// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"
#include "RHIDefinitions.h"
#include "RHIFwd.h"
#include "RHIShaderPlatform.h"

enum class ECustomDepthPassLocation : uint32
{
	// Renders custom depth before the base pass. Can be more efficient with AsyncCompute and enables use with DBuffer decals.
	BeforeBasePass,

	// Renders after the base pass.
	AfterBasePass
};

// Returns the location in the frame where custom depth is rendered.
extern ECustomDepthPassLocation GetCustomDepthPassLocation(EShaderPlatform Platform);

struct FCustomDepthTextures
{
	static FCustomDepthTextures Create(FRDGBuilder& GraphBuilder, FIntPoint CustomDepthExtent, EShaderPlatform ShaderPlatform, bool bRequireMultiView, uint16 MobileMultiViewRenderTargetNumLayers);
	bool IsValid() const
	{
		return Depth != nullptr;
	}

	FRDGTextureRef Depth{};
	FRDGTextureSRVRef Stencil{};

	// Denotes that the depth and stencil buffers had to be split to separate, non-depth textures (and thus Depth cannot be bound
	// as a depth/stencil buffer). This can happen when Nanite renders custom depth on platforms with HW that cannot write stencil
	// values per-pixel from a shader.
	bool bSeparateStencilBuffer = false;

	// Actions to use when initially rendering to custom depth / stencil.
	ERenderTargetLoadAction DepthAction = ERenderTargetLoadAction::EClear;
	ERenderTargetLoadAction StencilAction = ERenderTargetLoadAction::EClear;
};
