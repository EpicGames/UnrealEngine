// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCacheShaders.h"
#include "MeshPassProcessor.h"
#include "NaniteSceneProxy.h"

struct FMaterialCachePrimitiveData;
struct FNaniteShadingPipeline;
class FStaticMeshBatch;

struct FMaterialCacheLayerShadingCSCommand
{
	FMeshDrawShaderBindings     ShaderBindings;
	TShaderRef<FMaterialShader> ComputeShader;
};

struct FMaterialCacheMeshDrawCommand
{
	FMeshDrawCommand           Command;
	FCachedMeshDrawCommandInfo CommandInfo;
};

class FMaterialCacheMeshProcessor : public FSceneRenderingAllocatorObject<FMaterialCacheMeshProcessor>, public FMeshPassProcessor
{
public:
	FMaterialCacheMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FGuid& TagGuid, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext, const FMaterialRenderProxy* OverrideLayerMaterialProxy);

	/** FMeshPassProcessor */
	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial* Material);

	/** Optional, override layer material for the mesh */
	const FMaterialRenderProxy* OverrideLayerMaterialProxy = nullptr;

	/** Current tag being processed */
	FGuid TagGuid;
	
	FMeshPassProcessorRenderState PassDrawRenderState;
};

class FMaterialCacheMeshPassContext : public FMeshPassDrawListContext
{
public:
	/** FMeshPassDrawListContext */
	virtual FMeshDrawCommand& AddCommand(FMeshDrawCommand& Initializer, uint32 NumElements) override;
	virtual void FinalizeCommand(
		const FMeshBatch& MeshBatch, int32 BatchElementIndex, const FMeshDrawCommandPrimitiveIdInfo& IdInfo,
		ERasterizerFillMode MeshFillMode, ERasterizerCullMode MeshCullMode, FMeshDrawCommandSortKey SortKey, EFVisibleMeshDrawCommandFlags Flags,
		const FGraphicsMinimalPipelineStateInitializer& PipelineState, const FMeshProcessorShaders* ShadersForDebugging,
		FMeshDrawCommand& MeshDrawCommand) override;

	/** Compiled command */
	FMaterialCacheMeshDrawCommand Command;
};

#if WITH_EDITOR
/** Check if all materials needed for caching are ready, only relevant for editor */
extern bool IsMaterialCacheMaterialReady(EShaderPlatform InShaderPlatform, const FPrimitiveSceneProxy* Proxy);
#endif // WITH_EDITOR

/** Create a static mesh command with a layer material */
bool CreateMaterialCacheStaticLayerDrawCommand(
	FScene& Scene,
	const FPrimitiveSceneProxy* Proxy,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FStaticMeshBatch& MeshBatch,
	const FGuid& TagGuid,
	FMaterialCacheMeshDrawCommand& OutMeshCommand
);

/** Create a compute shading command with a layer material */
template<typename T>
extern bool CreateMaterialCacheComputeLayerShadingCommand(
	const FScene& Scene,
	const FPrimitiveSceneProxy* SceneProxy,
	const FMaterialRenderProxy* Material,
	bool bAllowDefaultFallback,
	const FGuid& TagGuid,
	FRHICommandListBase& RHICmdList,
	FMaterialCacheLayerShadingCSCommand& OutShadingCommand
);
