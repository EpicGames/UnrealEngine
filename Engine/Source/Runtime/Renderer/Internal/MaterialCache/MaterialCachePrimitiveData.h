// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCacheMeshProcessor.h"
#include "PrimitiveComponentId.h"

class FPrimitiveSceneProxy;

struct FMaterialCachePrimitiveCachedLayerCommands
{
	FMaterialCachePrimitiveCachedLayerCommands() = default;

	/** No copy or move construction */
	UE_NONCOPYABLE(FMaterialCachePrimitiveCachedLayerCommands);
	
	TArray<FMaterialCacheMeshDrawCommand>          StaticMeshBatchCommands;
	TOptional<FMaterialCacheLayerShadingCSCommand> NaniteLayerShadingCommand;
	TOptional<FMaterialCacheLayerShadingCSCommand> VertexInvariantShadingCommand;
};

struct FMaterialCachePrimitiveCachedTagCommands
{
	/** Lifetime of material tied to the proxy, any change invalidates the proxy, in turn clearing the cache */
	TMap<UMaterialInterface*, TUniquePtr<FMaterialCachePrimitiveCachedLayerCommands>> Layers;
};

struct FMaterialCachePrimitiveCachedCommands
{
	/** All cached material layers for a given teg */
	TMap<FGuid, FMaterialCachePrimitiveCachedTagCommands> Tags;
};

struct FMaterialCachePrimitiveData
{
	/** Proxy tied to the primitive */
	FPrimitiveSceneProxy* Proxy = nullptr;

	/** All cached commands */
	FMaterialCachePrimitiveCachedCommands CachedCommands;
};

struct FMaterialCacheAllocationSceneData
{
	/** All primitives associated with this allocation */
	TArray<FPrimitiveComponentId> Primitives;
};
