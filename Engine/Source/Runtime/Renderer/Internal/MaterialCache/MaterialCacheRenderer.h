// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCache/MaterialCacheAttribute.h"
#include "PrimitiveComponentId.h"
#include "Math/IntRect.h"
#include "Math/Box2D.h"

class  FRDGBuilder;
class  FSceneRenderer;
class  FPrimitiveSceneInfo;
struct IPooledRenderTarget;
struct FMaterialCacheVirtualTextureAllocation;

struct FMaterialCachePageEntry
{
	/** Destination page rectangle */
	FIntRect TileRect;

	/** Primitive UV rectangle associated with a given page */
	FBox2f UVRect;
};

struct FMaterialCacheSetup
{
	/** Owning allocation */
	FMaterialCacheVirtualTextureAllocation* Allocation;

	/** Destination render targets, must be UAV compatible */
	TArray<IPooledRenderTarget*, TInlineAllocator<MaterialCacheMaxRuntimeLayers>> PhysicalRenderTargets;

	/** Page size, includes border */
	FIntPoint TileSize = FIntPoint::ZeroValue;
};

/** Enqueue a set of pages for rendering */
void MaterialCacheEnqueuePages(
	FRDGBuilder& GraphBuilder,
	const FMaterialCacheTagLayout& TagLayout,
	const FMaterialCacheSetup& Setup,
	const TArrayView<FMaterialCachePageEntry>& Pages
);

/** Process all enqueued pages */
void MaterialCacheRenderPages(FRDGBuilder& GraphBuilder);
