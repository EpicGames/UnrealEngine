// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCacheRenderer.h"
#include "RendererInterface.h"

static constexpr uint32 ABufferPageIndexNotProduced = UINT32_MAX;

struct FMaterialCachePendingPageEntry
{
	/** Page to be produced */
	FMaterialCachePageEntry Page;

	/** Lazy allocated A-Buffer index */
	uint32 ABufferPageIndex = ABufferPageIndexNotProduced;
};

struct FMaterialCachePendingEntry
{
	/** General setup for the page */
	FMaterialCacheSetup Setup;

	/** All pages pending producing */
	TArray<FMaterialCachePendingPageEntry, SceneRenderingAllocator> Pages;
};

struct FMaterialCachePendingTagBucket
{
	/** Tag being rendered */
	FMaterialCacheTagLayout TagLayout;

	/** All entries for the given tag */
	TArray<FMaterialCachePendingEntry, SceneRenderingAllocator> PendingEntries;
};
