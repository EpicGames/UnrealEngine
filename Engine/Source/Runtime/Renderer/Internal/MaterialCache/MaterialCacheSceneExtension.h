// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCacheMeshProcessor.h"
#include "MaterialCacheSceneData.h"
#include "PrimitiveComponentId.h"
#include "SceneExtensions.h"

class FPrimitiveSceneProxy;
class IAllocatedVirtualTexture;
struct FMaterialCacheSceneExtensionData;
struct FMaterialCachePrimitiveData;
struct FMaterialCacheProviderData;
class UMaterialCacheVirtualTextureTag;
struct FMaterialCacheAllocationSceneData;

class FMaterialCacheSceneExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FMaterialCacheSceneExtension);

public:
	FMaterialCacheSceneExtension(FScene& InScene);

	/** Get the primitive data associated with a primitive id, nullptr if not found */
	FMaterialCachePrimitiveData* GetPrimitiveData(FPrimitiveComponentId PrimitiveComponentId) const;
	
	/** Get the allocation scene data, nullptr if not found */
	FMaterialCacheAllocationSceneData* GetAllocationSceneData(FMaterialCacheVirtualTextureAllocation* Allocation) const;
	
	/** Update the uniform data of a specific primitive */
	void UpdateTagUniforms(FPrimitiveComponentId PrimitiveComponentId);
	
	/** Update the uniform data of a specific primitive */
	void UpdateTagUniforms(FMaterialCacheVirtualTextureAllocation* Allocation);
	
	/** Register a new allocation */
	void Register(FMaterialCacheVirtualTextureAllocation* Allocation);
	
	/** Unregister an allocation, removing primitive associations */
	void Unregister(FMaterialCacheVirtualTextureAllocation* Allocation);

	/** Clear all cached primitive command data */
	void ClearCachedPrimitiveData();

public:
	/** All pending tags, lifetime tied to the scene's renderer */
	TMap<FGuid, FMaterialCachePendingTagBucket> TagBuckets;
	
public: /** ISceneExtension */
	static bool ShouldCreateExtension(FScene& Scene);
	virtual ISceneExtensionUpdater* CreateUpdater() override;

private:
	TUniquePtr<FMaterialCacheSceneExtensionData> Data;
};
