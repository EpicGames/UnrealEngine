// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshMaterialShader.h"
#include "MaterialCache/MaterialCacheAttribute.h"
#include "ShaderParameterStruct.h"

class FMaterialCacheUnwrapVSBase : public FMeshMaterialShader
{
	typedef FMeshMaterialShader Super;
	DECLARE_INLINE_TYPE_LAYOUT(FMaterialCacheUnwrapVSBase, NonVirtual);

public:
	class FTagIndex          : SHADER_PERMUTATION_INT("TAG_INDEX", MaterialCacheMaxTagsPerPrimitive);
	using FPermutationDomain = TShaderPermutationDomain<FTagIndex>;
	
	FMaterialCacheUnwrapVSBase(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FMeshMaterialShader(Initializer)
	{
		
	}

	FMaterialCacheUnwrapVSBase() = default;
};

template<bool bSupportsViewportFromVS>
class FMaterialCacheUnwrapVS : public FMaterialCacheUnwrapVSBase
{
	DECLARE_SHADER_TYPE(FMaterialCacheUnwrapVS, MeshMaterial);

public:	
	FMaterialCacheUnwrapVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FMaterialCacheUnwrapVSBase(Initializer)
	{
		
	}
	
	FMaterialCacheUnwrapVS() = default;
	
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

class FMaterialCacheUnwrapPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FMaterialCacheUnwrapPS, MeshMaterial);

public:
	class FTagIndex          : SHADER_PERMUTATION_INT("TAG_INDEX", MaterialCacheMaxTagsPerPrimitive);
	using FPermutationDomain = TShaderPermutationDomain<FTagIndex>;
	
	FMaterialCacheUnwrapPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FMeshMaterialShader(Initializer)
	{
		
	}

	FMaterialCacheUnwrapPS() = default;

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

class FMaterialCacheNaniteShadeCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FMaterialCacheNaniteShadeCS, MeshMaterial);

public:
	class FTagIndex          : SHADER_PERMUTATION_INT("TAG_INDEX", MaterialCacheMaxTagsPerPrimitive);
	using FPermutationDomain = TShaderPermutationDomain<FTagIndex>;
	
	FMaterialCacheNaniteShadeCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FMaterialCacheNaniteShadeCS() = default;

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	void SetPassParameters(FRHIBatchedShaderParameters& BatchedParameters, const FUintVector4& PassData, FRHIShaderResourceView* PageIndirections);

private:
	LAYOUT_FIELD(FShaderResourceParameter, PageIndirectionsParam);
	LAYOUT_FIELD(FShaderParameter, PassDataParam);
};

class FMaterialCacheShadeCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FMaterialCacheShadeCS, MeshMaterial);

public:
	FMaterialCacheShadeCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);
	FMaterialCacheShadeCS() = default;

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	void SetPassParameters(FRHIBatchedShaderParameters& BatchedParameters, const FUintVector4& PassData, FRHIShaderResourceView* PageIndirections);

private:
	LAYOUT_FIELD(FShaderResourceParameter, PageIndirectionsParam);
	LAYOUT_FIELD(FShaderParameter, PassDataParam);
};

class FMaterialCacheABufferFillBorderHardwarePagesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMaterialCacheABufferFillBorderHardwarePagesCS);
	SHADER_USE_PARAMETER_STRUCT(FMaterialCacheABufferFillBorderHardwarePagesCS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FMaterialCacheBinData>, PageWriteData)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, RWABuffer)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<uint>, R8Mask)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		SHADER_PARAMETER(uint32, BorderWidth)
		SHADER_PARAMETER(uint32, ThreadCount)
	END_SHADER_PARAMETER_STRUCT()
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

class FMaterialCacheABufferWritePagesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMaterialCacheABufferWritePagesCS);
	SHADER_USE_PARAMETER_STRUCT(FMaterialCacheABufferWritePagesCS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FMaterialCacheBinData>, PageWriteData)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint2>, RWVTLayerCompressed64)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint4>, RWVTLayerCompressed128)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWVTLayerUncompressed)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<float4>, ABuffer)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		SHADER_PARAMETER(uint32, BlockOrThreadCount)
		SHADER_PARAMETER(FVector2f, TexelSize)
		SHADER_PARAMETER(uint32, bYCoCg)
		SHADER_PARAMETER(uint32, bSRGB)
	END_SHADER_PARAMETER_STRUCT()
	
	class FCompressMode        : SHADER_PERMUTATION_INT("COMPRESS_MODE", 7);
	using FPermutationDomain = TShaderPermutationDomain<FCompressMode>;

	static int32 GetCompressMode(EPixelFormat Format);
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

struct FNaniteMaterialCacheData
{
	TShaderRef<FMaterialCacheNaniteShadeCS> TypedShader;
};
