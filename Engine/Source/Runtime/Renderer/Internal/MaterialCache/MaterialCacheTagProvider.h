// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCache/IMaterialCacheTagProvider.h"
#include "RenderGraphBuilder.h"

struct FMaterialCacheTagLayout;
namespace UE::HLSL { struct FMaterialCacheTagEntry; }

class FMaterialCacheTagProvider : public IMaterialCacheTagProvider
{
public:
	/** Allocate a new primitive tag offset, each tag may serve a number of tags (backing virtual textures) */
	uint32 AllocatePrimitiveTagOffset();
	
	/** Free a primitive tag offset */
	void FreePrimitiveTagOffset(uint32 TagOffset);
	
	/** Set the tag entry */
	void SetTagEntry(uint32 TagOffset, const FGuid& TagGuid, const UE::HLSL::FMaterialCacheTagEntry& Entry);
	
	/** Update the GPU state */
	void Update(FRDGBuilder& GraphBuilder);

	/** Call all pending invalidation callbacks */
	void CallPendingCallbacks();
	
public: /** IMaterialCacheTagProvider */
	virtual FMaterialCacheTagBindingData GetBindingData(const FGuid& Guid) override;
	virtual FMaterialCacheTagUniformData GetUniformData(const FGuid& Guid) override;
	virtual void Register(FSceneInterface* Scene, FPrimitiveComponentId PrimitiveComponentId, FMaterialCacheVirtualTextureAllocation* VirtualTexture) override;
	virtual void Unregister(FSceneInterface* Scene, FPrimitiveComponentId PrimitiveComponentId, FMaterialCacheVirtualTextureAllocation* VirtualTexture) override;
	virtual void Register(FMaterialCacheVirtualTextureAllocation* VirtualTexture) override;
	virtual void Unregister(FMaterialCacheVirtualTextureAllocation* VirtualTexture) override;
	virtual void Flush(FSceneInterface* Scene, FPrimitiveComponentId PrimitiveComponentId, const FGuid& TagGuid) override;
	virtual void AddTagSceneInvalidationCallback(const FGuid& TagGuid, FMaterialCacheTagProviderSceneInvalidationDelegate Delegate, void* Baton) override;
	virtual void RemoveTagSceneInvalidationCallbacks(void* Baton) override;
	virtual void NotifyAllTagSceneInvalidation() override;

public: /** Installation */
	/** Initialize the global tag provider */
	static void Initialize();
	
	/** Shutdown the global tag provider */
	static void Shutdown();
	
	/** Get the tag provider */
	static FMaterialCacheTagProvider& Get();

private:
	/** Create the backing buffers or resize them */
	void CreateDeviceBuffersOrResize(struct FMaterialCacheTagBucket& Bucket);

	/** Notify a tag invalidation */
	void NotifyTagSceneInvalidation(const FGuid& TagGuid);

private:
	FCriticalSection Mutex;
	
	/** All registered tag buckets */
	TMap<FGuid, struct FMaterialCacheTagBucket*> TagBuckets;

	/** All pending invalidations */
	TSet<FGuid> PendingTagSceneInvalidations;

	/** All registered virtual textures */
	TMap<TTuple<FPrimitiveComponentId, FGuid>, FMaterialCacheVirtualTextureAllocation*> VirtualTextures;

	/** All free indices */
	TArray<uint32> FreeTagIndices;

	/** Linear allocator */
	uint32 TagOffsetAllocator = 0;

private:
	struct FTagSceneListener
	{
		FMaterialCacheTagProviderSceneInvalidationDelegate Delegate = nullptr;
		void* Baton = nullptr;
	};
	
	struct FTagBatonEntry
	{
		int32 CallbackIndex = UINT32_MAX;
		FGuid TagGuid;
	};

	/** All scene invalidation callbacks */
	TArray<FTagSceneListener>  TagSceneInvalidationListeners;

	/** All free callback indices */
	TArray<int32> FreeTagSceneInvalidationCallbackIndices;

	/** All listeners to tag scene data invalidation */
	TMultiMap<FGuid, int32>  TagCallbackIndices;
	TMultiMap<void*, FTagBatonEntry> BatonCallbackIndices;
};
