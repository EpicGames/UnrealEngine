// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCache/MaterialCacheAttribute.h"
#include "Engine/Texture2D.h"
#include "VirtualTexturing.h"
#include "RHIFwd.h"

class FScene;
struct FMaterialCacheVirtualTextureAllocation;

struct FMaterialCacheTileEntry
{
	/** Destination layers */
	TArray<FVTProduceTargetLayer, TInlineAllocator<MaterialCacheMaxRuntimeLayers>> TargetLayers;

	/** Destination address (morton encoded page x/y) */
	uint64 Address = 0;

	/** Destination level */
	uint8 Level = 0;
};

class FMaterialCacheVirtualFinalizer : public IVirtualTextureFinalizer
{
public:
	FMaterialCacheVirtualFinalizer(FScene* InScene, FMaterialCacheVirtualTextureAllocation* Allocation, const FMaterialCacheTagLayout& TagLayout, const FVTProducerDescription& InProducerDesc);
	virtual ~FMaterialCacheVirtualFinalizer() = default;

	/** Add a new tile for processing */
	void AddTile(const FMaterialCacheTileEntry& InEntry);

public: /** IVirtualTextureFinalizer */
	virtual void RenderFinalize(FRDGBuilder& GraphBuilder, ISceneRenderer* SceneRenderingContext) override;
	virtual void Finalize(FRDGBuilder& GraphBuilder) override;

protected:
	/** Render scene, lifetime tied to the parent game virtual texture */
	FScene* Scene = nullptr;
	
	/** Owning allocation */
	FMaterialCacheVirtualTextureAllocation* Allocation;

	/** Runtime layout of the tag we're producing for */
	FMaterialCacheTagLayout TagLayout;
	
	FVTProducerDescription ProducerDesc;
	EPixelFormat SourceFormat;
	EPixelFormat DestFormat;
	EPixelFormat IntermediateFormat;

private:
	TArray<FMaterialCacheTileEntry> TilesToRender;
};
