// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCache/MaterialCacheAttribute.h"
#include "MaterialCache/MaterialCacheVirtualFinalizer.h"
#include "VirtualTexturing.h"
#include "RHIFwd.h"

class FScene;
struct FMaterialCacheVirtualTextureAllocation;

class FMaterialCacheVirtualProducer : public IVirtualTexture
{
public:
	FMaterialCacheVirtualProducer(
		FScene* Scene, 
		const FMaterialCacheTagLayout& TagLayout,
		const FVTProducerDescription& InProducerDesc,
		FMaterialCacheVirtualTextureAllocation* Allocation
	);

public: /** IVirtualTexture */
	virtual ~FMaterialCacheVirtualProducer() override = default;

	virtual bool IsPageStreamed(uint8 vLevel, uint32 vAddress) const override;

	virtual FVTRequestPageResult RequestPageData(
		FRHICommandListBase& RHICmdList,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		EVTRequestPagePriority Priority
	) override;

	virtual IVirtualTextureFinalizer* ProducePageData(
		FRHICommandListBase& RHICmdList,
		ERHIFeatureLevel::Type FeatureLevel,
		EVTProducePageFlags Flags,
		const FVirtualTextureProducerHandle& ProducerHandle,
		uint8 LayerMask,
		uint8 vLevel,
		uint64 vAddress,
		uint64 RequestHandle,
		const FVTProduceTargetLayer* TargetLayers
	) override;

	/** Single finalizer per producer */
	FMaterialCacheVirtualFinalizer Finalizer;
	
private:
	/** Render scene, lifetime tied to the parent game virtual texture */
	FScene* Scene = nullptr;

	/** Owning allocation */
	FMaterialCacheVirtualTextureAllocation* Allocation = nullptr;
	
	FVTProducerDescription ProducerDesc;
};
