// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialCacheVirtualTextureAllocatorStats.h"
#include "MaterialCache/IMaterialCacheVirtualTextureAllocator.h"
#include "RenderGraphBuilder.h"
#include "Containers/BinaryHeap.h"
#include "MaterialCache/MaterialCacheAttribute.h"

class FMaterialCacheVirtualProducer;

class FMaterialCacheVirtualTextureAllocator : public IMaterialCacheVirtualTextureAllocator
{
public:
	~FMaterialCacheVirtualTextureAllocator() override = default;

	/** Update all allocations */
	void UpdateAllocations(FRHICommandListBase& RHICmdList, uint32 Frame);

	/** Debug only, recreate all allocations from their most coarse representations */
	void DebugRecreateCoarse(FRHICommandListBase& RHICmdList);
	
public: /** IMaterialCacheVirtualTextureAllocator */
	virtual FMaterialCacheVirtualTextureAllocation* Allocate(FRHICommandListBase& RHICmdList, FSceneInterface* Scene, const FMaterialCacheVirtualTextureDescription& Desc) override;
	virtual void Reallocate(FRHICommandListBase& RHICmdList, FMaterialCacheVirtualTextureAllocation* Allocation) override;
	virtual void Deallocate(FMaterialCacheVirtualTextureAllocation* Texture) override;
	virtual void Flush(FMaterialCacheVirtualTextureAllocation* Allocation, const FVector2f& MinUV, const FVector2f& MaxUV) override;

	/** Installation */
	/** Initialize the global allocator */
	static void Initialize();
	
	/** Shutdown the global allocator */
	static void Shutdown();
	
	/** Get the allocator */
	static FMaterialCacheVirtualTextureAllocator& Get();
	
private:
	struct FAllocationEntry
	{
		static constexpr uint32 UnassignedRequestedLevel = 255;
		
		/** The backing allocation */
		FMaterialCacheVirtualTextureAllocation* Allocation = nullptr;
		
		/** Owning scene, allocator may represent multiple */
		class FScene* Scene = nullptr;
		
		/** Current requested level */
		uint32 RequestedLevel = UnassignedRequestedLevel;
	};

	void ProcessFeedback(FRHICommandListBase& RHICmdList, uint32 Frame);
	void ProcessPassiveLRUEvictions(FRHICommandListBase& RHICmdList, uint32 Frame);
	void ProcessPendingAllocations(FRHICommandListBase& RHICmdList);
	
	bool UpdateAllocation(FRHICommandListBase& RHICmdList, FAllocationEntry& Entry, uint32 Frame);
	void ReallocateBackingAllocation(FRHICommandListBase& RHICmdList, FAllocationEntry& Entry, uint32 Frame, bool bRemap);
	bool AllocateBackingAllocation(FRHICommandListBase& RHICmdList, FAllocationEntry& Entry);
	bool TryAllocateBackingAllocation(FRHICommandListBase& RHICmdList, FAllocationEntry& Entry);
	void DeallocateBackingAllocation(FMaterialCacheVirtualTextureAllocation& Allocation);
	void RemapBackingAllocation(
		const FMaterialCacheVirtualTextureAllocation& OldAllocation,
		const FMaterialCacheVirtualTextureAllocation& NewAllocation,
		uint32 Frame
	);
	
	void FreeLRU(FRHICommandListBase& RHICmdList, uint32 Frame);
	
	uint32 GetRequiredTileCount(FAllocationEntry& Entry);
	bool CanAllocate(uint32 TileCount);
	
private:
	FMaterialCacheVirtualTextureAllocatorStats Stats; 
	
#if !UE_BUILD_SHIPPING
	void DrawDebugDisplay(class UCanvas*, class APlayerController*);
	FDelegateHandle	DrawDelegate;
#endif // !UE_BUILD_SHIPPING
	
private:
	/** All allocations */
	TArray<FAllocationEntry> AllocationEntries;
	
	/** Pending entries that failed backing allocations */
	TArray<FMaterialCacheVirtualTextureAllocation*> PendingAllocations;
	
	/** Per allocation LRU */
	FBinaryHeap<uint32, uint32> LRUHeap;
	
private:
	/** Book-keeping */
	int32 LiveBackingAllocations = 0;
	int32 TotalMappedTileCount   = 0;
	
	/** All allocations should share a single space */
	uint8 DesignatedSpace = VIRTUALTEXTURE_INVALID_FORCE_SPACEID; 
};
