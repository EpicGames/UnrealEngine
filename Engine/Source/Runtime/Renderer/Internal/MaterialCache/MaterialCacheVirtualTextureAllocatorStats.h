// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FMaterialCacheVirtualTextureAllocatorStats
{
public:
	static constexpr uint32 HistorySize = 128;
	
	enum EStatID
	{
		Realloc,
		LRUEvict,
		FailedAlloc,
		FailedAllocSpace,
		LateAlloc,
		TileMapping,
		Count
	};
	
	void CycleStats()
	{
#if !UE_BUILD_SHIPPING
		RingIndex = (RingIndex + 1) % HistorySize;
		
		for (uint32 i = 0; i < Count; i++)
		{
			History[i][RingIndex] = 0;
		}
#endif // !UE_BUILD_SHIPPING
	}
	
	void AddStat(EStatID ID, float Value = 1)
	{
#if !UE_BUILD_SHIPPING
		History[ID][RingIndex] += Value;
#endif // !UE_BUILD_SHIPPING
	}
	
	void SetStat(EStatID ID, float Value)
	{
#if !UE_BUILD_SHIPPING
		History[ID][RingIndex] = Value;
#endif // !UE_BUILD_SHIPPING
	}
	
	float* GetHistoryBuffer(EStatID ID)
	{
#if !UE_BUILD_SHIPPING
		return History[ID];
#else  // !UE_BUILD_SHIPPING
		return nullptr;
#endif // !UE_BUILD_SHIPPING
	}
	
	uint32 GetRingIndex()
	{
#if !UE_BUILD_SHIPPING
		return RingIndex;
#else  // !UE_BUILD_SHIPPING
		return 0;
#endif // !UE_BUILD_SHIPPING
	}
	
private:
#if !UE_BUILD_SHIPPING
	uint32 RingIndex = 0;
	
	float History[Count][HistorySize]{};
#endif // !UE_BUILD_SHIPPING
};
