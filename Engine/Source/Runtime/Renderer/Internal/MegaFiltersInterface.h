// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "SceneView.h"

class FSceneView;
class FSceneViewFamily;

namespace UE::Renderer::Private
{

enum class EMegaFiltersPassType : uint32
{
	Default,
	MegaLights,
	MAX
};

/** Interface for implementing different filters */
class IMegaFilters : public ISceneViewFamilyExtention
{
public:

	/** Ref counted history to be saved in the history*/
	class IHistory : public FRefCountedObject
	{
	public:
		virtual ~IHistory() {}

		/** Debug name of the history. Must exactly point to the same const TCHAR* as IMegaFilters::GetDebugName().
		* This is used for debugging GPU memory uses of a viewport, but also to ensure IHistory is fed to a compatible IMegaFilters.
		*/
		virtual const TCHAR* GetDebugName() const = 0;

		/** Size of the history on the GPU in bytes. */
		virtual uint64 GetGPUSizeBytes() const = 0;
	};

	/** Inputs of the filter*/
	struct FInputs
	{
		/** Which filter pass this input is used for*/
		EMegaFiltersPassType MegaFiltersPassType;

		/** Outputs view rect that must be on FOutputs::FullRes::ViewRect. */
		TArray<FIntRect> OutputViewRects;

		/** Configuration variables per pass (e.g., TemporalJitter and PreExposure)*/
		TArray<TVariant<float, int, FVector2f, FVector4f>> Variables;

		/** Input texture and buffer*/
		TArray<FScreenPassTexture> ScreenPassTextures;

		TArray<FRDGTextureRef> Textures;

		TArray<FRDGBufferRef> Buffers;

		/** The history of the previous frame set by FOutputs::NewHistory. PrevHistory->GetDebugName() is guarentee to match the IMegaFilters.*/
		TRefCountPtr<IHistory> PrevHistory;
	};

	/** Outputs of the third party MegaFilters. */
	struct FOutputs
	{
		/** Output of the MegaFilters. FullRes[i].ViewRect must match FInputs::OutputViewRects[i]. */
		TArray<FScreenPassTexture> FullRes;

		/** New history to be kept alive for next frame. NewHistory->GetDebugName() must exactly point to the same const TCHAR* as IMegaFilters::GetDebugName(). 
		Meaningful when bPassThrough = true;
		*/
		TRefCountPtr<IHistory> NewHistory;

		/** Indicate the pass does nothing*/
		bool bPassThrough = false;
	};

	virtual ~IMegaFilters() {};

	/** Debug name of the history. Must exactly point to the same const TCHAR* as IMegaFilters::IHistory::GetDebugName(). */
	virtual const TCHAR* GetDebugName() const = 0;

	/** Adds the necessary passes into RDG for filtering or temporal upscaling the rendering resolution to desired output res. */
	virtual FOutputs AddPasses(
		FRDGBuilder& GraphBuilder,
		const FSceneView& View,
		const FInputs& Inputs) const = 0;

	virtual IMegaFilters* Fork_GameThread(const class FSceneViewFamily& ViewFamily) const = 0;
};

} // namespace UE::Renderer::Private



