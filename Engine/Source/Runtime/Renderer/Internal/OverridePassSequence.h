// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "SceneViewExtension.h"


/** This class provides a solution for overriding the final render target / view rect in a chain of render graph passes.
 *  RDG pass setup is immediate, which makes it difficult to determine the final pass in a chain in order to swap a final
 *  texture target (e.g. the view family target). This class helps by providing some utilities to mark passes as enabled
 *  prior to setup, then seamlessly assign the override texture to the correct last pass. It assumes that all passes adhere
 *  to the pattern of using an input struct that derives from FScreenPassRenderTarget (though this is not strictly required).
 *
 *  The user provides a custom enum class that is the set of ordered passes. The user can enable them in any order, but
 *  all passes must be assigned (i.e. have SetEnabled called on them). After enabling / disabling all passes. Call Finalize
 *  to compute internal state. You cannot enable / disable passes after finalizing. Each pass must be accepted in order,
 *  and all passes marked as enabled must be accepted. The helper, AcceptOverrideIfLastPass, will check and override if
 *  the pass is the last one, and then accept the pass.
 */
template <typename EPass>
class TOverridePassSequence final
{
public:
	TOverridePassSequence(const FScreenPassRenderTarget& InOverrideOutput)
		: OverrideOutput(InOverrideOutput)
	{}

	~TOverridePassSequence()
	{
#if RDG_ENABLE_DEBUG
		if (bFinalized)
		{
			for (int32 PassIndex = 0; PassIndex < PassCountMax; ++PassIndex)
			{
				if (Passes[PassIndex].bEnabled)
				{
					checkf(Passes[PassIndex].bAccepted, TEXT("Pass was enabled but not accepted: %s."), Passes[PassIndex].Name);
				}
			}
		}
#endif
	}

	void SetName(EPass Pass, const TCHAR* Name)
	{
#if RDG_ENABLE_DEBUG
		Passes[(int32)Pass].Name = Name;
#endif
	}

	void SetNames(const TCHAR* const* Names, uint32 NameCount)
	{
#if RDG_ENABLE_DEBUG
		check(NameCount == PassCountMax);
		for (int32 PassIndex = 0; PassIndex < PassCountMax; ++PassIndex)
		{
			Passes[PassIndex].Name = Names[PassIndex];
		}
#endif
	}

	void SetEnabled(EPass Pass, bool bEnabled)
	{
		const int32 PassIndex = (int32)Pass;

#if RDG_ENABLE_DEBUG
		check(!bFinalized);
		checkf(!Passes[PassIndex].bAssigned, TEXT("Pass cannot be assigned multiple times: %s."), Passes[PassIndex].Name);
		Passes[PassIndex].bAssigned = true;
#endif

		Passes[PassIndex].bEnabled = bEnabled;
	}

	bool IsEnabled(EPass Pass) const
	{
		const int32 PassIndex = (int32)Pass;
#if RDG_ENABLE_DEBUG
		check(Passes[PassIndex].bAssigned);
#endif
		return Passes[PassIndex].bEnabled;
	}

	bool IsLastPass(EPass Pass) const
	{
#if RDG_ENABLE_DEBUG
		check(bFinalized);
#endif
		return Pass == LastPass;
	}

	void AcceptPass(EPass Pass)
	{
#if RDG_ENABLE_DEBUG
		const int32 PassIndex = (int32)Pass;

		check(bFinalized);
		checkf(NextPass == Pass, TEXT("Pass was accepted out of order: %s. Expected %s."), Passes[PassIndex].Name, Passes[(int32)NextPass].Name);
		checkf(Passes[PassIndex].bEnabled, TEXT("Only accepted passes can be enabled: %s."), Passes[PassIndex].Name);

		Passes[PassIndex].bAccepted = true;

		// Walk the remaining passes until we hit one that's enabled. This will be the next pass to add.
		for (int32 NextPassIndex = int32(NextPass) + 1; NextPassIndex < PassCountMax; ++NextPassIndex)
		{
			if (Passes[NextPassIndex].bEnabled)
			{
				NextPass = EPass(NextPassIndex);
				break;
			}
		}
#endif
	}

	bool AcceptOverrideIfLastPass(EPass Pass, FScreenPassRenderTarget& OutTargetToOverride, const TOptional<int32>& AfterPassCallbackIndex = TOptional<int32>())
	{
		const int32 PassIndex = (int32)Pass;
		bool bHasMoreAfterPassCallbacks;

		if (AfterPassCallbackIndex)
		{
			bHasMoreAfterPassCallbacks = AfterPassCallbackIndex.GetValue() < AfterPass[PassIndex].Num() - 1;
		}
		else
		{
			bHasMoreAfterPassCallbacks = AfterPass[PassIndex].Num() > 0;

			// Display debug information for a Pass unless it is an after pass.
			AcceptPass(Pass);
		}

		// We need to override output only if this is the last pass and the last after pass.
		if (AcceptOverrideInPassIndex == PassIndex && !bHasMoreAfterPassCallbacks)
		{
			OutTargetToOverride = OverrideOutput;
			return true;
		}

		return false;
	}

	void Finalize()
	{
#if RDG_ENABLE_DEBUG
		check(!bFinalized);
		bFinalized = true;

		for (int32 PassIndex = 0; PassIndex < PassCountMax; ++PassIndex)
		{
			checkf(Passes[PassIndex].bAssigned, TEXT("Pass was not assigned to enabled or disabled: %s."), Passes[PassIndex].Name);
		}
#endif

		bool bFirstPass = true;

		for (int32 PassIndex = 0; PassIndex < PassCountMax; ++PassIndex)
		{
			if (Passes[PassIndex].bEnabled)
			{
				if (bFirstPass)
				{
#if RDG_ENABLE_DEBUG
					NextPass = (EPass)PassIndex;
#endif
					bFirstPass = false;
				}
				LastPass = (EPass)PassIndex;
				AcceptOverrideInPassIndex = PassIndex;
			}

			// We can have callbacks for disabled passes which come after the last enabled pass. In that case we only
			// accept the output override for the last callback of that pass, regardless of its state.
			if (AfterPass[PassIndex].Num() > 0)
			{
				AcceptOverrideInPassIndex = PassIndex;
			}
		}
	}

	FAfterPassCallbackDelegateArray& GetAfterPassCallbacks(EPass Pass)
	{ 
		const int32 PassIndex = (int32)Pass;
		return AfterPass[PassIndex]; 
	}

private:
	static const int32 PassCountMax = (int32)EPass::MAX;

	struct FPassInfo
	{
#if RDG_ENABLE_DEBUG
		const TCHAR* Name = nullptr;
		bool bAssigned = false;
		bool bAccepted = false;
#endif
		bool bEnabled = false;
	};

	FScreenPassRenderTarget OverrideOutput;
	TStaticArray<FPassInfo, PassCountMax> Passes;
	TStaticArray<FAfterPassCallbackDelegateArray, PassCountMax> AfterPass;
	EPass LastPass = EPass::MAX;
	int32 AcceptOverrideInPassIndex = PassCountMax;

#if RDG_ENABLE_DEBUG
	EPass NextPass = EPass(0);
	bool bFinalized = false;
#endif
};
