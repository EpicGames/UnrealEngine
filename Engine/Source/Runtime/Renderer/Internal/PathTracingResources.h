// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FRDGTexture;

struct FPathTracingResources
{
	FRDGTexture* DenoisedRadiance = nullptr;
	FRDGTexture* Radiance = nullptr;
	FRDGTexture* Albedo = nullptr;
	FRDGTexture* Normal = nullptr;
	FRDGTexture* Variance = nullptr;
	
	bool bPostProcessEnabled = false;
};

struct FPathTracingRealtimeDenoiserResources
{
	FRDGTexture* DiffuseColor = nullptr;
	FRDGTexture* SpecularColor = nullptr;
	FRDGTexture* Roughness = nullptr;
	FRDGTexture* Normal = nullptr;
	FRDGTexture* LinearDepth = nullptr;
};
