// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/MathFwd.h"
#include "ScreenPass.h"

class FRDGBuilder;
class FRDGTexture;
class FViewInfo;
class FRDGTextureSRV;
enum EShaderPlatform : uint16;

namespace UE::ColorParade
{
	struct FDrawParameters
	{
		// If false, the channels are overlayed on top of eachother (waveform instead of parade)
		bool bDrawSeparateColumnPerChannel = true;
		// If false, the graph is grayscale
		bool bColorize = true;
		// If true, columns with NaNs and Infs are highlighted in the graph
		bool bHighlightNaNsAndInfs = true;

		// Range of values that is plotted on the Y-axis
		float MinValue = 0.0f;
		float MaxValue = 1.0f;

		// Parameters to control how the histogram values are tonemapped to graph colors
		float Brightness = 50.0f;
		float Gamma = 1.0f / 2.2f;
		float DesaturationPower = 2.5f;
	};

	bool IsPlatformSupported(EShaderPlatform Platform);

	void Draw(
		FRDGBuilder& GraphBuilder,
		FRDGTextureSRV* InputSRV, FIntRect InputViewRect,
		FRDGTexture* Output, FIntRect OutputViewRect,
		const FDrawParameters& Parameters);

	inline void Draw(
		FRDGBuilder& GraphBuilder,
		FRDGTextureSRV* InputSRV, FIntRect InputViewRect,
		FRDGTexture* Output, FIntRect OutputViewRect)
	{
		FDrawParameters DrawParameters{};
		Draw(GraphBuilder, InputSRV, InputViewRect, Output, OutputViewRect, DrawParameters);
	}
} // namespace UE::ColorParade

namespace UE::VectorScope
{
	struct FDrawParameters
	{
		// If false, the graph is grayscale
		bool bColorize = true;

		// Rotation of the graph, in radians
		float Rotation = PI / 2.0f;

		// Include only pixel values with luminance in a certain range
		float LuminanceFilterMin = 0;
		float LuminanceFilterMax = 1;

		// Parameters to control how the histogram values are tonemapped to graph colors
		float Brightness = 50.0f;
		float Gamma = 1.0f / 2.2f;
		float DesaturationPower = 2.5f;
	};

	bool IsPlatformSupported(EShaderPlatform Platform);

	void Draw(
		FRDGBuilder& GraphBuilder,
		FRDGTextureSRV* InputSRV, FIntRect InputViewRect,
		FRDGTexture* Output, FIntRect OutputViewRect,
		const FDrawParameters& Parameters);

	inline void Draw(
		FRDGBuilder& GraphBuilder,
		FRDGTextureSRV* InputSRV, FIntRect InputViewRect,
		FRDGTexture* Output, FIntRect OutputViewRect)
	{
		FDrawParameters DrawParameters{};
		Draw(GraphBuilder, InputSRV, InputViewRect, Output, OutputViewRect, DrawParameters);
	}
} // namespace UE::VectorScope

namespace UE::ColorHistogram
{
	struct FDrawParameters
	{
		// If false, the channels are overlayed on top of eachother
		bool bDrawSeparateRowPerChannel = true;
		// If false, the graph is grayscale
		bool bColorize = true;

		// Range of values that is plotted on the Y-axis
		float MinValue = 0.0f;
		float MaxValue = 1.0f;
	};

	bool IsPlatformSupported(EShaderPlatform Platform);

	void Draw(
		FRDGBuilder& GraphBuilder,
		FRDGTextureSRV* InputSRV, FIntRect InputViewRect,
		FRDGTexture* Output, FIntRect OutputViewRect,
		const FDrawParameters& Parameters);

	inline void Draw(
		FRDGBuilder& GraphBuilder,
		FRDGTextureSRV* InputSRV, FIntRect InputViewRect,
		FRDGTexture* Output, FIntRect OutputViewRect)
	{
		FDrawParameters DrawParameters{};
		Draw(GraphBuilder, InputSRV, InputViewRect, Output, OutputViewRect, DrawParameters);
	}
} // namespace UE::ColorHistogram
