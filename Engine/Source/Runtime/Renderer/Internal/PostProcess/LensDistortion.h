// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

struct FLensDistortionLUT
{
	/** Expected pixel format of the LUT. 16bit floating point do not work when distorting in TSR. */
	static constexpr EPixelFormat kFormat = PF_G32R32F;

	/** DistortedViewportUV = UndistortedViewportUV + DistortingDisplacementTexture.Sample(UndistortedViewportUV) */
	FRDGTextureRef DistortingDisplacementTexture = nullptr;

	/** UndistortedViewportUV = DistortedViewportUV + UndistortingDisplacementTexture.BilinearSample(DistortedViewportUV) */
	FRDGTextureRef UndistortingDisplacementTexture = nullptr;

	/** Resolution fraction of the upscaling happening due to distortion. */
	float ResolutionFraction = 1.0f;

	/** For distortion maps that don't fill the whole frustum, this is the amount of overscan they require but do not fill */
	float DistortionOverscan = 1.0f;

	/** The distortion warp grid dimensions to use when distorting during the upscale pass */
	FIntPoint DistortionGridDimensions = FIntPoint(32, 20);
	
	/** Returns whether the displacement is enabled. */
	bool IsEnabled() const
	{
		return DistortingDisplacementTexture && UndistortingDisplacementTexture;
	}
};

struct FPaniniProjectionConfig
{
	FPaniniProjectionConfig() = default;

	/** Returns whether the panini is enabled by cvar. */
	RENDERER_API static bool IsEnabledByCVars();

	/** Returns teh cvars' configuration. */
	RENDERER_API static FPaniniProjectionConfig ReadCVars();

	bool IsEnabled() const
	{
		return D > 0.01f;
	}

	void Sanitize()
	{
		D = FMath::Max(D, 0.0f);
	}

	// 0=none..1=full, must be >= 0.
	float D = 0.0f;

	// Panini hard vertical compression lerp (0=no vertical compression, 1=hard compression).
	float S = 0.0f;


	/** Add a RDG pass to generate the lens distortion LUT from the settings. */
	FLensDistortionLUT GenerateLUTPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View) const;

	/**
	 * Add a RDG pass to generate the lens distortion LUT from the settings.
	 * Unsafe to due internal argument cast from "FSceneView" to "FViewInfo".
	 */
	RENDERER_API FLensDistortionLUT GenerateLUTPassesUnsafe(FRDGBuilder& GraphBuilder, const FSceneView& InView) const;
};

namespace LensDistortion
{
	// Possible pass locations of the lens distortion application in post-processing.
	enum class EPassLocation : uint8
	{
		TSR,
		PrimaryUpscale
	};

	/**
	 * Get the pass location of the lens distortion application in post-processing.
	 *
	 * @param InViewInfo Active view info.
	 * @return Location enum.
	 */
	EPassLocation GetPassLocation(const FViewInfo& InViewInfo);

	/**
	 * Get the pass location of the lens distortion application in post-processing.
	 * Unsafe to due internal argument cast from "FSceneView" to "FViewInfo".
	 *
	 * @param InView Active view info.
	 * @return EPassLocation enum.
	 */
	RENDERER_API EPassLocation GetPassLocationUnsafe(const FSceneView& InView);

	/**
	 * Get the view lens distortion LUT.
	 * Unsafe to due internal argument cast from "FSceneView" to "FViewInfo".
	 *
	 * @param InView Active view info.
	 * @return FLensDistortionLUT texture struct.
	 */
	RENDERER_API const FLensDistortionLUT& GetLUTUnsafe(const FSceneView& InView);

	/**
	 * Set the view lens distortion LUT.
	 * Unsafe to due internal argument cast from "FSceneView" to "FViewInfo".
	 *
	 * @param InView Active view info.
	 * @param DistortionLUT The LUT object to set on the FViewInfo
	 */
	RENDERER_API void SetLUTUnsafe(FSceneView& InView, const FLensDistortionLUT& DistortionLUT);

} // end namespace LensDistortion

