// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

class INeuralPostProcessInterface
{
public:

	virtual ~INeuralPostProcessInterface() = default;

	virtual void Apply(FRDGBuilder& GraphBuilder, int32 NeuralProfileId,
		FRDGTexture* NeuralTexture, FIntRect ViewRect, FRDGBufferRef InputSourceType,
		FRDGBufferRef& OutputNeuralBuffer, FVector4f& BufferDimension) = 0;

	virtual void AllocateBuffer(FRDGBuilder& GraphBuilder, const FScreenPassTextureViewport& Viewport,
		int32 NeuralProfileId, FRDGBufferRef& InputNeuralBuffer, FVector4f& InputBufferDimension) = 0;
};

extern RENDERER_API TUniquePtr<INeuralPostProcessInterface> GNeuralPostProcess;
