// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "OverridePassSequence.h"

enum class EFXAAQuality : uint32
{
	// Lowest Quality / Fastest
	Q0,
	Q1,
	Q2,
	Q3,
	Q4,
	Q5,
	// Highest Quality / Slowest
	MAX
};

EFXAAQuality GetFXAAQuality();

struct FFXAAInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] HDR scene color to filter.
	FScreenPassTexture SceneColor;

	// FXAA filter quality.
	EFXAAQuality Quality = EFXAAQuality::MAX;
};

FScreenPassTexture RENDERER_API AddFXAAPass(FRDGBuilder& GraphBuilder, const FSceneView& View, const FFXAAInputs& Inputs);