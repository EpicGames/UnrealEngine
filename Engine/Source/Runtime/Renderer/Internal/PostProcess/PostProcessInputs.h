// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TranslucentPassResource.h"
#include "PathTracingResources.h"

struct FPostProcessingInputs
{
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures = nullptr;
	FRDGTextureRef ViewFamilyTexture = nullptr;
	FRDGTextureRef ViewFamilyDepthTexture = nullptr;
	FRDGTextureRef CustomDepthTexture = nullptr;
	FRDGTextureRef ExposureIlluminance = nullptr;
	FRDGTextureRef OriginalSceneDepthForStencil = nullptr;
	FTranslucencyViewResourcesMap TranslucencyViewResourcesMap;
	FPathTracingResources PathTracingResources;

	bool bSeparateCustomStencil = false;

	void Validate() const
	{
		check(SceneTextures);
		check(ViewFamilyTexture);
		check(TranslucencyViewResourcesMap.IsValid());
	}
};
