// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "OverridePassSequence.h"

enum class ESMAAQuality : uint32
{
	// Lowest Quality / Fastest
	Q0,
	Q1,
	Q2,
	Q3,
	// Highest Quality / Slowest
	MAX
};

ESMAAQuality GetSMAAQuality();

struct FSMAAInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] HDR scene color to filter.
	FScreenPassTexture SceneColor;

	FScreenPassTexture SceneColorBeforeTonemap;

	// FXAA filter quality.
	ESMAAQuality Quality = ESMAAQuality::MAX;
};

FScreenPassTexture RENDERER_API AddSMAAPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FSMAAInputs& Inputs);


