// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneView.h"
#include "RenderGraphUtils.h"
#include "CustomDepthRendering.h"
#include "SceneRenderTargetParameters.h"
#include "GBufferInfo.h"
#include "ScreenPass.h"

struct FDBufferTextures;
struct FSceneTextures;
struct FSubstrateSceneData;
class FViewInfo;
class FViewFamilyInfo;

/** Initializes a scene textures config instance from the view family. */
extern RENDERER_API void InitializeSceneTexturesConfig(FSceneTexturesConfig& Config, const FSceneViewFamily& ViewFamily, FIntPoint ExtentOverride = FIntPoint(0,0));

struct FTransientUserSceneTexture
{
	FRDGTextureRef Texture{};
	FIntPoint ResolutionDivisor;
	uint16 AllocationOrder;				// Order in which item of a given Name was allocated, mainly for differentiating items in texture visualizer
	bool bUsed;							// Tracks whether output was used as an input, for debugging
	uint32 ViewMask;					// Tracks which views this texture has been rendered in, so we can detect the first render in a given view
};

#if !(UE_BUILD_SHIPPING)
enum class EUserSceneTextureEvent
{
	MissingInput,
	CollidingInput,		// Input matches the output, and has been unbound as a result
	FoundInput,
	Output,
	Pass,				// Marker for the end of events for a given material pass with UserSceneTexture inputs or outputs
	CustomRenderPass	// Marker for a custom render pass that writes to a UserSceneTexture -- AllocationOrder contains ERenderOutput enum, MaterialInterface contains CustomRenderPassBase pointer
};

struct FUserSceneTextureEventData
{
	EUserSceneTextureEvent Event;
	FName Name;
	uint16 AllocationOrder;							// ERenderOutput stored here for EUserSceneTextureEvent::CustomRenderPass
	uint16 ViewIndex;								// Necessary to differentiate events from multiple views in split screen
	const UMaterialInterface* MaterialInterface;	// FCustomRenderPassBase stored here for EUserSceneTextureEvent::CustomRenderPass
	FIntPoint RectSize;								// Only filled in for EUserSceneTextureEvent::Output
};
#endif

/** RDG struct containing the minimal set of scene textures common across all rendering configurations. */
struct FMinimalSceneTextures
{
	// Initializes the minimal scene textures structure in the FViewFamilyInfo
	static RENDERER_API void InitializeViewFamily(FRDGBuilder& GraphBuilder, FViewFamilyInfo& ViewFamily);

	// Structure may be pointed to by multiple FViewFamilyInfo during scene rendering, through CustomRenderPasses.  The Owner
	// handles deleting the structure when the scene renderer is destroyed.  TRefCountPtr doesn't work, because the structure
	// is also copied by value, and the copy constructor is disabled for reference counted structures.
	const FViewFamilyInfo* Owner = nullptr;
	bool bIsSceneTexturesInitialized = false;

	// Immutable copy of the config used to create scene textures.
	FSceneTexturesConfig Config;

	// Uniform buffers for deferred or mobile.
	TRDGUniformBufferRef<FSceneTextureUniformParameters> UniformBuffer{};
	TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> MobileUniformBuffer{};

	// Setup modes used when creating uniform buffers. These are updated on demand.
	ESceneTextureSetupMode SetupMode = ESceneTextureSetupMode::None;
	EMobileSceneTextureSetupMode MobileSetupMode = EMobileSceneTextureSetupMode::None;

	// Texture containing scene color information with lighting but without post processing. Will be two textures if MSAA.
	FRDGTextureMSAA Color{};

	// Texture containing scene depth. Will be two textures if MSAA.
	FRDGTextureMSAA Depth{};

	// Texture containing a stencil view of the resolved (if MSAA) scene depth. 
	FRDGTextureSRVRef Stencil{};

	// Textures containing primary depth buffer copied before other meshes are rendered in the secondary depth pass.
	FRDGTextureMSAA PartialDepth{};

	// Textures containing depth / stencil information from the custom depth pass.
	FCustomDepthTextures CustomDepth{};

	// Dynamically allocated user scene textures, stored by name.  An array of textures per name is used, as it's possible the
	// same name is allocated with different resolution divisors.  The most recently written texture resolution with a given name
	// will be used as an input to other materials, by swapping to the front of the array.
	mutable TMap<FName, TArray<FTransientUserSceneTexture>> UserSceneTextures;

#if !(UE_BUILD_SHIPPING)
	mutable TArray<FUserSceneTextureEventData> UserSceneTextureEvents;
#endif

	RENDERER_API FSceneTextureShaderParameters GetSceneTextureShaderParameters(ERHIFeatureLevel::Type FeatureLevel) const;

	FRDGTextureRef FindOrAddUserSceneTexture(FRDGBuilder& GraphBuilder, int32 ViewIndex, FName Name, FIntPoint ResolutionDivisor, bool& bOutFirstRender, const UMaterialInterface* MaterialInterface, const FIntRect& OutputRect) const;
	FScreenPassTextureSlice GetUserSceneTexture(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 ViewIndex, FName Name, const UMaterialInterface* MaterialInterface) const;
	FIntPoint GetUserSceneTextureDivisor(FName Name) const;

#if !(UE_BUILD_SHIPPING)
	const FTransientUserSceneTexture* FindUserSceneTextureByEvent(const FUserSceneTextureEventData& Event) const;
#endif
};

/** RDG struct containing the complete set of scene textures for the deferred or mobile renderers. */
struct FSceneTextures : public FMinimalSceneTextures
{
	// Initializes the scene textures structure in the FViewFamilyInfo
	static RENDERER_API void InitializeViewFamily(FRDGBuilder& GraphBuilder, FViewFamilyInfo& ViewFamily, FIntPoint FamilySize);
	static RENDERER_API EPixelFormat GetGBufferFFormatAndCreateFlags(ETextureCreateFlags& OutCreateFlags);

	// Configures an array of render targets for the GBuffer pass.
	RENDERER_API uint32 GetGBufferRenderTargets(
		TArrayView<FTextureRenderTargetBinding> RenderTargets,
		EGBufferLayout Layout = GBL_Default) const;
	RENDERER_API uint32 GetGBufferRenderTargets(
		ERenderTargetLoadAction LoadAction,
		FRenderTargetBindingSlots& RenderTargets,
		EGBufferLayout Layout = GBL_Default) const;

	// Returns list of valid textures in this structure
	RENDERER_API TArray<FRDGTextureRef> EnumerateSceneTextures() const;
	
	// (Deferred) Texture containing conservative downsampled depth for occlusion.
	FRDGTextureRef SmallDepth{};

	// (Deferred) Textures containing geometry information for deferred shading.
	FRDGTextureRef GBufferA{};
	FRDGTextureRef GBufferB{};
	FRDGTextureRef GBufferC{};
	FRDGTextureRef GBufferD{};
	FRDGTextureRef GBufferE{};
	FRDGTextureRef GBufferF{};
	FRDGTextureRef GBufferSGGX{};

	// Texture containing dithered downsampled half res depth for occlusion.
	FRDGTextureRef DitheredHalfResDepth{};

	// Additional Buffer texture used by mobile
	FRDGTextureMSAA DepthAux{};

	// Texture containing dynamic motion vectors. Can be bound by the base pass or its own velocity pass.
	FRDGTextureRef Velocity{};

	// Texture containing the screen space ambient occlusion result.
	FRDGTextureRef ScreenSpaceAO{};

	// Additional texture used by some debug view mode when enabled.
	FRDGTextureRef DebugAux{};

	// (Mobile Stereo) Textures containing motion vectors and depth for OpenXR frame synthesis, using Vulkan-NDC format rather than UE's encoded velocity format.
	// The size of both textures is determined independently of the view family size, via IStereoRenderTargetManager::GetRecommendedMotionVectorTextureSize.
	FRDGTextureRef StereoMotionVectors{};
	FRDGTextureRef StereoMotionVectorDepth{};

	// Textures containing the accumulated diffuse luminance from SceneColor for efficient SSS.
	FRDGTextureRef SeparatedSubsurfaceDiffuse{};

	// Textures used to composite editor primitives. Also used by the base pass when in wireframe mode.
#if WITH_EDITOR
	FRDGTextureRef EditorPrimitiveColor{};
	FRDGTextureRef EditorPrimitiveDepth{};
#endif

	// The following private structures are allocated in FSceneTextures::InitializeViewFamily.
	// Decal related textures.
	TPimplPtr<FDBufferTextures, EPimplPtrMode::DeepCopy> DBufferTextures;

	// Substrate data shared between all views.  Contains Substrate specific GBuffer textures, corresponding to those in FSceneTextures, in addition to general settings.
	TPimplPtr<FSubstrateSceneData, EPimplPtrMode::DeepCopy> SubstrateSceneData;
};

/** Extracts scene textures into the global extraction instance. */
void QueueSceneTextureExtractions(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);
