// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "RHIResources.h"

class FRHIGPUTextureReadback;
struct IPooledRenderTarget;

struct FSceneViewStateSystemMemoryTexture
{
	FRHITextureDesc Desc;
	const TCHAR* DebugName;
	TUniquePtr<FRHIGPUTextureReadback> Readback;
	TMap<int32, TArray<uint8>> Instances;
};

// Context to handle mirroring of scene view state textures and buffers to system memory.  Provides a mechanism for very high
// resolution tiled rendering, beyond what can fit in GPU memory.
struct FSceneViewStateSystemMemoryMirror
{
	// Key is an offset in the FSceneView structure for the texture reference being mirrored.  Value is an array of 
	// unique texture descriptions, and readback and storage for instances of those textures per view key.  Assumption
	// is that with tiled rendering, all the tiles have the same resolution, and a single readback buffer can be shared
	// for all of them.
	TMap<int64, TArray<FSceneViewStateSystemMemoryTexture>> TextureMirrors;

	// Transient used internally
	TArray<TRefCountPtr<IPooledRenderTarget>> TemporaryTextures;
};
