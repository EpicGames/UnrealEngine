// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "SystemTextures.h"


// enum instead of bool to get better visibility when we pass around multiple bools, also allows for easier extensions
namespace ETranslucencyPass
{
	enum Type : int
	{
		TPT_TranslucencyStandard,
		TPT_TranslucencyStandardModulate,
		TPT_TranslucencyAfterDOF,
		TPT_TranslucencyAfterDOFModulate,
		TPT_TranslucencyAfterMotionBlur,
		TPT_TranslucencyHoldout,
		/** Drawing all translucency, regardless of separate or standard.  Used when drawing translucency outside of the main renderer, eg FRendererModule::DrawTile. */
		TPT_AllTranslucency,
		TPT_MAX
	};
};

/** Resources of a translucency pass. */
struct FTranslucencyPassResources
{
	ETranslucencyPass::Type Pass = ETranslucencyPass::TPT_MAX;
	FIntRect ViewRect = FIntRect(0, 0, 0, 0);
	FRDGTextureMSAA ColorTexture;
	FRDGTextureMSAA ColorModulateTexture;
	FRDGTextureMSAA DepthTexture;

	inline bool IsValid() const
	{
		return ViewRect.Width() > 0 && ViewRect.Height() > 0;
	}

	inline FRDGTextureRef GetColorForRead(FRDGBuilder& GraphBuilder) const
	{
		if (!ColorTexture.IsValid())
		{
			return GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder);
		}
		return ColorTexture.Resolve;
	}

	inline FRDGTextureRef GetColorModulateForRead(FRDGBuilder& GraphBuilder) const
	{
		if (!ColorModulateTexture.IsValid())
		{
			return GSystemTextures.GetWhiteDummy(GraphBuilder);
		}
		return ColorModulateTexture.Resolve;
	}

	inline FRDGTextureRef GetDepthForRead(FRDGBuilder& GraphBuilder) const
	{
		if (!DepthTexture.IsValid())
		{
			return GSystemTextures.GetMaxFP16Depth(GraphBuilder);
		}
		return DepthTexture.Resolve;
	}

	inline FScreenPassTextureViewport GetTextureViewport() const
	{
		check(IsValid());
		return FScreenPassTextureViewport(ColorTexture.Target->Desc.Extent, ViewRect);
	}
};

/** All resources of all translucency passes for a view family. */
struct FTranslucencyPassResourcesMap
{
	FTranslucencyPassResourcesMap(int32 NumViews);

	inline FTranslucencyPassResources& Get(int32 ViewIndex, ETranslucencyPass::Type Translucency)
	{
		return Array[ViewIndex][int32(Translucency)];
	};

	inline const FTranslucencyPassResources& Get(int32 ViewIndex, ETranslucencyPass::Type Translucency) const
	{
		check(ViewIndex < Array.Num());
		return Array[ViewIndex][int32(Translucency)];
	};

private:
	TArray<TStaticArray<FTranslucencyPassResources, ETranslucencyPass::TPT_MAX>, TInlineAllocator<4>> Array;
};

/** All resources of all translucency for one view. */
struct FTranslucencyViewResourcesMap
{
	FTranslucencyViewResourcesMap() = default;

	FTranslucencyViewResourcesMap(const FTranslucencyPassResourcesMap& InTranslucencyPassResourcesMap, int32 InViewIndex)
		: TranslucencyPassResourcesMap(&InTranslucencyPassResourcesMap)
		, ViewIndex(InViewIndex)
	{ }

	bool IsValid() const
	{
		return TranslucencyPassResourcesMap != nullptr;
	}

	inline const FTranslucencyPassResources& Get(ETranslucencyPass::Type Translucency) const
	{
		check(IsValid());
		return TranslucencyPassResourcesMap->Get(ViewIndex, Translucency);
	};

private:
	const FTranslucencyPassResourcesMap* TranslucencyPassResourcesMap = nullptr;
	int32 ViewIndex = 0;
};
