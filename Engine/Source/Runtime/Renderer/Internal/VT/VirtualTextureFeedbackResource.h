// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"

class FRDGBuffer;
class FRDGBuilder;
class FViewUniformShaderParameters;

namespace VirtualTexture
{
	/** Shader parameters needed when writing virtual texture feedback from GPU. */
	struct FFeedbackShaderParams
	{
		FRHIUnorderedAccessView* BufferUAV = nullptr;
		uint32 BufferSize = 0;
		uint32 TileShift = 0;
		uint32 TileMask = 0;
		uint32 TileJitterOffset = 0;
		uint32 SampleOffset = 0;
		uint32 ExtendedDebugBufferSize = 0;
	};

	/** 
	 * Begin a virtual texture feedback scope. 
	 * The feedback buffer size will be InBufferSize. If InBufferSize is 0 a default size is used.
	 * Note that there is only one global feedback buffer alive at any time and its buffer size is later used internally by GetFeedbackShaderParams().
	 */
	RENDERER_API void BeginFeedback(FRDGBuilder& GraphBuilder, uint32 InBufferSize = 0, ERHIFeatureLevel::Type InFeatureLevel = GMaxRHIFeatureLevel);

	/** 
	 * Begin a virtual texture feedback scope. 
	 * InViewportSize and InVirtualTextureFeedbackTileSize are used to calculate the feedback buffer size. 
	 * If bInExtendFeedbackForDebug is true we will allocate room at the end of the feedback buffer to capture extra debug info.
	 * Note that there is only one global feedback buffer alive at any time and its buffer size is later used internally by GetFeedbackShaderParams().
	 */
	RENDERER_API void BeginFeedback(FRDGBuilder& GraphBuilder, FIntPoint InViewportSize, uint32 InVirtualTextureFeedbackTileSize, bool bInExtendFeedbackForDebug, ERHIFeatureLevel::Type InFeatureLevel);

	/** End a virtual texture feedback scope. */
	RENDERER_API void EndFeedback(FRDGBuilder& GraphBuilder);

	/** 
	 * Get virtual texture feedback parameters to use for binding to any shader that is sampling virtual textures. 
	 * This is only valid within a BeginFeedback()/EndFeedback() scope.
	 */
	RENDERER_API void GetFeedbackShaderParams(int32 InFrameIndex, int32 InVirtualTextureFeedbackTileSize, FFeedbackShaderParams& OutParams);
	
	/** 
	 * Get virtual texture feedback parameters to use for binding to any shader that is sampling virtual textures. 
	 * This version uses an internal scene frame counter, and the project default feedback tile size.
	 * This is only valid within a BeginFeedback()/EndFeedback() scope.
	 */
	RENDERER_API void GetFeedbackShaderParams(FFeedbackShaderParams& OutParams);
	
	/** Helper to copy the virtual texture feedback shader parameters into the view parameters to be bound in the view uniform buffer. */
	RENDERER_API void UpdateViewUniformShaderParameters(FFeedbackShaderParams const& InParams, FViewUniformShaderParameters& ViewUniformShaderParameters);

	/** 
	 * This resolves any extended debug information that is currently stored at the end of the feedback buffer and returns a buffer that contains it.
	 * This is only valid within a BeginFeedback()/EndFeedback() scope.
	 */
	FRDGBuffer* ResolveExtendedDebugBuffer(FRDGBuilder& GraphBuilder);
}
