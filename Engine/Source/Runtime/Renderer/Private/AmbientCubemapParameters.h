// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AmbientCubemapParameters.h: Shared shader parameters for ambient cubemap
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "FinalPostProcessSettings.h"
#include "ShaderParameterMacros.h"

/** Shader parameters needed for deferred passes sampling the ambient cube map. */
BEGIN_SHADER_PARAMETER_STRUCT(FAmbientCubemapParameters, )
	SHADER_PARAMETER(FLinearColor, AmbientCubemapColor)
	SHADER_PARAMETER(FVector4f, AmbientCubemapMipAdjust)
	SHADER_PARAMETER_TEXTURE(TextureCube, AmbientCubemap)
	SHADER_PARAMETER_SAMPLER(SamplerState, AmbientCubemapSampler)
END_SHADER_PARAMETER_STRUCT()

void SetupAmbientCubemapParameters(const FFinalPostProcessSettings::FCubemapEntry& Entry, FAmbientCubemapParameters* OutParameters);
