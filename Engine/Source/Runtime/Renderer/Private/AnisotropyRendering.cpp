// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnisotropyRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PrimitiveSceneProxy.h"
#include "MeshPassProcessor.inl"
#include "ScenePrivate.h"
#include "DeferredShadingRenderer.h"
#include "RenderCore.h"

DECLARE_GPU_STAT_NAMED(RenderAnisotropyPass, TEXT("Render Anisotropy Pass"));

static int32 GAnisotropicMaterials = 0;
static FAutoConsoleVariableRef CVarAnisotropicMaterials(
	TEXT("r.AnisotropicMaterials"),
	GAnisotropicMaterials,
	TEXT("Whether anisotropic BRDF is used for material with anisotropy."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

bool SupportsAnisotropicMaterials(ERHIFeatureLevel::Type FeatureLevel, EShaderPlatform ShaderPlatform)
{
	return GAnisotropicMaterials
		&& FeatureLevel >= ERHIFeatureLevel::SM5
		&& FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(ShaderPlatform);
}

bool SupportsAnisotropicMaterialPass(ERHIFeatureLevel::Type FeatureLevel, EShaderPlatform ShaderPlatform)
{
		const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
		return SupportsAnisotropicMaterials(FeatureLevel, ShaderPlatform)
			&& (!bSubstrateEnabled || (bSubstrateEnabled && Substrate::IsSubstrateBlendableGBufferEnabled(ShaderPlatform))); // Substrate renders anisotropy surface natively, without extra pass. Unless blendable gbuffer is used.
}

static bool IsAnisotropyPassCompatible(const EShaderPlatform Platform, FMaterialShaderParameters MaterialParameters)
{
	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
	return
		FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Platform) &&
		MaterialParameters.bHasAnisotropyConnected &&
		!IsTranslucentBlendMode(MaterialParameters) &&
		MaterialParameters.ShadingModels.HasAnyShadingModel({ MSM_DefaultLit, MSM_ClearCoat, MSM_Strata }) &&
		(!bSubstrateEnabled || (bSubstrateEnabled && Substrate::IsSubstrateBlendableGBufferEnabled(Platform))); // Substrate renders anisotropy surface natively, without extra pass. Unless blendable gbuffer is used.
}

class FAnisotropyVS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FAnisotropyVS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		// Compile if supported by the hardware.
		const bool bIsFeatureSupported = IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);

		return 
			bIsFeatureSupported && 
			IsAnisotropyPassCompatible(Parameters.Platform, Parameters.MaterialParameters) &&
			FMeshMaterialShader::ShouldCompilePermutation(Parameters);
	}

	FAnisotropyVS() = default;
	FAnisotropyVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}
};

class FAnisotropyPS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FAnisotropyPS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return FAnisotropyVS::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	FAnisotropyPS() = default;
	FAnisotropyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}
};

IMPLEMENT_SHADER_TYPE(, FAnisotropyVS, TEXT("/Engine/Private/AnisotropyPassShader.usf"), TEXT("MainVertexShader"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(, FAnisotropyPS, TEXT("/Engine/Private/AnisotropyPassShader.usf"), TEXT("MainPixelShader"), SF_Pixel);
IMPLEMENT_SHADERPIPELINE_TYPE_VSPS(AnisotropyPipeline, FAnisotropyVS, FAnisotropyPS, true);

FAnisotropyMeshProcessor::FAnisotropyMeshProcessor(
	const FScene* Scene, 
	ERHIFeatureLevel::Type InFeatureLevel,
	const FSceneView* InViewIfDynamicMeshCommand,
	const FMeshPassProcessorRenderState& InPassDrawRenderState, 
	FMeshPassDrawListContext* InDrawListContext
	)
	: FMeshPassProcessor(EMeshPass::AnisotropyPass, Scene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InPassDrawRenderState)
{
}

FMeshPassProcessor* CreateAnisotropyPassProcessor(ERHIFeatureLevel::Type InFeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	const ERHIFeatureLevel::Type FeatureLevel = InViewIfDynamicMeshCommand ? InViewIfDynamicMeshCommand->GetFeatureLevel() : InFeatureLevel;

	FMeshPassProcessorRenderState AnisotropyPassState;

	AnisotropyPassState.SetBlendState(TStaticBlendState<>::GetRHI());
	AnisotropyPassState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Equal>::GetRHI());

	return new FAnisotropyMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, AnisotropyPassState, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(AnisotropyPass, CreateAnisotropyPassProcessor, EShadingPath::Deferred, EMeshPass::AnisotropyPass, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);

bool GetAnisotropyPassShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	TShaderRef<FAnisotropyVS>& VertexShader,
	TShaderRef<FAnisotropyPS>& PixelShader
	)
{
	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.PipelineType = &AnisotropyPipeline;
	ShaderTypes.AddShaderType<FAnisotropyVS>();
	ShaderTypes.AddShaderType<FAnisotropyPS>();

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	check(VertexShader.IsValid() && PixelShader.IsValid());

	return true;
}

static bool ShouldDraw(const FMaterial& Material, bool bMaterialUsesAnisotropy, bool bMaterialIsSubstrateToon)
{
	const bool bIsNotTranslucent = IsOpaqueOrMaskedBlendMode(Material);
	return bMaterialUsesAnisotropy && bIsNotTranslucent && (Material.GetShadingModels().HasAnyShadingModel({ MSM_DefaultLit, MSM_ClearCoat }) || (Material.GetShadingModels().HasAnyShadingModel({ MSM_Strata }) && bMaterialIsSubstrateToon));
}

void FAnisotropyMeshProcessor::AddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch, 
	uint64 BatchElementMask, 
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, 
	int32 StaticMeshId /* = -1 */ 
	)
{
	if (SupportsAnisotropicMaterialPass(FeatureLevel, GShaderPlatformForFeatureLevel[FeatureLevel]) && MeshBatch.bUseForMaterial)
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material)
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

bool FAnisotropyMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	bool bResult = true;
	if (ShouldDraw(Material, Material.MaterialUsesAnisotropy_RenderThread(), uint16(Material.MaterialGetSubstrateMaterialBsdfFeatures_RenderThread() & ESubstrateBsdfFeature::Toon) > 0))
	{
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

		bResult = Process(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, MaterialRenderProxy, Material, MeshFillMode, MeshCullMode);
	}

	return bResult;
}

bool FAnisotropyMeshProcessor::Process(
	const FMeshBatch& MeshBatch, 
	uint64 BatchElementMask, 
	int32 StaticMeshId, 
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy, 
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode, 
	ERasterizerCullMode MeshCullMode 
	)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		FAnisotropyVS,
		FAnisotropyPS> AnisotropyPassShaders;

	if (!GetAnisotropyPassShaders(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		AnisotropyPassShaders.VertexShader,
		AnisotropyPassShaders.PixelShader))
	{
		return false;
	}

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(AnisotropyPassShaders.VertexShader, AnisotropyPassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		AnisotropyPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData
		);

	return true;
}


void FAnisotropyMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	if (ShouldDraw(Material, Material.MaterialUsesAnisotropy_GameThread(), uint16(Material.MaterialGetSubstrateMaterialBsdfFeatures_GameThread() & ESubstrateBsdfFeature::Toon) > 0) &&
		SupportsAnisotropicMaterialPass(FeatureLevel, GShaderPlatformForFeatureLevel[FeatureLevel]))
	{
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

		TMeshProcessorShaders<
			FAnisotropyVS,
			FAnisotropyPS> AnisotropyPassShaders;

		if (!GetAnisotropyPassShaders(
			Material,
			VertexFactoryData.VertexFactoryType,
			FeatureLevel,
			AnisotropyPassShaders.VertexShader,
			AnisotropyPassShaders.PixelShader))
		{
			return;
		}

		FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
		RenderTargetsInfo.NumSamples = 1;

		ETextureCreateFlags GBufferFCreateFlags;
		EPixelFormat GBufferFPixelFormat = FSceneTextures::GetGBufferFFormatAndCreateFlags(GBufferFCreateFlags);
		AddRenderTargetInfo(GBufferFPixelFormat, GBufferFCreateFlags, RenderTargetsInfo);
		SetupDepthStencilInfo(PF_DepthStencil, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilNop, RenderTargetsInfo);

		AddGraphicsPipelineStateInitializer(
			VertexFactoryData,
			Material,
			PassDrawRenderState,
			RenderTargetsInfo,
			AnisotropyPassShaders,
			MeshFillMode,
			MeshCullMode,
			(EPrimitiveType)PreCacheParams.PrimitiveType,
			EMeshPassFeatures::Default,
			true /*bRequired*/,
			PSOInitializers);
	}
}

bool ShouldRenderAnisotropicLighting(const FViewInfo& View)
{
	if (!SupportsAnisotropicMaterials(View.FeatureLevel, View.GetShaderPlatform()))
	{
		return false;
	}

	if (SupportsAnisotropicMaterialPass(View.FeatureLevel, View.GetShaderPlatform()) && View.ShouldRenderView() && !HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::AnisotropyPass]))
	{
		// The setup requires a anisotropy pass for material data but nothing is rendered in it: then skip rendering anisotropy.
		return false;
	}

	return true;
}

bool ShouldRenderAnisotropyPass(const FViewInfo& View)
{
	if (!SupportsAnisotropicMaterialPass(View.FeatureLevel, View.GetShaderPlatform()))
	{
		return false;
	}

	if (IsForwardShadingEnabled(GetFeatureLevelShaderPlatform(View.FeatureLevel)))
	{
		return false;
	}

	// The anisotropy GBuffer is used for lighting, and not needed for custom render passes, which don't run lighting.
	if (View.CustomRenderPass)
	{
		return false;
	}

	if (View.ShouldRenderView() && HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::AnisotropyPass]))
	{
		return true;
	}

	return false;
}

bool ShouldRenderAnisotropyPass(TArrayView<FViewInfo> Views)
{
	for (const FViewInfo& View : Views)
	{
		if (ShouldRenderAnisotropyPass(View))
		{
			return true;
		}
	}

	return false;
}

BEGIN_SHADER_PARAMETER_STRUCT(FAnisotropyPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FDeferredShadingSceneRenderer::RenderAnisotropyPass(
	FRDGBuilder& GraphBuilder, 
	TArrayView<FViewInfo> InViews,
	FSceneTextures& SceneTextures,
	const FScene* Scene,
	bool bDoParallelPass)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderAnisotropyPass);
	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_RenderAnisotropyPass, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_AnisotropyPassDrawTime);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, RenderAnisotropyPass, "RenderAnisotropyPass");

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
	{
		FViewInfo& View = InViews[ViewIndex];

		if (View.ShouldRenderView())
		{
			if (!View.ParallelMeshDrawCommandPasses[EMeshPass::AnisotropyPass])
			{
				continue;
			}

			FParallelMeshDrawCommandPass& ParallelMeshPass = *View.ParallelMeshDrawCommandPasses[EMeshPass::AnisotropyPass];

			if (!ParallelMeshPass.HasAnyDraw())
			{
				continue;
			}

			View.BeginRenderView();

			auto* PassParameters = GraphBuilder.AllocParameters<FAnisotropyPassParameters>();
			PassParameters->View = View.GetShaderParameters();
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilNop);

			ParallelMeshPass.BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);
			if (bDoParallelPass)
			{
				if (ViewIndex == 0)
				{
					AddClearRenderTargetPass(GraphBuilder, SceneTextures.GBufferF);
				}

				PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.GBufferF, ERenderTargetLoadAction::ELoad);

				GraphBuilder.AddDispatchPass(
					RDG_EVENT_NAME("AnisotropyPassParallel"),
					PassParameters,
					ERDGPassFlags::Raster,
					[&View, &ParallelMeshPass, PassParameters](FRDGDispatchPassBuilder& DispatchPassBuilder)
				{
					ParallelMeshPass.Dispatch(DispatchPassBuilder, &PassParameters->InstanceCullingDrawParams);
				});
			}
			else
			{
				PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.GBufferF, ViewIndex == 0 ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("AnisotropyPass"),
					PassParameters,
					ERDGPassFlags::Raster,
					[&View, &ParallelMeshPass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					SetStereoViewport(RHICmdList, View);

					ParallelMeshPass.Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
				});
			}
		}
	}
}
