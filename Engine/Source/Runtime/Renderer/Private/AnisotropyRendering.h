// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RendererInterface.h"
#include "MeshPassProcessor.h"

class FScene;

class FAnisotropyMeshProcessor : public FSceneRenderingAllocatorObject<FAnisotropyMeshProcessor>, public FMeshPassProcessor
{
public:
	FAnisotropyMeshProcessor(
		const FScene* Scene,
		ERHIFeatureLevel::Type InFeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand,
		const FMeshPassProcessorRenderState& InPassDrawRenderState,
		FMeshPassDrawListContext* InDrawListContext
		);

	FMeshPassProcessorRenderState PassDrawRenderState;

	virtual void AddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId = -1
		) override final;

	virtual void CollectPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams, 
		TArray<FPSOPrecacheData>& PSOInitializers) override final;

protected:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode
		);
};

bool ShouldRenderAnisotropicLighting(const FViewInfo& View);
bool ShouldRenderAnisotropyPass(const FViewInfo& View);
bool ShouldRenderAnisotropyPass(TArrayView<FViewInfo> Views);