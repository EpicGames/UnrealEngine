// Copyright Epic Games, Inc. All Rights Reserved.

#include "BasePassRendering.h"
#include "CoreMinimal.h"
#include "AnisotropyRendering.h"
#include "BatchedElements.h"
#include "ClearQuad.h"
#include "Containers/ArrayView.h"
#include "Containers/IndirectArray.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DBufferTextures.h"
#include "DebugProbeRendering.h"
#include "DebugViewModeHelpers.h"
#include "DebugViewModeRendering.h"
#include "DeferredShadingRenderer.h"
#include "DepthRendering.h"
#include "EditorPrimitivesRendering.h"
#include "Engine/EngineTypes.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "FogRendering.h"
#include "GBufferInfo.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "LightFunctionAtlas.h"
#include "LightMapDensityRendering.h"
#include "LightMapRendering.h"
#include "LocalFogVolumeRendering.h"
#include "Logging/LogMacros.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "MeshBatch.h"
#include "MeshDrawCommands.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "Misc/LargeWorldCoordinates.h"
#include "MultiGPU.h"
#include "Nanite/NaniteCullRaster.h"
#include "Nanite/NaniteShading.h"
#include "Nanite/NaniteShared.h"
#include "Nanite/NaniteVisualize.h"
#include "OIT/OIT.h"
#include "OneColorShader.h"
#include "PipelineStateCache.h"
#include "PixelFormat.h"
#include "PlanarReflectionRendering.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "PSOPrecache.h"
#include "PSOPrecacheMaterial.h"
#include "PSOPrecacheSettings.h"
#include "ReflectionEnvironment.h"
#include "RenderCore.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneCore.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneTypes.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "Shader.h"
#include "ShaderCompiler.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShadowRendering.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "TranslucentPassResource.h"
#include "TranslucentRendering.h"
#include "UniformBuffer.h"
#include "VariableRateShadingImageManager.h"
#include "VelocityRendering.h"
#include "VertexFactory.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"

#include "BasePassRendering.inl"

// Instantiate the common policies
template class TBasePassVertexShaderPolicyParamType<FUniformLightMapPolicy>;
template class TBasePassPixelShaderPolicyParamType<FUniformLightMapPolicy>;
template class TBasePassComputeShaderPolicyParamType<FUniformLightMapPolicy>;

// Changing this causes a full shader recompile
static TAutoConsoleVariable<int32> CVarSelectiveBasePassOutputs(
	TEXT("r.SelectiveBasePassOutputs"),
	0,
	TEXT("Enables shaders to only export to relevant rendertargets.\n") \
	TEXT(" 0: Export in all rendertargets.\n") \
	TEXT(" 1: Export only into relevant rendertarget.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

// Changing this causes a full shader recompile
static TAutoConsoleVariable<int32> CVarGlobalClipPlane(
	TEXT("r.AllowGlobalClipPlane"),
	0,
	TEXT("Enables mesh shaders to support a global clip plane, needed for planar reflections, which adds about 15% BasePass GPU cost on PS4."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

// Changing this causes a full shader recompile
static TAutoConsoleVariable<int32> CVarVertexFoggingForOpaque(
	TEXT("r.VertexFoggingForOpaque"),
	1,
	TEXT("Causes opaque materials to use per-vertex fogging, which costs less and integrates properly with MSAA.  Only supported with forward shading."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSupportStationarySkylight(
	TEXT("r.SupportStationarySkylight"),
	1,
	TEXT("Enables Stationary Skylight support for opaque base pass shaders."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSupportLowQualityLightmaps(
	TEXT("r.SupportLowQualityLightmaps"),
	1,
	TEXT("Support low quality lightmap shader permutations"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSupportAllShaderPermutations(
	TEXT("r.SupportAllShaderPermutations"),
	0,
	TEXT("Local user config override to force all shader permutation features on."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarParallelBasePass(
	TEXT("r.ParallelBasePass"),
	1,
	TEXT("Toggles parallel base pass rendering. Parallel rendering must be enabled for this to have an effect."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarClearGBufferDBeforeBasePass(
	TEXT("r.ClearGBufferDBeforeBasePass"),
	1,
	TEXT("Whether to clear GBuffer D before basepass"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarAllow128BitBPPSCompilation(
	TEXT("r.128BitBPPSCompilation.Allow"),
	true,
	TEXT("Whether or not to allow 128 Bit BasePass Pixel Shader compilation on platforms which require them.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

// Scene color alpha is used during scene captures and planar reflections.  1 indicates background should be shown, 0 indicates foreground is fully present.
static const float kSceneColorClearAlpha = 1.0f;

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSharedBasePassUniformParameters, "BasePass");
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FOpaqueBasePassUniformParameters, "OpaqueBasePass", SceneTextures);
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FTranslucentBasePassUniformParameters, "TranslucentBasePass", SceneTextures);

// Typedef is necessary because the C preprocessor thinks the comma in the template parameter list is a comma in the macro parameter list.
#define IMPLEMENT_BASEPASS_MESHSHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	typedef TBasePassMS< LightMapPolicyType > TBasePassMS##LightMapPolicyName ; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TBasePassMS##LightMapPolicyName,TEXT("/Engine/Private/BasePassVertexShader.usf"),TEXT("Main"),SF_Mesh);

#define IMPLEMENT_BASEPASS_VERTEXSHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	typedef TBasePassVS< LightMapPolicyType > TBasePassVS##LightMapPolicyName ; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TBasePassVS##LightMapPolicyName,TEXT("/Engine/Private/BasePassVertexShader.usf"),TEXT("Main"),SF_Vertex);

#define IMPLEMENT_BASEPASS_PIXELSHADER_TYPE(LightMapPolicyType,LightMapPolicyName,GBufferLayout,LayoutName) \
	typedef TBasePassPS<LightMapPolicyType, GBufferLayout> TBasePassPS##LightMapPolicyName##LayoutName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TBasePassPS##LightMapPolicyName##LayoutName,TEXT("/Engine/Private/BasePassPixelShader.usf"),TEXT("MainPS"),SF_Pixel);

#define IMPLEMENT_BASEPASS_COMPUTESHADER_TYPE(LightMapPolicyType,LightMapPolicyName,bVoxel,VoxelName,bCurve,CurveName) \
	typedef TBasePassCS<LightMapPolicyType, bVoxel, bCurve, SF_Compute> TBasePassCS##LightMapPolicyName##VoxelName##CurveName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TBasePassCS##LightMapPolicyName##VoxelName##CurveName,TEXT("/Engine/Private/BasePassPixelShader.usf"),TEXT("MainCS"),SF_Compute);

#define IMPLEMENT_BASEPASS_WORKGRAPHSHADER_TYPE(LightMapPolicyType,LightMapPolicyName,bVoxel,VoxelName,bCurve,CurveName) \
	typedef TBasePassCS<LightMapPolicyType, bVoxel, bCurve, SF_WorkGraphComputeNode> TBasePassWorkGraphCS##LightMapPolicyName##VoxelName##CurveName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TBasePassWorkGraphCS##LightMapPolicyName##VoxelName##CurveName,TEXT("/Engine/Private/BasePassPixelShader.usf"),TEXT("MainCS"),SF_WorkGraphComputeNode);

// Implement pixel and compute shader types for valid permutations, and one vertex shader that will be shared between them
#define IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	IMPLEMENT_BASEPASS_MESHSHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	IMPLEMENT_BASEPASS_VERTEXSHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	IMPLEMENT_BASEPASS_PIXELSHADER_TYPE(LightMapPolicyType,LightMapPolicyName,GBL_Default,) \
	IMPLEMENT_BASEPASS_PIXELSHADER_TYPE(LightMapPolicyType,LightMapPolicyName,GBL_ForceVelocity,ForceVelocity) \
	IMPLEMENT_BASEPASS_COMPUTESHADER_TYPE  (LightMapPolicyType,LightMapPolicyName,false,     ,true, Curve) \
	IMPLEMENT_BASEPASS_COMPUTESHADER_TYPE  (LightMapPolicyType,LightMapPolicyName,true, Voxel,false,     ) \
	IMPLEMENT_BASEPASS_COMPUTESHADER_TYPE  (LightMapPolicyType,LightMapPolicyName,false,     ,false,     ) \
	IMPLEMENT_BASEPASS_WORKGRAPHSHADER_TYPE(LightMapPolicyType,LightMapPolicyName,false,     ,true, Curve) \
	IMPLEMENT_BASEPASS_WORKGRAPHSHADER_TYPE(LightMapPolicyType,LightMapPolicyName,true, Voxel,false,     ) \
	IMPLEMENT_BASEPASS_WORKGRAPHSHADER_TYPE(LightMapPolicyType,LightMapPolicyName,false,     ,false,     )

// Implement shader types per lightmap policy
// If renaming or refactoring these, remember to update FMaterialResource::GetRepresentativeInstructionCounts and FPreviewMaterial::ShouldCache().
IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( FSelfShadowedTranslucencyPolicy, FSelfShadowedTranslucencyPolicy );
IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( FSelfShadowedCachedPointIndirectLightingPolicy, FSelfShadowedCachedPointIndirectLightingPolicy );
IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( FSelfShadowedVolumetricLightmapPolicy, FSelfShadowedVolumetricLightmapPolicy );

IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( TUniformLightMapPolicy<LMP_NO_LIGHTMAP>, FNoLightMapPolicy );
IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( TUniformLightMapPolicy<LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING>, FPrecomputedVolumetricLightmapLightingPolicy );
IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( TUniformLightMapPolicy<LMP_CACHED_VOLUME_INDIRECT_LIGHTING>, FCachedVolumeIndirectLightingPolicy );
IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( TUniformLightMapPolicy<LMP_CACHED_POINT_INDIRECT_LIGHTING>, FCachedPointIndirectLightingPolicy );
IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( TUniformLightMapPolicy<LMP_LQ_LIGHTMAP>, TLightMapPolicyLQ );
IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( TUniformLightMapPolicy<LMP_HQ_LIGHTMAP>, TLightMapPolicyHQ );
IMPLEMENT_BASEPASS_LIGHTMAPPED_SHADER_TYPE( TUniformLightMapPolicy<LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP>, TDistanceFieldShadowsAndLightMapPolicyHQ  );

F128BitRTBasePassPS::F128BitRTBasePassPS() = default;
F128BitRTBasePassPS::F128BitRTBasePassPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
	TBasePassPS<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>, GBL_Default>(Initializer)
{}

bool F128BitRTBasePassPS::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return FDataDrivenShaderPlatformInfo::GetRequiresExplicit128bitRT(Parameters.Platform) && 
		   Allow128BitBasePassPSCompilation(Parameters.Platform);
}

void F128BitRTBasePassPS::ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetRenderTargetOutputFormat(0, PF_A32B32G32R32F);
	TBasePassPS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

IMPLEMENT_MATERIAL_SHADER_TYPE(, F128BitRTBasePassPS, TEXT("/Engine/Private/BasePassPixelShader.usf"), TEXT("MainPS"), SF_Pixel);

DEFINE_GPU_DRAWCALL_STAT(Basepass);

DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer ClearGBufferAtMaxZ"), STAT_FDeferredShadingSceneRenderer_ClearGBufferAtMaxZ, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer ViewExtensionPostRenderBasePass"), STAT_FDeferredShadingSceneRenderer_ViewExtensionPostRenderBasePass, STATGROUP_SceneRendering);

DEFINE_GPU_STAT(NaniteBasePass);

static bool IsPrimitiveAlphaHoldoutEnabled()
{
	static const auto CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Deferred.SupportPrimitiveAlphaHoldout"));
	return CVar && CVar->GetBool();
}

template<uint32 StencilRef> void SetTranslucentPassDepthStencilState(FMeshPassProcessorRenderState& DrawRenderState, bool bDisableDepthTest)
{
	DrawRenderState.SetDepthStencilState(GetTranslucentPassDepthStencilState<StencilRef>(bDisableDepthTest));
	DrawRenderState.SetStencilRef(StencilRef);
}

void SetTranslucentRenderState(FMeshPassProcessorRenderState& DrawRenderState, const FMaterial& Material, const EShaderPlatform Platform, ETranslucencyPass::Type InTranslucencyPassType)
{
	FRHIBlendState* BlendState = GetTranslucentBlendState(Material, InTranslucencyPassType);
	if (BlendState)
	{
		DrawRenderState.SetBlendState(BlendState);
	}

	const bool bDisableDepthTest = Material.ShouldDisableDepthTest();
	const bool bEnableResponsiveAA = Material.ShouldEnableResponsiveAA();
	const bool bIsPostMotionBlur = Material.IsTranslucencyAfterMotionBlurEnabled();

	// When separate standard translucent are used, we must mark the distortion bit for the composition to happen correctly for any BeforeDoF translucent.
	const bool bSeparatedStandardTranslucent = (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyStandard || InTranslucencyPassType == ETranslucencyPass::TPT_AllTranslucency) && IsStandardTranslucencyPassSeparated();

	if (bEnableResponsiveAA && !bIsPostMotionBlur)
	{
		if (bSeparatedStandardTranslucent)
		{
			SetTranslucentPassDepthStencilState<STENCIL_TEMPORAL_RESPONSIVE_AA_MASK | DISTORTION_STENCIL_MASK_BIT>(DrawRenderState, bDisableDepthTest);
		}
		else
		{
			SetTranslucentPassDepthStencilState<STENCIL_TEMPORAL_RESPONSIVE_AA_MASK>(DrawRenderState, bDisableDepthTest);
		}
		}
	else if (bSeparatedStandardTranslucent)
	{
		SetTranslucentPassDepthStencilState<DISTORTION_STENCIL_MASK_BIT>(DrawRenderState, bDisableDepthTest);
	}
	else if (bDisableDepthTest)
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
	}
}

FMeshDrawCommandSortKey CalculateTranslucentMeshStaticSortKey(const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, uint16 MeshIdInPrimitive)
{
	uint16 SortKeyPriority = 0;
	float DistanceOffset = 0.0f;

	if (PrimitiveSceneProxy)
	{
		const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
		SortKeyPriority = (uint16)((int32)PrimitiveSceneInfo->Proxy->GetTranslucencySortPriority() - (int32)SHRT_MIN);
		DistanceOffset = PrimitiveSceneInfo->Proxy->GetTranslucencySortDistanceOffset();
	}

	FMeshDrawCommandSortKey SortKey;
	SortKey.Translucent.MeshIdInPrimitive = MeshIdInPrimitive;
	SortKey.Translucent.Priority = SortKeyPriority;
	SortKey.Translucent.Distance = *(uint32*)(&DistanceOffset); // View specific, so will be filled later inside VisibleMeshCommands.

	return SortKey;
}

static FMeshDrawCommandSortKey CalculateBasePassMeshStaticSortKey(EDepthDrawingMode EarlyZPassMode, const bool bIsMasked, const bool bBackground, const FMeshMaterialShader* VertexShader, const FMeshMaterialShader* PixelShader)
{
	FMeshDrawCommandSortKey SortKey;
	SortKey.BasePass.VertexShaderHash = (VertexShader ? VertexShader->GetSortKey() : 0) & 0xFFFF;
	SortKey.BasePass.PixelShaderHash = PixelShader ? PixelShader->GetSortKey() : 0;
	SortKey.BasePass.Background = bBackground;
	if (EarlyZPassMode != DDM_None)
	{
		SortKey.BasePass.Masked = bIsMasked ? 0 : 1;
	}
	else
	{
		SortKey.BasePass.Masked = bIsMasked ? 1 : 0;
	}

	return SortKey;
}

template<bool bDepthTest, ECompareFunction CompareFunction, uint32 StencilWriteMask>
void SetDepthStencilStateForBasePass_Internal(FMeshPassProcessorRenderState& InDrawRenderState)
{
	InDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
		bDepthTest, CompareFunction,
		true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		0xFF, StencilWriteMask>::GetRHI());
}

template<bool bDepthTest, ECompareFunction CompareFunction>
void SetDepthStencilStateForBasePass_Internal(FMeshPassProcessorRenderState& InDrawRenderState, ERHIFeatureLevel::Type FeatureLevel)
{	
	const static bool bSubstrateDufferPassEnabled = Substrate::IsSubstrateEnabled() && Substrate::IsDBufferPassEnabled(GShaderPlatformForFeatureLevel[FeatureLevel]);
	if (bSubstrateDufferPassEnabled)
	{
		SetDepthStencilStateForBasePass_Internal<bDepthTest, CompareFunction, GET_STENCIL_BIT_MASK(SUBSTRATE_RECEIVE_DBUFFER_NORMAL, 1) | GET_STENCIL_BIT_MASK(SUBSTRATE_RECEIVE_DBUFFER_DIFFUSE, 1) | GET_STENCIL_BIT_MASK(SUBSTRATE_RECEIVE_DBUFFER_ROUGHNESS, 1) | GET_STENCIL_BIT_MASK(RAY_TRACING_REPRESENTATION, 1) | STENCIL_LIGHTING_CHANNELS_MASK(0x7)>(InDrawRenderState);
	}
	else
	{
		SetDepthStencilStateForBasePass_Internal<bDepthTest, CompareFunction, GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1) | GET_STENCIL_BIT_MASK(CAST_SHADOW, 1) | GET_STENCIL_BIT_MASK(RAY_TRACING_REPRESENTATION, 1) | STENCIL_LIGHTING_CHANNELS_MASK(0x7)>(InDrawRenderState);
	}
}

void SetDepthStencilStateForBasePass(
	FMeshPassProcessorRenderState& DrawRenderState,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bDitheredLODTransition,
	const FMaterial& MaterialResource,
	bool bEnableReceiveDecalOutput,
	bool bForceEnableStencilDitherState)
{
	const bool bMaskedInEarlyPass = (IsMaskedBlendMode(MaterialResource) || bDitheredLODTransition) && MaskedInEarlyPass(GShaderPlatformForFeatureLevel[FeatureLevel]);
	if (bEnableReceiveDecalOutput)
	{
		if (bMaskedInEarlyPass)
		{
			SetDepthStencilStateForBasePass_Internal<false, CF_Equal>(DrawRenderState, FeatureLevel);
		}
		else if (DrawRenderState.GetDepthStencilAccess() & FExclusiveDepthStencil::DepthWrite)
		{
			SetDepthStencilStateForBasePass_Internal<true, CF_GreaterEqual>(DrawRenderState, FeatureLevel);
		}
		else
		{
			SetDepthStencilStateForBasePass_Internal<false, CF_GreaterEqual>(DrawRenderState, FeatureLevel);
		}
	}
	else if (bMaskedInEarlyPass)
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Equal>::GetRHI());
	}

	if (bForceEnableStencilDitherState)
	{
		SetDepthStencilStateForBasePass_Internal<false, CF_Equal>(DrawRenderState, FeatureLevel);
	}
}

void SetupBasePassState(FExclusiveDepthStencil::Type BasePassDepthStencilAccess, const bool bShaderComplexity, FMeshPassProcessorRenderState& DrawRenderState)
{
	DrawRenderState.SetDepthStencilAccess(BasePassDepthStencilAccess);

	if (bShaderComplexity)
	{
		// Additive blending when shader complexity viewmode is enabled.
		DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_One>::GetRHI());
		// Disable depth writes as we have a full depth prepass.
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
	}
	else
	{
		// Opaque blending for all G buffer targets, depth tests and writes.
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.BasePassOutputsVelocityDebug"));
		if (CVar && CVar->GetValueOnRenderThread() == 2)
		{
			DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA, CW_RGBA, CW_RGBA, CW_RGBA, CW_RGBA, CW_RGBA, CW_NONE>::GetRHI());
		}
		else
		{
			DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA, CW_RGBA, CW_RGBA, CW_RGBA>::GetRHI());
		}

		if (DrawRenderState.GetDepthStencilAccess() & FExclusiveDepthStencil::DepthWrite)
		{
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
		}
		else
		{
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
		}
	}
}

template <ELightMapPolicyType Policy, EGBufferLayout GBufferLayout>
void AddUniformBasePassPixelShader(bool bUse128bitRT, bool bIsDebug, FMaterialShaderTypes& OutShaderTypes, bool bIsForOITPass = false)
{
	int32 PermutationId = 0;
	if (bIsDebug || bIsForOITPass)
	{
		using FMyShader = TBasePassPS<TUniformLightMapPolicy<Policy>, GBufferLayout>;
		typename FMyShader::FPermutationDomain PermutationVector;
		PermutationVector.template Set<typename FMyShader::FVisualizeDim>(bIsDebug);
		PermutationVector.template Set<typename FMyShader::FSupportOITDim>(bIsForOITPass);
		PermutationId = PermutationVector.ToDimensionValueId();
	}

	if (bUse128bitRT && (Policy == LMP_NO_LIGHTMAP) && (GBufferLayout == GBL_Default))
	{
		OutShaderTypes.AddShaderType<F128BitRTBasePassPS>(PermutationId);
	}
	else
	{
		OutShaderTypes.AddShaderType<TBasePassPS<TUniformLightMapPolicy<Policy>, GBufferLayout>>(PermutationId);
	}
}

/**
 * Get shader templates allowing to redirect between compatible shaders.
 */

template <ELightMapPolicyType Policy>
bool GetUniformBasePassShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bUse128bitRT,
	bool bIsDebug,
	EGBufferLayout GBufferLayout,
	TShaderRef<TBasePassMeshShaderPolicyParamType<FUniformLightMapPolicy>>* MeshShader,
	TShaderRef<TBasePassVertexShaderPolicyParamType<FUniformLightMapPolicy>>* VertexShader,
	TShaderRef<TBasePassPixelShaderPolicyParamType<FUniformLightMapPolicy>>* PixelShader,
	bool bIsForOITPass = false
)
{
	FMaterialShaderTypes ShaderTypes;

	if (VertexShader)
	{
		ShaderTypes.AddShaderType<TBasePassVS<TUniformLightMapPolicy<Policy>>>();
	}

	if (MeshShader)
	{
		ShaderTypes.AddShaderType<TBasePassMS<TUniformLightMapPolicy<Policy>>>();
	}

	if (PixelShader)
	{
		switch (GBufferLayout)
		{
		case GBL_Default:
			AddUniformBasePassPixelShader<Policy, GBL_Default>(bUse128bitRT, bIsDebug, ShaderTypes, bIsForOITPass);
			break;
		case GBL_ForceVelocity:
			AddUniformBasePassPixelShader<Policy, GBL_ForceVelocity>(bUse128bitRT, bIsDebug, ShaderTypes, bIsForOITPass);
			break;
		default:
			check(false);
			break;
		}
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetMeshShader(MeshShader);
	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

template <>
bool GetBasePassShaders<FUniformLightMapPolicy>(
	const FMaterial& Material, 
	const FVertexFactoryType* VertexFactoryType, 
	FUniformLightMapPolicy LightMapPolicy, 
	ERHIFeatureLevel::Type FeatureLevel,
    bool bUse128bitRT,
	bool bIsDebug,
	EGBufferLayout GBufferLayout,
	TShaderRef<TBasePassMeshShaderPolicyParamType<FUniformLightMapPolicy>>* MeshShader,
	TShaderRef<TBasePassVertexShaderPolicyParamType<FUniformLightMapPolicy>>* VertexShader,
	TShaderRef<TBasePassPixelShaderPolicyParamType<FUniformLightMapPolicy>>* PixelShader,
	bool bIsForOITPass
	)
{
	switch (LightMapPolicy.GetIndirectPolicy())
	{
	case LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING:
		return GetUniformBasePassShaders<LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING>(Material, VertexFactoryType, FeatureLevel, bUse128bitRT, bIsDebug, GBufferLayout, MeshShader, VertexShader, PixelShader, bIsForOITPass);
	case LMP_CACHED_VOLUME_INDIRECT_LIGHTING:
		return GetUniformBasePassShaders<LMP_CACHED_VOLUME_INDIRECT_LIGHTING>(Material, VertexFactoryType, FeatureLevel, bUse128bitRT, bIsDebug, GBufferLayout, MeshShader, VertexShader, PixelShader, bIsForOITPass);
	case LMP_CACHED_POINT_INDIRECT_LIGHTING:
		return GetUniformBasePassShaders<LMP_CACHED_POINT_INDIRECT_LIGHTING>(Material, VertexFactoryType, FeatureLevel, bUse128bitRT, bIsDebug, GBufferLayout, MeshShader, VertexShader, PixelShader, bIsForOITPass);
	case LMP_LQ_LIGHTMAP:
		return GetUniformBasePassShaders<LMP_LQ_LIGHTMAP>(Material, VertexFactoryType, FeatureLevel, bUse128bitRT, bIsDebug, GBufferLayout, MeshShader, VertexShader, PixelShader, bIsForOITPass);
	case LMP_HQ_LIGHTMAP:
		return GetUniformBasePassShaders<LMP_HQ_LIGHTMAP>(Material, VertexFactoryType, FeatureLevel, bUse128bitRT, bIsDebug, GBufferLayout, MeshShader, VertexShader, PixelShader, bIsForOITPass);
	case LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP:
		return GetUniformBasePassShaders<LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP>(Material, VertexFactoryType, FeatureLevel, bUse128bitRT, bIsDebug, GBufferLayout, MeshShader, VertexShader, PixelShader, bIsForOITPass);
	case LMP_NO_LIGHTMAP:
		return GetUniformBasePassShaders<LMP_NO_LIGHTMAP>(Material, VertexFactoryType, FeatureLevel, bUse128bitRT, bIsDebug, GBufferLayout, MeshShader, VertexShader, PixelShader, bIsForOITPass);
	default:
		check(false);
		return false;
	}
}

template <ELightMapPolicyType Policy>
void AddUniformBasePassComputeShader(bool bVoxel, bool bCurve, bool bIsDebug, EShaderFrequency ShaderFrequency, FMaterialShaderTypes& OutShaderTypes)
{
	int32 PermutationId = 0;
	if (bIsDebug)
	{
		using FMyShader = TBasePassCS<TUniformLightMapPolicy<Policy>, true, true, SF_Compute>;
		typename FMyShader::FPermutationDomain PermutationVector;
		PermutationVector.template Set<typename FMyShader::FVisualizeDim>(bIsDebug);
		PermutationId = PermutationVector.ToDimensionValueId();
	}

	if (ShaderFrequency == SF_Compute)
	{
		if (bCurve)
		{
			OutShaderTypes.AddShaderType<TBasePassCS<TUniformLightMapPolicy<Policy>, false, true, SF_Compute>>(PermutationId);
		}
		else if (bVoxel)
		{
			OutShaderTypes.AddShaderType<TBasePassCS<TUniformLightMapPolicy<Policy>, true, false, SF_Compute>>(PermutationId);
		}
		else
		{
			OutShaderTypes.AddShaderType<TBasePassCS<TUniformLightMapPolicy<Policy>, false, false, SF_Compute>>(PermutationId);
		}
	}
	else if (ShaderFrequency == SF_WorkGraphComputeNode)
	{
		if (bCurve)
		{
			OutShaderTypes.AddShaderType<TBasePassCS<TUniformLightMapPolicy<Policy>, false, true, SF_WorkGraphComputeNode>>(PermutationId);
		}
		else if (bVoxel)
		{
			OutShaderTypes.AddShaderType<TBasePassCS<TUniformLightMapPolicy<Policy>, true, false, SF_WorkGraphComputeNode>>(PermutationId);
		}
		else
		{
			OutShaderTypes.AddShaderType<TBasePassCS<TUniformLightMapPolicy<Policy>, false, false, SF_WorkGraphComputeNode>>(PermutationId);
		}
	}
}

template <ELightMapPolicyType Policy>
bool GetUniformBasePassShader(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bVoxel,
	bool bCurve,
	bool bIsDebug,
	EShaderFrequency ShaderFrequency,
	TShaderRef<TBasePassComputeShaderPolicyParamType<FUniformLightMapPolicy>>* ComputeShader
)
{
	FMaterialShaderTypes ShaderTypes;

	if (ComputeShader)
	{
		AddUniformBasePassComputeShader<Policy>(bVoxel, bCurve, bIsDebug, ShaderFrequency, ShaderTypes);
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetShader(ShaderFrequency, ComputeShader);
	return true;
}

template <>
bool GetBasePassShader<FUniformLightMapPolicy>(
	const FMaterial& Material, 
	const FVertexFactoryType* VertexFactoryType,
	FUniformLightMapPolicy LightMapPolicy,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bVoxel,
	bool bCurve,
	bool bIsDebug,
	EShaderFrequency ShaderFrequency,
	TShaderRef<TBasePassComputeShaderPolicyParamType<FUniformLightMapPolicy>>* ComputeShader
	)
{
	switch (LightMapPolicy.GetIndirectPolicy())
	{
	case LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING:
		return GetUniformBasePassShader<LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING>(Material, VertexFactoryType, FeatureLevel, bVoxel, bCurve, bIsDebug, ShaderFrequency, ComputeShader);
	case LMP_CACHED_VOLUME_INDIRECT_LIGHTING:
		return GetUniformBasePassShader<LMP_CACHED_VOLUME_INDIRECT_LIGHTING>(Material, VertexFactoryType, FeatureLevel, bVoxel, bCurve, bIsDebug, ShaderFrequency, ComputeShader);
	case LMP_CACHED_POINT_INDIRECT_LIGHTING:
		return GetUniformBasePassShader<LMP_CACHED_POINT_INDIRECT_LIGHTING>(Material, VertexFactoryType, FeatureLevel, bVoxel, bCurve, bIsDebug, ShaderFrequency, ComputeShader);
	case LMP_LQ_LIGHTMAP:
		return GetUniformBasePassShader<LMP_LQ_LIGHTMAP>(Material, VertexFactoryType, FeatureLevel, bVoxel, bCurve, bIsDebug, ShaderFrequency, ComputeShader);
	case LMP_HQ_LIGHTMAP:
		return GetUniformBasePassShader<LMP_HQ_LIGHTMAP>(Material, VertexFactoryType, FeatureLevel, bVoxel, bCurve, bIsDebug, ShaderFrequency, ComputeShader);
	case LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP:
		return GetUniformBasePassShader<LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP>(Material, VertexFactoryType, FeatureLevel, bVoxel, bCurve, bIsDebug, ShaderFrequency, ComputeShader);
	case LMP_NO_LIGHTMAP:
		return GetUniformBasePassShader<LMP_NO_LIGHTMAP>(Material, VertexFactoryType, FeatureLevel, bVoxel, bCurve, bIsDebug, ShaderFrequency, ComputeShader);
	default:
		check(false);
		return false;
	}
}

extern void SetForwardLightUniformParametersFromView(FRDGBuilder& GraphBuilder, const FViewInfo& View, FForwardLightUniformParameters& ForwardLightUniformParameters);
extern void SetForwardDirLightShadowParametersFromView(FRDGBuilder& GraphBuilder, const FViewInfo& View, FForwardDirectionalLightShadowMapParameters& Parameters);

void SetupSharedBasePassParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const int32 ViewIndex,
	FSharedBasePassUniformParameters& SharedParameters,
	bool bForRealtimeSkyCapture)
{
	SetForwardLightUniformParametersFromView(GraphBuilder, View, SharedParameters.Forward);
	SetForwardDirLightShadowParametersFromView(GraphBuilder, View, SharedParameters.ForwardDirLightShadow);

	SetupFogUniformParameters(GraphBuilder, View, SharedParameters.Fog, bForRealtimeSkyCapture);

	if (View.IsInstancedStereo() && IStereoRendering::IsAPrimaryView(View))
	{
		const FViewInfo& InstancedView = *View.GetInstancedView();
		SetupFogUniformParameters(GraphBuilder, (FViewInfo&)InstancedView, SharedParameters.FogISR, bForRealtimeSkyCapture);
	}
	else
	{
		SharedParameters.FogISR = SharedParameters.Fog;
	}

	if (bForRealtimeSkyCapture)
	{
		// LFV are not allowed in real time capture since they are local effects.
		// Furthermore, this avoid having the sky capture pass on async compute being dependent on LFV culling pass resources,
		// because those resources are in the View UB so they most be ready to read.
		// So we bind dummy parameters and resources for real time capture.
		SetDummyLocalFogVolumeUniformParametersStruct(GraphBuilder, SharedParameters.LFV);
	}
	else
	{
		SharedParameters.LFV = View.LocalFogVolumeViewData.UniformParametersStruct;
	}

	SharedParameters.LightFunctionAtlas = *LightFunctionAtlas::GetGlobalParametersStruct(GraphBuilder, View);

	const FScene* Scene = View.Family->Scene ? View.Family->Scene->GetRenderScene() : nullptr;
	const FPlanarReflectionSceneProxy* ReflectionSceneProxy = Scene ? Scene->GetForwardPassGlobalPlanarReflection() : nullptr;

	SetupReflectionUniformParameters(GraphBuilder, View, SharedParameters.Reflection);
	SetupPlanarReflectionUniformParameters(View, ReflectionSceneProxy, SharedParameters.PlanarReflection);
}

TRDGUniformBufferRef<FOpaqueBasePassUniformParameters> CreateOpaqueBasePassUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const int32 ViewIndex,
	const FForwardBasePassTextures& ForwardBasePassTextures,
	const FDBufferTextures& DBufferTextures,
	bool bEnableSkyLight,
	bool bForRealtimeSkyCapture)
{
	FOpaqueBasePassUniformParameters& BasePassParameters = *GraphBuilder.AllocParameters<FOpaqueBasePassUniformParameters>();
	SetupSharedBasePassParameters(GraphBuilder, View, ViewIndex, BasePassParameters.Shared, bForRealtimeSkyCapture);

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	// Forward shading
	{
		BasePassParameters.UseForwardScreenSpaceShadowMask = 0;
		BasePassParameters.ForwardScreenSpaceShadowMaskTexture = SystemTextures.White;
		BasePassParameters.IndirectOcclusionTexture = SystemTextures.White;
		BasePassParameters.ResolvedSceneDepthTexture = SystemTextures.White;

		if (ForwardBasePassTextures.ScreenSpaceShadowMask)
		{
			BasePassParameters.UseForwardScreenSpaceShadowMask = 1;
			BasePassParameters.ForwardScreenSpaceShadowMaskTexture = ForwardBasePassTextures.ScreenSpaceShadowMask;
		}

		if (HasBeenProduced(ForwardBasePassTextures.ScreenSpaceAO))
		{
			BasePassParameters.IndirectOcclusionTexture = ForwardBasePassTextures.ScreenSpaceAO;
		}

		if (ForwardBasePassTextures.SceneDepthIfResolved)
		{
			BasePassParameters.ResolvedSceneDepthTexture = ForwardBasePassTextures.SceneDepthIfResolved;
		}
		BasePassParameters.Is24BitUnormDepthStencil = ForwardBasePassTextures.bIs24BitUnormDepthStencil ? 1 : 0;
	}

	// DBuffer Decals
	BasePassParameters.DBuffer = GetDBufferParameters(GraphBuilder, DBufferTextures, View.GetShaderPlatform());

	// Substrate
	Substrate::BindSubstrateBasePassUniformParameters(GraphBuilder, View, BasePassParameters.Substrate);

	// Skylight enable for shaders that have skylight support compiled
	BasePassParameters.Shared.UseBasePassSkylight = bEnableSkyLight;

	// Misc
	BasePassParameters.PreIntegratedGFTexture = GSystemTextures.PreintegratedGF->GetRHI();
	BasePassParameters.PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	BasePassParameters.EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));

	return GraphBuilder.CreateUniformBuffer(&BasePassParameters);
}

static void ClearGBufferAtMaxZ(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FRenderTargetBindingSlots& BasePassRenderTargets,
	FLinearColor ClearColor0)
{
	check(Views.Num() > 0);

	SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_ClearGBufferAtMaxZ);
	RDG_EVENT_SCOPE(GraphBuilder, "ClearGBufferAtMaxZ");

	const uint32 ActiveTargetCount = BasePassRenderTargets.GetActiveCount();
	FGlobalShaderMap* ShaderMap = Views[0].ShaderMap;

	TShaderMapRef<TOneColorVS<true> > VertexShader(ShaderMap);
	TOneColorPixelShaderMRT::FPermutationDomain PermutationVector;
	PermutationVector.Set<TOneColorPixelShaderMRT::TOneColorPixelShaderNumOutputs>(ActiveTargetCount);
	TShaderMapRef<TOneColorPixelShaderMRT>PixelShader(ShaderMap, PermutationVector);

	auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
	PassParameters->RenderTargets = BasePassRenderTargets;

	// Clear each viewport by drawing background color at MaxZ depth
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		GraphBuilder.AddPass(
			{},
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, VertexShader, PixelShader, ActiveTargetCount, ClearColor0](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			const FLinearColor ClearColors[MaxSimultaneousRenderTargets] =
			{
				ClearColor0,
				FLinearColor(0.5f,0.5f,0.5f,0),
				FLinearColor(0,0,0,1),
				FLinearColor(0,0,0,0),
				FLinearColor(0,1,1,1),
				FLinearColor(1,1,1,1),
				FLinearColor::Transparent,
				FLinearColor::Transparent
			};

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			// Opaque rendering, depth test but no depth writes
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParametersLegacyVS(RHICmdList, VertexShader, 0.f);

			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1);

			TOneColorPixelShaderMRT::FParameters PixelParameters;
			PixelShader->FillParameters(PixelParameters, ClearColors, ActiveTargetCount);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PixelParameters);

			RHICmdList.SetStreamSource(0, GClearVertexBuffer.VertexBufferRHI, 0);
			RHICmdList.DrawPrimitive(0, 2, 1);
		});
	}
}

bool IsGBufferLayoutSupportedForMaterial(EGBufferLayout Layout, const FMeshMaterialShaderPermutationParameters& Parameters)
{
	switch (Layout)
	{
	case GBL_Default:
		// All Nanite and non-Nanite base pass shaders support the default layout
		return true;

	case GBL_ForceVelocity:
		// Only Nanite materials with WPO and SingleLayerWater materials support this GBuffer layout
		// NOTE: FMaterialShaderParameters::bHasVertexPositionOffsetConnected means that the material *could* have WPO.
		// It's still possible for the material to disable WPO after translation.
		return (!IsUsingBasePassVelocity(Parameters.Platform) && 
			Parameters.VertexFactoryType->SupportsNaniteRendering() &&
			Parameters.MaterialParameters.bIsUsedWithNanite &&
			Parameters.MaterialParameters.bHasVertexPositionOffsetConnected)
			|| Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);

	default:
		checkf(false, TEXT("Unhandled GBuffer Layout!"));
		return false;
	}
}

void ModifyBasePassCSPSCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, EGBufferLayout GBufferLayout, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), Parameters.MaterialParameters.MaterialDomain != MD_Surface);
	OutEnvironment.SetDefine(TEXT("ENABLE_DBUFFER_TEXTURES"), Parameters.MaterialParameters.MaterialDomain == MD_Surface);
	OutEnvironment.SetDefine(TEXT("PLATFORM_FORCE_SIMPLE_SKY_DIFFUSE"), ForceSimpleSkyDiffuse(Parameters.Platform));
	OutEnvironment.SetDefine(TEXT("GBUFFER_LAYOUT"), GBufferLayout);

	// This define simply lets the compilation environment know that we are using BasePassPixelShader.usf, so that we can check for more
	// complicated defines later in the compilation pipe.
	OutEnvironment.SetDefine(TEXT("IS_BASE_PASS"), 1);
	OutEnvironment.SetDefine(TEXT("IS_MOBILE_BASE_PASS"), 0);

	// Compile skylight support where the project settings requires it.
	static const auto AllowStaticLightingCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AllowStaticLighting"));
	static bool bProjectAllowStaticLighting = AllowStaticLightingCVar && AllowStaticLightingCVar->GetValueOnAnyThread() != 0;
	static bool bProjectSupportsStationarySkylight = CVarSupportStationarySkylight.GetValueOnAnyThread() != 0;
	static bool bProjectSupportAllPermutations = CVarSupportAllShaderPermutations.GetValueOnAnyThread() != 0;
	static bool bProjectEnableSkyLight = (bProjectAllowStaticLighting && bProjectSupportsStationarySkylight) || bProjectSupportAllPermutations;
	
	// Also forward rendering and translucent materials need to compile skylight support for MOVABLE skylights.
	const bool bShaderPlatformUsesForward = IsForwardShadingEnabled(Parameters.Platform);
	const bool bMaterialIsTranslucent = IsTranslucentBlendMode(Parameters.MaterialParameters);
	const bool bMaterialIsSingleLayerWater = Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);

	// Only need skylight for lit materials.
	// Note that some entire passes like Sky and WaterInfo are guaranteed to _only_ use unlit materials and so we can assume they won't have shaders with skylight enabled.
	const bool bMaterialIsLit = Parameters.MaterialParameters.ShadingModels.IsLit();
	const bool bEnableSkyLight = bMaterialIsLit && (bProjectEnableSkyLight || bShaderPlatformUsesForward || bMaterialIsTranslucent || bMaterialIsSingleLayerWater);
	OutEnvironment.SetDefine(TEXT("ENABLE_SKY_LIGHT"), bEnableSkyLight);

	const bool bSupportVirtualShadowMap = DoesPlatformSupportVirtualShadowMaps(Parameters.Platform);
	if (bSupportVirtualShadowMap && (bMaterialIsTranslucent || bMaterialIsSingleLayerWater))
	{
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
	}

	OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
	Substrate::SetBasePassRenderTargetOutputFormat(Parameters.Platform, Parameters.MaterialParameters, OutEnvironment, GBufferLayout);

	const bool bOutputVelocity = (GBufferLayout == GBL_ForceVelocity) ||
		FVelocityRendering::BasePassCanOutputVelocity(Parameters.Platform);
	if (bOutputVelocity)
	{
		// As defined in BasePassPixelShader.usf. Also account for Substrate setting velocity in slot 1 as described in FetchLegacyGBufferInfo.
		const int32 VelocityIndex = (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(Parameters.Platform)) ? 1 : (IsForwardShadingEnabled(Parameters.Platform) ? 1 : 4);
		OutEnvironment.SetRenderTargetOutputFormat(VelocityIndex, PF_G16R16);
	}

	const bool bNeedsSeparateMainDirLightTexture = IsWaterDistanceFieldShadowEnabled(Parameters.Platform) || IsWaterVirtualShadowMapFilteringEnabled(Parameters.Platform);
	if (bMaterialIsSingleLayerWater && bNeedsSeparateMainDirLightTexture)
	{
		const FGBufferParams GBufferParams = FShaderCompileUtilities::FetchGBufferParamsRuntime(Parameters.Platform, GBufferLayout);
		const FGBufferInfo BufferInfo = FetchFullGBufferInfo(GBufferParams);
		const uint32 TargetSeparatedMainDirLight = BufferInfo.Slots[GBS_SeparatedMainDirLight].Packing[0].TargetIndex;

		OutEnvironment.SetRenderTargetOutputFormat(TargetSeparatedMainDirLight, PF_FloatR11G11B10);
	}

	const bool bSingleLayerWaterUsesLightFunctionAtlas = bMaterialIsSingleLayerWater && GetSingleLayerWaterUsesLightFunctionAtlas();
	const bool bTranslucentUsesLightFunctionAtlas = bMaterialIsTranslucent && GetTranslucentUsesLightFunctionAtlas();
	OutEnvironment.SetDefine(TEXT("USE_LIGHT_FUNCTION_ATLAS"), (bSingleLayerWaterUsesLightFunctionAtlas || bTranslucentUsesLightFunctionAtlas) ? TEXT("1") : TEXT("0"));
}

void FDeferredShadingSceneRenderer::RenderBasePass(
	FDeferredShadingSceneRenderer& Renderer,
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> InViews,
	FSceneTextures& SceneTextures,
	FExclusiveDepthStencil::Type BasePassDepthStencilAccess,
	FRDGTextureRef ForwardShadowMaskTexture,
	FInstanceCullingManager& InstanceCullingManager,
	bool bNaniteEnabled,
	FNaniteShadingCommands& NaniteBasePassShadingCommands,
	const TArrayView<Nanite::FRasterResults>& NaniteRasterResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDeferredShadingSceneRenderer::RenderBasePass);

	// This function has been deliberately made static, so it doesn't have access to the ViewFamily member, to support Custom Render Passes,
	// which have a separate ViewFamily.  Create a local reference to the first view's Family here.
	FViewFamilyInfo& ViewFamily = *(FViewFamilyInfo*)InViews[0].Family;

	const bool bEnableParallelBasePasses = GRHICommandList.UseParallelAlgorithms() && CVarParallelBasePass.GetValueOnRenderThread();

	static const auto ClearMethodCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ClearSceneMethod"));
	bool bRequiresRHIClear = true;
	bool bRequiresFarZQuadClear = false;

	if (ClearMethodCVar)
	{
		int32 ClearMethod = ClearMethodCVar->GetValueOnRenderThread();

		if (ClearMethod == 0 && !ViewFamily.EngineShowFlags.Game)
		{
			// Do not clear the scene only if the view family is in game mode.
			ClearMethod = 1;
		}

		switch (ClearMethod)
		{
		case 0: // No clear
			bRequiresRHIClear = false;
			bRequiresFarZQuadClear = false;
			break;

		case 1: // RHICmdList.Clear
			bRequiresRHIClear = true;
			bRequiresFarZQuadClear = false;
			break;

		case 2: // Clear using far-z quad
			bRequiresFarZQuadClear = true;
			bRequiresRHIClear = false;
			break;
		}
	}

	// Always perform a full buffer clear for wireframe, shader complexity view mode, and stationary light overlap viewmode.
	if (ViewFamily.EngineShowFlags.Wireframe || ViewFamily.EngineShowFlags.ShaderComplexity || ViewFamily.EngineShowFlags.StationaryLightOverlap)
	{
		bRequiresRHIClear = true;
		bRequiresFarZQuadClear = false;
	}

	const bool bIsWireframeRenderpass = ViewFamily.EngineShowFlags.Wireframe && FSceneRenderer::ShouldCompositeEditorPrimitives(InViews[0]);
	const bool bDebugViewMode = ViewFamily.UseDebugViewPS();
	const bool bRenderLightmapDensity = ViewFamily.EngineShowFlags.LightMapDensity && AllowDebugViewmodes();
	const bool bDoParallelBasePass = bEnableParallelBasePasses && !bDebugViewMode && !bRenderLightmapDensity; // DebugView and LightmapDensity are non-parallel substitutions inside BasePass
	const bool bNeedsBeginRender = AllowDebugViewmodes() &&
		(ViewFamily.EngineShowFlags.RequiredTextureResolution ||
			ViewFamily.EngineShowFlags.MaterialTextureScaleAccuracy ||
			ViewFamily.EngineShowFlags.MeshUVDensityAccuracy ||
			ViewFamily.EngineShowFlags.PrimitiveDistanceAccuracy ||
			ViewFamily.EngineShowFlags.ShaderComplexity ||
			ViewFamily.EngineShowFlags.LODColoration ||
			ViewFamily.EngineShowFlags.HLODColoration);

	const bool bForwardShadingEnabled = IsForwardShadingEnabled(SceneTextures.Config.ShaderPlatform);

	const FExclusiveDepthStencil ExclusiveDepthStencil(BasePassDepthStencilAccess);

	TStaticArray<FTextureRenderTargetBinding, MaxSimultaneousRenderTargets> BasePassTextures;
	uint32 BasePassTextureCount = SceneTextures.GetGBufferRenderTargets(BasePassTextures);
	Substrate::AppendSubstrateMRTs(Renderer, BasePassTextureCount, BasePassTextures);
	TArrayView<FTextureRenderTargetBinding> BasePassTexturesView = MakeArrayView(BasePassTextures.GetData(), BasePassTextureCount);
	FRDGTextureRef BasePassDepthTexture = SceneTextures.Depth.Target;
	FLinearColor SceneColorClearValue = FLinearColor::Black;

	if (bRequiresRHIClear)
	{
		ClearDebugAux(GraphBuilder, ViewFamily, SceneTextures.DebugAux);

		if (ViewFamily.EngineShowFlags.ShaderComplexity || ViewFamily.EngineShowFlags.StationaryLightOverlap)
		{
			SceneColorClearValue = FLinearColor(0, 0, 0, kSceneColorClearAlpha);
		}
		else
		{
			SceneColorClearValue = FLinearColor(InViews[0].BackgroundColor.R, InViews[0].BackgroundColor.G, InViews[0].BackgroundColor.B, kSceneColorClearAlpha);
		}

		ERenderTargetLoadAction ColorLoadAction = ERenderTargetLoadAction::ELoad;

		if (SceneTextures.Color.Target->Desc.ClearValue.GetClearColor() == SceneColorClearValue)
		{
			ColorLoadAction = ERenderTargetLoadAction::EClear;
		}
		else
		{
			ColorLoadAction = ERenderTargetLoadAction::ENoAction;
		}

		auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
		PassParameters->RenderTargets = GetRenderTargetBindings(ColorLoadAction, BasePassTexturesView);

		const FGBufferBindings& GBufferBindings = SceneTextures.Config.GBufferBindings[GBL_Default];
		if (!CVarClearGBufferDBeforeBasePass.GetValueOnRenderThread() && GBufferBindings.GBufferD.Index > 0 && GBufferBindings.GBufferD.Index < (int32)BasePassTextureCount)
		{
			PassParameters->RenderTargets[GBufferBindings.GBufferD.Index].SetLoadAction(ERenderTargetLoadAction::ENoAction);
		}

		GraphBuilder.AddPass(RDG_EVENT_NAME("GBufferClear"), PassParameters, ERDGPassFlags::Raster,
			[PassParameters, ColorLoadAction, SceneColorClearValue](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			// If no fast-clear action was used, we need to do an MRT shader clear.
			if (ColorLoadAction == ERenderTargetLoadAction::ENoAction)
			{
				if (Substrate::IsSubstrateEnabled())
				{
					const FRenderTargetBindingSlots& RenderTargets = PassParameters->RenderTargets;
					FLinearColor ClearColors[MaxSimultaneousRenderTargets];
					int32 MRTCount = 0;
					uint8 NumUintOutputs = 0;	// Substrate textures are always integers and we assume they are all at the end of the MRTs.

					// Clear Only non-UINT targets.
					RenderTargets.Enumerate([&](const FRenderTargetBinding& RenderTarget)
						{
							FRHITexture* TextureRHI = RenderTarget.GetTexture()->GetRHI();

							EPixelFormat Format = TextureRHI->GetFormat();
							if (!IsInteger(Format))
							{
								ClearColors[MRTCount] = MRTCount == 0 ? SceneColorClearValue : TextureRHI->GetClearColor();
								++MRTCount;

								// We do not support non integer textures after we have identified integer textures starting point.
								check(NumUintOutputs == 0);
							}
							else
							{
								ClearColors[MRTCount] = TextureRHI->GetClearColor();
								++NumUintOutputs;
								++MRTCount;
							}
						});

					// Clear color only; depth-stencil is fast cleared.
					DrawClearQuadMRTWithUints(RHICmdList, true, MRTCount, ClearColors, false, 0, false, 0, NumUintOutputs);
				}
				else
				{
					const FRenderTargetBindingSlots& RenderTargets = PassParameters->RenderTargets;
					FLinearColor ClearColors[MaxSimultaneousRenderTargets];
					FRHITexture* Textures[MaxSimultaneousRenderTargets];
					int32 TextureIndex = 0;

					RenderTargets.Enumerate([&](const FRenderTargetBinding& RenderTarget)
						{
							FRHITexture* TextureRHI = RenderTarget.GetTexture()->GetRHI();
							ClearColors[TextureIndex] = TextureIndex == 0 ? SceneColorClearValue : TextureRHI->GetClearColor();
							Textures[TextureIndex] = TextureRHI;
							++TextureIndex;
						});

					// Clear color only; depth-stencil is fast cleared.
					DrawClearQuadMRT(RHICmdList, true, TextureIndex, ClearColors, false, 0, false, 0);
				}
			}
		});

		if (ShouldRenderSkyAtmosphereEditorNotifications(InViews))
		{
			// We only render this warning text when bRequiresRHIClear==true to make sure the scene color buffer is allocated at this stage.
			// When false, the option specifies that all pixels must be written to by a sky dome anyway.
			Renderer.RenderSkyAtmosphereEditorNotifications(GraphBuilder, InViews, SceneTextures.Color.Target);
		}
	}

#if WITH_EDITOR
	if (ViewFamily.EngineShowFlags.Wireframe)
	{
		checkf(ExclusiveDepthStencil.IsDepthWrite(), TEXT("Wireframe base pass requires depth-write, but it is set to read-only."));

		BasePassTextureCount = 1;
		BasePassTextures[0] = SceneTextures.EditorPrimitiveColor;
		BasePassTexturesView = MakeArrayView(BasePassTextures.GetData(), BasePassTextureCount);

		BasePassDepthTexture = SceneTextures.EditorPrimitiveDepth;

		auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
		PassParameters->RenderTargets = GetRenderTargetBindings(ERenderTargetLoadAction::EClear, BasePassTexturesView);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(BasePassDepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, ExclusiveDepthStencil);

		GraphBuilder.AddPass(RDG_EVENT_NAME("WireframeClear"), PassParameters, ERDGPassFlags::Raster, [](FRDGAsyncTask, FRHICommandList&) {});
	}
#endif

	// Render targets bindings should remain constant at this point.
	FRenderTargetBindingSlots BasePassRenderTargets = GetRenderTargetBindings(ERenderTargetLoadAction::ELoad, BasePassTexturesView);
	BasePassRenderTargets.DepthStencil = FDepthStencilBinding(BasePassDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, ExclusiveDepthStencil);
	
	FForwardBasePassTextures ForwardBasePassTextures{};

	if (bForwardShadingEnabled)
	{
		ForwardBasePassTextures.SceneDepthIfResolved = SceneTextures.Depth.IsSeparate() ? SceneTextures.Depth.Resolve : nullptr;
		ForwardBasePassTextures.ScreenSpaceAO = SceneTextures.ScreenSpaceAO;
		ForwardBasePassTextures.ScreenSpaceShadowMask = ForwardShadowMaskTexture;
	}
	else if (!ExclusiveDepthStencil.IsDepthWrite())
	{
		// If depth write is not enabled, we can bound the depth texture as read only
		ForwardBasePassTextures.SceneDepthIfResolved = SceneTextures.Depth.Resolve;
	}
	ForwardBasePassTextures.bIs24BitUnormDepthStencil = ForwardBasePassTextures.SceneDepthIfResolved ? GPixelFormats[ForwardBasePassTextures.SceneDepthIfResolved->Desc.Format].bIs24BitUnormDepthStencil : 1;

	RenderBasePassInternal(Renderer, GraphBuilder, InViews, SceneTextures, BasePassRenderTargets, BasePassDepthStencilAccess, ForwardBasePassTextures, bDoParallelBasePass, bRenderLightmapDensity, InstanceCullingManager, bNaniteEnabled, NaniteBasePassShadingCommands, NaniteRasterResults);
	
	for (const TSharedRef<ISceneViewExtension>& ViewExtension : ViewFamily.ViewExtensions)
	{
		for (FViewInfo& View : InViews)
		{
			ViewExtension->PostRenderBasePassDeferred_RenderThread(GraphBuilder, View, BasePassRenderTargets, SceneTextures.UniformBuffer);
		}
	}

	if (bRequiresFarZQuadClear)
	{
		ClearGBufferAtMaxZ(GraphBuilder, InViews, BasePassRenderTargets, SceneColorClearValue);
	}

	// The anisotropy GBuffer is used for lighting
	if (ShouldRenderAnisotropyPass(InViews) && ViewFamily.EngineShowFlags.Lighting)
	{
		RenderAnisotropyPass(GraphBuilder, InViews, SceneTextures, ViewFamily.Scene->GetRenderScene(), bEnableParallelBasePasses);
	}

#if !(UE_BUILD_SHIPPING)
	if (!bForwardShadingEnabled)
	{
		StampDeferredDebugProbeMaterialPS(GraphBuilder, InViews, BasePassRenderTargets, SceneTextures);
	}
#endif
}

BEGIN_SHADER_PARAMETER_STRUCT(FOpaqueBasePassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_REF(FReflectionCaptureShaderData, ReflectionCapture)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FOpaqueBasePassUniformParameters, BasePass)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static void RenderEditorPrimitivesForDPG(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FOpaqueBasePassParameters* PassParameters,
	const FMeshPassProcessorRenderState& DrawRenderState,
	ESceneDepthPriorityGroup DepthPriorityGroup,
	FInstanceCullingManager& InstanceCullingManager)
{
	const FScene* Scene = View.Family->Scene->GetRenderScene();

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("EditorPrimitives: %s", *UEnum::GetValueAsString(DepthPriorityGroup)),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, DrawRenderState, DepthPriorityGroup](FRHICommandList& RHICmdList)
	{
		FSceneRenderer::SetStereoViewport(RHICmdList, View, 1.0f);

		View.SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, DepthPriorityGroup, View.GetStereoPassInstanceFactor());

		if (!View.Family->EngineShowFlags.CompositeEditorPrimitives)
		{
			DrawDynamicMeshPass(View, RHICmdList,
				[&View, &DrawRenderState](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FEditorPrimitivesBasePassMeshProcessor PassMeshProcessor(
					View.Family->Scene->GetRenderScene(),
					View.GetFeatureLevel(),
					&View,
					DrawRenderState,
					false,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;

				for (int32 MeshIndex = 0; MeshIndex < View.ViewMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatch& MeshBatch = View.ViewMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
				}
			});

			const FBatchedElements& BatchedViewElements = DepthPriorityGroup == SDPG_World ? View.BatchedViewElements : View.TopBatchedViewElements;

			DrawDynamicMeshPass(View, RHICmdList,
				[&View, &DrawRenderState](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FEditorPrimitivesBasePassMeshProcessor PassMeshProcessor(
					View.Family->Scene->GetRenderScene(),
					View.GetFeatureLevel(),
					&View,
					DrawRenderState,
					false,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;

				for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
				}
			});

			// Draw the view's batched simple elements(lines, sprites, etc).
			View.TopBatchedViewElements.Draw(RHICmdList, DrawRenderState, View.GetFeatureLevel(), View, false, 1.0f, EBlendModeFilter::All, View.GetStereoPassInstanceFactor());
		}
	});
}

static bool HasEditorPrimitivesForDPG(const FViewInfo& View, ESceneDepthPriorityGroup DepthPriorityGroup)
{
	bool bHasPrimitives = View.SimpleElementCollector.HasPrimitives(DepthPriorityGroup);

	if (!View.Family->EngineShowFlags.CompositeEditorPrimitives)
	{
		const TIndirectArray<FMeshBatch, SceneRenderingAllocator>& ViewMeshElementList = (DepthPriorityGroup == SDPG_Foreground ? View.TopViewMeshElements : View.ViewMeshElements);
		bHasPrimitives |= ViewMeshElementList.Num() > 0;

		const FBatchedElements& BatchedViewElements = DepthPriorityGroup == SDPG_World ? View.BatchedViewElements : View.TopBatchedViewElements;
		bHasPrimitives |= BatchedViewElements.HasPrimsToDraw();
	}

	return bHasPrimitives;
}

static void RenderEditorPrimitives(
	FRDGBuilder& GraphBuilder,
	FOpaqueBasePassParameters* PassParameters,
	const FViewInfo& View,
	const FMeshPassProcessorRenderState& DrawRenderState,
	FInstanceCullingManager& InstanceCullingManager)
{
	RDG_EVENT_SCOPE(GraphBuilder, "EditorPrimitives");

	RenderEditorPrimitivesForDPG(GraphBuilder, View, PassParameters, DrawRenderState, SDPG_World, InstanceCullingManager);

	if (HasEditorPrimitivesForDPG(View, SDPG_Foreground))
	{
		// Write foreground primitives into depth buffer without testing 
		{
			auto* DepthWritePassParameters = GraphBuilder.AllocParameters<FOpaqueBasePassParameters>();
			*DepthWritePassParameters = *PassParameters;

			// Change to depth writable
			DepthWritePassParameters->RenderTargets.DepthStencil.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite);

			FMeshPassProcessorRenderState NoDepthTestDrawRenderState(DrawRenderState);
			NoDepthTestDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_Always>::GetRHI());
			NoDepthTestDrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite);
			RenderEditorPrimitivesForDPG(GraphBuilder, View, DepthWritePassParameters, NoDepthTestDrawRenderState, SDPG_Foreground, InstanceCullingManager);
		}

		// Render foreground primitives with depth testing
		RenderEditorPrimitivesForDPG(GraphBuilder, View, PassParameters, DrawRenderState, SDPG_Foreground, InstanceCullingManager);
	}
}

void FDeferredShadingSceneRenderer::RenderBasePassInternal(
	FDeferredShadingSceneRenderer& Renderer,
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> InViews,
	const FSceneTextures& SceneTextures,
	const FRenderTargetBindingSlots& BasePassRenderTargets,
	FExclusiveDepthStencil::Type BasePassDepthStencilAccess,
	const FForwardBasePassTextures& ForwardBasePassTextures,
	bool bParallelBasePass,
	bool bRenderLightmapDensity,
	FInstanceCullingManager& InstanceCullingManager,
	bool bNaniteEnabled,
	FNaniteShadingCommands& NaniteBasePassShadingCommands,
	const TArrayView<Nanite::FRasterResults>& NaniteRasterResults)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderBasePass);
	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_RenderBasePass, FColor::Emerald);

	// This function has been deliberately made static, so it doesn't have access to the ViewFamily member, to support Custom Render Passes,
	// which have a separate ViewFamily.  Create a local reference to the first view's Family here.
	FViewFamilyInfo& ViewFamily = *(FViewFamilyInfo*)InViews[0].Family;
	FScene* Scene = ViewFamily.Scene->GetRenderScene();

#if WITH_DEBUG_VIEW_MODES
	Nanite::EDebugViewMode NaniteDebugViewMode = Nanite::EDebugViewMode::None;
	if (ViewFamily.EngineShowFlags.Wireframe)
	{
		NaniteDebugViewMode = Nanite::EDebugViewMode::Wireframe;
	}
	else if (bRenderLightmapDensity)
	{
		NaniteDebugViewMode = Nanite::EDebugViewMode::LightmapDensity;
	}
	else if (ViewFamily.EngineShowFlags.ActorColoration)
	{
		NaniteDebugViewMode = Nanite::EDebugViewMode::PrimitiveColor;
	}
	else if (ViewFamily.UseDebugViewPS())
	{
	    switch (ViewFamily.GetDebugViewShaderMode())
	    {
	    case DVSM_ShaderComplexity:							// Default shader complexity viewmode
	    case DVSM_ShaderComplexityContainedQuadOverhead:	// Show shader complexity with quad overdraw scaling the PS instruction count.
	    case DVSM_ShaderComplexityBleedingQuadOverhead:		// Show shader complexity with quad overdraw bleeding the PS instruction count over the quad.
	    case DVSM_QuadComplexity:							// Show quad overdraw only.
		    NaniteDebugViewMode = Nanite::EDebugViewMode::ShaderComplexity;
		    break;
		case DVSM_LWCComplexity:							// Show LWC function usage in materials
			NaniteDebugViewMode = Nanite::EDebugViewMode::LWCComplexity;
			break;
		case DVSM_StreamingDeficit:
			NaniteDebugViewMode = Nanite::EDebugViewMode::StreamingDeficit;
			break;
	    default:
		    break;
	    }
	}
#endif

	FRDGTextureRef NaniteColorTarget = SceneTextures.Color.Target;
	FRDGTextureRef NaniteDepthTarget = SceneTextures.Depth.Target;
#if WITH_EDITOR && WITH_DEBUG_VIEW_MODES
	if (NaniteDebugViewMode == Nanite::EDebugViewMode::Wireframe)
	{
		NaniteColorTarget = SceneTextures.EditorPrimitiveColor;
		NaniteDepthTarget = SceneTextures.EditorPrimitiveDepth;
	}
#endif

	auto RenderNaniteBasePass = [&](FViewInfo& View, int32 ViewIndex)
	{
		Nanite::FRasterResults& RasterResults = NaniteRasterResults[ViewIndex];
	#if WITH_DEBUG_VIEW_MODES
		if (NaniteDebugViewMode != Nanite::EDebugViewMode::None)
		{
			FRDGTextureRef NaniteDepthTargetCopy = GraphBuilder.CreateTexture(NaniteDepthTarget->Desc, TEXT("NaniteDepthTargetCopy"));
			AddCopyTexturePass(GraphBuilder, NaniteDepthTarget, NaniteDepthTargetCopy);

			Nanite::RenderDebugViewMode(
				GraphBuilder,
				NaniteDebugViewMode,
				*Scene,
				View,
				ViewFamily,
				RasterResults,
				NaniteColorTarget,
				NaniteDepthTargetCopy,
				NaniteDepthTarget
			);
		}
		else
	#endif
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, NaniteBasePass, "NaniteBasePass");

			Nanite::DispatchBasePass(
				GraphBuilder,
				NaniteBasePassShadingCommands,
				Renderer,
				SceneTextures,
				BasePassRenderTargets,
				*Scene,
				View,
				uint32(ViewIndex),
				RasterResults
			);
		}
	};

	if (bRenderLightmapDensity || ViewFamily.UseDebugViewPS())
	{
		// Debug view support for Nanite
		if (bNaniteEnabled)
		{
			// Should always have a full Z prepass with Nanite
			check(Renderer.ShouldRenderPrePass());

			for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
			{
				FViewInfo& View = InViews[ViewIndex];
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
				RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);

				RenderNaniteBasePass(View, ViewIndex);
			}
		}

		if (bRenderLightmapDensity)
		{
			// Override the base pass with the lightmap density pass if the viewmode is enabled.
			RenderLightMapDensities(GraphBuilder, InViews, BasePassRenderTargets);
		}
		else if (ViewFamily.UseDebugViewPS())
		{
			// Override the base pass with one of the debug view shader mode (see EDebugViewShaderMode) if required.
			RenderDebugViewMode(GraphBuilder, InViews, SceneTextures.DebugAux, BasePassRenderTargets[0], BasePassRenderTargets.DepthStencil);
		}
	}
	else
	{
		SCOPE_CYCLE_COUNTER(STAT_BasePassDrawTime);
		RDG_EVENT_SCOPE_STAT(GraphBuilder, Basepass, "BasePass");

		const bool bDrawSceneViewsInOneNanitePass = InViews.Num() > 1 && Nanite::ShouldDrawSceneViewsInOneNanitePass(InViews[0]);
		if (bParallelBasePass)
		{
			for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
			{
				FViewInfo& View = InViews[ViewIndex];
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
				RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);
				View.BeginRenderView();

				// Skip base pass skylight if Lumen GI is enabled, as Lumen handles the skylight.
				const bool bShouldRenderSkylightInBasePass = Scene->ShouldRenderSkylightInBasePass(/*bIsTranslucent*/false);
				const bool bLumenGIEnabled = Renderer.IsLumenGIEnabled(View);
				const bool bEnableSkylight = bShouldRenderSkylightInBasePass && !bLumenGIEnabled;

				FMeshPassProcessorRenderState DrawRenderState;
				SetupBasePassState(BasePassDepthStencilAccess, ViewFamily.EngineShowFlags.ShaderComplexity, DrawRenderState);

				FOpaqueBasePassParameters* PassParameters = GraphBuilder.AllocParameters<FOpaqueBasePassParameters>();
				PassParameters->View = View.GetShaderParameters();
				PassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;
				PassParameters->BasePass = CreateOpaqueBasePassUniformBuffer(GraphBuilder, View, ViewIndex, ForwardBasePassTextures, *SceneTextures.DBufferTextures, bEnableSkylight);
				PassParameters->RenderTargets = BasePassRenderTargets;
				PassParameters->RenderTargets.ShadingRateTexture = GVRSImageManager.GetVariableRateShadingImage(GraphBuilder, View, FVariableRateShadingImageManager::EVRSPassType::BasePass);

				const bool bShouldRenderView = View.ShouldRenderView();

				if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::BasePass]; Pass && bShouldRenderView)
				{
					Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

					GraphBuilder.AddDispatchPass(
						RDG_EVENT_NAME("BasePassParallel"),
						PassParameters,
						ERDGPassFlags::Raster,
						[Pass, PassParameters](FRDGDispatchPassBuilder& DispatchPassBuilder)
					{
						Pass->Dispatch(DispatchPassBuilder, &PassParameters->InstanceCullingDrawParams);
					});
				}
				else
				{
					InstanceCullingManager.SetDummyCullingParams(GraphBuilder, PassParameters->InstanceCullingDrawParams);
				}

				const bool bShouldRenderViewForNanite = bNaniteEnabled
					&& !View.bHasNoVisiblePrimitive
					&& (!bDrawSceneViewsInOneNanitePass || ViewIndex == 0); // when bDrawSceneViewsInOneNanitePass, the first view should cover all the other atlased ones
				if (bShouldRenderViewForNanite)
				{
					// Should always have a full Z prepass with Nanite
					check(Renderer.ShouldRenderPrePass());

					RenderNaniteBasePass(View, ViewIndex);
				}

				RenderEditorPrimitives(GraphBuilder, PassParameters, View, DrawRenderState, InstanceCullingManager);

				if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::SkyPass]; Pass && bShouldRenderView && View.Family->EngineShowFlags.Atmosphere)
				{
					FOpaqueBasePassParameters* SkyPassPassParameters = GraphBuilder.AllocParameters<FOpaqueBasePassParameters>();
					SkyPassPassParameters->BasePass = PassParameters->BasePass;
					SkyPassPassParameters->RenderTargets = BasePassRenderTargets;
					SkyPassPassParameters->View = View.GetShaderParameters();
					SkyPassPassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;

					// Remove all but the SceneColor
					for (uint32 i = 1; i < MaxSimultaneousRenderTargets; ++i)
					{
						SkyPassPassParameters->RenderTargets[i] = FRenderTargetBinding();
					}

					Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, SkyPassPassParameters->InstanceCullingDrawParams);

					GraphBuilder.AddDispatchPass(
						RDG_EVENT_NAME("SkyPassParallel"),
						SkyPassPassParameters,
						ERDGPassFlags::Raster,
						[Pass, SkyPassPassParameters](FRDGDispatchPassBuilder& DispatchPassBuilder)
					{
						Pass->Dispatch(DispatchPassBuilder, &SkyPassPassParameters->InstanceCullingDrawParams);
					});
				}
			}
		}
		else
		{
			for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
			{
				FViewInfo& View = InViews[ViewIndex];
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
				RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);
				View.BeginRenderView();

				// Skip base pass skylight if Lumen GI is enabled, as Lumen handles the skylight.
				const bool bShouldRenderSkylightInBasePass = Scene->ShouldRenderSkylightInBasePass(/*bIsTranslucent*/false);
				const bool bLumenGIEnabled = Renderer.IsLumenGIEnabled(View);
				const bool bEnableSkylight = bShouldRenderSkylightInBasePass && !bLumenGIEnabled;

				FMeshPassProcessorRenderState DrawRenderState;
				SetupBasePassState(BasePassDepthStencilAccess, ViewFamily.EngineShowFlags.ShaderComplexity, DrawRenderState);

				FOpaqueBasePassParameters* PassParameters = GraphBuilder.AllocParameters<FOpaqueBasePassParameters>();
				PassParameters->View = View.GetShaderParameters();
				PassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;
				PassParameters->BasePass = CreateOpaqueBasePassUniformBuffer(GraphBuilder, View, ViewIndex, ForwardBasePassTextures, *SceneTextures.DBufferTextures, bEnableSkylight);
				PassParameters->RenderTargets = BasePassRenderTargets;
				PassParameters->RenderTargets.ShadingRateTexture = GVRSImageManager.GetVariableRateShadingImage(GraphBuilder, View, FVariableRateShadingImageManager::EVRSPassType::BasePass);

				const bool bShouldRenderView = View.ShouldRenderView();

				if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::BasePass]; Pass && bShouldRenderView)
				{
					Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

					GraphBuilder.AddPass(
						RDG_EVENT_NAME("BasePass"),
						PassParameters,
						ERDGPassFlags::Raster,
						[&View, Pass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
					{
						SetStereoViewport(RHICmdList, View, 1.0f);
						Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
					});
				}
				else
				{
					InstanceCullingManager.SetDummyCullingParams(GraphBuilder, PassParameters->InstanceCullingDrawParams);
				}

				const bool bShouldRenderViewForNanite = bNaniteEnabled && (!bDrawSceneViewsInOneNanitePass || ViewIndex == 0); // when bDrawSceneViewsInOneNanitePass, the first view should cover all the other atlased ones
				if (bShouldRenderViewForNanite)
				{
					// Should always have a full Z prepass with Nanite
					check(Renderer.ShouldRenderPrePass());

					RenderNaniteBasePass(View, ViewIndex);
				}

				RenderEditorPrimitives(GraphBuilder, PassParameters, View, DrawRenderState, InstanceCullingManager);

				if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::SkyPass]; Pass && bShouldRenderView && View.Family->EngineShowFlags.Atmosphere)
				{
					FOpaqueBasePassParameters* SkyPassParameters = GraphBuilder.AllocParameters<FOpaqueBasePassParameters>();
					SkyPassParameters->BasePass = PassParameters->BasePass;
					SkyPassParameters->RenderTargets = BasePassRenderTargets;
					SkyPassParameters->View = View.GetShaderParameters();
					SkyPassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;

					// Remove all but the SceneColor
					for (uint32 i = 1; i < MaxSimultaneousRenderTargets; ++i)
					{
						SkyPassParameters->RenderTargets[i] = FRenderTargetBinding();
					}

					Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, SkyPassParameters->InstanceCullingDrawParams);

					GraphBuilder.AddPass(
						RDG_EVENT_NAME("SkyPass"),
						SkyPassParameters,
						ERDGPassFlags::Raster,
						[&View, Pass, SkyPassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
						{
							SetStereoViewport(RHICmdList, View, 1.0f);
							Pass->Draw(RHICmdList, &SkyPassParameters->InstanceCullingDrawParams);
						}
					);
				}
			}
		}
	}
}

template<typename LightMapPolicyType>
void FBasePassMeshProcessor::CollectPSOInitializersForLMPolicy(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams,
	const FMaterial& RESTRICT MaterialResource,
	FMaterialShadingModelField ShadingModels,
	const bool bDitheredLODTransition,
	const LightMapPolicyType& RESTRICT LightMapPolicy,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	EPrimitiveType PrimitiveType, 
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	// Get the shaders if possible for given vertex factory
	TMeshProcessorShaders<
		TBasePassVertexShaderPolicyParamType<LightMapPolicyType>,
		TBasePassPixelShaderPolicyParamType<LightMapPolicyType>> BasePassShaders;
	if (!GetBasePassShaders<LightMapPolicyType>(
		MaterialResource,
		VertexFactoryData.VertexFactoryType,
		LightMapPolicy,
		FeatureLevel,
		Get128BitRequirement(),
		bIsDebug,
		GBL_Default, // Currently only Nanite supports non-default layout)
		nullptr, // Mesh Shader
		&BasePassShaders.VertexShader,
		&BasePassShaders.PixelShader,
		bOITBasePass
		))
	{
		return;
	}

	// Generate multiple PSO's for LOD transition support? Have to check runtime hit for this?
	// Could do it only when hint is given from the PSOPrecacheParams
	bool bForceEnableStencilDitherState = false;

	// Setup the draw state
	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);
	SetDepthStencilStateForBasePass(
		DrawRenderState,
		FeatureLevel,
		bDitheredLODTransition,
		MaterialResource,
		bEnableReceiveDecalOutput,
		bForceEnableStencilDitherState);
	if (bTranslucentBasePass)
	{
		SetTranslucentRenderState(DrawRenderState, MaterialResource, GShaderPlatformForFeatureLevel[FeatureLevel], TranslucencyPassType);
	}

	// Setup the render target info for basepass
	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;

	// If translucent use different render target setup (see FDeferredShadingSceneRenderer::RenderTranslucencyInner)
	if (bTranslucentBasePass)
	{
		// Extent & scale is not important for PSOs
		FSeparateTranslucencyDimensions SeparateTranslucencyDimensions;
		SeparateTranslucencyDimensions.NumSamples = RenderTargetsInfo.NumSamples;

		EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
		const float DownsampleScale = 1.0;
		const bool bIsModulate = TranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOFModulate;
		const bool bDepthTest = TranslucencyPassType != ETranslucencyPass::TPT_TranslucencyAfterMotionBlur;
		const bool bRenderInSeparateTranslucency = IsSeparateTranslucencyEnabled(TranslucencyPassType, SeparateTranslucencyDimensions.Scale);

		// Always create PSO without separate translucency (could be used when under water for all translucent passes)
		EPixelFormat SceneColorFormat = SceneTexturesConfig.ColorFormat;
		ETextureCreateFlags SceneColorCreateFlags = SceneTexturesConfig.ColorCreateFlags;
		AddRenderTargetInfo(SceneColorFormat, SceneColorCreateFlags, RenderTargetsInfo);

		if (bDepthTest)
		{
			ETextureCreateFlags DepthStencilCreateFlags = SceneTexturesConfig.DepthCreateFlags;
			SetupDepthStencilInfo(PF_DepthStencil, DepthStencilCreateFlags, ERenderTargetLoadAction::ELoad,
				ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite, RenderTargetsInfo);
		}

		if (!bRenderInSeparateTranslucency || UPSOPrecacheSettingsManager::GetSettings().bTranslucencyUnderWater)
		{
			AddGraphicsPipelineStateInitializer(
				VertexFactoryData,
				MaterialResource,
				DrawRenderState,
				RenderTargetsInfo,
				BasePassShaders,
				MeshFillMode,
				MeshCullMode,
				PrimitiveType,
				EMeshPassFeatures::Default,
				true /*bRequired*/,
				PSOInitializers);
		}

		// Add another PSO when render in separate translucency because render target format could have changed
		if (bRenderInSeparateTranslucency)
		{
			const FRDGTextureDesc TextureDesc = GetPostDOFTranslucentTextureDesc(TranslucencyPassType, SeparateTranslucencyDimensions, bIsModulate, ShaderPlatform);
			RenderTargetsInfo.RenderTargetFormats[0] = TextureDesc.Format;
			RenderTargetsInfo.RenderTargetFlags[0] = TextureDesc.Flags;

			AddGraphicsPipelineStateInitializer(
				VertexFactoryData,
				MaterialResource,
				DrawRenderState,
				RenderTargetsInfo,
				BasePassShaders,
				MeshFillMode,
				MeshCullMode,
				PrimitiveType,
				EMeshPassFeatures::Default,
				true /*bRequired*/,
				PSOInitializers);
		}
	}
	else
	{
		if (PreCacheParams.bCanvasMaterial)
		{
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
			SetTranslucentRenderState(DrawRenderState, MaterialResource, GShaderPlatformForFeatureLevel[FeatureLevel], ETranslucencyPass::TPT_AllTranslucency);
			AddRenderTargetInfo(PreCacheParams.GetBassPixelFormat() != PF_Unknown ? PreCacheParams.GetBassPixelFormat() : PF_B8G8R8A8, TexCreate_RenderTargetable | TexCreate_ShaderResource, RenderTargetsInfo);
		}
		else
		{
			// Regular base pass with gbuffer bindings
			SetupGBufferRenderTargetInfo(SceneTexturesConfig, RenderTargetsInfo, true /*bSetupDepthStencil*/);
		}

		AddBasePassGraphicsPipelineStateInitializer(
			FeatureLevel,
			VertexFactoryData,
			MaterialResource,
			DrawRenderState,
			RenderTargetsInfo,
			BasePassShaders,
			MeshFillMode,
			MeshCullMode,
			PrimitiveType,
			true /*bPrecacheAlphaColorChannel*/,
			PSOCollectorIndex,
			PSOInitializers);
	}	
}

template<typename LightMapPolicyType>
bool FBasePassMeshProcessor::Process(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	const bool bIsMasked,
	const bool bIsTranslucent,
	FMaterialShadingModelField ShadingModels,
	const LightMapPolicyType& RESTRICT LightMapPolicy,
	const typename LightMapPolicyType::ElementDataType& RESTRICT LightMapElementData,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		TBasePassVertexShaderPolicyParamType<LightMapPolicyType>,
		TBasePassPixelShaderPolicyParamType<LightMapPolicyType>> BasePassShaders;

	if (!GetBasePassShaders<LightMapPolicyType>(
		MaterialResource,
		VertexFactory->GetType(),
		LightMapPolicy,
		FeatureLevel,
		Get128BitRequirement(),
		bIsDebug,
		GBL_Default, // Currently only Nanite uses non-default layout
		nullptr, // Mesh Shader
		&BasePassShaders.VertexShader,
		&BasePassShaders.PixelShader,
		bOITBasePass
		))
	{
		return false;
	}

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);

	bool bForceEnableStencilDitherState = false;
	if (ViewIfDynamicMeshCommand && StaticMeshId >= 0 && MeshBatch.bDitheredLODTransition)
	{
		checkSlow(ViewIfDynamicMeshCommand->bIsViewInfo);
		const FViewInfo* ViewInfo = static_cast<const FViewInfo*>(ViewIfDynamicMeshCommand);
		if (ViewInfo->bAllowStencilDither)
		{
			if (ViewInfo->StaticMeshFadeOutDitheredLODMap[StaticMeshId] || ViewInfo->StaticMeshFadeInDitheredLODMap[StaticMeshId])
			{
				bForceEnableStencilDitherState = true;
			}
		}
	}

	SetDepthStencilStateForBasePass(
		DrawRenderState,
		FeatureLevel,
		MeshBatch.bDitheredLODTransition,
		MaterialResource,
		bEnableReceiveDecalOutput,
		bForceEnableStencilDitherState);
	
	if (bEnableReceiveDecalOutput)
	{
		static const bool bSubstrateDufferPassEnabled = Substrate::IsSubstrateEnabled() && Substrate::IsDBufferPassEnabled(GShaderPlatformForFeatureLevel[FeatureLevel]);

		bool bHasRayTracingRepresentation = false;

		if (PrimitiveSceneProxy != nullptr)
		{
#if RHI_RAYTRACING
			bHasRayTracingRepresentation = IsRayTracingEnabled()
				? PrimitiveSceneProxy->HasRayTracingRepresentation() && PrimitiveSceneProxy->IsVisibleInRayTracing()
				: PrimitiveSceneProxy->HasDistanceFieldRepresentation();
#else
			bHasRayTracingRepresentation = PrimitiveSceneProxy->HasDistanceFieldRepresentation();
#endif
		}

		// Set stencil value for this draw call
		// This is effectively extending the GBuffer using the stencil bits
		uint8 StencilValue = 0;
		if (bSubstrateDufferPassEnabled)
		{
			// Set material's decal responsness through stencil bit. This is only used when Substrate DBuffer pass is enabled.
			// This 'stencil marking' allows Substrate's DBuffer pass to blend decal appropriately. It is only used for Simple 
			// and Single materials, as other materials (Complex, ...) get decal applied during the base pass.
			const uint8 SubstrateMaterialType	= MaterialResource.MaterialGetSubstrateMaterialType_RenderThread();
			const uint32 DecalResponse		= MaterialResource.GetMaterialDecalResponse();
			const bool bSupportDBufferPass	= SubstrateMaterialType == SUBSTRATE_MATERIAL_TYPE_SIMPLE || SubstrateMaterialType == SUBSTRATE_MATERIAL_TYPE_SINGLE;
			const bool bRoughnessResponse	= bSupportDBufferPass && (DecalResponse == MDR_ColorNormalRoughness || DecalResponse == MDR_ColorRoughness	|| DecalResponse == MDR_NormalRoughness || DecalResponse == MDR_Roughness);
			const bool bColorResponse		= bSupportDBufferPass && (DecalResponse == MDR_ColorNormalRoughness || DecalResponse == MDR_ColorNormal		|| DecalResponse == MDR_ColorRoughness	|| DecalResponse == MDR_Color);
			const bool bNormalResponse		= bSupportDBufferPass && (DecalResponse == MDR_ColorNormalRoughness || DecalResponse == MDR_ColorNormal		|| DecalResponse == MDR_NormalRoughness || DecalResponse == MDR_Normal);

			StencilValue =
			  GET_STENCIL_BIT_MASK(SUBSTRATE_RECEIVE_DBUFFER_NORMAL, bNormalResponse ? 0x1 : 0x0)
			| GET_STENCIL_BIT_MASK(SUBSTRATE_RECEIVE_DBUFFER_DIFFUSE, bColorResponse ? 0x1 : 0x0)
			| GET_STENCIL_BIT_MASK(SUBSTRATE_RECEIVE_DBUFFER_ROUGHNESS, bRoughnessResponse ? 0x1 : 0x0)
			| GET_STENCIL_BIT_MASK(RAY_TRACING_REPRESENTATION, bHasRayTracingRepresentation ? 0x1 : 0x0)
			| STENCIL_LIGHTING_CHANNELS_MASK(PrimitiveSceneProxy ? PrimitiveSceneProxy->GetLightingChannelStencilValue() : 0x00);
		}
		else
		{
			const bool bCastShadow = PrimitiveSceneProxy ? PrimitiveSceneProxy->CastsDynamicShadow() : true;

			StencilValue = 
			  GET_STENCIL_BIT_MASK(RECEIVE_DECAL, PrimitiveSceneProxy ? !!PrimitiveSceneProxy->ReceivesDecals() : 0x00)
			| GET_STENCIL_BIT_MASK(CAST_SHADOW, bCastShadow)
			| GET_STENCIL_BIT_MASK(RAY_TRACING_REPRESENTATION, bHasRayTracingRepresentation)
			| STENCIL_LIGHTING_CHANNELS_MASK(PrimitiveSceneProxy ? PrimitiveSceneProxy->GetLightingChannelStencilValue() : 0x00);
		}
		DrawRenderState.SetStencilRef(StencilValue);
	}

	if (bTranslucentBasePass)
	{
		SetTranslucentRenderState(DrawRenderState, MaterialResource, GShaderPlatformForFeatureLevel[FeatureLevel], TranslucencyPassType);
	}

	TBasePassShaderElementData<LightMapPolicyType> ShaderElementData(LightMapElementData);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

	FMeshDrawCommandSortKey SortKey = FMeshDrawCommandSortKey::Default;

	if (bTranslucentBasePass)
	{
		SortKey = CalculateTranslucentMeshStaticSortKey(PrimitiveSceneProxy, MeshBatch.MeshIdInPrimitive);
	}
	else
	{
		SortKey = CalculateBasePassMeshStaticSortKey(
			EarlyZPassMode,
			bIsMasked,
			PrimitiveSceneProxy && PrimitiveSceneProxy->TreatAsBackgroundForOcclusion(),
			BasePassShaders.VertexShader.GetShader(),
			BasePassShaders.PixelShader.GetShader());
	}

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		DrawRenderState,
		BasePassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

void FBasePassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (MeshBatch.bUseForMaterial)
	{
		// Determine the mesh's material and blend mode.
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material && Material->GetRenderingThreadShaderMap())
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

bool FBasePassMeshProcessor::ShouldDraw(const FMaterial& Material)
{
	// Determine the mesh's material and blend mode.
	const EBlendMode BlendMode = Material.GetBlendMode();
	const bool bIsTranslucent = IsTranslucentBlendMode(BlendMode);
	
	bool bShouldDraw = false;
	if (bTranslucentBasePass)
	{
		bShouldDraw = ShouldDrawInTranslucentBasePass(Material, TranslucencyPassType, AutoBeforeDOFTranslucencyBoundary);
	}
	else
	{
		bShouldDraw = !bIsTranslucent;
	}

	return bShouldDraw;
}

bool FBasePassMeshProcessor::TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material)
{
	// Determine the mesh's material and blend mode.
	const EBlendMode BlendMode = Material.GetBlendMode();
	const FMaterialShadingModelField ShadingModels = Material.GetShadingModels();
	const bool bIsMasked = IsMaskedBlendMode(BlendMode);
	const bool bIsTranslucent = IsTranslucentBlendMode(BlendMode);
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
	const bool bIsValidTranslucentNonDeferredDecal = PrimitiveSceneProxy && bIsTranslucent && !Material.IsDeferredDecal();

	bool bShouldDraw = false;
	if (AutoBeforeDOFTranslucencyBoundary > 0.0f && bIsValidTranslucentNonDeferredDecal)
	{
		check(ViewIfDynamicMeshCommand);
		check(TranslucencyPassType != ETranslucencyPass::TPT_MAX);

		const FVector& BoundsOrigin = PrimitiveSceneProxy->GetBounds().Origin;

		const FVector& ViewOrigin = ViewIfDynamicMeshCommand->ViewMatrices.GetViewOrigin();
		const FVector ViewForward = ViewIfDynamicMeshCommand->ViewMatrices.GetOverriddenTranslatedViewMatrix().GetColumn(2);

		const FVector CameraToObject = BoundsOrigin - ViewOrigin;
		float Distance = FVector::DotProduct(CameraToObject, ViewForward);
		bool bIsInDOFBackground = Distance > AutoBeforeDOFTranslucencyBoundary;

		bool bIsStandardTranslucency = !Material.IsTranslucencyAfterDOFEnabled() && !Material.IsTranslucencyAfterMotionBlurEnabled();
		bool bIsAfterDOF = Material.IsTranslucencyAfterDOFEnabled() && !Material.IsTranslucencyAfterMotionBlurEnabled() && BlendMode != BLEND_ColoredTransmittanceOnly;
		bool bIsAfterDOFModulate = Material.IsTranslucencyAfterDOFEnabled() && (Material.IsDualBlendingEnabled() || IsModulateBlendMode(BlendMode));

		// When AutoBeforeDOFTranslucencyBoundary is valid, we automatically move After DOF translucent meshes (never blurred by DOF)
		// before DOF if those elements are behind the focus distance.
		if (TranslucencyPassType == ETranslucencyPass::TPT_TranslucencyStandard)
		{
			if (bIsStandardTranslucency)
			{
				bShouldDraw = true;
			}
			else if (bIsInDOFBackground)
			{
				bShouldDraw = bIsAfterDOF || bIsAfterDOFModulate;
			}
		}
		else if (TranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOF)
		{
			bShouldDraw = bIsAfterDOF && !bIsInDOFBackground;
		}
		else if (TranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOFModulate)
		{
			bShouldDraw = bIsAfterDOFModulate && !bIsInDOFBackground;
		}
		else if (TranslucencyPassType == ETranslucencyPass::TPT_TranslucencyHoldout)
		{
			bShouldDraw = IsPrimitiveAlphaHoldoutEnabled();
		}
		else
		{
			unimplemented();
		}
	}
	else
	{
		bShouldDraw = ShouldDraw(Material);

		// Only translucent materials should be drawn in the holdout pass.
		if (TranslucencyPassType == ETranslucencyPass::TPT_TranslucencyHoldout)
		{
			bShouldDraw = bIsValidTranslucentNonDeferredDecal && IsPrimitiveAlphaHoldoutEnabled();
		}
	}

	// Only draw opaque materials.
	bool bResult = true;
	if (bShouldDraw
		&& (!PrimitiveSceneProxy || PrimitiveSceneProxy->ShouldRenderInMainPass())
		&& ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain())
		&& ShouldIncludeMaterialInDefaultOpaquePass(Material))
	{
		// Check for a cached light-map.
		const bool bIsLitMaterial = ShadingModels.IsLit();
		const bool bAllowStaticLighting = IsStaticLightingAllowed();

		const FLightMapInteraction LightMapInteraction = (bAllowStaticLighting && MeshBatch.LCI && bIsLitMaterial)
			? MeshBatch.LCI->GetLightMapInteraction(FeatureLevel)
			: FLightMapInteraction();

		const bool bAllowIndirectLightingCache = Scene && Scene->PrecomputedLightVolumes.Num() > 0;
		const bool bUseVolumetricLightmap = Scene && Scene->VolumetricLightmapSceneData.HasData();

		FMeshMaterialShaderElementData MeshMaterialShaderElementData;
		MeshMaterialShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

		// Render volumetric translucent self-shadowing only for >= SM4 and fallback to non-shadowed for lesser shader models
		if (bIsLitMaterial
			&& bIsTranslucent
			&& PrimitiveSceneProxy
			&& PrimitiveSceneProxy->CastsVolumetricTranslucentShadow())
		{
			checkSlow(ViewIfDynamicMeshCommand && ViewIfDynamicMeshCommand->bIsViewInfo);
			const FViewInfo* ViewInfo = (FViewInfo*)ViewIfDynamicMeshCommand;

			const int32 PrimitiveIndex = PrimitiveSceneProxy->GetPrimitiveSceneInfo()->GetIndex();

			const FUniformBufferRHIRef* UniformBufferPtr = ViewInfo->TranslucentSelfShadowUniformBufferMap.Find(PrimitiveIndex);

			FSelfShadowLightCacheElementData ElementData;
			ElementData.LCI = MeshBatch.LCI;
			ElementData.SelfShadowTranslucencyUniformBuffer = UniformBufferPtr ? (*UniformBufferPtr).GetReference() : GEmptyTranslucentSelfShadowUniformBuffer.GetUniformBufferRHI();

			if (bIsLitMaterial
				&& bAllowStaticLighting
				&& bUseVolumetricLightmap
				&& PrimitiveSceneProxy)
			{
				bResult = Process< FSelfShadowedVolumetricLightmapPolicy >(
					MeshBatch,
					BatchElementMask,
					StaticMeshId,
					PrimitiveSceneProxy,
					MaterialRenderProxy,
					Material,
					bIsMasked,
					bIsTranslucent,
					ShadingModels,
					FSelfShadowedVolumetricLightmapPolicy(),
					ElementData,
					MeshFillMode,
					MeshCullMode);
			}
			else if (IsIndirectLightingCacheAllowed(FeatureLevel)
				&& bAllowIndirectLightingCache
				&& PrimitiveSceneProxy)
			{
				// Apply cached point indirect lighting as well as self shadowing if needed
				bResult = Process< FSelfShadowedCachedPointIndirectLightingPolicy >(
					MeshBatch,
					BatchElementMask,
					StaticMeshId,
					PrimitiveSceneProxy,
					MaterialRenderProxy,
					Material,
					bIsMasked,
					bIsTranslucent,
					ShadingModels,
					FSelfShadowedCachedPointIndirectLightingPolicy(),
					ElementData,
					MeshFillMode,
					MeshCullMode);
			}
			else
			{
				bResult = Process< FSelfShadowedTranslucencyPolicy >(
					MeshBatch,
					BatchElementMask,
					StaticMeshId,
					PrimitiveSceneProxy,
					MaterialRenderProxy,
					Material,
					bIsMasked,
					bIsTranslucent,
					ShadingModels,
					FSelfShadowedTranslucencyPolicy(),
					ElementData.SelfShadowTranslucencyUniformBuffer,
					MeshFillMode,
					MeshCullMode);
			}
		}
		else
		{
			ELightMapPolicyType UniformLightMapPolicyType = GetUniformLightMapPolicyType(FeatureLevel, Scene, MeshBatch.LCI, PrimitiveSceneProxy, Material);
			bResult = Process< FUniformLightMapPolicy >(
				MeshBatch,
				BatchElementMask,
				StaticMeshId,
				PrimitiveSceneProxy,
				MaterialRenderProxy,
				Material,
				bIsMasked,
				bIsTranslucent,
				ShadingModels,
				FUniformLightMapPolicy(UniformLightMapPolicyType),
				MeshBatch.LCI,
				MeshFillMode,
				MeshCullMode);
		}
	}

	return bResult;
}

ELightMapPolicyType FBasePassMeshProcessor::GetUniformLightMapPolicyType(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FLightCacheInterface* LCI, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const FMaterial& Material)
{
	// Check for a cached light-map.
	const bool bIsTranslucent = IsTranslucentBlendMode(Material);
	const FMaterialShadingModelField ShadingModels = Material.GetShadingModels();
	const bool bIsLitMaterial = ShadingModels.IsLit();
	const bool bAllowStaticLighting = IsStaticLightingAllowed();

	const FLightMapInteraction LightMapInteraction = (bAllowStaticLighting && LCI && bIsLitMaterial)
		? LCI->GetLightMapInteraction(FeatureLevel)
		: FLightMapInteraction();

	// force LQ lightmaps based on system settings
	const bool bPlatformAllowsHighQualityLightMaps = AllowHighQualityLightmaps(FeatureLevel);
	const bool bAllowHighQualityLightMaps = bPlatformAllowsHighQualityLightMaps && LightMapInteraction.AllowsHighQualityLightmaps();

	const bool bAllowIndirectLightingCache = Scene && Scene->PrecomputedLightVolumes.Num() > 0;
	const bool bUseVolumetricLightmap = Scene && Scene->VolumetricLightmapSceneData.HasData();

	static const auto CVarSupportLowQualityLightmap = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
	const bool bAllowLowQualityLightMaps = (!CVarSupportLowQualityLightmap) || (CVarSupportLowQualityLightmap->GetValueOnAnyThread() != 0);

	ELightMapPolicyType LightMapPolicyType = LMP_DUMMY;
	switch (LightMapInteraction.GetType())
	{
	case LMIT_Texture:
		if (bAllowHighQualityLightMaps)
		{
			const FShadowMapInteraction ShadowMapInteraction = (bAllowStaticLighting && LCI && bIsLitMaterial)
				? LCI->GetShadowMapInteraction(FeatureLevel)
				: FShadowMapInteraction();

			if (ShadowMapInteraction.GetType() == SMIT_Texture)
			{
				LightMapPolicyType = LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP;
			}
			else
			{
				LightMapPolicyType = LMP_HQ_LIGHTMAP;
			}
		}
		else if (bAllowLowQualityLightMaps)
		{
			LightMapPolicyType = LMP_LQ_LIGHTMAP;
		}
		else
		{
			LightMapPolicyType = LMP_NO_LIGHTMAP;
		}
		break;
	default:
		if (bIsLitMaterial
			&& bAllowStaticLighting
			&& bUseVolumetricLightmap
			&& PrimitiveSceneProxy
			&& (PrimitiveSceneProxy->IsMovable()
				|| PrimitiveSceneProxy->NeedsUnbuiltPreviewLighting()
				|| PrimitiveSceneProxy->GetLightmapType() == ELightmapType::ForceVolumetric))
		{
			LightMapPolicyType = LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING;
		}
		else if (bIsLitMaterial
			&& IsIndirectLightingCacheAllowed(FeatureLevel)
			&& Scene
			&& Scene->PrecomputedLightVolumes.Num() > 0
			&& PrimitiveSceneProxy)
		{
			const FIndirectLightingCacheAllocation* IndirectLightingCacheAllocation = PrimitiveSceneProxy->GetPrimitiveSceneInfo()->IndirectLightingCacheAllocation;
			const bool bPrimitiveIsMovable = PrimitiveSceneProxy->IsMovable();
			const bool bPrimitiveUsesILC = PrimitiveSceneProxy->GetIndirectLightingCacheQuality() != ILCQ_Off;

			// Use the indirect lighting cache shaders if the object has a cache allocation
			// This happens for objects with unbuilt lighting
			if (bPrimitiveUsesILC &&
				((IndirectLightingCacheAllocation && IndirectLightingCacheAllocation->IsValid())
					// Use the indirect lighting cache shaders if the object is movable, it may not have a cache allocation yet because that is done in InitViews
					// And movable objects are sometimes rendered in the static draw lists
					|| bPrimitiveIsMovable))
			{
				if (CanIndirectLightingCacheUseVolumeTexture(FeatureLevel)
					// Translucency forces point sample for pixel performance
					&& !bIsTranslucent
					&& ((IndirectLightingCacheAllocation && !IndirectLightingCacheAllocation->bPointSample)
						|| (bPrimitiveIsMovable && PrimitiveSceneProxy->GetIndirectLightingCacheQuality() == ILCQ_Volume)))
				{
					// Use a lightmap policy that supports reading indirect lighting from a volume texture for dynamic objects
					LightMapPolicyType = LMP_CACHED_VOLUME_INDIRECT_LIGHTING;
				}
				else
				{
					// Use a lightmap policy that supports reading indirect lighting from a single SH sample
					LightMapPolicyType = LMP_CACHED_POINT_INDIRECT_LIGHTING;
				}
			}
			else
			{
				LightMapPolicyType = LMP_NO_LIGHTMAP;
			}
		}
		else
		{
			LightMapPolicyType = LMP_NO_LIGHTMAP;
		}
		break;
	};

	check(LightMapPolicyType != LMP_DUMMY);

	return LightMapPolicyType;
}

TArray<ELightMapPolicyType, TInlineAllocator<2>> FBasePassMeshProcessor::GetUniformLightMapPolicyTypeForPSOCollection(ERHIFeatureLevel::Type FeatureLevel, const FMaterial& Material)
{
	const FMaterialShadingModelField ShadingModels = Material.GetShadingModels();

	const bool bIsTranslucent = IsTranslucentBlendMode(Material);
	const bool bIsLitMaterial = ShadingModels.IsLit();
	const bool bAllowStaticLighting = IsStaticLightingAllowed();

	const bool bPlatformAllowsHighQualityLightMaps = AllowHighQualityLightmaps(FeatureLevel);
	static const auto CVarSupportLowQualityLightmap = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
	const bool bAllowLowQualityLightMaps = (!CVarSupportLowQualityLightmap) || (CVarSupportLowQualityLightmap->GetValueOnAnyThread() != 0);

	// Retrieve those values or have as global precache params (if not known then we have to assume they can be used for now)
	const bool bAllowIndirectLightingCache = true;// Scene&& Scene->PrecomputedLightVolumes.Num() > 0;
	const bool bUseVolumetricLightmap = true;// Scene&& Scene->VolumetricLightmapSceneData.HasData();

	TArray<ELightMapPolicyType, TInlineAllocator<2>> UniformLightMapPolicyTypes;

	// always add the fallback no lightmap mode
	UniformLightMapPolicyTypes.Add(LMP_NO_LIGHTMAP);

	if (UPSOPrecacheSettingsManager::GetSettings().bBasePassApplyLightMapPolicies)
	{
		// Simplified version of GetUniformLightMapPolicyType with worst case to collect all types which could be enabled at runtime	

		// LightMapInterationType::LMIT_Texture
		{
			if (bPlatformAllowsHighQualityLightMaps)
			{
				// EShadowMapInteractionType::SMIT_Texture
				if (bAllowStaticLighting && bIsLitMaterial)
				{
					UniformLightMapPolicyTypes.Add(LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP);
				}
				// else
				{
					UniformLightMapPolicyTypes.Add(LMP_HQ_LIGHTMAP);
				}
			}
			if (bAllowLowQualityLightMaps)
			{
				UniformLightMapPolicyTypes.Add(LMP_LQ_LIGHTMAP);
			}
		}
		// else LightMapInterationType
		{
			if (bIsLitMaterial && bAllowStaticLighting && bUseVolumetricLightmap)
			{
				UniformLightMapPolicyTypes.Add(LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING);
			}

			if (IsIndirectLightingCacheAllowed(FeatureLevel) && bAllowIndirectLightingCache)
			{
				if (CanIndirectLightingCacheUseVolumeTexture(FeatureLevel)
					// Translucency forces point sample for pixel performance
					&& !bIsTranslucent)
				{
					UniformLightMapPolicyTypes.Add(LMP_CACHED_VOLUME_INDIRECT_LIGHTING);
				}
				// else
				{
					UniformLightMapPolicyTypes.Add(LMP_CACHED_POINT_INDIRECT_LIGHTING);
				}
			}
		}
	}

	return UniformLightMapPolicyTypes;
}

void FBasePassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	// Check if material should be rendered
	bool bShouldDraw = ShouldDraw(Material)
		&& ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain())
		&& ShouldIncludeMaterialInDefaultOpaquePass(Material);
	bShouldDraw = bShouldDraw || (PreCacheParams.bCanvasMaterial && MeshPassType == EMeshPass::BasePass);
	if (!bShouldDraw || !PreCacheParams.bRenderInMainPass)
	{
		return;
	}

	// Ignore UI materials for PSO precaching on base pass
	if (Material.IsUIMaterial())
	{
		return;
	}

	// PSO precaching enabled for TranslucencyAll
	if (MeshPassType == EMeshPass::TranslucencyAll && !UPSOPrecacheSettingsManager::GetSettings().bTranslucencyAll)
	{
		return;
	}

	// Is hold out enabled
	if (MeshPassType == EMeshPass::TranslucencyHoldout && !IsPrimitiveAlphaHoldoutEnabled())
	{
		return;
	}

	// Determine the mesh's material and blend mode.
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

	bool bMovable = PreCacheParams.Mobility == EComponentMobility::Movable || PreCacheParams.Mobility == EComponentMobility::Stationary;
	bool bDitheredLODTransition = !bMovable && Material.IsDitheredLODTransition() && !PreCacheParams.bForceLODModel;

	CollectPSOInitializers(SceneTexturesConfig, VertexFactoryData, PreCacheParams, Material, bDitheredLODTransition, MeshFillMode, MeshCullMode, (EPrimitiveType)PreCacheParams.PrimitiveType, PSOInitializers);
}

void FBasePassMeshProcessor::CollectPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams,
	const FMaterial& RESTRICT Material,
	const bool bDitheredLODTransition,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	EPrimitiveType PrimitiveType, 
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	const FMaterialShadingModelField ShadingModels = Material.GetShadingModels();

	const bool bIsTranslucent = IsTranslucentBlendMode(Material);
	const bool bIsLitMaterial = ShadingModels.IsLit();
	
	if (UPSOPrecacheSettingsManager::GetSettings().bBasePassApplyLightMapPolicies && bIsLitMaterial && bIsTranslucent)
	{
		const bool bAllowStaticLighting = IsStaticLightingAllowed();

		// Retrieve those values or have as global precache params (if not known then we have to assume they can be used for now)
		const bool bAllowIndirectLightingCache = true;// Scene&& Scene->PrecomputedLightVolumes.Num() > 0;
		const bool bUseVolumetricLightmap = true;// Scene&& Scene->VolumetricLightmapSceneData.HasData();

		if (bAllowStaticLighting && bUseVolumetricLightmap)
		{
			CollectPSOInitializersForLMPolicy< FSelfShadowedVolumetricLightmapPolicy >(
				SceneTexturesConfig, VertexFactoryData, PreCacheParams, Material, ShadingModels, bDitheredLODTransition,
				FSelfShadowedVolumetricLightmapPolicy(), MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
		}

		if (IsIndirectLightingCacheAllowed(FeatureLevel) && bAllowIndirectLightingCache)
		{
			CollectPSOInitializersForLMPolicy< FSelfShadowedCachedPointIndirectLightingPolicy >(
				SceneTexturesConfig, VertexFactoryData, PreCacheParams, Material, ShadingModels, bDitheredLODTransition,
				FSelfShadowedCachedPointIndirectLightingPolicy(), MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
		}

		CollectPSOInitializersForLMPolicy< FSelfShadowedTranslucencyPolicy >(
			SceneTexturesConfig, VertexFactoryData, PreCacheParams, Material, ShadingModels, bDitheredLODTransition,
			FSelfShadowedTranslucencyPolicy(), MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
	}

	TArray<ELightMapPolicyType, TInlineAllocator<2>> UniformLightMapPolicyTypes = GetUniformLightMapPolicyTypeForPSOCollection(FeatureLevel, Material);
	for (ELightMapPolicyType LightMapPolicyType : UniformLightMapPolicyTypes)
	{
		CollectPSOInitializersForLMPolicy< FUniformLightMapPolicy >(
			SceneTexturesConfig, VertexFactoryData, PreCacheParams, Material, ShadingModels, bDitheredLODTransition,
			FUniformLightMapPolicy(LightMapPolicyType), MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
	}
}

FBasePassMeshProcessor::FBasePassMeshProcessor(
	EMeshPass::Type InMeshPassType,
	const FScene* Scene,
	ERHIFeatureLevel::Type InFeatureLevel,
	const FSceneView* InViewIfDynamicMeshCommand,
	const FMeshPassProcessorRenderState& InDrawRenderState,
	FMeshPassDrawListContext* InDrawListContext,
	EFlags Flags,
	ETranslucencyPass::Type InTranslucencyPassType)
	: FMeshPassProcessor(InMeshPassType, Scene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InDrawRenderState)
	, bIsDebug(Scene ? Scene->RequiresDebugMaterials() : false)
	, TranslucencyPassType(InTranslucencyPassType)
	, bTranslucentBasePass(InTranslucencyPassType != ETranslucencyPass::TPT_MAX)
	, bOITBasePass(InTranslucencyPassType != ETranslucencyPass::TPT_MAX && OIT::IsSortedPixelsEnabled(GetFeatureLevelShaderPlatform(InFeatureLevel)))
	, bEnableReceiveDecalOutput((Flags & EFlags::CanUseDepthStencil) == EFlags::CanUseDepthStencil)
	, EarlyZPassMode(Scene ? Scene->EarlyZPassMode : DDM_None)
	, bRequiresExplicit128bitRT((Flags & EFlags::bRequires128bitRT) == EFlags::bRequires128bitRT)
{
	if (InTranslucencyPassType != ETranslucencyPass::TPT_MAX && InViewIfDynamicMeshCommand && ViewIfDynamicMeshCommand->bIsViewInfo)
	{
		const FViewInfo* ViewInfo = (FViewInfo*)ViewIfDynamicMeshCommand;
		if (InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyStandard ||
			InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOF ||
			InTranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOFModulate)
		{
			AutoBeforeDOFTranslucencyBoundary = ViewInfo->AutoBeforeDOFTranslucencyBoundary;
		}
	}
}

FMeshPassProcessor* CreateBasePassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	const FExclusiveDepthStencil::Type DefaultBasePassDepthStencilAccess = FScene::GetDefaultBasePassDepthStencilAccess(FeatureLevel);
	
	FMeshPassProcessorRenderState PassDrawRenderState;
	SetupBasePassState(DefaultBasePassDepthStencilAccess, false, PassDrawRenderState);

	const FBasePassMeshProcessor::EFlags Flags = FBasePassMeshProcessor::EFlags::CanUseDepthStencil;

	return new FBasePassMeshProcessor(EMeshPass::BasePass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags);
}

FMeshPassProcessor* CreateTranslucencyStandardPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	const FBasePassMeshProcessor::EFlags Flags = FBasePassMeshProcessor::EFlags::CanUseDepthStencil;

	return new FBasePassMeshProcessor(EMeshPass::TranslucencyStandard, Scene, FeatureLevel, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_TranslucencyStandard);
}

FMeshPassProcessor* CreateTranslucencyStandardModulatePassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	const FBasePassMeshProcessor::EFlags Flags = FBasePassMeshProcessor::EFlags::CanUseDepthStencil;

	return new FBasePassMeshProcessor(EMeshPass::TranslucencyStandardModulate, Scene, FeatureLevel, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_TranslucencyStandardModulate);
}

FMeshPassProcessor* CreateTranslucencyAfterDOFProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	const FBasePassMeshProcessor::EFlags Flags = FBasePassMeshProcessor::EFlags::CanUseDepthStencil;

	return new FBasePassMeshProcessor(EMeshPass::TranslucencyAfterDOF, Scene, FeatureLevel, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_TranslucencyAfterDOF);
}

FMeshPassProcessor* CreateTranslucencyAfterDOFModulateProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	const FBasePassMeshProcessor::EFlags Flags = FBasePassMeshProcessor::EFlags::CanUseDepthStencil;

	return new FBasePassMeshProcessor(EMeshPass::TranslucencyAfterDOFModulate, Scene, FeatureLevel, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_TranslucencyAfterDOFModulate);
}

FMeshPassProcessor* CreateTranslucencyAfterMotionBlurProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;	
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
	PassDrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthNop_StencilNop);

	const FBasePassMeshProcessor::EFlags Flags = FBasePassMeshProcessor::EFlags::None;

	return new FBasePassMeshProcessor(EMeshPass::TranslucencyAfterMotionBlur, Scene, FeatureLevel, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_TranslucencyAfterMotionBlur);
}

FMeshPassProcessor* CreateTranslucencyHoldoutPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	const FBasePassMeshProcessor::EFlags Flags = FBasePassMeshProcessor::EFlags::CanUseDepthStencil;

	return new FBasePassMeshProcessor(EMeshPass::TranslucencyHoldout, Scene, FeatureLevel, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_TranslucencyHoldout);
}

FMeshPassProcessor* CreateTranslucencyAllPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	const FBasePassMeshProcessor::EFlags Flags = FBasePassMeshProcessor::EFlags::CanUseDepthStencil;

	return new FBasePassMeshProcessor(EMeshPass::TranslucencyAll, Scene, FeatureLevel, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_AllTranslucency);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(BasePass, CreateBasePassProcessor, EShadingPath::Deferred, EMeshPass::BasePass, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(TranslucencyStandardPass, CreateTranslucencyStandardPassProcessor, EShadingPath::Deferred, EMeshPass::TranslucencyStandard, EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(TranslucencyStandardModulatePass, CreateTranslucencyStandardModulatePassProcessor, EShadingPath::Deferred, EMeshPass::TranslucencyStandardModulate, EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(TranslucencyAfterDOFPass, CreateTranslucencyAfterDOFProcessor, EShadingPath::Deferred, EMeshPass::TranslucencyAfterDOF, EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(TranslucencyAfterDOFModulatePass, CreateTranslucencyAfterDOFModulateProcessor, EShadingPath::Deferred, EMeshPass::TranslucencyAfterDOFModulate, EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(TranslucencyAfterMotionBlurPass, CreateTranslucencyAfterMotionBlurProcessor, EShadingPath::Deferred, EMeshPass::TranslucencyAfterMotionBlur, EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(TranslucencyHoldoutPass, CreateTranslucencyHoldoutPassProcessor, EShadingPath::Deferred, EMeshPass::TranslucencyHoldout, EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(TranslucencyAllPass, CreateTranslucencyAllPassProcessor, EShadingPath::Deferred, EMeshPass::TranslucencyAll, EMeshPassFlags::MainView);
