// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BasePassRendering.h: Base pass rendering definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "HitProxies.h"
#include "RHIStaticStates.h"
#include "SceneManagement.h"
#include "Materials/Material.h"
#include "PostProcess/SceneRenderTargets.h"
#include "DBufferTextures.h"
#include "LightMapRendering.h"
#include "VelocityRendering.h"
#include "MeshMaterialShaderType.h"
#include "MeshMaterialShader.h"
#include "ShaderBaseClasses.h"
#include "FogRendering.h"
#include "TranslucentLighting.h"
#include "PlanarReflectionRendering.h"
#include "UnrealEngine.h"
#include "ReflectionEnvironment.h"
#include "Substrate/Substrate.h"
#include "OIT/OITParameters.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VolumetricCloudRendering.h"
#include "Nanite/NaniteMaterials.h"
#include "BlueNoise.h"
#include "LocalFogVolumeRendering.h"
#include "LightFunctionAtlas.h"
#include "RenderUtils.h"
#include "SceneTexturesConfig.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "PSOPrecacheSettings.h"

class FScene;

template<typename TBufferStruct> class TUniformBufferRef;

struct FSceneWithoutWaterTextures;

class FViewInfo;
class UMaterialExpressionSingleLayerWaterMaterialOutput;

namespace OIT
{
	bool IsSortedPixelsEnabledForProject(EShaderPlatform InPlatform);
}

/** Whether to allow the indirect lighting cache to be applied to dynamic objects. */
extern int32 GIndirectLightingCache;

class FForwardLightData
{
public:
	FVector4f LightPositionAndInvRadius;
	FVector4f LightColorAndIdAndFalloffExponentAndRayEndBias;
	FVector4f LightDirectionAndSceneInfoExtraDataPacked;
	FVector4f SpotAnglesAndSourceRadiusPacked;
	FVector4f LightTangentAndIESDataAndSpecularScale;
	FVector4f RectDataAndVirtualShadowMapIdOrPrevLocalLightIndex;
};

struct FForwardBasePassTextures
{
	FRDGTextureRef ScreenSpaceAO = nullptr;
	FRDGTextureRef ScreenSpaceShadowMask = nullptr;
	FRDGTextureRef SceneDepthIfResolved = nullptr;
	bool bIs24BitUnormDepthStencil = false;
};

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSharedBasePassUniformParameters,)
	SHADER_PARAMETER_STRUCT(FForwardLightUniformParameters, Forward)
	SHADER_PARAMETER_STRUCT(FForwardDirectionalLightShadowMapParameters, ForwardDirLightShadow)
	SHADER_PARAMETER_STRUCT(FReflectionUniformParameters, Reflection)
	SHADER_PARAMETER_STRUCT(FPlanarReflectionUniformParameters, PlanarReflection) // Single global planar reflection for the forward pass.
	SHADER_PARAMETER_STRUCT(FFogUniformParameters, Fog)
	SHADER_PARAMETER_STRUCT(FFogUniformParameters, FogISR)
	SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
	SHADER_PARAMETER_STRUCT(LightFunctionAtlas::FLightFunctionAtlasGlobalParameters, LightFunctionAtlas)
	SHADER_PARAMETER(uint32, UseBasePassSkylight)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FOpaqueBasePassUniformParameters,)
	SHADER_PARAMETER_STRUCT(FSharedBasePassUniformParameters, Shared)
	SHADER_PARAMETER_STRUCT(FSubstrateBasePassUniformParameters, Substrate)
	// Forward shading 
	SHADER_PARAMETER(int32, UseForwardScreenSpaceShadowMask)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ForwardScreenSpaceShadowMaskTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, IndirectOcclusionTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ResolvedSceneDepthTexture)
	// DBuffer decals
	SHADER_PARAMETER_STRUCT_INCLUDE(FDBufferParameters, DBuffer)
	// Misc
	SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGFTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)
	SHADER_PARAMETER(int32, Is24BitUnormDepthStencil)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FNaniteTranslucencyParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedNaniteView>, InViews)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FNaniteRasterGroupMeta>, RasterGroupMeta)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FNaniteRasterBinMeta>, RasterBinMeta)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, RasterBinData)
END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FTranslucentBasePassUniformParameters,)
	SHADER_PARAMETER_STRUCT(FSharedBasePassUniformParameters, Shared)
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT(FSubstrateForwardPassUniformParameters, Substrate)
	SHADER_PARAMETER_STRUCT(FLightCloudTransmittanceParameters, ForwardDirLightCloudShadow)
	SHADER_PARAMETER_STRUCT(FOITBasePassUniformParameters, OIT)
	// Material SSR
	SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
	SHADER_PARAMETER(FVector4f, PrevScreenPositionScaleBias)
	SHADER_PARAMETER(FVector2f, PrevSceneColorBilinearUVMin)
	SHADER_PARAMETER(FVector2f, PrevSceneColorBilinearUVMax)
	SHADER_PARAMETER(float, PrevSceneColorPreExposureInv)
	SHADER_PARAMETER(int32, SSRQuality)

	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, PrevSceneColor)
	SHADER_PARAMETER_SAMPLER(SamplerState, PrevSceneColorSampler)
	// Volumetric cloud
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VolumetricCloudColor)
	SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudColorSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VolumetricCloudDepth)
	SHADER_PARAMETER_SAMPLER(SamplerState, VolumetricCloudDepthSampler)
	SHADER_PARAMETER(float, ApplyVolumetricCloudOnTransparent)
	SHADER_PARAMETER(float, SoftBlendingDistanceKm)
	SHADER_PARAMETER(FVector2f, VolumetricCloudColorUVScale)
	SHADER_PARAMETER(FVector2f, VolumetricCloudColorUVMax)
	// Translucency Lighting Volume
	SHADER_PARAMETER_STRUCT_INCLUDE(FTranslucencyLightingVolumeParameters, TranslucencyLightingVolume)
	SHADER_PARAMETER_STRUCT_INCLUDE(FFrontLayerTranslucencyParameters, FrontLayerTranslucency)
	SHADER_PARAMETER_STRUCT_INCLUDE(FLumenTranslucencyLightingParameters, LumenTranslucencyLighting)
	SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGFTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)
	// Misc
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorCopyTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorCopySampler)
	SHADER_PARAMETER_STRUCT(FBlueNoiseParameters, BlueNoise)
	SHADER_PARAMETER_STRUCT(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)
	SHADER_PARAMETER_STRUCT(FNaniteTranslucencyParameters, NaniteTranslucency)
	SHADER_PARAMETER(int, TranslucencyPass)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FTranslucentBasePassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_REF(FReflectionCaptureShaderData, ReflectionCapture)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FTranslucentBasePassUniformParameters, BasePass)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRasterUniformParameters, NaniteRaster)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteShadingUniformParameters, NaniteShading)
	RDG_BUFFER_ACCESS(NaniteIndirectArgs, ERHIAccess::IndirectArgs)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

DECLARE_GPU_DRAWCALL_STAT_EXTERN(Basepass);
DECLARE_GPU_STAT_NAMED_EXTERN(NaniteBasePass, TEXT("Nanite BasePass"));

extern void SetupSharedBasePassParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const int32 ViewIndex,
	FSharedBasePassUniformParameters& BasePassParameters,
	bool bForRealtimeSkyCapture = false);

extern TRDGUniformBufferRef<FOpaqueBasePassUniformParameters> CreateOpaqueBasePassUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const int32 ViewIndex = 0,
	const FForwardBasePassTextures& ForwardBasePassTextures = {},
	const FDBufferTextures& DBufferTextures = {},
	bool bEnableSkylight = false,
	bool bForRealtimeSkyCapture = false);

extern TRDGUniformBufferRef<FTranslucentBasePassUniformParameters> CreateTranslucentBasePassUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const int32 ViewIndex = 0,
	const FTranslucencyLightingVolumeTextures& TranslucencyLightingVolumeTextures = {},
	FRDGTextureRef SceneColorCopyTexture = nullptr,
	const ESceneTextureSetupMode SceneTextureSetupMode = ESceneTextureSetupMode::None,
	bool bLumenGIEnabled = false,
	const TArrayView<Nanite::FRasterResults>& NaniteRasterResults = {},
	ETranslucencyPass::Type TranslucencyPass = ETranslucencyPass::TPT_MAX);

extern bool IsGBufferLayoutSupportedForMaterial(EGBufferLayout Layout, const FMeshMaterialShaderPermutationParameters& Params);
extern void ModifyBasePassCSPSCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Params, EGBufferLayout GBufferLayout, FShaderCompilerEnvironment& OutEnvironment);

/** Parameters for computing forward lighting. */
class FForwardLightingParameters
{
public:

	static void ModifyCompilationEnvironment(EShaderPlatform Platform, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("FORWARD_LIGHT_DATA_STRIDE"), FMath::DivideAndRoundUp<int32>(sizeof(FForwardLightData), sizeof(FVector4f)));
		extern int32 NumCulledLightsGridStride;
		OutEnvironment.SetDefine(TEXT("NUM_CULLED_LIGHTS_GRID_STRIDE"), NumCulledLightsGridStride);
		extern int32 NumCulledGridPrimitiveTypes;
		OutEnvironment.SetDefine(TEXT("NUM_CULLED_GRID_PRIMITIVE_TYPES"), NumCulledGridPrimitiveTypes);
	}
};

template<typename LightMapPolicyType>
class TBasePassShaderElementData : public FMeshMaterialShaderElementData
{
public:
	TBasePassShaderElementData(const typename LightMapPolicyType::ElementDataType& InLightMapPolicyElementData) :
		LightMapPolicyElementData(InLightMapPolicyElementData)
	{}

	typename LightMapPolicyType::ElementDataType LightMapPolicyElementData;
};

/**
 * The base shader type for vertex shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 * The base type is shared between the versions with and without atmospheric fog.
 */

#define TBasePassMeshShaderPolicyParamType TBasePassVertexShaderPolicyParamType

template<typename LightMapPolicyType>
class TBasePassVertexShaderPolicyParamType : public FMeshMaterialShader, public LightMapPolicyType::VertexParametersType
{
	DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(TBasePassVertexShaderPolicyParamType, NonVirtual, FMeshMaterialShader, typename LightMapPolicyType::VertexParametersType);
protected:

	TBasePassVertexShaderPolicyParamType() {}
	TBasePassVertexShaderPolicyParamType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer):
		FMeshMaterialShader(Initializer)
	{
		LightMapPolicyType::VertexParametersType::Bind(Initializer.ParameterMap);
		ReflectionCaptureBuffer.Bind(Initializer.ParameterMap, TEXT("ReflectionCapture"));
	}

public:

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const TBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const;

	void GetElementShaderBindings(
		const FShaderMapPointerTable& PointerTable,
		const FScene* Scene,
		const FSceneView* ViewIfDynamicMeshCommand,
		const FVertexFactory* VertexFactory,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMeshBatch& MeshBatch,
		const FMeshBatchElement& BatchElement,
		const TBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const;

	LAYOUT_FIELD(FShaderUniformBufferParameter, ReflectionCaptureBuffer);
};

/**
 * The base shader type for vertex shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 * The base type is shared between the versions with and without atmospheric fog.
 */

template<typename LightMapPolicyType>
class TBasePassVertexShaderBaseType : public TBasePassVertexShaderPolicyParamType<LightMapPolicyType>
{
	typedef TBasePassVertexShaderPolicyParamType<LightMapPolicyType> Super;
	DECLARE_INLINE_TYPE_LAYOUT(TBasePassVertexShaderBaseType, NonVirtual);
protected:
	TBasePassVertexShaderBaseType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer) : Super(Initializer) {}
	TBasePassVertexShaderBaseType() {}

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return LightMapPolicyType::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		LightMapPolicyType::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		if (Parameters.VertexFactoryType && Parameters.VertexFactoryType->SupportsMeshShading())
		{
			OutEnvironment.SetCompileArgument(TEXT("USING_MESH_SHADING"), true);
		}

		if (HardwareVariableRateShadingSupportedByPlatform(Parameters.Platform) && Parameters.MaterialParameters.bAllowVariableRateShading)
		{
			OutEnvironment.SetCompileArgument(TEXT("USING_VARIABLE_RATE_SHADING"), true);
		}
	}
};

template<typename LightMapPolicyType>
class TBasePassMS : public TBasePassVertexShaderBaseType<LightMapPolicyType>
{
	DECLARE_SHADER_TYPE(TBasePassMS,MeshMaterial);
	typedef TBasePassVertexShaderBaseType<LightMapPolicyType> Super;

protected:

	TBasePassMS() {}
	TBasePassMS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer):
		Super(Initializer)
	{
	}

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		static const auto SupportAllShaderPermutations = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportAllShaderPermutations"));
		const bool bForceAllPermutations = SupportAllShaderPermutations && SupportAllShaderPermutations->GetValueOnAnyThread() != 0;
		const bool bShouldCache = Super::ShouldCompilePermutation(Parameters) || bForceAllPermutations;
		const bool bSupportMeshShading = Parameters.VertexFactoryType->SupportsMeshShading() && FDataDrivenShaderPlatformInfo::GetSupportsMeshShadersTier0(Parameters.Platform);
		return bShouldCache && bSupportMeshShading && (IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5));
	}

	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// @todo MetalMobile: Remove this hack and implement proper atmospheric-fog solution for Metal Mobile...
		OutEnvironment.SetDefine(TEXT("BASEPASS_SKYATMOSPHERE_AERIALPERSPECTIVE"), !(IsMetalMobileSM5Platform(Parameters.Platform)) ? 1 : 0);

		const uint32 MSThreadGroupSize = 32;
		OutEnvironment.SetDefine(TEXT("BASEPASS_MESH_SHADER_TG_SIZE"), MSThreadGroupSize);

		OutEnvironment.SetDefine(TEXT("BASEPASS_MESH_SHADER_BATCHING"), 1);

		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);

		// GBUFFER_LAYOUT is used to derive GBUFFER_HAS_VELOCITY, which influences whether VelocityPrevScreenPosition is computed by the vertex shader.
		// SLW can force velocity to be drawn in the water base pass even if the rest of the scene outputs in the depth prepass. SLW achieves this override
		// by using GBL_ForceVelocity, which requires setting the corresponding define on the vertex shader as well, as the calculation for VelocityPrevScreenPosition will be stripped otherwise.
		static const auto SLWForceVelocityCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Water.SingleLayer.ForceVelocity"));
		const bool bSLWForceVelocity = SLWForceVelocityCVar && SLWForceVelocityCVar->GetValueOnAnyThread() != 0;
		if (bSLWForceVelocity && Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater))
		{
			OutEnvironment.SetDefine(TEXT("GBUFFER_LAYOUT"), GBL_ForceVelocity);
		}
	}
};

template<typename LightMapPolicyType>
class TBasePassVS : public TBasePassVertexShaderBaseType<LightMapPolicyType>
{
	DECLARE_SHADER_TYPE(TBasePassVS,MeshMaterial);
	typedef TBasePassVertexShaderBaseType<LightMapPolicyType> Super;

protected:

	TBasePassVS() {}
	TBasePassVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer):
		Super(Initializer)
	{
	}

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		static const auto SupportAllShaderPermutations = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportAllShaderPermutations"));
		const bool bForceAllPermutations = SupportAllShaderPermutations && SupportAllShaderPermutations->GetValueOnAnyThread() != 0;
		bool bShouldCache = Super::ShouldCompilePermutation(Parameters) || bForceAllPermutations;
		return bShouldCache && (IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5));
	}

	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// @todo MetalMobile: Remove this hack and implement proper atmospheric-fog solution for Metal Mobile...
		OutEnvironment.SetDefine(TEXT("BASEPASS_SKYATMOSPHERE_AERIALPERSPECTIVE"), !(IsMetalMobileSM5Platform(Parameters.Platform)) ? 1 : 0);

		// GBUFFER_LAYOUT is used to derive GBUFFER_HAS_VELOCITY, which influences whether VelocityPrevScreenPosition is computed by the vertex shader.
		// When SLW velocity outputs to the depth prepass, the base pass uses GBL_Default (no velocity RT). When velocity outputs to the base pass,
		// GBL_ForceVelocity is used.
		static const auto SLWVelocityOutputPass = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Water.SingleLayer.VelocityOutputPass"));
		const int32 VelocityMode = SLWVelocityOutputPass ? SLWVelocityOutputPass->GetValueOnAnyThread() : 1;
		if (Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater))
		{
			// Must match GetSingleLayerWaterGBufferLayout: mode 0 = GBL_Default, mode 1 with prepass = GBL_Default,
			// mode 1 without prepass or forward renderer, or mode 2 = GBL_ForceVelocity
			// Forward+MSAA requires multisampled depth target while velocity RT stays as single sampled. Need to fallback to GBL_ForceVelocity. 
			const bool bDepthPrepass = IsSingleLayerWaterDepthPrepassEnabled(Parameters.Platform, GetMaxSupportedFeatureLevel(Parameters.Platform));
			const bool bForwardMSAA = IsForwardShadingEnabled(Parameters.Platform);
			const bool bUseForceVelocity = (VelocityMode == 2) || (VelocityMode == 1 && (!bDepthPrepass || bForwardMSAA));
			OutEnvironment.SetDefine(TEXT("GBUFFER_LAYOUT"), bUseForceVelocity ? GBL_ForceVelocity : GBL_Default);
		}
	}
};

BEGIN_UNIFORM_BUFFER_STRUCT(FComputeShadingOutputs, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTarget0)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTarget1)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTarget2)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTarget3)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTarget4)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTarget5)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTarget6)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTarget7)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutSGGX)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, OutTargets)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<SUBSTRATE_TOP_LAYER_TYPE>, OutTopLayerTarget)
END_UNIFORM_BUFFER_STRUCT()

/**
 * The base type for compute shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 * The base type is shared between the versions with and without sky light.
 */
template<typename LightMapPolicyType>
class TBasePassComputeShaderPolicyParamType : public FMeshMaterialShader, public LightMapPolicyType::ComputeParametersType
{
	DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(TBasePassComputeShaderPolicyParamType, NonVirtual, FMeshMaterialShader, typename LightMapPolicyType::ComputeParametersType);

public:
	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		if (ParameterMap.ContainsParameterAllocation(FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
		{
			OutError.Add(TEXT("Base pass shaders cannot read from the SceneTexturesStruct."));
			return false;
		}

		return true;
	}

	/** Initialization constructor. */
	TBasePassComputeShaderPolicyParamType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer):
		FMeshMaterialShader(Initializer)
	{
		LightMapPolicyType::ComputeParametersType::Bind(Initializer.ParameterMap);

		ReflectionCaptureBuffer.Bind(Initializer.ParameterMap, TEXT("ReflectionCapture"));
		ShadingOutputsParam.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs"));

		ViewRectParam.Bind(Initializer.ParameterMap, TEXT("ViewRect"));
		PassDataParam.Bind(Initializer.ParameterMap, TEXT("PassData"));

		Target0.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs.OutTarget0"));
		Target1.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs.OutTarget1"));
		Target2.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs.OutTarget2"));
		Target3.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs.OutTarget3"));
		Target4.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs.OutTarget4"));
		Target5.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs.OutTarget5"));
		Target6.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs.OutTarget6"));
		Target7.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs.OutTarget7"));

		Targets.Bind(Initializer.ParameterMap, TEXT("ComputeShadingOutputs.OutTargets"));

		// These parameters should only be used nested in the base pass uniform buffer
		check(!Initializer.ParameterMap.ContainsParameterAllocation(FFogUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()));
		check(!Initializer.ParameterMap.ContainsParameterAllocation(FReflectionUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()));
		check(!Initializer.ParameterMap.ContainsParameterAllocation(FPlanarReflectionUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()));
	}
	TBasePassComputeShaderPolicyParamType() {}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const TBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const;

	void SetPassParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FUintVector4& ViewRect,
		const FUintVector4& PassData,
		FRHIUniformBuffer* ShadingOutputs
	);

	uint32 GetBoundTargetMask() const;

private:
	LAYOUT_FIELD(FShaderUniformBufferParameter,       ReflectionCaptureBuffer);
	LAYOUT_FIELD(FShaderUniformBufferParameter,       ShadingOutputsParam);
	LAYOUT_FIELD(FShaderParameter,                    ViewRectParam);
	LAYOUT_FIELD(FShaderParameter,                    PassDataParam);
	LAYOUT_FIELD(FShaderUniformBufferMemberParameter, Target0);
	LAYOUT_FIELD(FShaderUniformBufferMemberParameter, Target1);
	LAYOUT_FIELD(FShaderUniformBufferMemberParameter, Target2);
	LAYOUT_FIELD(FShaderUniformBufferMemberParameter, Target3);
	LAYOUT_FIELD(FShaderUniformBufferMemberParameter, Target4);
	LAYOUT_FIELD(FShaderUniformBufferMemberParameter, Target5);
	LAYOUT_FIELD(FShaderUniformBufferMemberParameter, Target6);
	LAYOUT_FIELD(FShaderUniformBufferMemberParameter, Target7);
	LAYOUT_FIELD(FShaderUniformBufferMemberParameter, Targets);
};

/**
 * The base type for compute shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 * The base type is shared between the versions with and without sky light.
 */
template<typename LightMapPolicyType>
class TBasePassComputeShaderBaseType : public TBasePassComputeShaderPolicyParamType<LightMapPolicyType>
{
	typedef TBasePassComputeShaderPolicyParamType<LightMapPolicyType> Super;
	DECLARE_INLINE_TYPE_LAYOUT(TBasePassComputeShaderBaseType, NonVirtual);

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return LightMapPolicyType::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		LightMapPolicyType::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	/** Initialization constructor. */
	TBasePassComputeShaderBaseType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer) : Super(Initializer) {}

	TBasePassComputeShaderBaseType() {}
};

/** The concrete base pass compute shader type. */
template<typename LightMapPolicyType, bool bVoxel, bool bCurve, EShaderFrequency ShaderFrequency>
class TBasePassCS : public TBasePassComputeShaderBaseType<LightMapPolicyType>
{
	DECLARE_SHADER_TYPE(TBasePassCS,MeshMaterial);

	class FVisualizeDim : SHADER_PERMUTATION_BOOL("VISUALIZE");
	using FPermutationDomain = TShaderPermutationDomain<FVisualizeDim>;

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		// Debug visualization shaders are ODSC only.
		FPermutationDomain PermutationVector{ Parameters.PermutationId };
		if (PermutationVector.template Get<FVisualizeDim>() && !EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::IsODSCOnly))
		{
			return false;
		}

		const bool bVoxelTest = (!bVoxel || (NaniteVoxelsSupported() && Parameters.MaterialParameters.bIsUsedWithVoxels && Parameters.MaterialParameters.bIsUsedWithNanite));
		const bool bCurveTest = (!bCurve || (NaniteCurvesSupported() && Parameters.MaterialParameters.bIsUsedWithCurves && Parameters.MaterialParameters.bIsUsedWithNanite));

		return 
			(IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
			&& Parameters.VertexFactoryType->SupportsComputeShading()
			&& bVoxelTest
			&& DoesMaterialQualitySupportNanite(Parameters.Platform, Parameters.MaterialParameters.QualityLevel)
			&& bCurveTest
			&& TBasePassComputeShaderBaseType<LightMapPolicyType>::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		ModifyBasePassCSPSCompilationEnvironment(Parameters, GBL_ForceVelocity, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADED"), 1);

		if (ShaderFrequency == SF_WorkGraphComputeNode)
		{
			OutEnvironment.SetDefine(TEXT("WORKGRAPH_NODE"), 1);
		}

		if (bVoxel)
		{
			OutEnvironment.SetDefine(TEXT("NANITE_SHADING_PASS_FORCE_VOXELS"), 1);
		}
		else if (bCurve)
		{
			OutEnvironment.SetDefine(TEXT("NANITE_SHADING_PASS_FORCE_CURVES"), 1);
		}
		else
		{
			OutEnvironment.SetDefine(TEXT("NANITE_SHADING_PASS_FORCE_TRIANGLES"), 1);
		}

		TBasePassComputeShaderBaseType<LightMapPolicyType>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
	
	/** Initialization constructor. */
	TBasePassCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		TBasePassComputeShaderBaseType<LightMapPolicyType>(Initializer)
	{}

	/** Default constructor. */
	TBasePassCS() {}
};

/**
 * The base type for pixel shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 * The base type is shared between the versions with and without sky light.
 */
template<typename LightMapPolicyType>
class TBasePassPixelShaderPolicyParamType : public FMeshMaterialShader, public LightMapPolicyType::PixelParametersType
{
	DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(TBasePassPixelShaderPolicyParamType, NonVirtual, FMeshMaterialShader, typename LightMapPolicyType::PixelParametersType);
public:

	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);

		if (Parameters.VertexFactoryType && Parameters.VertexFactoryType->SupportsMeshShading())
		{
			OutEnvironment.SetCompileArgument(TEXT("USING_MESH_SHADING"), true);
		}

		if (HardwareVariableRateShadingSupportedByPlatform(Parameters.Platform) && Parameters.MaterialParameters.bAllowVariableRateShading)
		{
			OutEnvironment.SetCompileArgument(TEXT("USING_VARIABLE_RATE_SHADING"), true);
		}
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		if (ParameterMap.ContainsParameterAllocation(FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
		{
			OutError.Add(TEXT("Base pass shaders cannot read from the SceneTexturesStruct."));
			return false;
		}

		return true;
	}

	/** Initialization constructor. */
	TBasePassPixelShaderPolicyParamType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer):
		FMeshMaterialShader(Initializer)
	{
		LightMapPolicyType::PixelParametersType::Bind(Initializer.ParameterMap);
		ReflectionCaptureBuffer.Bind(Initializer.ParameterMap, TEXT("ReflectionCapture"));

		// These parameters should only be used nested in the base pass uniform buffer
		check(!Initializer.ParameterMap.ContainsParameterAllocation(FFogUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()));
		check(!Initializer.ParameterMap.ContainsParameterAllocation(FReflectionUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()));
		check(!Initializer.ParameterMap.ContainsParameterAllocation(FPlanarReflectionUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()));
	}
	TBasePassPixelShaderPolicyParamType() {}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const TBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const;

private:
	LAYOUT_FIELD(FShaderUniformBufferParameter, ReflectionCaptureBuffer);
};

/**
 * The base type for pixel shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 * The base type is shared between the versions with and without sky light.
 */
template<typename LightMapPolicyType>
class TBasePassPixelShaderBaseType : public TBasePassPixelShaderPolicyParamType<LightMapPolicyType>
{
	typedef TBasePassPixelShaderPolicyParamType<LightMapPolicyType> Super;
	DECLARE_INLINE_TYPE_LAYOUT(TBasePassPixelShaderBaseType, NonVirtual);
public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return LightMapPolicyType::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		LightMapPolicyType::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	/** Initialization constructor. */
	TBasePassPixelShaderBaseType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer) : Super(Initializer) {}

	TBasePassPixelShaderBaseType() {}
};

/** The concrete base pass pixel shader type. */
template<typename LightMapPolicyType, EGBufferLayout GBufferLayout>
class TBasePassPS : public TBasePassPixelShaderBaseType<LightMapPolicyType>
{
	DECLARE_SHADER_TYPE(TBasePassPS,MeshMaterial);

	class FVisualizeDim : SHADER_PERMUTATION_BOOL("VISUALIZE");
	class FSupportOITDim : SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORTS_OIT");
	using FPermutationDomain = TShaderPermutationDomain<FVisualizeDim, FSupportOITDim>;

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector{ Parameters.PermutationId };

		// Debug visualization shaders are ODSC only.
		if (PermutationVector.template Get<FVisualizeDim>() && !EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::IsODSCOnly))
		{
			return false;
		}

		// Only compiled OIT permutation for translucent surface, and if OIT sorted pixel is enabled for the current project
		const bool bTranslucent = IsTranslucentBlendMode(Parameters.MaterialParameters);
		if (PermutationVector.template Get<FSupportOITDim>())
		{
			if (!bTranslucent || !OIT::IsSortedPixelsEnabledForProject(Parameters.Platform))
			{
				return false;
			}
		}

		return
			IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			&& TBasePassPixelShaderBaseType<LightMapPolicyType>::ShouldCompilePermutation(Parameters)
			&& IsGBufferLayoutSupportedForMaterial(GBufferLayout, Parameters);
	}

	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		ModifyBasePassCSPSCompilationEnvironment(Parameters, GBufferLayout, OutEnvironment);

		const bool bIsSingleLayerWater = Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);
		if (bIsSingleLayerWater)
		{
			const bool bHasDepthPrepass = IsSingleLayerWaterDepthPrepassEnabled(Parameters.Platform, GetMaxSupportedFeatureLevel(Parameters.Platform));
			if (bHasDepthPrepass)
			{
				OutEnvironment.SetDefine(TEXT("SINGLE_LAYER_WATER_NO_DISCARD"), TEXT("1"));
			}
		}

		const bool bIsTranslucent = IsTranslucentBlendMode(Parameters.MaterialParameters);
		if (bIsTranslucent && DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform) && ShouldCompositeHeterogeneousVolumesWithTranslucency())
		{
			OutEnvironment.SetDefine(TEXT("ADAPTIVE_VOLUMETRIC_SHADOW_MAP"), TEXT("1"));
		}

		TBasePassPixelShaderBaseType<LightMapPolicyType>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
	
	/** Initialization constructor. */
	TBasePassPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		TBasePassPixelShaderBaseType<LightMapPolicyType>(Initializer)
	{}

	/** Default constructor. */
	TBasePassPS() {}
};

//Alternative base pass PS for 128 bit canvas render targets that need to be set at shader compilation time.
class F128BitRTBasePassPS : public TBasePassPS<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>, GBL_Default>
{
	DECLARE_SHADER_TYPE(F128BitRTBasePassPS, MeshMaterial);
public:
	F128BitRTBasePassPS();
	F128BitRTBasePassPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/**
 * Get shader templates allowing to redirect between compatible shaders.
 */

template <typename LightMapPolicyType>
void AddBasePassComputeShader(bool bVoxel, bool bCurve, bool bIsDebug, EShaderFrequency ShaderFrequency, FMaterialShaderTypes& OutShaderTypes)
{
	int32 PermutationId = 0;
	if (bIsDebug)
	{
		using FMyShader = TBasePassCS<LightMapPolicyType, true, false, SF_Compute>;
		typename FMyShader::FPermutationDomain PermutationVector;
		PermutationVector.template Set<typename FMyShader::FVisualizeDim>(bIsDebug);
		PermutationId = PermutationVector.ToDimensionValueId();
	}

	if (ShaderFrequency == SF_Compute)
	{
		if (bCurve)
		{
			OutShaderTypes.AddShaderType<TBasePassCS<LightMapPolicyType, false, true, SF_Compute>>(PermutationId);
		}
		else if (bVoxel)
		{
			OutShaderTypes.AddShaderType<TBasePassCS<LightMapPolicyType, true, false, SF_Compute>>(PermutationId);
		}
		else
		{
			OutShaderTypes.AddShaderType<TBasePassCS<LightMapPolicyType, false, false, SF_Compute>>(PermutationId);
		}
		
	}
	else if (ShaderFrequency == SF_WorkGraphComputeNode)
	{
		if (bCurve)
		{
			OutShaderTypes.AddShaderType<TBasePassCS<LightMapPolicyType, false, true, SF_WorkGraphComputeNode>>(PermutationId);
		}
		else if (bVoxel)
		{
			OutShaderTypes.AddShaderType<TBasePassCS<LightMapPolicyType, true, false, SF_WorkGraphComputeNode>>(PermutationId);
		}
		else
		{
			OutShaderTypes.AddShaderType<TBasePassCS<LightMapPolicyType, false, false, SF_WorkGraphComputeNode>>(PermutationId);
		}
	}
}

template <typename LightMapPolicyType>
bool GetBasePassShader(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	LightMapPolicyType LightMapPolicy,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bVoxel,
	bool bCurve,
	bool bIsDebug,
	EShaderFrequency ShaderFrequency,
	TShaderRef<TBasePassComputeShaderPolicyParamType<LightMapPolicyType>>* ComputeShader
)
{
	FMaterialShaderTypes ShaderTypes;

	if (ComputeShader)
	{
		AddBasePassComputeShader<LightMapPolicyType>(bVoxel, bCurve, bIsDebug, ShaderFrequency, ShaderTypes);
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetShader(ShaderFrequency, ComputeShader);
	return true;
}

template <>
bool GetBasePassShader<FUniformLightMapPolicy>(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	FUniformLightMapPolicy LightMapPolicy,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bVoxel,
	bool bCurve,
	bool bIsDebug,
	EShaderFrequency ShaderFrequency,
	TShaderRef<TBasePassComputeShaderPolicyParamType<FUniformLightMapPolicy>>* ComputeShader
);

/**
 * Get shader templates allowing to redirect between compatible shaders.
 */

template <typename LightMapPolicyType, EGBufferLayout GBufferLayout>
void AddBasePassPixelShader(bool bIsDebug, FMaterialShaderTypes& OutShaderTypes, bool bIsForOITPass = false)
{
	int32 PermutationId = 0;
	if (bIsDebug || bIsForOITPass)
	{
		using FMyShader = TBasePassPS<LightMapPolicyType, GBufferLayout>;
		typename FMyShader::FPermutationDomain PermutationVector;
		PermutationVector.template Set<typename FMyShader::FVisualizeDim>(bIsDebug);
		PermutationVector.template Set<typename FMyShader::FSupportOITDim>(bIsForOITPass);
		PermutationId = PermutationVector.ToDimensionValueId();
	}

	OutShaderTypes.AddShaderType<TBasePassPS<LightMapPolicyType, GBufferLayout>>(PermutationId);
}

template <typename LightMapPolicyType>
bool GetBasePassShaders(
	const FMaterial& Material, 
	const FVertexFactoryType* VertexFactoryType, 
	LightMapPolicyType LightMapPolicy, 
	ERHIFeatureLevel::Type FeatureLevel,
	bool bUse128bitRT,
	bool bIsDebug,
	EGBufferLayout GBufferLayout,
	TShaderRef<TBasePassMeshShaderPolicyParamType<LightMapPolicyType>>* MeshShader,
	TShaderRef<TBasePassVertexShaderPolicyParamType<LightMapPolicyType>>* VertexShader,
	TShaderRef<TBasePassPixelShaderPolicyParamType<LightMapPolicyType>>* PixelShader,
	bool bIsForOITPass = false
)
{
	FMaterialShaderTypes ShaderTypes;

	if (VertexShader)
	{
		ShaderTypes.AddShaderType<TBasePassVS<LightMapPolicyType>>();
	}

	if (MeshShader)
	{
		ShaderTypes.AddShaderType<TBasePassMS<LightMapPolicyType>>();
	}

	if (PixelShader)
	{
		switch (GBufferLayout)
		{
		case GBL_Default:
			AddBasePassPixelShader<LightMapPolicyType, GBL_Default>(bIsDebug, ShaderTypes, bIsForOITPass);
			break;
		case GBL_ForceVelocity:
			AddBasePassPixelShader<LightMapPolicyType, GBL_ForceVelocity>(bIsDebug, ShaderTypes, bIsForOITPass);
			break;
		default:
			check(false);
			break;
		}
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetMeshShader(MeshShader);
	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

template <>
bool GetBasePassShaders<FUniformLightMapPolicy>(
	const FMaterial& Material, 
	const FVertexFactoryType* VertexFactoryType, 
	FUniformLightMapPolicy LightMapPolicy, 
	ERHIFeatureLevel::Type FeatureLevel,
	bool bUse128bitRT,
	bool bIsDebug,
	EGBufferLayout GBufferLayout,
	TShaderRef<TBasePassMeshShaderPolicyParamType<FUniformLightMapPolicy>>* MeshShader,
	TShaderRef<TBasePassVertexShaderPolicyParamType<FUniformLightMapPolicy>>* VertexShader,
	TShaderRef<TBasePassPixelShaderPolicyParamType<FUniformLightMapPolicy>>* PixelShader,
	bool bIsForOITPass
	);

class FBasePassMeshProcessor : public FSceneRenderingAllocatorObject<FBasePassMeshProcessor>, public FMeshPassProcessor
{
public:
	enum class EFlags
	{
		None = 0,

		// Informs the processor whether a depth-stencil target is bound when processed draw commands are issued.
		CanUseDepthStencil = (1 << 0),
		bRequires128bitRT = (1 << 1)
	};

	FBasePassMeshProcessor(
		EMeshPass::Type InMeshPassType,
		const FScene* InScene,
		ERHIFeatureLevel::Type InFeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand,
		const FMeshPassProcessorRenderState& InDrawRenderState,
		FMeshPassDrawListContext* InDrawListContext,
		EFlags Flags,
		ETranslucencyPass::Type InTranslucencyPassType = ETranslucencyPass::TPT_MAX);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

	FMeshPassProcessorRenderState PassDrawRenderState;

	FORCEINLINE_DEBUGGABLE void Set128BitRequirement(const bool Required)
	{
		bRequiresExplicit128bitRT = Required;
	}

	FORCEINLINE_DEBUGGABLE bool Get128BitRequirement() const
	{
		return bRequiresExplicit128bitRT;
	}

	static ELightMapPolicyType GetUniformLightMapPolicyType(ERHIFeatureLevel::Type FeatureLevelconst, const FScene* Scene, const FLightCacheInterface* LCI, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, const FMaterial& Material);
	static TArray<ELightMapPolicyType, TInlineAllocator<2>> GetUniformLightMapPolicyTypeForPSOCollection(ERHIFeatureLevel::Type FeatureLevel, const FMaterial& Material);

	template<typename PassShadersType>
	static void AddBasePassGraphicsPipelineStateInitializer(
		ERHIFeatureLevel::Type InFeatureLevel,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		const FMeshPassProcessorRenderState& RESTRICT DrawRenderState,
		const FGraphicsPipelineRenderTargetsInfo& RESTRICT RenderTargetsInfo,
		const PassShadersType& PassShaders,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		EPrimitiveType PrimitiveType,
		bool bPrecacheAlphaColorChannel,
		int InPSOCollectorIndex,
		TArray<FPSOPrecacheData>& PSOInitializers)
	{
		AddGraphicsPipelineStateInitializer(
			VertexFactoryData,
			MaterialResource,
			DrawRenderState,
			RenderTargetsInfo,
			PassShaders,
			MeshFillMode,
			MeshCullMode,
			PrimitiveType,
			EMeshPassFeatures::Default,
			ESubpassHint::None,
			0,
			true /*bRequired*/,
			InPSOCollectorIndex,
			PSOInitializers);

		// Planar reflections and scene captures use scene color alpha to keep track of where content has been rendered, for compositing into a different scene later
		if (bPrecacheAlphaColorChannel && UPSOPrecacheSettingsManager::GetSettings().bBasePassColorWithAlphaChannel)
		{
			FGraphicsPipelineRenderTargetsInfo AlphaColorRenderTargetsInfo = RenderTargetsInfo;

			bool bRequiresAlphaChannel = true;
			ETextureCreateFlags ExtraSceneColorCreateFlags = ETextureCreateFlags::None;
			EPixelFormat SceneColorFormatWithAlpha;
			ETextureCreateFlags SceneColorCreateFlagsWithAlpha;
			GetSceneColorFormatAndCreateFlags(InFeatureLevel, bRequiresAlphaChannel, ExtraSceneColorCreateFlags, RenderTargetsInfo.NumSamples, false, SceneColorFormatWithAlpha, SceneColorCreateFlagsWithAlpha);

			AlphaColorRenderTargetsInfo.RenderTargetFormats[0] = SceneColorFormatWithAlpha;
			AlphaColorRenderTargetsInfo.RenderTargetFlags[0] = SceneColorCreateFlagsWithAlpha;

			AddGraphicsPipelineStateInitializer(
				VertexFactoryData,
				MaterialResource,
				DrawRenderState,
				AlphaColorRenderTargetsInfo,
				PassShaders,
				MeshFillMode,
				MeshCullMode,
				PrimitiveType,
				EMeshPassFeatures::Default,
				ESubpassHint::None,
				0,
				true,
				InPSOCollectorIndex,
				PSOInitializers);
		}
	}

private:

	bool TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material);
	bool ShouldDraw(const FMaterial& Material);
	
	template<typename LightMapPolicyType>
	bool Process(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		const bool bIsMasked,
		const bool bIsTranslucent,
		FMaterialShadingModelField ShadingModels,
		const LightMapPolicyType& RESTRICT LightMapPolicy,
		const typename LightMapPolicyType::ElementDataType& RESTRICT LightMapElementData,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	void CollectPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		const FMaterial& RESTRICT MaterialResource,
		const bool bDitheredLODTransition,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		EPrimitiveType PrimitiveType, 
		TArray<FPSOPrecacheData>& PSOInitializers);

	template<typename LightMapPolicyType>
	void CollectPSOInitializersForLMPolicy(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		const FMaterial& RESTRICT MaterialResource,
		FMaterialShadingModelField ShadingModels,
		const bool bDitheredLODTransition,
		const LightMapPolicyType& RESTRICT LightMapPolicy,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode, 
		EPrimitiveType PrimitiveType, 
		TArray<FPSOPrecacheData>& PSOInitializers);

	const bool bIsDebug;
	const ETranslucencyPass::Type TranslucencyPassType;
	const bool bTranslucentBasePass;
	const bool bOITBasePass;
	const bool bEnableReceiveDecalOutput;
	EDepthDrawingMode EarlyZPassMode;
	bool bRequiresExplicit128bitRT;
	float AutoBeforeDOFTranslucencyBoundary = 0.0f;
};

ENUM_CLASS_FLAGS(FBasePassMeshProcessor::EFlags);

extern void SetupBasePassState(FExclusiveDepthStencil::Type BasePassDepthStencilAccess, const bool bShaderComplexity, FMeshPassProcessorRenderState& DrawRenderState);
extern FMeshDrawCommandSortKey CalculateTranslucentMeshStaticSortKey(const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, uint16 MeshIdInPrimitive);

struct FNaniteBasePassData
{
	TShaderRef<TBasePassComputeShaderPolicyParamType<FUniformLightMapPolicy>> TypedShader;
};
