// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BasePassRendering.inl: Base pass rendering implementations.
		(Due to forward declaration issues)
=============================================================================*/

#pragma once

#include "CoreFwd.h"
#include "BasePassRendering.h"

class FMeshMaterialShader;
class FPrimitiveSceneProxy;
class FRHICommandList;
class FSceneView;
class FVertexFactory;
class FViewInfo;
struct FMeshBatch;
struct FMeshBatchElement;
struct FMeshDrawingRenderState;

template<typename LightMapPolicyType>
void TBasePassVertexShaderPolicyParamType<LightMapPolicyType>::GetShaderBindings(
	const FScene* Scene,
	ERHIFeatureLevel::Type FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	const TBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings) const
{
	FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

	LightMapPolicyType::GetVertexShaderBindings(
		PrimitiveSceneProxy,
		ShaderElementData.LightMapPolicyElementData,
		this,
		ShaderBindings);
}

template<typename LightMapPolicyType>
void TBasePassVertexShaderPolicyParamType<LightMapPolicyType>::GetElementShaderBindings(
	const FShaderMapPointerTable& PointerTable,
	const FScene* Scene, 
	const FSceneView* ViewIfDynamicMeshCommand, 
	const FVertexFactory* VertexFactory,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMeshBatch& MeshBatch,
	const FMeshBatchElement& BatchElement, 
	const TBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams) const
{
	FMeshMaterialShader::GetElementShaderBindings(PointerTable, Scene, ViewIfDynamicMeshCommand, VertexFactory, InputStreamType, FeatureLevel, PrimitiveSceneProxy, MeshBatch, BatchElement, ShaderElementData, ShaderBindings, VertexStreams);
}

template<typename LightMapPolicyType>
void TBasePassPixelShaderPolicyParamType<LightMapPolicyType>::GetShaderBindings(
	const FScene* Scene,
	ERHIFeatureLevel::Type FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	const TBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings) const
{
	FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

	LightMapPolicyType::GetPixelShaderBindings(
		PrimitiveSceneProxy,
		ShaderElementData.LightMapPolicyElementData,
		this,
		ShaderBindings);
}

template<typename LightMapPolicyType>
void TBasePassComputeShaderPolicyParamType<LightMapPolicyType>::GetShaderBindings(
	const FScene* Scene,
	ERHIFeatureLevel::Type FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	const TBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings) const
{
	FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

	LightMapPolicyType::GetComputeShaderBindings(
		PrimitiveSceneProxy,
		ShaderElementData.LightMapPolicyElementData,
		this,
		ShaderBindings);
}

template<typename LightMapPolicyType>
void TBasePassComputeShaderPolicyParamType<LightMapPolicyType>::SetPassParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FUintVector4& ViewRect,
	const FUintVector4& PassData,
	FRHIUniformBuffer* ShadingOutputs
)
{
	SetUniformBufferParameter(BatchedParameters, ShadingOutputsParam, ShadingOutputs);
	SetShaderValue(BatchedParameters, ViewRectParam, ViewRect);
	SetShaderValue(BatchedParameters, PassDataParam, PassData);
}

template<typename LightMapPolicyType>
uint32 TBasePassComputeShaderPolicyParamType<LightMapPolicyType>::GetBoundTargetMask() const
{
	uint32 TargetMask = 0u;
	TargetMask |= Target0.IsBound() ? (1u << 0u) : 0u;
	TargetMask |= Target1.IsBound() ? (1u << 1u) : 0u;
	TargetMask |= Target2.IsBound() ? (1u << 2u) : 0u;
	TargetMask |= Target3.IsBound() ? (1u << 3u) : 0u;
	TargetMask |= Target4.IsBound() ? (1u << 4u) : 0u;
	TargetMask |= Target5.IsBound() ? (1u << 5u) : 0u;
	TargetMask |= Target6.IsBound() ? (1u << 6u) : 0u;
	TargetMask |= Target7.IsBound() ? (1u << 7u) : 0u;
	TargetMask |= Targets.IsBound() ? (1u << 8u) : 0u;
	return TargetMask;
}
