// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BlueNoise.cpp: Resources for Blue-Noise vectors on the GPU.
=============================================================================*/

#include "BlueNoise.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "TextureResource.h"
#include "SystemTextures.h"
#include "GlobalRenderResources.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FBlueNoise, "BlueNoise");

FBlueNoiseParameters GetBlueNoiseDummyParameters()
{
	FBlueNoiseParameters Out;
	Out.Dimensions = FIntVector(1,1,1);
	Out.ModuloMasks = FIntVector(1,1,1);
	Out.ScalarTexture = GSystemTextures.BlackDummy->GetRHI();
	Out.Vec2Texture = GSystemTextures.BlackDummy->GetRHI();
	return Out;
}

static void FillUpBlueNoiseParametersFromTexture(FBlueNoiseParameters& Out)
{
	const FIntVector BlueNoiseSize = Out.ScalarTexture->GetSizeXYZ();

	Out.Dimensions = FIntVector(
		BlueNoiseSize.X,
		BlueNoiseSize.X,
		BlueNoiseSize.Y / FMath::Max<int32>(1, BlueNoiseSize.X));

	Out.ModuloMasks = FIntVector(
		(1u << FMath::FloorLog2(Out.Dimensions.X)) - 1,
		(1u << FMath::FloorLog2(Out.Dimensions.Y)) - 1,
		(1u << FMath::FloorLog2(Out.Dimensions.Z)) - 1);

	check((Out.ModuloMasks.X + 1) == Out.Dimensions.X
		&& (Out.ModuloMasks.Y + 1) == Out.Dimensions.Y
		&& (Out.ModuloMasks.Z + 1) == Out.Dimensions.Z);
}

FBlueNoiseParameters GetBlueNoiseParameters()
{
	FBlueNoiseParameters Out;
	check(GEngine);

	check(GEngine->BlueNoiseScalarTexture && GEngine->BlueNoiseScalarTexture->GetResource());
	Out.ScalarTexture = GEngine->BlueNoiseScalarTexture->GetResource()->TextureRHI;

	if (GEngine->BlueNoiseVec2Texture != nullptr && GEngine->BlueNoiseVec2Texture->GetResource() != nullptr)
	{
		Out.Vec2Texture = GEngine->BlueNoiseVec2Texture->GetResource()->TextureRHI;
	}
	else
	{
		Out.Vec2Texture = GSystemTextures.BlackDummy->GetRHI();
	}

	FillUpBlueNoiseParametersFromTexture(Out);
	return Out;
}

FBlueNoise GetBlueNoiseGlobalParameters()
{
	FBlueNoise Out;
	if (GEngine->BlueNoiseScalarTexture != nullptr && GEngine->BlueNoiseScalarTexture->GetResource() != nullptr)
	{
		Out.BlueNoise = GetBlueNoiseParameters();
	}
	else
	{
		ensure(IsRunningCommandlet() && IsAllowCommandletRendering());	//	If running a commandlet, the load path won't be visited so the blue noise textures not present. Allow the fallback only in that case.
		Out.BlueNoise = GetBlueNoiseDummyParameters();
	}
	return Out;
}

FBlueNoiseParameters GetBlueNoiseParametersForView()
{
	FBlueNoiseParameters Out;
	check(GEngine);

	if (GEngine->BlueNoiseScalarTexture != nullptr && GEngine->BlueNoiseScalarTexture->GetResource() != nullptr)
	{
		Out.ScalarTexture = GEngine->BlueNoiseScalarTexture->GetResource()->TextureRHI;
	}
	else
	{
		Out.ScalarTexture = GBlackVolumeTexture->TextureRHI.GetReference();
	}
	Out.Vec2Texture = GBlackVolumeTexture->TextureRHI.GetReference();

	FillUpBlueNoiseParametersFromTexture(Out);
	return Out;
}