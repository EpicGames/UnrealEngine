// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	BlueNoise.h: Resources for Blue-Noise vectors on the GPU.
=============================================================================*/

#pragma once

#include "Math/IntVector.h"
#include "ShaderParameterMacros.h"

BEGIN_SHADER_PARAMETER_STRUCT(FBlueNoiseParameters, RENDERER_API)
	SHADER_PARAMETER(FIntVector, Dimensions)
	SHADER_PARAMETER(FIntVector, ModuloMasks)
	SHADER_PARAMETER_TEXTURE(Texture2D<float>, ScalarTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D<float2>, Vec2Texture)
END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FBlueNoise, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FBlueNoiseParameters, BlueNoise)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

// These GetBlueNoiseXXX functions should only be called from a rendering system, when this system make it possible 
// to load the blue noise textures when calling GEngine->LoadBlueNoiseTexture from SceneRendering.cpp.
extern RENDERER_API FBlueNoise GetBlueNoiseGlobalParameters();
extern RENDERER_API FBlueNoiseParameters GetBlueNoiseParameters();
extern RENDERER_API FBlueNoiseParameters GetBlueNoiseDummyParameters();

// This function is for filling up the View blue noise parameters used for materials.
// Sometimes views are created for rendering without GSystemTextures initialised, e.g. HLOD baking, canvas DrawTile.
// So we have to rely on global default GPU resources in this case.
extern RENDERER_API FBlueNoiseParameters GetBlueNoiseParametersForView();
