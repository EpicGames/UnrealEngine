// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CapsuleShadowRendering.cpp: Functionality for rendering shadows from capsules
=============================================================================*/

#include "CapsuleShadowRendering.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "PostProcess/SceneRenderTargets.h"
#include "GlobalShader.h"
#include "SceneRenderTargetParameters.h"
#include "ShadowRendering.h"
#include "DeferredShadingRenderer.h"
#include "MaterialShaderType.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "DistanceFieldLightingPost.h"
#include "DistanceFieldLightingShared.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "RendererPrivateUtils.h"
#include "Substrate/Substrate.h"

DECLARE_GPU_STAT_NAMED(CapsuleShadows, TEXT("Capsule Shadows"));

int32 GCapsuleShadows = 1;
FAutoConsoleVariableRef CVarCapsuleShadows(
	TEXT("r.CapsuleShadows"),
	GCapsuleShadows,
	TEXT("Whether to allow capsule shadowing on skinned components with bCastCapsuleDirectShadow or bCastCapsuleIndirectShadow enabled."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GCapsuleDirectShadows = 1;
FAutoConsoleVariableRef CVarCapsuleDirectShadows(
	TEXT("r.CapsuleDirectShadows"),
	GCapsuleDirectShadows,
	TEXT("Whether to allow capsule direct shadowing on skinned components with bCastCapsuleDirectShadow enabled."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GCapsuleIndirectShadows = 1;
FAutoConsoleVariableRef CVarCapsuleIndirectShadows(
	TEXT("r.CapsuleIndirectShadows"),
	GCapsuleIndirectShadows,
	TEXT("Whether to allow capsule indirect shadowing on skinned components with bCastCapsuleIndirectShadow enabled."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GCapsuleShadowsFullResolution = 0;
FAutoConsoleVariableRef CVarCapsuleShadowsFullResolution(
	TEXT("r.CapsuleShadowsFullResolution"),
	GCapsuleShadowsFullResolution,
	TEXT("Whether to compute capsule shadows at full resolution."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GCapsuleMaxDirectOcclusionDistance = 400;
FAutoConsoleVariableRef CVarCapsuleMaxDirectOcclusionDistance(
	TEXT("r.CapsuleMaxDirectOcclusionDistance"),
	GCapsuleMaxDirectOcclusionDistance,
	TEXT("Maximum cast distance for direct shadows from capsules.  This has a big impact on performance."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GCapsuleMaxIndirectOcclusionDistance = 200;
FAutoConsoleVariableRef CVarCapsuleMaxIndirectOcclusionDistance(
	TEXT("r.CapsuleMaxIndirectOcclusionDistance"),
	GCapsuleMaxIndirectOcclusionDistance,
	TEXT("Maximum cast distance for indirect shadows from capsules.  This has a big impact on performance."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GCapsuleShadowFadeAngleFromVertical = PI / 3;
FAutoConsoleVariableRef CVarCapsuleShadowFadeAngleFromVertical(
	TEXT("r.CapsuleShadowFadeAngleFromVertical"),
	GCapsuleShadowFadeAngleFromVertical,
	TEXT("Angle from vertical up to start fading out the indirect shadow, to avoid self shadowing artifacts."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GCapsuleIndirectConeAngle = PI / 8;
FAutoConsoleVariableRef CVarCapsuleIndirectConeAngle(
	TEXT("r.CapsuleIndirectConeAngle"),
	GCapsuleIndirectConeAngle,
	TEXT("Light source angle used when the indirect shadow direction is derived from precomputed indirect lighting (no stationary skylight present)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GCapsuleSkyAngleScale = .6f;
FAutoConsoleVariableRef CVarCapsuleSkyAngleScale(
	TEXT("r.CapsuleSkyAngleScale"),
	GCapsuleSkyAngleScale,
	TEXT("Scales the light source angle derived from the precomputed unoccluded sky vector (stationary skylight present)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GCapsuleMinSkyAngle = 15;
FAutoConsoleVariableRef CVarCapsuleMinSkyAngle(
	TEXT("r.CapsuleMinSkyAngle"),
	GCapsuleMinSkyAngle,
	TEXT("Minimum light source angle derived from the precomputed unoccluded sky vector (stationary skylight present)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

const int32 GComputeLightDirectionFromVolumetricLightmapGroupSize = 64;

class FComputeLightDirectionFromVolumetricLightmapCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FComputeLightDirectionFromVolumetricLightmapCS);
	SHADER_USE_PARAMETER_STRUCT(FComputeLightDirectionFromVolumetricLightmapCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER(uint32, NumLightDirectionData)
		SHADER_PARAMETER(uint32, SkyLightMode)
		SHADER_PARAMETER(float, CapsuleIndirectConeAngle)
		SHADER_PARAMETER(float, CapsuleSkyAngleScale)
		SHADER_PARAMETER(float, CapsuleMinSkyAngle)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, LightDirectionData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, RWComputedLightDirectionData)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return !IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GComputeLightDirectionFromVolumetricLightmapGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), 1);
		OutEnvironment.SetDefine(TEXT("LIGHT_SOURCE_MODE"), TEXT("LIGHT_SOURCE_FROM_CAPSULE"));
	}
};

IMPLEMENT_GLOBAL_SHADER(FComputeLightDirectionFromVolumetricLightmapCS, "/Engine/Private/CapsuleShadowShaders.usf", "ComputeLightDirectionFromVolumetricLightmapCS", SF_Compute);

int32 GShadowShapeTileSize = 8;

int32 GetCapsuleShadowDownsampleFactor()
{
	return GCapsuleShadowsFullResolution ? 1 : 2;
}

FIntPoint GetBufferSizeForCapsuleShadows(const FViewInfo& View)
{
	return FIntPoint::DivideAndRoundDown(View.GetSceneTexturesConfig().Extent, GetCapsuleShadowDownsampleFactor());
}

enum class ECapsuleShadowingType
{
	DirectionalLightTiledCulling,
	PointLightTiledCulling,
	IndirectTiledCulling,
	MovableSkylightTiledCulling,
	MovableSkylightTiledCullingGatherFromReceiverBentNormal,

	MAX
};

enum class EIndirectShadowingPrimitiveTypes
{
	CapsuleShapes,
	MeshDistanceFields,
	CapsuleShapesAndMeshDistanceFields,

	MAX
};

enum class ELightSourceMode
{
	// must match CapsuleShadowShaders.usf
	Punctual = 0,
	FromCapsule = 1,
	FromReceiver = 2,
};

class FCapsuleShadowingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCapsuleShadowingCS);
public:
	SHADER_USE_PARAMETER_STRUCT(FCapsuleShadowingCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)

		SHADER_PARAMETER(uint32, EyeIndex)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWShadowFactors)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWBentNormalTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ReceiverBentNormalTexture)
		SHADER_PARAMETER(FIntPoint, TileDimensions)
		SHADER_PARAMETER(FVector2f, NumGroups)

		SHADER_PARAMETER(FVector4f, LightTranslatedPositionAndInvRadius)
		SHADER_PARAMETER(FVector3f, LightDirection)
		SHADER_PARAMETER(float, LightSourceRadius)
		SHADER_PARAMETER(FVector3f, LightAngleAndNormalThreshold)
		SHADER_PARAMETER(float, RayStartOffsetDepthScale)

		SHADER_PARAMETER(FIntRect, ScissorRectMinAndSize)

		SHADER_PARAMETER(float, IndirectCapsuleSelfShadowingIntensity)
		SHADER_PARAMETER(uint32, DownsampleFactor)
		SHADER_PARAMETER(float, MaxOcclusionDistance)

		SHADER_PARAMETER(FVector2f, CosFadeStartAngle)

		SHADER_PARAMETER(uint32, NumShadowCapsules)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FCapsuleShape3f>, ShadowCapsuleShapes)

		SHADER_PARAMETER(uint32, NumMeshDistanceFieldCasters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, MeshDistanceFieldCasterIndices)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, LightDirectionData)

		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DFObjectBufferParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DFAtlasParameters)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWTileIntersectionCounts)
	END_SHADER_PARAMETER_STRUCT()

	class FShapeShadow : SHADER_PERMUTATION_ENUM_CLASS("SHAPE_SHADOW", ECapsuleShadowingType);
	class FIndirectPrimitiveType : SHADER_PERMUTATION_ENUM_CLASS("INDIRECT_PRIMITIVE_TYPE", EIndirectShadowingPrimitiveTypes);
	using FPermutationDomain = TShaderPermutationDomain<FShapeShadow, FIndirectPrimitiveType>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GShadowShapeTileSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GShadowShapeTileSize);

		ECapsuleShadowingType ShadowingType = PermutationVector.Get<FShapeShadow>();

		OutEnvironment.SetDefine(TEXT("POINT_LIGHT"), ShadowingType == ECapsuleShadowingType::PointLightTiledCulling);

		ELightSourceMode LightSourceMode = (ELightSourceMode)0;

		if (ShadowingType == ECapsuleShadowingType::DirectionalLightTiledCulling || ShadowingType == ECapsuleShadowingType::PointLightTiledCulling)
		{
			LightSourceMode = ELightSourceMode::Punctual;
		}
		else if (ShadowingType == ECapsuleShadowingType::IndirectTiledCulling || ShadowingType == ECapsuleShadowingType::MovableSkylightTiledCulling)
		{
			LightSourceMode = ELightSourceMode::FromCapsule;
		}
		else if (ShadowingType == ECapsuleShadowingType::MovableSkylightTiledCullingGatherFromReceiverBentNormal)
		{
			LightSourceMode = ELightSourceMode::FromReceiver;
		}
		else
		{
			check(0);
		}

		OutEnvironment.SetDefine(TEXT("LIGHT_SOURCE_MODE"), (uint32)LightSourceMode);
		const bool bMovableSkyLightCulling = (ShadowingType == ECapsuleShadowingType::MovableSkylightTiledCulling || ShadowingType == ECapsuleShadowingType::MovableSkylightTiledCullingGatherFromReceiverBentNormal);
		const bool bApplyToBentNormal = !IsMobilePlatform(Parameters.Platform) ? bMovableSkyLightCulling : false;
		OutEnvironment.SetDefine(TEXT("APPLY_TO_BENT_NORMAL"), bApplyToBentNormal);

		EIndirectShadowingPrimitiveTypes PrimitiveTypes = PermutationVector.Get<FIndirectPrimitiveType>();

		if (PrimitiveTypes == EIndirectShadowingPrimitiveTypes::CapsuleShapes || PrimitiveTypes == EIndirectShadowingPrimitiveTypes::CapsuleShapesAndMeshDistanceFields)
		{
			OutEnvironment.SetDefine(TEXT("SUPPORT_CAPSULE_SHAPES"), 1);
		}

		if (!IsMobilePlatform(Parameters.Platform) && (PrimitiveTypes == EIndirectShadowingPrimitiveTypes::MeshDistanceFields || PrimitiveTypes == EIndirectShadowingPrimitiveTypes::CapsuleShapesAndMeshDistanceFields))
		{
			OutEnvironment.SetDefine(TEXT("SUPPORT_MESH_DISTANCE_FIELDS"), 1);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FCapsuleShadowingCS, "/Engine/Private/CapsuleShadowShaders.usf", "CapsuleShadowingCS", SF_Compute);

class FCapsuleShadowingUpsampleVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCapsuleShadowingUpsampleVS);
	SHADER_USE_PARAMETER_STRUCT(FCapsuleShadowingUpsampleVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileIntersectionCounts)
		SHADER_PARAMETER(FIntPoint, TileDimensions)
		SHADER_PARAMETER(FVector2f, TileSize)
		SHADER_PARAMETER(FIntRect, ScissorRectMinAndSize)
		SHADER_PARAMETER(uint32, EyeIndex)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TILES_PER_INSTANCE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCapsuleShadowingUpsampleVS, "/Engine/Private/CapsuleShadowShaders.usf", "CapsuleShadowingUpsampleVS", SF_Vertex);

class FCapsuleShadowingUpsamplePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCapsuleShadowingUpsamplePS);
public:
	SHADER_USE_PARAMETER_STRUCT(FCapsuleShadowingUpsamplePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowFactorsTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowFactorsSampler)
		SHADER_PARAMETER(FIntRect, ScissorRectMinAndSize)
		SHADER_PARAMETER(FVector2f, ShadowFactorsUVBilinearMax)
		SHADER_PARAMETER(float, OutputtingToLightAttenuation)
		SHADER_PARAMETER(uint32, EyeIndex)
	END_SHADER_PARAMETER_STRUCT()

	class FUpsampleRequired : SHADER_PERMUTATION_BOOL("SHADOW_FACTORS_UPSAMPLE_REQUIRED");
	class FApplySSAO : SHADER_PERMUTATION_BOOL("APPLY_TO_SSAO");
	using FPermutationDomain = TShaderPermutationDomain<FUpsampleRequired, FApplySSAO>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("UPSAMPLE_PASS"), 1);
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), 2);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCapsuleShadowingUpsamplePS, "/Engine/Private/CapsuleShadowShaders.usf", "CapsuleShadowingUpsamplePS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FUpsampleCapsuleShadowParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FCapsuleShadowingUpsampleVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FCapsuleShadowingUpsamplePS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void SetupCapsuleShadowingParameters(
	FRDGBuilder& GraphBuilder,
	FCapsuleShadowingCS::FParameters& Parameters,
	ECapsuleShadowingType ShadowingType,
	FRDGTextureUAVRef OutputUAV,
	FIntPoint TileDimensions,
	FRDGTextureRef ReceiverBentNormalTexture,
	FVector2D NumGroups,
	const FLightSceneInfo* LightSceneInfo,
	const FIntRect& ScissorRect,
	int32 DownsampleFactor,
	float MaxOcclusionDistance,
	const FScene* Scene,
	const FViewInfo& View,
	const uint32 ViewIndex,

	uint32 NumShadowCapsules,
	FRDGBufferSRVRef ShadowCapsuleShapesBuffer,
	uint32 NumMeshDistanceFieldCasters,
	FRDGBufferSRVRef MeshDistanceFieldCasterIndicesSRV,

	FRDGBufferSRVRef LightDirectionDataSRV,

	FRDGBufferUAVRef CapsuleTileIntersectionCountsUAV
)
{
	Parameters.SceneTextures = GetSceneTextureShaderParameters(View);
	Parameters.View = View.ViewUniformBuffer;
	Parameters.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

	if (ShadowingType == ECapsuleShadowingType::MovableSkylightTiledCulling)
	{
		Parameters.RWShadowFactors = nullptr;
		Parameters.RWBentNormalTexture = OutputUAV;
	}
	else
	{
		Parameters.RWShadowFactors = OutputUAV;
		Parameters.RWBentNormalTexture = nullptr;
	}

	Parameters.RWTileIntersectionCounts = CapsuleTileIntersectionCountsUAV;

	Parameters.TileDimensions = TileDimensions;

	if (ShadowingType == ECapsuleShadowingType::MovableSkylightTiledCulling)
	{
		check(ReceiverBentNormalTexture);
		Parameters.ReceiverBentNormalTexture = ReceiverBentNormalTexture;
	}
	else
	{
		Parameters.ReceiverBentNormalTexture = nullptr;
	}

	Parameters.NumGroups = FVector2f(NumGroups);

	if (LightSceneInfo)
	{
		check(ShadowingType == ECapsuleShadowingType::DirectionalLightTiledCulling || ShadowingType == ECapsuleShadowingType::PointLightTiledCulling);

		const FLightSceneProxy& LightProxy = *LightSceneInfo->Proxy;

		FLightRenderParameters LightParameters;
		LightProxy.GetLightShaderParameters(LightParameters);

		Parameters.LightDirection = LightParameters.Direction;

		FVector4f LightTranslatedPositionAndInvRadius((FVector3f)(LightParameters.WorldPosition + View.ViewMatrices.GetPreViewTranslation()), LightParameters.InvRadius);
		Parameters.LightTranslatedPositionAndInvRadius = LightTranslatedPositionAndInvRadius;

		// Default light source radius of 0 gives poor results
		Parameters.LightSourceRadius = LightParameters.SourceRadius == 0 ? 20 : FMath::Clamp(LightParameters.SourceRadius, .001f, 1.0f / (4 * LightParameters.InvRadius));

		Parameters.RayStartOffsetDepthScale = LightProxy.GetRayStartOffsetDepthScale();

		const float LightSourceAngle = FMath::Clamp(LightProxy.GetLightSourceAngle() * 5, 1.0f, 30.0f) * PI / 180.0f;
		const FVector3f LightAngleAndNormalThreshold(LightSourceAngle, FMath::Cos(PI / 2 + LightSourceAngle), LightProxy.GetTraceDistance());
		Parameters.LightAngleAndNormalThreshold = LightAngleAndNormalThreshold;
	}
	else
	{
		check(ShadowingType == ECapsuleShadowingType::IndirectTiledCulling || ShadowingType == ECapsuleShadowingType::MovableSkylightTiledCulling || ShadowingType == ECapsuleShadowingType::MovableSkylightTiledCullingGatherFromReceiverBentNormal);
	}

	Parameters.EyeIndex = ViewIndex;

	Parameters.ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());
	Parameters.DownsampleFactor = DownsampleFactor;

	Parameters.IndirectCapsuleSelfShadowingIntensity = Scene->DynamicIndirectShadowsSelfShadowingIntensity;
	Parameters.MaxOcclusionDistance = MaxOcclusionDistance;

	Parameters.NumShadowCapsules = NumShadowCapsules;
	Parameters.ShadowCapsuleShapes = ShadowCapsuleShapesBuffer;

	Parameters.NumMeshDistanceFieldCasters = NumMeshDistanceFieldCasters;
	Parameters.MeshDistanceFieldCasterIndices = MeshDistanceFieldCasterIndicesSRV;
	Parameters.LightDirectionData = LightDirectionDataSRV;

	const float CosFadeStartAngleValue = FMath::Cos(GCapsuleShadowFadeAngleFromVertical);
	Parameters.CosFadeStartAngle = FVector2f(CosFadeStartAngleValue, 1.0f / (1.0f - CosFadeStartAngleValue));

	Parameters.DFObjectBufferParameters = DistanceField::SetupObjectBufferParameters(GraphBuilder, Scene->DistanceFieldSceneData);
	Parameters.DFAtlasParameters = DistanceField::SetupAtlasParameters(GraphBuilder, Scene->DistanceFieldSceneData);
}

bool FSceneRenderer::RenderCapsuleDirectShadows(
	FRDGBuilder& GraphBuilder,
	const FLightSceneInfo& LightSceneInfo,
	FRDGTextureRef ScreenShadowMaskTexture,
	TArrayView<const FProjectedShadowInfo* const> CapsuleShadows,
	bool bProjectingForForwardShading) const
{
	bool bAllViewsHaveViewState = true;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (!View.ViewState)
		{
			bAllViewsHaveViewState = false;
		}
	}

	if (!IsCapsuleDirectShadowsEnabled(ShaderPlatform)
		|| CapsuleShadows.Num() == 0
		|| !ViewFamily.EngineShowFlags.CapsuleShadows
		|| !bAllViewsHaveViewState)
	{
		return false;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_RenderCapsuleShadows);

	const FIntPoint BufferSize = GetBufferSizeForCapsuleShadows(Views[0]);

	FRDGTextureRef RayTracedShadowsRT = nullptr;
	{
		const FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(BufferSize, PF_G16R16F, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV));
		RayTracedShadowsRT = GraphBuilder.CreateTexture(Desc, TEXT("CapsuleShadows.ShadowFactors"));
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		const FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_STAT(GraphBuilder, CapsuleShadows, "CapsuleShadows View%d", ViewIndex);

		TArray<FCapsuleShape3f> CapsuleShapeData;

		for (int32 ShadowIndex = 0; ShadowIndex < CapsuleShadows.Num(); ShadowIndex++)
		{
			const FProjectedShadowInfo* Shadow = CapsuleShadows[ShadowIndex];

			int32 OriginalCapsuleIndex = CapsuleShapeData.Num();

			TArray<const FPrimitiveSceneInfo*, SceneRenderingAllocator> ShadowGroupPrimitives;
			Shadow->GetParentSceneInfo()->GatherLightingAttachmentGroupPrimitives(ShadowGroupPrimitives);

			for (int32 ChildIndex = 0; ChildIndex < ShadowGroupPrimitives.Num(); ChildIndex++)
			{
				const FPrimitiveSceneInfo* PrimitiveSceneInfo = ShadowGroupPrimitives[ChildIndex];

				if (PrimitiveSceneInfo->Proxy->CastsDynamicShadow())
				{
					PrimitiveSceneInfo->Proxy->GetShadowShapes(PreViewTranslation, CapsuleShapeData);
				}
			}

			const float FadeRadiusScale = Shadow->FadeAlphas[ViewIndex];

			for (int32 ShapeIndex = OriginalCapsuleIndex; ShapeIndex < CapsuleShapeData.Num(); ShapeIndex++)
			{
				CapsuleShapeData[ShapeIndex].Radius *= FadeRadiusScale;
			}
		}

		if (CapsuleShapeData.Num() > 0)
		{
			const bool bDirectionalLight = LightSceneInfo.Proxy->GetLightType() == LightType_Directional;
			FIntRect ScissorRect;

			if (!LightSceneInfo.Proxy->GetScissorRect(ScissorRect, View, View.ViewRect))
			{
				ScissorRect = View.ViewRect;
			}

			// CapsuleShadowingCS always outputs at rect with min = (0,0)
			const FIntRect DownsampledViewRect(0, 0, ScissorRect.Width() / GetCapsuleShadowDownsampleFactor(), ScissorRect.Height() / GetCapsuleShadowDownsampleFactor());

			const FIntPoint GroupSize = FIntPoint(
				FMath::DivideAndRoundUp(DownsampledViewRect.Width(), GShadowShapeTileSize),
				FMath::DivideAndRoundUp(DownsampledViewRect.Height(), GShadowShapeTileSize))
				.ComponentMax(FIntPoint(1, 1));

			FRDGBufferRef CapsuleTileIntersectionCountsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), GroupSize.X * GroupSize.Y), TEXT("CapsuleTileIntersectionCountsBuffer"));
			FRDGBufferUAVRef CapsuleTileIntersectionCountsUAV = GraphBuilder.CreateUAV(CapsuleTileIntersectionCountsBuffer);
			FRDGBufferSRVRef CapsuleTileIntersectionCountsSRV = GraphBuilder.CreateSRV(CapsuleTileIntersectionCountsBuffer);

			AddClearUAVPass(GraphBuilder, CapsuleTileIntersectionCountsUAV, 0);

			// Upload capsule shape data
			// TODO: Use FRDGUploadData
			FRDGBufferRef ShadowCapsuleShapesBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("ShadowCapsuleShapesBuffer"), CapsuleShapeData);

			{
				ECapsuleShadowingType ShadowingType = bDirectionalLight ? ECapsuleShadowingType::DirectionalLightTiledCulling : ECapsuleShadowingType::PointLightTiledCulling;

				auto* PassParameters = GraphBuilder.AllocParameters<FCapsuleShadowingCS::FParameters>();

				SetupCapsuleShadowingParameters(
					GraphBuilder,
					*PassParameters,
					ShadowingType,
					GraphBuilder.CreateUAV(RayTracedShadowsRT),
					GroupSize,
					nullptr,
					FVector2D(GroupSize.X, GroupSize.Y),
					&LightSceneInfo,
					ScissorRect,
					GetCapsuleShadowDownsampleFactor(),
					GCapsuleMaxDirectOcclusionDistance,
					Scene,
					View,
					ViewIndex,
					CapsuleShapeData.Num(),
					GraphBuilder.CreateSRV(ShadowCapsuleShapesBuffer),
					0,
					nullptr,
					nullptr,
					CapsuleTileIntersectionCountsUAV
				);

				FCapsuleShadowingCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FCapsuleShadowingCS::FShapeShadow>(ShadowingType);
				PermutationVector.Set<FCapsuleShadowingCS::FIndirectPrimitiveType>(EIndirectShadowingPrimitiveTypes::CapsuleShapes);
				auto ComputeShader = View.ShaderMap->GetShader<FCapsuleShadowingCS>(PermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("TiledCapsuleShadowing"),
					ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FIntVector(GroupSize.X, GroupSize.Y, 1));
			}

			{
				const FVector2f RayTracedShadowsTextureExtent(RayTracedShadowsRT->Desc.Extent);
				const FVector2f RayTracedShadowsTextureExtentInverse(1.0f / RayTracedShadowsTextureExtent.X, 1.0f / RayTracedShadowsTextureExtent.Y);
				const FVector2f RayTracedShadowsTextureViewportMax(DownsampledViewRect.Max);

				auto VertexShader = View.ShaderMap->GetShader<FCapsuleShadowingUpsampleVS>();

				FCapsuleShadowingUpsamplePS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FCapsuleShadowingUpsamplePS::FUpsampleRequired>(!GCapsuleShadowsFullResolution);
				PermutationVector.Set<FCapsuleShadowingUpsamplePS::FApplySSAO>(false);
				auto PixelShader = View.ShaderMap->GetShader<FCapsuleShadowingUpsamplePS>(PermutationVector);

				FUpsampleCapsuleShadowParameters* PassParameters = GraphBuilder.AllocParameters<FUpsampleCapsuleShadowParameters>();

				if (GRHISupportsBindingTexArrayPerSlice) // Native mobile multi view
				{
					PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenShadowMaskTexture, GetLoadActionIfProduced(ScreenShadowMaskTexture, ERenderTargetLoadAction::EClear), /*MipIndex*/ 0, /*ArraySlice*/ ViewIndex);
				}
				else // Mobile multi view fallback or not stereo
				{
					// D3D12 RHI does not support setting slice 1 of a texture 2d array without creating it with TexCreate_TargetArraySlicesIndependently.
					// Unfortunately, this flag does not allow to target the array with SV_RenderTargetArrayIndex anymore so this is not viable for mobile multi view fallback.
					// In this case, the PS draw calls are rerouted to the correct slice with SV_RenderTargetArrayIndex as a workaround.
					PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenShadowMaskTexture, GetLoadActionIfProduced(ScreenShadowMaskTexture, ERenderTargetLoadAction::EClear));
				}

				PassParameters->SceneTextures = GetSceneTextureShaderParameters(View);
				PassParameters->VS.View = GetShaderBinding(View.ViewUniformBuffer);
				PassParameters->VS.TileIntersectionCounts = CapsuleTileIntersectionCountsSRV;
				PassParameters->VS.TileDimensions = GroupSize;
				PassParameters->VS.TileSize = FVector2f(GShadowShapeTileSize * GetCapsuleShadowDownsampleFactor(), GShadowShapeTileSize * GetCapsuleShadowDownsampleFactor());
				PassParameters->VS.ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());
				PassParameters->VS.EyeIndex = ViewIndex;

				PassParameters->PS.View = GetShaderBinding(View.ViewUniformBuffer);
				PassParameters->PS.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
				PassParameters->PS.ShadowFactorsTexture = RayTracedShadowsRT;
				PassParameters->PS.ShadowFactorsSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
				PassParameters->PS.ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());
				PassParameters->PS.ShadowFactorsUVBilinearMax = (RayTracedShadowsTextureViewportMax - 0.5f) * RayTracedShadowsTextureExtentInverse;
				PassParameters->PS.OutputtingToLightAttenuation = 1.0f;
				PassParameters->PS.EyeIndex = ViewIndex;

				ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
				ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("UpsampleCapsuleShadow %dx%d", ScissorRect.Width(), ScissorRect.Height()),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, VertexShader, PixelShader, &View, &LightSceneInfo, RayTracedShadowsRT, GroupSize, ScissorRect, bProjectingForForwardShading](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

					RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

					GraphicsPSOInit.BlendState = FProjectedShadowInfo::GetBlendStateForProjection(
						LightSceneInfo.GetDynamicShadowMapChannel(),
						false,
						false,
						bProjectingForForwardShading,
						false);

					GraphicsPSOInit.PrimitiveType = PT_TriangleList;
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GTileVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

					RHICmdList.SetStreamSource(0, GetOneTileQuadVertexBuffer(), 0);
					RHICmdList.DrawIndexedPrimitive(GetOneTileQuadIndexBuffer(),
						0, 
						0, 
						4,
						0,
						2,
						FMath::DivideAndRoundUp(GroupSize.X * GroupSize.Y, 1));
				});
			}
		}
	}
	return true;
}

void FDeferredShadingSceneRenderer::CreateIndirectCapsuleShadows()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CreateIndirectCapsuleShadows);

	if (!SupportsCapsuleIndirectShadows(ShaderPlatform))
	{
		return;
	}

	for (int32 PrimitiveIndex = 0; PrimitiveIndex < Scene->DynamicIndirectCasterPrimitives.Num(); PrimitiveIndex++)
	{
		FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene->DynamicIndirectCasterPrimitives[PrimitiveIndex];
		FPrimitiveSceneProxy* PrimitiveProxy = PrimitiveSceneInfo->Proxy;

		if (PrimitiveProxy->CastsDynamicShadow() && PrimitiveProxy->CastsDynamicIndirectShadow())
		{
			TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> ShadowGroupPrimitives;
			PrimitiveSceneInfo->GatherLightingAttachmentGroupPrimitives(ShadowGroupPrimitives);

			// Compute the composite bounds of this group of shadow primitives.
			FBoxSphereBounds LightingGroupBounds = ShadowGroupPrimitives[0]->Proxy->GetBounds();

			for (int32 ChildIndex = 1; ChildIndex < ShadowGroupPrimitives.Num(); ChildIndex++)
			{
				const FPrimitiveSceneInfo* ShadowChild = ShadowGroupPrimitives[ChildIndex];

				if (ShadowChild->Proxy->CastsDynamicShadow())
				{
					LightingGroupBounds = LightingGroupBounds + ShadowChild->Proxy->GetBounds();
				}
			}

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				FViewInfo& View = Views[ViewIndex];

				float EffectiveMaxIndirectOcclusionDistance = GCapsuleMaxIndirectOcclusionDistance;

				if (PrimitiveProxy->HasDistanceFieldRepresentation())
				{
					// Increase max occlusion distance based on object size for distance field casters
					// This improves the solidness of the shadows, since the fadeout distance causes internal structure of objects to become visible
					EffectiveMaxIndirectOcclusionDistance += .5f * LightingGroupBounds.SphereRadius;
				}

				if (View.ViewFrustum.IntersectBox(LightingGroupBounds.Origin, LightingGroupBounds.BoxExtent + FVector(EffectiveMaxIndirectOcclusionDistance)))
				{
					View.IndirectShadowPrimitives.Add(PrimitiveSceneInfo);
				}
			}
		}
	}
}

struct IndirectCapsuleShadowsResources
{
	FRDGBufferSRVRef IndirectShadowCapsuleShapesSRV;
	FRDGBufferSRVRef IndirectShadowMeshDistanceFieldCasterIndicesSRV;
	FRDGBufferSRVRef IndirectShadowLightDirectionSRV;
};

static IndirectCapsuleShadowsResources CreateIndirectCapsuleShadowsResources(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View, 
	int32& NumCapsuleShapes, 
	int32& NumMeshesWithCapsules, 
	int32& NumMeshDistanceFieldCasters)
{
	const FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();

	IndirectCapsuleShadowsResources Output;
	Output.IndirectShadowCapsuleShapesSRV = nullptr;
	Output.IndirectShadowMeshDistanceFieldCasterIndicesSRV = nullptr;
	Output.IndirectShadowLightDirectionSRV = nullptr;

	const float CosFadeStartAngle = FMath::Cos(GCapsuleShadowFadeAngleFromVertical);
	const FSkyLightSceneProxy* SkyLight = Scene ? Scene->SkyLight : NULL;

	static TArray<FCapsuleShape3f> CapsuleShapeData;
	static TArray<FVector4f> CapsuleLightSourceData;
	static TArray<int32, TInlineAllocator<1>> MeshDistanceFieldCasterIndices;
	static TArray<FVector4f> DistanceFieldCasterLightSourceData;

	CapsuleShapeData.Reset();
	MeshDistanceFieldCasterIndices.Reset();
	CapsuleLightSourceData.Reset();
	DistanceFieldCasterLightSourceData.Reset();

	const bool bComputeLightDataFromVolumetricLightmapOrGpuSkyEnvMapIrradiance = Scene && (Scene->VolumetricLightmapSceneData.HasData() || (Scene->SkyLight && Scene->SkyLight->bRealTimeCaptureEnabled));

	for (int32 PrimitiveIndex = 0; PrimitiveIndex < View.IndirectShadowPrimitives.Num(); PrimitiveIndex++)
	{
		FPrimitiveSceneInfo* PrimitiveSceneInfo = View.IndirectShadowPrimitives[PrimitiveIndex];
		const FIndirectLightingCacheAllocation* Allocation = PrimitiveSceneInfo->IndirectLightingCacheAllocation;

		FVector4f PackedLightDirection(0, 0, 1, PI / 16);
		float ShapeFadeAlpha = 1;

		if (bComputeLightDataFromVolumetricLightmapOrGpuSkyEnvMapIrradiance)
		{
			// Encode object position for ComputeLightDirectionsFromVolumetricLightmapCS
			PackedLightDirection = FVector4f((FVector3f)PrimitiveSceneInfo->Proxy->GetBounds().Origin, 0);
		}
		else if (SkyLight 
			&& !SkyLight->bHasStaticLighting
			&& SkyLight->bWantsStaticShadowing
			&& View.Family->EngineShowFlags.SkyLighting
			&& Allocation)
		{
			// Stationary sky light case
			// Get the indirect shadow direction from the unoccluded sky direction
			const float ConeAngle = FMath::Max(Allocation->CurrentSkyBentNormal.W * GCapsuleSkyAngleScale * .5f * PI, GCapsuleMinSkyAngle * PI / 180.0f);
			PackedLightDirection = FVector4f(FVector3f(Allocation->CurrentSkyBentNormal), ConeAngle);
		}
		else if (SkyLight 
			&& !SkyLight->bHasStaticLighting 
			&& !SkyLight->bWantsStaticShadowing
			&& View.Family->EngineShowFlags.SkyLighting)
		{
			// Movable sky light case
			const FSHVector2 SkyLightingIntensity = FSHVectorRGB2(SkyLight->IrradianceEnvironmentMap).GetLuminance();
			const FVector ExtractedMaxDirection = SkyLightingIntensity.GetMaximumDirection();

			// Get the indirect shadow direction from the primary sky lighting direction
			PackedLightDirection = FVector4f((FVector3f)ExtractedMaxDirection, GCapsuleIndirectConeAngle);
		}
		else if (Allocation)
		{
			// Static sky light or no sky light case
			FSHVectorRGB2 IndirectLighting;
			IndirectLighting.R = FSHVector2(Allocation->SingleSamplePacked0[0]);
			IndirectLighting.G = FSHVector2(Allocation->SingleSamplePacked0[1]);
			IndirectLighting.B = FSHVector2(Allocation->SingleSamplePacked0[2]);
			const FSHVector2 IndirectLightingIntensity = IndirectLighting.GetLuminance();
			const FVector ExtractedMaxDirection = IndirectLightingIntensity.GetMaximumDirection();

			// Get the indirect shadow direction from the primary indirect lighting direction
			PackedLightDirection = FVector4f((FVector3f)ExtractedMaxDirection, GCapsuleIndirectConeAngle);
		}

		if (CosFadeStartAngle < 1 && !bComputeLightDataFromVolumetricLightmapOrGpuSkyEnvMapIrradiance)
		{
			// Fade out when nearly vertical up due to self shadowing artifacts
			ShapeFadeAlpha = 1 - FMath::Clamp(2 * (-PackedLightDirection.Z - CosFadeStartAngle) / (1 - CosFadeStartAngle), 0.0f, 1.0f);
		}
			
		if (ShapeFadeAlpha > 0)
		{
			const int32 OriginalNumCapsuleShapes = CapsuleShapeData.Num();
			const int32 OriginalNumMeshDistanceFieldCasters = MeshDistanceFieldCasterIndices.Num();

			TArray<const FPrimitiveSceneInfo*, SceneRenderingAllocator> ShadowGroupPrimitives;
			PrimitiveSceneInfo->GatherLightingAttachmentGroupPrimitives(ShadowGroupPrimitives);

			for (int32 ChildIndex = 0; ChildIndex < ShadowGroupPrimitives.Num(); ChildIndex++)
			{
				const FPrimitiveSceneInfo* GroupPrimitiveSceneInfo = ShadowGroupPrimitives[ChildIndex];

				if (GroupPrimitiveSceneInfo->Proxy->CastsDynamicShadow())
				{
					GroupPrimitiveSceneInfo->Proxy->GetShadowShapes(PreViewTranslation, CapsuleShapeData);
					
					if (GroupPrimitiveSceneInfo->Proxy->HasDistanceFieldRepresentation())
					{
						MeshDistanceFieldCasterIndices.Append(GroupPrimitiveSceneInfo->DistanceFieldInstanceIndices);
					}
				}
			}

			// Pack both values into a single float to keep float4 alignment
			const FFloat16 LightAngle16f = FFloat16(PackedLightDirection.W);
			const FFloat16 MinVisibility16f = FFloat16(PrimitiveSceneInfo->Proxy->GetDynamicIndirectShadowMinVisibility());
			const uint32 PackedWInt = ((uint32)LightAngle16f.Encoded) | ((uint32)MinVisibility16f.Encoded << 16);
			PackedLightDirection.W = *(float*)&PackedWInt;

			//@todo - remove entries with 0 fade alpha
			for (int32 ShapeIndex = OriginalNumCapsuleShapes; ShapeIndex < CapsuleShapeData.Num(); ShapeIndex++)
			{
				CapsuleLightSourceData.Add(PackedLightDirection);
			}

			for (int32 CasterIndex = OriginalNumMeshDistanceFieldCasters; CasterIndex < MeshDistanceFieldCasterIndices.Num(); CasterIndex++)
			{
				DistanceFieldCasterLightSourceData.Add(PackedLightDirection);
			}

			NumMeshesWithCapsules++;
		}
	}

	if (CapsuleShapeData.Num() > 0 || MeshDistanceFieldCasterIndices.Num() > 0)
	{
		if (CapsuleShapeData.Num() > 0)
		{
			// Upload capsule shape data
			// TODO: Use FRDGUploadData
			FRDGBufferRef IndirectShadowCapsuleShapesVertexBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("IndirectShadowCapsuleShapesVertexBuffer"), CapsuleShapeData);
			Output.IndirectShadowCapsuleShapesSRV = GraphBuilder.CreateSRV(IndirectShadowCapsuleShapesVertexBuffer);
		}

		if (MeshDistanceFieldCasterIndices.Num() > 0)
		{
			// Upload mesh distance field caster indices
			// TODO: Use FRDGUploadData
			FRDGBufferRef IndirectShadowMeshDistanceFieldCasterIndicesVertexBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("IndirectShadowMeshDistanceFieldCasterIndicesVertexBuffer"), MeshDistanceFieldCasterIndices);
			Output.IndirectShadowMeshDistanceFieldCasterIndicesSRV = GraphBuilder.CreateSRV(IndirectShadowMeshDistanceFieldCasterIndicesVertexBuffer);
		}

		const int32 NumLightDataElements = CapsuleLightSourceData.Num() + DistanceFieldCasterLightSourceData.Num();

		FRDGBufferRef IndirectShadowLightDirectionVertexBuffer = nullptr;
		FRDGBufferSRVRef IndirectShadowLightDirectionSRV = nullptr;
		{
			// Upload light data
			// Light data for distance fields is placed after capsule light data
			// This packing behavior must match GetLightDirectionData

			const size_t CapsuleLightSourceDataSize = CapsuleLightSourceData.Num() * CapsuleLightSourceData.GetTypeSize();
			const size_t DistanceFieldCasterLightSourceDataSize = DistanceFieldCasterLightSourceData.Num() * DistanceFieldCasterLightSourceData.GetTypeSize();

			FRDGUploadData<FVector4f> TempData(GraphBuilder, NumLightDataElements);
			FPlatformMemory::Memcpy(TempData.GetData(), CapsuleLightSourceData.GetData(), CapsuleLightSourceDataSize);
			FPlatformMemory::Memcpy((char*)TempData.GetData() + CapsuleLightSourceDataSize, DistanceFieldCasterLightSourceData.GetData(), DistanceFieldCasterLightSourceDataSize);

			IndirectShadowLightDirectionVertexBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("IndirectShadowLightDirectionVertexBuffer"), TempData);

			IndirectShadowLightDirectionSRV = GraphBuilder.CreateSRV(IndirectShadowLightDirectionVertexBuffer);
			Output.IndirectShadowLightDirectionSRV = IndirectShadowLightDirectionSRV;
		}

		if (bComputeLightDataFromVolumetricLightmapOrGpuSkyEnvMapIrradiance)
		{
			FRDGBufferRef IndirectShadowVolumetricLightmapDerivedLightDirection = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FVector4f), NumLightDataElements), TEXT("IndirectShadowVolumetricLightmapDerivedLightDirection"));
			FRDGBufferUAVRef ComputedLightDirectionUAV = GraphBuilder.CreateUAV(IndirectShadowVolumetricLightmapDerivedLightDirection);
			Output.IndirectShadowLightDirectionSRV = GraphBuilder.CreateSRV(IndirectShadowVolumetricLightmapDerivedLightDirection);

			TShaderMapRef<FComputeLightDirectionFromVolumetricLightmapCS> ComputeShader(View.ShaderMap);

			uint32 SkyLightMode = Scene->SkyLight && Scene->SkyLight->bWantsStaticShadowing ? 1 : 0;
			SkyLightMode = Scene->SkyLight && Scene->SkyLight->bRealTimeCaptureEnabled ? 2 : SkyLightMode;

			const int32 GroupSize = FMath::DivideAndRoundUp(NumLightDataElements, GComputeLightDirectionFromVolumetricLightmapGroupSize);

			FComputeLightDirectionFromVolumetricLightmapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComputeLightDirectionFromVolumetricLightmapCS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->NumLightDirectionData = NumLightDataElements;
			PassParameters->SkyLightMode = SkyLightMode;
			PassParameters->CapsuleIndirectConeAngle = GCapsuleIndirectConeAngle;
			PassParameters->CapsuleSkyAngleScale = GCapsuleSkyAngleScale;
			PassParameters->CapsuleMinSkyAngle = GCapsuleMinSkyAngle;
			PassParameters->RWComputedLightDirectionData = ComputedLightDirectionUAV;
			PassParameters->LightDirectionData = IndirectShadowLightDirectionSRV;

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("LightDirectionFromVolumetricLightmap"),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FIntVector(GroupSize, 1, 1));
		}
	}

	NumCapsuleShapes = CapsuleShapeData.Num();
	NumMeshDistanceFieldCasters = MeshDistanceFieldCasterIndices.Num();

	return Output;
}

void FDeferredShadingSceneRenderer::RenderIndirectCapsuleShadows(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures) const
{
	if (!SupportsCapsuleIndirectShadows(ShaderPlatform)
		|| !ViewFamily.EngineShowFlags.DynamicShadows
		|| !ViewFamily.EngineShowFlags.CapsuleShadows)
	{
		return;
	}

	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderIndirectCapsuleShadows);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RenderIndirectCapsuleShadows);

	bool bAnyViewsUseCapsuleShadows = false;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (View.IndirectShadowPrimitives.Num() > 0 && View.ViewState)
		{
			bAnyViewsUseCapsuleShadows = true;
		}
	}

	if (!bAnyViewsUseCapsuleShadows)
	{
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "IndirectCapsuleShadows");

	const FIntPoint BufferSize = GetBufferSizeForCapsuleShadows(Views[0]);

	FRDGTextureRef RayTracedShadowsRT = nullptr;
	{
		const FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(BufferSize, PF_G16R16F, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV));
		RayTracedShadowsRT = GraphBuilder.CreateTexture(Desc, TEXT("CapsuleShadows.ShadowFactors"));
	}

	TArray<FRenderTargetBinding, TInlineAllocator<2>> RenderTargets;

	if (SceneTextures.Color.IsValid())
	{
		RenderTargets.Emplace(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
	}

	check(SceneTextures.ScreenSpaceAO);
	RenderTargets.Emplace(SceneTextures.ScreenSpaceAO, SceneTextures.ScreenSpaceAO->HasBeenProduced() ? ERenderTargetLoadAction::ELoad : ERenderTargetLoadAction::EClear);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (View.IndirectShadowPrimitives.Num() > 0 && View.ViewState)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_STAT(GraphBuilder, CapsuleShadows, "CapsuleShadows");

			int32 NumCapsuleShapes = 0;
			int32 NumMeshesWithCapsules = 0;
			int32 NumMeshDistanceFieldCasters = 0;
			IndirectCapsuleShadowsResources Resources = CreateIndirectCapsuleShadowsResources(GraphBuilder, Scene, View, NumCapsuleShapes, NumMeshesWithCapsules, NumMeshDistanceFieldCasters);

			if (NumCapsuleShapes == 0 && NumMeshDistanceFieldCasters == 0)
			{
				continue;
			}

			check(Resources.IndirectShadowLightDirectionSRV);

			const FIntRect ScissorRect = View.ViewRect;

			// CapsuleShadowingCS always outputs at rect with min = (0,0)
			const FIntRect DownsampledViewRect(0, 0, ScissorRect.Width() / GetCapsuleShadowDownsampleFactor(), ScissorRect.Height() / GetCapsuleShadowDownsampleFactor());

			const FIntPoint GroupSize(
				FMath::DivideAndRoundUp(DownsampledViewRect.Width(), GShadowShapeTileSize),
				FMath::DivideAndRoundUp(DownsampledViewRect.Height(), GShadowShapeTileSize));

			FRDGBufferRef CapsuleTileIntersectionCountsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), GroupSize.X * GroupSize.Y), TEXT("CapsuleTileIntersectionCountsBuffer"));
			FRDGBufferUAVRef CapsuleTileIntersectionCountsUAV = GraphBuilder.CreateUAV(CapsuleTileIntersectionCountsBuffer);
			FRDGBufferSRVRef CapsuleTileIntersectionCountsSRV = GraphBuilder.CreateSRV(CapsuleTileIntersectionCountsBuffer);

			AddClearUAVPass(GraphBuilder, CapsuleTileIntersectionCountsUAV, 0);

			{
				auto* PassParameters = GraphBuilder.AllocParameters<FCapsuleShadowingCS::FParameters>();

				SetupCapsuleShadowingParameters(
					GraphBuilder,
					*PassParameters,
					ECapsuleShadowingType::IndirectTiledCulling,
					GraphBuilder.CreateUAV(RayTracedShadowsRT),
					GroupSize,
					nullptr,
					FVector2D(GroupSize.X, GroupSize.Y),
					nullptr,
					ScissorRect,
					GetCapsuleShadowDownsampleFactor(),
					GCapsuleMaxIndirectOcclusionDistance,
					Scene,
					View,
					ViewIndex,
					NumCapsuleShapes,
					Resources.IndirectShadowCapsuleShapesSRV,
					NumMeshDistanceFieldCasters,
					Resources.IndirectShadowMeshDistanceFieldCasterIndicesSRV,
					Resources.IndirectShadowLightDirectionSRV,
					CapsuleTileIntersectionCountsUAV
				);

				EIndirectShadowingPrimitiveTypes PrimitiveTypes;

				if (NumCapsuleShapes > 0 && NumMeshDistanceFieldCasters > 0)
				{
					PrimitiveTypes = EIndirectShadowingPrimitiveTypes::CapsuleShapesAndMeshDistanceFields;
				}
				else if (NumCapsuleShapes > 0)
				{
					PrimitiveTypes = EIndirectShadowingPrimitiveTypes::CapsuleShapes;
				}
				else
				{
					check(NumMeshDistanceFieldCasters > 0);
					PrimitiveTypes = EIndirectShadowingPrimitiveTypes::MeshDistanceFields;
				}

				FCapsuleShadowingCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FCapsuleShadowingCS::FShapeShadow>(ECapsuleShadowingType::IndirectTiledCulling);
				PermutationVector.Set<FCapsuleShadowingCS::FIndirectPrimitiveType>(PrimitiveTypes);
				auto ComputeShader = View.ShaderMap->GetShader<FCapsuleShadowingCS>(PermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("TiledCapsuleShadowing %u capsules among %u meshes", NumCapsuleShapes, NumMeshesWithCapsules),
					ERDGPassFlags::Compute,
					ComputeShader,
					PassParameters,
					FIntVector(GroupSize.X, GroupSize.Y, 1));
			}

			{
				const FVector2f RayTracedShadowsTextureExtent(RayTracedShadowsRT->Desc.Extent);
				const FVector2f RayTracedShadowsTextureExtentInverse(1.0f / RayTracedShadowsTextureExtent.X, 1.0f / RayTracedShadowsTextureExtent.Y);
				const FVector2f RayTracedShadowsTextureViewportMax(DownsampledViewRect.Max);

				const int32 RenderTargetCount = RenderTargets.Num();

				auto VertexShader = View.ShaderMap->GetShader<FCapsuleShadowingUpsampleVS>();

				FCapsuleShadowingUpsamplePS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FCapsuleShadowingUpsamplePS::FUpsampleRequired>(!GCapsuleShadowsFullResolution);
				PermutationVector.Set<FCapsuleShadowingUpsamplePS::FApplySSAO>(RenderTargetCount > 1);
				auto PixelShader = View.ShaderMap->GetShader<FCapsuleShadowingUpsamplePS>(PermutationVector);

				FUpsampleCapsuleShadowParameters* PassParameters = GraphBuilder.AllocParameters<FUpsampleCapsuleShadowParameters>();
				for (int32 Index = 0; Index < RenderTargetCount; ++Index)
				{
					PassParameters->RenderTargets[Index] = RenderTargets[Index];

					// Only allow clears for the first use of the render target.
					RenderTargets[Index].SetLoadAction(ERenderTargetLoadAction::ELoad);
				}

				PassParameters->SceneTextures = GetSceneTextureShaderParameters(View);

				PassParameters->VS.View = GetShaderBinding(View.ViewUniformBuffer);
				PassParameters->VS.TileIntersectionCounts = CapsuleTileIntersectionCountsSRV;
				PassParameters->VS.TileDimensions = GroupSize;
				PassParameters->VS.TileSize = FVector2f(GShadowShapeTileSize * GetCapsuleShadowDownsampleFactor(), GShadowShapeTileSize * GetCapsuleShadowDownsampleFactor());
				PassParameters->VS.ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());

				PassParameters->PS.View = GetShaderBinding(View.ViewUniformBuffer);
				PassParameters->PS.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
				PassParameters->PS.ShadowFactorsTexture = RayTracedShadowsRT;
				PassParameters->PS.ShadowFactorsSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
				PassParameters->PS.ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());
				PassParameters->PS.ShadowFactorsUVBilinearMax = (RayTracedShadowsTextureViewportMax - 0.5f) * RayTracedShadowsTextureExtentInverse;
				PassParameters->PS.OutputtingToLightAttenuation = 0.0f;

				ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
				ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("UpsampleCapsuleShadow %dx%d", ScissorRect.Width(), ScissorRect.Height()),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, VertexShader, PixelShader, &View, RayTracedShadowsRT, RenderTargetCount, GroupSize, ScissorRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

					RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

					// Modulative blending against scene color for application to indirect diffuse
					if (RenderTargetCount > 1)
					{
						GraphicsPSOInit.BlendState = TStaticBlendState<
							CW_RGB, BO_Add, BF_DestColor, BF_Zero, BO_Add, BF_Zero, BF_One,
							CW_RED, BO_Add, BF_DestColor, BF_Zero, BO_Add, BF_Zero, BF_One>::GetRHI();
					}
					// Modulative blending against SSAO occlusion value for application to indirect specular, since Reflection Environment pass masks by AO
					else
					{
						GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_Zero>::GetRHI();
					}

					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GTileVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

					RHICmdList.SetStreamSource(0, GetOneTileQuadVertexBuffer(), 0);
					RHICmdList.DrawIndexedPrimitive(GetOneTileQuadIndexBuffer(),
						0,
						0,
						4,
						0,
						2,
						FMath::DivideAndRoundUp(GroupSize.X * GroupSize.Y, 1));
				});
			}
		}
	}
}

bool ShouldPrepareForDFInsetIndirectShadow(EShaderPlatform ShaderPlatform, const FEngineShowFlags& EngineShowFlags)
{
	return SupportsCapsuleIndirectShadows(ShaderPlatform) && EngineShowFlags.CapsuleShadows;
}

void FSceneRenderer::RenderCapsuleShadowsForMovableSkylight(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef& BentNormalOutput) const
{
	if (SupportsCapsuleIndirectShadows(ShaderPlatform)
		&& ViewFamily.EngineShowFlags.CapsuleShadows)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RenderCapsuleShadowsSkylight);

		if (View.IndirectShadowPrimitives.Num() > 0 && View.ViewState)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_STAT(GraphBuilder, CapsuleShadows, "IndirectCapsuleShadows");

			FRDGTextureRef NewBentNormal = nullptr;
			AllocateOrReuseAORenderTarget(GraphBuilder, View, NewBentNormal, TEXT("CapsuleBentNormal"), PF_FloatRGBA);

			int32 NumCapsuleShapes = 0;
			int32 NumMeshesWithCapsules = 0;
			int32 NumMeshDistanceFieldCasters = 0;
			IndirectCapsuleShadowsResources Resources = CreateIndirectCapsuleShadowsResources(GraphBuilder, Scene, View, NumCapsuleShapes, NumMeshesWithCapsules, NumMeshDistanceFieldCasters);

			// Don't render indirect occlusion from mesh distance fields when operating on a movable skylight,
			// DFAO is responsible for indirect occlusion from meshes with distance fields on a movable skylight.
			// A single mesh should only provide indirect occlusion for a given lighting component in one way.
			NumMeshDistanceFieldCasters = 0;

			if (NumCapsuleShapes > 0)
			{
				check(Resources.IndirectShadowLightDirectionSRV);

				FIntRect ScissorRect = View.ViewRect;

				{
					uint32 GroupSizeX = FMath::DivideAndRoundUp(ScissorRect.Size().X / GAODownsampleFactor, GShadowShapeTileSize);
					uint32 GroupSizeY = FMath::DivideAndRoundUp(ScissorRect.Size().Y / GAODownsampleFactor, GShadowShapeTileSize);

					auto* PassParameters = GraphBuilder.AllocParameters<FCapsuleShadowingCS::FParameters>();

					SetupCapsuleShadowingParameters(
						GraphBuilder,
						*PassParameters,
						ECapsuleShadowingType::MovableSkylightTiledCulling,
						GraphBuilder.CreateUAV(NewBentNormal),
						FIntPoint(GroupSizeX, GroupSizeY),
						BentNormalOutput,
						FVector2D(GroupSizeX, GroupSizeY),
						nullptr,
						ScissorRect,
						GAODownsampleFactor,
						GCapsuleMaxIndirectOcclusionDistance,
						Scene,
						View,
						0, // Not used in stereo rendering
						NumCapsuleShapes,
						Resources.IndirectShadowCapsuleShapesSRV,
						NumMeshDistanceFieldCasters,
						Resources.IndirectShadowMeshDistanceFieldCasterIndicesSRV,
						Resources.IndirectShadowLightDirectionSRV,
						nullptr
					);

					FCapsuleShadowingCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FCapsuleShadowingCS::FShapeShadow>(ECapsuleShadowingType::MovableSkylightTiledCulling);
					PermutationVector.Set<FCapsuleShadowingCS::FIndirectPrimitiveType>(EIndirectShadowingPrimitiveTypes::CapsuleShapes);
					auto ComputeShader = View.ShaderMap->GetShader<FCapsuleShadowingCS>(PermutationVector);

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("TiledCapsuleShadowing % u capsules among % u meshes", NumCapsuleShapes, NumMeshesWithCapsules),
						ERDGPassFlags::Compute,
						ComputeShader,
						PassParameters,
						FIntVector(GroupSizeX, GroupSizeY, 1));
				}

				// Replace the pipeline output with our output that has capsule shadows applied
				BentNormalOutput = NewBentNormal;
			}
		}
	}
}
