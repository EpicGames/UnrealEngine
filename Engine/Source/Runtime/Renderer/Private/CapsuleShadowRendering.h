// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CapsuleShadowRendering.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderUtils.h"

extern int32 GCapsuleShadows;
extern int32 GCapsuleDirectShadows;
extern int32 GCapsuleIndirectShadows;

inline bool IsCapsuleShadowsEnabled(FStaticShaderPlatform ShaderPlatform)
{
	return GCapsuleShadows && (!IsMobilePlatform(ShaderPlatform) || IsMobileCapsuleShadowsEnabled(ShaderPlatform));
}

inline bool IsCapsuleDirectShadowsEnabled(FStaticShaderPlatform ShaderPlatform)
{
	return GCapsuleDirectShadows && IsCapsuleShadowsEnabled(ShaderPlatform) && (!IsMobilePlatform(ShaderPlatform) || IsMobileCapsuleDirectShadowsEnabled(ShaderPlatform));
}

inline bool SupportsCapsuleIndirectShadows(FStaticShaderPlatform ShaderPlatform)
{
	return GCapsuleIndirectShadows && IsCapsuleShadowsEnabled(ShaderPlatform) && !IsMobilePlatform(ShaderPlatform);
}
