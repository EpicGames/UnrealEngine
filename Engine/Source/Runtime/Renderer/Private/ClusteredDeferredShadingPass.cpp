// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ClusteredDeferredShadingPass.cpp: Implementation of tiled deferred shading
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "EngineGlobals.h"
#include "RHI.h"
#include "UniformBuffer.h"
#include "ShaderParameters.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "PostProcess/SceneRenderTargets.h"
#include "LightSceneInfo.h"
#include "GlobalShader.h"
#include "SceneRenderTargetParameters.h"
#include "BasePassRendering.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "ShaderPrintParameters.h"
#include "ShaderPrint.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "Substrate/Substrate.h"
#include "LightFunctionAtlas.h"
#include "AnisotropyRendering.h"
#include "HairStrands/HairStrandsTransmittance.h"

static FAutoConsoleVariableDeprecated CVarUseClusteredDeferredShadingDep(TEXT("r.UseClusteredDeferredShading"), TEXT("r.UseClusteredDeferredShading_ToBeRemoved"), TEXT("5.7"));

// This is used to switch on and off the clustered deferred shading implementation, that uses the light grid to perform shading.
int32 GUseClusteredDeferredShading = 0;
static FAutoConsoleVariableRef CVarUseClusteredDeferredShading(
	TEXT("r.UseClusteredDeferredShading_ToBeRemoved"),
	GUseClusteredDeferredShading,
	TEXT("NOTE: The clustered deferred shading implementation will be removed in a future release due to low utility and thus use.\n")
	TEXT("Toggle use of clustered deferred shading for lights that support it. 0 is off (default), 1 is on (also required is SM5 to actually turn on)."),
	ECVF_RenderThreadSafe
);

// The project setting (disable runtime and shader code)
static TAutoConsoleVariable<int32> CVarClusteredDeferredShadingEnableForProject(
	TEXT("r.ClusteredDeferredShading.EnableForProject"),
	1,
	TEXT("Enables Clustered Deferred Shading rendering and shader code."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

DECLARE_GPU_STAT_NAMED(ClusteredShading, TEXT("Clustered Shading"));

using namespace LightFunctionAtlas;

bool FDeferredShadingSceneRenderer::ShouldUseClusteredDeferredShading(EShaderPlatform InPlatform) const
{
	// The feature level is the same as in the shader compile conditions below, maybe we don't need SM5?
	// NOTE: should also take into account the conditions for building the light grid, since these 
	//       shaders might have another feature level.
	return CVarClusteredDeferredShadingEnableForProject.GetValueOnAnyThread() > 0 && GUseClusteredDeferredShading != 0 && Scene->GetFeatureLevel() >= ERHIFeatureLevel::SM6 && DoesPlatformSupportVirtualShadowMaps(InPlatform);
}

bool FDeferredShadingSceneRenderer::AreLightsInLightGrid() const
{
	return bAreLightsInLightGrid;
}


/**
 * Clustered deferred shading shader. Use a custom vertex shader for hair strands lighting, to covered all sample in sample space
 */
class FClusteredShadingVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClusteredShadingVS);
	SHADER_USE_PARAMETER_STRUCT(FClusteredShadingVS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && CVarClusteredDeferredShadingEnableForProject.GetValueOnAnyThread() > 0;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FClusteredShadingVS, "/Engine/Private/ClusteredDeferredShadingVertexShader.usf", "ClusteredShadingVertexShader", SF_Vertex);

/**
 * Clustered deferred shading shader, used in a full-screen pass to apply all lights in the light grid.
 */
class FClusteredShadingPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClusteredShadingPS);
	SHADER_USE_PARAMETER_STRUCT(FClusteredShadingPS, FGlobalShader)

	class FVisualizeLightCullingDim : SHADER_PERMUTATION_BOOL("VISUALIZE_LIGHT_CULLING");
	class FHairStrandsLighting : SHADER_PERMUTATION_BOOL("USE_HAIR_LIGHTING");
	class FLightFunctionAtlasDim : SHADER_PERMUTATION_BOOL("USE_LIGHT_FUNCTION_ATLAS");
	class FRectLight : SHADER_PERMUTATION_BOOL("USE_RECT_LIGHT");
	class FAnistropicMaterials : SHADER_PERMUTATION_BOOL("SUPPORTS_ANISOTROPIC_MATERIALS");
	using FPermutationDomain = TShaderPermutationDomain<FVisualizeLightCullingDim, FHairStrandsLighting, Substrate::FSubstrateTileType, FLightFunctionAtlasDim, FRectLight, FAnistropicMaterials>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLightFunctionAtlasGlobalParameters, LightFunctionAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneLightingChannelParameters, LightingChannelParameters)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowMaskBits)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, LightingChannelsTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HairTransmittanceBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (!Substrate::ShouldCompileSubstrateTileTypePermutations(PermutationVector.Get<Substrate::FSubstrateTileType>(), Parameters.Platform))
		{
			return false;
		}
		else if (Substrate::IsSubstrateEnabled() && PermutationVector.Get<FAnistropicMaterials>())
		{
			return false;
		}

		// Cluster shading requires VSM support
		if (!DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			return false;
		}

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM6) && CVarClusteredDeferredShadingEnableForProject.GetValueOnAnyThread() > 0;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
        OutEnvironment.SetDefine(TEXT("SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE"), PlatformSupportsSeparatedSubsurfaceDiffuse(Parameters.Platform) ? 1 : 0);
		// Occupancy is very poor on this shader - this helps a bit in the mean time
		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
	}

	static EShaderCompileJobPriority GetOverrideJobPriority()
	{
		// FClusteredShadingPS takes up to 11s on average
		return EShaderCompileJobPriority::ExtraHigh;
	}
};

IMPLEMENT_GLOBAL_SHADER(FClusteredShadingPS, "/Engine/Private/ClusteredDeferredShadingPixelShader.usf", "ClusteredShadingPixelShader", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FClusteredShadingParameters, )
	SHADER_PARAMETER_STRUCT(FClusteredShadingPS::FParameters, PS)
	SHADER_PARAMETER_STRUCT(Substrate::FSubstrateTilePassVS::FParameters, VS)
END_SHADER_PARAMETER_STRUCT()

enum class EClusterPassInputType : uint8
{
	GBuffer,
	Substrate,
	HairStrands
};

static void InternalAddClusteredDeferredShadingPass(
	FRDGBuilder& GraphBuilder,
	int32 ViewIndex,
	FViewInfo& View,
	const FMinimalSceneTextures& SceneTextures,
	const FSortedLightSetSceneInfo& SortedLightsSet,
	EClusterPassInputType InputType,
	ESubstrateTileType TileType,
	FRDGTextureRef LightingChannelsTexture,
	FRDGTextureRef ShadowMaskBits,
	FVirtualShadowMapArray& VirtualShadowMapArray,
	FRDGBufferSRVRef HairTransmittanceBuffer,
	FSubstrateSceneData* SubstrateSceneData)
{
	check(SortedLightsSet.ClusteredSupportedEnd > 0);
	const FIntPoint SceneTextureExtent = SceneTextures.Config.Extent;
	const bool bHairStrands = InputType == EClusterPassInputType::HairStrands;
	const bool bSubstrate = Substrate::IsSubstrateEnabled() && !bHairStrands;
	const bool bHasRectLights = SortedLightsSet.bHasRectLights;
	const bool bLightFunctionAtlas = LightFunctionAtlas::IsEnabled(View, ELightFunctionAtlasSystem::DeferredLighting);	
	const bool bSupportAnisotropyPermutation = ShouldRenderAnisotropyPass(View) && !Substrate::IsSubstrateEnabled(); // Strata managed anisotropy differently than legacy path. No need for special permutation.

	FClusteredShadingParameters* PassParameters = GraphBuilder.AllocParameters<FClusteredShadingParameters>();
	FClusteredShadingPS::FParameters* PSParameters = &PassParameters->PS;
	PSParameters->View = View.ViewUniformBuffer;
	PSParameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
	PSParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	PSParameters->SceneTextures = SceneTextures.UniformBuffer;
	PSParameters->ShadowMaskBits = ShadowMaskBits ? ShadowMaskBits : GSystemTextures.GetZeroUIntDummy(GraphBuilder);
	PSParameters->LightingChannelParameters = GetSceneLightingChannelParameters(GraphBuilder, View, LightingChannelsTexture);
	PSParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
	PSParameters->HairTransmittanceBuffer = HairTransmittanceBuffer;
	PSParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PSParameters->LightFunctionAtlas = LightFunctionAtlas::BindGlobalParameters(GraphBuilder, View);
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PSParameters->ShaderPrintUniformBuffer);

	PSParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
	if (bHairStrands)
	{
		PSParameters->RenderTargets[0] = FRenderTargetBinding(View.HairStrandsViewData.VisibilityData.SampleLightingTexture, ERenderTargetLoadAction::ELoad);
	}
	if (Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()))
	{
		check(SubstrateSceneData);
		PSParameters->RenderTargets[1] = FRenderTargetBinding(SubstrateSceneData->SeparatedOpaqueRoughRefractionSceneColor, ERenderTargetLoadAction::ELoad);
		PSParameters->RenderTargets[2] = FRenderTargetBinding(SubstrateSceneData->SeparatedSubSurfaceSceneColor, ERenderTargetLoadAction::ELoad);
	}
	else if (UseSeparatedSubsurfaceDiffuse(View.GetShaderPlatform()) && !bHairStrands)
	{
		PSParameters->RenderTargets[1] = FRenderTargetBinding(View.GetSceneTextures().SeparatedSubsurfaceDiffuse, ERenderTargetLoadAction::ELoad);
	}

	// VS - Substrate tile parameters
	EPrimitiveType PrimitiveType = PT_TriangleList;
	if (Substrate::IsSubstrateEnabled())
	{
		PassParameters->VS = Substrate::SetTileParameters(GraphBuilder, View, TileType, PrimitiveType);
	}
	
	const TCHAR* TileTypeName = ToString(TileType);
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Light::ClusteredDeferredShading(%s,Lights:%d%s%s%s%s)", bHairStrands ? TEXT("HairStrands") : (bSubstrate ? TEXT("Substrate") : TEXT("GBuffer")), SortedLightsSet.ClusteredSupportedEnd, bSubstrate ? TEXT(",Tile:") : TEXT(""), bSubstrate ? TileTypeName : TEXT(""), bLightFunctionAtlas ? TEXT(",LFAtlas") : TEXT(""), bHasRectLights ? TEXT(",RectLight") : TEXT("")),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, &View, SceneTextureExtent, bHasRectLights, bLightFunctionAtlas, bHairStrands, bSubstrate, bSupportAnisotropyPermutation, TileType, PrimitiveType](FRDGAsyncTask, FRHICommandList& InRHICmdList)
	{
		TShaderMapRef<FClusteredShadingVS> HairVertexShader(View.ShaderMap);
		TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);

		Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
		VSPermutationVector.Set<Substrate::FSubstrateTilePassVS::FEnableDebug>(false);
		VSPermutationVector.Set<Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector>(true);
		TShaderMapRef<Substrate::FSubstrateTilePassVS> TileVertexShader(View.ShaderMap, VSPermutationVector);

		FClusteredShadingPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FClusteredShadingPS::FVisualizeLightCullingDim>(View.Family->EngineShowFlags.VisualizeLightCulling);
		PermutationVector.Set<FClusteredShadingPS::FHairStrandsLighting>(bHairStrands);
		PermutationVector.Set<Substrate::FSubstrateTileType>(bSubstrate ? TileType : 0);
		PermutationVector.Set<FClusteredShadingPS::FRectLight>(bHasRectLights);
		PermutationVector.Set<FClusteredShadingPS::FLightFunctionAtlasDim>(bLightFunctionAtlas);
		PermutationVector.Set<FClusteredShadingPS::FAnistropicMaterials>(bSupportAnisotropyPermutation);
		TShaderMapRef<FClusteredShadingPS> PixelShader(View.ShaderMap, PermutationVector);
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			InRHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			// Additive blend to accumulate lighting contributions.
			if (Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()))
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
			}
			else if (UseSeparatedSubsurfaceDiffuse(View.GetShaderPlatform()) && !bHairStrands)
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
			}
			else
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
			}

			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = bHairStrands ? HairVertexShader.GetVertexShader() : (bSubstrate ? TileVertexShader.GetVertexShader() : VertexShader.GetVertexShader());
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PrimitiveType;
			SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);
		}

		SetShaderParameters(InRHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

		if (bHairStrands)
		{
			InRHICmdList.SetViewport(0, 0, 0.0f, View.HairStrandsViewData.VisibilityData.SampleLightingViewportResolution.X, View.HairStrandsViewData.VisibilityData.SampleLightingViewportResolution.Y, 1.0f);

			FClusteredShadingVS::FParameters VertexParameters;
			VertexParameters.View = PassParameters->PS.View;
			VertexParameters.HairStrands = PassParameters->PS.HairStrands;
			VertexParameters.SceneTextures = PassParameters->PS.SceneTextures;
			SetShaderParameters(InRHICmdList, HairVertexShader, HairVertexShader.GetVertexShader(), VertexParameters);
			InRHICmdList.SetStreamSource(0, nullptr, 0);
			InRHICmdList.DrawPrimitive(0, 1, 1);
		}
		else if (bSubstrate)
		{
			InRHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
			SetShaderParameters(InRHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), PassParameters->VS);
			InRHICmdList.DrawPrimitiveIndirect(PassParameters->VS.TileIndirectBuffer->GetIndirectRHICallBuffer(), Substrate::TileTypeDrawIndirectArgOffset(TileType));
		}
		else
		{
			InRHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
			DrawRectangle(InRHICmdList, 0, 0, View.ViewRect.Width(), View.ViewRect.Height(),
				View.ViewRect.Min.X, View.ViewRect.Min.Y, View.ViewRect.Width(), View.ViewRect.Height(),
				FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()), SceneTextureExtent, VertexShader);
		}
	});
}


void FDeferredShadingSceneRenderer::AddClusteredDeferredShadingPass(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const FSortedLightSetSceneInfo &SortedLightsSet,
	FRDGTextureRef ShadowMaskBits,
	FRDGTextureRef HairStrandsShadowMaskBits,
	FRDGTextureRef LightingChannelsTexture)
{
	check(ShouldUseClusteredDeferredShading(ViewFamily.GetShaderPlatform()));

	const int32 NumLightsToRender = SortedLightsSet.ClusteredSupportedEnd;

	if (NumLightsToRender > 0)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, ClusteredShading, "ClusteredShading");

		for (int32 ViewIndex = 0, Num = Views.Num(); ViewIndex < Num; ViewIndex++)
		{
			FViewInfo& View = Views[ViewIndex];

			FSubstrateSceneData* SubstrateSceneData = nullptr;

			if (Substrate::IsSubstrateEnabled())
			{
				SubstrateSceneData = SceneTextures.SubstrateSceneData.Get();
				check(SubstrateSceneData);

				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplexSpecial))
				{
					InternalAddClusteredDeferredShadingPass(
						GraphBuilder,
						ViewIndex,
						View,
						SceneTextures,
						SortedLightsSet,
						EClusterPassInputType::Substrate,
						ESubstrateTileType::EComplexSpecial,
						LightingChannelsTexture,
						ShadowMaskBits,
						VirtualShadowMapArray,
						nullptr,
						SubstrateSceneData);
				}

				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplex))
				{
					InternalAddClusteredDeferredShadingPass(
						GraphBuilder,
						ViewIndex,
						View,
						SceneTextures,
						SortedLightsSet,
						EClusterPassInputType::Substrate,
						ESubstrateTileType::EComplex,
						LightingChannelsTexture,
						ShadowMaskBits,
						VirtualShadowMapArray,
						nullptr,
						SubstrateSceneData);
				}

				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESingle))
				{
					InternalAddClusteredDeferredShadingPass(
						GraphBuilder,
						ViewIndex,
						View,
						SceneTextures,
						SortedLightsSet,
						EClusterPassInputType::Substrate,
						ESubstrateTileType::ESingle,
						LightingChannelsTexture,
						ShadowMaskBits,
						VirtualShadowMapArray,
						nullptr,
						SubstrateSceneData);
				}

				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESimple))
				{
					InternalAddClusteredDeferredShadingPass(
						GraphBuilder,
						ViewIndex,
						View,
						SceneTextures,
						SortedLightsSet,
						EClusterPassInputType::Substrate,
						ESubstrateTileType::ESimple,
						LightingChannelsTexture,
						ShadowMaskBits,
						VirtualShadowMapArray,
						nullptr, 
						SubstrateSceneData);
				}
			}
			else
			{
				InternalAddClusteredDeferredShadingPass(
					GraphBuilder,
					ViewIndex,
					View,
					SceneTextures,
					SortedLightsSet,
					EClusterPassInputType::GBuffer,
					ESubstrateTileType::ECount,
					LightingChannelsTexture,
					ShadowMaskBits,
					VirtualShadowMapArray,
					nullptr,
					SubstrateSceneData);
			}

			if (HairStrands::HasViewHairStrandsData(View))
			{
				FHairStrandsTransmittanceMaskData TransmittanceMask = RenderHairStrandsOnePassTransmittanceMask(GraphBuilder, View, ViewIndex, HairStrandsShadowMaskBits, VirtualShadowMapArray);
				InternalAddClusteredDeferredShadingPass(
					GraphBuilder,
					ViewIndex,
					View,
					SceneTextures,
					SortedLightsSet,
					EClusterPassInputType::HairStrands,
					ESubstrateTileType::ECount,
					LightingChannelsTexture,
					HairStrandsShadowMaskBits,
					VirtualShadowMapArray,
					GraphBuilder.CreateSRV(TransmittanceMask.TransmittanceMask, FHairStrandsTransmittanceMaskData::Format),
					SubstrateSceneData);
			}
		}
	}
}
