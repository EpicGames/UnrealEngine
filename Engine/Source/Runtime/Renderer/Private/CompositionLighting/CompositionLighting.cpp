// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CompositionLighting.cpp: The center for all deferred lighting activities.
=============================================================================*/

#include "CompositionLighting/CompositionLighting.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "PostProcess/PostProcessing.h"
#include "CompositionLighting/PostProcessAmbientOcclusion.h"
#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "DecalRenderingShared.h"
#include "VisualizeTexture.h"
#include "RayTracing/RaytracingOptions.h"
#include "SceneTextureParameters.h"
#include "RenderGraphUtils.h"
#include "SceneViewState.h"

DECLARE_GPU_STAT_NAMED(CompositionBeforeBasePass, TEXT("Composition BeforeBasePass") );
DECLARE_GPU_STAT_NAMED(CompositionPreLighting, TEXT("Composition PreLighting") );
DECLARE_GPU_STAT_NAMED(CompositionPostLighting, TEXT("Composition PostLighting") );

static TAutoConsoleVariable<int32> CVarSSAOSmoothPass(
	TEXT("r.AmbientOcclusion.Compute.Smooth"),
	1,
	TEXT("Whether to smooth SSAO output when TAA is disabled"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarGTAODownsample(
	TEXT("r.GTAO.Downsample"),
	0,
	TEXT("Perform GTAO at Halfres \n ")
	TEXT("0: Off \n ")
	TEXT("1: On (default)\n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarGTAOTemporalFilter(
	TEXT("r.GTAO.TemporalFilter"),
	1,
	TEXT("Enable Temporal Filter for GTAO \n ")
	TEXT("0: Off \n ")
	TEXT("1: On (default)\n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarGTAOSpatialFilter(
	TEXT("r.GTAO.SpatialFilter"),
	1,
	TEXT("Enable Spatial Filter for GTAO \n ")
	TEXT("0: Off \n ")
	TEXT("1: On (default)\n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

bool IsAmbientCubemapPassRequired(const FSceneView& View)
{
	return View.FinalPostProcessSettings.ContributingCubemaps.Num() != 0 && IsUsingGBuffers(View.GetShaderPlatform());
}

static bool IsReflectionEnvironmentActive(const FSceneView& View)
{
	FScene* Scene = (FScene*)View.Family->Scene;

	// LPV & Screenspace Reflections : Reflection Environment active if either LPV (assumed true if this was called), Reflection Captures or SSR active

	bool IsReflectingEnvironment = View.Family->EngineShowFlags.ReflectionEnvironment;
	bool HasReflectionCaptures = (Scene->ReflectionSceneData.RegisteredReflectionCaptures.Num() > 0);
	bool HasSSR = View.Family->EngineShowFlags.ScreenSpaceReflections;

	return (Scene->GetFeatureLevel() >= ERHIFeatureLevel::SM5 && IsReflectingEnvironment && (HasReflectionCaptures || HasSSR) && !IsForwardShadingEnabled(View.GetShaderPlatform()));
}

static bool IsSkylightActive(const FViewInfo& View)
{
	FScene* Scene = (FScene*)View.Family->Scene;
	return Scene->SkyLight 
		&& Scene->SkyLight->ProcessedTexture
		&& View.Family->EngineShowFlags.SkyLighting;
}

bool ShouldRenderScreenSpaceAmbientOcclusion(const FViewInfo& View, bool bLumenWantsSSAO)
{
	bool bEnabled = true;

	bEnabled = View.FinalPostProcessSettings.AmbientOcclusionIntensity > 0
		&& View.Family->EngineShowFlags.Lighting
		&& View.FinalPostProcessSettings.AmbientOcclusionRadius >= 0.1f
		&& !View.Family->UseDebugViewPS()
		&& (FSSAOHelper::GetNumAmbientOcclusionLevels() != 0)
		&& (FSSAOHelper::IsBasePassAmbientOcclusionRequired(View) || IsAmbientCubemapPassRequired(View) || IsReflectionEnvironmentActive(View) || IsSkylightActive(View) || IsForwardShadingEnabled(View.GetShaderPlatform()) || View.Family->EngineShowFlags.VisualizeBuffer || bLumenWantsSSAO);

	bEnabled &= !ShouldRenderRayTracingAmbientOcclusion(View);

	return bEnabled;
}

static ESSAOType GetDownscaleSSAOType(const FViewInfo& View)
{
	return FSSAOHelper::IsAmbientOcclusionCompute() ? ESSAOType::ECS : ESSAOType::EPS;
}

static ESSAOType GetFullscreenSSAOType(const FViewInfo& View, uint32 Levels)
{
	if (FSSAOHelper::IsAmbientOcclusionCompute())
	{
		if (FSSAOHelper::IsAmbientOcclusionAsyncCompute(View, Levels))
		{
			return ESSAOType::EAsyncCS;
		}

		return ESSAOType::ECS;
	}

	return ESSAOType::EPS;
}

FSSAOCommonParameters GetSSAOCommonParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	uint32 Levels,
	bool bAllowGBufferRead)
{
	const FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, View);

	FSSAOCommonParameters CommonParameters;
	CommonParameters.SceneTextures = GetSceneTextureShaderParameters(View);
	CommonParameters.SceneTexturesViewport = FScreenPassTextureViewport(SceneTextureParameters.SceneDepthTexture, View.ViewRect);

	CommonParameters.GBufferA = bAllowGBufferRead ? FScreenPassTexture(SceneTextureParameters.GBufferATexture, View.ViewRect) : FScreenPassTexture();
	CommonParameters.SceneDepth = FScreenPassTexture(SceneTextureParameters.SceneDepthTexture, View.ViewRect);

	CommonParameters.Levels = Levels;
	CommonParameters.ShaderQuality = FSSAOHelper::GetAmbientOcclusionShaderLevel(View);
	CommonParameters.DownscaleType = GetDownscaleSSAOType(View);
	CommonParameters.FullscreenType = GetFullscreenSSAOType(View, Levels);

	// If there is no temporal upsampling, we need a smooth pass to get rid of the grid pattern.
	// Pixel shader version has relatively smooth result so no need to do extra work.
	CommonParameters.bNeedSmoothingPass = CommonParameters.FullscreenType != ESSAOType::EPS && !IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod) && CVarSSAOSmoothPass.GetValueOnRenderThread();

	return CommonParameters;
}

FGTAOCommonParameters GetGTAOCommonParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	EGTAOType GTAOType
	)
{
	const FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, View);
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
	FRDGTextureRef VelocityTexture = GetIfProduced(SceneTextureParameters.GBufferVelocityTexture, SystemTextures.Black);

	FGTAOCommonParameters CommonParameters;
	CommonParameters.SceneTextures = GetSceneTextureShaderParameters(View);
	CommonParameters.SceneTexturesViewport = FScreenPassTextureViewport(SceneTextureParameters.SceneDepthTexture, View.ViewRect);

	CommonParameters.SceneDepth = FScreenPassTexture(SceneTextureParameters.SceneDepthTexture, View.ViewRect);
	CommonParameters.SceneVelocity = FScreenPassTexture(VelocityTexture, View.ViewRect);

	CommonParameters.ShaderQuality = FSSAOHelper::GetAmbientOcclusionShaderLevel(View);
	CommonParameters.DownscaleFactor = CVarGTAODownsample.GetValueOnRenderThread() > 0 ? 2 : 1;
	CommonParameters.GTAOType = GTAOType;

	CommonParameters.DownsampledViewRect = GetDownscaledRect(View.ViewRect, CommonParameters.DownscaleFactor);

	return CommonParameters;
}

// Async Passes of the GTAO.
// This can either just be the Horizon search if GBuffer Normals are needed or it can be
// Combined Horizon search and Integrate followed by the Spatial filter if no normals are needed
static FGTAOHorizonSearchOutputs AddPostProcessingGTAOAsyncPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassRenderTarget GTAOHorizons
	)
{
	check(CommonParameters.GTAOType == EGTAOType::EAsyncHorizonSearch || CommonParameters.GTAOType == EGTAOType::EAsyncCombinedSpatial);

	const bool bSpatialPass = (CVarGTAOSpatialFilter.GetValueOnRenderThread() == 1);

	FGTAOHorizonSearchOutputs HorizonSearchOutputs;

	if (CommonParameters.GTAOType == EGTAOType::EAsyncHorizonSearch)
	{
		HorizonSearchOutputs =
			AddGTAOHorizonSearchPass(
				GraphBuilder,
				View,
				CommonParameters,
				CommonParameters.SceneDepth,
				GTAOHorizons);
	}
	else
	{
		HorizonSearchOutputs =
			AddGTAOHorizonSearchIntegratePass(
				GraphBuilder,
				View,
				CommonParameters,
				CommonParameters.SceneDepth);

		if (bSpatialPass)
		{
			FScreenPassTexture SpatialOutput =
				AddGTAOSpatialFilter(
					GraphBuilder,
					View,
					CommonParameters,
					HorizonSearchOutputs.Color,
					CommonParameters.SceneDepth,
					GTAOHorizons);
		}
	}

	return MoveTemp(HorizonSearchOutputs);
}

// The whole GTAO stack is run on the Gfx Pipe
static FScreenPassTexture AddPostProcessingGTAOAllPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassRenderTarget FinalTarget)
{
	FSceneViewState* ViewState = View.ViewState;

	const bool bSpatialPass = (CVarGTAOSpatialFilter.GetValueOnRenderThread() == 1);
	const bool bTemporalPass = (ViewState && CVarGTAOTemporalFilter.GetValueOnRenderThread() == 1);

	{
		FGTAOHorizonSearchOutputs HorizonSearchOutputs =
			AddGTAOHorizonSearchIntegratePass(
				GraphBuilder,
				View,
				CommonParameters,
				CommonParameters.SceneDepth);

		FScreenPassTexture CurrentOutput = HorizonSearchOutputs.Color;
		if (bSpatialPass)
		{
			CurrentOutput =
				AddGTAOSpatialFilter(
					GraphBuilder,
					View,
					CommonParameters,
					HorizonSearchOutputs.Color,
					CommonParameters.SceneDepth);
		}

		if (bTemporalPass)
		{
			const FGTAOTAAHistory& InputHistory = View.PrevViewInfo.GTAOHistory;
			FGTAOTAAHistory* OutputHistory = &View.ViewState->PrevFrameViewInfo.GTAOHistory;

			FScreenPassTextureViewport HistoryViewport(InputHistory.ReferenceBufferSize, InputHistory.ViewportRect);

			FScreenPassTexture HistoryColor;

			if (InputHistory.IsValid())
			{
				HistoryColor = FScreenPassTexture(GraphBuilder.RegisterExternalTexture(InputHistory.RT, TEXT("GTAOHistoryColor")), HistoryViewport.Rect);
			}
			else
			{
				HistoryColor = FScreenPassTexture(GraphBuilder.RegisterExternalTexture(GSystemTextures.WhiteDummy, TEXT("GTAODummyTexture")));
				HistoryViewport = FScreenPassTextureViewport(HistoryColor);
			}

			FGTAOTemporalOutputs TemporalOutputs =
				AddGTAOTemporalPass(
					GraphBuilder,
					View,
					CommonParameters,
					CurrentOutput,
					CommonParameters.SceneDepth,
					CommonParameters.SceneVelocity,
					HistoryColor,
					HistoryViewport);

			OutputHistory->SafeRelease();
			GraphBuilder.QueueTextureExtraction(TemporalOutputs.OutputAO.Texture, &OutputHistory->RT);

			OutputHistory->ReferenceBufferSize = TemporalOutputs.TargetExtent;
			OutputHistory->ViewportRect = TemporalOutputs.ViewportRect;

			CurrentOutput = TemporalOutputs.OutputAO;
		}

		FScreenPassTexture FinalOutput = CurrentOutput;
		// TODO: Can't switch outputs since it's an external texture. Won't be a problem when we're fully over to RDG.
		//if (DownsampleFactor > 1)
		{
			FinalOutput =
				AddGTAOUpsamplePass(
					GraphBuilder,
					View,
					CurrentOutput,
					CommonParameters.SceneDepth,
					FinalTarget);
		}
	}

	return MoveTemp(FinalTarget);
}

// These are the passes run after Async where some are run before on the Async pipe
static FScreenPassTexture AddPostProcessingGTAOPostAsync(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture GTAOHorizons,
	FScreenPassRenderTarget FinalTarget)
{
	FSceneViewState* ViewState = View.ViewState;

	const bool bSpatialPass = (CVarGTAOSpatialFilter.GetValueOnRenderThread() == 1);
	const bool bTemporalPass = (ViewState && CVarGTAOTemporalFilter.GetValueOnRenderThread() == 1);

	{
		FScreenPassTexture CurrentOutput;

		if (CommonParameters.GTAOType == EGTAOType::EAsyncHorizonSearch)
		{
			CurrentOutput =
				AddGTAOInnerIntegratePass(
					GraphBuilder,
					View,
					CommonParameters,
					CommonParameters.SceneDepth,
					GTAOHorizons);

			if (bSpatialPass)
			{
				CurrentOutput =
					AddGTAOSpatialFilter(
						GraphBuilder,
						View,
						CommonParameters,
						CommonParameters.SceneDepth,
						CurrentOutput);
			}
		}
		else
		{
			// If the Spatial Filter is running as part of the async then we'll render to the R channel of the horizons texture so it can be read in as part of the temporal
			CurrentOutput = GTAOHorizons;
		}

		if (bTemporalPass)
		{
			const FGTAOTAAHistory& InputHistory = View.PrevViewInfo.GTAOHistory;
			FGTAOTAAHistory* OutputHistory = &ViewState->PrevFrameViewInfo.GTAOHistory;

			FScreenPassTextureViewport HistoryViewport(InputHistory.ReferenceBufferSize, InputHistory.ViewportRect);

			FScreenPassTexture HistoryColor;

			if (InputHistory.IsValid())
			{
				HistoryColor = FScreenPassTexture(GraphBuilder.RegisterExternalTexture(InputHistory.RT, TEXT("GTAOHistoryColor")), HistoryViewport.Rect);
			}
			else
			{
				HistoryColor = FScreenPassTexture(GraphBuilder.RegisterExternalTexture(GSystemTextures.WhiteDummy, TEXT("GTAODummyTexture")));
				HistoryViewport = FScreenPassTextureViewport(HistoryColor);
			}

			FGTAOTemporalOutputs TemporalOutputs =
				AddGTAOTemporalPass(
					GraphBuilder,
					View,
					CommonParameters,
					CurrentOutput,
					CommonParameters.SceneDepth,
					CommonParameters.SceneVelocity,
					HistoryColor,
					HistoryViewport);

			OutputHistory->SafeRelease();
			GraphBuilder.QueueTextureExtraction(TemporalOutputs.OutputAO.Texture, &OutputHistory->RT);

			OutputHistory->ReferenceBufferSize = TemporalOutputs.TargetExtent;
			OutputHistory->ViewportRect = TemporalOutputs.ViewportRect;

			CurrentOutput = TemporalOutputs.OutputAO;
		}

		FScreenPassTexture FinalOutput = CurrentOutput;

		// TODO: Can't switch outputs since it's an external texture. Won't be a problem when we're fully over to RDG.
		//if (DownsampleFactor > 1)
		{
			FinalOutput =
				AddGTAOUpsamplePass(
					GraphBuilder,
					View,
					CurrentOutput,
					CommonParameters.SceneDepth,
					FinalTarget);
		}
	}

	return MoveTemp(FinalTarget);
}

// @param Levels 0..3, how many different resolution levels we want to render
static FScreenPassTexture AddPostProcessingAmbientOcclusion(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	FScreenPassRenderTarget FinalTarget)
{
	check(CommonParameters.Levels >= 0 && CommonParameters.Levels <= 3);

	FScreenPassTexture AmbientOcclusionInMip1;
	FScreenPassTexture AmbientOcclusionPassMip1;
	if (CommonParameters.Levels >= 2)
	{
		AmbientOcclusionInMip1 =
			AddAmbientOcclusionSetupPass(
				GraphBuilder,
				View,
				CommonParameters,
				CommonParameters.SceneDepth);

		FScreenPassTexture AmbientOcclusionPassMip2;
		if (CommonParameters.Levels >= 3)
		{
			FScreenPassTexture AmbientOcclusionInMip2 =
				AddAmbientOcclusionSetupPass(
					GraphBuilder,
					View,
					CommonParameters,
					AmbientOcclusionInMip1);

			AmbientOcclusionPassMip2 =
				AddAmbientOcclusionStepPass(
					GraphBuilder,
					View,
					CommonParameters,
					AmbientOcclusionInMip2,
					AmbientOcclusionInMip2,
					FScreenPassTexture());
		}

		AmbientOcclusionPassMip1 =
			AddAmbientOcclusionStepPass(
				GraphBuilder,
				View,
				CommonParameters,
				AmbientOcclusionInMip1,
				AmbientOcclusionInMip1,
				AmbientOcclusionPassMip2);
	}

	FScreenPassTexture SetupTexture = CommonParameters.GBufferA;
	if (Substrate::IsSubstrateEnabled())
	{
		// For Substrate, we invalidate the setup texture for the final pass:
		//	- We do not need GBufferA, the Substrate TopLayer texture will fill in for that.
		//	- Setting it to nullptr will make the AddAmbientOcclusionPass use a valid viewport from SceneTextures.
		SetupTexture.Texture = nullptr;
	}

	FScreenPassTexture FinalOutput =
		AddAmbientOcclusionFinalPass(
			GraphBuilder,
			View,
			CommonParameters,
			SetupTexture,
			AmbientOcclusionInMip1,
			AmbientOcclusionPassMip1,
			FinalTarget);

	return FinalOutput;
}

FCompositionLighting::FCompositionLighting(FDecalVisibilityTaskData* InDecalVisibility, TArrayView<FViewInfo> InViews, const FSceneTextures& InSceneTextures, TUniqueFunction<bool(int32)> RequestSSAOFunction)
	: Views(InViews)
	, ViewFamily(*InViews[0].Family)
	, SceneTextures(InSceneTextures)
	, DecalVisibility(InDecalVisibility)
{
	ViewAOConfigs.SetNum(Views.Num());

	for (int32 Index = 0; Index < Views.Num(); ++Index)
	{
		ViewAOConfigs[Index].bRequested = RequestSSAOFunction(Index);
	}
}

FCompositionLighting::~FCompositionLighting()
{
	const int32 DecalViewOffset = DecalVisibility ? DecalVisibility->GetViewOffset(Views) : INDEX_NONE;

	if (DecalViewOffset != INDEX_NONE)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			DecalVisibility->Finish(DecalViewOffset, ViewIndex);
		}
	}
}

void FCompositionLighting::TryInit()
{
	if (bInitialized)
	{
		return;
	}

	const bool bForwardShading = IsForwardShadingEnabled(SceneTextures.Config.ShaderPlatform);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		FAOConfig& ViewConfig = ViewAOConfigs[ViewIndex];

		if (!ViewConfig.bRequested)
		{
			continue;
		}

		ViewConfig.Levels = FSSAOHelper::ComputeAmbientOcclusionPassCount(View);

		if (!bForwardShading)
		{
			ViewConfig.GTAOType = FSSAOHelper::GetGTAOPassType(View, ViewConfig.Levels);
		}

		if (ViewConfig.GTAOType == EGTAOType::EOff && ViewConfig.Levels > 0)
		{
			ViewConfig.bSSAOAsync = FSSAOHelper::IsAmbientOcclusionAsyncCompute(View, ViewConfig.Levels);

			ViewConfig.SSAOLocation = IsHZBValid(View, EHZBType::FurthestHZB) && (ViewConfig.bSSAOAsync || bForwardShading)
				? ESSAOLocation::BeforeBasePass
				: ESSAOLocation::AfterBasePass;
		}
	}

	bInitialized = true;
}

void FCompositionLighting::ProcessBeforeBasePass(FRDGBuilder& GraphBuilder, FDBufferTextures& DBufferTextures, FInstanceCullingManager& InstanceCullingManager, const FSubstrateSceneData& SubstrateSceneData)
{
	if (HasRayTracedOverlay(ViewFamily))
	{
		return;
	}

	TryInit();

	RDG_EVENT_SCOPE_STAT(GraphBuilder, CompositionBeforeBasePass, "CompositionBeforeBasePass");

	const int32 DecalViewOffset = DecalVisibility ? DecalVisibility->GetViewOffset(Views) : INDEX_NONE;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];
		const FAOConfig& ViewConfig = ViewAOConfigs[ViewIndex];

		const bool bRunDbufferBeforeBasePass = !Substrate::IsDBufferPassEnabled(View.GetShaderPlatform());
		const bool bEnableSSAO = ViewConfig.SSAOLocation == ESSAOLocation::BeforeBasePass;
		const bool bEnableDecals = bRunDbufferBeforeBasePass && DecalViewOffset != INDEX_NONE && DecalVisibility->HasStage(DecalViewOffset, ViewIndex, EDecalRenderStage::BeforeBasePass);

		if (!bEnableDecals && !bEnableSSAO)
		{
			continue;
		}

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		View.BeginRenderView();

		// decals are before AmbientOcclusion so the decal can output a normal that AO is affected by
		if (bEnableDecals)
		{
			FDeferredDecalPassTextures DecalPassTextures = GetDeferredDecalPassTextures(GraphBuilder, View, SceneTextures, &DBufferTextures, EDecalRenderStage::BeforeBasePass);
			TConstArrayView<const FVisibleDecal*> SortedDecals = DecalVisibility->FinishRelevantDecals(DecalViewOffset, ViewIndex, EDecalRenderStage::BeforeBasePass);
			AddDeferredDecalPass(GraphBuilder, View, SortedDecals, DecalPassTextures, InstanceCullingManager, EDecalRenderStage::BeforeBasePass);
		}

		if (bEnableSSAO)
		{
			FSSAOCommonParameters Parameters = GetSSAOCommonParameters(GraphBuilder, View, ViewConfig.Levels, false);
			FScreenPassRenderTarget FinalTarget = FScreenPassRenderTarget(SceneTextures.ScreenSpaceAO, View.ViewRect, ERenderTargetLoadAction::ENoAction);

			AddPostProcessingAmbientOcclusion(
				GraphBuilder,
				View,
				Parameters,
				FinalTarget);
		}
	}
}

void FCompositionLighting::ProcessAfterBasePass(FRDGBuilder& GraphBuilder, FInstanceCullingManager& InstanceCullingManager, EProcessAfterBasePassMode Mode, const FSubstrateSceneData& SubstrateSceneData)
{
	if (HasRayTracedOverlay(ViewFamily))
	{
		return;
	}

	check(bInitialized);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, CompositionPreLighting, "LightCompositionTasks_PreLighting");

	const int32 DecalViewOffset = DecalVisibility ? DecalVisibility->GetViewOffset(Views) : INDEX_NONE;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];
		const FAOConfig& ViewConfig = ViewAOConfigs[ViewIndex];

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		View.BeginRenderView();

		const auto AddDeferredDecalPassLambda = [&] (const FDeferredDecalPassTextures& DecalPassTextures, EDecalRenderStage Stage)
		{
			AddDeferredDecalPass(GraphBuilder, View, DecalVisibility->FinishRelevantDecals(DecalViewOffset, ViewIndex, Stage), DecalPassTextures, InstanceCullingManager, Stage);
		};

		if (DecalViewOffset != INDEX_NONE)
		{
			if (DecalVisibility->HasStage(DecalViewOffset, ViewIndex, EDecalRenderStage::BeforeLighting) && Mode != EProcessAfterBasePassMode::SkipBeforeLightingDecals)
			{
				// We can disable this pass if using DBuffer decals
				// Decals are before AmbientOcclusion so the decal can output a normal that AO is affected by
				FDeferredDecalPassTextures DecalPassTextures = GetDeferredDecalPassTextures(GraphBuilder, View, SceneTextures, nullptr, EDecalRenderStage::BeforeLighting);
				AddDeferredDecalPassLambda(DecalPassTextures, EDecalRenderStage::BeforeLighting);
			}

			if (DecalVisibility->HasStage(DecalViewOffset, ViewIndex, EDecalRenderStage::Emissive) && Mode != EProcessAfterBasePassMode::OnlyBeforeLightingDecals)
			{
				// DBuffer decals with emissive component
				FDeferredDecalPassTextures DecalPassTextures = GetDeferredDecalPassTextures(GraphBuilder, View, SceneTextures, nullptr, EDecalRenderStage::Emissive);
				AddDeferredDecalPassLambda(DecalPassTextures, EDecalRenderStage::Emissive);
			}
		}

		// Forward shading SSAO is applied before the base pass using only the depth buffer.
		if (!IsForwardShadingEnabled(View.GetShaderPlatform()) && Mode != EProcessAfterBasePassMode::OnlyBeforeLightingDecals)
		{
			if (ViewConfig.Levels > 0)
			{
				const bool bScreenSpaceAOIsProduced = SceneTextures.ScreenSpaceAO->HasBeenProduced();
				FScreenPassRenderTarget FinalTarget = FScreenPassRenderTarget(SceneTextures.ScreenSpaceAO, View.ViewRect, bScreenSpaceAOIsProduced ? ERenderTargetLoadAction::ELoad : ERenderTargetLoadAction::ENoAction);

				FScreenPassTexture AmbientOcclusion = bScreenSpaceAOIsProduced ? FinalTarget : FScreenPassTexture();

				// If doing the Split GTAO method then we need to do the second part here.
				if (ViewConfig.GTAOType == EGTAOType::EAsyncHorizonSearch || ViewConfig.GTAOType == EGTAOType::EAsyncCombinedSpatial)
				{
					check(HorizonsTexture);

					FGTAOCommonParameters Parameters = GetGTAOCommonParameters(GraphBuilder, View, ViewConfig.GTAOType);

					FScreenPassTexture GTAOHorizons(HorizonsTexture, Parameters.DownsampledViewRect);
					AmbientOcclusion = AddPostProcessingGTAOPostAsync(GraphBuilder, View, Parameters, GTAOHorizons, FinalTarget);

					ensureMsgf(
						DecalViewOffset == INDEX_NONE || !DecalVisibility->HasStage(DecalViewOffset, ViewIndex, EDecalRenderStage::AmbientOcclusion),
						TEXT("Ambient occlusion decals are not supported with Async compute SSAO."));
				}
				else
				{
					if (ViewConfig.GTAOType == EGTAOType::ENonAsync)
					{
						FGTAOCommonParameters Parameters = GetGTAOCommonParameters(GraphBuilder, View, ViewConfig.GTAOType);
						AmbientOcclusion = AddPostProcessingGTAOAllPasses(GraphBuilder, View, Parameters, FinalTarget);
					}
					else if (ViewConfig.SSAOLocation == ESSAOLocation::AfterBasePass)
					{
						FSSAOCommonParameters Parameters = GetSSAOCommonParameters(GraphBuilder, View, ViewConfig.Levels, true);
						AmbientOcclusion = AddPostProcessingAmbientOcclusion(GraphBuilder, View, Parameters, FinalTarget);
					}

					if (DecalViewOffset != INDEX_NONE)
					{
						FDeferredDecalPassTextures DecalPassTextures = GetDeferredDecalPassTextures(GraphBuilder, View, SceneTextures, nullptr, EDecalRenderStage::AmbientOcclusion);
						DecalPassTextures.ScreenSpaceAO = AmbientOcclusion.Texture;
						AddDeferredDecalPassLambda(DecalPassTextures, EDecalRenderStage::AmbientOcclusion);
					}
				}
			}
		}
	}
}

void FCompositionLighting::ProcessAfterOcclusion(FRDGBuilder& GraphBuilder)
{
	if (HasRayTracedOverlay(ViewFamily))
	{
		return;
	}

	TryInit();

	RDG_ASYNC_COMPUTE_BUDGET_SCOPE(GraphBuilder, FSSAOHelper::GetAmbientOcclusionAsyncComputeBudget());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];
		const FAOConfig& ViewConfig = ViewAOConfigs[ViewIndex];

		if (ViewConfig.GTAOType == EGTAOType::EAsyncCombinedSpatial || ViewConfig.GTAOType == EGTAOType::EAsyncHorizonSearch)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			FGTAOCommonParameters CommonParameters = GetGTAOCommonParameters(GraphBuilder, View, ViewConfig.GTAOType);

			const ERenderTargetLoadAction LoadAction = View.DecayLoadAction(ERenderTargetLoadAction::ENoAction);

			if (!HorizonsTexture)
			{
				const FIntPoint HorizonTextureSize = FIntPoint::DivideAndRoundUp(SceneTextures.Config.Extent, CommonParameters.DownscaleFactor);
				const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(HorizonTextureSize, PF_R8G8, FClearValueBinding::White, TexCreate_UAV | TexCreate_RenderTargetable);
				HorizonsTexture = GraphBuilder.CreateTexture(Desc, TEXT("ScreenSpaceGTAOHorizons"));
			}

			FScreenPassRenderTarget GTAOHorizons(HorizonsTexture, CommonParameters.DownsampledViewRect, LoadAction);

			AddPostProcessingGTAOAsyncPasses(
				GraphBuilder,
				View,
				CommonParameters,
				GTAOHorizons);
		}
	}
}