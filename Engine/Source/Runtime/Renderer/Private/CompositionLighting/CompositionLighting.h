// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CompositionLighting.h: The center for all deferred lighting activities.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "SceneRendering.h"
#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "CompositionLighting/PostProcessAmbientOcclusion.h"
#include "DecalRenderingShared.h"

/**
 * The center for all screen space processing activities (e.g. G-buffer manipulation, lighting).
 */
class FCompositionLighting
{
public:
	FCompositionLighting(FDecalVisibilityTaskData* InDecalVisibility, TArrayView<FViewInfo> InViews, const FSceneTextures& InSceneTextures, TUniqueFunction<bool(int32)> RequestSSAOFunction);
	~FCompositionLighting();

	void ProcessAfterOcclusion(FRDGBuilder& GraphBuilder);

	void ProcessBeforeBasePass(FRDGBuilder& GraphBuilder, FDBufferTextures& DBufferTextures, FInstanceCullingManager& InstanceCullingManager, const FSubstrateSceneData& SubstrateSceneData);

	enum class EProcessAfterBasePassMode
	{
		OnlyBeforeLightingDecals,
		SkipBeforeLightingDecals,
		All
	};

	void ProcessAfterBasePass(FRDGBuilder& GraphBuilder, FInstanceCullingManager& InstanceCullingManager, EProcessAfterBasePassMode Mode, const FSubstrateSceneData& SubstrateSceneData);

private:
	void TryInit();

	const TArrayView<FViewInfo> Views;
	const FSceneViewFamily& ViewFamily;
	const FSceneTextures& SceneTextures;

	enum class ESSAOLocation
	{
		None,
		BeforeBasePass,
		AfterBasePass
	};

	struct FAOConfig
	{
		uint32 Levels = 0;
		EGTAOType GTAOType = EGTAOType::EOff;
		ESSAOLocation SSAOLocation = ESSAOLocation::None;
		bool bSSAOAsync = false;
		bool bRequested = false;
	};

	FDecalVisibilityTaskData* DecalVisibility = nullptr;
	TArray<FAOConfig, TInlineAllocator<8>> ViewAOConfigs;
	FRDGTextureRef HorizonsTexture = nullptr;
	bool bInitialized = false;
};

extern bool ShouldRenderScreenSpaceAmbientOcclusion(const FViewInfo& View, bool bLumenWantsSSAO);
