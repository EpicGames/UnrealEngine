// Copyright Epic Games, Inc. All Rights Reserved.

#include "AmbientCubemapParameters.h"
#include "Engine/Texture.h"
#include "GlobalRenderResources.h"
#include "TextureResource.h"

void SetupAmbientCubemapParameters(const FFinalPostProcessSettings::FCubemapEntry& Entry, FAmbientCubemapParameters* OutParameters)
{
	// floats to render the cubemap
	{
		float MipCount = 0;

		if(Entry.AmbientCubemap)
		{
			int32 CubemapWidth = Entry.AmbientCubemap->GetSurfaceWidth();
			MipCount = FMath::Log2(static_cast<float>(CubemapWidth)) + 1.0f;
		}

		OutParameters->AmbientCubemapColor = Entry.AmbientCubemapTintMulScaleValue;

		OutParameters->AmbientCubemapMipAdjust.X =  1.0f - GDiffuseConvolveMipLevel / MipCount;
		OutParameters->AmbientCubemapMipAdjust.Y = (MipCount - 1.0f) * OutParameters->AmbientCubemapMipAdjust.X;
		OutParameters->AmbientCubemapMipAdjust.Z = MipCount - GDiffuseConvolveMipLevel;
		OutParameters->AmbientCubemapMipAdjust.W = MipCount;
	}

	// cubemap texture
	{
		FTexture* AmbientCubemapTexture = Entry.AmbientCubemap && Entry.AmbientCubemap->GetResource() ? Entry.AmbientCubemap->GetResource() : (FTexture*)GBlackTextureCube;
		OutParameters->AmbientCubemap = AmbientCubemapTexture->TextureRHI;
		OutParameters->AmbientCubemapSampler = AmbientCubemapTexture->SamplerStateRHI;
	}
}