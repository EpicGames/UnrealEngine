// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessAmbientOcclusion.cpp: Post processing ambient occlusion implementation.
=============================================================================*/

#include "CompositionLighting/PostProcessAmbientOcclusion.h"
#include "CompositionLighting/CompositionLighting.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PostProcess/SceneFilterRendering.h"
#include "SceneTextureParameters.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "ClearQuad.h"
#include "VariableRateShadingImageManager.h"
#include "SceneViewState.h"

DECLARE_GPU_STAT_NAMED(SSAOSetup, TEXT("ScreenSpace AO Setup") );
DECLARE_GPU_STAT_NAMED(SSAO, TEXT("ScreenSpace AO") );
DECLARE_GPU_STAT_NAMED(BasePassAO, TEXT("BasePass AO") );
DECLARE_GPU_STAT_NAMED(SSAOSmooth, TEXT("SSAO smooth"));
DECLARE_GPU_STAT_NAMED(GTAO_HorizonSearch,				TEXT("GTAO HorizonSearch"));
DECLARE_GPU_STAT_NAMED(GTAO_HorizonSearchIntegrate,		TEXT("GTAO HorizonSearch And Integrate"));
DECLARE_GPU_STAT_NAMED(GTAO_InnerIntegrate,				TEXT("GTAO InnerIntegrate"));
DECLARE_GPU_STAT_NAMED(GTAO_TemporalFilter,				TEXT("GTAO Temportal Filter"));
DECLARE_GPU_STAT_NAMED(GTAO_SpatialFilter,				TEXT("GTAO Spatial Filter"));
DECLARE_GPU_STAT_NAMED(GTAO_Upsample,					TEXT("GTAO Upsample"));

static TAutoConsoleVariable<int32> CVarAmbientOcclusionCompute(
	TEXT("r.AmbientOcclusion.Compute"),
	0,
	TEXT("If SSAO should use ComputeShader (not available on all platforms) or PixelShader.\n")
	TEXT("The [Async] Compute Shader version is WIP, not optimized, requires hardware support (not mobile/DX10/OpenGL3),\n")
	TEXT("does not use normals which allows it to run right after EarlyZPass (better performance when used with AyncCompute)\n")
	TEXT(" 0: PixelShader (default)\n")
	TEXT(" 1: (WIP) Use ComputeShader if possible, otherwise fall back to '0'\n")
	TEXT(" 2: (WIP) Use AsyncCompute if efficient, otherwise fall back to '1'\n")
	TEXT(" 3: (WIP) Use AsyncCompute if possible, otherwise fall back to '1'"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarAmbientOcclusionMaxQuality(
	TEXT("r.AmbientOcclusionMaxQuality"),
	100.0f,
	TEXT("Defines the max clamping value from the post process volume's quality level for ScreenSpace Ambient Occlusion\n")
	TEXT("     100: don't override quality level from the post process volume (default)\n")
	TEXT("   0..99: clamp down quality level from the post process volume to the maximum set by this cvar\n")
	TEXT(" -100..0: Enforces a different quality (the absolute value) even if the postprocessvolume asks for a lower quality."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarAmbientOcclusionStepMipLevelFactor(
	TEXT("r.AmbientOcclusionMipLevelFactor"),
	0.5f,
	TEXT("Controls mipmap level according to the SSAO step id\n")
	TEXT(" 0: always look into the HZB mipmap level 0 (memory cache trashing)\n")
	TEXT(" 0.5: sample count depends on post process settings (default)\n")
	TEXT(" 1: Go into higher mipmap level (quality loss)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAmbientOcclusionLevels(
	TEXT("r.AmbientOcclusionLevels"),
	-1,
	TEXT("Defines how many mip levels are using during the ambient occlusion calculation. This is useful when tweaking the algorithm.\n")
	TEXT("<0: decide based on the quality setting in the postprocess settings/volume and r.AmbientOcclusionMaxQuality (default)\n")
	TEXT(" 0: none (disable AmbientOcclusion)\n")
	TEXT(" 1: one\n")
	TEXT(" 2: two (costs extra performance, soft addition)\n")
	TEXT(" 3: three (larger radius cost less but can flicker)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAmbientOcclusionAsyncComputeBudget(
	TEXT("r.AmbientOcclusion.AsyncComputeBudget"),
	1,
	TEXT("Defines which level of EAsyncComputeBudget to use for balancing AsyncCompute work against Gfx work.\n")
	TEXT("Only matters if the compute version of SSAO is active (requires CS support, enabled by cvar, single pass, no normals)\n")
	TEXT("This is a low level developer tweak to get best performance on hardware that supports AsyncCompute.\n")
	TEXT(" 0: least AsyncCompute\n")
	TEXT(" 1: .. (default)\n")
	TEXT(" 2: .. \n")
	TEXT(" 3: .. \n")
	TEXT(" 4: most AsyncCompute"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAmbientOcclusionDepthBoundsTest(
	TEXT("r.AmbientOcclusion.DepthBoundsTest"),
	1,
	TEXT("Whether to use depth bounds test to cull distant pixels during AO pass. This option is only valid when pixel shader path is used (r.AmbientOcclusion.Compute=0), without upsampling."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAmbientOcclusionFramebufferCompression(
	TEXT("r.AmbientOcclusion.FramebufferCompression"),
	0,
	TEXT("Whether to use framebuffer compression for AO texture if platform supports it.\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAmbientOcclusionMethod(
	TEXT("r.AmbientOcclusion.Method"),
	0,
	TEXT("Select between SSAO methods \n ")
	TEXT("0: SSAO (default)\n ")
	TEXT("1: GTAO\n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarGTAOUseNormals(
	TEXT("r.GTAO.UseNormals"),
	0,
	TEXT("Whether to use GBuffer Normals or Depth Derived normals \n ")
	TEXT("0: Off (default) \n ")
	TEXT("1: On\n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarGTAOFilterWidth(
	TEXT("r.GTAO.FilterWidth"),
	5,
	TEXT("Size of the noise pattern and filter width\n ")
	TEXT("5: 5x5 Pattern (default) \n ")
	TEXT("4: 4x4 Pattern \n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarGTAOThicknessBlend(
	TEXT("r.GTAO.ThicknessBlend"),
	0.5f,
	TEXT("A heuristic to bias occlusion for thin or thick objects. \n ")
	TEXT("0  : Off \n ")
	TEXT(">0 : On - Bigger values lead to reduced occlusion \n ")
	TEXT("0.5: On (default)\n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarGTAOFalloffEnd(
	TEXT("r.GTAO.FalloffEnd"),
	200.0f,
	TEXT("Distance at when the occlusion completes the fall off.  \n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarGTAOFalloffStartRatio(
	TEXT("r.GTAO.FalloffStartRatio"),
	0.5f,
	TEXT("Ratio of the r.GTAO.FalloffEnd value at which it starts to fall off. \n ")
	TEXT("Must be Between 0 and 1. \n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarGTAONumAngles(
	TEXT("r.GTAO.NumAngles"),
	2,
	TEXT("How Many Angles we choose per pixel \n ")
	TEXT("Must be Between 1 and 16. \n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarGTAOPauseJitter(
	TEXT("r.GTAO.PauseJitter"),
	0,
	TEXT("Whether to pause Jitter when Temporal filter is off \n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);


static TAutoConsoleVariable<int32> CVarGTAOUpsample(
	TEXT("r.GTAO.Upsample"),
	1,
	TEXT("Enable Simple or Depth aware upsample filter for GTAO \n ")
	TEXT("0: Simple \n ")
	TEXT("1: DepthAware (default)\n "),
	ECVF_RenderThreadSafe | ECVF_Scalability);

float FSSAOHelper::GetAmbientOcclusionQualityRT(const FSceneView& View)
{
	float CVarValue = CVarAmbientOcclusionMaxQuality.GetValueOnRenderThread();

	if (CVarValue < 0)
	{
		return FMath::Clamp(-CVarValue, 0.0f, 100.0f);
	}
	else
	{
		return FMath::Min(CVarValue, View.FinalPostProcessSettings.AmbientOcclusionQuality);
	}
}

int32 FSSAOHelper::GetAmbientOcclusionShaderLevel(const FSceneView& View)
{
	float QualityPercent = GetAmbientOcclusionQualityRT(View);

	return	(QualityPercent > 75.0f) +
		(QualityPercent > 55.0f) +
		(QualityPercent > 25.0f) +
		(QualityPercent > 5.0f);
}

bool FSSAOHelper::IsAmbientOcclusionCompute()
{
	return CVarAmbientOcclusionCompute.GetValueOnAnyThread() >= 1;
}

int32 FSSAOHelper::GetNumAmbientOcclusionLevels()
{
	return CVarAmbientOcclusionLevels.GetValueOnRenderThread();
}

float FSSAOHelper::GetAmbientOcclusionStepMipLevelFactor()
{
	return CVarAmbientOcclusionStepMipLevelFactor.GetValueOnRenderThread();
}

EAsyncComputeBudget FSSAOHelper::GetAmbientOcclusionAsyncComputeBudget()
{
	int32 RawBudget = CVarAmbientOcclusionAsyncComputeBudget.GetValueOnRenderThread();

	return (EAsyncComputeBudget)FMath::Clamp(RawBudget, (int32)EAsyncComputeBudget::ELeast_0, (int32)EAsyncComputeBudget::EAll_4);
}

bool FSSAOHelper::IsBasePassAmbientOcclusionRequired(const FViewInfo& View)
{
	// the BaseAO pass is only worth with some AO
	return (View.FinalPostProcessSettings.AmbientOcclusionStaticFraction >= 1 / 100.0f) && IsUsingGBuffers(View.GetShaderPlatform());
}

bool FSSAOHelper::IsAmbientOcclusionAsyncCompute(const FViewInfo& View, uint32 AOPassCount)
{
	// if AsyncCompute is feasible
	// only single level is allowed.  more levels end up reading from gbuffer normals atm which is not allowed.
	if(IsAmbientOcclusionCompute() && (AOPassCount == 1))
	{
		int32 ComputeCVar = CVarAmbientOcclusionCompute.GetValueOnRenderThread();

		if(ComputeCVar >= 2)
		{
			// we might want AsyncCompute

			if(ComputeCVar == 3)
			{
				// enforced, no matter if efficient hardware support
				return true;
			}

			// depends on efficient hardware support
			return GSupportsEfficientAsyncCompute;
		}
	}

	return false;
}

// @return 0:off, 0..3
uint32 FSSAOHelper::ComputeAmbientOcclusionPassCount(const FViewInfo& View)
{
	// 0:off / 1 / 2 / 3
	uint32 Ret = 0;

	{
		int32 CVarLevel = GetNumAmbientOcclusionLevels();

		if (IsAmbientOcclusionCompute() || IsForwardShadingEnabled(View.GetShaderPlatform()))
		{	
			if (CVarLevel<0)
			{
				CVarLevel = 1;
			}
			// Compute and forward only support one pass currently.
			return FMath::Min<int32>(CVarLevel, 1);
		}

		// usually in the range 0..100 
		float QualityPercent = GetAmbientOcclusionQualityRT(View);

		// don't expose 0 as the lowest quality should still render
		Ret = 1 +
			(QualityPercent > 70.0f) +
			(QualityPercent > 35.0f);

		if (CVarLevel >= 0)
		{
			// cvar can override (for scalability or to profile/test)
			Ret = CVarLevel;
		}

		// bring into valid range
		Ret = FMath::Min<uint32>(Ret, 3);
	}

	return Ret;
}


// Helper function to get what type of method we are using
// EGTAOType::EOff					: This is when r.AmbientOcclusion.Method == 0
// EGTAOType::EAsyncHorizonSearch   : This is when We need GBuffer normals and the hardware Supports Async Compute. The trace pass is on the 
//									  Async Pipe and the Intergrate, Spatial and temporal Filters are on the Gfx pipe after the base pass
// EGTAOType::EAsyncCombinedSpatial	: This is when we use Derived normals from the depth buffer and the hardware Supports Async Compute. 
//									  All passes will be on the async compute pipe
// EGTAOType::ENonAsync				: All passes are are on the graphics pipe. Can use either gbuffer normals or derived depth normals.

EGTAOType FSSAOHelper::GetGTAOPassType(const FViewInfo& View, uint32 Levels)
{
	int32 Method		= CVarAmbientOcclusionMethod.GetValueOnRenderThread();
	int32 UseNormals	= CVarGTAOUseNormals.GetValueOnRenderThread();

	if (Method == 1)
	{
		if (IsAmbientOcclusionAsyncCompute(View, Levels))
		{
			if (UseNormals)
			{
				return EGTAOType::EAsyncHorizonSearch;
			}
			else
			{
				return EGTAOType::EAsyncCombinedSpatial;
			}
		}
		else
		{
			return EGTAOType::ENonAsync;
		}
	}
	return EGTAOType::EOff;
}

bool FSSAOHelper::IsDepthBoundsTestEnabled()
{
	return GSupportsDepthBoundsTest && CVarAmbientOcclusionDepthBoundsTest.GetValueOnRenderThread() != 0;
}

FRDGTextureDesc GetScreenSpaceAOTextureDesc(ERHIFeatureLevel::Type FeatureLevel, FIntPoint Extent)
{
	ETextureCreateFlags TextureCreateFlags = TexCreate_UAV | TexCreate_RenderTargetable | TexCreate_ShaderResource | GFastVRamConfig.ScreenSpaceAO;
	if (FSSAOHelper::IsAmbientOcclusionCompute())
	{
		TextureCreateFlags |= TexCreate_NoFastClear;
	}
	return FRDGTextureDesc(FRDGTextureDesc::Create2D(Extent, PF_G8, FClearValueBinding::White, TextureCreateFlags));
}

FRDGTextureRef CreateScreenSpaceAOTexture(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FIntPoint Extent)
{	
	return GraphBuilder.CreateTexture(GetScreenSpaceAOTextureDesc(FeatureLevel, Extent), TEXT("ScreenSpaceAO"));
}

FRDGTextureRef GetScreenSpaceAOFallback(const FRDGSystemTextures& SystemTextures)
{
	return SystemTextures.White;
}

//----------------------------------------------------------------------------------------------------------------------

static const uint32 kSSAOParametersArraySize = 5;

BEGIN_SHADER_PARAMETER_STRUCT(FSSAOShaderParameters, )
	SHADER_PARAMETER_ARRAY(FVector4f, ScreenSpaceAOParams, [kSSAOParametersArraySize])

	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, AOViewport)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, AOSceneViewport)
END_SHADER_PARAMETER_STRUCT();

static FSSAOShaderParameters GetSSAOShaderParameters(
	const FViewInfo& View,
	const FScreenPassTextureViewport& InputViewport,
	const FScreenPassTextureViewport& OutputViewport,
	const FScreenPassTextureViewport& SceneViewport,
	EAOTechnique AOTechnique)
{
	const FFinalPostProcessSettings& Settings = View.FinalPostProcessSettings;

	const FIntPoint RandomizationSize = GSystemTextures.SSAORandomization->GetDesc().Extent * InputViewport.Extent / OutputViewport.Extent;
	FVector2D ViewportUVToRandomUV(InputViewport.Extent.X / (float)RandomizationSize.X, InputViewport.Extent.Y / (float)RandomizationSize.Y);

	// e.g. 4 means the input texture is 4x smaller than the buffer size
	uint32 ScaleToFullRes = SceneViewport.Extent.X / InputViewport.Extent.X;

	FIntRect ViewRect = FIntRect::DivideAndRoundUp(View.ViewRect, ScaleToFullRes);

	float AORadiusInShader = Settings.AmbientOcclusionRadius;
	float ScaleRadiusInWorldSpace = 1.0f;

	if (!Settings.AmbientOcclusionRadiusInWS)
	{
		// radius is defined in view space in 400 units
		AORadiusInShader /= 400.0f;
		ScaleRadiusInWorldSpace = 0.0f;
	}

	// /4 is an adjustment for usage with multiple mips
	float f = FMath::Log2(static_cast<float>(ScaleToFullRes));
	float g = pow(Settings.AmbientOcclusionMipScale, f);
	AORadiusInShader *= pow(Settings.AmbientOcclusionMipScale, f) / 4.0f;

	float Ratio = View.UnscaledViewRect.Width() / (float)View.UnscaledViewRect.Height();

	// Grab this and pass into shader so we can negate the fov influence of projection on the screen pos.
	float InvTanHalfFov = View.ViewMatrices.GetInvTanHalfFov().X;

	float StaticFraction = FMath::Clamp(Settings.AmbientOcclusionStaticFraction, 0.0f, 1.0f);

	// clamp to prevent user error
	float FadeRadius = FMath::Max(1.0f, Settings.AmbientOcclusionFadeRadius);
	float InvFadeRadius = 1.0f / FadeRadius;

	FVector2D TemporalOffset(0.0f, 0.0f);

	if (View.State)
	{
		TemporalOffset = (View.State->GetCurrentTemporalAASampleIndex() % 8) * FVector2D(2.48f, 7.52f) / (float)RandomizationSize.X;
	}
	const float HzbStepMipLevelFactorValue = FMath::Clamp(FSSAOHelper::GetAmbientOcclusionStepMipLevelFactor(), 0.0f, 100.0f);
	const float InvAmbientOcclusionDistance = 1.0f / FMath::Max(Settings.AmbientOcclusionDistance_DEPRECATED, KINDA_SMALL_NUMBER);

	FSSAOShaderParameters Result{};

	// /1000 to be able to define the value in that distance
	Result.ScreenSpaceAOParams[0] = FVector4f(Settings.AmbientOcclusionPower, Settings.AmbientOcclusionBias / 1000.0f, InvAmbientOcclusionDistance, Settings.AmbientOcclusionIntensity);
	Result.ScreenSpaceAOParams[1] = FVector4f(ViewportUVToRandomUV.X, ViewportUVToRandomUV.Y, AORadiusInShader, Ratio);
	Result.ScreenSpaceAOParams[2] = FVector4f(ScaleToFullRes, Settings.AmbientOcclusionMipThreshold / ScaleToFullRes, ScaleRadiusInWorldSpace, Settings.AmbientOcclusionMipBlend);
	Result.ScreenSpaceAOParams[3] = FVector4f(TemporalOffset.X, TemporalOffset.Y, StaticFraction, InvTanHalfFov);
	Result.ScreenSpaceAOParams[4] = FVector4f(InvFadeRadius, -(Settings.AmbientOcclusionFadeDistance - FadeRadius) * InvFadeRadius, HzbStepMipLevelFactorValue, Settings.AmbientOcclusionFadeDistance);

	Result.AOViewport = GetScreenPassTextureViewportParameters(OutputViewport);
	Result.AOSceneViewport = GetScreenPassTextureViewportParameters(SceneViewport);

	return Result;
}

//----------------------------------------------------------------------------------------------------------------------

static const uint32 kGTAOParametersArraySize = 5;

BEGIN_SHADER_PARAMETER_STRUCT(FGTAOShaderParameters, )
	SHADER_PARAMETER_ARRAY(FVector4f, GTAOParams, [kGTAOParametersArraySize])
END_SHADER_PARAMETER_STRUCT();

static FGTAOShaderParameters GetGTAOShaderParameters(const FViewInfo& View, FIntPoint DestSize)
{
	const FFinalPostProcessSettings& Settings = View.FinalPostProcessSettings;

	const FSceneViewState* ViewState = static_cast<const FSceneViewState*>(View.State);

	uint32 Frame = 0;
	if (ViewState && (CVarGTAOPauseJitter.GetValueOnRenderThread() != 1))
	{
		Frame = ViewState->GetFrameIndex();
	}

	FGTAOShaderParameters Result{};

	const float Rots[6] = { 60.0f, 300.0f, 180.0f, 240.0f, 120.0f, 0.0f };
	const float Offsets[4] = { 0.1f, 0.6f, 0.35f, 0.85f };

	float TemporalAngle = Rots[Frame % 6] * (PI / 360.0f);

	// Angles of rotation that are set per frame
	float SinAngle, CosAngle;
	FMath::SinCos(&SinAngle, &CosAngle, TemporalAngle);

	Result.GTAOParams[0] = FVector4f(CosAngle, SinAngle, Offsets[(Frame / 6) % 4] * 0.25, Offsets[Frame % 4]);

	// Frame X = number , Y = Thickness param, 
	float ThicknessBlend = CVarGTAOThicknessBlend.GetValueOnRenderThread();
	ThicknessBlend = FMath::Clamp(1.0f - (ThicknessBlend * ThicknessBlend), 0.0f, 0.99f);
	Result.GTAOParams[1] = FVector4f(Frame, ThicknessBlend, 0.0f, 0.0f);

	// Destination buffer Size and InvSize
	float Fx = float(DestSize.X);
	float Fy = float(DestSize.Y);
	Result.GTAOParams[2] = FVector4f(Fx, Fy, 1.0f / Fx, 1.0f / Fy);

	// Fall Off Params
	float FallOffEnd = CVarGTAOFalloffEnd.GetValueOnRenderThread();
	float FallOffStartRatio = FMath::Clamp(CVarGTAOFalloffStartRatio.GetValueOnRenderThread(), 0.0f, 0.999f);
	float FallOffStart = FallOffEnd * FallOffStartRatio;
	float FallOffStartSq = FallOffStart * FallOffStart;
	float FallOffEndSq = FallOffEnd * FallOffEnd;

	float FallOffScale = 1.0f / (FallOffEndSq - FallOffStartSq);
	float FallOffBias = -FallOffStartSq * FallOffScale;

	Result.GTAOParams[3] = FVector4f(FallOffStart, FallOffEnd, FallOffScale, FallOffBias);

	float TemporalBlendWeight = FMath::Clamp(Settings.AmbientOcclusionTemporalBlendWeight, 0.01f, 1.0f);

	float NumAngles = FMath::Clamp(CVarGTAONumAngles.GetValueOnRenderThread(), 1.0f, 16.0f);
	float SinDeltaAngle, CosDeltaAngle;
	FMath::SinCos(&SinDeltaAngle, &CosDeltaAngle, PI / NumAngles);

	Result.GTAOParams[4] = FVector4f(Settings.AmbientOcclusionTemporalBlendWeight, NumAngles, SinDeltaAngle, CosDeltaAngle);

	return Result;
}

//----------------------------------------------------------------------------------------------------------------------

FHZBParameters GetHZBParametersForAO(FRDGBuilder& GraphBuilder, const FViewInfo& View, FIntPoint InputTextureSize, EAOTechnique AOTechnique)
{
	FHZBParameters OutParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);

	FScreenTransform HZBRemapping;
	if (AOTechnique == EAOTechnique::SSAO)
	{
		const FVector2f HZBScaleFactor(
			float(View.ViewRect.Width()) / float(2 * View.HZBMipmap0Size.X),
			float(View.ViewRect.Height()) / float(2 * View.HZBMipmap0Size.Y));

		// from -1..1 to UV 0..1*HZBScaleFactor
		HZBRemapping.Scale = FVector2f(0.5f * HZBScaleFactor.X, -0.5f * HZBScaleFactor.Y);
		HZBRemapping.Bias = FVector2f(0.5f * HZBScaleFactor.X, 0.5f * HZBScaleFactor.Y);
	}
	else
	{
		const FVector2f HZBScaleFactor(
			float(InputTextureSize.X) / float(2 * View.HZBMipmap0Size.X),
			float(InputTextureSize.Y) / float(2 * View.HZBMipmap0Size.Y)
		);

		HZBRemapping.Scale = HZBScaleFactor;
		HZBRemapping.Bias = FVector2f(0.0f, 0.0f);
	}
	
	OutParameters.ScreenPosToHZBUVScaleBias = FVector4f(
		HZBRemapping.Scale.X, 
		HZBRemapping.Scale.Y, 
		HZBRemapping.Bias.X, 
		HZBRemapping.Bias.Y);

	return OutParameters;
}

//----------------------------------------------------------------------------------------------------------------------

class FAmbientOcclusionSetupPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FAmbientOcclusionSetupPS);
	SHADER_USE_PARAMETER_STRUCT(FAmbientOcclusionSetupPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSSAOShaderParameters, SSAOParameters)

		SHADER_PARAMETER(float, ThresholdInverse)
		SHADER_PARAMETER(FVector2f, InputExtentInverse)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FAmbientOcclusionSetupPS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "MainSetupPS", SF_Pixel);

FScreenPassTexture AddAmbientOcclusionSetupPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	FScreenPassTexture Input)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, SSAOSetup, "ScreenSpace AO Setup");

	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(GetDownscaledViewport(InputViewport, 2));

	FScreenPassRenderTarget Output;

	{
		FRDGTextureDesc OutputDesc = Input.Texture->Desc;
		OutputDesc.Reset();
		OutputDesc.Format = PF_FloatRGBA;
		OutputDesc.ClearValue = FClearValueBinding::None;
		OutputDesc.Flags &= ~TexCreate_DepthStencilTargetable;
		OutputDesc.Flags |= TexCreate_RenderTargetable;
		OutputDesc.Extent = OutputViewport.Extent;

		Output.Texture = GraphBuilder.CreateTexture(OutputDesc, TEXT("AmbientOcclusionSetup"));
		Output.ViewRect = OutputViewport.Rect;
		Output.LoadAction = ERenderTargetLoadAction::ENoAction;
		Output.UpdateVisualizeTextureExtent();
	}

	const FFinalPostProcessSettings& Settings = View.FinalPostProcessSettings;
	const float ThresholdInverseValue = Settings.AmbientOcclusionMipThreshold * ((float)OutputViewport.Extent.X / (float)CommonParameters.SceneTexturesViewport.Extent.X);


	FAmbientOcclusionSetupPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAmbientOcclusionSetupPS::FParameters>();
	PassParameters->View = View.GetShaderParameters();
	PassParameters->SceneTextures = CommonParameters.SceneTextures;
	PassParameters->SSAOParameters = GetSSAOShaderParameters(View, InputViewport, OutputViewport, CommonParameters.SceneTexturesViewport, EAOTechnique::SSAO);
	PassParameters->ThresholdInverse = ThresholdInverseValue;
	PassParameters->InputExtentInverse = FVector2f(1.0f) / FVector2f(InputViewport.Extent);
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	TShaderMapRef<FAmbientOcclusionSetupPS> PixelShader(View.ShaderMap);

	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("AmbientOcclusionSetup %dx%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
		View,
		OutputViewport,
		InputViewport,
		PixelShader,
		PassParameters);

	return MoveTemp(Output);
}

//----------------------------------------------------------------------------------------------------------------------

class FAmbientOcclusionSmoothCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FAmbientOcclusionSmoothCS);
	SHADER_USE_PARAMETER_STRUCT(FAmbientOcclusionSmoothCS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), 8);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), 8);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, SSAOSmoothOutputViewport)
		SHADER_PARAMETER(FScreenTransform, SSAOSmoothOutputToInput)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SSAOSmoothInputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SSAOSmoothInputSampler)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, SSAOSmoothOutputTexture)
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FAmbientOcclusionSmoothCS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "MainSSAOSmoothCS", SF_Compute);


class FAmbientOcclusionSmoothPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FAmbientOcclusionSmoothPS);
	SHADER_USE_PARAMETER_STRUCT(FAmbientOcclusionSmoothPS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 0);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, SSAOSmoothOutputViewport)
		SHADER_PARAMETER(FScreenTransform, SSAOSmoothOutputToInput)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SSAOSmoothInputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SSAOSmoothInputSampler)
		
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FAmbientOcclusionSmoothPS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "MainSSAOSmoothPS", SF_Pixel);

void AddAmbientOcclusionSmoothPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	ESSAOType SSAOType,
	FScreenPassTexture Input,
	FScreenPassRenderTarget Output)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, SSAOSmooth, "SSAO smooth");

	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(Output);

	const FScreenPassTextureViewportParameters InputViewportParameters = GetScreenPassTextureViewportParameters(InputViewport);
	const FScreenPassTextureViewportParameters OutputViewportParameters = GetScreenPassTextureViewportParameters(OutputViewport);

	if (SSAOType == ESSAOType::EPS)
	{
		FAmbientOcclusionSmoothPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAmbientOcclusionSmoothPS::FParameters>();
		PassParameters->SSAOSmoothOutputViewport = OutputViewportParameters;
		PassParameters->SSAOSmoothOutputToInput = FScreenTransform::ChangeTextureUVCoordinateFromTo(OutputViewport, InputViewport);
		PassParameters->SSAOSmoothInputTexture = Input.Texture;
		PassParameters->SSAOSmoothInputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		TShaderMapRef<FAmbientOcclusionSmoothPS> PixelShader(View.ShaderMap);

		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("AmbientOcclusionSmooth %dx%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
			View,
			OutputViewport,
			InputViewport,
			PixelShader,
			PassParameters);
	}
	else
	{
		FAmbientOcclusionSmoothCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAmbientOcclusionSmoothCS::FParameters>();
		PassParameters->SSAOSmoothOutputViewport = OutputViewportParameters;
		PassParameters->SSAOSmoothOutputToInput = FScreenTransform::ChangeTextureUVCoordinateFromTo(OutputViewport, InputViewport);
		PassParameters->SSAOSmoothInputTexture = Input.Texture;
		PassParameters->SSAOSmoothInputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->SSAOSmoothOutputTexture = GraphBuilder.CreateUAV(Output.Texture);

		TShaderMapRef<FAmbientOcclusionSmoothCS> ComputeShader(View.ShaderMap);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("AmbientOcclusionSmooth %dx%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
			SSAOType == ESSAOType::EAsyncCS ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), 8));
	}
}

//----------------------------------------------------------------------------------------------------------------------

/**
 * Encapsulates the post processing ambient occlusion pixel shader.
 * @param bAOSetupAsInput true:use AO setup instead of full resolution depth and normal
 * @param bDoUpsample true:we have lower resolution pass data we need to upsample, false otherwise
 * @param ShaderQuality 0..4, 0:low 4:high
 */

BEGIN_SHADER_PARAMETER_STRUCT(FAmbientOcclusionParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)

	SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSSAOShaderParameters, SSAOParameters)

	SHADER_PARAMETER(FVector2f, SSAO_DownsampledAOInverseSize)
	SHADER_PARAMETER(FVector2f, SSAO_DownsampledAOUVViewportMin)
	SHADER_PARAMETER(FVector2f, SSAO_DownsampledAOUVViewportMax)
	SHADER_PARAMETER(FVector2f, SSAO_SvPositionScaleBias)
	SHADER_PARAMETER(uint32, SSAO_ShadingModelsExcludeMask)

	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SSAO_SetupTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SSAO_NormalsTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SSAO_DownsampledAO)

	SHADER_PARAMETER_SAMPLER(SamplerState, SSAO_Sampler)

	SHADER_PARAMETER_TEXTURE(Texture2D, RandomNormalTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, RandomNormalTextureSampler)
END_SHADER_PARAMETER_STRUCT();

struct FAmbientOcclusionPermutations
{
	class FUseUpsampleDim		: SHADER_PERMUTATION_BOOL("USE_UPSAMPLE");
	class FUseAoSetupAsInputDim : SHADER_PERMUTATION_BOOL("USE_AO_SETUP_AS_INPUT");
	class FOutputDepthDim		: SHADER_PERMUTATION_BOOL("OUTPUT_DEPTH");
	class FUseSMExcludeMaskDim	: SHADER_PERMUTATION_BOOL("USE_SM_EXCLUDE_MASK");
	class FShaderQualityDim		: SHADER_PERMUTATION_INT("SHADER_QUALITY", 5);

	using FPermutationDomain = TShaderPermutationDomain<FUseUpsampleDim, FUseAoSetupAsInputDim, FUseSMExcludeMaskDim, FOutputDepthDim, FShaderQualityDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters, EShaderFrequency Frequency)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (!IsMobilePlatform(Parameters.Platform))
		{
			// Mobile specific features
			if (PermutationVector.Get<FUseSMExcludeMaskDim>() || PermutationVector.Get<FOutputDepthDim>())
			{
				return false;
			}
		}

		// SM mask requires access to GBuffer, only pixel shader version can do it
		const bool bSMAccess = !IsMobilePlatform(Parameters.Platform) || (IsMobileDeferredShadingEnabled(Parameters.Platform) && MobileUsesGBufferCustomData(Parameters.Platform));
		if (PermutationVector.Get<FUseSMExcludeMaskDim>() && (Frequency != SF_Pixel || !bSMAccess))
		{
			return false;
		}

		// OUTPUT_DEPTH is only used in a specific setup with a bilateral upsample
		if (PermutationVector.Get<FOutputDepthDim>() &&
			(PermutationVector.Get<FUseUpsampleDim>() || PermutationVector.Get<FUseAoSetupAsInputDim>()))
		{
			return false;
		}
		
		return true;
	}

	static FPermutationDomain BuildPermutationVector(bool bDoUpsample, bool bAOSetupAsInput, int32 ShaderQuality, bool bOutputDepth, uint32 SMExcludeMask)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FUseUpsampleDim>(bDoUpsample);
		PermutationVector.Set<FUseAoSetupAsInputDim>(bAOSetupAsInput);
		PermutationVector.Set<FShaderQualityDim>(ShaderQuality);
		PermutationVector.Set<FOutputDepthDim>(bOutputDepth);
		PermutationVector.Set<FUseSMExcludeMaskDim>(SMExcludeMask != 0);
		return PermutationVector;
	}
};

class FAmbientOcclusionPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FAmbientOcclusionPS);
	SHADER_USE_PARAMETER_STRUCT(FAmbientOcclusionPS, FGlobalShader);

	using FPermutationDomain = FAmbientOcclusionPermutations::FPermutationDomain;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 0);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FAmbientOcclusionPermutations::ShouldCompilePermutation(Parameters, SF_Pixel);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FAmbientOcclusionParameters, SharedParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FAmbientOcclusionPS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "MainPS", SF_Pixel);

class FAmbientOcclusionCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FAmbientOcclusionCS);
	SHADER_USE_PARAMETER_STRUCT(FAmbientOcclusionCS, FGlobalShader);

	using FPermutationDomain = FAmbientOcclusionPermutations::FPermutationDomain;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), 16);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), 16);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FAmbientOcclusionPermutations::ShouldCompilePermutation(Parameters, SF_Compute);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FAmbientOcclusionParameters, SharedParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FAmbientOcclusionCS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "MainCS", SF_Compute);

FScreenPassRenderTarget CreateAmbientOcclusionOutputTarget(
	FRDGBuilder& GraphBuilder,
	FRDGTextureDesc OutputDesc,
	FIntRect ViewRect,
	ESSAOType AOType,
	EPixelFormat IntermediateFormatOverride)
{
	const bool bUsingUAVOutput = (AOType == ESSAOType::ECS || AOType == ESSAOType::EAsyncCS);

	OutputDesc.Reset();
	OutputDesc.ClearValue = FClearValueBinding::None;
	OutputDesc.Flags &= ~TexCreate_DepthStencilTargetable;

	if (bUsingUAVOutput)
	{
		// UAV allowed format
		OutputDesc.Format = PF_FloatRGBA;
		OutputDesc.Flags |= TexCreate_UAV;
	}
	else
	{
		// R:AmbientOcclusion, GBA:used for normal
		OutputDesc.Format = PF_B8G8R8A8;
		OutputDesc.Flags |= TexCreate_RenderTargetable;
	}

	if (IntermediateFormatOverride != PF_Unknown)
	{
		OutputDesc.Format = IntermediateFormatOverride;
	}

	if (UE::PixelFormat::HasCapabilities(OutputDesc.Format, EPixelFormatCapabilities::LossyCompressible) 
		&& CVarAmbientOcclusionFramebufferCompression.GetValueOnRenderThread() != 0)
	{
		OutputDesc.Flags |= TexCreate_LossyCompressionLowBitrate;
	}

	return FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, TEXT("AmbientOcclusion")), ViewRect, ERenderTargetLoadAction::ENoAction);
}

void AddAmbientOcclusionPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	const FScreenPassTexture& SetupTexture,
	const FScreenPassTexture& NormalsTexture,
	const FScreenPassTexture& DownsampledAO,
	FScreenPassRenderTarget Output,
	ESSAOType AOType,
	bool bAOSetupAsInput)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, SSAO, "ScreenSpace AO");

	// No setup texture falls back to a depth scene texture fetch.
	const FScreenPassTextureViewport InputViewport = SetupTexture.IsValid()
		? FScreenPassTextureViewport(SetupTexture)
		: CommonParameters.SceneTexturesViewport;

	const FScreenPassTextureViewport OutputViewport(Output);

	bool bDoUpsample = DownsampledAO.IsValid();
	if (CommonParameters.MobileHalfResolutionSetting > 0)
	{
		// these are not supported with half-res SSAO
		bDoUpsample = false;
		bAOSetupAsInput = false;
	}
	
	FAmbientOcclusionParameters SharedParameters;
	SharedParameters.View = View.GetShaderParameters();
	SharedParameters.SceneTextures = CommonParameters.SceneTextures;
	SharedParameters.HZBParameters = GetHZBParametersForAO(GraphBuilder, View, CommonParameters.SceneTexturesViewport.Extent, EAOTechnique::SSAO);
	SharedParameters.SSAOParameters = GetSSAOShaderParameters(View, InputViewport, OutputViewport, CommonParameters.SceneTexturesViewport, EAOTechnique::SSAO);

	SharedParameters.SSAO_SetupTexture = SetupTexture.Texture;

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	if (NormalsTexture.IsValid())
	{
		SharedParameters.SSAO_NormalsTexture = NormalsTexture.Texture;
	}
	else
	{
		SharedParameters.SSAO_NormalsTexture = SystemTextures.Black;
	}

	if (DownsampledAO.IsValid())
	{
		SharedParameters.SSAO_DownsampledAO = DownsampledAO.Texture;
		SharedParameters.SSAO_DownsampledAOInverseSize = FVector2f(1.0f) / FVector2f(DownsampledAO.Texture->Desc.Extent);

		const FVector2f ViewportMin(DownsampledAO.ViewRect.Min.X, DownsampledAO.ViewRect.Min.Y);
		const FVector2f ViewportMax(DownsampledAO.ViewRect.Max.X, DownsampledAO.ViewRect.Max.Y);

		SharedParameters.SSAO_DownsampledAOUVViewportMin = (ViewportMin + 0.5f) * SharedParameters.SSAO_DownsampledAOInverseSize;
		SharedParameters.SSAO_DownsampledAOUVViewportMax = (ViewportMax - 0.5f) * SharedParameters.SSAO_DownsampledAOInverseSize;
	}
	else
	{
		SharedParameters.SSAO_DownsampledAO = SystemTextures.Black;
		SharedParameters.SSAO_DownsampledAOInverseSize = FVector2f(1.0f, 1.0f);
		SharedParameters.SSAO_DownsampledAOUVViewportMin = FVector2f(0.0f, 0.0f);
		SharedParameters.SSAO_DownsampledAOUVViewportMax = FVector2f(1.0f, 1.0f);
	}

	if (CommonParameters.MobileHalfResolutionSetting > 0)
	{
		// Downscaled 
		SharedParameters.SSAO_SvPositionScaleBias = FVector2f(2, -0.5);
	}
	else
	{
		SharedParameters.SSAO_SvPositionScaleBias = FVector2f(1, 0);
	}
	
	SharedParameters.SSAO_ShadingModelsExcludeMask = CommonParameters.MobileShadingModelsExcludeMask;
			
	SharedParameters.SSAO_Sampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	SharedParameters.RandomNormalTexture = GSystemTextures.SSAORandomization->GetRHI();
	SharedParameters.RandomNormalTextureSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

	FRDGEventName EventName(TEXT("AmbientOcclusion%s %dx%d SetupAsInput=%d Upsample=%d ShaderQuality=%d"),
		(AOType == ESSAOType::EPS) ? TEXT("PS") : TEXT("CS"), OutputViewport.Rect.Width(), OutputViewport.Rect.Height(), bAOSetupAsInput, bDoUpsample, CommonParameters.ShaderQuality);

	if (AOType == ESSAOType::ECS || AOType == ESSAOType::EAsyncCS)
	{
		// Compute Shader Path
		FAmbientOcclusionCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAmbientOcclusionCS::FParameters>();
		PassParameters->SharedParameters = MoveTemp(SharedParameters);
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->OutTexture = GraphBuilder.CreateUAV(Output.Texture);

		FAmbientOcclusionCS::FPermutationDomain PermutationVector = FAmbientOcclusionPermutations::BuildPermutationVector(
			bDoUpsample, 
			bAOSetupAsInput, 
			CommonParameters.ShaderQuality, 
			CommonParameters.MobileHalfResolutionSetting > 1,
			0// SMExcludeMask
		);

		TShaderMapRef<FAmbientOcclusionCS> ComputeShader(View.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			MoveTemp(EventName),
			AOType == ESSAOType::EAsyncCS ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), 16));
	}
	else
	{
		const bool bFullRes = (CommonParameters.SceneTexturesViewport == OutputViewport);
		const bool bDepthBoundsTestEnabled =
			!bDoUpsample
			&& bFullRes
			&& FSSAOHelper::IsDepthBoundsTestEnabled()
			&& CommonParameters.SceneDepth.IsValid()
			&& CommonParameters.SceneDepth.Texture->Desc.NumSamples == 1;

		FDepthStencilBinding DepthStencilBinding{};
		float DepthFar = 0.0f;

		if (bDepthBoundsTestEnabled)
		{
			DepthStencilBinding = FDepthStencilBinding(CommonParameters.SceneDepth.Texture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);
			
			const FFinalPostProcessSettings& Settings = View.FinalPostProcessSettings;
			const FMatrix& ProjectionMatrix = View.ViewMatrices.GetViewToClip();
			const FVector4f Far = (FVector4f)ProjectionMatrix.TransformFVector4(FVector4(0, 0, Settings.AmbientOcclusionFadeDistance));
			DepthFar = FMath::Min(1.0f, Far.Z / Far.W);

			FRenderTargetParameters* ClearParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
			ClearParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
			ClearParameters->RenderTargets.DepthStencil = DepthStencilBinding;

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("DepthBounds ClearQuad(%s)", Output.Texture->Name),
				ClearParameters,
				ERDGPassFlags::Raster,
			[OutputViewport, DepthFar](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				// We must clear all pixels that won't be touched by AO shader.
				FClearQuadCallbacks Callbacks;
				Callbacks.PSOModifier = [](FGraphicsPipelineStateInitializer& PSOInitializer)
				{
					PSOInitializer.bDepthBounds = true;
				};
				Callbacks.PreClear = [DepthFar](FRHICommandList& InRHICmdList)
				{
					// This is done by rendering a clear quad over a depth range from AmbientOcclusionFadeDistance to far plane.
					InRHICmdList.SetDepthBounds(0, DepthFar);	// NOTE: Inverted depth
				};
				Callbacks.PostClear = [DepthFar](FRHICommandList& InRHICmdList)
				{
					// Set depth bounds test to cover everything from near plane to AmbientOcclusionFadeDistance and run AO pixel shader.
					InRHICmdList.SetDepthBounds(DepthFar, 1.0f);
				};

				RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

				DrawClearQuad(RHICmdList, FLinearColor::White, Callbacks);
			});

			// Make sure the following pass doesn't clear or ignore the data
			Output.LoadAction = ERenderTargetLoadAction::ELoad;
		}

		// Pixel Shader Path
		FAmbientOcclusionPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAmbientOcclusionPS::FParameters>();
		PassParameters->SharedParameters = MoveTemp(SharedParameters);
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		PassParameters->RenderTargets.ShadingRateTexture = GVRSImageManager.GetVariableRateShadingImage(GraphBuilder, View, FVariableRateShadingImageManager::EVRSPassType::SSAO);
		if (bDepthBoundsTestEnabled)
		{
			PassParameters->RenderTargets.DepthStencil = DepthStencilBinding;
		}
		
		FAmbientOcclusionPS::FPermutationDomain PermutationVector = FAmbientOcclusionPermutations::BuildPermutationVector(
			bDoUpsample, 
			bAOSetupAsInput, 
			CommonParameters.ShaderQuality, 
			CommonParameters.MobileHalfResolutionSetting > 1,
			CommonParameters.MobileShadingModelsExcludeMask
		);

		TShaderMapRef<FAmbientOcclusionPS> PixelShader(View.ShaderMap, PermutationVector);
		TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);

		check(PassParameters);
		ClearUnusedGraphResources(PixelShader, PassParameters);

		GraphBuilder.AddPass(
			MoveTemp(EventName),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, OutputViewport, InputViewport, VertexShader, PixelShader, PassParameters, bDepthBoundsTestEnabled, DepthFar] (FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			const FIntRect InputRect = InputViewport.Rect;
			const FIntPoint InputSize = InputViewport.Extent;
			const FIntRect OutputRect = OutputViewport.Rect;
			const FIntPoint OutputSize = OutputRect.Size();

			RHICmdList.SetViewport(OutputRect.Min.X, OutputRect.Min.Y, 0.0f, OutputRect.Max.X, OutputRect.Max.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			GraphicsPSOInit.bDepthBounds = bDepthBoundsTestEnabled;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			if (bDepthBoundsTestEnabled)
			{
				RHICmdList.SetDepthBounds(DepthFar, 1.0f);
			}

			DrawPostProcessPass(
				RHICmdList,
				0, 0, OutputSize.X, OutputSize.Y,
				InputRect.Min.X, InputRect.Min.Y, InputRect.Width(), InputRect.Height(),
				OutputSize,
				InputSize,
				VertexShader,
				View.StereoViewIndex,
				false,
				EDRF_UseTriangleOptimization);

			if (bDepthBoundsTestEnabled)
			{
				RHICmdList.SetDepthBounds(0, 1.0f);
			}
		});
	}
}

FScreenPassTexture AddAmbientOcclusionStepPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	const FScreenPassTexture& SetupTexture,
	const FScreenPassTexture& NormalsTexture,
	const FScreenPassTexture& DownsampledAO)
{
	FScreenPassRenderTarget Output = CreateAmbientOcclusionOutputTarget(GraphBuilder, SetupTexture.Texture->Desc, SetupTexture.ViewRect, CommonParameters.DownscaleType, PF_Unknown);
	AddAmbientOcclusionPass(
		GraphBuilder,
		View,
		CommonParameters,
		SetupTexture,
		NormalsTexture,
		DownsampledAO,
		Output,
		CommonParameters.DownscaleType,
		true);
	return MoveTemp(Output);
}

FScreenPassTexture AddAmbientOcclusionFinalPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	const FScreenPassTexture& SetupTexture,
	const FScreenPassTexture& NormalsTexture,
	const FScreenPassTexture& DownsampledAO,
	FScreenPassRenderTarget FinalOutput)
{
	FScreenPassRenderTarget CurrentOutput = CommonParameters.bNeedSmoothingPass ?
		CreateAmbientOcclusionOutputTarget(GraphBuilder, FinalOutput.Texture->Desc, FinalOutput.ViewRect, CommonParameters.FullscreenType, PF_G8):
		FinalOutput;

	AddAmbientOcclusionPass(
		GraphBuilder,
		View,
		CommonParameters,
		SetupTexture,
		NormalsTexture,
		DownsampledAO,
		CurrentOutput,
		CommonParameters.FullscreenType,
		false);

	if (CommonParameters.bNeedSmoothingPass)
	{
		AddAmbientOcclusionSmoothPass(
				GraphBuilder,
				View,
				CommonParameters.FullscreenType,
				CurrentOutput,
				FinalOutput);

		CurrentOutput = FinalOutput;
	}

	return MoveTemp(CurrentOutput);
}

//----------------------------------------------------------------------------------------------------------------------

class FGTAOHorizonSearchAndIntegrateCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGTAOHorizonSearchAndIntegrateCS);
	SHADER_USE_PARAMETER_STRUCT(FGTAOHorizonSearchAndIntegrateCS, FGlobalShader);

	class FShaderQualityDim : SHADER_PERMUTATION_INT("SHADER_QUALITY", 5);
	class FUseNormalBufferDim : SHADER_PERMUTATION_BOOL("USE_NORMALBUFFER");

	using FPermutationDomain = TShaderPermutationDomain<FShaderQualityDim, FUseNormalBufferDim>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), 8);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), 8);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSSAOShaderParameters, SSAOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGTAOShaderParameters, GTAOParameters)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FGTAOHorizonSearchAndIntegrateCS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "GTAOCombinedCS", SF_Compute);

FGTAOHorizonSearchOutputs AddGTAOHorizonSearchIntegratePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture SceneDepth)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, GTAO_HorizonSearchIntegrate, "GTAO HorizonSearch And Integrate");

	const FScreenPassTextureViewport SceneViewport(SceneDepth);
	const FScreenPassTextureViewport OutputViewport(GetDownscaledViewport(SceneViewport, CommonParameters.DownscaleFactor));

	FScreenPassRenderTarget Output;

	{
		FRDGTextureDesc OutputDesc = SceneDepth.Texture->Desc;
		OutputDesc.Reset();
		OutputDesc.Format = PF_G8;
		OutputDesc.ClearValue = FClearValueBinding::None;
		OutputDesc.Flags &= ~TexCreate_DepthStencilTargetable;
		OutputDesc.Flags |= TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV;
		OutputDesc.Extent = OutputViewport.Extent;

		Output.Texture = GraphBuilder.CreateTexture(OutputDesc, TEXT("GTAOCombined"));
		Output.ViewRect = OutputViewport.Rect;
		Output.LoadAction = ERenderTargetLoadAction::ENoAction;
		Output.UpdateVisualizeTextureExtent();
	}

	const bool bUseNormals = CVarGTAOUseNormals.GetValueOnRenderThread() >= 1;

	FGTAOHorizonSearchAndIntegrateCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGTAOHorizonSearchAndIntegrateCS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneTextures = CommonParameters.SceneTextures;
	PassParameters->HZBParameters = GetHZBParametersForAO(GraphBuilder, View, SceneViewport.Extent, EAOTechnique::GTAO);
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PassParameters->SSAOParameters = GetSSAOShaderParameters(View, SceneViewport, OutputViewport, CommonParameters.SceneTexturesViewport, EAOTechnique::GTAO);
	PassParameters->GTAOParameters = GetGTAOShaderParameters(View, OutputViewport.Extent);

	PassParameters->OutTexture = GraphBuilder.CreateUAV(Output.Texture);

	FGTAOHorizonSearchAndIntegrateCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FGTAOHorizonSearchAndIntegrateCS::FShaderQualityDim>(CommonParameters.ShaderQuality);
	PermutationVector.Set<FGTAOHorizonSearchAndIntegrateCS::FUseNormalBufferDim>(bUseNormals);

	TShaderMapRef<FGTAOHorizonSearchAndIntegrateCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("GTAOCombinedCS %dx%d ShaderQuality=%d UseNormals=%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height(), CommonParameters.ShaderQuality, bUseNormals ? 1 : 0),
		CommonParameters.GTAOType == EGTAOType::EAsyncCombinedSpatial ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), 8));

	FGTAOHorizonSearchOutputs Outputs;
	Outputs.Color = Output;
	return MoveTemp(Outputs);
}

//----------------------------------------------------------------------------------------------------------------------

class FGTAOInnerIntegratePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGTAOInnerIntegratePS);
	SHADER_USE_PARAMETER_STRUCT(FGTAOInnerIntegratePS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 0);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSSAOShaderParameters, SSAOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGTAOShaderParameters, GTAOParameters)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HorizonsTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, HorizonsTextureSampler)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FGTAOInnerIntegratePS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "GTAOInnerIntegratePS", SF_Pixel);

FScreenPassTexture AddGTAOInnerIntegratePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture SceneDepth,
	FScreenPassTexture HorizonsTexture)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, GTAO_InnerIntegrate, "GTAO InnerIntegrate");

	const FScreenPassTextureViewport InputViewport(SceneDepth);
	const FScreenPassTextureViewport OutputViewport(GetDownscaledViewport(InputViewport, CommonParameters.DownscaleFactor));

	FScreenPassRenderTarget Output;

	{
		FRDGTextureDesc OutputDesc = SceneDepth.Texture->Desc;
		OutputDesc.Reset();
		OutputDesc.Format = PF_G8;
		OutputDesc.ClearValue = FClearValueBinding::None;
		OutputDesc.Flags &= ~TexCreate_DepthStencilTargetable;
		OutputDesc.Flags |= TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV;
		OutputDesc.Extent = OutputViewport.Extent;

		Output.Texture = GraphBuilder.CreateTexture(OutputDesc, TEXT("GTAOInnerIntegrate"));
		Output.ViewRect = OutputViewport.Rect;
		Output.LoadAction = ERenderTargetLoadAction::ENoAction;
		Output.UpdateVisualizeTextureExtent();
	}

	FGTAOInnerIntegratePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGTAOInnerIntegratePS::FParameters>();

	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneTextures = CommonParameters.SceneTextures;
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PassParameters->SSAOParameters = GetSSAOShaderParameters(View, InputViewport, OutputViewport, CommonParameters.SceneTexturesViewport, EAOTechnique::GTAO);
	PassParameters->GTAOParameters = GetGTAOShaderParameters(View, OutputViewport.Extent);

	PassParameters->HorizonsTexture = HorizonsTexture.Texture;
	PassParameters->HorizonsTextureSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	TShaderMapRef<FGTAOInnerIntegratePS> PixelShader(View.ShaderMap);

	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("GTAOInnerIntegratePS %dx%d Downscale=%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height(), CommonParameters.DownscaleFactor),
		View,
		OutputViewport,
		InputViewport,
		PixelShader,
		PassParameters);

	return MoveTemp(Output);
}

//----------------------------------------------------------------------------------------------------------------------

class FGTAOHorizonSearchCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGTAOHorizonSearchCS);
	SHADER_USE_PARAMETER_STRUCT(FGTAOHorizonSearchCS, FGlobalShader);

	class FShaderQualityDim : SHADER_PERMUTATION_INT("SHADER_QUALITY", 5);

	using FPermutationDomain = TShaderPermutationDomain<FShaderQualityDim>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), 8);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), 8);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSSAOShaderParameters, SSAOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGTAOShaderParameters, GTAOParameters)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, HorizonOutTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DepthOutTexture)
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FGTAOHorizonSearchCS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "HorizonSearchCS", SF_Compute);

FGTAOHorizonSearchOutputs AddGTAOHorizonSearchPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture SceneDepth,
	FScreenPassRenderTarget HorizonOutput)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, GTAO_HorizonSearch, "GTAO HorizonSearch");

	const FScreenPassTextureViewport SceneViewport(SceneDepth);
	const FScreenPassTextureViewport OutputViewport(HorizonOutput);

	FGTAOHorizonSearchCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGTAOHorizonSearchCS::FParameters>();

	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneTextures = CommonParameters.SceneTextures;
	PassParameters->HZBParameters = GetHZBParametersForAO(GraphBuilder, View, SceneViewport.Extent, EAOTechnique::GTAO);
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PassParameters->SSAOParameters = GetSSAOShaderParameters(View, SceneViewport, OutputViewport, CommonParameters.SceneTexturesViewport, EAOTechnique::GTAO);
	PassParameters->GTAOParameters = GetGTAOShaderParameters(View, OutputViewport.Extent);

	PassParameters->HorizonOutTexture = GraphBuilder.CreateUAV(HorizonOutput.Texture);

	FGTAOHorizonSearchCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FGTAOHorizonSearchCS::FShaderQualityDim>(CommonParameters.ShaderQuality);

	TShaderMapRef<FGTAOHorizonSearchCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HorizonSearchCS %dx%d ShaderQuality=%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height(), CommonParameters.ShaderQuality),
		ERDGPassFlags::AsyncCompute,
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), 8));

	FGTAOHorizonSearchOutputs Outputs;
	Outputs.Color = HorizonOutput;

	return MoveTemp(Outputs);
}

//----------------------------------------------------------------------------------------------------------------------

class FGTAOTemporalFilterCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGTAOTemporalFilterCS);
	SHADER_USE_PARAMETER_STRUCT(FGTAOTemporalFilterCS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), 8);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), 8);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSSAOShaderParameters, SSAOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGTAOShaderParameters, GTAOParameters)

		SHADER_PARAMETER(FVector4f, PrevScreenPositionScaleBias)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GTAOTemporalInput)
		SHADER_PARAMETER_SAMPLER(SamplerState, GTAOTemporalSampler)
		SHADER_PARAMETER(FVector2f, GTAOTemporalInputPixelSize)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HistoryTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, HistoryTextureSampler)
		SHADER_PARAMETER(FVector2f, HistoryTextureSize)
		SHADER_PARAMETER(FVector2f, HistoryTexturePixelSize)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ZCurrTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ZCurrTextureSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneVelocityTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneVelocityTextureSampler)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FGTAOTemporalFilterCS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "GTAOTemporalFilterCS", SF_Compute);

FGTAOTemporalOutputs AddGTAOTemporalPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture Input,
	FScreenPassTexture SceneDepth,
	FScreenPassTexture SceneVelocity,
	FScreenPassTexture HistoryColor,
	FScreenPassTextureViewport HistoryViewport)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, GTAO_TemporalFilter, "GTAO Temportal Filter");

	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(InputViewport);

	FScreenPassRenderTarget OutputAO;

	{
		FRDGTextureDesc OutputDesc = Input.Texture->Desc;
		OutputDesc.Reset();
		OutputDesc.ClearValue = FClearValueBinding::None;
		OutputDesc.Flags &= ~TexCreate_DepthStencilTargetable;
		OutputDesc.Flags |= TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV;
		OutputDesc.Extent = OutputViewport.Extent;

		OutputDesc.Format = PF_G8;
		OutputAO.Texture = GraphBuilder.CreateTexture(OutputDesc, TEXT("GTAOTemporalOutput"));
		OutputAO.ViewRect = OutputViewport.Rect;
		OutputAO.LoadAction = ERenderTargetLoadAction::ENoAction;
		OutputAO.UpdateVisualizeTextureExtent();
	}

	const FVector2f HistoryTextureSize = FVector2f(HistoryColor.Texture->Desc.Extent);
	const FVector2f HistoryTexturePixelSize = FVector2f(1.0f) / HistoryTextureSize;

	const FIntPoint ViewportOffset = HistoryViewport.Rect.Min;
	const FIntPoint ViewportExtent = HistoryViewport.Rect.Size();
	const FIntPoint BufferSize = HistoryViewport.Extent;

	const FVector4f PrevScreenPositionScaleBiasValue = FVector4f(
		ViewportExtent.X * 0.5f / BufferSize.X,
		-ViewportExtent.Y * 0.5f / BufferSize.Y,
		(ViewportExtent.X * 0.5f + ViewportOffset.X) / BufferSize.X,
		(ViewportExtent.Y * 0.5f + ViewportOffset.Y) / BufferSize.Y);

	FGTAOTemporalFilterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGTAOTemporalFilterCS::FParameters>();

	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneTextures = CommonParameters.SceneTextures;
	PassParameters->SSAOParameters = GetSSAOShaderParameters(View, InputViewport, OutputViewport, CommonParameters.SceneTexturesViewport, EAOTechnique::GTAO);
	PassParameters->GTAOParameters = GetGTAOShaderParameters(View, OutputViewport.Extent);

	PassParameters->PrevScreenPositionScaleBias = PrevScreenPositionScaleBiasValue;

	PassParameters->GTAOTemporalInput = Input.Texture;
	PassParameters->GTAOTemporalSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->GTAOTemporalInputPixelSize = FVector2f(1.0f) / FVector2f(InputViewport.Extent);

	PassParameters->HistoryTexture = HistoryColor.Texture;
	PassParameters->HistoryTextureSampler = TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border, 0, 0, 0xffffffff >::GetRHI();
	PassParameters->HistoryTextureSize = HistoryTextureSize;
	PassParameters->HistoryTexturePixelSize = HistoryTexturePixelSize;

	PassParameters->ZCurrTexture		= SceneDepth.Texture;
	PassParameters->ZCurrTextureSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

	PassParameters->SceneVelocityTexture = SceneVelocity.Texture;
	PassParameters->SceneVelocityTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();

	PassParameters->OutTexture = GraphBuilder.CreateUAV(OutputAO.Texture);

	TShaderMapRef<FGTAOTemporalFilterCS> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("GTAOTemporalFilterCS %dx%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
		ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), 8));

	FGTAOTemporalOutputs Outputs;
	Outputs.OutputAO = OutputAO;
	Outputs.TargetExtent = OutputViewport.Extent;
	Outputs.ViewportRect = OutputViewport.Rect;

	return MoveTemp(Outputs);
}

//----------------------------------------------------------------------------------------------------------------------

class FGTAOSpatialFilterCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGTAOSpatialFilterCS);
	SHADER_USE_PARAMETER_STRUCT(FGTAOSpatialFilterCS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), 8);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), 8);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSSAOShaderParameters, SSAOParameters)

		SHADER_PARAMETER(FIntPoint, GTAOSpatialFilterExtents)
		SHADER_PARAMETER(FVector4f, GTAOSpatialFilterParams)
		SHADER_PARAMETER(FVector4f, GTAOSpatialFilterWidth)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GTAOSpatialFilterTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GTAOSpatialFilterDepthTexture)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FGTAOSpatialFilterCS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "GTAOSpatialFilterCS", SF_Compute);

FScreenPassTexture AddGTAOSpatialFilter(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture Input,
	FScreenPassTexture InputDepth,
	FScreenPassRenderTarget SuggestedOutput)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, GTAO_SpatialFilter, "GTAO Spatial Filter");

	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(InputViewport);

	FScreenPassRenderTarget Output = SuggestedOutput;
	if (!Output.IsValid())
	{
		FRDGTextureDesc OutputDesc = Input.Texture->Desc;
		OutputDesc.Reset();
		OutputDesc.Format = PF_G8;
		OutputDesc.ClearValue = FClearValueBinding::None;
		OutputDesc.Flags &= ~TexCreate_DepthStencilTargetable;
		OutputDesc.Flags |= TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV;
		OutputDesc.Extent = OutputViewport.Extent;

		Output.Texture = GraphBuilder.CreateTexture(OutputDesc, TEXT("GTAOFilter"));
		Output.ViewRect = OutputViewport.Rect;
		Output.LoadAction = ERenderTargetLoadAction::ENoAction;
		Output.UpdateVisualizeTextureExtent();
	}

	FGTAOSpatialFilterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGTAOSpatialFilterCS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneTextures = CommonParameters.SceneTextures;
	PassParameters->SSAOParameters = GetSSAOShaderParameters(View, InputViewport, OutputViewport, CommonParameters.SceneTexturesViewport, EAOTechnique::GTAO);

	PassParameters->GTAOSpatialFilterExtents = OutputViewport.Rect.Size();


	FVector4f FilterWidthParamsValue(0.0f, 0.0f, 0.0f, 0.0f);
	float FilterWidth = CVarGTAOFilterWidth.GetValueOnRenderThread();

	if (FilterWidth == 3.0f)
	{
		FilterWidthParamsValue.X = -1.0f;
		FilterWidthParamsValue.Y = 1.0f;
	}
	else if (FilterWidth == 4.0f)
	{
		FilterWidthParamsValue.X = -1.0f;
		FilterWidthParamsValue.Y = 2.0f;
	}
	else
	{
		FilterWidthParamsValue.X = -2.0f;
		FilterWidthParamsValue.Y = 2.0f;
	}
	PassParameters->GTAOSpatialFilterWidth = FilterWidthParamsValue;

	float DownsampleFactor = 1.0;
	FVector4f FilterParamsValue((float)DownsampleFactor, 0.0f, 0.0f, 0.0f); // JDW TODO
	PassParameters->GTAOSpatialFilterParams = FilterParamsValue;

	PassParameters->GTAOSpatialFilterTexture = Input.Texture;
	PassParameters->GTAOSpatialFilterDepthTexture = InputDepth.Texture;

	PassParameters->OutTexture = GraphBuilder.CreateUAV(Output.Texture);

	TShaderMapRef<FGTAOSpatialFilterCS> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("GTAOSpatialFilterCS %dx%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
		CommonParameters.GTAOType == EGTAOType::EAsyncCombinedSpatial ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), 8));

	return MoveTemp(Output);
}

//----------------------------------------------------------------------------------------------------------------------

class FGTAOUpsamplePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGTAOUpsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FGTAOUpsamplePS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_SHADER"), 0);
		OutEnvironment.SetDefine(TEXT("UPSAMPLE_FILTER_SHADER"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GTAOUpsampleTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, GTAOUpsampleSampler)
		SHADER_PARAMETER(FVector2f, GTAOUpsamplePixelSize)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();
};
IMPLEMENT_GLOBAL_SHADER(FGTAOUpsamplePS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "GTAOUpsamplePS", SF_Pixel);

FScreenPassTexture AddGTAOUpsamplePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture Input,
	FScreenPassTexture SceneDepth,
	FScreenPassRenderTarget Output)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, GTAO_Upsample, "GTAO Upsample");

	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(Output);

	// Pixel Shader
	FGTAOUpsamplePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGTAOUpsamplePS::FParameters>();

	PassParameters->GTAOUpsampleTexture = Input.Texture;
	PassParameters->GTAOUpsampleSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->GTAOUpsamplePixelSize = FVector2f(1.0f) / FVector2f(InputViewport.Extent);

	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FGTAOUpsamplePS> PixelShader(View.ShaderMap);
	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("GTAOUpsamplePS %dx%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
		View,
		OutputViewport,
		InputViewport,
		VertexShader,
		PixelShader,
		TStaticBlendState<>::GetRHI(),
		TStaticDepthStencilState<false, CF_Always>::GetRHI(),
		PassParameters);

	return MoveTemp(Output);
}
