// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessAmbientOcclusion.h: Post processing ambient occlusion implementation.
=============================================================================*/

#pragma once

#include "ScreenPass.h"
#include "UniformBuffer.h"
#include "RendererInterface.h"
#include "HZB.h"
#include "SceneTexturesConfig.h"

class FViewInfo;

struct FRDGSystemTextures;

enum class ESSAOType
{
	// pixel shader
	EPS,
	// non async compute shader
	ECS,
	// async compute shader
	EAsyncCS,
};

enum class EGTAOType
{
	// Not on (use legacy if at all)
	EOff,

	// Async compute shader where the Horizon Search and Inner Integrate are combined and the spatial filter is run on the Async Pipe
	// Temporal and Upsample are run on the GFX Pipe as Temporal requires velocity Buffer
	EAsyncCombinedSpatial,

	// Async compute shader where the Horizon Search is run on the Async compute pipe
	// Integrate, Spatial, Temporal and Upsample are run on the GFX pipe as these require GBuffer channels
	EAsyncHorizonSearch,

	// Non async version where all passes are run on the GFX Pipe
	ENonAsync,
};

enum EGTAOPass
{
	EGTAOPass_None						= 0x0,
	EGTAOPass_HorizonSearch				= 0x1,
	EGTAOPass_HorizonSearchIntegrate	= 0x2,
	EGTAOPass_Integrate					= 0x4,
	EGTAOPass_SpatialFilter				= 0x8,
	EGTAOPass_TemporalFilter			= 0x10,
	EGTAOPass_Upsample					= 0x20,
};

FRDGTextureDesc GetScreenSpaceAOTextureDesc(ERHIFeatureLevel::Type FeatureLevel, FIntPoint Extent);

FRDGTextureRef CreateScreenSpaceAOTexture(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FIntPoint Extent);

FRDGTextureRef GetScreenSpaceAOFallback(const FRDGSystemTextures& SystemTextures);

class FGTAOContext
{
public:
	EGTAOType GTAOType;
	uint32 FinalPass;
	uint32 DownsampleFactor;

	bool bUseNormals;
	bool bHalfRes;
	bool bHasSpatialFilter;
	bool bHasTemporalFilter;

	FGTAOContext(EGTAOType Type);
	FGTAOContext();

	bool IsFinalPass(EGTAOPass);
};


class FSSAOHelper
{
public:

	// Utility functions for deciding AO logic.
	// for render thread
	// @return usually in 0..100 range but could be outside, combines the view with the cvar setting
	static float GetAmbientOcclusionQualityRT(const FSceneView& View);

	// @return returns actual shader quality level to use. 0-4 currently.
	static int32 GetAmbientOcclusionShaderLevel(const FSceneView& View);

	// @return whether AmbientOcclusion should run a compute shader.
	static bool IsAmbientOcclusionCompute();

	static int32 GetNumAmbientOcclusionLevels();
	static float GetAmbientOcclusionStepMipLevelFactor();
	static EAsyncComputeBudget GetAmbientOcclusionAsyncComputeBudget();

	static bool IsBasePassAmbientOcclusionRequired(const FViewInfo& View);
	static bool IsAmbientOcclusionAsyncCompute(const FViewInfo& View, uint32 AOPassCount);

	// @return 0:off, 0..3
	static uint32 ComputeAmbientOcclusionPassCount(const FViewInfo& View);

	static EGTAOType GetGTAOPassType(const FViewInfo& View, uint32 Levels);

	// @return Whether depth bounds test for AO is enabled
	static bool IsDepthBoundsTestEnabled();
};

// SSAO

struct FSSAOCommonParameters
{
	FSceneTextureShaderParameters SceneTextures;
	FScreenPassTextureViewport SceneTexturesViewport;

	FScreenPassTexture GBufferA;
	FScreenPassTexture SceneDepth;

	uint32 Levels = 1;
	int32 ShaderQuality = 4;
	int32 MobileHalfResolutionSetting = 0;
	uint32 MobileShadingModelsExcludeMask = 0;
	ESSAOType DownscaleType = ESSAOType::EPS;
	ESSAOType FullscreenType = ESSAOType::EPS;
	bool bNeedSmoothingPass = true;
};

FScreenPassTexture AddAmbientOcclusionSetupPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	FScreenPassTexture Input);

FScreenPassTexture AddAmbientOcclusionStepPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	const FScreenPassTexture& Input0,
	const FScreenPassTexture& Input1,
	const FScreenPassTexture& Input2);

FScreenPassTexture AddAmbientOcclusionFinalPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	const FScreenPassTexture& Input0,
	const FScreenPassTexture& Input1,
	const FScreenPassTexture& Input2,
	FScreenPassRenderTarget FinalOutput);

// GTAO

struct FGTAOCommonParameters
{
	FSceneTextureShaderParameters SceneTextures;
	FScreenPassTextureViewport SceneTexturesViewport;
	
	FScreenPassTexture SceneDepth;
	FScreenPassTexture SceneVelocity;

	FIntRect DownsampledViewRect;

	int32 ShaderQuality = 4;
	uint32 DownscaleFactor = 1;
	EGTAOType GTAOType = EGTAOType::EOff;
};

struct FGTAOHorizonSearchOutputs
{
	FScreenPassTexture Color;
};

FGTAOHorizonSearchOutputs AddGTAOHorizonSearchPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture SceneDepth,
	FScreenPassRenderTarget HorizonOutput);

FScreenPassTexture AddGTAOInnerIntegratePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture SceneDepth,
	FScreenPassTexture HorizonsTexture);

FGTAOHorizonSearchOutputs AddGTAOHorizonSearchIntegratePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture SceneDepth);

struct FGTAOTemporalOutputs
{
	FScreenPassRenderTarget OutputAO;

	FIntPoint TargetExtent;
	FIntRect ViewportRect;
};

FGTAOTemporalOutputs AddGTAOTemporalPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture Input,
	FScreenPassTexture SceneDepth,
	FScreenPassTexture SceneVelocity,
	FScreenPassTexture HistoryColor,
	FScreenPassTextureViewport HistoryViewport);

FScreenPassTexture AddGTAOSpatialFilter(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGTAOCommonParameters& CommonParameters,
	FScreenPassTexture Input,
	FScreenPassTexture InputDepth,
	FScreenPassRenderTarget SuggestedOutput = FScreenPassRenderTarget());

FScreenPassTexture AddGTAOUpsamplePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture Input,
	FScreenPassTexture SceneDepth,
	FScreenPassRenderTarget Output);

BEGIN_SHADER_PARAMETER_STRUCT(FTextureBinding, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Texture)
	SHADER_PARAMETER(FIntPoint, TextureSize)
	SHADER_PARAMETER(FVector2f, InverseTextureSize)
END_SHADER_PARAMETER_STRUCT();

enum class EAOTechnique
{
	SSAO,
	GTAO,
};

FHZBParameters GetHZBParametersForAO(FRDGBuilder& GraphBuilder, const FViewInfo& View, FIntPoint InputTextureSize, EAOTechnique AOTechnique);
