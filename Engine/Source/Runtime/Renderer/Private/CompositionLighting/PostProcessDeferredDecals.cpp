// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "CompositionLighting/PostProcessAmbientOcclusion.h"

#include "ClearQuad.h"
#include "DBufferTextures.h"
#include "DecalRenderingShared.h"
#include "PipelineStateCache.h"
#include "PostProcess/SceneRenderTargets.h"
#include "RendererUtils.h"
#include "SceneUtils.h"
#include "ScenePrivate.h"
#include "SceneProxies/DeferredDecalProxy.h"
#include "SystemTextures.h"
#include "VelocityRendering.h"
#include "VisualizeTexture.h"
#include "RenderCore.h"
#include "VariableRateShadingImageManager.h"
#include "PSOPrecacheValidation.h"
#include "Rendering/CustomRenderPass.h"
#include "SceneViewState.h"
#include "LocalVertexFactory.h"

static TAutoConsoleVariable<float> CVarStencilSizeThreshold(
	TEXT("r.Decal.StencilSizeThreshold"),
	0.1f,
	TEXT("Control a per decal stencil pass that allows to large (screen space) decals faster. It adds more overhead per decals so this\n")
	TEXT("  <0: optimization is disabled\n")
	TEXT("   0: optimization is enabled no matter how small (screen space) the decal is\n")
	TEXT("0..1: optimization is enabled, value defines the minimum size (screen space) to trigger the optimization (default 0.1)")
);

static TAutoConsoleVariable<float> CVarDBufferDecalNormalReprojectionThresholdLow(
	TEXT("r.Decal.NormalReprojectionThresholdLow"),
	0.990f, 
	TEXT("When reading the normal from a SceneTexture node in a DBuffer decal shader, ")
	TEXT("the normal is a mix of the geometry normal (extracted from the depth buffer) and the normal from the reprojected ")
	TEXT("previous frame. When the dot product of the geometry and reprojected normal is below the r.Decal.NormalReprojectionThresholdLow, ")
	TEXT("the geometry normal is used. When that value is above r.Decal.NormalReprojectionThresholdHigh, the reprojected ")
	TEXT("normal is used. Otherwise it uses a lerp between them."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarDBufferDecalNormalReprojectionThresholdHigh(
	TEXT("r.Decal.NormalReprojectionThresholdHigh"),
	0.995f, 
	TEXT("When reading the normal from a SceneTexture node in a DBuffer decal shader, ")
	TEXT("the normal is a mix of the geometry normal (extracted from the depth buffer) and the normal from the reprojected ")
	TEXT("previous frame. When the dot product of the geometry and reprojected normal is below the r.Decal.NormalReprojectionThresholdLow, ")
	TEXT("the geometry normal is used. When that value is above r.Decal.NormalReprojectionThresholdHigh, the reprojected ")
	TEXT("normal is used. Otherwise it uses a lerp between them."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarDBufferDecalNormalReprojectionEnabled(
	TEXT("r.Decal.NormalReprojectionEnabled"),
	false, 
	TEXT("If true, normal reprojection from the previous frame is allowed in SceneTexture nodes on DBuffer decals, provided that motion ")
	TEXT("in depth prepass is enabled as well (r.VelocityOutputPass=0). Otherwise the fallback is the normal extracted from the depth buffer."),
	ECVF_RenderThreadSafe);

bool AreDecalsEnabled(const FViewInfo& View)
{
	const FSceneViewFamily& ViewFamily = *View.Family;
	return ViewFamily.EngineShowFlags.Decals && !ViewFamily.EngineShowFlags.VisualizeLightCulling && !ViewFamily.EngineShowFlags.HitProxies &&
		(!View.CustomRenderPass || View.CustomRenderPass->GetRenderMode() == FCustomRenderPassBase::ERenderMode::DepthAndBasePass);
}

bool IsDBufferEnabled(const FSceneViewFamily& ViewFamily, EShaderPlatform ShaderPlatform)
{
	return IsUsingDBuffers(ShaderPlatform) && !ViewFamily.EngineShowFlags.ShaderComplexity;
}

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FDecalPassUniformParameters, "DecalPass", SceneTextures);

FDeferredDecalPassTextures GetDeferredDecalPassTextures(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View,
	const FSceneTextures& SceneTextures, 
	FDBufferTextures* DBufferTextures,
	EDecalRenderStage DecalRenderStage)
{
	FDeferredDecalPassTextures PassTextures;

	auto* Parameters = GraphBuilder.AllocParameters<FDecalPassUniformParameters>(); //


	const bool bIsMobile = (View.GetFeatureLevel() == ERHIFeatureLevel::ES3_1);
	ESceneTextureSetupMode TextureReadAccess = ESceneTextureSetupMode::None;
	EMobileSceneTextureSetupMode MobileTextureReadAccess = EMobileSceneTextureSetupMode::None;
	if (bIsMobile)
	{
		MobileTextureReadAccess = EMobileSceneTextureSetupMode::SceneDepth | EMobileSceneTextureSetupMode::CustomDepth;
	}
	else
	{
		TextureReadAccess = ESceneTextureSetupMode::GBufferA | ESceneTextureSetupMode::SceneDepth | ESceneTextureSetupMode::CustomDepth;
	}

	SetupSceneTextureUniformParameters(GraphBuilder, &SceneTextures, View.FeatureLevel, TextureReadAccess, Parameters->SceneTextures);
	SetupMobileSceneTextureUniformParameters(GraphBuilder, &SceneTextures, MobileTextureReadAccess, Parameters->MobileSceneTextures);
	Parameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));

	// When using SubstrateDbufferPass, the DBuffer are rendered after the BasePass. In this case, the toplayer data can be piped to the DBuffer base pass for normal reading.
	if (DecalRenderStage == EDecalRenderStage::Emissive || (DecalRenderStage == EDecalRenderStage::BeforeBasePass && Substrate::IsDBufferPassEnabled(View.GetShaderPlatform())))
	{
		Substrate::BindSubstratePublicGlobalUniformParameters(GraphBuilder, SceneTextures.SubstrateSceneData.Get(), Parameters->SubstratePublic);
	}
	else
	{
		Substrate::BindSubstratePublicGlobalUniformParameters(GraphBuilder, nullptr, Parameters->SubstratePublic); // nullptr for default
	}
	PassTextures.DecalPassUniformBuffer = GraphBuilder.CreateUniformBuffer(Parameters);

	PassTextures.Depth = SceneTextures.Depth;
	PassTextures.Color = SceneTextures.Color.Target;

	// Mobile deferred renderer does not use dbuffer 
	if (!bIsMobile)
	{
		PassTextures.GBufferA = SceneTextures.GBufferA;
		PassTextures.GBufferB = SceneTextures.GBufferB;
		PassTextures.GBufferC = SceneTextures.GBufferC;
		PassTextures.GBufferE = SceneTextures.GBufferE;
	}

	PassTextures.DBufferTextures = DBufferTextures;

	return PassTextures;
}

void GetDeferredDecalRenderTargetsInfo(
	const FSceneTexturesConfig& Config,
	EDecalRenderTargetMode RenderTargetMode,
	FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo)
{
	const FGBufferBindings& Bindings = Config.GBufferBindings[GBL_Default];
	switch (RenderTargetMode)
	{
	case EDecalRenderTargetMode::SceneColorAndGBuffer:
		AddRenderTargetInfo(Config.ColorFormat, Config.ColorCreateFlags, RenderTargetsInfo);
		AddRenderTargetInfo(Bindings.GBufferA.Format, Bindings.GBufferA.Flags, RenderTargetsInfo);
		AddRenderTargetInfo(Bindings.GBufferB.Format, Bindings.GBufferB.Flags, RenderTargetsInfo);
		AddRenderTargetInfo(Bindings.GBufferC.Format, Bindings.GBufferC.Flags, RenderTargetsInfo);
		break;
	case EDecalRenderTargetMode::SceneColorAndGBufferNoNormal:
		AddRenderTargetInfo(Config.ColorFormat, Config.ColorCreateFlags, RenderTargetsInfo);
		AddRenderTargetInfo(Bindings.GBufferB.Format, Bindings.GBufferB.Flags, RenderTargetsInfo);
		AddRenderTargetInfo(Bindings.GBufferC.Format, Bindings.GBufferC.Flags, RenderTargetsInfo);
		break;
	case EDecalRenderTargetMode::SceneColor:
		AddRenderTargetInfo(Config.ColorFormat, Config.ColorCreateFlags, RenderTargetsInfo);
		break;

	case EDecalRenderTargetMode::DBuffer:
	{
		const FDBufferTexturesDesc DBufferTexturesDesc = GetDBufferTexturesDesc(Config.Extent, Config.ShaderPlatform);

		AddRenderTargetInfo(DBufferTexturesDesc.DBufferADesc.Format, DBufferTexturesDesc.DBufferADesc.Flags, RenderTargetsInfo);
		AddRenderTargetInfo(DBufferTexturesDesc.DBufferBDesc.Format, DBufferTexturesDesc.DBufferBDesc.Flags, RenderTargetsInfo);
		AddRenderTargetInfo(DBufferTexturesDesc.DBufferCDesc.Format, DBufferTexturesDesc.DBufferCDesc.Flags, RenderTargetsInfo);

		if (DBufferTexturesDesc.DBufferMaskDesc.Format != PF_Unknown)
		{
			AddRenderTargetInfo(DBufferTexturesDesc.DBufferMaskDesc.Format, DBufferTexturesDesc.DBufferMaskDesc.Flags, RenderTargetsInfo);
		}
		break;
	}
	case EDecalRenderTargetMode::AmbientOcclusion:
	{		
		const FRDGTextureDesc AOTextureDesc = GetScreenSpaceAOTextureDesc(Config.FeatureLevel, Config.Extent);
		AddRenderTargetInfo(AOTextureDesc.Format, AOTextureDesc.Flags, RenderTargetsInfo);
		break;
	}

	default:
		checkNoEntry();
	}

	if (Config.bRequiresDepthAux)
	{
		switch (RenderTargetMode)
		{
		case EDecalRenderTargetMode::SceneColorAndGBuffer:
		case EDecalRenderTargetMode::SceneColorAndGBufferNoNormal:
		case EDecalRenderTargetMode::SceneColor:
			AddRenderTargetInfo(Config.bPreciseDepthAux ? PF_R32_FLOAT : PF_R16F, TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead, RenderTargetsInfo);	
		};
	}
	if (Config.bCustomResolveSubpass)
	{
		// resolve target as an additional color attachment
		AddRenderTargetInfo(IsAndroidPlatform(Config.ShaderPlatform) ? PF_R8G8B8A8 : PF_B8G8R8A8, TexCreate_RenderTargetable | TexCreate_ShaderResource, RenderTargetsInfo);
	}

	RenderTargetsInfo.NumSamples = Config.NumSamples;

	SetupDepthStencilInfo(PF_DepthStencil, Config.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite, RenderTargetsInfo);
}

void GetDeferredDecalPassParameters(
	FRDGBuilder &GraphBuilder,
	const FViewInfo& View,
	const FDeferredDecalPassTextures& Textures,
	EDecalRenderStage DecalRenderStage,
	EDecalRenderTargetMode RenderTargetMode,
	FDeferredDecalPassParameters& PassParameters)
{
	PassParameters.View = View.GetShaderParameters();
	PassParameters.Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
	PassParameters.DeferredDecal = CreateDeferredDecalUniformBuffer(View);
	PassParameters.DecalPass = Textures.DecalPassUniformBuffer;
	
	FRDGTextureRef DepthTexture = Textures.Depth.Target;

	FRenderTargetBindingSlots& RenderTargets = PassParameters.RenderTargets;
	PassParameters.RenderTargets.ShadingRateTexture = GVRSImageManager.GetVariableRateShadingImage(GraphBuilder, View, FVariableRateShadingImageManager::EVRSPassType::Decals);
	PassParameters.RenderTargets.MultiViewCount = View.GetMultiViewCount();
	uint32 ColorTargetIndex = 0;

	const auto AddColorTarget = [&](FRDGTextureRef Texture, ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ELoad, FRDGTextureRef TextureArray = nullptr, bool bIsMobileMultiView = false)
	{
		if (bIsMobileMultiView)
		{
			checkf(TextureArray, TEXT("Attempting to bind decal render targets, but the texture array is null."));
			RenderTargets[ColorTargetIndex++] = FRenderTargetBinding(TextureArray, LoadAction);
		}
		else
		{
			checkf(Texture, TEXT("Attempting to bind decal render targets, but the texture is null."));
			RenderTargets[ColorTargetIndex++] = FRenderTargetBinding(Texture, LoadAction);
		}
	};

	switch (RenderTargetMode)
	{
	case EDecalRenderTargetMode::SceneColorAndGBuffer:
		AddColorTarget(Textures.Color);
		AddColorTarget(Textures.GBufferA);
		AddColorTarget(Textures.GBufferB);
		AddColorTarget(Textures.GBufferC);
		break;
	case EDecalRenderTargetMode::SceneColorAndGBufferNoNormal:
		AddColorTarget(Textures.Color);
		AddColorTarget(Textures.GBufferB);
		AddColorTarget(Textures.GBufferC);
		break;
	case EDecalRenderTargetMode::SceneColor:
		AddColorTarget(Textures.Color);
		break;

	case EDecalRenderTargetMode::DBuffer:
	{
		check(Textures.DBufferTextures);

		FDBufferTextures& DBufferTextures = *Textures.DBufferTextures;

		const bool bUseTextureArrays = View.UsesMobileMultiViewShaders();
		const ERenderTargetLoadAction LoadAction = (bUseTextureArrays ? DBufferTextures.bDBufferTexArrayProduced : DBufferTextures.bDBufferProduced)
			? ERenderTargetLoadAction::ELoad
			: ERenderTargetLoadAction::EClear;

		if (bUseTextureArrays)
		{
			if (DBufferTextures.DBufferATexArray)
			{
				DBufferTextures.bDBufferTexArrayProduced = true;
			}
		}
		else
		{
			if (DBufferTextures.DBufferA)
			{
				DBufferTextures.bDBufferProduced = true;
			}
		}

		AddColorTarget(DBufferTextures.DBufferA, LoadAction, DBufferTextures.DBufferATexArray, bUseTextureArrays);
		AddColorTarget(DBufferTextures.DBufferB, LoadAction, DBufferTextures.DBufferBTexArray, bUseTextureArrays);
		AddColorTarget(DBufferTextures.DBufferC, LoadAction, DBufferTextures.DBufferCTexArray, bUseTextureArrays);

		const EDecalDBufferMaskTechnique DBufferMaskTechnique = GetDBufferMaskTechnique(View.Family->GetShaderPlatform());
		if (DBufferMaskTechnique == EDecalDBufferMaskTechnique::PerPixel)
		{
			AddColorTarget(DBufferTextures.DBufferMask, LoadAction);
		}

		// D-Buffer always uses the resolved depth; no MSAA.
		DepthTexture = Textures.Depth.Resolve;
		break;
	}
	case EDecalRenderTargetMode::AmbientOcclusion:
	{
		AddColorTarget(Textures.ScreenSpaceAO);
		break;
	}

	default:
		checkNoEntry();
	}

	RenderTargets.DepthStencil = FDepthStencilBinding(
		DepthTexture,
		ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad,
		FExclusiveDepthStencil::DepthRead_StencilWrite);
}

TUniformBufferRef<FDeferredDecalUniformParameters> CreateDeferredDecalUniformBuffer(const FViewInfo& View)
{
	const bool bIsMotionInDepth = FVelocityRendering::DepthPassCanOutputVelocity(View.GetFeatureLevel());
	// if we have early motion vectors (bIsMotionInDepth) and the cvar is enabled and we actually have a buffer from the previous frame (View.PrevViewInfo.GBufferA.IsValid())
	const bool bIsNormalReprojectionEnabled = (bIsMotionInDepth && CVarDBufferDecalNormalReprojectionEnabled.GetValueOnRenderThread() && View.PrevViewInfo.GBufferA.IsValid());

	FDeferredDecalUniformParameters UniformParameters;
	UniformParameters.NormalReprojectionThresholdLow  = CVarDBufferDecalNormalReprojectionThresholdLow .GetValueOnRenderThread();
	UniformParameters.NormalReprojectionThresholdHigh = CVarDBufferDecalNormalReprojectionThresholdHigh.GetValueOnRenderThread();
	UniformParameters.NormalReprojectionEnabled = bIsNormalReprojectionEnabled ? 1 : 0;
	
	// the algorithm is:
	//    value = (dot - low)/(high - low)
	// so calculate the divide in the helper to turn the math into:
	//    helper = 1.0f/(high - low)
	//    value = (dot - low)*helper;
	// also check for the case where high <= low.
	float Denom = FMath::Max(UniformParameters.NormalReprojectionThresholdHigh - UniformParameters.NormalReprojectionThresholdLow,1e-4f);
	UniformParameters.NormalReprojectionThresholdScaleHelper = 1.0f / Denom;

	UniformParameters.PreviousFrameNormal = (bIsNormalReprojectionEnabled) ? View.PrevViewInfo.GBufferA->GetRHI() : GSystemTextures.BlackDummy->GetRHI();

	UniformParameters.NormalReprojectionJitter = FVector2f(View.PrevViewInfo.ViewMatrices.GetTemporalAAJitter());

	return TUniformBufferRef<FDeferredDecalUniformParameters>::CreateUniformBufferImmediate(UniformParameters, UniformBuffer_SingleFrame);
}

enum EDecalDepthInputState
{
	DDS_Undefined,
	DDS_Always,
	DDS_DepthTest,
	DDS_DepthAlways_StencilEqual1,
	DDS_DepthAlways_StencilEqual1_IgnoreMask,
	DDS_DepthAlways_StencilEqual0,
	DDS_DepthTest_StencilEqual1,
	DDS_DepthTest_StencilEqual1_IgnoreMask,
	DDS_DepthTest_StencilEqual0,
};

struct FDecalDepthState
{
	EDecalDepthInputState DepthTest;
	bool bDepthOutput;

	FDecalDepthState()
		: DepthTest(DDS_Undefined)
		, bDepthOutput(false)
	{
	}

	bool operator !=(const FDecalDepthState &rhs) const
	{
		return DepthTest != rhs.DepthTest || bDepthOutput != rhs.bDepthOutput;
	}
};

static bool RenderPreStencil(FRHICommandList& RHICmdList, const FViewInfo& View, const FMatrix& ComponentToWorldMatrix, const FMatrix& FrustumComponentToClip)
{
	float Distance = (View.ViewMatrices.GetViewOrigin() - ComponentToWorldMatrix.GetOrigin()).Size();
	float Radius = ComponentToWorldMatrix.GetMaximumAxisScale();

	// if not inside
	if (Distance > Radius)
	{
		float EstimatedDecalSize = Radius / Distance;

		float StencilSizeThreshold = CVarStencilSizeThreshold.GetValueOnRenderThread();

		// Check if it's large enough on screen
		if (EstimatedDecalSize < StencilSizeThreshold)
		{
			return false;
		}
	}

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	// Set states, the state cache helps us avoiding redundant sets
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();

	// all the same to have DX10 working
	GraphicsPSOInit.BlendState = TStaticBlendState<
		CW_NONE, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Emissive
		CW_NONE, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
		CW_NONE, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
		CW_NONE, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One		// BaseColor
	>::GetRHI();

	// Carmack's reverse the sandbox stencil bit on the bounds
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
		false, CF_LessEqual,
		true, CF_Always, SO_Keep, SO_Keep, SO_Invert,
		true, CF_Always, SO_Keep, SO_Keep, SO_Invert,
		STENCIL_SANDBOX_MASK, STENCIL_SANDBOX_MASK
	>::GetRHI();

	DecalRendering::SetVertexShaderOnly(RHICmdList, GraphicsPSOInit, View, FrustumComponentToClip);

	// Set stream source after updating cached strides
	RHICmdList.SetStreamSource(0, GetUnitCubeVertexBuffer(), 0);

	// Render decal mask
	uint32 InstanceCount = View.UsesInstancedStereoShaders() ? 1 : View.GetStereoPassInstanceFactor();
	RHICmdList.DrawIndexedPrimitive(GetUnitCubeIndexBuffer(), 0, 0, 8, 0, UE_ARRAY_COUNT(GCubeIndices) / 3, InstanceCount);

	return true;
}

static FDecalDepthState ComputeDecalDepthState(EDecalRenderStage LocalDecalStage, bool bInsideDecal, bool bThisDecalUsesStencil)
{
	FDecalDepthState Ret;

	Ret.bDepthOutput = false;

	const bool bUseDecalMask = 
		LocalDecalStage == EDecalRenderStage::BeforeLighting || 
		LocalDecalStage == EDecalRenderStage::Emissive || 
		LocalDecalStage == EDecalRenderStage::AmbientOcclusion;

	if (bInsideDecal)
	{
		if (bThisDecalUsesStencil)
		{
			Ret.DepthTest = bUseDecalMask ? DDS_DepthAlways_StencilEqual1 : DDS_DepthAlways_StencilEqual1_IgnoreMask;
		}
		else
		{
			Ret.DepthTest = bUseDecalMask ? DDS_DepthAlways_StencilEqual0 : DDS_Always;
		}
	}
	else
	{
		if (bThisDecalUsesStencil)
		{
			Ret.DepthTest = bUseDecalMask ? DDS_DepthTest_StencilEqual1 : DDS_DepthTest_StencilEqual1_IgnoreMask;
		}
		else
		{
			Ret.DepthTest = bUseDecalMask ? DDS_DepthTest_StencilEqual0 : DDS_DepthTest;
		}
	}

	return Ret;
}

static FRHIDepthStencilState* GetDecalDepthState(uint32& StencilRef, FDecalDepthState DecalDepthState)
{
	switch (DecalDepthState.DepthTest)
	{
	case DDS_DepthAlways_StencilEqual1:
		check(!DecalDepthState.bDepthOutput);			// todo
		StencilRef = STENCIL_SANDBOX_MASK | GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1);
		return TStaticDepthStencilState<
			false, CF_Always,
			true, CF_Equal, SO_Zero, SO_Zero, SO_Zero,
			true, CF_Equal, SO_Zero, SO_Zero, SO_Zero,
			STENCIL_SANDBOX_MASK | GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1), STENCIL_SANDBOX_MASK>::GetRHI();

	case DDS_DepthAlways_StencilEqual1_IgnoreMask:
		check(!DecalDepthState.bDepthOutput);			// todo
		StencilRef = STENCIL_SANDBOX_MASK;
		return TStaticDepthStencilState<
			false, CF_Always,
			true, CF_Equal, SO_Zero, SO_Zero, SO_Zero,
			true, CF_Equal, SO_Zero, SO_Zero, SO_Zero,
			STENCIL_SANDBOX_MASK, STENCIL_SANDBOX_MASK>::GetRHI();

	case DDS_DepthAlways_StencilEqual0:
		check(!DecalDepthState.bDepthOutput);			// todo
		StencilRef = GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1);
		return TStaticDepthStencilState<
			false, CF_Always,
			true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			STENCIL_SANDBOX_MASK | GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1), 0x00>::GetRHI();

	case DDS_Always:
		check(!DecalDepthState.bDepthOutput);			// todo 
		StencilRef = 0;
		return TStaticDepthStencilState<false, CF_Always>::GetRHI();

	case DDS_DepthTest_StencilEqual1:
		check(!DecalDepthState.bDepthOutput);			// todo
		StencilRef = STENCIL_SANDBOX_MASK | GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1);
		return TStaticDepthStencilState<
			false, CF_DepthNearOrEqual,
			true, CF_Equal, SO_Zero, SO_Zero, SO_Zero,
			true, CF_Equal, SO_Zero, SO_Zero, SO_Zero,
			STENCIL_SANDBOX_MASK | GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1), STENCIL_SANDBOX_MASK>::GetRHI();

	case DDS_DepthTest_StencilEqual1_IgnoreMask:
		check(!DecalDepthState.bDepthOutput);			// todo
		StencilRef = STENCIL_SANDBOX_MASK;
		return TStaticDepthStencilState<
			false, CF_DepthNearOrEqual,
			true, CF_Equal, SO_Zero, SO_Zero, SO_Zero,
			true, CF_Equal, SO_Zero, SO_Zero, SO_Zero,
			STENCIL_SANDBOX_MASK, STENCIL_SANDBOX_MASK>::GetRHI();

	case DDS_DepthTest_StencilEqual0:
		check(!DecalDepthState.bDepthOutput);			// todo
		StencilRef = GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1);
		return TStaticDepthStencilState<
			false, CF_DepthNearOrEqual,
			true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			STENCIL_SANDBOX_MASK | GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1), 0x00>::GetRHI();

	case DDS_DepthTest:
		if (DecalDepthState.bDepthOutput)
		{
			StencilRef = 0;
			return TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
		}
		else
		{
			StencilRef = 0;
			return TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
		}

	default:
		check(0);
		return nullptr;
	}
}

static bool IsStencilOptimizationAvailable(EDecalRenderStage RenderStage)
{
	return RenderStage == EDecalRenderStage::BeforeLighting || RenderStage == EDecalRenderStage::BeforeBasePass || RenderStage == EDecalRenderStage::Emissive;
}

static const TCHAR* GetStageName(EDecalRenderStage Stage)
{
	switch (Stage)
	{
	case EDecalRenderStage::BeforeBasePass: return TEXT("BeforeBasePass");
	case EDecalRenderStage::BeforeLighting: return TEXT("BeforeLighting");
	case EDecalRenderStage::Mobile: return TEXT("Mobile");
	case EDecalRenderStage::MobileBeforeLighting: return TEXT("MobileBeforeLighting");
	case EDecalRenderStage::Emissive: return TEXT("Emissive");
	case EDecalRenderStage::AmbientOcclusion: return TEXT("AmbientOcclusion");
	}
	return TEXT("<UNKNOWN>");
}

void CollectDeferredDecalPassPSOInitializers(
	int32 PSOCollectorIndex,
	ERHIFeatureLevel::Type FeatureLevel,
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	EDecalRenderStage DecalRenderStage,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	const FDecalBlendDesc DecalBlendDesc = DecalRendering::ComputeDecalBlendDesc(ShaderPlatform, Material);
	EDecalRenderTargetMode DecalRenderTargetMode = DecalRendering::GetRenderTargetMode(ShaderPlatform, DecalBlendDesc, DecalRenderStage);


	TShaderRef<FShader> VertexShader, PixelShader;
	if (!DecalRendering::GetShaders(FeatureLevel, Material, DecalRenderStage, VertexShader, PixelShader))
	{
		return;
	}

	if (IsPSOShaderPreloadingEnabled())
	{
		FPSOPrecacheData PSOPrecacheData;
		PSOPrecacheData.bRequired = true;
		PSOPrecacheData.Type = FPSOPrecacheData::EType::Graphics;
		PSOPrecacheData.ShaderPreloadData.Shaders.Add(VertexShader);
		PSOPrecacheData.ShaderPreloadData.Shaders.Add(PixelShader);
#if PSO_PRECACHING_VALIDATE		
		PSOPrecacheData.ResourceName = Material.GetAssetName();
		PSOPrecacheData.PSOCollectorIndex = PSOCollectorIndex;
		PSOPrecacheData.VertexFactoryType = nullptr;
#endif // PSO_PRECACHING_VALIDATE	

		PSOInitializers.Add(MoveTemp(PSOPrecacheData));
		return;
	}

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;		
	GraphicsPSOInit.BlendState = DecalRendering::GetDecalBlendState(DecalBlendDesc, DecalRenderStage, DecalRenderTargetMode);

	if (!DecalRendering::SetupShaderState(FeatureLevel, Material, DecalRenderStage, GraphicsPSOInit.BoundShaderState, false))
	{
		return;
	}

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	GetDeferredDecalRenderTargetsInfo(SceneTexturesConfig, DecalRenderTargetMode, RenderTargetsInfo);
	ApplyTargetsInfo(GraphicsPSOInit, RenderTargetsInfo);

	if (FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		// subpass info set during the submission of the draws in a mobile renderer
		GraphicsPSOInit.SubpassHint = GetSubpassHint(SceneTexturesConfig.ShaderPlatform, SceneTexturesConfig.bIsUsingGBuffers, SceneTexturesConfig.bRequireMultiView, SceneTexturesConfig.NumSamples);
		// all decals use second sub-pass on mobile
		GraphicsPSOInit.SubpassIndex = (GraphicsPSOInit.SubpassHint != ESubpassHint::None) ? 1 : 0;
		
	}
		
	const auto AddDeferredDecalPSO = [&](bool bInsideDecal,	bool bReverseHanded, bool bReverseCulling, bool bDecalUsesStencil)
	{
		const EDecalRasterizerState DecalRasterizerState = DecalRendering::GetDecalRasterizerState(bInsideDecal, bReverseHanded, bReverseCulling);
		GraphicsPSOInit.RasterizerState = DecalRendering::GetDecalRasterizerState(DecalRasterizerState);

		uint32 StencilRef = 0;
		const FDecalDepthState DecalDepthState = ComputeDecalDepthState(DecalRenderStage, bInsideDecal, bDecalUsesStencil);
		GraphicsPSOInit.DepthStencilState = GetDecalDepthState(StencilRef, DecalDepthState);

		GraphicsPSOInit.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(GraphicsPSOInit);

		FPSOPrecacheData PSOPrecacheData;
		PSOPrecacheData.bRequired = true;
		PSOPrecacheData.Type = FPSOPrecacheData::EType::Graphics;
		PSOPrecacheData.GraphicsPSOInitializer = GraphicsPSOInit;
#if PSO_PRECACHING_VALIDATE
		PSOPrecacheData.ResourceName = Material.GetAssetName();
		PSOPrecacheData.PSOCollectorIndex = PSOCollectorIndex;
		PSOPrecacheData.VertexFactoryType = nullptr;
#endif // PSO_PRECACHING_VALIDATE		

		PSOInitializers.Add(MoveTemp(PSOPrecacheData));
	};

	const auto AddDeferredDecalPSOInsideOutside = [&](bool bReverseHanded, bool bReverseCulling, bool bDecalUsesStencil)
	{
		AddDeferredDecalPSO(false /*bInsideDecal*/, bReverseHanded, bReverseCulling, bDecalUsesStencil);
		AddDeferredDecalPSO(true /*bInsideDecal*/, bReverseHanded, bReverseCulling, bDecalUsesStencil);
	};

	const auto AddDeferredDecalPSOReverseHanded = [&](bool bReverseCulling, bool bDecalUsesStencil)
	{
		AddDeferredDecalPSOInsideOutside(false /*bReverseHanded*/, bReverseCulling, bDecalUsesStencil);
		AddDeferredDecalPSOInsideOutside(true /*bReverseHanded*/, bReverseCulling, bDecalUsesStencil);
	};

	const auto AddDeferredDecalPSOReverseCulling = [&](bool bDecalUsesStencil)
	{
		AddDeferredDecalPSOReverseHanded(false /*bReverseCulling*/, bDecalUsesStencil);
		AddDeferredDecalPSOReverseHanded(true /*bReverseCulling*/, bDecalUsesStencil);
	};

	AddDeferredDecalPSOReverseCulling(false /*bDecalUsesStencil*/);	
	const bool bStencilThisDecal = IsStencilOptimizationAvailable(DecalRenderStage);
	if (bStencilThisDecal)
	{
		AddDeferredDecalPSOReverseCulling(true /*bDecalUsesStencil*/);
	}
}

void AddDeferredDecalPass(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	TConstArrayView<const FVisibleDecal*> SortedDecals,
	const FDeferredDecalPassTextures& PassTextures,
	FInstanceCullingManager& InstanceCullingManager,
	EDecalRenderStage DecalRenderStage)
{
	check(PassTextures.Depth.IsValid());
	check(DecalRenderStage != EDecalRenderStage::BeforeBasePass || PassTextures.DBufferTextures);

	const FSceneViewFamily& ViewFamily = *(View.Family);

	// Debug view framework does not yet support decals.
	// todo: Handle shader complexity mode here for deferred decal.
	if (!ViewFamily.EngineShowFlags.Decals || ViewFamily.UseDebugViewPS())
	{
		return;
	}

	const FScene& Scene = *(FScene*)ViewFamily.Scene;
	const EShaderPlatform ShaderPlatform = View.GetShaderPlatform();
	const ERHIFeatureLevel::Type FeatureLevel = View.GetFeatureLevel();
	const uint32 DecalCount = Scene.Decals.Num();
	uint32 SortedDecalCount = SortedDecals.Num();
	INC_DWORD_STAT_BY(STAT_Decals, SortedDecalCount);

	checkf(DecalRenderStage != EDecalRenderStage::AmbientOcclusion || PassTextures.ScreenSpaceAO, TEXT("Attepting to render AO decals without SSAO having emitted a valid render target."));
	checkf(DecalRenderStage != EDecalRenderStage::BeforeBasePass || IsUsingDBuffers(ShaderPlatform), TEXT("Only DBuffer decals are supported before the base pass."));

	const bool bHasAnyDrawCommandDecalCount = HasAnyDrawCommandDecalCount(DecalRenderStage, View);
	const bool bVisibleDecalsInView = SortedDecalCount > 0 || bHasAnyDrawCommandDecalCount;
	const bool bShaderComplexity = View.Family->EngineShowFlags.ShaderComplexity;
	const bool bStencilSizeThreshold = CVarStencilSizeThreshold.GetValueOnRenderThread() >= 0;

	// Attempt to clear the D-Buffer if it's appropriate for this view.
	const EDecalDBufferMaskTechnique DBufferMaskTechnique = GetDBufferMaskTechnique(ShaderPlatform);

	const auto RenderDecals = [&](uint32 DecalIndexBegin, uint32 DecalIndexEnd, EDecalRenderTargetMode RenderTargetMode)
	{
		// Sanity check - Substrate only support DBuffer, SceneColor, or AO decals
		if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(ShaderPlatform))
		{
			const bool bDecalSupported = RenderTargetMode == EDecalRenderTargetMode::DBuffer || RenderTargetMode == EDecalRenderTargetMode::SceneColor || RenderTargetMode == EDecalRenderTargetMode::AmbientOcclusion;
			if (!bDecalSupported)
			{
				return;
			}
		}

		auto* PassParameters = GraphBuilder.AllocParameters<FDeferredDecalPassParameters>();
		GetDeferredDecalPassParameters(GraphBuilder, View, PassTextures, DecalRenderStage, RenderTargetMode, *PassParameters);

		FRDGPass* Pass = GraphBuilder.AddPass(
			RDG_EVENT_NAME("Batch [%d, %d]", DecalIndexBegin, DecalIndexEnd - 1),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, FeatureLevel, ShaderPlatform, DecalIndexBegin, DecalIndexEnd, SortedDecals, DecalRenderStage, RenderTargetMode, bStencilSizeThreshold, bShaderComplexity](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

#if PSO_PRECACHING_VALIDATE			
			int32 PSOCollectorIndex = FPassProcessorManager::GetPSOCollectorIndex(EShadingPath::Deferred, DecalRendering::GetMeshPassType(RenderTargetMode));
#endif // PSO_PRECACHING_VALIDATE

			for (uint32 DecalIndex = DecalIndexBegin; DecalIndex < DecalIndexEnd; ++DecalIndex)
			{
				const FVisibleDecal& VisibleDecal = *SortedDecals[DecalIndex];
				const FMatrix ComponentToWorldMatrix = VisibleDecal.ComponentTrans.ToMatrixWithScale();
				const FMatrix FrustumComponentToClip = DecalRendering::ComputeComponentToClipMatrix(View, ComponentToWorldMatrix);
				const bool bStencilThisDecal = IsStencilOptimizationAvailable(DecalRenderStage);

				bool bThisDecalUsesStencil = false;

				if (bStencilThisDecal && bStencilSizeThreshold)
				{
					bThisDecalUsesStencil = RenderPreStencil(RHICmdList, View, ComponentToWorldMatrix, FrustumComponentToClip);
				}

				const bool bInsideDecal = ((FVector)View.ViewMatrices.GetViewOrigin() - ComponentToWorldMatrix.GetOrigin()).SizeSquared() < FMath::Square(VisibleDecal.ConservativeRadius * 1.05f + View.NearClippingDistance * 2.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

				{
					// Account for the reversal of handedness caused by negative scale on the decal
					const FVector Scale = VisibleDecal.ComponentTrans.GetScale3D();
					const bool bReverseHanded = Scale.X * Scale.Y * Scale.Z < 0.0f;
					const EDecalRasterizerState DecalRasterizerState = DecalRendering::GetDecalRasterizerState(bInsideDecal, bReverseHanded, View.bReverseCulling);
					GraphicsPSOInit.RasterizerState = DecalRendering::GetDecalRasterizerState(DecalRasterizerState);
				}

				uint32 StencilRef = 0;

				{
					const FDecalDepthState DecalDepthState = ComputeDecalDepthState(DecalRenderStage, bInsideDecal, bThisDecalUsesStencil);
					GraphicsPSOInit.DepthStencilState = GetDecalDepthState(StencilRef, DecalDepthState);
				}

				GraphicsPSOInit.BlendState = DecalRendering::GetDecalBlendState(VisibleDecal.BlendDesc, DecalRenderStage, RenderTargetMode);
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;

				DecalRendering::SetShader(RHICmdList, GraphicsPSOInit, StencilRef, View, VisibleDecal, DecalRenderStage, FrustumComponentToClip);

#if PSO_PRECACHING_VALIDATE
				if (PSOCollectorStats::IsFullPrecachingValidationEnabled())
				{
					PSOCollectorStats::CheckFullPipelineStateInCache(GraphicsPSOInit, EPSOPrecacheResult::Unknown, VisibleDecal.MaterialProxy, &FLocalVertexFactory::StaticType, nullptr, PSOCollectorIndex);
				}
#endif // PSO_PRECACHING_VALIDATE

				uint32 InstanceCount = View.UsesInstancedStereoShaders() ? 1 : View.GetStereoPassInstanceFactor();
				RHICmdList.DrawIndexedPrimitive(GetUnitCubeIndexBuffer(), 0, 0, 8, 0, UE_ARRAY_COUNT(GCubeIndices) / 3, InstanceCount);
			}
		});

		GraphBuilder.SetPassWorkload(Pass, DecalIndexEnd - DecalIndexBegin);
	};

	if (bVisibleDecalsInView)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "DeferredDecals %s", GetStageName(DecalRenderStage));

		if (bHasAnyDrawCommandDecalCount && (DecalRenderStage == EDecalRenderStage::BeforeBasePass || DecalRenderStage == EDecalRenderStage::BeforeLighting || DecalRenderStage == EDecalRenderStage::Emissive || DecalRenderStage == EDecalRenderStage::AmbientOcclusion))
		{
			// Sanity check - Substrate only support DBuffer, SceneColor, or AO decals
			bool bDecalSupported = true;
			if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(ShaderPlatform) && DecalRenderStage == EDecalRenderStage::BeforeLighting)
			{
				bDecalSupported = false;
			}

			if (bDecalSupported)
			{
				RenderMeshDecals(GraphBuilder, Scene, View, PassTextures, InstanceCullingManager, DecalRenderStage);
			}
		}

		if (SortedDecalCount > 0)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "Decals (Relevant: %d, Total: %d)", SortedDecalCount, DecalCount);

			const int32 MaxNumDecals = 128;

			uint32 NumDecals = 0;
			uint32 SortedDecalIndex = 1;
			uint32 LastSortedDecalIndex = 0;
			EDecalRenderTargetMode LastRenderTargetMode = DecalRendering::GetRenderTargetMode(ShaderPlatform, SortedDecals[0]->BlendDesc, DecalRenderStage);

			for (; SortedDecalIndex < SortedDecalCount; ++SortedDecalIndex, ++NumDecals)
			{
				const EDecalRenderTargetMode RenderTargetMode = DecalRendering::GetRenderTargetMode(ShaderPlatform, SortedDecals[SortedDecalIndex]->BlendDesc, DecalRenderStage);

				if (LastRenderTargetMode != RenderTargetMode || NumDecals > MaxNumDecals)
				{
					RenderDecals(LastSortedDecalIndex, SortedDecalIndex, LastRenderTargetMode);
					LastRenderTargetMode = RenderTargetMode;
					LastSortedDecalIndex = SortedDecalIndex;
					NumDecals = 0;
				}
			}

			if (LastSortedDecalIndex != SortedDecalIndex)
			{
				RenderDecals(LastSortedDecalIndex, SortedDecalIndex, LastRenderTargetMode);
			}
		}
	}

	// Last D-Buffer pass in the frame decodes the write mask (if supported and decals were rendered).
	if (DBufferMaskTechnique == EDecalDBufferMaskTechnique::WriteMask &&
		DecalRenderStage == EDecalRenderStage::BeforeBasePass &&
		PassTextures.DBufferTextures->IsValid() &&
		View.IsLastInFamily())
	{
		// Combine DBuffer RTWriteMasks; will end up in one texture we can load from in the base pass PS and decide whether to do the actual work or not.
		FRDGTextureRef Textures[] = { PassTextures.DBufferTextures->DBufferA, PassTextures.DBufferTextures->DBufferB, PassTextures.DBufferTextures->DBufferC };
		FRenderTargetWriteMask::Decode(GraphBuilder, View.ShaderMap, MakeArrayView(Textures), PassTextures.DBufferTextures->DBufferMask, GFastVRamConfig.DBufferMask, TEXT("DBufferMaskCombine"));
	}
}

void ExtractNormalsForNextFrameReprojection(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures, const TArray<FViewInfo>& Views)
{
	// save the previous frame if early motion vectors are enabled and normal reprojection is enabled, so there should be no cost if these options are off
	const bool bIsNormalReprojectionEnabled = CVarDBufferDecalNormalReprojectionEnabled.GetValueOnRenderThread();

	if (bIsNormalReprojectionEnabled)
	{
		for (int32 Index = 0; Index < Views.Num(); Index++)
		{
			if (FVelocityRendering::DepthPassCanOutputVelocity(Views[Index].GetFeatureLevel()))
			{
				if (Views[Index].bStatePrevViewInfoIsReadOnly == false)
				{
					GraphBuilder.QueueTextureExtraction(SceneTextures.GBufferA, &Views[Index].ViewState->PrevFrameViewInfo.GBufferA);
				}
			}
		}
	}
}
