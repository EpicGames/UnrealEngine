// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraph.h"
#include "SceneTextureParameters.h"
#include "SceneView.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "Substrate/Substrate.h"

enum class EDecalRenderStage : uint8;
enum class EDecalRenderTargetMode : uint8;
struct FVisibleDecal;
struct FDBufferTextures;
struct FSceneTextures;
class FViewInfo;

DECLARE_UNIFORM_BUFFER_STRUCT(FSceneUniformParameters, RENDERER_API)

bool AreDecalsEnabled(const FViewInfo& View);
bool IsDBufferEnabled(const FSceneViewFamily& ViewFamily, EShaderPlatform ShaderPlatform);

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FDecalPassUniformParameters, )
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT(FMobileSceneTextureUniformParameters, MobileSceneTextures)
	SHADER_PARAMETER_STRUCT(FSubstratePublicParameters, SubstratePublic)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

struct FDeferredDecalPassTextures
{
	TRDGUniformBufferRef<FDecalPassUniformParameters> DecalPassUniformBuffer = nullptr;

	// Potential render targets for the decal pass.
	FRDGTextureMSAA Depth;
	FRDGTextureRef Color = nullptr;
	FRDGTextureRef ScreenSpaceAO = nullptr;
	FRDGTextureRef GBufferA = nullptr;
	FRDGTextureRef GBufferB = nullptr;
	FRDGTextureRef GBufferC = nullptr;
	FRDGTextureRef GBufferE = nullptr;
	FDBufferTextures* DBufferTextures = nullptr;
};

FDeferredDecalPassTextures GetDeferredDecalPassTextures(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FDBufferTextures* DBufferTextures,
	EDecalRenderStage DecalRenderStage);

void AddDeferredDecalPass(
	FRDGBuilder& GraphBuilder,
	FViewInfo& ViewInfo,
	TConstArrayView<const FVisibleDecal*> SortedDecals,
	const FDeferredDecalPassTextures& Textures,
	FInstanceCullingManager& InstanceCullingManager,
	EDecalRenderStage RenderStage);

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FDeferredDecalUniformParameters, )
	SHADER_PARAMETER_TEXTURE(Texture2D, PreviousFrameNormal)
	SHADER_PARAMETER(int32, NormalReprojectionEnabled)
	SHADER_PARAMETER(float, NormalReprojectionThresholdLow)
	SHADER_PARAMETER(float, NormalReprojectionThresholdHigh)
	SHADER_PARAMETER(float, NormalReprojectionThresholdScaleHelper)
	SHADER_PARAMETER(FVector2f, NormalReprojectionJitter)
END_SHADER_PARAMETER_STRUCT()

TUniformBufferRef<FDeferredDecalUniformParameters> CreateDeferredDecalUniformBuffer(const FViewInfo& View);

BEGIN_SHADER_PARAMETER_STRUCT(FDeferredDecalPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_STRUCT_REF(FDeferredDecalUniformParameters, DeferredDecal)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FDecalPassUniformParameters, DecalPass)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void GetDeferredDecalRenderTargetsInfo(
	const FSceneTexturesConfig& Config,
	EDecalRenderTargetMode RenderTargetMode,
	FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo);

void CollectDeferredDecalPassPSOInitializers(
	int32 PSOCollectorIndex,
	ERHIFeatureLevel::Type FeatureLevel,
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	EDecalRenderStage DecalRenderStage,
	TArray<FPSOPrecacheData>& PSOInitializers);

void GetDeferredDecalPassParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FDeferredDecalPassTextures& DecalPassTextures,
	EDecalRenderStage DecalRenderStage,
	EDecalRenderTargetMode RenderTargetMode,
	FDeferredDecalPassParameters& PassParameters);

void RenderMeshDecals(
	FRDGBuilder& GraphBuilder,
	const FScene& Scene,
	FViewInfo& View,
	const FDeferredDecalPassTextures& DecalPassTextures,
	FInstanceCullingManager& InstanceCullingManager, 
	EDecalRenderStage DecalRenderStage);

bool HasAnyDrawCommandDecalCount(
	EDecalRenderStage DecalRenderStage, 
	FViewInfo& View);

void ExtractNormalsForNextFrameReprojection(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const TArray<FViewInfo>& Views);
