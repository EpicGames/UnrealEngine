// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "RHI.h"
#include "HitProxies.h"
#include "Shader.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "PostProcess/SceneRenderTargets.h"
#include "MaterialShaderType.h"
#include "MeshMaterialShader.h"
#include "ShaderBaseClasses.h"
#include "DepthRendering.h"
#include "DecalRenderingCommon.h"
#include "DecalRenderingShared.h"
#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "SceneRendering.h"
#include "ScenePrivate.h"
#include "UnrealEngine.h"
#include "DebugViewModeRendering.h"
#include "MeshPassProcessor.inl"
#include "SimpleMeshDrawCommandPass.h"
#include "ComponentRecreateRenderStateContext.h"
#include "PSOPrecacheSettings.h"

static TAutoConsoleVariable<int32> CVarParallelMeshDecal(
	TEXT("r.ParallelMeshDecal"),
	1,
	TEXT("Toggles parallel mesh decal rendering. Parallel rendering must be enabled for this to have an effect."),
	ECVF_RenderThreadSafe
);

static constexpr uint32 GetMeshDecalStencilRef()
{
	return GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1);
}

static FRHIDepthStencilState* GetMeshDecalDepthStencilState(EDecalRenderTargetMode InRenderTargetMode)
{
	if (InRenderTargetMode == EDecalRenderTargetMode::DBuffer)
	{
		// DBuffer handles ReceiveDecal state in the receiver material.
		return TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
	}
	else
	{
		// GBuffer handles ReceiveDecal state using stencil.
		return TStaticDepthStencilState<
			false, CF_DepthNearOrEqual,
			true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			GetMeshDecalStencilRef(), 0>::GetRHI();
	}
}


class FMeshDecalsVS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FMeshDecalsVS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return (Parameters.MaterialParameters.MaterialDomain == MD_DeferredDecal);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
	}

	FMeshDecalsVS() = default;
	FMeshDecalsVS(const ShaderMetaType::CompiledShaderInitializerType & Initializer)
		: FMeshMaterialShader(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(,FMeshDecalsVS,TEXT("/Engine/Private/MeshDecals.usf"),TEXT("MainVS"),SF_Vertex); 

class FMeshDecalsPS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FMeshDecalsPS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return (Parameters.MaterialParameters.MaterialDomain == MD_DeferredDecal) &&
			DecalRendering::GetBaseRenderStage(DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters)) != EDecalRenderStage::None;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		DecalRendering::ModifyCompilationEnvironment(Parameters.Platform, DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::None, OutEnvironment);
	}

	FMeshDecalsPS() = default;
	FMeshDecalsPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(,FMeshDecalsPS,TEXT("/Engine/Private/MeshDecals.usf"),TEXT("MainPS"),SF_Pixel);

class FMeshDecalsEmissivePS : public FMeshDecalsPS
{
public:
	DECLARE_SHADER_TYPE(FMeshDecalsEmissivePS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return (Parameters.MaterialParameters.MaterialDomain == MD_DeferredDecal) &&
			DecalRendering::IsCompatibleWithRenderStage(DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::Emissive);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		DecalRendering::ModifyCompilationEnvironment(Parameters.Platform, DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::Emissive, OutEnvironment);
	}

	FMeshDecalsEmissivePS() = default;
	FMeshDecalsEmissivePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshDecalsPS(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FMeshDecalsEmissivePS, TEXT("/Engine/Private/MeshDecals.usf"), TEXT("MainPS"), SF_Pixel);

class FMeshDecalsAmbientOcclusionPS : public FMeshDecalsPS
{
public:
	DECLARE_SHADER_TYPE(FMeshDecalsAmbientOcclusionPS, MeshMaterial);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return (Parameters.MaterialParameters.MaterialDomain == MD_DeferredDecal) &&
			DecalRendering::IsCompatibleWithRenderStage(DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::AmbientOcclusion);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		DecalRendering::ModifyCompilationEnvironment(Parameters.Platform, DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::AmbientOcclusion, OutEnvironment);
	}

	FMeshDecalsAmbientOcclusionPS() = default;
	FMeshDecalsAmbientOcclusionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshDecalsPS(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FMeshDecalsAmbientOcclusionPS, TEXT("/Engine/Private/MeshDecals.usf"), TEXT("MainPS"), SF_Pixel);

class FMeshDecalMeshProcessor : public FMeshPassProcessor
{
public:
	FMeshDecalMeshProcessor(const FScene* Scene, 
		ERHIFeatureLevel::Type FeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand,
		EDecalRenderTargetMode InRenderTargetMode,
		EShadingPath ShadingPath,
		FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;

	virtual void CollectPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams, 
		TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	void CollectDeferredDecalMeshPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		const FMaterial& Material,
		const FDecalBlendDesc DecalBlendDesc,
		EDecalRenderStage DecalRenderStage,
		TArray<FPSOPrecacheData>& PSOInitializers);

	FMeshPassProcessorRenderState PassDrawRenderState;
	const EDecalRenderStage PassDecalStage;
	const EDecalRenderTargetMode RenderTargetMode;
};

IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(DeferredDecals);
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FDeferredDecalUniformParameters, "DeferredDecal", DeferredDecals);


FMeshDecalMeshProcessor::FMeshDecalMeshProcessor(const FScene* Scene, 
	ERHIFeatureLevel::Type InFeatureLevel,
	const FSceneView* InViewIfDynamicMeshCommand, 
	EDecalRenderTargetMode InRenderTargetMode,
	EShadingPath ShadingPath,
	FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(DecalRendering::GetMeshPassType(InRenderTargetMode), Scene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDecalStage(DecalRendering::GetRenderStage(InRenderTargetMode, ShadingPath))
	, RenderTargetMode(InRenderTargetMode)
{
	PassDrawRenderState.SetDepthStencilState(GetMeshDecalDepthStencilState(RenderTargetMode));
}

void FMeshDecalMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (MeshBatch.bUseForMaterial && MeshBatch.IsDecal(FeatureLevel))
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material)
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

bool FMeshDecalMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	if (Material.IsDeferredDecal())
	{
		// We have no special engine material for decals since we don't want to eat the compilation & memory cost, so just skip if it failed to compile
		if (Material.GetRenderingThreadShaderMap())
		{
			const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
			const FDecalBlendDesc DecalBlendDesc = DecalRendering::ComputeDecalBlendDesc(ShaderPlatform, Material);

			const bool bShouldRender =
				DecalRendering::IsCompatibleWithRenderStage(DecalBlendDesc, PassDecalStage) &&
				DecalRendering::GetRenderTargetMode(ShaderPlatform, DecalBlendDesc, PassDecalStage) == RenderTargetMode;

			if (bShouldRender)
			{
				const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
				ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
				ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

				if (ViewIfDynamicMeshCommand && ViewIfDynamicMeshCommand->Family->UseDebugViewPS())
				{
					// Deferred decals can only use translucent blend mode
					if (ViewIfDynamicMeshCommand->Family->EngineShowFlags.ShaderComplexity)
					{
						// If we are in the translucent pass then override the blend mode, otherwise maintain additive blending.
						PassDrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_One>::GetRHI());
					}
					else if (ViewIfDynamicMeshCommand->Family->GetDebugViewShaderMode() != DVSM_OutputMaterialTextureScales)
					{
						// Otherwise, force translucent blend mode (shaders will use an hardcoded alpha).
						PassDrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI());
					}
				}
				else
				{
					PassDrawRenderState.SetBlendState(DecalRendering::GetDecalBlendState(DecalBlendDesc, PassDecalStage, RenderTargetMode));
				}

				if (RenderTargetMode != EDecalRenderTargetMode::DBuffer)
				{
					PassDrawRenderState.SetStencilRef(GetMeshDecalStencilRef());
				}

				return Process(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, MaterialRenderProxy, Material, MeshFillMode, MeshCullMode);
			}
		}
	}

	return true;
}

bool FMeshDecalMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;
	FVertexFactoryType* VertexFactoryType = VertexFactory->GetType();

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<FMeshDecalsVS>();

	if (PassDecalStage == EDecalRenderStage::Emissive)
	{
		ShaderTypes.AddShaderType<FMeshDecalsEmissivePS>();
	}
	else if (PassDecalStage == EDecalRenderStage::AmbientOcclusion)
	{
		ShaderTypes.AddShaderType<FMeshDecalsAmbientOcclusionPS>();
	}
	else
	{
		ShaderTypes.AddShaderType<FMeshDecalsPS>();
	}

	FMaterialShaders Shaders;
	if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		// Skip rendering if any shaders missing
		return false;
	}

	TMeshProcessorShaders<
		FMeshDecalsVS,
		FMeshDecalsPS> MeshDecalPassShaders;
	Shaders.TryGetVertexShader(MeshDecalPassShaders.VertexShader);
	Shaders.TryGetPixelShader(MeshDecalPassShaders.PixelShader);

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

	// Use BasePass sort key layout but replace the "Masked" highest priority bits with TranslucencySortPriority.
	FMeshDrawCommandSortKey SortKey;
	SortKey.BasePass.VertexShaderHash = (MeshDecalPassShaders.VertexShader.IsValid() ? MeshDecalPassShaders.VertexShader->GetSortKey() : 0) & 0xFFFF;
	SortKey.BasePass.PixelShaderHash = MeshDecalPassShaders.PixelShader.IsValid() ? MeshDecalPassShaders.PixelShader->GetSortKey() : 0;
	SortKey.BasePass.Masked = PrimitiveSceneProxy->GetTranslucencySortPriority();

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		MeshDecalPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

void FMeshDecalMeshProcessor::CollectDeferredDecalMeshPSOInitializers(	
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams,
	const FMaterial& Material,
	const FDecalBlendDesc DecalBlendDesc,
	EDecalRenderStage DecalRenderStage,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	EDecalRenderTargetMode LocalRenderTargetMode = DecalRendering::GetRenderTargetMode(ShaderPlatform, DecalBlendDesc, DecalRenderStage);
	PassDrawRenderState.SetBlendState(DecalRendering::GetDecalBlendState(DecalBlendDesc, DecalRenderStage, LocalRenderTargetMode));
	PassDrawRenderState.SetDepthStencilState(GetMeshDecalDepthStencilState(LocalRenderTargetMode));

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<FMeshDecalsVS>();
	if (DecalRenderStage == EDecalRenderStage::Emissive)
	{
		ShaderTypes.AddShaderType<FMeshDecalsEmissivePS>();
	}
	else if (DecalRenderStage == EDecalRenderStage::AmbientOcclusion)
	{
		ShaderTypes.AddShaderType<FMeshDecalsAmbientOcclusionPS>();
	}
	else
	{
		ShaderTypes.AddShaderType<FMeshDecalsPS>();
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryData.VertexFactoryType, Shaders))
	{
		return;
	}

	TMeshProcessorShaders<
		FMeshDecalsVS,
		FMeshDecalsPS> MeshDecalPassShaders;
	Shaders.TryGetVertexShader(MeshDecalPassShaders.VertexShader);
	Shaders.TryGetPixelShader(MeshDecalPassShaders.PixelShader);

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	GetDeferredDecalRenderTargetsInfo(SceneTexturesConfig, LocalRenderTargetMode, RenderTargetsInfo);

	uint8 SubpassIndex = 0;
	ESubpassHint SubpassHint = ESubpassHint::None;
	if (FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		// subpass info set during the submission of the draws in a mobile renderer
		SubpassHint = GetSubpassHint(SceneTexturesConfig.ShaderPlatform, SceneTexturesConfig.bIsUsingGBuffers, SceneTexturesConfig.bRequireMultiView, SceneTexturesConfig.NumSamples);
		// all decals use second sub-pass on mobile
		SubpassIndex = (SubpassHint != ESubpassHint::None) ? 1 : 0;
	}

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		Material,
		PassDrawRenderState,
		RenderTargetsInfo,
		MeshDecalPassShaders,
		MeshFillMode,
		MeshCullMode,
		PT_TriangleList,
		EMeshPassFeatures::Default,
		SubpassHint,
		SubpassIndex,
		true /*bRequired*/,
		PSOCollectorIndex,
		PSOInitializers);
}

void FMeshDecalMeshProcessor::CollectPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams, 
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	if (!Material.IsDeferredDecal() || !UPSOPrecacheSettingsManager::GetSettings().bDeferredDecals)
	{
		return;
	}

	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	const FDecalBlendDesc DecalBlendDesc = DecalRendering::ComputeDecalBlendDesc(ShaderPlatform, Material);
	const bool bPlatformUsesDBuffers = IsUsingDBuffers(ShaderPlatform);

	for (uint32 DecalStageIter = 0; DecalStageIter < (uint32)EDecalRenderStage::Num; ++DecalStageIter)
	{
		EDecalRenderStage LocalDecalRenderStage = EDecalRenderStage(DecalStageIter);
				
		const bool bShouldRender = DecalRendering::IsCompatibleWithRenderStage(DecalBlendDesc, LocalDecalRenderStage);
		if (!bShouldRender)
		{
			continue;
		}

		EDecalRenderTargetMode LocalRenderTargetMode = DecalRendering::GetRenderTargetMode(ShaderPlatform, DecalBlendDesc, LocalDecalRenderStage);
		if (LocalRenderTargetMode == EDecalRenderTargetMode::DBuffer && !bPlatformUsesDBuffers)
		{
			continue;
		}
		
		// Collect decal pass PSOs
		CollectDeferredDecalPassPSOInitializers(PSOCollectorIndex, FeatureLevel, SceneTexturesConfig, Material, LocalDecalRenderStage, PSOInitializers);
		
		// Collect decal mesh PSOs
		CollectDeferredDecalMeshPSOInitializers(SceneTexturesConfig, VertexFactoryData, PreCacheParams, Material, DecalBlendDesc, LocalDecalRenderStage, PSOInitializers);
	}
}

FMeshPassProcessor* CreateMeshDecalDBufferMeshProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FMeshDecalMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, EDecalRenderTargetMode::DBuffer, EShadingPath::Deferred, InDrawListContext);
}

FMeshPassProcessor* CreateMeshDecalSceneColorAndGBufferMeshProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FMeshDecalMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, EDecalRenderTargetMode::SceneColorAndGBuffer, EShadingPath::Deferred, InDrawListContext);
}

FMeshPassProcessor* CreateMeshDecalSceneColorAndGBufferNoNormalMeshProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FMeshDecalMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, EDecalRenderTargetMode::SceneColorAndGBufferNoNormal, EShadingPath::Deferred, InDrawListContext);
}

FMeshPassProcessor* CreateMeshDecalSceneColorMeshProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FMeshDecalMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, EDecalRenderTargetMode::SceneColor, EShadingPath::Deferred, InDrawListContext);
}

FMeshPassProcessor* CreateMeshDecalAmbientOcclusionProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FMeshDecalMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, EDecalRenderTargetMode::AmbientOcclusion, EShadingPath::Deferred, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MeshDecalPass_DBuffer, CreateMeshDecalDBufferMeshProcessor, EShadingPath::Deferred, EMeshPass::MeshDecal_DBuffer, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MeshDecal_SceneColorAndGBuffer, CreateMeshDecalSceneColorAndGBufferMeshProcessor, EShadingPath::Deferred, EMeshPass::MeshDecal_SceneColorAndGBuffer, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MeshDecal_SceneColorAndGBufferNoNormal, CreateMeshDecalSceneColorAndGBufferNoNormalMeshProcessor, EShadingPath::Deferred, EMeshPass::MeshDecal_SceneColorAndGBufferNoNormal, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MeshDecal_SceneColor, CreateMeshDecalSceneColorMeshProcessor, EShadingPath::Deferred, EMeshPass::MeshDecal_SceneColor, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MeshDecal_AmbientOcclusion, CreateMeshDecalAmbientOcclusionProcessor, EShadingPath::Deferred, EMeshPass::MeshDecal_AmbientOcclusion, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);

FMeshPassProcessor* CreateMeshDecalSceneColorAndGBufferMeshProcessor_Mobile(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FMeshDecalMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, EDecalRenderTargetMode::SceneColorAndGBuffer, EShadingPath::Mobile, InDrawListContext);
}

FMeshPassProcessor* CreateMeshDecalSceneColorAndGBufferNoNormalMeshProcessor_Mobile(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FMeshDecalMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, EDecalRenderTargetMode::SceneColorAndGBufferNoNormal, EShadingPath::Mobile, InDrawListContext);
}

FMeshPassProcessor* CreateMeshDecalSceneColorMeshProcessor_Mobile(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FMeshDecalMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, EDecalRenderTargetMode::SceneColor, EShadingPath::Mobile, InDrawListContext);
}

FMeshPassProcessor* CreateMeshDecalDBufferMeshProcessor_Mobile(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FMeshDecalMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, EDecalRenderTargetMode::DBuffer, EShadingPath::Mobile, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(Mobile_MeshDecal_SceneColorAndGBuffer, CreateMeshDecalSceneColorAndGBufferMeshProcessor_Mobile, EShadingPath::Mobile, EMeshPass::MeshDecal_SceneColorAndGBuffer, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(Mobile_MeshDecal_SceneColorAndGBufferNoNormal, CreateMeshDecalSceneColorAndGBufferNoNormalMeshProcessor_Mobile, EShadingPath::Mobile, EMeshPass::MeshDecal_SceneColorAndGBufferNoNormal, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(Mobile_MeshDecal_SceneColor, CreateMeshDecalSceneColorMeshProcessor_Mobile, EShadingPath::Mobile, EMeshPass::MeshDecal_SceneColor, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(Mobile_MeshDecal_DBuffer, CreateMeshDecalDBufferMeshProcessor_Mobile, EShadingPath::Mobile, EMeshPass::MeshDecal_DBuffer, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);

void DrawDecalMeshCommands(
	FRDGBuilder& GraphBuilder,
	const FScene& Scene,
	FViewInfo& View,
	const FDeferredDecalPassTextures& DecalPassTextures,
	FInstanceCullingManager& InstanceCullingManager,
	EDecalRenderStage DecalRenderStage,
	EDecalRenderTargetMode RenderTargetMode)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FDeferredDecalPassParameters>();
	GetDeferredDecalPassParameters(GraphBuilder, View, DecalPassTextures, DecalRenderStage, RenderTargetMode, *PassParameters);

	EMeshPass::Type DecalMeshPassType = DecalRendering::GetMeshPassType(RenderTargetMode);
	if (auto* Pass = View.ParallelMeshDrawCommandPasses[DecalMeshPassType]; HasAnyDraw(Pass))
	{
		const bool bRenderInParallel = GRHICommandList.UseParallelAlgorithms() && CVarParallelMeshDecal.GetValueOnRenderThread() == 1;

		Pass->BuildRenderingCommands(GraphBuilder, Scene.GPUScene, PassParameters->InstanceCullingDrawParams);

		if (bRenderInParallel)
		{
			// Make sure we are not clearing any render targets via the pass parameters (otherwise it might be cleared multiple times)
			for (FRenderTargetBinding& RenderTarget : PassParameters->RenderTargets.Output)
			{
				if (RenderTarget.GetLoadAction() == ERenderTargetLoadAction::EClear)
				{
					AddClearRenderTargetPass(GraphBuilder, RenderTarget.GetTexture());
					RenderTarget.SetLoadAction(ERenderTargetLoadAction::ELoad);
				}
			}

			GraphBuilder.AddDispatchPass(
				RDG_EVENT_NAME("MeshDecals: %s", GetMeshPassName(DecalMeshPassType)),
				PassParameters,
				ERDGPassFlags::Raster,
				[Pass, PassParameters, DecalMeshPassType](FRDGDispatchPassBuilder& DispatchPassBuilder)
				{
					Pass->Dispatch(DispatchPassBuilder, &PassParameters->InstanceCullingDrawParams);
				});
		}
		else
		{
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("MeshDecals: %s", GetMeshPassName(DecalMeshPassType)),
				PassParameters,
				ERDGPassFlags::Raster,
				[&View, Pass, PassParameters, DecalMeshPassType](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					FSceneRenderer::SetStereoViewport(RHICmdList, View, 1.0f);
					Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
				});
		}
	}
}

bool HasAnyDrawCommandDecalCount(
	EDecalRenderStage DecalRenderStage,
	FViewInfo& View)
{
	switch (DecalRenderStage)
	{
	case EDecalRenderStage::BeforeBasePass:	
		return HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::MeshDecal_DBuffer]);

	case EDecalRenderStage::BeforeLighting:
	case EDecalRenderStage::MobileBeforeLighting:
		return HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::MeshDecal_SceneColorAndGBuffer]) || HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::MeshDecal_SceneColorAndGBufferNoNormal]);

	case EDecalRenderStage::Mobile:
		return HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::MeshDecal_SceneColor]);

	case EDecalRenderStage::Emissive:
		return HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::MeshDecal_SceneColor]);

	case EDecalRenderStage::AmbientOcclusion:
		return HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::MeshDecal_AmbientOcclusion]);
	}
	return false;
}

void RenderMeshDecals(
	FRDGBuilder& GraphBuilder,
	const FScene& Scene,
	FViewInfo& View,
	const FDeferredDecalPassTextures& DecalPassTextures,
	FInstanceCullingManager& InstanceCullingManager,
	EDecalRenderStage DecalRenderStage)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRenderer_RenderMeshDecals);

	switch (DecalRenderStage)
	{
	case EDecalRenderStage::BeforeBasePass:
		DrawDecalMeshCommands(GraphBuilder, Scene, View, DecalPassTextures, InstanceCullingManager, DecalRenderStage, EDecalRenderTargetMode::DBuffer);
		break;

	case EDecalRenderStage::BeforeLighting:
	case EDecalRenderStage::MobileBeforeLighting:
		DrawDecalMeshCommands(GraphBuilder, Scene, View, DecalPassTextures, InstanceCullingManager, DecalRenderStage, EDecalRenderTargetMode::SceneColorAndGBuffer);
		DrawDecalMeshCommands(GraphBuilder, Scene, View, DecalPassTextures, InstanceCullingManager, DecalRenderStage, EDecalRenderTargetMode::SceneColorAndGBufferNoNormal);
		break;

	case EDecalRenderStage::Mobile:
		DrawDecalMeshCommands(GraphBuilder, Scene, View, DecalPassTextures, InstanceCullingManager, DecalRenderStage, EDecalRenderTargetMode::SceneColor);
		break;

	case EDecalRenderStage::Emissive:
		DrawDecalMeshCommands(GraphBuilder, Scene, View, DecalPassTextures, InstanceCullingManager, DecalRenderStage, EDecalRenderTargetMode::SceneColor);
		break;

	case EDecalRenderStage::AmbientOcclusion:
		DrawDecalMeshCommands(GraphBuilder, Scene, View, DecalPassTextures, InstanceCullingManager, DecalRenderStage, EDecalRenderTargetMode::AmbientOcclusion);
		break;
	}
}
