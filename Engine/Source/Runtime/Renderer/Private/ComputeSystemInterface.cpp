// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComputeSystemInterface.h"

namespace ComputeSystemInterface
{
	TArray<IComputeSystem*> GRegisteredSystems;

	void RegisterSystem(IComputeSystem* InSystem)
	{
		GRegisteredSystems.AddUnique(InSystem);
	}

	void UnregisterSystem(IComputeSystem* InSystem)
	{
		for (int32 SystemIndex = 0; SystemIndex < GRegisteredSystems.Num(); ++SystemIndex)
		{
			if (GRegisteredSystems[SystemIndex] == InSystem)
			{
				GRegisteredSystems.RemoveAtSwap(SystemIndex, EAllowShrinking::No);
				break;
			}
		}
	}

	void CreateWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& OutWorkders)
	{
		for (IComputeSystem* System : GRegisteredSystems)
		{
			System->CreateWorkers(InScene, OutWorkders);
		}
	}

	void DestroyWorkers(FSceneInterface const* InScene, TArray<IComputeTaskWorker*>& InOutWorkders)
	{
		for (IComputeSystem* System : GRegisteredSystems)
		{
			System->DestroyWorkers(InScene, InOutWorkders);
		}

		ensure(InOutWorkders.Num() == 0);
	}
}
