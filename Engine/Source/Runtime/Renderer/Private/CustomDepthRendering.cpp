// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomDepthRendering.h"
#include "CoreMinimal.h"
#include "CanvasTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DepthRendering.h"
#include "Engine/EngineBaseTypes.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "Logging/LogMacros.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShared.h"
#include "Math/MathFwd.h"
#include "MeshBatch.h"
#include "MeshDrawCommands.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.inl"
#include "Misc/LargeWorldCoordinates.h"
#include "Nanite/NaniteComposition.h"
#include "Nanite/NaniteCullRaster.h"
#include "Nanite/NaniteShared.h"
#include "Nanite/NaniteVisibility.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "PSOPrecache.h"
#include "PSOPrecacheMaterial.h"
#include "PSOPrecacheSettings.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/NaniteResources.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTexturesConfig.h"
#include "SceneTypes.h"
#include "SceneUniformBuffer.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterMetadata.h"
#include "Stats/Stats.h"
#include "StereoRenderUtils.h"
#include "UnrealEngine.h"
#include "VertexFactory.h"


static TAutoConsoleVariable<int32> CVarCustomDepthOrder(
	TEXT("r.CustomDepth.Order"),
	2,
	TEXT("When CustomDepth (and CustomStencil) is getting rendered\n")
	TEXT("  0: Before Base Pass (Allows samping in DBuffer pass. Can be more efficient with AsyncCompute.)\n")
	TEXT("  1: After Base Pass\n")
	TEXT("  2: Default (Before Base Pass if DBuffer enabled.)\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarCustomDepthTemporalAAJitter(
	TEXT("r.CustomDepthTemporalAAJitter"),
	1,
	TEXT("If disabled the Engine will remove the TemporalAA Jitter from the Custom Depth Pass. Only has effect when TemporalAA is used."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarCustomDepthEnableFastClear(
	TEXT("r.CustomDepthEnableFastClear"), false,
	TEXT("Enable HTile on the custom depth buffer (default:false).\n"),
	ECVF_RenderThreadSafe);

DECLARE_DWORD_COUNTER_STAT(TEXT("Nanite Custom Depth Instances"), STAT_NaniteCustomDepthInstances, STATGROUP_Nanite);

DECLARE_GPU_DRAWCALL_STAT_NAMED(CustomDepth, TEXT("Custom Depth"));

using FNaniteCustomDepthDrawList = TArray<Nanite::FInstanceDraw, SceneRenderingAllocator>;

ECustomDepthPassLocation GetCustomDepthPassLocation(EShaderPlatform Platform)
{
	const int32 CustomDepthOrder = CVarCustomDepthOrder.GetValueOnRenderThread();
	const bool bCustomDepthBeforeBasePase = CustomDepthOrder == 0 || (CustomDepthOrder == 2 && IsUsingDBuffers(Platform));
	return bCustomDepthBeforeBasePase ? ECustomDepthPassLocation::BeforeBasePass : ECustomDepthPassLocation::AfterBasePass;
}

bool IsCustomDepthPassWritingStencil()
{
	return GetCustomDepthMode() == ECustomDepthMode::EnabledWithStencil;
}

FCustomDepthTextures FCustomDepthTextures::Create(FRDGBuilder& GraphBuilder, FIntPoint CustomDepthExtent, EShaderPlatform ShaderPlatform, bool bRequireMultiView, uint16 MobileMultiViewRenderTargetNumLayers)
{
	if (!IsCustomDepthPassEnabled())
	{
		return {};
	}

	const bool bWritesCustomStencil = IsCustomDepthPassWritingStencil();

	FCustomDepthTextures CustomDepthTextures;

	ETextureCreateFlags CreateFlags = GFastVRamConfig.CustomDepth | TexCreate_DepthStencilTargetable | TexCreate_ShaderResource;

	// For Nanite, check to create the depth texture as a UAV and force HTILE.
	if (UseNanite(ShaderPlatform) && UseComputeDepthExport() && Nanite::GetSupportsCustomDepthRendering())
	{
		CreateFlags |= TexCreate_UAV;
	}
	else if (!CVarCustomDepthEnableFastClear.GetValueOnRenderThread())
	{
		CreateFlags |= TexCreate_NoFastClear;
	}
	FRDGTextureDesc CustomDepthDesc = FRDGTextureDesc::CreateRenderTargetTextureDesc(CustomDepthExtent, PF_DepthStencil, FClearValueBinding::DepthFar, CreateFlags, bRequireMultiView, MobileMultiViewRenderTargetNumLayers);
	CustomDepthDesc.AliasableFormats.Add(PF_X24_G8);
	
	CustomDepthTextures.Depth = GraphBuilder.CreateTexture(CustomDepthDesc, TEXT("CustomDepth"));

	CustomDepthTextures.DepthAction = ERenderTargetLoadAction::EClear;
	CustomDepthTextures.StencilAction = bWritesCustomStencil ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ENoAction;

	return CustomDepthTextures;
}

BEGIN_SHADER_PARAMETER_STRUCT(FCustomDepthPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static FViewShaderParameters CreateViewShaderParametersWithoutJitter(const FViewInfo& View, uint32 ViewIndex, const TArrayView<Nanite::FPackedView>& OutNaniteViews)
{
	const auto SetupParameters = [](const FViewInfo& View, FViewUniformShaderParameters& Parameters)
	{
		FBox VolumeBounds[TVC_MAX];
		FViewMatrices ModifiedViewMatrices = View.ViewMatrices;
		ModifiedViewMatrices.HackRemoveTemporalAAProjectionJitter();

		Parameters = *View.CachedViewUniformShaderParameters;
		View.SetupUniformBufferParameters(ModifiedViewMatrices, ModifiedViewMatrices, VolumeBounds, TVC_MAX, Parameters);
	};
	const auto CopyIntoNaniteParameters = [&](uint32 ViewIndex, const FViewUniformShaderParameters& Parameters)
	{
		if (OutNaniteViews.IsValidIndex(ViewIndex))
		{
			OutNaniteViews[ViewIndex].TranslatedWorldToClip	= Parameters.TranslatedWorldToClip;
			OutNaniteViews[ViewIndex].ViewToClip			= Parameters.ViewToClip;
			OutNaniteViews[ViewIndex].ClipToRelativeWorld	= Parameters.ClipToRelativeWorld;
		}
	};

	FViewUniformShaderParameters ViewUniformParameters;
	SetupParameters(View, ViewUniformParameters);
	CopyIntoNaniteParameters(ViewIndex, ViewUniformParameters);

	FViewShaderParameters Parameters;
	Parameters.View = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(ViewUniformParameters, UniformBuffer_SingleFrame);

	if (View.bShouldBindInstancedViewUB)
	{
		FInstancedViewUniformShaderParameters LocalInstancedViewUniformShaderParameters;
		InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, ViewUniformParameters, 0);

		if (const FViewInfo* InstancedView = View.GetInstancedView())
		{
			SetupParameters(*InstancedView, ViewUniformParameters);
			CopyIntoNaniteParameters(ViewIndex + 1, ViewUniformParameters);
			InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, ViewUniformParameters, 1);
		}

		Parameters.InstancedView = TUniformBufferRef<FInstancedViewUniformShaderParameters>::CreateUniformBufferImmediate(
			LocalInstancedViewUniformShaderParameters,
			UniformBuffer_SingleFrame);
	}

	return Parameters;
}

static FNaniteCustomDepthDrawList BuildNaniteCustomDepthDrawList(
	const FViewInfo& View,
	uint32 NumViews,
	const FNaniteVisibilityResults* VisibilityResults)
{
	FNaniteCustomDepthDrawList Output;
	for (uint32 ViewId = 0; ViewId < NumViews; ++ViewId)
	{
		for (const FPrimitiveInstanceRange& InstanceRange : View.NaniteCustomDepthInstances)
		{
			if (!VisibilityResults || VisibilityResults->ShouldRenderCustomDepthPrimitive(InstanceRange.PrimitiveIndex))
			{
				const uint32 FirstOutputIndex = Output.Num();
				Output.AddUninitialized(InstanceRange.NumInstances);
				for (uint32 RelativeInstanceIndex = 0; RelativeInstanceIndex < uint32(InstanceRange.NumInstances); ++RelativeInstanceIndex)
				{
					const uint32 OutputIndex = FirstOutputIndex + RelativeInstanceIndex;
					const uint32 InstanceId = InstanceRange.InstanceSceneDataOffset + RelativeInstanceIndex;
					const Nanite::FInstanceDraw Draw { InstanceId, ViewId };
					Output[OutputIndex] = Draw;
				}
			}
		}
	}

	return MoveTemp(Output);
}

bool FSceneRenderer::RenderCustomDepthPass(
	FRDGBuilder& GraphBuilder,
	FCustomDepthTextures& CustomDepthTextures,
	const FSceneTextureShaderParameters& SceneTextures,
	TConstArrayView<Nanite::FRasterResults> PrimaryNaniteRasterResults,
	TConstArrayView<Nanite::FPackedView> PrimaryNaniteViews)
{
	if (!CustomDepthTextures.IsValid())
	{
		return false;
	}

	struct FTempViewParams
	{
		FViewShaderParameters ViewParams;
		FNaniteCustomDepthDrawList NaniteDrawList;
	};

	TArray<FTempViewParams, FSceneRenderingArrayAllocator> TempViewParams;
	TempViewParams.SetNum(Views.Num());

	TArray<Nanite::FPackedView, FSceneRenderingArrayAllocator> TempNaniteViews;
	TempNaniteViews.Append(PrimaryNaniteViews);

	const bool bWriteCustomStencil = IsCustomDepthPassWritingStencil();
	const bool bDrawSceneViewsInOneNanitePass = Views.Num() > 1 && Nanite::ShouldDrawSceneViewsInOneNanitePass(Views[0]);
	const bool bRemoveTAAJitter = CVarCustomDepthTemporalAAJitter.GetValueOnRenderThread() == 0;

	// Determine if any of the views have custom depth and if any of them have Nanite that is rendering custom depth
	bool bAnyCustomDepth = false;
	uint32 TotalNaniteInstances = 0;
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];
		if (!View.ShouldRenderView() || !View.bHasCustomDepthPrimitives)
		{
			continue;
		}

		if (PrimaryNaniteRasterResults.IsValidIndex(ViewIndex))
		{
			FNaniteVisibilityQuery* VisibilityQuery = PrimaryNaniteRasterResults[ViewIndex].VisibilityQuery;

			// Get the Nanite instance draw list for this view.
			TempViewParams[ViewIndex].NaniteDrawList = BuildNaniteCustomDepthDrawList(
				View,
				bDrawSceneViewsInOneNanitePass ? Views.Num() : 1u,
				Nanite::GetVisibilityResults(VisibilityQuery)
			);

			TotalNaniteInstances += TempViewParams[ViewIndex].NaniteDrawList.Num();
		}

		// User requested jitter-free custom depth.
		if (bRemoveTAAJitter && IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod))
		{
			TempViewParams[ViewIndex].ViewParams = CreateViewShaderParametersWithoutJitter(View, ViewIndex, TempNaniteViews);
		}
		else
		{
			TempViewParams[ViewIndex].ViewParams = View.GetShaderParameters();
		}

		bAnyCustomDepth = true;
	}

	SET_DWORD_STAT(STAT_NaniteCustomDepthInstances, TotalNaniteInstances);

	if (!bAnyCustomDepth)
	{
		return false;
	}

	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderCustomDepthPass);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, CustomDepth, "CustomDepth");

	// Render non-Nanite Custom Depth primitives
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];

		if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::CustomDepth]; Pass && View.ShouldRenderView() && View.bHasCustomDepthPrimitives)
		{
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			View.BeginRenderView();

			FCustomDepthPassParameters* PassParameters = GraphBuilder.AllocParameters<FCustomDepthPassParameters>();
			PassParameters->SceneTextures = SceneTextures;
			PassParameters->View = TempViewParams[ViewIndex].ViewParams;

			const ERenderTargetLoadAction DepthLoadAction = GetLoadActionIfProduced(CustomDepthTextures.Depth, CustomDepthTextures.DepthAction);
			const ERenderTargetLoadAction StencilLoadAction = (View.Family->ViewMode == VMI_VisualizeBuffer) ? ERenderTargetLoadAction::EClear : GetLoadActionIfProduced(CustomDepthTextures.Depth, CustomDepthTextures.StencilAction);

			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
				CustomDepthTextures.Depth,
				DepthLoadAction,
				StencilLoadAction,
				FExclusiveDepthStencil::DepthWrite_StencilWrite);
			PassParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();
			Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("CustomDepth"),
				PassParameters,
				ERDGPassFlags::Raster,
				[&View, Pass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				SetStereoViewport(RHICmdList, View, 1.0f);
				Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
			});
		}
	}

	if (TotalNaniteInstances > 0)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "Nanite CustomDepth");

		const FIntPoint RasterTextureSize = CustomDepthTextures.Depth->Desc.Extent;
		FIntRect RasterTextureRect(0, 0, RasterTextureSize.X, RasterTextureSize.Y);
		if (Views.Num() == 1)
		{
			const FViewInfo& View = Views[0];
			if (View.ViewRect.Min.X == 0 && View.ViewRect.Min.Y == 0)
			{
				RasterTextureRect = View.ViewRect;
			}
		}

		Nanite::FSharedContext SharedContext{};
		SharedContext.FeatureLevel = Scene->GetFeatureLevel();
		SharedContext.ShaderMap = GetGlobalShaderMap(SharedContext.FeatureLevel);
		SharedContext.Pipeline = Nanite::ERasterPipeline::Primary;

		// TODO: If !bWriteCustomStencil, we could copy off the depth and rasterize depth-only (probable optimization)
		const Nanite::FRasterContextInitParams RasterInitParams =
		{
			.TextureSize = RasterTextureSize,
			.TextureRect = RasterTextureRect,
			.RasterMode = Nanite::EOutputBufferMode::VisBuffer,
			.bCustomPass = true
		};
		Nanite::FRasterContext RasterContext = Nanite::InitRasterContext(
			GraphBuilder,
			SharedContext,
			ViewFamily,
			RasterInitParams
		);

		Nanite::FCustomDepthContext CustomDepthContext = Nanite::InitCustomDepthStencilContext(
			GraphBuilder,
			CustomDepthTextures,
			bWriteCustomStencil);

		Nanite::FConfiguration CullingConfig = { 0 };
		CullingConfig.bUpdateStreaming = true;

		const int32 NumViewsToRender = bDrawSceneViewsInOneNanitePass ? 1 : Views.Num();
		auto ViewArray = bDrawSceneViewsInOneNanitePass ?
			Nanite::FPackedViewArray::Create(GraphBuilder, TempNaniteViews.Num(), MoveTemp(TempNaniteViews)) : nullptr;
		for (int32 ViewIndex = 0; ViewIndex < NumViewsToRender; ++ViewIndex)
		{
			if (TempViewParams[ViewIndex].NaniteDrawList.Num() == 0)
			{
				continue;
			}

			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1 && !bDrawSceneViewsInOneNanitePass, "View%d", ViewIndex);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1 && bDrawSceneViewsInOneNanitePass, "View%d (together with %d more)", ViewIndex, Views.Num() - 1);
			
			FViewInfo& View = Views[ViewIndex];

			const FIntRect HZBViewRect = bDrawSceneViewsInOneNanitePass ? FIntRect(0, 0, FamilySize.X, FamilySize.Y) : GetHZBViewRect(View);

			auto NaniteRenderer = Nanite::IRenderer::Create(
				GraphBuilder,
				*Scene,
				View,
				GetSceneUniforms(),
				SharedContext,
				RasterContext,
				CullingConfig,
				HZBViewRect,
				/* PrevHZB = */ nullptr
			);

			if (!bDrawSceneViewsInOneNanitePass)
			{
				ViewArray = Nanite::FPackedViewArray::Create(GraphBuilder, TempNaniteViews[ViewIndex]);
			}

			NaniteRenderer->DrawGeometry(
				Scene->NaniteRasterPipelines[ENaniteMeshPass::BasePass],
				PrimaryNaniteRasterResults[ViewIndex].VisibilityQuery,
				*ViewArray,
				TempViewParams[ViewIndex].NaniteDrawList
			);

			Nanite::FRasterResults RasterResults;
			NaniteRenderer->ExtractResults( RasterResults );

			// Emit depth
			Nanite::EmitCustomDepthStencilTargets(
				GraphBuilder,
				*Scene,
				View,
				bDrawSceneViewsInOneNanitePass,
				RasterResults.PageConstants,
				RasterResults.VisibleClustersSWHW,
				RasterResults.ViewsBuffer,
				RasterContext.VisBuffer64,
				CustomDepthContext
			);
		}

		Nanite::FinalizeCustomDepthStencil(GraphBuilder, CustomDepthContext, CustomDepthTextures);
	}
	else
	{
		const FSceneTexturesConfig& Config = FSceneTexturesConfig::Get();
		// TextureView is not supported in GLES, so we can't lookup CustomDepth and CustomStencil from a single texture
		// Do a copy of the CustomDepthStencil texture if CustomStencil is sampled in a shader.
		if (IsOpenGLPlatform(ShaderPlatform))
		{
			if (Config.bSamplesCustomStencil)
			{
				FRDGTextureRef CustomStencil = GraphBuilder.CreateTexture(CustomDepthTextures.Depth->Desc, TEXT("CustomStencil"));
				AddCopyTexturePass(GraphBuilder, CustomDepthTextures.Depth, CustomStencil);
				CustomDepthTextures.Stencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(CustomStencil, PF_X24_G8));
			}
		}
		else
		{
			CustomDepthTextures.Stencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(CustomDepthTextures.Depth, PF_X24_G8));
		}
		CustomDepthTextures.bSeparateStencilBuffer = false;
	}

	return true;
}

class FCustomDepthPassMeshProcessor : public FSceneRenderingAllocatorObject<FCustomDepthPassMeshProcessor>, public FMeshPassProcessor
{
public:
	FCustomDepthPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	template<bool bPositionOnly>
	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	bool UseDefaultMaterial(const FMaterial& Material, bool bMaterialModifiesMeshPosition, bool bSupportPositionOnlyStream, bool bVFTypeSupportsNullPixelShader, bool& bPositionOnly, bool& bIgnoreThisMaterial);

	void CollectDefaultMaterialPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		EPrimitiveType PrimitiveType,
		TArray<FPSOPrecacheData>& PSOInitializers);

	template<bool bPositionOnly>
	void CollectPSOInitializersInternal(
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode, 
		EPrimitiveType PrimitiveType, 
		TArray<FPSOPrecacheData>& PSOInitializers);

	FMeshPassProcessorRenderState PassDrawRenderState;
};

FCustomDepthPassMeshProcessor::FCustomDepthPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::CustomDepth, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
{
	PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
}

void FCustomDepthPassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (PrimitiveSceneProxy->ShouldRenderCustomDepth())
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material)
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

FRHIDepthStencilState* GetCustomDepthStencilState(bool bWriteCustomStencilValues, EStencilMask StencilWriteMask)
{
	if (bWriteCustomStencilValues)
	{
		static FRHIDepthStencilState* StencilStates[EStencilMask::SM_Count] =
		{
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Keep, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 255>::GetRHI(),
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 255>::GetRHI(),
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 1>::GetRHI(),
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 2>::GetRHI(),
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 4>::GetRHI(),
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 8>::GetRHI(),
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 16>::GetRHI(),
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 32>::GetRHI(),
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 64>::GetRHI(),
			TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 128>::GetRHI()
		};
		checkSlow(EStencilMask::SM_Count == UE_ARRAY_COUNT(StencilStates));
		return StencilStates[(int32)StencilWriteMask];
	}
	else
	{
		return TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
	}
}

bool FCustomDepthPassMeshProcessor::UseDefaultMaterial(const FMaterial& Material, bool bMaterialModifiesMeshPosition, bool bSupportPositionOnlyStream, bool bVFTypeSupportsNullPixelShader, bool& bPositionOnly, bool& bIgnoreThisMaterial)
{
	bool bUseDefaultMaterial = false;
	bIgnoreThisMaterial = false;

	const bool bIsOpaque = IsOpaqueBlendMode(Material);
	const bool bIsTranslucent = IsTranslucentBlendMode(Material);
	if (bIsOpaque
		&& bSupportPositionOnlyStream
		&& !bMaterialModifiesMeshPosition
		&& Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader))
	{
		bUseDefaultMaterial = true;
		bPositionOnly = true;
	}
	else if (!bIsTranslucent || Material.IsTranslucencyWritingCustomDepth())
	{
		const bool bMaterialMasked = !Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader) || Material.IsTranslucencyWritingCustomDepth();
		if (!bMaterialMasked && !bMaterialModifiesMeshPosition)
		{
			bUseDefaultMaterial = true;
			bPositionOnly = false;
		}
	}
	else
	{
		// E.g., ignore translucent materials without allowing custom depth writes.
		bIgnoreThisMaterial = true;
	}

	return bUseDefaultMaterial;
}

bool FCustomDepthPassMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	// Setup the depth stencil state
	const bool bWriteCustomStencilValues = IsCustomDepthPassWritingStencil();
	PassDrawRenderState.SetDepthStencilState(GetCustomDepthStencilState(bWriteCustomStencilValues, PrimitiveSceneProxy->GetStencilWriteMask()));
	if (bWriteCustomStencilValues)
	{
		const uint32 CustomDepthStencilValue = PrimitiveSceneProxy->GetCustomDepthStencilValue();
		PassDrawRenderState.SetStencilRef(CustomDepthStencilValue);
	}

	// Using default material?
	bool bIgnoreThisMaterial = false;
	bool bPositionOnly = false;
	const bool bSupportPositionOnlyStream = MeshBatch.VertexFactory->SupportsPositionOnlyStream();
	const bool bVFTypeSupportsNullPixelShader = MeshBatch.VertexFactory->SupportsNullPixelShader();
	const bool bModifiesMeshPosition = DoMaterialAndPrimitiveModifyMeshPosition(Material, PrimitiveSceneProxy);
	bool bUseDefaultMaterial = UseDefaultMaterial(Material, bModifiesMeshPosition, bSupportPositionOnlyStream, bVFTypeSupportsNullPixelShader, bPositionOnly, bIgnoreThisMaterial);
	if (bIgnoreThisMaterial)
	{
		return true;
	}
	// Swap to default material
	const FMaterialRenderProxy* EffectiveMaterialRenderProxy = &MaterialRenderProxy;
	const FMaterial* EffectiveMaterial = &Material;
	if (bUseDefaultMaterial)
	{
		// Override with the default material
		EffectiveMaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
		EffectiveMaterial = EffectiveMaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		check(EffectiveMaterial);
	}

	// Get the fill & cull mode
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

	if (bPositionOnly)
	{
		return Process<true>(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *EffectiveMaterialRenderProxy, *EffectiveMaterial, MeshFillMode, MeshCullMode);
	}
	else
	{
		return Process<false>(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *EffectiveMaterialRenderProxy, *EffectiveMaterial, MeshFillMode, MeshCullMode);
	}
}

template<bool bPositionOnly>
bool FCustomDepthPassMeshProcessor::Process(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		TDepthOnlyVS<bPositionOnly>,
		FDepthOnlyPS> DepthPassShaders;

	FShaderPipelineRef ShaderPipeline;
	if (!GetDepthPassShaders<bPositionOnly>(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		MaterialResource.MaterialUsesPixelDepthOffset_RenderThread(),
		DepthPassShaders.VertexShader,
		DepthPassShaders.PixelShader,
		ShaderPipeline
		))
	{
		return false;
	}

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(DepthPassShaders.VertexShader, DepthPassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		DepthPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		bPositionOnly ? EMeshPassFeatures::PositionOnly : EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

void FCustomDepthPassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	int32 CustomDepthPrecacheMode = UPSOPrecacheSettingsManager::GetSettings().CustomDepthMode;
	if (CustomDepthPrecacheMode <= 0)
	{
		return;
	}

	// Setup the depth stencil state to use
	const bool bWriteCustomStencilValues = IsCustomDepthPassWritingStencil();
	PassDrawRenderState.SetDepthStencilState(GetCustomDepthStencilState(bWriteCustomStencilValues, PreCacheParams.GetStencilWriteMask()));

	// Are we currently collecting PSO's for the default material
	if (PreCacheParams.bDefaultMaterial)
	{		
		CollectDefaultMaterialPSOInitializers(SceneTexturesConfig, Material, VertexFactoryData, (EPrimitiveType)PreCacheParams.PrimitiveType, PSOInitializers);
		return;
	}

	// assume we can always do this when collecting PSO's for now (vertex factory instance might actually not support it)
	const bool bSupportPositionOnlyStream = VertexFactoryData.VertexFactoryType->SupportsPositionOnly();
	const bool bVFTypeSupportsNullPixelShader = VertexFactoryData.VertexFactoryType->SupportsNullPixelShader();
	bool bIgnoreThisMaterial = false;
	bool bPositionOnly = false;
	bool bUseDefaultMaterial = UseDefaultMaterial(Material, Material.MaterialModifiesMeshPosition_GameThread(), bSupportPositionOnlyStream, bVFTypeSupportsNullPixelShader, bPositionOnly, bIgnoreThisMaterial);

	if (!bIgnoreThisMaterial)
	{
		const FMaterial* EffectiveMaterial = &Material;
		if (bUseDefaultMaterial && ((!bSupportPositionOnlyStream && VertexFactoryData.CustomDefaultVertexDeclaration) || PreCacheParams.PrimitiveType != PT_TriangleList))
		{
			EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			EffectiveMaterial = UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialResource(SceneTexturesConfig.ShaderPlatform, ActiveQualityLevel);
			bUseDefaultMaterial = false;
		}		
				
		bool bPrecacheCustomDepth = PreCacheParams.bRenderCustomDepth;

		// If requested precache for all primitives in depth pass as well
		if (CustomDepthPrecacheMode == 2)
		{
			bPrecacheCustomDepth = bPrecacheCustomDepth || PreCacheParams.bRenderInDepthPass;
		}

		if (!bUseDefaultMaterial && bPrecacheCustomDepth)
		{
			check(!bPositionOnly);

			const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
			const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
			const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

			CollectPSOInitializersInternal<false>(VertexFactoryData, *EffectiveMaterial, MeshFillMode, MeshCullMode, (EPrimitiveType)PreCacheParams.PrimitiveType, PSOInitializers);
		}
	}
}

void FCustomDepthPassMeshProcessor::CollectDefaultMaterialPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig, 
	const FMaterial& Material, 
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	EPrimitiveType PrimitiveType,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	const ERasterizerFillMode MeshFillMode = FM_Solid;

	// TODO: Should do this for each stencil write mask?
	
	// Collect PSOs for all possible default material combinations
	{
		ERasterizerCullMode MeshCullMode = CM_None;
		CollectPSOInitializersInternal<true>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
		CollectPSOInitializersInternal<false>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
	}
	{
		ERasterizerCullMode MeshCullMode = CM_CW;
		CollectPSOInitializersInternal<true>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
		CollectPSOInitializersInternal<false>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
	}
	{
		ERasterizerCullMode MeshCullMode = CM_CCW;
		CollectPSOInitializersInternal<true>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
		CollectPSOInitializersInternal<false>(VertexFactoryData, Material, MeshFillMode, MeshCullMode, PrimitiveType, PSOInitializers);
	}
}

template<bool bPositionOnly>
void FCustomDepthPassMeshProcessor::CollectPSOInitializersInternal(
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode, 
	EPrimitiveType PrimitiveType, 
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	TMeshProcessorShaders<
		TDepthOnlyVS<bPositionOnly>,
		FDepthOnlyPS> DepthPassShaders;

	FShaderPipelineRef ShaderPipeline;
	if (!GetDepthPassShaders<bPositionOnly>(
		MaterialResource,
		VertexFactoryData.VertexFactoryType,
		FeatureLevel,
		MaterialResource.MaterialUsesPixelDepthOffset_GameThread(),
		DepthPassShaders.VertexShader,
		DepthPassShaders.PixelShader,
		ShaderPipeline
		))
	{
		return;
	}

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;

	ETextureCreateFlags CustomDepthStencilCreateFlags = GFastVRamConfig.CustomDepth | TexCreate_NoFastClear | TexCreate_DepthStencilTargetable | TexCreate_ShaderResource;
	SetupDepthStencilInfo(PF_DepthStencil, CustomDepthStencilCreateFlags, ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite, RenderTargetsInfo);

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		MaterialResource,
		PassDrawRenderState,
		RenderTargetsInfo,
		DepthPassShaders,
		MeshFillMode,
		MeshCullMode,
		PrimitiveType,
		bPositionOnly ? EMeshPassFeatures::PositionOnly : EMeshPassFeatures::Default,
		true /*bRequired*/,
		PSOInitializers);
}

FMeshPassProcessor* CreateCustomDepthPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FCustomDepthPassMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(RegisterCustomDepthPass, CreateCustomDepthPassProcessor, EShadingPath::Deferred, EMeshPass::CustomDepth, EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(RegisterMobileCustomDepthPass, CreateCustomDepthPassProcessor, EShadingPath::Mobile, EMeshPass::CustomDepth, EMeshPassFlags::MainView);
