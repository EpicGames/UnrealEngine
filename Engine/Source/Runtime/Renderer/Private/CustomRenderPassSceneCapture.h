// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	CustomRenderPassSceneCapture.h: Internal user data for scene capture custom render passes
=============================================================================*/

#include "Rendering/CustomRenderPass.h"

//
// FSceneCaptureCustomRenderPassUserData
//
class FSceneCaptureCustomRenderPassUserData : public ICustomRenderPassUserData
{
public:
	IMPLEMENT_CUSTOM_RENDER_PASS_USER_DATA(FSceneCaptureCustomRenderPassUserData);

	bool bMainViewFamily = false;
	bool bMainViewResolution = false;
	bool bMainViewCamera = false;
	bool bIgnoreScreenPercentage = false;
	bool bExcludeFromSceneTextureExtents = false;
	FIntPoint SceneTextureDivisor = FIntPoint(1, 1);
	FName UserSceneTextureBaseColor;
	FName UserSceneTextureNormal;
	FName UserSceneTextureSceneColor;
#if !UE_BUILD_SHIPPING
	FString CaptureActorName;
#endif

	static const FSceneCaptureCustomRenderPassUserData& Get(const FCustomRenderPassBase* CustomRenderPass)
	{
		const FSceneCaptureCustomRenderPassUserData* UserData = CustomRenderPass->GetUserDataTyped<FSceneCaptureCustomRenderPassUserData>();
		return UserData ? *UserData : GDefaultData;
	}

private:
	// Default data to return if data isn't present, to simplify renderer logic using the data (no need for conditional that data exists everywhere the structure is used)
	static FSceneCaptureCustomRenderPassUserData GDefaultData;
};
