// Copyright Epic Games, Inc. All Rights Reserved.

#include "DBufferTextures.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "RendererUtils.h"
#include "RenderGraphUtils.h"
#include "SceneUtils.h"
#include "SystemTextures.h"
#include "SceneRendering.h"

bool FDBufferTextures::IsValid() const
{
	check((!DBufferA || (DBufferB && DBufferC)) || (!DBufferATexArray || (DBufferBTexArray && DBufferCTexArray)));
	return bDBufferProduced || bDBufferTexArrayProduced;
}

EDecalDBufferMaskTechnique GetDBufferMaskTechnique(EShaderPlatform ShaderPlatform)
{
	const bool bWriteMaskDBufferMask = RHISupportsRenderTargetWriteMask(ShaderPlatform);
	const bool bPerPixelDBufferMask = FDataDrivenShaderPlatformInfo::GetSupportsPerPixelDBufferMask(ShaderPlatform);
	checkf(!bWriteMaskDBufferMask || !bPerPixelDBufferMask, TEXT("The WriteMask and PerPixel DBufferMask approaches cannot be enabled at the same time. They are mutually exclusive."));

	if (bWriteMaskDBufferMask)
	{
		return EDecalDBufferMaskTechnique::WriteMask;
	}
	else if (bPerPixelDBufferMask)
	{
		return EDecalDBufferMaskTechnique::PerPixel;
	}
	return EDecalDBufferMaskTechnique::Disabled;
}

FDBufferTexturesDesc GetDBufferTexturesDesc(FIntPoint Extent, EShaderPlatform ShaderPlatform)
{
	FDBufferTexturesDesc DBufferTexturesDesc;

	if (IsUsingDBuffers(ShaderPlatform))
	{
		const EDecalDBufferMaskTechnique DBufferMaskTechnique = GetDBufferMaskTechnique(ShaderPlatform);
		const ETextureCreateFlags WriteMaskFlags = DBufferMaskTechnique == EDecalDBufferMaskTechnique::WriteMask ? TexCreate_NoFastClearFinalize | TexCreate_DisableDCC : TexCreate_None;
		const ETextureCreateFlags BaseFlags = WriteMaskFlags | TexCreate_ShaderResource | TexCreate_RenderTargetable;
		
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Extent, PF_B8G8R8A8, FClearValueBinding::None, BaseFlags);
		FRDGTextureDesc ArrayDesc = FRDGTextureDesc::Create2DArray(Extent, PF_B8G8R8A8, FClearValueBinding::None, BaseFlags, 2);

		Desc.Flags = ArrayDesc.Flags = BaseFlags | GFastVRamConfig.DBufferA;
		Desc.ClearValue = ArrayDesc.ClearValue = FClearValueBinding::Black;
		DBufferTexturesDesc.DBufferADesc = Desc;
		DBufferTexturesDesc.DBufferATexArrayDesc = ArrayDesc;
		DBufferTexturesDesc.DBufferADesc.Flags |= TexCreate_SRGB;
		DBufferTexturesDesc.DBufferATexArrayDesc.Flags |= TexCreate_SRGB;

		Desc.Flags = ArrayDesc.Flags = BaseFlags | GFastVRamConfig.DBufferB;
		Desc.ClearValue = ArrayDesc.ClearValue = FClearValueBinding(FLinearColor(128.0f / 255.0f, 128.0f / 255.0f, 128.0f / 255.0f, 1));
		DBufferTexturesDesc.DBufferBDesc = Desc;
		DBufferTexturesDesc.DBufferBTexArrayDesc = ArrayDesc;

		Desc.Flags = ArrayDesc.Flags = BaseFlags | GFastVRamConfig.DBufferC;
		Desc.ClearValue = ArrayDesc.ClearValue = FClearValueBinding(FLinearColor(0, 0, 0, 1));
		DBufferTexturesDesc.DBufferCDesc = Desc;
		DBufferTexturesDesc.DBufferCTexArrayDesc = ArrayDesc;

		if (DBufferMaskTechnique == EDecalDBufferMaskTechnique::PerPixel)
		{
			// Note: 32bpp format is used here to utilize color compression hardware (same as other DBuffer targets).
			// This significantly reduces bandwidth for clearing, writing and reading on some GPUs.
			// While a smaller format, such as R8_UINT, will use less video memory, it will result in slower clears and higher bandwidth requirements.
			check(Desc.Format == PF_B8G8R8A8);
			// On mobile platforms using PF_B8G8R8A8 has no benefits over R8.
			if (IsMobilePlatform(ShaderPlatform))
			{
				Desc.Format = PF_R8;
			}
			Desc.Flags = TexCreate_ShaderResource | TexCreate_RenderTargetable;
			Desc.ClearValue = FClearValueBinding::Transparent;
			DBufferTexturesDesc.DBufferMaskDesc = Desc;
		}
	}

	return DBufferTexturesDesc;
}

FDBufferTextures CreateDBufferTextures(FRDGBuilder& GraphBuilder, FIntPoint Extent, EShaderPlatform ShaderPlatform, const bool bIsMobileMultiView)
{
	FDBufferTextures DBufferTextures;

	if (IsUsingDBuffers(ShaderPlatform))
	{
		FDBufferTexturesDesc TexturesDesc = GetDBufferTexturesDesc(Extent, ShaderPlatform);

		const EDecalDBufferMaskTechnique DBufferMaskTechnique = GetDBufferMaskTechnique(ShaderPlatform);
		const ERDGTextureFlags TextureFlags = DBufferMaskTechnique != EDecalDBufferMaskTechnique::Disabled
			? ERDGTextureFlags::MaintainCompression
			: ERDGTextureFlags::None;
		if (!bIsMobileMultiView)
		{
			DBufferTextures.DBufferA = GraphBuilder.CreateTexture(TexturesDesc.DBufferADesc, TEXT("DBufferA"), TextureFlags);
			DBufferTextures.DBufferB = GraphBuilder.CreateTexture(TexturesDesc.DBufferBDesc, TEXT("DBufferB"), TextureFlags);
			DBufferTextures.DBufferC = GraphBuilder.CreateTexture(TexturesDesc.DBufferCDesc, TEXT("DBufferC"), TextureFlags);
		}
		else
		{
			DBufferTextures.DBufferATexArray = GraphBuilder.CreateTexture(TexturesDesc.DBufferATexArrayDesc, TEXT("DBufferATexArray"), TextureFlags);
			DBufferTextures.DBufferBTexArray = GraphBuilder.CreateTexture(TexturesDesc.DBufferBTexArrayDesc, TEXT("DBufferBTexArray"), TextureFlags);
			DBufferTextures.DBufferCTexArray = GraphBuilder.CreateTexture(TexturesDesc.DBufferCTexArrayDesc, TEXT("DBufferCTexArray"), TextureFlags);
		}

		if (DBufferMaskTechnique == EDecalDBufferMaskTechnique::PerPixel)
		{
			DBufferTextures.DBufferMask = GraphBuilder.CreateTexture(TexturesDesc.DBufferMaskDesc, TEXT("DBufferMask"));
		}
	}

	return DBufferTextures;
}

FDBufferParameters GetDBufferParameters(FRDGBuilder& GraphBuilder, const FDBufferTextures& DBufferTextures, EShaderPlatform ShaderPlatform, const bool bIsMobileMultiView)
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	FDBufferParameters Parameters;
	Parameters.DBufferATextureSampler = TStaticSamplerState<>::GetRHI();
	Parameters.DBufferBTextureSampler = TStaticSamplerState<>::GetRHI();
	Parameters.DBufferCTextureSampler = TStaticSamplerState<>::GetRHI();
	Parameters.DBufferATexture = SystemTextures.BlackAlphaOne;
	Parameters.DBufferBTexture = SystemTextures.DefaultNormal8Bit;
	Parameters.DBufferCTexture = SystemTextures.BlackAlphaOne;
	Parameters.DBufferATextureArray = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, EPixelFormat::PF_B8G8R8A8, FClearValueBinding::Black);
	Parameters.DBufferBTextureArray = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, EPixelFormat::PF_B8G8R8A8, FClearValueBinding::Black);
	Parameters.DBufferCTextureArray = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, EPixelFormat::PF_B8G8R8A8, FClearValueBinding::Black);

	Parameters.DBufferRenderMask = SystemTextures.White;

	if (DBufferTextures.IsValid())
	{
		if (bIsMobileMultiView)
		{
			Parameters.DBufferATextureArray = DBufferTextures.DBufferATexArray;
			Parameters.DBufferBTextureArray = DBufferTextures.DBufferBTexArray;
			Parameters.DBufferCTextureArray = DBufferTextures.DBufferCTexArray;
		}
		else
		{
			Parameters.DBufferATexture = DBufferTextures.DBufferA;
			Parameters.DBufferBTexture = DBufferTextures.DBufferB;
			Parameters.DBufferCTexture = DBufferTextures.DBufferC;
		}

		if (DBufferTextures.DBufferMask)
		{
			Parameters.DBufferRenderMask = DBufferTextures.DBufferMask;
		}
	}

	return Parameters;
}
