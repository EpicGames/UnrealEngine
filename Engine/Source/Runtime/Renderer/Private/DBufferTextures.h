// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphDefinitions.h"
#include "RHIDefinitions.h"
#include "ShaderParameterMacros.h"

struct FSceneTextures;
class FSceneViewFamily;
class FSceneTextureUniformParameters;
class FViewInfo;

enum class EDecalDBufferMaskTechnique
{
	Disabled,	// DBufferMask is not enabled.
	PerPixel,	// DBufferMask is written explicitly by the shader during the DBuffer pass.
	WriteMask,	// DBufferMask is constructed after the DBuffer pass by compositing DBuffer write mask planes together in a compute shader.
};

EDecalDBufferMaskTechnique GetDBufferMaskTechnique(EShaderPlatform ShaderPlatform);

struct FDBufferTexturesDesc
{
	FRDGTextureDesc DBufferADesc;
	FRDGTextureDesc DBufferBDesc;
	FRDGTextureDesc DBufferCDesc;
	FRDGTextureDesc DBufferATexArrayDesc;
	FRDGTextureDesc DBufferBTexArrayDesc;
	FRDGTextureDesc DBufferCTexArrayDesc;
	FRDGTextureDesc DBufferMaskDesc;
};

struct FDBufferTextures
{
	bool IsValid() const;

	FRDGTextureRef DBufferA = nullptr;
	FRDGTextureRef DBufferB = nullptr;
	FRDGTextureRef DBufferC = nullptr;
	FRDGTextureRef DBufferATexArray = nullptr;
	FRDGTextureRef DBufferBTexArray = nullptr;
	FRDGTextureRef DBufferCTexArray = nullptr;
	FRDGTextureRef DBufferMask = nullptr;

	// We use an external flag to test whether the render targets have been produced, rather than "HasBeenProduced()", for the purpose of deciding whether
	// the render targets need to be cleared.  This is necessary for Custom Render Passes, where the render targets need to be cleared again when the main
	// view renders.  Without this, decals from Custom Render Passes will still be in the buffer and incorrectly render on top of the main view.
	bool bDBufferProduced = false;
	bool bDBufferTexArrayProduced = false;
};

FDBufferTexturesDesc GetDBufferTexturesDesc(FIntPoint Extent, EShaderPlatform ShaderPlatform);
FDBufferTextures CreateDBufferTextures(FRDGBuilder& GraphBuilder, FIntPoint Extent, EShaderPlatform ShaderPlatform, const bool bIsMobileMultiView = false);

BEGIN_SHADER_PARAMETER_STRUCT(FDBufferParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DBufferATexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DBufferBTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DBufferCTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, DBufferATextureArray)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, DBufferBTextureArray)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, DBufferCTextureArray)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, DBufferRenderMask)
	SHADER_PARAMETER_SAMPLER(SamplerState, DBufferATextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, DBufferBTextureSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, DBufferCTextureSampler)
END_SHADER_PARAMETER_STRUCT()

FDBufferParameters GetDBufferParameters(FRDGBuilder& GraphBuilder, const FDBufferTextures& DBufferTextures, EShaderPlatform ShaderPlatform, const bool bIsMobileMultiView = false);
