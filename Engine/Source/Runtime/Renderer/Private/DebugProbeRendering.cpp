// Copyright Epic Games, Inc. All Rights Reserved.

#include "DebugProbeRendering.h"
#include "PixelShaderUtils.h"
#include "ShaderParameterStruct.h"
#include "SceneRendering.h"
#include "SceneTextures.h"
#include "SceneTextureParameters.h"
#include "Substrate/Substrate.h"
#include "DataDrivenShaderPlatformInfo.h"

// We do not want such debug in shipping build or when the editor is not available.
#define DEBUG_PROBE_ENABLED (!UE_BUILD_SHIPPING && WITH_EDITORONLY_DATA)

//
//	Deferred probes are only stamped in deferred mode.
//

// Changing this causes a full shader recompile
static TAutoConsoleVariable<int32> CVarVisualizeLightingOnProbes(
	TEXT("r.VisualizeLightingOnProbes"),
	0,
	TEXT("Enables debug probes rendering to visualise diffuse/specular lighting (direct and indirect) on simple sphere scattered in the world.") \
	TEXT(" 0: disabled.\n")
	TEXT(" 1: camera probes only.\n")
	TEXT(" 2: world probes only.\n")
	TEXT(" 3: camera and world probes.\n")
	,
	ECVF_RenderThreadSafe);


DECLARE_GPU_STAT(StampDeferredDebugProbe);

extern bool IsIlluminanceMeterSupportedByView(const FViewInfo& View);

// Must match DebugProbes.usf
#define RENDER_DEPTHPREPASS  0
#define RENDER_BASEPASS	     1
#define RENDER_VELOCITYPASS  2

static bool IsDeferredDebugProbeSupportedByShaderPlatform(const FStaticShaderPlatform ShaderPlatform)
{
	// Only SM6 since SM5 is limited to 8 UAV.
	return IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM6);
}

class FStampDeferredDebugProbePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FStampDeferredDebugProbePS);
	SHADER_USE_PARAMETER_STRUCT(FStampDeferredDebugProbePS, FGlobalShader);

	class FRenderPass : SHADER_PERMUTATION_RANGE_INT("PERMUTATION_PASS", 0, 3);
	class FIlluminanceMeter : SHADER_PERMUTATION_BOOL("PERMUTATION_ILLUMINANCEMETER");
	using FPermutationDomain = TShaderPermutationDomain<FRenderPass, FIlluminanceMeter>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, MaterialTextureArrayUAV)
		SHADER_PARAMETER(uint32, MaxBytesPerPixel)
		SHADER_PARAMETER(uint32, bRoughDiffuse)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER(int32, DebugProbesMode)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsDeferredDebugProbeSupportedByShaderPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FStampDeferredDebugProbePS, "/Engine/Private/DebugProbes.usf", "MainPS", SF_Pixel);

#if DEBUG_PROBE_ENABLED

static bool ViewRequiresAndSupportsIlluminanceMeter(const FViewInfo& View)
{
	if (View.Family->EngineShowFlags.VisualizeHDR		// This is the debug view contain the illuminance meter
		&& IsIlluminanceMeterSupportedByView(View)		// For instance: forward shading does not work with illuminance meter (we need a material buffer to work with)
		&& !View.Family->EngineShowFlags.Wireframe)		// Disable when in wireframe mode
	{
		return true;
	}
	return false;
}
 
template<bool bEnableDepthWrite, ECompareFunction CompareFunction>
static void CommonStampDeferredDebugProbeDrawCall(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, 
	FStampDeferredDebugProbePS::FParameters* PassParameters,
	int32 RenderPass,
	bool bIlluminanceMeter)
{
	check(FPlatformProperties::HasEditorOnlyData());

	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->MaterialTextureArrayUAV = View.SubstrateViewData.SceneData->MaterialTextureArrayUAVWithoutRTs;
	PassParameters->MaxBytesPerPixel = View.SubstrateViewData.SceneData->EffectiveMaxBytesPerPixel;
	PassParameters->bRoughDiffuse = View.SubstrateViewData.SceneData->bRoughDiffuse ? 1 : 0;
	PassParameters->DebugProbesMode = View.Family->EngineShowFlags.VisualizeLightingOnProbes ? 3 : FMath::Clamp(CVarVisualizeLightingOnProbes.GetValueOnRenderThread(), 0, 3);
		
	FStampDeferredDebugProbePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FStampDeferredDebugProbePS::FRenderPass>(RenderPass);
	PermutationVector.Set<FStampDeferredDebugProbePS::FIlluminanceMeter>(bIlluminanceMeter);
	TShaderMapRef<FStampDeferredDebugProbePS> PixelShader(View.ShaderMap, PermutationVector);

	FPixelShaderUtils::AddFullscreenPass<FStampDeferredDebugProbePS>(
		GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("StampDeferredDebugProbePS"),
		PixelShader, PassParameters, View.ViewRect,
		TStaticBlendState<>::GetRHI(),
		TStaticRasterizerState<FM_Solid, CM_None>::GetRHI(),
		TStaticDepthStencilState<bEnableDepthWrite, CompareFunction>::GetRHI());
}

#endif // DEBUG_PROBE_ENABLED

void StampDeferredDebugProbeDepthPS(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FRDGTextureRef SceneDepthTexture)
{
#if DEBUG_PROBE_ENABLED
	if (!FPlatformProperties::HasEditorOnlyData())
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, StampDeferredDebugProbe, "StampDeferredDebugProbeDepth");

	const bool bVisualizeLightingOnProbes = CVarVisualizeLightingOnProbes.GetValueOnRenderThread() > 0;
	for (const FViewInfo& View : Views)
	{
		const bool bViewRequiresAndSupportsIlluminanceMeter = ViewRequiresAndSupportsIlluminanceMeter(View);
		if (!(bVisualizeLightingOnProbes || View.Family->EngineShowFlags.VisualizeLightingOnProbes) || View.bIsReflectionCapture || bViewRequiresAndSupportsIlluminanceMeter
			|| !IsDeferredDebugProbeSupportedByShaderPlatform(View.GetShaderPlatform()))
		{
			// When "Visualizing HDR with Illuminance Meter", we want to evaluated the illuminance on the surface behind the target square patch.  
			// So we do not want write depth or velocity in this case.
			continue;
		}

		FStampDeferredDebugProbePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStampDeferredDebugProbePS::FParameters>();
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);

		CommonStampDeferredDebugProbeDrawCall<true, CF_DepthNearOrEqual>(GraphBuilder, View, PassParameters, RENDER_DEPTHPREPASS, bViewRequiresAndSupportsIlluminanceMeter);
	}
#endif // DEBUG_PROBE_ENABLED
}

void StampDeferredDebugProbeMaterialPS(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FRenderTargetBindingSlots& BasePassRenderTargets,
	const FMinimalSceneTextures& SceneTextures)
{
#if DEBUG_PROBE_ENABLED
	if (!FPlatformProperties::HasEditorOnlyData())
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, StampDeferredDebugProbe, "StampDeferredDebugProbeMaterial");

	const bool bVisualizeLightingOnProbes = CVarVisualizeLightingOnProbes.GetValueOnRenderThread() > 0;
	for (const FViewInfo& View : Views)
	{
		const bool bViewRequiresAndSupportsIlluminanceMeter = ViewRequiresAndSupportsIlluminanceMeter(View);
		if (!(bVisualizeLightingOnProbes || View.Family->EngineShowFlags.VisualizeLightingOnProbes || bViewRequiresAndSupportsIlluminanceMeter) || View.bIsReflectionCapture
			|| !IsDeferredDebugProbeSupportedByShaderPlatform(View.GetShaderPlatform()))
		{
			continue;
		}

		FStampDeferredDebugProbePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStampDeferredDebugProbePS::FParameters>();
		PassParameters->RenderTargets = BasePassRenderTargets;

		if (bViewRequiresAndSupportsIlluminanceMeter)
		{
			// We do not want to update the normal of the material in this case,
			// because in this case we want to use the scene normal as the direction of the hemisphere over which we evaluate Illuminance.
			// So for legacy, we unbind the normal buffer and for Substrate we unbind the toplayer data (that is enough since the material written is going to be of simple type)
			if (Substrate::IsSubstrateEnabled())
			{
				// Search for the TopLayerTexture render target and nullify it to not update it.
				for (int32 i = 0; i < MaxSimultaneousRenderTargets; ++i)
				{
					if (PassParameters->RenderTargets.Output[i].GetTexture() == View.SubstrateViewData.SceneData->TopLayerTexture)
					{
						PassParameters->RenderTargets.Output[i] = FRenderTargetBinding();
					}
				}
			}
			else
			{
				// Do not write to the GBuffer normal render target, use a dummy render target to avoid validation issue with target textures that must be packed.
				PassParameters->RenderTargets.Output[1] = FRenderTargetBinding(GraphBuilder.CreateTexture(PassParameters->RenderTargets.Output[1].GetTexture()->Desc, TEXT("DummyGBufferNormalTexture")), ERenderTargetLoadAction::ELoad);
			}
		}

		if (Substrate::IsSubstrateEnabled())
		{
			// Make sure we do not write depth so that we can safely read it from texture parameters
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding();
			PassParameters->SceneTextures = CreateSceneTextureShaderParameters(GraphBuilder, View.GetSceneTexturesChecked(), View.GetFeatureLevel(), ESceneTextureSetupMode::SceneDepth);

			CommonStampDeferredDebugProbeDrawCall<false, CF_Always>(GraphBuilder, View, PassParameters, RENDER_BASEPASS, bViewRequiresAndSupportsIlluminanceMeter);
		}
		else
		{
			CommonStampDeferredDebugProbeDrawCall<false, CF_DepthNearOrEqual>(GraphBuilder, View, PassParameters, RENDER_BASEPASS, bViewRequiresAndSupportsIlluminanceMeter);
		}
	}
#endif // DEBUG_PROBE_ENABLED
}

void StampDeferredDebugProbeVelocityPS(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FRenderTargetBindingSlots& BasePassRenderTargets)
{
#if DEBUG_PROBE_ENABLED
	if (!FPlatformProperties::HasEditorOnlyData())
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, StampDeferredDebugProbe, "StampDeferredDebugProbeVelocity");

	const bool bVisualizeLightingOnProbes = CVarVisualizeLightingOnProbes.GetValueOnRenderThread() > 0;
	for (const FViewInfo& View : Views)
	{
		const bool bViewRequiresAndSupportsIlluminanceMeter = ViewRequiresAndSupportsIlluminanceMeter(View);
		if (!(bVisualizeLightingOnProbes || View.Family->EngineShowFlags.VisualizeLightingOnProbes) || View.bIsReflectionCapture || bViewRequiresAndSupportsIlluminanceMeter
			|| !IsDeferredDebugProbeSupportedByShaderPlatform(View.GetShaderPlatform()))
		{
			// When "Visualizing HDR with Illuminance Meter", we want to evaluated the illuminance on the surface behind the target square patch. 
			// So we do not want write depth or velocity in this case.
			continue;
		}

		FStampDeferredDebugProbePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FStampDeferredDebugProbePS::FParameters>();
		PassParameters->RenderTargets = BasePassRenderTargets;

		const bool bRenderVelocity = true;
		CommonStampDeferredDebugProbeDrawCall<false, CF_DepthNearOrEqual>(GraphBuilder, View, PassParameters, RENDER_VELOCITYPASS, bViewRequiresAndSupportsIlluminanceMeter);
	}
#endif // DEBUG_PROBE_ENABLED
}

