// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DebugProbeRendering.h: Debug probe pass rendering definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphBuilder.h"
#include "SceneRendering.h"


void StampDeferredDebugProbeDepthPS(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FRDGTextureRef SceneDepthTexture);


void StampDeferredDebugProbeMaterialPS(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FRenderTargetBindingSlots& BasePassRenderTargets,
	const FMinimalSceneTextures& SceneTextures);


void StampDeferredDebugProbeVelocityPS(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FRenderTargetBindingSlots& BasePassRenderTargets);

