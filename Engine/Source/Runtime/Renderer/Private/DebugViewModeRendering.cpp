// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
DebugViewModeRendering.cpp: Contains definitions for rendering debug viewmodes.
=============================================================================*/

#include "DebugViewModeRendering.h"
#include "CoreMinimal.h"
#include "BasePassRendering.h"
#include "CanvasTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DebugViewModeHelpers.h"
#include "DebugViewModeInterface.h"
#include "DecalRenderingShared.h"
#include "DepthRendering.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/StreamableRenderAsset.h"
#include "Engine/Texture.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "GBufferInfo.h"
#include "HAL/IConsoleManager.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "LightMapRendering.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "MeshBatch.h"
#include "MeshDrawCommands.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "MeshUVChannelInfo.h"
#include "Misc/LargeWorldCoordinates.h"
#include "MobileBasePassRendering.h"
#include "MultiGPU.h"
#include "PostProcess/PostProcessVisualizeComplexity.h"
#include "PrimitiveSceneProxy.h"
#include "RenderGraphBuilder.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextures.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShowFlags.h"
#include "TextureResource.h"
#include "VelocityRendering.h"
#include "VertexFactory.h"

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FDebugViewModeUniformParameters, "DebugViewModeStruct");
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FDebugViewModePassUniformParameters, "DebugViewModePass", SceneTextures);

#if WITH_DEBUG_VIEW_MODES

int32 GShaderComplexityBaselineForwardVS = 134;
static FAutoConsoleVariableRef CVarShaderComplexityBaselineForwardVS(
	TEXT("r.ShaderComplexity.Baseline.Forward.VS"),
	GShaderComplexityBaselineForwardVS,
	TEXT("Minimum number of instructions for vertex shaders in forward shading (default=134)"),
	ECVF_Default
);

int32 GShaderComplexityBaselineForwardPS = 635;
static FAutoConsoleVariableRef CVarShaderComplexityBaselineForwardPS(
	TEXT("r.ShaderComplexity.Baseline.Forward.PS"),
	GShaderComplexityBaselineForwardPS,
	TEXT("Minimum number of instructions for pixel shaders in forward shading (default=635)"),
	ECVF_Default
);

int32 GShaderComplexityBaselineForwardUnlitPS = 47;
static FAutoConsoleVariableRef CVarShaderComplexityBaselineForwardUnlitPS(
	TEXT("r.ShaderComplexity.Baseline.Forward.UnlitPS"),
	GShaderComplexityBaselineForwardUnlitPS,
	TEXT("Minimum number of instructions for unlit material pixel shaders in forward shading (default=47)"),
	ECVF_Default
);

int32 GShaderComplexityBaselineDeferredVS = 41;
static FAutoConsoleVariableRef CVarShaderComplexityBaselineDeferredVS(
	TEXT("r.ShaderComplexity.Baseline.Deferred.VS"),
	GShaderComplexityBaselineDeferredVS,
	TEXT("Minimum number of instructions for vertex shaders in deferred shading (default=41)"),
	ECVF_Default
);

int32 GShaderComplexityBaselineDeferredPS = 111;
static FAutoConsoleVariableRef CVarShaderComplexityBaselineDeferredPS(
	TEXT("r.ShaderComplexity.Baseline.Deferred.PS"),
	GShaderComplexityBaselineDeferredPS,
	TEXT("Minimum number of instructions for pixel shaders in deferred shading (default=111)"),
	ECVF_Default
);

int32 GShaderComplexityBaselineDeferredUnlitPS = 33;
static FAutoConsoleVariableRef CVarShaderComplexityBaselineDeferredUnlitPS(
	TEXT("r.ShaderComplexity.Baseline.Deferred.UnlitPS"),
	GShaderComplexityBaselineDeferredUnlitPS,
	TEXT("Minimum number of instructions for unlit material pixel shaders in deferred shading (default=33)"),
	ECVF_Default
);

float GShaderComplexityMobileMaskedCostMultiplier = 1.5f;
static FAutoConsoleVariableRef CVarShaderComplexityMobileMaskedCostMultiplier(
	TEXT("r.ShaderComplexity.MobileMaskedCostMultiplier"),
	GShaderComplexityMobileMaskedCostMultiplier,
	TEXT("Extra cost of masked materials if we do not use masked in early Z optimization"),
	ECVF_Default
);

IMPLEMENT_MATERIAL_SHADER_TYPE(, FDebugViewModePS, TEXT("/Engine/Private/DebugViewModePixelShader.usf"), TEXT("Main"), SF_Pixel);

int32 GetQuadOverdrawUAVIndex(EShaderPlatform Platform)
{
	if (IsForwardShadingEnabled(Platform))
	{
		return FVelocityRendering::BasePassCanOutputVelocity(Platform) ? 2 : 1;
	}
	else // GBuffer
	{
		return FVelocityRendering::BasePassCanOutputVelocity(Platform) ? 7 : 6;
	}
}

void SetupDebugViewModePassUniformBufferConstants(const FViewInfo& ViewInfo, FDebugViewModeUniformParameters& Parameters)
{
	// Accuracy colors
	{
		const int32 NumEngineColors = FMath::Min<int32>(GEngine->StreamingAccuracyColors.Num(), NumStreamingAccuracyColors);
		int32 ColorIndex = 0;
		for (; ColorIndex < NumEngineColors; ++ColorIndex)
		{
			Parameters.AccuracyColors[ColorIndex] = GEngine->StreamingAccuracyColors[ColorIndex];
		}
		for (; ColorIndex < NumStreamingAccuracyColors; ++ColorIndex)
		{
			Parameters.AccuracyColors[ColorIndex] = FLinearColor::Black;
		}
	}
	// LOD / HLOD colors
	{
		const TArray<FLinearColor>* Colors = nullptr;
		if (ViewInfo.Family->EngineShowFlags.LODColoration)
		{
			Colors = &(GEngine->LODColorationColors);
		}
		else if (ViewInfo.Family->EngineShowFlags.HLODColoration)
		{
			Colors = &GEngine->HLODColorationColors;
		}
		else if (ViewInfo.Family->EngineShowFlags.StreamingTextureDeficit
		      || ViewInfo.Family->EngineShowFlags.StreamingTextureResidency
		      || ViewInfo.Family->EngineShowFlags.StreamingMeshLODDeficit
		      || ViewInfo.Family->EngineShowFlags.StreamingMeshLODResidency)
		{
			Colors = &GEngine->StreamingDeficitColors;
		}

		const int32 NumColors = Colors ? FMath::Min<int32>(NumLODColorationColors, Colors->Num()) : 0;
		int32 ColorIndex = 0;
		for (; ColorIndex < NumColors; ++ColorIndex)
		{
			Parameters.LODColors[ColorIndex] = (*Colors)[ColorIndex];
		}
		for (; ColorIndex < NumLODColorationColors; ++ColorIndex)
		{
			Parameters.LODColors[ColorIndex] = NumColors > 0 ? Colors->Last() : FLinearColor::Black;
		}
	}
}

TRDGUniformBufferRef<FDebugViewModePassUniformParameters> CreateDebugViewModePassUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef DebugAuxTexture)
{
	FRDGTextureRef QuadOverdrawTexture = nullptr;

	EDebugViewShaderMode DebugViewMode = View.Family->GetDebugViewShaderMode();
	if (DebugViewMode == EDebugViewShaderMode::DVSM_QuadComplexity || DebugViewMode == DVSM_ShaderComplexityContainedQuadOverhead)
	{
		QuadOverdrawTexture = DebugAuxTexture;
		DebugAuxTexture = nullptr;
	}

	if (!QuadOverdrawTexture)
	{
		QuadOverdrawTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV), TEXT("DummyOverdrawUAV"));
	}

	auto* UniformBufferParameters = GraphBuilder.AllocParameters<FDebugViewModePassUniformParameters>();
	SetupSceneTextureUniformParameters(GraphBuilder, View.GetSceneTexturesChecked(), View.FeatureLevel, ESceneTextureSetupMode::None, UniformBufferParameters->SceneTextures);
	SetupDebugViewModePassUniformBufferConstants(View, UniformBufferParameters->DebugViewMode);
	UniformBufferParameters->QuadOverdraw = GraphBuilder.CreateUAV(QuadOverdrawTexture);
	return GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
}

static bool ShouldCompileDebugViewModeShaders(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	if (ForceEnableDebugViewSupport(Parameters.Platform))
	{
		return true;
	}

	if (ForceDisableDebugViewSupport(Parameters.Platform))
	{
		return false;
	}

	if (Parameters.VertexFactoryType && Parameters.VertexFactoryType->SupportsNaniteRendering())
	{
		return false;
	}

	EShaderPermutationFlags Flags = IsDebugViewShaderModeODSCOnly() ? EShaderPermutationFlags::IsODSCOnly : EShaderPermutationFlags::HasEditorOnlyData | EShaderPermutationFlags::IsODSCOnly;
	return EnumHasAnyFlags(Parameters.Flags, Flags);
}

FDebugViewModeVS::FDebugViewModeVS() = default;
FDebugViewModeVS::FDebugViewModeVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
	: FMeshMaterialShader(Initializer)
{}

bool FDebugViewModeVS::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return ShouldCompileDebugViewModeShaders(Parameters);
}

void FDebugViewModeVS::GetShaderBindings(
	const FScene* Scene,
	ERHIFeatureLevel::Type FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	const FDebugViewModeShaderElementData& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings) const
{
	FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
}

void FDebugViewModeVS::SetCommonDefinitions(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	// SM4 has less input interpolants. Also instanced meshes use more interpolants.
	if (Parameters.MaterialParameters.bIsDefaultMaterial || (IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && !Parameters.MaterialParameters.bIsUsedWithInstancedStaticMeshes))
	{	// Force the default material to pass enough texcoords to the pixel shaders (even though not using them).
		// This is required to allow material shaders to have access to the sampled coords.
		OutEnvironment.SetDefine(TEXT("MIN_MATERIAL_TEXCOORDS"), (uint32)4);
	}
	else // Otherwise still pass at minimum amount to have debug shader using a texcoord to work (material might not use any).
	{
		OutEnvironment.SetDefine(TEXT("MIN_MATERIAL_TEXCOORDS"), (uint32)2);
	}

}

void FDebugViewModeVS::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	SetCommonDefinitions(Parameters, OutEnvironment);
	FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

IMPLEMENT_MATERIAL_SHADER_TYPE(,FDebugViewModeVS,TEXT("/Engine/Private/DebugViewModeVertexShader.usf"),TEXT("Main"),SF_Vertex);

BEGIN_SHADER_PARAMETER_STRUCT(FDebugViewModePassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FDebugViewModePassUniformParameters, Pass)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void ClearDebugAux(FRDGBuilder& GraphBuilder, const FViewFamilyInfo& ViewFamily, FRDGTextureRef DebugAuxTexture)
{
	if (DebugAuxTexture)
	{
		EDebugViewShaderMode DebugViewMode = ViewFamily.GetDebugViewShaderMode();
		FRDGTextureUAVRef DebugAuxTextureUAV = GraphBuilder.CreateUAV(DebugAuxTexture);

		if (DebugViewMode == DVSM_QuadComplexity || DebugViewMode == DVSM_ShaderComplexityContainedQuadOverhead)
		{
			AddClearUAVPass(GraphBuilder, DebugAuxTextureUAV, FUintVector4(0, 0, 0, 0));
		}
		else if (DebugViewMode == DVSM_ShadowCasters)
		{
			const uint32 InvalidInstanceId = 0xFFFFFFFF;
			AddClearUAVPass(GraphBuilder, DebugAuxTextureUAV, FUintVector4(InvalidInstanceId, 0, 0, 0));
		}
	}
}

void RenderDebugViewMode(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views, FRDGTextureRef DebugAuxTexture, const FRenderTargetBinding& SceneColorRTBinding, const FDepthStencilBinding& SceneDepthBinding)
{
	RDG_EVENT_SCOPE(GraphBuilder, "DebugViewMode");

	check(SceneColorRTBinding.GetTexture());

	if (Views.Num() == 0)
	{
		return;
	}
	
	FIntPoint SceneColorExtent = SceneColorRTBinding.GetTexture()->Desc.Extent;
	
	// Dummy needs to have the correct size and type to prevent validation errors, but we can use R8 instead of R32.
	const FRDGTextureDesc RenderTarget1Desc = FRDGTextureDesc::Create2D(SceneColorExtent, PF_R8_UINT, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
	FRDGTextureRef DummyRenderTarget1 = GraphBuilder.CreateTexture(RenderTarget1Desc, TEXT("DummyDebugAux"));

	FRenderTargetBindingSlots DebugRenderTargets{};
	DebugRenderTargets.DepthStencil = SceneDepthBinding;
	DebugRenderTargets[0] = SceneColorRTBinding;
	DebugRenderTargets[1] = FRenderTargetBinding(DummyRenderTarget1, ERenderTargetLoadAction::ENoAction);

	FRDGTextureRef QuadOverdrawTexture = nullptr;

	EDebugViewShaderMode DebugViewMode = Views[0].Family->GetDebugViewShaderMode();
	if (DebugViewMode == EDebugViewShaderMode::DVSM_QuadComplexity || DebugViewMode == DVSM_ShaderComplexityContainedQuadOverhead)
	{
		QuadOverdrawTexture = DebugAuxTexture;
		DebugAuxTexture = nullptr;
	}
	else if (DebugViewMode == EDebugViewShaderMode::DVSM_ShadowCasters)
	{
		DebugRenderTargets[1] = FRenderTargetBinding(DebugAuxTexture, ERenderTargetLoadAction::ELoad);
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];

		if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::DebugViewMode])
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			auto* PassParameters = GraphBuilder.AllocParameters<FDebugViewModePassParameters>();
			PassParameters->View = View.GetShaderParameters();
			PassParameters->Pass = CreateDebugViewModePassUniformBuffer(GraphBuilder, View, QuadOverdrawTexture);
			PassParameters->RenderTargets = DebugRenderTargets;

			FScene* Scene = View.Family->Scene->GetRenderScene();
			check(Scene != nullptr);

			Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

			GraphBuilder.AddPass(
				{},
				PassParameters,
				ERDGPassFlags::Raster,
				[&View, Pass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1);
				Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
			});
		}
	}
}

FDebugViewModePS::FDebugViewModePS() = default;
FDebugViewModePS::FDebugViewModePS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
	: FMeshMaterialShader(Initializer)
{
	OneOverCPUTexCoordScalesParameter.Bind(Initializer.ParameterMap, TEXT("OneOverCPUTexCoordScales"));
	TexCoordIndicesParameter.Bind(Initializer.ParameterMap, TEXT("TexCoordIndices"));
	CPUTexelFactorParameter.Bind(Initializer.ParameterMap, TEXT("CPUTexelFactor"));
	NormalizedComplexity.Bind(Initializer.ParameterMap, TEXT("NormalizedComplexity"));
	AnalysisParamsParameter.Bind(Initializer.ParameterMap, TEXT("AnalysisParams"));
	PrimitiveAlphaParameter.Bind(Initializer.ParameterMap, TEXT("PrimitiveAlpha"));
	PrimitiveBlendModeParameter.Bind(Initializer.ParameterMap, TEXT("PrimitiveBlendMode"));
	TexCoordAnalysisIndexParameter.Bind(Initializer.ParameterMap, TEXT("TexCoordAnalysisIndex"));
	CPULogDistanceParameter.Bind(Initializer.ParameterMap, TEXT("CPULogDistance"));
	ShowQuadOverdraw.Bind(Initializer.ParameterMap, TEXT("bShowQuadOverdraw"));
	OutputQuadOverdrawParameter.Bind(Initializer.ParameterMap, TEXT("bOutputQuadOverdraw"));
	LODIndexParameter.Bind(Initializer.ParameterMap, TEXT("LODIndex"));
	SkinCacheDebugColorParameter.Bind(Initializer.ParameterMap, TEXT("SkinCacheDebugColor"));
	VisualizeModeParameter.Bind(Initializer.ParameterMap, TEXT("VisualizeMode"));
	QuadBufferUAV.Bind(Initializer.ParameterMap, TEXT("RWQuadBuffer"));
}

bool FDebugViewModePS::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return ShouldCompileDebugViewModeShaders(Parameters);
}

void FDebugViewModePS::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("UNDEFINED_ACCURACY"), UndefinedStreamingAccuracyIntensity);
	OutEnvironment.SetDefine(TEXT("MAX_NUM_TEX_COORD"), (uint32)TEXSTREAM_MAX_NUM_UVCHANNELS);
	OutEnvironment.SetDefine(TEXT("INITIAL_GPU_SCALE"), (uint32)TEXSTREAM_INITIAL_GPU_SCALE);
	OutEnvironment.SetDefine(TEXT("TILE_RESOLUTION"), (uint32)TEXSTREAM_TILE_RESOLUTION);
	OutEnvironment.SetDefine(TEXT("MAX_NUM_TEXTURE_REGISTER"), (uint32)TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL);
	OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1u);
	TCHAR BufferRegister[] = { 'u', '0', 0 };
	BufferRegister[1] += GetQuadOverdrawUAVIndex(Parameters.Platform);
	OutEnvironment.SetDefine(TEXT("QUAD_BUFFER_REGISTER"), BufferRegister);
	OutEnvironment.SetDefine(TEXT("OUTPUT_QUAD_OVERDRAW"), SupportDebugViewShaderMode(DVSM_QuadComplexity, Parameters.Platform));

	OutEnvironment.SetDefine(TEXT("BLEND_Opaque"), (int)BLEND_Opaque);
	OutEnvironment.SetDefine(TEXT("BLEND_Masked"), (int)BLEND_Masked);

	for (int i = 0; i < DVSM_MAX; ++i)
	{
		OutEnvironment.SetDefine(DebugViewShaderModeToString(static_cast<EDebugViewShaderMode>(i)), i);
	}

	OutEnvironment.SetDefine(TEXT("NUM_LOD_COLORATION_COLORS"), (uint32)NumLODColorationColors);

	FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

void FDebugViewModePS::GetElementShaderBindings(
	const FShaderMapPointerTable& PointerTable,
	const FScene* Scene,
	const FSceneView* ViewIfDynamicMeshCommand,
	const FVertexFactory* VertexFactory,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMeshBatch& MeshBatch,
	const FMeshBatchElement& BatchElement,
	const FDebugViewModeShaderElementData& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams) const
{
	FMeshMaterialShader::GetElementShaderBindings(PointerTable, Scene, ViewIfDynamicMeshCommand, VertexFactory, InputStreamType, FeatureLevel, PrimitiveSceneProxy, MeshBatch, BatchElement, ShaderElementData, ShaderBindings, VertexStreams);

	int8 VisualizeElementIndex = 0;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	VisualizeElementIndex = BatchElement.VisualizeElementIndex;
#endif

	const FDebugViewModeInterface* Interface = FDebugViewModeInterface::GetInterface(ShaderElementData.DebugViewMode);
	if (ensure(Interface))
	{
		Interface->GetDebugViewModeShaderBindings(
			*this,
			PrimitiveSceneProxy,
			ShaderElementData.MaterialRenderProxy,
			ShaderElementData.Material,
			ShaderElementData.DebugViewMode,
			ShaderElementData.ViewOrigin,
			ShaderElementData.VisualizeLODIndex,
			ShaderElementData.SkinCacheDebugColor,
			VisualizeElementIndex,
			ShaderElementData.NumVSInstructions,
			ShaderElementData.NumPSInstructions,
			ShaderElementData.LWCComplexityVS,
			ShaderElementData.LWCComplexityPS,
			ShaderElementData.ViewModeParam,
			ShaderElementData.ViewModeParamName,
			ShaderBindings
		);
	}
}

FDebugViewModeMeshProcessor::FDebugViewModeMeshProcessor(
	const FScene* InScene, 
	ERHIFeatureLevel::Type InFeatureLevel,
	const FSceneView* InViewIfDynamicMeshCommand, 
	bool bTranslucentBasePass,
	FMeshPassDrawListContext* InDrawListContext
)
	: FMeshPassProcessor(EMeshPass::DebugViewMode, InScene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, DebugViewMode(DVSM_None)
	, ViewModeParam(INDEX_NONE)
	, DebugViewModeInterface(nullptr)
{
	if (InViewIfDynamicMeshCommand)
	{
		DebugViewMode = InViewIfDynamicMeshCommand->Family->GetDebugViewShaderMode();
		ViewModeParam = InViewIfDynamicMeshCommand->Family->GetViewModeParam();
		ViewModeParamName = InViewIfDynamicMeshCommand->Family->GetViewModeParamName();

		DebugViewModeInterface = FDebugViewModeInterface::GetInterface(DebugViewMode);
	}
}

/** Compute the worst-case texture streaming deficit across all textures in a material (uncached). */
int32 ComputeTextureStreamingDeficitForMaterial(const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material, bool bUseWantedMips)
{
#if !UE_BUILD_SHIPPING
	int32 MaxDeficit = 0;
	FMaterialRenderContext MaterialContext(&MaterialRenderProxy, Material, nullptr);
	const FUniformExpressionSet& UniformExpressions = Material.GetUniformExpressions();
	EMaterialTextureParameterType TextureTypes[] = { EMaterialTextureParameterType::Standard2D, EMaterialTextureParameterType::Virtual };

	for (EMaterialTextureParameterType TextureType : TextureTypes)
	{
		for (int32 ParameterIndex = 0; ParameterIndex < UniformExpressions.GetNumTextures(TextureType); ++ParameterIndex)
		{
			const UTexture* Texture = nullptr;
			UniformExpressions.GetTextureValue(TextureType, ParameterIndex, MaterialContext, Material, Texture);
			if (!Texture || !Texture->IsStreamable())
			{
				continue;
			}

			const FStreamableRenderResourceState State = Texture->GetStreamableResourceState();
			if (!State.bSupportsStreaming || State.MaxNumLODs == 0)
			{
				continue;
			}

			int32 Deficit;
			if (bUseWantedMips)
			{
				const uint8 WantedLODs = Texture->GetCachedNumWantedLODs();
				Deficit = (WantedLODs > 0) ? FMath::Max(0, (int32)WantedLODs - (int32)State.NumResidentLODs) : 0;
			}
			else
			{
				Deficit = FMath::Max(0, (int32)State.MaxNumLODs - (int32)State.NumResidentLODs);
			}

			MaxDeficit = FMath::Max(MaxDeficit, Deficit);
		}
	}
	return FMath::Clamp(MaxDeficit, 0, NumLODColorationColors - 1);
#else
	return 0;
#endif
}

/** Cached lookup: returns per-proxy texture streaming deficit, computing only on first access. */
static int32 ComputeTextureStreamingDeficit(const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material, bool bUseWantedMips, TMap<const FMaterialRenderProxy*, int32>& Cache)
{
#if !UE_BUILD_SHIPPING
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeTextureStreamingDeficit);

	// Cache by FMaterialRenderProxy
	const FMaterialRenderProxy* CacheKey = &MaterialRenderProxy;
	if (int32* Cached = Cache.Find(CacheKey))
	{
		return *Cached;
	}

	int32 Result = ComputeTextureStreamingDeficitForMaterial(MaterialRenderProxy, Material, bUseWantedMips);
	Cache.Add(CacheKey, Result);
	return Result;
#else
	return 0;
#endif
}

/** Compute mesh LOD streaming deficit from the scene proxy. */
static int32 ComputeMeshStreamingDeficit(const FPrimitiveSceneProxy* PrimitiveSceneProxy, bool bUseWantedMips)
{
#if !UE_BUILD_SHIPPING
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeMeshStreamingDeficit);
	if (!PrimitiveSceneProxy)
	{
		return 0;
	}

	if (bUseWantedMips)
	{
		uint8 Deficit = PrimitiveSceneProxy->GetStreamingLODWantedDeficit();
		return FMath::Clamp((int32)Deficit, 0, NumLODColorationColors - 1);
	}
	else
	{
		// Absolute residency: FirstLODIdx == number of missing LODs from the top
		uint8 FirstAvailableLOD = PrimitiveSceneProxy->GetCurrentFirstLODIdx_RenderThread();
		return FMath::Clamp((int32)FirstAvailableLOD, 0, NumLODColorationColors - 1);
	}
#else
	return 0;
#endif
}

void FDebugViewModeMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (!DebugViewModeInterface)
	{
		return;
	}

	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	const FMaterial* BatchMaterial = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
	if (!BatchMaterial)
	{
		return;
	}

	const FMaterial* Material = BatchMaterial;

	FVertexFactoryType* VertexFactoryType = MeshBatch.VertexFactory->GetType();

	FMaterialShaderTypes ShaderTypes;
	DebugViewModeInterface->AddShaderTypes(FeatureLevel, VertexFactoryType, ShaderTypes);
	if (!Material->ShouldCacheShaders(ShaderTypes, VertexFactoryType))
	{
		return;
	}

	FMaterialShaders Shaders;
	if (!Material->TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return;
	}

	TMeshProcessorShaders<FDebugViewModeVS,	FDebugViewModePS> DebugViewModePassShaders;
	Shaders.TryGetVertexShader(DebugViewModePassShaders.VertexShader);
	Shaders.TryGetPixelShader(DebugViewModePassShaders.PixelShader);

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*BatchMaterial, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(*BatchMaterial, OverrideSettings);
	const bool bDitheredLODTransition = MeshBatch.bDitheredLODTransition && MeshBatch.CachedLODDither != FMeshBatch::EDitherState::None;

	FMeshPassProcessorRenderState DrawRenderState;
	FDebugViewModeInterface::FRenderState InterfaceRenderState;
	DebugViewModeInterface->SetDrawRenderState(DebugViewMode, BatchMaterial->GetBlendMode(), InterfaceRenderState, Scene ? (Scene->GetShadingPath() == EShadingPath::Deferred && Scene->EarlyZPassMode != DDM_NonMaskedOnly) : false, bDitheredLODTransition);
	DrawRenderState.SetBlendState(InterfaceRenderState.BlendState);
	DrawRenderState.SetDepthStencilState(InterfaceRenderState.DepthStencilState);

	int32 DebugLODIndex = MeshBatch.VisualizeLODIndex;
	if (ViewIfDynamicMeshCommand)
	{
		const FEngineShowFlags& ViewFlags = ViewIfDynamicMeshCommand->Family->EngineShowFlags;
		if (ViewFlags.StreamingTextureDeficit)
		{
			DebugLODIndex = ComputeTextureStreamingDeficit(*MaterialRenderProxy, *Material, /*bUseWantedMips=*/ true, TextureStreamingDeficitCache);
		}
		else if (ViewFlags.StreamingTextureResidency)
		{
			DebugLODIndex = ComputeTextureStreamingDeficit(*MaterialRenderProxy, *Material, /*bUseWantedMips=*/ false, TextureStreamingDeficitCache);
		}
		else if (ViewFlags.StreamingMeshLODDeficit)
		{
			DebugLODIndex = ComputeMeshStreamingDeficit(PrimitiveSceneProxy, /*bUseWantedMips=*/ true);
		}
		else if (ViewFlags.StreamingMeshLODResidency)
		{
			DebugLODIndex = ComputeMeshStreamingDeficit(PrimitiveSceneProxy, /*bUseWantedMips=*/ false);
		}
		else if (ViewFlags.HLODColoration)
		{
			DebugLODIndex = MeshBatch.VisualizeHLODIndex;
		}
	}

	FDebugViewModeShaderElementData ShaderElementData(
		*MaterialRenderProxy,
		*Material,
		DebugViewMode,
		ViewIfDynamicMeshCommand ? ViewIfDynamicMeshCommand->ViewMatrices.GetViewOrigin() : FVector::ZeroVector,
		DebugLODIndex,
		(ViewIfDynamicMeshCommand && ViewIfDynamicMeshCommand->Family->EngineShowFlags.VisualizeGPUSkinCache) ? MeshBatch.Elements[0].SkinCacheDebugColor : FColor::White,
		ViewModeParam,
		ViewModeParamName);

	// Shadermap can be null while shaders are compiling.
	UpdateInstructionCount(ShaderElementData, BatchMaterial, VertexFactoryType);

	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(DebugViewModePassShaders.VertexShader, DebugViewModePassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		*MaterialRenderProxy,
		*Material,
		DrawRenderState,
		DebugViewModePassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);
}

void FDebugViewModeMeshProcessor::UpdateInstructionCount(FDebugViewModeShaderElementData& OutShaderElementData, const FMaterial* InBatchMaterial, FVertexFactoryType* InVertexFactoryType)
{
	check(InBatchMaterial && InVertexFactoryType);

	if (Scene)
	{
		if (Scene->GetShadingPath() == EShadingPath::Deferred)
		{
			const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InBatchMaterial->GetFeatureLevel());

			FMaterialShaderTypes ShaderTypes;
			ShaderTypes.AddShaderType<TBasePassVS<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>>>();

			ShaderTypes.AddShaderType<TBasePassPS<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>, GBL_Default>>();

			FMaterialShaders Shaders;
			if (InBatchMaterial->TryGetShaders(ShaderTypes, InVertexFactoryType, Shaders))
			{
				OutShaderElementData.NumVSInstructions = Shaders.Shaders[SF_Vertex]->GetNumInstructions();
				OutShaderElementData.NumPSInstructions = Shaders.Shaders[SF_Pixel]->GetNumInstructions();

				if (IsForwardShadingEnabled(ShaderPlatform) && !IsTranslucentBlendMode(*InBatchMaterial))
				{
					const bool bLit = InBatchMaterial->GetShadingModels().IsLit();

					// Those numbers are taken from a simple material where common inputs are bound to vector parameters (to prevent constant optimizations).
					OutShaderElementData.NumVSInstructions -= GShaderComplexityBaselineForwardVS - GShaderComplexityBaselineDeferredVS;
					OutShaderElementData.NumPSInstructions -= bLit ? (GShaderComplexityBaselineForwardPS - GShaderComplexityBaselineDeferredPS) : (GShaderComplexityBaselineForwardUnlitPS - GShaderComplexityBaselineDeferredUnlitPS);
				}

				OutShaderElementData.NumVSInstructions = FMath::Max<int32>(0, OutShaderElementData.NumVSInstructions);
				OutShaderElementData.NumPSInstructions = FMath::Max<int32>(0, OutShaderElementData.NumPSInstructions);
				OutShaderElementData.NumCSInstructions = 0;
			}
		}
		else // EShadingPath::Mobile
		{
			TShaderRef<TMobileBasePassVSPolicyParamType<FUniformLightMapPolicy>> MobileVS;
			TShaderRef<TMobileBasePassPSPolicyParamType<FUniformLightMapPolicy>> MobilePS;
			if (MobileBasePass::GetShaders(FUniformLightMapPolicy(LMP_NO_LIGHTMAP), EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED, *InBatchMaterial, InVertexFactoryType, MobileVS, MobilePS))
			{
				OutShaderElementData.NumVSInstructions = MobileVS.IsValid() ? MobileVS->GetNumInstructions() : 0;
				OutShaderElementData.NumPSInstructions = MobilePS.IsValid() ? MobilePS->GetNumInstructions() : 0;
				OutShaderElementData.NumCSInstructions = 0;
			}

			const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InBatchMaterial->GetFeatureLevel());
			if (IsMaskedBlendMode(*InBatchMaterial) && !MaskedInEarlyPass(ShaderPlatform))
			{
				OutShaderElementData.NumPSInstructions *= GShaderComplexityMobileMaskedCostMultiplier;
			}
		}
		
#if WITH_EDITOR
		InBatchMaterial->GetRenderingThreadShaderMap()->GetEstimatedLWCFuncUsageComplexity(
			OutShaderElementData.LWCComplexityVS, OutShaderElementData.LWCComplexityPS, OutShaderElementData.LWCComplexityCS);
#endif
	}
}

void FDebugViewModeImplementation::AddShaderTypes(ERHIFeatureLevel::Type InFeatureLevel,
	const FVertexFactoryType* InVertexFactoryType,
	FMaterialShaderTypes& OutShaderTypes) const
{
	OutShaderTypes.AddShaderType<FDebugViewModeVS>();
	OutShaderTypes.AddShaderType<FDebugViewModePS>();
}

float GMaxLWCComplexity = 40000.0f; // Arbitrary, chosen to make contrast visible
static FAutoConsoleVariableRef CVarMaxLWCComplexity(
	TEXT("r.ShaderComplexity.MaxLWCComplexity"),
	GMaxLWCComplexity,
	TEXT("Value used for scaling the material LWC usage visualization"));

void FDebugViewModeImplementation::GetDebugViewModeShaderBindings(
	const FDebugViewModePS& Shader,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	EDebugViewShaderMode DebugViewMode,
	const FVector& ViewOrigin,
	int32 VisualizeLODIndex,
	const FColor& SkinCacheDebugColor,
	int32 VisualizeElementIndex,
	int32 NumVSInstructions,
	int32 NumPSInstructions,
	int32 LWCComplexityVS,
	int32 LWCComplexityPS,
	int32 ViewModeParam,
	FName ViewModeParamName,
	FMeshDrawSingleShaderBindings& ShaderBindings) const
{
	FVector4f OneOverCPUTexCoordScales[TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL / 4];
	FIntVector4 TexCoordIndices[TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL / 4];
	FVector4 WorldUVDensities {};
	FVector4f NormalizedComplexityValue {};
	FIntPoint AnalysisParameter {};
	const float PrimitiveAlpha = (!PrimitiveSceneProxy || PrimitiveSceneProxy->IsSelected()) ? 1.f : .2f;
	const int32 TexCoordAnalysisIndex = ViewModeParam >= 0 ? FMath::Clamp<int32>(ViewModeParam, 0, MAX_TEXCOORDS - 1) : -1;
	float CPULogDistance = -1.0f;
	int32 bShowQuadOverdraw = 0;
	const int32 boolOutputQuadOverdraw = (DebugViewMode == DVSM_QuadComplexity) || (DebugViewMode == DVSM_ShaderComplexityContainedQuadOverhead) ? 1 : 0;
	const int32 LODIndex = FMath::Clamp(VisualizeLODIndex, 0, NumLODColorationColors - 1);

	FMemory::Memzero(OneOverCPUTexCoordScales); // 0 remap to irrelevant data.
	FMemory::Memzero(TexCoordIndices);

	// Gather Data
#if WITH_EDITORONLY_DATA
	if (PrimitiveSceneProxy)
	{
		PrimitiveSceneProxy->GetMaterialTextureScales(VisualizeLODIndex, VisualizeElementIndex, nullptr, OneOverCPUTexCoordScales, TexCoordIndices);
		PrimitiveSceneProxy->GetMeshUVDensities(VisualizeLODIndex, VisualizeElementIndex, WorldUVDensities);

		float Distance = 0;
		if (PrimitiveSceneProxy->GetPrimitiveDistance(VisualizeLODIndex, VisualizeElementIndex, ViewOrigin, Distance))
		{
			// Because the streamer use FMath::FloorToFloat, here we need to use -1 to have a useful result.
			CPULogDistance = FMath::Max<float>(0.f, FMath::Log2(FMath::Max<float>(1.f, Distance)));
		}
	}
#endif

	if (DebugViewMode == DVSM_OutputMaterialTextureScales || DebugViewMode == DVSM_MaterialTextureScaleAccuracy)
	{
		const bool bOutputScales = DebugViewMode == DVSM_OutputMaterialTextureScales;
		const int32 AnalysisIndex = ViewModeParam >= 0 ? FMath::Clamp<int32>(ViewModeParam, 0, TEXSTREAM_MAX_NUM_TEXTURES_PER_MATERIAL - 1) : -1;
		AnalysisParameter = FIntPoint(bOutputScales ? -1 : AnalysisIndex, bOutputScales ? 1 : 0);
	}
	else if (DebugViewMode == DVSM_RequiredTextureResolution)
	{
		int32 AnalysisIndex = INDEX_NONE;
		int32 TextureResolution = 64;
		FMaterialRenderContext MaterialContext(&MaterialRenderProxy, Material, nullptr);
		const FUniformExpressionSet& UniformExpressions = Material.GetUniformExpressions();
		EMaterialTextureParameterType TextureTypes[] = { EMaterialTextureParameterType::Standard2D, EMaterialTextureParameterType::Virtual };
		if (ViewModeParam != INDEX_NONE && ViewModeParamName == NAME_None) // If displaying texture per texture indices
		{
			for (EMaterialTextureParameterType TextureType : TextureTypes)
			{
				for (int32 ParameterIndex = 0; ParameterIndex < UniformExpressions.GetNumTextures(TextureType); ++ParameterIndex)
				{
					const FMaterialTextureParameterInfo& Parameter = UniformExpressions.GetTextureParameter(TextureType, ParameterIndex);
					if (Parameter.TextureIndex == ViewModeParam)
					{
						const UTexture* Texture = nullptr;
						UniformExpressions.GetTextureValue(TextureType, ParameterIndex, MaterialContext, Material, Texture);
						if (Texture && Texture->GetResource())
						{
							AnalysisIndex = ViewModeParam;

							if (Texture->IsStreamable())
							{
								TextureResolution = 1 << FMath::Max((Texture->GetResource()->GetCurrentMipCount() - 1), 0);
							}
							else
							{
								TextureResolution = FMath::Max(Texture->GetResource()->GetSizeX(), Texture->GetResource()->GetSizeY());
							}
						}
					}
				}
			}
		}
		else if (ViewModeParam != INDEX_NONE) // Otherwise show only texture matching the given name
		{
			for (EMaterialTextureParameterType TextureType : TextureTypes)
			{
				for (int32 ParameterIndex = 0; ParameterIndex < UniformExpressions.GetNumTextures(TextureType); ++ParameterIndex)
				{
					const UTexture* Texture = nullptr;
					UniformExpressions.GetTextureValue(TextureType, ParameterIndex, MaterialContext, Material, Texture);
					if (Texture && Texture->GetResource() && Texture->GetFName() == ViewModeParamName)
					{
						const FMaterialTextureParameterInfo& Parameter = UniformExpressions.GetTextureParameter(TextureType, ParameterIndex);
						AnalysisIndex = Parameter.TextureIndex;

						if (Texture->IsStreamable())
						{
							TextureResolution = 1 << FMath::Max((Texture->GetResource()->GetCurrentMipCount() - 1), 0);
						}
						else
						{
							TextureResolution = FMath::Max(Texture->GetResource()->GetSizeX(), Texture->GetResource()->GetSizeY());
						}
					}
				}
			}
		}

		AnalysisParameter = FIntPoint(AnalysisIndex, TextureResolution);
	}

	if (DebugViewMode == DVSM_QuadComplexity)
	{
		NormalizedComplexityValue = FVector4f(NormalizedQuadComplexityValue);
		bShowQuadOverdraw = true;
	}
	else if (DebugViewMode == DVSM_LWCComplexity)
	{
		// Set minimum complexity to 1, to differentiate between 0 cost and missing data
		LWCComplexityPS++;
		LWCComplexityVS++;
		NormalizedComplexityValue = FVector4f(LWCComplexityPS, LWCComplexityVS, 0.f, 0.f) / GMaxLWCComplexity;
		bShowQuadOverdraw = false;
	}
	else if (DebugViewMode == DVSM_ShadowCasters)
	{
		bShowQuadOverdraw = false;
	}
	else
	{
		// normalize the complexity so we can fit it in a low precision scene color which is necessary on some platforms
		// late value is for overdraw which can be problematic with a low precision float format, at some point the precision isn't there any more and it doesn't accumulate
		const float NormalizeMul = 1.0f / GetMaxShaderComplexityCount(Material.GetFeatureLevel());
		NormalizedComplexityValue = FVector4f(NumPSInstructions * NormalizeMul, NumVSInstructions * NormalizeMul, 1 / 32.0f);

		bShowQuadOverdraw = DebugViewMode != DVSM_ShaderComplexity ? 1 : 0;
	}

	// Bind Data
	ShaderBindings.Add(Shader.OneOverCPUTexCoordScalesParameter, OneOverCPUTexCoordScales);
	ShaderBindings.Add(Shader.TexCoordIndicesParameter, TexCoordIndices);
	ShaderBindings.Add(Shader.CPUTexelFactorParameter, FVector4f(WorldUVDensities));
	ShaderBindings.Add(Shader.NormalizedComplexity, NormalizedComplexityValue);
	ShaderBindings.Add(Shader.AnalysisParamsParameter, AnalysisParameter);
	ShaderBindings.Add(Shader.PrimitiveAlphaParameter, PrimitiveAlpha);
	ShaderBindings.Add(Shader.PrimitiveBlendModeParameter, (int)Material.GetBlendMode());
	ShaderBindings.Add(Shader.TexCoordAnalysisIndexParameter, TexCoordAnalysisIndex);
	ShaderBindings.Add(Shader.CPULogDistanceParameter, CPULogDistance);
	ShaderBindings.Add(Shader.ShowQuadOverdraw, bShowQuadOverdraw);
	ShaderBindings.Add(Shader.LODIndexParameter, LODIndex);
	ShaderBindings.Add(Shader.SkinCacheDebugColorParameter, FVector3f(SkinCacheDebugColor.R / 255.f, SkinCacheDebugColor.G / 255.f, SkinCacheDebugColor.B / 255.f));
	ShaderBindings.Add(Shader.OutputQuadOverdrawParameter, boolOutputQuadOverdraw);
	ShaderBindings.Add(Shader.VisualizeModeParameter, DebugViewMode);
}

FMeshPassProcessor* CreateDebugViewModePassProcessor(ERHIFeatureLevel::Type InFeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	const ERHIFeatureLevel::Type FeatureLevel = InViewIfDynamicMeshCommand ? InViewIfDynamicMeshCommand->GetFeatureLevel() : InFeatureLevel;
	return new FDebugViewModeMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, false, InDrawListContext);
}

FRegisterPassProcessorCreateFunction RegisterDebugViewModeMobilePass(&CreateDebugViewModePassProcessor, EShadingPath::Mobile, EMeshPass::DebugViewMode, EMeshPassFlags::MainView);
FRegisterPassProcessorCreateFunction RegisterDebugViewModePass(&CreateDebugViewModePassProcessor, EShadingPath::Deferred, EMeshPass::DebugViewMode, EMeshPassFlags::MainView);

void InitDebugViewModeInterface()
{
	FDebugViewModeInterface::SetInterface(new FDebugViewModeImplementation());
}

#else // !WITH_DEBUG_VIEW_MODES

void ClearDebugAux(FRDGBuilder& GraphBuilder, const FViewFamilyInfo& ViewFamily, FRDGTextureRef DebugAuxTexture)
{}

void RenderDebugViewMode(
	FRDGBuilder& GraphBuilder, 
	TArrayView<FViewInfo> Views, 
	FRDGTextureRef DebugAuxTexture, 
	const FRenderTargetBinding& SceneColorRTBinding, 
	const FDepthStencilBinding& SceneDepthBinding)
{}

#endif // WITH_DEBUG_VIEW_MODES
