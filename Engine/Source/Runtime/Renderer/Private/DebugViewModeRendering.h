// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
DebugViewModeRendering.h: Contains definitions for rendering debug viewmodes.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "Engine/TextureStreamingTypes.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "MeshMaterialShaderType.h"
#include "MeshMaterialShader.h"
#include "ShaderBaseClasses.h"
#include "MeshPassProcessor.h"
#include "DebugViewModeInterface.h"
#include "SceneRenderTargetParameters.h"

class FPrimitiveSceneProxy;
struct FMeshBatchElement;
struct FMeshDrawingRenderState;
class FDebugViewModeInterface;
class FViewFamilyInfo;

static const int32 NumStreamingAccuracyColors = 5;
static const int32 NumLODColorationColors = 8;
static const float UndefinedStreamingAccuracyIntensity = .015f;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FDebugViewModeUniformParameters, )
	SHADER_PARAMETER_ARRAY(FLinearColor, AccuracyColors, [NumStreamingAccuracyColors])
	SHADER_PARAMETER_ARRAY(FLinearColor, LODColors, [NumLODColorationColors])
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FDebugViewModePassUniformParameters, )
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT(FDebugViewModeUniformParameters, DebugViewMode)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, QuadOverdraw)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

#if WITH_DEBUG_VIEW_MODES

void SetupDebugViewModePassUniformBufferConstants(const FViewInfo& ViewInfo, FDebugViewModeUniformParameters& Parameters);

#if !UE_BUILD_SHIPPING
/** Compute the worst-case texture streaming deficit across all textures in a material (uncached). */
int32 ComputeTextureStreamingDeficitForMaterial(const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material, bool bUseWantedMips);
#endif
TRDGUniformBufferRef<FDebugViewModePassUniformParameters> CreateDebugViewModePassUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef DebugAuxTexture);

/** Returns the RT index where the QuadOverdrawUAV will be bound. */
extern int32 GetQuadOverdrawUAVIndex(EShaderPlatform Platform);

class FDebugViewModeShaderElementData : public FMeshMaterialShaderElementData
{
public:

	FDebugViewModeShaderElementData(
		const FMaterialRenderProxy& InMaterialRenderProxy,
		const FMaterial& InMaterial,
		EDebugViewShaderMode InDebugViewMode, 
		const FVector& InViewOrigin, 
		int32 InVisualizeLODIndex, 
		const FColor& InSkinCacheDebugColor,
		int32 InViewModeParam, 
		const FName& InViewModeParamName) 
		: MaterialRenderProxy(InMaterialRenderProxy)
		, Material(InMaterial)
		, DebugViewMode(InDebugViewMode)
		, ViewOrigin(InViewOrigin)
		, VisualizeLODIndex(InVisualizeLODIndex)
		, SkinCacheDebugColor(InSkinCacheDebugColor)
		, ViewModeParam(InViewModeParam)
		, ViewModeParamName(InViewModeParamName)
		, NumVSInstructions(0)
		, NumPSInstructions(0)
		, NumCSInstructions(0)
		, LWCComplexityVS(0)
		, LWCComplexityPS(0)
		, LWCComplexityCS(0)
	{}

	const FMaterialRenderProxy& MaterialRenderProxy;
	const FMaterial& Material;

	EDebugViewShaderMode DebugViewMode;
	FVector ViewOrigin;
	int32 VisualizeLODIndex;
	FColor SkinCacheDebugColor;
	int32 ViewModeParam;
	FName ViewModeParamName;

	int32 NumVSInstructions;
	int32 NumPSInstructions;
	int32 NumCSInstructions;

	uint32 LWCComplexityVS;
	uint32 LWCComplexityPS;
	uint32 LWCComplexityCS;
};

/**
 * Vertex shader for quad overdraw. Required because overdraw shaders need to have SV_Position as first PS interpolant.
 */
class FDebugViewModeVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FDebugViewModeVS,MeshMaterial);
protected:

	FDebugViewModeVS();
	FDebugViewModeVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer);

public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FDebugViewModeShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const;

	static void SetCommonDefinitions(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

class FDebugViewModePS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FDebugViewModePS, MeshMaterial);

	FDebugViewModePS();
	FDebugViewModePS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer);

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	void GetElementShaderBindings(
		const FShaderMapPointerTable& PointerTable,
		const FScene* Scene,
		const FSceneView* ViewIfDynamicMeshCommand,
		const FVertexFactory* VertexFactory,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMeshBatch& MeshBatch,
		const FMeshBatchElement& BatchElement,
		const FDebugViewModeShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const;

	LAYOUT_FIELD(FShaderParameter, OneOverCPUTexCoordScalesParameter);
	LAYOUT_FIELD(FShaderParameter, TexCoordIndicesParameter);
	LAYOUT_FIELD(FShaderParameter, CPUTexelFactorParameter);
	LAYOUT_FIELD(FShaderParameter, NormalizedComplexity);
	LAYOUT_FIELD(FShaderParameter, AnalysisParamsParameter);
	LAYOUT_FIELD(FShaderParameter, PrimitiveAlphaParameter);
	LAYOUT_FIELD(FShaderParameter, PrimitiveBlendModeParameter);
	LAYOUT_FIELD(FShaderParameter, TexCoordAnalysisIndexParameter);
	LAYOUT_FIELD(FShaderParameter, CPULogDistanceParameter);
	LAYOUT_FIELD(FShaderParameter, ShowQuadOverdraw);
	LAYOUT_FIELD(FShaderParameter, LODIndexParameter);
	LAYOUT_FIELD(FShaderParameter, SkinCacheDebugColorParameter);
	LAYOUT_FIELD(FShaderParameter, OutputQuadOverdrawParameter);
	LAYOUT_FIELD(FShaderParameter, VisualizeModeParameter);
	LAYOUT_FIELD(FShaderResourceParameter, QuadBufferUAV);
};

class FDebugViewModeMeshProcessor : public FSceneRenderingAllocatorObject<FDebugViewModeMeshProcessor>, public FMeshPassProcessor
{
public:
	FDebugViewModeMeshProcessor(const FScene* InScene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, bool bTranslucentBasePass, FMeshPassDrawListContext* InDrawListContext);
	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;

private:

	void UpdateInstructionCount(FDebugViewModeShaderElementData& OutShaderElementData, const FMaterial* InBatchMaterial, FVertexFactoryType* InVertexFactoryType);

	EDebugViewShaderMode DebugViewMode;
	int32 ViewModeParam;
	FName ViewModeParamName;

	const FDebugViewModeInterface* DebugViewModeInterface;

	/** Per-proxy cache for texture streaming deficit. Keyed on proxy (not FMaterial*) because
	    material instances sharing the same compiled material can have different texture overrides. */
	TMap<const FMaterialRenderProxy*, int32> TextureStreamingDeficitCache;
};

class FDebugViewModeImplementation : public FDebugViewModeInterface
{
public:
	FDebugViewModeImplementation() : FDebugViewModeInterface() {}

	virtual void AddShaderTypes(ERHIFeatureLevel::Type InFeatureLevel,
		const FVertexFactoryType* InVertexFactoryType,
		FMaterialShaderTypes& OutShaderTypes) const override;

	virtual void GetDebugViewModeShaderBindings(
		const FDebugViewModePS& BaseShader,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT Material,
		EDebugViewShaderMode DebugViewMode,
		const FVector& ViewOrigin,
		int32 VisualizeLODIndex,
		const FColor& SkinCacheDebugColor,
		int32 VisualizeElementIndex,
		int32 NumVSInstructions,
		int32 NumPSInstructions,
		int32 LWCComplexityVS,
		int32 LWCComplexityPS,
		int32 ViewModeParam,
		FName ViewModeParamName,
		FMeshDrawSingleShaderBindings& ShaderBindings
	) const override;
};

#endif // WITH_DEBUG_VIEW_MODES

void ClearDebugAux(FRDGBuilder& GraphBuilder, const FViewFamilyInfo& ViewFamily, FRDGTextureRef DebugAuxTexture);

void RenderDebugViewMode(
	FRDGBuilder& GraphBuilder, 
	TArrayView<FViewInfo> Views,
	FRDGTextureRef DebugAuxTexture,
	const FRenderTargetBinding& SceneColorRTBinding, 
	const FDepthStencilBinding& SceneDepthBinding);
