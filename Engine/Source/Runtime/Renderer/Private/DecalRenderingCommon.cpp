// Copyright Epic Games, Inc. All Rights Reserved.

#include "DecalRenderingCommon.h"
#include "RHIStaticStates.h"
#include "RenderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Substrate/Substrate.h"
#include "MaterialShared.h"
#include "StaticMeshBatch.h"

static_assert(DecalRendering::DecalRenderTargetMode_NumBits == (int32)EDecalRenderTargetMode::Num);

namespace DecalRendering
{
	inline bool IsOpaqueBlendMode(const FDecalBlendDesc& In)			{ return IsOpaqueBlendMode((EBlendMode)In.BlendMode); }
	inline bool IsOpaqueOrMaskedBlendMode(const FDecalBlendDesc& In)	{ return IsOpaqueOrMaskedBlendMode((EBlendMode)In.BlendMode); }
	inline bool IsMaskedBlendMode(const FDecalBlendDesc& In)			{ return IsMaskedBlendMode((EBlendMode)In.BlendMode); }
	inline bool IsTranslucentOnlyBlendMode(const FDecalBlendDesc& In)	{ return IsTranslucentOnlyBlendMode((EBlendMode)In.BlendMode); }
	inline bool IsTranslucentBlendMode(const FDecalBlendDesc& In)		{ return IsTranslucentBlendMode((EBlendMode)In.BlendMode); }
	inline bool IsAlphaHoldoutBlendMode(const FDecalBlendDesc& In)		{ return IsAlphaHoldoutBlendMode((EBlendMode)In.BlendMode); }
	inline bool IsModulateBlendMode(const FDecalBlendDesc& In)			{ return IsModulateBlendMode((EBlendMode)In.BlendMode); }
	inline bool IsAlphaCompositeBlendMode(const FDecalBlendDesc& In)	{ return IsAlphaCompositeBlendMode((EBlendMode)In.BlendMode); }
	
	/** Used at cook and runtime time to determine whether decals support normal G-Buffer sampling (requires Deferred MultiPass mode). */
	static bool MobileSupportDecalNormalSampling(const FStaticShaderPlatform Platform)
	{
		return IsMobileDeferredShadingEnabled(Platform) && !MobileAllowFramebufferFetch(Platform);
	}

	/** Finalize the initialization of FDecalBlendDesc after BlendMode and bWrite flags have all been set. */
	void FinalizeBlendDesc(EShaderPlatform Platform, FDecalBlendDesc& Desc)
	{
		const bool bIsSubstrateEnabled = Substrate::IsSubstrateEnabled();
		const bool bIsMobilePlatform = IsMobilePlatform(Platform);
		const bool bIsMobileDeferredPlatform = bIsMobilePlatform && IsMobileDeferredShadingEnabled(Platform);
		const bool bIsDBufferPlatform = IsUsingDBuffers(Platform);
		const bool bIsDBufferMaskPlatform = bIsDBufferPlatform && FDataDrivenShaderPlatformInfo::GetSupportsPerPixelDBufferMask(Platform);
		const bool bIsMobileNormalBlendable = bIsDBufferPlatform || (bIsMobileDeferredPlatform && MobileUsesExtendedGBuffer(Platform) && !IsStaticLightingAllowed());

		Desc.bWriteDBufferMask = bIsDBufferMaskPlatform;

		// Enforce platform blend mode limitations.
		if (!IsTranslucentOnlyBlendMode(Desc) && !IsAlphaCompositeBlendMode(Desc) && !IsModulateBlendMode(Desc))
		{
			Desc.BlendMode = BLEND_Translucent;
		}
		if (bIsDBufferPlatform && IsModulateBlendMode(Desc))
		{
			Desc.BlendMode = BLEND_Translucent;
		}

		// Enforce platform output limitations.
		if (bIsMobilePlatform && !bIsMobileDeferredPlatform && !bIsDBufferPlatform)
		{
			Desc.bWriteRoughnessSpecularMetallic = false;
		}
		if (bIsMobilePlatform && !bIsMobileNormalBlendable)
		{
			// Note that we disable writing normals for mobile AND mobile deferred.
			// Mobile deferred has octahedral encoding for normals and so doesn't support decal normal blending.
			Desc.bWriteNormal = false;
		}
		if (bIsMobilePlatform)
		{
			Desc.bWriteAmbientOcclusion = false;
		}

		// Enforce blend modes output limitations.
		if (IsAlphaCompositeBlendMode(Desc))
		{
			Desc.bWriteNormal = false;
		}

		// Calculate main decal render stage. We set only one (or none) of these for any decal.
		if ((bIsMobileDeferredPlatform && !bIsDBufferPlatform) && (Desc.bWriteEmissive || Desc.bWriteBaseColor || Desc.bWriteNormal || Desc.bWriteRoughnessSpecularMetallic))
		{
			Desc.RenderStageMask |= 1 << (uint32)EDecalRenderStage::MobileBeforeLighting;
		}
		else if ((bIsMobilePlatform && !bIsDBufferPlatform) && (Desc.bWriteEmissive || Desc.bWriteBaseColor))
		{
			Desc.RenderStageMask |= 1 << (uint32)EDecalRenderStage::Mobile;
		}
		else if (bIsDBufferPlatform && (Desc.bWriteBaseColor || Desc.bWriteNormal || Desc.bWriteRoughnessSpecularMetallic))
		{
			Desc.RenderStageMask |= 1 << (uint32)EDecalRenderStage::BeforeBasePass;
		}
		else if (Desc.bWriteEmissive || Desc.bWriteBaseColor || Desc.bWriteNormal || Desc.bWriteRoughnessSpecularMetallic)
		{
			Desc.RenderStageMask |= 1 << (uint32)EDecalRenderStage::BeforeLighting;
		}

		// Calculate additional decal render stages.
		if (Desc.bWriteEmissive && bIsDBufferPlatform)
		{
			Desc.RenderStageMask |= 1 << (uint32)EDecalRenderStage::Emissive;
		}
		if (Desc.bWriteAmbientOcclusion)
		{
			Desc.RenderStageMask |= 1 << (uint32)EDecalRenderStage::AmbientOcclusion;
		}
	}

	FDecalBlendDesc ComputeDecalBlendDesc(EShaderPlatform Platform, const FMaterial& Material)
	{
		FDecalBlendDesc Desc;
		if (Substrate::IsSubstrateEnabled() && Material.IsSubstrateMaterial())
		{
			const bool bUseDiffuseAlbedoAndF0 =
				Material.HasMaterialPropertyConnected(EMaterialProperty::MP_DiffuseColor) ||	// This is used for Substrate Slab using (DiffuseAlbedo | F0) parameterization
				Material.HasMaterialPropertyConnected(EMaterialProperty::MP_SpecularColor);	// This is used for Substrate Slab using (DiffuseAlbedo | F0) parameterization

			Desc.BlendMode = Material.GetBlendMode();
			Desc.bWriteBaseColor = Material.HasMaterialPropertyConnected(EMaterialProperty::MP_BaseColor) || bUseDiffuseAlbedoAndF0;
			Desc.bWriteNormal = Material.HasMaterialPropertyConnected(EMaterialProperty::MP_Normal);
			Desc.bWriteRoughnessSpecularMetallic =
				bUseDiffuseAlbedoAndF0 ||
				Material.HasMaterialPropertyConnected(EMaterialProperty::MP_Metallic) ||
				Material.HasMaterialPropertyConnected(EMaterialProperty::MP_Specular) ||
				Material.HasMaterialPropertyConnected(EMaterialProperty::MP_Roughness);
			Desc.bWriteEmissive=
				Material.HasMaterialPropertyConnected(EMaterialProperty::MP_EmissiveColor);
			Desc.bWriteAmbientOcclusion =
				Material.HasMaterialPropertyConnected(EMaterialProperty::MP_AmbientOcclusion);
		}
		else
		{
			Desc.BlendMode = Material.GetBlendMode();
			Desc.bWriteBaseColor = Material.HasBaseColorConnected();
			Desc.bWriteNormal = Material.HasNormalConnected();
			Desc.bWriteRoughnessSpecularMetallic = Material.HasRoughnessConnected() || Material.HasSpecularConnected() || Material.HasMetallicConnected();
			Desc.bWriteEmissive = Material.HasEmissiveColorConnected();
			Desc.bWriteAmbientOcclusion = Material.HasAmbientOcclusionConnected();
		}
		FinalizeBlendDesc(Platform, Desc);
		return Desc;
	}

	FDecalBlendDesc ComputeDecalBlendDesc(EShaderPlatform Platform, FMaterialShaderParameters const& MaterialShaderParameters)
	{
		FDecalBlendDesc Desc;
		if (Substrate::IsSubstrateEnabled() && MaterialShaderParameters.bIsSubstrateMaterial)
		{
			const bool bUseDiffuseAlbedoAndF0 = 
				MaterialShaderParameters.bHasDiffuseAlbedoConnected || 
				MaterialShaderParameters.bHasF0Connected;
			
			Desc.BlendMode = MaterialShaderParameters.BlendMode;
			Desc.bWriteBaseColor = MaterialShaderParameters.bHasBaseColorConnected || bUseDiffuseAlbedoAndF0;
			Desc.bWriteNormal = MaterialShaderParameters.bHasNormalConnected;
			Desc.bWriteRoughnessSpecularMetallic = 
				bUseDiffuseAlbedoAndF0 ||
				MaterialShaderParameters.bHasRoughnessConnected || 
				MaterialShaderParameters.bHasSpecularConnected || 
				MaterialShaderParameters.bHasMetallicConnected;
			Desc.bWriteEmissive = MaterialShaderParameters.bHasEmissiveColorConnected;
			Desc.bWriteAmbientOcclusion = MaterialShaderParameters.bHasAmbientOcclusionConnected;
		}
		else
		{
			Desc.BlendMode = MaterialShaderParameters.BlendMode;
			Desc.bWriteBaseColor = MaterialShaderParameters.bHasBaseColorConnected;
			Desc.bWriteNormal = MaterialShaderParameters.bHasNormalConnected;
			Desc.bWriteRoughnessSpecularMetallic = MaterialShaderParameters.bHasRoughnessConnected || MaterialShaderParameters.bHasSpecularConnected || MaterialShaderParameters.bHasMetallicConnected;
			Desc.bWriteEmissive = MaterialShaderParameters.bHasEmissiveColorConnected;
			Desc.bWriteAmbientOcclusion = MaterialShaderParameters.bHasAmbientOcclusionConnected;
		}
		FinalizeBlendDesc(Platform, Desc);
		return Desc;
	}

	bool IsCompatibleWithRenderStage(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage)
	{
		return (DecalBlendDesc.RenderStageMask & (1 << (uint32)DecalRenderStage)) != 0;
	}

	EDecalRenderStage GetBaseRenderStage(FDecalBlendDesc DecalBlendDesc)
	{
		if (DecalBlendDesc.RenderStageMask & (1 << (uint32)EDecalRenderStage::BeforeBasePass))
		{
			return EDecalRenderStage::BeforeBasePass;
		}
		if (DecalBlendDesc.RenderStageMask & (1 << (uint32)EDecalRenderStage::BeforeLighting))
		{
			return EDecalRenderStage::BeforeLighting;
		}
		if (DecalBlendDesc.RenderStageMask & (1 << (uint32)EDecalRenderStage::Mobile))
		{
			return EDecalRenderStage::Mobile;
		}
		if (DecalBlendDesc.RenderStageMask & (1 << (uint32)EDecalRenderStage::MobileBeforeLighting))
		{
			return EDecalRenderStage::MobileBeforeLighting;
		}

		return EDecalRenderStage::None;
	}

	uint8 GetDecalRenderTargetModeMask(const FMaterial& Material, ERHIFeatureLevel::Type FeatureLevel)
	{
		uint8 Mask = 0;

		const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
		const FDecalBlendDesc DecalBlendDesc = ComputeDecalBlendDesc(ShaderPlatform, Material);

		for (int32 DecalRenderStageNum = 0; DecalRenderStageNum < (int32)EDecalRenderStage::Num; ++DecalRenderStageNum)
		{
			EDecalRenderStage DecalRenderStage = (EDecalRenderStage)DecalRenderStageNum;
			if (IsCompatibleWithRenderStage(DecalBlendDesc, DecalRenderStage))
			{
				EDecalRenderTargetMode DecalRenderTargetMode = GetRenderTargetMode(ShaderPlatform, DecalBlendDesc, DecalRenderStage);
				Mask |= (1 << (uint32)DecalRenderTargetMode);
			}
		}

		return Mask;
	}

	bool IsCompatibleWithRenderTargetMode(uint8 DecalRenderTargetModeMask, EDecalRenderTargetMode DecalRenderTargetMode)
	{
		return (DecalRenderTargetModeMask & (1 << (uint32)DecalRenderTargetMode)) != 0;
	}

	EMeshPass::Type GetMeshPassType(EDecalRenderTargetMode RenderTargetMode)
	{
		switch (RenderTargetMode)
		{
		case EDecalRenderTargetMode::None:
			return EMeshPass::Num;
		case EDecalRenderTargetMode::DBuffer:
			return EMeshPass::MeshDecal_DBuffer;
		case EDecalRenderTargetMode::SceneColorAndGBuffer:
			return EMeshPass::MeshDecal_SceneColorAndGBuffer;
		case EDecalRenderTargetMode::SceneColorAndGBufferNoNormal:
			return EMeshPass::MeshDecal_SceneColorAndGBufferNoNormal;
		case EDecalRenderTargetMode::SceneColor:
			return EMeshPass::MeshDecal_SceneColor;
		case EDecalRenderTargetMode::AmbientOcclusion:
			return EMeshPass::MeshDecal_AmbientOcclusion;
		}

		return EMeshPass::Num;
	}

	EDecalRenderStage GetRenderStage(EDecalRenderTargetMode RenderTargetMode, EShadingPath ShadingPath)
	{
		switch (RenderTargetMode)
		{
			case EDecalRenderTargetMode::None:
				return EDecalRenderStage::None;
			case EDecalRenderTargetMode::DBuffer:
				return EDecalRenderStage::BeforeBasePass;
			case EDecalRenderTargetMode::SceneColorAndGBuffer:
			case EDecalRenderTargetMode::SceneColorAndGBufferNoNormal:
				return (ShadingPath == EShadingPath::Deferred) ? EDecalRenderStage::BeforeLighting : EDecalRenderStage::MobileBeforeLighting;
			case EDecalRenderTargetMode::SceneColor:
				return (ShadingPath == EShadingPath::Deferred) ? EDecalRenderStage::Emissive : EDecalRenderStage::Mobile;
			case EDecalRenderTargetMode::AmbientOcclusion:
				return EDecalRenderStage::AmbientOcclusion;
		}

		return EDecalRenderStage::None;
	}

	EDecalRenderTargetMode GetRenderTargetMode(EShaderPlatform Platform, FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage)
	{
		switch (DecalRenderStage)
		{
		case EDecalRenderStage::BeforeBasePass:
			return EDecalRenderTargetMode::DBuffer;
		case EDecalRenderStage::BeforeLighting:
			return DecalBlendDesc.bWriteNormal ? EDecalRenderTargetMode::SceneColorAndGBuffer : EDecalRenderTargetMode::SceneColorAndGBufferNoNormal;
		case EDecalRenderStage::Mobile:
			return EDecalRenderTargetMode::SceneColor;
		case EDecalRenderStage::MobileBeforeLighting:
			return (MobileSupportDecalNormalSampling(Platform) && !DecalBlendDesc.bWriteNormal) ? EDecalRenderTargetMode::SceneColorAndGBufferNoNormal : EDecalRenderTargetMode::SceneColorAndGBuffer;
		case EDecalRenderStage::Emissive:
			return EDecalRenderTargetMode::SceneColor;
		case EDecalRenderStage::AmbientOcclusion:
			return EDecalRenderTargetMode::AmbientOcclusion;
		}

		return EDecalRenderTargetMode::None;
	}

	const TCHAR* GetRenderTargetModeName(EDecalRenderTargetMode DecalRenderTargetMode)
	{
		switch (DecalRenderTargetMode)
		{
		case EDecalRenderTargetMode::DBuffer:
			return TEXT("DBuffer");
		case EDecalRenderTargetMode::SceneColorAndGBuffer:
			return TEXT("SceneColorAndGBuffer");
		case EDecalRenderTargetMode::SceneColorAndGBufferNoNormal:
			return TEXT("SceneColorAndGBufferNoNormal");
		case EDecalRenderTargetMode::SceneColor:
			return TEXT("SceneColor");
		case EDecalRenderTargetMode::AmbientOcclusion:
			return TEXT("AmbientOcclusion");
		default:
			break;
		}

		return TEXT("None");
	}

	uint32 GetRenderTargetCount(FDecalBlendDesc DecalBlendDesc, EDecalRenderTargetMode RenderTargetMode)
	{
		switch (RenderTargetMode)
		{
		case EDecalRenderTargetMode::DBuffer:
			return DecalBlendDesc.bWriteDBufferMask ? 4 : 3;
		case EDecalRenderTargetMode::SceneColorAndGBuffer:
			return 4;
		case EDecalRenderTargetMode::SceneColorAndGBufferNoNormal:
			return 3;
		case EDecalRenderTargetMode::SceneColor:
			return 1;
		case EDecalRenderTargetMode::AmbientOcclusion:
			return 1;
		}

		return 0;
	}

	uint32 GetRenderTargetWriteMask(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage, EDecalRenderTargetMode RenderTargetMode)
	{
		if (RenderTargetMode == EDecalRenderTargetMode::DBuffer)
		{
			return (DecalBlendDesc.bWriteBaseColor ? 1 : 0) + (DecalBlendDesc.bWriteNormal ? 2 : 0) + (DecalBlendDesc.bWriteRoughnessSpecularMetallic ? 4 : 0) + (DecalBlendDesc.bWriteDBufferMask ? 8 : 0);
		}
		else if (RenderTargetMode == EDecalRenderTargetMode::SceneColorAndGBuffer)
		{
			return (DecalBlendDesc.bWriteEmissive ? 1 : 0) + (DecalBlendDesc.bWriteNormal ? 2 : 0) + (DecalBlendDesc.bWriteRoughnessSpecularMetallic ? 4 : 0) + (DecalBlendDesc.bWriteBaseColor ? 8 : 0);
		}
		else if (RenderTargetMode == EDecalRenderTargetMode::SceneColorAndGBufferNoNormal)
		{
			return (DecalBlendDesc.bWriteEmissive ? 1 : 0) + (DecalBlendDesc.bWriteRoughnessSpecularMetallic ? 2 : 0) + (DecalBlendDesc.bWriteBaseColor ? 4 : 0);
		}
		else if (RenderTargetMode == EDecalRenderTargetMode::SceneColor)
		{
			if (DecalRenderStage == EDecalRenderStage::Mobile)
			{
				return ((DecalBlendDesc.bWriteEmissive || DecalBlendDesc.bWriteBaseColor) ? 1 : 0);
			}
			return (DecalBlendDesc.bWriteEmissive ? 1 : 0);
		}
		else if (RenderTargetMode == EDecalRenderTargetMode::AmbientOcclusion)
		{
			return (DecalBlendDesc.bWriteAmbientOcclusion ? 1 : 0);
		}

		// Enable all render targets by default.
		return (1 << GetRenderTargetCount(DecalBlendDesc, RenderTargetMode)) - 1;
	}

	FRHIBlendState* GetDecalBlendState_DBuffer(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage)
	{
		// Ignore DBuffer mask bit and always set that MRT active.
		const uint32 Mask = GetRenderTargetWriteMask(DecalBlendDesc, DecalRenderStage, EDecalRenderTargetMode::DBuffer) & 0x7;

		if (IsTranslucentOnlyBlendMode(DecalBlendDesc))
		{
			switch (Mask)
			{
			case 1:
				return TStaticBlendState<
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha, // BaseColor
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			case 2:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// BaseColor
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha, // Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			case 3:
				return TStaticBlendState<
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha, // BaseColor
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,	// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			case 4:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// BaseColor
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Normal
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,	// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			case 5:
				return TStaticBlendState<
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha, // BaseColor
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Normal
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,	// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			case 6:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// BaseColor
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,	// Normal
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,	// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			case 7:
				return TStaticBlendState<
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,	// BaseColor
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,	// Normal
					CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,	// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			}
		}
		else if (IsAlphaCompositeBlendMode(DecalBlendDesc))
		{
			ensure((Mask & 2) == 0); // AlphaComposite shouldn't write normal.

			switch (Mask)
			{
			case 1:
				return TStaticBlendState<
					CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,			// BaseColor
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			case 4:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// BaseColor
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Normal
					CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,			// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			case 5:
				return TStaticBlendState<
					CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,			// BaseColor
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,										// Normal
					CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,			// Metallic, Specular, Roughness
					CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One											// DBuffer mask
				>::GetRHI();
			}
		}

		ensure(0);
		return TStaticBlendState<>::GetRHI();
	}

	FRHIBlendState* GetDecalBlendState_SceneColorAndGBuffer(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage)
	{
		const uint32 Mask = GetRenderTargetWriteMask(DecalBlendDesc, DecalRenderStage, EDecalRenderTargetMode::SceneColorAndGBuffer);

		if (IsTranslucentOnlyBlendMode(DecalBlendDesc))
		{
			switch (Mask)
			{
			case 1:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 2:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 3:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 4:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 5:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 6:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 7:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 8:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 9:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 10:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 11:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 12:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 13:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 14:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 15:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			}
		}
		else if (IsAlphaCompositeBlendMode(DecalBlendDesc))
		{
			ensure((Mask & 2) == 0); // AlphaComposite shouldn't write normal.

			switch (Mask)
			{
			case 1:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 4:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,			// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 5:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,			// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 8:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One			// BaseColor
				>::GetRHI();
			case 9:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One			// BaseColor
				>::GetRHI();
			case 12:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,			// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One			// BaseColor
				>::GetRHI();
			case 13:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,			// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One			// BaseColor
				>::GetRHI();
			}
		}
		else if (IsModulateBlendMode(DecalBlendDesc))
		{
			switch (Mask)
			{
			case 1:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 2:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 3:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 4:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 5:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 6:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 7:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 8:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 9:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 10:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 11:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 12:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 13:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 14:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 15:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Normal
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			}
		}

		ensure(0);
		return TStaticBlendState<>::GetRHI();
	}

	FRHIBlendState* GetDecalBlendState_SceneColorAndGBufferNoNormal(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage)
	{
		const uint32 Mask = GetRenderTargetWriteMask(DecalBlendDesc, DecalRenderStage, EDecalRenderTargetMode::SceneColorAndGBufferNoNormal);

		if (IsTranslucentOnlyBlendMode(DecalBlendDesc))
		{
			switch (Mask)
			{
			case 1:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 2:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 3:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 4:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 5:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 6:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 7:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			}
		}
		else if (IsAlphaCompositeBlendMode(DecalBlendDesc))
		{
			switch (Mask)
			{
			case 1:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 2:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,			// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 3:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,			// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 4:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One			// BaseColor
				>::GetRHI();
			case 5:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One			// BaseColor
				>::GetRHI();
			case 6:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,			// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One			// BaseColor
				>::GetRHI();
			case 7:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,			// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One			// BaseColor
				>::GetRHI();
			}
		}
		else if (IsModulateBlendMode(DecalBlendDesc))
		{
			switch (Mask)
			{
			case 1:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 2:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 3:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One						// BaseColor
				>::GetRHI();
			case 4:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 5:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 6:
				return TStaticBlendState<
					CW_NONE, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_One,						// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			case 7:
				return TStaticBlendState<
					CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One,				// Emissive
					CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One,	// Metallic, Specular, Roughness
					CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One	// BaseColor
				>::GetRHI();
			}
		}

		ensure(0);
		return TStaticBlendState<>::GetRHI();
	}

	FRHIBlendState* GetDecalBlendState_SceneColor(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage)
	{
		if (DecalRenderStage == EDecalRenderStage::Mobile)
		{
			if (IsTranslucentOnlyBlendMode(DecalBlendDesc))
			{
				if (DecalBlendDesc.bWriteEmissive)
				{
					// Treat blend as emissive
					return TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_One, BO_Add, BF_Zero, BF_One, CW_NONE>::GetRHI();
				}
				else
				{
					// Treat blend as non-emissive
					return TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One, CW_NONE>::GetRHI();
				}
			}
			else if (IsAlphaCompositeBlendMode(DecalBlendDesc))
			{
				return TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One, CW_NONE>::GetRHI();
			}
			else if (IsModulateBlendMode(DecalBlendDesc))
			{
				return TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One, CW_NONE>::GetRHI();
			}
		}
		else
		{
			return TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_One>::GetRHI();
		}

		ensure(0);
		return TStaticBlendState<>::GetRHI();
	}

	FRHIBlendState* GetDecalBlendState_AmbientOcclusion(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage)
	{
		// Modulate with AO target.
		return TStaticBlendState<CW_RED, BO_Add, BF_DestColor, BF_Zero>::GetRHI();
	}

	FRHIBlendState* GetDecalBlendState(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage, EDecalRenderTargetMode RenderTargetMode)
	{
		// As we force the opacity in the shader we don't always _need_ to set different blend states per MRT.
		// But we want to give the driver as much information as possible about where output isn't required.
		// An alternative is to call SetRenderTarget per state change. But that is likely be slower (would need testing on various platforms to confirm that).

		switch(RenderTargetMode)
		{
			case EDecalRenderTargetMode::DBuffer:
				return GetDecalBlendState_DBuffer(DecalBlendDesc, DecalRenderStage);
			case EDecalRenderTargetMode::SceneColorAndGBuffer:
				return GetDecalBlendState_SceneColorAndGBuffer(DecalBlendDesc, DecalRenderStage);
			case EDecalRenderTargetMode::SceneColorAndGBufferNoNormal:
				return GetDecalBlendState_SceneColorAndGBufferNoNormal(DecalBlendDesc, DecalRenderStage);
			case EDecalRenderTargetMode::SceneColor:
				return GetDecalBlendState_SceneColor(DecalBlendDesc, DecalRenderStage);
			case EDecalRenderTargetMode::AmbientOcclusion:
				return GetDecalBlendState_AmbientOcclusion(DecalBlendDesc, DecalRenderStage);
		}

		return TStaticBlendState<>::GetRHI();
	}

	EDecalRasterizerState GetDecalRasterizerState(bool bInsideDecal, bool bIsInverted, bool ViewReverseCulling)
	{
		bool bClockwise = bInsideDecal;

		if (ViewReverseCulling)
		{
			bClockwise = !bClockwise;
		}

		if (bIsInverted)
		{
			bClockwise = !bClockwise;
		}
		
		return bClockwise ? EDecalRasterizerState::CW : EDecalRasterizerState::CCW;
	}

	FRHIRasterizerState* GetDecalRasterizerState(EDecalRasterizerState DecalRasterizerState)
	{
		switch (DecalRasterizerState)
		{
		case EDecalRasterizerState::CW:
			return TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();
		case EDecalRasterizerState::CCW:
			return TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
		}

		check(0); 
		return nullptr;
	}

	void ModifyCompilationEnvironment(EShaderPlatform Platform, FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage, FShaderCompilerEnvironment& OutEnvironment)
	{
		DecalRenderStage = DecalRenderStage != EDecalRenderStage::None ? DecalRenderStage : GetBaseRenderStage(DecalBlendDesc);
		check(DecalRenderStage != EDecalRenderStage::None);

		const EDecalRenderTargetMode RenderTargetMode = GetRenderTargetMode(Platform, DecalBlendDesc, DecalRenderStage);
		check(RenderTargetMode != EDecalRenderTargetMode::None);

		const uint32 RenderTargetCount = GetRenderTargetCount(DecalBlendDesc, RenderTargetMode);
		const uint32 RenderTargetWriteMask = GetRenderTargetWriteMask(DecalBlendDesc, DecalRenderStage, RenderTargetMode);

		OutEnvironment.SetDefine(TEXT("IS_DECAL"), 1);
		OutEnvironment.SetDefine(TEXT("IS_DBUFFER_DECAL"), DecalRenderStage == EDecalRenderStage::BeforeBasePass ? 1 : 0);

		OutEnvironment.SetDefine(TEXT("DECAL_RENDERSTAGE"), (uint32)DecalRenderStage);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERTARGETMODE"), (uint32)RenderTargetMode);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERTARGET_COUNT"), RenderTargetCount);

		OutEnvironment.SetDefineAndCompileArgument(TEXT("DECAL_OUT_MRT0"), (RenderTargetWriteMask & 1) != 0 ? true : false);
		OutEnvironment.SetDefineAndCompileArgument(TEXT("DECAL_OUT_MRT1"), (RenderTargetWriteMask & 2) != 0 ? true : false);
		OutEnvironment.SetDefineAndCompileArgument(TEXT("DECAL_OUT_MRT2"), (RenderTargetWriteMask & 4) != 0 ? true : false);
		OutEnvironment.SetDefineAndCompileArgument(TEXT("DECAL_OUT_MRT3"), (RenderTargetWriteMask & 8) != 0 ? true : false);

		OutEnvironment.SetDefine(TEXT("DECAL_RENDERSTAGE_BEFOREBASEPASS"), (uint32)EDecalRenderStage::BeforeBasePass);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERSTAGE_BEFORELIGHTING"), (uint32)EDecalRenderStage::BeforeLighting);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERSTAGE_MOBILE"), (uint32)EDecalRenderStage::Mobile);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERSTAGE_MOBILEBEFORELIGHTING"), (uint32)EDecalRenderStage::MobileBeforeLighting);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERSTAGE_EMISSIVE"), (uint32)EDecalRenderStage::Emissive);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERSTAGE_AO"), (uint32)EDecalRenderStage::AmbientOcclusion);

		OutEnvironment.SetDefine(TEXT("DECAL_RENDERTARGETMODE_DBUFFER"), (uint32)EDecalRenderTargetMode::DBuffer);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERTARGETMODE_GBUFFER"), (uint32)EDecalRenderTargetMode::SceneColorAndGBuffer);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERTARGETMODE_GBUFFER_NONORMAL"), (uint32)EDecalRenderTargetMode::SceneColorAndGBufferNoNormal);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERTARGETMODE_SCENECOLOR"), (uint32)EDecalRenderTargetMode::SceneColor);
		OutEnvironment.SetDefine(TEXT("DECAL_RENDERTARGETMODE_AO"), (uint32)EDecalRenderTargetMode::AmbientOcclusion);

		// Decals needs to both read Substrate data (deferred path) and write (inline path)
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_DEFERRED_SHADING"), 1);

		if (IsMobilePlatform(Platform))
		{
			// On mobile decals are rendered in a "depth read" sub-pass
			const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Platform);
			OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEPTHREAD_SUBPASS"), bMobileForceDepthRead ? 0u : 1u);
		}
	}
}
