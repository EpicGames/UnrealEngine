// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/Material.h"
#include "RenderUtils.h"
#include "MeshPassProcessor.h"

class FMaterial;
class FRHIBlendState;
class FRHIRasterizerState;
struct FMaterialShaderParameters;
struct FShaderCompilerEnvironment;

/** Packed decal description which contains all the information to define render passes and shader compilation setup. */
union FDecalBlendDesc
{
	uint32 Packed = 0;

	struct
	{
		uint32 BlendMode : 8;
		uint32 RenderStageMask : 8;
		uint32 bWriteBaseColor : 1;
		uint32 bWriteNormal : 1;
		uint32 bWriteRoughnessSpecularMetallic : 1;
		uint32 bWriteEmissive : 1;
		uint32 bWriteAmbientOcclusion : 1;
		uint32 bWriteDBufferMask : 1;
	};
};

/** Enumeration of the points in the frame for decal rendering. */
enum class EDecalRenderStage : uint8
{
	None = 0,

	// DBuffer decal pass.
	BeforeBasePass = 1,
	// GBuffer decal pass.
	BeforeLighting = 2,

	// Mobile decal pass with limited functionality.
	Mobile = 3,
	// Mobile decal pass for mobile deferred platforms.
	MobileBeforeLighting = 4,

	// Emissive decal pass.
	// DBuffer decals with an emissive component will use this pass.
	Emissive = 5,
	// Ambient occlusion decal pass.
	// A decal can write regular attributes in another pass and then AO in this pass.
	AmbientOcclusion = 6,

	Num,
};

/** Enumeration of the render target layouts for decal rendering. */
enum class EDecalRenderTargetMode : uint8
{
	None = 0,

	DBuffer = 1,
	SceneColorAndGBuffer = 2,
	// GBuffer with no normal is necessary for decals sampling the normal from the GBuffer.
	SceneColorAndGBufferNoNormal = 3,
	SceneColor = 4,
	AmbientOcclusion = 5,

	Num,
};

/** Enumeration of decal rasterization states. */
enum class EDecalRasterizerState : uint8
{
	Undefined,
	CCW,
	CW,
};

/**
 * Shared decal functionality for render pass and shader setup.
 */
namespace DecalRendering
{
	/** Build the packed decal description from a decal material. */
	FDecalBlendDesc ComputeDecalBlendDesc(EShaderPlatform Platform, const FMaterial& Material);

	/** Build the packed decal description from a decal material. */
	FDecalBlendDesc ComputeDecalBlendDesc(EShaderPlatform Platform, FMaterialShaderParameters const& MaterialShaderParameters);

	/** Returns true if a decal should be rendered in the render stage. */
	bool IsCompatibleWithRenderStage(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage);

	/** Returns the main render stage from render target mode and shading path */
	EDecalRenderStage GetRenderStage(EDecalRenderTargetMode RenderTargetMode, EShadingPath ShadingPath);

	/** Returns the main render stage for a decal (does not include the emissive, and AO stages). Can return EDecalRenderStage::None if there is no valid main render stage. */
	EDecalRenderStage GetBaseRenderStage(FDecalBlendDesc DecalBlendDesc);

	/** Get the render target mode that a decal uses for a given stage at runtime and at cook time. Can return EDecalRenderTargetMode::None if there is no valid render target mode. */
	EDecalRenderTargetMode GetRenderTargetMode(EShaderPlatform Platform, FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage);

	/** Get the render target mode string */
	const TCHAR* GetRenderTargetModeName(EDecalRenderTargetMode DecalRenderTargetMode);


	/** Setup a mask for each render target mode where the material should render to */
	uint8 GetDecalRenderTargetModeMask(const FMaterial& Material, ERHIFeatureLevel::Type FeatureLevel);

	/** Returns true if a decal should be rendered in the render target mode. */
	bool IsCompatibleWithRenderTargetMode(uint8 DecalRenderTargetModeMask, EDecalRenderTargetMode DecalRenderTargetMode);

	/** Get the mesh pass processer type to use for given render target mode */
	EMeshPass::Type GetMeshPassType(EDecalRenderTargetMode RenderTargetMode);

	/** Get render target count for the given render target mode. */
	uint32 GetRenderTargetCount(FDecalBlendDesc DecalBlendDesc, EDecalRenderTargetMode RenderTargetMode);

	/** Get render target write mask as a bitmask for the given stage and render target mode. */
	uint32 GetRenderTargetWriteMask(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage, EDecalRenderTargetMode RenderTargetMode);

	/** Get the blend state for rendering a decal at the given stage and render target mode. */
	FRHIBlendState* GetDecalBlendState(FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage, EDecalRenderTargetMode RenderTargetMode);

	/** Get the EDecalRasterizerState enum from the view setup. */
	EDecalRasterizerState GetDecalRasterizerState(bool bInsideDecal, bool bIsInverted, bool ViewReverseCulling);

	/** Get the rasterizer state object for a EDecalRasterizerState enum. */
	FRHIRasterizerState* GetDecalRasterizerState(EDecalRasterizerState DecalRasterizerState);

	/** Modify the shader compilation environment for a given decal and stage. */
	void ModifyCompilationEnvironment(EShaderPlatform Platform, FDecalBlendDesc DecalBlendDesc, EDecalRenderStage DecalRenderStage, FShaderCompilerEnvironment& OutEnvironment);
};
