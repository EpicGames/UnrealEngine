// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DecalRenderingShared.cpp
=============================================================================*/

#include "DecalRenderingShared.h"
#include "CoreMinimal.h"
#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "Containers/ArrayView.h"
#include "GameTime.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "Math/DoubleFloat.h"
#include "Math/Plane.h"
#include "Math/TransformVectorized.h"
#include "MeshMaterialShader.h"
#include "Misc/LargeWorldCoordinates.h"
#include "MobileBasePassRendering.h"
#include "PipelineStateCache.h"
#include "PrimitiveUniformShaderParameters.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RayTracing/RaytracingOptions.h"
#include "ReadOnlyCVARCache.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderResource.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIShaderParameters.h"
#include "ScenePrivate.h"
#include "SceneProxies/DeferredDecalProxy.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneRendering.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "StereoRenderUtils.h"
#include "Templates/ResolveTypeAmbiguity.h"
#include "UniformBuffer.h"

static TAutoConsoleVariable<float> CVarDecalFadeScreenSizeMultiplier(
	TEXT("r.Decal.FadeScreenSizeMult"),
	1.0f,
	TEXT("Control the per decal fade screen size. Multiplies with the per-decal screen size fade threshold.")
	TEXT("  Smaller means decals fade less aggressively.")
	);

FVisibleDecal::FVisibleDecal(const FDeferredDecalProxy& InDecalProxy, float InConservativeRadius, float InFadeAlpha, EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel)
	: MaterialProxy(InDecalProxy.DecalMaterial->GetRenderProxy())
	, Component((uintptr_t)InDecalProxy.Component)
	, SortOrder(InDecalProxy.SortOrder)
	, ConservativeRadius(InConservativeRadius)
	, FadeAlpha(InFadeAlpha)
	, InvFadeDuration(InDecalProxy.InvFadeDuration)
	, InvFadeInDuration(InDecalProxy.InvFadeInDuration)
	, FadeStartDelayNormalized(InDecalProxy.FadeStartDelayNormalized)
	, FadeInStartDelayNormalized(InDecalProxy.FadeInStartDelayNormalized)
	, DecalColor(InDecalProxy.DecalColor)
	, ComponentTrans(InDecalProxy.ComponentTrans)
	, BoxBounds(InDecalProxy.GetBounds().GetBox())
{
	// Build BlendDesc from a potentially incomplete material.
	// If our shader isn't compiled yet then we will potentially render later with a different fallback material.
	FMaterial const& MaterialResource = MaterialProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
	BlendDesc = DecalRendering::ComputeDecalBlendDesc(ShaderPlatform, MaterialResource);
}

FDecalVisibilityTaskData* FDecalVisibilityTaskData::Launch(FRDGBuilder& GraphBuilder, const FScene& Scene, TConstArrayView<FViewInfo*> Views)
{
	// Allocate decal visibility task data if any view needs decals
	for (FViewInfo* View : Views)
	{
		if (AreDecalsEnabled(*View) && !HasRayTracedOverlay(*View->Family))
		{
			return GraphBuilder.AllocObject<FDecalVisibilityTaskData>(Scene, Views);
		}
	}
	return nullptr;
}

FDecalVisibilityTaskData::FDecalVisibilityTaskData(const FScene& Scene, TConstArrayView<FViewInfo*> Views)
{
	ViewPackets.Reserve(Views.Num());

	for (const FViewInfo* View : Views)
	{
		if (AreDecalsEnabled(*View) && !HasRayTracedOverlay(*View->Family))
		{
			ViewPackets.Emplace(*this, Scene, *View);
		}
	}
}

FDecalVisibilityViewPacket::FDecalVisibilityViewPacket(const FDecalVisibilityTaskData& InTaskData, const FScene& Scene, const FViewInfo& InView)
	: TaskData(InTaskData)
	, View(InView)
{
	RelevantDecalsMap.Reserve((int32)EDecalRenderStage::Num);

	VisibleDecals.Task = LaunchSceneRenderTask(TEXT("BuildVisibleDecalList"), [&OutputList = VisibleDecals.List, &InputList = Scene.Decals, &View = View]
	{
		OutputList = DecalRendering::BuildVisibleDecalList(InputList, View);
	});

	AllTasksEvent.AddPrerequisites(VisibleDecals.Task);

	const auto LaunchRelevantDecalTask = [this] (EDecalRenderStage Stage)
	{
		FRelevantDecals& RelevantDecals = RelevantDecalsMap.Emplace(Stage);

		RelevantDecals.Task = LaunchSceneRenderTask(TEXT("BuildRelevantDecalList"), [&OutputList = RelevantDecals.List, &InputList = VisibleDecals.List, Stage]
		{
			OutputList = DecalRendering::BuildRelevantDecalList(InputList, Stage);

		}, VisibleDecals.Task);

		AllTasksEvent.AddPrerequisites(RelevantDecals.Task);
	};

	EShaderPlatform ShaderPlatform = View.Family->GetShaderPlatform();
	const bool bDBufferEnabled = ::IsDBufferEnabled(*View.Family, ShaderPlatform);
	const bool bGBufferEnabled = IsUsingGBuffers(ShaderPlatform);

	// DBuffer Passes
	if (bDBufferEnabled)
	{
		LaunchRelevantDecalTask(EDecalRenderStage::BeforeBasePass);
		LaunchRelevantDecalTask(EDecalRenderStage::Emissive);
	}
	// GBuffer Passes
	else if (bGBufferEnabled)
	{
		LaunchRelevantDecalTask(EDecalRenderStage::BeforeLighting);
	}

	// AmbientOcclusion Pass gets built on-demand, since we don't know if it will be enabled until later in the pipeline.
}

TConstArrayView<FVisibleDecal> FDecalVisibilityViewPacket::FinishVisibleDecals()
{
	check(IsInRenderingThread());
	if (VisibleDecals.Task.IsValid())
	{
		VisibleDecals.Task.Wait();
		VisibleDecals.Task = {};
	}
	return VisibleDecals.List;
}

TConstArrayView<const FVisibleDecal*> FDecalVisibilityViewPacket::FinishRelevantDecals(EDecalRenderStage Stage)
{
	check(IsInRenderingThread());
	FRelevantDecals* RelevantDecals = RelevantDecalsMap.Find(Stage);
	if (RelevantDecals)
	{
		if (RelevantDecals->Task.IsValid())
		{
			RelevantDecals->Task.Wait();
			RelevantDecals->Task = {};
		}
	}
	else
	{
		// Wasn't requested as a task. Build on-demand.
		RelevantDecals = &RelevantDecalsMap.Emplace(Stage);
		RelevantDecals->List = DecalRendering::BuildRelevantDecalList(FinishVisibleDecals(), Stage);
	}
	return RelevantDecals->List;
}

/**
 * A vertex shader for projecting a deferred decal onto the scene.
 */
class FDeferredDecalVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDeferredDecalVS);
	SHADER_USE_PARAMETER_STRUCT(FDeferredDecalVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FMatrix44f, FrustumComponentToClip)
		SHADER_PARAMETER_STRUCT_REF(FPrimitiveUniformShaderParameters, PrimitiveUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FDeferredDecalVS, "/Engine/Private/DeferredDecal.usf", "MainVS" ,SF_Vertex);

/**
 * A pixel shader for projecting a deferred decal onto the scene.
 */
class FDeferredDecalPS : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FDeferredDecalPS,Material);

public:
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return (Parameters.MaterialParameters.MaterialDomain == MD_DeferredDecal) &&
			DecalRendering::GetBaseRenderStage(DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters)) != EDecalRenderStage::None;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		DecalRendering::ModifyCompilationEnvironment(Parameters.Platform, DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::None, OutEnvironment);
	}

	FDeferredDecalPS() {}
	FDeferredDecalPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMaterialShader(Initializer)
	{
		DecalPositionHigh.Bind(Initializer.ParameterMap, TEXT("DecalPositionHigh"));
		SvPositionToDecal.Bind(Initializer.ParameterMap,TEXT("SvPositionToDecal"));
		RightEyeSvPositionToDecal.Bind(Initializer.ParameterMap, TEXT("RightEyeSvPositionToDecal"));
		DecalToWorld.Bind(Initializer.ParameterMap,TEXT("DecalToWorld"));
		DecalToWorldInvScale.Bind(Initializer.ParameterMap, TEXT("DecalToWorldInvScale"));
		DecalOrientation.Bind(Initializer.ParameterMap,TEXT("DecalOrientation"));
		DecalParams.Bind(Initializer.ParameterMap, TEXT("DecalParams"));
		DecalColorParam.Bind(Initializer.ParameterMap, TEXT("DecalColorParam"));
		MobileBasePassUniformBuffer.Bind(Initializer.ParameterMap, FMobileBasePassUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		MobileDirectionLightBufferParam.Bind(Initializer.ParameterMap, FMobileDirectionalLightShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		MobileReflectionCaptureParam.Bind(Initializer.ParameterMap, FMobileReflectionCaptureShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FViewInfo& View, const FVisibleDecal& VisibleDecal, const FMaterialRenderProxy* MaterialProxy, const FMaterial* MaterialResource, const float FadeAlphaValue = 1.0f, const FScene* Scene = nullptr)
	{
		auto& PrimitivePS = GetUniformBufferParameter<FPrimitiveUniformShaderParameters>();
		SetUniformBufferParameter(BatchedParameters, PrimitivePS, GIdentityPrimitiveUniformBuffer);

		FMaterialShader::SetParameters(BatchedParameters, MaterialProxy, *MaterialResource, View);

		const FMatrix DecalToWorldMatrix = VisibleDecal.ComponentTrans.ToMatrixWithScale();
		const FMatrix WorldToDecalMatrix = VisibleDecal.ComponentTrans.ToInverseMatrixWithScale();
		const FDFVector3 AbsoluteOrigin(DecalToWorldMatrix.GetOrigin());
		const FVector3f PositionHigh = AbsoluteOrigin.High;
		const FMatrix44f RelativeDecalToWorldMatrix = FDFMatrix::MakeToRelativeWorldMatrix(PositionHigh, DecalToWorldMatrix).M;
		const FVector3f OrientationVector = (FVector3f)VisibleDecal.ComponentTrans.GetUnitAxis(EAxis::X);

		if (DecalPositionHigh.IsBound())
		{
			SetShaderValue(BatchedParameters, DecalPositionHigh, PositionHigh);
		}
		if(SvPositionToDecal.IsBound())
		{
			FVector2D InvViewSize = FVector2D(1.0f / View.ViewRect.Width(), 1.0f / View.ViewRect.Height());

			// setup a matrix to transform float4(SvPosition.xyz,1) directly to Decal (quality, performance as we don't need to convert or use interpolator)
			//	new_xy = (xy - ViewRectMin.xy) * ViewSizeAndInvSize.zw * float2(2,-2) + float2(-1, 1);
			//  transformed into one MAD:  new_xy = xy * ViewSizeAndInvSize.zw * float2(2,-2)      +       (-ViewRectMin.xy) * ViewSizeAndInvSize.zw * float2(2,-2) + float2(-1, 1);

			float Mx = 2.0f * InvViewSize.X;
			float My = -2.0f * InvViewSize.Y;
			float Ax = -1.0f - 2.0f * View.ViewRect.Min.X * InvViewSize.X;
			float Ay = 1.0f + 2.0f * View.ViewRect.Min.Y * InvViewSize.Y;

			const FMatrix SvPositionToDecalBase(
				FPlane(Mx, 0, 0, 0),
				FPlane(0, My, 0, 0),
				FPlane(0, 0, 1, 0),
				FPlane(Ax, Ay, 0, 1)
			);

			// todo: we could use InvTranslatedViewProjectionMatrix and TranslatedWorldToComponent for better quality
			FMatrix44f SvPositionToDecalValue = FMatrix44f(										// LWC_TODO: Precision loss
				SvPositionToDecalBase * View.ViewMatrices.GetClipToWorld() * WorldToDecalMatrix);
			
			SetShaderValue(BatchedParameters, SvPositionToDecal, SvPositionToDecalValue);

			if (RightEyeSvPositionToDecal.IsBound())
			{
				const FViewInfo* InstancedView = View.GetInstancedView();
				if (InstancedView)
				{
					FMatrix44f RightEyeSvPositionToDecalValue = FMatrix44f(										// LWC_TODO: Precision loss
						SvPositionToDecalBase * InstancedView->ViewMatrices.GetClipToWorld() * WorldToDecalMatrix);
					SetShaderValue(BatchedParameters, RightEyeSvPositionToDecal, RightEyeSvPositionToDecalValue);
				}
			}
		}
		if(DecalToWorld.IsBound())
		{
			SetShaderValue(BatchedParameters, DecalToWorld, RelativeDecalToWorldMatrix);
		}
		if (DecalToWorldInvScale.IsBound())
		{
			SetShaderValue(BatchedParameters, DecalToWorldInvScale, static_cast<FVector3f>(DecalToWorldMatrix.GetScaleVector().Reciprocal()));
		}
		if (DecalOrientation.IsBound())
		{
			SetShaderValue(BatchedParameters, DecalOrientation, OrientationVector);
		}
		
		float LifetimeAlpha = 1.0f;

		// Certain engine captures (e.g. environment reflection) don't have a tick. Default to fully opaque.
		if (View.Family->Time.GetWorldTimeSeconds())
		{
			LifetimeAlpha = FMath::Clamp(FMath::Min(View.Family->Time.GetWorldTimeSeconds() * -VisibleDecal.InvFadeDuration + VisibleDecal.FadeStartDelayNormalized, View.Family->Time.GetWorldTimeSeconds() * VisibleDecal.InvFadeInDuration + VisibleDecal.FadeInStartDelayNormalized), 0.0f, 1.0f);
		}
 
		SetShaderValue(BatchedParameters, DecalParams, FVector2f(FadeAlphaValue, LifetimeAlpha));
		SetShaderValue(BatchedParameters, DecalColorParam, VisibleDecal.DecalColor);

		if (MobileDirectionLightBufferParam.IsBound() && Scene)
		{
			const int UniformBufferIndex = FMath::Clamp(FReadOnlyCVARCache::MobileForwardDecalLighting(), 1, 3);
			SetUniformBufferParameter(BatchedParameters, MobileDirectionLightBufferParam, Scene->UniformBuffers.MobileDirectionalLightUniformBuffers[UniformBufferIndex]);
		}

		if (MobileReflectionCaptureParam.IsBound())
		{
			if (Scene &&
				(
					(Scene->SkyLight && Scene->SkyLight->ProcessedTexture && Scene->SkyLight->ProcessedTexture->TextureRHI)
					|| Scene->CanSampleSkyLightRealTimeCaptureData()
				))
			{
				SetUniformBufferParameter(BatchedParameters, MobileReflectionCaptureParam, Scene->UniformBuffers.MobileSkyReflectionUniformBuffer);
			}
			else
			{
				SetUniformBufferParameter(BatchedParameters, MobileReflectionCaptureParam, GDefaultMobileReflectionCaptureUniformBuffer.GetUniformBufferRHI());
			}
		}
	}

private:
	LAYOUT_FIELD(FShaderParameter, SvPositionToDecal);
	LAYOUT_FIELD(FShaderParameter, RightEyeSvPositionToDecal);
	LAYOUT_FIELD(FShaderParameter, DecalPositionHigh);
	LAYOUT_FIELD(FShaderParameter, DecalToWorld);
	LAYOUT_FIELD(FShaderParameter, DecalToWorldInvScale);
	LAYOUT_FIELD(FShaderParameter, DecalOrientation);
	LAYOUT_FIELD(FShaderParameter, DecalParams);
	LAYOUT_FIELD(FShaderParameter, DecalColorParam);
	LAYOUT_FIELD(FShaderUniformBufferParameter, MobileBasePassUniformBuffer);
	LAYOUT_FIELD(FShaderUniformBufferParameter, MobileDirectionLightBufferParam);	
	LAYOUT_FIELD(FShaderUniformBufferParameter, MobileReflectionCaptureParam);
};

IMPLEMENT_MATERIAL_SHADER_TYPE(,FDeferredDecalPS,TEXT("/Engine/Private/DeferredDecal.usf"),TEXT("MainPS"),SF_Pixel);

class FDeferredDecalEmissivePS : public FDeferredDecalPS
{
	DECLARE_SHADER_TYPE(FDeferredDecalEmissivePS, Material);

public:
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return (Parameters.MaterialParameters.MaterialDomain == MD_DeferredDecal) &&
			DecalRendering::IsCompatibleWithRenderStage(DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::Emissive);
	}
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		DecalRendering::ModifyCompilationEnvironment(Parameters.Platform, DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::Emissive, OutEnvironment);
	}

	FDeferredDecalEmissivePS() {}
	FDeferredDecalEmissivePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FDeferredDecalPS(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FDeferredDecalEmissivePS, TEXT("/Engine/Private/DeferredDecal.usf"), TEXT("MainPS"), SF_Pixel);

class FDeferredDecalAmbientOcclusionPS : public FDeferredDecalPS
{
	DECLARE_SHADER_TYPE(FDeferredDecalAmbientOcclusionPS, Material);

public:
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return (Parameters.MaterialParameters.MaterialDomain == MD_DeferredDecal) &&
			DecalRendering::IsCompatibleWithRenderStage(DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::AmbientOcclusion);
	}
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		DecalRendering::ModifyCompilationEnvironment(Parameters.Platform, DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::AmbientOcclusion, OutEnvironment);
	}

	FDeferredDecalAmbientOcclusionPS() {}
	FDeferredDecalAmbientOcclusionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FDeferredDecalPS(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FDeferredDecalAmbientOcclusionPS, TEXT("/Engine/Private/DeferredDecal.usf"), TEXT("MainPS"), SF_Pixel);


class FDeferredDecalMobilePS : public FDeferredDecalPS
{
	DECLARE_SHADER_TYPE(FDeferredDecalMobilePS, Material);

public:
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return (Parameters.MaterialParameters.MaterialDomain == MD_DeferredDecal) &&
			DecalRendering::IsCompatibleWithRenderStage(DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::Mobile);
	}
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		DecalRendering::ModifyCompilationEnvironment(Parameters.Platform, DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters), EDecalRenderStage::Mobile, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("DECAL_MOBILE_FORWARD_LIT"), FReadOnlyCVARCache::MobileForwardDecalLighting() != 0 ? 1u : 0u);
	}

	FDeferredDecalMobilePS() {}
	FDeferredDecalMobilePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FDeferredDecalPS(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FDeferredDecalMobilePS, TEXT("/Engine/Private/DeferredDecal.usf"), TEXT("MainPS"), SF_Pixel);

namespace DecalRendering
{
	float GetDecalFadeScreenSizeMultiplier()
	{
		return CVarDecalFadeScreenSizeMultiplier.GetValueOnRenderThread();
	}

	float CalculateDecalFadeAlpha(float DecalFadeScreenSize, const FMatrix& ComponentToWorldMatrix, const FViewInfo& View, float FadeMultiplier)
	{
		check(View.IsPerspectiveProjection());

		float Distance = (View.ViewMatrices.GetViewOrigin() - ComponentToWorldMatrix.GetOrigin()).Size();
		float Radius = ComponentToWorldMatrix.GetMaximumAxisScale();
		float CurrentScreenSize = ((Radius / Distance) * FadeMultiplier);

		// fading coefficient needs to increase with increasing field of view and decrease with increasing resolution
		// FadeCoeffScale is an empirically determined constant to bring us back roughly to fraction of screen size for FadeScreenSize
		const float FadeCoeffScale = 600.0f;
		float FOVFactor = ((2.0f / View.ViewMatrices.GetViewToClip().M[0][0]) / View.ViewRect.Width()) * FadeCoeffScale;
		float FadeCoeff = DecalFadeScreenSize * FOVFactor;
		float FadeRange = FadeCoeff * 0.5f;

		float Alpha = (CurrentScreenSize - FadeCoeff) / FadeRange;
		return FMath::Clamp(Alpha, 0.0f, 1.0f);
	}

	void SortDecalList(FRelevantDecalList& Decals)
	{
		Decals.Sort([](const FVisibleDecal& A, const FVisibleDecal& B)
		{
			// Sort by sort order to allow control over composited result
			if (B.SortOrder != A.SortOrder)
			{ 
				return A.SortOrder < B.SortOrder;
			}
			// Then sort decals by state to reduce render target switches
			if (B.BlendDesc.bWriteNormal != A.BlendDesc.bWriteNormal)
			{
				// bWriteNormal here has priority because we want to render decals that output normals before those could read normals.
				// Also this is the only flag that can trigger a change of EDecalRenderTargetMode inside a single EDecalRenderStage, and we batch according to this.
				return B.BlendDesc.bWriteNormal < A.BlendDesc.bWriteNormal; // < so that those outputting normal are first.
			}
			// Sort decals by blend mode to reduce render target switches
			if (B.BlendDesc.Packed != A.BlendDesc.Packed)
			{
				// Sorting by the FDecalBlendDesc contents will reduce blend state changes.
				return (int32)B.BlendDesc.Packed < (int32)A.BlendDesc.Packed;
			}
			if (B.MaterialProxy != A.MaterialProxy)
			{
				// Batch decals with the same material together
				return B.MaterialProxy < A.MaterialProxy;
			}
			// Also sort by component since Sort() is not stable
			return B.Component < A.Component;
		});
	}

	FVisibleDecalList BuildVisibleDecalList(TConstArrayView<FDeferredDecalProxy*> Decals, const FViewInfo& View)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildVisibleDecalList);

		// Don't draw for shader complexity mode.
		// todo: Handle shader complexity mode for deferred decal.
		if (Decals.IsEmpty() || View.Family->EngineShowFlags.ShaderComplexity)
		{
			return {};
		}

		FVisibleDecalList VisibleDecals;
		VisibleDecals.Reserve(Decals.Num());

		const float FadeMultiplier = GetDecalFadeScreenSizeMultiplier();
		const EShaderPlatform ShaderPlatform = View.GetShaderPlatform();

		const bool bIsPerspectiveProjection = View.IsPerspectiveProjection();

		for (const FDeferredDecalProxy* DecalProxy : Decals)
		{
			if (!DecalProxy->DecalMaterial || !DecalProxy->DecalMaterial->IsValidLowLevelFast())
			{
				continue;
			}

			if (!DecalProxy->IsShown(&View))
			{
				continue;
			}

			const FMatrix ComponentToWorldMatrix = DecalProxy->ComponentTrans.ToMatrixWithScale();

			// can be optimized as we test against a sphere around the box instead of the box itself
			const float ConservativeRadius = FMath::Sqrt(
				ComponentToWorldMatrix.GetScaledAxis(EAxis::X).SizeSquared() +
				ComponentToWorldMatrix.GetScaledAxis(EAxis::Y).SizeSquared() +
				ComponentToWorldMatrix.GetScaledAxis(EAxis::Z).SizeSquared());

			const bool bIsVisibleInFirstView = View.ViewFrustum.IntersectSphere(ComponentToWorldMatrix.GetOrigin(), ConservativeRadius);
			const FViewInfo* InstancedView = View.GetInstancedView();
			const bool bIsVisibleInSecondView = InstancedView ? InstancedView->ViewFrustum.IntersectSphere(ComponentToWorldMatrix.GetOrigin(), ConservativeRadius) : false;

			// can be optimized as the test is too conservative (sphere instead of OBB)
			if (ConservativeRadius < SMALL_NUMBER || !(bIsVisibleInFirstView || bIsVisibleInSecondView))
			{
				continue;
			}

			float FadeAlpha = 1.0f;

			if (bIsPerspectiveProjection && DecalProxy->FadeScreenSize != 0.0f)
			{
				FadeAlpha = CalculateDecalFadeAlpha(DecalProxy->FadeScreenSize, ComponentToWorldMatrix, View, FadeMultiplier);
			}

			const bool bShouldRender = FadeAlpha > 0.0f;

			if (!bShouldRender)
			{
				continue;
			}

			VisibleDecals.Emplace(*DecalProxy, ConservativeRadius, FadeAlpha, ShaderPlatform, View.GetFeatureLevel());
		}

		return VisibleDecals;
	}

	FRelevantDecalList BuildRelevantDecalList(TConstArrayView<FVisibleDecal> Decals, EDecalRenderStage DecalRenderStage)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildRelevantDecalList);

		FRelevantDecalList RelevantDecals;
		RelevantDecals.Reserve(Decals.Num());

		for (const FVisibleDecal& VisibleDecal : Decals)
		{
			if (IsCompatibleWithRenderStage(VisibleDecal.BlendDesc, DecalRenderStage))
			{
				RelevantDecals.Emplace(&VisibleDecal);
			}
		}

		SortDecalList(RelevantDecals);
		return RelevantDecals;
	}

	bool HasRelevantDecals(TConstArrayView<FVisibleDecal> Decals, EDecalRenderStage DecalRenderStage)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HasRelevantDecals);

		for (const FVisibleDecal& VisibleDecal : Decals)
		{
			if (IsCompatibleWithRenderStage(VisibleDecal.BlendDesc, DecalRenderStage))
			{
				return true;
			}
		}

		return false;
	}

	FMatrix ComputeComponentToClipMatrix(const FViewInfo& View, const FMatrix& DecalComponentToWorld)
	{
		if (View.UsesMobileMultiViewShaders())
		{
			// In multi view, the rest of the matrix that is multiplied with DecalComponentToWorld in the non-multi view
			// case is split out in ViewUniformShaderParameters.MobileMultiviewDecalTransform so we can multiply
			// it later in the shader.
			return DecalComponentToWorld;
		}
		else
		{
			FMatrix ComponentToWorldMatrixTrans = DecalComponentToWorld.ConcatTranslation(View.ViewMatrices.GetPreViewTranslation());
			return ComponentToWorldMatrixTrans * View.ViewMatrices.GetTranslatedWorldToClip();
		}
	}

	bool TryGetDeferredDecalShaders(
		FMaterial const& Material,
		ERHIFeatureLevel::Type FeatureLevel,
		EDecalRenderStage DecalRenderStage,
		TShaderRef<FDeferredDecalPS>& OutPixelShader)
	{
		FMaterialShaderTypes ShaderTypes;
		
		if (DecalRenderStage == EDecalRenderStage::Emissive)
		{
			ShaderTypes.AddShaderType<FDeferredDecalEmissivePS>();
		}
		else if (DecalRenderStage == EDecalRenderStage::AmbientOcclusion)
		{
			ShaderTypes.AddShaderType<FDeferredDecalAmbientOcclusionPS>();
		}
		else if (DecalRenderStage == EDecalRenderStage::Mobile)
		{
			ShaderTypes.AddShaderType<FDeferredDecalMobilePS>();
		}
		else
		{
			ShaderTypes.AddShaderType<FDeferredDecalPS>();
		}

		FMaterialShaders Shaders;
		if (!Material.TryGetShaders(ShaderTypes, nullptr, Shaders))
		{
			return false;
		}

		Shaders.TryGetPixelShader(OutPixelShader);
		return OutPixelShader.IsValid();
	}

	bool GetShaders(ERHIFeatureLevel::Type FeatureLevel, 
		const FMaterial& Material, 
		EDecalRenderStage DecalRenderStage, 
		TShaderRef<FShader>& OutVertexShader,
		TShaderRef<FShader>& OutPixelShader)
	{
		TShaderRef<FDeferredDecalPS> PixelShader;
		if (!TryGetDeferredDecalShaders(Material, FeatureLevel, DecalRenderStage, PixelShader))
		{
			return false;
		}

		TShaderMapRef<FDeferredDecalVS> VertexShader(GetGlobalShaderMap(FeatureLevel));
		OutVertexShader = VertexShader;
		OutPixelShader = PixelShader;

		return true;
	}

	bool SetupShaderState(
		ERHIFeatureLevel::Type FeatureLevel,
		const FMaterial& Material,
		EDecalRenderStage DecalRenderStage,
		FBoundShaderStateInput& OutBoundShaderState,
		bool bRequired)
	{
		TShaderRef<FShader> VertexShader;
		TShaderRef<FShader> PixelShader;
		if (!GetShaders( FeatureLevel, Material, DecalRenderStage, VertexShader, PixelShader))
		{
			return false;
		}

		FRHIVertexShader* VertexShaderRHI = VertexShader.GetVertexShader(bRequired);
		FRHIPixelShader* PixelShaderRHI = PixelShader.GetPixelShader(bRequired);
		if (!VertexShaderRHI || !PixelShaderRHI)
		{
			return false;
		}

		OutBoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		OutBoundShaderState.VertexShaderRHI = VertexShaderRHI;
		OutBoundShaderState.PixelShaderRHI = PixelShaderRHI;

		return true;
	}

	FMaterialRenderProxy const* TryGetDeferredDecalMaterial(
		FMaterialRenderProxy const* MaterialProxy, 
		ERHIFeatureLevel::Type FeatureLevel,
		EDecalRenderStage DecalRenderStage,
		FMaterial const*& OutMaterialResource,
		TShaderRef<FDeferredDecalPS>& OutPixelShader)
	{
		OutMaterialResource = nullptr;

		while (MaterialProxy != nullptr)
		{
			OutMaterialResource = MaterialProxy->GetMaterialNoFallback(FeatureLevel);
			if (OutMaterialResource != nullptr)
			{
				if (TryGetDeferredDecalShaders(*OutMaterialResource, FeatureLevel, DecalRenderStage, OutPixelShader))
				{
					break;
				}
			}

			MaterialProxy = MaterialProxy->GetFallback(FeatureLevel);
		}

		return MaterialProxy;
	}

	void SetShader(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, uint32 StencilRef, const FViewInfo& View,
		const FVisibleDecal& VisibleDecal, EDecalRenderStage DecalRenderStage, const FMatrix& FrustumComponentToClip, const FScene* Scene)
	{
		FMaterial const* MaterialResource = nullptr;
		TShaderRef<FDeferredDecalPS> PixelShader;
		FMaterialRenderProxy const* MaterialProxy = TryGetDeferredDecalMaterial(VisibleDecal.MaterialProxy, View.GetFeatureLevel(), DecalRenderStage, MaterialResource, PixelShader);

		TShaderMapRef<FDeferredDecalVS> VertexShader(View.ShaderMap);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

		// Set vertex shader parameters.
		{
			FDeferredDecalVS::FParameters ShaderParameters;
			ShaderParameters.FrustumComponentToClip = FMatrix44f(FrustumComponentToClip); // LWC_TODO: Precision loss?
			ShaderParameters.PrimitiveUniformBuffer = GIdentityPrimitiveUniformBuffer.GetUniformBufferRef();
			ShaderParameters.View = View.GetShaderParameters();
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ShaderParameters);
		}

		// Set pixel shader parameters.
		{
			SetShaderParametersLegacyPS(RHICmdList, PixelShader, View, VisibleDecal, MaterialProxy, MaterialResource, VisibleDecal.FadeAlpha, Scene);
		}

		// Set stream source after updating cached strides
		RHICmdList.SetStreamSource(0, GetUnitCubeVertexBuffer(), 0);
	}

	void SetVertexShaderOnly(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, const FViewInfo& View, const FMatrix& FrustumComponentToClip)
	{
		TShaderMapRef<FDeferredDecalVS> VertexShader(View.ShaderMap);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		// Set vertex shader parameters.
		{
			FDeferredDecalVS::FParameters ShaderParameters;
			ShaderParameters.FrustumComponentToClip = FMatrix44f(FrustumComponentToClip); // LWC_TODO: Precision loss
			ShaderParameters.PrimitiveUniformBuffer = GIdentityPrimitiveUniformBuffer.GetUniformBufferRef();
			ShaderParameters.View = View.GetShaderParameters();
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ShaderParameters);
		}
	}
}
