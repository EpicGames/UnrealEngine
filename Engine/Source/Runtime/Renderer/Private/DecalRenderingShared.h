// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "DecalRenderingCommon.h"

class FDeferredDecalProxy;
class FMaterial;
class FMaterialRenderProxy;
class FScene;
class FViewInfo;
class FDecalVisibilityTaskData;

class FShader;
class FShaderMapPointerTable;
template<typename ShaderType, typename PointerTableType> class TShaderRefBase;
template<typename ShaderType> using TShaderRef = TShaderRefBase<ShaderType, FShaderMapPointerTable>;

/**
 * Compact deferred decal data for rendering.
 */
struct FVisibleDecal
{
	FVisibleDecal(const FDeferredDecalProxy& InDecalProxy, float InConservativeRadius, float InFadeAlpha, EShaderPlatform ShaderPlatform, ERHIFeatureLevel::Type FeatureLevel);

	const FMaterialRenderProxy* MaterialProxy;
	uintptr_t Component;
	uint32 SortOrder;
	FDecalBlendDesc BlendDesc;
	float ConservativeRadius;
	float FadeAlpha;
	float InvFadeDuration;
	float InvFadeInDuration;
	float FadeStartDelayNormalized;
	float FadeInStartDelayNormalized;
	FLinearColor DecalColor;
	FTransform ComponentTrans;
	FBox BoxBounds;
};

using FVisibleDecalList = TArray<FVisibleDecal, SceneRenderingAllocator>;
using FRelevantDecalList = TArray<const FVisibleDecal*, SceneRenderingAllocator>;

class FDecalVisibilityViewPacket
{
public:
	FDecalVisibilityViewPacket(const FDecalVisibilityTaskData& InTaskData, const FScene& Scene, const FViewInfo& InView);

	~FDecalVisibilityViewPacket()
	{
		check(bFinishCalled);
		check(AllTasksEvent.IsCompleted());
	}

	TConstArrayView<FVisibleDecal> FinishVisibleDecals();
	TConstArrayView<const FVisibleDecal*> FinishRelevantDecals(EDecalRenderStage Stage);

	void Finish()
	{
		bFinishCalled = true;
		AllTasksEvent.Trigger();
		AllTasksEvent.Wait();
	}

	bool HasStage(EDecalRenderStage Stage) const
	{
		return RelevantDecalsMap.Contains(Stage);
	}

private:
	const FDecalVisibilityTaskData& TaskData;
	const FViewInfo& View;

	struct FVisibleDecals
	{
		FVisibleDecalList List;
		UE::Tasks::FTask Task;

	} VisibleDecals;

	struct FRelevantDecals
	{
		FRelevantDecalList List;
		UE::Tasks::FTask Task;
	};

	TMap<EDecalRenderStage, FRelevantDecals> RelevantDecalsMap;
	UE::Tasks::FTaskEvent AllTasksEvent{ UE_SOURCE_LOCATION };
	bool bFinishCalled = false;

	friend class FDecalVisibilityTaskData;
};

class FDecalVisibilityTaskData
{
public:
	static FDecalVisibilityTaskData* Launch(FRDGBuilder& GraphBuilder, const FScene& Scene, TConstArrayView<FViewInfo*> Views);

	// Custom Render Passes have their own array of FViewInfo, outside the scene renderer's Views array.  When accessing decal visibility
	// for a specific view array, an offset must be added to account for where the relevant data is located in the visibility task data.
	// To enforce this, the view offset is passed separately to the accessor functions below.  INDEX_NONE is returned if decals are disabled
	// for the given array of Views, where the task data constructor won't have added them to the list.  It's up to the caller to check for
	// INDEX_NONE and skip decal rendering (the accessors will assert if INDEX_NONE is passed in for ViewOffset).
	int32 GetViewOffset(TConstArrayView<FViewInfo> Views) const
	{
		for (int32 ViewIndex = 0; ViewIndex < ViewPackets.Num(); ViewIndex++)
		{
			if (&ViewPackets[ViewIndex].View == Views.GetData())
			{
				return ViewIndex;
			}
		}
		return INDEX_NONE;
	}

	TConstArrayView<const FVisibleDecal*> FinishRelevantDecals(int32 ViewOffset, int32 ViewIndex, EDecalRenderStage Stage)
	{
		check(ViewOffset != INDEX_NONE);
		return ViewPackets[ViewOffset + ViewIndex].FinishRelevantDecals(Stage);
	}

	bool HasStage(int32 ViewOffset, int32 ViewIndex, EDecalRenderStage Stage) const
	{
		check(ViewOffset != INDEX_NONE);
		return ViewPackets[ViewOffset + ViewIndex].HasStage(Stage);
	}

	void Finish(int32 ViewOffset, int32 ViewIndex)
	{
		check(ViewOffset != INDEX_NONE);
		ViewPackets[ViewOffset + ViewIndex].Finish();
	}

private:
	FDecalVisibilityTaskData(const FScene& Scene, TConstArrayView<FViewInfo*> Views);

	TArray<FDecalVisibilityViewPacket, FRDGArrayAllocator> ViewPackets;

	friend class FDecalVisibilityViewPacket;
	RDG_FRIEND_ALLOCATOR_FRIEND(FDecalVisibilityTaskData);
};

struct FDecalViewInfo
{
	FDecalViewInfo(int32 InViewOffset, int32 InViewIndex, EDecalRenderStage InRenderStage, EDecalRenderTargetMode InRenderTargetMode, const TConstArrayView<const FVisibleDecal*>& InRelevantDeferredDecals) :
		ViewOffset(InViewOffset),
		ViewIndex(InViewIndex),
		RenderStage(InRenderStage),
		RenderTargetMode(InRenderTargetMode),
		RelevantDeferredDecals(InRelevantDeferredDecals)
	{
	}
	int32 ViewOffset;
	int32 ViewIndex;
	EDecalRenderStage RenderStage;
	EDecalRenderTargetMode RenderTargetMode;
	TConstArrayView<const FVisibleDecal*> RelevantDeferredDecals;
};
/**
 * Shared deferred decal functionality.
 */
namespace DecalRendering
{
	float GetDecalFadeScreenSizeMultiplier();
	float CalculateDecalFadeAlpha(float DecalFadeScreenSize, const FMatrix& ComponentToWorldMatrix, const FViewInfo& View, float FadeMultiplier);
	FMatrix ComputeComponentToClipMatrix(const FViewInfo& View, const FMatrix& DecalComponentToWorld);
	void SetVertexShaderOnly(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, const FViewInfo& View, const FMatrix& FrustumComponentToClip);
	void SortDecalList(FRelevantDecalList& Decals);
	FVisibleDecalList BuildVisibleDecalList(TConstArrayView<FDeferredDecalProxy*> Decals, const FViewInfo& View);
	FRelevantDecalList BuildRelevantDecalList(TConstArrayView<FVisibleDecal> Decals, EDecalRenderStage DecalRenderStage);
	bool HasRelevantDecals(TConstArrayView<FVisibleDecal> Decals, EDecalRenderStage DecalRenderStage);
	bool GetShaders(ERHIFeatureLevel::Type FeatureLevel, const FMaterial& Material, EDecalRenderStage DecalRenderStage, TShaderRef<FShader>& OutVertexShader, TShaderRef<FShader>& OutPixelShader);
	bool SetupShaderState(ERHIFeatureLevel::Type FeatureLevel, const FMaterial& Material, EDecalRenderStage DecalRenderStage, FBoundShaderStateInput& OutBoundShaderState, bool bRequired = true);
	void SetShader(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, uint32 StencilRef, const FViewInfo& View, const FVisibleDecal& DecalData, EDecalRenderStage DecalRenderStage, const FMatrix& FrustumComponentToClip, const FScene* Scene = nullptr);
};