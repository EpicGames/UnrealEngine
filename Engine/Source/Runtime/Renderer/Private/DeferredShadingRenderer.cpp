// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeferredShadingRenderer.cpp: Top level rendering loop for deferred shading
=============================================================================*/

#include "DeferredShadingRenderer.h"
#include "CoreMinimal.h"
#include "DeferredShadingRendererTypes.h"
#include "BasePassRendering.h"
#include "CanvasTypes.h"
#include "ComponentRecreateRenderStateContext.h"
#include "CompositionLighting/CompositionLighting.h"
#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "Containers/ArrayView.h"
#include "Containers/SetUtilities.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "Containers/StridedView.h"
#include "ConvexVolume.h"
#include "CustomDepthRendering.h"
#include "CustomRenderPassSceneCapture.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DBufferTextures.h"
#include "DebugViewModeHelpers.h"
#include "DepthRendering.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "DistanceFieldLightingShared.h"
#include "DynamicRenderScaling.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "Engine/SpecularProfile.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/ToonProfile.h"
#include "EngineModule.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "ExtensibleUniformBuffer.h"
#include "FogRendering.h"
#include "FrontLayerTranslucency.h"
#include "Froxel/Froxel.h"
#include "FXSystem.h"
#include "GenerateMips.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GlobalShader.h"
#include "GPUMessaging.h"
#include "GPUScene.h"
#include "GPUSortManager.h"
#include "HairStrands/HairStrandsComposition.h"
#include "HairStrands/HairStrandsData.h"
#include "HairStrands/HairStrandsDebug.h"
#include "HairStrands/HairStrandsEnvironment.h"
#include "HairStrands/HairStrandsRendering.h"
#include "HairStrands/HairStrandsUtils.h"
#include "HairStrandsInterface.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "HitProxies.h"
#include "IESTextureManager.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "InstanceCulling/InstanceCullingOcclusionQuery.h"
#include "LightFunctionAtlas.h"
#include "LightRendering.h"
#include "LightSceneInfo.h"
#include "LocalFogVolumeRendering.h"
#include "Logging/LogMacros.h"
#include "LogRenderer.h"
#include "Lumen/Lumen.h"
#include "Lumen/LumenHardwareRayTracingCommon.h"
#include "Lumen/LumenSceneData.h"
#include "Lumen/LumenScreenProbeGather.h"
#include "MaterialCache/MaterialCacheTagProvider.h"
#include "Materials/MaterialInterface.h"
#include "Math/Color.h"
#include "MegaLights/MegaLights.h"
#include "MeshBatch.h"
#include "MeshMaterialShader.h"
#include "Misc/ScopeExit.h"
#include "MultiGPU.h"
#include "Nanite/Nanite.h"
#include "Nanite/NaniteComposition.h"
#include "Nanite/NaniteRayTracing.h"
#include "Nanite/NaniteShading.h"
#include "Nanite/Voxel.h"
#include "NaniteDefinitions.h"
#include "NaniteVisualizationData.h"
#include "OIT/OIT.h"
#include "PathTracing.h"
#include "PhysicsFieldRendering.h"
#include "PostProcess/DebugAlphaChannel.h"
#include "PostProcess/PostProcessEyeAdaptation.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessInputs.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "PostProcess/PostProcessVisualizeCalibrationMaterial.h"
#include "PostProcess/TemporalAA.h"
#include "PrimitiveSceneInfo.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/TagTrace.h"
#include "RayTracing/RayTracing.h"
#include "RayTracing/RayTracingDecals.h"
#include "RayTracing/RayTracingLighting.h"
#include "RayTracing/RayTracingMaterialHitShaders.h"
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracing/RayTracingShaderBindingTable.h"
#include "RayTracingDebugTypes.h"
#include "RayTracingDynamicGeometryUpdateManager.h"
#include "RayTracingGeometryManagerInterface.h"
#include "RayTracingMeshDrawCommands.h"
#include "RayTracingVisualizationData.h"
#include "RectLightTextureManager.h"
#include "RenderCore.h"
#include "RendererInterface.h"
#include "RendererModule.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/CustomRenderPass.h"
#include "Rendering/NaniteResources.h"
#include "Rendering/NaniteStreamingManager.h"
#include "Rendering/RayTracingGeometryManager.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIPipeline.h"
#include "RHIResources.h"
#include "SceneCaptureRendering.h"
#include "SceneCulling/SceneCullingRenderer.h"
#include "SceneExtensions.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "SceneOcclusion.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextureReductions.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "SceneVisibility.h"
#include "ScreenPass.h"
#include "ScreenSpaceRayTracing.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderPrint.h"
#include "ShadingEnergyConservation.h"
#include "Shadows/FirstPersonSelfShadow.h"
#include "Shadows/ShadowSceneRenderer.h"
#include "ShowFlags.h"
#include "SingleLayerWaterRendering.h"
#include "SkyAtmosphereRendering.h"
#include "SparseVolumeTexture/ISparseVolumeTextureStreamingManager.h"
#include "SparseVolumeTexture/SparseVolumeTextureViewerRendering.h"
#include "Stats/Stats.h"
#include "StereoRendering.h"
#include "Substrate/Glint/GlintShadingLUTs.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "Tasks/Task.h"
#include "TranslucencyPass.h"
#include "TranslucentLighting.h"
#include "TranslucentPassResource.h"
#include "TranslucentRendering.h"
#include "VariableRateShadingImageManager.h"
#include "VelocityRendering.h"
#include "ViewData.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VirtualShadowMaps/VirtualShadowMapCacheManager.h"
#include "VolumetricCloudRendering.h"
#include "VolumetricRenderTarget.h"
#include "VT/VirtualTextureFeedbackResource.h"
#include "VT/VirtualTextureSystem.h"
#include "WaterInfoTextureRendering.h"
#include "SceneViewState.h"
#include "FogSeparateComposition.h"
#include "DepthCopy.h"

#if !UE_BUILD_SHIPPING
#include "RenderCaptureInterface.h"
#endif

extern int32 GNaniteShowStats;
extern int32 GNanitePickingDomain;

extern DynamicRenderScaling::FBudget GDynamicNaniteScalingPrimary;

static TAutoConsoleVariable<int32> CVarNanitePrimeHZBMode(
	TEXT("r.Nanite.PrimeHZB"),
	0,
	TEXT("If enabled, a separate pass is rendered to prime the HZB before Nanite visbuffer if there is no HZB present.\n")
	TEXT("   0 == off (default)\n")
	TEXT("   1 == run if no HZB available.")
	TEXT("   2 == Force on, mainly for testing / debugging purposes."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<bool> CVarNanitePrimeHZBOnlyRTFarField(
	TEXT("r.Nanite.PrimeHZB.DrawOnlyRTFarField"),
	true,
	TEXT("If enabled, draw only geometry marked as ray tracing far field in the HZB priming pass."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarNanitePrimeHZBRenderSizeBias(
	TEXT("r.Nanite.PrimeHZB.RenderSizeBias"),
	2,
	TEXT("Log2 scale factor by which to downsize the rendered HZB."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarNanitePrimeHZBDepthBias(
	TEXT("r.Nanite.PrimeHZB.SceneDepthBias"),
	150.0f,
	TEXT("Bias in world units by which to shift the HZB depth to avoid self occlusion from coarse representations drawn into the HZB."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarNanitePrimeHZBMPPE(
	TEXT("r.Nanite.PrimeHZB.MaxPixelsPerEdgeMultiplier"),
	32.0f,
	TEXT("Max pixel per edge scale factor used to aggressively reduce the rendered geometrical detail."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<bool> CVarNanitePrimeHZBSampleNonNanite(
	TEXT("r.Nanite.PrimeHZB.SampleNonNanite"),
	false,
	TEXT("Sample the scene depth buffer if available."),
	ECVF_RenderThreadSafe
);
static TAutoConsoleVariable<int32> CVarClearCoatNormal(
	TEXT("r.ClearCoatNormal"),
	0,
	TEXT("0 to disable clear coat normal.\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarIrisNormal(
	TEXT("r.IrisNormal"),
	0,
	TEXT("0 to disable iris normal.\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on"),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarTranslucencyLightingVolumeAsyncComputeClear(
	TEXT("r.TranslucencyLightingVolume.AsyncComputeClear"),
	false, // @todo: disabled due to GPU crashes
	TEXT("Whether to clear the translucency lighting volume using async compute.\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

#if !UE_BUILD_SHIPPING

static int32 GCaptureNextDeferredShadingRendererFrame = -1;
static FAutoConsoleVariableRef CVarCaptureNextRenderFrame(
	TEXT("r.CaptureNextDeferredShadingRendererFrame"),
	GCaptureNextDeferredShadingRendererFrame,
	TEXT("0 to capture the immideately next frame using e.g. RenderDoc or PIX.\n")
	TEXT(" > 0: N frames delay\n")
	TEXT(" < 0: disabled"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarDebugDrawOnlyRTFarField(
	TEXT("r.Nanite.Debug.RenderOnlyRayTracingFarField"),
	false,
	TEXT("Debug utility to render on the geometry marked as ray tracing far field into the visbuffer."),
	ECVF_RenderThreadSafe);

#endif

static TAutoConsoleVariable<int32> CVarRayTracing(
	TEXT("r.RayTracing"),
	0,
	TEXT("0 to disable ray tracing.\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarRayTracingTextureLod(
	TEXT("r.RayTracing.UseTextureLod"),
	0,
	TEXT("Enable automatic texture mip level selection in ray tracing material shaders.\n")
	TEXT(" 0: highest resolution mip level is used for all texture (default).\n")
	TEXT(" 1: texture LOD is approximated based on total ray length, output resolution and texel density at hit point (ray cone method)."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarForceAllRayTracingEffects(
	TEXT("r.RayTracing.ForceAllRayTracingEffects"),
	-1,
	TEXT("Force all ray tracing effects ON/OFF.\n")
	TEXT(" -1: Do not force (default) \n")
	TEXT(" 0: All ray tracing effects disabled\n")
	TEXT(" 1: All ray tracing effects enabled"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingAllowInline(
	TEXT("r.RayTracing.AllowInline"),
	1,
	TEXT("Allow use of Inline Ray Tracing if supported (default=1)."),	
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingAllowPipeline(
	TEXT("r.RayTracing.AllowPipeline"),
	1,
	TEXT("Allow use of Ray Tracing pipelines if supported (default=1)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingAsyncBuild(
	TEXT("r.RayTracing.AsyncBuild"),
	0,
	TEXT("Whether to build ray tracing acceleration structures on async compute queue.\n"),
	ECVF_RenderThreadSafe
);

int32 GRayTracingSBTImmediateRelease = 1;
static FAutoConsoleVariableRef CVarRayTracingSBTImmediateRelease(
	TEXT("r.RayTracing.SBT.ImmediateRelease"),
	GRayTracingSBTImmediateRelease,
	TEXT("When enabled, SBTs are released immediately when not needed by the current ViewFamily\n")
	TEXT(" 0: SBTs are kept alive for re-use during the same frame and can improve performance of multi-viewfamily scenarios\n")
	TEXT(" 1: SBTs are released immediately when not needed by current ViewFamily"),
	ECVF_RenderThreadSafe
);

static int32 GRayTracingMultiGpuTLASMask = 0;
static FAutoConsoleVariableRef CVarRayTracingMultiGpuTLASMask(
	TEXT("r.RayTracing.MultiGpuMaskTLAS"),
	GRayTracingMultiGpuTLASMask,
	TEXT("For Multi-GPU, controls which GPUs TLAS and material pipeline updates run on.  (default = 0)\n")
	TEXT(" 0: Run TLAS and material pipeline updates on all GPUs.  Original behavior -- the optimized version is disabled for now due to a bug.\n")
	TEXT(" 1: Run TLAS and material pipeline updates masked to the active view's GPUs to improve performance.  BLAS updates still run on all GPUs.")
);

static TAutoConsoleVariable<int32> CVarSceneDepthHZBAsyncCompute(
	TEXT("r.SceneDepthHZBAsyncCompute"), 0,
	TEXT("Selects whether HZB for scene depth buffer should be built with async compute.\n")
	TEXT(" 0: Don't use async compute (default)\n")
	TEXT(" 1: Use async compute, start as soon as possible\n")
	TEXT(" 2: Use async compute, start after ComputeLightGrid.CompactLinks pass"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowMapsRenderEarly(
	TEXT("r.shadow.ShadowMapsRenderEarly"), 0,
	TEXT("If enabled, shadows will render earlier in the frame. This can help async compute scheduling on some platforms\n")
	TEXT("Note: This is not compatible with VSMs\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarTranslucencyVelocity(
	TEXT("r.Translucency.Velocity"), 1,
	TEXT("Whether translucency can draws depth/velocity (enabled by default)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarTranslucencyEarlyVelocityPass(
	TEXT("r.Translucency.EarlyVelocityPass"), 2,
	TEXT("Enable translucency velocity pass to run earlier than the translucency pass so later passes like TSR can start earlier on async pipe.\n")
	TEXT("When it's enabled, stencil is maintained in the original scene depth. DOF PP reads depth from early depth and stencil from original depth.\n")
	TEXT(" 0: Disabled. Velocity renders in the standard translucency block.\n")
	TEXT(" 1: After deferred lighting. Maximum TSR async overlap.\n")
	TEXT(" 2: After volumetric cloud reconstruction. Avoid TSR/cloud bandwidth contention (Default)\n")
	TEXT(" 3: At the beginning of Translucency. Best if translucency occupancy is low and has a high rendering cost."),
	ECVF_RenderThreadSafe);

static FAutoConsoleCommand RecreateRenderStateContextCmd(
	TEXT("r.RecreateRenderStateContext"),
	TEXT("Recreate render state."),
	FConsoleCommandDelegate::CreateStatic([] { FGlobalComponentRecreateRenderStateContext Context; }));

#if !UE_BUILD_SHIPPING
static TAutoConsoleVariable<int32> CVarForceBlackVelocityBuffer(
	TEXT("r.Test.ForceBlackVelocityBuffer"), 0,
	TEXT("Force the velocity buffer to have no motion vector for debugging purpose."),
	ECVF_RenderThreadSafe);
#endif

static TAutoConsoleVariable<int32> CVarNaniteViewMeshLODBiasEnable(
	TEXT("r.Nanite.ViewMeshLODBias.Enable"), 1,
	TEXT("Whether LOD offset to apply for rasterized Nanite meshes for the main viewport should be based off TSR's ScreenPercentage (Enabled by default)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarNaniteViewMeshLODBiasOffset(
	TEXT("r.Nanite.ViewMeshLODBias.Offset"), 0.0f,
	TEXT("LOD offset to apply for rasterized Nanite meshes for the main viewport when using TSR (Default = 0)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarNaniteViewMeshLODBiasMin(
	TEXT("r.Nanite.ViewMeshLODBias.Min"), -2.0f,
	TEXT("Minimum LOD offset for rasterizing Nanite meshes for the main viewport (Default = -2)."),
	ECVF_RenderThreadSafe);

namespace Lumen
{
	extern bool AnyLumenHardwareRayTracingPassEnabled();
}
namespace Nanite
{
	extern bool IsStatFilterActive(const FString& FilterName);
	extern void ListStatFilters(FSceneRenderer* SceneRenderer);
}

namespace RayTracingDebug
{
	extern bool UseInlineHardwareRayTracing(const FSceneViewFamily& ViewFamily);
}

DECLARE_CYCLE_STAT(TEXT("InitViews Intentional Stall"), STAT_InitViews_Intentional_Stall, STATGROUP_InitViews);

DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer UpdateDownsampledDepthSurface"), STAT_FDeferredShadingSceneRenderer_UpdateDownsampledDepthSurface, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer Render Init"), STAT_FDeferredShadingSceneRenderer_Render_Init, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer FXSystem PreRender"), STAT_FDeferredShadingSceneRenderer_FXSystem_PreRender, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer AllocGBufferTargets"), STAT_FDeferredShadingSceneRenderer_AllocGBufferTargets, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer PrepareForwardLightData"), STAT_FDeferredShadingSceneRenderer_PrepareForwardLightData, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer DBuffer"), STAT_FDeferredShadingSceneRenderer_DBuffer, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer ResolveDepth After Basepass"), STAT_FDeferredShadingSceneRenderer_ResolveDepth_After_Basepass, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer Resolve After Basepass"), STAT_FDeferredShadingSceneRenderer_Resolve_After_Basepass, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer FXSystem PostRenderOpaque"), STAT_FDeferredShadingSceneRenderer_FXSystem_PostRenderOpaque, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer AfterBasePass"), STAT_FDeferredShadingSceneRenderer_AfterBasePass, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer Lighting"), STAT_FDeferredShadingSceneRenderer_Lighting, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer RenderLightShaftOcclusion"), STAT_FDeferredShadingSceneRenderer_RenderLightShaftOcclusion, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer RenderSkyAtmosphere"), STAT_FDeferredShadingSceneRenderer_RenderSkyAtmosphere, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer RenderFog"), STAT_FDeferredShadingSceneRenderer_RenderFog, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer RenderLocalFogVolume"), STAT_FDeferredShadingSceneRenderer_RenderLocalFogVolume, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer RenderLightShaftBloom"), STAT_FDeferredShadingSceneRenderer_RenderLightShaftBloom, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer RenderFinish"), STAT_FDeferredShadingSceneRenderer_RenderFinish, STATGROUP_SceneRendering);

DECLARE_GPU_STAT(RayTracingScene);
DECLARE_GPU_STAT(RayTracingGeometry);

DEFINE_GPU_STAT(Postprocessing);
DECLARE_GPU_STAT(VisibilityCommands);
DECLARE_GPU_STAT(RenderDeferredLighting);
DECLARE_GPU_STAT(AllocateRendertargets);
DECLARE_GPU_STAT(FrameRenderFinish);
DECLARE_GPU_STAT(PostRenderOpsFX);
DECLARE_GPU_STAT(WaterRendering);
DECLARE_GPU_STAT(HairRendering);
DECLARE_GPU_STAT(UploadDynamicBuffers);
DECLARE_GPU_STAT(PostOpaqueExtensions);
DEFINE_GPU_STAT(CustomRenderPasses);
DECLARE_GPU_STAT(Substrate);

DECLARE_GPU_STAT_NAMED(NaniteVisBuffer, TEXT("Nanite VisBuffer"));
DECLARE_GPU_STAT_NAMED(NanitePrimeHZB, TEXT("Nanite PrimeHZB"));

DECLARE_DWORD_COUNTER_STAT(TEXT("BasePass Total Raster Bins"), STAT_NaniteBasePassTotalRasterBins, STATGROUP_Nanite);
DECLARE_DWORD_COUNTER_STAT(TEXT("BasePass Visible Raster Bins"), STAT_NaniteBasePassVisibleRasterBins, STATGROUP_Nanite);

DECLARE_DWORD_COUNTER_STAT(TEXT("BasePass Total Shading Bins"), STAT_NaniteBasePassTotalShadingBins, STATGROUP_Nanite);
DECLARE_DWORD_COUNTER_STAT(TEXT("BasePass Visible Shading Bins"), STAT_NaniteBasePassVisibleShadingBins, STATGROUP_Nanite);

CSV_DEFINE_CATEGORY(LightCount, true);

static bool ShouldClearTranslucencyLightingVolumeInAsyncCompute()
{
	return CVarTranslucencyLightingVolumeAsyncComputeClear.GetValueOnRenderThread() && GSupportsEfficientAsyncCompute;
}

/*-----------------------------------------------------------------------------
	Global Illumination Plugin Function Delegates
-----------------------------------------------------------------------------*/

static FGlobalIlluminationPluginDelegates::FAnyRayTracingPassEnabled GIPluginAnyRaytracingPassEnabledDelegate;
FGlobalIlluminationPluginDelegates::FAnyRayTracingPassEnabled& FGlobalIlluminationPluginDelegates::AnyRayTracingPassEnabled()
{
	return GIPluginAnyRaytracingPassEnabledDelegate;
}

static FGlobalIlluminationPluginDelegates::FPrepareRayTracing GIPluginPrepareRayTracingDelegate;
FGlobalIlluminationPluginDelegates::FPrepareRayTracing& FGlobalIlluminationPluginDelegates::PrepareRayTracing()
{
	return GIPluginPrepareRayTracingDelegate;
}

static FGlobalIlluminationPluginDelegates::FRenderDiffuseIndirectLight GIPluginRenderDiffuseIndirectLightDelegate;
FGlobalIlluminationPluginDelegates::FRenderDiffuseIndirectLight& FGlobalIlluminationPluginDelegates::RenderDiffuseIndirectLight()
{
	return GIPluginRenderDiffuseIndirectLightDelegate;
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static FGlobalIlluminationPluginDelegates::FRenderDiffuseIndirectVisualizations GIPluginRenderDiffuseIndirectVisualizationsDelegate;
FGlobalIlluminationPluginDelegates::FRenderDiffuseIndirectVisualizations& FGlobalIlluminationPluginDelegates::RenderDiffuseIndirectVisualizations()
{
	return GIPluginRenderDiffuseIndirectVisualizationsDelegate;
}
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

const TCHAR* GetDepthPassReason(bool bDitheredLODTransitionsUseStencil, EShaderPlatform ShaderPlatform)
{
	if (IsForwardShadingEnabled(ShaderPlatform))
	{
		return TEXT("(Forced by ForwardShading)");
	}

	if (UseNanite(ShaderPlatform))
	{
		return TEXT("(Forced by Nanite)");
	}

	if (IsUsingDBuffers(ShaderPlatform))
	{
		return TEXT("(Forced by DBuffer)");
	}

	if (UseVirtualTexturing(ShaderPlatform))
	{
		return TEXT("(Forced by VirtualTexture)");
	}

	if (bDitheredLODTransitionsUseStencil)
	{
		return TEXT("(Forced by StencilLODDither)");
	}

	return TEXT("");
}

/*-----------------------------------------------------------------------------
	FDeferredShadingSceneRenderer
-----------------------------------------------------------------------------*/

FDeferredShadingSceneRenderer::FDeferredShadingSceneRenderer(const FSceneViewFamily* InViewFamily, FHitProxyConsumer* HitProxyConsumer)
	: FSceneRenderer(InViewFamily, HitProxyConsumer)
	, bAreLightsInLightGrid(false)
{
	ViewPipelineStates.SetNum(AllViews.Num());
	// Initialize scene renderer extensions here, after the rest of the renderer has been initialized
	InitSceneExtensionsRenderers(ViewFamily.EngineShowFlags, true);
}

/** 
* Renders the view family. 
*/
DECLARE_CYCLE_STAT(TEXT("Wait RayTracing Dynamic Bindings"), STAT_WaitRayTracingDynamicBindings, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("Wait Ray Tracing Scene Initialization"), STAT_WaitRayTracingSceneInitTask, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("Wait Ray Tracing Visible Shader Bindings Finalize"), STAT_WaitRayTracingVisibleShaderBindingsFinalizeTask, STATGROUP_SceneRendering);

DECLARE_CYCLE_STAT(TEXT("Wait Gather And Sort Lights"), STAT_WaitGatherAndSortLightsTask, STATGROUP_SceneRendering);

/**
 * Returns true if the depth Prepass needs to run
 */
bool FDeferredShadingSceneRenderer::ShouldRenderPrePass() const
{
	return (DepthPass.EarlyZPassMode != DDM_None || DepthPass.bEarlyZPassMovable != 0);
}

/**
 * Returns true if the Nanite rendering needs to run
 */
bool FDeferredShadingSceneRenderer::ShouldRenderNanite() const
{
	return UseNanite(ShaderPlatform) && ViewFamily.EngineShowFlags.NaniteMeshes && Nanite::GStreamingManager.HasResourceEntries();
}

bool FDeferredShadingSceneRenderer::RenderHzb(FRDGBuilder& GraphBuilder, FRDGTextureRef SceneDepthTexture, const FBuildHZBAsyncComputeParams* AsyncComputeParams, Froxel::FRenderer& FroxelRenderer)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HZB, "HZB");

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		FSceneViewState* ViewState = View.ViewState;
		const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);


		if (ViewPipelineState.bClosestHZB || ViewPipelineState.bFurthestHZB)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "BuildHZB(ViewId=%d)", ViewIndex);

			FRDGTextureRef ClosestHZBTexture = nullptr;
			FRDGTextureRef FurthestHZBTexture = nullptr;

			BuildHZB(
				GraphBuilder,
				SceneDepthTexture,
				/* VisBufferTexture = */ nullptr,
				GetHZBViewRect(View),
				View.GetFeatureLevel(),
				View.GetShaderPlatform(),
				TEXT("HZBClosest"),
				/* OutClosestHZBTexture = */ ViewPipelineState.bClosestHZB ? &ClosestHZBTexture : nullptr,
				TEXT("HZBFurthest"),
				/* OutFurthestHZBTexture = */ &FurthestHZBTexture,
				BuildHZBDefaultPixelFormat,
				AsyncComputeParams,
				FroxelRenderer.GetView(ViewIndex));

			// Update the view.
			{
				View.HZBMipmap0Size = FurthestHZBTexture->Desc.Extent;
				View.HZB = FurthestHZBTexture;

				// Extract furthest HZB texture.
				if (View.ViewState)
				{
					if (ShouldRenderNanite() || FInstanceCullingContext::IsOcclusionCullingEnabled())
					{
						GraphBuilder.QueueTextureExtraction(FurthestHZBTexture, &View.ViewState->PrevFrameViewInfo.HZB);
					}
					else
					{
						View.ViewState->PrevFrameViewInfo.HZB = nullptr;
					}
				}

				// Extract closest HZB texture.
				if (ViewPipelineState.bClosestHZB)
				{
					View.ClosestHZB = ClosestHZBTexture;
				}
			}
		}

		if (FamilyPipelineState->bHZBOcclusion && ViewState && ViewState->HZBOcclusionTests.GetNum() != 0)
		{
			check(ViewState->HZBOcclusionTests.IsValidFrame(ViewState->OcclusionFrameCounter));
			ViewState->HZBOcclusionTests.Submit(GraphBuilder, View);
		}

		if (Scene->InstanceCullingOcclusionQueryRenderer && View.ViewState)
		{
			// Render per-instance occlusion queries and save the mask to interpret results on the next frame
			const uint32 OcclusionQueryMaskForThisView = Scene->InstanceCullingOcclusionQueryRenderer->Render(GraphBuilder, Scene->GPUScene, View);
			View.ViewState->PrevFrameViewInfo.InstanceOcclusionQueryMask = OcclusionQueryMaskForThisView;
		}
	}

	return FamilyPipelineState->bHZBOcclusion;
}

BEGIN_SHADER_PARAMETER_STRUCT(FRenderOpaqueFXPassParameters, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
END_SHADER_PARAMETER_STRUCT()

static void RenderOpaqueFX(
	FRDGBuilder& GraphBuilder,
	TConstStridedView<FSceneView> Views,
	FSceneUniformBuffer &SceneUniformBuffer,
	FFXSystemInterface* FXSystem,
	ERHIFeatureLevel::Type FeatureLevel,
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformBuffer)
{
	// Notify the FX system that opaque primitives have been rendered and we now have a valid depth buffer.
	if (FXSystem && Views.Num() > 0)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, PostRenderOpsFX, "PostRenderOpsFX");
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderOpaqueFX);

		const ERDGPassFlags UBPassFlags = ERDGPassFlags::Compute | ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass | ERDGPassFlags::NeverCull;

		if (HasRayTracedOverlay(*Views[0].Family))
		{
			// In the case of Path Tracing/RT Debug -- we have not yet written to the SceneColor buffer, so make a dummy set of textures instead
			SceneTexturesUniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, nullptr, FeatureLevel, ESceneTextureSetupMode::SceneVelocity);
		}

		// Add a pass which extracts the RHI handle from the scene textures UB and sends it to the FX system.
		FRenderOpaqueFXPassParameters* ExtractUBPassParameters = GraphBuilder.AllocParameters<FRenderOpaqueFXPassParameters>();
		ExtractUBPassParameters->SceneTextures = SceneTexturesUniformBuffer;
		GraphBuilder.AddPass(RDG_EVENT_NAME("SetSceneTexturesUniformBuffer"), ExtractUBPassParameters, UBPassFlags, [ExtractUBPassParameters, FXSystem](FRHICommandListImmediate&)
		{
			FXSystem->SetSceneTexturesUniformBuffer(ExtractUBPassParameters->SceneTextures->GetRHIRef());
		});

		FXSystem->PostRenderOpaque(GraphBuilder, Views, SceneUniformBuffer, true /*bAllowGPUParticleUpdate*/);

		// Clear the scene textures UB pointer on the FX system. Use the same pass parameters to extend resource lifetimes.
		GraphBuilder.AddPass(RDG_EVENT_NAME("UnsetSceneTexturesUniformBuffer"), ExtractUBPassParameters, UBPassFlags, [FXSystem](FRHICommandListImmediate&)
		{
			FXSystem->SetSceneTexturesUniformBuffer({});
		});

		if (FGPUSortManager* GPUSortManager = FXSystem->GetGPUSortManager())
		{
			GPUSortManager->OnPostRenderOpaque(GraphBuilder);
		}
	}
}

#if RHI_RAYTRACING

static bool ShouldPrepareRayTracingDecals(const FScene& Scene, const FSceneViewFamily& ViewFamily)
{
	if (!IsRayTracingEnabled() || !RHISupportsRayTracingCallableShaders(ViewFamily.GetShaderPlatform()))
	{
		return false;
	}

	if (Scene.Decals.Num() == 0 || RayTracing::ShouldExcludeDecals())
	{
		return false;
	}

	return ViewFamily.EngineShowFlags.PathTracing && PathTracing::UsesDecals(ViewFamily);
}

static void DeduplicateRayGenerationShaders(TArray< FRHIRayTracingShader*>& RayGenShaders)
{
	TSet<FRHIRayTracingShader*> UniqueRayGenShaders;
	for (FRHIRayTracingShader* Shader : RayGenShaders)
	{
		UniqueRayGenShaders.Add(Shader);
	}
	RayGenShaders = UniqueRayGenShaders.Array();
}

BEGIN_SHADER_PARAMETER_STRUCT(FSetRayTracingBindingsPassParams, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRayTracingLightGrid, LightGridPacked)
	SHADER_PARAMETER_STRUCT_REF(FLumenHardwareRayTracingUniformBufferParameters, LumenHardwareRayTracingUniformBuffer)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSetRayTracingBindingsInlinePassParams, )
	RDG_BUFFER_ACCESS(InlineRayTracingBindingDataBuffer, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

bool FDeferredShadingSceneRenderer::SetupRayTracingPipelineStatesAndSBT(FRDGBuilder& GraphBuilder, bool bAnyInlineHardwareRayTracingPassEnabled, bool& bOutIsUsingFallbackRTPSO)
{
	if (!IsRayTracingEnabled() || Views.Num() == 0)
	{
		return false;
	}

	if (!FamilyPipelineState[&FFamilyPipelineState::bRayTracing])
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDeferredShadingSceneRenderer::SetupRayTracingPipelineStatesAndSBT);

	if (!GRHISupportsRayTracingShaders && !GRHISupportsInlineRayTracing)
	{
		return false;
	}

	const bool bIsPathTracing = ViewFamily.EngineShowFlags.PathTracing;

	// Get the max required local binding data size - SBTs are versioned together so using single initializer for now
	uint32 MaxLocalBindingDataSize = 0;
	ERayTracingShaderBindingMode ShaderBindingMode = ERayTracingShaderBindingMode::Disabled;

	bool bMaterialRayTracingSBTRequired = false;
	bool bLumenRayTracingSBTRequired = false;

	// Existing Scene SBTs (Material, Lumen, Inline) are populated even if this particular ViewFamily doesn't use them all,
	// to keep records consistent across multiple ViewFamilies rendering the same Scene.

	if (GRHISupportsRayTracingShaders)
	{
		// #dxr_todo: UE-72565: refactor ray tracing effects to not be member functions of DeferredShadingRenderer. 
		// Should register each effect at startup and just loop over them automatically to gather all required shaders.

		TArray<FRHIRayTracingShader*> RayGenShaders;

		// We typically see ~120 raygen shaders, but allow some headroom to avoid reallocation if our estimate is wrong.
		RayGenShaders.Reserve(256);

		if (bIsPathTracing)
		{
			// This view only needs the path tracing raygen shaders as all other
			// passes should be disabled.
			for (const FViewInfo& View : Views)
			{
				PreparePathTracing(View, *Scene, RayGenShaders);
			}
		}
		else
		{
			// Path tracing is disabled, get all other possible raygen shaders
			PrepareRayTracingDebug(ViewFamily, RayGenShaders);

			// These other cases do potentially depend on the camera position since they are
			// driven by FinalPostProcessSettings, which is why we need to merge them across views
			if (!IsForwardShadingEnabled(ShaderPlatform))
			{
				for (const FViewInfo& View : Views)
				{
					PrepareRayTracingShadows(View, *Scene, RayGenShaders);
					PrepareRayTracingAmbientOcclusion(View, RayGenShaders);
					PrepareRayTracingSkyLight(View, *Scene, RayGenShaders);
					PrepareRayTracingGlobalIlluminationPlugin(View, RayGenShaders);
					PrepareRayTracingTranslucency(View, RayGenShaders);
					PrepareRayTracingVolumetricFogShadows(View, *Scene, RayGenShaders);

					if (DoesPlatformSupportLumenGI(ShaderPlatform) 
						&& Lumen::UseHardwareRayTracing(ViewFamily))
					{
						if (IsLumenEnabled(View))
						{
							PrepareLumenHardwareRayTracingScreenProbeGather(View, RayGenShaders);
							PrepareLumenHardwareRayTracingShortRangeAO(View, RayGenShaders);
							PrepareLumenHardwareRayTracingRadianceCache(View, RayGenShaders);
							PrepareLumenHardwareRayTracingReflections(View, RayGenShaders);
							PrepareLumenHardwareRayTracingReSTIR(View, RayGenShaders);
							PrepareLumenHardwareRayTracingVisualize(View, RayGenShaders);
						}

						PrepareHardwareRayTracingTranslucency(View, RayGenShaders);
					}

					PrepareMegaLightsHardwareRayTracing(View, *Scene, RayGenShaders);
					PrepareMegaLightsHardwareRayTracingVisualize(View, *Scene, RayGenShaders);
				}
			}
		}

		if (Views.Num() > 1)
		{
			// If we have more than one View, chances are we got many duplicates, so compact the list here
			DeduplicateRayGenerationShaders(RayGenShaders);
		}

		if (RayGenShaders.Num())
		{
			// Create RTPSO and kick off high-level material parameter binding tasks which will be consumed during RDG execution in BindRayTracingMaterialPipeline()
			CreateMaterialRayTracingMaterialPipeline(GraphBuilder, RayGenShaders, MaxLocalBindingDataSize, bOutIsUsingFallbackRTPSO);

			// Need RTPSO
			EnumAddFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO);

			bMaterialRayTracingSBTRequired = true;

			auto* SBT = bIsPathTracing ? &(Scene->RayTracingPersistentSBTState.PathTracingSBT) : &(Scene->RayTracingPersistentSBTState.MaterialSBT);

			// Allocate persistent SBT RHI if necessary
			// (only when shader binding layout is enabled which is required to be able to share SBT between views)
			if (RHIGetStaticShaderBindingLayoutSupport(ShaderPlatform) != ERHIStaticShaderBindingLayoutSupport::Unsupported
				&& SBT->ID == INDEX_NONE)
			{
				SBT->ID = Scene->RayTracingSBT.AllocatePersistentSBTID(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::RTPSO);
			}

			// Material SBT release handled by ReleaseUnusedRayTracingSBTs if GRayTracingSBTImmediateRelease is 0.
			SBT->bUsedThisFrame = true;
		}
		else if (GRayTracingSBTImmediateRelease)
		{
			FScene::FRayTracingPersistentSBTState::FTrackedSBT& SBT = 
				bIsPathTracing ? 
				(Scene->RayTracingPersistentSBTState.PathTracingSBT) 
				: (Scene->RayTracingPersistentSBTState.MaterialSBT);

			if (SBT.ID != INDEX_NONE)
			{
				Scene->RayTracingSBT.ReleasePersistentSBT(SBT.ID);
				SBT.Reset();
			}
		}
	}

	if (GRHISupportsRayTracingShaders)
	{
		// Create Lumen hardware ray tracing SBT and material pipeline

		TArray<FRHIRayTracingShader*> LumenHardwareRayTracingRayGenShaders;

		if (!bIsPathTracing)
		{
			if (DoesPlatformSupportLumenGI(ShaderPlatform) && Lumen::UseHardwareRayTracing(ViewFamily))
			{
				for (const FViewInfo& View : Views)
				{
					if (IsLumenEnabled(View))
					{
						PrepareLumenHardwareRayTracingVisualizeLumenMaterial(View, LumenHardwareRayTracingRayGenShaders);
						PrepareLumenHardwareRayTracingRadianceCacheLumenMaterial(View, LumenHardwareRayTracingRayGenShaders);
						PrepareLumenHardwareRayTracingTranslucencyVolumeLumenMaterial(View, LumenHardwareRayTracingRayGenShaders);
						PrepareLumenHardwareRayTracingRadiosityLumenMaterial(View, LumenHardwareRayTracingRayGenShaders);
						PrepareLumenHardwareRayTracingReflectionsLumenMaterial(View, LumenHardwareRayTracingRayGenShaders);
						PrepareLumenHardwareRayTracingReSTIRLumenMaterial(View, LumenHardwareRayTracingRayGenShaders);
						PrepareLumenHardwareRayTracingScreenProbeGatherLumenMaterial(View, LumenHardwareRayTracingRayGenShaders);
						PrepareLumenHardwareRayTracingDirectLightingLumenMaterial(View, LumenHardwareRayTracingRayGenShaders);
					}
				}
			}

			for (const FViewInfo& View : Views)
			{
				PrepareMegaLightsHardwareRayTracingMaterial(View, *Scene, LumenHardwareRayTracingRayGenShaders);
				PrepareMegaLightsHardwareRayTracingVisualizeMaterial(View, *Scene, LumenHardwareRayTracingRayGenShaders);
			}
		}

		DeduplicateRayGenerationShaders(LumenHardwareRayTracingRayGenShaders);

		if (LumenHardwareRayTracingRayGenShaders.Num())
		{
			CreateLumenHardwareRayTracingMaterialPipeline(GraphBuilder, LumenHardwareRayTracingRayGenShaders, MaxLocalBindingDataSize);

			// Need RTPSO
			EnumAddFlags(ShaderBindingMode, ERayTracingShaderBindingMode::RTPSO);

			bLumenRayTracingSBTRequired = true;

			// Allocate persistent SBT RHI if necessary
			// (only when shader binding layout is enabled which is required to be able to share SBT between views)
			if (RHIGetStaticShaderBindingLayoutSupport(ShaderPlatform) != ERHIStaticShaderBindingLayoutSupport::Unsupported
				&& Scene->RayTracingPersistentSBTState.LumenSBT.ID == INDEX_NONE)
			{
				Scene->RayTracingPersistentSBTState.LumenSBT.ID = Scene->RayTracingSBT.AllocatePersistentSBTID(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::RTPSO);
			}

			// Lumen SBT release handled by ReleaseUnusedRayTracingSBTs if GRayTracingSBTImmediateRelease is 0.
			Scene->RayTracingPersistentSBTState.LumenSBT.bUsedThisFrame = true;
		}
		else if (GRayTracingSBTImmediateRelease && (Scene->RayTracingPersistentSBTState.LumenSBT.ID != INDEX_NONE))
		{
			Scene->RayTracingSBT.ReleasePersistentSBT(Scene->RayTracingPersistentSBTState.LumenSBT.ID);
			Scene->RayTracingPersistentSBTState.LumenSBT.Reset();
		}
	}

	// Check if inline SBT is needed or not
	if (bAnyInlineHardwareRayTracingPassEnabled && GRHIGlobals.RayTracing.RequiresInlineRayTracingSBT)
	{
		if (Scene->RayTracingPersistentSBTState.InlineSBT.ID == INDEX_NONE)
		{
			Scene->RayTracingPersistentSBTState.InlineSBT.ID = Scene->RayTracingSBT.AllocatePersistentSBTID(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::Inline);
		}

		// Inline SBT release handled by ReleaseUnusedRayTracingSBTs if GRayTracingSBTImmediateRelease is 0.
		Scene->RayTracingPersistentSBTState.InlineSBT.bUsedThisFrame = true;

		EnumAddFlags(ShaderBindingMode, ERayTracingShaderBindingMode::Inline);
	}
	else if (GRayTracingSBTImmediateRelease && Scene->RayTracingPersistentSBTState.InlineSBT.ID != INDEX_NONE)
	{
		Scene->RayTracingSBT.ReleasePersistentSBT(Scene->RayTracingPersistentSBTState.InlineSBT.ID);
		Scene->RayTracingPersistentSBTState.InlineSBT.Reset();
	}

	if (ShaderBindingMode != ERayTracingShaderBindingMode::Disabled)
	{
		Scene->RayTracingSBT.CheckPersistentRHI(GraphBuilder.RHICmdList, MaxLocalBindingDataSize);

		if (RHIGetStaticShaderBindingLayoutSupport(ShaderPlatform) != ERHIStaticShaderBindingLayoutSupport::Unsupported)
		{
			// share persistent SBT across all views

			bool bFirstViewWithRayTracingPass = true;

			for (FViewInfo& View : Views)
			{
				if (View.bHasAnyRayTracingPass)
				{
					auto* SBT = bIsPathTracing ? &(Scene->RayTracingPersistentSBTState.PathTracingSBT) : &(Scene->RayTracingPersistentSBTState.MaterialSBT);
					if (SBT->ID != INDEX_NONE)
					{
						View.MaterialRayTracingData.ShaderBindingTable = Scene->RayTracingSBT.GetPersistentSBT(SBT->ID);

						// Cache the PipelineState in Scene if View has one, so other ViewFamilies can reuse it
						if (!GRayTracingSBTImmediateRelease)
						{
							if (View.MaterialRayTracingData.PipelineState)
							{
								SBT->LastValidPipelineState = View.MaterialRayTracingData.PipelineState;
							}
							else if (SBT->LastValidPipelineState)
							{
								View.MaterialRayTracingData.PipelineState = SBT->LastValidPipelineState;
							}
						}
					}
					else
					{
						View.MaterialRayTracingData.ShaderBindingTable = nullptr;
					}

					if (Scene->RayTracingPersistentSBTState.LumenSBT.ID != INDEX_NONE)
					{
						View.LumenRayTracingData.ShaderBindingTable = Scene->RayTracingSBT.GetPersistentSBT(Scene->RayTracingPersistentSBTState.LumenSBT.ID);

						// Cache the PipelineState in Scene if View has one, so other ViewFamilies can reuse it
						if (!GRayTracingSBTImmediateRelease)
						{
							if (View.LumenRayTracingData.PipelineState)
							{
								Scene->RayTracingPersistentSBTState.LumenSBT.LastValidPipelineState = View.LumenRayTracingData.PipelineState;
							}
							else if (Scene->RayTracingPersistentSBTState.LumenSBT.LastValidPipelineState)
							{
								View.LumenRayTracingData.PipelineState = Scene->RayTracingPersistentSBTState.LumenSBT.LastValidPipelineState;
							}
						}
					}
					else
					{
						View.LumenRayTracingData.ShaderBindingTable = nullptr;
					}

					View.bOwnsShaderBindingTables = bFirstViewWithRayTracingPass;

					bFirstViewWithRayTracingPass = false;
				}
			}
		}
		else
		{
			// need transient SBTs per View

			for (FViewInfo& View : Views)
			{
				if (!View.bHasAnyRayTracingPass)
				{
					continue;
				}

				View.bTransientShaderBindingTables = true;

				if (bMaterialRayTracingSBTRequired)
				{
					View.MaterialRayTracingData.ShaderBindingTable = Scene->RayTracingSBT.AllocateTransientRHI(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Allow, MaxLocalBindingDataSize);
				}

				if (bLumenRayTracingSBTRequired)
				{
					View.LumenRayTracingData.ShaderBindingTable = Scene->RayTracingSBT.AllocateTransientRHI(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Allow, MaxLocalBindingDataSize);
				}
			}
		}

		if (Scene->RayTracingPersistentSBTState.InlineSBT.ID != INDEX_NONE)
		{
			// set the same InlineBindingDataBuffer on all views
			FRDGBufferRef InlineBindingDataBuffer = Scene->RayTracingSBT.GetPersistentInlineBindingDataBuffer(GraphBuilder, Scene->RayTracingPersistentSBTState.InlineSBT.ID);

			for (FViewInfo& View : Views)
			{
				if (View.bHasAnyRayTracingPass)
				{
					View.InlineRayTracingBindingDataBuffer = InlineBindingDataBuffer;
				}
			}
		}
	}

	return true;
}

void FDeferredShadingSceneRenderer::SetupRayTracingLightDataForViews(FRDGBuilder& GraphBuilder)
{
	if (!FamilyPipelineState[&FFamilyPipelineState::bRayTracing])
	{
		return;
	}

	const bool bPathTracingEnabled = ViewFamily.EngineShowFlags.PathTracing && FDataDrivenShaderPlatformInfo::GetSupportsPathTracing(Scene->GetShaderPlatform());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];

		bool bBuildLightGrid = false;

		// Path Tracing currently uses its own code to manage lights, so doesn't need to run this.
		if (!bPathTracingEnabled)
		{
			if (Lumen::IsUsingRayTracingLightingGrid(ViewFamily, View, GetViewPipelineState(View).DiffuseIndirectMethod)
				|| GetRayTracingTranslucencyOptions(View).bEnabled
				|| ViewFamily.EngineShowFlags.RayTracingDebug)
			{
				bBuildLightGrid = true;
			}
		}

		// The light data is built in TranslatedWorld space so must be built per view
		View.RayTracingLightGridUniformBuffer = CreateRayTracingLightData(GraphBuilder, Scene, View, View.ShaderMap, bBuildLightGrid);
	}
}

bool FDeferredShadingSceneRenderer::DispatchRayTracingWorldUpdates(FRDGBuilder& GraphBuilder, FRDGBufferRef& OutDynamicGeometryScratchBuffer, ERHIPipeline ResourceAccessPipelines)
{
	OutDynamicGeometryScratchBuffer = nullptr;

	if (!FamilyPipelineState[&FFamilyPipelineState::bRayTracing])
	{
		// - Nanite ray tracing instances are already pointing at the new BLASes and RayTracingDataOffsets in GPUScene have been updated
		Nanite::GRayTracingManager.ProcessBuildRequests(GraphBuilder);
		return false;
	}

	check(IsRayTracingEnabled() && !Views.IsEmpty());

	TRACE_CPUPROFILER_EVENT_SCOPE(FDeferredShadingSceneRenderer::DispatchRayTracingWorldUpdates);

	FRayTracingScene& RayTracingScene = Scene->RayTracingScene;

	{
		SCOPE_CYCLE_COUNTER(STAT_WaitRayTracingSceneInitTask);
		RayTracingScene.InitTask.Wait();
		RayTracingScene.InitTask = {};
	}

	const bool bRayTracingAsyncBuild = CVarRayTracingAsyncBuild.GetValueOnRenderThread() != 0 && GRHISupportsRayTracingAsyncBuildAccelerationStructure;
	const ERDGPassFlags ComputePassFlags = bRayTracingAsyncBuild ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute;

	{
		Nanite::GRayTracingManager.UpdateStreaming(GraphBuilder, Views, GetSceneUniforms(), GetActiveSceneTexturesConfig().Extent);
		Nanite::GRayTracingManager.ProcessUpdateRequests(GraphBuilder, GetSceneUniforms());

		const bool bAnyBlasRebuilt = Nanite::GRayTracingManager.ProcessBuildRequests(GraphBuilder);
		if (bAnyBlasRebuilt)
		{
			for (FViewInfo& View : Views)
			{
				if (View.ViewState != nullptr && !View.bIsOfflineRender)
				{
					// don't invalidate in the offline case because we only get one attempt at rendering each sample
					View.ViewState->PathTracingInvalidate();
				}
			}
		}
	}

	// Keep mask the same as what's already set (which will be the view mask) if TLAS updates should be masked to the view
	RDG_GPU_MASK_SCOPE(GraphBuilder, GRayTracingMultiGpuTLASMask ? GraphBuilder.RHICmdList.GetGPUMask() : FRHIGPUMask::All());

	FRayTracingDynamicGeometryUpdateManager* DynamicGeometryUpdateManager = Scene->GetRayTracingDynamicGeometryUpdateManager();
	DynamicGeometryUpdateManager->AddDynamicGeometryUpdatePass(GraphBuilder, ComputePassFlags, GetSceneUniformBufferRef(GraphBuilder), ResourceAccessPipelines, OutDynamicGeometryScratchBuffer);

	((FRayTracingGeometryManager*)GRayTracingGeometryManager)->ResetVisibleGeometries();

	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, RayTracingScene, "RayTracingScene");

		RayTracingScene.Update(GraphBuilder, GetSceneUniforms(), &Scene->GPUScene, ComputePassFlags);
		RayTracingScene.Build(GraphBuilder, ComputePassFlags | ERDGPassFlags::NeverCull, OutDynamicGeometryScratchBuffer);
	}

	GraphBuilder.AddDispatchHint();

	return true;
}

void FDeferredShadingSceneRenderer::SetupRayTracingRenderingData(FRDGBuilder& GraphBuilder, RayTracing::FGatherInstancesTaskData& RayTracingGatherInstancesTaskData)
{
	check(FamilyPipelineState[&FFamilyPipelineState::bRayTracing]);

	TRACE_CPUPROFILER_EVENT_SCOPE(FDeferredShadingSceneRenderer::SetupRayTracingRenderingData);

	// Keep mask the same as what's already set (which will be the view mask) if TLAS updates should be masked to the view
	RDG_GPU_MASK_SCOPE(GraphBuilder, GRayTracingMultiGpuTLASMask ? GraphBuilder.RHICmdList.GetGPUMask() : FRHIGPUMask::All());

	const bool bIsPathTracing = ViewFamily.EngineShowFlags.PathTracing;

	bool bAnyInlineHardwareRayTracingPassEnabled = false;

	for (int32 ViewExt = 0; ViewExt < ViewFamily.ViewExtensions.Num(); ++ViewExt)
	{
		if (EnumHasAnyFlags(ViewFamily.ViewExtensions[ViewExt]->GetFlags(), ESceneViewExtensionFlags::RequiresHardwareInlineRayTracing))
		{
			bAnyInlineHardwareRayTracingPassEnabled = true;
		}
	}

	for (FViewInfo& View : Views)
	{
		if (View.bHasAnyRayTracingPass)
		{
			SetupLumenHardwareRayTracingUniformBuffer(View);
		}
		
		if (Lumen::AnyLumenHardwareInlineRayTracingPassEnabled(Scene, View)
			|| MegaLights::UseInlineHardwareRayTracing(ViewFamily)
			|| RayTracingDebug::UseInlineHardwareRayTracing(ViewFamily))
		{
			bAnyInlineHardwareRayTracingPassEnabled = true;
		}
	}

	const bool bShouldRenderNanite = ShouldRenderNanite();

	Nanite::GRayTracingManager.UpdateUniformBuffer(GraphBuilder, bShouldRenderNanite);

	{
		SCOPE_CYCLE_COUNTER(STAT_WaitRayTracingDynamicBindings);

		// Need to wait for dynamic mesh batches tasks to finish before executing SetupRayTracingPipelineStatesAndSBT(...)
		// since they can request new materials that need to be included in RTPSO.
		// Also waits for dynamic range allocations in SBT to be complete
		// since CheckPersistentRHI (inside SetupRayTracingPipelineStatesAndSBT) reads MaxNumDynamicGeometrySegments.
		RayTracing::WaitForDynamicBindings(RayTracingGatherInstancesTaskData);
	}

	bool bIsUsingFallbackRTPSO = false;
	SetupRayTracingPipelineStatesAndSBT(GraphBuilder, bAnyInlineHardwareRayTracingPassEnabled, bIsUsingFallbackRTPSO);

	{
		SCOPE_CYCLE_COUNTER(STAT_WaitRayTracingVisibleShaderBindingsFinalizeTask);
		RayTracing::FinishGatherVisibleShaderBindings(RayTracingGatherInstancesTaskData);
	}

	TConstArrayView<FRayTracingShaderBindingData> VisibleRayTracingShaderBindings = RayTracing::GetVisibleShaderBindings(RayTracingGatherInstancesTaskData);

	FRayTracingShaderBindingDataOneFrameArray& DirtyPersistentRayTracingShaderBindings = GraphBuilder.AllocArray<FRayTracingShaderBindingData>();

	bool bRequireBindingsUpdate = false;

	for (const FViewInfo& View : Views)
	{
		// Check if any SBT exists in the Scene (not just this ViewFamily)
		bRequireBindingsUpdate |= View.MaterialRayTracingData.ShaderBindingTable || View.LumenRayTracingData.ShaderBindingTable || (Scene->RayTracingPersistentSBTState.InlineSBT.ID != INDEX_NONE);
	}

	if (bRequireBindingsUpdate)
	{
		// If fallback RTPSO then mark all bindings as dirty because they need to bound again when final RTPSO is ready (Shader identifier could have changed)
		const bool bForceAllDirty = bIsUsingFallbackRTPSO;

		// Build the dirty persistent shader bindings from the visible shader bindings (SBT version is updated after the PSO and SBTs have been created)
		DirtyPersistentRayTracingShaderBindings = Scene->RayTracingSBT.GetDirtyBindings(VisibleRayTracingShaderBindings, bForceAllDirty);
	}

	FRDGBufferRef LumenHardwareRayTracingHitDataBuffer = nullptr;
	if (bAnyInlineHardwareRayTracingPassEnabled)
	{
		// TODO: Could have a persistent HardwareRayTracingHitDataBuffer and update using DirtyPersistentRayTracingShaderBindings
		// instead of always recreating the buffer using VisibleRayTracingShaderBindings
		LumenHardwareRayTracingHitDataBuffer = SetupLumenHardwareRayTracingHitGroupBuffer(GraphBuilder, VisibleRayTracingShaderBindings);
	}

	for (FViewInfo& View : Views)
	{
		View.LumenHardwareRayTracingHitDataBuffer = LumenHardwareRayTracingHitDataBuffer;

		if (!View.bOwnsShaderBindingTables)
		{
			continue;
		}

		// Prepare the local ray tracing shader binding data to update on RHI timeline for Material and Lumen
		if (View.MaterialRayTracingData.ShaderBindingTable)
		{
			SetupMaterialRayTracingHitGroupBindings(GraphBuilder, View, View.bTransientShaderBindingTables ? VisibleRayTracingShaderBindings : DirtyPersistentRayTracingShaderBindings);
		}

		// All ViewFamilies write to Lumen SBT if it exists, using the shared PipelineState
		if (View.LumenRayTracingData.ShaderBindingTable)
		{
			SetupLumenHardwareRayTracingHitGroupBindings(GraphBuilder, View, View.bTransientShaderBindingTables ? VisibleRayTracingShaderBindings : DirtyPersistentRayTracingShaderBindings);
		}

		FSetRayTracingBindingsPassParams* PassParams = GraphBuilder.AllocParameters<FSetRayTracingBindingsPassParams>();
		PassParams->Scene = GetSceneUniformBufferRef(GraphBuilder);
		PassParams->LightGridPacked = bIsPathTracing ? nullptr : View.RayTracingLightGridUniformBuffer; // accessed by FRayTracingLightingMS // Is this needed for anything?
		PassParams->LumenHardwareRayTracingUniformBuffer = View.LumenHardwareRayTracingUniformBuffer;

		const FRayTracingLightFunctionMap* RayTracingLightFunctionMap = GraphBuilder.Blackboard.Get<FRayTracingLightFunctionMap>();
		GraphBuilder.AddPass(RDG_EVENT_NAME("SetRayTracingBindings"), PassParams, ERDGPassFlags::Copy | ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			[this, PassParams, bIsPathTracing, &View, RayTracingLightFunctionMap](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SetRayTracingBindings);

				check(View.MaterialRayTracingData.PipelineState || View.MaterialRayTracingData.MaterialBindings.Num() == 0);

				if (View.MaterialRayTracingData.PipelineState && (View.MaterialRayTracingData.MaterialBindings.Num() || View.MaterialRayTracingData.CallableBindings.Num()))
				{
					SetRayTracingShaderBindings(RHICmdList, Allocator, View.MaterialRayTracingData);

					if (bIsPathTracing)
					{
						SetupPathTracingDefaultMissShader(RHICmdList, View);

						BindLightFunctionShadersPathTracing(RHICmdList, Scene, RayTracingLightFunctionMap, View);
					}
					else
					{
						SetupRayTracingDefaultMissShader(RHICmdList, View);
						SetupRayTracingLightingMissShader(RHICmdList, View);

						BindLightFunctionShaders(RHICmdList, Scene, RayTracingLightFunctionMap, View);
					}

					RHICmdList.CommitShaderBindingTable(View.MaterialRayTracingData.ShaderBindingTable);
				}

				if (!bIsPathTracing)
				{
					if (View.LumenRayTracingData.PipelineState && View.LumenRayTracingData.ShaderBindingTable)
					{
						RHICmdList.SetRayTracingMissShader(View.LumenRayTracingData.ShaderBindingTable, RAY_TRACING_MISS_SHADER_SLOT_DEFAULT, View.LumenRayTracingData.PipelineState, 0 /* MissShaderPipelineIndex */, 0, nullptr, 0);
					}

					if (View.LumenRayTracingData.ShaderBindingTable)
					{
						SetRayTracingShaderBindings(RHICmdList, Allocator, View.LumenRayTracingData);
						RHICmdList.CommitShaderBindingTable(View.LumenRayTracingData.ShaderBindingTable);
					}
				}
			});
	}

	if (!bIsPathTracing && (Scene->RayTracingPersistentSBTState.InlineSBT.ID != INDEX_NONE))
	{
		FViewInfo::FRayTracingData InlineRayTracingData;
		InlineRayTracingData.ShaderBindingTable = Scene->RayTracingSBT.GetPersistentSBT(Scene->RayTracingPersistentSBTState.InlineSBT.ID);

		// Prepare the local ray tracing shader binding data to update on RHI timeline for Inline SBT
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FDeferredShadingSceneRenderer::SetupInlineHardwareRaytracingHitGroupBindings);

			const uint32 ShaderSlotsPerSegment = Scene->RayTracingSBT.GetNumShaderSlotsPerSegment();
			AddRayTracingLocalShaderBindingWriterTasks(GraphBuilder, DirtyPersistentRayTracingShaderBindings, InlineRayTracingData.MaterialBindings,
				[ShaderSlotsPerSegment, &RayTracingMeshCommands = Scene->CachedRayTracingMeshCommands](const FRayTracingShaderBindingData& RTShaderBindingData, FRayTracingLocalShaderBindingWriter* BindingWriter)
				{
					const FRayTracingMeshCommand& MeshCommand = RTShaderBindingData.GetRayTracingMeshCommand(RayTracingMeshCommands);

					for (uint32 SlotIndex = 0; SlotIndex < ShaderSlotsPerSegment; ++SlotIndex)
					{
						FRayTracingLocalShaderBindings& Binding = BindingWriter->AddWithExternalParameters();
						Binding.RecordIndex = RTShaderBindingData.SBTRecordIndex + SlotIndex;
						Binding.Geometry = RTShaderBindingData.RayTracingGeometry;
						Binding.SegmentIndex = MeshCommand.GeometrySegmentIndex;
						Binding.BindingType = RTShaderBindingData.BindingType;
						Binding.UserData = 0;
					}
				});
		}

		{
			FSetRayTracingBindingsInlinePassParams* PassParams = GraphBuilder.AllocParameters<FSetRayTracingBindingsInlinePassParams>();
			PassParams->InlineRayTracingBindingDataBuffer = Scene->RayTracingSBT.GetPersistentInlineBindingDataBuffer(GraphBuilder, Scene->RayTracingPersistentSBTState.InlineSBT.ID);

			GraphBuilder.AddPass(RDG_EVENT_NAME("SetRayTracingBindingsInline"), PassParams, ERDGPassFlags::Copy | ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				[PassParams, &Allocator = Allocator, InlineRayTracingData = MoveTemp(InlineRayTracingData)](FRDGAsyncTask, FRHICommandList& RHICmdList) mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(SetRayTracingBindingsInline);

					SetRayTracingShaderBindings(RHICmdList, Allocator, InlineRayTracingData);

					RHICmdList.CommitShaderBindingTable(InlineRayTracingData.ShaderBindingTable, PassParams->InlineRayTracingBindingDataBuffer->GetRHI());
				});
		}
	}

	GraphBuilder.AddPass(RDG_EVENT_NAME("UnlockStaticSBTAllocations"), ERDGPassFlags::NeverCull,
		[Scene = Scene](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			Scene->RayTracingSBT.ResetStaticAllocationLock();
		}
	);
}

#endif // RHI_RAYTRACING

void FDeferredShadingSceneRenderer::BeginInitDynamicShadows(FRDGBuilder& GraphBuilder, FInitViewTaskDatas& TaskDatas, FInstanceCullingManager& InstanceCullingManager)
{
	extern int32 GEarlyInitDynamicShadows;

	// This is called from multiple locations and will succeed if the visibility tasks are ready.
	if (!TaskDatas.DynamicShadows
		&& GEarlyInitDynamicShadows != 0
		&& ViewFamily.EngineShowFlags.DynamicShadows
		&& !ViewFamily.EngineShowFlags.HitProxies
		&& !HasRayTracedOverlay(ViewFamily)
		&& TaskDatas.VisibilityTaskData->IsTaskWaitingAllowed())
	{
		TaskDatas.DynamicShadows = FSceneRenderer::BeginInitDynamicShadows(GraphBuilder, true, TaskDatas.VisibilityTaskData, InstanceCullingManager);
	}
}

void FDeferredShadingSceneRenderer::FinishInitDynamicShadows(FRDGBuilder& GraphBuilder, FDynamicShadowsTaskData*& TaskData, FInstanceCullingManager& InstanceCullingManager)
{
	if (ViewFamily.EngineShowFlags.DynamicShadows && !ViewFamily.EngineShowFlags.HitProxies && !HasRayTracedOverlay(ViewFamily))
	{
		// Setup dynamic shadows.
		if (TaskData)
		{
			FSceneRenderer::FinishInitDynamicShadows(GraphBuilder, TaskData);
		}
		else
		{
			TaskData = InitDynamicShadows(GraphBuilder, InstanceCullingManager);
		}
	}
}

static TAutoConsoleVariable<float> CVarStallInitViews(
	TEXT("CriticalPathStall.AfterInitViews"),
	0.0f,
	TEXT("Sleep for the given time after InitViews. Time is given in ms. This is a debug option used for critical path analysis and forcing a change in the critical path."));

void FDeferredShadingSceneRenderer::CommitFinalPipelineState()
{
	// Family pipeline state
	{
		FamilyPipelineState.Set(&FFamilyPipelineState::bNanite, UseNanite(ShaderPlatform)); // TODO: Should this respect ViewFamily.EngineShowFlags.NaniteMeshes?

		static const auto ICVarHZBOcc = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HZBOcclusion"));
		FamilyPipelineState.Set(&FFamilyPipelineState::bHZBOcclusion, ICVarHZBOcc->GetInt() != 0);	
	}

	CommitIndirectLightingState();

	// Views pipeline states
	for (int32 ViewIndex = 0; ViewIndex < AllViews.Num(); ViewIndex++)
	{
		const FViewInfo& View = *AllViews[ViewIndex];
		TPipelineState<FPerViewPipelineState>& ViewPipelineState = GetViewPipelineStateWritable(View);

		// Commit HZB state
		{
			const bool bHasSSGI = ViewPipelineState[&FPerViewPipelineState::DiffuseIndirectMethod] == EDiffuseIndirectMethod::SSGI;
			const bool bUseLumen = ViewPipelineState[&FPerViewPipelineState::DiffuseIndirectMethod] == EDiffuseIndirectMethod::Lumen 
				|| ViewPipelineState[&FPerViewPipelineState::ReflectionsMethod] == EReflectionsMethod::Lumen;

			const ELumenFinalGatherMethod LumenFinalGatherMethod = Lumen::GetFinalGatherMethod(ViewFamily, ShaderPlatform);
			// Only HZB traversal needs Closest HZB
			const bool bLumenNeedsClosestHZB = (ViewPipelineState[&FPerViewPipelineState::DiffuseIndirectMethod] == EDiffuseIndirectMethod::Lumen && LumenFinalGatherMethod == ELumenFinalGatherMethod::ScreenProbeGather)
				|| ViewPipelineState[&FPerViewPipelineState::ReflectionsMethod] == EReflectionsMethod::Lumen;

			const bool bHasFirstPersonSelfShadow = ShouldRenderFirstPersonSelfShadow(ViewFamily);

			// Requires FurthestHZB
			ViewPipelineState.Set(&FPerViewPipelineState::bFurthestHZB,
				FamilyPipelineState[&FFamilyPipelineState::bHZBOcclusion] ||
				FamilyPipelineState[&FFamilyPipelineState::bNanite] ||
				ViewPipelineState[&FPerViewPipelineState::AmbientOcclusionMethod] == EAmbientOcclusionMethod::SSAO ||
				ViewPipelineState[&FPerViewPipelineState::ReflectionsMethod] == EReflectionsMethod::SSR ||
				bHasSSGI || bUseLumen);

			ViewPipelineState.Set(&FPerViewPipelineState::bClosestHZB, 
				bHasSSGI || bLumenNeedsClosestHZB || bHasFirstPersonSelfShadow || MegaLights::IsUsingClosestHZB(ViewFamily));
		}
	}

	// Commit all the pipeline states.
	{
		for (int32 ViewIndex = 0; ViewIndex < AllViews.Num(); ViewIndex++)
		{
			const FViewInfo& View = *AllViews[ViewIndex];

			GetViewPipelineStateWritable(View).Commit();
		}
		FamilyPipelineState.Commit();
	}
}

void FDeferredShadingSceneRenderer::RenderNanite(FRDGBuilder& GraphBuilder, const TArray<FViewInfo>& InViews, FSceneTextures& SceneTextures, bool bIsEarlyDepthComplete,
	FNaniteBasePassVisibility& InNaniteBasePassVisibility,
	TArray<Nanite::FRasterResults, TInlineAllocator<2>>& NaniteRasterResults,
	TArray<Nanite::FPackedView, SceneRenderingAllocator>& PrimaryNaniteViews,
	FRDGTextureRef FirstStageDepthBuffer)
{
	LLM_SCOPE_BYTAG(Nanite);
	TRACE_CPUPROFILER_EVENT_SCOPE(InitNaniteRaster);

	NaniteRasterResults.AddDefaulted(InViews.Num());
	if (InNaniteBasePassVisibility.Query != nullptr)
	{
		// For now we'll share the same visibility results across all views
		for (int32 ViewIndex = 0; ViewIndex < NaniteRasterResults.Num(); ++ViewIndex)
		{
			NaniteRasterResults[ViewIndex].VisibilityQuery = InNaniteBasePassVisibility.Query;
		}

#if STATS
		// Launch a setup task that will process stats when the visibility task completes.
		GraphBuilder.AddSetupTask([Query = InNaniteBasePassVisibility.Query]
		{
			const FNaniteVisibilityResults* VisibilityResults = Nanite::GetVisibilityResults(Query);

			uint32 TotalRasterBins = 0;
			uint32 VisibleRasterBins = 0;
			VisibilityResults->GetRasterBinStats(VisibleRasterBins, TotalRasterBins);

			uint32 TotalShadingBins = 0;
			uint32 VisibleShadingBins = 0;
			VisibilityResults->GetShadingBinStats(VisibleShadingBins, TotalShadingBins);

			SET_DWORD_STAT(STAT_NaniteBasePassTotalRasterBins, TotalRasterBins);
			SET_DWORD_STAT(STAT_NaniteBasePassVisibleRasterBins, VisibleRasterBins);

			SET_DWORD_STAT(STAT_NaniteBasePassTotalShadingBins, TotalShadingBins);
			SET_DWORD_STAT(STAT_NaniteBasePassVisibleShadingBins, VisibleShadingBins);

		}, Nanite::GetVisibilityTask(InNaniteBasePassVisibility.Query));
#endif
	}

	const FIntPoint RasterTextureSize = SceneTextures.Depth.Target->Desc.Extent;

	// Primary raster view
	{
		Nanite::FSharedContext SharedContext{};
		SharedContext.FeatureLevel = Scene->GetFeatureLevel();
		SharedContext.ShaderMap = GetGlobalShaderMap(SharedContext.FeatureLevel);
		SharedContext.Pipeline = Nanite::ERasterPipeline::Primary;

		FIntRect RasterTextureRect(0, 0, RasterTextureSize.X, RasterTextureSize.Y);
		if (InViews.Num() == 1)
		{
			const FViewInfo& View = InViews[0];
			if (View.ViewRect.Min.X == 0 && View.ViewRect.Min.Y == 0)
			{
				RasterTextureRect = View.ViewRect;
			}
		}

		Nanite::FRasterContext RasterContext;

		// Nanite::VisBuffer (Visibility Buffer Clear)
		{
			const FNaniteVisualizationData& VisualizationData = GetNaniteVisualizationData();

			bool bVisualizeActive = VisualizationData.IsActive() && ViewFamily.EngineShowFlags.VisualizeNanite;
			bool bVisualizeOverdraw = false;
			bool bEnableAssemblyMeta = false;
			if (bVisualizeActive)
			{
				const int32 ActiveMode = VisualizationData.GetActiveModeID();
				if (ActiveMode == 0) // Overview
				{
					bVisualizeOverdraw = VisualizationData.GetOverviewModeIDs().Contains(NANITE_VISUALIZE_OVERDRAW);
				}
				else
				{
					bVisualizeOverdraw = (ActiveMode == NANITE_VISUALIZE_OVERDRAW);
					bEnableAssemblyMeta = (ActiveMode == NANITE_VISUALIZE_ASSEMBLIES || ActiveMode == NANITE_VISUALIZE_PICKING);
				}
			}

			RDG_EVENT_SCOPE_STAT(GraphBuilder, NaniteVisBuffer, "Nanite::VisBuffer");

			const Nanite::FRasterContextInitParams RasterInitParams =
			{
				.TextureSize = RasterTextureSize,
				.TextureRect = RasterTextureRect,
				.RasterMode = Nanite::EOutputBufferMode::VisBuffer,
				.bVisualize = bVisualizeActive,
				.bVisualizeOverdraw = bVisualizeOverdraw,
				.bEnableAssemblyMeta = bEnableAssemblyMeta
			};
			RasterContext = Nanite::InitRasterContext(
				GraphBuilder,
				SharedContext,
				ViewFamily,
				RasterInitParams
			);
		}

		Nanite::FConfiguration CullingConfig = { 0 };
		CullingConfig.bTwoPassOcclusion = true;
		CullingConfig.bUpdateStreaming = true;
		CullingConfig.bPrimaryContext = true;
#if !UE_BUILD_SHIPPING
		CullingConfig.bDrawOnlyRayTracingFarField = CVarDebugDrawOnlyRTFarField.GetValueOnRenderThread();
#endif
		static FString EmptyFilterName; // Empty filter represents primary view.
		CullingConfig.bExtractStats = Nanite::IsStatFilterActive(EmptyFilterName);

		const bool bDrawSceneViewsInOneNanitePass = InViews.Num() > 1 && Nanite::ShouldDrawSceneViewsInOneNanitePass(InViews[0]);

		// creates one or more Nanite views (normally one per view unless drawing multiple views together - e.g. Stereo ISR views)
		auto CreateNaniteViews = [bDrawSceneViewsInOneNanitePass, &InViews, &PrimaryNaniteViews, &GraphBuilder](const FViewInfo& View, int32 ViewIndex, const FIntPoint& RasterTextureSize, FIntRect InHZBTestRect, float MaxPixelsPerEdgeMultipler, bool bUseCurrentAsPreviousForHZb, TArray<FConvexVolume> &OutViewsCullingVolumes) -> Nanite::FPackedViewArray*
		{
			Nanite::FPackedViewArray::ArrayType OutViews;

			// always add the primary view. In case of bDrawSceneViewsInOneNanitePass HZB is built from all views so using viewrects
			// to account for a rare case when the primary view doesn't start from 0, 0 (maybe can happen in splitscreen?)
			FIntRect HZBTestRect = bDrawSceneViewsInOneNanitePass ?
				View.PrevViewInfo.ViewRect :
				InHZBTestRect;

			Nanite::FPackedView PackedView = Nanite::CreatePackedViewFromViewInfo(
				View,
				RasterTextureSize,
				NANITE_VIEW_FLAG_HZBTEST | NANITE_VIEW_FLAG_NEAR_CLIP,
				/* StreamingPriorityCategory = */ 3,
				/* MinBoundsRadius = */ 0.0f,
				MaxPixelsPerEdgeMultipler,
				&HZBTestRect,
				bUseCurrentAsPreviousForHZb
			);
			OutViewsCullingVolumes.Add(View.ViewFrustum);
			OutViews.Add(PackedView);
			PrimaryNaniteViews.Add(PackedView);

			if (bDrawSceneViewsInOneNanitePass)
			{
				// All other views in the family will need to be rendered in one go, to cover both ISR and (later) split-screen
				for (int32 ViewIdx = 1, NumViews = InViews.Num(); ViewIdx < NumViews; ++ViewIdx)
				{
					const FViewInfo& SecondaryViewInfo = InViews[ViewIdx];

					/* viewport rect in HZB space. For instanced stereo passes HZB is built for all atlased views */
					FIntRect SecondaryHZBTestRect = SecondaryViewInfo.PrevViewInfo.ViewRect;
					Nanite::FPackedView SecondaryPackedView = Nanite::CreatePackedViewFromViewInfo(
						SecondaryViewInfo,
						RasterTextureSize,
						NANITE_VIEW_FLAG_HZBTEST | NANITE_VIEW_FLAG_NEAR_CLIP,
						/* StreamingPriorityCategory = */ 3,
						/* MinBoundsRadius = */ 0.0f,
						MaxPixelsPerEdgeMultipler,
						&SecondaryHZBTestRect
					);
					OutViewsCullingVolumes.Add(SecondaryViewInfo.ViewFrustum);
					OutViews.Add(SecondaryPackedView);
					PrimaryNaniteViews.Add(SecondaryPackedView);
				}
			}

			return Nanite::FPackedViewArray::Create(GraphBuilder, OutViews.Num(), MoveTemp(OutViews));
		};

		// in case of bDrawSceneViewsInOneNanitePass we only need one iteration
		uint32 ViewsToRender = (bDrawSceneViewsInOneNanitePass ? 1u : (uint32)InViews.Num());
		for (uint32 ViewIndex = 0; ViewIndex < ViewsToRender; ++ViewIndex)
		{
			Nanite::FRasterResults& RasterResults = NaniteRasterResults[ViewIndex];
			const FViewInfo& View = InViews[ViewIndex];
			// We don't check View.ShouldRenderView() since this is already taken care of by bDrawSceneViewsInOneNanitePass.
			// If bDrawSceneViewsInOneNanitePass is false, we need to render the secondary view even if ShouldRenderView() is false
			// NOTE: Except when there are no primitives to draw for the view
			if (View.bHasNoVisiblePrimitive)
			{
				continue;
			}
			
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1 && !bDrawSceneViewsInOneNanitePass, "View%u", ViewIndex);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1 && bDrawSceneViewsInOneNanitePass, "View%u (together with %d more)", ViewIndex, InViews.Num() - 1);

			FIntRect ViewRect = bDrawSceneViewsInOneNanitePass ? FIntRect(0, 0, FamilySize.X, FamilySize.Y) : GetHZBViewRect(View);
			CullingConfig.SetViewFlags(View);

			float LODScaleFactor = 1.0f;
			if (View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale &&
				CVarNaniteViewMeshLODBiasEnable.GetValueOnRenderThread() != 0)
			{
				float TemporalUpscaleFactor = ViewRect.Width() ? float(View.GetSecondaryViewRectSize().X) / float(ViewRect.Width()) : 1.0f;

				LODScaleFactor = TemporalUpscaleFactor * FMath::Exp2(-CVarNaniteViewMeshLODBiasOffset.GetValueOnRenderThread());
				LODScaleFactor = FMath::Min(LODScaleFactor, FMath::Exp2(-CVarNaniteViewMeshLODBiasMin.GetValueOnRenderThread()));
			}

			float MaxPixelsPerEdgeMultipler = 1.0f / LODScaleFactor;

			float QualityScale = Nanite::GStreamingManager.GetQualityScaleFactor();
			if (GDynamicNaniteScalingPrimary.GetSettings().IsEnabled())
			{
				QualityScale = FMath::Min(QualityScale, DynamicResolutionFractions[GDynamicNaniteScalingPrimary]);
			}
			MaxPixelsPerEdgeMultipler /= QualityScale;

			TRefCountPtr<IPooledRenderTarget> HZBToUseResource = !bIsEarlyDepthComplete ? View.PrevViewInfo.NaniteHZB : View.PrevViewInfo.HZB;

			FRDGTextureRef HZBToUse = HZBToUseResource.IsValid() ? GraphBuilder.RegisterExternalTexture(HZBToUseResource) : nullptr;

			// We don't support he multi-view render here.
			const bool bRenderNanitePrimeHZBPass = !bDrawSceneViewsInOneNanitePass
			 && ((CVarNanitePrimeHZBMode.GetValueOnRenderThread() != 0 && HZBToUse == nullptr) || CVarNanitePrimeHZBMode.GetValueOnRenderThread() == 2);

			FIntRect HZBTestRect = FIntRect(0, 0, View.PrevViewInfo.ViewRect.Width(), View.PrevViewInfo.ViewRect.Height());

			// Draw extra low detail pass to prime the HZB
			if (bRenderNanitePrimeHZBPass)
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, NanitePrimeHZB, "Nanite::PrimeHZB");

				// 1. Figure out the correct down-sampled view rect.
				uint32 RenderSizeBias = 1u + FMath::Max(0, CVarNanitePrimeHZBRenderSizeBias.GetValueOnRenderThread());
				FIntPoint HZBRenderSize = FIntPoint(
					FMath::Max(1, View.ViewRect.Width() >> RenderSizeBias),
					FMath::Max(1, View.ViewRect.Height() >> RenderSizeBias));

				const Nanite::FRasterContextInitParams RasterInitParams =
				{
					.TextureSize = HZBRenderSize,
					.TextureRect = RasterTextureRect,
					.RasterMode = Nanite::EOutputBufferMode::DepthOnly
				};
				Nanite::FRasterContext HzbRasterContext = Nanite::InitRasterContext(GraphBuilder, SharedContext, ViewFamily, RasterInitParams);
				Nanite::FConfiguration HzbCullingConfig = { 0 };
				HzbCullingConfig.bTwoPassOcclusion = false;
				HzbCullingConfig.bPrimaryContext = false;
				HzbCullingConfig.bDrawOnlyRayTracingFarField = CVarNanitePrimeHZBOnlyRTFarField.GetValueOnRenderThread();


				FIntRect HZBRenderViewRect = FIntRect(0, 0, HZBRenderSize.X, HZBRenderSize.Y);
				
				// Set up the HZB test rect for nanite render.
				HZBTestRect = HZBRenderViewRect * 2;

				Nanite::FPackedViewParams PackedViewParams = Nanite::CreateViewParamsFromViewInfo(
					View,
					HZBRenderSize,
					NANITE_VIEW_FLAG_HZBTEST | NANITE_VIEW_FLAG_NEAR_CLIP,
					/* StreamingPriorityCategory = */ 3,
					/* MinBoundsRadius = */ 0.0f,
					MaxPixelsPerEdgeMultipler * CVarNanitePrimeHZBMPPE.GetValueOnRenderThread(),
					&HZBRenderViewRect);

				PackedViewParams.ViewRect = HZBRenderViewRect;
				PackedViewParams.RasterContextSize = HZBRenderSize;

				Nanite::FPackedViewArray* NaniteViewsToRenderHZB = Nanite::FPackedViewArray::Create(GraphBuilder, Nanite::CreatePackedView(PackedViewParams));
				// TODO: there's really no need for a separate query, it also doesn't really do anything anymore for the broad cases, just grabs all the chunks.
				TArray<FConvexVolume> ViewsToRenderCullingVolumesHZB;
				ViewsToRenderCullingVolumesHZB.Add(View.ViewFrustum);
				FSceneInstanceCullingQuery* SceneInstanceCullQueryHZB = GetSceneExtensionsRenderers().GetRenderer<FSceneCullingRenderer>().CullInstances(GraphBuilder, ViewsToRenderCullingVolumesHZB);

				TUniquePtr< Nanite::IRenderer > NaniteHLODRenderer = Nanite::IRenderer::Create(
					GraphBuilder,
					*Scene,
					View,
					GetSceneUniforms(),
					SharedContext,
					HzbRasterContext,
					HzbCullingConfig,
					HZBRenderViewRect,
					nullptr
				);

				NaniteHLODRenderer->DrawGeometry(
					Scene->NaniteRasterPipelines[ENaniteMeshPass::BasePass],
					nullptr,
					*NaniteViewsToRenderHZB,
					SceneInstanceCullQueryHZB
				);

				FRDGTextureRef FurthestHZBTexture;
				BuildHZBFurthest(
					GraphBuilder,
					(SceneTextures.Depth.Resolve && CVarNanitePrimeHZBSampleNonNanite.GetValueOnRenderThread()) 
						? SceneTextures.Depth.Resolve
						: GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy),
					HzbRasterContext.DepthBuffer,
					HZBRenderViewRect,
					FeatureLevel,
					ShaderPlatform,
					TEXT("NanitePrimedHZB"),
					/* OutFurthestHZBTexture = */ &FurthestHZBTexture,
					BuildHZBDefaultPixelFormat,
					nullptr,
					// Configure HZB build to use bias & not rescale the base level.
					{ View.InvDeviceZToWorldZTransform, CVarNanitePrimeHZBDepthBias.GetValueOnRenderThread(), true });

					HZBToUse = FurthestHZBTexture;
				
			}

			TArray<FConvexVolume> ViewsToRenderCullingVolumes;
			Nanite::FPackedViewArray* NaniteViewsToRender = CreateNaniteViews(View, ViewIndex, RasterTextureSize, HZBTestRect, MaxPixelsPerEdgeMultipler, bRenderNanitePrimeHZBPass, ViewsToRenderCullingVolumes);

			TUniquePtr< Nanite::IRenderer > NaniteRenderer;

			// Nanite::VisBuffer (Culling and Rasterization)
			{
				DynamicRenderScaling::FRDGScope DynamicScalingScope(GraphBuilder, GDynamicNaniteScalingPrimary);

				RDG_EVENT_SCOPE_STAT(GraphBuilder, NaniteVisBuffer, "Nanite::VisBuffer");

				NaniteRenderer = Nanite::IRenderer::Create(
					GraphBuilder,
					*Scene,
					View,
					GetSceneUniforms(),
					SharedContext,
					RasterContext,
					CullingConfig,
					ViewRect,
					HZBToUse
				);

				FSceneInstanceCullingQuery* SceneInstanceCullQuery = GetSceneExtensionsRenderers().GetRenderer<FSceneCullingRenderer>().CullInstances(GraphBuilder, ViewsToRenderCullingVolumes);
				NaniteRenderer->DrawGeometry(
					Scene->NaniteRasterPipelines[ENaniteMeshPass::BasePass],
					RasterResults.VisibilityQuery,
					*NaniteViewsToRender,
					SceneInstanceCullQuery
				);

				NaniteRenderer->ExtractResults( RasterResults );
			}

			// Nanite::BasePass (Depth Pre-Pass and HZB Build)
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, NaniteBasePass, "NaniteBasePass");

				// Emit velocity with depth if not writing it in base pass.
				FRDGTexture* VelocityBuffer = !IsUsingBasePassVelocity(ShaderPlatform) ? SceneTextures.Velocity : nullptr;

				Nanite::EmitDepthTargets(
					GraphBuilder,
					*Scene,
					InViews[ViewIndex],
					bDrawSceneViewsInOneNanitePass,
					RasterResults,
					SceneTextures.Depth.Target,
					VelocityBuffer,
					FirstStageDepthBuffer
				);
				
				// Sanity check (always force Z prepass)
				check(bIsEarlyDepthComplete);
			}
		}
	}
}

#if RHI_RAYTRACING
extern void RenderRayTracingDebug(
	FRDGBuilder& GraphBuilder,
	const FScene& Scene,
	const FViewInfo& View,
	FSceneTextures& SceneTextures,
	TConstArrayView<FRayTracingShaderBindingData> VisibleRayTracingShaderBindings,
	FRayTracingPickingFeedback& PickingFeedback);
extern void RayTracingDebugDisplayOnScreenMessages(FScreenMessageWriter& Writer, const FViewInfo& View, bool bViewHasFarFieldInstances);
#endif // RHI_RAYTRACING

void FDeferredShadingSceneRenderer::Render(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderOther);

	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_Render, FColor::Emerald);

	for (FViewInfo& View : Views)
	{
		if (View.ViewState != nullptr)
		{
			if (View.ViewState->Scene == nullptr)
			{
				// link view state to the scene
				View.ViewState->Scene = Scene;
				Scene->ViewStates.Add(View.ViewState);
			}
		}
	}

	{
		FRayTracingVisualizationData& RayTracingVisualizationData = GetRayTracingVisualizationData();

		if (RayTracingVisualizationData.HasOverrides())
		{
			// When activating the view modes from the command line, automatically enable the RayTracingDebug show flag for convenience.
			ViewFamily.EngineShowFlags.SetRayTracingDebug(true);
		}
	}

	// If this is scene capture rendering depth pre-pass, we'll take the shortcut function RenderSceneCaptureDepth if optimization switch is on.
	const ERendererOutput RendererOutput = GetRendererOutput();

	const bool bNaniteEnabled = ShouldRenderNanite();
	const bool bHasRayTracedOverlay = HasRayTracedOverlay(ViewFamily);

#if !UE_BUILD_SHIPPING
	RenderCaptureInterface::FScopedCapture RenderCapture(GCaptureNextDeferredShadingRendererFrame-- == 0, GraphBuilder, TEXT("DeferredShadingSceneRenderer"));
	// Prevent overflow every 2B frames.
	GCaptureNextDeferredShadingRendererFrame = FMath::Max(-1, GCaptureNextDeferredShadingRendererFrame);
#endif

	GPU_MESSAGE_SCOPE(GraphBuilder);

#if RHI_RAYTRACING
	if (SceneUpdateInputs)
	{
		RHI_BREADCRUMB_EVENT_STAT(GraphBuilder.RHICmdList, RayTracingGeometry, "RayTracingGeometry");

		GRayTracingGeometryManager->Update(GraphBuilder.RHICmdList);

		// TODO: should only process build requests once per frame
		GRayTracingGeometryManager->ProcessBuildRequests(GraphBuilder.RHICmdList);
	}

	FRayTracingShaderBindingTable& RayTracingSBT = Scene->RayTracingSBT;
	FRayTracingScene& RayTracingScene = Scene->RayTracingScene;
	RayTracingSBT.ResetMissAndCallableShaders();

	for (FViewInfo& View : Views)
	{
		if (IStereoRendering::IsStereoEyeView(View) && IStereoRendering::IsASecondaryView(View))
		{
			continue;
		}

		View.SetRayTracingSceneViewHandle(RayTracingScene.AddView(View.GetViewKey()));
		RayTracingScene.SetViewParams(View.GetRayTracingSceneViewHandle(), View.ViewMatrices, View.RayTracingCullingParameters);
	}
#endif // RHI_RAYTRACING

	FInitViewTaskDatas InitViewTaskDatas = OnRenderBegin(GraphBuilder, SceneUpdateInputs);

#if RHI_RAYTRACING
	if (RendererOutput == FSceneRenderer::ERendererOutput::FinalSceneColor && FamilyPipelineState[&FFamilyPipelineState::bRayTracing])
	{
		GRayTracingGeometryManager->PreRender();
	}
#endif // RHI_RAYTRACING

	FUpdateExposureCompensationCurveLUTTaskData UpdateExposureCompensationCurveLUTTaskData;
	BeginUpdateExposureCompensationCurveLUT(Views, &UpdateExposureCompensationCurveLUTTaskData);

	FRDGExternalAccessQueue ExternalAccessQueue;
	TUniquePtr<FVirtualTextureUpdater> VirtualTextureUpdater;
	FLumenSceneFrameTemporaries LumenFrameTemporaries(Views);

	FGPUSceneScopeBeginEndHelper GPUSceneScopeBeginEndHelper(GraphBuilder, Scene->GPUScene, GPUSceneDynamicContext);

	const bool bUseVirtualTexturing = UseVirtualTexturing(ShaderPlatform);

	// Virtual texturing isn't needed for depth prepass
	if (bUseVirtualTexturing && RendererOutput != ERendererOutput::DepthPrepassOnly)
	{
		FVirtualTextureUpdateSettings Settings(ViewFamily);
		VirtualTextureUpdater = FVirtualTextureSystem::Get().BeginUpdate(GraphBuilder, FeatureLevel, this, Settings);
		VirtualTextureFeedbackBegin(GraphBuilder, Views, GetActiveSceneTexturesConfig().Extent);
	}

	if (SceneUpdateInputs)
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CommitFinalPipelineState);
			for (FSceneRenderer* Renderer : SceneUpdateInputs->Renderers)
			{
				// Compute & commit the final state of the entire dependency topology of the renderer.
				static_cast<FDeferredShadingSceneRenderer*>(Renderer)->CommitFinalPipelineState();
			}
		}

		// Initialize global system textures (pass-through if already initialized).
		GSystemTextures.InitializeTextures(GraphBuilder.RHICmdList, FeatureLevel);
	}

	UE::Tasks::TTask<void> UpdateLightFunctionAtlasTask;
	if (LightFunctionAtlas.IsLightFunctionAtlasEnabled())
	{
		UpdateLightFunctionAtlasTask = LaunchSceneRenderTask<void>(TEXT("UpdateLightFunctionAtlas"), [this]
			{
				UpdateLightFunctionAtlasTaskFunction();
			}, UE::Tasks::FTask());
	}

	FShadowSceneRenderer& ShadowSceneRenderer = GetSceneExtensionsRenderers().GetRenderer<FShadowSceneRenderer>();
	{
		// 1. Update global distance field view origin
		// This needs to be done prior to start Lumen scene lighting to ensure GlobalDistanceFieldData->CameraVelocityOffset is updated
		if (SceneUpdateInputs)
		{
			UpdateGlobalDistanceFieldViewOrigin(*SceneUpdateInputs);
		}

		if (RendererOutput == ERendererOutput::FinalSceneColor)
		{
			// 2. Update lumen scene

			InitViewTaskDatas.LumenFrameTemporaries = &LumenFrameTemporaries;
	
			// Important that this uses consistent logic throughout the frame, so evaluate once and pass in the flag from here
			// NOTE: Must be done after  system texture initialization
			const bool bEnableVirtualShadowMaps = UseVirtualShadowMaps(ShaderPlatform, FeatureLevel) && ViewFamily.EngineShowFlags.DynamicShadows && !bHasRayTracedOverlay;
			VirtualShadowMapArray.Initialize(GraphBuilder, Scene->GetVirtualShadowMapCache(), bEnableVirtualShadowMaps, ViewFamily.EngineShowFlags);
	
			if (InitViewTaskDatas.LumenFrameTemporaries)
			{
				BeginUpdateLumenSceneTasks(GraphBuilder, *InitViewTaskDatas.LumenFrameTemporaries);
			}
	
			BeginGatherLumenLights(*InitViewTaskDatas.LumenFrameTemporaries, InitViewTaskDatas.LumenDirectLighting, InitViewTaskDatas.VisibilityTaskData, UpdateLightFunctionAtlasTask);
		}

		if (bNaniteEnabled)
		{
			TArray<FConvexVolume, TInlineAllocator<2>> NaniteCullingViews;

			// For now we'll share the same visibility results across all views
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				FViewInfo& View = Views[ViewIndex];
				NaniteCullingViews.Add(View.ViewFrustum);
			}

			FNaniteVisibility& NaniteVisibility = Scene->NaniteVisibility[ENaniteMeshPass::BasePass];
			const FNaniteRasterPipelines&  NaniteRasterPipelines  = Scene->NaniteRasterPipelines[ENaniteMeshPass::BasePass];
			const FNaniteShadingPipelines& NaniteShadingPipelines = Scene->NaniteShadingPipelines[ENaniteMeshPass::BasePass];

			NaniteVisibility.BeginVisibilityFrame();

			NaniteBasePassVisibility.Visibility = &NaniteVisibility;
			NaniteBasePassVisibility.Query = NaniteVisibility.BeginVisibilityQuery(
				Allocator,
				*Scene,
				NaniteCullingViews,
				&NaniteRasterPipelines,
				&NaniteShadingPipelines,
				InitViewTaskDatas.VisibilityTaskData->GetComputeRelevanceTask()
			);
		}
	}
	ShaderPrint::BeginViews(GraphBuilder, Views);

	ON_SCOPE_EXIT
	{
		ShaderPrint::EndViews(Views);
	};

	GetSceneExtensionsRenderers().PreInitViews(GraphBuilder);

	if (SceneUpdateInputs)
	{
		PrepareDistanceFieldScene(GraphBuilder, ExternalAccessQueue, *SceneUpdateInputs);
	}

	if (RendererOutput == ERendererOutput::FinalSceneColor)
	{
		ShadingEnergyConservation::Init(GraphBuilder, Views);
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			FGlintShadingLUTsStateData::Init(GraphBuilder, View);
		}

#if RHI_RAYTRACING
		if (FamilyPipelineState[&FFamilyPipelineState::bRayTracing])
		{
			InitViewTaskDatas.RayTracingGatherInstances = RayTracing::CreateGatherInstancesTaskData(Allocator, *Scene, Views.Num());

			for (FViewInfo& View : Views)
			{
				const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);

				RayTracing::AddView(*InitViewTaskDatas.RayTracingGatherInstances, View, ViewPipelineState.DiffuseIndirectMethod, ViewPipelineState.ReflectionsMethod);
			}

			RayTracing::BeginGatherInstances(*InitViewTaskDatas.RayTracingGatherInstances, InitViewTaskDatas.VisibilityTaskData->GetFrustumCullTask());
		}
#endif
	}

	UE::SVT::GetStreamingManager().BeginAsyncUpdate(GraphBuilder);

	bool bVisualizeNanite = false;
	if (bNaniteEnabled)
	{
		Nanite::GGlobalResources.Update(GraphBuilder);
		Nanite::GStreamingManager.BeginAsyncUpdate(GraphBuilder);

		FNaniteVisualizationData& NaniteVisualization = GetNaniteVisualizationData();
		if (Views.Num() > 0)
		{
			FName NaniteViewMode = Views[0].CurrentNaniteVisualizationMode;
			
			EDebugViewShaderMode DebugViewShaderMode = ViewFamily.GetDebugViewShaderMode();
			if (DebugViewShaderMode == DVSM_ShadowCasters)
			{
				NaniteViewMode = FName("ShadowCasters");
				ViewFamily.EngineShowFlags.SetVisualizeNanite(true);
			}

			if (NaniteVisualization.Update(NaniteViewMode))
			{
				// When activating the view modes from the command line, automatically enable the VisualizeNanite show flag for convenience.
				ViewFamily.EngineShowFlags.SetVisualizeNanite(true);
			}

			bVisualizeNanite = NaniteVisualization.IsActive() && ViewFamily.EngineShowFlags.VisualizeNanite;
		}
	}

#if WITH_MGPU
	ComputeGPUMasks(&GraphBuilder.RHICmdList);
#endif // WITH_MGPU

	// By default, limit our GPU usage to only GPUs specified in the view masks.
	RDG_GPU_MASK_SCOPE(GraphBuilder, ViewFamily.EngineShowFlags.PathTracing ? FRHIGPUMask::All() : AllViewsGPUMask);
	RDG_EVENT_SCOPE(GraphBuilder, "Scene");
	
	if (RendererOutput == ERendererOutput::FinalSceneColor)
	{
		SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_Render_Init);
		RDG_RHI_EVENT_SCOPE_STAT(GraphBuilder, AllocateRendertargets, "AllocateRendertargets");

		// Force the subsurface profiles and specular profiles textures to be updated.
		SubsurfaceProfile::UpdateSubsurfaceProfileTexture(GraphBuilder, ShaderPlatform);
		SpecularProfile::UpdateSpecularProfileTextureAtlas(GraphBuilder, ShaderPlatform);
		ToonProfile::UpdateToonProfileTextureAtlas(GraphBuilder, ShaderPlatform);

		// Force the rect light texture & IES texture to be updated.
		RectLightAtlas::UpdateAtlasTexture(GraphBuilder, FeatureLevel);
		IESAtlas::UpdateAtlasTexture(GraphBuilder, ShaderPlatform);
	}

	FSceneTexturesConfig& SceneTexturesConfig = GetActiveSceneTexturesConfig();
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Create(GraphBuilder);

	const bool bAllowStaticLighting = !bHasRayTracedOverlay && IsStaticLightingAllowed();

	// if DDM_AllOpaqueNoVelocity was used, then velocity should have already been rendered as well
	const bool bIsEarlyDepthComplete = (DepthPass.EarlyZPassMode == DDM_AllOpaque || DepthPass.EarlyZPassMode == DDM_AllOpaqueNoVelocity);

	// Use read-only depth in the base pass if we have a full depth prepass.
	const bool bAllowReadOnlyDepthBasePass = bIsEarlyDepthComplete
		&& !ViewFamily.EngineShowFlags.ShaderComplexity
		&& !ViewFamily.UseDebugViewPS()
		&& !ViewFamily.EngineShowFlags.Wireframe
		&& !ViewFamily.EngineShowFlags.LightMapDensity;

	const FExclusiveDepthStencil::Type BasePassDepthStencilAccess =
		bAllowReadOnlyDepthBasePass
		? FExclusiveDepthStencil::DepthRead_StencilWrite
		: FExclusiveDepthStencil::DepthWrite_StencilWrite;

	FRendererViewDataManager& ViewDataManager = *GraphBuilder.AllocObject<FRendererViewDataManager>(GraphBuilder, *Scene, GetSceneUniforms(), AllViews);
	FInstanceCullingManager& InstanceCullingManager = *GraphBuilder.AllocObject<FInstanceCullingManager>(GraphBuilder, *Scene, GetSceneUniforms(), ViewDataManager);

	FSceneTextures::InitializeViewFamily(GraphBuilder, ViewFamily, FamilySize);
	FSceneTextures& SceneTextures = GetActiveSceneTextures();

	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, VisibilityCommands, "VisibilityCommands");
		BeginInitViews(GraphBuilder, SceneTexturesConfig, InstanceCullingManager, ExternalAccessQueue, InitViewTaskDatas);
	}

#if !UE_BUILD_SHIPPING
	if (CVarStallInitViews.GetValueOnRenderThread() > 0.0f)
	{
		SCOPE_CYCLE_COUNTER(STAT_InitViews_Intentional_Stall);
		FPlatformProcess::Sleep(CVarStallInitViews.GetValueOnRenderThread() / 1000.0f);
	}
#endif

	extern TSet<IPersistentViewUniformBufferExtension*> PersistentViewUniformBufferExtensions;

	for (IPersistentViewUniformBufferExtension* Extension : PersistentViewUniformBufferExtensions)
	{
		Extension->BeginFrame();

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			// Must happen before RHI thread flush so any tasks we dispatch here can land in the idle gap during the flush
			Extension->PrepareView(&Views[ViewIndex]);
		}
	}

	if (RendererOutput == ERendererOutput::FinalSceneColor)
	{
		// Prepare the scene for rendering this frame.

#if RHI_RAYTRACING
		if (ViewFamily.EngineShowFlags.PathTracing)
		{
			if (ShouldPrepareRayTracingDecals(*Scene, ViewFamily))
			{
				// Calculate decal grid for ray tracing per view since decal fade is view dependent
				// TODO: investigate reusing the same grid for all views (ie: different callable shader SBT entries for each view so fade alpha is still correct for each view)

				for (FViewInfo& View : Views)
				{
					View.RayTracingDecalUniformBuffer = CreateRayTracingDecalData(GraphBuilder, *Scene, View, RayTracingSBT.NumCallableShaderSlots);
					View.bHasRayTracingDecals = true;
					RayTracingSBT.NumCallableShaderSlots += Scene->Decals.Num();
				}
			}
			else
			{
				TRDGUniformBufferRef<FRayTracingDecals> NullRayTracingDecalUniformBuffer = CreateNullRayTracingDecalsUniformBuffer(GraphBuilder);

				for (FViewInfo& View : Views)
				{
					View.RayTracingDecalUniformBuffer = NullRayTracingDecalUniformBuffer;
					View.bHasRayTracingDecals = false;
				}
			}

			// If we might be path tracing the clouds -- call the path tracer's method for cloud callable shader setup
			// this will skip work if cloud rendering is not being used
			PreparePathTracingCloudMaterial(GraphBuilder, Scene, Views);
		}

		if (IsRayTracingEnabled(ViewFamily.GetShaderPlatform()) && ShouldCompileRayTracingShadersForProject(ViewFamily.GetShaderPlatform()))
		{
			if (!ViewFamily.EngineShowFlags.PathTracing)
			{
				// get the default lighting miss shader (to implicitly fill in the MissShader library before the RT pipeline is created)
				GetRayTracingLightingMissShader(GetGlobalShaderMap(FeatureLevel));
				RayTracingSBT.NumMissShaderSlots++;
			}

			if (ViewFamily.EngineShowFlags.LightFunctions)
			{
				// gather all the light functions that may be used (and also count how many miss shaders we will need)
				FRayTracingLightFunctionMap RayTracingLightFunctionMap;
				if (ViewFamily.EngineShowFlags.PathTracing)
				{
					RayTracingLightFunctionMap = GatherLightFunctionLightsPathTracing(Scene, ViewFamily.EngineShowFlags, FeatureLevel);
				}
				else
				{
					RayTracingLightFunctionMap = GatherLightFunctionLights(Scene, ViewFamily.EngineShowFlags, FeatureLevel);
				}
				if (!RayTracingLightFunctionMap.IsEmpty())
				{
					// If we got some light functions in our map, store them in the RDG blackboard so downstream functions can use them.
					// The map itself will be strictly read-only from this point on.
					GraphBuilder.Blackboard.Create<FRayTracingLightFunctionMap>(MoveTemp(RayTracingLightFunctionMap));
				}
			}
		}
#endif // RHI_RAYTRACING

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		Scene->DebugRender(Views);
#endif
	}

	InitViewTaskDatas.VisibilityTaskData->FinishGatherDynamicMeshElements(BasePassDepthStencilAccess, InstanceCullingManager, VirtualTextureUpdater.Get());

	// Notify the FX system that the scene is about to be rendered.
	// TODO: These should probably be moved to scene extensions
	if (FXSystem && Views.IsValidIndex(0))
	{
		SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_FXSystem_PreRender);
		const bool bAllowGPUParticleUpdate = IsHeadLink();
		FXSystem->PreRender(GraphBuilder, GetSceneViews(), GetSceneUniforms(), bAllowGPUParticleUpdate);
		if (FGPUSortManager* GPUSortManager = FXSystem->GetGPUSortManager())
		{
			GPUSortManager->OnPreRender(GraphBuilder);
		}
	}

	{
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, UpdateGPUScene);
		RDG_EVENT_SCOPE_STAT(GraphBuilder, GPUSceneUpdate, "GPUSceneUpdate");

		for (int32 ViewIndex = 0; ViewIndex < AllViews.Num(); ViewIndex++)
		{
			FViewInfo& View = *AllViews[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, View);
			Scene->GPUScene.DebugRender(GraphBuilder, GetSceneUniforms(), View);
		}

		// Must be called after all views have flushed the dynamic primitives.
		ViewDataManager.InitInstanceState(GraphBuilder);

		if (Views.Num() > 0)
		{
			FViewInfo& View = Views[0];
			Scene->UpdatePhysicsField(GraphBuilder, View);
		}
	}


	if (FSceneCullingRenderer* SceneCullingRenderer = GetSceneExtensionsRenderers().GetRendererPtr<FSceneCullingRenderer>())
	{
		SceneCullingRenderer->DebugRender(GraphBuilder, Views);
	}

	//SceneCullingInfo.SceneUniformBuffer = GetSceneUniforms().GetBuffer(GraphBuilder);
	GetSceneExtensionsRenderers().UpdateViewData(GraphBuilder, ViewDataManager);

	// Allow scene extensions to affect the scene uniform buffer after GPU scene has fully updated
	GetSceneExtensionsRenderers().UpdateSceneUniformBuffer(GraphBuilder, GetSceneUniforms());

	// Must happen after visibility state & scene UB has been updated.
	InstanceCullingManager.BeginDeferredCulling(GraphBuilder);

	// Separate fog composition can only happen when deferred fog is enabled.
	const bool bShouldRenderFogUsingSeparateComposition = ShouldRenderFogUsingSeparateComposition(Scene) && ShouldRenderDeferredFog(Scene) && ShouldRenderFog(ViewFamily);

	const bool bUseGBuffer = IsUsingGBuffers(ShaderPlatform);
	const bool bShouldRenderVolumetricFog = ShouldRenderVolumetricFog();
	const bool bShouldRenderLocalFogVolume = ShouldRenderLocalFogVolume(Scene, ViewFamily);
	const bool bShouldRenderLocalFogVolumeDuringHeightFogPass = ShouldRenderLocalFogVolumeDuringHeightFogPass(Scene, ViewFamily) || bShouldRenderFogUsingSeparateComposition;
	const bool bShouldRenderLocalFogVolumeInVolumetricFog = ShouldRenderLocalFogVolumeInVolumetricFog(Scene, ViewFamily, bShouldRenderLocalFogVolume);
	const bool bShouldRenderLocalFogVolumeVisualizationPass = ShouldRenderLocalFogVolumeVisualizationPass(Scene, ViewFamily);
	const bool bFogComposeLocalFogVolumes = (bShouldRenderLocalFogVolumeInVolumetricFog && bShouldRenderVolumetricFog) || bShouldRenderLocalFogVolumeDuringHeightFogPass;
	bool bHeightFogHasComposedLocalFogVolume = false;

	const bool bRenderDeferredLighting = ViewFamily.EngineShowFlags.Lighting
		&& FeatureLevel >= ERHIFeatureLevel::SM5
		&& ViewFamily.EngineShowFlags.DeferredLighting
		&& bUseGBuffer
		&& !bHasRayTracedOverlay;

	bool bAnyLumenEnabled = false;

	// Virtual texturing isn't needed for depth prepass
	if (bUseVirtualTexturing && RendererOutput != ERendererOutput::DepthPrepassOnly)
	{
		// Note, should happen after the GPU-Scene update to ensure rendering to runtime virtual textures is using the correctly updated scene
		FVirtualTextureSystem::Get().EndUpdate(GraphBuilder, MoveTemp(VirtualTextureUpdater), FeatureLevel);
	}

	FMaterialCacheTagProvider::Get().Update(GraphBuilder);

	UE::Tasks::TTask<FSortedLightSetSceneInfo*> GatherAndSortLightsTask;

	if (RendererOutput == ERendererOutput::FinalSceneColor)
	{
#if RHI_RAYTRACING
		if (FamilyPipelineState[&FFamilyPipelineState::bRayTracing])
		{
			RayTracing::FinishGatherInstances(
				GraphBuilder,
				*InitViewTaskDatas.RayTracingGatherInstances,
				RayTracingScene,
				RayTracingSBT);
		}
#endif // RHI_RAYTRACING

		if (!bHasRayTracedOverlay)
		{
			for (const FViewInfo& View : Views)
			{
				bAnyLumenEnabled = bAnyLumenEnabled
					|| GetViewPipelineState(View).DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen
					|| GetViewPipelineState(View).ReflectionsMethod == EReflectionsMethod::Lumen;
			}
		}

		{
			extern bool IsVSMOnePassProjectionEnabled(const FEngineShowFlags& ShowFlags);
			extern UE::Tasks::FTask GetGatherAndSortLightsPrerequisiteTask(const FDynamicShadowsTaskData* TaskData);

			auto* SortedLightSet = GraphBuilder.AllocObject<FSortedLightSetSceneInfo>();
			const bool bShadowedLightsInClustered = ShouldUseClusteredDeferredShading(ViewFamily.GetShaderPlatform())
				&& IsVSMOnePassProjectionEnabled(ViewFamily.EngineShowFlags)
				&& VirtualShadowMapArray.IsEnabled();

			TArray<UE::Tasks::FTask, TInlineAllocator<2>> IssuedTasksCompletionEvents;
			IssuedTasksCompletionEvents.Add(GetGatherAndSortLightsPrerequisiteTask(InitViewTaskDatas.DynamicShadows));
			IssuedTasksCompletionEvents.Add(UpdateLightFunctionAtlasTask);

			GatherAndSortLightsTask = LaunchSceneRenderTask<FSortedLightSetSceneInfo*>(UE_SOURCE_LOCATION, [this, SortedLightSet, bShadowedLightsInClustered]
			{
				GatherAndSortLights(*SortedLightSet, bShadowedLightsInClustered);
				return SortedLightSet;
			}, IssuedTasksCompletionEvents);
		}
	}

	// force using occ queries for wireframe if rendering is parented or frozen in the first view
	check(Views.Num());
	#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
		const bool bIsViewFrozen = false;
	#else
		const bool bIsViewFrozen = Views[0].State && ((FSceneViewState*)Views[0].State)->bIsFrozen;
	#endif

	
	const bool bIsOcclusionTesting = DoOcclusionQueries()
		&& (!ViewFamily.EngineShowFlags.Wireframe || bIsViewFrozen);
	const bool bNeedsPrePass = ShouldRenderPrePass();

	// Sanity check - Note: Nanite forces a Z prepass in ShouldForceFullDepthPass()
	check(!UseNanite(ShaderPlatform) || bNeedsPrePass);

	GetSceneExtensionsRenderers().PreRender(GraphBuilder);
	GEngine->GetPreRenderDelegateEx().Broadcast(GraphBuilder);

	if (DepthPass.IsComputeStencilDitherEnabled())
	{
		AddDitheredStencilFillPass(GraphBuilder, Views, SceneTextures.Depth.Target, DepthPass);
	}

	if (bNaniteEnabled)
	{
		// Must happen before any Nanite rendering in the frame
		Nanite::GStreamingManager.EndAsyncUpdate(GraphBuilder);

		const TMap<uint32, uint32> ModifiedResources = Nanite::GStreamingManager.GetAndClearModifiedResources();

		TSet<Nanite::FPageInfo> InstalledPages;
		TSet<Nanite::FPageInfo> UninstalledPages;
		Nanite::GStreamingManager.GetAndClearModifiedPages(InstalledPages, UninstalledPages);

#if RHI_RAYTRACING
		Nanite::GRayTracingManager.RequestUpdates(ModifiedResources, InstalledPages, UninstalledPages);
#endif
	}

	// Virtual texturing isn't needed for depth prepass
	if (bUseVirtualTexturing && RendererOutput != ERendererOutput::DepthPrepassOnly)
	{
		FVirtualTextureSystem::Get().FinalizeRequests(GraphBuilder, this);
	}

	{
		RDG_RHI_EVENT_SCOPE_STAT(GraphBuilder, VisibilityCommands, "VisibilityCommands");
		EndInitViews(GraphBuilder, LumenFrameTemporaries, InstanceCullingManager, InitViewTaskDatas);
	}

	// Substrate initialisation is always run even when not enabled.
	// Need to run after EndInitViews() to ensure ViewRelevance computation are completed
	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
	Substrate::InitialiseSubstrateFrameSceneData(GraphBuilder, *Scene, ViewFamily, AllViews, ShaderPlatform);

	UE::SVT::GetStreamingManager().EndAsyncUpdate(GraphBuilder);

	FHairStrandsBookmarkParameters& HairStrandsBookmarkParameters = *GraphBuilder.AllocObject<FHairStrandsBookmarkParameters>();
	if (IsHairStrandsEnabled(EHairStrandsShaderType::All, Scene->GetShaderPlatform()) && RendererOutput == ERendererOutput::FinalSceneColor)
	{
		CreateHairStrandsBookmarkParameters(Scene, Views, AllViews, HairStrandsBookmarkParameters);
		check(Scene->HairStrandsSceneData.TransientResources);
		HairStrandsBookmarkParameters.TransientResources = Scene->HairStrandsSceneData.TransientResources;
		RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessTasks, HairStrandsBookmarkParameters);

		// Interpolation needs to happen after the skin cache run as there is a dependency 
		// on the skin cache output.
		const bool bRunHairStrands = HairStrandsBookmarkParameters.HasInstances() && (Views.Num() > 0);
		if (bRunHairStrands)
		{
			RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessCardsAndMeshesInterpolation_PrimaryView, HairStrandsBookmarkParameters);
		}
		else
		{
			for (FViewInfo& View : Views)
			{
				View.HairStrandsViewData.UniformBuffer = HairStrands::CreateDefaultHairStrandsViewUniformBuffer(GraphBuilder);
			}
		}
	}

	ExternalAccessQueue.Submit(GraphBuilder);

	const bool bShouldRenderSkyAtmosphere = ShouldRenderSkyAtmosphere(Scene, ViewFamily.EngineShowFlags);
	const ESkyAtmospherePassLocation SkyAtmospherePassLocation = GetSkyAtmospherePassLocation();
	FSkyAtmospherePendingRDGResources SkyAtmospherePendingRDGResources;
	if (SkyAtmospherePassLocation == ESkyAtmospherePassLocation::BeforePrePass && bShouldRenderSkyAtmosphere)
	{
		// Generate the Sky/Atmosphere look up tables overlaping the pre-pass
		// Pass DynamicShadowsTaskData pointer in case that task needs to be waited before accessing VisibleLightInfo.ShadowsToProject
		// (i.e FilterDynamicShadows runs on a parallel task and writes ShadowsToProject concurrently)
		RenderSkyAtmosphereLookUpTables(GraphBuilder, /* out */ SkyAtmospherePendingRDGResources, InitViewTaskDatas.DynamicShadows);
	}

	RenderWaterInfoTexture(GraphBuilder, *this, Scene);

	const bool bShouldRenderVelocities = ShouldRenderVelocities();
	const EShaderPlatform Platform = GetViewFamilyInfo(Views).GetShaderPlatform();
	const bool bBasePassCanOutputVelocity = FVelocityRendering::BasePassCanOutputVelocity(Platform);
	const bool bHairStrandsEnable = HairStrandsBookmarkParameters.HasInstances() && Views.Num() > 0 && IsHairStrandsEnabled(EHairStrandsShaderType::Strands, Platform);
	const bool bForceVelocityOutput = bHairStrandsEnable || ShouldRenderDistortion();

	auto RenderPrepassAndVelocity = [&](auto& InViews, auto& InNaniteBasePassVisibility, auto& NaniteRasterResults, auto& PrimaryNaniteViews, FSceneTextures& LocalSceneTextures)
	{
		FRDGTextureRef FirstStageDepthBuffer = nullptr;
		{
			// Both compute approaches run earlier, so skip clearing stencil here, just load existing.
			const ERenderTargetLoadAction StencilLoadAction = DepthPass.IsComputeStencilDitherEnabled()
				? ERenderTargetLoadAction::ELoad
				: ERenderTargetLoadAction::EClear;

			const ERenderTargetLoadAction DepthLoadAction = ERenderTargetLoadAction::EClear;
			AddClearDepthStencilPass(GraphBuilder, LocalSceneTextures.Depth.Target, DepthLoadAction, StencilLoadAction);

			// Draw the scene pre-pass / early z pass, populating the scene depth buffer and HiZ
			if (bNeedsPrePass)
			{
				RenderPrePass(GraphBuilder, InViews, LocalSceneTextures.Depth.Target, InstanceCullingManager, &FirstStageDepthBuffer);
			}
			else
			{
				// We didn't do the prepass, but we still want the HMD mask if there is one
				RenderPrePassHMD(GraphBuilder, InViews, LocalSceneTextures.Depth.Target);
			}

			// special pass for DDM_AllOpaqueNoVelocity, which uses the velocity pass to finish the early depth pass write
			if (bShouldRenderVelocities && Scene->EarlyZPassMode == DDM_AllOpaqueNoVelocity && RendererOutput == ERendererOutput::FinalSceneColor)
			{
				// Render the velocities of movable objects.  Don't bind the velocity render target for custom render passes (it's not used downstream), to avoid needing to clear it again.
				RenderVelocities(GraphBuilder, InViews, LocalSceneTextures, EVelocityPass::Opaque, bForceVelocityOutput, /*bBindRenderTarget=*/ InViews[0].CustomRenderPass == nullptr);
			}
		}

		{
			Scene->WaitForCacheNaniteMaterialBinsTask();

			if (bNaniteEnabled && InViews.Num() > 0)
			{
				RenderNanite(GraphBuilder, InViews, LocalSceneTextures, bIsEarlyDepthComplete, InNaniteBasePassVisibility, NaniteRasterResults, PrimaryNaniteViews, FirstStageDepthBuffer);
			}
		}

		if (FirstStageDepthBuffer)
		{
			LocalSceneTextures.PartialDepth = FirstStageDepthBuffer;
			AddResolveSceneDepthPass(GraphBuilder, InViews, LocalSceneTextures.PartialDepth);
		}
		else
		{
			// Setup default partial depth to be scene depth so that it also works on transparent emitter when partial depth has not been generated.
			LocalSceneTextures.PartialDepth = LocalSceneTextures.Depth;
		}
		LocalSceneTextures.SetupMode = ESceneTextureSetupMode::SceneDepth;
		LocalSceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &LocalSceneTextures, FeatureLevel, LocalSceneTextures.SetupMode);

		AddResolveSceneDepthPass(GraphBuilder, InViews, LocalSceneTextures.Depth);
	};

	*SceneTextures.DBufferTextures = CreateDBufferTextures(GraphBuilder, SceneTextures.Config.Extent, ShaderPlatform);

	// Initialise local fog volume with dummy data before volumetric cloud view initialization (further down) which can bind LFV data.
	// Also need to do this before custom render passes (included in AllViews), as base pass rendering may bind LFV data.
	SetDummyLocalFogVolumeForViews(GraphBuilder, AllViews);

	if (CustomRenderPassInfos.Num() > 0)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_CustomRenderPasses);
		RDG_EVENT_SCOPE_STAT(GraphBuilder, CustomRenderPasses, "CustomRenderPasses");

		for (int32 i = 0; i < CustomRenderPassInfos.Num(); ++i)
		{
			FCustomRenderPassBase* CustomRenderPass = CustomRenderPassInfos[i].CustomRenderPass;
			TArray<FViewInfo>& CustomRenderPassViews = CustomRenderPassInfos[i].Views;
			FNaniteShadingCommands& NaniteBasePassShadingCommands = CustomRenderPassInfos[i].NaniteBasePassShadingCommands;
			check(CustomRenderPass);

			// Initialize scene textures if owned by this Custom Render Pass (not reusing the main view family or shared non-MSAA SceneTextures).
			FViewFamilyInfo& CustomRenderPassViewFamily = CustomRenderPassInfos[i].ViewFamily;
			if (CustomRenderPassViewFamily.IsSceneTexturesOwner())
			{
				FSceneTextures::InitializeViewFamily(GraphBuilder, CustomRenderPassViewFamily, FamilySize);
				Substrate::InitialiseSubstrateFrameSceneData(GraphBuilder, *Scene, CustomRenderPassViewFamily, AllViews, ShaderPlatform);
			}
			FSceneTextures* CustomRenderPassSceneTextures = &CustomRenderPassViewFamily.GetSceneTextures();

			// We want to reset the scene texture uniform buffer to its original state after custom render passes,
			// so they can't affect downstream rendering.  This includes both cases where CRPs use the main ViewFamily's
			// SceneTextures, and cases where CRPs share non-MSAA SceneTextures.
			ESceneTextureSetupMode OriginalSceneTextureSetupMode = CustomRenderPassSceneTextures->SetupMode;
			TRDGUniformBufferRef<FSceneTextureUniformParameters> OriginalSceneTextureUniformBuffer = CustomRenderPassSceneTextures->UniformBuffer;

			CustomRenderPass->BeginPass(GraphBuilder);

			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_CustomRenderPass);
				RDG_EVENT_SCOPE(GraphBuilder, "CustomRenderPass[%d] %s", i, *CustomRenderPass->GetDebugName());

				CustomRenderPass->PreRender(GraphBuilder);

				TArray<Nanite::FRasterResults, TInlineAllocator<2>> NaniteRasterResults;
				TArray<Nanite::FPackedView, SceneRenderingAllocator> PrimaryNaniteViews;
				FNaniteBasePassVisibility DummyNaniteBasePassVisibility;
				RenderPrepassAndVelocity(CustomRenderPassViews, DummyNaniteBasePassVisibility, NaniteRasterResults, PrimaryNaniteViews, *CustomRenderPassSceneTextures);

				const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult = nullptr;
				if (ShouldRenderSingleLayerWaterDepthPrepass(CustomRenderPassViews))
				{
					SingleLayerWaterPrePassResult = RenderSingleLayerWaterDepthPrepass(GraphBuilder, CustomRenderPassViews, *CustomRenderPassSceneTextures, ESingleLayerWaterPrepassLocation::BeforeBasePass, NaniteRasterResults);
				}

				const FSceneCaptureCustomRenderPassUserData& SceneCaptureUserData = FSceneCaptureCustomRenderPassUserData::Get(CustomRenderPass);

				if (CustomRenderPass->GetRenderMode() == FCustomRenderPassBase::ERenderMode::DepthAndBasePass)
				{
					if (CustomRenderPassViewFamily.IsSceneTexturesOwner())
					{
						*CustomRenderPassSceneTextures->DBufferTextures = CreateDBufferTextures(GraphBuilder, CustomRenderPassSceneTextures->Config.Extent, ShaderPlatform);
					}

					CustomRenderPassSceneTextures->SetupMode |= ESceneTextureSetupMode::SceneColor;
					CustomRenderPassSceneTextures->UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, CustomRenderPassSceneTextures, FeatureLevel, CustomRenderPassSceneTextures->SetupMode);

					if (bNaniteEnabled)
					{
						Nanite::BuildShadingCommands(GraphBuilder, *Scene, ENaniteMeshPass::BasePass, NaniteBasePassShadingCommands, Nanite::EBuildShadingCommandsMode::Custom);
					}

					// This class handles decal rendering
					FCompositionLighting CustomRenderPassCompositionLighting(InitViewTaskDatas.Decals, CustomRenderPassViews, *CustomRenderPassSceneTextures, [this](int32 ViewIndex) { return false; });
					CustomRenderPassCompositionLighting.ProcessBeforeBasePass(GraphBuilder, *CustomRenderPassSceneTextures->DBufferTextures, InstanceCullingManager, *CustomRenderPassSceneTextures->SubstrateSceneData);

					RenderBasePass(*this, GraphBuilder, CustomRenderPassViews, *CustomRenderPassSceneTextures, BasePassDepthStencilAccess, /*ForwardScreenSpaceShadowMaskTexture=*/nullptr, InstanceCullingManager, bNaniteEnabled, NaniteBasePassShadingCommands, NaniteRasterResults);

					CustomRenderPassCompositionLighting.ProcessAfterBasePass(GraphBuilder, InstanceCullingManager, FCompositionLighting::EProcessAfterBasePassMode::All, *CustomRenderPassSceneTextures->SubstrateSceneData);

					if (ShouldRenderSingleLayerWater(CustomRenderPassViews))
					{
						// GBuffer code paths in RenderSingleLayerWater don't use the bIsCameraUnderWater flag, so just pass in false.  Normally this is
						// computed by a render extension, but those aren't run for custom render passes.
						FSceneWithoutWaterTextures SceneWithoutWaterTextures;
						RenderSingleLayerWater(GraphBuilder, CustomRenderPassViews, *CustomRenderPassSceneTextures, SingleLayerWaterPrePassResult, /*bShouldRenderVolumetricCloud=*/false, SceneWithoutWaterTextures, LumenFrameTemporaries, /*bIsCameraUnderWater=*/false);
					}

					FCustomRenderPassBase::ERenderOutput RenderOutput = CustomRenderPass->GetRenderOutput();
					if (RenderOutput == FCustomRenderPassBase::ERenderOutput::BaseColor || RenderOutput == FCustomRenderPassBase::ERenderOutput::Normal ||
						!SceneCaptureUserData.UserSceneTextureBaseColor.IsNone() || !SceneCaptureUserData.UserSceneTextureNormal.IsNone() || !SceneCaptureUserData.UserSceneTextureSceneColor.IsNone())
					{
						// CopySceneCaptureComponentToTarget uses scene texture uniforms
						CustomRenderPassSceneTextures->SetupMode |= ESceneTextureSetupMode::GBuffers;
						CustomRenderPassSceneTextures->UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, CustomRenderPassSceneTextures, FeatureLevel, CustomRenderPassSceneTextures->SetupMode);
					}

					if (CustomRenderPass->IsTranslucentIncluded())
					{
						// Empty defaults
						FTranslucencyLightingVolumeTextures TranslucencyLightingVolumeTextures;
						FTranslucencyPassResourcesMap TranslucencyResourceMap(CustomRenderPassViews.Num());
						const bool bStandardTranslucentCanRenderSeparate = false;
						FRDGTextureMSAA TranslucencySharedDepthTexture;
						FSeparateTranslucencyDimensions CustomTranslucencyDimensions = { SceneTexturesConfig.Extent };

						FReflectionCaptureShaderData EmptyData;
						TUniformBufferRef<FReflectionCaptureShaderData> EmptyReflectionCaptureUniformBuffer = TUniformBufferRef<FReflectionCaptureShaderData>::CreateUniformBufferImmediate(EmptyData, UniformBuffer_SingleFrame);
						for (FViewInfo& View : CustomRenderPassViews)
						{
							View.ReflectionCaptureUniformBuffer = EmptyReflectionCaptureUniformBuffer;
						}

						RenderTranslucency(*this, GraphBuilder, *CustomRenderPassSceneTextures, TranslucencyLightingVolumeTextures, &TranslucencyResourceMap, CustomRenderPassViews, ETranslucencyView::AboveWater, CustomTranslucencyDimensions, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, SingleLayerWaterPrePassResult, TranslucencySharedDepthTexture, NaniteRasterResults);
					}

					const bool bNeedsClassificationPass = !Substrate::UsesStochasticLightingClassification(ShaderPlatform);
					if (bNeedsClassificationPass)
					{
						Substrate::AddSubstrateMaterialClassificationPass(GraphBuilder, *CustomRenderPassSceneTextures, CustomRenderPassViews);
					}
				}

				CopySceneCaptureComponentToTarget(GraphBuilder, *CustomRenderPassSceneTextures, CustomRenderPass->GetRenderTargetTexture(), ViewFamily, CustomRenderPassViews);

				if (!SceneCaptureUserData.UserSceneTextureBaseColor.IsNone())
				{
					// User Scene Textures are stored to "SceneTextures" for downstream use, not in "CustomRenderPassSceneTextures", only used during Custom Render Pass rendering
					bool bFirstRender;
					FRDGTextureRef BaseColorSceneTexture = SceneTextures.FindOrAddUserSceneTexture(GraphBuilder, 0, SceneCaptureUserData.UserSceneTextureBaseColor, SceneCaptureUserData.SceneTextureDivisor, bFirstRender, nullptr, CustomRenderPassViews[0].ViewRect);
#if !(UE_BUILD_SHIPPING)
					SceneTextures.UserSceneTextureEvents.Add({ EUserSceneTextureEvent::CustomRenderPass, NAME_None, (uint16)FCustomRenderPassBase::ERenderOutput::BaseColor, 0, (const UMaterialInterface*)CustomRenderPass });
#endif

					CustomRenderPass->OverrideRenderOutput(FCustomRenderPassBase::ERenderOutput::BaseColor);
					CopySceneCaptureComponentToTarget(GraphBuilder, *CustomRenderPassSceneTextures, BaseColorSceneTexture, ViewFamily, CustomRenderPassViews);
				}

				if (!SceneCaptureUserData.UserSceneTextureNormal.IsNone())
				{
					bool bFirstRender;
					FRDGTextureRef NormalSceneTexture = SceneTextures.FindOrAddUserSceneTexture(GraphBuilder, 0, SceneCaptureUserData.UserSceneTextureNormal, SceneCaptureUserData.SceneTextureDivisor, bFirstRender, nullptr, CustomRenderPassViews[0].ViewRect);
#if !(UE_BUILD_SHIPPING)
					SceneTextures.UserSceneTextureEvents.Add({ EUserSceneTextureEvent::CustomRenderPass, NAME_None, (uint16)FCustomRenderPassBase::ERenderOutput::Normal, 0, (const UMaterialInterface*)CustomRenderPass });
#endif

					CustomRenderPass->OverrideRenderOutput(FCustomRenderPassBase::ERenderOutput::Normal);
					CopySceneCaptureComponentToTarget(GraphBuilder, *CustomRenderPassSceneTextures, NormalSceneTexture, ViewFamily, CustomRenderPassViews);
				}

				if (!SceneCaptureUserData.UserSceneTextureSceneColor.IsNone())
				{
					bool bFirstRender;
					FRDGTextureRef SceneColorSceneTexture = SceneTextures.FindOrAddUserSceneTexture(GraphBuilder, 0, SceneCaptureUserData.UserSceneTextureSceneColor, SceneCaptureUserData.SceneTextureDivisor, bFirstRender, nullptr, CustomRenderPassViews[0].ViewRect);
#if !(UE_BUILD_SHIPPING)
					SceneTextures.UserSceneTextureEvents.Add({ EUserSceneTextureEvent::CustomRenderPass, NAME_None, (uint16)FCustomRenderPassBase::ERenderOutput::SceneColorAndAlpha, 0, (const UMaterialInterface*)CustomRenderPass });
#endif

					CustomRenderPass->OverrideRenderOutput(FCustomRenderPassBase::ERenderOutput::SceneColorAndAlpha);
					CopySceneCaptureComponentToTarget(GraphBuilder, *CustomRenderPassSceneTextures, SceneColorSceneTexture, ViewFamily, CustomRenderPassViews);
				}

				CustomRenderPass->PostRender(GraphBuilder);

				// Mips are normally generated in UpdateSceneCaptureContentDeferred_RenderThread, but that doesn't run when the
				// scene capture runs as a custom render pass.  The function does nothing if the render target doesn't have mips.
				if (CustomRenderPassViews[0].bIsSceneCapture)
				{
					FGenerateMips::Execute(GraphBuilder, FeatureLevel, CustomRenderPass->GetRenderTargetTexture(), FGenerateMipsParams());
				}

			#if WITH_MGPU
				DoCrossGPUTransfers(GraphBuilder, CustomRenderPass->GetRenderTargetTexture(), CustomRenderPassViews, false, FRHIGPUMask::All(), nullptr);
			#endif
			}

			CustomRenderPass->EndPass(GraphBuilder);

			// Restore original scene texture uniforms
			CustomRenderPassSceneTextures->SetupMode = OriginalSceneTextureSetupMode;
			CustomRenderPassSceneTextures->UniformBuffer = OriginalSceneTextureUniformBuffer;

			// Clear DBuffer produced flags after Custom Render Pass rendering, so they will be cleared when they render again
			CustomRenderPassSceneTextures->DBufferTextures->bDBufferProduced = false;
			CustomRenderPassSceneTextures->DBufferTextures->bDBufferTexArrayProduced = false;
		}
	}

	TArray<Nanite::FRasterResults, TInlineAllocator<2>> NaniteRasterResults;
	TArray<Nanite::FPackedView, SceneRenderingAllocator> PrimaryNaniteViews;
	RenderPrepassAndVelocity(Views, NaniteBasePassVisibility, NaniteRasterResults, PrimaryNaniteViews, SceneTextures);

	// Run Nanite compute commands early in the frame to allow some task overlap on the CPU until the base pass runs.
	if (bNaniteEnabled && RendererOutput != ERendererOutput::DepthPrepassOnly && !bHasRayTracedOverlay)
	{
		Nanite::BuildShadingCommands(GraphBuilder, *Scene, ENaniteMeshPass::BasePass, Scene->NaniteShadingCommands[ENaniteMeshPass::BasePass]);
		if (bAnyLumenEnabled && RendererOutput == ERendererOutput::FinalSceneColor)
		{
			Nanite::BuildShadingCommands(GraphBuilder, *Scene, ENaniteMeshPass::LumenCardCapture, Scene->NaniteShadingCommands[ENaniteMeshPass::LumenCardCapture]);
		}
	}

	FComputeLightGridOutput ComputeLightGridOutput = {};

	FCompositionLighting CompositionLighting(InitViewTaskDatas.Decals, Views, SceneTextures, [this](int32 ViewIndex)
	{
		return GetViewPipelineState(Views[ViewIndex]).AmbientOcclusionMethod == EAmbientOcclusionMethod::SSAO;
	});

	const auto RenderOcclusionLambda = [&]() -> Froxel::FRenderer 
	{
		const int32 AsyncComputeMode = CVarSceneDepthHZBAsyncCompute.GetValueOnRenderThread();
		bool bAsyncCompute = AsyncComputeMode != 0;

		FBuildHZBAsyncComputeParams AsyncComputeParams = {};
		if (AsyncComputeMode == 2)
		{
			AsyncComputeParams.Prerequisite = ComputeLightGridOutput.CompactLinksPass;
		}

		bool bShouldGenerateFroxels = DoesVSMWantFroxels(ShaderPlatform);

		Froxel::FRenderer FroxelRenderer(bShouldGenerateFroxels, GraphBuilder, Views);

		RenderOcclusion(GraphBuilder, SceneTextures, bIsOcclusionTesting,
			bAsyncCompute ? &AsyncComputeParams : nullptr, FroxelRenderer);

		CompositionLighting.ProcessAfterOcclusion(GraphBuilder);

		return FroxelRenderer;
	};

	const bool bShouldRenderVolumetricCloudBase = ShouldRenderVolumetricCloud(Scene, ViewFamily.EngineShowFlags);
	const bool bShouldRenderVolumetricCloud = bShouldRenderVolumetricCloudBase && (!ViewFamily.EngineShowFlags.VisualizeVolumetricCloudConservativeDensity && !ViewFamily.EngineShowFlags.VisualizeVolumetricCloudEmptySpaceSkipping);
	const bool bShouldVisualizeVolumetricCloud = bShouldRenderVolumetricCloudBase && (!!ViewFamily.EngineShowFlags.VisualizeVolumetricCloudConservativeDensity || !!ViewFamily.EngineShowFlags.VisualizeVolumetricCloudEmptySpaceSkipping);
	const bool bAsyncComputeVolumetricCloud = IsVolumetricRenderTargetEnabled() && IsVolumetricRenderTargetAsyncCompute();
	const bool bVolumetricRenderTargetRequired = bShouldRenderVolumetricCloud && !bHasRayTracedOverlay;

	Froxel::FRenderer FroxelRenderer;

	FRDGTextureRef ViewFamilyTexture = TryCreateViewFamilyTexture(GraphBuilder, ViewFamily);
	FRDGTextureRef ViewFamilyDepthTexture = TryCreateViewFamilyDepthTexture(GraphBuilder, ViewFamily);
	if (RendererOutput == ERendererOutput::DepthPrepassOnly)
	{
		const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult = nullptr;
		if (ShouldRenderSingleLayerWaterDepthPrepass(Views))
		{
			SingleLayerWaterPrePassResult = RenderSingleLayerWaterDepthPrepass(GraphBuilder, Views, SceneTextures, ESingleLayerWaterPrepassLocation::BeforeBasePass, NaniteRasterResults);
		}

		FroxelRenderer = RenderOcclusionLambda();

		CopySceneCaptureComponentToTarget(GraphBuilder, SceneTextures, ViewFamilyTexture, ViewFamilyDepthTexture, ViewFamily, Views);
	}
	else
	{
		GVRSImageManager.PrepareImageBasedVRS(GraphBuilder, ViewFamily, SceneTextures);

		if (!IsForwardShadingEnabled(ShaderPlatform))
		{
			// Dynamic shadows are synced later when using the deferred path to make more headroom for tasks.
			FinishInitDynamicShadows(GraphBuilder, InitViewTaskDatas.DynamicShadows, InstanceCullingManager);
		}

		// Update groom only visible in shadow
		if (IsHairStrandsEnabled(EHairStrandsShaderType::All, Scene->GetShaderPlatform()) && RendererOutput == ERendererOutput::FinalSceneColor)
		{
			UpdateHairStrandsBookmarkParameters(Scene, Views, HairStrandsBookmarkParameters);

			// Interpolation for cards/meshes only visible in shadow needs to happen after the shadow jobs are completed
			const bool bRunHairStrands = HairStrandsBookmarkParameters.HasInstances() && (Views.Num() > 0);
			if (bRunHairStrands)
			{
				RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessCardsAndMeshesInterpolation_ShadowView, HairStrandsBookmarkParameters);
			}
		}

		// Early occlusion queries
		const bool bOcclusionBeforeBasePass = ((DepthPass.EarlyZPassMode == EDepthDrawingMode::DDM_AllOccluders) || bIsEarlyDepthComplete);

		if (bOcclusionBeforeBasePass)
		{
			FroxelRenderer = RenderOcclusionLambda();
		}

		// End early occlusion queries

		for (FSceneViewExtensionRef& ViewExtension : ViewFamily.ViewExtensions)
		{
			ViewExtension->PreRenderBasePass_RenderThread(GraphBuilder, ShouldRenderPrePass() /*bDepthBufferIsPopulated*/);
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_WaitGatherAndSortLightsTask);
			GatherAndSortLightsTask.Wait();
		}

		{
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, PrepareForwardLightData);
			SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_PrepareForwardLightData);

			const FSortedLightSetSceneInfo* SortedLightSet = GatherAndSortLightsTask.GetResult();

			if (!ViewFamily.EngineShowFlags.PathTracing)
			{
				ComputeLightGridOutput = PrepareForwardLightData(GraphBuilder, true, *SortedLightSet);

				// Store this flag if lights are injected in the grids, check with 'AreLightsInLightGrid()'
				bAreLightsInLightGrid = true;
			}
			else
			{
				SetDummyForwardLightUniformBufferOnViews(GraphBuilder, ShaderPlatform, Views);
			}

			CSV_CUSTOM_STAT(LightCount, All,  float(SortedLightSet->SortedLights.Num()), ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(LightCount, Batched, float(SortedLightSet->UnbatchedLightStart), ECsvCustomStatOp::Set);
			CSV_CUSTOM_STAT(LightCount, Unbatched, float(SortedLightSet->SortedLights.Num()) - float(SortedLightSet->UnbatchedLightStart), ECsvCustomStatOp::Set);
		}

		LightFunctionAtlas.RenderLightFunctionAtlas(GraphBuilder, Views);

		ComputeMegaLightsLightPowerDelta(GraphBuilder);

		// Run before RenderSkyAtmosphereLookUpTables for cloud shadows to be valid.
		InitVolumetricCloudsForViews(GraphBuilder, bShouldRenderVolumetricCloudBase, InstanceCullingManager);

		BeginAsyncDistanceFieldShadowProjections(GraphBuilder, SceneTextures, InitViewTaskDatas.DynamicShadows);

		// Run local fog volume culling before base pass and after HZB generation to benefit from more culling.
		InitLocalFogVolumesForViews(Scene, Views, ViewFamily, GraphBuilder, bShouldRenderVolumetricFog, false /*bool bUseHalfResLocalFogVolume*/);

		if (bShouldRenderVolumetricCloudBase)
		{
			InitVolumetricRenderTargetForViews(GraphBuilder, Views, SceneTextures);
		}
		else
		{
			ResetVolumetricRenderTargetForViews(GraphBuilder, Views);
		}

		// Generate sky LUTs
		// TODO: Valid shadow maps (for volumetric light shafts) have not yet been generated at this point in the frame. Need to resolve dependency ordering!
		// This also must happen before the BasePass for Sky material to be able to sample valid LUTs.
		if (SkyAtmospherePassLocation == ESkyAtmospherePassLocation::BeforeBasePass && bShouldRenderSkyAtmosphere)
		{
			// Generate the Sky/Atmosphere look up tables
			RenderSkyAtmosphereLookUpTables(GraphBuilder, /* out */ SkyAtmospherePendingRDGResources);

			SkyAtmospherePendingRDGResources.CommitToSceneAndViewUniformBuffers(GraphBuilder, /* out */ ExternalAccessQueue);
		}
		else if (SkyAtmospherePassLocation == ESkyAtmospherePassLocation::BeforePrePass && bShouldRenderSkyAtmosphere)
		{
			SkyAtmospherePendingRDGResources.CommitToSceneAndViewUniformBuffers(GraphBuilder, /* out */ ExternalAccessQueue);
		}

		// Capture the SkyLight using the SkyAtmosphere and VolumetricCloud component if available.
		const bool bRealTimeSkyCaptureEnabled = Scene->SkyLight && Scene->SkyLight->bRealTimeCaptureEnabled && Views.Num() > 0 && ViewFamily.EngineShowFlags.SkyLighting;
		const bool bPathTracedAtmosphere = ViewFamily.EngineShowFlags.PathTracing && Views.Num() > 0 && PathTracing::UsesReferenceAtmosphere(Views[0]);
		if (bRealTimeSkyCaptureEnabled && !bPathTracedAtmosphere)
		{
			// Sky capture accesses the view uniform buffer which uses LUT's.
			ExternalAccessQueue.Submit(GraphBuilder);

			FViewInfo& MainView = Views[0];
			Scene->AllocateAndCaptureFrameSkyEnvMap(GraphBuilder, *this, MainView, bShouldRenderSkyAtmosphere, bShouldRenderVolumetricCloud, InstanceCullingManager, ExternalAccessQueue);
		}

		const ECustomDepthPassLocation CustomDepthPassLocation = GetCustomDepthPassLocation(ShaderPlatform);
		if (CustomDepthPassLocation == ECustomDepthPassLocation::BeforeBasePass)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_CustomDepthPass_BeforeBasePass);
			if (RenderCustomDepthPass(GraphBuilder, SceneTextures.CustomDepth, SceneTextures.GetSceneTextureShaderParameters(FeatureLevel), NaniteRasterResults, PrimaryNaniteViews))
			{
				SceneTextures.SetupMode |= ESceneTextureSetupMode::CustomDepth;
				SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);
			}
		}

		// Single layer water depth prepass. Needs to run before VSM page allocation. If there's a full depth prepass, it can run before the base pass, otherwise after.
		// Running before the base pass allows for some optimizations to save work in the base pass and lighting stages.
		const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult = nullptr;
		const ESingleLayerWaterPrepassLocation SingleLayerWaterPrepassLocation = GetSingleLayerWaterDepthPrepassLocation(bIsEarlyDepthComplete, CustomDepthPassLocation);
		const bool bShouldRenderSingleLayerWaterDepthPrepass = !bHasRayTracedOverlay && ShouldRenderSingleLayerWaterDepthPrepass(Views);
		if (bShouldRenderSingleLayerWaterDepthPrepass && SingleLayerWaterPrepassLocation == ESingleLayerWaterPrepassLocation::BeforeBasePass)
		{
			SingleLayerWaterPrePassResult = RenderSingleLayerWaterDepthPrepass(GraphBuilder, Views, SceneTextures, SingleLayerWaterPrepassLocation, NaniteRasterResults);
		}

		// Lumen updates need access to sky atmosphere LUT.
		ExternalAccessQueue.Submit(GraphBuilder);

		UpdateLumenScene(GraphBuilder, LumenFrameTemporaries);

		// Reset the common DitheredHalfResDepth texture.
		SceneTextures.DitheredHalfResDepth = nullptr;

		FRDGTextureRef HalfResolutionDepthMinMaxTexture = nullptr;
		FRDGTextureRef QuarterResolutionDepthMinMaxTexture = nullptr;
		bool bQuarterResMinMaxDepthRequired = bShouldRenderVolumetricCloud && ShouldVolumetricCloudTraceWithMinMaxDepth(Views);

		auto GenerateQuarterResDepthMinMaxTexture = [&](auto& GraphBuilder, auto& Views, auto& SceneDepthTexture)
		{
			if (bQuarterResMinMaxDepthRequired)
			{
				check(SceneDepthTexture != nullptr);					// Must receive a valid texture
				check(HalfResolutionDepthMinMaxTexture == nullptr);		// Only generate it once
				check(QuarterResolutionDepthMinMaxTexture == nullptr);	// Only generate it once
				CreateQuarterResolutionDepthMinAndMaxFromDepthTexture(GraphBuilder, Views, SceneDepthTexture, HalfResolutionDepthMinMaxTexture, QuarterResolutionDepthMinMaxTexture);
			}
			else
			{
				SceneTextures.DitheredHalfResDepth = CreateHalfResolutionDepthCheckerboardMinMax(GraphBuilder, Views, SceneDepthTexture);
			}
		};
		
		FRDGTextureRef ForwardScreenSpaceShadowMaskTexture = nullptr;
		FRDGTextureRef ForwardScreenSpaceShadowMaskHairTexture = nullptr;
		bool bShadowMapsRenderedEarly = false;
		if (IsForwardShadingEnabled(ShaderPlatform))
		{
			// With forward shading we need to render shadow maps early
			ensureMsgf(!VirtualShadowMapArray.IsEnabled(), TEXT("Virtual shadow maps are not supported in the forward shading path"));
			RenderShadowDepthMaps(GraphBuilder, InitViewTaskDatas.DynamicShadows, InstanceCullingManager, ExternalAccessQueue);
			bShadowMapsRenderedEarly = true;

			if (bHairStrandsEnable)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "Hair");

				RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessStrandsInterpolation, HairStrandsBookmarkParameters);
				if (!bHasRayTracedOverlay)
				{
					RenderHairPrePass(GraphBuilder, Scene, SceneTextures, Views, InstanceCullingManager, HairStrandsBookmarkParameters.CullingResults);
					RenderHairBasePass(GraphBuilder, Scene, SceneTextures, Views, InstanceCullingManager, nullptr);
				}
			}

			RenderForwardShadowProjections(GraphBuilder, SceneTextures, ForwardScreenSpaceShadowMaskTexture, ForwardScreenSpaceShadowMaskHairTexture);

			// With forward shading we need to render volumetric fog before the base pass
			ComputeVolumetricFog(GraphBuilder, SceneTextures);
		}
		else if ( CVarShadowMapsRenderEarly.GetValueOnRenderThread() )
		{
			// Disable early shadows if VSM is enabled, but warn
			ensureMsgf(!VirtualShadowMapArray.IsEnabled(), TEXT("Virtual shadow maps are not supported with r.shadow.ShadowMapsRenderEarly. Early shadows will be disabled"));
			if (!VirtualShadowMapArray.IsEnabled())
			{
				RenderShadowDepthMaps(GraphBuilder, InitViewTaskDatas.DynamicShadows, InstanceCullingManager, ExternalAccessQueue);
				bShadowMapsRenderedEarly = true;
			}
		}

		ExternalAccessQueue.Submit(GraphBuilder);

		{
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, DeferredShadingSceneRenderer_DBuffer);
			SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_DBuffer);
			CompositionLighting.ProcessBeforeBasePass(GraphBuilder, *SceneTextures.DBufferTextures, InstanceCullingManager, *SceneTextures.SubstrateSceneData);
		}
		
		if (IsForwardShadingEnabled(ShaderPlatform))
		{
			RenderIndirectCapsuleShadows(GraphBuilder, SceneTextures);
		}

		FTranslucencyLightingVolumeTextures TranslucencyLightingVolumeTextures;

		if (bRenderDeferredLighting && ShouldClearTranslucencyLightingVolumeInAsyncCompute())
		{
			TranslucencyLightingVolumeTextures.Init(GraphBuilder, Views, ERDGPassFlags::AsyncCompute);
		}

		FRDGBufferRef DynamicGeometryScratchBuffer = nullptr;
#if RHI_RAYTRACING
		
		ERHIPipeline DynamicRTResourceAccessPipelines = Lumen::UseAsyncCompute(ViewFamily) ? ERHIPipeline::All : ERHIPipeline::Graphics;

		// Async AS builds can potentially overlap with BasePass.
		bool bNeedToSetupRayTracingRenderingData = DispatchRayTracingWorldUpdates(GraphBuilder, DynamicGeometryScratchBuffer, DynamicRTResourceAccessPipelines);

		// Must be prepared before the SBT is built
		if (ViewFamily.EngineShowFlags.PathTracing)
		{
			PreparePathTracingHeterogeneousVolumes(GraphBuilder, Scene, Views);
		}

		/** Should be called somewhere before "SetupRayTracingRenderingData" */
		SetupRayTracingLightDataForViews(GraphBuilder);
#endif

		bool bHasMarkedTranslucencyVolume = false;
		bool bGatheredTranslucencyVolumeMarkedVoxels = false;
		auto MarkUsedTranslucencyVolumeVoxels = [this, &GraphBuilder, &SceneTextures, &bHasMarkedTranslucencyVolume]()
		{
			if (IsTranslucencyLightingVolumeUsingVoxelMarking() && !bHasMarkedTranslucencyVolume)
			{
				for (FViewInfo& View : Views)
				{
					bool bNeedsMark = false;
					for (int32 CascadeIndex = 0; CascadeIndex < TVC_MAX; ++CascadeIndex)
					{
						bNeedsMark = bNeedsMark || !View.TranslucencyVolumeMarkData[CascadeIndex].MarkTexture;
					}

					if (bNeedsMark)
					{
						LumenTranslucencyReflectionsMarkUsedProbes(GraphBuilder, *this, View, SceneTextures, nullptr);
					}
				}

				bHasMarkedTranslucencyVolume = true;
			}
		};

		TSharedPtr<FMegaLightsFrameTemporaries> MegaLightsContext = nullptr;

		if (!bHasRayTracedOverlay)
		{
			const FSortedLightSetSceneInfo& SortedLightSet = *GatherAndSortLightsTask.GetResult();
			if (bRenderDeferredLighting && SortedLightSet.MegaLightsLightStart < SortedLightSet.SortedLights.Num())
			{
				MegaLightsContext = CreateMegaLightsFrameTemporaries(GraphBuilder, SceneTextures);
			}

#if RHI_RAYTRACING
			const bool bMegaLightsNeedsRTData = MegaLights::UseHardwareRayTracing(ViewFamily) && MegaLights::VolumeUseAsyncCompute(MegaLightsContext);

			// Lumen scene lighting requires ray tracing scene to be ready if HWRT shadows are desired
			if (bNeedToSetupRayTracingRenderingData && (Lumen::UseHardwareRayTracedSceneLighting(ViewFamily) || bMegaLightsNeedsRTData))
			{
				SetupRayTracingRenderingData(GraphBuilder, *InitViewTaskDatas.RayTracingGatherInstances);
				bNeedToSetupRayTracingRenderingData = false;
			}
#endif
			{
				LLM_SCOPE_BYTAG(Lumen);
				BeginGatheringLumenSurfaceCacheFeedback(GraphBuilder, Views[0], LumenFrameTemporaries);
				RenderLumenSceneLighting(GraphBuilder, LumenFrameTemporaries, InitViewTaskDatas.LumenDirectLighting);
			}

			MarkLumenRadianceCacheForTranslucencyReflections(GraphBuilder, SceneTextures, LumenFrameTemporaries);

			if (MegaLights::VolumeUseAsyncCompute(MegaLightsContext) && MegaLights::UseTranslucencyVolumeMarkTexture(MegaLightsContext))
			{
				MarkUsedTranslucencyVolumeVoxels();
				GatherTranslucencyVolumeMarkedVoxels(GraphBuilder, SortedLightSet);
				bGatheredTranslucencyVolumeMarkedVoxels = true;
			}

			DispatchAsyncMegaLightsVolumePasses(GraphBuilder, MegaLightsContext);
		}

		{
			if (!bHasRayTracedOverlay)
			{
				RenderBasePass(*this, GraphBuilder, Views, SceneTextures, BasePassDepthStencilAccess, ForwardScreenSpaceShadowMaskTexture, InstanceCullingManager, bNaniteEnabled, Scene->NaniteShadingCommands[ENaniteMeshPass::BasePass], NaniteRasterResults);
			}

			if (!bAllowReadOnlyDepthBasePass)
			{
				AddResolveSceneDepthPass(GraphBuilder, Views, SceneTextures.Depth);
			}

			if (bNaniteEnabled)
			{
				if (bVisualizeNanite)
				{
					FNanitePickingFeedback PickingFeedback = { 0 };

					Nanite::AddVisualizationPasses(
						GraphBuilder,
						Scene,
						SceneTextures,
						ViewFamily.EngineShowFlags,
						Views,
						NaniteRasterResults,
						PickingFeedback,
						VirtualShadowMapArray
					);

					OnGetOnScreenMessages.AddLambda([this, PickingFeedback, RenderFlags = NaniteRasterResults[0].RenderFlags, ScenePtr = Scene](FScreenMessageWriter& ScreenMessageWriter)->void
					{
						Nanite::DisplayPicking(ScenePtr, PickingFeedback, RenderFlags, ScreenMessageWriter);
					});
				}
			}

			// VisualizeVirtualShadowMap TODO
		}

		FRDGTextureRef ExposureIlluminanceSetup = nullptr;
		if (!bHasRayTracedOverlay)
		{
			// Extract emissive from SceneColor (before lighting is applied)
			ExposureIlluminanceSetup = AddSetupExposureIlluminancePass(GraphBuilder, Views, SceneTextures);
		}

		if (ViewFamily.EngineShowFlags.VisualizeLightCulling)
		{
			FRDGTextureRef VisualizeLightCullingTexture = GraphBuilder.CreateTexture(SceneTextures.Color.Target->Desc, TEXT("SceneColorVisualizeLightCulling"));
			AddClearRenderTargetPass(GraphBuilder, VisualizeLightCullingTexture, FLinearColor::Transparent);
			SceneTextures.Color.Target = VisualizeLightCullingTexture;

			// When not in MSAA, assign to both targets.
			if (SceneTexturesConfig.NumSamples == 1)
			{
				SceneTextures.Color.Resolve = SceneTextures.Color.Target;
			}
		}

		if (bRenderDeferredLighting)
		{
			// mark GBufferA for saving for next frame if it's needed
			ExtractNormalsForNextFrameReprojection(GraphBuilder, SceneTextures, Views);
		}

		// Rebuild scene textures to include GBuffers.
		SceneTextures.SetupMode |= ESceneTextureSetupMode::GBuffers;
		if (bShouldRenderVelocities && (bBasePassCanOutputVelocity || Scene->EarlyZPassMode == DDM_AllOpaqueNoVelocity))
		{
			SceneTextures.SetupMode |= ESceneTextureSetupMode::SceneVelocity;
		}
		SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);

		if (bRealTimeSkyCaptureEnabled)
		{
			Scene->ValidateSkyLightRealTimeCapture(GraphBuilder, Views[0], SceneTextures.Color.Target);
		}

		VisualizeVolumetricLightmap(GraphBuilder, SceneTextures);

		// Occlusion after base pass
		if (!bOcclusionBeforeBasePass)
		{
			FroxelRenderer = RenderOcclusionLambda();
		}

		// End occlusion after base

		if (!bUseGBuffer)
		{
			AddResolveSceneColorPass(GraphBuilder, Views, SceneTextures.Color);
		}

		// Render hair
		if (bHairStrandsEnable && !IsForwardShadingEnabled(ShaderPlatform))
		{
			RDG_EVENT_SCOPE(GraphBuilder, "Hair");

			RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessStrandsInterpolation, HairStrandsBookmarkParameters);
			if (!bHasRayTracedOverlay)
			{
				RenderHairPrePass(GraphBuilder, Scene, SceneTextures, Views, InstanceCullingManager, HairStrandsBookmarkParameters.CullingResults);
				RenderHairBasePass(GraphBuilder, Scene, SceneTextures, Views, InstanceCullingManager, MegaLightsContext);
			}
		}

		if (ShouldRenderHeterogeneousVolumes(Scene) && !bHasRayTracedOverlay)
		{
			RenderHeterogeneousVolumeShadows(GraphBuilder, SceneTextures);
		}

		// Post base pass for material classification
		// This needs to run before virtual shadow map, in order to have ready&cleared classified SSS data
		if (Substrate::IsSubstrateEnabled() && !bHasRayTracedOverlay)
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, Substrate, "Substrate");

			// Substrate DBufferPass (optional)
			if (Substrate::IsDBufferPassEnabled(ShaderPlatform))
			{
				RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, DeferredShadingSceneRenderer_DBuffer);
				SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_DBuffer);
				Substrate::AddSubstrateDBufferBasePass(GraphBuilder, Views, SceneTextures, *SceneTextures.DBufferTextures, InitViewTaskDatas.Decals, InstanceCullingManager, *SceneTextures.SubstrateSceneData);
			}

			// Substrate classifation is done either in a standalone pass (here) or done within the StochasticLightingTileClassificationMark pass
			const bool bNeedsClassificationPass = !(RequiresStochasticLightingPass() && Substrate::UsesStochasticLightingClassification(ShaderPlatform));
			if (bNeedsClassificationPass)
			{
				Substrate::AddSubstrateMaterialClassificationPass(GraphBuilder, SceneTextures, Views);
			}
			{
				Substrate::AddSubstrateDBufferPass(GraphBuilder, SceneTextures, Views);
				Substrate::AddSubstrateSampleMaterialPass(GraphBuilder, Scene, SceneTextures, Views);
			}
		}

		FAsyncLumenIndirectLightingOutputs AsyncLumenIndirectLightingOutputs;
		const bool bAsyncMegaLightsGenerateSamples = MegaLights::GenerateSamplesUseAsyncCompute(MegaLightsContext);

		if (!bHasRayTracedOverlay && RequiresStochasticLightingPass())
		{
			// Decals may modify GBuffers so they need to be done first. Can decals read velocities and/or custom depth? If so, they need to be rendered earlier too.
			CompositionLighting.ProcessAfterBasePass(GraphBuilder, InstanceCullingManager, FCompositionLighting::EProcessAfterBasePassMode::OnlyBeforeLightingDecals, *SceneTextures.SubstrateSceneData);
			AsyncLumenIndirectLightingOutputs.bHasDrawnBeforeLightingDecals = true;

			RDG_EVENT_SCOPE_STAT(GraphBuilder, RenderDeferredLighting, "StochasticLighting");

			StochasticLightingTileClassificationMark(GraphBuilder, LumenFrameTemporaries, SceneTextures, bAsyncMegaLightsGenerateSamples);
		}

		// Copy lighting channels out of stencil before deferred decals which overwrite those values
		TArray<FRDGTextureRef, TInlineAllocator<2>> NaniteShadingMask;
		if (bNaniteEnabled && Views.Num() > 0)
		{
			check(Views.Num() == NaniteRasterResults.Num());
			for (const Nanite::FRasterResults& Results : NaniteRasterResults)
			{
				NaniteShadingMask.Add(Results.ShadingMask);
			}
		}
		FRDGTextureRef LightingChannelsTexture = CopyStencilToLightingChannelTexture(GraphBuilder, SceneTextures.Stencil, NaniteShadingMask);

		// Single layer water depth prepass. Needs to run before VSM page allocation.
		if (bShouldRenderSingleLayerWaterDepthPrepass && SingleLayerWaterPrepassLocation == ESingleLayerWaterPrepassLocation::AfterBasePass)
		{
			SingleLayerWaterPrePassResult = RenderSingleLayerWaterDepthPrepass(GraphBuilder, Views, SceneTextures, SingleLayerWaterPrepassLocation, NaniteRasterResults);
		}

		GraphBuilder.FlushSetupQueue();

		// Shadows, lumen and fog after base pass
		if (!bHasRayTracedOverlay)
		{
			if (bAsyncMegaLightsGenerateSamples)
			{
				GenerateMegaLightsSamples(
					GraphBuilder,
					MegaLightsContext,
					LumenFrameTemporaries,
					LightingChannelsTexture,
					ERDGPassFlags::AsyncCompute);
			}

#if RHI_RAYTRACING
			// When Lumen HWRT is running async we need to wait for ray tracing scene before dispatching the work
			if (bNeedToSetupRayTracingRenderingData && Lumen::UseAsyncCompute(ViewFamily))
			{
				SetupRayTracingRenderingData(GraphBuilder, *InitViewTaskDatas.RayTracingGatherInstances);
				bNeedToSetupRayTracingRenderingData = false;
			}
#endif // RHI_RAYTRACING

			DispatchAsyncLumenIndirectLightingWork(
				GraphBuilder,
				SceneTextures,
				InstanceCullingManager,
				LumenFrameTemporaries,
				InitViewTaskDatas.DynamicShadows,
				LightingChannelsTexture,
				AsyncLumenIndirectLightingOutputs);

			// Kick off volumetric clouds async dispatch after Lumen
			// Lumen has a dependency on the opaque so should run first
			// Volumetric Clouds have a depedency on translucent, so should run second and overlap opaque work after Lumen async is done
			if (bShouldRenderVolumetricCloud && bAsyncComputeVolumetricCloud)
			{
				GenerateQuarterResDepthMinMaxTexture(GraphBuilder, Views, SceneTextures.Depth.Resolve);

				bool bSkipVolumetricRenderTarget = false;
				bool bSkipPerPixelTracing = true;
				RenderVolumetricCloud(GraphBuilder, SceneTextures, bSkipVolumetricRenderTarget, bSkipPerPixelTracing,
					SceneTextures.DitheredHalfResDepth, QuarterResolutionDepthMinMaxTexture, true, InstanceCullingManager);
			}

			if (!bShadowMapsRenderedEarly && ShadowSceneRenderer.GetVirtualShadowMapArray().IsEnabled())
			{
				FFrontLayerTranslucencyData FrontLayerTranslucencyData = RenderFrontLayerTranslucency(GraphBuilder, Views, SceneTextures, MegaLightsContext, true /*VSM page marking*/);
				ShadowSceneRenderer.BeginMarkVirtualShadowMapPages(GraphBuilder, SingleLayerWaterPrePassResult, FrontLayerTranslucencyData, FroxelRenderer);
			}

			if (!bAsyncMegaLightsGenerateSamples)
			{
				// Do MegaLights sampling before VSM pages are marked and rendered so they can be specialized
				// based on the selected samples.
				GenerateMegaLightsSamples(
					GraphBuilder,
					MegaLightsContext,
					LumenFrameTemporaries,
					LightingChannelsTexture,
					ERDGPassFlags::Compute);
			}

			// If we haven't already rendered shadow maps, render them now (due to forward shading or r.shadow.ShadowMapsRenderEarly)
			if (!bShadowMapsRenderedEarly)
			{
				RenderShadowDepthMaps(GraphBuilder, InitViewTaskDatas.DynamicShadows, InstanceCullingManager, ExternalAccessQueue);
			}
			CheckShadowDepthRenderCompleted();

#if RHI_RAYTRACING
			// Lumen scene lighting requires ray tracing scene to be ready if HWRT shadows are desired
			if (bNeedToSetupRayTracingRenderingData && Lumen::UseHardwareRayTracedSceneLighting(ViewFamily))
			{
				SetupRayTracingRenderingData(GraphBuilder, *InitViewTaskDatas.RayTracingGatherInstances);
				bNeedToSetupRayTracingRenderingData = false;
			}
#endif // RHI_RAYTRACING
		}

		ExternalAccessQueue.Submit(GraphBuilder);

		// End shadow and fog after base pass

#if RHI_RAYTRACING
		if (IsRayTracingEnabled(ViewFamily.GetShaderPlatform()) && (GRHISupportsInlineRayTracing || GRHISupportsRayTracingShaders))
		{
			for (int32 ViewExt = 0; ViewExt < ViewFamily.ViewExtensions.Num(); ++ViewExt)
			{
				if (EnumHasAnyFlags(ViewFamily.ViewExtensions[ViewExt]->GetFlags(), ESceneViewExtensionFlags::SubscribesToPostTLASBuild))
				{
					if (bNeedToSetupRayTracingRenderingData)
					{
						SetupRayTracingRenderingData(GraphBuilder, *InitViewTaskDatas.RayTracingGatherInstances);
						bNeedToSetupRayTracingRenderingData = false;
					}

					for (int32 ViewIndex = 0; ViewIndex < ViewFamily.Views.Num(); ++ViewIndex)
					{
						ViewFamily.ViewExtensions[ViewExt]->PostTLASBuild_RenderThread(GraphBuilder, Views[ViewIndex]);
					}
				}
			}
		}
#endif // RHI_RAYTRACING

		if (bNaniteEnabled)
		{
			// Needs doing after shadows such that the checks for shadow atlases etc work.
			Nanite::ListStatFilters(this);

			if (GNaniteShowStats != 0)
			{
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					const FViewInfo& View = Views[ViewIndex];
					if (IStereoRendering::IsAPrimaryView(View))
					{
						Nanite::PrintStats(GraphBuilder, View);
					}
				}
			}
		}

		{
			if (FVirtualShadowMapArrayCacheManager* CacheManager = VirtualShadowMapArray.CacheManager)
			{
				// Do this even if VSMs are disabled this frame to clean up any previously extracted data
				CacheManager->ExtractFrameData(
					GraphBuilder,
					VirtualShadowMapArray,
					*this,
					ViewFamily.EngineShowFlags.VirtualShadowMapPersistentData);
			}
		}

		if (CustomDepthPassLocation == ECustomDepthPassLocation::AfterBasePass)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_CustomDepthPass_AfterBasePass);
			if (RenderCustomDepthPass(GraphBuilder, SceneTextures.CustomDepth, SceneTextures.GetSceneTextureShaderParameters(FeatureLevel), NaniteRasterResults, PrimaryNaniteViews))
			{
				SceneTextures.SetupMode |= ESceneTextureSetupMode::CustomDepth;
				SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);
			}
		}

		// If we are not rendering velocities in depth or base pass then do that here.
		if (bShouldRenderVelocities && !bBasePassCanOutputVelocity && (Scene->EarlyZPassMode != DDM_AllOpaqueNoVelocity))
		{
			RenderVelocities(GraphBuilder, Views, SceneTextures, EVelocityPass::Opaque, bHairStrandsEnable);
		}

		// Pre-lighting composition lighting stage
		// e.g. deferred decals, SSAO
		{
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, AfterBasePass);
			SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_AfterBasePass);

			if (!IsForwardShadingEnabled(ShaderPlatform))
			{
				AddResolveSceneDepthPass(GraphBuilder, Views, SceneTextures.Depth);
			}

			const FCompositionLighting::EProcessAfterBasePassMode Mode = AsyncLumenIndirectLightingOutputs.bHasDrawnBeforeLightingDecals ?
				FCompositionLighting::EProcessAfterBasePassMode::SkipBeforeLightingDecals : FCompositionLighting::EProcessAfterBasePassMode::All;

			CompositionLighting.ProcessAfterBasePass(GraphBuilder, InstanceCullingManager, Mode, *SceneTextures.SubstrateSceneData);
		}

		// Rebuild scene textures to include velocity, custom depth, and SSAO.
		SceneTextures.SetupMode |= ESceneTextureSetupMode::All;
		SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);

		if (!IsForwardShadingEnabled(ShaderPlatform))
		{
			// Clear stencil to 0 now that deferred decals are done using what was setup in the base pass.
			AddClearStencilPass(GraphBuilder, SceneTextures.Depth.Target);
		}

#if RHI_RAYTRACING
		// If Lumen did not force an earlier ray tracing scene sync, we must wait for it here.
		if (bNeedToSetupRayTracingRenderingData)
		{
			SetupRayTracingRenderingData(GraphBuilder, *InitViewTaskDatas.RayTracingGatherInstances);
			bNeedToSetupRayTracingRenderingData = false;
		}
#endif // RHI_RAYTRACING

		GraphBuilder.FlushSetupQueue();

		TArray<FRDGTextureRef> DynamicBentNormalAOTextures;
		TArray<FSubsurfaceTiles> PrebuiltSSSTiles;

		if (bRenderDeferredLighting)
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, RenderDeferredLighting, "RenderDeferredLighting");
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderLighting);

			SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_Lighting);
			SCOPED_NAMED_EVENT(RenderLighting, FColor::Emerald);

			// Initialize the separated subsurface diffuse for subsurface scattering and build SSS tiles.
			RenderSubsurfaceDiffuseInitPass(GraphBuilder,Views, SceneTextures, /*out*/PrebuiltSSSTiles);

			RenderDiffuseIndirectAndAmbientOcclusion(
				GraphBuilder,
				SceneTextures,
				LumenFrameTemporaries,
				LightingChannelsTexture,
				/* bCompositeRegularLumenOnly = */ false,
				/* bIsVisualizePass = */ false,
				AsyncLumenIndirectLightingOutputs);

			MarkUsedTranslucencyVolumeVoxels();

			// These modulate the scenecolor output from the basepass, which is assumed to be indirect lighting
			RenderIndirectCapsuleShadows(GraphBuilder, SceneTextures);

			// These modulate the scene color output from the base pass, which is assumed to be indirect lighting
			RenderDFAOAsIndirectShadowing(GraphBuilder, SceneTextures, DynamicBentNormalAOTextures);

			// Clear the translucent lighting volumes before we accumulate
			if (!ShouldClearTranslucencyLightingVolumeInAsyncCompute())
			{
				TranslucencyLightingVolumeTextures.Init(GraphBuilder, Views, ERDGPassFlags::Compute);
			}

#if RHI_RAYTRACING
			// Only used by ray traced shadows
			if (IsRayTracingEnabled() && Views[0].bHasRayTracingShadows && Views[0].IsRayTracingAllowedForView())
			{
				RenderDitheredLODFadingOutMask(GraphBuilder, Views[0], SceneTextures.Depth.Target);
			}
#endif

			const FSortedLightSetSceneInfo& SortedLightSet = *GatherAndSortLightsTask.GetResult();

			if (!bGatheredTranslucencyVolumeMarkedVoxels)
			{
				GatherTranslucencyVolumeMarkedVoxels(GraphBuilder, SortedLightSet);
			}

			RenderLights(GraphBuilder, SceneTextures, LightingChannelsTexture, SortedLightSet);

			if (MegaLightsContext.IsValid())
			{
				RenderMegaLights(
					GraphBuilder,
					MegaLightsContext,
					SceneTextures,
					LightingChannelsTexture);
			}

			RenderTranslucencyLightingVolume(GraphBuilder, TranslucencyLightingVolumeTextures, SortedLightSet);
		}

		const bool bShouldRenderTranslucency = !bHasRayTracedOverlay && ShouldRenderTranslucency();

		// Union of all translucency view render flags.
		ETranslucencyView TranslucencyViewsToRender = bShouldRenderTranslucency ? GetTranslucencyViews(Views) : ETranslucencyView::None;

		FFrontLayerTranslucencyData FrontLayerTranslucencyData;
		if (!bHasRayTracedOverlay && TranslucencyViewsToRender != ETranslucencyView::None)
		{
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderTranslucency);
			SCOPED_NAMED_EVENT(RenderTranslucency, FColor::Emerald);
			SCOPE_CYCLE_COUNTER(STAT_TranslucencyDrawTime);

			RDG_EVENT_SCOPE(GraphBuilder, "FrontLayerTranslucency");

			// Lumen / MegaLights / VSM translucency front layer
			FrontLayerTranslucencyData = RenderFrontLayerTranslucency(GraphBuilder, Views, SceneTextures, MegaLightsContext, false /*VSM page marking*/);

			if (FrontLayerTranslucencyData.IsValid())
			{
				StochasticLightingFrontLayerTranslucencyTileClassificationMark(GraphBuilder, SceneTextures, FrontLayerTranslucencyData, ERDGPassFlags::AsyncCompute);
			}

			if (MegaLightsContext.IsValid())
			{
				RenderMegaLightsFrontLayerTranslucency(GraphBuilder, MegaLightsContext, LumenFrameTemporaries, FrontLayerTranslucencyData, ERDGPassFlags::AsyncCompute);
			}
		}

		if (bRenderDeferredLighting)
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, RenderDeferredLighting, "RenderDeferredLighting");
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderLighting);

			SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_Lighting);
			SCOPED_NAMED_EVENT(RenderLighting, FColor::Emerald);

			// Do DiffuseIndirectComposite after Lights so that async Lumen work can overlap
			RenderDiffuseIndirectAndAmbientOcclusion(
				GraphBuilder,
				SceneTextures,
				LumenFrameTemporaries,
				LightingChannelsTexture,
				/* bCompositeRegularLumenOnly = */ true,
				/* bIsVisualizePass = */ false,
				AsyncLumenIndirectLightingOutputs);

			// Render diffuse sky lighting and reflections that only operate on opaque pixels
			RenderDeferredReflectionsAndSkyLighting(GraphBuilder, SceneTextures, LumenFrameTemporaries, DynamicBentNormalAOTextures);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			// Renders debug visualizations for global illumination plugins
			RenderGlobalIlluminationPluginVisualizations(GraphBuilder, LightingChannelsTexture);
#endif

			AddSubsurfacePass(GraphBuilder, SceneTextures, Views, PrebuiltSSSTiles);

			Substrate::AddSubstrateOpaqueRoughRefractionPasses(GraphBuilder, SceneTextures, Views);

			{
				RenderHairStrandsSceneColorScattering(GraphBuilder, SceneTextures.Color.Target, Scene, Views);
			}

		#if RHI_RAYTRACING
			if (ShouldRenderRayTracingSkyLight(Scene->SkyLight, Scene->GetShaderPlatform()) 
				//@todo - integrate RenderRayTracingSkyLight into RenderDiffuseIndirectAndAmbientOcclusion
				&& GetViewPipelineState(Views[0]).DiffuseIndirectMethod != EDiffuseIndirectMethod::Lumen
				&& ViewFamily.EngineShowFlags.GlobalIllumination)
			{
				FRDGTextureRef SkyLightTexture = nullptr;
				FRDGTextureRef SkyLightHitDistanceTexture = nullptr;
				RenderRayTracingSkyLight(GraphBuilder, SceneTextures.Color.Target, SkyLightTexture, SkyLightHitDistanceTexture);
				CompositeRayTracingSkyLight(GraphBuilder, SceneTextures, SkyLightTexture, SkyLightHitDistanceTexture);
			}
		#endif

			if (Substrate::IsSubstrateEnabled())
			{
				// Now remove all the Substrate tile stencil tags used by deferred tiled light passes. Make later marks such as responssive AA works.
				AddClearStencilPass(GraphBuilder, SceneTextures.Depth.Target);
			}
		}
		else if (HairStrands::HasViewHairStrandsData(Views) && ViewFamily.EngineShowFlags.Lighting)
		{
			const FSortedLightSetSceneInfo& SortedLightSet = *GatherAndSortLightsTask.GetResult();
			RenderLightsForHair(GraphBuilder, SceneTextures, SortedLightSet, ForwardScreenSpaceShadowMaskHairTexture, LightingChannelsTexture);
			RenderDeferredReflectionsAndSkyLightingHair(GraphBuilder);
		}

		// Early translucency velocity pass.
		FTranslucencyVelocityContext TranslucencyVelocityContext;
		int32 EarlyVelocityLocation = FMath::Clamp(CVarTranslucencyEarlyVelocityPass.GetValueOnRenderThread(), 0, 3);
		bool bEarlyTranslucencyVelocity = [&]()
		{
			if (!bShouldRenderTranslucency
				|| !bShouldRenderVelocities
				|| bHasRayTracedOverlay
				|| !GSupportsEfficientAsyncCompute
				|| CVarTranslucencyVelocity.GetValueOnRenderThread() == 0
				|| EarlyVelocityLocation == 0)
			{
				return false;
			}
			if (Views.Num() == 0 || GetMainTAAPassConfig(Views[0]) != EMainTAAPassConfig::TSR)
			{
				return false;
			}
			if (HairStrands::HasHairStrandsVisible(Views)
				&& GetHairStrandsComposition() != EHairStrandsCompositionType::BeforeTranslucent)
			{
				return false;
			}
			for (const FViewInfo& View : Views)
			{
				if (HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::TranslucentVelocity])
					|| HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::TranslucentVelocityClippedDepth]))
				{
					return true;
				}
			}
			return false;
		}();
		
		// SLW base pass can write velocity after locations 1/2, so defer early translucency velocity to location 3 to ensure TSR has complete motion vectors.
		const bool bShouldRenderSingleLayerWater = !bHasRayTracedOverlay && ShouldRenderSingleLayerWater(Views);
		const bool bSingleLayerWaterPrepassVelocityComplete = CanSingleLayerWaterDepthPrepassOutputVelocity(ShaderPlatform, FeatureLevel)
			|| GetSingleLayerWaterVelocityOutputPass() == ESingleLayerWaterVelocityOutputPass::None;
		const bool bSingleLayerWaterBasePassShouldWriteVelocity = bShouldRenderSingleLayerWater
			&& ((SingleLayerWaterPrePassResult && !bSingleLayerWaterPrepassVelocityComplete)
				|| !bShouldRenderSingleLayerWaterDepthPrepass);

		if (bEarlyTranslucencyVelocity
			&& bSingleLayerWaterBasePassShouldWriteVelocity
			&& EarlyVelocityLocation < 3)
		{
			EarlyVelocityLocation = 3;
		}

		auto RenderEarlyTranslucencyVelocity = [&]()
		{
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderTranslucency);
			SCOPED_NAMED_EVENT(RenderTranslucency, FColor::Emerald);
			SCOPE_CYCLE_COUNTER(STAT_TranslucencyDrawTime);
			RDG_EVENT_SCOPE(GraphBuilder, "EarlyTranslucencyVelocity");
			
			// Use SLW prepass texture if available (has opaque + water).
			FRDGTextureRef DepthSource = SingleLayerWaterPrePassResult ? SingleLayerWaterPrePassResult->DepthPrepassTexture.Target : SceneTextures.Depth.Resolve;
			const FRDGTextureDesc& DepthDesc = DepthSource->Desc;
			FRDGTextureRef EarlyDepthTexture = GraphBuilder.CreateTexture(DepthDesc, TEXT("EarlyTranslucencyVelocityDepth"));

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
			{
				FViewInfo& View = Views[ViewIndex];
				if (View.ShouldRenderView())
				{
					RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
					DepthCopy::AddViewDepthCopyPass(GraphBuilder, View, DepthSource, EarlyDepthTexture, DepthCopy::EDepthCopyStencilMode::Keep);
				}
			}

			const bool bRecreateSceneTextures = !HasBeenProduced(SceneTextures.Velocity);

			RenderVelocities(GraphBuilder, Views, SceneTextures, EVelocityPass::Translucent, false, true, SceneTextures.Velocity, EarlyDepthTexture);			
			
			if (bRecreateSceneTextures)
			{
				SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);
			}

			RenderVelocities(GraphBuilder, Views, SceneTextures, EVelocityPass::TranslucentClippedDepth, false, false, SceneTextures.Velocity, EarlyDepthTexture);
			
			TranslucencyVelocityContext.EarlyVelocity = SceneTextures.Velocity;
			TranslucencyVelocityContext.EarlyDepth = EarlyDepthTexture;
		};

		if (bEarlyTranslucencyVelocity && EarlyVelocityLocation == 1)
		{
			RenderEarlyTranslucencyVelocity();
		}

		// Volumetric fog after Lumen GI and shadow depths
		if (!IsForwardShadingEnabled(ShaderPlatform) && !bHasRayTracedOverlay)
		{
			ComputeVolumetricFog(GraphBuilder, SceneTextures);
		}

		if (ShouldRenderHeterogeneousVolumes(Scene) && !bHasRayTracedOverlay)
		{
			RenderHeterogeneousVolumes(GraphBuilder, SceneTextures);
		}

		GraphBuilder.FlushSetupQueue();

		if (bShouldRenderVolumetricCloud && !bHasRayTracedOverlay)
		{
			if (!bAsyncComputeVolumetricCloud)
			{
				if(IsVolumetricRenderTargetEnabled())
				{
					GenerateQuarterResDepthMinMaxTexture(GraphBuilder, Views, SceneTextures.Depth.Resolve);
				}

				// Generate the volumetric cloud render target
				bool bSkipVolumetricRenderTarget = false;
				bool bSkipPerPixelTracing = true;
				RenderVolumetricCloud(GraphBuilder, SceneTextures, bSkipVolumetricRenderTarget, bSkipPerPixelTracing,
					SceneTextures.DitheredHalfResDepth, QuarterResolutionDepthMinMaxTexture, false, InstanceCullingManager);
			}
			// Reconstruct the volumetric cloud render target to be ready to compose it over the scene
			ReconstructVolumetricRenderTarget(GraphBuilder, Views, SceneTextures.Depth.Resolve, SceneTextures.DitheredHalfResDepth, bAsyncComputeVolumetricCloud);
		}

		if (bEarlyTranslucencyVelocity && EarlyVelocityLocation == 2)
		{
			RenderEarlyTranslucencyVelocity();
		}

		bool bSceneHasSkyMaterial = false;
		TArray<FScreenPassTexture, TInlineAllocator<4>> TSRFlickeringInputTextures;
		if (!bHasRayTracedOverlay)
		{
			// Extract TSR's moire heuristic luminance before rendering translucency into the scene color.
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
			{
				FViewInfo& View = Views[ViewIndex];
				bSceneHasSkyMaterial |= View.bSceneHasSkyMaterial;
				if (NeedTSRAntiFlickeringPass(View))
				{
					if (TSRFlickeringInputTextures.Num() == 0)
					{
						TSRFlickeringInputTextures.SetNum(Views.Num());
					}

					TSRFlickeringInputTextures[ViewIndex] = AddTSRMeasureFlickeringLuma(GraphBuilder, View.ShaderMap, FScreenPassTexture(SceneTextures.Color.Target, View.ViewRect));
				}
			}
		}

		FTranslucencyPassResourcesMap TranslucencyResourceMap(Views.Num());

		const bool bIsCameraUnderWater = EnumHasAnyFlags(TranslucencyViewsToRender, ETranslucencyView::UnderWater);
		FRDGTextureRef LightShaftOcclusionTexture = nullptr;
		FSceneWithoutWaterTextures SceneWithoutWaterTextures;
		auto RenderLightShaftSkyFogAndCloud = [&]()
		{
			// Draw Lightshafts
			if (!bHasRayTracedOverlay && ViewFamily.EngineShowFlags.LightShafts)
			{
				SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_RenderLightShaftOcclusion);
				LightShaftOcclusionTexture = RenderLightShaftOcclusion(GraphBuilder, SceneTextures);
			}

			const bool bShouldRenderFog = ShouldRenderFog(ViewFamily);
			const bool bIsForwardShadingEnabled = IsForwardShadingEnabled(ShaderPlatform);
			const bool bRenderSkyAtmosphere = !bHasRayTracedOverlay && bShouldRenderSkyAtmosphere && !bIsForwardShadingEnabled;
			// Testing bSceneHasSkyMaterial is enough to not not render sky at half res when there is no sky dome and the sky is traced per pixel for instance.
			// We do not render the SkyAtmosphere in separate composition if there is no sky materials, because in this case we will trace fullscreen.
			const bool bRenderSkyAtmosphereInFogSeparateComposition = bRenderSkyAtmosphere && bShouldRenderFog && bShouldRenderFogUsingSeparateComposition && bSceneHasSkyMaterial;

			const bool bComposeWithWater = bIsCameraUnderWater ? false : bShouldRenderSingleLayerWater;

			TArray<FFogSeparateCompositionViewResources> FogSeparateCompositionViewResources;
			FogSeparateCompositionViewResources.AddDefaulted(Views.Num());

			// Generate half resolution min/max dithered depth buffer and render half resolution fog here, to overlap with Lumen work on the async pipe.
			if (!bHasRayTracedOverlay && bShouldRenderFogUsingSeparateComposition)
			{
				// Render half resolution dithered depth texture for volumetric after water contribution to depth buffer.
				if (!SceneTextures.DitheredHalfResDepth)
				{
					SceneTextures.DitheredHalfResDepth = CreateHalfResolutionDepthCheckerboardMinMax(GraphBuilder, Views, SceneTextures.Depth.Resolve);
				}

				// Render volumetric layer at desired resolution
				{
					RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderFog);
					SCOPED_NAMED_EVENT(RenderFog, FColor::Emerald);
					SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_RenderFog);
					RDG_EVENT_SCOPE(GraphBuilder, "Fog::RenderSeparateCompositionTextures");

					// Here we render the fog component and capture part of the scene to scatter:
					//  - SkyAtmosphere AP is accumulated (Note: part of the sky box can be captured for scattering in this pass such as blue sky or the sun).
					//  - Height fog, local fog volumes and volumetric fog are then also accumulated.
					//  - VolumetricCloud is also accumulated but it does not influence FSSS blur to not blur itself. As of today this is simpler for artistic control.
					RenderFogSeparateCompositionTextures(GraphBuilder, SceneTextures, FogSeparateCompositionViewResources, LightShaftOcclusionTexture, bFogComposeLocalFogVolumes, bRenderSkyAtmosphereInFogSeparateComposition);
					bHeightFogHasComposedLocalFogVolume = bFogComposeLocalFogVolumes;

					if (bVolumetricRenderTargetRequired)
					{
						ComposeVolumetricRenderTargetOverScene(
							GraphBuilder, Scene, Views, SceneTextures.Color.Target, SceneTextures.Depth.Target,
							bComposeWithWater,
							SceneWithoutWaterTextures, SceneTextures,
							&FogSeparateCompositionViewResources);
					}

					RenderFogScreenSpaceScatteringMipChain(GraphBuilder, SceneTextures, FogSeparateCompositionViewResources);
				}
			}

			// Draw the sky atmosphere
			if (!bHasRayTracedOverlay && bShouldRenderSkyAtmosphere && !bRenderSkyAtmosphereInFogSeparateComposition && !IsForwardShadingEnabled(ShaderPlatform) && ViewFamily.EngineShowFlags.DeferredAtmospherePass)
			{
				SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_RenderSkyAtmosphere);
				RenderSkyAtmosphere(GraphBuilder, SceneTextures);
			}

			if (!bHasRayTracedOverlay && bShouldRenderFog)
			{
				RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderFog);
				SCOPED_NAMED_EVENT(RenderFog, FColor::Emerald);
				SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_RenderFog);

				if (bShouldRenderFogUsingSeparateComposition)
				{
					// Draw fog when using separate composition.
					RDG_EVENT_SCOPE(GraphBuilder, "Fog::UpsampleFogSeparateCompositionTexture");
					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
					{
						FViewInfo& View = Views[ViewIndex];
						FFogSeparateCompositionViewResources& FogSeparateCompositionViewResource = FogSeparateCompositionViewResources[ViewIndex];

						const bool bShouldRenderFogForView = FogSeparateCompositionViewResource.FogSeparateCompositionTexture0 != nullptr && FogSeparateCompositionViewResource.FogSeparateCompositionTexture1 != nullptr;
						if (!bShouldRenderFogForView)
						{
							continue;
						}

						RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
						RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

						UpsampleFogSeparateCompositionTextureForView(
							GraphBuilder,
							SceneTextures,
							FogSeparateCompositionViewResource,
							View,
							Scene);
					}
				}
				else
				{
					// Draw fog when not using separate composition.
					RenderFog(GraphBuilder, SceneTextures, LightShaftOcclusionTexture, bFogComposeLocalFogVolumes);
					bHeightFogHasComposedLocalFogVolume = bFogComposeLocalFogVolumes;
				}
			}

			if (!bHasRayTracedOverlay) 
			{
				// Local Fog Volumes (LFV) rendering order is first HeightFog, then LFV, then volumetric fog on top.
				// LFVs are rendered as part of the regular height fog + volumetric fog pass when volumetric fog is enabled and it is requested to voxelise LFVs into volumetric fog.
				// Otherwise, they are rendered in an independent pass (this for instance make it independent of the near clip plane optimization).
				if (!bHeightFogHasComposedLocalFogVolume)
				{
					RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderLocalFogVolume);
					SCOPED_NAMED_EVENT(RenderLocalFogVolume, FColor::Emerald);
					SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_RenderLocalFogVolume);
					RenderLocalFogVolume(Scene, Views, ViewFamily, GraphBuilder, SceneTextures, LightShaftOcclusionTexture);
				}
				// Also compose on top the visualisation pass if enabled.
				if (bShouldRenderLocalFogVolumeVisualizationPass)
				{
					RenderLocalFogVolumeVisualization(Scene, Views, ViewFamily, GraphBuilder, SceneTextures);
				}
			}

			// After the height fog, Draw volumetric clouds (having fog applied on them already) when using per pixel tracing,
			if (!bHasRayTracedOverlay && bShouldRenderVolumetricCloud)
			{
				bool bSkipVolumetricRenderTarget = true;
				bool bSkipPerPixelTracing = false;
				RenderVolumetricCloud(GraphBuilder, SceneTextures, bSkipVolumetricRenderTarget, bSkipPerPixelTracing,
					SceneTextures.DitheredHalfResDepth, QuarterResolutionDepthMinMaxTexture, false, InstanceCullingManager);
			}

			// Or composite the off screen buffer over the scene.
			if (bVolumetricRenderTargetRequired && !bShouldRenderFogUsingSeparateComposition)
			{
				ComposeVolumetricRenderTargetOverScene(
					GraphBuilder, Scene, Views, SceneTextures.Color.Target, SceneTextures.Depth.Target,
					bComposeWithWater,
					SceneWithoutWaterTextures, SceneTextures);
			}
		};

		if (bShouldRenderSingleLayerWater)
		{
			if (bIsCameraUnderWater)
			{
				RenderLightShaftSkyFogAndCloud();

				RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderTranslucency);
				SCOPED_NAMED_EVENT(RenderTranslucency, FColor::Emerald);
				SCOPE_CYCLE_COUNTER(STAT_TranslucencyDrawTime);
				const bool bStandardTranslucentCanRenderSeparate = false;
				FRDGTextureMSAA SharedDepthTexture;
				RenderTranslucency(*this, GraphBuilder, SceneTextures, TranslucencyLightingVolumeTextures, &TranslucencyResourceMap, Views, ETranslucencyView::UnderWater, SeparateTranslucencyDimensions, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, SingleLayerWaterPrePassResult, SharedDepthTexture, NaniteRasterResults);
				EnumRemoveFlags(TranslucencyViewsToRender, ETranslucencyView::UnderWater);
			}

			RenderSingleLayerWater(GraphBuilder, Views, SceneTextures, SingleLayerWaterPrePassResult, bShouldRenderVolumetricCloud, SceneWithoutWaterTextures, LumenFrameTemporaries, bIsCameraUnderWater);

			// Replace main depth texture with the output of the SLW depth prepass which contains the scene + water. Stencil is cleared to 0.
			if (SingleLayerWaterPrePassResult)
			{
				SceneTextures.Depth = SingleLayerWaterPrePassResult->DepthPrepassTexture;
			}
		}

		// Rebuild scene textures to include scene color.
		SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);

		if (!bHasRayTracedOverlay)
		{
			// Extract TSR's thin geometry coverage after SLW but before rendering translucency into the scene color.
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
			{
				FViewInfo& View = Views[ViewIndex];
				if (NeedTSRAntiFlickeringPass(View))
				{
					if (TSRFlickeringInputTextures.Num() == 0)
					{
						TSRFlickeringInputTextures.SetNum(Views.Num());
					}

					AddTSRMeasureThinGeometryCoverage(GraphBuilder, View.ShaderMap, SceneTextures, TSRFlickeringInputTextures[ViewIndex]);
				}
			}
		}

		if (!bIsCameraUnderWater)
		{
			RenderLightShaftSkyFogAndCloud();
		}

		FRDGTextureRef ExposureIlluminance = nullptr;
		if (!bHasRayTracedOverlay)
		{
			ExposureIlluminance = AddCalculateExposureIlluminancePass(GraphBuilder, Views, SceneTextures, TranslucencyLightingVolumeTextures, ExposureIlluminanceSetup);
		}

		RenderOpaqueFX(GraphBuilder, GetSceneViews(), GetSceneUniforms(), FXSystem, FeatureLevel, SceneTextures.UniformBuffer);

		FRendererModule& RendererModule = static_cast<FRendererModule&>(GetRendererModule());
		RendererModule.RenderPostOpaqueExtensions(GraphBuilder, Views, SceneTextures);

		if (Scene->GPUScene.ExecuteDeferredGPUWritePass(GraphBuilder, Views, EGPUSceneGPUWritePass::PostOpaqueRendering))
		{
			InstanceCullingManager.BeginDeferredCulling(GraphBuilder);
		}

		if (GetHairStrandsComposition() == EHairStrandsCompositionType::BeforeTranslucent)
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, HairRendering, "HairRendering");

			RenderHairComposition(GraphBuilder, Views, SceneTextures.Color.Target, SceneTextures.Depth.Target, SceneTextures.Velocity, TranslucencyResourceMap, SingleLayerWaterPrePassResult);
		}

#if DEBUG_ALPHA_CHANNEL
		if (ShouldMakeDistantGeometryTranslucent())
		{
			SceneTextures.Color = MakeDistanceGeometryTranslucent(GraphBuilder, Views, SceneTextures);
			SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);
		}
#endif

		// Experimental voxel test code
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
		
			Nanite::DrawVisibleBricks( GraphBuilder, *Scene, View, SceneTextures );
		}

		// Composite Heterogeneous Volumes
		if (!bHasRayTracedOverlay && ShouldRenderHeterogeneousVolumes(Scene) &&
			(GetHeterogeneousVolumesComposition() == EHeterogeneousVolumesCompositionType::BeforeTranslucent))
		{
			CompositeHeterogeneousVolumes(GraphBuilder, SceneTextures);
		}

		// Draw translucency.
		FRDGTextureMSAA TranslucencySharedDepthTexture;
		if (!bHasRayTracedOverlay && TranslucencyViewsToRender != ETranslucencyView::None)
		{
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderTranslucency);
			SCOPED_NAMED_EVENT(RenderTranslucency, FColor::Emerald);
			SCOPE_CYCLE_COUNTER(STAT_TranslucencyDrawTime);

			RDG_EVENT_SCOPE(GraphBuilder, "Translucency");

			// Raytracing doesn't need the distortion effect.
			const bool bShouldRenderDistortion = TranslucencyViewsToRender != ETranslucencyView::RayTracing && ShouldRenderDistortion();
			
			if (bEarlyTranslucencyVelocity && EarlyVelocityLocation == 3)
			{
				RenderEarlyTranslucencyVelocity();
			}

#if RHI_RAYTRACING
			if (EnumHasAnyFlags(TranslucencyViewsToRender, ETranslucencyView::RayTracing))
			{
				if (!RenderRayTracedTranslucency(GraphBuilder, SceneTextures, LumenFrameTemporaries, FrontLayerTranslucencyData))
				{
					RenderRayTracingTranslucency(GraphBuilder, SceneTextures.Color);
				}

				EnumRemoveFlags(TranslucencyViewsToRender, ETranslucencyView::RayTracing);
			}
#endif

			for (FViewInfo& View : Views)
			{
				if (IsLumenFrontLayerTranslucencyEnabled(View))
				{
					RenderLumenFrontLayerTranslucencyReflections(GraphBuilder, View, SceneTextures, LumenFrameTemporaries, FrontLayerTranslucencyData);
				}
			}

			// Sort objects' triangles
			for (FViewInfo& View : Views)
			{
				if (OIT::IsSortedTrianglesEnabled(View.GetShaderPlatform()))
				{
					OIT::AddSortTrianglesPass(GraphBuilder, View, Scene->OITSceneData, FTriangleSortingOrder::BackToFront);
				}
			}

			{
				// Render all remaining translucency views.
				const bool bStandardTranslucentCanRenderSeparate = bShouldRenderDistortion; // It is only needed to render standard translucent as separate when there is distortion (non self distortion of transmittance/specular/etc.)
				RenderTranslucency(*this, GraphBuilder, SceneTextures, TranslucencyLightingVolumeTextures, &TranslucencyResourceMap, Views, TranslucencyViewsToRender, SeparateTranslucencyDimensions, InstanceCullingManager, bStandardTranslucentCanRenderSeparate, SingleLayerWaterPrePassResult, TranslucencySharedDepthTexture, NaniteRasterResults);
			}

			// Compose hair before velocity/distortion pass since these pass write depth value, 
			// and this would make the hair composition fails in this cases.
			if (GetHairStrandsComposition() == EHairStrandsCompositionType::AfterTranslucent)
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, HairRendering, "HairRendering");

				RenderHairComposition(GraphBuilder, Views, SceneTextures.Color.Target, SceneTextures.Depth.Target, SceneTextures.Velocity, TranslucencyResourceMap, SingleLayerWaterPrePassResult);
			}

			if (bShouldRenderDistortion)
			{
				RenderDistortion(GraphBuilder, SceneTextures.Color.Target, SceneTextures.Depth.Target, SceneTextures.Velocity, TranslucencyResourceMap);
			}

			if (bShouldRenderVelocities && !bEarlyTranslucencyVelocity)
			{
				const bool bRecreateSceneTextures = !HasBeenProduced(SceneTextures.Velocity);

				RenderVelocities(GraphBuilder, Views, SceneTextures, EVelocityPass::Translucent, false);

				if (bRecreateSceneTextures)
				{
					// Rebuild scene textures to include newly allocated velocity.
					SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);
				}

				RenderVelocities(GraphBuilder, Views, SceneTextures, EVelocityPass::TranslucentClippedDepth, false, /*bBindRenderTarget=*/ false);
			}
			else if (bEarlyTranslucencyVelocity)
			{
				// Keep all stencil bits.
				TranslucencyVelocityContext.OriginalSceneDepthForStencil = SceneTextures.Depth.Resolve;
				SceneTextures.Depth.Resolve = TranslucencyVelocityContext.EarlyDepth;
				SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);
			}
		}
		else if (GetHairStrandsComposition() == EHairStrandsCompositionType::AfterTranslucent)
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, HairRendering, "HairRendering");

			RenderHairComposition(GraphBuilder, Views, SceneTextures.Color.Target, SceneTextures.Depth.Target, SceneTextures.Velocity, TranslucencyResourceMap, SingleLayerWaterPrePassResult);
		}

		if (!bHasRayTracedOverlay)
		{
			RenderLightShapeVisualization(GraphBuilder, SceneTextures, TranslucencyResourceMap);
		}

#if !UE_BUILD_SHIPPING
		if (CVarForceBlackVelocityBuffer.GetValueOnRenderThread())
		{
			SceneTextures.Velocity = SystemTextures.Black;

			// Rebuild the scene texture uniform buffer to include black.
			SceneTextures.UniformBuffer = CreateSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, FeatureLevel, SceneTextures.SetupMode);
		}
#endif

		{
			if (HairStrandsBookmarkParameters.HasInstances())
			{
				HairStrandsBookmarkParameters.SceneColorTexture = SceneTextures.Color.Target;
				HairStrandsBookmarkParameters.SceneDepthTexture = SceneTextures.Depth.Target;
				RenderHairStrandsDebugInfo(GraphBuilder, Scene, Views, HairStrandsBookmarkParameters);
			}
		}

		if (VirtualShadowMapArray.IsEnabled())
		{
			VirtualShadowMapArray.RenderDebugInfo(GraphBuilder, Views);
		}

		for (FViewInfo& View : Views)
		{
			ShadingEnergyConservation::Debug(GraphBuilder, View, SceneTextures);
		}

		if (!bHasRayTracedOverlay && ViewFamily.EngineShowFlags.LightShafts)
		{
			SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_RenderLightShaftBloom);
			RenderLightShaftBloom(GraphBuilder, SceneTextures, /* inout */ TranslucencyResourceMap);
		}

		{
			// Light shaft (rendered just above) can render in separate transluceny at low resolution according to r.SeparateTranslucencyScreenPercentage. 
			// So we can only upsample that buffer if required after the light shaft bloom pass.
			UpscaleTranslucencyIfNeeded(GraphBuilder, SceneTextures, TranslucencyViewsToRender, /* inout */ &TranslucencyResourceMap, TranslucencySharedDepthTexture);
			TranslucencyViewsToRender = ETranslucencyView::None;
		}

		FPathTracingResources PathTracingResources;

#if RHI_RAYTRACING
		if (FamilyPipelineState[&FFamilyPipelineState::bRayTracing])
		{
			// Path tracer requires the full ray tracing pipeline support, as well as specialized extra shaders.
			// Most of the ray tracing debug visualizations also require the full pipeline, but some support inline mode.
			
			if (ViewFamily.EngineShowFlags.PathTracing 
				&& FDataDrivenShaderPlatformInfo::GetSupportsPathTracing(Scene->GetShaderPlatform()))
			{
				for (const FViewInfo& View : Views)
				{
					RenderPathTracing(GraphBuilder, View, SceneTextures.UniformBuffer, SceneTextures.Color.Target, SceneTextures.Depth.Target,PathTracingResources);
				}
			}
			else if (ViewFamily.EngineShowFlags.RayTracingDebug)
			{
				// TODO: This will include visible bindings for all views, but we could potentially also provide a way to get visible bindings for a single view
				// Although that would require running deduplication logic separately for each view in VisibleRayTracingShaderBindingsFinalizeTask
				TConstArrayView<FRayTracingShaderBindingData> VisibleRayTracingShaderBindings = RayTracing::GetVisibleShaderBindings(*InitViewTaskDatas.RayTracingGatherInstances);

				for (const FViewInfo& View : Views)
				{
					FRayTracingPickingFeedback PickingFeedback = {};
					RenderRayTracingDebug(GraphBuilder, *Scene, View, SceneTextures, VisibleRayTracingShaderBindings, PickingFeedback);

					const bool bViewHasFarFieldInstances = RayTracingScene.GetNumNativeInstances(ERayTracingSceneLayer::FarField, View.GetRayTracingSceneViewHandle()) > 0;
					OnGetOnScreenMessages.AddLambda([this, &View, PickingFeedback, bViewHasFarFieldInstances](FScreenMessageWriter& ScreenMessageWriter)->void
						{
							RayTracingDebugDisplayOnScreenMessages(ScreenMessageWriter, View, bViewHasFarFieldInstances);
							RayTracingDisplayPicking(PickingFeedback, ScreenMessageWriter);
						});
				}
			}
		}
#endif
		RendererModule.RenderOverlayExtensions(GraphBuilder, Views, SceneTextures);

		if (ViewFamily.EngineShowFlags.PhysicsField && Scene->PhysicsField)
		{
			RenderPhysicsField(GraphBuilder, Views, Scene->PhysicsField, SceneTextures.Color.Target);
		}

		if (ViewFamily.EngineShowFlags.VisualizeDistanceFieldAO && ShouldRenderDistanceFieldLighting(Scene->DistanceFieldSceneData, Views))
		{
			// Use the skylight's max distance if there is one, to be consistent with DFAO shadowing on the skylight
			const float OcclusionMaxDistance = Scene->SkyLight && !Scene->SkyLight->bWantsStaticShadowing ? Scene->SkyLight->OcclusionMaxDistance : Scene->DefaultMaxDistanceFieldOcclusionDistance;
			TArray<FRDGTextureRef> DummyOutput;
			RenderDistanceFieldLighting(GraphBuilder, SceneTextures, FDistanceFieldAOParameters(OcclusionMaxDistance), DummyOutput, false, ViewFamily.EngineShowFlags.VisualizeDistanceFieldAO);
		}

		// Draw visualizations just before use to avoid target contamination
		if (ViewFamily.EngineShowFlags.VisualizeMeshDistanceFields || ViewFamily.EngineShowFlags.VisualizeGlobalDistanceField)
		{
			RenderMeshDistanceFieldVisualization(GraphBuilder, SceneTextures);
		}

		if (bRenderDeferredLighting)
		{
			RenderLumenMiscVisualizations(GraphBuilder, SceneTextures, LumenFrameTemporaries);
			RenderDiffuseIndirectAndAmbientOcclusion(
				GraphBuilder,
				SceneTextures,
				LumenFrameTemporaries,
				LightingChannelsTexture,
				/* bCompositeRegularLumenOnly = */ false,
				/* bIsVisualizePass = */ true,
				AsyncLumenIndirectLightingOutputs);
		}

		if (ViewFamily.EngineShowFlags.StationaryLightOverlap)
		{
			RenderStationaryLightOverlap(GraphBuilder, SceneTextures, LightingChannelsTexture);
		}

		// Composite Heterogeneous Volumes
		if (!bHasRayTracedOverlay && ShouldRenderHeterogeneousVolumes(Scene) &&
			(GetHeterogeneousVolumesComposition() == EHeterogeneousVolumesCompositionType::AfterTranslucent))
		{
			CompositeHeterogeneousVolumes(GraphBuilder, SceneTextures);
		}

		if (bShouldVisualizeVolumetricCloud && !bHasRayTracedOverlay)
		{
			RenderVolumetricCloud(GraphBuilder, SceneTextures, false, true, SceneTextures.DitheredHalfResDepth, QuarterResolutionDepthMinMaxTexture, false, InstanceCullingManager);
			ReconstructVolumetricRenderTarget(GraphBuilder, Views, SceneTextures.Depth.Resolve, SceneTextures.DitheredHalfResDepth, false);
			ComposeVolumetricRenderTargetOverSceneForVisualization(GraphBuilder, Views, SceneTextures.Color.Target, SceneTextures);
			RenderVolumetricCloud(GraphBuilder, SceneTextures, true, false, SceneTextures.DitheredHalfResDepth, QuarterResolutionDepthMinMaxTexture, false, InstanceCullingManager);
		}

		if (!bHasRayTracedOverlay)
		{
			AddSparseVolumeTextureViewerRenderPass(GraphBuilder, *this, SceneTextures);
		}

		RenderTranslucencyVolumeVisualization(GraphBuilder, SceneTextures, TranslucencyLightingVolumeTextures);

		// Resolve the scene color for post processing.
		AddResolveSceneColorPass(GraphBuilder, Views, SceneTextures.Color);

		RendererModule.RenderPostResolvedSceneColorExtension(GraphBuilder, SceneTextures);

		CopySceneCaptureComponentToTarget(GraphBuilder, SceneTextures, ViewFamilyTexture, ViewFamilyDepthTexture, ViewFamily, Views);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];

			if (((View.FinalPostProcessSettings.DynamicGlobalIlluminationMethod == EDynamicGlobalIlluminationMethod::ScreenSpace && ScreenSpaceRayTracing::ShouldKeepBleedFreeSceneColor(View))
				|| GetViewPipelineState(View).DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen
				|| GetViewPipelineState(View).ReflectionsMethod == EReflectionsMethod::Lumen)
				&& !View.bStatePrevViewInfoIsReadOnly)
			{
				// Keep scene color and depth for next frame screen space ray tracing.
				FSceneViewState* ViewState = View.ViewState;
				GraphBuilder.QueueTextureExtraction(SceneTextures.Depth.Resolve, &ViewState->PrevFrameViewInfo.DepthBuffer);
				GraphBuilder.QueueTextureExtraction(SceneTextures.Color.Resolve, &ViewState->PrevFrameViewInfo.ScreenSpaceRayTracingInput);
			}
		}

		// Finish rendering for each view.
		if (ViewFamily.bResolveScene && ViewFamilyTexture)
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, Postprocessing, "PostProcessing");
			SCOPED_NAMED_EVENT(PostProcessing, FColor::Emerald);

			FinishUpdateExposureCompensationCurveLUT(GraphBuilder.RHICmdList, &UpdateExposureCompensationCurveLUTTaskData);

			FPostProcessingInputs PostProcessingInputs;
			PostProcessingInputs.ViewFamilyTexture = ViewFamilyTexture;
			PostProcessingInputs.ViewFamilyDepthTexture = ViewFamilyDepthTexture;
			PostProcessingInputs.CustomDepthTexture = SceneTextures.CustomDepth.Depth;
			PostProcessingInputs.ExposureIlluminance = ExposureIlluminance;
			PostProcessingInputs.SceneTextures = SceneTextures.UniformBuffer;
			PostProcessingInputs.bSeparateCustomStencil = SceneTextures.CustomDepth.bSeparateStencilBuffer;
			PostProcessingInputs.PathTracingResources = PathTracingResources;
			PostProcessingInputs.OriginalSceneDepthForStencil = TranslucencyVelocityContext.OriginalSceneDepthForStencil;

			FRDGTextureRef InstancedEditorDepthTexture = nullptr; // Used to pass instanced stereo depth data from primary to secondary views

			GraphBuilder.FlushSetupQueue();

			if (ViewFamily.UseDebugViewPS())
			{
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					const FViewInfo& View = Views[ViewIndex];
					const Nanite::FRasterResults* NaniteResults = bNaniteEnabled ? &NaniteRasterResults[ViewIndex] : nullptr;
					RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
					RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
					PostProcessingInputs.TranslucencyViewResourcesMap = FTranslucencyViewResourcesMap(TranslucencyResourceMap, ViewIndex);
					AddDebugViewPostProcessingPasses(GraphBuilder, View, ViewIndex, GetSceneUniforms(), PostProcessingInputs, NaniteResults, &VirtualShadowMapArray);
				}
			}
			else
			{
				for (int32 ViewExt = 0; ViewExt < ViewFamily.ViewExtensions.Num(); ++ViewExt)
				{
					for (int32 ViewIndex = 0; ViewIndex < ViewFamily.Views.Num(); ++ViewIndex)
					{
						FViewInfo& View = Views[ViewIndex];
						RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
						PostProcessingInputs.TranslucencyViewResourcesMap = FTranslucencyViewResourcesMap(TranslucencyResourceMap, ViewIndex);
						ViewFamily.ViewExtensions[ViewExt]->PrePostProcessPass_RenderThread(GraphBuilder, View, PostProcessingInputs);
					}
				}
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					const FViewInfo& View = Views[ViewIndex];
					const int32 NaniteResultsIndex = View.bIsInstancedStereoEnabled ? View.PrimaryViewIndex : ViewIndex;
					const Nanite::FRasterResults* NaniteResults = bNaniteEnabled ? &NaniteRasterResults[NaniteResultsIndex] : nullptr;
					RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
					RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

					PostProcessingInputs.TranslucencyViewResourcesMap = FTranslucencyViewResourcesMap(TranslucencyResourceMap, ViewIndex);

					if (IsPostProcessVisualizeCalibrationMaterialEnabled(View))
					{
						const UMaterialInterface* DebugMaterialInterface = GetPostProcessVisualizeCalibrationMaterialInterface(View);
						check(DebugMaterialInterface);

						AddVisualizeCalibrationMaterialPostProcessingPasses(GraphBuilder, View, PostProcessingInputs, DebugMaterialInterface);
					}
					else
					{
						const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);

						FScreenPassTexture TSRFlickeringInput;
						if (ViewIndex < TSRFlickeringInputTextures.Num())
						{
							TSRFlickeringInput = TSRFlickeringInputTextures[ViewIndex];
						}

						// If we're using instanced stereo, only the primary view simple element collectors will be populated with elements.
						// However, since post processing is always rendered per-view, we need to mirror the collectors to any instanced secondary views.
						if (View.bIsSinglePassStereo && View.StereoPass == EStereoscopicPass::eSSP_SECONDARY)
						{
							const FViewInfo& PrimaryView = Views[View.PrimaryViewIndex];

							View.SimpleElementCollector = PrimaryView.SimpleElementCollector;
							View.EditorSimpleElementCollector = PrimaryView.EditorSimpleElementCollector;
#if UE_ENABLE_DEBUG_DRAWING
							View.DebugSimpleElementCollector = PrimaryView.DebugSimpleElementCollector;
#endif
						}

						AddPostProcessingPasses(
							GraphBuilder,
							View, ViewIndex,
							GetSceneUniforms(),
							ViewPipelineState.DiffuseIndirectMethod,
							ViewPipelineState.ReflectionsMethod,
							PostProcessingInputs,
							NaniteResults,
							InstanceCullingManager,
							&VirtualShadowMapArray,
							LumenFrameTemporaries,
							MegaLightsContext,
							SceneWithoutWaterTextures,
							TSRFlickeringInput,
							InstancedEditorDepthTexture);
					}
				}
			}
		}

		if (bUseVirtualTexturing)
		{
			VirtualTexture::EndFeedback(GraphBuilder);
		}

		// After AddPostProcessingPasses in case of Lumen Visualizations writing to feedback
		FinishGatheringLumenSurfaceCacheFeedback(GraphBuilder, Views[0], LumenFrameTemporaries, SceneTextures);

		QueueExtractStochasticLighting(GraphBuilder, LumenFrameTemporaries, FrontLayerTranslucencyData, SceneTextures);

#if RHI_RAYTRACING
		RayTracingScene.PostRender(GraphBuilder);
#endif

		if (ViewFamily.bResolveScene && ViewFamilyTexture)
		{
			GVRSImageManager.DrawDebugPreview(GraphBuilder, ViewFamily, ViewFamilyTexture);
		}

		GEngine->GetPostRenderDelegateEx().Broadcast(GraphBuilder);
	}

	FinishUpdateExposureCompensationCurveLUT(GraphBuilder.RHICmdList, &UpdateExposureCompensationCurveLUTTaskData);
	
	GetSceneExtensionsRenderers().PostRender(GraphBuilder);

#if WITH_MGPU
	if (ViewFamily.bMultiGPUForkAndJoin)
	{
		DoCrossGPUTransfers(GraphBuilder, ViewFamilyTexture, Views, CrossGPUTransferFencesDefer.Num() > 0, RenderTargetGPUMask, CrossGPUTransferDeferred.GetReference());
	}
	FlushCrossGPUTransfers(GraphBuilder);
#endif

	{
		SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_RenderFinish);

		RDG_EVENT_SCOPE_STAT(GraphBuilder, FrameRenderFinish, "FrameRenderFinish");

		OnRenderFinish(GraphBuilder, ViewFamilyTexture);
		GraphBuilder.AddDispatchHint();
		GraphBuilder.FlushSetupQueue();
	}

	QueueSceneTextureExtractions(GraphBuilder, SceneTextures);

	HairStrands::PostRender(*Scene);
	HeterogeneousVolumes::PostRender(*Scene, Views);

#if RHI_RAYTRACING
	Nanite::GRayTracingManager.PostRender();
#endif // RHI_RAYTRACING

	// Release the view's previous frame histories so that their memory can be reused at the graph's execution.
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		Views[ViewIndex].PrevViewInfo = FPreviousViewInfo();
	}

	if (NaniteBasePassVisibility.Visibility)
	{
		NaniteBasePassVisibility.Visibility->FinishVisibilityFrame();
		NaniteBasePassVisibility.Visibility = nullptr;
	}

	if (Scene->InstanceCullingOcclusionQueryRenderer)
	{
		Scene->InstanceCullingOcclusionQueryRenderer->EndFrame(GraphBuilder);
	}
}

#if RHI_RAYTRACING

static bool AnyRayTracingPassEnabled(const FScene* Scene, const FViewInfo& View, bool bSceneHasRayTracedShadows)
{
	if (!IsRayTracingEnabled(View.GetShaderPlatform()) || Scene == nullptr)
	{
		return false;
	}

	// Path tracer, ray tracing visualization debug modes, and sky light ray tracing force ray tracing on, regardless of what the view says
	if (View.Family->EngineShowFlags.PathTracing
		|| View.Family->EngineShowFlags.RayTracingDebug
		|| ShouldRenderRayTracingSkyLight(Scene->SkyLight, View.GetShaderPlatform()))
	{
		return true;
	}

	if (!View.IsRayTracingAllowedForView())
	{
		return false;
	}

	return bSceneHasRayTracedShadows
		|| ShouldRenderRayTracingAmbientOcclusion(View)
		|| ShouldRenderRayTracingTranslucency(View)
		|| ShouldRenderRayTracingShadows(*View.Family)
		|| ShouldRenderPluginRayTracingGlobalIllumination(View)
        || Lumen::AnyLumenHardwareRayTracingPassEnabled(Scene, View)
		|| MegaLights::UseHardwareRayTracing(*View.Family);
}

static bool ShouldRenderRayTracingEffectInternal(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, EShaderPlatform ShaderPlatform)
{
	const bool bAllowPipeline = GRHISupportsRayTracingShaders &&
								ShouldCompileRayTracingShadersForProject(ShaderPlatform) && 
								CVarRayTracingAllowPipeline.GetValueOnRenderThread() &&
								EnumHasAnyFlags(CompatibilityFlags, ERayTracingPipelineCompatibilityFlags::FullPipeline);

	const bool bAllowInline = GRHISupportsInlineRayTracing && 
							  CVarRayTracingAllowInline.GetValueOnRenderThread() &&
							  EnumHasAnyFlags(CompatibilityFlags, ERayTracingPipelineCompatibilityFlags::Inline);

	// Disable the effect if current machine does not support the full ray tracing pipeline and the effect can't fall back to inline mode or vice versa.
	if (!bAllowPipeline && !bAllowInline)
	{
		return false;
	}

	const int32 OverrideMode = CVarForceAllRayTracingEffects.GetValueOnRenderThread();

	if (OverrideMode >= 0)
	{
		return OverrideMode > 0;
	}
	else
	{
		return bEffectEnabled;
	}
}

// Most ray tracing effects can be enabled or disabled per view, but the ray tracing sky light effect specifically requires base pass shaders
// in the FScene to be configured differently, and thus can't work if ray tracing is disabled.  There is logic in FScene::Update where
// bCachedShouldRenderSkylightInBasePass is updated based on the result of ShouldRenderSkylightInBasePass(), which is affected by whether sky light
// ray tracing is enabled.  When this value changes, bScenesPrimitivesNeedStaticMeshElementUpdate is set to true, forcing a rebuild of all static mesh
// elements in the scene.  This can't be done per frame (never mind per view), which would be required to allow this setting to vary, at least with
// the current implementation.  Sky light ray tracing is often used for cinematic capture, and not in games, so hopefully this isn't a big limitation.

bool ShouldRenderRayTracingEffect(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, EShaderPlatform ShaderPlatform)
{
	if (!IsRayTracingEnabled(ShaderPlatform))
	{
		return false;
	}

	return ShouldRenderRayTracingEffectInternal(bEffectEnabled, CompatibilityFlags, ShaderPlatform);
}

bool ShouldRenderRayTracingEffect(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, const FSceneView& View)
{
	if (!View.IsRayTracingAllowedForView())
	{
		return false;
	}

	return ShouldRenderRayTracingEffect(bEffectEnabled, CompatibilityFlags, View.GetShaderPlatform());
}

bool ShouldRenderRayTracingEffect(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, const FSceneViewFamily& ViewFamily)
{
	// TODO:  Should this check if ALL views have ray tracing?  ANY views have ray tracing?  Assert that all are the same?  All or any depending
	// on the specific feature or use case?  In practice, current examples (split screen or scene captures) will have ray tracing set the same
	// for all views, so we'll just check the first view of given a family, but having it be a separate function lets us reconsider that approach
	// in the future.
	return ShouldRenderRayTracingEffect(bEffectEnabled, CompatibilityFlags, *ViewFamily.Views[0]);
}

bool HasRaytracingDebugViewModeRaytracedOverlay(const FSceneViewFamily& ViewFamily);
bool HasRayTracedOverlay(const FSceneViewFamily& ViewFamily)
{
	// Return true if a full screen ray tracing pass will be displayed on top of the raster pass.
	// This can be used to skip certain calculations.
	// Must also verify ray tracing is actually supported by the RHI at runtime.
	if (ViewFamily.Views.IsEmpty() || !IsRayTracingEnabled(ViewFamily.Views[0]->GetShaderPlatform()))
	{
		return false;
	}

	return
		ViewFamily.EngineShowFlags.PathTracing ||
		(ViewFamily.EngineShowFlags.RayTracingDebug && HasRaytracingDebugViewModeRaytracedOverlay(ViewFamily));
}

void FDeferredShadingSceneRenderer::InitializeRayTracingFlags_RenderThread()
{
	bool bRayTracingShadows = false;
	bool bRayTracing = false;

	// We currently don't need a full list of RT lights, only whether there are any RT lights at all.
	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
	{
		if (GetLightOcclusionType(LightSceneInfoCompact, ViewFamily) == ELightOcclusionType::Raytraced)
		{
			bRayTracingShadows = true;
			break;
		}
	}

	for (FViewInfo& View : Views)
	{
		const bool bViewHasRayTracing = AnyRayTracingPassEnabled(Scene, View, bRayTracingShadows);

		View.bHasAnyRayTracingPass = bViewHasRayTracing;
		View.bHasRayTracingShadows = bRayTracingShadows;

		bRayTracing |= bViewHasRayTracing;
	}

	FamilyPipelineState.Set(&FFamilyPipelineState::bRayTracingShadows, bRayTracingShadows);
	FamilyPipelineState.Set(&FFamilyPipelineState::bRayTracing, bRayTracing);
}
#endif // RHI_RAYTRACING
