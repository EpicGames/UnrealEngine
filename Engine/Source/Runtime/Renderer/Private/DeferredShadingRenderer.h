// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeferredShadingRenderer.h: Scene rendering definitions.
=============================================================================*/

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "DeferredShadingRendererTypes.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "HAL/Platform.h"
#include "Lumen/LumenSceneCardCapture.h"
#include "Lumen/LumenIndirect.h"
#include "RendererInterface.h"
#include "RenderGraphFwd.h"
#include "RenderGraphUtils.h"
#include "RenderResource.h"
#include "RHIFwd.h"
#include "RHIResources.h"
#include "ScreenSpaceDenoise.h"
#include "SceneRendering.h"
#include "Tasks/Task.h"
#include "Templates/FunctionFwd.h"
#include "TranslucentRendering.h"
#include "HAL/PlatformMemory.h"


enum class ERayTracingPrimaryRaysFlag : uint32;
enum class ESingleLayerWaterPrepassLocation : uint8;
enum class ETranslucencyView;
namespace ETranslucencyPass { enum Type : int; }
enum class ERHIPipeline : uint8;

class FScene;
class FLumenCardUpdateContext;
class FSceneTextureParameters;
class FDistanceFieldCulledObjectBufferParameters;
class FTileIntersectionParameters;
class FDistanceFieldAOParameters;
class UStaticMeshComponent;
class FExponentialHeightFogSceneInfo;
class FLumenCardScatterContext;
class FLumenCardRenderer;
struct FLumenReflectionsConfig;
class FLumenMeshSDFGridParameters;
class FLumenScreenSpaceBentNormalParameters;
class FLumenUpsampleParameters;
struct FAsyncLumenIndirectLightingOutputs;
class FCardPageRenderData;
struct FCardCaptureAtlas;
struct FResampledCardCaptureAtlas;
namespace LumenRadianceCache
{
	class FRadianceCacheInputs;
	class FRadianceCacheInterpolationParameters;
	class FUpdateInputs;
}
namespace LumenRadiosity
{
	struct FFrameTemporaries;
}
namespace HybridIndirectLighting
{
	class FCommonParameters;
}
enum class ELumenReflectionPass;
class FRenderLightParameters;
class FRayTracingScene;
class FNaniteVisibility;
class FDecalVisibilityTaskData;
class FFogSeparateCompositionViewResources;
class FSSDSignalTextures;

struct FNaniteShadingCommands;
struct FNaniteVisibilityQuery;
class FNaniteVisibility;
namespace Nanite
{
	struct FRasterResults;
	struct FPackedView;
};
struct FFrontLayerTranslucencyData;
struct FSeparateTranslucencyDimensions;
struct FTranslucencyPassResourcesMap;
struct FTranslucencyLightingVolumeTextures;
struct FSceneWithoutWaterTextures;
struct FRayTracingReflectionOptions;
struct FHairStrandsTransmittanceMaskData;
struct FVolumetricFogLocalLightFunctionInfo;
struct FLumenSceneFrameTemporaries;
struct FSingleLayerWaterPrePassResult;
struct FBuildHZBAsyncComputeParams;
struct FForwardBasePassTextures;
struct FTranslucentLightInjectionCollector;
struct FRayTracingPickingFeedback;
struct FILCUpdatePrimTaskData;
struct FLumenDirectLightingTaskData;
struct FMegaLightsFrameTemporaries;
class FLightSceneInfo;
struct FSortedLightSetSceneInfo;
class FVirtualShadowMapUniformParameters;
class FHitProxyConsumer;
class FInstanceCullingManager;
struct FSceneTextures;
struct FSceneTexturesConfig;
struct FMinimalSceneTextures;
class FSceneTextureParameters;
class FSceneTextureUniformParameters;
struct FSubstrateSceneData;
struct FDynamicShadowsTaskData;
class FLightPrimitiveInteraction;
class FViewInfo;
class FSceneViewFamily;
struct FSceneRenderUpdateInputs;
class FGlobalShaderMap;
struct FPreviousViewInfo;
class FMaterialRenderProxy;
struct FPathTracingResources;
struct FRenderTargetBindingSlots;
class FProjectedShadowInfo;
struct FScreenMessageWriter;
class FRayTracingShaderBindingData;

namespace Froxel
{
	class FFroxelRenderer;
	class FRenderer;
}

class IVisibilityTaskData;

namespace RayTracing
{
	struct FGatherInstancesTaskData;
}

/** Encapsulation of the pipeline state of the renderer that have to deal with very large number of dimensions
 * and make sure there is no cycle dependencies in the dimensions by setting them ordered by memory offset in the structure.
 */
template<typename PermutationVectorType>
class TPipelineState
{
public:
	TPipelineState()
	{
		FPlatformMemory::Memset(&Vector, 0, sizeof(Vector));
	}

	/** Set a member of the pipeline state committed yet. */
	template<typename DimensionType>
	void Set(DimensionType PermutationVectorType::*Dimension, const DimensionType& DimensionValue)
	{
		SIZE_T ByteOffset = GetByteOffset(Dimension);

		// Make sure not updating a value of the pipeline already initialized, to ensure there is no cycle in the dependency of the different dimensions.
		checkf(ByteOffset >= InitializedOffset, TEXT("This member of the pipeline state has already been committed."));

		Vector.*Dimension = DimensionValue;

		// Update the initialised offset to make sure this is not set only once.
		InitializedOffset = ByteOffset + sizeof(DimensionType);
	}

	/** Commit the pipeline state to its final immutable value. */
	void Commit()
	{
		// Force the pipeline state to be initialized exactly once.
		checkf(!IsCommitted(), TEXT("Pipeline state has already been committed."));
		InitializedOffset = ~SIZE_T(0);
	}

	/** Returns whether the pipeline state has been fully committed to its final immutable value. */
	bool IsCommitted() const
	{
		return InitializedOffset == ~SIZE_T(0);
	}

	/** Access a member of the pipeline state, even when the pipeline state hasn't been fully committed to it's final value yet. */
	template<typename DimensionType>
	const DimensionType& operator [](DimensionType PermutationVectorType::*Dimension) const
	{
		SIZE_T ByteOffset = GetByteOffset(Dimension);

		checkf(ByteOffset < InitializedOffset, TEXT("This dimension has not been initialized yet."));

		return Vector.*Dimension;
	}

	/** Access the fully committed pipeline state structure. */
	const PermutationVectorType* operator->() const
	{
		// Make sure the pipeline state is committed to catch accesses to uninitialized settings. 
		checkf(IsCommitted(), TEXT("The pipeline state needs to be fully commited before being able to reference directly the pipeline state structure."));
		return &Vector;
	}

	/** Access the fully committed pipeline state structure. */
	const PermutationVectorType& operator * () const
	{
		// Make sure the pipeline state is committed to catch accesses to uninitialized settings. 
		checkf(IsCommitted(), TEXT("The pipeline state needs to be fully commited before being able to reference directly the pipeline state structure."));
		return Vector;
	}

private:

	template<typename DimensionType>
	static SIZE_T GetByteOffset(DimensionType PermutationVectorType::*Dimension)
	{
		return (SIZE_T)(&(((PermutationVectorType*) 0)->*Dimension));
	}

	PermutationVectorType Vector;

	SIZE_T InitializedOffset = 0;
};


/**
 * Encapsulates the resources and render targets used by global illumination plugins.
 */
class FGlobalIlluminationPluginResources : public FRenderResource

{
public:
	FRDGTextureRef GBufferA;
	FRDGTextureRef GBufferB;
	FRDGTextureRef GBufferC;
	FRDGTextureRef SceneDepthZ;
	FRDGTextureRef SceneColor;
	FRDGTextureRef LightingChannelsTexture;
};

/**
 * Delegate callback used by global illumination plugins
 */
class FGlobalIlluminationPluginDelegates
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FAnyRayTracingPassEnabled, bool& /*bAnyRayTracingPassEnabled*/);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FPrepareRayTracing, const FViewInfo& /*View*/, TArray<FRHIRayTracingShader*>& /*OutRayGenShaders*/);
	DECLARE_MULTICAST_DELEGATE_FourParams(FRenderDiffuseIndirectLight, const FScene& /*Scene*/, const FViewInfo& /*View*/, FRDGBuilder& /*GraphBuilder*/, FGlobalIlluminationPluginResources& /*Resources*/);

	static RENDERER_API FAnyRayTracingPassEnabled& AnyRayTracingPassEnabled();
	static RENDERER_API FPrepareRayTracing& PrepareRayTracing();
	static RENDERER_API FRenderDiffuseIndirectLight& RenderDiffuseIndirectLight();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	DECLARE_MULTICAST_DELEGATE_FourParams(FRenderDiffuseIndirectVisualizations, const FScene& /*Scene*/, const FViewInfo& /*View*/, FRDGBuilder& /*GraphBuilder*/, FGlobalIlluminationPluginResources& /*Resources*/);
	static RENDERER_API FRenderDiffuseIndirectVisualizations& RenderDiffuseIndirectVisualizations();
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
};

/**
 * Scene renderer that implements a deferred shading pipeline and associated features.
 */
class FDeferredShadingSceneRenderer : public FSceneRenderer
{
public:

	FLumenCardRenderer LumenCardRenderer;

	FDeferredShadingSceneRenderer(const FSceneViewFamily* InViewFamily, FHitProxyConsumer* HitProxyConsumer);

	/** Determine and commit the final state of the pipeline for the view family and views. */
	void CommitFinalPipelineState();

	/** Commit all the pipeline state for indirect lighting. */
	void CommitIndirectLightingState();

	/** Clears a view */
	void ClearView(FRHICommandListImmediate& RHICmdList);

	/**
	 * Renders the scene's prepass for a particular view
	 * @return true if anything was rendered
	 */
	void RenderPrePassView(FRHICommandList& RHICmdList, const FViewInfo& View);

	/**
	 * Renders the scene's prepass for a particular view in parallel
	 * @return true if the depth was cleared
	 */
	bool RenderPrePassViewParallel(const FViewInfo& View, FRHICommandListImmediate& ParentCmdList,TFunctionRef<void()> AfterTasksAreStarted, bool bDoPrePre);

	/** 
	 * Debug light grid content on screen.
	 */
	void DebugLightGrid(FRDGBuilder& GraphBuilder, FSceneTextures& SceneTextures, bool bNeedLightGrid);

	/**
	 * The following three functions are static, for compile time enforcement related to CustomRenderPass rendering, which uses these functions.
	 * Custom Render Passes have a separate ViewFamily, and making these functions static prevents the ViewFamily member in the scene renderer
	 * class from being inadvertently accessed.
	 */
	static void RenderBasePass(
		FDeferredShadingSceneRenderer& Renderer,
		FRDGBuilder& GraphBuilder,
		TArrayView<FViewInfo> InViews,
		FSceneTextures& SceneTextures,
		FExclusiveDepthStencil::Type BasePassDepthStencilAccess,
		FRDGTextureRef ForwardShadowMaskTexture,
		FInstanceCullingManager& InstanceCullingManager,
		bool bNaniteEnabled,
		struct FNaniteShadingCommands& NaniteBasePassShadingCommands,
		const TArrayView<Nanite::FRasterResults>& NaniteRasterResults);

	static void RenderBasePassInternal(
		FDeferredShadingSceneRenderer& Renderer,
		FRDGBuilder& GraphBuilder,
		TArrayView<FViewInfo> InViews,
		const FSceneTextures& SceneTextures,
		const FRenderTargetBindingSlots& BasePassRenderTargets,
		FExclusiveDepthStencil::Type BasePassDepthStencilAccess,
		const FForwardBasePassTextures& ForwardBasePassTextures,
		bool bParallelBasePass,
		bool bRenderLightmapDensity,
		FInstanceCullingManager& InstanceCullingManager,
		bool bNaniteEnabled,
		struct FNaniteShadingCommands& NaniteBasePassShadingCommands,
		const TArrayView<Nanite::FRasterResults>& NaniteRasterResults);

	static void RenderAnisotropyPass(
		FRDGBuilder& GraphBuilder,
		TArrayView<FViewInfo> InViews,
		FSceneTextures& SceneTextures,
		const FScene* Scene,
		bool bDoParallelPass);
	/**
	 * Runs water pre-pass if enabled and returns an RDG-allocated object with intermediates, or null.
	 */
	FSingleLayerWaterPrePassResult* RenderSingleLayerWaterDepthPrepass(
		FRDGBuilder& GraphBuilder,
		TArrayView<FViewInfo> InViews,
		const FSceneTextures& SceneTextures,
		ESingleLayerWaterPrepassLocation Location, 
		TConstArrayView<Nanite::FRasterResults> NaniteRasterResults);

	void RenderSingleLayerWater(
		FRDGBuilder& GraphBuilder,
		TArrayView<FViewInfo> InViews,
		const FSceneTextures& SceneTextures,
		const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult,
		bool bShouldRenderVolumetricCloud,
		FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
		FLumenSceneFrameTemporaries& LumenFrameTemporaries,
		bool bIsCameraUnderWater);

	void RenderSingleLayerWaterInner(
		FRDGBuilder& GraphBuilder,
		TArrayView<FViewInfo> InViews,
		const FSceneTextures& SceneTextures,
		const FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
		const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult);

	void RenderSingleLayerWaterReflections(
		FRDGBuilder& GraphBuilder,
		TArrayView<FViewInfo> InViews,
		const FSceneTextures& SceneTextures,
		const FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
		const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult,
		FLumenSceneFrameTemporaries& LumenFrameTemporaries);

	void RenderOcclusion(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		bool bIsOcclusionTesting,
		const FBuildHZBAsyncComputeParams* BuildHZBAsyncComputeParams,
		Froxel::FRenderer& FroxelRenderer);

	bool RenderHzb(FRDGBuilder& GraphBuilder, FRDGTextureRef SceneDepthTexture, const FBuildHZBAsyncComputeParams* AsyncComputeParams, Froxel::FRenderer& FroxelRenderer);

	/** Renders the view family. */
	virtual void Render(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs) override;

	/** Render the view family's hit proxies. */
	virtual void RenderHitProxies(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs) override;

	virtual bool ShouldRenderVelocities() const override;

	virtual bool ShouldRenderPrePass() const override;

	virtual bool ShouldRenderNanite() const override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	void RenderVisualizeTexturePool(FRHICommandListImmediate& RHICmdList);
#endif

private:

#if RHI_RAYTRACING
	void InitializeRayTracingFlags_RenderThread() override;
#endif

	/** Structure that contains the final state of deferred shading pipeline for a FViewInfo */
	struct FPerViewPipelineState
	{
		EDiffuseIndirectMethod DiffuseIndirectMethod;
		IScreenSpaceDenoiser::EMode DiffuseIndirectDenoiser;

		// Method to use for ambient occlusion.
		EAmbientOcclusionMethod AmbientOcclusionMethod;

		// Method to use for reflections. 
		EReflectionsMethod ReflectionsMethod;

		// Method to use for reflections on water.
		EReflectionsMethod ReflectionsMethodWater;

		// Whether there is planar reflection to compose to the reflection.
		bool bComposePlanarReflections;

		// Whether need to generate HZB from the depth buffer.
		bool bFurthestHZB;
		bool bClosestHZB;
	};

	// Structure that contains the final state of deferred shading pipeline for the FSceneViewFamily
	struct FFamilyPipelineState
	{
#if RHI_RAYTRACING
		// Whether the scene has lights with ray traced shadows.
		bool bRayTracingShadows = false;

		// Whether any ray tracing passes are enabled.
		bool bRayTracing = false;
#endif

		// Whether Nanite is enabled.
		bool bNanite;

		// Whether the scene occlusion is made using HZB.
		bool bHZBOcclusion;
	};

	/** Pipeline states that describe the high level topology of the entire renderer.
	 *
	 * Once initialized by CommitFinalPipelineState(), it becomes immutable for the rest of the execution of the renderer.
	 * The ViewPipelineStates array corresponds to Views in the FSceneRenderer.  Use "GetViewPipelineState" or
	 * "GetViewPipelineStateWritable" to access the pipeline state for a specific View.
	 */
	TArray<TPipelineState<FPerViewPipelineState>, TInlineAllocator<1>> ViewPipelineStates;
	TPipelineState<FFamilyPipelineState> FamilyPipelineState;

	FORCEINLINE const FPerViewPipelineState& GetViewPipelineState(const FViewInfo& View) const
	{
		return *ViewPipelineStates[View.SceneRendererPrimaryViewId];
	}

	FORCEINLINE TPipelineState<FPerViewPipelineState>& GetViewPipelineStateWritable(const FViewInfo& View)
	{
		return ViewPipelineStates[View.SceneRendererPrimaryViewId];
	}

	virtual bool IsLumenEnabled(const FViewInfo& View) const override
	{ 
		return (GetViewPipelineState(View).DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen || GetViewPipelineState(View).ReflectionsMethod == EReflectionsMethod::Lumen);
	}

	virtual bool IsLumenGIEnabled(const FViewInfo& View) const override
	{
		return GetViewPipelineState(View).DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen;
	}

	virtual bool AnyViewHasGIMethodSupportingDFAO() const override
	{
		bool bAnyViewHasGIMethodSupportingDFAO = false;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (GetViewPipelineState(Views[ViewIndex]).DiffuseIndirectMethod != EDiffuseIndirectMethod::Lumen)
			{
				bAnyViewHasGIMethodSupportingDFAO = true;
			}
		}

		return bAnyViewHasGIMethodSupportingDFAO;
	}

	FSeparateTranslucencyDimensions SeparateTranslucencyDimensions;

	/** Creates a per object projected shadow for the given interaction. */
	void CreatePerObjectProjectedShadow(
		FRHICommandListImmediate& RHICmdList,
		FLightPrimitiveInteraction* Interaction,
		bool bCreateTranslucentObjectShadow,
		bool bCreateInsetObjectShadow,
		const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ViewDependentWholeSceneShadows,
		TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& OutPreShadows);

	struct FInitViewTaskDatas
	{
		FInitViewTaskDatas(IVisibilityTaskData* InVisibilityTaskData)
			: VisibilityTaskData(InVisibilityTaskData)
		{}

		IVisibilityTaskData* VisibilityTaskData;
		FILCUpdatePrimTaskData* ILCUpdatePrim = nullptr;
		RayTracing::FGatherInstancesTaskData* RayTracingGatherInstances = nullptr;
		FDynamicShadowsTaskData* DynamicShadows = nullptr;
		FDecalVisibilityTaskData* Decals = nullptr;
		FLumenDirectLightingTaskData* LumenDirectLighting = nullptr;
		FLumenSceneFrameTemporaries* LumenFrameTemporaries = nullptr;
	};

	void PreVisibilityFrameSetup(FRDGBuilder& GraphBuilder);

	void BeginInitDynamicShadows(FRDGBuilder& GraphBuilder, FInitViewTaskDatas& TaskDatas, FInstanceCullingManager& InstanceCullingManager);
	void FinishInitDynamicShadows(FRDGBuilder& GraphBuilder, FDynamicShadowsTaskData*& TaskData, FInstanceCullingManager& InstanceCullingManager);

	void ComputeLightVisibility();

	/** Determines which primitives are visible for each view. */
	void BeginInitViews(
		FRDGBuilder& GraphBuilder,
		const FSceneTexturesConfig& SceneTexturesConfig,
		FInstanceCullingManager& InstanceCullingManager,
		FRDGExternalAccessQueue& ExternalAccessQueue,
		FInitViewTaskDatas& TaskDatas);

	void EndInitViews(
		FRDGBuilder& GraphBuilder,
		FLumenSceneFrameTemporaries& FrameTemporaries,
		FInstanceCullingManager& InstanceCullingManager,
		FInitViewTaskDatas& TaskDatas);

	void BeginUpdateLumenSceneTasks(FRDGBuilder& GraphBuilder, FLumenSceneFrameTemporaries& FrameTemporaries);
	void UpdateLumenScene(FRDGBuilder& GraphBuilder, FLumenSceneFrameTemporaries& FrameTemporaries);
	void RenderLumenSceneLighting(FRDGBuilder& GraphBuilder, const FLumenSceneFrameTemporaries& FrameTemporaries, const FLumenDirectLightingTaskData* DirectLightingTaskData);

	void BeginGatherLumenLights(const FLumenSceneFrameTemporaries& FrameTemporaries, FLumenDirectLightingTaskData*& TaskData, IVisibilityTaskData* VisibilityTaskData, UE::Tasks::FTask UpdateLightFunctionAtlasTask);

	void RenderDirectLightingForLumenScene(
		FRDGBuilder& GraphBuilder,
		const FLumenSceneFrameTemporaries& FrameTemporaries,
		const FLumenDirectLightingTaskData* DirectLightingTaskData,
		const FLumenCardUpdateContext& CardUpdateContext,
		ERDGPassFlags ComputePassFlags);
	
	void RenderRadiosityForLumenScene(
		FRDGBuilder& GraphBuilder,
		const FLumenSceneFrameTemporaries& FrameTemporaries,
		const LumenRadiosity::FFrameTemporaries& RadiosityFrameTemporaries,
		const FLumenCardUpdateContext& CardUpdateContext,
		ERDGPassFlags ComputePassFlags);

	void ClearLumenSurfaceCacheAtlas(
		FRDGBuilder& GraphBuilder,
		const FLumenSceneFrameTemporaries& FrameTemporaries,
		const FGlobalShaderMap* GlobalShaderMap);

	void DilateCardPageOneTexel(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		int32 NumPagesNeedDilation,
		FRDGBufferSRVRef CardCaptureRectBufferSRV,
		TConstArrayView<FCardPageRenderData> CardPagesToRender,
		struct FCardCaptureAtlas& CardCaptureAtlas);

	void UpdateLumenSurfaceCacheAtlas(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FLumenSceneFrameTemporaries& FrameTemporaries,
		const TArray<FCardPageRenderData, SceneRenderingAllocator>& CardPagesToRender,
		FRDGBufferSRVRef CardCaptureRectBufferSRV,
		const struct FCardCaptureAtlas& CardCaptureAtlas,
		const struct FResampledCardCaptureAtlas& ResampledCardCaptureAtlas);

	LumenRadianceCache::FUpdateInputs GetLumenTranslucencyGIVolumeRadianceCacheInputs(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View, 
		const FLumenSceneFrameTemporaries& FrameTemporaries,
		ERDGPassFlags ComputePassFlags);

	void ComputeLumenTranslucencyGIVolume(
		FRDGBuilder& GraphBuilder,
		FViewInfo& View,
		const FSceneTextures& SceneTextures,
		const FLumenSceneFrameTemporaries& FrameTemporaries,
		LumenRadianceCache::FRadianceCacheInterpolationParameters& RadianceCacheParameters,
		ERDGPassFlags ComputePassFlags);

	void CreateIndirectCapsuleShadows();

	void RenderPrePass(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FRDGTextureRef SceneDepthTexture, FInstanceCullingManager& InstanceCullingManager, FRDGTextureRef* FirstStageDepthBuffer);
	void RenderPrePassHMD(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FRDGTextureRef SceneDepthTexture);

	void RenderFog(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		FRDGTextureRef LightShaftOcclusionTexture,
		bool bFogComposeLocalFogVolumes);

	void RenderFogSeparateCompositionTextures(
		FRDGBuilder& GraphBuilder,
		FSceneTextures& SceneTextures,
		TArray<FFogSeparateCompositionViewResources>& FogSeparateCompositionViewResources,
		FRDGTextureRef LightShaftOcclusionTexture,
		bool bFogComposeLocalFogVolumes,
		const bool bRenderSkyAtmosphereInLayer);

	void RenderFogScreenSpaceScatteringMipChain(
		FRDGBuilder& GraphBuilder,
		FSceneTextures& SceneTextures,
		TArray<FFogSeparateCompositionViewResources>& FogSeparateCompositionViewResources);

	void RenderUnderWaterFog(
		FRDGBuilder& GraphBuilder,
		const FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
		TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesWithDepth);

	// TODO: Address tech debt to that directly in RenderDiffuseIndirectAndAmbientOcclusion()
	void SetupCommonDiffuseIndirectParameters(
		FRDGBuilder& GraphBuilder,
		const FSceneTextureParameters& SceneTextures,
		const FViewInfo& View,
		HybridIndirectLighting::FCommonParameters& OutCommonDiffuseParameters);

	/** Dispatch async Lumen work if possible. */
	void DispatchAsyncLumenIndirectLightingWork(
		FRDGBuilder& GraphBuilder,
		FSceneTextures& SceneTextures,
		FInstanceCullingManager& InstanceCullingManager,
		FLumenSceneFrameTemporaries& LumenFrameTemporaries,
		FDynamicShadowsTaskData* DynamicShadowsTaskData,
		FRDGTextureRef LightingChannelsTexture,
		FAsyncLumenIndirectLightingOutputs& Outputs);

	/** Render diffuse indirect (regardless of the method) of the views into the scene color. */
	void RenderDiffuseIndirectAndAmbientOcclusion(
		FRDGBuilder& GraphBuilder,
		FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& FrameTemporaries,
		FRDGTextureRef LightingChannelsTexture,
		bool bCompositeRegularLumenOnly,
		bool bIsVisualizePass,
		FAsyncLumenIndirectLightingOutputs& AsyncLumenIndirectLightingOutputs);

	/** Renders sky lighting and reflections that can be done in a deferred pass. */
	void RenderDeferredReflectionsAndSkyLighting(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& LumenFrameTemporaries,
		TArray<FRDGTextureRef>& DynamicBentNormalAOTexture);

	void RenderDeferredReflectionsAndSkyLightingHair(FRDGBuilder& GraphBuilder);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Renders debug visualizations for global illumination plugins. */
	void RenderGlobalIlluminationPluginVisualizations(FRDGBuilder& GraphBuilder, FRDGTextureRef LightingChannelsTexture);
#endif

	/** Computes DFAO, modulates it to scene color (which is assumed to contain diffuse indirect lighting), and stores the output bent normal for use occluding specular. */
	void RenderDFAOAsIndirectShadowing(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		TArray<FRDGTextureRef>& DynamicBentNormalAOTextures);

	TSharedPtr<FMegaLightsFrameTemporaries> CreateMegaLightsFrameTemporaries(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

	void DispatchAsyncMegaLightsVolumePasses(
		FRDGBuilder& GraphBuilder,
		const TSharedPtr<FMegaLightsFrameTemporaries>& MegaLightsFrameTemporaries);

	void GenerateMegaLightsSamples(
		FRDGBuilder& GraphBuilder,
		const TSharedPtr<FMegaLightsFrameTemporaries>& MegaLightsFrameTemporaries,
		FLumenSceneFrameTemporaries& FrameTemporaries,
		FRDGTextureRef LightingChannelsTexture,
		ERDGPassFlags ComputePassFlags);

	void RenderMegaLights(
		FRDGBuilder& GraphBuilder,
		const TSharedPtr<FMegaLightsFrameTemporaries>& MegaLightsFrameTemporaries,
		const FSceneTextures& SceneTextures,
		FRDGTextureRef LightingChannelsTexture);

	void RenderMegaLightsFrontLayerTranslucency(
		FRDGBuilder& GraphBuilder,
		TSharedPtr<FMegaLightsFrameTemporaries> FrameTemporaries,
		FLumenSceneFrameTemporaries& LumenFrameTemporaries,
		const FFrontLayerTranslucencyData& FrontLayerTranslucencyData,
		ERDGPassFlags ComputePassFlags);

	void RenderLightShapeVisualization(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		const FTranslucencyPassResourcesMap& TranslucencyResourceMap);

	FSSDSignalTextures RenderLumenFinalGather(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& FrameTemporaries,
		FRDGTextureRef LightingChannelsTexture,
		FViewInfo& View,
		int32 ViewIndex,
		FPreviousViewInfo* PreviousViewInfos,
		class FLumenMeshSDFGridParameters& MeshSDFGridParameters,
		LumenRadianceCache::FRadianceCacheInterpolationParameters& RadianceCacheParameters,
		class FLumenScreenSpaceBentNormalParameters& ScreenSpaceBentNormalParameters,
		class FLumenUpsampleParameters& UpsampleParameters,
		ERDGPassFlags ComputePassFlags);

	FSSDSignalTextures RenderLumenScreenProbeGather(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& FrameTemporaries,
		FRDGTextureRef LightingChannelsTexture,
		FViewInfo& View,
		int32 ViewIndex,
		FPreviousViewInfo* PreviousViewInfos,
		class FLumenMeshSDFGridParameters& MeshSDFGridParameters,
		LumenRadianceCache::FRadianceCacheInterpolationParameters& RadianceCacheParameters,
		class FLumenScreenSpaceBentNormalParameters& ScreenBentNormalParameters,
		LumenRadianceCache::FRadianceCacheInterpolationParameters& TranslucencyVolumeRadianceCacheParameters,
		ERDGPassFlags ComputePassFlags);

	void MarkLumenRadianceCacheForTranslucencyReflections(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& FrameTemporaries);

	FSSDSignalTextures RenderLumenReSTIRGather(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& FrameTemporaries,
		FRDGTextureRef LightingChannelsTexture,
		FViewInfo& View,
		FPreviousViewInfo* PreviousViewInfos,
		ERDGPassFlags ComputePassFlags,
		FLumenScreenSpaceBentNormalParameters& ScreenSpaceBentNormalParameters);

	bool RequiresStochasticLightingPass();
	void StochasticLightingTileClassificationMark(FRDGBuilder& GraphBuilder, FLumenSceneFrameTemporaries& FrameTemporaries, const FSceneTextures& SceneTextures, bool bAsyncMegaLightsGenerateSamples);
	void StochasticLightingFrontLayerTranslucencyTileClassificationMark(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures, FFrontLayerTranslucencyData& FrontLayerTranslucencyData, ERDGPassFlags ComputePassFlags);

	/** Extract current frame opaque (no water) depth and normal scene textures to use as history data. */
	void QueueExtractStochasticLighting(FRDGBuilder& GraphBuilder, FLumenSceneFrameTemporaries& FrameTemporaries, const FFrontLayerTranslucencyData& FrontLayerTranslucencyData, const FMinimalSceneTextures& SceneTextures);

	FSSDSignalTextures RenderLumenIrradianceFieldGather(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		const FLumenSceneFrameTemporaries& FrameTemporaries,
		FRDGTextureRef LightingChannelsTexture,
		FViewInfo& View,
		FLumenScreenSpaceBentNormalParameters& ScreenSpaceBentNormalParameters,
		class FLumenUpsampleParameters& UpsampleParameters,
		LumenRadianceCache::FRadianceCacheInterpolationParameters& TranslucencyVolumeRadianceCacheParameters,
		ERDGPassFlags ComputePassFlags);

	FRDGTextureRef RenderLumenReflections(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& FrameTemporaries,
		const class FLumenMeshSDFGridParameters& MeshSDFGridParameters,
		const LumenRadianceCache::FRadianceCacheInterpolationParameters& RadianceCacheParameters,
		ELumenReflectionPass ReflectionPass,
		const FLumenReflectionsConfig& ReflectionsConfig,
		ERDGPassFlags ComputePassFlags);

	void RenderRayTracedTranslucencyView(
		FRDGBuilder& GraphBuilder,
		FViewInfo& View,
		FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& FrameTemporaries,
		const FFrontLayerTranslucencyData& FrontLayerTranslucencyData,
		FRDGTextureRef& InOutFinalRadiance,
		FRDGTextureRef& InOutBackgroundVisibility);

	bool RenderRayTracedTranslucency(
		FRDGBuilder& GraphBuilder,
		FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& FrameTemporaries,
		const FFrontLayerTranslucencyData& FrontLayerTranslucencyData);

	void RenderLumenFrontLayerTranslucencyReflections(
		FRDGBuilder& GraphBuilder,
		FViewInfo& View,
		const FSceneTextures& SceneTextures,
		FLumenSceneFrameTemporaries& LumenFrameTemporaries, 
		const FFrontLayerTranslucencyData& FrontLayerTranslucencyData);
	
	FFrontLayerTranslucencyData RenderFrontLayerTranslucency(
		FRDGBuilder& GraphBuilder,
		TArray<FViewInfo>& Views,
		const FSceneTextures& SceneTextures,
		const TSharedPtr<FMegaLightsFrameTemporaries>& MegaLightsFrameTemporaries,
		bool bRenderOnlyForVSMPageMarking);

	bool IsLumenFrontLayerTranslucencyEnabled(const FViewInfo& View) const;
	bool IsMegaLightsFrontLayerTranslucencyEnabled(const FViewInfo& View) const;

	void RenderLumenMiscVisualizations(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures, const FLumenSceneFrameTemporaries& FrameTemporaries);
	void RenderLumenRadianceCacheVisualization(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures);
	void RenderLumenRadiosityProbeVisualization(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures, const FLumenSceneFrameTemporaries& FrameTemporaries);
	void LumenScenePDIVisualization(const FLumenSceneFrameTemporaries* FrameTemporaries);

	/** Mark time line for gathering Lumen virtual surface cache feedback. */
	void BeginGatheringLumenSurfaceCacheFeedback(FRDGBuilder& GraphBuilder, const FViewInfo& View, FLumenSceneFrameTemporaries& FrameTemporaries);
	void FinishGatheringLumenSurfaceCacheFeedback(FRDGBuilder& GraphBuilder, const FViewInfo& View, FLumenSceneFrameTemporaries& FrameTemporaries, const FMinimalSceneTextures& SceneTextures);

	void RenderTranslucencyVolumeVisualization(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures, const FTranslucencyLightingVolumeTextures& TranslucencyLightingVolumeTextures);

	/** 
	 * True if the 'r.UseClusteredDeferredShading' flag is 1 and sufficient feature level. 
	 */
	bool ShouldUseClusteredDeferredShading(EShaderPlatform InPlatform) const;

	/**
	 * Have the lights been injected into the light grid?
	 */
	bool AreLightsInLightGrid() const;


	/** Add a clustered deferred shading lighting render pass.	Note: in the future it should take the RenderGraph builder as argument */
	void AddClusteredDeferredShadingPass(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		const FSortedLightSetSceneInfo& SortedLightsSet,
		FRDGTextureRef ShadowMaskBits,
		FRDGTextureRef HairStrandsShadowMaskBits, 
		FRDGTextureRef ShadowMaskBitsLightingChannelsTexture);

	/** Renders the scene's lighting. */
	void RenderLights(
		FRDGBuilder& GraphBuilder,
		FSceneTextures& SceneTextures,
		FRDGTextureRef LightingChannelsTexture,
		const FSortedLightSetSceneInfo& SortedLightSet);

	void RenderTranslucencyLightingVolume(
		FRDGBuilder& GraphBuilder,
		FTranslucencyLightingVolumeTextures& Textures,
		const FSortedLightSetSceneInfo& SortedLightSet);

	void GatherTranslucencyVolumeMarkedVoxels(FRDGBuilder& GraphBuilder, const FSortedLightSetSceneInfo& SortedLightSet);

	/** Render stationary light overlap as complexity to scene color. */
	void RenderStationaryLightOverlap(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		FRDGTextureRef LightingChannelsTexture);
	
	/** Renders the scene's translucency passes. */
	static void RenderTranslucency(
		FDeferredShadingSceneRenderer& Renderer,
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		const FTranslucencyLightingVolumeTextures& TranslucencyLightingVolumeTextures,
		FTranslucencyPassResourcesMap* OutTranslucencyResourceMap,
		TArray<FViewInfo>& InViews,
		ETranslucencyView ViewsToRender,
		const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions,
		FInstanceCullingManager& InstanceCullingManager,
		bool bStandardTranslucentCanRenderSeparate,
		const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult,
		FRDGTextureMSAA& OutSharedDepthTexture,
		const TArrayView<Nanite::FRasterResults>& NaniteRasterResults);

	/** Renders the scene's translucency given a specific pass. */
	static void RenderTranslucencyInner(
		FDeferredShadingSceneRenderer& Renderer,
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const FTranslucencyLightingVolumeTextures& TranslucencyLightingVolumeTextures,
		FTranslucencyPassResourcesMap* OutTranslucencyResourceMap,
		FRDGTextureMSAA SharedDepthTexture,
		TArray<FViewInfo>& InViews,
		ETranslucencyView ViewsToRender,
		const FSeparateTranslucencyDimensions& SeparateTranslucencyDimensions,
		FRDGTextureRef SceneColorCopyTexture,
		ETranslucencyPass::Type TranslucencyPass,
		FInstanceCullingManager& InstanceCullingManager,
		bool bStandardTranslucentCanRenderSeparate,
		const TArrayView<Nanite::FRasterResults>& NaniteRasterResults);

	void UpscaleTranslucencyIfNeeded(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		ETranslucencyView ViewsToRender,
		FTranslucencyPassResourcesMap* OutTranslucencyResourceMap,
		FRDGTextureMSAA& InSharedDepthTexture);

	/** Renders the scene's light shafts */
	FRDGTextureRef RenderLightShaftOcclusion(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures);

	void RenderLightShaftBloom(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		FTranslucencyPassResourcesMap& OutTranslucencyResourceMap);

	bool ShouldRenderDistortion() const;
	void RenderDistortion(FRDGBuilder& GraphBuilder, 
		FRDGTextureRef SceneColorTexture, 
		FRDGTextureRef SceneDepthTexture,
		FRDGTextureRef SceneVelocityTexture,
		FTranslucencyPassResourcesMap& TranslucencyResourceMap);

	void CollectLightForTranslucencyLightingVolumeInjection(
		const FLightSceneInfo* LightSceneInfo,
		bool bSupportShadowMaps,
		FTranslucentLightInjectionCollector& Collector);

	/** Renders indirect shadows from capsules modulated onto scene color. */
	void RenderIndirectCapsuleShadows(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures) const;

	

	void RenderDeferredShadowProjections(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const FLightSceneInfo* LightSceneInfo,
		FRDGTextureRef ScreenShadowMaskTexture,
		FRDGTextureRef ScreenShadowMaskSubPixelTexture);

	void RenderForwardShadowProjections(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		FRDGTextureRef& ForwardScreenSpaceShadowMask,
		FRDGTextureRef& ForwardScreenSpaceShadowMaskSubPixel);

	/** Used by RenderLights to render a light function to the attenuation buffer. */
	bool RenderLightFunction(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const FLightSceneInfo* LightSceneInfo,
		FRDGTextureRef ScreenShadowMaskTexture,
		bool bLightAttenuationCleared,
		bool bProjectingForForwardShading, 
		bool bUseHairStrands);

	/** Renders a light function indicating that whole scene shadowing being displayed is for previewing only, and will go away in game. */
	bool RenderPreviewShadowsIndicator(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const FLightSceneInfo* LightSceneInfo,
		FRDGTextureRef ScreenShadowMaskTexture,
		bool bLightAttenuationCleared,
		bool bUseHairStrands);

	/** Renders a light function with the given material. */
	bool RenderLightFunctionForMaterial(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const FLightSceneInfo* LightSceneInfo,
		FRDGTextureRef ScreenShadowMaskTexture,
		const FMaterialRenderProxy* MaterialProxy,
		bool bLightAttenuationCleared,
		bool bProjectingForForwardShading,
		bool bRenderingPreviewShadowsIndicator, 
		bool bUseHairStrands);

	void RenderLightsForHair(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const FSortedLightSetSceneInfo& SortedLightSet,
		FRDGTextureRef InScreenShadowMaskSubPixelTexture,
		FRDGTextureRef LightingChannelsTexture);

	/** Specialized version of RenderLight for hair (run lighting evaluation on at sub-pixel rate, without depth bound) */
	void RenderLightForHair(
		FRDGBuilder& GraphBuilder,
		FViewInfo& View,
		const FMinimalSceneTextures& SceneTextures,
		const FLightSceneInfo* LightSceneInfo,
		FRDGTextureRef ScreenShadowMaskSubPixelTexture,
		FRDGTextureRef LightingChannelsTexture,
		const FHairStrandsTransmittanceMaskData& InTransmittanceMaskData,
		const bool bForwardRendering,
		const bool bCanLightUsesAtlasForUnbatchedLight,
		TRDGUniformBufferRef<FVirtualShadowMapUniformParameters> VirtualShadowMapUniformBuffer = nullptr,
		FRDGTextureRef ShadowMaskBits = nullptr,
		int32 VirtualShadowMapId = INDEX_NONE);

	/** Renders an array of simple lights using standard deferred shading. */
	void RenderSimpleLightsStandardDeferred(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const FSortedLightSetSceneInfo& SortedLightSet,
		FSubstrateSceneData& SubstrateSceneData);

	FRDGTextureRef CopyStencilToLightingChannelTexture(
		FRDGBuilder& GraphBuilder, 
		FRDGTextureSRVRef SceneStencilTexture,
		const TArrayView<FRDGTextureRef> NaniteResolveTextures);

	void RenderHeterogeneousVolumeShadows(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);
	void RenderHeterogeneousVolumes(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);
	void CompositeHeterogeneousVolumes(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

	void VisualizeVolumetricLightmap(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

	/** Render image based reflections (SSR, Env, SkyLight) without compute shaders */
	void RenderStandardDeferredImageBasedReflections(FRHICommandListImmediate& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, bool bReflectionEnv, const TRefCountPtr<IPooledRenderTarget>& DynamicBentNormalAO, TRefCountPtr<IPooledRenderTarget>& VelocityRT);

	bool HasDeferredPlanarReflections(const FViewInfo& View) const;
	void RenderDeferredPlanarReflections(FRDGBuilder& GraphBuilder, const FSceneTextureParameters& SceneTextures, const FViewInfo& View, FRDGTextureRef& ReflectionsOutput);

	void SetupImaginaryReflectionTextureParameters(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FSceneTextureParameters* OutTextures);

	void RenderRayTracingReflections(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		const FViewInfo& View,
		int DenoiserMode,
		const FRayTracingReflectionOptions& Options,
		IScreenSpaceDenoiser::FReflectionsInputs* OutDenoiserInputs);

	void RenderRayTracingDeferredReflections(
		FRDGBuilder& GraphBuilder,
		const FSceneTextureParameters& SceneTextures,
		const FViewInfo& View,
		int DenoiserMode,
		const FRayTracingReflectionOptions& Options,
		IScreenSpaceDenoiser::FReflectionsInputs* OutDenoiserInputs);

	void RenderDitheredLODFadingOutMask(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneDepthTexture);

	void RenderRayTracingShadows(
		FRDGBuilder& GraphBuilder,
		const FSceneTextureParameters& SceneTextures,
		const FViewInfo& View,
		const FLightSceneInfo& LightSceneInfo,
		const IScreenSpaceDenoiser::FShadowRayTracingConfig& RayTracingConfig,
		const IScreenSpaceDenoiser::EShadowRequirements DenoiserRequirements,
		FRDGTextureRef LightingChannelsTexture,
		FRDGTextureUAV* OutShadowMaskUAV,
		FRDGTextureUAV* OutRayHitDistanceUAV,
		FRDGTextureUAV* SubPixelRayTracingShadowMaskUAV);
	void CompositeRayTracingSkyLight(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		FRDGTextureRef SkyLightRT,
		FRDGTextureRef HitDistanceRT);
	
	void RenderRayTracingAmbientOcclusion(
		FRDGBuilder& GraphBuilder,
		FViewInfo& View,
		const FSceneTextureParameters& SceneTextures,
		FRDGTextureRef* OutAmbientOcclusionTexture);
	
#if RHI_RAYTRACING
	template <int TextureImportanceSampling>
	void RenderRayTracingRectLightInternal(
		FRDGBuilder& GraphBuilder,
		TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformBuffer,
		const TArray<FViewInfo>& Views,
		const FLightSceneInfo& RectLightSceneInfo,
		FRDGTextureRef ScreenShadowMaskTexture,
		FRDGTextureRef RayDistanceTexture);

	void RenderRayTracingSkyLight(
		FRDGBuilder& GraphBuilder,
		FRDGTextureRef SceneColorTexture,
		FRDGTextureRef& OutSkyLightTexture,
		FRDGTextureRef& OutHitDistanceTexture);

	void RenderRayTracingTranslucency(FRDGBuilder& GraphBuilder, FRDGTextureMSAA SceneColorTexture);

	void RenderRayTracingTranslucencyView(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FRDGTextureRef* OutColorTexture,
		FRDGTextureRef* OutRayHitDistanceTexture,
		int32 SamplePerPixel,
		int32 HeightFog,
		float ResolutionFraction);

	/** Setup the default miss shader (required for any raytracing pipeline) */
	void SetupRayTracingDefaultMissShader(FRHICommandList& RHICmdList, const FViewInfo& View);
	void SetupPathTracingDefaultMissShader(FRHICommandList& RHICmdList, const FViewInfo& View);

	/** Lighting Evaluation shader setup (used by ray traced reflections and translucency) */
	void SetupRayTracingLightingMissShader(FRHICommandList& RHICmdList, const FViewInfo& View);

	/** Path tracing functions. */
	void RenderPathTracing(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformBuffer,
		FRDGTextureRef SceneColorOutputTexture,
		FRDGTextureRef SceneDepthOutputTexture,
		struct FPathTracingResources& PathTracingResources);

	void ComputePathCompaction(FRHICommandListImmediate& RHICmdList, const FViewInfo& View, FRHITexture* RadianceTexture, FRHITexture* SampleCountTexture, FRHITexture* PixelPositionTexture,
		FRHIUnorderedAccessView* RadianceSortedRedUAV, FRHIUnorderedAccessView* RadianceSortedGreenUAV, FRHIUnorderedAccessView* RadianceSortedBlueUAV, FRHIUnorderedAccessView* RadianceSortedAlphaUAV, FRHIUnorderedAccessView* SampleCountSortedUAV);

	void SetupRayTracingRenderingData(FRDGBuilder& GraphBuilder, RayTracing::FGatherInstancesTaskData& RayTracingGatherInstances);

	/** Debug ray tracing functions. */
	void RayTracingDisplayPicking(const FRayTracingPickingFeedback& PickingFeedback, FScreenMessageWriter& Writer);

	bool SetupRayTracingPipelineStatesAndSBT(FRDGBuilder& GraphBuilder, bool bAnyInlineHardwareRayTracingPassEnabled, bool& bOutIsUsingFallbackRTPSO);
	void SetupRayTracingLightDataForViews(FRDGBuilder& GraphBuilder);
	bool DispatchRayTracingWorldUpdates(FRDGBuilder& GraphBuilder, FRDGBufferRef& OutDynamicGeometryScratchBuffer, ERHIPipeline ResourceAccessPipelines);

	/** Functions to create ray tracing pipeline state objects for various effects */
	void CreateMaterialRayTracingMaterialPipeline(FRDGBuilder& GraphBuilder, const TArrayView<FRHIRayTracingShader*>& RayGenShaderTable, uint32& OutMaxLocalBindingDataSize, bool& bOutIsUsingFallbackRTPSO);
	void SetupMaterialRayTracingHitGroupBindings(FRDGBuilder& GraphBuilder, FViewInfo& View, TConstArrayView<FRayTracingShaderBindingData> RayTracingShaderBindings);
	
	void CreateLumenHardwareRayTracingMaterialPipeline(FRDGBuilder& GraphBuilder, const TArrayView<FRHIRayTracingShader*>& RayGenShaderTable, uint32& OutMaxLocalBindingDataSize);
	void SetupLumenHardwareRayTracingHitGroupBindings(FRDGBuilder& GraphBuilder, FViewInfo& View, TConstArrayView<FRayTracingShaderBindingData> RayTracingShaderBindings);

	FRDGBufferRef SetupLumenHardwareRayTracingHitGroupBuffer(FRDGBuilder& GraphBuilder, TConstArrayView<FRayTracingShaderBindingData> VisibleRayTracingShaderBindings);
	void SetupLumenHardwareRayTracingUniformBuffer(FViewInfo& View);
	
	// #dxr_todo: UE-72565: refactor ray tracing effects to not be member functions of DeferredShadingRenderer. Register each effect at startup and just loop over them automatically
	static void PrepareRayTracingShadows(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareRayTracingAmbientOcclusion(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareRayTracingSkyLight(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareRayTracingGlobalIlluminationPlugin(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareRayTracingTranslucency(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareRayTracingVolumetricFogShadows(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareRayTracingDebug(const FSceneViewFamily& ViewFamily, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PreparePathTracing(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareLumenHardwareRayTracingShortRangeAO(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingScreenProbeGather(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingRadianceCache(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingReflections(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareHardwareRayTracingTranslucency(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingReSTIR(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingVisualize(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareMegaLightsHardwareRayTracing(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareMegaLightsHardwareRayTracingVisualize(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders);

	// Versions for setting up the lumen material pipeline
	static void PrepareLumenHardwareRayTracingTranslucencyVolumeLumenMaterial(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingVisualizeLumenMaterial(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingReflectionsLumenMaterial(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingReSTIRLumenMaterial(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingScreenProbeGatherLumenMaterial(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	void PrepareLumenHardwareRayTracingRadianceCacheLumenMaterial(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareLumenHardwareRayTracingRadiosityLumenMaterial(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareLumenHardwareRayTracingDirectLightingLumenMaterial(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareMegaLightsHardwareRayTracingMaterial(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
	static void PrepareMegaLightsHardwareRayTracingVisualizeMaterial(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders);
#endif // RHI_RAYTRACING


	struct FNaniteBasePassVisibility
	{
		FNaniteVisibilityQuery* Query = nullptr;
		FNaniteVisibility* Visibility = nullptr;

	} NaniteBasePassVisibility;

	void RenderNanite(FRDGBuilder& GraphBuilder, const TArray<FViewInfo>& InViews, FSceneTextures& SceneTextures, bool bIsEarlyDepthComplete,
		FNaniteBasePassVisibility& InNaniteBasePassVisibility,
		TArray<Nanite::FRasterResults, TInlineAllocator<2>>& NaniteRasterResults,
		TArray<Nanite::FPackedView, SceneRenderingAllocator>& PrimaryNaniteViews,
		FRDGTextureRef FirstStageDepthBuffer);

	// FSceneRendererBase interface
	virtual FDeferredShadingSceneRenderer* GetDeferredShadingSceneRenderer() override { return this; }

	/** Set to true if lights were injected into the light grid (this controlled by somewhat complex logic, this flag is used to cross-check). */
	bool bAreLightsInLightGrid;
};
