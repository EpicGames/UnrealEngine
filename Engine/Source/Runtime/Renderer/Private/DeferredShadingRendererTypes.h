// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

enum class EDiffuseIndirectMethod
{
	Disabled,
	SSGI,
	Lumen,
	Plugin,
};

enum class EAmbientOcclusionMethod
{
	Disabled,
	SSAO,
	SSGI, // SSGI can produce AO buffer at same time to correctly comp SSGI within the other indirect light such as skylight and lightmass.
	RTAO,
};

enum class EReflectionsMethod
{
	Disabled,
	SSR,
	Lumen
};
