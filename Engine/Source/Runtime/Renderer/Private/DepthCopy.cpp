// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DepthCopy.cpp: Depth rendering implementation.
=============================================================================*/

#include "DepthCopy.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PixelShaderUtils.h"
#include "SceneRendering.h"

namespace DepthCopy
{

static TAutoConsoleVariable<int32> CVarDepthCopyFastPath(
	TEXT("r.DepthCopy.FastPath"), 1,
	TEXT("Use compute pass to compute HTile along with depth copy when supported."),
	ECVF_RenderThreadSafe);

class FViewDepthCopyCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FViewDepthCopyCS)
		SHADER_USE_PARAMETER_STRUCT(FViewDepthCopyCS, FGlobalShader)

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, RWDepthTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		END_SHADER_PARAMETER_STRUCT()

		using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static int32 GetGroupSize()
	{
		return 8;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

class FCopyDepthAndBuildHTileCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyDepthAndBuildHTileCS)
	SHADER_USE_PARAMETER_STRUCT(FCopyDepthAndBuildHTileCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, PlatformConfig)
		SHADER_PARAMETER(uint32, PixelsWide)
		SHADER_PARAMETER(FUint32Vector4, ViewRect)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SrcDepth)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, DstDepth)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTextureMetadata, DstHTile)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsNanite(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FViewDepthCopyCS, "/Engine/Private/CopyDepthTextureCS.usf", "CopyDepthCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FCopyDepthAndBuildHTileCS, "/Engine/Private/CopyDepthTextureCS.usf", "CopyDepthAndBuildHTileCS", SF_Compute);

void AddViewDepthCopyCSPass(FRDGBuilder& GraphBuilder, FViewInfo& View, FRDGTextureRef SourceSceneDepthTexture, FRDGTextureRef DestinationDepthTexture)
{
	FViewDepthCopyCS::FPermutationDomain PermutationVector;
	auto ComputeShader = View.ShaderMap->GetShader<FViewDepthCopyCS>(PermutationVector);

	FViewDepthCopyCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FViewDepthCopyCS::FParameters>();
	PassParameters->RWDepthTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(DestinationDepthTexture));
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneDepthTexture = SourceSceneDepthTexture;

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("CopyViewDepthCS"),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), FViewDepthCopyCS::GetGroupSize()));
}

void AddViewDepthCopyHTileCSPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SourceDepthTexture, FRDGTextureRef DestinationDepthTexture)
{
	const FIntRect& ViewRect = View.ViewRect;
	const int32 kHTileSize = 8;
	check((ViewRect.Min.X % kHTileSize) == 0 && (ViewRect.Min.Y % kHTileSize) == 0);
	check(GRHISupportsDepthUAV && GRHISupportsExplicitHTile);

	const uint32 PlatformConfig = RHIGetHTilePlatformConfig(DestinationDepthTexture->Desc);
	const FIntPoint SceneTexturesExtent = DestinationDepthTexture->Desc.Extent;

	FCopyDepthAndBuildHTileCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyDepthAndBuildHTileCS::FParameters>();
	PassParameters->PlatformConfig = PlatformConfig;
	PassParameters->PixelsWide = SceneTexturesExtent.X;
	PassParameters->ViewRect = FUint32Vector4(ViewRect.Min.X, ViewRect.Min.Y, ViewRect.Max.X, ViewRect.Max.Y);
	PassParameters->SrcDepth = SourceDepthTexture;
	PassParameters->DstDepth = GraphBuilder.CreateUAV(FRDGTextureUAVDesc::CreateForMetaData(DestinationDepthTexture, ERDGTextureMetaDataAccess::CompressedSurface));
	PassParameters->DstHTile = GraphBuilder.CreateUAV(FRDGTextureUAVDesc::CreateForMetaData(DestinationDepthTexture, ERDGTextureMetaDataAccess::HTile));

	TShaderMapRef<FCopyDepthAndBuildHTileCS> ComputeShader(View.ShaderMap);

	const FIntVector DispatchDim = FComputeShaderUtils::GetGroupCount(ViewRect.Size(), kHTileSize);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("CopyDepthAndBuildHTileCS"), ComputeShader, PassParameters, DispatchDim);
}

IMPLEMENT_GLOBAL_SHADER(FCopyDepthPS, "/Engine/Private/CopyDepthTexture.usf", "CopyDepthPS", SF_Pixel);

void AddViewDepthCopyPSPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SourceSceneDepthTexture, FRDGTextureRef DestinationDepthTexture, bool bKeepDestStencil)
{
	const FRDGTextureDesc& SrcDesc = SourceSceneDepthTexture->Desc;

	FCopyDepthPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyDepthPS::FParameters>();
	if (SrcDesc.NumSamples > 1)
	{
		PassParameters->DepthTextureMS = SourceSceneDepthTexture;
	}
	else
	{
		PassParameters->DepthTexture = SourceSceneDepthTexture;
	}
	FDepthStencilBinding ClearDepthStencilBinding = FDepthStencilBinding(DestinationDepthTexture, ERenderTargetLoadAction::ENoAction, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);
	FDepthStencilBinding KeepDestinationDepthStencilBinding = FDepthStencilBinding(DestinationDepthTexture, ERenderTargetLoadAction::ENoAction, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);

	PassParameters->RenderTargets.DepthStencil = bKeepDestStencil ? KeepDestinationDepthStencilBinding : ClearDepthStencilBinding;

	FCopyDepthPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FCopyDepthPS::FMSAASampleCount>(SrcDesc.NumSamples);
	TShaderMapRef<FCopyDepthPS> PixelShader(View.ShaderMap, PermutationVector);

	// Set depth test to always pass and stencil test to reset to 0.
	FRHIDepthStencilState* DepthStencilResetZeroState = TStaticDepthStencilState<
		true, CF_Always,										// depth
		true, CF_Always, SO_Zero, SO_Zero, SO_Zero,				// frontface stencil
		true, CF_Always, SO_Zero, SO_Zero, SO_Zero				// backface stencil
	>::GetRHI();

	FRHIDepthStencilState* DepthStencilNop = TStaticDepthStencilState<
		true, CF_Always										    // depth
	>::GetRHI();

	FRHIDepthStencilState* DepthStencilState = bKeepDestStencil ? DepthStencilNop : DepthStencilResetZeroState;

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME("CopyViewDepthPS"),
		PixelShader,
		PassParameters,
		View.ViewRect,
		nullptr, /*BlendState*/
		nullptr, /*RasterizerState*/
		DepthStencilState,
		0 /*StencilRef*/);

	// The above copy technique loses HTILE data during the copy, so until AddCopyTexturePass() supports depth buffer copies on all platforms,
	// This is the best we can do: regenerate HTile from depth texture.
	AddResummarizeHTilePass(GraphBuilder, DestinationDepthTexture);
}


void AddViewDepthCopyPass(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View,
	FRDGTextureRef SourceDepthTexture, 
	FRDGTextureRef DestinationDepthTexture,
	EDepthCopyStencilMode StencilMode)
{

	// CS+HTILE path: fastest, no resummarize, but can't handle stencil or MSAA
	if ((StencilMode == EDepthCopyStencilMode::Keep)
		&& GRHISupportsDepthUAV 
		&& GRHISupportsExplicitHTile
		&& SourceDepthTexture->Desc.NumSamples <= 1
		&& DestinationDepthTexture->Desc.NumSamples <= 1
		&& CVarDepthCopyFastPath.GetValueOnRenderThread() != 0)
	{
		AddViewDepthCopyHTileCSPass(GraphBuilder, View, SourceDepthTexture, DestinationDepthTexture);
	}
	else
	{
		const bool bKeepDestStencil = (StencilMode == EDepthCopyStencilMode::Keep);
		AddViewDepthCopyPSPass(GraphBuilder, View, SourceDepthTexture, DestinationDepthTexture, bKeepDestStencil);
	}
}

};
