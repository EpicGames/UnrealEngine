// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DepthCopy.h: Depth copy utilities.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphUtils.h"
#include "GlobalShader.h"

class FViewInfo;

namespace DepthCopy
{

// This is a temporary workaround while we get AddCopyTexturePass to do a proper copy of depth texture (with source texture HTile maintained).
// On some platforms this is not the case: depth is decompressed so that the depth format can be read through SRV and HTile optimizations are thus lost on the source texture.
// While we wait for such support, we do a simple copy from SRV to UAV.
void AddViewDepthCopyCSPass(FRDGBuilder& GraphBuilder, FViewInfo& View, FRDGTextureRef SourceSceneDepthTexture, FRDGTextureRef DestinationDepthTexture);

class FCopyDepthPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FCopyDepthPS, FGlobalShader);

	class FMSAASampleCount : SHADER_PERMUTATION_SPARSE_INT("MSAA_SAMPLE_COUNT", 1, 2, 4, 8);
	using FPermutationDomain = TShaderPermutationDomain<FMSAASampleCount>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS, DepthTextureMS)
		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()
};

// This is a temporary workaround while we get AddCopyTexturePass to do a proper copy of depth texture (with source texture HTile maintained).
// This one does a depth buffer copy via pixel shader depth output. This is valid on some platform having moore complex HTile management.
void AddViewDepthCopyPSPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SourceSceneDepthTexture, FRDGTextureRef DestinationDepthTexture, bool bKeepDestStencil = false);

enum class EDepthCopyStencilMode : uint8
{
	Clear,  // Zero stencil - PS path
	Keep,   // Preserve destination stencil - allows fast CS+HTILE path when available
};

void AddViewDepthCopyPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SourceDepthTexture, FRDGTextureRef DestinationDepthTexture, EDepthCopyStencilMode StencilMode = EDepthCopyStencilMode::Clear);

};

