// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DepthRendering.cpp: Depth rendering implementation.
=============================================================================*/

#include "DepthRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Engine.h"
#include "RendererInterface.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "EngineGlobals.h"
#include "Materials/Material.h"
#include "PostProcess/SceneRenderTargets.h"
#include "GlobalShader.h"
#include "MaterialShaderType.h"
#include "MeshMaterialShaderType.h"
#include "MeshMaterialShader.h"
#include "SceneRendering.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "OneColorShader.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "ScreenRendering.h"
#include "PostProcess/SceneFilterRendering.h"
#include "DynamicPrimitiveDrawing.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "MeshPassProcessor.inl"
#include "PixelShaderUtils.h"
#include "RenderGraphUtils.h"
#include "SceneRenderingUtils.h"
#include "DebugProbeRendering.h"
#include "RenderCore.h"
#include "SimpleMeshDrawCommandPass.h"
#include "UnrealEngine.h"
#include "DepthCopy.h"
#include "PSOPrecacheSettings.h"
#include "VelocityRendering.h"

static TAutoConsoleVariable<int32> CVarParallelPrePass(
	TEXT("r.ParallelPrePass"),
	1,
	TEXT("Toggles parallel zprepass rendering. Parallel rendering must be enabled for this to have an effect."),
	ECVF_RenderThreadSafe);

static int32 GEarlyZSortMasked = 1;
static FAutoConsoleVariableRef CVarSortPrepassMasked(
	TEXT("r.EarlyZSortMasked"),
	GEarlyZSortMasked,
	TEXT("Sort EarlyZ masked draws to the end of the draw order.\n"),
	ECVF_Default
);

static TAutoConsoleVariable<int32> CVarStencilLODDitherMode(
	TEXT("r.StencilLODMode"),
	2,
	TEXT("Specifies the dither LOD stencil mode.\n")
	TEXT(" 0: Graphics pass.\n")
	TEXT(" 1: Compute pass (on supported platforms).\n")
	TEXT(" 2: Compute async pass (on supported platforms)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarStencilForLODDither(
	TEXT("r.StencilForLODDither"),
	0,
	TEXT("Whether to use stencil tests in the prepass, and depth-equal tests in the base pass to implement LOD dithering.\n")
	TEXT("If disabled, LOD dithering will be done through clip() instructions in the prepass and base pass, which disables EarlyZ.\n")
	TEXT("Forces a full prepass when enabled."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

extern bool IsHMDHiddenAreaMaskActive();

FDepthPassInfo GetDepthPassInfo(const FScene* Scene)
{
	FDepthPassInfo Info;
	Info.EarlyZPassMode = Scene ? Scene->EarlyZPassMode : DDM_None;
	Info.bEarlyZPassMovable = Scene ? Scene->bEarlyZPassMovable : false;
	Info.bDitheredLODTransitionsUseStencil = IsStencilForLODDitherEnabled(Scene ? Scene->GetShaderPlatform() : GMaxRHIShaderPlatform);
	Info.StencilDitherPassFlags = ERDGPassFlags::Raster;

	if (GRHISupportsDepthUAV && !IsHMDHiddenAreaMaskActive())
	{
		switch (CVarStencilLODDitherMode.GetValueOnAnyThread())
		{
		case 1:
			Info.StencilDitherPassFlags = ERDGPassFlags::Compute;
			break;
		case 2:
			Info.StencilDitherPassFlags = ERDGPassFlags::AsyncCompute;
			break;
		}
	}

	return Info;
}

BEGIN_SHADER_PARAMETER_STRUCT(FDepthPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

FDepthPassParameters* GetDepthPassParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef DepthTexture)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FDepthPassParameters>();
	PassParameters->View = View.GetShaderParameters();
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);
	return PassParameters;
}

const TCHAR* GetDepthDrawingModeString(EDepthDrawingMode Mode)
{
	switch (Mode)
	{
	case DDM_None:
		return TEXT("DDM_None");
	case DDM_NonMaskedOnly:
		return TEXT("DDM_NonMaskedOnly");
	case DDM_AllOccluders:
		return TEXT("DDM_AllOccluders");
	case DDM_AllOpaque:
		return TEXT("DDM_AllOpaque");
	case DDM_AllOpaqueNoVelocity:
		return TEXT("DDM_AllOpaqueNoVelocity");
	default:
		check(0);
	}

	return TEXT("");
}

DECLARE_GPU_DRAWCALL_STAT(Prepass);

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TDepthOnlyVS<true>,TEXT("/Engine/Private/PositionOnlyDepthVertexShader.usf"),TEXT("Main"),SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TDepthOnlyVS<false>,TEXT("/Engine/Private/DepthOnlyVertexShader.usf"),TEXT("Main"),SF_Vertex);

IMPLEMENT_MATERIAL_SHADER_TYPE(,FDepthOnlyPS,TEXT("/Engine/Private/DepthOnlyPixelShader.usf"),TEXT("Main"),SF_Pixel);

IMPLEMENT_SHADERPIPELINE_TYPE_VS(DepthNoPixelPipeline, TDepthOnlyVS<false>, true);
IMPLEMENT_SHADERPIPELINE_TYPE_VS(DepthPosOnlyNoPixelPipeline, TDepthOnlyVS<true>, true);
IMPLEMENT_SHADERPIPELINE_TYPE_VSPS(DepthPipeline, TDepthOnlyVS<false>, FDepthOnlyPS, true);

template <bool bPositionOnly>
bool GetDepthPassShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bMaterialUsesPixelDepthOffset,
	TShaderRef<TDepthOnlyVS<bPositionOnly>>& VertexShader,
	TShaderRef<FDepthOnlyPS>& PixelShader,
	FShaderPipelineRef& ShaderPipeline)
{
	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<TDepthOnlyVS<bPositionOnly>>();

	if (bPositionOnly)
	{
		ShaderTypes.PipelineType = &DepthPosOnlyNoPixelPipeline;
		/*ShaderPipeline = UseShaderPipelines(FeatureLevel) ? Material.GetShaderPipeline(&DepthPosOnlyNoPixelPipeline, VertexFactoryType) : FShaderPipelineRef();
		VertexShader = ShaderPipeline.IsValid()
			? ShaderPipeline.GetShader<TDepthOnlyVS<bPositionOnly> >()
			: Material.GetShader<TDepthOnlyVS<bPositionOnly> >(VertexFactoryType, 0, false);
		return VertexShader.IsValid();*/
	}
	else
	{
		const bool bVFTypeSupportsNullPixelShader = VertexFactoryType->SupportsNullPixelShader();
		const bool bNeedsPixelShader = !Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader) || bMaterialUsesPixelDepthOffset || Material.IsTranslucencyWritingCustomDepth();
		if (bNeedsPixelShader)
		{
			ShaderTypes.AddShaderType<FDepthOnlyPS>();
			ShaderTypes.PipelineType = &DepthPipeline;
		}
		else
		{
			ShaderTypes.PipelineType = &DepthNoPixelPipeline;
		}
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetPipeline(ShaderPipeline);
	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

#define IMPLEMENT_GetDepthPassShaders( bPositionOnly ) \
	template bool GetDepthPassShaders< bPositionOnly >( \
		const FMaterial& Material, \
		const FVertexFactoryType* VertexFactoryType, \
		ERHIFeatureLevel::Type FeatureLevel, \
		bool bMaterialUsesPixelDepthOffset, \
		TShaderRef<TDepthOnlyVS<bPositionOnly>>& VertexShader, \
		TShaderRef<FDepthOnlyPS>& PixelShader, \
		FShaderPipelineRef& ShaderPipeline \
	);

IMPLEMENT_GetDepthPassShaders( true );
IMPLEMENT_GetDepthPassShaders( false );

FDepthStencilStateRHIRef GetDitheredLODTransitionDepthStencilState()
{
	return TStaticDepthStencilState<true, CF_DepthNearOrEqual,
		true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		STENCIL_SANDBOX_MASK, STENCIL_SANDBOX_MASK
		>::GetRHI();
}

TOptional<uint32> GetOptionalDitheringFlag(
	const FSceneView* SceneView,
	const FMeshBatch& RESTRICT Mesh,
	int32 StaticMeshId)
{
#if UE_ENABLE_DEBUG_DRAWING
	// Check cached value from GetDynamicMeshElements first for debug view mode.
	if (Mesh.CachedLODDither != FMeshBatch::EDitherState::None)
	{
		if (Mesh.CachedLODDither == FMeshBatch::EDitherState::FadeOut)
		{
			return STENCIL_SANDBOX_MASK;
		}
		else if (Mesh.CachedLODDither == FMeshBatch::EDitherState::FadeIn)
		{
			return 0;
		}
	}
#endif
	if (SceneView && StaticMeshId >= 0 && Mesh.bDitheredLODTransition)
	{
		checkSlow(SceneView->bIsViewInfo);
		const FViewInfo* ViewInfo = (FViewInfo*)SceneView;

		if (ViewInfo->bAllowStencilDither)
		{
			if (ViewInfo->StaticMeshFadeOutDitheredLODMap[StaticMeshId])
			{
				return STENCIL_SANDBOX_MASK;
			}
			else if (ViewInfo->StaticMeshFadeInDitheredLODMap[StaticMeshId])
			{
				return 0;
			}
		}
	}

	return TOptional<uint32>();
}

/** A pixel shader used to fill the stencil buffer with the current dithered transition mask. */
class FDitheredTransitionStencilPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDitheredTransitionStencilPS);
	SHADER_USE_PARAMETER_STRUCT(FDitheredTransitionStencilPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER(float, DitheredTransitionFactor)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

};

IMPLEMENT_GLOBAL_SHADER(FDitheredTransitionStencilPS, "/Engine/Private/DitheredTransitionStencil.usf", "Main", SF_Pixel);

/** A compute shader used to fill the stencil buffer with the current dithered transition mask. */
class FDitheredTransitionStencilCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDitheredTransitionStencilCS);
	SHADER_USE_PARAMETER_STRUCT(FDitheredTransitionStencilCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, StencilOutput)
		SHADER_PARAMETER(float, DitheredTransitionFactor)
		SHADER_PARAMETER(FIntVector4, StencilOffsetAndValues)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDitheredTransitionStencilCS, "/Engine/Private/DitheredTransitionStencil.usf", "MainCS", SF_Compute);

void AddDitheredStencilFillPass(FRDGBuilder& GraphBuilder, TConstArrayView<FViewInfo> Views, FRDGTextureRef DepthTexture, const FDepthPassInfo& DepthPass)
{
	RDG_EVENT_SCOPE(GraphBuilder, "DitheredStencilPrePass");

	checkf(EnumHasAnyFlags(DepthPass.StencilDitherPassFlags, ERDGPassFlags::Raster | ERDGPassFlags::Compute | ERDGPassFlags::AsyncCompute), TEXT("Stencil dither fill pass flags are invalid."));

	if (DepthPass.StencilDitherPassFlags == ERDGPassFlags::Raster)
	{
		FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState<false, CF_Always,
			true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			STENCIL_SANDBOX_MASK, STENCIL_SANDBOX_MASK>::GetRHI();

		const uint32 StencilRef = STENCIL_SANDBOX_MASK;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			TShaderMapRef<FDitheredTransitionStencilPS> PixelShader(View.ShaderMap);

			auto* PassParameters = GraphBuilder.AllocParameters<FDitheredTransitionStencilPS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->DitheredTransitionFactor = View.GetTemporalLODTransition();
			PassParameters->RenderTargets.DepthStencil = IsMobilePlatform(View.GetShaderPlatform()) ?
				FDepthStencilBinding(DepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite):
				FDepthStencilBinding(DepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);

			FPixelShaderUtils::AddFullscreenPass(GraphBuilder, View.ShaderMap, {}, PixelShader, PassParameters, View.ViewRect, nullptr, nullptr, DepthStencilState, StencilRef);
		}
	}
	else
	{
		const int32 MaskedValue = (STENCIL_SANDBOX_MASK & 0xFF);
		const int32 ClearedValue = 0;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			TShaderMapRef<FDitheredTransitionStencilCS> ComputeShader(View.ShaderMap);

			auto* PassParameters = GraphBuilder.AllocParameters<FDitheredTransitionStencilCS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->StencilOutput = GraphBuilder.CreateUAV(FRDGTextureUAVDesc::CreateForMetaData(DepthTexture, ERDGTextureMetaDataAccess::Stencil));
			PassParameters->DitheredTransitionFactor = View.GetTemporalLODTransition();
			PassParameters->StencilOffsetAndValues = FIntVector4(View.ViewRect.Min.X, View.ViewRect.Min.Y, MaskedValue, ClearedValue);

			const FIntPoint SubExtent(
				FMath::Min(DepthTexture->Desc.Extent.X, View.ViewRect.Width()),
				FMath::Min(DepthTexture->Desc.Extent.Y, View.ViewRect.Height()));
			check(SubExtent.X > 0 && SubExtent.Y > 0);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				{},
				DepthPass.StencilDitherPassFlags,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(SubExtent, FComputeShaderUtils::kGolden2DGroupSize));
		}
	}
}

FDepthStencilStateRHIRef GetDitheredStencilClearPassDepthStencilState(bool bClearAllStencilBits)
{
	if (bClearAllStencilBits)
	{
		return TStaticDepthStencilState<false, CF_Always,
			true, CF_Always, SO_Replace, SO_Replace, SO_Replace,
			false, CF_Always, SO_Replace, SO_Replace, SO_Replace,
			0xff, 0xff
		>::GetRHI();
	}
	else
	{
		return TStaticDepthStencilState<false, CF_Always,
			true, CF_Always, SO_Replace, SO_Replace, SO_Replace,
			false, CF_Always, SO_Replace, SO_Replace, SO_Replace,
			STENCIL_SANDBOX_MASK, STENCIL_SANDBOX_MASK
		>::GetRHI();
	}
}

void AddDitheredStencilClearPass(FRDGBuilder& GraphBuilder, FRDGTextureRef DepthTexture, EShaderPlatform ShaderPlatform, bool bClearAllStencilBits)
{
	auto* PassParameters = GraphBuilder.AllocParameters<TOneColorPixelShaderMRT::FParameters>();
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);
	FIntPoint DepthSize = DepthTexture->Desc.Extent;
	const auto ShaderMap = GetGlobalShaderMap(ShaderPlatform);
	TOneColorPixelShaderMRT::FPermutationDomain PermutationVector;
	TShaderMapRef<TOneColorPixelShaderMRT> PixelShader(ShaderMap, PermutationVector);
	FRHIDepthStencilState* DepthStencilState = GetDitheredStencilClearPassDepthStencilState(bClearAllStencilBits);

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder, 
		ShaderMap,
		RDG_EVENT_NAME("AddDitheredStencilClearPass"),
		PixelShader, 
		PassParameters, 
		FIntRect(0, 0, DepthSize.X, DepthSize.Y), 
		nullptr, 
		nullptr, 
		DepthStencilState, 
		0);
}

// GPUCULL_TODO: Move to Utils file and make templated on params and mesh pass processor
static void AddViewMeshElementsPass(const TIndirectArray<FMeshBatch>& MeshElements, FRDGBuilder& GraphBuilder, FDepthPassParameters* PassParameters, const FScene* Scene, const FViewInfo& View, const FMeshPassProcessorRenderState& DrawRenderState, bool bRespectUseAsOccluderFlag, EDepthDrawingMode DepthDrawingMode, FInstanceCullingManager& InstanceCullingManager)
{
	AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, View, &InstanceCullingManager, RDG_EVENT_NAME("ViewMeshElementsPass"), View.ViewRect,
		[&View, Scene, DrawRenderState, &MeshElements, bRespectUseAsOccluderFlag, DepthDrawingMode](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
		{
			FDepthPassMeshProcessor PassMeshProcessor(
				EMeshPass::DepthPass,
				View.Family->Scene->GetRenderScene(),
				View.GetFeatureLevel(),
				&View,
				DrawRenderState,
				bRespectUseAsOccluderFlag,
				DepthDrawingMode,
				false,
				false,
				DynamicMeshPassContext);

			const uint64 DefaultBatchElementMask = ~0ull;

			for (const FMeshBatch& MeshBatch : MeshElements)
			{
				PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
			}
		}
	);
}


static void RenderPrePassEditorPrimitives(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FDepthPassParameters* PassParameters,
	const FMeshPassProcessorRenderState& DrawRenderState,
	EDepthDrawingMode DepthDrawingMode)
{
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("EditorPrimitives"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, DrawRenderState, DepthDrawingMode](FRHICommandList& RHICmdList)
	{
		const bool bRespectUseAsOccluderFlag = true;

		FSceneRenderer::SetStereoViewport(RHICmdList, View, 1.0f);

		View.SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_World, View.GetStereoPassInstanceFactor());
		View.SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_Foreground, View.GetStereoPassInstanceFactor());

		if (!View.Family->EngineShowFlags.CompositeEditorPrimitives)
		{
			DrawDynamicMeshPass(View, RHICmdList, [&](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FDepthPassMeshProcessor PassMeshProcessor(
					EMeshPass::DepthPass,
					View.Family->Scene->GetRenderScene(),
					View.GetFeatureLevel(),
					&View,
					DrawRenderState,
					bRespectUseAsOccluderFlag,
					DepthDrawingMode,
					false,
					false,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;

				for (int32 MeshIndex = 0; MeshIndex < View.ViewMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatch& MeshBatch = View.ViewMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
				}
			});

			// Draw the view's batched simple elements(lines, sprites, etc).
			View.BatchedViewElements.Draw(RHICmdList, DrawRenderState, View.FeatureLevel, View, false, 1.0f, EBlendModeFilter::All, View.GetStereoPassInstanceFactor());

			DrawDynamicMeshPass(View, RHICmdList, [&](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FDepthPassMeshProcessor PassMeshProcessor(
					EMeshPass::DepthPass,
					View.Family->Scene->GetRenderScene(),
					View.GetFeatureLevel(),
					&View,
					DrawRenderState,
					bRespectUseAsOccluderFlag,
					DepthDrawingMode,
					false,
					false,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;

				for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
				}
			});

			// Draw the view's batched simple elements(lines, sprites, etc).
			View.TopBatchedViewElements.Draw(RHICmdList, DrawRenderState, View.FeatureLevel, View, false, 1.0f, EBlendModeFilter::All, View.GetStereoPassInstanceFactor());
		}
	});
}

void SetupDepthPassState(FMeshPassProcessorRenderState& DrawRenderState)
{
	// Disable color writes, enable depth tests and writes.
	DrawRenderState.SetBlendState(TStaticBlendState<CW_NONE>::GetRHI());
	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
}

extern const TCHAR* GetDepthPassReason(bool bDitheredLODTransitionsUseStencil, EShaderPlatform ShaderPlatform);

void FDeferredShadingSceneRenderer::RenderPrePass(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FRDGTextureRef SceneDepthTexture, FInstanceCullingManager& InstanceCullingManager, FRDGTextureRef* FirstStageDepthBuffer)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, Prepass, "PrePass %s %s", GetDepthDrawingModeString(DepthPass.EarlyZPassMode), GetDepthPassReason(DepthPass.bDitheredLODTransitionsUseStencil, ShaderPlatform));
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderPrePass);

	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_RenderPrePass, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_DepthDrawTime);

	const bool bParallelDepthPass = GRHICommandList.UseParallelAlgorithms() && CVarParallelPrePass.GetValueOnRenderThread();

	RenderPrePassHMD(GraphBuilder, InViews, SceneDepthTexture);

	if (DepthPass.IsRasterStencilDitherEnabled())
	{
		AddDitheredStencilFillPass(GraphBuilder, InViews, SceneDepthTexture, DepthPass);
	}

	auto RenderDepthPass = [&](uint8 DepthMeshPass)
	{
		check(DepthMeshPass == EMeshPass::DepthPass || DepthMeshPass == EMeshPass::SecondStageDepthPass);
		const bool bSecondStageDepthPass = DepthMeshPass == EMeshPass::SecondStageDepthPass;

		if (bParallelDepthPass)
		{
			for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
			{
				FViewInfo& View = InViews[ViewIndex];
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
				RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);

				FMeshPassProcessorRenderState DrawRenderState;
				SetupDepthPassState(DrawRenderState);

				const bool bShouldRenderView = View.ShouldRenderView() && (bSecondStageDepthPass ? View.bUsesSecondStageDepthPass : true);
				if (bShouldRenderView)
				{
					View.BeginRenderView();

					FDepthPassParameters* PassParameters = GetDepthPassParameters(GraphBuilder, View, SceneDepthTexture);

					if (auto* Pass = View.ParallelMeshDrawCommandPasses[DepthMeshPass])
					{
						Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

						GraphBuilder.AddDispatchPass(
							bSecondStageDepthPass ? RDG_EVENT_NAME("SecondStageDepthPassParallel") : RDG_EVENT_NAME("DepthPassParallel"),
							PassParameters,
							ERDGPassFlags::Raster,
							[Pass, PassParameters, DepthMeshPass](FRDGDispatchPassBuilder& DispatchPassBuilder)
						{
							Pass->Dispatch(DispatchPassBuilder, &PassParameters->InstanceCullingDrawParams);
						});
					}
					else
					{
						InstanceCullingManager.SetDummyCullingParams(GraphBuilder, PassParameters->InstanceCullingDrawParams);
					}

					RenderPrePassEditorPrimitives(GraphBuilder, View, PassParameters, DrawRenderState, DepthPass.EarlyZPassMode);
				}
			}
		}
		else
		{
			for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
			{
				FViewInfo& View = InViews[ViewIndex];
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
				RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, InViews.Num() > 1, "View%d", ViewIndex);

				FMeshPassProcessorRenderState DrawRenderState;
				SetupDepthPassState(DrawRenderState);

				const bool bShouldRenderView = View.ShouldRenderView() && (bSecondStageDepthPass ? View.bUsesSecondStageDepthPass : true);
				if (bShouldRenderView)
				{
					View.BeginRenderView();

					FDepthPassParameters* PassParameters = GetDepthPassParameters(GraphBuilder, View, SceneDepthTexture);

					if (auto* Pass = View.ParallelMeshDrawCommandPasses[DepthMeshPass])
					{
						Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

						GraphBuilder.AddPass(
							bSecondStageDepthPass ? RDG_EVENT_NAME("SecondStageDepthPass") : RDG_EVENT_NAME("DepthPass"),
							PassParameters,
							ERDGPassFlags::Raster,
							[&View, Pass, PassParameters, DepthMeshPass](FRDGAsyncTask, FRHICommandList& RHICmdList)
						{
							SetStereoViewport(RHICmdList, View, 1.0f);
							Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
						});
					}
					else
					{
						InstanceCullingManager.SetDummyCullingParams(GraphBuilder, PassParameters->InstanceCullingDrawParams);
					}

					RenderPrePassEditorPrimitives(GraphBuilder, View, PassParameters, DrawRenderState, DepthPass.EarlyZPassMode);
				}
			}
		}
	};

	// Draw a depth pass to avoid overdraw in the other passes.
	if (DepthPass.EarlyZPassMode != DDM_None)
	{
		// Render primary depth pass.
		RenderDepthPass(EMeshPass::DepthPass);

		// Evaluate if any second stage depth buffer processing is required
		bool bUsesSecondStageDepthPass = false;
		for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
		{
			FViewInfo& View = InViews[ViewIndex];
			bUsesSecondStageDepthPass |= View.bUsesSecondStageDepthPass;
		}

		// Copy depth buffer and render secondary depth pass if needed.
		if(bUsesSecondStageDepthPass)
		{
			FRDGTextureDesc FirstStageDepthBufferDesc = FRDGTextureDesc::Create2D(SceneDepthTexture->Desc.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
			*FirstStageDepthBuffer = GraphBuilder.CreateTexture(FirstStageDepthBufferDesc, TEXT("FirstStageDepthBuffer"));

			for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
			{
				FViewInfo& View = InViews[ViewIndex];
				if (View.bUsesSecondStageDepthPass)
				{
					DepthCopy::AddViewDepthCopyPSPass(GraphBuilder, View, SceneDepthTexture, *FirstStageDepthBuffer);
				}
			}

			// Dispatch and render the meshes
			RenderDepthPass(EMeshPass::SecondStageDepthPass);
		}
	}

	// Dithered transition stencil mask clear, accounting for all active viewports
	if (DepthPass.bDitheredLODTransitionsUseStencil)
	{
		AddDitheredStencilClearPass(GraphBuilder, SceneDepthTexture, ShaderPlatform, true);
	}

#if !(UE_BUILD_SHIPPING)
	const bool bForwardShadingEnabled = IsForwardShadingEnabled(ShaderPlatform);
	if (!bForwardShadingEnabled)
	{
		StampDeferredDebugProbeDepthPS(GraphBuilder, InViews, SceneDepthTexture);
	}
#endif
}

bool FMobileSceneRenderer::ShouldRenderPrePass() const
{
	// Draw a depth pass to avoid overdraw in the other passes.
	return Scene->EarlyZPassMode == DDM_MaskedOnly || Scene->EarlyZPassMode == DDM_AllOpaque || Scene->EarlyZPassMode == DDM_AllOpaqueNoVelocity;
}

void FMobileSceneRenderer::RenderPrePass(FRHICommandList& RHICmdList, const FViewInfo& View, const FInstanceCullingDrawParams* InstanceCullingDrawParams)
{
	if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::DepthPass])
	{
		checkSlow(RHICmdList.IsInsideRenderPass());

		SCOPED_NAMED_EVENT(FMobileSceneRenderer_RenderPrePass, FColor::Emerald);
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, Prepass, "MobileRenderPrePass");

		SCOPE_CYCLE_COUNTER(STAT_DepthDrawTime);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderPrePass);

		SetStereoViewport(RHICmdList, View);
		Pass->Draw(RHICmdList, InstanceCullingDrawParams);
	}
}

void FDeferredShadingSceneRenderer::RenderPrePassHMD(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FRDGTextureRef DepthTexture)
{
	// Early out before we change any state if there's not a mask to render
	if (!IsHMDHiddenAreaMaskActive())
	{
		return;
	}

	auto* HMDDevice = GEngine->XRSystem->GetHMDDevice();
	if (!HMDDevice)
	{
		return;
	}

	for (const FViewInfo& View : InViews)
	{
		// Don't draw the hidden area mesh in scene captures as they are not displayed
		// through the HMD lenses.
		const bool bIsCapture = View.bIsSceneCapture || View.bIsPlanarReflection;
		if (IStereoRendering::IsStereoEyeView(View) && !bIsCapture)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			FDepthPassParameters* PassParameters = GetDepthPassParameters(GraphBuilder, View, DepthTexture);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("HiddenAreaMask"),
				PassParameters,
				ERDGPassFlags::Raster,
				[this, &View, HMDDevice](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				extern TGlobalResource<FFilterVertexDeclaration, FRenderResource::EInitPhase::Pre> GFilterVertexDeclaration;

				TShaderMapRef<TOneColorVS<true>> VertexShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_NONE>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParametersLegacyVS(RHICmdList, VertexShader, 1.0f);
				HMDDevice->DrawHiddenAreaMesh(RHICmdList, View.StereoViewIndex);
			});
		}
	}
}

FMeshDrawCommandSortKey CalculateDepthPassMeshStaticSortKey(bool bIsMasked, const FMeshMaterialShader* VertexShader, const FMeshMaterialShader* PixelShader)
{
	FMeshDrawCommandSortKey SortKey;
	if (GEarlyZSortMasked)
	{
		SortKey.BasePass.VertexShaderHash = (VertexShader ? VertexShader->GetSortKey() : 0) & 0xFFFF;
		SortKey.BasePass.PixelShaderHash = PixelShader ? PixelShader->GetSortKey() : 0;
		SortKey.BasePass.Masked = bIsMasked ? 1 : 0;
	}
	else
	{
		SortKey.Generic.VertexShaderHash = VertexShader ? VertexShader->GetSortKey() : 0;
		SortKey.Generic.PixelShaderHash = PixelShader ? PixelShader->GetSortKey() : 0;
	}
	
	return SortKey;
}

template<bool bPositionOnly>
bool FDepthPassMeshProcessor::Process(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		TDepthOnlyVS<bPositionOnly>,
		FDepthOnlyPS> DepthPassShaders;

	FShaderPipelineRef ShaderPipeline;

	if (!GetDepthPassShaders<bPositionOnly>(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		MaterialResource.MaterialUsesPixelDepthOffset_RenderThread(),
		DepthPassShaders.VertexShader,
		DepthPassShaders.PixelShader,
		ShaderPipeline))
	{
		return false;
	}

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);
	if (!bDitheredLODFadingOutMaskPass && !bShadowProjection)
	{
		TOptional<uint32> DitheringFlag = GetOptionalDitheringFlag(ViewIfDynamicMeshCommand, MeshBatch, StaticMeshId);
		// Use StencilMask for DecalOutput on mobile
		if (FeatureLevel == ERHIFeatureLevel::ES3_1)
		{
			extern void SetMobileBasePassDepthState(
				FMeshPassProcessorRenderState & DrawRenderState,
				const FPrimitiveSceneProxy * PrimitiveSceneProxy,
				const FMaterial & Material,
				FMaterialShadingModelField ShadingModels,
				bool bUsesDeferredShading,
				TOptional<uint32> DitheringFlag,
				bool bKeepStencilDither);

			// *Don't* get shading models from MaterialResource since it's for a default material
			FMaterialShadingModelField ShadingModels = MeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(ERHIFeatureLevel::ES3_1).GetShadingModels();
			bool bUsesDeferredShading = IsMobileDeferredShadingEnabled(GetFeatureLevelShaderPlatform(FeatureLevel));
			SetMobileBasePassDepthState(
				DrawRenderState,
				PrimitiveSceneProxy,
				MaterialResource,
				ShadingModels,
				bUsesDeferredShading,
				DitheringFlag,
				true);
		}
		else if (DitheringFlag.IsSet())
		{
			DrawRenderState.SetDepthStencilState(GetDitheredLODTransitionDepthStencilState());
			DrawRenderState.SetStencilRef(DitheringFlag.GetValue());
		}
	}

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

	const bool bIsMasked = IsMaskedBlendMode(MaterialResource);
	const FMeshDrawCommandSortKey SortKey = CalculateDepthPassMeshStaticSortKey(bIsMasked, DepthPassShaders.VertexShader.GetShader(), DepthPassShaders.PixelShader.GetShader());

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		DrawRenderState,
		DepthPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		bPositionOnly ? EMeshPassFeatures::PositionOnly : EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

template<bool bPositionOnly>
void FDepthPassMeshProcessor::CollectPSOInitializersInternal(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	bool bDitheredLODTransition,
	EPrimitiveType PrimitiveType,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	TMeshProcessorShaders<
		TDepthOnlyVS<bPositionOnly>,
		FDepthOnlyPS> DepthPassShaders;

	FShaderPipelineRef ShaderPipeline;

	if (!GetDepthPassShaders<bPositionOnly>(
		MaterialResource,
		VertexFactoryData.VertexFactoryType,
		FeatureLevel,
		MaterialResource.MaterialUsesPixelDepthOffset_GameThread(),
		DepthPassShaders.VertexShader,
		DepthPassShaders.PixelShader,
		ShaderPipeline))
	{
		return;
	}

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);

	// If bDitheredLODTransition option is set, then swap to that depth stencil state (see logic in SetDepthPassDitheredLODTransitionState())
	if (!bDitheredLODFadingOutMaskPass && !bShadowProjection && bDitheredLODTransition)
	{
		DrawRenderState.SetDepthStencilState(GetDitheredLODTransitionDepthStencilState());
	}

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;

	ETextureCreateFlags DepthStencilCreateFlags = SceneTexturesConfig.DepthCreateFlags;
	SetupDepthStencilInfo(PF_DepthStencil, DepthStencilCreateFlags, ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite, RenderTargetsInfo);

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		MaterialResource,
		DrawRenderState,
		RenderTargetsInfo,
		DepthPassShaders,
		MeshFillMode,
		MeshCullMode,
		PrimitiveType,
		bPositionOnly ? EMeshPassFeatures::PositionOnly : EMeshPassFeatures::Default,
		true /*bRequired*/,
		PSOInitializers);

	// Also cache with project shadow depth stencil state (see FProjectedShadowInfo::SetupMeshDrawCommandsForProjectionStenciling)
	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(SceneTexturesConfig.FeatureLevel);
	bool bUseVirtualShadowMaps = UseVirtualShadowMaps(ShaderPlatform, SceneTexturesConfig.FeatureLevel);
	if (UPSOPrecacheSettingsManager::GetSettings().bDepthProjectedShadowsState && !bUseVirtualShadowMaps)
	{
		// Set stencil to one.
		DrawRenderState.SetDepthStencilState(
			TStaticDepthStencilState<
			false, CF_DepthNearOrEqual,
			true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			0xff, 0xff
			>::GetRHI());

		AddRenderTargetInfo(PF_B8G8R8A8, TexCreate_RenderTargetable | TexCreate_ShaderResource, RenderTargetsInfo);

		AddGraphicsPipelineStateInitializer(
			VertexFactoryData,
			MaterialResource,
			DrawRenderState,
			RenderTargetsInfo,
			DepthPassShaders,
			MeshFillMode,
			MeshCullMode,
			PrimitiveType,
			bPositionOnly ? EMeshPassFeatures::PositionOnly : EMeshPassFeatures::Default,
			true /*bRequired*/,
			PSOInitializers);
	}
}

bool FDepthPassMeshProcessor::ShouldRender(const FMaterial& Material, bool bMaterialModifiesMeshPosition, bool bSupportPositionOnlyStream, bool bVFTypeSupportsNullPixelShader, bool& bUseDefaultMaterial, bool& bPositionOnly)
{
	bool bShouldRender = false;
	bUseDefaultMaterial = false;
	bPositionOnly = false;

	if (FeatureLevel == ERHIFeatureLevel::ES3_1 && EarlyZPassMode == DDM_None)
	{
		// Do not cache MDC and do not pre-cache PSOs for a depth pass if it's never going to be used on mobile platforms
		return false;
	}

	if (IsOpaqueBlendMode(Material)
		&& EarlyZPassMode != DDM_MaskedOnly
		&& bSupportPositionOnlyStream
		&& !bMaterialModifiesMeshPosition
		&& Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader))
	{
		bShouldRender = true;
		bUseDefaultMaterial = true;
		bPositionOnly = true;
	}
	else
	{
		// still possible to use default material
		const bool bMaterialMasked = !Material.WritesEveryPixel(false, bVFTypeSupportsNullPixelShader) || Material.IsTranslucencyWritingCustomDepth();
		if ((!bMaterialMasked && EarlyZPassMode != DDM_MaskedOnly) || (bMaterialMasked && EarlyZPassMode != DDM_NonMaskedOnly))
		{
			bShouldRender = true;

			if (!bMaterialMasked && !bMaterialModifiesMeshPosition)
			{
				bUseDefaultMaterial = true;
				bPositionOnly = false;
			}
		}
	}

	return bShouldRender;
}

bool FDepthPassMeshProcessor::TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material)
{
	const bool bIsTranslucent = IsTranslucentBlendMode(Material);
	bool ShouldRenderInDepthPass = (!PrimitiveSceneProxy || PrimitiveSceneProxy->ShouldRenderInDepthPass());

	bool bResult = true;
	if (!bIsTranslucent
		&& ShouldRenderInDepthPass
		&& ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain())
		&& ShouldIncludeMaterialInDefaultOpaquePass(Material))
	{
		const bool bSupportPositionOnlyStream = MeshBatch.VertexFactory->SupportsPositionOnlyStream();
		const bool bVFTypeSupportsNullPixelShader = MeshBatch.VertexFactory->SupportsNullPixelShader();
		const bool bModifiesMeshPosition = DoMaterialAndPrimitiveModifyMeshPosition(Material, PrimitiveSceneProxy);
		bool bPositionOnly = false;
		bool bUseDefaultMaterial = false;
		if (ShouldRender(Material, bModifiesMeshPosition, bSupportPositionOnlyStream, bVFTypeSupportsNullPixelShader, bUseDefaultMaterial, bPositionOnly))
		{
			const FMaterialRenderProxy* EffectiveMaterialRenderProxy = &MaterialRenderProxy;
			const FMaterial* EffectiveMaterial = &Material;
			if (bUseDefaultMaterial)
			{
				// Override with the default material
				EffectiveMaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
				EffectiveMaterial = EffectiveMaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
				check(EffectiveMaterial);
			}

			const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
			const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
			const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

			if (bPositionOnly)
			{
				bResult = Process<true>(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *EffectiveMaterialRenderProxy, *EffectiveMaterial, MeshFillMode, MeshCullMode);
			}
			else
			{
				bResult = Process<false>(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *EffectiveMaterialRenderProxy, *EffectiveMaterial, MeshFillMode, MeshCullMode);
			}
		}
	}

	return bResult;
}

void FDepthPassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	bool bDraw = MeshBatch.bUseForDepthPass;
	
	// Filter by occluder flags and settings if required.
	if (bDraw && bRespectUseAsOccluderFlag && !MeshBatch.bUseAsOccluder && EarlyZPassMode < DDM_AllOpaque)
	{
		if (PrimitiveSceneProxy)
		{
			// Only render primitives marked as occluders.
			bDraw = PrimitiveSceneProxy->ShouldUseAsOccluder()
			// Only render static objects unless movable are requested.
			&& (!PrimitiveSceneProxy->IsMovable() || bEarlyZPassMovable);

			// Filter dynamic mesh commands by screen size.
			if (ViewIfDynamicMeshCommand)
			{
				extern float GMinScreenRadiusForDepthPrepass;
				const float LODFactorDistanceSquared = (PrimitiveSceneProxy->GetBounds().Origin - ViewIfDynamicMeshCommand->ViewMatrices.GetViewOrigin()).SizeSquared() * FMath::Square(ViewIfDynamicMeshCommand->LODDistanceFactor);
				bDraw = bDraw && FMath::Square(PrimitiveSceneProxy->GetBounds().SphereRadius) > GMinScreenRadiusForDepthPrepass * GMinScreenRadiusForDepthPrepass * LODFactorDistanceSquared;
			}
		}
		else
		{
			bDraw = false;
		}
	}

	// When using DDM_AllOpaqueNoVelocity we skip objects that will write depth+velocity in the subsequent velocity pass.
	if (EarlyZPassMode == DDM_AllOpaqueNoVelocity && PrimitiveSceneProxy)
	{
		// We should ideally check to see if we this primitive is using the FOpaqueVelocityMeshProcessor or FTranslucentVelocityMeshProcessor. 
		// But for the object to get here, it would already be culled if it was translucent, so we can assume FOpaqueVelocityMeshProcessor.
		// This logic needs to match the logic in FOpaqueVelocityMeshProcessor::AddMeshBatch()
		// todo: Move that logic to a single place.

		const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
		if (FOpaqueVelocityMeshProcessor::PrimitiveCanHaveVelocity(ShaderPlatform, PrimitiveSceneProxy))
		{
			if (ViewIfDynamicMeshCommand)
			{
				if (FOpaqueVelocityMeshProcessor::PrimitiveHasVelocityForFrame(ShaderPlatform, PrimitiveSceneProxy))
				{
					checkSlow(ViewIfDynamicMeshCommand->bIsViewInfo);
					FViewInfo* ViewInfo = (FViewInfo*)ViewIfDynamicMeshCommand;

					if (FOpaqueVelocityMeshProcessor::PrimitiveHasVelocityForView(*ViewInfo, PrimitiveSceneProxy))
					{
						bDraw = false;
					}
				}
			}
		}
	}

	if (bDraw)
	{
		// Determine the mesh's material and blend mode.
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material && Material->GetRenderingThreadShaderMap())
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

void FDepthPassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{		
	if (FeatureLevel == ERHIFeatureLevel::ES3_1 && EarlyZPassMode == DDM_None)
	{
		// Do not cache MDC and do not pre-cache PSOs for a depth pass if it's never going to be used on mobile platforms
		return;
	}

	// Are we currently collecting PSO's for the default material
	if (PreCacheParams.bDefaultMaterial)
	{
		CollectDefaultMaterialPSOInitializers(SceneTexturesConfig, Material, VertexFactoryData, PSOInitializers);
		return;
	}

	// PSO precaching enabled for DitheredLODFadingOutMaskPass
	if (MeshPassType == EMeshPass::DitheredLODFadingOutMaskPass && !UPSOPrecacheSettingsManager::GetSettings().bDitheredLODFadingOutMask)
	{
		return;
	}

	const bool bIsTranslucent = IsTranslucentBlendMode(Material);

	// Early out if translucent or material shouldn't be used during this pass
	if (bIsTranslucent ||
		!PreCacheParams.bRenderInDepthPass ||
		!ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain()) || 
		!ShouldIncludeMaterialInDefaultOpaquePass(Material))
	{
		return;
	}

	// assume we can always do this when collecting PSO's for now (vertex factory instance might actually not support it)
	const bool bSupportPositionOnlyStream = VertexFactoryData.VertexFactoryType->SupportsPositionOnly();  
	const bool bVFTypeSupportsNullPixelShader = VertexFactoryData.VertexFactoryType->SupportsNullPixelShader();
	bool bPositionOnly = false;
	bool bUseDefaultMaterial = false;
	if (ShouldRender(Material, Material.MaterialModifiesMeshPosition_GameThread(), bSupportPositionOnlyStream, bVFTypeSupportsNullPixelShader, bUseDefaultMaterial, bPositionOnly))
	{
		bool bCollectPSOs = !bUseDefaultMaterial;

		// Collect PSOs for default material if there is a custom vertex declaration
		const FMaterial* EffectiveMaterial = &Material;
		if (bUseDefaultMaterial && !bSupportPositionOnlyStream && VertexFactoryData.CustomDefaultVertexDeclaration)
		{
			EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
			EffectiveMaterial = UMaterial::GetDefaultMaterial(MD_Surface)->GetMaterialResource(SceneTexturesConfig.ShaderPlatform, ActiveQualityLevel);
			bCollectPSOs = true;
		}

		if (bCollectPSOs)
		{
			check(!bPositionOnly);

			const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
			const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
			const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

			bool bIsMoveable = PreCacheParams.IsMoveable();
			const bool bAllowDitheredLODTransition = !bIsMoveable && Material.IsDitheredLODTransition();

			bool bDitheredLODTransition = false;
			CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, *EffectiveMaterial, MeshFillMode, MeshCullMode, bDitheredLODTransition, (EPrimitiveType)PreCacheParams.PrimitiveType, PSOInitializers);
			if (bAllowDitheredLODTransition)
			{
				bDitheredLODTransition = true;
				CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, *EffectiveMaterial, MeshFillMode, MeshCullMode, bDitheredLODTransition, (EPrimitiveType)PreCacheParams.PrimitiveType, PSOInitializers);
			}
		}
	}
}

void FDepthPassMeshProcessor::CollectDefaultMaterialPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	const ERasterizerFillMode MeshFillMode = FM_Solid;

	// Collect PSOs for all possible default material combinations
	{
		ERasterizerCullMode MeshCullMode = CM_None;
		bool bDitheredLODTransition = false;

		CollectPSOInitializersInternal<true>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);
		CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);

		bDitheredLODTransition = true;
		CollectPSOInitializersInternal<true>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);
		CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);
	}
	{
		ERasterizerCullMode MeshCullMode = CM_CW;
		bool bDitheredLODTransition = false;

		CollectPSOInitializersInternal<true>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);
		CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);

		bDitheredLODTransition = true;
		CollectPSOInitializersInternal<true>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);
		CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);
	}
	{
		ERasterizerCullMode MeshCullMode = CM_CCW;
		bool bDitheredLODTransition = false;

		CollectPSOInitializersInternal<true>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);
		CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);

		bDitheredLODTransition = true;
		CollectPSOInitializersInternal<true>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);
		CollectPSOInitializersInternal<false>(SceneTexturesConfig, VertexFactoryData, Material, MeshFillMode, MeshCullMode, bDitheredLODTransition, PT_TriangleList, PSOInitializers);
	}
}

FDepthPassMeshProcessor::FDepthPassMeshProcessor(
	EMeshPass::Type InMeshPassType, 
	const FScene* Scene,
	ERHIFeatureLevel::Type FeatureLevel,
	const FSceneView* InViewIfDynamicMeshCommand,
	const FMeshPassProcessorRenderState& InPassDrawRenderState,
	const bool InbRespectUseAsOccluderFlag,
	const EDepthDrawingMode InEarlyZPassMode,
	const bool InbEarlyZPassMovable,
	const bool bDitheredLODFadingOutMaskPass,
	FMeshPassDrawListContext* InDrawListContext,
	const bool bInShadowProjection,
	const bool bInSecondStageDepthPass)
	: FMeshPassProcessor(InMeshPassType, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, bRespectUseAsOccluderFlag(InbRespectUseAsOccluderFlag)
	, EarlyZPassMode(InEarlyZPassMode)
	, bEarlyZPassMovable(InbEarlyZPassMovable)
	, bDitheredLODFadingOutMaskPass(bDitheredLODFadingOutMaskPass)
	, bShadowProjection(bInShadowProjection)
	, bSecondStageDepthPass(bInSecondStageDepthPass)
{
	PassDrawRenderState = InPassDrawRenderState;
}

FMeshPassProcessor* CreateDepthPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	EDepthDrawingMode EarlyZPassMode;
	bool bEarlyZPassMovable;
	FScene::GetEarlyZPassMode(FeatureLevel, EarlyZPassMode, bEarlyZPassMovable);

	FMeshPassProcessorRenderState DepthPassState;
	SetupDepthPassState(DepthPassState);
		
	return new FDepthPassMeshProcessor(EMeshPass::DepthPass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, DepthPassState, true, EarlyZPassMode, bEarlyZPassMovable, false, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(DepthPass, CreateDepthPassProcessor, EShadingPath::Deferred, EMeshPass::DepthPass, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileDepthPass, CreateDepthPassProcessor, EShadingPath::Mobile, EMeshPass::DepthPass, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);

FMeshPassProcessor* CreateSecondStageDepthPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	EDepthDrawingMode EarlyZPassMode;
	bool bEarlyZPassMovable;
	FScene::GetEarlyZPassMode(FeatureLevel, EarlyZPassMode, bEarlyZPassMovable);

	FMeshPassProcessorRenderState DepthPassState;
	SetupDepthPassState(DepthPassState);

	return new FDepthPassMeshProcessor(EMeshPass::SecondStageDepthPass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, DepthPassState, true, EarlyZPassMode, bEarlyZPassMovable, false, InDrawListContext, false , true);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(SecondStageDepthPass, CreateSecondStageDepthPassProcessor, EShadingPath::Deferred, EMeshPass::SecondStageDepthPass, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
//Secondary depth pass is not implemented on mobile so far (see SceneVisibility.cpp)
//FRegisterPassProcessorCreateFunction RegisterMobileDepthPass(&CreateSecondStageDepthPassProcessor, EShadingPath::Mobile, EMeshPass::SecondStageDepthPass, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);

FMeshPassProcessor* CreateDitheredLODFadingOutMaskPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	EDepthDrawingMode EarlyZPassMode;
	bool bEarlyZPassMovable;
	FScene::GetEarlyZPassMode(FeatureLevel, EarlyZPassMode, bEarlyZPassMovable);

	FMeshPassProcessorRenderState DrawRenderState;

	DrawRenderState.SetBlendState(TStaticBlendState<CW_NONE>::GetRHI());
	DrawRenderState.SetDepthStencilState(
		TStaticDepthStencilState<true, CF_Equal,
		true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		STENCIL_SANDBOX_MASK, STENCIL_SANDBOX_MASK
		>::GetRHI());
	DrawRenderState.SetStencilRef(STENCIL_SANDBOX_MASK);

	return new FDepthPassMeshProcessor(EMeshPass::DitheredLODFadingOutMaskPass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, DrawRenderState, true, EarlyZPassMode, bEarlyZPassMovable, true, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(DitheredLODFadingOutMaskPass, CreateDitheredLODFadingOutMaskPassProcessor, EShadingPath::Deferred, EMeshPass::DitheredLODFadingOutMaskPass, EMeshPassFlags::MainView);
