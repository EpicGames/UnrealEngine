// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DepthRendering.h: Depth rendering definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "HitProxies.h"
#include "ShaderBaseClasses.h"
#include "MeshPassProcessor.h"
#include "RenderGraphDefinitions.h"

class FPrimitiveSceneProxy;
class FScene;
class FStaticMeshBatch;
class FViewInfo;

enum EDepthDrawingMode
{
	// tested at a higher level
	DDM_None			= 0,
	// Opaque materials only
	DDM_NonMaskedOnly	= 1,
	// Opaque and masked materials, but no objects with bUseAsOccluder disabled
	DDM_AllOccluders	= 2,
	// Full prepass, every object must be drawn and every pixel must match the base pass depth
	DDM_AllOpaque		= 3,
	// Masked materials only
	DDM_MaskedOnly = 4,
	// Full prepass, every object must be drawn and every pixel must match the base pass depth, except dynamic geometry which will render in the Velocity pass
	DDM_AllOpaqueNoVelocity	= 5,
};

extern const TCHAR* GetDepthDrawingModeString(EDepthDrawingMode Mode);

struct FDepthPassInfo
{
	bool IsComputeStencilDitherEnabled() const
	{
		return StencilDitherPassFlags != ERDGPassFlags::Raster && bDitheredLODTransitionsUseStencil;
	}

	bool IsRasterStencilDitherEnabled() const
	{
		return StencilDitherPassFlags == ERDGPassFlags::Raster && bDitheredLODTransitionsUseStencil;
	}

	EDepthDrawingMode EarlyZPassMode = DDM_None;
	bool bEarlyZPassMovable = false;
	bool bDitheredLODTransitionsUseStencil = false;
	ERDGPassFlags StencilDitherPassFlags = ERDGPassFlags::Raster;
};

extern FDepthPassInfo GetDepthPassInfo(const FScene* Scene);

void AddDitheredStencilFillPass(FRDGBuilder& GraphBuilder, TConstArrayView<FViewInfo> Views, FRDGTextureRef DepthTexture, const FDepthPassInfo& DepthPass);
void AddDitheredStencilClearPass(FRDGBuilder& GraphBuilder, FRDGTextureRef DepthTexture, EShaderPlatform ShaderPlatform, bool bClearAllStencilBits);

FMeshDrawCommandSortKey CalculateDepthPassMeshStaticSortKey(const bool bIsMasked, const FMeshMaterialShader* VertexShader, const FMeshMaterialShader* PixelShader);

/**
 * A vertex shader for rendering the depth of a mesh.
 */
template <bool bUsePositionOnlyStream>
class TDepthOnlyVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(TDepthOnlyVS,MeshMaterial);
protected:

	TDepthOnlyVS() {}

	TDepthOnlyVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer) :
		FMeshMaterialShader(Initializer)
	{}

public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		// Only the local vertex factory supports the position-only stream
		if (bUsePositionOnlyStream)
		{
			return Parameters.VertexFactoryType->SupportsPositionOnly() && Parameters.MaterialParameters.bIsSpecialEngineMaterial;
		}
		
		if (IsTranslucentBlendMode(Parameters.MaterialParameters))
		{
			return Parameters.MaterialParameters.bIsTranslucencyWritingCustomDepth;
		}

		// Only compile for the default material and masked materials
		return (
			Parameters.MaterialParameters.bIsSpecialEngineMaterial ||
			!Parameters.MaterialParameters.bWritesEveryPixel ||
			Parameters.MaterialParameters.bMaterialMayModifyMeshPosition)
			&& !Parameters.VertexFactoryType->SupportsNaniteRendering();
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FMeshMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
	}
};

/**
* A pixel shader for rendering the depth of a mesh.
*/
class FDepthOnlyPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FDepthOnlyPS,MeshMaterial);
public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		if (IsTranslucentBlendMode(Parameters.MaterialParameters))
		{
			return Parameters.MaterialParameters.bIsTranslucencyWritingCustomDepth;
		}
		
		return
			// Compile for materials that are masked
			(!Parameters.MaterialParameters.bWritesEveryPixel || Parameters.MaterialParameters.bHasPixelDepthOffsetConnected)
			&& !Parameters.VertexFactoryType->SupportsNaniteRendering();
	}

	FDepthOnlyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FMeshMaterialShader(Initializer)
	{
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		
		OutEnvironment.SetDefine(TEXT("ALLOW_DEBUG_VIEW_MODES"), AllowDebugViewmodes(Parameters.Platform));

		// Regular opaque materials do not support scene textures, but SLW does support reading custom depth/stencil,
		// so we need to make an exception here for the water depth prepass.
		const bool bIsSingleLayerWater = Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);
		OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), bIsSingleLayerWater ? 0u : 1u);
	}

	FDepthOnlyPS() {}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FMeshMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
	}
};

template <bool bPositionOnly>
bool GetDepthPassShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	bool bMaterialUsesPixelDepthOffset,
	TShaderRef<TDepthOnlyVS<bPositionOnly>>& VertexShader,
	TShaderRef<FDepthOnlyPS>& PixelShader,
	FShaderPipelineRef& ShaderPipeline);

class FDepthPassMeshProcessor : public FSceneRenderingAllocatorObject<FDepthPassMeshProcessor>, public FMeshPassProcessor
{
public:

	FDepthPassMeshProcessor(
		EMeshPass::Type InMeshPassType,
		const FScene* Scene, 
		ERHIFeatureLevel::Type FeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand, 
		const FMeshPassProcessorRenderState& InPassDrawRenderState, 
		const bool InbRespectUseAsOccluderFlag,
		const EDepthDrawingMode InEarlyZPassMode,
		const bool InbEarlyZPassMovable,
		/** Whether this mesh processor is being reused for rendering a pass that marks all fading out pixels on the screen */
		const bool bDitheredLODFadingOutMaskPass,
		FMeshPassDrawListContext* InDrawListContext,
		const bool bShadowProjection = false,
		const bool bSecondStageDepthPass = false);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:

	bool TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material);
	
	template<bool bPositionOnly>
	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	bool ShouldRender(const FMaterial& Material, bool bMaterialModifiesMeshPosition, bool bSupportPositionOnlyStream, bool bVFTypeSupportsNullPixelShader, bool& bUseDefaultMaterial, bool& bPositionOnly);

	void CollectDefaultMaterialPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig, 
		const FMaterial& Material, 
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		TArray<FPSOPrecacheData>& PSOInitializers);

	template<bool bPositionOnly>
	void CollectPSOInitializersInternal(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		bool bDitheredLODTransition, 
		EPrimitiveType PrimitiveType,
		TArray<FPSOPrecacheData>& PSOInitializers);

	FMeshPassProcessorRenderState PassDrawRenderState;

	const bool bRespectUseAsOccluderFlag;
	const EDepthDrawingMode EarlyZPassMode;
	const bool bEarlyZPassMovable;
	const bool bDitheredLODFadingOutMaskPass;
	const bool bShadowProjection;
	const bool bSecondStageDepthPass;
};

extern void SetupDepthPassState(FMeshPassProcessorRenderState& DrawRenderState);

class FRayTracingDitheredLODMeshProcessor : public FMeshPassProcessor
{
public:

	FRayTracingDitheredLODMeshProcessor(const FScene* Scene,
		const FSceneView* InViewIfDynamicMeshCommand,
		const FMeshPassProcessorRenderState& InPassDrawRenderState,
		const bool InbRespectUseAsOccluderFlag,
		const EDepthDrawingMode InEarlyZPassMode,
		const bool InbEarlyZPassMovable,
		FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;

private:

	template<bool bPositionOnly>
	void Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		EBlendMode BlendMode,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	FMeshPassProcessorRenderState PassDrawRenderState;

	const bool bRespectUseAsOccluderFlag;
	const EDepthDrawingMode EarlyZPassMode;
	const bool bEarlyZPassMovable;
};
