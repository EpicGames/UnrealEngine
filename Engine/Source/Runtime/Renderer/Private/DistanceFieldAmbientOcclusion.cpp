// Copyright Epic Games, Inc. All Rights Reserved.

#include "DistanceFieldAmbientOcclusion.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DeferredShadingRenderer.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "DistanceFieldLightingShared.h"
#include "ScreenRendering.h"
#include "DistanceFieldLightingPost.h"
#include "OneColorShader.h"
#include "GlobalDistanceField.h"
#include "FXSystem.h"
#include "LogRenderer.h"
#include "PipelineStateCache.h"
#include "VisualizeTexture.h"
#include "RayTracing/RaytracingOptions.h"
#include "Lumen/Lumen.h"
#include "MegaLights/MegaLights.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "Substrate/Substrate.h"

DEFINE_LOG_CATEGORY(LogDistanceField);

int32 GDistanceFieldAO = 1;
FAutoConsoleVariableRef CVarDistanceFieldAO(
	TEXT("r.DistanceFieldAO"),
	GDistanceFieldAO,
	TEXT("Whether the distance field AO feature is allowed, which is used to implement shadows of Movable sky lights from static meshes."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GMobileDistanceFieldAO = 0;
FAutoConsoleVariableRef CVarMobileDistanceFieldAO(
	TEXT("r.Mobile.DistanceFieldAO"),
	GMobileDistanceFieldAO,
	TEXT("Whether the distance field AO feature is allowed for Mobile Platforms, which is used to implement shadows of Movable sky lights from static meshes."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

int32 GDistanceFieldAOMultiView = 1;
FAutoConsoleVariableRef CVarDistanceFieldAOMultiView(
	TEXT("r.DistanceFieldAO.MultiView"),
	GDistanceFieldAOMultiView,
	TEXT("Whether the distance field AO feature is allowed when rendering multiple views."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GDistanceFieldAOQuality = 2;
FAutoConsoleVariableRef CVarDistanceFieldAOQuality(
	TEXT("r.AOQuality"),
	GDistanceFieldAOQuality,
	TEXT("Defines the distance field AO method which allows to adjust for quality or performance.\n")
	TEXT(" 0:off, 1:medium, 2:high (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GDistanceFieldAOApplyToStaticIndirect = 0;
FAutoConsoleVariableRef CVarDistanceFieldAOApplyToStaticIndirect(
	TEXT("r.AOApplyToStaticIndirect"),
	GDistanceFieldAOApplyToStaticIndirect,
	TEXT("Whether to apply DFAO as indirect shadowing even to static indirect sources (lightmaps + stationary skylight + reflection captures)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GDistanceFieldAOSpecularOcclusionMode = 1;
FAutoConsoleVariableRef CVarDistanceFieldAOSpecularOcclusionMode(
	TEXT("r.AOSpecularOcclusionMode"),
	GDistanceFieldAOSpecularOcclusionMode,
	TEXT("Determines how specular should be occluded by DFAO\n")
	TEXT("0: Apply non-directional AO to specular.\n")
	TEXT("1: (default) Intersect the reflection cone with the unoccluded cone produced by DFAO.  This gives more accurate occlusion than 0, but can bring out DFAO sampling artifacts.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GAOStepExponentScale = .5f;
FAutoConsoleVariableRef CVarAOStepExponentScale(
	TEXT("r.AOStepExponentScale"),
	GAOStepExponentScale,
	TEXT("Exponent used to distribute AO samples along a cone direction."),
	ECVF_RenderThreadSafe
	);

float GAOMaxViewDistance = 20000;
FAutoConsoleVariableRef CVarAOMaxViewDistance(
	TEXT("r.AOMaxViewDistance"),
	GAOMaxViewDistance,
	TEXT("The maximum distance that AO will be computed at."),
	ECVF_RenderThreadSafe
	);

int32 GAOComputeShaderNormalCalculation = 0;
FAutoConsoleVariableRef CVarAOComputeShaderNormalCalculation(
	TEXT("r.AOComputeShaderNormalCalculation"),
	GAOComputeShaderNormalCalculation,
	TEXT("Whether to use the compute shader version of the distance field normal computation."),
	ECVF_RenderThreadSafe
	);

int32 GAOSampleSet = 1;
FAutoConsoleVariableRef CVarAOSampleSet(
	TEXT("r.AOSampleSet"),
	GAOSampleSet,
	TEXT("0 = Original set, 1 = Relaxed set"),
	ECVF_RenderThreadSafe
	);

int32 GAOOverwriteSceneColor = 0;
FAutoConsoleVariableRef CVarAOOverwriteSceneColor(
	TEXT("r.AOOverwriteSceneColor"),
	GAOOverwriteSceneColor,
	TEXT(""),
	ECVF_RenderThreadSafe
	);

int32 GAOJitterConeDirections = 0;
FAutoConsoleVariableRef CVarAOJitterConeDirections(
	TEXT("r.AOJitterConeDirections"),
	GAOJitterConeDirections,
	TEXT(""),
	ECVF_RenderThreadSafe
	);

int32 GAOObjectDistanceField = 1;
FAutoConsoleVariableRef CVarAOObjectDistanceField(
	TEXT("r.AOObjectDistanceField"),
	GAOObjectDistanceField,
	TEXT("Determines whether object distance fields are used to compute ambient occlusion.\n")
	TEXT("Only global distance field will be used when this option is disabled.\n"),
	ECVF_RenderThreadSafe
);

int32 GAOGlobalDistanceFieldDetailedNecessityCheck = 1;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldDetailedCheck(
	TEXT("r.AOGlobalDistanceField.DetailedNecessityCheck"),
	GAOGlobalDistanceFieldDetailedNecessityCheck,
	TEXT("Whether to perform detailed necessity check in FSceneRenderer::ShouldPrepareGlobalDistanceField()."),
	ECVF_RenderThreadSafe
);

// if the detailed necessity check (bShouldPrepareForMaterialsOrNiagara) succeeds, this cvar can be used
// to control whether the GDF is prepared, regardless of whether DFAO is being used
int32 GGlobalDistanceFieldDetailedNecessitySufficient = 0;
FAutoConsoleVariableRef CVarGlobalDistanceFieldDetailedSufficient(
	TEXT("r.GlobalDistanceField.DetailedNecessitySufficient"),
	GGlobalDistanceFieldDetailedNecessitySufficient,
	TEXT("Whether detailed necessity checks in FSceneRenderer::ShouldPrepareGlobalDistanceField() is sufficient to enabled GDF."),
	ECVF_RenderThreadSafe
);

bool UseDistanceFieldAO()
{
	return GDistanceFieldAO && GDistanceFieldAOQuality >= 1;
}

bool UseAOObjectDistanceField()
{
	return GAOObjectDistanceField && GDistanceFieldAOQuality >= 2;
}

int32 GDistanceFieldAOTileSizeX = 16;
int32 GDistanceFieldAOTileSizeY = 16;

FDistanceFieldAOParameters::FDistanceFieldAOParameters(float InOcclusionMaxDistance, float InContrast)
{
	Contrast = FMath::Clamp(InContrast, .01f, 2.0f);
	InOcclusionMaxDistance = FMath::Clamp(InOcclusionMaxDistance, 2.0f, 3000.0f);

	if (GAOGlobalDistanceField != 0)
	{
		extern float GAOGlobalDFStartDistance;
		ObjectMaxOcclusionDistance = FMath::Min(InOcclusionMaxDistance, GAOGlobalDFStartDistance);
		GlobalMaxOcclusionDistance = InOcclusionMaxDistance >= GAOGlobalDFStartDistance ? InOcclusionMaxDistance : 0;
	}
	else
	{
		ObjectMaxOcclusionDistance = InOcclusionMaxDistance;
		GlobalMaxOcclusionDistance = 0;
	}
}

void TileIntersectionModifyCompilationEnvironment(EShaderPlatform Platform, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("CULLED_TILE_DATA_STRIDE"), CulledTileDataStride);
	OutEnvironment.SetDefine(TEXT("CULLED_TILE_SIZEX"), GDistanceFieldAOTileSizeX);
	extern int32 GConeTraceDownsampleFactor;
	OutEnvironment.SetDefine(TEXT("TRACE_DOWNSAMPLE_FACTOR"), GConeTraceDownsampleFactor);
	OutEnvironment.SetDefine(TEXT("CONE_TRACE_OBJECTS_THREADGROUP_SIZE"), ConeTraceObjectsThreadGroupSize);
}

FIntPoint GetBufferSizeForAO(const FViewInfo& View)
{
	return FIntPoint::DivideAndRoundDown(View.GetSceneTexturesConfig().Extent, GAODownsampleFactor);
}

// Sample set restricted to not self-intersect a surface based on cone angle .475882232
// Coverage of hemisphere = 0.755312979
const FVector SpacedVectors9[] = 
{
	FVector(-0.573257625, 0.625250816, 0.529563010),
	FVector(0.253354192, -0.840093017, 0.479640961),
	FVector(-0.421664953, -0.718063235, 0.553700149),
	FVector(0.249163717, 0.796005428, 0.551627457),
	FVector(0.375082791, 0.295851320, 0.878512800),
	FVector(-0.217619032, 0.00193520682, 0.976031899),
	FVector(-0.852834642, 0.0111727007, 0.522061586),
	FVector(0.745701790, 0.239393353, 0.621787369),
	FVector(-0.151036426, -0.465937436, 0.871831656)
};

// Generated from SpacedVectors9 by applying repulsion forces until convergence
const FVector RelaxedSpacedVectors9[] = 
{
	FVector(-0.467612, 0.739424, 0.484347),
	FVector(0.517459, -0.705440, 0.484346),
	FVector(-0.419848, -0.767551, 0.484347),
	FVector(0.343077, 0.804802, 0.484347),
	FVector(0.364239, 0.244290, 0.898695),
	FVector(-0.381547, 0.185815, 0.905481),
	FVector(-0.870176, -0.090559, 0.484347),
	FVector(0.874448, 0.027390, 0.484346),
	FVector(0.032967, -0.435625, 0.899524)
};

float TemporalHalton2( int32 Index, int32 Base )
{
	float Result = 0.0f;
	float InvBase = 1.0f / Base;
	float Fraction = InvBase;
	while( Index > 0 )
	{
		Result += ( Index % Base ) * Fraction;
		Index /= Base;
		Fraction *= InvBase;
	}
	return Result;
}

void GetSpacedVectors(uint32 FrameNumber, TArray<FVector, TInlineAllocator<9> >& OutVectors)
{
	OutVectors.Empty(UE_ARRAY_COUNT(SpacedVectors9));

	if (GAOSampleSet == 0)
	{
		for (int32 i = 0; i < UE_ARRAY_COUNT(SpacedVectors9); i++)
		{
			OutVectors.Add(SpacedVectors9[i]);
		}
	}
	else
	{
		for (int32 i = 0; i < UE_ARRAY_COUNT(RelaxedSpacedVectors9); i++)
		{
			OutVectors.Add(RelaxedSpacedVectors9[i]);
		}
	}

	if (GAOJitterConeDirections)
	{
		float RandomAngle = TemporalHalton2(FrameNumber & 1023, 2) * 2 * PI;
		float CosRandomAngle = FMath::Cos(RandomAngle);
		float SinRandomAngle = FMath::Sin(RandomAngle);

		for (int32 i = 0; i < OutVectors.Num(); i++)
		{
			FVector ConeDirection = OutVectors[i];
			FVector2D ConeDirectionXY(ConeDirection.X, ConeDirection.Y);
			ConeDirectionXY = FVector2D(FVector2D::DotProduct(ConeDirectionXY, FVector2D(CosRandomAngle, -SinRandomAngle)), FVector2D::DotProduct(ConeDirectionXY, FVector2D(SinRandomAngle, CosRandomAngle)));
			OutVectors[i].X = ConeDirectionXY.X;
			OutVectors[i].Y = ConeDirectionXY.Y;
		}
	}
}

// Cone half angle derived from each cone covering an equal solid angle
float GAOConeHalfAngle = FMath::Acos(1 - 1.0f / (float)UE_ARRAY_COUNT(SpacedVectors9));

// Number of AO sample positions along each cone
// Must match shader code
uint32 GAONumConeSteps = 10;

extern float GAOViewFadeDistanceScale;

FAOParameters DistanceField::SetupAOShaderParameters(const FDistanceFieldAOParameters& Parameters)
{
	const float AOLargestSampleOffset = Parameters.ObjectMaxOcclusionDistance / (1 + FMath::Tan(GAOConeHalfAngle));

	FAOParameters ShaderParameters;
	ShaderParameters.AOObjectMaxDistance = Parameters.ObjectMaxOcclusionDistance;
	ShaderParameters.AOStepScale = AOLargestSampleOffset / FMath::Pow(2.0f, GAOStepExponentScale * (GAONumConeSteps - 1));
	ShaderParameters.AOStepExponentScale = GAOStepExponentScale;
	ShaderParameters.AOMaxViewDistance = GetMaxAOViewDistance();
	ShaderParameters.AOGlobalMaxOcclusionDistance = Parameters.GlobalMaxOcclusionDistance;

	return ShaderParameters;
}

FDFAOUpsampleParameters DistanceField::SetupAOUpsampleParameters(const FViewInfo& View, FRDGTextureRef DistanceFieldAOBentNormal)
{
	const float DistanceFadeScaleValue = 1.0f / ((1.0f - GAOViewFadeDistanceScale) * GetMaxAOViewDistance());

	const FIntPoint AOBufferSize = GetBufferSizeForAO(View);
	const FIntPoint AOViewSize = View.ViewRect.Size() / GAODownsampleFactor;
	const FVector2f UVMax(
		(AOViewSize.X - 0.51f) / AOBufferSize.X, // 0.51 - so bilateral gather4 won't sample invalid texels
		(AOViewSize.Y - 0.51f) / AOBufferSize.Y);

	FDFAOUpsampleParameters ShaderParameters;
	ShaderParameters.BentNormalAOTexture = DistanceFieldAOBentNormal;
	ShaderParameters.BentNormalAOSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	ShaderParameters.AOBufferBilinearUVMax = UVMax;
	ShaderParameters.DistanceFadeScale = DistanceFadeScaleValue;
	ShaderParameters.AOMaxViewDistance = GetMaxAOViewDistance();

	return ShaderParameters;
}

bool bListMemoryNextFrame = false;

void OnListMemory(UWorld* InWorld)
{
	bListMemoryNextFrame = true;
}

FAutoConsoleCommandWithWorld ListMemoryConsoleCommand(
	TEXT("r.AOListMemory"),
	TEXT(""),
	FConsoleCommandWithWorldDelegate::CreateStatic(OnListMemory)
	);

bool bListMeshDistanceFieldsMemoryNextFrame = false;

void OnListMeshDistanceFields(UWorld* InWorld)
{
	bListMeshDistanceFieldsMemoryNextFrame = true;
}

FAutoConsoleCommandWithWorld ListMeshDistanceFieldsMemoryConsoleCommand(
	TEXT("r.AOListMeshDistanceFields"),
	TEXT(""),
	FConsoleCommandWithWorldDelegate::CreateStatic(OnListMeshDistanceFields)
	);

bool ShouldCompileDFNormalShaders(EShaderPlatform ShaderPlatform)
{
	return ShouldCompileDistanceFieldAOShaders(ShaderPlatform);
}

class FComputeDistanceFieldNormalPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FComputeDistanceFieldNormalPS);
	SHADER_USE_PARAMETER_STRUCT(FComputeDistanceFieldNormalPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDFNormalShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), GAODownsampleFactor);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GDistanceFieldAOTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GDistanceFieldAOTileSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FComputeDistanceFieldNormalPS, "/Engine/Private/DistanceFieldScreenGridLighting.usf", "ComputeDistanceFieldNormalPS", SF_Pixel);

class FComputeDistanceFieldNormalCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FComputeDistanceFieldNormalCS);
	SHADER_USE_PARAMETER_STRUCT(FComputeDistanceFieldNormalCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWDistanceFieldNormal)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDFNormalShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), GAODownsampleFactor);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GDistanceFieldAOTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GDistanceFieldAOTileSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FComputeDistanceFieldNormalCS, "/Engine/Private/DistanceFieldScreenGridLighting.usf", "ComputeDistanceFieldNormalCS", SF_Compute);

void ComputeDistanceFieldNormal(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef DistanceFieldNormal,
	const FDistanceFieldAOParameters& Parameters)
{
	const FIntPoint DFAOViewSize = View.ViewRect.Size() / GAODownsampleFactor;

	if (GAOComputeShaderNormalCalculation)
	{
		uint32 GroupSizeX = FMath::DivideAndRoundUp(DFAOViewSize.X, GDistanceFieldAOTileSizeX);
		uint32 GroupSizeY = FMath::DivideAndRoundUp(DFAOViewSize.Y, GDistanceFieldAOTileSizeY);

		auto* PassParameters = GraphBuilder.AllocParameters<FComputeDistanceFieldNormalCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->AOParameters = DistanceField::SetupAOShaderParameters(Parameters);
		PassParameters->RWDistanceFieldNormal = GraphBuilder.CreateUAV(DistanceFieldNormal);

		TShaderMapRef<FComputeDistanceFieldNormalCS> ComputeShader(View.ShaderMap);

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("ComputeNormalCS"), ComputeShader, PassParameters, FIntVector(GroupSizeX, GroupSizeY, 1));
	}
	else
	{
		auto* PassParameters = GraphBuilder.AllocParameters<FComputeDistanceFieldNormalPS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->AOParameters = DistanceField::SetupAOShaderParameters(Parameters);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(DistanceFieldNormal, ERenderTargetLoadAction::EClear);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("ComputeNormal"),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, PassParameters, DFAOViewSize](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(0, 0, 0.0f, DFAOViewSize.X, DFAOViewSize.Y, 1.0f);

			TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);

			TShaderMapRef<FComputeDistanceFieldNormalPS> PixelShader(View.ShaderMap);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			DrawRectangle(
				RHICmdList,
				0, 0,
				DFAOViewSize.X, DFAOViewSize.Y,
				0, 0,
				View.ViewRect.Width(), View.ViewRect.Height(),
				DFAOViewSize,
				View.GetSceneTexturesConfig().Extent,
				VertexShader);
		});
	}
}

/** Generates a pseudo-random position inside the unit sphere, uniformly distributed over the volume of the sphere. */
FVector GetUnitPosition2(FRandomStream& RandomStream)
{
	FVector Result;
	// Use rejection sampling to generate a valid sample
	do
	{
		Result.X = RandomStream.GetFraction() * 2 - 1;
		Result.Y = RandomStream.GetFraction() * 2 - 1;
		Result.Z = RandomStream.GetFraction() * 2 - 1;
	} while( Result.SizeSquared() > 1.f );
	return Result;
}

/** Generates a pseudo-random unit vector, uniformly distributed over all directions. */
FVector GetUnitVector2(FRandomStream& RandomStream)
{
	return GetUnitPosition2(RandomStream).GetUnsafeNormal();
}

void GenerateBestSpacedVectors()
{
	static bool bGenerated = false;
	bool bApplyRepulsion = false;

	if (bApplyRepulsion && !bGenerated)
	{
		bGenerated = true;

		FVector OriginalSpacedVectors9[UE_ARRAY_COUNT(SpacedVectors9)];

		for (int32 i = 0; i < UE_ARRAY_COUNT(OriginalSpacedVectors9); i++)
		{
			OriginalSpacedVectors9[i] = SpacedVectors9[i];
		}

		float CosHalfAngle = 1 - 1.0f / (float)UE_ARRAY_COUNT(OriginalSpacedVectors9);
		// Used to prevent self-shadowing on a plane
		float AngleBias = .03f;
		float MinAngle = FMath::Acos(CosHalfAngle) + AngleBias;
		float MinZ = FMath::Sin(MinAngle);

		// Relaxation iterations by repulsion
		for (int32 Iteration = 0; Iteration < 10000; Iteration++)
		{
			for (int32 i = 0; i < UE_ARRAY_COUNT(OriginalSpacedVectors9); i++)
			{
				FVector Force(0.0f, 0.0f, 0.0f);

				for (int32 j = 0; j < UE_ARRAY_COUNT(OriginalSpacedVectors9); j++)
				{
					if (i != j)
					{
						FVector Distance = OriginalSpacedVectors9[i] - OriginalSpacedVectors9[j];
						float Dot = OriginalSpacedVectors9[i] | OriginalSpacedVectors9[j];

						if (Dot > 0)
						{
							// Repulsion force
							Force += .001f * Distance.GetSafeNormal() * Dot * Dot * Dot * Dot;
						}
					}
				}

				FVector NewPosition = OriginalSpacedVectors9[i] + Force;
				NewPosition.Z = FMath::Max<FVector::FReal>(NewPosition.Z, MinZ);
				NewPosition = NewPosition.GetSafeNormal();
				OriginalSpacedVectors9[i] = NewPosition;
			}
		}

		for (int32 i = 0; i < UE_ARRAY_COUNT(OriginalSpacedVectors9); i++)
		{
			UE_LOGF(LogDistanceField, Log, "FVector(%f, %f, %f),", OriginalSpacedVectors9[i].X, OriginalSpacedVectors9[i].Y, OriginalSpacedVectors9[i].Z);
		}

		int32 temp = 0;
	}

	bool bBruteForceGenerateConeDirections = false;

	if (bBruteForceGenerateConeDirections)
	{
		FVector BestSpacedVectors9[9];
		float BestCoverage = 0;
		// Each cone covers an area of ConeSolidAngle = HemisphereSolidAngle / NumCones
		// HemisphereSolidAngle = 2 * PI
		// ConeSolidAngle = 2 * PI * (1 - cos(ConeHalfAngle))
		// cos(ConeHalfAngle) = 1 - 1 / NumCones
		float CosHalfAngle = 1 - 1.0f / (float)UE_ARRAY_COUNT(BestSpacedVectors9);
		// Prevent self-intersection in sample set
		float MinAngle = FMath::Acos(CosHalfAngle);
		float MinZ = FMath::Sin(MinAngle);
		FRandomStream RandomStream(123567);

		// Super slow random brute force search
		for (int i = 0; i < 1000000; i++)
		{
			FVector CandidateSpacedVectors[UE_ARRAY_COUNT(BestSpacedVectors9)];

			for (int j = 0; j < UE_ARRAY_COUNT(CandidateSpacedVectors); j++)
			{
				FVector NewSample;

				// Reject invalid directions until we get a valid one
				do 
				{
					NewSample = GetUnitVector2(RandomStream);
				} 
				while (NewSample.Z <= MinZ);

				CandidateSpacedVectors[j] = NewSample;
			}

			float Coverage = 0;
			int NumSamples = 10000;

			// Determine total cone coverage with monte carlo estimation
			for (int sample = 0; sample < NumSamples; sample++)
			{
				FVector NewSample;

				do 
				{
					NewSample = GetUnitVector2(RandomStream);
				} 
				while (NewSample.Z <= 0);

				bool bIntersects = false;

				for (int j = 0; j < UE_ARRAY_COUNT(CandidateSpacedVectors); j++)
				{
					if (FVector::DotProduct(CandidateSpacedVectors[j], NewSample) > CosHalfAngle)
					{
						bIntersects = true;
						break;
					}
				}

				Coverage += bIntersects ? 1 / (float)NumSamples : 0;
			}

			if (Coverage > BestCoverage)
			{
				BestCoverage = Coverage;

				for (int j = 0; j < UE_ARRAY_COUNT(CandidateSpacedVectors); j++)
				{
					BestSpacedVectors9[j] = CandidateSpacedVectors[j];
				}
			}
		}

		int32 temp = 0;
	}
}

void AllocateTileIntersectionBuffers(
	FRDGBuilder& GraphBuilder,
	FIntPoint TileListGroupSize,
	uint32 MaxSceneObjects,
	bool bAllow16BitIndices,
	FRDGBufferRef& OutObjectTilesIndirectArguments,
	FTileIntersectionParameters& OutParameters)
{
	// Can only use 16 bit for CulledTileDataArray if few enough objects and tiles
	const bool b16BitObjectIndices = MaxSceneObjects < (1 << 16);
	const bool b16BitCulledTileIndexBuffer = bAllow16BitIndices && b16BitObjectIndices && (TileListGroupSize.X * TileListGroupSize.Y < (1 << 16));

	int32 TileCount = TileListGroupSize.X * TileListGroupSize.Y;

	FRDGBufferRef TileConeAxisAndCos = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FVector4f), TileCount), TEXT("TileConeAxisAndCos"));
	OutParameters.RWTileConeAxisAndCos = GraphBuilder.CreateUAV(TileConeAxisAndCos);
	OutParameters.TileConeAxisAndCos = GraphBuilder.CreateSRV(TileConeAxisAndCos);

	FRDGBufferRef TileConeDepthRanges = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FVector4f), TileCount), TEXT("TileConeDepthRanges"));
	OutParameters.RWTileConeDepthRanges = GraphBuilder.CreateUAV(TileConeDepthRanges);
	OutParameters.TileConeDepthRanges = GraphBuilder.CreateSRV(TileConeDepthRanges);

	FRDGBufferRef NumCulledTilesArray = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxSceneObjects), TEXT("NumCulledTilesArray"));
	OutParameters.RWNumCulledTilesArray = GraphBuilder.CreateUAV(NumCulledTilesArray);
	OutParameters.NumCulledTilesArray = GraphBuilder.CreateSRV(NumCulledTilesArray);

	FRDGBufferRef CulledTilesStartOffsetArray = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxSceneObjects), TEXT("CulledTilesStartOffsetArray"));
	OutParameters.RWCulledTilesStartOffsetArray = GraphBuilder.CreateUAV(CulledTilesStartOffsetArray);
	OutParameters.CulledTilesStartOffsetArray = GraphBuilder.CreateSRV(CulledTilesStartOffsetArray);

	extern int32 GAverageDistanceFieldObjectsPerCullTile;

	FRDGBufferRef CulledTileDataArray = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(b16BitCulledTileIndexBuffer ? sizeof(uint16) : sizeof(uint32), GAverageDistanceFieldObjectsPerCullTile * TileCount * CulledTileDataStride),
		TEXT("CulledTileDataArray"));
	OutParameters.RWCulledTileDataArray = GraphBuilder.CreateUAV(CulledTileDataArray, b16BitCulledTileIndexBuffer ? PF_R16_UINT : PF_R32_UINT);
	OutParameters.CulledTileDataArray = GraphBuilder.CreateSRV(CulledTileDataArray, b16BitCulledTileIndexBuffer ? PF_R16_UINT : PF_R32_UINT);

	OutObjectTilesIndirectArguments = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(), TEXT("ObjectTilesIndirectArguments"));
	OutParameters.RWObjectTilesIndirectArguments = GraphBuilder.CreateUAV(OutObjectTilesIndirectArguments, PF_R32_UINT);

	OutParameters.TileListGroupSize = TileListGroupSize;
}

void ListDistanceFieldLightingMemory(const FScene* Scene, FSceneRenderer& SceneRenderer)
{
#if !NO_LOGGING
	UE_LOGF(LogRenderer, Log, "Shared GPU memory (excluding render targets)");

	if (Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0)
	{
		UE_LOGF(LogRenderer, Log, "   Scene Object data %.3fMb", Scene->DistanceFieldSceneData.GetCurrentObjectBuffers()->GetSizeBytes() / 1024.0f / 1024.0f);
	}
#endif // !NO_LOGGING
}

bool SupportsDistanceFieldAO(ERHIFeatureLevel::Type FeatureLevel, EShaderPlatform ShaderPlatform)
{
	return GDistanceFieldAO && GDistanceFieldAOQuality > 0
		// Pre-GCN AMD cards have a driver bug that prevents the global distance field from being generated correctly
		// Better to disable entirely than to display garbage
		&& !GRHIDeviceIsAMDPreGCNArchitecture
		&& DoesPlatformSupportDistanceFieldAO(ShaderPlatform)
		&& IsUsingDistanceFields(ShaderPlatform);
}

bool ShouldRenderDeferredDynamicSkyLight(const FScene* Scene, const FSceneViewFamily& ViewFamily)
{
	return Scene->SkyLight
		&& (Scene->SkyLight->ProcessedTexture || Scene->SkyLight->bRealTimeCaptureEnabled)
		&& !Scene->SkyLight->bWantsStaticShadowing
		&& !Scene->SkyLight->bHasStaticLighting
		&& ViewFamily.EngineShowFlags.SkyLighting
		&& !IsForwardShadingEnabled(Scene->GetShaderPlatform())
		&& !ViewFamily.EngineShowFlags.VisualizeLightCulling
		&& !ShouldRenderRayTracingSkyLight(Scene->SkyLight, Scene->GetShaderPlatform()); // Disable diffuse sky contribution if evaluated by RT Sky;
}

bool ShouldDoReflectionEnvironment(const FScene* Scene, const FSceneViewFamily& ViewFamily)
{
	const ERHIFeatureLevel::Type SceneFeatureLevel = Scene->GetFeatureLevel();

	return IsReflectionEnvironmentAvailable(SceneFeatureLevel)
		&& Scene->ReflectionSceneData.RegisteredReflectionCaptures.Num()
		&& ViewFamily.EngineShowFlags.ReflectionEnvironment;
}

bool ShouldPrepareForDistanceFieldAO(const FScene* Scene, const FSceneViewFamily& ViewFamily, bool bAnyViewHasGIMethodSupportingDFAO)
{
	if (SupportsDistanceFieldAO(Scene->GetFeatureLevel(), Scene->GetShaderPlatform()))
	{
		if (bAnyViewHasGIMethodSupportingDFAO && ViewFamily.EngineShowFlags.DistanceFieldAO)
		{
			if (GDistanceFieldAOApplyToStaticIndirect)
			{
				return true;
			}

			if (ShouldRenderDeferredDynamicSkyLight(Scene, ViewFamily) && Scene->SkyLight->bCastShadows)
			{
				return true;
			}
		}

		if (ViewFamily.EngineShowFlags.VisualizeDistanceFieldAO)
		{
			return true;
		}
	}

	return false;
}

bool ShouldPrepareForDistanceFieldAO(const FSceneRenderUpdateInputs& SceneUpdateInputs)
{
	for (const FSceneRenderer* Renderer : SceneUpdateInputs.Renderers)
	{
		if (ShouldPrepareForDistanceFieldAO(SceneUpdateInputs.Scene, Renderer->ViewFamily, Renderer->AnyViewHasGIMethodSupportingDFAO()))
		{
			return true;
		}
	}

	return false;
}

bool ShouldPrepareGlobalDistanceField(const FSceneRenderer* Renderer)
{
	if (!DoesProjectSupportDistanceFields() || !UseGlobalDistanceField())
	{
		return false;
	}

	const FScene* Scene = Renderer->Scene;
	const FFXSystemInterface* FXSystem = Scene->FXSystem;
	const FViewInfo& View = Renderer->Views[0];
	const FSceneViewFamily& ViewFamily = Renderer->ViewFamily;

	const bool bShouldPrepareForMaterialsOrNiagara = !GAOGlobalDistanceFieldDetailedNecessityCheck
		|| View.PrevViewInfo.bUsesGlobalDistanceField // use previous frame relevance because this frame relevance flags haven't been calculate yet
		|| ((FXSystem != nullptr) && FXSystem->UsesGlobalDistanceField());

	if (ViewFamily.EngineShowFlags.PathTracing)
	{
		return bShouldPrepareForMaterialsOrNiagara;
	}

	if (GGlobalDistanceFieldDetailedNecessitySufficient && bShouldPrepareForMaterialsOrNiagara)
	{
		return true;
	}

	if (SupportsDistanceFieldAO(Renderer->FeatureLevel, Renderer->ShaderPlatform) && (bShouldPrepareForMaterialsOrNiagara || ShouldPrepareForDistanceFieldAO(Scene, ViewFamily, Renderer->AnyViewHasGIMethodSupportingDFAO())))
	{
		return true;
	}

	if (Renderer->IsLumenEnabled(View) && Lumen::UseGlobalSDFObjectGrid(ViewFamily))
	{
		return true;
	}

	if (MegaLights::IsUsingGlobalSDF(ViewFamily))
	{
		return true;
	}

	if (ViewFamily.EngineShowFlags.VisualizeGlobalDistanceField)
	{
		return true;
	}

	return false;
}

bool ShouldPrepareDistanceFieldScene(const FSceneRenderUpdateInputs& SceneUpdateInputs)
{
	extern bool ShouldPrepareForDFInsetIndirectShadow(EShaderPlatform, const FEngineShowFlags&);
	extern bool ShouldPrepareForDistanceFieldShadows(const FSceneRenderUpdateInputs& SceneUpdateInputs);

	if (!DoesProjectSupportDistanceFields())
	{
		return false;
	}

	if (ShouldPrepareForDistanceFieldAO(SceneUpdateInputs))
	{
		return true;
	}

	if (ShouldPrepareForDistanceFieldShadows(SceneUpdateInputs))
	{
		return true;
	}

	for (const FSceneRenderer* Renderer : SceneUpdateInputs.Renderers)
	{
		const FViewFamilyInfo& ViewFamily = Renderer->ViewFamily;

		if (ShouldPrepareGlobalDistanceField(Renderer))
		{
			return true;
		}

		if (ShouldPrepareForDFInsetIndirectShadow(SceneUpdateInputs.ShaderPlatform, ViewFamily.EngineShowFlags))
		{
			return true;
		}

		if (ViewFamily.EngineShowFlags.VisualizeGlobalDistanceField || ViewFamily.EngineShowFlags.VisualizeMeshDistanceFields)
		{
			return true;
		}
	}

	return false;
}

bool ShouldRenderDistanceFieldLighting(const FDistanceFieldSceneData& SceneData, TConstArrayView<FViewInfo> Views)
{
	if (!GDistanceFieldAOMultiView && Views.Num() > 1)
	{
		return false;
	}

	bool bSupportsDistanceFieldAO = true;

	for (int32_t ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (!SupportsDistanceFieldAO(View.GetFeatureLevel(), View.GetShaderPlatform()))
		{
			bSupportsDistanceFieldAO = false;
			break;
		}
	}

	return bSupportsDistanceFieldAO && SceneData.NumObjectsInBuffer;
}

void FDeferredShadingSceneRenderer::RenderDFAOAsIndirectShadowing(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	TArray<FRDGTextureRef>& DynamicBentNormalAOTextures)
{
	if (GDistanceFieldAOApplyToStaticIndirect && ShouldRenderDistanceFieldAO(Views, ViewFamily.EngineShowFlags, AnyViewHasGIMethodSupportingDFAO()) && ShouldRenderDistanceFieldLighting(Scene->DistanceFieldSceneData, Views))
	{
		// Use the skylight's max distance if there is one, to be consistent with DFAO shadowing on the skylight
		const float OcclusionMaxDistance = Scene->SkyLight && !Scene->SkyLight->bWantsStaticShadowing ? Scene->SkyLight->OcclusionMaxDistance : Scene->DefaultMaxDistanceFieldOcclusionDistance;
		RenderDistanceFieldLighting(GraphBuilder, SceneTextures, FDistanceFieldAOParameters(OcclusionMaxDistance), DynamicBentNormalAOTextures, true, false);
	}
}

void FSceneRenderer::RenderDistanceFieldLighting(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const FDistanceFieldAOParameters& Parameters,
	TArray<FRDGTextureRef>& OutDynamicBentNormalAOTextures,
	bool bModulateToSceneColor,
	bool bVisualizeAmbientOcclusion,
	bool bModulateToScreenSpaceAO)
{
	check(ShouldRenderDistanceFieldLighting(Scene->DistanceFieldSceneData, Views));
	check(!Scene->DistanceFieldSceneData.HasPendingOperations());

	RDG_EVENT_SCOPE(GraphBuilder, "DistanceFieldLighting");
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RenderDistanceFieldLighting);

	GenerateBestSpacedVectors();

	if (bListMemoryNextFrame)
	{
		bListMemoryNextFrame = false;
		ListDistanceFieldLightingMemory(Scene, *this);
	}

	if (bListMeshDistanceFieldsMemoryNextFrame)
	{
		bListMeshDistanceFieldsMemoryNextFrame = false;
		Scene->DistanceFieldSceneData.ListMeshDistanceFields(true);
	}
	
	// We only need this texture if we need to copy multiple view outputs to an texture atlas
	OutDynamicBentNormalAOTextures.Reset();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		FRDGTextureRef DistanceFieldNormal = nullptr;

		{
			const FIntPoint BufferSize = GetBufferSizeForAO(View);
			const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(BufferSize, PF_FloatRGBA, FClearValueBinding::Transparent, GFastVRamConfig.DistanceFieldNormal | TexCreate_RenderTargetable | TexCreate_UAV | TexCreate_ShaderResource);
			DistanceFieldNormal = GraphBuilder.CreateTexture(Desc, TEXT("DistanceFieldNormal"));
		}

		ComputeDistanceFieldNormal(GraphBuilder, View, SceneTextures, DistanceFieldNormal, Parameters);

		const FIntPoint TileListGroupSize = GetTileListGroupSizeForView(View);
		const int32 MaxSceneObjects = FMath::DivideAndRoundUp(Scene->DistanceFieldSceneData.NumObjectsInBuffer, 256) * 256;
		const bool bAllow16BitIndices = !IsMetalPlatform(GShaderPlatformForFeatureLevel[View.FeatureLevel]);

		FRDGBufferRef ObjectTilesIndirectArguments = nullptr;
		FTileIntersectionParameters TileIntersectionParameters;

		AllocateTileIntersectionBuffers(GraphBuilder, TileListGroupSize, MaxSceneObjects, bAllow16BitIndices, ObjectTilesIndirectArguments, TileIntersectionParameters);

		FRDGBufferRef ObjectIndirectArguments = nullptr;
		FDistanceFieldCulledObjectBufferParameters CulledObjectBufferParameters;

		if (UseAOObjectDistanceField())
		{
			AllocateDistanceFieldCulledObjectBuffers(
				GraphBuilder,
				MaxSceneObjects,
				ObjectIndirectArguments,
				CulledObjectBufferParameters);

			CullObjectsToView(GraphBuilder, *Scene, View, Parameters, CulledObjectBufferParameters);

			BuildTileObjectLists(GraphBuilder, *Scene, View, SceneTextures, ObjectIndirectArguments, CulledObjectBufferParameters, TileIntersectionParameters, DistanceFieldNormal, Parameters);
		}

		// Render to a per-view BentNormal first because it also needs to be stored by the view history.
		// The per-view output can be copied back to the appropriate region in the output BentNormal texture.
		FRDGTextureRef PerViewBentNormal = nullptr;

		RenderDistanceFieldAOScreenGrid(
			GraphBuilder,
			SceneTextures,
			View,
			CulledObjectBufferParameters,
			ObjectTilesIndirectArguments,
			TileIntersectionParameters,
			Parameters,
			DistanceFieldNormal,
			PerViewBentNormal);

		RenderCapsuleShadowsForMovableSkylight(GraphBuilder, View, PerViewBentNormal);

		// Upsample to full resolution, write to output in case of debug AO visualization or scene color modulation (standard upsampling is done later together with sky lighting and reflection environment)
		if (bModulateToSceneColor || bModulateToScreenSpaceAO || bVisualizeAmbientOcclusion)
		{
			UpsampleBentNormalAO(GraphBuilder, View, SceneTextures, bModulateToScreenSpaceAO ? SceneTextures.ScreenSpaceAO: SceneTextures.Color.Target, PerViewBentNormal, bModulateToSceneColor && !bVisualizeAmbientOcclusion, bModulateToScreenSpaceAO && !bVisualizeAmbientOcclusion);
		}

		OutDynamicBentNormalAOTextures.Add(PerViewBentNormal);
	}
}

bool ShouldRenderDistanceFieldAO(TConstArrayView<FViewInfo> Views, const FEngineShowFlags& EngineShowFlags, bool bAnyViewHasGIMethodSupportingDFAO)
{
	bool bShouldRenderRTAO = false;
	for (int ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		bShouldRenderRTAO = bShouldRenderRTAO || ShouldRenderRayTracingAmbientOcclusion(Views[ViewIndex]);
	}

	return EngineShowFlags.DistanceFieldAO
		&& bAnyViewHasGIMethodSupportingDFAO
		&& !bShouldRenderRTAO
		&& !EngineShowFlags.VisualizeDistanceFieldAO
		&& !EngineShowFlags.VisualizeMeshDistanceFields
		&& !EngineShowFlags.VisualizeGlobalDistanceField
		&& !EngineShowFlags.PathTracing;
}
