// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldAmbientOcclusion.h
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "ShaderParameterMacros.h"

class FDistanceFieldCulledObjectBufferParameters;
class FRDGBuffer;
class FRDGBuilder;
class FRDGTexture;
class FScene;
class FSceneTextureUniformParameters;
class FSceneViewFamily;
class FViewInfo;
struct FSceneTextures;
struct FShaderCompilerEnvironment;
using FRDGBufferRef = FRDGBuffer*;
using FRDGTextureRef = FRDGTexture*;
template <typename TUniformStruct> using TRDGUniformBufferRef = TRDGUniformBuffer<TUniformStruct>*;

/** Base downsample factor that all distance field AO operations are done at. */
const int32 GAODownsampleFactor = 2;

extern const uint32 UpdateObjectsGroupSize;

extern FIntPoint GetBufferSizeForAO(const FViewInfo& View);

class FDistanceFieldAOParameters
{
public:
	float GlobalMaxOcclusionDistance;
	float ObjectMaxOcclusionDistance;
	float Contrast;

	FDistanceFieldAOParameters(float InOcclusionMaxDistance, float InContrast = 0);
};

BEGIN_SHADER_PARAMETER_STRUCT(FTileIntersectionParameters, )
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVector4f>, RWTileConeAxisAndCos)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVector4f>, RWTileConeDepthRanges)

	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWNumCulledTilesArray)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCulledTilesStartOffsetArray)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWCulledTileDataArray)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWObjectTilesIndirectArguments)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector4f>, TileConeAxisAndCos)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVector4f>, TileConeDepthRanges)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, NumCulledTilesArray)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CulledTilesStartOffsetArray)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, CulledTileDataArray)

	SHADER_PARAMETER(FIntPoint, TileListGroupSize)
END_SHADER_PARAMETER_STRUCT()

static const int32 CulledTileDataStride = 2;
static const int32 ConeTraceObjectsThreadGroupSize = 64;

void TileIntersectionModifyCompilationEnvironment(EShaderPlatform Platform, FShaderCompilerEnvironment& OutEnvironment);

BEGIN_SHADER_PARAMETER_STRUCT(FAOScreenGridParameters, )
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWScreenGridConeVisibility)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ScreenGridConeVisibility)
	SHADER_PARAMETER(FIntPoint, ScreenGridConeVisibilitySize)
END_SHADER_PARAMETER_STRUCT()

extern void GetSpacedVectors(uint32 FrameNumber, TArray<FVector, TInlineAllocator<9> >& OutVectors);

inline float GetMaxAOViewDistance()
{
	extern float GAOMaxViewDistance;
	// Scene depth stored in fp16 alpha, must fade out before it runs out of range
	// The fade extends past GAOMaxViewDistance a bit
	return FMath::Min(GAOMaxViewDistance, 65000.0f);
}

BEGIN_SHADER_PARAMETER_STRUCT(FAOParameters, )
	SHADER_PARAMETER(float, AOObjectMaxDistance)
	SHADER_PARAMETER(float, AOStepScale)
	SHADER_PARAMETER(float, AOStepExponentScale)
	SHADER_PARAMETER(float, AOMaxViewDistance)
	SHADER_PARAMETER(float, AOGlobalMaxOcclusionDistance)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FDFAOUpsampleParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BentNormalAOTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, BentNormalAOSampler)
	SHADER_PARAMETER(FVector2f, AOBufferBilinearUVMax)
	SHADER_PARAMETER(float, DistanceFadeScale)
	SHADER_PARAMETER(float, AOMaxViewDistance)
END_SHADER_PARAMETER_STRUCT()

namespace DistanceField
{
	FAOParameters SetupAOShaderParameters(const FDistanceFieldAOParameters& AOParameters);
	FDFAOUpsampleParameters SetupAOUpsampleParameters(const FViewInfo& View, FRDGTextureRef DistanceFieldAOBentNormal);
};

extern void TrackGPUProgress(FRHICommandListImmediate& RHICmdList, uint32 DebugId);

extern bool ShouldRenderDeferredDynamicSkyLight(const FScene* Scene, const FSceneViewFamily& ViewFamily);
extern bool ShouldDoReflectionEnvironment(const FScene* Scene, const FSceneViewFamily& ViewFamily);

extern void CullObjectsToView(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View, const FDistanceFieldAOParameters& Parameters, FDistanceFieldCulledObjectBufferParameters& CulledObjectBuffers);
extern void BuildTileObjectLists(FRDGBuilder& GraphBuilder,
	const FScene& Scene,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FRDGBufferRef ObjectIndirectArguments,
	const FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters,
	FTileIntersectionParameters TileIntersectionParameters,
	FRDGTextureRef DistanceFieldNormal,
	const FDistanceFieldAOParameters& Parameters);
extern FIntPoint GetTileListGroupSizeForView(const FViewInfo& View);
