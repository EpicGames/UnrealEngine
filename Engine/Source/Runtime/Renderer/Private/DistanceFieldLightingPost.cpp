// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldLightingPost.cpp
=============================================================================*/

#include "DistanceFieldLightingPost.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "DistanceFieldLightingShared.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "CompositionLighting/PostProcessAmbientOcclusion.h"
#include "PipelineStateCache.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderTargetPool.h"

int32 GAOUseHistory = 1;
FAutoConsoleVariableRef CVarAOUseHistory(
	TEXT("r.AOUseHistory"),
	GAOUseHistory,
	TEXT("Whether to apply a temporal filter to the distance field AO, which reduces flickering but also adds trails when occluders are moving."),
	ECVF_RenderThreadSafe
	);

int32 GAOClearHistory = 0;
FAutoConsoleVariableRef CVarAOClearHistory(
	TEXT("r.AOClearHistory"),
	GAOClearHistory,
	TEXT(""),
	ECVF_RenderThreadSafe
	);

int32 GAOHistoryStabilityPass = 1;
FAutoConsoleVariableRef CVarAOHistoryStabilityPass(
	TEXT("r.AOHistoryStabilityPass"),
	GAOHistoryStabilityPass,
	TEXT("Whether to gather stable results to fill in holes in the temporal reprojection.  Adds some GPU cost but improves temporal stability with foliage."),
	ECVF_RenderThreadSafe
	);

float GAOHistoryWeight = .85f;
FAutoConsoleVariableRef CVarAOHistoryWeight(
	TEXT("r.AOHistoryWeight"),
	GAOHistoryWeight,
	TEXT("Amount of last frame's AO to lerp into the final result.  Higher values increase stability, lower values have less streaking under occluder movement."),
	ECVF_RenderThreadSafe
	);

float GAOHistoryDistanceThreshold = 30;
FAutoConsoleVariableRef CVarAOHistoryDistanceThreshold(
	TEXT("r.AOHistoryDistanceThreshold"),
	GAOHistoryDistanceThreshold,
	TEXT("World space distance threshold needed to discard last frame's DFAO results.  Lower values reduce ghosting from characters when near a wall but increase flickering artifacts."),
	ECVF_RenderThreadSafe
	);

float GAOViewFadeDistanceScale = .7f;
FAutoConsoleVariableRef CVarAOViewFadeDistanceScale(
	TEXT("r.AOViewFadeDistanceScale"),
	GAOViewFadeDistanceScale,
	TEXT("Distance over which AO will fade out as it approaches r.AOMaxViewDistance, as a fraction of r.AOMaxViewDistance."),
	ECVF_RenderThreadSafe
	);

bool UseAOHistoryStabilityPass()
{
	extern int32 GDistanceFieldAOQuality;
	return GAOHistoryStabilityPass && GDistanceFieldAOQuality >= 2;
}

bool ShouldCompileDFLightingPostShaders(EShaderPlatform ShaderPlatform)
{
	return ShouldCompileDistanceFieldAOShaders(ShaderPlatform);
}

BEGIN_SHADER_PARAMETER_STRUCT(FGeometryAwareUpsampleParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistanceFieldNormalTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, DistanceFieldNormalSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BentNormalAOTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, BentNormalAOSampler)
	SHADER_PARAMETER(FScreenTransform, UVToScreenPos)
	SHADER_PARAMETER(FVector4f, BentNormalBufferAndTexelSize)
	SHADER_PARAMETER(FVector2f, DistanceFieldGBufferTexelSize)
	SHADER_PARAMETER(FVector2f, DistanceFieldGBufferJitterOffset)
	SHADER_PARAMETER(FVector2f, JitterOffset)
	SHADER_PARAMETER(float, DistanceFadeScale)
END_SHADER_PARAMETER_STRUCT()

FGeometryAwareUpsampleParameters SetupGeometryAwareUpsampleParameters(const FViewInfo& View, FRDGTextureRef DistanceFieldNormal, FRDGTextureRef DistanceFieldAOBentNormal)
{
	extern FVector2f GetJitterOffset(const FViewInfo& View);
	FVector2f const JitterOffsetValue = GetJitterOffset(View);

	const FIntPoint DownsampledBufferSize = GetBufferSizeForAO(View);
	const FVector2f BaseLevelTexelSizeValue(1.0f / DownsampledBufferSize.X, 1.0f / DownsampledBufferSize.Y);

	extern FIntPoint GetBufferSizeForConeTracing(const FViewInfo& View);
	const FIntPoint ConeTracingBufferSize = GetBufferSizeForConeTracing(View);
	const FVector4f BentNormalBufferAndTexelSizeValue(ConeTracingBufferSize.X, ConeTracingBufferSize.Y, 1.0f / ConeTracingBufferSize.X, 1.0f / ConeTracingBufferSize.Y);

	extern float GAOViewFadeDistanceScale;
	const float DistanceFadeScaleValue = 1.0f / ((1.0f - GAOViewFadeDistanceScale) * GetMaxAOViewDistance());

	const FIntRect AOViewRect = FIntRect(FIntPoint::ZeroValue, FIntPoint::DivideAndRoundDown(View.ViewRect.Size(), GAODownsampleFactor));

	FGeometryAwareUpsampleParameters ShaderParameters;
	ShaderParameters.DistanceFieldNormalTexture = DistanceFieldNormal;
	ShaderParameters.DistanceFieldNormalSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	ShaderParameters.BentNormalAOTexture = DistanceFieldAOBentNormal;
	ShaderParameters.BentNormalAOSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	ShaderParameters.UVToScreenPos = FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(DownsampledBufferSize, AOViewRect), FScreenTransform::ETextureBasis::TextureUV, FScreenTransform::ETextureBasis::ScreenPosition);
	ShaderParameters.BentNormalBufferAndTexelSize = BentNormalBufferAndTexelSizeValue;
	ShaderParameters.DistanceFieldGBufferTexelSize = BaseLevelTexelSizeValue;
	ShaderParameters.DistanceFieldGBufferJitterOffset = BaseLevelTexelSizeValue * JitterOffsetValue;
	ShaderParameters.JitterOffset = JitterOffsetValue;
	ShaderParameters.DistanceFadeScale = DistanceFadeScaleValue;

	return ShaderParameters;
}

class FUpdateHistoryDepthRejectionPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FUpdateHistoryDepthRejectionPS);
	SHADER_USE_PARAMETER_STRUCT(FUpdateHistoryDepthRejectionPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGeometryAwareUpsampleParameters, GeometryAwareUpsampleParameters)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BentNormalHistoryTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BentNormalHistorySampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VelocityTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, VelocityTextureSampler)
		SHADER_PARAMETER(FVector4f, HistoryScreenPositionScaleBias)
		SHADER_PARAMETER(FVector4f, HistoryUVMinMax)
		SHADER_PARAMETER(float, HistoryWeight)
		SHADER_PARAMETER(float, HistoryDistanceThreshold)
		SHADER_PARAMETER(float, UseHistoryFilter)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDFLightingPostShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), GAODownsampleFactor);
	}
};

IMPLEMENT_GLOBAL_SHADER(FUpdateHistoryDepthRejectionPS, "/Engine/Private/DistanceFieldLightingPost.usf", "UpdateHistoryDepthRejectionPS", SF_Pixel);

class FFilterHistoryPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FFilterHistoryPS);
	SHADER_USE_PARAMETER_STRUCT(FFilterHistoryPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BentNormalAOTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BentNormalAOSampler)
		SHADER_PARAMETER(FVector2f, BentNormalAOTexelSize)
		SHADER_PARAMETER(FVector2f, MaxSampleBufferUV)
		SHADER_PARAMETER(float, HistoryWeight)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	class FManuallyClampUV : SHADER_PERMUTATION_BOOL("MANUALLY_CLAMP_UV");
	using FPermutationDomain = TShaderPermutationDomain<FManuallyClampUV>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDFLightingPostShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), GAODownsampleFactor);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFilterHistoryPS, "/Engine/Private/DistanceFieldLightingPost.usf", "FilterHistoryPS", SF_Pixel);

class FGeometryAwareUpsamplePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FGeometryAwareUpsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FGeometryAwareUpsamplePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGeometryAwareUpsampleParameters, GeometryAwareUpsampleParameters)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDFLightingPostShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), GAODownsampleFactor);
	}
};

IMPLEMENT_SHADER_TYPE(,FGeometryAwareUpsamplePS, TEXT("/Engine/Private/DistanceFieldLightingPost.usf"), TEXT("GeometryAwareUpsamplePS"), SF_Pixel);

void AllocateOrReuseAORenderTarget(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef& Texture, const TCHAR* Name, EPixelFormat Format, ETextureCreateFlags Flags)
{
	if (!Texture)
	{
		const FIntPoint BufferSize = GetBufferSizeForAO(View);
		const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(BufferSize, Format, FClearValueBinding::None, Flags | TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
		Texture = GraphBuilder.CreateTexture(Desc, Name);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Texture), FLinearColor::Black);
	}
}

void GeometryAwareUpsample(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef DistanceFieldAOBentNormal, FRDGTextureRef DistanceFieldNormal, FRDGTextureRef BentNormalInterpolation, const FDistanceFieldAOParameters& Parameters)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FGeometryAwareUpsamplePS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->AOParameters = DistanceField::SetupAOShaderParameters(Parameters);
	PassParameters->GeometryAwareUpsampleParameters = SetupGeometryAwareUpsampleParameters(View, DistanceFieldNormal, BentNormalInterpolation ? BentNormalInterpolation : GSystemTextures.GetWhiteDummy(GraphBuilder));
	PassParameters->RenderTargets[0] = FRenderTargetBinding(DistanceFieldAOBentNormal, ERenderTargetLoadAction::ELoad);

	auto VertexShader = View.ShaderMap->GetShader<FPostProcessVS>();
	auto PixelShader = View.ShaderMap->GetShader<FGeometryAwareUpsamplePS>();

	ClearUnusedGraphResources(PixelShader, PassParameters);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("GeometryAwareUpsample"),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, VertexShader, PixelShader, &View](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		const FIntPoint AOBufferSize = GetBufferSizeForAO(View);
		const FIntPoint AOViewSize = View.ViewRect.Size() / GAODownsampleFactor;

		RHICmdList.SetViewport(0, 0, 0.0f, AOViewSize.X, AOViewSize.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			AOViewSize.X, AOViewSize.Y,
			0, 0,
			AOViewSize.X, AOViewSize.Y,
			AOViewSize,
			AOBufferSize,
			VertexShader);
	});
}

bool DistanceFieldAOUseHistory(const FViewInfo& View)
{
	// Disable AO history for cube map captures to save memory.  AO history (DistanceFieldAOHistoryRT) is sized to the scene texture extents, meaning
	// the size is proportional to the front buffer resolution, regardless of the resolution of the cube map capture itself.  This then gets multiplied
	// by 6 for cube map faces.  This totals around 25 MB at 1080p or 100 MB at 4K resolution.
	//
	// JHOERNER_TODO:  Ideally, the code would be rewritten so DistanceFieldAOHistoryRT is sized to the view rectangle, rather than scene texture extents.
	// The goal of scene texture extents is to reuse render target memory on platforms like PC, by making all scene renders use the same sized render
	// targets for things like gbuffers.  This advantage doesn't apply to temporal history buffers, since these persist across frames and are unique per
	// scene view state, and never shared.  As an example, temporal volumetric cloud buffers (VolumetricCloudRenderTarget) are scaled to the view rect,
	// and so don't involve a similar memory waste.
	//
	// I did investigate fixing the memory waste myself, and started attempting it, but it's an involved process due to the number of places in the code
	// you'd need to touch.  I figured it would be better if someone that has worked with the code before did the changes, to make sure nothing relevant
	// was missed, and hopefully they'd be able to visually tell if it was working correctly.
	//
	// At a basic level, you'd modify UpdateHistory below so instead of using "View.GetSceneTexturesConfig().Extent" to determine the history dimensions,
	// you'd use the view rect (multiple places in that function).  You'd eliminate "HistoryScreenPositionScaleBiasValue" and "HistoryUVMinMaxValue" scale
	// and clamping factors when reading from the history in shaders.  The input scene textures (VelocityTexture, etc) would still be in scene texture
	// space, so there would be a mix of UV math going on, depending on what textures you were sampling from.
	//
	// If it was just the shaders in UpdateHistory, I'd give it a shot, but there are additional shaders referencing "BentNormalAOTexture", including from
	// DistanceFieldAOShared.ush and DistanceFieldLightingPost.usf.  You'd need to update all the shaders, and possibly add extra parameters to make
	// information available about the resolution of the history (can't use View.BufferSizeAndInvSize).  Any typos could cause artifacts which will only
	// be visually obvious if you set up a synthetic test where the scene texture extents don't match the view rect (you should be able to accomplish
	// this for debug purposes by modifying FSceneTextureExtentState::Compute to add some padding to the scene textures).
	//
	return GAOUseHistory && !View.bIsSceneCaptureCube;
}

void UpdateHistory(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, 
	const TCHAR* BentNormalHistoryRTName,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef VelocityTexture,
	FRDGTextureRef DistanceFieldNormal,
	FRDGTextureRef BentNormalInterpolation,
	/** Contains last frame's history, if non-NULL.  This will be updated with the new frame's history. */
	FIntRect* DistanceFieldAOHistoryViewRect,
	TRefCountPtr<IPooledRenderTarget>* BentNormalHistoryState,
	/** Output of Temporal Reprojection for the next step in the pipeline. */
	FRDGTextureRef& BentNormalHistoryOutput,
	const FDistanceFieldAOParameters& Parameters)
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	const FIntPoint AOBufferSize = GetBufferSizeForAO(View);
	const FIntPoint AOViewSize = View.ViewRect.Size() / GAODownsampleFactor;

	if (BentNormalHistoryState && DistanceFieldAOUseHistory(View))
	{
		if (*BentNormalHistoryState 
			&& !View.bCameraCut 
			&& !View.bPrevTransformsReset
			&& !GAOClearHistory
			// If the scene render targets reallocate, toss the history so we don't read uninitialized data
			&& (*BentNormalHistoryState)->GetDesc().Extent == AOBufferSize)
		{
			FRDGTextureRef BentNormalHistoryTexture = GraphBuilder.RegisterExternalTexture(*BentNormalHistoryState);

			ETextureCreateFlags HistoryPassOutputFlags = ETextureCreateFlags(UseAOHistoryStabilityPass() ? GFastVRamConfig.DistanceFieldAOHistory : TexCreate_None);
			// Reuse a render target from the pool with a consistent name, for vis purposes
			FRDGTextureRef NewBentNormalHistory = nullptr;
			AllocateOrReuseAORenderTarget(GraphBuilder, View, NewBentNormalHistory, BentNormalHistoryRTName, PF_FloatRGBA, HistoryPassOutputFlags);

			{
				FIntRect PrevHistoryViewRect = *DistanceFieldAOHistoryViewRect;

				auto* PassParameters = GraphBuilder.AllocParameters<FUpdateHistoryDepthRejectionPS::FParameters>();
				PassParameters->View = View.ViewUniformBuffer;
				PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);;
				PassParameters->AOParameters = DistanceField::SetupAOShaderParameters(Parameters);
				PassParameters->GeometryAwareUpsampleParameters = SetupGeometryAwareUpsampleParameters(View, DistanceFieldNormal, BentNormalInterpolation);
				PassParameters->BentNormalHistoryTexture = BentNormalHistoryTexture;
				PassParameters->BentNormalHistorySampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->VelocityTexture = GetIfProduced(VelocityTexture, SystemTextures.Black);
				PassParameters->VelocityTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

				PassParameters->HistoryWeight = GAOHistoryWeight;
				PassParameters->HistoryDistanceThreshold = GAOHistoryDistanceThreshold;
				PassParameters->UseHistoryFilter = UseAOHistoryStabilityPass() ? 1.0f : 0.0f;

				{
					const float InvBufferSizeX = 1.0f / AOBufferSize.X;
					const float InvBufferSizeY = 1.0f / AOBufferSize.Y;

					const FVector4f HistoryScreenPositionScaleBiasValue(
						PrevHistoryViewRect.Width() * InvBufferSizeX / +2.0f,
						PrevHistoryViewRect.Height() * InvBufferSizeY / (-2.0f * GProjectionSignY),
						(PrevHistoryViewRect.Height() / 2.0f + PrevHistoryViewRect.Min.Y) * InvBufferSizeY,
						(PrevHistoryViewRect.Width() / 2.0f + PrevHistoryViewRect.Min.X) * InvBufferSizeX);

					// Pull in the max UV to exclude the region which will read outside the viewport due to bilinear filtering
					const FVector4f HistoryUVMinMaxValue(
						(PrevHistoryViewRect.Min.X + 0.5f) * InvBufferSizeX,
						(PrevHistoryViewRect.Min.Y + 0.5f) * InvBufferSizeY,
						(PrevHistoryViewRect.Max.X - 0.5f) * InvBufferSizeX,
						(PrevHistoryViewRect.Max.Y - 0.5f) * InvBufferSizeY);

					PassParameters->HistoryScreenPositionScaleBias = HistoryScreenPositionScaleBiasValue;
					PassParameters->HistoryUVMinMax = HistoryUVMinMaxValue;
				}

				PassParameters->RenderTargets[0] = FRenderTargetBinding(NewBentNormalHistory, ERenderTargetLoadAction::ELoad);

				TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);
				TShaderMapRef<FUpdateHistoryDepthRejectionPS> PixelShader(View.ShaderMap);

				ClearUnusedGraphResources(PixelShader, PassParameters);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("UpdateHistory"),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, VertexShader, PixelShader, &View, AOBufferSize, AOViewSize]
					(FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					RHICmdList.SetViewport(0, 0, 0.0f, AOViewSize.X, AOViewSize.Y, 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

					DrawRectangle(
						RHICmdList,
						0, 0,
						AOViewSize.X, AOViewSize.Y,
						0, 0,
						AOViewSize.X, AOViewSize.Y,
						AOViewSize,
						AOBufferSize,
						VertexShader);
				});
			}

			if (UseAOHistoryStabilityPass())
			{
				const FRDGTextureDesc& HistoryDesc = BentNormalHistoryTexture->Desc;

				// Reallocate history if buffer sizes have changed
				if (HistoryDesc.Extent != AOBufferSize)
				{
					GRenderTargetPool.FreeUnusedResource(*BentNormalHistoryState);
					*BentNormalHistoryState = nullptr;
					// Update the view state's render target reference with the new history
					AllocateOrReuseAORenderTarget(GraphBuilder, View, BentNormalHistoryTexture, BentNormalHistoryRTName, PF_FloatRGBA);
				}

				const bool bManuallyClampUV = View.ViewRect.Min != FIntPoint::ZeroValue || View.ViewRect.Max != AOBufferSize;
				FFilterHistoryPS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FFilterHistoryPS::FManuallyClampUV>(bManuallyClampUV);

				auto VertexShader = View.ShaderMap->GetShader<FPostProcessVS>();
				auto PixelShader = View.ShaderMap->GetShader<FFilterHistoryPS>(PermutationVector);

				FVector2f MaxSampleBufferUV(
					(AOViewSize.X - 0.5f - GAODownsampleFactor) / AOBufferSize.X,
					(AOViewSize.Y - 0.5f - GAODownsampleFactor) / AOBufferSize.Y);

				auto* PassParameters = GraphBuilder.AllocParameters<FFilterHistoryPS::FParameters>();
				PassParameters->View = View.ViewUniformBuffer;
				PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);;
				PassParameters->BentNormalAOTexture = NewBentNormalHistory;
				PassParameters->BentNormalAOSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->BentNormalAOTexelSize = FVector2f(1.0f / AOBufferSize.X, 1.0f / AOBufferSize.Y);
				PassParameters->MaxSampleBufferUV = MaxSampleBufferUV;
				PassParameters->HistoryWeight = GAOHistoryWeight;
				PassParameters->RenderTargets[0] = FRenderTargetBinding(BentNormalHistoryTexture, ERenderTargetLoadAction::ELoad);

				ClearUnusedGraphResources(PixelShader, PassParameters);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("UpdateHistoryStability"),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, VertexShader, PixelShader, &View, AOBufferSize, AOViewSize]
					(FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					RHICmdList.SetViewport(0, 0, 0.0f, AOViewSize.X, AOViewSize.Y, 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

					DrawRectangle(
						RHICmdList,
						0, 0,
						AOViewSize.X, AOViewSize.Y,
						0, 0,
						AOViewSize.X, AOViewSize.Y,
						AOViewSize ,
						AOBufferSize,
						VertexShader);
				});

				GraphBuilder.QueueTextureExtraction(BentNormalHistoryTexture, BentNormalHistoryState);
				BentNormalHistoryOutput = BentNormalHistoryTexture;
			}
			else
			{
				// Update the view state's render target reference with the new history
				GraphBuilder.QueueTextureExtraction(NewBentNormalHistory, BentNormalHistoryState);
				BentNormalHistoryOutput = NewBentNormalHistory;
			}
		}
		else
		{
			// Use the current frame's upscaled mask for next frame's history
			FRDGTextureRef DistanceFieldAOBentNormal = nullptr;
			AllocateOrReuseAORenderTarget(GraphBuilder, View, DistanceFieldAOBentNormal, TEXT("PerViewDistanceFieldBentNormalAO"), PF_FloatRGBA, GFastVRamConfig.DistanceFieldAOHistory);

			GeometryAwareUpsample(GraphBuilder, View, DistanceFieldAOBentNormal, DistanceFieldNormal, BentNormalInterpolation, Parameters);

			GraphBuilder.QueueTextureExtraction(DistanceFieldAOBentNormal, BentNormalHistoryState);
			BentNormalHistoryOutput = DistanceFieldAOBentNormal;
		}

		DistanceFieldAOHistoryViewRect->Min = FIntPoint::ZeroValue;
		DistanceFieldAOHistoryViewRect->Max = View.ViewRect.Size() / FIntPoint(GAODownsampleFactor, GAODownsampleFactor);
	}
	else
	{
		// Temporal reprojection is disabled or there is no view state - just upscale
		FRDGTextureRef DistanceFieldAOBentNormal = nullptr;
		AllocateOrReuseAORenderTarget(GraphBuilder, View, DistanceFieldAOBentNormal, TEXT("PerViewDistanceFieldBentNormalAO"), PF_FloatRGBA, GFastVRamConfig.DistanceFieldAOHistory);

		GeometryAwareUpsample(GraphBuilder, View, DistanceFieldAOBentNormal, DistanceFieldNormal, BentNormalInterpolation, Parameters);

		BentNormalHistoryOutput = DistanceFieldAOBentNormal;
	}
}

class FDistanceFieldAOUpsamplePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDistanceFieldAOUpsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FDistanceFieldAOUpsamplePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDFAOUpsampleParameters, DFAOUpsampleParameters)
		SHADER_PARAMETER(float, MinIndirectDiffuseOcclusion)
		RDG_TEXTURE_ACCESS(DistanceFieldAOBentNormal, ERHIAccess::SRVGraphics)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	class FModulateToSceneColorDim : SHADER_PERMUTATION_BOOL("MODULATE_SCENE_COLOR");
	using FPermutationDomain = TShaderPermutationDomain<FModulateToSceneColorDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDFLightingPostShaders(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDistanceFieldAOUpsamplePS, "/Engine/Private/DistanceFieldLightingPost.usf", "AOUpsamplePS", SF_Pixel);

void UpsampleBentNormalAO(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef DistanceFieldAOBentNormal,
	bool bModulateSceneColor,
	bool bModulateToScreenSpaceAO)
{
	FScene* Scene = (FScene*)View.Family->Scene;

	RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

	auto* PassParameters = GraphBuilder.AllocParameters<FDistanceFieldAOUpsamplePS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);;
	PassParameters->DFAOUpsampleParameters = DistanceField::SetupAOUpsampleParameters(View, DistanceFieldAOBentNormal ? DistanceFieldAOBentNormal : GSystemTextures.GetWhiteDummy(GraphBuilder));
	PassParameters->MinIndirectDiffuseOcclusion = Scene->SkyLight ? Scene->SkyLight->MinOcclusion : 0;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);
	PassParameters->DistanceFieldAOBentNormal = DistanceFieldAOBentNormal;

	TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);

	FDistanceFieldAOUpsamplePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FDistanceFieldAOUpsamplePS::FModulateToSceneColorDim>(bModulateSceneColor || bModulateToScreenSpaceAO);
	TShaderMapRef<FDistanceFieldAOUpsamplePS> PixelShader(View.ShaderMap, PermutationVector);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("UpsampleAO"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, PassParameters, &View, DistanceFieldAOBentNormal, bModulateSceneColor, bModulateToScreenSpaceAO](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			if (bModulateSceneColor || bModulateToScreenSpaceAO)
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_Zero, BO_Add, BF_Zero, BF_One>::GetRHI();
			}
			else
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			}

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			const int32 DownsampleFactor = bModulateToScreenSpaceAO ? 1.f : GAODownsampleFactor;
			DrawRectangle(
				RHICmdList,
				0, 0,
				View.ViewRect.Width(), View.ViewRect.Height(),
				View.ViewRect.Min.X / DownsampleFactor, View.ViewRect.Min.Y / DownsampleFactor,
				View.ViewRect.Width() / DownsampleFactor, View.ViewRect.Height() / DownsampleFactor,
				FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()),
				GetBufferSizeForAO(View),
				VertexShader);
		});
}