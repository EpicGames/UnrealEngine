// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldLightingPost.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "SceneRendering.h"

class FDistanceFieldAOParameters;

extern void AllocateOrReuseAORenderTarget(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef& Target, const TCHAR* Name, EPixelFormat Format, ETextureCreateFlags Flags = TexCreate_None);

extern void UpdateHistory(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const TCHAR* BentNormalHistoryRTName,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef VelocityTexture,
	FRDGTextureRef BentNormalInterpolation,
	FRDGTextureRef DistanceFieldNormal,
	/** Contains last frame's history, if non-NULL.  This will be updated with the new frame's history. */
	FIntRect* DistanceFieldAOHistoryViewRect,
	TRefCountPtr<IPooledRenderTarget>* BentNormalHistoryState,
	/** Output of Temporal Reprojection for the next step in the pipeline. */
	FRDGTextureRef& BentNormalHistoryOutput,
	const FDistanceFieldAOParameters& Parameters);

extern void UpsampleBentNormalAO(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef DistanceFieldAOBentNormal,
	bool bModulateSceneColor,
	bool bModulateToScreenSpaceAO);

extern void MobileAggregateDFAOToSSAO(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef DistanceFieldAOBentNormal);
