// Copyright Epic Games, Inc. All Rights Reserved.

#include "DistanceFieldLightingShared.h"
#include "RenderGraphUtils.h"

FDistanceFieldObjectBuffers::FDistanceFieldObjectBuffers() = default;
FDistanceFieldObjectBuffers::~FDistanceFieldObjectBuffers() = default;

void FDistanceFieldObjectBuffers::Initialize()
{
}

void FDistanceFieldObjectBuffers::Release()
{
	Bounds = nullptr;
	Data = nullptr;
}

size_t FDistanceFieldObjectBuffers::GetSizeBytes() const
{
	return TryGetSize(Bounds) + TryGetSize(Data);
}
