// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldLightingShared.h
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "ShaderParameterMacros.h"

class FDistanceFieldSceneData;
class FLightSceneProxy;
class FMaterialRenderProxy;
class FPrimitiveSceneInfo;
class FRDGBuffer;
class FRDGBuilder;
class FRDGPooledBuffer;
class FScene;
class FSceneRenderer;
class FSceneViewFamily;
class FShaderParameterMap;
class FViewInfo;

struct FEngineShowFlags;

using FRDGBufferRef = FRDGBuffer*;

DECLARE_LOG_CATEGORY_EXTERN(LogDistanceField, Log, All);

/** Tile sized used for most AO compute shaders. */
extern int32 GDistanceFieldAOTileSizeX;
extern int32 GDistanceFieldAOTileSizeY;
extern int32 GAverageObjectsPerShadowCullTile;
extern int32 GAverageHeightFieldObjectsPerShadowCullTile;

extern bool UseDistanceFieldAO();
extern bool UseAOObjectDistanceField();

enum EDistanceFieldPrimitiveType
{
	DFPT_SignedDistanceField,
	DFPT_HeightField,
	DFPT_Num
};

// Must match equivalent shader defines
static const int32 GDistanceFieldObjectDataStride = 10;
static const int32 GDistanceFieldObjectBoundsStride = 3;

static const int32 GHeightFieldObjectDataStride = 7;
static const int32 GHeightFieldObjectBoundsStride = 3;

class FDistanceFieldObjectBuffers
{
public:
	TRefCountPtr<FRDGPooledBuffer> Bounds;
	TRefCountPtr<FRDGPooledBuffer> Data;

	FDistanceFieldObjectBuffers();
	~FDistanceFieldObjectBuffers();

	void Initialize();
	void Release();

	size_t GetSizeBytes() const;
};

BEGIN_SHADER_PARAMETER_STRUCT(FDistanceFieldObjectBufferParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, SceneObjectBounds)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, SceneObjectData)
	SHADER_PARAMETER(uint32, NumSceneObjects)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, SceneHeightfieldObjectBounds)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, SceneHeightfieldObjectData)
	SHADER_PARAMETER(uint32, NumSceneHeightfieldObjects)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FDistanceFieldAtlasParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, SceneDistanceFieldAssetData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, DistanceFieldIndirectionTable)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, DistanceFieldIndirection2Table)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, DistanceFieldIndirectionAtlas)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, DistanceFieldBrickTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, DistanceFieldSampler)
	SHADER_PARAMETER(FVector3f, DistanceFieldBrickSize)
	SHADER_PARAMETER(FVector3f, DistanceFieldUniqueDataBrickSize)
	SHADER_PARAMETER(FIntVector, DistanceFieldBrickAtlasSizeInBricks)
	SHADER_PARAMETER(FIntVector, DistanceFieldBrickAtlasMask)
	SHADER_PARAMETER(FIntVector, DistanceFieldBrickAtlasSizeLog2)
	SHADER_PARAMETER(FVector3f, DistanceFieldBrickAtlasTexelSize)
	SHADER_PARAMETER(FVector3f, DistanceFieldBrickAtlasHalfTexelSize)
	SHADER_PARAMETER(FVector3f, DistanceFieldBrickOffsetToAtlasUVScale)
	SHADER_PARAMETER(FVector3f, DistanceFieldUniqueDataBrickSizeInAtlasTexels)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FHeightFieldAtlasParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HeightFieldTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HFVisibilityTexture)
END_SHADER_PARAMETER_STRUCT()

namespace DistanceField
{
	constexpr uint32 MinPrimitiveModifiedBoundsAllocation = 1024;

	FDistanceFieldObjectBufferParameters SetupObjectBufferParameters(FRDGBuilder& GraphBuilder, const FDistanceFieldSceneData& DistanceFieldSceneData);
	FDistanceFieldAtlasParameters SetupAtlasParameters(FRDGBuilder& GraphBuilder, const FDistanceFieldSceneData& DistanceFieldSceneData);
};

BEGIN_SHADER_PARAMETER_STRUCT(FDistanceFieldCulledObjectBufferParameters, )
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWObjectIndirectArguments)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWCulledObjectIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ObjectIndirectArguments)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, CulledObjectIndices)
END_SHADER_PARAMETER_STRUCT()

extern void AllocateDistanceFieldCulledObjectBuffers(
	FRDGBuilder& GraphBuilder,
	uint32 MaxObjects,
	FRDGBufferRef& OutObjectIndirectArguments,
	FDistanceFieldCulledObjectBufferParameters& OutParameters);

BEGIN_SHADER_PARAMETER_STRUCT(FLightTileIntersectionParameters, )
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWShadowTileNumCulledObjects)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWShadowTileStartOffsets)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWNextStartOffset)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWShadowTileArrayData)

	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ShadowTileNumCulledObjects)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ShadowTileStartOffsets)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, NextStartOffset)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ShadowTileArrayData)

	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HeightfieldShadowTileNumCulledObjects)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HeightfieldShadowTileStartOffsets)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HeightfieldShadowTileArrayData)

	SHADER_PARAMETER(FIntPoint, ShadowTileListGroupSize)
	SHADER_PARAMETER(uint32, ShadowMaxObjectsPerTile)
END_SHADER_PARAMETER_STRUCT()

extern void CullDistanceFieldObjectsForLight(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightSceneProxy* LightSceneProxy, 
	EDistanceFieldPrimitiveType PrimitiveType,
	const FMatrix& WorldToShadowValue, 
	int32 NumPlanes, 
	const FPlane* PlaneData,
	const FVector& PrePlaneTranslation,
	const FVector4f& ShadowBoundingSphere,
	float ShadowBoundingRadius,
	bool bCullingForDirectShadowing,
	bool bCullHeighfieldsNotInAtlas,
	const FDistanceFieldObjectBufferParameters& ObjectBufferParameters,
	FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters,
	FLightTileIntersectionParameters& LightTileIntersectionParameters);

extern bool ShouldPrepareForDistanceFieldAO(const FScene* Scene, const FSceneViewFamily& ViewFamily, bool bAnyViewHasGIMethodSupportingDFAO);
extern bool ShouldRenderDistanceFieldAO(TConstArrayView<FViewInfo> Views, const FEngineShowFlags& EngineShowFlags, bool bAnyViewHasGIMethodSupportingDFAO);

extern bool SupportsDistanceFieldAO(ERHIFeatureLevel::Type FeatureLevel, EShaderPlatform ShaderPlatform);

extern bool ShouldRenderDistanceFieldLighting(const FDistanceFieldSceneData& SceneData, TConstArrayView<FViewInfo> Views);