// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldObjectCulling.cpp
=============================================================================*/

#include "DistanceFieldAmbientOcclusion.h"
#include "DeferredShadingRenderer.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "DistanceFieldLightingShared.h"
#include "ScreenRendering.h"
#include "DistanceFieldLightingPost.h"
#include "LightRendering.h"
#include "OneColorShader.h"
#include "GlobalDistanceField.h"
#include "FXSystem.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "ScenePrivate.h"
#include "ShaderCompilerCore.h"

int32 GAOScatterTileCulling = 1;
FAutoConsoleVariableRef CVarAOScatterTileCulling(
	TEXT("r.AOScatterTileCulling"),
	GAOScatterTileCulling,
	TEXT("Whether to use the rasterizer for binning occluder objects into screenspace tiles."),
	ECVF_RenderThreadSafe
	);

int32 GAverageDistanceFieldObjectsPerCullTile = 512;
FAutoConsoleVariableRef CVarMaxDistanceFieldObjectsPerCullTile(
	TEXT("r.AOAverageObjectsPerCullTile"),
	GAverageDistanceFieldObjectsPerCullTile,
	TEXT("Determines how much memory should be allocated in distance field object culling data structures.  Too much = memory waste, too little = flickering due to buffer overflow."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
	);

class FCullObjectsForViewCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCullObjectsForViewCS);
	SHADER_USE_PARAMETER_STRUCT(FCullObjectsForViewCS, FGlobalShader);
public:

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, CulledObjectBufferParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, ObjectBufferParameters)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER(uint32, NumConvexHullPlanes)
		SHADER_PARAMETER_ARRAY(FVector4f, ViewFrustumConvexHull, [6])
		SHADER_PARAMETER(uint32, ObjectBoundingGeometryIndexCount)
		SHADER_PARAMETER(float, AOObjectMaxDistance)
		SHADER_PARAMETER(float, AOMaxViewDistance)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		OutEnvironment.SetDefine(TEXT("UPDATEOBJECTS_THREADGROUP_SIZE"), UpdateObjectsGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCullObjectsForViewCS, "/Engine/Private/DistanceFieldObjectCulling.usf", "CullObjectsForViewCS", SF_Compute);

void CullObjectsToView(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View, const FDistanceFieldAOParameters& Parameters, FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters)
{
	AddClearUAVPass(GraphBuilder, CulledObjectBufferParameters.RWObjectIndirectArguments, 0);

	{
		const int32 NumObjectsInBuffer = Scene.DistanceFieldSceneData.NumObjectsInBuffer;

		auto* PassParameters = GraphBuilder.AllocParameters<FCullObjectsForViewCS::FParameters>();

		PassParameters->CulledObjectBufferParameters = CulledObjectBufferParameters;
		PassParameters->ObjectBufferParameters = DistanceField::SetupObjectBufferParameters(GraphBuilder, Scene.DistanceFieldSceneData);

		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->NumConvexHullPlanes = View.ViewFrustum.Planes.Num();

		for (int32 i = 0; i < View.ViewFrustum.Planes.Num(); i++)
		{
			const FPlane4f Plane(View.ViewFrustum.Planes[i].TranslateBy(View.ViewMatrices.GetPreViewTranslation()));
			PassParameters->ViewFrustumConvexHull[i] = FVector4f(FVector3f(Plane), Plane.W);
		}

		PassParameters->ObjectBoundingGeometryIndexCount = StencilingGeometry::GLowPolyStencilSphereIndexBuffer.GetIndexCount();
		PassParameters->AOObjectMaxDistance = Parameters.ObjectMaxOcclusionDistance;
		PassParameters->AOMaxViewDistance = GetMaxAOViewDistance();

		auto ComputeShader = View.ShaderMap->GetShader<FCullObjectsForViewCS>();
		const FIntVector GroupCount = FComputeShaderUtils::GetGroupCountWrapped(NumObjectsInBuffer, UpdateObjectsGroupSize);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ObjectFrustumCulling"),
			ComputeShader,
			PassParameters,
			GroupCount);
	}
}

/**  */
class FBuildTileConesCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FBuildTileConesCS);
	SHADER_USE_PARAMETER_STRUCT(FBuildTileConesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVector4f>, RWTileConeAxisAndCos)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVector4f>, RWTileConeDepthRanges)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistanceFieldNormalTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistanceFieldNormalSampler)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER(FUintVector4, ViewDimensions)
		SHADER_PARAMETER(FVector2f, NumGroups)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GDistanceFieldAOTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GDistanceFieldAOTileSizeY);
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), GAODownsampleFactor);
		
		// To reduce shader compile time of compute shaders with shared memory, doesn't have an impact on generated code with current compiler (June 2010 DX SDK)
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
	}
};

IMPLEMENT_GLOBAL_SHADER(FBuildTileConesCS, "/Engine/Private/DistanceFieldObjectCulling.usf", "BuildTileConesMain", SF_Compute);

/**  */
class FObjectCullVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FObjectCullVS);
	SHADER_USE_PARAMETER_STRUCT(FObjectCullVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, DistanceFieldCulledObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER(float, ConservativeRadiusScale)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FObjectCullVS, "/Engine/Private/DistanceFieldObjectCulling.usf", "ObjectCullVS", SF_Vertex);

class FObjectCullPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FObjectCullPS);
	SHADER_USE_PARAMETER_STRUCT(FObjectCullPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FTileIntersectionParameters, TileIntersectionParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, DistanceFieldCulledObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER(FVector2f, NumGroups)
	END_SHADER_PARAMETER_STRUCT()

	class FCountingPass : SHADER_PERMUTATION_BOOL("SCATTER_CULLING_COUNT_PASS");
	using FPermutationDomain = TShaderPermutationDomain<FCountingPass>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		TileIntersectionModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), GAODownsampleFactor);
	}
};

IMPLEMENT_GLOBAL_SHADER(FObjectCullPS, "/Engine/Private/DistanceFieldObjectCulling.usf", "ObjectCullPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FObjectCullParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FObjectCullVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FObjectCullPS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	RDG_BUFFER_ACCESS(ObjectIndirectArguments, ERHIAccess::IndirectArgs)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

const uint32 ComputeStartOffsetGroupSize = 64;

/**  */
class FComputeCulledTilesStartOffsetCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComputeCulledTilesStartOffsetCS);
	SHADER_USE_PARAMETER_STRUCT(FComputeCulledTilesStartOffsetCS, FGlobalShader);

public:

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FTileIntersectionParameters, TileIntersectionParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, DistanceFieldCulledObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		TileIntersectionModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("COMPUTE_START_OFFSET_GROUP_SIZE"), ComputeStartOffsetGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FComputeCulledTilesStartOffsetCS, "/Engine/Private/DistanceFieldObjectCulling.usf", "ComputeCulledTilesStartOffsetCS", SF_Compute);

void ScatterTilesToObjects(
	FRDGBuilder& GraphBuilder,
	bool bCountingPass,
	const FViewInfo& View,
	const FDistanceFieldSceneData& DistanceFieldSceneData,
	FIntPoint TileListGroupSize,
	const FDistanceFieldAOParameters& Parameters,
	FRDGBufferRef ObjectIndirectArguments,
	const FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters,
	const FTileIntersectionParameters& TileIntersectionParameters,
	const FSceneTextures& SceneTextures)
{
	FDistanceFieldObjectBufferParameters DistanceFieldObjectBuffers = DistanceField::SetupObjectBufferParameters(GraphBuilder, DistanceFieldSceneData);

	FObjectCullPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FObjectCullPS::FCountingPass>(bCountingPass);

	auto VertexShader = View.ShaderMap->GetShader<FObjectCullVS>();
	auto PixelShader = View.ShaderMap->GetShader<FObjectCullPS>(PermutationVector);

	auto* PassParameters = GraphBuilder.AllocParameters<FObjectCullParameters>();
	PassParameters->VS.View = GetShaderBinding(View.ViewUniformBuffer);
	PassParameters->VS.DistanceFieldObjectBuffers = DistanceFieldObjectBuffers;
	PassParameters->VS.DistanceFieldCulledObjectBuffers = CulledObjectBufferParameters;
	PassParameters->VS.DistanceFieldAtlas = DistanceField::SetupAtlasParameters(GraphBuilder, DistanceFieldSceneData);
	PassParameters->VS.AOParameters = DistanceField::SetupAOShaderParameters(Parameters);

	{
		const int32 NumRings = StencilingGeometry::GLowPolyStencilSphereVertexBuffer.GetNumRings();
		const float RadiansPerRingSegment = PI / (float)NumRings;

		// Boost the effective radius so that the edges of the sphere approximation lie on the sphere, instead of the vertices
		PassParameters->VS.ConservativeRadiusScale = 1.0f / FMath::Cos(RadiansPerRingSegment);
	}

	PassParameters->PS.View = GetShaderBinding(View.ViewUniformBuffer);
	PassParameters->PS.TileIntersectionParameters = TileIntersectionParameters;
	PassParameters->PS.DistanceFieldObjectBuffers = DistanceFieldObjectBuffers;
	PassParameters->PS.DistanceFieldCulledObjectBuffers = CulledObjectBufferParameters;
	PassParameters->PS.DistanceFieldAtlas = DistanceField::SetupAtlasParameters(GraphBuilder, DistanceFieldSceneData);
	PassParameters->PS.AOParameters = DistanceField::SetupAOShaderParameters(Parameters);
	PassParameters->PS.NumGroups = FVector2f(TileListGroupSize.X, TileListGroupSize.Y);

	PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);
	PassParameters->ObjectIndirectArguments = ObjectIndirectArguments;

	if (GRHIRequiresRenderTargetForPixelShaderUAVs)
	{
		FRDGTextureDesc DummyDesc = FRDGTextureDesc::Create2D(TileListGroupSize, PF_B8G8R8A8, FClearValueBinding::Black, TexCreate_RenderTargetable);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(GraphBuilder.CreateTexture(DummyDesc, TEXT("Dummy")), ERenderTargetLoadAction::ENoAction);
	}

	ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

	GraphBuilder.AddPass(
		bCountingPass ? RDG_EVENT_NAME("CountTileObjectIntersections") : RDG_EVENT_NAME("CullTilesToObjects"),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, VertexShader, PixelShader, &View, TileListGroupSize, ObjectIndirectArguments](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			RHICmdList.SetViewport(0, 0, 0.0f, TileListGroupSize.X, TileListGroupSize.Y, 1.0f);

			// Render backfaces since camera may intersect
			GraphicsPSOInit.RasterizerState = View.bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

			RHICmdList.SetStreamSource(0, StencilingGeometry::GLowPolyStencilSphereVertexBuffer.VertexBufferRHI, 0);

			RHICmdList.DrawIndexedPrimitiveIndirect(
				StencilingGeometry::GLowPolyStencilSphereIndexBuffer.IndexBufferRHI,
				ObjectIndirectArguments->GetIndirectRHICallBuffer(),
				0);
		});
}

FIntPoint GetTileListGroupSizeForView(const FViewInfo& View)
{
	const FIntPoint AOViewSize = View.ViewRect.Size() / GAODownsampleFactor;
	return FIntPoint(
		FMath::DivideAndRoundUp(FMath::Max(AOViewSize.X, 1), GDistanceFieldAOTileSizeX),
		FMath::DivideAndRoundUp(FMath::Max(AOViewSize.Y, 1), GDistanceFieldAOTileSizeY));
}

void BuildTileObjectLists(
	FRDGBuilder& GraphBuilder,
	const FScene& Scene,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FRDGBufferRef ObjectIndirectArguments,
	const FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters,
	FTileIntersectionParameters TileIntersectionParameters,
	FRDGTextureRef DistanceFieldNormal,
	const FDistanceFieldAOParameters& Parameters)
{
	ensure(GAOScatterTileCulling);

	RDG_EVENT_SCOPE(GraphBuilder, "BuildTileList");
	RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

	const FIntPoint TileListGroupSize = GetTileListGroupSizeForView(View);

	{
		auto* PassParameters = GraphBuilder.AllocParameters<FBuildTileConesCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->RWTileConeAxisAndCos = TileIntersectionParameters.RWTileConeAxisAndCos;
		PassParameters->RWTileConeDepthRanges = TileIntersectionParameters.RWTileConeDepthRanges;
		PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);
		PassParameters->DistanceFieldNormalTexture = DistanceFieldNormal;
		PassParameters->DistanceFieldNormalSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->AOParameters = DistanceField::SetupAOShaderParameters(Parameters);
		PassParameters->NumGroups = FVector2f(TileListGroupSize.X, TileListGroupSize.Y);

		auto ComputeShader = View.ShaderMap->GetShader<FBuildTileConesCS>();

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("BuildTileCones"), ComputeShader, PassParameters, FIntVector(TileListGroupSize.X, TileListGroupSize.Y, 1));
	}

	// Start at 0 tiles per object
	AddClearUAVPass(GraphBuilder, TileIntersectionParameters.RWNumCulledTilesArray, 0);

	// Start at 0 threadgroups
	AddClearUAVPass(GraphBuilder, TileIntersectionParameters.RWObjectTilesIndirectArguments, 0);

	// Rasterize object bounding shapes and intersect with screen tiles to compute how many tiles intersect each object
	ScatterTilesToObjects(GraphBuilder, true, View, Scene.DistanceFieldSceneData, TileListGroupSize, Parameters, ObjectIndirectArguments, CulledObjectBufferParameters, TileIntersectionParameters, SceneTextures);

	{
		auto* PassParameters = GraphBuilder.AllocParameters<FComputeCulledTilesStartOffsetCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->TileIntersectionParameters = TileIntersectionParameters;
		PassParameters->DistanceFieldCulledObjectBuffers = CulledObjectBufferParameters;
		PassParameters->DistanceFieldAtlas = DistanceField::SetupAtlasParameters(GraphBuilder, Scene.DistanceFieldSceneData);
		PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);

		auto ComputeShader = View.ShaderMap->GetShader<FComputeCulledTilesStartOffsetCS>();
		const FIntVector GroupCount = FComputeShaderUtils::GetGroupCountWrapped(Scene.DistanceFieldSceneData.NumObjectsInBuffer, ComputeStartOffsetGroupSize);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ComputeStartOffsets"),
			ComputeShader,
			PassParameters,
			GroupCount);
	}

	// Start at 0 tiles per object
	AddClearUAVPass(GraphBuilder, TileIntersectionParameters.RWNumCulledTilesArray, 0);

	// Rasterize object bounding shapes and intersect with screen tiles, and write out intersecting tile indices for the cone tracing pass
	ScatterTilesToObjects(GraphBuilder, false, View, Scene.DistanceFieldSceneData, TileListGroupSize, Parameters, ObjectIndirectArguments, CulledObjectBufferParameters, TileIntersectionParameters, SceneTextures);
}
