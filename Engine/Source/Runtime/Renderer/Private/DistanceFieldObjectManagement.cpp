// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "Async/ParallelFor.h"
#include "RHI.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "SceneUtils.h"
#include "GlobalShader.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "DistanceFieldAtlas.h"
#include "DistanceFieldLightingShared.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "ComponentRecreateRenderStateContext.h"
#include "GlobalDistanceField.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "UnrealEngine.h"
#include "InstanceDataSceneProxy.h"
#include "Lumen/Lumen.h"
#include "Templates/Greater.h"

DECLARE_GPU_STAT(DistanceFields);

extern int32 GDistanceFieldOffsetDataStructure;

float GMeshDistanceFieldsMaxObjectBoundingRadius = 100000;
FAutoConsoleVariableRef CVarMeshDistanceFieldsMaxObjectBoundingRadius(
	TEXT("r.DistanceFields.MaxObjectBoundingRadius"),
	GMeshDistanceFieldsMaxObjectBoundingRadius,
	TEXT("Objects larger than this will not be included in the Mesh Distance Field scene, to improve performance."),
	ECVF_RenderThreadSafe
	);

int32 GDFParallelUpdate = 0;
FAutoConsoleVariableRef CVarDFParallelUpdate(
	TEXT("r.DistanceFields.ParallelUpdate"),
	GDFParallelUpdate,
	TEXT(""),
	ECVF_RenderThreadSafe
);

int32 GDFReverseAtlasAllocationOrder = 0;
FAutoConsoleVariableRef CVarDFReverseAtlasAllocationOrder(
	TEXT("r.DistanceFields.ReverseAtlasAllocationOrder"),
	GDFReverseAtlasAllocationOrder,
	TEXT(""),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
);

float GMeshSDFSurfaceBiasExpand = .25f;
FAutoConsoleVariableRef CVarMeshSDFSurfaceBiasExpand(
	TEXT("r.DistanceFields.SurfaceBiasExpand"),
	GMeshSDFSurfaceBiasExpand,
	TEXT("Fraction of a Mesh SDF voxel to expand the surface during intersection.  Expanding the surface improves representation quality, at the cost of over-occlusion."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
);

int32 GDFPreviousReverseAtlasAllocationOrder = 0;

#if ENABLE_LOW_LEVEL_MEM_TRACKER
DECLARE_LLM_MEMORY_STAT(TEXT("DistanceFields"), STAT_DistanceFieldsLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("DistanceFields"), STAT_DistanceFieldsSummaryLLM, STATGROUP_LLM);
LLM_DEFINE_TAG(DistanceFields, NAME_None, NAME_None, GET_STATFNAME(STAT_DistanceFieldsLLM), GET_STATFNAME(STAT_DistanceFieldsSummaryLLM));
#endif // ENABLE_LOW_LEVEL_MEM_TRACKER

FDistanceFieldObjectBufferParameters DistanceField::SetupObjectBufferParameters(FRDGBuilder& GraphBuilder, const FDistanceFieldSceneData& DistanceFieldSceneData)
{
	FDistanceFieldObjectBufferParameters ObjectBufferParameters;

	ObjectBufferParameters.NumSceneObjects = DistanceFieldSceneData.NumObjectsInBuffer;
	ObjectBufferParameters.NumSceneHeightfieldObjects = DistanceFieldSceneData.HeightFieldObjectBuffers ? DistanceFieldSceneData.HeightfieldPrimitives.Num() : 0;

	if (ObjectBufferParameters.NumSceneObjects > 0)
	{
		check(DistanceFieldSceneData.GetCurrentObjectBuffers());
		ObjectBufferParameters.SceneObjectBounds = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(DistanceFieldSceneData.GetCurrentObjectBuffers()->Bounds));
		ObjectBufferParameters.SceneObjectData = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(DistanceFieldSceneData.GetCurrentObjectBuffers()->Data));
	}
	else
	{
		ObjectBufferParameters.SceneObjectBounds = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f)));
		ObjectBufferParameters.SceneObjectData = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f)));
	}

	if (ObjectBufferParameters.NumSceneHeightfieldObjects > 0)
	{
		check(DistanceFieldSceneData.GetHeightFieldObjectBuffers());
		ObjectBufferParameters.SceneHeightfieldObjectBounds = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(DistanceFieldSceneData.GetHeightFieldObjectBuffers()->Bounds));
		ObjectBufferParameters.SceneHeightfieldObjectData = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(DistanceFieldSceneData.GetHeightFieldObjectBuffers()->Data));
	}
	else
	{
		ObjectBufferParameters.SceneHeightfieldObjectBounds = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f)));
		ObjectBufferParameters.SceneHeightfieldObjectData = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f)));
	}

	return ObjectBufferParameters;
}

FDistanceFieldAtlasParameters DistanceField::SetupAtlasParameters(FRDGBuilder& GraphBuilder, const FDistanceFieldSceneData& DistanceFieldSceneData)
{
	FDistanceFieldAtlasParameters SceneParameters;

	if (DistanceFieldSceneData.DistanceFieldBrickVolumeTexture)
	{
		SceneParameters.SceneDistanceFieldAssetData = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(DistanceFieldSceneData.AssetDataBuffer));

		if (GDistanceFieldOffsetDataStructure == 0 || GDistanceFieldOffsetDataStructure == 1)
		{
			if (DistanceFieldSceneData.IndirectionTable)
			{
				FRDGBufferSRV* IndirectTableSRV = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(DistanceFieldSceneData.IndirectionTable));

				if (GDistanceFieldOffsetDataStructure == 0)
				{
					SceneParameters.DistanceFieldIndirectionTable = IndirectTableSRV;
				}
				else
				{
					SceneParameters.DistanceFieldIndirection2Table = IndirectTableSRV;
				}
			}
		}
		else if (DistanceFieldSceneData.IndirectionAtlas)
		{
			SceneParameters.DistanceFieldIndirectionAtlas = GraphBuilder.RegisterExternalTexture(DistanceFieldSceneData.IndirectionAtlas);
		}

		SceneParameters.DistanceFieldBrickTexture = GraphBuilder.RegisterExternalTexture(DistanceFieldSceneData.DistanceFieldBrickVolumeTexture);
		SceneParameters.DistanceFieldSampler = TStaticSamplerState<SF_Bilinear,AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

		SceneParameters.DistanceFieldBrickSize = FVector3f(DistanceField::BrickSize);
		SceneParameters.DistanceFieldUniqueDataBrickSize = FVector3f(DistanceField::UniqueDataBrickSize);
		SceneParameters.DistanceFieldBrickAtlasSizeInBricks = DistanceFieldSceneData.BrickTextureDimensionsInBricks;
		SceneParameters.DistanceFieldBrickAtlasMask = DistanceFieldSceneData.BrickTextureDimensionsInBricks - FIntVector(1);
		SceneParameters.DistanceFieldBrickAtlasSizeLog2 = FIntVector(
			FMath::FloorLog2(DistanceFieldSceneData.BrickTextureDimensionsInBricks.X),
			FMath::FloorLog2(DistanceFieldSceneData.BrickTextureDimensionsInBricks.Y),
			FMath::FloorLog2(DistanceFieldSceneData.BrickTextureDimensionsInBricks.Z));
		SceneParameters.DistanceFieldBrickAtlasTexelSize = FVector3f(1.0f) / FVector3f(DistanceFieldSceneData.BrickTextureDimensionsInBricks * DistanceField::BrickSize);

		SceneParameters.DistanceFieldBrickAtlasHalfTexelSize = 0.5f * SceneParameters.DistanceFieldBrickAtlasTexelSize;
		SceneParameters.DistanceFieldUniqueDataBrickSizeInAtlasTexels = SceneParameters.DistanceFieldUniqueDataBrickSize * SceneParameters.DistanceFieldBrickAtlasTexelSize;

		SceneParameters.DistanceFieldBrickOffsetToAtlasUVScale = GDistanceFieldOffsetDataStructure == 0 ?
			SceneParameters.DistanceFieldBrickSize * SceneParameters.DistanceFieldBrickAtlasTexelSize :
			SceneParameters.DistanceFieldBrickSize * SceneParameters.DistanceFieldBrickAtlasTexelSize * (DistanceField::MaxIndirectionDimension - 1);
	}
	else
	{
		SceneParameters.SceneDistanceFieldAssetData = nullptr;
		SceneParameters.DistanceFieldIndirectionTable = nullptr;
		SceneParameters.DistanceFieldIndirection2Table = nullptr;
		SceneParameters.DistanceFieldIndirectionAtlas = nullptr;
		SceneParameters.DistanceFieldBrickTexture = nullptr;
		SceneParameters.DistanceFieldSampler = nullptr;
	}

	return SceneParameters;
}

const uint32 UpdateObjectsGroupSize = 64;

struct FParallelUpdateRangeDFO
{
	int32 ItemStart;
	int32 ItemCount;
};

struct FParallelUpdateRangesDFO
{
	FParallelUpdateRangeDFO Range[4];
};

// TODO: Improve and move to shared utility location.
static int32 PartitionUpdateRangesDFO(FParallelUpdateRangesDFO& Ranges, int32 ItemCount, bool bAllowParallel)
{
	if (ItemCount < 256 || !bAllowParallel)
	{
		Ranges.Range[0].ItemStart = 0;
		Ranges.Range[0].ItemCount = ItemCount;
		return 1;
	}

	const int32 RangeCount = Align(ItemCount, 4) >> 2;

	Ranges.Range[0].ItemCount = RangeCount;
	Ranges.Range[1].ItemCount = RangeCount;
	Ranges.Range[2].ItemCount = RangeCount;

	Ranges.Range[0].ItemStart = 0;
	Ranges.Range[1].ItemStart = RangeCount;
	Ranges.Range[2].ItemStart = RangeCount * 2;
	Ranges.Range[3].ItemStart = RangeCount * 3;
	Ranges.Range[3].ItemCount = ItemCount - Ranges.Range[3].ItemStart;

	return Ranges.Range[3].ItemCount > 0 ? 4 : 3;
}

void AddModifiedBounds(FDistanceFieldSceneData& DistanceFieldSceneData, FGlobalDFCacheType CacheType, const FBox& Bounds)
{
	DistanceFieldSceneData.PrimitiveModifiedBounds[CacheType].Add(Bounds);
}

static void RemoveDistanceFieldInstance(int32 RemoveIndex, FDistanceFieldSceneData& DistanceFieldSceneData)
{
	--DistanceFieldSceneData.NumObjectsInBuffer;

	if (RemoveIndex < DistanceFieldSceneData.NumObjectsInBuffer)
	{
		const int32 MoveFromIndex = DistanceFieldSceneData.NumObjectsInBuffer;

		FPrimitiveAndInstance& PrimitiveAndInstanceBeingMoved = DistanceFieldSceneData.PrimitiveInstanceMapping[MoveFromIndex];

		// Fixup indices of the primitive that is being moved
		check(PrimitiveAndInstanceBeingMoved.Primitive && PrimitiveAndInstanceBeingMoved.Primitive->DistanceFieldInstanceIndices.Num() > 0);
		PrimitiveAndInstanceBeingMoved.Primitive->DistanceFieldInstanceIndices[PrimitiveAndInstanceBeingMoved.InstanceIndex] = RemoveIndex;
	}

	DistanceFieldSceneData.PrimitiveInstanceMapping.RemoveAtSwap(RemoveIndex, EAllowShrinking::No);

	if(!DistanceFieldSceneData.IndicesToUpdateInObjectBuffersSet.Contains(RemoveIndex))
	{
		DistanceFieldSceneData.IndicesToUpdateInObjectBuffers.Add(RemoveIndex);
		DistanceFieldSceneData.IndicesToUpdateInObjectBuffersSet.Add(RemoveIndex);
	}
}

void ProcessDistanceFieldObjectRemoves(FDistanceFieldSceneData& DistanceFieldSceneData, TArray<FSetElementId>& DistanceFieldAssetRemoves)
{
	if (DistanceFieldSceneData.PendingRemoveOperations.Num() > 0)
	{
		TArray<int32, SceneRenderingAllocator> PendingRemoveOperations;

		for (int32 RemoveIndex = 0; RemoveIndex < DistanceFieldSceneData.PendingRemoveOperations.Num(); RemoveIndex++)
		{
			const FPrimitiveRemoveInfo& PrimitiveRemoveInfo = DistanceFieldSceneData.PendingRemoveOperations[RemoveIndex];
			
			FSetElementId AssetSetId = DistanceFieldSceneData.AssetStateArray.FindId(PrimitiveRemoveInfo.DistanceFieldData);
			FDistanceFieldAssetState& AssetState = DistanceFieldSceneData.AssetStateArray[AssetSetId];
			AssetState.RefCount--;
			
			if (AssetState.RefCount == 0)
			{
				DistanceFieldAssetRemoves.Add(AssetSetId);
			}

			// Can't dereference the primitive here, it has already been deleted
			const FPrimitiveSceneInfo* Primitive = PrimitiveRemoveInfo.Primitive;
			const TArray<int32, TInlineAllocator<1>>& DistanceFieldInstanceIndices = PrimitiveRemoveInfo.DistanceFieldInstanceIndices;

			for (int32 RemoveInstanceIndex = 0; RemoveInstanceIndex < DistanceFieldInstanceIndices.Num(); RemoveInstanceIndex++)
			{
				const int32 InstanceIndex = DistanceFieldInstanceIndices[RemoveInstanceIndex];

				// InstanceIndex will be -1 with zero scale meshes
				if (InstanceIndex >= 0)
				{
					// Mark region covered by instance in global distance field as modified
					FGlobalDFCacheType CacheType = PrimitiveRemoveInfo.bOftenMoving ? GDF_Full : GDF_MostlyStatic;
					AddModifiedBounds(DistanceFieldSceneData, CacheType, DistanceFieldSceneData.PrimitiveInstanceMapping[InstanceIndex].GetWorldBounds());

					// Add individual instances to temporary array for processing in the next pass
					PendingRemoveOperations.Add(InstanceIndex);
				}
			}
		}

		DistanceFieldSceneData.PendingRemoveOperations.Reset();

		if (PendingRemoveOperations.Num() > 0)
		{
			check(DistanceFieldSceneData.NumObjectsInBuffer >= PendingRemoveOperations.Num());

			// Sort from largest to smallest so we can safely RemoveAtSwap without invalidating indices in this array
			PendingRemoveOperations.Sort(TGreater<int32>());

			for (int32 RemoveIndex : PendingRemoveOperations)
			{
				RemoveDistanceFieldInstance(RemoveIndex, DistanceFieldSceneData);
			}

			PendingRemoveOperations.Reset();
		}
	}
}

void LogDistanceFieldUpdate(FPrimitiveSceneInfo const* PrimitiveSceneInfo, float BoundingRadius, bool bIsAddOperation)
{
	extern int32 GGlobalDistanceFieldDebugLogModifiedPrimitives;

	if (GGlobalDistanceFieldDebugLogModifiedPrimitives == 1
		|| (GGlobalDistanceFieldDebugLogModifiedPrimitives == 2 && !PrimitiveSceneInfo->Proxy->IsOftenMoving()))
	{
		UE_LOGF(LogDistanceField, Log,
			"Global Distance Field %ls primitive %ls %ls %ls bounding radius %.1f",
			PrimitiveSceneInfo->Proxy->IsOftenMoving() ? TEXT("Movable") : TEXT("CACHED"),
			(bIsAddOperation ? TEXT("add") : TEXT("update")),
			*PrimitiveSceneInfo->Proxy->GetOwnerName().ToString(),
			*PrimitiveSceneInfo->Proxy->GetResourceName().ToString(),
			BoundingRadius);
	}
}

/** Gathers the information needed to represent a single object's distance field and appends it to the upload buffers. */
void ProcessPrimitiveUpdate(
	bool bIsAddOperation,
	FDistanceFieldSceneData& DistanceFieldSceneData,
	FPrimitiveSceneInfo* PrimitiveSceneInfo,
	TArray<FMatrix>& InstanceLocalToWorldTmpStorage,
	TArray<FDistanceFieldAssetMipId>& DistanceFieldAssetAdds,
	TArray<FSetElementId>& DistanceFieldAssetRemoves)
{
	const FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;

	InstanceLocalToWorldTmpStorage.Reset();

	const FDistanceFieldVolumeData* DistanceFieldData = nullptr;
	float SelfShadowBias;
	Proxy->GetDistanceFieldAtlasData(DistanceFieldData, SelfShadowBias);

	TConstArrayView<FMatrix> InstanceLocalToWorldTransforms;
	const FInstanceSceneDataBuffers *InstanceData = PrimitiveSceneInfo->GetInstanceSceneDataBuffers();
	if (InstanceData)
	{
		// GPU-only instance data not currently supported - instances must be available on CPU.
		check(!InstanceData->IsInstanceDataGPUOnly());

		for (int32 InstanceIndex = 0; InstanceIndex < InstanceData->GetNumInstances(); ++InstanceIndex)
		{
			InstanceLocalToWorldTmpStorage.Add(InstanceData->GetInstanceToWorld(InstanceIndex));
		}
		InstanceLocalToWorldTransforms = InstanceLocalToWorldTmpStorage;
	}
	else
	{
		InstanceLocalToWorldTransforms = MakeArrayView(&Proxy->GetLocalToWorld(), 1);
	}

	if (DistanceFieldData && DistanceFieldData->IsValid() && InstanceLocalToWorldTransforms.Num() > 0)
	{
		const float BoundingRadius = Proxy->GetBounds().SphereRadius;
		const FGlobalDFCacheType CacheType = Proxy->IsOftenMoving() ? GDF_Full : GDF_MostlyStatic;

		// Proxy bounds are only useful if single instance
		if (InstanceLocalToWorldTransforms.Num() > 1 || BoundingRadius < GMeshDistanceFieldsMaxObjectBoundingRadius)
		{
			if (bIsAddOperation)
			{
				PrimitiveSceneInfo->DistanceFieldInstanceIndices.Empty(InstanceLocalToWorldTransforms.Num());
				PrimitiveSceneInfo->DistanceFieldInstanceIndices.AddZeroed(InstanceLocalToWorldTransforms.Num());

				FSetElementId AddSetId = DistanceFieldSceneData.AssetStateArray.FindId(DistanceFieldData);

				if (AddSetId.IsValidId())
				{
					FDistanceFieldAssetState& AssetState = DistanceFieldSceneData.AssetStateArray[AddSetId];
					AssetState.RefCount++;

					if (AssetState.RefCount == 1)
					{
						DistanceFieldAssetRemoves.Remove(AddSetId);
					}
				}
				else
				{
					FDistanceFieldAssetState NewAssetState;
					NewAssetState.RefCount = 1;
					NewAssetState.BuiltData = DistanceFieldData;
					FSetElementId AssetSetId = DistanceFieldSceneData.AssetStateArray.Add(NewAssetState);
					DistanceFieldAssetAdds.Add(FDistanceFieldAssetMipId(AssetSetId));
				}
			}

			for (int32 TransformIndex = 0; TransformIndex < InstanceLocalToWorldTransforms.Num(); TransformIndex++)
			{
				const int32 bNewInstance = bIsAddOperation || (PrimitiveSceneInfo->DistanceFieldInstanceIndices[TransformIndex] == -1);

				const bool bInstanceCountOverflow = bNewInstance && (DistanceFieldSceneData.NumObjectsInBuffer + 1 > MAX_INSTANCE_ID);

				static bool bWarnOnce = true;
				if (bInstanceCountOverflow && bWarnOnce)
				{
					bWarnOnce = false;
					UE_LOGF(LogDistanceField, Warning, "Max instance count in Distance Field Scene reached. New instances might not be represented.");
				}

				const FMatrix LocalToWorld = InstanceLocalToWorldTransforms[TransformIndex];

				const FMatrix::FReal MinScale = LocalToWorld.GetMinimumAxisScale();

				// Don't include degenerate instances or when instance count limit is reached
				if (MinScale < 0.0001f || bInstanceCountOverflow)
				{
					if (!bNewInstance)
					{
						// remove existing instance
						const int32 RemoveIndex = PrimitiveSceneInfo->DistanceFieldInstanceIndices[TransformIndex];
						RemoveDistanceFieldInstance(RemoveIndex, DistanceFieldSceneData);
					}

					PrimitiveSceneInfo->DistanceFieldInstanceIndices[TransformIndex] = -1;
					continue;
				}

				uint32 UploadIndex;

				if (bNewInstance)
				{
					UploadIndex = DistanceFieldSceneData.NumObjectsInBuffer;
					++DistanceFieldSceneData.NumObjectsInBuffer;
				}
				else
				{
					UploadIndex = PrimitiveSceneInfo->DistanceFieldInstanceIndices[TransformIndex];
				}

				if (!DistanceFieldSceneData.IndicesToUpdateInObjectBuffersSet.Contains(UploadIndex))
				{
					DistanceFieldSceneData.IndicesToUpdateInObjectBuffers.Add(UploadIndex);
					DistanceFieldSceneData.IndicesToUpdateInObjectBuffersSet.Add(UploadIndex);
				}

				const FBox WorldBounds = ((FBox)DistanceFieldData->LocalSpaceMeshBounds).TransformBy(LocalToWorld);

				if (bNewInstance)
				{
					const int32 MappingIndex = DistanceFieldSceneData.PrimitiveInstanceMapping.Add(FPrimitiveAndInstance(LocalToWorld, WorldBounds, PrimitiveSceneInfo, TransformIndex));
					PrimitiveSceneInfo->DistanceFieldInstanceIndices[TransformIndex] = UploadIndex;

					AddModifiedBounds(DistanceFieldSceneData, CacheType, WorldBounds);
					LogDistanceFieldUpdate(PrimitiveSceneInfo, BoundingRadius, bIsAddOperation);
				}
				else 
				{
					const int32 InstanceIndex = PrimitiveSceneInfo->DistanceFieldInstanceIndices[TransformIndex];
					check(InstanceIndex >= 0);

					FPrimitiveAndInstance& Mapping = DistanceFieldSceneData.PrimitiveInstanceMapping[InstanceIndex];

					const FMatrix PrevLocalToWorld = Mapping.GetLocalToWorld();
					const FBox PrevWorldBounds = Mapping.GetWorldBounds();

					// Filter out global distance field updates which were too small
					if (!PrevWorldBounds.GetExtent().Equals(WorldBounds.GetExtent(), 0.01f)
						|| !PrevLocalToWorld.Equals(LocalToWorld, 0.01f))
					{
						// decide if we want to make a single global distance field update or two updates for large movement (teleport) case
						const FBox MergedBounds = PrevWorldBounds + WorldBounds;
						const FVector MergedExtentIncrease = MergedBounds.GetExtent() - PrevWorldBounds.GetExtent() - WorldBounds.GetExtent();
						if (MergedExtentIncrease.GetMax() < 100.0f)
						{
							AddModifiedBounds(DistanceFieldSceneData, CacheType, MergedBounds);
						}
						else
						{
							AddModifiedBounds(DistanceFieldSceneData, CacheType, PrevWorldBounds);
							AddModifiedBounds(DistanceFieldSceneData, CacheType, WorldBounds);
						}
						LogDistanceFieldUpdate(PrimitiveSceneInfo, BoundingRadius, bIsAddOperation);

						Mapping.SetTransformAndBounds(LocalToWorld, WorldBounds);
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogDistanceField,Verbose,"Primitive %ls %ls excluded due to huge bounding radius %f", *Proxy->GetOwnerName().ToString(), *Proxy->GetResourceName().ToString(), BoundingRadius);
		}
	}
}

bool bVerifySceneIntegrity = false;

void FDistanceFieldSceneData::UpdateDistanceFieldObjectBuffers(
	FRDGBuilder& GraphBuilder,
	FRDGExternalAccessQueue& ExternalAccessQueue,
	FScene* Scene,
	TArray<FDistanceFieldAssetMipId>& DistanceFieldAssetAdds,
	TArray<FSetElementId>& DistanceFieldAssetRemoves)
{
	// Mask should be set in FSceneRenderer::PrepareDistanceFieldScene before calling this
	check(GraphBuilder.RHICmdList.GetGPUMask() == FRHIGPUMask::All());

	const bool bExecuteInParallel = GDFParallelUpdate != 0 && FApp::ShouldUseThreadingForPerformance();

	if (HasPendingOperations() || HasPendingUploads())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateDistanceFieldObjectBuffers);
		RDG_EVENT_SCOPE(GraphBuilder, "UpdateDistanceFieldObjectBuffers");

		if (ObjectBuffers == nullptr)
		{
			ObjectBuffers = new FDistanceFieldObjectBuffers();
		}

		// Process removes before adds, as the adds will overwrite primitive allocation info
		// This also prevents re-uploading distance fields on render state recreation
		ProcessDistanceFieldObjectRemoves(*this, DistanceFieldAssetRemoves);

		if ((PendingAddOperations.Num() > 0 || PendingUpdateOperations.Num() > 0) && GDFReverseAtlasAllocationOrder == GDFPreviousReverseAtlasAllocationOrder)
		{
			TArray<FMatrix> InstanceLocalToPrimitiveTransforms;
			int32 OriginalNumObjects = NumObjectsInBuffer;
			for (FPrimitiveSceneInfo* PrimitiveSceneInfo : PendingAddOperations)
			{
				ProcessPrimitiveUpdate(
					true,
					*this,
					PrimitiveSceneInfo,
					InstanceLocalToPrimitiveTransforms,
					DistanceFieldAssetAdds,
					DistanceFieldAssetRemoves);
			}

			for (FPrimitiveSceneInfo* PrimitiveSceneInfo : PendingUpdateOperations)
			{
				ProcessPrimitiveUpdate(
					false,
					*this,
					PrimitiveSceneInfo,
					InstanceLocalToPrimitiveTransforms,
					DistanceFieldAssetAdds,
					DistanceFieldAssetRemoves);
			}

			PendingAddOperations.Reset();
			PendingUpdateOperations.Reset();
		}

		GDFPreviousReverseAtlasAllocationOrder = GDFReverseAtlasAllocationOrder;

		// Upload buffer changes
		if (HasPendingUploads())
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_UploadDistanceFieldObjectDataAndBounds);

			// Upload DF object data and bounds
			check(NumObjectsInBuffer <= MAX_INSTANCE_ID);

			const uint32 NumDFObjectsRoundedUp = FMath::RoundUpToPowerOfTwo(NumObjectsInBuffer);

			const uint32 DFObjectDataNumBytes = NumDFObjectsRoundedUp * GDistanceFieldObjectDataStride * sizeof(FVector4f);
			FRDGBuffer* DFObjectDataBuffer = ResizeStructuredBufferIfNeeded(GraphBuilder, ObjectBuffers->Data, DFObjectDataNumBytes, TEXT("DistanceFields.DFObjectData"));

			const uint32 DFObjectBoundsNumBytes = NumDFObjectsRoundedUp * GDistanceFieldObjectBoundsStride * sizeof(FVector4f);
			FRDGBuffer* DFObjectBoundsBuffer = ResizeStructuredBufferIfNeeded(GraphBuilder, ObjectBuffers->Bounds, DFObjectBoundsNumBytes, TEXT("DistanceFields.DFObjectBounds"));

			// Limit number of distance field object uploads per frame to 2M
			// The bottleneck is GetMaxUploadBufferElements() used by FRDGScatterUploadBuffer
			// This is not expected to be hit during gameplay.
			static const int32 MAX_NUM_DISTANCE_FIELD_OBJECT_UPLOADS = (2 << 20);

			check(IndicesToUpdateInObjectBuffers.Num() == IndicesToUpdateInObjectBuffersSet.Num());

			const int32 NumDFObjectUploads = FMath::Min(IndicesToUpdateInObjectBuffers.Num(), MAX_NUM_DISTANCE_FIELD_OBJECT_UPLOADS);

			static FCriticalSection DFUpdateCS;

			if (NumDFObjectUploads > 0)
			{
				UploadDistanceFieldDataBuffer.Init(GraphBuilder, NumDFObjectUploads, GDistanceFieldObjectDataStride * sizeof(FVector4f), true, TEXT("DistanceFields.DFObjectDataUploadBuffer"));
				UploadDistanceFieldBoundsBuffer.Init(GraphBuilder, NumDFObjectUploads, GDistanceFieldObjectBoundsStride * sizeof(FVector4f), true, TEXT("DistanceFields.DFObjectBoundsUploadBuffer"));

				const TScenePrimitiveArray<FPrimitiveBounds>& PrimitiveBounds = Scene->PrimitiveBounds;

				FParallelUpdateRangesDFO ParallelRanges;

				int32 RangeCount = PartitionUpdateRangesDFO(ParallelRanges, NumDFObjectUploads, bExecuteInParallel);

				ParallelFor(RangeCount,
					[this, &ParallelRanges, &PrimitiveBounds, RangeCount](int32 RangeIndex)
					{
						for (int32 ItemIndex = ParallelRanges.Range[RangeIndex].ItemStart; ItemIndex < ParallelRanges.Range[RangeIndex].ItemStart + ParallelRanges.Range[RangeIndex].ItemCount; ++ItemIndex)
						{
							const int32 Index = IndicesToUpdateInObjectBuffers[ItemIndex];
							checkf(Index >= 0, TEXT("Invalid instances should've been skipped in ProcessPrimitiveUpdate(...)"));

							if (Index < PrimitiveInstanceMapping.Num())
							{
								const FPrimitiveAndInstance& PrimAndInst = PrimitiveInstanceMapping[Index];
								const FPrimitiveSceneProxy* PrimitiveSceneProxy = PrimAndInst.Primitive->Proxy;

								if (RangeCount > 1)
								{
									DFUpdateCS.Lock();
								}

								FVector4f* UploadObjectData = (FVector4f*)UploadDistanceFieldDataBuffer.Add_GetRef(Index);
								FVector4f* UploadObjectBounds = (FVector4f*)UploadDistanceFieldBoundsBuffer.Add_GetRef(Index);

								if (RangeCount > 1)
								{
									DFUpdateCS.Unlock();
								}

								const FDistanceFieldVolumeData* DistanceFieldData = nullptr;
								float SelfShadowBias;
								PrimitiveSceneProxy->GetDistanceFieldAtlasData(DistanceFieldData, SelfShadowBias);

								const FBox3f LocalSpaceMeshBounds = DistanceFieldData->LocalSpaceMeshBounds;

								// Uniformly scale our Volume space to lie within [-1, 1] at the max extent
								// This is mirrored in the SDF encoding
								const FBox3f::FReal LocalToVolumeScale = 1.0f / LocalSpaceMeshBounds.GetExtent().GetMax();

								const FDFVector3 WorldPosition(PrimAndInst.Origin + FVector(PrimAndInst.WorldBoundsRelativeToOrigin.GetCenter()));

								FMatrix44f LocalToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrix(WorldPosition.High, PrimAndInst.GetLocalToWorld()).M;
								FMatrix44f RelativeWorldToLocal = FMatrix44f(LocalToRelativeWorld.InverseFast());

								{
									const FVector3f BoundsExtent = PrimAndInst.WorldBoundsRelativeToOrigin.GetExtent();
									const FVector4f ObjectBoundingSphere(WorldPosition.Low, BoundsExtent.Size());

									UploadObjectBounds[0] = WorldPosition.High;
									UploadObjectBounds[1] = ObjectBoundingSphere;

									const FGlobalDFCacheType CacheType = PrimitiveSceneProxy->IsOftenMoving() ? GDF_Full : GDF_MostlyStatic;
									const bool bOftenMoving = CacheType == GDF_Full;
									const bool bCastShadow = PrimitiveSceneProxy->CastsDynamicShadow();
									const bool bIsNaniteMesh = PrimitiveSceneProxy->IsNaniteMesh();
									const bool bEmissiveLightSource = PrimitiveSceneProxy->IsEmissiveLightSource();
									const bool bVisible = PrimitiveSceneProxy->IsDrawnInGame(); // Distance field object can be invisible in main view, but cast shadows
									const bool bAffectIndirectLightingWhileHidden = PrimitiveSceneProxy->AffectsIndirectLightingWhileHidden();

									uint32 Flags = 0;
									Flags |= bOftenMoving ? 1u : 0;
									Flags |= bCastShadow ? 2u : 0;
									Flags |= bIsNaniteMesh ? 4u : 0;
									Flags |= bEmissiveLightSource ? 8u : 0;
									Flags |= bVisible ? 16u : 0;
									Flags |= bAffectIndirectLightingWhileHidden ? 32u : 0;

									FVector4f ObjectWorldExtentAndFlags(BoundsExtent, 0.0f);
									ObjectWorldExtentAndFlags.W = *(const float*)&Flags;
									UploadObjectBounds[2] = ObjectWorldExtentAndFlags;
								}

								const FMatrix44f VolumeToRelativeWorld = FScaleMatrix44f(1.0f / LocalToVolumeScale) * FTranslationMatrix44f(LocalSpaceMeshBounds.GetCenter()) * LocalToRelativeWorld;
								const FMatrix44f RelativeWorldToVolume = RelativeWorldToLocal * FTranslationMatrix44f(-LocalSpaceMeshBounds.GetCenter()) * FScaleMatrix44f(LocalToVolumeScale);

								UploadObjectData[0] = WorldPosition.High;
								const FMatrix44f WorldToVolumeT = RelativeWorldToVolume.GetTransposed();
								// WorldToVolumeT
								UploadObjectData[1] = (*(FVector4f*)&WorldToVolumeT.M[0]);
								UploadObjectData[2] = (*(FVector4f*)&WorldToVolumeT.M[1]);
								UploadObjectData[3] = (*(FVector4f*)&WorldToVolumeT.M[2]);

								const FVector3f VolumePositionExtent = LocalSpaceMeshBounds.GetExtent() * LocalToVolumeScale;

								// Minimal surface bias which increases chance that ray hit will a surface located between two texels
								float ExpandSurfaceDistance = (GMeshSDFSurfaceBiasExpand * VolumePositionExtent / FVector3f(DistanceFieldData->Mips[0].IndirectionDimensions * DistanceField::UniqueDataBrickSize)).Size();

								const float WSign = DistanceFieldData->bMostlyTwoSided ? -1 : 1;
								UploadObjectData[4] = FVector4f(VolumePositionExtent, WSign * ExpandSurfaceDistance);

								const int32 PrimIdx = PrimAndInst.Primitive->GetIndex();
								const FPrimitiveBounds& PrimBounds = PrimitiveBounds[PrimIdx];
								float MinDrawDist2 = FMath::Square(PrimBounds.MinDrawDistance);
								// For IEEE compatible machines, float operations goes to inf if overflow
								// In this case, it will effectively disable max draw distance culling
								float MaxDrawDist = FMath::Max(PrimBounds.MaxCullDistance, 0.f) * GetCachedScalabilityCVars().ViewDistanceScale;

								const uint32 GPUSceneInstanceIndex = PrimitiveSceneProxy->SupportsInstanceDataBuffer() ?
									PrimAndInst.Primitive->GetInstanceSceneDataOffset() + PrimAndInst.InstanceIndex :
									PrimAndInst.Primitive->GetInstanceSceneDataOffset();

								// Bypass NaN checks in FVector4f ctor
								FVector4f Vector4;
								Vector4.X = MinDrawDist2;
								Vector4.Y = MaxDrawDist * MaxDrawDist;
								Vector4.Z = SelfShadowBias;
								Vector4.W = *(const float*)&GPUSceneInstanceIndex;
								UploadObjectData[5] = Vector4;

								const FMatrix44f VolumeToWorldT = VolumeToRelativeWorld.GetTransposed();
								UploadObjectData[6] = *(FVector4f*)&VolumeToWorldT.M[0];
								UploadObjectData[7] = *(FVector4f*)&VolumeToWorldT.M[1];
								UploadObjectData[8] = *(FVector4f*)&VolumeToWorldT.M[2];

								FVector4f FloatVector8(FVector3f(VolumeToRelativeWorld.GetScaleVector()), 0.0f);

								// Bypass NaN checks in FVector4f ctor
								FSetElementId AssetStateSetId = AssetStateArray.FindId(DistanceFieldData);
								check(AssetStateSetId.IsValidId());
								const int32 AssetStateInt = AssetStateSetId.AsInteger();
								FloatVector8.W = *(const float*)&AssetStateInt;

								UploadObjectData[9] = FloatVector8;
							}
						}
					},
					RangeCount == 1
				);

				UploadDistanceFieldDataBuffer.ResourceUploadTo(GraphBuilder, DFObjectDataBuffer);
				UploadDistanceFieldBoundsBuffer.ResourceUploadTo(GraphBuilder, DFObjectBoundsBuffer);

				ExternalAccessQueue.Add(DFObjectDataBuffer, ERHIAccess::SRVMask, ERHIPipeline::All);
				ExternalAccessQueue.Add(DFObjectBoundsBuffer, ERHIAccess::SRVMask, ERHIPipeline::All);

				IndicesToUpdateInObjectBuffersSet.Reset();

				if (IndicesToUpdateInObjectBuffers.Num() > NumDFObjectUploads)
				{
					// this is not expected to happen frequently since we can perform up to MAX_NUM_DISTANCE_FIELD_OBJECT_UPLOADS per frame
					// RemoveAtSwap would be more efficient but could potentially result in starvation
					IndicesToUpdateInObjectBuffers.RemoveAt(0, NumDFObjectUploads, EAllowShrinking::Yes); // allow array to shrink since getting into this code path means array is very large

					for (int32 Index : IndicesToUpdateInObjectBuffers)
					{
						IndicesToUpdateInObjectBuffersSet.Add(Index);
					}

					check(IndicesToUpdateInObjectBuffers.Num() == IndicesToUpdateInObjectBuffersSet.Num());
				}
				else
				{
					IndicesToUpdateInObjectBuffers.Reset();
				}
			}
		}

		check(NumObjectsInBuffer == PrimitiveInstanceMapping.Num());

		if (bVerifySceneIntegrity)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateObjectData_VerifyIntegrity);
			VerifyIntegrity();
		}
	}
}

void UpdateGlobalHeightFieldObjectBuffers(FRDGBuilder& GraphBuilder, const TArray<uint32>& IndicesToUpdateInHeightFieldObjectBuffers, FDistanceFieldSceneData& DistanceFieldSceneData)
{
	// Mask should be set in FSceneRenderer::PrepareDistanceFieldScene before calling this
	check(GraphBuilder.RHICmdList.GetGPUMask() == FRHIGPUMask::All());

	check(DistanceFieldSceneData.PendingHeightFieldAddOps.IsEmpty());
	check(DistanceFieldSceneData.PendingHeightFieldRemoveOps.IsEmpty());

	bool bUpdateAllEntries = false;

	if (!DistanceFieldSceneData.HeightFieldObjectBuffers)
	{
		DistanceFieldSceneData.HeightFieldObjectBuffers = new FDistanceFieldObjectBuffers();

		bUpdateAllEntries = true;
	}

	if (DistanceFieldSceneData.HeightFieldAtlasGeneration != GHeightFieldTextureAtlas.GetGeneration()
		|| DistanceFieldSceneData.HFVisibilityAtlasGenerattion != GHFVisibilityTextureAtlas.GetGeneration())
	{
		DistanceFieldSceneData.HeightFieldAtlasGeneration = GHeightFieldTextureAtlas.GetGeneration();
		DistanceFieldSceneData.HFVisibilityAtlasGenerattion = GHFVisibilityTextureAtlas.GetGeneration();

		bUpdateAllEntries = true;
	}
	
	const uint32 NumHeightFieldObjects = DistanceFieldSceneData.HeightfieldPrimitives.Num();
	const uint32 NumHeightFieldObjectUploads = bUpdateAllEntries ? NumHeightFieldObjects : IndicesToUpdateInHeightFieldObjectBuffers.Num();
	check(NumHeightFieldObjectUploads <= NumHeightFieldObjects);

	if (NumHeightFieldObjectUploads > 0)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateHeightFieldSceneObjectData);
		RDG_EVENT_SCOPE(GraphBuilder, "UpdateHeightFieldSceneObjectData");

		// Upload height field object data and bounds

		FDistanceFieldObjectBuffers*& ObjectBuffers = DistanceFieldSceneData.HeightFieldObjectBuffers;

		const uint32 HeighFieldObjectDataNumFloat4s = FMath::RoundUpToPowerOfTwo(NumHeightFieldObjects * GHeightFieldObjectDataStride);
		const uint32 HeighFieldObjectDataNumBytes = HeighFieldObjectDataNumFloat4s * sizeof(FVector4f);
		FRDGBuffer* HeightfieldObjectDataBuffer = ResizeStructuredBufferIfNeeded(GraphBuilder, ObjectBuffers->Data, HeighFieldObjectDataNumBytes, TEXT("HeighFieldObjectData"));

		const uint32 HeighFieldObjectBoundsNumFloat4s = FMath::RoundUpToPowerOfTwo(NumHeightFieldObjects * GHeightFieldObjectBoundsStride);
		const uint32 HeighFieldObjectBoundsNumBytes = HeighFieldObjectBoundsNumFloat4s * sizeof(FVector4f);
		FRDGBuffer* HeightfieldObjectBoundsBuffer = ResizeStructuredBufferIfNeeded(GraphBuilder, ObjectBuffers->Bounds, HeighFieldObjectBoundsNumBytes, TEXT("HeighFieldObjectBounds"));

		DistanceFieldSceneData.UploadHeightFieldDataBuffer.Init(GraphBuilder, NumHeightFieldObjectUploads, GHeightFieldObjectDataStride * sizeof(FVector4f), true, TEXT("HeighFieldObjectDataUploadBuffer"));
		DistanceFieldSceneData.UploadHeightFieldBoundsBuffer.Init(GraphBuilder, NumHeightFieldObjectUploads, GHeightFieldObjectBoundsStride * sizeof(FVector4f), true, TEXT("HeighFieldObjectBoundsUploadBuffer"));

		for (uint32 Index = 0; Index < NumHeightFieldObjectUploads; ++Index)
		{
			const uint32 PrimitiveIndex = bUpdateAllEntries ? Index : IndicesToUpdateInHeightFieldObjectBuffers[Index];
			check(PrimitiveIndex < (uint32)DistanceFieldSceneData.HeightfieldPrimitives.Num());

			FPrimitiveSceneInfo* Primitive = DistanceFieldSceneData.HeightfieldPrimitives[PrimitiveIndex];

			FVector4f* UploadObjectData = (FVector4f*)DistanceFieldSceneData.UploadHeightFieldDataBuffer.Add_GetRef(PrimitiveIndex);
			FVector4f* UploadObjectBounds = (FVector4f*)DistanceFieldSceneData.UploadHeightFieldBoundsBuffer.Add_GetRef(PrimitiveIndex);

			UTexture2D* HeightNormalTexture;
			UTexture2D* VisibilityTexture;
			FHeightfieldComponentDescription HeightFieldCompDesc(Primitive->Proxy->GetLocalToWorld(), Primitive->GetInstanceSceneDataOffset());
			Primitive->Proxy->GetHeightfieldRepresentation(HeightNormalTexture, VisibilityTexture, HeightFieldCompDesc);

			const bool bInAtlas = HeightNormalTexture && (GHeightFieldTextureAtlas.GetAllocationHandle(HeightNormalTexture) != INDEX_NONE);

			{
				const FBoxSphereBounds& Bounds = Primitive->Proxy->GetBounds();
				const FBox BoxBound = Bounds.GetBox();

				const FDFVector3 AbsoluteWorldPosition(BoxBound.GetCenter());

				const FVector4f ObjectBoundingSphere(AbsoluteWorldPosition.Low, Bounds.SphereRadius);

				uint32 Flags = 0;
				Flags |= bInAtlas ? 1u : 0;

				FVector4f BoxBoundExtentAndFlags((FVector3f)BoxBound.GetExtent(), 0.0f);
				BoxBoundExtentAndFlags.W = *(const float*)&Flags;

				UploadObjectBounds[0] = AbsoluteWorldPosition.High;
				UploadObjectBounds[1] = ObjectBoundingSphere;
				UploadObjectBounds[2] = BoxBoundExtentAndFlags;
			}

			const FMatrix& LocalToWorld = HeightFieldCompDesc.LocalToWorld;
			check(LocalToWorld.GetMaximumAxisScale() > 0.f);

			const FDFVector3 WorldPosition(LocalToWorld.GetOrigin());

			// Inverse on FMatrix44f can generate NaNs if the source matrix contains large scaling, so do it in double precision.
			FMatrix LocalToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrixDouble(FVector(WorldPosition.High), LocalToWorld);

			UploadObjectData[0] = WorldPosition.High;

			const FMatrix44f WorldToLocalT = FMatrix44f(LocalToRelativeWorld.Inverse().GetTransposed());
			UploadObjectData[1] = *(const FVector4f*)&WorldToLocalT.M[0];
			UploadObjectData[2] = *(const FVector4f*)&WorldToLocalT.M[1];
			UploadObjectData[3] = *(const FVector4f*)&WorldToLocalT.M[2];

			const FIntRect& HeightFieldRect = HeightFieldCompDesc.HeightfieldRect;
			const float WorldToLocalScale = FMath::Min3(
				WorldToLocalT.GetColumn(0).Size(),
				WorldToLocalT.GetColumn(1).Size(),
				WorldToLocalT.GetColumn(2).Size());
			UploadObjectData[4] = FVector4f(HeightFieldRect.Width(), HeightFieldRect.Height(), WorldToLocalScale, 0.f);

			FVector4f HeightUVScaleBias(ForceInitToZero);
			if (HeightNormalTexture)
			{
				const uint32 HeightNormalTextureHandle = GHeightFieldTextureAtlas.GetAllocationHandle(HeightNormalTexture);
				if (HeightNormalTextureHandle != INDEX_NONE)
				{
					const FVector4f HeightFieldScaleBias = HeightFieldCompDesc.HeightfieldScaleBias;
					check(HeightFieldScaleBias.Y >= 0.f && HeightFieldScaleBias.Z >= 0.f && HeightFieldScaleBias.W >= 0.f);

					const FVector4f ScaleBias = GHeightFieldTextureAtlas.GetAllocationScaleBias(HeightNormalTextureHandle);
					HeightUVScaleBias.Set(FMath::Abs(HeightFieldScaleBias.X) * ScaleBias.X,
						HeightFieldScaleBias.Y * ScaleBias.Y,
						HeightFieldScaleBias.Z * ScaleBias.X + ScaleBias.Z,
						HeightFieldScaleBias.W * ScaleBias.Y + ScaleBias.W);
				}
			}
			UploadObjectData[5] = HeightUVScaleBias;

			FVector4f VisUVScaleBias(ForceInitToZero);
			if (VisibilityTexture)
			{
				const uint32 VisHandle = GHFVisibilityTextureAtlas.GetAllocationHandle(VisibilityTexture);
				if (VisHandle != INDEX_NONE)
				{
					const FVector4f ScaleBias = GHFVisibilityTextureAtlas.GetAllocationScaleBias(VisHandle);
					VisUVScaleBias = FVector4f(1.f / HeightFieldRect.Width() * ScaleBias.X, 1.f / HeightFieldRect.Height() * ScaleBias.Y, ScaleBias.Z, ScaleBias.W);
				}
			}
			UploadObjectData[6] = VisUVScaleBias;
		}

		DistanceFieldSceneData.UploadHeightFieldDataBuffer.ResourceUploadTo(GraphBuilder, HeightfieldObjectDataBuffer);
		DistanceFieldSceneData.UploadHeightFieldBoundsBuffer.ResourceUploadTo(GraphBuilder, HeightfieldObjectBoundsBuffer);
	}
}

void ProcessPendingHeightFieldPrimitiveAddAndRemoveOps(FScene* Scene, TArray<uint32>& IndicesToUpdateInHeightFieldObjectBuffers)
{
	FDistanceFieldSceneData& SceneData = Scene->DistanceFieldSceneData;

	if (SceneData.HeightFieldObjectBuffers)
	{
		// When using HeightFieldObjectBuffers we need to track the indices of primitives that need to be updated
		IndicesToUpdateInHeightFieldObjectBuffers.Reserve(SceneData.PendingHeightFieldAddOps.Num() + SceneData.PendingHeightFieldRemoveOps.Num());
	}
	
	check(SceneData.PendingHeightFieldRemoveOps.Num() <= SceneData.HeightfieldPrimitives.Num());

	// First process removes
	
	// Need to gather indices that are pending removal since they need to be sorted to be able to use RemoveAtSwap
	TArray<int32, SceneRenderingAllocator> PendingRemoveIndices;
	for (int32 Idx = 0; Idx < SceneData.PendingHeightFieldRemoveOps.Num(); ++Idx)
	{
		const FHeightFieldPrimitiveRemoveInfo& RemoveInfo = SceneData.PendingHeightFieldRemoveOps[Idx];
		
		checkf(RemoveInfo.DistanceFieldInstanceIndices.Num() == 1, TEXT("Heightfield primitives should only have one distance field instance"));
		PendingRemoveIndices.Add(RemoveInfo.DistanceFieldInstanceIndices[0]);

		const FGlobalDFCacheType CacheType = RemoveInfo.bOftenMoving ? GDF_Full : GDF_MostlyStatic;
		AddModifiedBounds(SceneData, CacheType, RemoveInfo.WorldBounds);
	}

	SceneData.PendingHeightFieldRemoveOps.Reset();

	// Sort in descending order to be able to use RemoveAtSwap
	PendingRemoveIndices.Sort(TGreater<int32>());

	const int32 HeightfieldPrimitivesSizeAfterRemoves = SceneData.HeightfieldPrimitives.Num() - PendingRemoveIndices.Num();
	check(HeightfieldPrimitivesSizeAfterRemoves >= 0);

	// Actually remove entries from SceneData.HeightfieldPrimitives
	for (int32 RemoveIndex : PendingRemoveIndices)
	{
		const int32 MoveFromIndex = SceneData.HeightfieldPrimitives.Num() - 1;
		FPrimitiveSceneInfo* PrimitiveBeingMoved = SceneData.HeightfieldPrimitives[MoveFromIndex];

		if (RemoveIndex != MoveFromIndex)
		{
			// Fixup indices of the primitive that is being moved
			checkf(PrimitiveBeingMoved && PrimitiveBeingMoved->DistanceFieldInstanceIndices.Num() == 1, TEXT("Heightfield primitives should only have one distance field instance"));
			PrimitiveBeingMoved->DistanceFieldInstanceIndices[0] = RemoveIndex;

			// only add index to update entry if there's valid buffers and the index will be a valid entry after all removes are processed
			if (SceneData.HeightFieldObjectBuffers && RemoveIndex < HeightfieldPrimitivesSizeAfterRemoves)
			{
				IndicesToUpdateInHeightFieldObjectBuffers.Add(RemoveIndex);
			}
		}

		SceneData.HeightfieldPrimitives.RemoveAtSwap(RemoveIndex);
	}

	PendingRemoveIndices.Reset();

	// After processing removes, we now process adds

	for (FPrimitiveSceneInfo* Primitive : SceneData.PendingHeightFieldAddOps)
	{
		check(Primitive->DistanceFieldInstanceIndices.IsEmpty());

		const int32 AddIndex = SceneData.HeightfieldPrimitives.Add(Primitive);
		Primitive->DistanceFieldInstanceIndices.Add(AddIndex);

		if (SceneData.HeightFieldObjectBuffers)
		{
			IndicesToUpdateInHeightFieldObjectBuffers.Add(AddIndex);
		}

		const FGlobalDFCacheType CacheType = Primitive->Proxy->IsOftenMoving() ? GDF_Full : GDF_MostlyStatic;
		const FBoxSphereBounds& Bounds = Primitive->Proxy->GetBounds();
		AddModifiedBounds(SceneData, CacheType, Bounds.GetBox());
	}

	SceneData.PendingHeightFieldAddOps.Reset();
}

void PrepareDistanceFieldScene(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue, const FSceneRenderUpdateInputs& SceneUpdateInputs)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, PrepareDistanceFieldScene);
	TRACE_CPUPROFILER_EVENT_SCOPE(FSceneRenderer::PrepareDistanceFieldScene);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PrepareDistanceFieldScene);
	LLM_SCOPE_BYTAG(DistanceFields);
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());
	RDG_RHI_EVENT_SCOPE_STAT(GraphBuilder, DistanceFields, "DistanceFields");

	extern bool ShouldPrepareDistanceFieldScene(const FSceneRenderUpdateInputs & SceneUpdateInputs);
	extern bool ShouldPrepareGlobalDistanceField(const FSceneRenderer*);
	extern bool ShouldPrepareHeightFieldScene(const FSceneRenderUpdateInputs& SceneUpdateInputs);

	const bool bShouldPrepareHeightFieldScene = ShouldPrepareHeightFieldScene(SceneUpdateInputs);
	const bool bShouldPrepareDistanceFieldScene = ShouldPrepareDistanceFieldScene(SceneUpdateInputs);

	if (!bShouldPrepareDistanceFieldScene && !bShouldPrepareHeightFieldScene)
	{
		return;
	}

	FScene* Scene = SceneUpdateInputs.Scene;
	FDistanceFieldSceneData& DistanceFieldSceneData = Scene->DistanceFieldSceneData;

	TArray<uint32> IndicesToUpdateInHeightFieldObjectBuffers;

	ProcessPendingHeightFieldPrimitiveAddAndRemoveOps(Scene, IndicesToUpdateInHeightFieldObjectBuffers);

	if (bShouldPrepareHeightFieldScene)
	{
		extern int32 GHFShadowQuality;
		if (GHFShadowQuality > 2)
		{
			GHFVisibilityTextureAtlas.UpdateAllocations(GraphBuilder, SceneUpdateInputs.FeatureLevel);
		}
		GHeightFieldTextureAtlas.UpdateAllocations(GraphBuilder, SceneUpdateInputs.FeatureLevel);

		UpdateGlobalHeightFieldObjectBuffers(GraphBuilder, IndicesToUpdateInHeightFieldObjectBuffers, DistanceFieldSceneData);
	}
	else if (DistanceFieldSceneData.HeightFieldObjectBuffers)
	{
		// if we don't need HeightFieldScene release the buffers
		delete DistanceFieldSceneData.HeightFieldObjectBuffers;
		DistanceFieldSceneData.HeightFieldObjectBuffers = nullptr;
		DistanceFieldSceneData.HeightFieldAtlasGeneration = 0;
		DistanceFieldSceneData.HFVisibilityAtlasGenerattion = 0;
	}

	if (bShouldPrepareDistanceFieldScene)
	{
		TArray<FDistanceFieldAssetMipId> DistanceFieldAssetAdds;
		TArray<FSetElementId> DistanceFieldAssetRemoves;
		DistanceFieldSceneData.UpdateDistanceFieldObjectBuffers(GraphBuilder, ExternalAccessQueue, Scene, DistanceFieldAssetAdds, DistanceFieldAssetRemoves);

		DistanceFieldSceneData.UpdateDistanceFieldAtlas(GraphBuilder, ExternalAccessQueue, SceneUpdateInputs, DistanceFieldAssetAdds, DistanceFieldAssetRemoves);

		SceneUpdateInputs.ForEachView([&] (const FSceneRenderer* Renderer, FViewInfo& View)
		{
			if (ShouldPrepareGlobalDistanceField(Renderer))
			{
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

				float OcclusionMaxDistance = Scene->DefaultMaxDistanceFieldOcclusionDistance;

				// Use the skylight's max distance if there is one
				if (Scene->SkyLight && Scene->SkyLight->bCastShadows && !Scene->SkyLight->bWantsStaticShadowing)
				{
					OcclusionMaxDistance = Scene->SkyLight->OcclusionMaxDistance;
				}

				const bool bUseLumenGlobalDistanceFieldSettings = Renderer->IsLumenEnabled(View) && Lumen::IsUsingGlobalSDF(*View.Family);

				UpdateGlobalDistanceFieldVolume(GraphBuilder, ExternalAccessQueue, View, Scene, OcclusionMaxDistance, bUseLumenGlobalDistanceFieldSettings, View.GlobalDistanceFieldInfo);
			}
			return true;
		});
	}
}