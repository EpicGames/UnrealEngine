// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldScreenGridLighting.cpp
=============================================================================*/

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "ShaderParameters.h"
#include "RendererInterface.h"
#include "RHIStaticStates.h"
#include "GlobalDistanceFieldParameters.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "ScenePrivate.h"
#include "SceneUtils.h"
#include "PostProcess/SceneRenderTargets.h"
#include "GlobalShader.h"
#include "SceneRenderTargetParameters.h"
#include "DeferredShadingRenderer.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "DistanceFieldLightingShared.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "DistanceFieldLightingPost.h"
#include "GlobalDistanceField.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "VisualizeTexture.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneViewState.h"

/** Number of cone traced directions. */
static const int32 NumConeSampleDirections = 9;

static int32 GAOUseJitter = 1;
static FAutoConsoleVariableRef CVarAOUseJitter(
	TEXT("r.AOUseJitter"),
	GAOUseJitter,
	TEXT("Whether to use 4x temporal supersampling with Screen Grid DFAO.  When jitter is disabled, a shorter history can be used but there will be more spatial aliasing."),
	ECVF_RenderThreadSafe
);

static int32 GDistanceFieldAOTraverseMips = 1;
static FAutoConsoleVariableRef CVarDistanceFieldAOTraverseMips(
	TEXT("r.DistanceFieldAO.TraverseMips"),
	GDistanceFieldAOTraverseMips,
	TEXT("Whether to traverse mips while tracing AO cones against object SDFs."),
	ECVF_RenderThreadSafe
);

int32 GConeTraceDownsampleFactor = 4;

FIntPoint GetBufferSizeForConeTracing(const FViewInfo& View)
{
	const FIntPoint ConeTracingBufferSize = FIntPoint::DivideAndRoundDown(GetBufferSizeForAO(View), GConeTraceDownsampleFactor);
	return FIntPoint(FMath::Max(ConeTracingBufferSize.X, 1), FMath::Max(ConeTracingBufferSize.Y, 1));
}

FVector2f JitterOffsets[4] = 
{
	FVector2f(.25f, 0),
	FVector2f(.75f, .25f),
	FVector2f(.5f, .75f),
	FVector2f(0, .5f)
};

extern float GAOConeHalfAngle;
extern bool DistanceFieldAOUseHistory(const FViewInfo& View);

bool ShouldCompileDFScreenGridLightingShaders(EShaderPlatform ShaderPlatform)
{
	return ShouldCompileDistanceFieldAOShaders(ShaderPlatform);
}

FVector2f GetJitterOffset(const FViewInfo& View)
{
	if (GAOUseJitter && DistanceFieldAOUseHistory(View))
	{
		return JitterOffsets[View.GetDistanceFieldTemporalSampleIndex()] * GConeTraceDownsampleFactor;
	}

	return FVector2f(0, 0);
}

BEGIN_SHADER_PARAMETER_STRUCT(FAOSampleParameters,)
	SHADER_PARAMETER_ARRAY(FVector4f,SampleDirections,[NumConeSampleDirections])
	SHADER_PARAMETER(float, BentNormalNormalizeFactor)
END_SHADER_PARAMETER_STRUCT()

static FAOSampleParameters SetupAOSampleParameters(uint32 FrameNumber)
{
	TArray<FVector, TInlineAllocator<9> > SampleDirections;
	GetSpacedVectors(FrameNumber, SampleDirections);

	FVector UnoccludedVector(0);
	for (int32 SampleIndex = 0; SampleIndex < NumConeSampleDirections; SampleIndex++)
	{
		UnoccludedVector += SampleDirections[SampleIndex];
	}

	// LWC_TODO: Was float BentNormalNormalizeFactorValue = 1.0f / (UnoccludedVector / NumConeSampleDirections).Size();
	// This causes "error C4723: potential divide by 0" in msvc, implying the compiler has managed to evaluate (UnoccludedVector / NumConeSampleDirections).Size() as 0 at compile time.
	// Ensuring Size() is called in a seperate line stops the warning, but this needs investigating. Clang seems happy with it. Possible compiler bug?
	const float ConeSampleAverageSize = (UnoccludedVector / NumConeSampleDirections).Size();

	FAOSampleParameters ShaderParameters;
	ShaderParameters.BentNormalNormalizeFactor = ConeSampleAverageSize ? 1.0f / ConeSampleAverageSize : 0.f;

	for (int32 SampleIndex = 0; SampleIndex < NumConeSampleDirections; SampleIndex++)
	{
		ShaderParameters.SampleDirections[SampleIndex] = FVector3f(SampleDirections[SampleIndex]);
	}

	return ShaderParameters;
}

BEGIN_SHADER_PARAMETER_STRUCT(FScreenGridParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistanceFieldNormalTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, DistanceFieldNormalSampler)
	SHADER_PARAMETER(FVector2f, BaseLevelTexelSize)
	SHADER_PARAMETER(FVector2f, JitterOffset)
END_SHADER_PARAMETER_STRUCT()

static FScreenGridParameters SetupScreenGridParameters(const FViewInfo& View, FRDGTextureRef DistanceFieldNormal)
{
	const FIntPoint DownsampledBufferSize = GetBufferSizeForAO(View);

	FScreenGridParameters ShaderParameters;
	ShaderParameters.DistanceFieldNormalTexture = DistanceFieldNormal;
	ShaderParameters.DistanceFieldNormalSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	ShaderParameters.BaseLevelTexelSize = FVector2f(1.0f / DownsampledBufferSize.X, 1.0f / DownsampledBufferSize.Y);
	ShaderParameters.JitterOffset = GetJitterOffset(View);

	return ShaderParameters;
}

class FConeTraceScreenGridObjectOcclusionCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FConeTraceScreenGridObjectOcclusionCS);
	SHADER_USE_PARAMETER_STRUCT(FConeTraceScreenGridObjectOcclusionCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, DistanceFieldCulledObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FTileIntersectionParameters, TileIntersectionParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOScreenGridParameters, AOScreenGridParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FScreenGridParameters, ScreenGridParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGlobalDistanceFieldParameters2, GlobalDistanceFieldParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOSampleParameters, AOSampleParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER(float, TanConeHalfAngle)
		RDG_BUFFER_ACCESS(ObjectTilesIndirectArguments, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	class FUseGlobalDistanceField : SHADER_PERMUTATION_BOOL("USE_GLOBAL_DISTANCE_FIELD");
	class FTraverseMips : SHADER_PERMUTATION_BOOL("SDF_TRACING_TRAVERSE_MIPS");
	using FPermutationDomain = TShaderPermutationDomain<FUseGlobalDistanceField, FTraverseMips>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDFScreenGridLightingShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		TileIntersectionModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);

		// To reduce shader compile time of compute shaders with shared memory, doesn't have an impact on generated code with current compiler (June 2010 DX SDK)
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
	}
};

IMPLEMENT_GLOBAL_SHADER(FConeTraceScreenGridObjectOcclusionCS, "/Engine/Private/DistanceFieldScreenGridLighting.usf", "ConeTraceObjectOcclusionCS", SF_Compute);

const int32 GConeTraceGlobalDFTileSize = 8;

class FConeTraceScreenGridGlobalOcclusionCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FConeTraceScreenGridGlobalOcclusionCS);
	SHADER_USE_PARAMETER_STRUCT(FConeTraceScreenGridGlobalOcclusionCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, DistanceFieldCulledObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOScreenGridParameters, AOScreenGridParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FScreenGridParameters, ScreenGridParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGlobalDistanceFieldParameters2, GlobalDistanceFieldParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOSampleParameters, AOSampleParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<FVector4f>, TileConeDepthRanges)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER(FIntPoint, TileListGroupSize)
		SHADER_PARAMETER(float, TanConeHalfAngle)
	END_SHADER_PARAMETER_STRUCT()

	class FConeTraceObjects : SHADER_PERMUTATION_BOOL("CONE_TRACE_OBJECTS");
	using FPermutationDomain = TShaderPermutationDomain<FConeTraceObjects>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDFScreenGridLightingShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		OutEnvironment.SetDefine(TEXT("CONE_TRACE_GLOBAL_DISPATCH_SIZEX"), GConeTraceGlobalDFTileSize);
		OutEnvironment.SetDefine(TEXT("OUTPUT_VISIBILITY_DIRECTLY"), TEXT("1"));
		OutEnvironment.SetDefine(TEXT("TRACE_DOWNSAMPLE_FACTOR"), GConeTraceDownsampleFactor);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_DISTANCE_FIELD"), TEXT("1"));

		// To reduce shader compile time of compute shaders with shared memory, doesn't have an impact on generated code with current compiler (June 2010 DX SDK)
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
	}
};

IMPLEMENT_GLOBAL_SHADER(FConeTraceScreenGridGlobalOcclusionCS, "/Engine/Private/DistanceFieldScreenGridLighting.usf", "ConeTraceGlobalOcclusionCS", SF_Compute);

const int32 GCombineConesSizeX = 8;

class FCombineConeVisibilityCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCombineConeVisibilityCS);
	SHADER_USE_PARAMETER_STRUCT(FCombineConeVisibilityCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOScreenGridParameters, AOScreenGridParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FScreenGridParameters, ScreenGridParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOSampleParameters, AOSampleParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWDistanceFieldBentNormal)
		SHADER_PARAMETER(FIntPoint, ConeBufferMax)
		SHADER_PARAMETER(FVector2f, DFNormalBufferUVMax)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDFScreenGridLightingShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMBINE_CONES_SIZEX"), GCombineConesSizeX);
		OutEnvironment.SetDefine(TEXT("TRACE_DOWNSAMPLE_FACTOR"), GConeTraceDownsampleFactor);

		if (IsMobilePlatform(Parameters.Platform) && !IsMobileDeferredShadingEnabled(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("ROTATE_WITH_TANGENT"), 0);
		}
		else
		{
			OutEnvironment.SetDefine(TEXT("ROTATE_WITH_TANGENT"), 1);
		}

		// To reduce shader compile time of compute shaders with shared memory, doesn't have an impact on generated code with current compiler (June 2010 DX SDK)
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCombineConeVisibilityCS, "/Engine/Private/DistanceFieldScreenGridLighting.usf", "CombineConeVisibilityCS", SF_Compute);

void PostProcessBentNormalAOScreenGrid(
	FRDGBuilder& GraphBuilder, 
	const FDistanceFieldAOParameters& Parameters, 
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef VelocityTexture,
	FRDGTextureRef BentNormalInterpolation,
	FRDGTextureRef DistanceFieldNormal,
	FRDGTextureRef& BentNormalOutput)
{
	FSceneViewState* ViewState = (FSceneViewState*)View.State;
	FIntRect* DistanceFieldAOHistoryViewRect = ViewState ? &ViewState->DistanceFieldAOHistoryViewRect : nullptr;
	TRefCountPtr<IPooledRenderTarget>* BentNormalHistoryState = ViewState ? &ViewState->DistanceFieldAOHistoryRT : NULL;

	UpdateHistory(
		GraphBuilder,
		View,
		TEXT("DistanceFieldAOHistory"),
		SceneTextures,
		VelocityTexture,
		DistanceFieldNormal,
		BentNormalInterpolation,
		DistanceFieldAOHistoryViewRect,
		BentNormalHistoryState,
		BentNormalOutput,
		Parameters);
}

void FSceneRenderer::RenderDistanceFieldAOScreenGrid(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const FViewInfo& View,
	const FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters,
	FRDGBufferRef ObjectTilesIndirectArguments,
	const FTileIntersectionParameters& TileIntersectionParameters,
	const FDistanceFieldAOParameters& Parameters,
	FRDGTextureRef DistanceFieldNormal,
	FRDGTextureRef& OutDynamicBentNormalAO)
{
	const bool bUseGlobalDistanceField = UseGlobalDistanceField(Parameters) && Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0;
	const bool bUseObjectDistanceField = UseAOObjectDistanceField();

	const FIntPoint ConeTraceBufferSize = GetBufferSizeForConeTracing(View);
	const FIntPoint TileListGroupSize = GetTileListGroupSizeForView(View);

	FAOScreenGridParameters AOScreenGridParameters;

	{
		AOScreenGridParameters.ScreenGridConeVisibilitySize = ConeTraceBufferSize;
		//@todo - 2d textures
		//@todo - FastVRamFlag
		// @todo - ScreenGridConeVisibility should probably be R32_FLOAT format.
		FRDGBufferRef ScreenGridConeVisibility = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), NumConeSampleDirections * ConeTraceBufferSize.X * ConeTraceBufferSize.Y), TEXT("ScreenGridConeVisibility"));
		AOScreenGridParameters.RWScreenGridConeVisibility = GraphBuilder.CreateUAV(ScreenGridConeVisibility, PF_R32_UINT);
		AOScreenGridParameters.ScreenGridConeVisibility = GraphBuilder.CreateSRV(ScreenGridConeVisibility, PF_R32_UINT);
	}

	float ConeVisibilityClearValue = 1.0f;
	AddClearUAVPass(GraphBuilder, AOScreenGridParameters.RWScreenGridConeVisibility, *(uint32*)&ConeVisibilityClearValue);

	const FDistanceFieldSceneData& DistanceFieldSceneData = Scene->DistanceFieldSceneData;

	// Note: no transition, want to overlap object cone tracing and global DF cone tracing since both shaders use atomics to ScreenGridConeVisibility

	if (bUseGlobalDistanceField)
	{
		check(View.GlobalDistanceFieldInfo.Clipmaps.Num() > 0);

		auto* PassParameters = GraphBuilder.AllocParameters<FConeTraceScreenGridGlobalOcclusionCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->DistanceFieldCulledObjectBuffers = CulledObjectBufferParameters;
		PassParameters->DistanceFieldAtlas = DistanceField::SetupAtlasParameters(GraphBuilder, DistanceFieldSceneData);
		PassParameters->AOScreenGridParameters = AOScreenGridParameters;
		PassParameters->AOParameters = DistanceField::SetupAOShaderParameters(Parameters);
		PassParameters->ScreenGridParameters = SetupScreenGridParameters(View, DistanceFieldNormal);
		PassParameters->GlobalDistanceFieldParameters = SetupGlobalDistanceFieldParameters(View.GlobalDistanceFieldInfo.ParameterData);
		PassParameters->AOSampleParameters = SetupAOSampleParameters(View.Family->FrameNumber);
		PassParameters->TileConeDepthRanges = TileIntersectionParameters.TileConeDepthRanges;
		PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);;
		PassParameters->TileListGroupSize = TileListGroupSize;
		PassParameters->TanConeHalfAngle = FMath::Tan(GAOConeHalfAngle);

		FConeTraceScreenGridGlobalOcclusionCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FConeTraceScreenGridGlobalOcclusionCS::FConeTraceObjects>(bUseObjectDistanceField);

		auto ComputeShader = View.ShaderMap->GetShader<FConeTraceScreenGridGlobalOcclusionCS>(PermutationVector);

		const uint32 GroupSizeX = FMath::DivideAndRoundUp(ConeTraceBufferSize.X, GConeTraceGlobalDFTileSize);
		const uint32 GroupSizeY = FMath::DivideAndRoundUp(ConeTraceBufferSize.Y, GConeTraceGlobalDFTileSize);

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("ConeTraceGlobal"), ComputeShader, PassParameters, FIntVector(GroupSizeX, GroupSizeY, 1));
	}

	if (bUseObjectDistanceField)
	{
		check(!bUseGlobalDistanceField || View.GlobalDistanceFieldInfo.Clipmaps.Num() > 0);

		auto* PassParameters = GraphBuilder.AllocParameters<FConeTraceScreenGridObjectOcclusionCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->DistanceFieldObjectBuffers = DistanceField::SetupObjectBufferParameters(GraphBuilder, DistanceFieldSceneData);
		PassParameters->DistanceFieldCulledObjectBuffers = CulledObjectBufferParameters;
		PassParameters->DistanceFieldAtlas = DistanceField::SetupAtlasParameters(GraphBuilder, DistanceFieldSceneData);
		PassParameters->TileIntersectionParameters = TileIntersectionParameters;
		PassParameters->AOScreenGridParameters = AOScreenGridParameters;
		PassParameters->AOParameters = DistanceField::SetupAOShaderParameters(Parameters);
		PassParameters->ScreenGridParameters = SetupScreenGridParameters(View, DistanceFieldNormal);
		if (bUseGlobalDistanceField)
		{
			PassParameters->GlobalDistanceFieldParameters = SetupGlobalDistanceFieldParameters(View.GlobalDistanceFieldInfo.ParameterData);
		}
		PassParameters->AOSampleParameters = SetupAOSampleParameters(View.Family->FrameNumber);
		PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.FeatureLevel);;
		PassParameters->TanConeHalfAngle = FMath::Tan(GAOConeHalfAngle);
		PassParameters->ObjectTilesIndirectArguments = ObjectTilesIndirectArguments;

		FConeTraceScreenGridObjectOcclusionCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FConeTraceScreenGridObjectOcclusionCS::FUseGlobalDistanceField>(bUseGlobalDistanceField);
		PermutationVector.Set<FConeTraceScreenGridObjectOcclusionCS::FTraverseMips>(GDistanceFieldAOTraverseMips != 0);

		auto ComputeShader = View.ShaderMap->GetShader<FConeTraceScreenGridObjectOcclusionCS>(PermutationVector);

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("ConeTraceObjects"), ComputeShader, PassParameters, ObjectTilesIndirectArguments, 0);
	}

	FRDGTextureRef DownsampledBentNormal = nullptr;

	{
		const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(ConeTraceBufferSize, PF_FloatRGBA, FClearValueBinding::None, GFastVRamConfig.DistanceFieldAODownsampledBentNormal | TexCreate_RenderTargetable | TexCreate_UAV | TexCreate_ShaderResource);
		DownsampledBentNormal = GraphBuilder.CreateTexture(Desc, TEXT("DownsampledBentNormal"));
	}

	{
		const uint32 GroupSizeX = FMath::DivideAndRoundUp(ConeTraceBufferSize.X, GCombineConesSizeX);
		const uint32 GroupSizeY = FMath::DivideAndRoundUp(ConeTraceBufferSize.Y, GCombineConesSizeX);

		const FIntPoint DFNormalBufferSize = GetBufferSizeForAO(View);
		const FVector2f DFNormalBufferUVMaxValue(
			(View.ViewRect.Width() / GAODownsampleFactor - 0.5f) / DFNormalBufferSize.X,
			(View.ViewRect.Height() / GAODownsampleFactor - 0.5f) / DFNormalBufferSize.Y);

		auto* PassParameters = GraphBuilder.AllocParameters<FCombineConeVisibilityCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->AOScreenGridParameters = AOScreenGridParameters;
		PassParameters->ScreenGridParameters = SetupScreenGridParameters(View, DistanceFieldNormal);
		PassParameters->AOSampleParameters = SetupAOSampleParameters(View.Family->FrameNumber);
		PassParameters->RWDistanceFieldBentNormal = GraphBuilder.CreateUAV(DownsampledBentNormal);
		PassParameters->ConeBufferMax = FIntPoint(ConeTraceBufferSize.X - 1, ConeTraceBufferSize.Y - 1);
		PassParameters->DFNormalBufferUVMax = DFNormalBufferUVMaxValue;

		auto ComputeShader = View.ShaderMap->GetShader<FCombineConeVisibilityCS>();

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("CombineCones"), ComputeShader, PassParameters, FIntVector(GroupSizeX, GroupSizeY, 1));
	}

	PostProcessBentNormalAOScreenGrid(
		GraphBuilder,
		Parameters,
		View,
		SceneTextures,
		SceneTextures.Velocity,
		DownsampledBentNormal,
		DistanceFieldNormal,
		OutDynamicBentNormalAO);
}
