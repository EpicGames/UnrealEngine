// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldShadowing.cpp
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "ShaderParameters.h"
#include "RenderResource.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "PostProcess/SceneRenderTargets.h"
#include "LightSceneInfo.h"
#include "GlobalShader.h"
#include "ScenePrivate.h"
#include "SceneRenderTargetParameters.h"
#include "ShadowRendering.h"
#include "DeferredShadingRenderer.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "DistanceFieldAtlas.h"
#include "DistanceFieldLightingShared.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "Substrate/Substrate.h"
#include "PixelShaderUtils.h"
#include "Engine/Engine.h"

int32 GDistanceFieldShadowing = 1;
FAutoConsoleVariableRef CVarDistanceFieldShadowing(
	TEXT("r.DistanceFieldShadowing"),
	GDistanceFieldShadowing,
	TEXT("Whether the distance field shadowing feature is allowed."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GDFShadowQuality = 3;
FAutoConsoleVariableRef CVarDFShadowQuality(
	TEXT("r.DFShadowQuality"),
	GDFShadowQuality,
	TEXT("Defines the distance field shadow method which allows to adjust for quality or performance.\n")
	TEXT(" 0:off, 1:low (20 steps, no SSS), 2:medium (32 steps, no SSS), 3:high (64 steps, SSS, default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GFullResolutionDFShadowing = 0;
FAutoConsoleVariableRef CVarFullResolutionDFShadowing(
	TEXT("r.DFFullResolution"),
	GFullResolutionDFShadowing,
	TEXT("1 = full resolution distance field shadowing, 0 = half resolution with bilateral upsample."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GShadowScatterTileCulling = 1;
FAutoConsoleVariableRef CVarShadowScatterTileCulling(
	TEXT("r.DFShadowScatterTileCulling"),
	GShadowScatterTileCulling,
	TEXT("Whether to use the rasterizer to scatter objects onto the tile grid for culling."),
	ECVF_RenderThreadSafe
	);

float GShadowCullTileWorldSize = 200.0f;
FAutoConsoleVariableRef CVarShadowCullTileWorldSize(
	TEXT("r.DFShadowCullTileWorldSize"),
	GShadowCullTileWorldSize,
	TEXT("World space size of a tile used for culling for directional lights."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GDFShadowTwoSidedMeshDistanceBiasScale = 1.0f;
FAutoConsoleVariableRef CVarShadowTwoSidedMeshDistanceBiasScale(
	TEXT("r.DFShadow.TwoSidedMeshDistanceBiasScale"),
	GDFShadowTwoSidedMeshDistanceBiasScale,
	TEXT("Scale applied to distance bias when calculating distance field shadows of two sided meshes. This is useful to get tree shadows to match up with standard shadow mapping."),
	ECVF_RenderThreadSafe
	);

int32 GAverageObjectsPerShadowCullTile = 128;
FAutoConsoleVariableRef CVarAverageObjectsPerShadowCullTile(
	TEXT("r.DFShadowAverageObjectsPerCullTile"),
	GAverageObjectsPerShadowCullTile,
	TEXT("Determines how much memory should be allocated in distance field object culling data structures.  Too much = memory waste, too little = flickering due to buffer overflow."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
	);

int32 GDFShadowAsyncCompute = 0;
static FAutoConsoleVariableRef CVarDFShadowAsyncCompute(
	TEXT("r.DFShadowAsyncCompute"),
	GDFShadowAsyncCompute,
	TEXT("Whether render distance field shadows using async compute if possible"),
	ECVF_RenderThreadSafe);

static int32 GHeightFieldShadowing = 0;
FAutoConsoleVariableRef CVarHeightFieldShadowing(
	TEXT("r.HeightFieldShadowing"),
	GHeightFieldShadowing,
	TEXT("Whether the height field shadowing feature is allowed."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

int32 GHFShadowQuality = 2;
FAutoConsoleVariableRef CVarHFShadowQuality(
	TEXT("r.HFShadowQuality"),
	GHFShadowQuality,
	TEXT("Defines the height field shadow method which allows to adjust for quality or performance.\n")
	TEXT(" 0:off, 1:low (8 steps), 2:medium (16 steps, default), 3:high (32 steps, hole aware)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static float GMinDirectionalLightAngleForRTHF = 27.f;
static FAutoConsoleVariableRef CVarMinDirectionalLightAngleForRTHF(
	TEXT("r.Shadow.MinDirectionalLightAngleForRTHF"),
	GMinDirectionalLightAngleForRTHF,
	TEXT(""),
	ECVF_RenderThreadSafe);

int32 GAverageHeightFieldObjectsPerShadowCullTile = 16;
FAutoConsoleVariableRef CVarAverageHeightFieldObjectsPerShadowCullTile(
	TEXT("r.HFShadowAverageObjectsPerCullTile"),
	GAverageHeightFieldObjectsPerShadowCullTile,
	TEXT("Determines how much memory should be allocated in height field object culling data structures.  Too much = memory waste, too little = flickering due to buffer overflow."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

int32 GDFShadowCompactCulledObjects = 1;
static FAutoConsoleVariableRef CVarCompactCulledObjects(
	TEXT("r.DFShadowCompactCulledObjects"),
	GDFShadowCompactCulledObjects,
	TEXT("Whether to compact culled object indices when using scattered tile culling. ")
	TEXT("Note that each tile can only hold up to r.DFShadowAverageObjectsPerCullTile number of objects when compaction is not used."),
	ECVF_RenderThreadSafe);

bool GDFShadowCullingSubsampleDepth = false;
static FAutoConsoleVariableRef CVarCullingSubsampleDepth(
	TEXT("r.DFShadowCullingSubsampleDepth"),
	GDFShadowCullingSubsampleDepth,
	TEXT("When deciding whether to cull DF shadows for a pixel, subsample the depthbuffer instead of checking all relevant depth texels. ")
	TEXT("Decreases bandwidth, but produces artifacts on edges and pixel-sized holes"),
	ECVF_RenderThreadSafe);

int32 const GDistanceFieldShadowTileSizeX = 8;
int32 const GDistanceFieldShadowTileSizeY = 8;

int32 GetDFShadowDownsampleFactor()
{
	return GFullResolutionDFShadowing ? 1 : GAODownsampleFactor;
}

FIntPoint GetBufferSizeForDFShadows(const FViewInfo& View)
{
	// TODO: Use GetDownscaledExtent(...) but need to check different rounding doesn't cause issues
	return FIntPoint::DivideAndRoundDown(View.GetSceneTexturesConfig().Extent, GetDFShadowDownsampleFactor());
}

FIntRect GetScissorRectForDFShadows(FIntRect ScissorRect)
{
	// TODO: Use GetDownscaledRect(...) but need to check different rounding doesn't cause issues
	return ScissorRect / GetDFShadowDownsampleFactor();
}

#define MAX_NUM_SHADOW_CONVEX_HULL_PLANES (12)

class FCullObjectsForShadowCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCullObjectsForShadowCS);
	SHADER_USE_PARAMETER_STRUCT(FCullObjectsForShadowCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, ObjectBufferParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, CulledObjectBufferParameters)
		SHADER_PARAMETER(uint32, ObjectBoundingGeometryIndexCount)
		SHADER_PARAMETER(FMatrix44f, TranslatedWorldToShadow)
		SHADER_PARAMETER(uint32, NumShadowHullPlanes)
		SHADER_PARAMETER(uint32, bDrawNaniteMeshes)
		SHADER_PARAMETER(uint32, bCullHeighfieldsNotInAtlas)
		SHADER_PARAMETER(FVector4f, ShadowBoundingSphere)
		SHADER_PARAMETER_ARRAY(FVector4f,ShadowConvexHull,[MAX_NUM_SHADOW_CONVEX_HULL_PLANES])
	END_SHADER_PARAMETER_STRUCT()

	class FPrimitiveType : SHADER_PERMUTATION_INT("DISTANCEFIELD_PRIMITIVE_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FPrimitiveType>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		OutEnvironment.SetDefine(TEXT("UPDATEOBJECTS_THREADGROUP_SIZE"), UpdateObjectsGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCullObjectsForShadowCS, "/Engine/Private/DistanceFieldShadowing.usf", "CullObjectsForShadowCS", SF_Compute);

class FShadowObjectCullVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FShadowObjectCullVS);
	SHADER_USE_PARAMETER_STRUCT(FShadowObjectCullVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, ObjectBufferParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, CulledObjectBufferParameters)
		SHADER_PARAMETER(FMatrix44f, TranslatedWorldToShadow)
		SHADER_PARAMETER(float, MinExpandRadius)
	END_SHADER_PARAMETER_STRUCT()

	class FPrimitiveType : SHADER_PERMUTATION_INT("DISTANCEFIELD_PRIMITIVE_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FPrimitiveType>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FShadowObjectCullVS, "/Engine/Private/DistanceFieldShadowing.usf", "ShadowObjectCullVS", SF_Vertex);

class FShadowObjectCullPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FShadowObjectCullPS);
	SHADER_USE_PARAMETER_STRUCT(FShadowObjectCullPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, ObjectBufferParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, CulledObjectBufferParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightTileIntersectionParameters, LightTileIntersectionParameters)
		SHADER_PARAMETER(FMatrix44f, TranslatedWorldToShadow)
		SHADER_PARAMETER(float, ObjectExpandScale)
	END_SHADER_PARAMETER_STRUCT()

	class FPrimitiveType : SHADER_PERMUTATION_INT("DISTANCEFIELD_PRIMITIVE_TYPE", 2);
	class FCountingPass : SHADER_PERMUTATION_BOOL("SCATTER_CULLING_COUNT_PASS");
	class FCompactCulledObjects : SHADER_PERMUTATION_BOOL("COMPACT_CULLED_SHADOW_OBJECTS");
	using FPermutationDomain = TShaderPermutationDomain<FPrimitiveType, FCountingPass, FCompactCulledObjects>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (!PermutationVector.Get<FCompactCulledObjects>() && PermutationVector.Get<FCountingPass>())
		{
			// We don't need a counting pass if we don't compact culled objects
			return false;
		}

		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FShadowObjectCullPS, "/Engine/Private/DistanceFieldShadowing.usf", "ShadowObjectCullPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FShadowMeshSDFObjectCull, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FShadowObjectCullVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FShadowObjectCullPS::FParameters, PS)
	RDG_BUFFER_ACCESS(MeshSDFIndirectArgs, ERHIAccess::IndirectArgs)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

enum EDistanceFieldShadowingType
{
	DFS_DirectionalLightScatterTileCulling,
	DFS_DirectionalLightTiledCulling,
	DFS_PointLightTiledCulling
};

int32 GetDFShadowQuality()
{
	return FMath::Clamp(GDFShadowQuality, 0, 3);
}

int32 GetHFShadowQuality()
{
	return FMath::Clamp(GHFShadowQuality, 0, 3);
}

class FDistanceFieldShadowingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDistanceFieldShadowingCS);
	SHADER_USE_PARAMETER_STRUCT(FDistanceFieldShadowingCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float2>, RWShadowFactors)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER(FVector2f, NumGroups)
		SHADER_PARAMETER(FVector3f, LightDirection)
		SHADER_PARAMETER(FVector4f, LightTranslatedPositionAndInvRadius)
		SHADER_PARAMETER(float, LightSourceRadius)
		SHADER_PARAMETER(float, RayStartOffsetDepthScale)
		SHADER_PARAMETER(FVector3f, TanLightAngleAndNormalThreshold)
		SHADER_PARAMETER(FIntRect, ScissorRectMinAndSize)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, ObjectBufferParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, CulledObjectBufferParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightTileIntersectionParameters, LightTileIntersectionParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlasParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHeightFieldAtlasParameters, HeightFieldAtlasParameters)
		SHADER_PARAMETER(FMatrix44f, TranslatedWorldToShadow)
		SHADER_PARAMETER(float, TwoSidedMeshDistanceBiasScale)
		SHADER_PARAMETER(float, MinDepth)
		SHADER_PARAMETER(float, MaxDepth)
		SHADER_PARAMETER(uint32, DownsampleFactor)
		SHADER_PARAMETER(FVector2f, InvOutputBufferSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowFactorsTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowFactorsSampler)
		SHADER_PARAMETER(FMatrix44f, ScreenToView)
	END_SHADER_PARAMETER_STRUCT()

	class FCullingType : SHADER_PERMUTATION_INT("CULLING_TYPE", 3);
	class FShadowQuality : SHADER_PERMUTATION_INT("DF_SHADOW_QUALITY", 3);
	class FPrimitiveType : SHADER_PERMUTATION_INT("DISTANCEFIELD_PRIMITIVE_TYPE", 2);
	class FHasPreviousOutput : SHADER_PERMUTATION_BOOL("HAS_PREVIOUS_OUTPUT");
	class FOffsetDataStructure : SHADER_PERMUTATION_INT("OFFSET_DATA_STRUCT", 3);
	class FCompactCulledObjects : SHADER_PERMUTATION_BOOL("COMPACT_CULLED_SHADOW_OBJECTS");
	class FCullingSubSampleDepth : SHADER_PERMUTATION_BOOL("CULLING_SUBSAMPLE_DEPTH");
	class FStereoRendering : SHADER_PERMUTATION_BOOL("STEREO_RENDERING");
	using FPermutationDomain = TShaderPermutationDomain<FCullingType, FShadowQuality, FPrimitiveType, FHasPreviousOutput, FOffsetDataStructure, FCompactCulledObjects, FCullingSubSampleDepth, FStereoRendering>;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FCullingType>() != DFS_DirectionalLightScatterTileCulling)
		{
			// Compacting culled objects is only relevant when we do scattered tile culling
			PermutationVector.Set<FCompactCulledObjects>(false);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}

		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector = RemapPermutation(FPermutationDomain(Parameters.PermutationId));

		EDistanceFieldPrimitiveType PrimitiveType = (EDistanceFieldPrimitiveType)PermutationVector.Get<FPrimitiveType>();
		const int32 DFShadowQuality = (PrimitiveType == DFPT_HeightField ? GetHFShadowQuality() : GetDFShadowQuality()) - 1;
		if (PermutationVector.Get<FShadowQuality>() != DFShadowQuality)
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		extern int32 GDistanceFieldOffsetDataStructure;
		if (PermutationVector.Get<FOffsetDataStructure>() != GDistanceFieldOffsetDataStructure)
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		if (PermutationVector.Get<FCompactCulledObjects>() != (GDFShadowCompactCulledObjects != 0 && (PermutationVector.Get<FCullingType>() == DFS_DirectionalLightScatterTileCulling)))
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		return EShaderPermutationPrecacheRequest::Precached;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GDistanceFieldShadowTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GDistanceFieldShadowTileSizeY);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), 1);
		OutEnvironment.SetDefine(TEXT("PLATFORM_SUPPORTS_TYPED_UAV_LOAD"), (int32)RHISupports4ComponentUAVReadWrite(Parameters.Platform));
	}
};

IMPLEMENT_GLOBAL_SHADER(FDistanceFieldShadowingCS, "/Engine/Private/DistanceFieldShadowing.usf", "DistanceFieldShadowingCS", SF_Compute);

class FDistanceFieldShadowingUpsamplePS : public FGlobalShader
	{
	DECLARE_GLOBAL_SHADER(FDistanceFieldShadowingUpsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FDistanceFieldShadowingUpsamplePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowFactorsTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowFactorsSampler)
		SHADER_PARAMETER(FIntRect, ScissorRectMinAndSize)
		SHADER_PARAMETER(FVector2f, ShadowFactorsUVBilinearMax)
		SHADER_PARAMETER(float, FadePlaneOffset)
		SHADER_PARAMETER(float, InvFadePlaneLength)
		SHADER_PARAMETER(float, NearFadePlaneOffset)
		SHADER_PARAMETER(float, InvNearFadePlaneLength)
		SHADER_PARAMETER(float, OneOverDownsampleFactor)
		SHADER_PARAMETER(float, ShadowFadeFraction)
		SHADER_PARAMETER(float, MinDepth)
		SHADER_PARAMETER(float, MaxDepth)
	END_SHADER_PARAMETER_STRUCT()

	class FUpsample : SHADER_PERMUTATION_BOOL("SHADOW_FACTORS_UPSAMPLE_REQUIRED");
	using FPermutationDomain = TShaderPermutationDomain<FUpsample>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("UPSAMPLE_PASS"), 1);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDistanceFieldShadowingUpsamplePS, "/Engine/Private/DistanceFieldShadowing.usf", "DistanceFieldShadowingUpsamplePS", SF_Pixel);


bool UseShadowIndirectDraw(EShaderPlatform ShaderPlatform)
{
	return IsFeatureLevelSupported(ShaderPlatform, ERHIFeatureLevel::SM5)
		&& !IsVulkanMobilePlatform(ShaderPlatform)
		&& FDataDrivenShaderPlatformInfo::GetSupportsManualVertexFetch(ShaderPlatform);
}

class FShadowTileVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FShadowTileVS);
	SHADER_USE_PARAMETER_STRUCT(FShadowTileVS, FGlobalShader);

	class FTileType : SHADER_PERMUTATION_INT("PERMUTATION_TILE_TYPE",2);
	using FPermutationDomain = TShaderPermutationDomain<FTileType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileListData)
	END_SHADER_PARAMETER_STRUCT()

	static int32 GetTileSize()
	{
		return 8;
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return UseShadowIndirectDraw(Parameters.Platform) && ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SHADOW_TILE_VS"), 1);
		OutEnvironment.SetDefine(TEXT("WORK_TILE_SIZE"), GetTileSize());
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FShadowTileVS, "/Engine/Private/DistanceFieldShadowing.usf", "ShadowTileVS", SF_Vertex);

const uint32 ComputeCulledObjectStartOffsetGroupSize = 8;

/**  */
class FComputeCulledObjectStartOffsetCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComputeCulledObjectStartOffsetCS);
	SHADER_USE_PARAMETER_STRUCT(FComputeCulledObjectStartOffsetCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightTileIntersectionParameters, LightTileIntersectionParameters)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTE_START_OFFSET_GROUP_SIZE"), ComputeCulledObjectStartOffsetGroupSize);
	}
};

IMPLEMENT_SHADER_TYPE(,FComputeCulledObjectStartOffsetCS,TEXT("/Engine/Private/DistanceFieldShadowing.usf"),TEXT("ComputeCulledTilesStartOffsetCS"),SF_Compute);

void ScatterObjectsToShadowTiles(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, 
	const FMatrix& WorldToShadowValue, 
	float ShadowBoundingRadius,
	bool bCountingPass, 
	EDistanceFieldPrimitiveType PrimitiveType,
	FIntPoint LightTileDimensions, 
	FRDGBufferRef ObjectIndirectArguments,
	const FDistanceFieldObjectBufferParameters& ObjectBufferParameters,
	const FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters,
	const FLightTileIntersectionParameters& LightTileIntersectionParameters)
{
	{
		FShadowMeshSDFObjectCull* PassParameters = GraphBuilder.AllocParameters<FShadowMeshSDFObjectCull>();

		if (GRHIRequiresRenderTargetForPixelShaderUAVs)
		{
			FRDGTextureDesc DummyDesc = FRDGTextureDesc::Create2D(LightTileDimensions, PF_B8G8R8A8, FClearValueBinding::Black, TexCreate_RenderTargetable);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(GraphBuilder.CreateTexture(DummyDesc, TEXT("Dummy")), ERenderTargetLoadAction::ENoAction);
		}

		const float MinExpandRadiusValue = (PrimitiveType == DFPT_HeightField ? 0.87f : 1.414f) * ShadowBoundingRadius / FMath::Min(LightTileDimensions.X, LightTileDimensions.Y);

		PassParameters->VS.View = GetShaderBinding(View.ViewUniformBuffer);
		PassParameters->VS.ObjectBufferParameters = ObjectBufferParameters;
		PassParameters->VS.CulledObjectBufferParameters = CulledObjectBufferParameters;
		PassParameters->VS.TranslatedWorldToShadow = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToShadowValue);
		PassParameters->VS.MinExpandRadius = MinExpandRadiusValue;
		PassParameters->PS.View = GetShaderBinding(View.ViewUniformBuffer);
		PassParameters->PS.ObjectBufferParameters = ObjectBufferParameters;
		PassParameters->PS.CulledObjectBufferParameters = CulledObjectBufferParameters;
		PassParameters->PS.LightTileIntersectionParameters = LightTileIntersectionParameters;
		PassParameters->PS.TranslatedWorldToShadow = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToShadowValue);
		PassParameters->PS.ObjectExpandScale = PrimitiveType == DFPT_HeightField ? 0.f : WorldToShadowValue.GetMaximumAxisScale();

		PassParameters->MeshSDFIndirectArgs = ObjectIndirectArguments;

		FShadowObjectCullVS::FPermutationDomain VSPermutationVector;
		VSPermutationVector.Set< FShadowObjectCullVS::FPrimitiveType >(PrimitiveType);
		auto VertexShader = View.ShaderMap->GetShader< FShadowObjectCullVS >(VSPermutationVector);

		FShadowObjectCullPS::FPermutationDomain PermutationVector;
		PermutationVector.Set< FShadowObjectCullPS::FPrimitiveType >(PrimitiveType);
		PermutationVector.Set< FShadowObjectCullPS::FCountingPass >(bCountingPass);
		PermutationVector.Set<FShadowObjectCullPS::FCompactCulledObjects>(GDFShadowCompactCulledObjects != 0);
		auto PixelShader = View.ShaderMap->GetShader< FShadowObjectCullPS >(PermutationVector);

		const bool bReverseCulling = View.bReverseCulling;

		ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
		ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("ScatterMeshSDFsToLightGrid %ux%u", LightTileDimensions.X, LightTileDimensions.Y),
			PassParameters,
			ERDGPassFlags::Raster,
			[LightTileDimensions, bReverseCulling, VertexShader, PixelShader, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			RHICmdList.SetViewport(0, 0, 0.0f, LightTileDimensions.X, LightTileDimensions.Y, 1.0f);

			// Render backfaces since camera may intersect
			GraphicsPSOInit.RasterizerState = bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

			RHICmdList.SetStreamSource(0, GetUnitCubeVertexBuffer(), 0);

			RHICmdList.DrawIndexedPrimitiveIndirect(
				GetUnitCubeIndexBuffer(),
				PassParameters->MeshSDFIndirectArgs->GetIndirectRHICallBuffer(),
				0);
		});
	}
}

void AllocateDistanceFieldCulledObjectBuffers(
	FRDGBuilder& GraphBuilder, 
	uint32 MaxObjects, 
	FRDGBufferRef& OutObjectIndirectArguments,
	FDistanceFieldCulledObjectBufferParameters& OutParameters)
{
	check(MaxObjects > 0);
	OutObjectIndirectArguments = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndexedIndirectParameters>(), TEXT("DistanceField.ObjectIndirectArguments"));
	FRDGBufferRef ObjectIndices = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxObjects), TEXT("DistanceField.ObjectIndices"));

	OutParameters.RWObjectIndirectArguments = GraphBuilder.CreateUAV(OutObjectIndirectArguments, PF_R32_UINT);
	OutParameters.RWCulledObjectIndices = GraphBuilder.CreateUAV(ObjectIndices);

	OutParameters.ObjectIndirectArguments = GraphBuilder.CreateSRV(OutObjectIndirectArguments, PF_R32_UINT);
	OutParameters.CulledObjectIndices = GraphBuilder.CreateSRV(ObjectIndices);
}

void CullDistanceFieldObjectsForLight(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightSceneProxy* LightSceneProxy, 
	EDistanceFieldPrimitiveType PrimitiveType,
	const FMatrix& WorldToShadowValue, 
	int32 NumPlanes, 
	const FPlane* PlaneData,
	const FVector& PrePlaneTranslation,
	const FVector4f& ShadowBoundingSphere,
	float ShadowBoundingRadius,
	bool bCullingForDirectShadowing,
	bool bCullHeighfieldsNotInAtlas,
	const FDistanceFieldObjectBufferParameters& ObjectBufferParameters,
	FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters,
	FLightTileIntersectionParameters& LightTileIntersectionParameters)
{
	const bool bIsHeightfield = PrimitiveType == DFPT_HeightField;
	const FScene* Scene = (const FScene*)(View.Family->Scene);
	FRDGBufferRef ObjectIndirectArguments = nullptr;

	RDG_EVENT_SCOPE(GraphBuilder, "CullDistanceFieldObjectsForLight %s", PrimitiveType == DFPT_SignedDistanceField ? TEXT("MeshSDF") : TEXT("Heightfield"));

	const int32 NumObjectsInBuffer = bIsHeightfield ? ObjectBufferParameters.NumSceneHeightfieldObjects : ObjectBufferParameters.NumSceneObjects;

	AllocateDistanceFieldCulledObjectBuffers(
		GraphBuilder, 
		FMath::DivideAndRoundUp(NumObjectsInBuffer, 256) * 256, 
		ObjectIndirectArguments,
		CulledObjectBufferParameters);

	AddClearUAVPass(GraphBuilder, CulledObjectBufferParameters.RWObjectIndirectArguments, 0);

	ensureMsgf(NumPlanes <= MAX_NUM_SHADOW_CONVEX_HULL_PLANES, TEXT("NumPlanes (%d) is expected to be less or equal to %d."), NumPlanes, MAX_NUM_SHADOW_CONVEX_HULL_PLANES);
	NumPlanes = FMath::Clamp(NumPlanes, 0, MAX_NUM_SHADOW_CONVEX_HULL_PLANES);

	{
		FCullObjectsForShadowCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCullObjectsForShadowCS::FParameters>();
			
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->ObjectBufferParameters = ObjectBufferParameters;
		PassParameters->CulledObjectBufferParameters = CulledObjectBufferParameters;
		PassParameters->ObjectBoundingGeometryIndexCount = UE_ARRAY_COUNT(GCubeIndices);
		PassParameters->TranslatedWorldToShadow = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToShadowValue);
		PassParameters->NumShadowHullPlanes = NumPlanes;
		PassParameters->ShadowBoundingSphere = ShadowBoundingSphere;
		// Disable Nanite meshes for directional lights that use VSM since they draw into the VSM unconditionally (and would get double shadow)
		PassParameters->bDrawNaniteMeshes = !(LightSceneProxy->UseVirtualShadowMaps() && LightSceneProxy->GetLightType() == LightType_Directional) || !bCullingForDirectShadowing;
		PassParameters->bCullHeighfieldsNotInAtlas = bCullHeighfieldsNotInAtlas;

		for (int32 Index = 0; Index < NumPlanes; ++Index)
		{
			// translated planes from translated-shadow-space to translated-world-space
			const FPlane4f Plane(PlaneData[Index].TranslateBy(View.ViewMatrices.GetPreViewTranslation() - PrePlaneTranslation));
			PassParameters->ShadowConvexHull[Index] = FVector4f(FVector3f(Plane), Plane.W);
		}

		FCullObjectsForShadowCS::FPermutationDomain PermutationVector;
		PermutationVector.Set< FCullObjectsForShadowCS::FPrimitiveType >(PrimitiveType);
		auto ComputeShader = View.ShaderMap->GetShader<FCullObjectsForShadowCS>(PermutationVector);

		const FIntVector GroupCount = FComputeShaderUtils::GetGroupCountWrapped(NumObjectsInBuffer, UpdateObjectsGroupSize);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("CullMeshSDFObjectsToFrustum"),
			ComputeShader,
			PassParameters,
			GroupCount);
	}

	if (LightSceneProxy->GetLightType() == LightType_Directional && GShadowScatterTileCulling)
	{
		// Allocate tile resolution based on world space size
		const float LightTiles = FMath::Min(ShadowBoundingRadius / GShadowCullTileWorldSize + 1.0f, 256.0f);
		FIntPoint LightTileDimensions(Align(FMath::TruncToInt(LightTiles), 64), Align(FMath::TruncToInt(LightTiles), 64));

		const bool b16BitObjectIndices = Scene->DistanceFieldSceneData.CanUse16BitObjectIndices();

		FRDGBufferRef ShadowTileNumCulledObjects = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), LightTileDimensions.X * LightTileDimensions.Y), TEXT("ShadowTileNumCulledObjects"));
		LightTileIntersectionParameters.RWShadowTileNumCulledObjects = GraphBuilder.CreateUAV(ShadowTileNumCulledObjects, PF_R32_UINT);
		LightTileIntersectionParameters.ShadowTileNumCulledObjects = GraphBuilder.CreateSRV(ShadowTileNumCulledObjects, PF_R32_UINT);

		const uint32 MaxNumObjectsPerTile = bIsHeightfield ? GAverageHeightFieldObjectsPerShadowCullTile : GAverageObjectsPerShadowCullTile;
		FRDGBufferRef ShadowTileArrayData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(b16BitObjectIndices ? sizeof(uint16) : sizeof(uint32), MaxNumObjectsPerTile * LightTileDimensions.X * LightTileDimensions.Y), TEXT("ShadowTileArrayData"));
		LightTileIntersectionParameters.RWShadowTileArrayData = GraphBuilder.CreateUAV(ShadowTileArrayData, b16BitObjectIndices ? PF_R16_UINT : PF_R32_UINT);
		LightTileIntersectionParameters.ShadowTileArrayData = GraphBuilder.CreateSRV(ShadowTileArrayData, b16BitObjectIndices ? PF_R16_UINT : PF_R32_UINT);
		LightTileIntersectionParameters.ShadowTileListGroupSize = LightTileDimensions;
		LightTileIntersectionParameters.ShadowMaxObjectsPerTile = MaxNumObjectsPerTile;

		if (GDFShadowCompactCulledObjects != 0)
		{
			FRDGBufferRef ShadowTileStartOffsets = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), LightTileDimensions.X * LightTileDimensions.Y), TEXT("ShadowTileStartOffsets"));
			LightTileIntersectionParameters.RWShadowTileStartOffsets = GraphBuilder.CreateUAV(ShadowTileStartOffsets, PF_R32_UINT);
			LightTileIntersectionParameters.ShadowTileStartOffsets = GraphBuilder.CreateSRV(ShadowTileStartOffsets, PF_R32_UINT);

			FRDGBufferRef NextStartOffset = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1), TEXT("ShadowNextStartOffset"));
			LightTileIntersectionParameters.RWNextStartOffset = GraphBuilder.CreateUAV(NextStartOffset, PF_R32_UINT);
			LightTileIntersectionParameters.NextStartOffset = GraphBuilder.CreateSRV(NextStartOffset, PF_R32_UINT);

			// Start at 0 tiles per object
			AddClearUAVPass(GraphBuilder, LightTileIntersectionParameters.RWShadowTileNumCulledObjects, 0);

			// Rasterize object bounding shapes and intersect with shadow tiles to compute how many objects intersect each tile
			ScatterObjectsToShadowTiles(GraphBuilder, View, WorldToShadowValue, ShadowBoundingRadius, true, PrimitiveType, LightTileDimensions, ObjectIndirectArguments, ObjectBufferParameters, CulledObjectBufferParameters, LightTileIntersectionParameters);

			AddClearUAVPass(GraphBuilder, LightTileIntersectionParameters.RWNextStartOffset, 0);

			// Compute the start offset for each tile's culled object data
			{
				FComputeCulledObjectStartOffsetCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComputeCulledObjectStartOffsetCS::FParameters>();

				PassParameters->LightTileIntersectionParameters = LightTileIntersectionParameters;
				auto ComputeShader = View.ShaderMap->GetShader<FComputeCulledObjectStartOffsetCS>();
				uint32 GroupSizeX = FMath::DivideAndRoundUp<int32>(LightTileDimensions.X, ComputeCulledObjectStartOffsetGroupSize);
				uint32 GroupSizeY = FMath::DivideAndRoundUp<int32>(LightTileDimensions.Y, ComputeCulledObjectStartOffsetGroupSize);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("ComputeCulledObjectStartOffset"),
					ComputeShader,
					PassParameters,
					FIntVector(GroupSizeX, GroupSizeY, 1));
			}
		}
		else
		{
			LightTileIntersectionParameters.RWShadowTileStartOffsets = nullptr;
			LightTileIntersectionParameters.ShadowTileStartOffsets = nullptr;

			LightTileIntersectionParameters.RWNextStartOffset = nullptr;
			LightTileIntersectionParameters.NextStartOffset = nullptr;
		}

		// Start at 0 tiles per object
		AddClearUAVPass(GraphBuilder, LightTileIntersectionParameters.RWShadowTileNumCulledObjects, 0);

		// Rasterize object bounding shapes and intersect with shadow tiles, and write out intersecting tile indices for the cone tracing pass
		ScatterObjectsToShadowTiles(GraphBuilder, View, WorldToShadowValue, ShadowBoundingRadius, false, PrimitiveType, LightTileDimensions, ObjectIndirectArguments, ObjectBufferParameters, CulledObjectBufferParameters, LightTileIntersectionParameters);
	}
}

bool SupportsDistanceFieldShadows(ERHIFeatureLevel::Type FeatureLevel, EShaderPlatform ShaderPlatform)
{
	return GDistanceFieldShadowing 
		&& GetDFShadowQuality() > 0
		&& ShouldCompileDistanceFieldShaders(ShaderPlatform);
}

bool SupportsHeightFieldShadows(ERHIFeatureLevel::Type FeatureLevel, EShaderPlatform ShaderPlatform)
{
	return GHeightFieldShadowing
		&& GetHFShadowQuality() > 0
		&& FeatureLevel >= ERHIFeatureLevel::SM5
		&& ShouldCompileDistanceFieldShaders(ShaderPlatform);
}

bool ShouldPrepareForDistanceFieldShadows(const FSceneRenderUpdateInputs& SceneUpdateInputs)
{
	if (!SceneUpdateInputs.HasAnyShowFlags([] (const FEngineShowFlags& EngineShowFlags) { return EngineShowFlags.DynamicShadows && EngineShowFlags.RayTracedDistanceFieldShadows; }) ||
		!SupportsDistanceFieldShadows(SceneUpdateInputs.FeatureLevel, SceneUpdateInputs.ShaderPlatform))
	{
		return false;
	}

	return true;
}

bool ShouldPrepareHeightFieldScene(const FSceneRenderUpdateInputs& SceneUpdateInputs)
{
	return SceneUpdateInputs.HasAnyShowFlags([](const FEngineShowFlags& ShowFlags) { return ShowFlags.DynamicShadows && !ShowFlags.PathTracing; })
		&& SupportsHeightFieldShadows(SceneUpdateInputs.FeatureLevel, SceneUpdateInputs.ShaderPlatform);
}

void GetDistanceFieldShadowRange(const FProjectedShadowInfo* ProjectedShadowInfo, EDistanceFieldPrimitiveType PrimitiveType, float& OutMinDepth, float& OutMaxDepth)
{
	if (ProjectedShadowInfo->bDirectionalLight)
	{
		OutMinDepth = ProjectedShadowInfo->CascadeSettings.SplitNear - ProjectedShadowInfo->CascadeSettings.SplitNearFadeRegion;
		OutMaxDepth = ProjectedShadowInfo->CascadeSettings.SplitFar;
	}
	else
	{
		check(PrimitiveType != DFPT_HeightField);
		//@todo - set these up for point lights as well
		OutMinDepth = 0.0f;
		OutMaxDepth = HALF_WORLD_MAX;
	}
}

void RayTraceShadows(
	FRDGBuilder& GraphBuilder,
	bool bAsyncCompute,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef OutputTexture,
	const FViewInfo& View,
	const FIntRect& ScissorRect,
	const FIntRect& DownsampledScissorRect,
	const FDistanceFieldSceneData& DistanceFieldSceneData,
	const FProjectedShadowInfo* ProjectedShadowInfo,
	EDistanceFieldPrimitiveType PrimitiveType,
	bool bHasPrevOutput,
	FRDGTextureRef PrevOutputTexture,
	const FDistanceFieldObjectBufferParameters& ObjectBufferParameters,
	const FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters,
	const FLightTileIntersectionParameters& LightTileIntersectionParameters)
{
	const int32 DFShadowQuality = (PrimitiveType == DFPT_HeightField ? GetHFShadowQuality() : GetDFShadowQuality()) - 1;
	check(DFShadowQuality >= 0);

	EDistanceFieldShadowingType DistanceFieldShadowingType;

	if (ProjectedShadowInfo->bDirectionalLight && GShadowScatterTileCulling)
	{
		DistanceFieldShadowingType = DFS_DirectionalLightScatterTileCulling;
	}
	else if (ProjectedShadowInfo->bDirectionalLight)
	{
		DistanceFieldShadowingType = DFS_DirectionalLightTiledCulling;
	}
	else
	{
		DistanceFieldShadowingType = DFS_PointLightTiledCulling;
	}

	check(DistanceFieldShadowingType != DFS_PointLightTiledCulling || PrimitiveType != DFPT_HeightField);

	FHeightFieldAtlasParameters HeightFieldAtlasParameters;
	HeightFieldAtlasParameters.HeightFieldTexture = GHeightFieldTextureAtlas.GetAtlasTexture(GraphBuilder);
	HeightFieldAtlasParameters.HFVisibilityTexture = GHFVisibilityTextureAtlas.GetAtlasTexture(GraphBuilder);

	{
		FDistanceFieldShadowingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDistanceFieldShadowingCS::FParameters>();
			
		PassParameters->RWShadowFactors = GraphBuilder.CreateUAV(OutputTexture);
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.GetFeatureLevel());

		const FLightSceneProxy& LightProxy = *(ProjectedShadowInfo->GetLightSceneInfo().Proxy);
		FLightRenderParameters LightParameters;
		LightProxy.GetLightShaderParameters(LightParameters);

		PassParameters->LightDirection = LightParameters.Direction;
		PassParameters->LightTranslatedPositionAndInvRadius = FVector4f(FVector3f(LightParameters.WorldPosition + View.ViewMatrices.GetPreViewTranslation()), LightParameters.InvRadius);
		// Default light source radius of 0 gives poor results
		PassParameters->LightSourceRadius = LightParameters.SourceRadius == 0 ? 20 : FMath::Clamp(LightParameters.SourceRadius, .001f, 1.0f / (4 * LightParameters.InvRadius));
		PassParameters->RayStartOffsetDepthScale = LightProxy.GetRayStartOffsetDepthScale();

		const bool bHeightfield = PrimitiveType == DFPT_HeightField;
		const float MaxLightAngle = bHeightfield ? 45.0f : 5.0f;
		const float MinLightAngle = bHeightfield ? FMath::Min(GMinDirectionalLightAngleForRTHF, MaxLightAngle) : 0.001f;
		const float LightSourceAngle = FMath::Clamp(LightProxy.GetLightSourceAngle(), MinLightAngle, MaxLightAngle) * PI / 180.0f;
		PassParameters->TanLightAngleAndNormalThreshold = FVector3f(FMath::Tan(LightSourceAngle), FMath::Cos(PI / 2 + LightSourceAngle), LightProxy.GetTraceDistance());
		PassParameters->ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());
		PassParameters->ObjectBufferParameters = ObjectBufferParameters;
		PassParameters->CulledObjectBufferParameters = CulledObjectBufferParameters;
		PassParameters->LightTileIntersectionParameters = LightTileIntersectionParameters;
		PassParameters->DistanceFieldAtlasParameters = DistanceField::SetupAtlasParameters(GraphBuilder, DistanceFieldSceneData);
		PassParameters->HeightFieldAtlasParameters = HeightFieldAtlasParameters;
		PassParameters->TranslatedWorldToShadow = FMatrix44f(FTranslationMatrix(ProjectedShadowInfo->PreShadowTranslation - View.ViewMatrices.GetPreViewTranslation()) * FMatrix(ProjectedShadowInfo->TranslatedWorldToClipInnerMatrix));
		PassParameters->TwoSidedMeshDistanceBiasScale = GDFShadowTwoSidedMeshDistanceBiasScale;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

		GetDistanceFieldShadowRange(ProjectedShadowInfo, PrimitiveType, PassParameters->MinDepth, PassParameters->MaxDepth);

		PassParameters->DownsampleFactor = GetDFShadowDownsampleFactor();
		const FIntPoint OutputBufferSize = OutputTexture->Desc.Extent;
		PassParameters->InvOutputBufferSize = FVector2f(1.f / OutputBufferSize.X, 1.f / OutputBufferSize.Y);
		PassParameters->ShadowFactorsTexture = PrevOutputTexture;
		PassParameters->ShadowFactorsSampler = TStaticSamplerState<>::GetRHI();
		PassParameters->ScreenToView = FMatrix44f(View.ViewMatrices.GetScreenToClipMatrix() * View.ViewMatrices.GetClipToTranslatedWorld() * View.ViewMatrices.GetOverriddenTranslatedViewMatrix());
		


		FDistanceFieldShadowingCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FDistanceFieldShadowingCS::FCullingType>((uint32)DistanceFieldShadowingType);
		PermutationVector.Set<FDistanceFieldShadowingCS::FShadowQuality>(DFShadowQuality);
		PermutationVector.Set<FDistanceFieldShadowingCS::FPrimitiveType>(PrimitiveType);
		PermutationVector.Set<FDistanceFieldShadowingCS::FHasPreviousOutput>(bHasPrevOutput);
		extern int32 GDistanceFieldOffsetDataStructure;
		PermutationVector.Set<FDistanceFieldShadowingCS::FOffsetDataStructure>(GDistanceFieldOffsetDataStructure);
		PermutationVector.Set<FDistanceFieldShadowingCS::FCompactCulledObjects>(GDFShadowCompactCulledObjects != 0);
		PermutationVector.Set<FDistanceFieldShadowingCS::FCullingSubSampleDepth>(GDFShadowCullingSubsampleDepth);
		PermutationVector.Set<FDistanceFieldShadowingCS::FStereoRendering>(GEngine && GEngine->IsStereoscopic3D());

		PermutationVector = FDistanceFieldShadowingCS::RemapPermutation(PermutationVector);

		auto ComputeShader = View.ShaderMap->GetShader< FDistanceFieldShadowingCS >(PermutationVector);

		uint32 GroupSizeX = FMath::DivideAndRoundUp(DownsampledScissorRect.Size().X, GDistanceFieldShadowTileSizeX);
		uint32 GroupSizeY = FMath::DivideAndRoundUp(DownsampledScissorRect.Size().Y, GDistanceFieldShadowTileSizeY);
		PassParameters->NumGroups = FVector2f(GroupSizeX, GroupSizeY);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("DistanceFieldShadowing %ux%u", GroupSizeX * GDistanceFieldShadowTileSizeX, GroupSizeY * GDistanceFieldShadowTileSizeY),
			bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FIntVector(GroupSizeX, GroupSizeY, 1));
	}
}

FScreenPassTexture FProjectedShadowInfo::RenderRayTracedDistanceFieldProjection(
	FRDGBuilder& GraphBuilder,
	bool bAsyncCompute, 
	const FMinimalSceneTextures& SceneTextures,
	const FViewInfo& View,
	const FIntRect& ScissorRect)
{
	DistanceFieldShadowViewGPUData& SDFShadowViewGPUData = CachedDistanceFieldShadowViewGPUData.FindOrAdd(&View);

	FIntRect DownsampledScissorRect = GetScissorRectForDFShadows(ScissorRect);
	
	// DistanceFieldShadowingCS always outputs at rect with min = (0,0)
	DownsampledScissorRect = DownsampledScissorRect - DownsampledScissorRect.Min; // Beware - this gives different results than using '-='!

	if (SDFShadowViewGPUData.RayTracedShadowsTexture)
	{
		// Ray traced distance field shadows were already calculated, simply return previous result.
		return FScreenPassTexture(SDFShadowViewGPUData.RayTracedShadowsTexture, DownsampledScissorRect);
	}

	if (DownsampledScissorRect.Area() <= 0)
	{
		// skip calculating DF shadows
		return FScreenPassTexture();
	}

	const bool bDFShadowSupported = SupportsDistanceFieldShadows(View.GetFeatureLevel(), View.GetShaderPlatform());
	const bool bHFShadowSupported = SupportsHeightFieldShadows(View.GetFeatureLevel(), View.GetShaderPlatform());
	const FScene* Scene = (const FScene*)(View.Family->Scene);

	if (bDFShadowSupported && View.Family->EngineShowFlags.RayTracedDistanceFieldShadows)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_BeginRenderRayTracedDistanceFieldShadows);
		RDG_EVENT_SCOPE(GraphBuilder, "BeginRayTracedDistanceFieldShadow");

		if (Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0)
		{
			ensure(!Scene->DistanceFieldSceneData.HasPendingOperations());

			FDistanceFieldObjectBufferParameters ObjectBufferParameters = DistanceField::SetupObjectBufferParameters(GraphBuilder, Scene->DistanceFieldSceneData);

			if (SDFShadowViewGPUData.SDFCulledObjectBufferParameters == nullptr)
			{
				int32 NumPlanes = 0;
				const FPlane* PlaneData = NULL;
				FVector4f ShadowBoundingSphere = FVector4f::Zero();
				FVector PrePlaneTranslation = FVector::ZeroVector;

				if (bDirectionalLight)
				{
					NumPlanes = CascadeSettings.ShadowBoundsAccurate.Planes.Num();
					PlaneData = CascadeSettings.ShadowBoundsAccurate.Planes.GetData();
				}
				else if (IsWholeScenePointLightShadow())
				{
					ShadowBoundingSphere = FVector4f(FVector3f(ShadowBounds.Center + View.ViewMatrices.GetPreViewTranslation()), ShadowBounds.W);
				}
				else
				{
					NumPlanes = CasterOuterFrustum.Planes.Num();
					PlaneData = CasterOuterFrustum.Planes.GetData();
					PrePlaneTranslation = PreShadowTranslation;
				}

				const FMatrix WorldToShadowValue = FTranslationMatrix(PreShadowTranslation) * FMatrix(TranslatedWorldToClipInnerMatrix);

				SDFShadowViewGPUData.SDFCulledObjectBufferParameters = GraphBuilder.AllocObject<FDistanceFieldCulledObjectBufferParameters>();
				SDFShadowViewGPUData.SDFLightTileIntersectionParameters = GraphBuilder.AllocObject<FLightTileIntersectionParameters>();

				const bool bCullingForDirectShadowing = true;
				const bool bCullHeighfieldsNotInAtlas = false;

				CullDistanceFieldObjectsForLight(
					GraphBuilder,
					View,
					LightSceneInfo->Proxy,
					DFPT_SignedDistanceField,
					WorldToShadowValue,
					NumPlanes,
					PlaneData,
					PrePlaneTranslation,
					ShadowBoundingSphere,
					ShadowBounds.W,
					bCullingForDirectShadowing,
					bCullHeighfieldsNotInAtlas,
					ObjectBufferParameters,
					*SDFShadowViewGPUData.SDFCulledObjectBufferParameters,
					*SDFShadowViewGPUData.SDFLightTileIntersectionParameters
				);
			}

			{
				const FIntPoint BufferSize = GetBufferSizeForDFShadows(View);
				FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(BufferSize, PF_G16R16F, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource));
				Desc.Flags |= GFastVRamConfig.DistanceFieldShadows;
				SDFShadowViewGPUData.RayTracedShadowsTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracedShadows"));
			}

			RayTraceShadows(
				GraphBuilder,
				bAsyncCompute,
				SceneTextures,
				SDFShadowViewGPUData.RayTracedShadowsTexture,
				View,
				ScissorRect,
				DownsampledScissorRect,
				Scene->DistanceFieldSceneData,
				this,
				DFPT_SignedDistanceField,
				false,
				nullptr,
				ObjectBufferParameters,
				*SDFShadowViewGPUData.SDFCulledObjectBufferParameters,
				*SDFShadowViewGPUData.SDFLightTileIntersectionParameters);
		}
	}

	if (bDirectionalLight
		&& View.Family->EngineShowFlags.RayTracedDistanceFieldShadows
		&& GHeightFieldTextureAtlas.HasAtlasTexture()
		&& Scene->DistanceFieldSceneData.HeightFieldObjectBuffers
		&& Scene->DistanceFieldSceneData.HeightfieldPrimitives.Num() > 0
		&& bHFShadowSupported)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_BeginRenderRayTracedHeightFieldShadows);
		RDG_EVENT_SCOPE(GraphBuilder, "BeginRenderRayTracedHeightFieldShadows");

		check(!Scene->DistanceFieldSceneData.HasPendingHeightFieldOperations());

		FDistanceFieldObjectBufferParameters ObjectBufferParameters = DistanceField::SetupObjectBufferParameters(GraphBuilder, Scene->DistanceFieldSceneData);

		if (SDFShadowViewGPUData.HeightFieldCulledObjectBufferParameters == nullptr)
		{
			const int32 NumPlanes = CascadeSettings.ShadowBoundsAccurate.Planes.Num();
			const FPlane* PlaneData = CascadeSettings.ShadowBoundsAccurate.Planes.GetData();
			FVector PrePlaneTranslation = FVector::ZeroVector;
			const FVector4f ShadowBoundingSphere = FVector4f::Zero();
			const FMatrix WorldToShadowValue = FTranslationMatrix(PreShadowTranslation) * FMatrix(TranslatedWorldToClipInnerMatrix);

			SDFShadowViewGPUData.HeightFieldCulledObjectBufferParameters = GraphBuilder.AllocObject<FDistanceFieldCulledObjectBufferParameters>();
			SDFShadowViewGPUData.HeightFieldLightTileIntersectionParameters = GraphBuilder.AllocObject<FLightTileIntersectionParameters>();

			const bool bCullingForDirectShadowing = true;
			const bool bCullHeighfieldsNotInAtlas = true;

			CullDistanceFieldObjectsForLight(
				GraphBuilder,
				View,
				LightSceneInfo->Proxy,
				DFPT_HeightField,
				WorldToShadowValue,
				NumPlanes,
				PlaneData,
				PrePlaneTranslation,
				ShadowBoundingSphere,
				ShadowBounds.W,
				bCullingForDirectShadowing,
				bCullHeighfieldsNotInAtlas,
				ObjectBufferParameters,
				*SDFShadowViewGPUData.HeightFieldCulledObjectBufferParameters,
				*SDFShadowViewGPUData.HeightFieldLightTileIntersectionParameters
			);

		}

		const bool bHasPrevOutput = !!SDFShadowViewGPUData.RayTracedShadowsTexture;

		FRDGTextureRef PrevOutputTexture = nullptr;

		if (!RHISupports4ComponentUAVReadWrite(View.GetShaderPlatform()))
		{
			PrevOutputTexture = SDFShadowViewGPUData.RayTracedShadowsTexture;
			SDFShadowViewGPUData.RayTracedShadowsTexture = nullptr;
		}

		if (!SDFShadowViewGPUData.RayTracedShadowsTexture)
		{
			const FIntPoint BufferSize = GetBufferSizeForDFShadows(View);
			FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(BufferSize, PF_G16R16F, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource));
			Desc.Flags |= GFastVRamConfig.DistanceFieldShadows;
			SDFShadowViewGPUData.RayTracedShadowsTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracedShadows"));
		}

		RayTraceShadows(
			GraphBuilder,
			bAsyncCompute,
			SceneTextures,
			SDFShadowViewGPUData.RayTracedShadowsTexture,
			View,
			ScissorRect,
			DownsampledScissorRect,
			Scene->DistanceFieldSceneData,
			this,
			DFPT_HeightField,
			bHasPrevOutput,
			PrevOutputTexture,
			ObjectBufferParameters,
			*SDFShadowViewGPUData.HeightFieldCulledObjectBufferParameters,
			*SDFShadowViewGPUData.HeightFieldLightTileIntersectionParameters);
	}

	return FScreenPassTexture(SDFShadowViewGPUData.RayTracedShadowsTexture, DownsampledScissorRect);
}

BEGIN_SHADER_PARAMETER_STRUCT(FDistanceFieldShadowingUpsample, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldShadowingUpsamplePS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FShadowTileVS::FParameters, VS)
	RDG_BUFFER_ACCESS(IndirectDrawParameter, ERHIAccess::IndirectArgs)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FProjectedShadowInfo::RenderRayTracedDistanceFieldProjection(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ScreenShadowMaskTexture,
	const FViewInfo& View,
	const int32 ViewIndex,
	FIntRect ScissorRect,
	bool bProjectingForForwardShading,
	bool bForceRGBModulation,
	FTiledShadowRendering* TiledShadowRendering)
{
	check(ScissorRect.Area() > 0);
	const bool bRunTiled = UseShadowIndirectDraw(View.GetShaderPlatform()) && TiledShadowRendering != nullptr;

	FScreenPassTexture RayTracedShadowsTexture = RenderRayTracedDistanceFieldProjection(GraphBuilder, false, SceneTextures, View, ScissorRect);

	if (RayTracedShadowsTexture.IsValid())
	{
		const FVector2f RayTracedShadowsTextureExtent(RayTracedShadowsTexture.Texture->Desc.Extent);
		const FVector2f RayTracedShadowsTextureExtentInverse(1.0f / RayTracedShadowsTextureExtent.X, 1.0f / RayTracedShadowsTextureExtent.Y);
		const FVector2f RayTracedShadowsTextureViewportMax(RayTracedShadowsTexture.ViewRect.Max);

		FDistanceFieldShadowingUpsample* PassParameters = GraphBuilder.AllocParameters<FDistanceFieldShadowingUpsample>();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenShadowMaskTexture, GetLoadActionIfProduced(ScreenShadowMaskTexture, ERenderTargetLoadAction::EClear));
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
		
		PassParameters->PS.View = GetShaderBinding(View.ViewUniformBuffer);
		PassParameters->PS.SceneTextures = SceneTextures.GetSceneTextureShaderParameters(View.GetFeatureLevel());
		PassParameters->PS.ShadowFactorsTexture = RayTracedShadowsTexture.Texture;
		PassParameters->PS.ShadowFactorsSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->PS.ScissorRectMinAndSize = FIntRect(ScissorRect.Min, ScissorRect.Size());
		PassParameters->PS.ShadowFactorsUVBilinearMax = (RayTracedShadowsTextureViewportMax - 0.5f) * RayTracedShadowsTextureExtentInverse;
		PassParameters->PS.OneOverDownsampleFactor = 1.0f / GetDFShadowDownsampleFactor();
		PassParameters->PS.ShadowFadeFraction = FadeAlphas[ViewIndex];

		GetDistanceFieldShadowRange(this, DFPT_SignedDistanceField, PassParameters->PS.MinDepth, PassParameters->PS.MaxDepth);

		if (bDirectionalLight && CascadeSettings.FadePlaneLength > 0)
		{
			PassParameters->PS.FadePlaneOffset = CascadeSettings.FadePlaneOffset;
			PassParameters->PS.InvFadePlaneLength = 1.0f / FMath::Max(CascadeSettings.FadePlaneLength, .00001f);
		}
		else
		{
			PassParameters->PS.FadePlaneOffset = 0.0f;
			PassParameters->PS.InvFadePlaneLength = 0.0f;
		}

		if (bDirectionalLight && CascadeSettings.SplitNearFadeRegion > 0)
		{
			PassParameters->PS.NearFadePlaneOffset = CascadeSettings.SplitNear - CascadeSettings.SplitNearFadeRegion;
			PassParameters->PS.InvNearFadePlaneLength = 1.0f / FMath::Max(CascadeSettings.SplitNearFadeRegion, .00001f);
		}
		else
		{
			PassParameters->PS.NearFadePlaneOffset = -1.0f;
			PassParameters->PS.InvNearFadePlaneLength = 1.0f;
		}

		if (bRunTiled)
		{
			PassParameters->IndirectDrawParameter = TiledShadowRendering->DrawIndirectParametersBuffer;
			PassParameters->VS.ViewUniformBuffer = GetShaderBinding(View.ViewUniformBuffer);
			PassParameters->VS.TileListData = TiledShadowRendering->TileListDataBufferSRV;
		}

		FDistanceFieldShadowingUpsamplePS::FPermutationDomain PermutationVector;
		PermutationVector.Set< FDistanceFieldShadowingUpsamplePS::FUpsample >(GetDFShadowDownsampleFactor() != 1);
		auto PixelShader = View.ShaderMap->GetShader< FDistanceFieldShadowingUpsamplePS >(PermutationVector);

		const bool bReverseCulling = View.bReverseCulling;

		ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

		if (bRunTiled)
		{
			check(TiledShadowRendering->TileSize == FShadowTileVS::GetTileSize());

			FShadowTileVS::FPermutationDomain VSPermutationVector;
			VSPermutationVector.Set<FShadowTileVS::FTileType>(TiledShadowRendering->TileType == FTiledShadowRendering::ETileType::Tile16bits ? 0 : 1);
			auto VertexShader = View.ShaderMap->GetShader<FShadowTileVS>(VSPermutationVector);
			ClearUnusedGraphResources(VertexShader, &PassParameters->VS);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("TiledUpsample"),
				PassParameters,
				ERDGPassFlags::Raster,
				[this, &View, VertexShader, PixelShader, ScissorRect, bProjectingForForwardShading, PassParameters, bForceRGBModulation](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(ScissorRect.Min.X, ScissorRect.Min.Y, 0.0f, ScissorRect.Max.X, ScissorRect.Max.Y, 1.0f);
				RHICmdList.SetScissorRect(true, ScissorRect.Min.X, ScissorRect.Min.Y, ScissorRect.Max.X, ScissorRect.Max.Y);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

				if (bForceRGBModulation)
				{
					// This has the shadow contribution modulate all the channels, e.g. used for water rendering to apply distance field shadow on the main light RGB luminance for the updated depth buffer with water in it.
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_SourceColor, BO_Add, BF_Zero, BF_One>::GetRHI();;
				}
				else
				{
					GraphicsPSOInit.BlendState = GetBlendStateForProjection(bProjectingForForwardShading, false);
				}
				GraphicsPSOInit.bDepthBounds = bDirectionalLight;

				GraphicsPSOInit.PrimitiveType = GRHISupportsRectTopology ? PT_RectList : PT_TriangleList;
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);

				//@todo - depth bounds test for local lights
				if (bDirectionalLight)
				{
					SetDepthBoundsTest(RHICmdList, CascadeSettings.SplitNear - CascadeSettings.SplitNearFadeRegion, CascadeSettings.SplitFar, View.ViewMatrices.GetViewToClip());
				}

				PassParameters->IndirectDrawParameter->MarkResourceAsUsed();
				RHICmdList.DrawPrimitiveIndirect(PassParameters->IndirectDrawParameter->GetIndirectRHICallBuffer(), 0);

				RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
			});
		}
		else
		{
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("Upsample"),
				PassParameters,
				ERDGPassFlags::Raster,
				[this, &View, PixelShader, ScissorRect, bProjectingForForwardShading, PassParameters, bForceRGBModulation](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(ScissorRect.Min.X, ScissorRect.Min.Y, 0.0f, ScissorRect.Max.X, ScissorRect.Max.Y, 1.0f);
				RHICmdList.SetScissorRect(true, ScissorRect.Min.X, ScissorRect.Min.Y, ScissorRect.Max.X, ScissorRect.Max.Y);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, View.ShaderMap, PixelShader, GraphicsPSOInit);

				if (bForceRGBModulation)
				{
					// This has the shadow contribution modulate all the channels, e.g. used for water rendering to apply distance field shadow on the main light RGB luminance for the updated depth buffer with water in it.
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_SourceColor, BO_Add, BF_Zero, BF_One>::GetRHI();;
				}
				else
				{
					GraphicsPSOInit.BlendState = GetBlendStateForProjection(bProjectingForForwardShading, false);
				}
				GraphicsPSOInit.bDepthBounds = bDirectionalLight;

				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

				//@todo - depth bounds test for local lights
				if (bDirectionalLight)
				{
					SetDepthBoundsTest(RHICmdList, CascadeSettings.SplitNear - CascadeSettings.SplitNearFadeRegion, CascadeSettings.SplitFar, View.ViewMatrices.GetViewToClip());
				}

				FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList);
				RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
			});
		}
	}
}
