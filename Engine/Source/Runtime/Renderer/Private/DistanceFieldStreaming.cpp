// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldStreaming.cpp
=============================================================================*/

#include "CoreMinimal.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "IO/IoDispatcher.h"
#include "Async/ParallelFor.h"
#include "RHI.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "SceneUtils.h"
#include "GlobalShader.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "DistanceFieldAtlas.h"
#include "DistanceFieldLightingShared.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "GlobalDistanceField.h"
#include "ProfilingDebugging/IoStoreTrace.h"

CSV_DEFINE_CATEGORY(DistanceField, true);

extern int32 GDFReverseAtlasAllocationOrder;

static TAutoConsoleVariable<int32> CVarBrickAtlasSizeXYInBricks(
	TEXT("r.DistanceFields.BrickAtlasSizeXYInBricks"),
	128,	
	TEXT("Controls the allocation granularity of the atlas, which grows in Z."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMaxAtlasDepthInBricks(
	TEXT("r.DistanceFields.BrickAtlasMaxSizeZ"),
	32,	
	TEXT("Target for maximum depth of the Mesh Distance Field atlas, in 8^3 bricks.  32 => 128 * 128 * 32 * 8^3 = 256Mb.  Actual atlas size can go over since mip2 is always loaded."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarTextureUploadLimitKBytes(
	TEXT("r.DistanceFields.TextureUploadLimitKBytes"),
	8192,	
	TEXT("Max KB of distance field texture data to upload per frame from streaming requests."),
	ECVF_RenderThreadSafe);

int32 GDistanceFieldOffsetDataStructure = 0;
static FAutoConsoleVariableRef CVarShadowOffsetDataStructure(
	TEXT("r.DistanceFields.OffsetDataStructure"),
	GDistanceFieldOffsetDataStructure,
	TEXT("Which data structure to store offset in, 0 - base, 1 - buffer, 2 - texture"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static int32 MinIndirectionAtlasSizeXYZ = 64;
static FAutoConsoleVariableRef CVarMinIndirectionAtlasSizeXYZ(
	TEXT("r.DistanceFields.MinIndirectionAtlasSizeXYZ"),
	MinIndirectionAtlasSizeXYZ,
	TEXT("Minimum size of indirection atlas texture"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMaxIndirectionAtlasSizeXYZ(
	TEXT("r.DistanceFields.MaxIndirectionAtlasSizeXYZ"),
	512,
	TEXT("Maximum size of indirection atlas texture"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarDefragmentIndirectionAtlas(
	TEXT("r.DistanceFields.DefragmentIndirectionAtlas"),
	1,
	TEXT("Whether to defragment the Distance Field indirection atlas when it requires resizing."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarResizeAtlasEveryFrame(
	TEXT("r.DistanceFields.Debug.ResizeAtlasEveryFrame"),
	0,	
	TEXT("Whether to resize the Distance Field atlas every frame, which is useful for debugging."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDebugForceNumMips(
	TEXT("r.DistanceFields.Debug.ForceNumMips"),
	0,	
	TEXT("When set to > 0, overrides the requested number of mips for streaming.  1 = only lowest resolution mip loaded, 3 = all mips loaded.  Mips will still be clamped by available space in the atlas."),
	ECVF_RenderThreadSafe);

static int32 GDistanceFieldAtlasLogStats = 0;
static FAutoConsoleVariableRef CVarDistanceFieldAtlasLogStats(
	TEXT("r.DistanceFields.LogAtlasStats"),
	GDistanceFieldAtlasLogStats,
	TEXT("Set to 1 to dump atlas stats, set to 2 to dump atlas and SDF asset stats."),
	ECVF_RenderThreadSafe
	);

static int32 GDistanceFieldBlockAllocatorSizeInBricks = 16;
static FAutoConsoleVariableRef CVarDistanceFieldBlockAllocatorSizeInBricks(
	TEXT("r.DistanceFields.BlockAllocatorSizeInBricks"),
	GDistanceFieldBlockAllocatorSizeInBricks,
	TEXT("Allocation granularity of the distance field block allocator. Higher number may cause more memory wasted on padding but allocation may be faster."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static const int32 MaxStreamingRequests = 4095;
static const float IndirectionAtlasGrowMult = 2.0f;

class FCopyDistanceFieldAtlasCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyDistanceFieldAtlasCS)
	SHADER_USE_PARAMETER_STRUCT(FCopyDistanceFieldAtlasCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<UNORM float>, RWDistanceFieldBrickAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 4;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FCopyDistanceFieldAtlasCS, "/Engine/Private/DistanceFieldStreaming.usf", "CopyDistanceFieldAtlasCS", SF_Compute);

class FScatterUploadDistanceFieldAtlasCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScatterUploadDistanceFieldAtlasCS)
	SHADER_USE_PARAMETER_STRUCT(FScatterUploadDistanceFieldAtlasCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<UNORM float>, RWDistanceFieldBrickAtlas)
		SHADER_PARAMETER_SRV(Buffer<uint3>, BrickUploadCoordinates)
		SHADER_PARAMETER_SRV(Buffer<float>, BrickUploadData)
		SHADER_PARAMETER(uint32, StartBrickIndex)
		SHADER_PARAMETER(uint32, NumBrickUploads)
		SHADER_PARAMETER(uint32, BrickSize)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 4;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FScatterUploadDistanceFieldAtlasCS, "/Engine/Private/DistanceFieldStreaming.usf", "ScatterUploadDistanceFieldAtlasCS", SF_Compute);

class FScatterUploadDistanceFieldIndirectionAtlasCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScatterUploadDistanceFieldIndirectionAtlasCS)
	SHADER_USE_PARAMETER_STRUCT(FScatterUploadDistanceFieldIndirectionAtlasCS, FGlobalShader)
		
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWIndirectionAtlas)
		SHADER_PARAMETER_SRV(Buffer<uint>, IndirectionUploadIndices)
		SHADER_PARAMETER_SRV(Buffer<float4>, IndirectionUploadData)
		SHADER_PARAMETER(FIntVector, IndirectionAtlasSize)
		SHADER_PARAMETER(uint32, NumIndirectionUploads)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}
	
	static int32 GetGroupSize()
	{
		return 64;
	}
	
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FScatterUploadDistanceFieldIndirectionAtlasCS, "/Engine/Private/DistanceFieldStreaming.usf", "ScatterUploadDistanceFieldIndirectionAtlasCS", SF_Compute);

class FCopyDistanceFieldIndirectionAtlasCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyDistanceFieldIndirectionAtlasCS)
	SHADER_USE_PARAMETER_STRUCT(FCopyDistanceFieldIndirectionAtlasCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWIndirectionAtlas)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float4>, DistanceFieldIndirectionAtlas)
		SHADER_PARAMETER(FIntVector, IndirectionDimensions)
		SHADER_PARAMETER(FIntVector, SrcPosition)
		SHADER_PARAMETER(FIntVector, DstPosition)
		SHADER_PARAMETER(uint32, NumAssets)
	END_SHADER_PARAMETER_STRUCT()
	
	using FPermutationDomain = TShaderPermutationDomain<>;
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 64;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FCopyDistanceFieldIndirectionAtlasCS, "/Engine/Private/DistanceFieldStreaming.usf", "CopyDistanceFieldIndirectionAtlasCS", SF_Compute);

class FComputeDistanceFieldAssetWantedMipsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComputeDistanceFieldAssetWantedMipsCS)
	SHADER_USE_PARAMETER_STRUCT(FComputeDistanceFieldAssetWantedMipsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWDistanceFieldAssetWantedNumMips)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWDistanceFieldAssetStreamingRequests)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER(int32, DebugForceNumMips)
		SHADER_PARAMETER(FVector3f, Mip1WorldTranslatedCenter)
		SHADER_PARAMETER(FVector3f, Mip1WorldExtent)
		SHADER_PARAMETER(FVector3f, Mip2WorldTranslatedCenter)
		SHADER_PARAMETER(FVector3f, Mip2WorldExtent)
		SHADER_PARAMETER(FVector3f, PreViewTranslationHigh)
		SHADER_PARAMETER(FVector3f, PreViewTranslationLow)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 64;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FComputeDistanceFieldAssetWantedMipsCS, "/Engine/Private/DistanceFieldStreaming.usf", "ComputeDistanceFieldAssetWantedMipsCS", SF_Compute);


class FGenerateDistanceFieldAssetStreamingRequestsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGenerateDistanceFieldAssetStreamingRequestsCS)
	SHADER_USE_PARAMETER_STRUCT(FGenerateDistanceFieldAssetStreamingRequestsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWDistanceFieldAssetStreamingRequests)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, DistanceFieldAssetWantedNumMips)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlasParameters)
		SHADER_PARAMETER(uint32, NumDistanceFieldAssets)
		SHADER_PARAMETER(uint32, MaxNumStreamingRequests)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 64;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FGenerateDistanceFieldAssetStreamingRequestsCS, "/Engine/Private/DistanceFieldStreaming.usf", "GenerateDistanceFieldAssetStreamingRequestsCS", SF_Compute);

const int32 AssetDataMipStrideFloat4s = 3;

FIntVector GetBrickCoordinate(int32 BrickIndex, FIntVector BrickAtlasSize)
{
	return FIntVector(
		BrickIndex % BrickAtlasSize.X,
		(BrickIndex / BrickAtlasSize.X) % BrickAtlasSize.Y,
		BrickIndex / (BrickAtlasSize.X * BrickAtlasSize.Y));
}

class FDistanceFieldAtlasUpload
{
public:
	FReadBuffer& BrickUploadCoordinatesBuffer;
	FReadBuffer& BrickUploadDataBuffer;

	FIntVector4* BrickUploadCoordinatesPtr = nullptr;
	uint8* BrickUploadDataPtr = nullptr;

	uint32 NumBrickUploadCoordinatesBytes = 0;
	uint32 NumBrickUploadDataBytes = 0;

	FDistanceFieldAtlasUpload(
		FReadBuffer& InBrickUploadCoordinatesBuffer,
		FReadBuffer& InBrickUploadDataBuffer) :
		BrickUploadCoordinatesBuffer(InBrickUploadCoordinatesBuffer),
		BrickUploadDataBuffer(InBrickUploadDataBuffer)
	{}

	void Allocate(FRHICommandListBase& RHICmdList, uint32 NumBrickUploads, uint32 BrickSize)
	{
		const uint32 NumCoordElements = FMath::RoundUpToPowerOfTwo(NumBrickUploads);
		const uint32 CoordNumBytesPerElement = GPixelFormats[PF_R32G32B32A32_UINT].BlockBytes;

		NumBrickUploadCoordinatesBytes = NumCoordElements * CoordNumBytesPerElement;

		if (BrickUploadCoordinatesBuffer.NumBytes < NumBrickUploadCoordinatesBytes)
		{
			BrickUploadCoordinatesBuffer.Initialize(RHICmdList, TEXT("DistanceFields.BrickUploadCoordinatesBuffer"), CoordNumBytesPerElement, NumCoordElements, PF_R32G32B32A32_UINT, BUF_Volatile);
		}

		const uint32 NumBrickDataElements = FMath::RoundUpToPowerOfTwo(NumBrickUploads) * BrickSize * BrickSize * BrickSize;
		const uint32 BrickDataNumBytesPerElement = GPixelFormats[DistanceField::DistanceFieldFormat].BlockBytes;

		NumBrickUploadDataBytes = NumBrickDataElements * BrickDataNumBytesPerElement;

		if (BrickUploadDataBuffer.NumBytes < NumBrickUploadDataBytes || BrickUploadDataBuffer.NumBytes > 2 * NumBrickUploadDataBytes)
		{
			BrickUploadDataBuffer.Initialize(RHICmdList, TEXT("DistanceFields.BrickUploadDataBuffer"), BrickDataNumBytesPerElement, NumBrickDataElements, DistanceField::DistanceFieldFormat, BUF_Volatile);
		}
	}

	void Lock(FRHICommandListBase& RHICmdList)
	{
		if (NumBrickUploadCoordinatesBytes != 0)
		{
			BrickUploadCoordinatesPtr = (FIntVector4*)RHICmdList.LockBuffer(BrickUploadCoordinatesBuffer.Buffer, 0, NumBrickUploadCoordinatesBytes, RLM_WriteOnly);
		}

		if (NumBrickUploadDataBytes != 0)
		{
			BrickUploadDataPtr = (uint8*)RHICmdList.LockBuffer(BrickUploadDataBuffer.Buffer, 0, NumBrickUploadDataBytes, RLM_WriteOnly);
		}
	}

	void Unlock(FRHICommandListBase& RHICmdList)
	{
		if (BrickUploadCoordinatesPtr)
		{
			RHICmdList.UnlockBuffer(BrickUploadCoordinatesBuffer.Buffer);
		}

		if (BrickUploadDataPtr)
		{
			RHICmdList.UnlockBuffer(BrickUploadDataBuffer.Buffer);
		}
	}
};

class FDistanceFieldIndirectionAtlasUpload
{
public:
	FReadBuffer& IndirectionUploadIndicesBuffer;
	FReadBuffer& IndirectionUploadDataBuffer;

	uint32* IndirectionUploadIndicesPtr = nullptr;
	FVector4f* IndirectionUploadDataPtr = nullptr;

	uint32 NumIndirectionUploadIndicesBytes = 0;
	uint32 NumIndirectionUploadDataBytes = 0;

	FDistanceFieldIndirectionAtlasUpload(
		FReadBuffer& InIndirectionUploadIndicesBuffer,
		FReadBuffer& InIndirectionUploadDataBuffer) :
		IndirectionUploadIndicesBuffer(InIndirectionUploadIndicesBuffer),
		IndirectionUploadDataBuffer(InIndirectionUploadDataBuffer)
	{}

	void Allocate(FRHICommandListBase& RHICmdList, uint32 NumIndirectionUploads)
	{
		const uint32 NumElements = FMath::RoundUpToPowerOfTwo(NumIndirectionUploads);

		NumIndirectionUploadIndicesBytes = NumElements * sizeof(uint32);

		if (IndirectionUploadIndicesBuffer.NumBytes < NumIndirectionUploadIndicesBytes)
		{
			IndirectionUploadIndicesBuffer.Initialize(RHICmdList, TEXT("DistanceFields.IndirectionUploadIndicesBuffer"), sizeof(uint32), NumElements, PF_R32_UINT, BUF_Volatile);
		}

		NumIndirectionUploadDataBytes = NumElements * sizeof(FVector4f);

		if (IndirectionUploadDataBuffer.NumBytes < NumIndirectionUploadDataBytes)
		{
			IndirectionUploadDataBuffer.Initialize(RHICmdList, TEXT("DistanceFields.IndirectionUploadDataBuffer"), sizeof(FVector4f), NumElements, PF_A32B32G32R32F, BUF_Volatile);
		}
	}

	void Lock(FRHICommandListBase& RHICmdList)
	{
		if (NumIndirectionUploadIndicesBytes != 0)
		{
			IndirectionUploadIndicesPtr = (uint32*)RHICmdList.LockBuffer(IndirectionUploadIndicesBuffer.Buffer, 0, NumIndirectionUploadIndicesBytes, RLM_WriteOnly);
		}

		if (NumIndirectionUploadDataBytes != 0)
		{
			IndirectionUploadDataPtr = (FVector4f*)RHICmdList.LockBuffer(IndirectionUploadDataBuffer.Buffer, 0, NumIndirectionUploadDataBytes, RLM_WriteOnly);
		}
	}

	void Unlock(FRHICommandListBase& RHICmdList) const
	{
		if (IndirectionUploadIndicesPtr)
		{
			RHICmdList.UnlockBuffer(IndirectionUploadIndicesBuffer.Buffer);
		}

		if (IndirectionUploadDataPtr)
		{
			RHICmdList.UnlockBuffer(IndirectionUploadDataBuffer.Buffer);
		}
	}
};

void FDistanceFieldBlockAllocator::Allocate(int32 NumBlocks, TArray<int32, TInlineAllocator<4>>& OutBlocks)
{
	OutBlocks.Empty(NumBlocks);
	OutBlocks.AddUninitialized(NumBlocks);

	const int32 NumFree = FMath::Min(NumBlocks, FreeBlocks.Num());

	if (NumFree > 0)
	{
		for (int32 i = 0; i < NumFree; i++)
		{
			OutBlocks[i] = FreeBlocks[FreeBlocks.Num() - i - 1];
		}

		FreeBlocks.RemoveAt(FreeBlocks.Num() - NumFree, NumFree, EAllowShrinking::No);
	}
		
	const int32 NumRemaining = NumBlocks - NumFree;

	for (int32 i = 0; i < NumRemaining; i++)
	{
		OutBlocks[i + NumFree] = MaxNumBlocks + i;
	}
	MaxNumBlocks += NumRemaining;
}

void FDistanceFieldBlockAllocator::Free(const TArray<int32, TInlineAllocator<4>>& ElementRange)
{
	FreeBlocks.Append(ElementRange);
}

struct FDistanceFieldReadRequest
{
	// SDF scene context
	FSetElementId AssetSetId;
	int32 ReversedMipIndex = 0;
	int32 NumDistanceFieldBricks = 0;
	uint64 BuiltDataId = 0;

	// Used when BulkData is nullptr
	const uint8* AlwaysLoadedDataPtr = nullptr;

	// Inputs of read request
	const FByteBulkData* BulkData = nullptr;
	uint32 BulkOffset = 0;
	uint32 BulkSize = 0;

	// Outputs of read request
	uint8* ReadOutputDataPtr = nullptr;
	FIoRequest Request;
	IAsyncReadFileHandle* AsyncHandle = nullptr;
	IAsyncReadRequest* AsyncRequest = nullptr;
};

struct FDistanceFieldAsyncUpdateParameters
{
	FDistanceFieldAsyncUpdateParameters(
		FReadBuffer& InBrickUploadCoordinatesBuffer,
		FReadBuffer& InBrickUploadDataBuffer,
		FReadBuffer& InIndirectionUploadIndicesBuffer,
		FReadBuffer& InIndirectionUploadDataBuffer)
		: AtlasUpload(InBrickUploadCoordinatesBuffer, InBrickUploadDataBuffer)
		, IndirectionAtlasUpload(InIndirectionUploadIndicesBuffer, InIndirectionUploadDataBuffer)
	{}

	void Lock(FRHICommandListBase& RHICmdList)
	{
		AtlasUpload.Lock(RHICmdList);
		IndirectionAtlasUpload.Lock(RHICmdList);

		if (IndirectionTableUploader)
		{
			IndirectionTableUploader->Lock(RHICmdList);
		}
	}

	void Unlock(FRHICommandListBase& RHICmdList)
	{
		AtlasUpload.Unlock(RHICmdList);
		IndirectionAtlasUpload.Unlock(RHICmdList);

		if (IndirectionTableUploader)
		{
			IndirectionTableUploader->Unlock(RHICmdList);
		}
	}

	FDistanceFieldSceneData* DistanceFieldSceneData = nullptr;

	FDistanceFieldAtlasUpload AtlasUpload;
	FDistanceFieldIndirectionAtlasUpload IndirectionAtlasUpload;

	FRDGScatterUploader* IndirectionTableUploader = nullptr;

	TArray<FDistanceFieldReadRequest> NewReadRequests;
	TArray<FDistanceFieldReadRequest> ReadRequestsToUpload;
	TArray<FDistanceFieldReadRequest> ReadRequestsToCleanUp;
};

FDistanceFieldSceneData::FDistanceFieldSceneData(FDistanceFieldSceneData&&) = default;

FDistanceFieldSceneData::FDistanceFieldSceneData(EShaderPlatform ShaderPlatform)
	: NumObjectsInBuffer(0)
	, IndirectionAtlasLayout(8, 8, 8, 512, 512, 512, false, true, false)
	, HeightFieldAtlasGeneration(0)
	, HFVisibilityAtlasGenerattion(0)
{
	ObjectBuffers = nullptr;
	HeightFieldObjectBuffers = nullptr;

	bTrackAllPrimitives = ShouldAllPrimitivesHaveDistanceField(ShaderPlatform);

	bCanUse16BitObjectIndices = RHISupportsBufferLoadTypeConversion(ShaderPlatform);

	StreamingRequestReadbackBuffers.AddZeroed(MaxStreamingReadbackBuffers);
}

FDistanceFieldSceneData::~FDistanceFieldSceneData()
{
	delete ObjectBuffers;
	delete HeightFieldObjectBuffers;
}

void FDistanceFieldSceneData::AsyncUpdate(FRHICommandListBase& RHICmdList, FDistanceFieldAsyncUpdateParameters& UpdateParameters)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FDistanceFieldSceneData_AsyncUpdate);
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldSceneData::AsyncUpdate);

	const uint32 BrickSizeBytes = GPixelFormats[DistanceField::DistanceFieldFormat].BlockBytes * DistanceField::BrickSize * DistanceField::BrickSize * DistanceField::BrickSize;
	const FIntVector IndirectionTextureSize = IndirectionAtlas ? IndirectionAtlas->GetDesc().GetSize() : FIntVector::ZeroValue;
	const float InvMaxIndirectionDimensionMinusOne = 1.f / (DistanceField::MaxIndirectionDimension - 1);

	int32 BrickUploadIndex = 0;
	int32 IndirectionUploadIndex = 0;

	UpdateParameters.Lock(RHICmdList);

	for (FDistanceFieldReadRequest ReadRequest : UpdateParameters.ReadRequestsToUpload)
	{
		const FDistanceFieldAssetState& AssetState = AssetStateArray[ReadRequest.AssetSetId];
		const int32 ReversedMipIndex = ReadRequest.ReversedMipIndex;
		const FDistanceFieldAssetMipState& MipState = AssetState.ReversedMips[ReversedMipIndex];
		const int32 MipIndex = AssetState.BuiltData->Mips.Num() - ReversedMipIndex - 1;
		const FSparseDistanceFieldMip& MipBuiltData = AssetState.BuiltData->Mips[MipIndex];

		const uint8* BulkDataReadPtr = ReadRequest.BulkData ? ReadRequest.ReadOutputDataPtr : ReadRequest.AlwaysLoadedDataPtr;

#if WITH_EDITOR
		if (ReadRequest.BulkData)
		{	
			check((ReadRequest.BulkData->IsBulkDataLoaded() ||ReadRequest.BulkData->CanLoadFromDisk()) && ReadRequest.BulkData->GetBulkDataSize() > 0);
			BulkDataReadPtr = (const uint8*)ReadRequest.BulkData->LockReadOnly() + ReadRequest.BulkOffset;
		}
#endif
		const int32 NumIndirectionEntries = MipBuiltData.IndirectionDimensions.X * MipBuiltData.IndirectionDimensions.Y * MipBuiltData.IndirectionDimensions.Z;
		const uint32 ExpectedBulkSize = NumIndirectionEntries * sizeof(uint32) + ReadRequest.NumDistanceFieldBricks * BrickSizeBytes;

		check(ReadRequest.BuiltDataId == AssetState.BuiltData->GetId());
		checkf(ReadRequest.BulkSize == ExpectedBulkSize, 
			TEXT("Bulk size mismatch: BulkSize %u, ExpectedSize %u, NumIndirectionEntries %u, NumBricks %u, ReversedMip %u"),
			ReadRequest.BulkSize,
			ExpectedBulkSize,
			NumIndirectionEntries,
			ReadRequest.NumDistanceFieldBricks,
			ReversedMipIndex);

		const uint32* SourceIndirectionTable = (const uint32*)BulkDataReadPtr;
		const int32* RESTRICT GlobalBlockOffsets = MipState.AllocatedBlocks.GetData();

		uint32* DestIndirectionTable = nullptr;
		FVector4f* DestIndirection2Table = nullptr;
		if (GDistanceFieldOffsetDataStructure == 0)
		{
			DestIndirectionTable = (uint32*)UpdateParameters.IndirectionTableUploader->Add_GetRef(MipState.IndirectionTableOffset, NumIndirectionEntries);
		}
		else if (GDistanceFieldOffsetDataStructure == 1)
		{
			DestIndirection2Table = (FVector4f*)UpdateParameters.IndirectionTableUploader->Add_GetRef(MipState.IndirectionTableOffset, NumIndirectionEntries);
		}

		// Add global allocated brick offset to indirection table entries as we upload them
		for (int32 i = 0; i < NumIndirectionEntries; i++)
		{
			const uint32 BrickIndex = SourceIndirectionTable[i];
			uint32 GlobalBrickIndex = DistanceField::InvalidBrickIndex;
			FVector4f BrickOffset(0, 0, 0, 0);

			if (BrickIndex != DistanceField::InvalidBrickIndex)
			{
				const int32 BlockIndex = BrickIndex / GDistanceFieldBlockAllocatorSizeInBricks;

				if (BlockIndex < MipState.AllocatedBlocks.Num())
				{
					GlobalBrickIndex = BrickIndex % GDistanceFieldBlockAllocatorSizeInBricks + GlobalBlockOffsets[BlockIndex] * GDistanceFieldBlockAllocatorSizeInBricks;

					const FIntVector BrickTextureCoordinate = GetBrickCoordinate(GlobalBrickIndex, BrickTextureDimensionsInBricks);
					BrickOffset.X = BrickTextureCoordinate.X * InvMaxIndirectionDimensionMinusOne;
					BrickOffset.Y = BrickTextureCoordinate.Y * InvMaxIndirectionDimensionMinusOne;
					BrickOffset.Z = BrickTextureCoordinate.Z * InvMaxIndirectionDimensionMinusOne;
					BrickOffset.W = 1.0f;
				}
			}
			
			if (GDistanceFieldOffsetDataStructure == 0)
			{
				DestIndirectionTable[i] = GlobalBrickIndex;
			}
			else if (GDistanceFieldOffsetDataStructure == 1)
			{
				// This null check isn't really needed but to make static analysis happy
				if (DestIndirection2Table)
				{
					DestIndirection2Table[i] = BrickOffset;
				}
			}
			else if (GDistanceFieldOffsetDataStructure == 2)
			{
				const FIntVector IndirectionCoord = FIntVector(
					i % MipState.IndirectionDimensions.X,
					i / MipState.IndirectionDimensions.X % MipState.IndirectionDimensions.Y,
					i / (MipState.IndirectionDimensions.X * MipState.IndirectionDimensions.Y));
				const FIntVector IndirectionAtlasPosition = MipState.IndirectionAtlasOffset + IndirectionCoord;
				const uint32 IndirectionIndex = IndirectionAtlasPosition.X + IndirectionTextureSize.X * (IndirectionAtlasPosition.Y + IndirectionTextureSize.Y * IndirectionAtlasPosition.Z);

				UpdateParameters.IndirectionAtlasUpload.IndirectionUploadIndicesPtr[IndirectionUploadIndex] = IndirectionIndex;
				UpdateParameters.IndirectionAtlasUpload.IndirectionUploadDataPtr[IndirectionUploadIndex] = BrickOffset;

				++IndirectionUploadIndex;
			}
		}

		check(MipState.NumBricks == ReadRequest.NumDistanceFieldBricks);
		const uint8* DistanceFieldBrickDataPtr = BulkDataReadPtr + NumIndirectionEntries * sizeof(uint32);
		const SIZE_T DistanceFieldBrickDataSizeBytes = ReadRequest.NumDistanceFieldBricks * BrickSizeBytes;
		FMemory::Memcpy(UpdateParameters.AtlasUpload.BrickUploadDataPtr + BrickUploadIndex * BrickSizeBytes, DistanceFieldBrickDataPtr, DistanceFieldBrickDataSizeBytes);

		for (int32 BrickIndex = 0; BrickIndex < MipState.NumBricks; BrickIndex++)
		{
			const int32 GlobalBrickIndex = BrickIndex % GDistanceFieldBlockAllocatorSizeInBricks + GlobalBlockOffsets[BrickIndex / GDistanceFieldBlockAllocatorSizeInBricks] * GDistanceFieldBlockAllocatorSizeInBricks;
			const FIntVector BrickTextureCoordinate = GetBrickCoordinate(GlobalBrickIndex, BrickTextureDimensionsInBricks);
			UpdateParameters.AtlasUpload.BrickUploadCoordinatesPtr[BrickUploadIndex + BrickIndex] = FIntVector4(BrickTextureCoordinate.X, BrickTextureCoordinate.Y, BrickTextureCoordinate.Z, 0);
		}

#if WITH_EDITOR
		if (ReadRequest.BulkData)
		{
			ReadRequest.BulkData->Unlock();
		}
#endif

		BrickUploadIndex += MipState.NumBricks;
	}

#if !WITH_EDITOR

	for (FDistanceFieldReadRequest ReadRequest : UpdateParameters.ReadRequestsToCleanUp)
	{
		if (ReadRequest.AsyncRequest)
		{
			check(ReadRequest.AsyncRequest->PollCompletion());	
			delete ReadRequest.AsyncRequest;
			delete ReadRequest.AsyncHandle;
		}
		else
		{
			check(ReadRequest.Request.Status().IsCompleted());
		}

		FMemory::Free(ReadRequest.ReadOutputDataPtr);
	}

	FIoBatch Batch;

	for (FDistanceFieldReadRequest& ReadRequest : UpdateParameters.NewReadRequests)
	{
		check(ReadRequest.BulkSize > 0);
		ReadRequest.ReadOutputDataPtr = (uint8*)FMemory::Malloc(ReadRequest.BulkSize);
		const bool bIODispatcher = ReadRequest.BulkData->IsUsingIODispatcher();

		if (bIODispatcher)
		{
			TRACE_IOSTORE_METADATA_SCOPE_TAG("DistanceField");

			// Use IODispatcher when available
			FIoChunkId ChunkID = ReadRequest.BulkData->CreateChunkId();
			FIoReadOptions ReadOptions;
			ReadOptions.SetRange(ReadRequest.BulkData->GetBulkDataOffsetInFile() + ReadRequest.BulkOffset, ReadRequest.BulkSize);
			ReadOptions.SetTargetVa(ReadRequest.ReadOutputDataPtr);
			ReadRequest.Request = Batch.Read(ChunkID, ReadOptions, IoDispatcherPriority_Low);
		}
		else
		{
			// Compatibility path without IODispatcher
			ReadRequest.AsyncHandle = ReadRequest.BulkData->OpenAsyncReadHandle();
			ReadRequest.AsyncRequest = ReadRequest.AsyncHandle->ReadRequest(ReadRequest.BulkData->GetBulkDataOffsetInFile() + ReadRequest.BulkOffset, ReadRequest.BulkSize, AIOP_Low, nullptr, ReadRequest.ReadOutputDataPtr);
		}
	}

	Batch.Issue();

#endif

	ReadRequests.Append(UpdateParameters.NewReadRequests);

	UpdateParameters.Unlock(RHICmdList);
}

bool AssetHasOutstandingRequest(FSetElementId AssetSetId, const TArray<FDistanceFieldReadRequest>& ReadRequests)
{
	for (const FDistanceFieldReadRequest& ReadRequest : ReadRequests)
	{
		if (ReadRequest.AssetSetId == AssetSetId)
		{
			return true;
		}
	}

	return false;
}

void FDistanceFieldSceneData::ProcessStreamingRequestsFromGPU(
	TArray<FDistanceFieldReadRequest>& NewReadRequests,
	TArray<FDistanceFieldAssetMipId>& AssetDataUploads)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_DistanceFieldProcessStreamingRequests);
	TRACE_CPUPROFILER_EVENT_SCOPE(DistanceFieldProcessStreamingRequests);

	FRHIGPUBufferReadback* LatestReadbackBuffer = nullptr;

	{
		// Find latest buffer that is ready
		while (ReadbackBuffersNumPending > 0)
		{
			uint32 Index = (ReadbackBuffersWriteIndex + MaxStreamingReadbackBuffers - ReadbackBuffersNumPending) % MaxStreamingReadbackBuffers;
			if (StreamingRequestReadbackBuffers[Index]->IsReady())	
			{
				ReadbackBuffersNumPending--;
				LatestReadbackBuffer = StreamingRequestReadbackBuffers[Index];
			}
			else
			{
				break;
			}
		}
	}

	const int32 BrickAtlasSizeXYInBricks = CVarBrickAtlasSizeXYInBricks.GetValueOnRenderThread();
	const int32 NumBricksBeforeDroppingMips = FMath::Max((CVarMaxAtlasDepthInBricks.GetValueOnRenderThread() - 1) * BrickAtlasSizeXYInBricks * BrickAtlasSizeXYInBricks, 0);
	int32 NumAllocatedDistanceFieldBricks = DistanceFieldAtlasBlockAllocator.GetAllocatedSize() * GDistanceFieldBlockAllocatorSizeInBricks;

	for (const FDistanceFieldReadRequest& ReadRequest : ReadRequests)
	{
		// Account for size that will be added when all async read requests complete
		NumAllocatedDistanceFieldBricks += ReadRequest.NumDistanceFieldBricks;
	}

	if (LatestReadbackBuffer)
	{
		const uint32* LatestReadbackBufferPtr = (const uint32*)LatestReadbackBuffer->Lock((MaxStreamingRequests * 2 + 1) * sizeof(uint32));

		const uint32 NumStreamingRequests = FMath::Min<uint32>(LatestReadbackBufferPtr[0], MaxStreamingRequests);

		// Process streaming requests in two passes so that mip1 requests will be allocated before mip2
		for (int32 PassIndex = 0; PassIndex < 2; PassIndex++)
		{
			const bool bFirstPass = PassIndex == 0;

			for (uint32 StreamingRequestIndex = 0; StreamingRequestIndex < NumStreamingRequests; StreamingRequestIndex++)
			{
				const int32 AssetIndex = LatestReadbackBufferPtr[1 + StreamingRequestIndex * 2 + 0];
				const FSetElementId AssetSetId = FSetElementId::FromInteger(AssetIndex);

				if (AssetStateArray.IsValidId(AssetSetId))
				{
					FDistanceFieldAssetState& AssetState = AssetStateArray[AssetSetId];

					const int32 WantedNumMips = LatestReadbackBufferPtr[1 + StreamingRequestIndex * 2 + 1];
					check(WantedNumMips <= DistanceField::NumMips && WantedNumMips <= AssetState.BuiltData->Mips.Num());
					AssetState.WantedNumMips = WantedNumMips;

					if (WantedNumMips < AssetState.ReversedMips.Num() && bFirstPass)
					{
						check(AssetState.ReversedMips.Num() > 1);
						const FDistanceFieldAssetMipState MipState = AssetState.ReversedMips.Pop();
						
						if (GDistanceFieldOffsetDataStructure == 0 || GDistanceFieldOffsetDataStructure == 1)
						{
							IndirectionTableAllocator.Free(MipState.IndirectionTableOffset, MipState.IndirectionDimensions.X * MipState.IndirectionDimensions.Y * MipState.IndirectionDimensions.Z);
						}
						else
						{
							IndirectionAtlasLayout.RemoveElement(MipState.IndirectionAtlasOffset.X, MipState.IndirectionAtlasOffset.Y, MipState.IndirectionAtlasOffset.Z,
								MipState.IndirectionDimensions.X, MipState.IndirectionDimensions.Y, MipState.IndirectionDimensions.Z);
						}
						
						if (MipState.NumBricks > 0)
						{
							check(MipState.AllocatedBlocks.Num() > 0);
							DistanceFieldAtlasBlockAllocator.Free(MipState.AllocatedBlocks);
						}

						// Re-upload mip0 to push the new NumMips to the shader
						AssetDataUploads.Add(FDistanceFieldAssetMipId(AssetSetId, 0));
					}
					else if (WantedNumMips > AssetState.ReversedMips.Num())
					{
						const int32 ReversedMipIndexToAdd = AssetState.ReversedMips.Num();
						// Don't allocate mip if we are close to the max size
						const bool bAllowedToAllocateMipBricks = NumAllocatedDistanceFieldBricks <= NumBricksBeforeDroppingMips;
						// Only allocate mip2 requests in the second pass after all mip1 requests have succeeded
						const bool bShouldProcessThisPass = ((bFirstPass && ReversedMipIndexToAdd < DistanceField::NumMips - 1) || (!bFirstPass && ReversedMipIndexToAdd == DistanceField::NumMips - 1));

						if (bAllowedToAllocateMipBricks 
							&& bShouldProcessThisPass 
							// Only allow one IO request in flight for a given asset
							&& !AssetHasOutstandingRequest(AssetSetId, ReadRequests))
						{
							const int32 MipIndexToAdd = AssetState.BuiltData->Mips.Num() - ReversedMipIndexToAdd - 1;
							const FSparseDistanceFieldMip& MipBuiltData = AssetState.BuiltData->Mips[MipIndexToAdd];

							//@todo - this condition shouldn't be possible as the built data always has non-zero size, needs more investigation
							if (MipBuiltData.BulkSize > 0)
							{
								FDistanceFieldReadRequest ReadRequest;
								ReadRequest.AssetSetId = AssetSetId;
								ReadRequest.BuiltDataId = AssetState.BuiltData->GetId();
								ReadRequest.ReversedMipIndex = ReversedMipIndexToAdd;
								ReadRequest.NumDistanceFieldBricks = MipBuiltData.NumDistanceFieldBricks;
								ReadRequest.BulkData = &AssetState.BuiltData->StreamableMips;
								ReadRequest.BulkOffset = MipBuiltData.BulkOffset;
								ReadRequest.BulkSize = MipBuiltData.BulkSize;
								check(ReadRequest.BulkSize > 0);
								NewReadRequests.Add(MoveTemp(ReadRequest));

								NumAllocatedDistanceFieldBricks += MipBuiltData.NumDistanceFieldBricks;
							}
						}
					}
				}
			}
		}
		
		LatestReadbackBuffer->Unlock();
	}
}

void FDistanceFieldSceneData::ProcessReadRequests(
	TArray<FDistanceFieldAssetMipId>& AssetDataUploads,
	TArray<FDistanceFieldAssetMipId>& DistanceFieldAssetMipAdds,
	TArray<FDistanceFieldReadRequest>& ReadRequestsToUpload,
	TArray<FDistanceFieldReadRequest>& ReadRequestsToCleanUp)
{
	const uint32 BrickSizeBytes = GPixelFormats[DistanceField::DistanceFieldFormat].BlockBytes * DistanceField::BrickSize * DistanceField::BrickSize * DistanceField::BrickSize;
	const SIZE_T TextureUploadLimitBytes = (SIZE_T)CVarTextureUploadLimitKBytes.GetValueOnRenderThread() * 1024;

	SIZE_T TextureUploadBytes = 0;

	// At this point DistanceFieldAssetMipAdds contains only lowest resolution mip adds which are always loaded
	// Forward these to the Requests to Upload list, with a null BulkData
	for (FDistanceFieldAssetMipId AssetMipAdd : DistanceFieldAssetMipAdds)
	{
		const FDistanceFieldAssetState& AssetState = AssetStateArray[AssetMipAdd.AssetId];
		const int32 ReversedMipIndex = AssetMipAdd.ReversedMipIndex;
		check(ReversedMipIndex == 0);
		const int32 MipIndex = AssetState.BuiltData->Mips.Num() - ReversedMipIndex - 1;
		const FSparseDistanceFieldMip& MipBuiltData = AssetState.BuiltData->Mips[MipIndex];
		TextureUploadBytes += MipBuiltData.NumDistanceFieldBricks * BrickSizeBytes;

		FDistanceFieldReadRequest NewReadRequest;
		NewReadRequest.AssetSetId = AssetMipAdd.AssetId;
		NewReadRequest.BuiltDataId = AssetState.BuiltData->GetId();
		NewReadRequest.ReversedMipIndex = AssetMipAdd.ReversedMipIndex;
		NewReadRequest.NumDistanceFieldBricks = MipBuiltData.NumDistanceFieldBricks;
		NewReadRequest.AlwaysLoadedDataPtr = AssetState.BuiltData->AlwaysLoadedMip.GetData();
		NewReadRequest.BulkSize = AssetState.BuiltData->AlwaysLoadedMip.Num();
		ReadRequestsToUpload.Add(MoveTemp(NewReadRequest));
	}

	for (int32 RequestIndex = 0; RequestIndex < ReadRequests.Num(); RequestIndex++)
	{
		const FDistanceFieldReadRequest ReadRequest = ReadRequests[RequestIndex];

		bool bReady = true;

#if !WITH_EDITOR
		if (ReadRequest.AsyncRequest)
		{
			bReady = bReady && ReadRequest.AsyncRequest->PollCompletion();
		}
		else
		{
			bReady = bReady && ReadRequest.Request.Status().IsCompleted();
		}
#endif

		if (bReady)
		{
			ReadRequests.RemoveAtSwap(RequestIndex);
			RequestIndex--;

			if (AssetStateArray.IsValidId(ReadRequest.AssetSetId) 
				// Prevent attempting to upload after a different asset has been allocated at the same index
				&& ReadRequest.BuiltDataId == AssetStateArray[ReadRequest.AssetSetId].BuiltData->GetId()
				// Shader requires sequential reversed mips starting from 0, skip upload if the IO request got out of sync with the streaming feedback requests
				&& ReadRequest.ReversedMipIndex == AssetStateArray[ReadRequest.AssetSetId].ReversedMips.Num())
			{
				TextureUploadBytes += ReadRequest.NumDistanceFieldBricks * BrickSizeBytes;

				DistanceFieldAssetMipAdds.Add(FDistanceFieldAssetMipId(ReadRequest.AssetSetId, ReadRequest.ReversedMipIndex));
				// Re-upload mip0 to push the new NumMips to the shader
				AssetDataUploads.Add(FDistanceFieldAssetMipId(ReadRequest.AssetSetId, 0));
				ReadRequestsToUpload.Add(ReadRequest);
			}

			ReadRequestsToCleanUp.Add(ReadRequest);
		}

		// Stop uploading when we reach the limit
		// In practice we can still exceed the limit with a single large upload request
		if (TextureUploadBytes >= TextureUploadLimitBytes)
		{
			break;
		}
	}

	// Re-upload asset data for all mips we are uploading this frame
	AssetDataUploads.Append(DistanceFieldAssetMipAdds);
}

FRDGTexture* FDistanceFieldSceneData::ResizeBrickAtlasIfNeeded(FRDGBuilder& GraphBuilder, FGlobalShaderMap* GlobalShaderMap)
{
	// Mask should be set in FSceneRenderer::PrepareDistanceFieldScene before calling this
	check(GraphBuilder.RHICmdList.GetGPUMask() == FRHIGPUMask::All());

	const int32 BrickAtlasSizeXYInBricks = CVarBrickAtlasSizeXYInBricks.GetValueOnRenderThread();
	int32 DesiredZSizeInBricks = FMath::DivideAndRoundUp(DistanceFieldAtlasBlockAllocator.GetMaxSize() * GDistanceFieldBlockAllocatorSizeInBricks, BrickAtlasSizeXYInBricks * BrickAtlasSizeXYInBricks);

	if (DesiredZSizeInBricks <= CVarMaxAtlasDepthInBricks.GetValueOnRenderThread())
	{
		DesiredZSizeInBricks = FMath::RoundUpToPowerOfTwo(DesiredZSizeInBricks);
	}
	else
	{
		DesiredZSizeInBricks = FMath::DivideAndRoundUp(DesiredZSizeInBricks, 4) * 4;
	}

	const FIntVector DesiredBrickTextureDimensionsInBricks = FIntVector(BrickAtlasSizeXYInBricks, BrickAtlasSizeXYInBricks, DesiredZSizeInBricks);
	const bool bResizeAtlasEveryFrame = CVarResizeAtlasEveryFrame.GetValueOnRenderThread() != 0;

	if (!DistanceFieldBrickVolumeTexture 
		|| DistanceFieldBrickVolumeTexture->GetDesc().GetSize() != DesiredBrickTextureDimensionsInBricks * DistanceField::BrickSize
		|| bResizeAtlasEveryFrame)
	{
		const FRDGTextureDesc BrickVolumeTextureDesc = FRDGTextureDesc::Create3D(
			DesiredBrickTextureDimensionsInBricks * DistanceField::BrickSize,
			DistanceField::DistanceFieldFormat,
			FClearValueBinding::Black, 
			TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling);

		FRDGTextureRef DistanceFieldBrickVolumeTextureRDG = GraphBuilder.CreateTexture(BrickVolumeTextureDesc, TEXT("DistanceFields.DistanceFieldBrickTexture"));

		if (DistanceFieldBrickVolumeTexture)
		{
			FCopyDistanceFieldAtlasCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyDistanceFieldAtlasCS::FParameters>();

			PassParameters->RWDistanceFieldBrickAtlas = GraphBuilder.CreateUAV(DistanceFieldBrickVolumeTextureRDG);
			PassParameters->DistanceFieldAtlas = DistanceField::SetupAtlasParameters(GraphBuilder, *this);

			auto ComputeShader = GlobalShaderMap->GetShader<FCopyDistanceFieldAtlasCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("CopyDistanceFieldAtlas"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(DistanceFieldBrickVolumeTexture->GetDesc().GetSize(), FCopyDistanceFieldAtlasCS::GetGroupSize()));
		}

		BrickTextureDimensionsInBricks = DesiredBrickTextureDimensionsInBricks;
		DistanceFieldBrickVolumeTexture = GraphBuilder.ConvertToExternalTexture(DistanceFieldBrickVolumeTextureRDG);
		return DistanceFieldBrickVolumeTextureRDG;
	}

	return GraphBuilder.RegisterExternalTexture(DistanceFieldBrickVolumeTexture);
}

static FIntVector CalculateDesiredSize(FIntVector CurrentSize, FIntVector RequiredSize)
{
	FIntVector DesiredSize = CurrentSize;

	DesiredSize.X = FMath::Max(DesiredSize.X, MinIndirectionAtlasSizeXYZ);
	DesiredSize.Y = FMath::Max(DesiredSize.Y, MinIndirectionAtlasSizeXYZ);
	DesiredSize.Z = FMath::Max(DesiredSize.Z, MinIndirectionAtlasSizeXYZ);

	while (DesiredSize.X < RequiredSize.X)
	{
		DesiredSize.X *= IndirectionAtlasGrowMult;
	}

	while (DesiredSize.Y < RequiredSize.Y)
	{
		DesiredSize.Y *= IndirectionAtlasGrowMult;
	}

	while (DesiredSize.Z < RequiredSize.Z)
	{
		DesiredSize.Z *= IndirectionAtlasGrowMult;
	}

	return DesiredSize;
}

bool FDistanceFieldSceneData::ResizeIndirectionAtlasIfNeeded(FRDGBuilder& GraphBuilder, FGlobalShaderMap* GlobalShaderMap, FRDGTexture*& OutTexture)
{
	// Mask should be set in FSceneRenderer::PrepareDistanceFieldScene before calling this
	check(GraphBuilder.RHICmdList.GetGPUMask() == FRHIGPUMask::All());

	QUICK_SCOPE_CYCLE_COUNTER(STAT_ResizeIndirectionAtlasIfNeeded);
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldSceneData::ResizeIndirectionAtlasIfNeeded);
	
	const FIntVector CurrentSize = IndirectionAtlas ? IndirectionAtlas->GetDesc().GetSize() : FIntVector::ZeroValue;
	FIntVector DesiredSize = CalculateDesiredSize(CurrentSize, IndirectionAtlasLayout.GetSize());
	const bool bDefragment = (CurrentSize != DesiredSize) && CVarDefragmentIndirectionAtlas.GetValueOnRenderThread() != 0;
	TArray<FDistanceFieldAssetMipRelocation> Relocations;

	if (bDefragment)
	{
		CSV_EVENT(DistanceField, TEXT("DefragmentIndirectionAtlas"));

		DefragmentIndirectionAtlas(FIntVector(MinIndirectionAtlasSizeXYZ), Relocations);
		DesiredSize = CalculateDesiredSize(CurrentSize, IndirectionAtlasLayout.GetSize());
	}
	else if (CurrentSize != DesiredSize)
	{
		for (TSparseSet<FDistanceFieldAssetState, TFDistanceFieldAssetStateFuncs>::TConstIterator It(AssetStateArray); It; ++It)
		{
			const FDistanceFieldAssetState& AssetState = *It;
			for (int32 ReversedMipIndex = 0; ReversedMipIndex < AssetState.ReversedMips.Num(); ReversedMipIndex++)
			{
				const FDistanceFieldAssetMipState& MipState = AssetState.ReversedMips[ReversedMipIndex];
				Relocations.Add(FDistanceFieldAssetMipRelocation(MipState.IndirectionDimensions, MipState.IndirectionAtlasOffset, MipState.IndirectionAtlasOffset));
			}
		}
	}

	if (!IndirectionAtlas || (CurrentSize != DesiredSize) || !Relocations.IsEmpty())
	{
		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create3D(
			DesiredSize,
			PF_A2B10G10R10,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling);
		FRDGTextureRef NewIndirectAtlasRDG = GraphBuilder.CreateTexture(TextureDesc, TEXT("DistanceFields.DistanceFieldIndirectionAtlas"));
		check(NewIndirectAtlasRDG);

		if (IndirectionAtlas)
		{
			TShaderMapRef<FCopyDistanceFieldIndirectionAtlasCS> ComputeShader(GlobalShaderMap);

			FRDGTexture* OldIndirectAtlasRDG = GraphBuilder.RegisterExternalTexture(IndirectionAtlas);

			FRDGTextureUAV* NewIndirectAtlasUAV = GraphBuilder.CreateUAV(NewIndirectAtlasRDG, ERDGUnorderedAccessViewFlags::SkipBarrier);

			for (int32 RelocationIndex = 0; RelocationIndex < Relocations.Num(); ++RelocationIndex)
			{
				const FDistanceFieldAssetMipRelocation& Relocation = Relocations[RelocationIndex];

				auto* PassParameters = GraphBuilder.AllocParameters<FCopyDistanceFieldIndirectionAtlasCS::FParameters>();
				PassParameters->RWIndirectionAtlas = NewIndirectAtlasUAV;
				PassParameters->DistanceFieldIndirectionAtlas = OldIndirectAtlasRDG;
				PassParameters->IndirectionDimensions = Relocation.IndirectionDimensions;
				PassParameters->SrcPosition = Relocation.SrcPosition;
				PassParameters->DstPosition = Relocation.DstPosition;
				PassParameters->NumAssets = 1;

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("CopyDistanceFieldIndirectionAtlas (Index %d)", RelocationIndex),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(
						Relocation.IndirectionDimensions.X * Relocation.IndirectionDimensions.Y * Relocation.IndirectionDimensions.Z,
						FCopyDistanceFieldIndirectionAtlasCS::GetGroupSize()));
			}
		}

		IndirectionAtlas = GraphBuilder.ConvertToExternalTexture(NewIndirectAtlasRDG);
		OutTexture = NewIndirectAtlasRDG;

		UE_LOGF(LogDistanceField, Log, "New indirection table size: %ls (%ls required)", *IndirectionAtlas->GetDesc().GetSize().ToString(), *IndirectionAtlasLayout.GetSize().ToString());
		return true;
	}

	OutTexture = GraphBuilder.RegisterExternalTexture(IndirectionAtlas);
	return false;
}

void FDistanceFieldSceneData::DefragmentIndirectionAtlas(FIntVector MinSize, TArray<FDistanceFieldAssetMipRelocation>& Relocations)
{
	struct FEntry
	{
		FSetElementId AssetSetId;
		uint32 ReversedMipIndex;
		SIZE_T Size;
	};
	TArray<FEntry> Entries;

	for (TSparseSet<FDistanceFieldAssetState, TFDistanceFieldAssetStateFuncs>::TConstIterator It(AssetStateArray); It; ++It)
	{
		const FDistanceFieldAssetState& AssetState = *It;
		for (int32 ReversedMipIndex = 0; ReversedMipIndex < AssetState.ReversedMips.Num(); ReversedMipIndex++)
		{
			const FDistanceFieldAssetMipState& MipState = AssetState.ReversedMips[ReversedMipIndex];
			FEntry Entry;
			Entry.AssetSetId = It.GetId();
			Entry.ReversedMipIndex = ReversedMipIndex;
			Entry.Size = MipState.IndirectionDimensions.X * MipState.IndirectionDimensions.Y * MipState.IndirectionDimensions.Z;
			Entries.Add(Entry);
		}
	}

	struct FMeshDistanceFieldSorter
	{
		bool operator()(const FEntry& A, const FEntry& B) const
		{
			return A.Size > B.Size;
		}
	};
	Entries.Sort(FMeshDistanceFieldSorter());

	const int32 MaxIndirectionAtlasSizeXYZ = CVarMaxIndirectionAtlasSizeXYZ.GetValueOnRenderThread();
	IndirectionAtlasLayout = FTextureLayout3d(MinSize.X, MinSize.Y, MinSize.Z, MaxIndirectionAtlasSizeXYZ, MaxIndirectionAtlasSizeXYZ, MaxIndirectionAtlasSizeXYZ, false, false);

	for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
	{
		const FEntry& Entry = Entries[EntryIndex];
		const FDistanceFieldAssetState& AssetState = AssetStateArray[Entry.AssetSetId];
		const FDistanceFieldAssetMipState& MipState = AssetState.ReversedMips[Entry.ReversedMipIndex];
		FIntVector PrevPosition = MipState.IndirectionAtlasOffset;
		IndirectionAtlasLayout.AddElement((uint32&)MipState.IndirectionAtlasOffset.X, (uint32&)MipState.IndirectionAtlasOffset.Y, (uint32&)MipState.IndirectionAtlasOffset.Z,
			MipState.IndirectionDimensions.X, MipState.IndirectionDimensions.Y, MipState.IndirectionDimensions.Z);
		Relocations.Add(FDistanceFieldAssetMipRelocation(MipState.IndirectionDimensions, PrevPosition, MipState.IndirectionAtlasOffset));
	}
}

void FDistanceFieldSceneData::GenerateStreamingRequests(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs& SceneUpdateInputs)
{
	// It is not safe to EnqueueCopy on a buffer that already has a pending copy.
	if (ReadbackBuffersNumPending < MaxStreamingReadbackBuffers && NumObjectsInBuffer > 0)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

		if (!StreamingRequestReadbackBuffers[ReadbackBuffersWriteIndex])
		{
			FRHIGPUBufferReadback* GPUBufferReadback = new FRHIGPUBufferReadback(TEXT("DistanceFields.StreamingRequestReadBack"));
			StreamingRequestReadbackBuffers[ReadbackBuffersWriteIndex] = GPUBufferReadback;
		}

		const uint32 NumAssets = AssetStateArray.GetMaxIndex();
		FRDGBufferDesc WantedNumMipsDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::RoundUpToPowerOfTwo(NumAssets));
		FRDGBufferRef WantedNumMips = GraphBuilder.CreateBuffer(WantedNumMipsDesc, TEXT("DistanceFields.DistanceFieldAssetWantedNumMips"));

		// Every asset wants at least 1 mipmap
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(FRDGBufferUAVDesc(WantedNumMips)), 1);

		FRDGBufferDesc StreamingRequestsDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxStreamingRequests * 2 + 1);
		StreamingRequestsDesc.Usage = EBufferUsageFlags(StreamingRequestsDesc.Usage | BUF_SourceCopy);
		FRDGBufferRef StreamingRequestsBuffer = GraphBuilder.CreateBuffer(StreamingRequestsDesc, TEXT("DistanceFields.DistanceFieldStreamingRequests"));

		SceneUpdateInputs.ForEachView([&] (const FSceneRenderer* Renderer, const FViewInfo& View)
		{
			const bool bLumenEnabled = Renderer->IsLumenEnabled(View);

			FDFVector3 PreViewTranslation(View.ViewMatrices.GetPreViewTranslation());

			FComputeDistanceFieldAssetWantedMipsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComputeDistanceFieldAssetWantedMipsCS::FParameters>();
			checkf(DistanceField::NumMips == 3, TEXT("Shader needs to be updated"));
			PassParameters->RWDistanceFieldAssetWantedNumMips = GraphBuilder.CreateUAV(WantedNumMips);
			PassParameters->RWDistanceFieldAssetStreamingRequests = GraphBuilder.CreateUAV(StreamingRequestsBuffer);
			PassParameters->DistanceFieldObjectBuffers = DistanceField::SetupObjectBufferParameters(GraphBuilder, *this);
			PassParameters->DebugForceNumMips = FMath::Clamp(CVarDebugForceNumMips.GetValueOnRenderThread(), 0, DistanceField::NumMips);
			extern int32 GAOGlobalDistanceFieldNumClipmaps;
			// Request Mesh SDF mips based off of the Global SDF clipmaps
			PassParameters->Mip1WorldTranslatedCenter = FVector3f(View.ViewMatrices.GetViewOrigin() + View.ViewMatrices.GetPreViewTranslation());
			PassParameters->Mip1WorldExtent = FVector3f(GlobalDistanceField::GetClipmapExtent(GAOGlobalDistanceFieldNumClipmaps - 1, SceneUpdateInputs.Scene, bLumenEnabled));
			PassParameters->Mip2WorldTranslatedCenter = FVector3f(View.ViewMatrices.GetViewOrigin() + View.ViewMatrices.GetPreViewTranslation());
			PassParameters->Mip2WorldExtent = FVector3f(GlobalDistanceField::GetClipmapExtent(FMath::Max<int32>(GAOGlobalDistanceFieldNumClipmaps / 2 - 1, 0), SceneUpdateInputs.Scene, bLumenEnabled));
			PassParameters->PreViewTranslationHigh = PreViewTranslation.High;
			PassParameters->PreViewTranslationLow = PreViewTranslation.Low;

			auto ComputeShader = SceneUpdateInputs.GlobalShaderMap->GetShader<FComputeDistanceFieldAssetWantedMipsCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ComputeWantedMips"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCountWrapped(NumObjectsInBuffer, FComputeDistanceFieldAssetWantedMipsCS::GetGroupSize()));

			return true;
		});

		{
			FGenerateDistanceFieldAssetStreamingRequestsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGenerateDistanceFieldAssetStreamingRequestsCS::FParameters>();
			PassParameters->RWDistanceFieldAssetStreamingRequests = GraphBuilder.CreateUAV(StreamingRequestsBuffer);
			PassParameters->DistanceFieldAssetWantedNumMips = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(WantedNumMips));
			PassParameters->DistanceFieldObjectBuffers = DistanceField::SetupObjectBufferParameters(GraphBuilder, *this);
			PassParameters->DistanceFieldAtlasParameters = DistanceField::SetupAtlasParameters(GraphBuilder, *this);
			PassParameters->NumDistanceFieldAssets = NumAssets;
			PassParameters->MaxNumStreamingRequests = MaxStreamingRequests;

			auto ComputeShader = SceneUpdateInputs.GlobalShaderMap->GetShader<FGenerateDistanceFieldAssetStreamingRequestsCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("GenerateStreamingRequests"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(NumAssets, FGenerateDistanceFieldAssetStreamingRequestsCS::GetGroupSize()));
		}

		FRHIGPUBufferReadback* ReadbackBuffer = StreamingRequestReadbackBuffers[ReadbackBuffersWriteIndex];

		AddReadbackBufferPass(GraphBuilder, RDG_EVENT_NAME("DistanceFieldAssetReadback"), StreamingRequestsBuffer,
			[ReadbackBuffer, StreamingRequestsBuffer](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			ReadbackBuffer->EnqueueCopy(RHICmdList, StreamingRequestsBuffer->GetRHI(), 0u);
		});

		ReadbackBuffersWriteIndex = (ReadbackBuffersWriteIndex + 1u) % MaxStreamingReadbackBuffers;
		ReadbackBuffersNumPending = FMath::Min(ReadbackBuffersNumPending + 1u, MaxStreamingReadbackBuffers);
	}
}

void EncodeAssetData(const FDistanceFieldAssetState& AssetState, const int32 ReversedMipIndex, FVector4f* OutAssetData)
{
	const FDistanceFieldAssetMipState& MipState = AssetState.ReversedMips[ReversedMipIndex];
	const int32 MipIndex = AssetState.BuiltData->Mips.Num() - ReversedMipIndex - 1;
	const FSparseDistanceFieldMip& MipBuiltData = AssetState.BuiltData->Mips[MipIndex];
	const FVector2f DistanceFieldToVolumeScaleBias = MipBuiltData.DistanceFieldToVolumeScaleBias;
	const int32 NumMips = AssetState.ReversedMips.Num();

	check(NumMips <= DistanceField::NumMips);
	check(DistanceField::NumMips < 4);
	check(MipBuiltData.IndirectionDimensions.X < DistanceField::MaxIndirectionDimension
		&& MipBuiltData.IndirectionDimensions.Y < DistanceField::MaxIndirectionDimension
		&& MipBuiltData.IndirectionDimensions.Z < DistanceField::MaxIndirectionDimension);

	uint32 IntVector0[4] =
	{
		(uint32)MipBuiltData.IndirectionDimensions.X | (uint32)(MipBuiltData.IndirectionDimensions.Y << 10) | (uint32)(MipBuiltData.IndirectionDimensions.Z << 20) | (uint32)(NumMips << 30),
		(uint32)MipState.IndirectionTableOffset,
		0,
		0
	};

	// Bypass NaN checks in FVector4f ctors
	FVector4f FloatVector0;
	FloatVector0.X = *(const float*)&IntVector0[0];
	FloatVector0.Y = *(const float*)&IntVector0[1];
	FloatVector0.Z = *(const float*)&IntVector0[2];
	FloatVector0.W = *(const float*)&IntVector0[3];

	FVector4f VolumeToIndirectionScale = FVector4f(MipBuiltData.VolumeToVirtualUVScale, DistanceFieldToVolumeScaleBias.X);
	VolumeToIndirectionScale.X *= MipBuiltData.IndirectionDimensions.X;
	VolumeToIndirectionScale.Y *= MipBuiltData.IndirectionDimensions.Y;
	VolumeToIndirectionScale.Z *= MipBuiltData.IndirectionDimensions.Z;

	FVector4f VolumeToIndirectionAdd = FVector4f(MipBuiltData.VolumeToVirtualUVAdd, DistanceFieldToVolumeScaleBias.Y);
	VolumeToIndirectionAdd.X *= MipBuiltData.IndirectionDimensions.X;
	VolumeToIndirectionAdd.Y *= MipBuiltData.IndirectionDimensions.Y;
	VolumeToIndirectionAdd.Z *= MipBuiltData.IndirectionDimensions.Z;

	if (GDistanceFieldOffsetDataStructure != 0)
	{
		VolumeToIndirectionAdd.X += MipState.IndirectionAtlasOffset.X;
		VolumeToIndirectionAdd.Y += MipState.IndirectionAtlasOffset.Y;
		VolumeToIndirectionAdd.Z += MipState.IndirectionAtlasOffset.Z;
	}

	OutAssetData[0] = FloatVector0;
	OutAssetData[1] = VolumeToIndirectionScale;
	OutAssetData[2] = VolumeToIndirectionAdd;
}

void FDistanceFieldSceneData::UploadAssetData(FRDGBuilder& GraphBuilder, const TArray<FDistanceFieldAssetMipId>& AssetDataUploads, FRDGBuffer* AssetDataBufferRDG)
{
	if (AssetDataUploads.IsEmpty())
	{
		return;
	}

	// Mask should be set in FSceneRenderer::PrepareDistanceFieldScene before calling this
	check(GraphBuilder.RHICmdList.GetGPUMask() == FRHIGPUMask::All());

	AssetDataUploadBuffer.Init(GraphBuilder, AssetDataUploads.Num(), AssetDataMipStrideFloat4s * sizeof(FVector4f), true, TEXT("DistanceFields.DFAssetDataAssetDataUploadBuffer"));

	for (FDistanceFieldAssetMipId AssetMipUpload : AssetDataUploads)
	{
		const int32 ReversedMipIndex = AssetMipUpload.ReversedMipIndex;
		FVector4f* UploadAssetData = (FVector4f*)AssetDataUploadBuffer.Add_GetRef(AssetMipUpload.AssetId.AsInteger() * DistanceField::NumMips + ReversedMipIndex);

		if (AssetStateArray.IsValidId(AssetMipUpload.AssetId))
		{
			const FDistanceFieldAssetState& AssetState = AssetStateArray[AssetMipUpload.AssetId];
			EncodeAssetData(AssetState, ReversedMipIndex, UploadAssetData);
		}
		else
		{
			// Clear invalid entries to zero
			UploadAssetData[0] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
			UploadAssetData[1] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
			UploadAssetData[2] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		}
	}

	AssetDataUploadBuffer.ResourceUploadTo(GraphBuilder, AssetDataBufferRDG);
}

void FDistanceFieldSceneData::UploadAllAssetData(FRDGBuilder& GraphBuilder, FRDGBuffer* AssetDataBufferRDG)
{
	uint32 NumUploads = AssetStateArray.Num() * DistanceField::NumMips;
	if (NumUploads == 0)
	{
		return;
	}

	// Mask should be set in FSceneRenderer::PrepareDistanceFieldScene before calling this
	check(GraphBuilder.RHICmdList.GetGPUMask() == FRHIGPUMask::All());

	AssetDataUploadBuffer.Init(GraphBuilder, NumUploads, AssetDataMipStrideFloat4s * sizeof(FVector4f), true, TEXT("DistanceFields.DFAssetDataUploadBuffer"));

	for (TSparseSet<FDistanceFieldAssetState, TFDistanceFieldAssetStateFuncs>::TConstIterator It(AssetStateArray); It; ++It)
	{
		const FDistanceFieldAssetState& AssetState = *It;
		const FSetElementId AssetId = It.GetId();

		if (AssetStateArray.IsValidId(AssetId))
		{
			for (int32 ReversedMipIndex = 0; ReversedMipIndex < AssetState.ReversedMips.Num(); ReversedMipIndex++)
			{
				//const FDistanceFieldAssetMipState& MipState = AssetState.ReversedMips[ReversedMipIndex];
				FVector4f* UploadAssetData = (FVector4f*)AssetDataUploadBuffer.Add_GetRef(AssetId.AsInteger() * DistanceField::NumMips + ReversedMipIndex);
				EncodeAssetData(AssetStateArray[AssetId], ReversedMipIndex, UploadAssetData);
			}
		}
		else
		{
			FVector4f* UploadAssetData = (FVector4f*)AssetDataUploadBuffer.Add_GetRef(AssetId.AsInteger() * DistanceField::NumMips);
			// Clear invalid entries to zero
			UploadAssetData[0] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
			UploadAssetData[1] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
			UploadAssetData[2] = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		}
	}

	AssetDataUploadBuffer.ResourceUploadTo(GraphBuilder, AssetDataBufferRDG);
}

void FDistanceFieldSceneData::UpdateDistanceFieldAtlas(
	FRDGBuilder& GraphBuilder,
	FRDGExternalAccessQueue& ExternalAccessQueue,
	const FSceneRenderUpdateInputs& SceneUpdateInputs,
	TArray<FDistanceFieldAssetMipId>& DistanceFieldAssetMipAdds,
	TArray<FSetElementId>& DistanceFieldAssetRemoves)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateDistanceFieldAtlas);
	TRACE_CPUPROFILER_EVENT_SCOPE(FDistanceFieldSceneData::UpdateDistanceFieldAtlas);
	RDG_EVENT_SCOPE(GraphBuilder, "UpdateDistanceFieldAtlas");

	// Mask should be set in FSceneRenderer::PrepareDistanceFieldScene before calling this
	check(GraphBuilder.RHICmdList.GetGPUMask() == FRHIGPUMask::All());

	TArray<FDistanceFieldAssetMipId> AssetDataUploads;

	for (FSetElementId AssetSetId : DistanceFieldAssetRemoves)
	{
		const FDistanceFieldAssetState& AssetState = AssetStateArray[AssetSetId];
		check(AssetState.RefCount == 0);

		for (const FDistanceFieldAssetMipState& MipState : AssetState.ReversedMips)
		{
			if (GDistanceFieldOffsetDataStructure == 0 || GDistanceFieldOffsetDataStructure == 1)
			{
				IndirectionTableAllocator.Free(MipState.IndirectionTableOffset, MipState.IndirectionDimensions.X * MipState.IndirectionDimensions.Y * MipState.IndirectionDimensions.Z);
			}
			else
			{
				IndirectionAtlasLayout.RemoveElement(MipState.IndirectionAtlasOffset.X, MipState.IndirectionAtlasOffset.Y, MipState.IndirectionAtlasOffset.Z,
					MipState.IndirectionDimensions.X, MipState.IndirectionDimensions.Y, MipState.IndirectionDimensions.Z);
			}

			if (MipState.NumBricks > 0)
			{
				check(MipState.AllocatedBlocks.Num() > 0);
				DistanceFieldAtlasBlockAllocator.Free(MipState.AllocatedBlocks);
			}
		}
		
		// Clear GPU data for removed asset
		AssetDataUploads.Add(FDistanceFieldAssetMipId(AssetSetId, 0));

		AssetStateArray.Remove(AssetSetId);
	}

	TArray<FDistanceFieldReadRequest> NewReadRequests;
	// Lock the most recent streaming request buffer from the GPU, create new read requests for mips we want to load in the Async Task
	ProcessStreamingRequestsFromGPU(NewReadRequests, AssetDataUploads);

	TArray<FDistanceFieldReadRequest> ReadRequestsToUpload;
	TArray<FDistanceFieldReadRequest> ReadRequestsToCleanUp;
	// Build a list of completed read requests that should be uploaded to the GPU this frame
	ProcessReadRequests(AssetDataUploads, DistanceFieldAssetMipAdds, ReadRequestsToUpload, ReadRequestsToCleanUp);

	int32 NumIndirectionTableAdds = 0;
	int32 NumBrickUploads = 0;

	// Allocate the mips we are adding this frame from the IndirectionTable and BrickAtlas
	for (int32 MipAddIndex = 0; MipAddIndex < DistanceFieldAssetMipAdds.Num(); MipAddIndex++)
	{
		const int32 Index = GDFReverseAtlasAllocationOrder ? DistanceFieldAssetMipAdds.Num() - MipAddIndex - 1 : MipAddIndex;
		FSetElementId AssetSetId = DistanceFieldAssetMipAdds[Index].AssetId;
		FDistanceFieldAssetState& AssetState = AssetStateArray[AssetSetId];

		const int32 ReversedMipIndex = DistanceFieldAssetMipAdds[Index].ReversedMipIndex;

		// Shader requires sequential reversed mips starting from 0
		check(ReversedMipIndex == AssetState.ReversedMips.Num());
			
		const int32 MipIndex = AssetState.BuiltData->Mips.Num() - ReversedMipIndex - 1;
		const FSparseDistanceFieldMip& MipBuiltData = AssetState.BuiltData->Mips[MipIndex];
		FDistanceFieldAssetMipState NewMipState;
		NewMipState.NumBricks = MipBuiltData.NumDistanceFieldBricks;
		DistanceFieldAtlasBlockAllocator.Allocate(FMath::DivideAndRoundUp(MipBuiltData.NumDistanceFieldBricks, GDistanceFieldBlockAllocatorSizeInBricks), NewMipState.AllocatedBlocks);
		NewMipState.IndirectionDimensions = MipBuiltData.IndirectionDimensions;
		const int32 NumIndirectionEntries = NewMipState.IndirectionDimensions.X * NewMipState.IndirectionDimensions.Y * NewMipState.IndirectionDimensions.Z;
		
		if (GDistanceFieldOffsetDataStructure == 0 || GDistanceFieldOffsetDataStructure == 1)
		{
			NewMipState.IndirectionTableOffset = IndirectionTableAllocator.Allocate(NumIndirectionEntries);
		}
		else
		{
			IndirectionAtlasLayout.AddElement((uint32&)NewMipState.IndirectionAtlasOffset.X, (uint32&)NewMipState.IndirectionAtlasOffset.Y, (uint32&)NewMipState.IndirectionAtlasOffset.Z,
				NewMipState.IndirectionDimensions.X, NewMipState.IndirectionDimensions.Y, NewMipState.IndirectionDimensions.Z);
		}

		AssetState.ReversedMips.Add(MoveTemp(NewMipState));

		NumIndirectionTableAdds += NumIndirectionEntries;
		NumBrickUploads += MipBuiltData.NumDistanceFieldBricks;
	}

	// Now that DistanceFieldAtlasBlockAllocator has been modified, potentially resize the atlas
	FRDGTextureRef DistanceFieldBrickVolumeTextureRDG = ResizeBrickAtlasIfNeeded(GraphBuilder, SceneUpdateInputs.GlobalShaderMap);

	const uint32 NumAssets = AssetStateArray.GetMaxIndex();
	const int32 AssetDataStrideFloat4s = DistanceField::NumMips * AssetDataMipStrideFloat4s;

	bool bIndirectionAtlasResized = false;

	const uint32 AssetDataSizeBytes = FMath::RoundUpToPowerOfTwo(NumAssets) * AssetDataStrideFloat4s * sizeof(FVector4f);
	FRDGBuffer* AssetDataBufferRDG = ResizeStructuredBufferIfNeeded(GraphBuilder, AssetDataBuffer, AssetDataSizeBytes, TEXT("DistanceFields.AssetData"));

	FRDGBuffer* IndirectionTableRDG = nullptr;
	FRDGTexture* IndirectionAtlasRDG = nullptr;

	if (GDistanceFieldOffsetDataStructure == 0)
	{
		const uint32 IndirectionTableSizeBytes = FMath::Max<uint32>(FMath::RoundUpToPowerOfTwo(IndirectionTableAllocator.GetMaxSize()) * sizeof(uint32), 16);
		IndirectionTableRDG = ResizeByteAddressBufferIfNeeded(GraphBuilder, IndirectionTable, IndirectionTableSizeBytes, TEXT("DistanceFields.IndirectionTable.Uint"));
	}
	else if (GDistanceFieldOffsetDataStructure == 1)
	{
		const uint32 Indirection2TableNumElements = FMath::Max<uint32>(FMath::RoundUpToPowerOfTwo(IndirectionTableAllocator.GetMaxSize()), 16);
		IndirectionTableRDG = ResizeBufferIfNeeded(GraphBuilder, IndirectionTable, PF_A2B10G10R10, Indirection2TableNumElements, TEXT("DistanceFields.IndirectionTable.Float"));
	}
	else
	{
		bIndirectionAtlasResized = ResizeIndirectionAtlasIfNeeded(GraphBuilder, SceneUpdateInputs.GlobalShaderMap, IndirectionAtlasRDG);
	}

	{
		const FIntVector AtlasDimensions = BrickTextureDimensionsInBricks * DistanceField::BrickSize;
		const SIZE_T AtlasSizeBytes = AtlasDimensions.X * AtlasDimensions.Y * AtlasDimensions.Z * GPixelFormats[DistanceField::DistanceFieldFormat].BlockBytes;
		const SIZE_T IndirectionTableBytes = TryGetSize(IndirectionTable);

		const FIntVector IndirectionAtlasSize = IndirectionAtlas ? IndirectionAtlas->GetDesc().GetSize() : FIntVector::ZeroValue;
		const SIZE_T IndirectionTextureBytes = IndirectionAtlasSize.X * IndirectionAtlasSize.Y * IndirectionAtlasSize.Z * GPixelFormats[PF_A2B10G10R10].BlockBytes;

		float AtlasSizeMB = float(AtlasSizeBytes) / (1024.0f * 1024.0f);
		float IndirectionTableSizeMB = float(IndirectionTableBytes) / (1024.0f * 1024.0f);
		float IndirectionAtlasSizeMB = float(IndirectionTextureBytes) / (1024.0f * 1024.0f);
		CSV_CUSTOM_STAT(DistanceField, AtlasMB, AtlasSizeMB, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(DistanceField, IndirectionTableMB, IndirectionTableSizeMB, ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(DistanceField, IndirectionAtlasMB, IndirectionAtlasSizeMB, ECsvCustomStatOp::Set);
	}

	{
		FDistanceFieldAsyncUpdateParameters& UpdateParameters = *GraphBuilder.AllocObject<FDistanceFieldAsyncUpdateParameters>(BrickUploadCoordinatesBuffer, BrickUploadDataBuffer, IndirectionUploadIndicesBuffer, IndirectionUploadDataBuffer);
		UpdateParameters.DistanceFieldSceneData = this;

		FDistanceFieldIndirectionAtlasUpload& IndirectionAtlasUpload = UpdateParameters.IndirectionAtlasUpload;
		FDistanceFieldAtlasUpload& AtlasUpload = UpdateParameters.AtlasUpload;

		check(ReadRequestsToUpload.Num() == 0 && NumIndirectionTableAdds == 0 || ReadRequestsToUpload.Num() > 0 && NumIndirectionTableAdds > 0);

		if (NumIndirectionTableAdds > 0)
		{
			// Allocate staging buffer space for the indirection table compute scatter
			if (GDistanceFieldOffsetDataStructure == 0)
			{
				UpdateParameters.IndirectionTableUploader = IndirectionTableUploadBuffer.Begin(GraphBuilder, IndirectionTableRDG, NumIndirectionTableAdds, sizeof(uint32), TEXT("DistanceFields.IndirectionTableUploadBuffer.Uint"));
			}
			else if (GDistanceFieldOffsetDataStructure == 1)
			{
				UpdateParameters.IndirectionTableUploader = IndirectionTableUploadBuffer.Begin(GraphBuilder, IndirectionTableRDG, NumIndirectionTableAdds, sizeof(FVector4f), TEXT("DistanceFields.IndirectionFloatUploadBuffer.Float"));
			}
			else
			{
				// Allocate staging buffer space for the indirection atlas compute scatter
				IndirectionAtlasUpload.Allocate(GraphBuilder.RHICmdList, NumIndirectionTableAdds);
			}
		}

		if (NumBrickUploads > 0)
		{
			// Allocate staging buffer space for the brick atlas compute scatter
			AtlasUpload.Allocate(GraphBuilder.RHICmdList, NumBrickUploads, DistanceField::BrickSize);
		}

		// Refresh the snapshot used by HasPendingStreaming() before the AsyncUpdate task below appends NewReadRequests into ReadRequests on a worker thread.
		// Render-thread callers running in parallel with that task (e.g. ComputeUpdateRegionsAndUpdateViewState) read the cached value to avoid racing with the append.
		bHasPendingStreaming = (ReadRequests.Num() + NewReadRequests.Num()) > 0;

		if (NewReadRequests.Num() || ReadRequestsToUpload.Num() || ReadRequestsToCleanUp.Num())
		{
			UpdateParameters.NewReadRequests = MoveTemp(NewReadRequests);
			UpdateParameters.ReadRequestsToUpload = MoveTemp(ReadRequestsToUpload);
			UpdateParameters.ReadRequestsToCleanUp = MoveTemp(ReadRequestsToCleanUp);

			GraphBuilder.AddCommandListSetupTask([this, &UpdateParameters] (FRHICommandListBase& RHICmdList)
			{
				AsyncUpdate(RHICmdList, UpdateParameters);
			});
		}

		if (NumBrickUploads > 0 || NumIndirectionTableAdds > 0)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_WaitOnDistanceFieldStreamingUpdate);
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitOnDistanceFieldStreamingUpdate);

			if (NumIndirectionTableAdds > 0)
			{
				if (GDistanceFieldOffsetDataStructure == 0 || GDistanceFieldOffsetDataStructure == 1)
				{
					IndirectionTableUploadBuffer.End(GraphBuilder, UpdateParameters.IndirectionTableUploader);
					ExternalAccessQueue.Add(IndirectionTableRDG, ERHIAccess::SRVMask, ERHIPipeline::All);
				}
				else
				{
					TShaderMapRef<FScatterUploadDistanceFieldIndirectionAtlasCS> ComputeShader(SceneUpdateInputs.GlobalShaderMap);

					auto* PassParameters = GraphBuilder.AllocParameters<FScatterUploadDistanceFieldIndirectionAtlasCS::FParameters>();
					PassParameters->RWIndirectionAtlas = GraphBuilder.CreateUAV(IndirectionAtlasRDG);
					PassParameters->IndirectionUploadIndices = IndirectionAtlasUpload.IndirectionUploadIndicesBuffer.SRV;
					PassParameters->IndirectionUploadData = IndirectionAtlasUpload.IndirectionUploadDataBuffer.SRV;
					PassParameters->IndirectionAtlasSize = IndirectionAtlasRDG->Desc.GetSize();
					PassParameters->NumIndirectionUploads = NumIndirectionTableAdds;

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("ScatterUploadDistanceFieldIndirectionAtlas"),
						ComputeShader,
						PassParameters,
						FComputeShaderUtils::GetGroupCount(NumIndirectionTableAdds, FScatterUploadDistanceFieldIndirectionAtlasCS::GetGroupSize()));

					ExternalAccessQueue.Add(IndirectionAtlasRDG, ERHIAccess::SRVMask);
				}
			}
		}

		if (NumBrickUploads > 0)
		{
			// GRHIMaxDispatchThreadGroupsPerDimension can be MAX_int32 so we need to do this math in 64-bit.
			const int32 MaxBrickUploadsPerPass = (int32)FMath::Min<int64>((int64)GRHIMaxDispatchThreadGroupsPerDimension.Z * FScatterUploadDistanceFieldAtlasCS::GetGroupSize() / DistanceField::BrickSize, MAX_int32);

			for (int32 StartBrickIndex = 0; StartBrickIndex < NumBrickUploads; StartBrickIndex += MaxBrickUploadsPerPass)
			{
				const int32 NumBrickUploadsThisPass = FMath::Min(MaxBrickUploadsPerPass, NumBrickUploads - StartBrickIndex);
				FScatterUploadDistanceFieldAtlasCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FScatterUploadDistanceFieldAtlasCS::FParameters>();

				PassParameters->RWDistanceFieldBrickAtlas = GraphBuilder.CreateUAV(DistanceFieldBrickVolumeTextureRDG);
				PassParameters->BrickUploadCoordinates = AtlasUpload.BrickUploadCoordinatesBuffer.SRV;
				PassParameters->BrickUploadData = AtlasUpload.BrickUploadDataBuffer.SRV;
				PassParameters->StartBrickIndex = StartBrickIndex;
				PassParameters->NumBrickUploads = NumBrickUploadsThisPass;
				PassParameters->BrickSize = DistanceField::BrickSize;

				auto ComputeShader = SceneUpdateInputs.GlobalShaderMap->GetShader<FScatterUploadDistanceFieldAtlasCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("ScatterUploadDistanceFieldAtlas"),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(FIntVector(DistanceField::BrickSize, DistanceField::BrickSize, NumBrickUploadsThisPass * DistanceField::BrickSize), FScatterUploadDistanceFieldAtlasCS::GetGroupSize()));
			}
		}
	}
	
	if (bIndirectionAtlasResized)
	{
		UploadAllAssetData(GraphBuilder, AssetDataBufferRDG);
	}
	else
	{
		UploadAssetData(GraphBuilder, AssetDataUploads, AssetDataBufferRDG);
	}

	GenerateStreamingRequests(GraphBuilder, SceneUpdateInputs);

	if (GDistanceFieldAtlasLogStats)
	{
		const bool bDumpAssetStats = GDistanceFieldAtlasLogStats > 1;
		ListMeshDistanceFields(bDumpAssetStats);
		GDistanceFieldAtlasLogStats = 0;
	}

	ExternalAccessQueue.Add(DistanceFieldBrickVolumeTextureRDG, ERHIAccess::SRVMask, ERHIPipeline::All);
	ExternalAccessQueue.Add(AssetDataBufferRDG, ERHIAccess::SRVMask, ERHIPipeline::All);
}

void FDistanceFieldSceneData::ListMeshDistanceFields(bool bDumpAssetStats) const
{
	SIZE_T BlockAllocatorWasteBytes = 0;

	struct FMeshDistanceFieldStats
	{
		int32 LoadedMips;
		int32 WantedMips;
		SIZE_T BrickMemoryBytes;
		SIZE_T IndirectionMemoryBytes;
		FIntVector Resolution;
		FName AssetName;
	};

	struct FMipStats
	{
		SIZE_T BrickMemoryBytes;
		SIZE_T IndirectionMemoryBytes;
	};

	TArray<FMeshDistanceFieldStats> AssetStats;
	TArray<FMipStats> MipStats;
	MipStats.AddZeroed(DistanceField::NumMips);

	const uint32 BrickSizeBytes = GPixelFormats[DistanceField::DistanceFieldFormat].BlockBytes * DistanceField::BrickSize * DistanceField::BrickSize * DistanceField::BrickSize;

	for (TSparseSet<FDistanceFieldAssetState, TFDistanceFieldAssetStateFuncs>::TConstIterator It(AssetStateArray); It; ++It)
	{
		const FDistanceFieldAssetState& AssetState = *It;

		FMeshDistanceFieldStats Stats;
		Stats.Resolution = AssetState.BuiltData->Mips[0].IndirectionDimensions * DistanceField::UniqueDataBrickSize;
		Stats.BrickMemoryBytes = 0;
		Stats.IndirectionMemoryBytes = 0;
		Stats.AssetName = AssetState.BuiltData->AssetName;
		Stats.LoadedMips = AssetState.ReversedMips.Num();
		Stats.WantedMips = AssetState.WantedNumMips;

		for (int32 ReversedMipIndex = 0; ReversedMipIndex < AssetState.ReversedMips.Num(); ReversedMipIndex++)
		{
			const FDistanceFieldAssetMipState& MipState = AssetState.ReversedMips[ReversedMipIndex];
			const SIZE_T MipBrickBytes = MipState.NumBricks * BrickSizeBytes;

			BlockAllocatorWasteBytes += MipState.AllocatedBlocks.Num() * GDistanceFieldBlockAllocatorSizeInBricks * BrickSizeBytes - MipBrickBytes;
			MipStats[ReversedMipIndex].BrickMemoryBytes += MipBrickBytes;
			Stats.BrickMemoryBytes += MipBrickBytes;

			const SIZE_T MipIndirectionBytes = MipState.IndirectionDimensions.X * MipState.IndirectionDimensions.Y * MipState.IndirectionDimensions.Z * sizeof(uint32);
			MipStats[ReversedMipIndex].IndirectionMemoryBytes += MipIndirectionBytes;
			Stats.IndirectionMemoryBytes += MipIndirectionBytes;
		}

		AssetStats.Add(Stats);
	}

	struct FMeshDistanceFieldStatsSorter
	{
		bool operator()( const FMeshDistanceFieldStats& A, const FMeshDistanceFieldStats& B ) const
		{
			return A.BrickMemoryBytes > B.BrickMemoryBytes;
		}
	};

	AssetStats.Sort(FMeshDistanceFieldStatsSorter());

	const FIntVector AtlasDimensions = BrickTextureDimensionsInBricks * DistanceField::BrickSize;
	const SIZE_T AtlasSizeBytes = AtlasDimensions.X * AtlasDimensions.Y * AtlasDimensions.Z * GPixelFormats[DistanceField::DistanceFieldFormat].BlockBytes;
	const SIZE_T AtlasUsedBytes = DistanceFieldAtlasBlockAllocator.GetAllocatedSize() * GDistanceFieldBlockAllocatorSizeInBricks * BrickSizeBytes;
	const float BlockAllocatorWasteMb = BlockAllocatorWasteBytes / 1024.0f / 1024.0f;
	const SIZE_T IndirectionTableBytes = TryGetSize(IndirectionTable);
	const FIntVector IndirectionAtlasSize = IndirectionAtlas ? IndirectionAtlas->GetDesc().GetSize() : FIntVector::ZeroValue;
	const SIZE_T IndirectionTextureBytes = IndirectionAtlasSize.X * IndirectionAtlasSize.Y * IndirectionAtlasSize.Z * GPixelFormats[PF_A2B10G10R10].BlockBytes;
	const int32 BrickAtlasSizeXYInBricks = CVarBrickAtlasSizeXYInBricks.GetValueOnRenderThread();
	const float MaxAtlasSizeMb = CVarMaxAtlasDepthInBricks.GetValueOnRenderThread() * BrickAtlasSizeXYInBricks * BrickAtlasSizeXYInBricks * BrickSizeBytes / 1024.0f / 1024.0f;

	UE_LOGF(LogDistanceField, Log,
		"Mesh Distance Field Atlas %ux%ux%u = %.1fMb (%.1fMb target max), with %.1fMb free, %.1fMb block allocator waste, Indirection Table %.1fMb, Indirection Texture %.1fMb", 
		AtlasDimensions.X,
		AtlasDimensions.Y,
		AtlasDimensions.Z,
		AtlasSizeBytes / 1024.0f / 1024.0f,
		MaxAtlasSizeMb,
		(AtlasSizeBytes - AtlasUsedBytes) / 1024.0f / 1024.0f,
		BlockAllocatorWasteMb,
		IndirectionTableBytes / 1024.0f / 1024.0f,
		IndirectionTextureBytes / 1024.0f / 1024.0f);

	for (int32 ReversedMipIndex = 0; ReversedMipIndex < DistanceField::NumMips; ReversedMipIndex++)
	{
		UE_LOGF(LogDistanceField, Log, "   Bricks at Mip%u: %.1fMb, %.1f%%", 
			ReversedMipIndex,
			MipStats[ReversedMipIndex].BrickMemoryBytes / 1024.0f / 1024.0f,
			100.0f * MipStats[ReversedMipIndex].BrickMemoryBytes / (float)AtlasUsedBytes);
	}

	if (bDumpAssetStats)
	{
		UE_LOGF(LogDistanceField, Log, "");
		UE_LOGF(LogDistanceField, Log, "Dumping mesh distance fields for %u mesh assets", AssetStats.Num());
		UE_LOGF(LogDistanceField, Log, "   Memory Mb, Loaded Mips / Wanted Mips, Mip0 Resolution, Asset Name");

		for (int32 EntryIndex = 0; EntryIndex < AssetStats.Num(); EntryIndex++)
		{
			const FMeshDistanceFieldStats& MeshStats = AssetStats[EntryIndex];

			UE_LOGF(LogDistanceField, Log, "   %.2fMb, %u%ls, %dx%dx%d, %ls", 
				(MeshStats.BrickMemoryBytes + MeshStats.IndirectionMemoryBytes) / 1024.0f / 1024.0f, 
				MeshStats.LoadedMips,
				MeshStats.LoadedMips == MeshStats.WantedMips ? TEXT("") : *FString::Printf(TEXT(" / %u"), MeshStats.WantedMips),
				MeshStats.Resolution.X, 
				MeshStats.Resolution.Y,
				MeshStats.Resolution.Z, 
				*MeshStats.AssetName.ToString());
		}
	}
}
