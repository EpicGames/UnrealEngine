// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistanceFieldVisualization.cpp
=============================================================================*/

#include "DistanceFieldAmbientOcclusion.h"
#include "DeferredShadingRenderer.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "DistanceFieldLightingShared.h"
#include "ScenePrivate.h"
#include "ScreenRendering.h"
#include "DistanceFieldLightingPost.h"
#include "OneColorShader.h"
#include "GlobalDistanceField.h"
#include "FXSystem.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "PipelineStateCache.h"

int32 GDistanceFieldsVisualizeMonochrome = 0;
FAutoConsoleVariableRef CVarDistanceFieldsVisualizeMonochrome(
	TEXT("r.DistanceFields.Visualize.Monochrome"),
	GDistanceFieldsVisualizeMonochrome,
	TEXT("Whether to render the distance field visualization in monochrome"),
	ECVF_RenderThreadSafe
);

class FVisualizeMeshDistanceFieldCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeMeshDistanceFieldCS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeMeshDistanceFieldCS, FGlobalShader);

	class FUseGlobalDistanceFieldDim : SHADER_PERMUTATION_BOOL("USE_GLOBAL_DISTANCE_FIELD");
	class FMonochromeVisualizationDim : SHADER_PERMUTATION_BOOL("MONOCHROME_VISUALIZATION");
	class FCoverageBasedExpand : SHADER_PERMUTATION_BOOL("GLOBALSDF_USE_COVERAGE_BASED_EXPAND");
	class FSimpleCoverageBasedExpand : SHADER_PERMUTATION_BOOL("GLOBALSDF_SIMPLE_COVERAGE_BASED_EXPAND");
	class FOffsetDataStructure : SHADER_PERMUTATION_INT("OFFSET_DATA_STRUCT", 3);
	using FPermutationDomain = TShaderPermutationDomain<FUseGlobalDistanceFieldDim, FMonochromeVisualizationDim, FCoverageBasedExpand, FSimpleCoverageBasedExpand, FOffsetDataStructure>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FReflectionUniformParameters, ReflectionStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldCulledObjectBufferParameters, DistanceFieldCulledObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAOParameters, AOParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGlobalDistanceFieldParameters2, GlobalDistanceFieldParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileSceneTextureUniformParameters, MobileSceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWVisualizeMeshDistanceFields)
		SHADER_PARAMETER(FVector2f, NumGroups)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (!PermutationVector.Get<FCoverageBasedExpand>() && PermutationVector.Get<FSimpleCoverageBasedExpand>())
		{
			return false;
		}

		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), GAODownsampleFactor);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GDistanceFieldAOTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GDistanceFieldAOTileSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeMeshDistanceFieldCS, "/Engine/Private/DistanceFieldVisualization.usf", "VisualizeMeshDistanceFieldCS", SF_Compute);

class FVisualizeDistanceFieldUpsamplePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeDistanceFieldUpsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeDistanceFieldUpsamplePS, FGlobalShader);
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VisualizeDistanceFieldTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, VisualizeDistanceFieldSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLE_FACTOR"), GAODownsampleFactor);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeDistanceFieldUpsamplePS, "/Engine/Private/DistanceFieldVisualization.usf", "VisualizeDistanceFieldUpsamplePS", SF_Pixel);

void FSceneRenderer::RenderMeshDistanceFieldVisualization(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures)
{
	const bool bVisualizeGlobalDistanceField = ViewFamily.EngineShowFlags.VisualizeGlobalDistanceField;
	const bool bVisualizeMeshDistanceFields = ViewFamily.EngineShowFlags.VisualizeMeshDistanceFields
		&& (Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0)
		&& !bVisualizeGlobalDistanceField; // only one visualization enabled at a time

	if (!DoesPlatformSupportDistanceFields(ShaderPlatform)
		|| !IsUsingDistanceFields(ShaderPlatform)
		|| !(bVisualizeGlobalDistanceField || bVisualizeMeshDistanceFields))
	{
		return;
	}

	check(!Scene->DistanceFieldSceneData.HasPendingOperations());

	QUICK_SCOPE_CYCLE_COUNTER(STAT_AOIssueGPUWork);

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeMeshDistanceFields");

	FRDGTextureRef VisualizeResultTexture = nullptr;
	{
		const FViewInfo& FirstView = Views[0];
		const FIntPoint BufferSize = GetBufferSizeForAO(FirstView);
		const FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(BufferSize, PF_FloatRGBA, FClearValueBinding::None, TexCreate_UAV));
		VisualizeResultTexture = GraphBuilder.CreateTexture(Desc, TEXT("VisualizeDistanceField"));
	}

	const FDistanceFieldAOParameters DFAOParameters = FDistanceFieldAOParameters(Scene->DefaultMaxDistanceFieldOcclusionDistance);

	for (const FViewInfo& View : Views)
	{
		FRDGBufferRef ObjectIndirectArguments = nullptr;
		FDistanceFieldCulledObjectBufferParameters CulledObjectBufferParameters;

		if (bVisualizeMeshDistanceFields)
		{
			AllocateDistanceFieldCulledObjectBuffers(
				GraphBuilder,
				FMath::DivideAndRoundUp(Scene->DistanceFieldSceneData.NumObjectsInBuffer, 256) * 256,
				ObjectIndirectArguments,
				CulledObjectBufferParameters);

			CullObjectsToView(GraphBuilder, *Scene, View, DFAOParameters, CulledObjectBufferParameters);
		}

		uint32 GroupSizeX = FMath::DivideAndRoundUp(View.ViewRect.Size().X / GAODownsampleFactor, GDistanceFieldAOTileSizeX);
		uint32 GroupSizeY = FMath::DivideAndRoundUp(View.ViewRect.Size().Y / GAODownsampleFactor, GDistanceFieldAOTileSizeY);

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		auto* PassParameters = GraphBuilder.AllocParameters<FVisualizeMeshDistanceFieldCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		PassParameters->ReflectionStruct = CreateReflectionUniformBuffer(GraphBuilder, View);
		PassParameters->DistanceFieldObjectBuffers = DistanceField::SetupObjectBufferParameters(GraphBuilder, Scene->DistanceFieldSceneData);
		PassParameters->DistanceFieldCulledObjectBuffers = CulledObjectBufferParameters;
		PassParameters->DistanceFieldAtlas = DistanceField::SetupAtlasParameters(GraphBuilder, Scene->DistanceFieldSceneData);
		PassParameters->AOParameters = DistanceField::SetupAOShaderParameters(DFAOParameters);
		PassParameters->GlobalDistanceFieldParameters = SetupGlobalDistanceFieldParameters(View.GlobalDistanceFieldInfo.ParameterData);
		PassParameters->NumGroups = FVector2f(GroupSizeX, GroupSizeY);
		PassParameters->SceneTextures = SceneTextures.UniformBuffer;
		PassParameters->MobileSceneTextures = SceneTextures.MobileUniformBuffer;
		PassParameters->RWVisualizeMeshDistanceFields = GraphBuilder.CreateUAV(VisualizeResultTexture);

		FVisualizeMeshDistanceFieldCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FVisualizeMeshDistanceFieldCS::FUseGlobalDistanceFieldDim>(bVisualizeGlobalDistanceField);
		PermutationVector.Set<FVisualizeMeshDistanceFieldCS::FMonochromeVisualizationDim>(GDistanceFieldsVisualizeMonochrome != 0);
		PermutationVector.Set<FVisualizeMeshDistanceFieldCS::FCoverageBasedExpand>(IsLumenEnabled(View));
		PermutationVector.Set<FVisualizeMeshDistanceFieldCS::FSimpleCoverageBasedExpand>(IsLumenEnabled(View) && Lumen::UseGlobalSDFSimpleCoverageBasedExpand());
		extern int32 GDistanceFieldOffsetDataStructure;
		PermutationVector.Set<FVisualizeMeshDistanceFieldCS::FOffsetDataStructure>(GDistanceFieldOffsetDataStructure);

		auto ComputeShader = View.ShaderMap->GetShader<FVisualizeMeshDistanceFieldCS>(PermutationVector);

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("VisualizeMeshDistanceFieldCS"), ComputeShader, PassParameters, FIntVector(GroupSizeX, GroupSizeY, 1));
	}

	for (const FViewInfo& View : Views)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		auto* PassParameters = GraphBuilder.AllocParameters<FVisualizeDistanceFieldUpsamplePS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->VisualizeDistanceFieldTexture = VisualizeResultTexture;
		PassParameters->VisualizeDistanceFieldSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);

		const FScreenPassTextureViewport InputViewport(VisualizeResultTexture, GetDownscaledRect(View.ViewRect, GAODownsampleFactor));
		const FScreenPassTextureViewport OutputViewport(SceneTextures.Color.Target, View.ViewRect);

		auto PixelShader = View.ShaderMap->GetShader<FVisualizeDistanceFieldUpsamplePS>();

		AddDrawScreenPass(GraphBuilder, {}, View, OutputViewport, InputViewport, PixelShader, PassParameters);
	}
}
