// Copyright Epic Games, Inc. All Rights Reserved.

#include "DistortionRendering.h"
#include "CoreMinimal.h"
#include "BlueNoise.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DecalRenderingShared.h"
#include "DeferredShadingRenderer.h"
#include "DynamicRHI.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "Logging/LogMacros.h"
#include "MaterialDomain.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "MeshBatch.h"
#include "MeshDrawCommands.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "Misc/LargeWorldCoordinates.h"
#include "MobileDistortionPass.h"
#include "MultiGPU.h"
#include "PostProcess/TemporalAA.h"
#include "PrimitiveSceneProxy.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "PSOPrecache.h"
#include "PSOPrecacheMaterial.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "ScreenRendering.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderPlatformCachedIniValue.h"
#include "ShowFlags.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "TranslucencyPass.h"
#include "TranslucentPassResource.h"
#include "TranslucentRendering.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"
#include "SceneViewState.h"


DEFINE_GPU_DRAWCALL_STAT(Distortion);

static TAutoConsoleVariable<int32> CVarDisableDistortion(
	TEXT("r.DisableDistortion"),
	0,
	TEXT("Prevents distortion effects from rendering.  Saves a full-screen framebuffer's worth of memory."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarRefractionBlur(
	TEXT("r.Refraction.Blur"),
	0, // Substrate 60fps by default
	TEXT("Enables rough refractions, i.e. blurring of the background. Only enabled when Substrate is enabled."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRefractionOffsetQuality(
	TEXT("r.Refraction.OffsetQuality"),
	0,
	TEXT("When enabled, the offset buffer is made float for higher quality. This is important to maintain the softness of the blurred scene buffer."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRefractionBlurMaxStandardDeviationInScreenPercent(
	TEXT("r.Refraction.Blur.MaxStandardDeviationInScreenRatio"),
	5.0f,
	TEXT("This will clamp the maximum refraction blur radius on screen."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRefractionBlurTemporalAA(
	TEXT("r.Refraction.Blur.TemporalAA"),
	1,
	TEXT("Enables temporal AA of the scene color buffer in order to avoid flickering in rough refractions."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRefractionBlurMaxExposedLuminance(
	TEXT("r.Refraction.Blur.MaxExposedLuminance"),
	10.0f,
	TEXT("Clamp scene pre-exposed luminance to this maximum value. It helps to reduce bright specular highlight flickering, even when r.Refraction.Blur.TemporalAA=1."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRefractionBlurDitheringScaleFactor(
	TEXT("r.Refraction.Blur.DitheringScaleFactor"),
	0.3f,
	TEXT("Represents the radius scale factor of the dithered noise applied on the rough refraction UV sample. A value of 1 means that the UV offset will be rotated on circle of radius = pixel standard deviation. A large value can confuse TAA/TSR and result on color splotches. Disabled when 0, accept [0,1] range."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRefractionUseAVSM(
	TEXT("r.HeterogeneousVolumes.CompositeWithTranslucency.Refraction.UseAVSM"),
	1,
	TEXT("Enables AVSM lookup (Default = 1)\n")
	TEXT("Requires enabling Heterogeneous Volumes Project Setting: 'Composite with Translucency'"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRefractionTransmittanceThreshold(
	TEXT("r.HeterogeneousVolumes.CompositeWithTranslucency.Refraction.TransmittanceThreshold"),
	0.9f,
	TEXT("Minimum transmittance threshold to apply distortion (Default = 0.9)\n")
	TEXT("Requires enabling Heterogeneous Volumes Project Setting: 'Composite with Translucency'"),
	ECVF_RenderThreadSafe);

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FDistortionPassUniformParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT(FSubstrateForwardPassUniformParameters, Substrate)
	SHADER_PARAMETER(FVector4f, DistortionParams)
	SHADER_PARAMETER_STRUCT(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)
	SHADER_PARAMETER(int32, UseAVSM)
	SHADER_PARAMETER(float, TransmittanceThreshold)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FDistortionPassUniformParameters, "DistortionPass", SceneTextures);

uint32 GetRefractionOffsetQuality(EShaderPlatform InPlatform)
{
	static FShaderPlatformCachedIniValue<int32> CVarRefractionOffsetQualityPlatform(TEXT("r.Refraction.OffsetQuality"));
	return CVarRefractionOffsetQualityPlatform.Get(InPlatform);
}

int32 GetGlobalRefractionQuality()
{
	static const auto ICVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RefractionQuality"));
	return ICVar->GetValueOnAnyThread();
}

int32 FSceneRenderer::GetRefractionQuality(const FSceneViewFamily& ViewFamily)
{
	return ViewFamily.EngineShowFlags.Refraction ? GetGlobalRefractionQuality() : 0;
}

void SetupDistortionParams(FVector4f& DistortionParams, const FViewInfo& View)
{
	float Ratio = View.UnscaledViewRect.Width() / (float)View.UnscaledViewRect.Height();
	DistortionParams.X = View.ViewMatrices.GetViewToClip().M[0][0];
	DistortionParams.Y = Ratio;
	DistortionParams.Z = (float)View.UnscaledViewRect.Width();
	DistortionParams.W = (float)View.UnscaledViewRect.Height();

	// When ISR or MMV is enabled we store two FOVs in the distortion parameters and compute the aspect ratio in the shader instead.
	if (View.bIsSinglePassStereo)
	{
		const FSceneView* InstancedView = View.GetInstancedView();
		if (InstancedView)
		{
			DistortionParams.Y = InstancedView->ViewMatrices.GetViewToClip().M[0][0];
		}
	}
}

TRDGUniformBufferRef<FDistortionPassUniformParameters> CreateDistortionPassUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	auto* Parameters = GraphBuilder.AllocParameters<FDistortionPassUniformParameters>();
	SetupSceneTextureUniformParameters(GraphBuilder, View.GetSceneTexturesChecked(), View.FeatureLevel, ESceneTextureSetupMode::All, Parameters->SceneTextures);
	SetupDistortionParams(Parameters->DistortionParams, View);
	Substrate::BindSubstrateForwardPasslUniformParameters(GraphBuilder, View, Parameters->Substrate);
	Parameters->AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricCameraMapParameters(GraphBuilder, View.ViewState);
	Parameters->UseAVSM = CVarRefractionUseAVSM.GetValueOnRenderThread() != 0;
	Parameters->TransmittanceThreshold = FMath::Clamp(CVarRefractionTransmittanceThreshold.GetValueOnRenderThread(), 0.0, 1.0);

	return GraphBuilder.CreateUniformBuffer(Parameters);
}

bool GetSubstrateEnabledUseRoughRefraction()
{
	return Substrate::IsSubstrateEnabled() && !CVarDisableDistortion.GetValueOnAnyThread() && CVarRefractionBlur.GetValueOnAnyThread() > 0; // Any thread since it can be called when creating the PassProcessor
}

class FDistortionScreenPS : public FGlobalShader
{
public:
	class FUseMSAADim : SHADER_PERMUTATION_BOOL("USE_MSAA");
	class FUseRoughRefractionDim : SHADER_PERMUTATION_BOOL("USE_ROUGH_REFRACTION");
	class FCompositedStandardTranslucentDim : SHADER_PERMUTATION_BOOL("COMPOSITE_STANDARD_TRANSLUCENT");
	using FPermutationDomain = TShaderPermutationDomain<FUseMSAADim, FUseRoughRefractionDim, FCompositedStandardTranslucentDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float4>, RefractionVarianceCoverageMSAATexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float4>, RefractionClosestDepthMeterMSAATexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float4>, DistortionMSAATexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float4>, SceneColorMSAATexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float4>, SceneDepthMeterMSAATexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float4>, StandardTranslucentColorAlphaMSAATexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMS<float4>, StandardTranslucentTransmittanceMSAATexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RefractionVarianceCoverageTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RefractionClosestDepthMeterTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistortionTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthMeterTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, StandardTranslucentColorAlphaTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, StandardTranslucentTransmittanceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, RefractionVarianceCoverageSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, RefractionClosestDepthMeterSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistortionTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthMeterTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, StandardTranslucentColorAlphaTextureSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, StandardTranslucentTransmittanceTextureSampler)
		SHADER_PARAMETER(float, RefractionMaxMipLevel)
		SHADER_PARAMETER(float, StandardDevInPixelForMip0)
		SHADER_PARAMETER(float, BlurDitheringScaleFactor)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		return !PermutationVector.Get<FUseMSAADim>() || IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	FDistortionScreenPS() = default;
	FDistortionScreenPS(const FGlobalShaderType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

/** A pixel shader for rendering the full screen refraction pass */
class FDistortionApplyScreenPS : public FDistortionScreenPS
{
public:
	DECLARE_GLOBAL_SHADER(FDistortionApplyScreenPS);
	SHADER_USE_PARAMETER_STRUCT(FDistortionApplyScreenPS, FDistortionScreenPS);
};

IMPLEMENT_GLOBAL_SHADER(FDistortionApplyScreenPS, "/Engine/Private/DistortApplyScreenPS.usf", "Main", SF_Pixel);

/** A pixel shader that applies the distorted image to the scene */
class FDistortionMergeScreenPS : public FDistortionScreenPS
{
public:
	DECLARE_GLOBAL_SHADER(FDistortionMergeScreenPS);
	SHADER_USE_PARAMETER_STRUCT(FDistortionMergeScreenPS, FDistortionScreenPS);
};

IMPLEMENT_GLOBAL_SHADER(FDistortionMergeScreenPS, "/Engine/Private/DistortApplyScreenPS.usf", "Merge", SF_Pixel);

class FDistortionMeshVS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FDistortionMeshVS,MeshMaterial);

	FDistortionMeshVS() = default;

	FDistortionMeshVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsTranslucentBlendMode(Parameters.MaterialParameters) && Parameters.MaterialParameters.bIsDistorted;
	}
};

class FDistortionMeshPS : public FMeshMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FDistortionMeshPS,MeshMaterial);

	FDistortionMeshPS() = default;

	FDistortionMeshPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		:	FMeshMaterialShader(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsTranslucentBlendMode(Parameters.MaterialParameters) && Parameters.MaterialParameters.bIsDistorted;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Skip the material clip if depth test should not be done
		OutEnvironment.SetDefine(TEXT("MATERIAL_SHOULD_DISABLE_DEPTH_TEST"), Parameters.MaterialParameters.bShouldDisableDepthTest ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("ADAPTIVE_VOLUMETRIC_SHADOW_MAP"), !IsMobilePlatform(Parameters.Platform) && ShouldCompositeHeterogeneousVolumesWithTranslucency());
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FDistortionMeshVS, TEXT("/Engine/Private/DistortAccumulateVS.usf"), TEXT("Main"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(, FDistortionMeshPS,TEXT("/Engine/Private/DistortAccumulatePS.usf"),TEXT("Main"),SF_Pixel);

bool FDeferredShadingSceneRenderer::ShouldRenderDistortion() const
{
	static const auto DisableDistortionCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DisableDistortion"));
	const bool bAllowDistortion = DisableDistortionCVar->GetValueOnAnyThread() != 1;

	if (GetRefractionQuality(ViewFamily) <= 0 || !bAllowDistortion)
	{
		return false;
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (View.bHasDistortionPrimitives && View.ShouldRenderView() && HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::Distortion]))
		{
			return true;
		}
	}
	return false;
}

BEGIN_SHADER_PARAMETER_STRUCT(FDistortionPassParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FDistortionPassUniformParameters, Pass)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

//////////////////////////////////////////////////////////////////////////

class FCopySceneColorTexturePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopySceneColorTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FCopySceneColorTexturePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthSampler)
		SHADER_PARAMETER(float, MaxExposedLuminance)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COPYSCENECOLORDEPTHTEXTUREPS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCopySceneColorTexturePS, "/Engine/Private/DistortFiltering.usf", "CopySceneColorDepthTexturePS", SF_Pixel);

static void AddCopySceneColorDepthPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneColorTexture, FRDGTextureRef SceneDepthTexture, FRDGTextureRef SceneColorCopyTexture, FRDGTextureRef SceneDepthCopyTexture)
{
	const FScreenPassTextureViewport Viewport(SceneColorCopyTexture, View.ViewRect);

	TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FCopySceneColorTexturePS> PixelShader(View.ShaderMap);

	auto* PassParameters = GraphBuilder.AllocParameters<FCopySceneColorTexturePS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneColorTexture = SceneColorTexture;
	PassParameters->SceneDepthTexture = SceneDepthTexture;
	PassParameters->SceneColorSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	PassParameters->SceneDepthSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorCopyTexture, ERenderTargetLoadAction::ENoAction);
	PassParameters->RenderTargets[1] = FRenderTargetBinding(SceneDepthCopyTexture, ERenderTargetLoadAction::ENoAction);
	PassParameters->MaxExposedLuminance = FMath::Max(0.0f, CVarRefractionBlurMaxExposedLuminance.GetValueOnRenderThread());

	// The scene color is copied into from multi-view-rect layout into a single-rect layout.
	FIntRect ViewRect;
	ViewRect.Min = FIntPoint(0, 0);
	ViewRect.Max = FIntPoint(View.ViewRect.Width(), View.ViewRect.Height());

	const FScreenPassTextureViewport InputViewport(SceneColorTexture, ViewRect);
	const FScreenPassTextureViewport OutputViewport(SceneColorCopyTexture);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Distortion::CopySceneColorDepth"), View, InputViewport, OutputViewport, VertexShader, PixelShader, PassParameters);
}

//////////////////////////////////////////////////////////////////////////

class FDownsampleSceneColorCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDownsampleSceneColorCS);
	SHADER_USE_PARAMETER_STRUCT(FDownsampleSceneColorCS, FGlobalShader);

	static const uint32 ThreadGroupSize = 8;

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, SrcMipResolution)
		SHADER_PARAMETER(FIntPoint, DstMipResolution)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SourceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTextureMipColor)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLECOLORCS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FDownsampleSceneColorCS, "/Engine/Private/DistortFiltering.usf", "DownsampleColorCS", SF_Compute);

static void AddDownsampleSceneColorPass(FRDGBuilder& GraphBuilder, FDownsampleSceneColorCS::FParameters* PassParameters, const FViewInfo& View)
{
	FIntVector NumGroups = FIntVector::DivideAndRoundUp(
		FIntVector(PassParameters->DstMipResolution.X, PassParameters->DstMipResolution.Y, 1), 
		FIntVector(FDownsampleSceneColorCS::ThreadGroupSize, FDownsampleSceneColorCS::ThreadGroupSize, 1));

	FDownsampleSceneColorCS::FPermutationDomain PermutationVector;
	TShaderMapRef<FDownsampleSceneColorCS> ComputeShader(View.ShaderMap, PermutationVector);

	// Dispatch with GenerateMips: reading from a slice through SRV and writing into lower mip through UAV.
	ClearUnusedGraphResources(ComputeShader, PassParameters);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("DistoMipGen"), ComputeShader, PassParameters, NumGroups);
}

//////////////////////////////////////////////////////////////////////////

class FGaussianFiltering
{
public:
	// Evaluates an unnormalized normal distribution PDF around 0 at given X with Variance.
	static float NormalDistributionUnscaled(float X, float Sigma)
	{
		const float DX = FMath::Abs(X);

		// Unscaled so no front factor, because Compute1DGaussianFilterKernel normalizes weights after
		const float Gaussian = FMath::Exp(-0.5f * FMath::Square(DX / Sigma)); 
		return Gaussian;
	}

	// Sigma is Standard Deviation in pixel
	// Variance = sigma x sigma
	static uint32 Compute1DGaussianFilterKernel(FVector2D* OutOffsetAndWeight, uint32 SampleCountRadius, uint32 MaxSampleCountRadius, float Sigma)
	{
		const float IntegerSampleCountRadius = FMath::Clamp<float>(SampleCountRadius, 1, MaxSampleCountRadius);

		uint32 SampleCount = 0;
		float WeightSum = 0.0f;

		for (int32 SampleIndex = -IntegerSampleCountRadius; SampleIndex <= IntegerSampleCountRadius; SampleIndex += 2)
		{
			float Weight0 = NormalDistributionUnscaled(SampleIndex, Sigma);
			float Weight1 = 0.0f;

			// We use the bilinear filter optimization for gaussian blur. However, we don't want to bias the
			// last sample off the edge of the filter kernel, so the very last tap just is on the pixel center.
			if (SampleIndex != IntegerSampleCountRadius)
			{
				Weight1 = NormalDistributionUnscaled(SampleIndex + 1, Sigma);
			}

			const float TotalWeight = Weight0 + Weight1;
			OutOffsetAndWeight[SampleCount].X = SampleIndex + (Weight1 / TotalWeight);
			OutOffsetAndWeight[SampleCount].Y = TotalWeight;
			WeightSum += TotalWeight;
			SampleCount++;
		}

		// Normalize blur weights.
		const float WeightSumInverse = 1.0f / WeightSum;
		for (uint32 SampleIndex = 0; SampleIndex < SampleCount; ++SampleIndex)
		{
			OutOffsetAndWeight[SampleIndex].Y *= WeightSumInverse;
		}

		return SampleCount;
	}
};

// If this is update, please update DistortFiltering.usf
#define MAX_FILTER_SAMPLE_COUNT_RADIUS	32
#define MAX_FILTER_SAMPLE_COUNT	(MAX_FILTER_SAMPLE_COUNT_RADIUS*2+1)

class FFilterSceneColorCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FFilterSceneColorCS);
	SHADER_USE_PARAMETER_STRUCT(FFilterSceneColorCS, FGlobalShader);

	static const uint32 ThreadGroupSize = 8;

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, SampleCount)
		SHADER_PARAMETER(FIntPoint, SrcMipResolution)
		SHADER_PARAMETER(FIntPoint, DstMipResolution)
		SHADER_PARAMETER(FVector4f, BlurDirection)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SourceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTextureMipColor)
		SHADER_PARAMETER_ARRAY(FVector4f, SampleOffsetsWeights, [MAX_FILTER_SAMPLE_COUNT])
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("FILTERCOLORCS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FFilterSceneColorCS, "/Engine/Private/DistortFiltering.usf", "FilterColorCS", SF_Compute);

static void AddFilterSceneColorPass(FRDGBuilder& GraphBuilder, FFilterSceneColorCS::FParameters* PassParameters, const FViewInfo& View)
{
	FIntVector NumGroups = FIntVector::DivideAndRoundUp(
		FIntVector(PassParameters->DstMipResolution.X, PassParameters->DstMipResolution.Y, 1),
		FIntVector(FFilterSceneColorCS::ThreadGroupSize, FFilterSceneColorCS::ThreadGroupSize, 1));

	FFilterSceneColorCS::FPermutationDomain PermutationVector;
	TShaderMapRef<FFilterSceneColorCS> ComputeShader(View.ShaderMap, PermutationVector);

	// Dispatch with GenerateMips: reading from a slice through SRV and writing into lower mip through UAV.
	ClearUnusedGraphResources(ComputeShader, PassParameters);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("FilterMipGen"), ComputeShader, PassParameters, NumGroups);
}

//////////////////////////////////////////////////////////////////////////

void FDeferredShadingSceneRenderer::RenderDistortion(
	FRDGBuilder& GraphBuilder, 
	FRDGTextureRef SceneColorTexture, 
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef SceneVelocityTexture,
	FTranslucencyPassResourcesMap& TranslucencyResourceMap)
{
	check(SceneDepthTexture);
	check(SceneColorTexture);

	if (!ShouldRenderDistortion())
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRenderer_RenderDistortion);
	SCOPED_NAMED_EVENT(RenderDistortion, FColor::Emerald);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, Distortion, "Distortion");

	const FDepthStencilBinding StencilReadBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
	FDepthStencilBinding StencilWriteBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);

	FRDGTextureRef DistortionTexture = nullptr;
	FRDGTextureRef RefractionVarianceCoverage = nullptr;
	FRDGTextureRef RefractionClosestDepthMeter = nullptr;
	FRDGTextureRef TempSceneColorMipchainTexture = nullptr;
	FRDGTextureRef TempSceneDepthMeterMipchainTexture = nullptr;
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

	const float FilterSampleCountRadius = 5.0f;	// The maximum sample count we will run. This is scaled down by FilterSizeScale.
	const float FilterSigmaPixel = 2.0f;		// Variance in pixel, even though it is not really exact since weights are renormalizes, but it does control the spread

	// Create a texture of the scene color with MIP levels, unfiltered
	// NOTE: we cannot use mips on the scene color render target it self because multiple views are continuous in there and they would leak color one onto another.
	TArray<FRDGTextureRef> ViewsSceneColorMipchain;
	TArray<FRDGTextureRef> ViewsSceneDepthMeterMipchain;
	ViewsSceneColorMipchain.SetNum(Views.Num());
	ViewsSceneDepthMeterMipchain.SetNum(Views.Num());
	const bool bUseRoughRefraction = GetSubstrateEnabledUseRoughRefraction();
	if(bUseRoughRefraction)
	{
		FVector2D OffsetAndWeight[MAX_FILTER_SAMPLE_COUNT];
		uint32 SampleCount = FGaussianFiltering::Compute1DGaussianFilterKernel(OffsetAndWeight, FilterSampleCountRadius, MAX_FILTER_SAMPLE_COUNT_RADIUS, FilterSigmaPixel);

		for (int32 ViewIndex = 0, Num = Views.Num(); ViewIndex < Num; ++ViewIndex)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "Rough Refraction View%d", ViewIndex);

			const FViewInfo& View = Views[ViewIndex];
			const FIntPoint SceneColorMip0Resolution = View.ViewRect.Size();

			// We do not use max and do not add 1 to the result, this to not go down to the lowest mip level as this is not required.
			const int32 MipCount = FMath::Max((int32)1, (int32)FMath::CeilLogTwo(FMath::Min(SceneColorMip0Resolution.X, SceneColorMip0Resolution.Y)));

			FRDGTextureDesc SceneColorMipchainDesc = FRDGTextureDesc::Create2D(SceneColorMip0Resolution, PF_FloatR11G11B10, FClearValueBinding::None,
				TexCreate_TargetArraySlicesIndependently | TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable, MipCount);
			FRDGTextureRef SceneColorMipchainTexture = GraphBuilder.CreateTexture(SceneColorMipchainDesc, TEXT("SceneColorMipchain"));
			ViewsSceneColorMipchain[ViewIndex] = SceneColorMipchainTexture;

			FRDGTextureDesc SceneDepthMeterMipchainDesc = SceneColorMipchainDesc;
			SceneDepthMeterMipchainDesc.Format = PF_R16F; // Store depth form camera in meters, depth up to 64Km.
			FRDGTextureRef SceneDepthMeterMipchainTexture = GraphBuilder.CreateTexture(SceneDepthMeterMipchainDesc, TEXT("SceneDepthMeterMipchain"));
			ViewsSceneDepthMeterMipchain[ViewIndex] = SceneDepthMeterMipchainTexture;

			if (!TempSceneColorMipchainTexture || TempSceneColorMipchainTexture->Desc.Extent != SceneColorMipchainTexture->Desc.Extent)
			{
				// Only allocate a new temporary if it needs to have another resolution.
				TempSceneColorMipchainTexture = GraphBuilder.CreateTexture(SceneColorMipchainDesc, TEXT("TempSceneColorMipchain"));
				TempSceneDepthMeterMipchainTexture = GraphBuilder.CreateTexture(SceneDepthMeterMipchainDesc, TEXT("TempSceneDepthMeterMipchain"));
			}

			// Apply temporal AA on the current scene buffer to avoid large flickering in rough refraction

			FRDGTextureRef TAASceneColorTexture = SceneColorTexture;
			if(View.ViewState && CVarRefractionBlurTemporalAA.GetValueOnRenderThread() > 0)
			{
				FTAAPassParameters TAASettings(View);
				TAASettings.SceneDepthTexture = SceneDepthTexture;

				// We need a valid velocity buffer texture. Use black (no velocity) if it's not produced.
				TAASettings.SceneVelocityTexture = GetIfProduced(SceneVelocityTexture, GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy));
				TAASettings.Pass = ETAAPassConfig::Main;		// Reusing main config for now. We could add a ReoughRefraction config forcing 111110 format.
				TAASettings.SceneColorInput = TAASceneColorTexture;
				TAASettings.bOutputRenderTargetable = true;

				FTAAOutputs TAAOutputs = AddTemporalAAPass(
					GraphBuilder,
					View,
					TAASettings,
					View.PrevViewInfo.RoughRefractionHistory,
					&View.ViewState->PrevFrameViewInfo.RoughRefractionHistory);

				TAASceneColorTexture = TAAOutputs.SceneColor;
			}

			// Copy scene color into the first mip level

			AddCopySceneColorDepthPass(GraphBuilder, View, TAASceneColorTexture, SceneDepthTexture, SceneColorMipchainTexture, SceneDepthMeterMipchainTexture);

			// Now render the mip chain
			// SUBSTRATE_TODO we could optimize that pass by doing one pass with a tile of 16x16 writing out the 8x8, 4x4, 2x2 and 1x1 down sampled output

			{
				RDG_EVENT_SCOPE(GraphBuilder, "SceneColorMipChain");

				FIntPoint SrcMipResolution = SceneColorMip0Resolution;
				FIntPoint DstMipResolution = SceneColorMip0Resolution / 2;
				for (int32 DstMipIndex = 1; DstMipIndex < MipCount; DstMipIndex++)
				{
					FDownsampleSceneColorCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDownsampleSceneColorCS::FParameters>();
					PassParameters->SrcMipResolution = SrcMipResolution;
					PassParameters->DstMipResolution = DstMipResolution;
					PassParameters->SourceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

					PassParameters->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SceneColorMipchainTexture, DstMipIndex - 1));
					PassParameters->OutTextureMipColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SceneColorMipchainTexture, DstMipIndex));
					AddDownsampleSceneColorPass(GraphBuilder, PassParameters, View);

					FDownsampleSceneColorCS::FParameters* PassParametersDepth = GraphBuilder.AllocParameters<FDownsampleSceneColorCS::FParameters>();
					*PassParametersDepth = *PassParameters;
					PassParametersDepth->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SceneDepthMeterMipchainTexture, DstMipIndex - 1));
					PassParametersDepth->OutTextureMipColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SceneDepthMeterMipchainTexture, DstMipIndex));
					AddDownsampleSceneColorPass(GraphBuilder, PassParametersDepth, View);

					SrcMipResolution = DstMipResolution;
					DstMipResolution = DstMipResolution / 2;
				}
			}

			// Now the horizontal blur

			{
				RDG_EVENT_SCOPE(GraphBuilder, "SceneColorDepthMipHBlur");

				FIntPoint SrcMipResolution = SceneColorMip0Resolution;
				FIntPoint DstMipResolution = SceneColorMip0Resolution / 2;
				for (int32 MipIndex = 1; MipIndex < MipCount; MipIndex++)
				{
					const FVector2D InverseFilterTextureExtent(1.0f / static_cast<float>(DstMipResolution.X), 1.0f / static_cast<float>(DstMipResolution.Y));

					FFilterSceneColorCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFilterSceneColorCS::FParameters>();
					PassParameters->SrcMipResolution = SrcMipResolution;
					PassParameters->DstMipResolution = DstMipResolution;
					PassParameters->SourceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
					PassParameters->SampleCount = SampleCount;
					PassParameters->BlurDirection = FVector4f(1.0f, 0.0f, 0.0f, 0.0f);

					for (uint32 i = 0; i < SampleCount; ++i)
					{
						PassParameters->SampleOffsetsWeights[i] = FVector4f(InverseFilterTextureExtent.X * OffsetAndWeight[i].X, OffsetAndWeight[i].Y);
					}

					PassParameters->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SceneColorMipchainTexture, MipIndex));
					PassParameters->OutTextureMipColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(TempSceneColorMipchainTexture, MipIndex));
					AddFilterSceneColorPass(GraphBuilder, PassParameters, View);

					FFilterSceneColorCS::FParameters* PassParametersDepth = GraphBuilder.AllocParameters<FFilterSceneColorCS::FParameters>();
					*PassParametersDepth = *PassParameters;
					PassParametersDepth->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SceneDepthMeterMipchainTexture, MipIndex));
					PassParametersDepth->OutTextureMipColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(TempSceneDepthMeterMipchainTexture, MipIndex));
					AddFilterSceneColorPass(GraphBuilder, PassParametersDepth, View);

					SrcMipResolution = DstMipResolution;
					DstMipResolution = DstMipResolution / 2;
				}
			}

			// Now the vertical blur

			{
				RDG_EVENT_SCOPE(GraphBuilder, "SceneColorMipVBlur");

				FIntPoint SrcMipResolution = SceneColorMip0Resolution;
				FIntPoint DstMipResolution = SceneColorMip0Resolution / 2;
				for (int32 MipIndex = 1; MipIndex < MipCount; MipIndex++)
				{
					const FVector2D InverseFilterTextureExtent(1.0f / static_cast<float>(DstMipResolution.X), 1.0f / static_cast<float>(DstMipResolution.Y));

					FFilterSceneColorCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFilterSceneColorCS::FParameters>();
					PassParameters->SrcMipResolution = SrcMipResolution;
					PassParameters->DstMipResolution = DstMipResolution;
					PassParameters->SourceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
					PassParameters->SampleCount = SampleCount;
					PassParameters->BlurDirection = FVector4f(0.0f, 1.0f, 0.0f, 0.0f);

					for (uint32 i = 0; i < SampleCount; ++i)
					{
						PassParameters->SampleOffsetsWeights[i] = FVector4f(InverseFilterTextureExtent.Y * OffsetAndWeight[i].X, OffsetAndWeight[i].Y);
					}

					PassParameters->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(TempSceneColorMipchainTexture, MipIndex));
					PassParameters->OutTextureMipColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SceneColorMipchainTexture, MipIndex));
					AddFilterSceneColorPass(GraphBuilder, PassParameters, View);

					FFilterSceneColorCS::FParameters* PassParametersDepth = GraphBuilder.AllocParameters<FFilterSceneColorCS::FParameters>();
					*PassParametersDepth = *PassParameters;
					PassParametersDepth->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(TempSceneDepthMeterMipchainTexture, MipIndex));
					PassParametersDepth->OutTextureMipColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SceneDepthMeterMipchainTexture, MipIndex));
					AddFilterSceneColorPass(GraphBuilder, PassParametersDepth, View);

					SrcMipResolution = DstMipResolution;
					DstMipResolution = DstMipResolution / 2;
				}
			}
		}
	}


	// Use stencil mask to optimize cases with lower screen coverage.
	// Note: This adds an extra pass which is actually slower as distortion tends towards full-screen.
	//       It could be worth testing object screen bounds then reverting to a target flip and single pass.

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRenderer_RenderDistortion_Accumulate);

		// Use RGBA8 light target for accumulating distortion offsets.
		// R = positive X offset
		// G = positive Y offset
		// B = negative X offset
		// A = negative Y offset

		DistortionTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(
				SceneDepthTexture->Desc.Extent,
				GetRefractionOffsetQuality(ShaderPlatform) > 0 ? PF_FloatRGBA : PF_B8G8R8A8,
				FClearValueBinding::Transparent,
				GFastVRamConfig.Distortion | TexCreate_RenderTargetable | TexCreate_ShaderResource,
				1,
				SceneDepthTexture->Desc.NumSamples),
			TEXT("Distortion"));

		if (bUseRoughRefraction)
		{
			// This is the texture containing information about the surface back scattering process
			RefractionVarianceCoverage = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(
					SceneDepthTexture->Desc.Extent,
					PF_G16R16F,
					FClearValueBinding::Black,
					GFastVRamConfig.Distortion | TexCreate_RenderTargetable | TexCreate_ShaderResource,
					1,
					SceneDepthTexture->Desc.NumSamples),
				TEXT("RefractionVarianceCoverage"));
			RefractionClosestDepthMeter = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(
					SceneDepthTexture->Desc.Extent,
					PF_R16F,
					FClearValueBinding(FLinearColor(65504.0f, 1.0f, 1.0f, 1.0f)),	// Max 16bits float depth
					GFastVRamConfig.Distortion | TexCreate_RenderTargetable | TexCreate_ShaderResource,
					1,
					SceneDepthTexture->Desc.NumSamples),
				TEXT("RefractionClosestDepthMeter"));
		}

		ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::EClear;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FViewInfo& View = Views[ViewIndex];
			const ETranslucencyView TranslucencyView = GetTranslucencyView(View);

			if (!View.ShouldRenderView() && !EnumHasAnyFlags(TranslucencyView, ETranslucencyView::RayTracing))
			{
				continue;
			}

			if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::Distortion])
			{
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
				View.BeginRenderView();

				auto* PassParameters = GraphBuilder.AllocParameters<FDistortionPassParameters>();
				PassParameters->View = View.GetShaderParameters();
				PassParameters->Pass = CreateDistortionPassUniformBuffer(GraphBuilder, View);
				PassParameters->RenderTargets[0] = FRenderTargetBinding(DistortionTexture, LoadAction);
				if (bUseRoughRefraction)
				{
					PassParameters->RenderTargets[1] = FRenderTargetBinding(RefractionVarianceCoverage, LoadAction);
					PassParameters->RenderTargets[2] = FRenderTargetBinding(RefractionClosestDepthMeter, LoadAction);
				}
				PassParameters->RenderTargets.DepthStencil = StencilWriteBinding;

				Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("Distortion::Accumulate View%d", ViewIndex),
					PassParameters,
					ERDGPassFlags::Raster,
					[&View, Pass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRender_RenderDistortion_Accumulate_Meshes);
					SetStereoViewport(RHICmdList, View);
					Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
				});

				LoadAction = ERenderTargetLoadAction::ELoad;
			}
		}

		if (LoadAction == ERenderTargetLoadAction::EClear)
		{
			AddClearRenderTargetPass(GraphBuilder, DistortionTexture);
		}
	}

	FRDGTextureDesc DistortedSceneColorDesc = SceneColorTexture->Desc;
	//Remove fast clear flag on the DistoredSceneColor which is used in the Apply and Merge passes. 
	// This can save the Fast clear eliminate in the Merge pass when the RTV is transient allocated.
	EnumAddFlags(DistortedSceneColorDesc.Flags, TexCreate_NoFastClear);
	EnumRemoveFlags(DistortedSceneColorDesc.Flags, TexCreate_FastVRAM);

	FRDGTextureRef DistortionSceneColorTexture = GraphBuilder.CreateTexture(DistortedSceneColorDesc, TEXT("DistortedSceneColor"));

	FDistortionScreenPS::FParameters CommonParameters;
	CommonParameters.DistortionMSAATexture = DistortionTexture;
	CommonParameters.DistortionTexture = DistortionTexture;
	if (bUseRoughRefraction)
	{
		CommonParameters.RefractionVarianceCoverageMSAATexture = RefractionVarianceCoverage;
		CommonParameters.RefractionVarianceCoverageTexture = RefractionVarianceCoverage;
		CommonParameters.RefractionClosestDepthMeterMSAATexture = RefractionClosestDepthMeter;
		CommonParameters.RefractionClosestDepthMeterTexture = RefractionClosestDepthMeter;
		FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
		CommonParameters.BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);
	}
	CommonParameters.SceneColorTextureSampler = bUseRoughRefraction ? TStaticSamplerState<SF_Trilinear>::GetRHI() : TStaticSamplerState<>::GetRHI();
	CommonParameters.SceneDepthMeterTextureSampler = bUseRoughRefraction ? TStaticSamplerState<SF_Trilinear>::GetRHI() : TStaticSamplerState<>::GetRHI();
	CommonParameters.DistortionTextureSampler = TStaticSamplerState<>::GetRHI();
	CommonParameters.RefractionVarianceCoverageSampler = TStaticSamplerState<>::GetRHI();
	CommonParameters.RefractionClosestDepthMeterSampler = TStaticSamplerState<>::GetRHI();
	CommonParameters.StandardTranslucentColorAlphaTextureSampler = TStaticSamplerState<>::GetRHI();
	CommonParameters.StandardTranslucentTransmittanceTextureSampler = TStaticSamplerState<>::GetRHI();
	CommonParameters.StandardDevInPixelForMip0 = FilterSigmaPixel;
	CommonParameters.BlurDitheringScaleFactor = FMath::Clamp(CVarRefractionBlurDitheringScaleFactor.GetValueOnRenderThread(), 0.0f, 1.0f);

	const bool bAllowStandardTranslucencySeparated = ViewFamily.AllowStandardTranslucencySeparated();
	FDistortionScreenPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FDistortionScreenPS::FUseMSAADim>(SceneColorTexture->Desc.NumSamples > 1);
	PermutationVector.Set<FDistortionScreenPS::FUseRoughRefractionDim>(bUseRoughRefraction);
	PermutationVector.Set<FDistortionScreenPS::FCompositedStandardTranslucentDim>(bAllowStandardTranslucencySeparated);

	TShaderMapRef<FScreenPassVS> VertexShader(ShaderMap);
	TShaderMapRef<FDistortionApplyScreenPS> ApplyPixelShader(ShaderMap, PermutationVector);
	TShaderMapRef<FDistortionMergeScreenPS> MergePixelShader(ShaderMap, PermutationVector);

	FScreenPassPipelineState PipelineState(VertexShader, {});
	FScreenPassTextureViewport Viewport(SceneColorTexture);

	auto GetViewRefractionMaxMipLevel = [](const FViewInfo& View, float StandardDevInPixelForMip0)
	{
		float MaxStandardDeviationInScreenRatio = FMath::Clamp(CVarRefractionBlurMaxStandardDeviationInScreenPercent.GetValueOnRenderThread() / 100.0f, 0.0f, 1.0f);
		float MaxStandardDevInPixel = MaxStandardDeviationInScreenRatio * View.ViewRect.Width();
		float MaxMipLevel = FMath::Loge(MaxStandardDevInPixel / StandardDevInPixelForMip0) / FMath::Loge(2.0f); // This formula is explained in DistortApplyScreenPS.usf.
		return MaxMipLevel;
	};

	// Apply distortion and store off-screen.
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRenderer_RenderDistortion_Apply);
		CommonParameters.SceneColorMSAATexture = SceneColorTexture;
		CommonParameters.SceneColorTexture = SceneColorTexture;
		CommonParameters.RenderTargets.DepthStencil = StencilReadBinding;
		PipelineState.PixelShader = ApplyPixelShader;

		// Test against stencil mask but don't clear.
		PipelineState.DepthStencilState = TStaticDepthStencilState<
			false, CF_Always,
			true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			DISTORTION_STENCIL_MASK_BIT, DISTORTION_STENCIL_MASK_BIT>::GetRHI();
		PipelineState.StencilRef = DISTORTION_STENCIL_MASK_BIT;

		ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ENoAction;

		for (int32 ViewIndex = 0, Num = Views.Num(); ViewIndex < Num; ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

			auto* PassParameters = GraphBuilder.AllocParameters<FDistortionScreenPS::FParameters>();
			*PassParameters = CommonParameters;
			if (bUseRoughRefraction)
			{
				PassParameters->SceneColorMSAATexture = ViewsSceneColorMipchain[ViewIndex];
				PassParameters->SceneColorTexture = ViewsSceneColorMipchain[ViewIndex];
				PassParameters->SceneDepthMeterMSAATexture = ViewsSceneDepthMeterMipchain[ViewIndex];
				PassParameters->SceneDepthMeterTexture = ViewsSceneDepthMeterMipchain[ViewIndex];
				PassParameters->RefractionMaxMipLevel = GetViewRefractionMaxMipLevel(View, PassParameters->StandardDevInPixelForMip0);
			}
			PassParameters->View = View.GetShaderParameters();
			PassParameters->RenderTargets[0] = FRenderTargetBinding(DistortionSceneColorTexture, LoadAction);

			Viewport.Rect = View.ViewRect;

			ClearUnusedGraphResources(ApplyPixelShader, PassParameters);
			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Distortion::Apply View%d", ViewIndex), FScreenPassViewInfo(View, 1), Viewport, Viewport, PipelineState, PassParameters,
				[ApplyPixelShader, PassParameters](FRHICommandList& RHICmdList)
			{
				SetShaderParameters(RHICmdList, ApplyPixelShader, ApplyPixelShader.GetPixelShader(), *PassParameters);
			});

			LoadAction = ERenderTargetLoadAction::ELoad;
		}
	}

	// Merge distortion back to scene color.
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRenderer_RenderDistortion_Merge);
		CommonParameters.SceneColorMSAATexture = DistortionSceneColorTexture;
		CommonParameters.SceneColorTexture = DistortionSceneColorTexture;
		CommonParameters.RenderTargets.DepthStencil = StencilWriteBinding;
		PipelineState.PixelShader = MergePixelShader;

		// Test against stencil mask and clear it.
		PipelineState.DepthStencilState = TStaticDepthStencilState<
			false, CF_Always,
			true, CF_Equal, SO_Keep, SO_Keep, SO_Zero,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			DISTORTION_STENCIL_MASK_BIT, DISTORTION_STENCIL_MASK_BIT>::GetRHI();
		PipelineState.StencilRef = DISTORTION_STENCIL_MASK_BIT;
		
		// When holdout is enabled, we retain the alpha to keep the translucent holdout alpha.
		if (IsPrimitiveAlphaHoldoutEnabledForAnyView(Views))
		{
			PipelineState.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_Zero, BF_One>::GetRHI();
		}

		for (int32 ViewIndex = 0, Num = Views.Num(); ViewIndex < Num; ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			auto* PassParameters = GraphBuilder.AllocParameters<FDistortionScreenPS::FParameters>();
			*PassParameters = CommonParameters;
			PassParameters->View = View.GetShaderParameters();
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);

			// Fetch the separated before dof translucent data if available for composition
			FTranslucencyPassResources& TranslucencyPassResources = TranslucencyResourceMap.Get(ViewIndex, ETranslucencyPass::TPT_TranslucencyStandard);
			PassParameters->StandardTranslucentColorAlphaTexture = TranslucencyPassResources.GetColorForRead(GraphBuilder);
			PassParameters->StandardTranslucentColorAlphaMSAATexture = TranslucencyPassResources.ColorTexture.Target;
			PassParameters->StandardTranslucentTransmittanceTexture = TranslucencyPassResources.GetColorModulateForRead(GraphBuilder);
			PassParameters->StandardTranslucentTransmittanceMSAATexture = TranslucencyPassResources.ColorModulateTexture.Target;
			Viewport.Rect = View.ViewRect;

			ClearUnusedGraphResources(MergePixelShader, PassParameters);
			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Distortion::Merge View%d", ViewIndex), FScreenPassViewInfo(View, 1), Viewport, Viewport, PipelineState, PassParameters,
				[MergePixelShader, PassParameters](FRHICommandList& RHICmdList)
			{
				SetShaderParameters(RHICmdList, MergePixelShader, MergePixelShader.GetPixelShader(), *PassParameters);
			});
		}
	}
}

bool GetDistortionPassShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	TShaderRef<FDistortionMeshVS>& VertexShader,
	TShaderRef<FDistortionMeshPS>& PixelShader)
{
	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<FDistortionMeshVS>();
	ShaderTypes.AddShaderType<FDistortionMeshPS>();

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

void FDistortionMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (MeshBatch.bUseForMaterial)
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material)
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

static bool ShouldDraw(const FMaterial& Material)
{
	return (GetGlobalRefractionQuality() > 0
		&& IsTranslucentBlendMode(Material)
		&& ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain())
		&& Material.IsDistorted());
}

void FDistortionMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	if (!ShouldDraw(Material))
	{
		return;
	}

	TMeshProcessorShaders<
		FDistortionMeshVS,
		FDistortionMeshPS> DistortionPassShaders;
	if (!GetDistortionPassShaders(
		Material,
		VertexFactoryData.VertexFactoryType,
		FeatureLevel,
		DistortionPassShaders.VertexShader,
		DistortionPassShaders.PixelShader))
	{
		return;
	}

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
		
	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;
	AddRenderTargetInfo(GetRefractionOffsetQuality(SceneTexturesConfig.ShaderPlatform) > 0 ? PF_FloatRGBA : PF_B8G8R8A8, GFastVRamConfig.Distortion | TexCreate_RenderTargetable | TexCreate_ShaderResource, RenderTargetsInfo);
	if (GetSubstrateEnabledUseRoughRefraction())
	{
		AddRenderTargetInfo(PF_R16F, GFastVRamConfig.Distortion | TexCreate_RenderTargetable | TexCreate_ShaderResource, RenderTargetsInfo);
	}
	ETextureCreateFlags DepthStencilCreateFlags = SceneTexturesConfig.DepthCreateFlags;
	SetupDepthStencilInfo(PF_DepthStencil, DepthStencilCreateFlags, ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite, RenderTargetsInfo);

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		Material,
		PassDrawRenderState,
		RenderTargetsInfo,
		DistortionPassShaders,
		MeshFillMode,
		MeshCullMode,
		(EPrimitiveType)PreCacheParams.PrimitiveType,
		EMeshPassFeatures::Default,
		true /*bRequired*/,
		PSOInitializers);
}

bool FDistortionMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	bool bResult = true;
	if (ShouldDraw(Material)
		&& (!PrimitiveSceneProxy || PrimitiveSceneProxy->ShouldRenderInMainPass()))
	{
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

		bResult = Process(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, MeshFillMode, MeshCullMode);
	}

	return bResult;
}

bool FDistortionMeshProcessor::Process(
	const FMeshBatch& MeshBatch, 
	uint64 BatchElementMask, 
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		FDistortionMeshVS,
		FDistortionMeshPS> DistortionPassShaders;

	if (!GetDistortionPassShaders(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		DistortionPassShaders.VertexShader,
		DistortionPassShaders.PixelShader))
	{
		return false;
	}

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(DistortionPassShaders.VertexShader, DistortionPassShaders.PixelShader);

	const bool bDisableDepthTest = MaterialResource.ShouldDisableDepthTest();

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		bDisableDepthTest ? PassDrawRenderStateNoDepthTest : PassDrawRenderState,
		DistortionPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

FDistortionMeshProcessor::FDistortionMeshProcessor(
	const FScene* Scene,
	ERHIFeatureLevel::Type FeatureLevel,
	const FSceneView* InViewIfDynamicMeshCommand,
	const FMeshPassProcessorRenderState& InPassDrawRenderState,
	const FMeshPassProcessorRenderState& InDistortionPassStateNoDepthTest,
	FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::Distortion, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InPassDrawRenderState)
	, PassDrawRenderStateNoDepthTest(InDistortionPassStateNoDepthTest)
{}

FMeshPassProcessor* CreateDistortionPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState DistortionPassState;
	
	// test against depth and write stencil mask
	DistortionPassState.SetDepthStencilState(TStaticDepthStencilState<
		false, CF_DepthNearOrEqual,
		true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		DISTORTION_STENCIL_MASK_BIT, DISTORTION_STENCIL_MASK_BIT>::GetRHI());

	DistortionPassState.SetStencilRef(DISTORTION_STENCIL_MASK_BIT);

	if (GetSubstrateEnabledUseRoughRefraction())
	{
		DistortionPassState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
															CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
															CW_RED,  BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI());
	}
	else
	{
		// additive blending of offsets (or complexity if the shader complexity viewmode is enabled)
		DistortionPassState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI());
	}

	FMeshPassProcessorRenderState DistortionPassStateNoDepthTest = DistortionPassState;
	DistortionPassStateNoDepthTest.SetDepthStencilState(TStaticDepthStencilState<
		false, CF_Always,
		true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		DISTORTION_STENCIL_MASK_BIT, DISTORTION_STENCIL_MASK_BIT>::GetRHI());
	DistortionPassStateNoDepthTest.SetStencilRef(DISTORTION_STENCIL_MASK_BIT);

	return new FDistortionMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, DistortionPassState, DistortionPassStateNoDepthTest, InDrawListContext);
}

FMeshPassProcessor* CreateMobileDistortionPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	if (IsMobileDistortionBeforeTranslucency())
	{
		return CreateDistortionPassProcessor(FeatureLevel, Scene, InViewIfDynamicMeshCommand, InDrawListContext);
	}
	else
	{
		FMeshPassProcessorRenderState DistortionPassState;
		// We don't have depth, render all pixels, pixel shader will sample SceneDepth from SceneColor.A and discard if occluded
		DistortionPassState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

		if (GetSubstrateEnabledUseRoughRefraction())
		{
			DistortionPassState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
				CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
				CW_RED, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI());
		}
		else
		{
			// additive blending of offsets
			DistortionPassState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI());
		}
		return new FDistortionMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, DistortionPassState, DistortionPassState, InDrawListContext);
	}
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(DistortionPass, CreateDistortionPassProcessor, EShadingPath::Deferred, EMeshPass::Distortion, EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileDistortionPass, CreateMobileDistortionPassProcessor, EShadingPath::Mobile, EMeshPass::Distortion, EMeshPassFlags::MainView);
