// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DistortionRendering.h: Distortion rendering implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "MeshPassProcessor.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "SceneRenderingAllocator.h"

class FPrimitiveSceneProxy;
struct FMeshPassProcessorRenderState;
class FViewInfo;
class FScene;
class FSceneView;
struct FSceneTexturesConfig;
class FMaterial;
class FMaterialRenderProxy;

DECLARE_GPU_DRAWCALL_STAT_EXTERN(Distortion);

extern void SetupDistortionParams(FVector4f& DistortionParams, const FViewInfo& View);

class FDistortionMeshProcessor : public FSceneRenderingAllocatorObject<FDistortionMeshProcessor>, public FMeshPassProcessor
{
public:

	FDistortionMeshProcessor(
		const FScene* Scene, 
		ERHIFeatureLevel::Type FeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand, 
		const FMeshPassProcessorRenderState& InPassDrawRenderState, 
		const FMeshPassProcessorRenderState& InDistortionPassStateNoDepthTest,
		FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

	FMeshPassProcessorRenderState PassDrawRenderState;
	FMeshPassProcessorRenderState PassDrawRenderStateNoDepthTest;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);
};