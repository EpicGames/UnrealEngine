// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicBVH.h"
#include "Misc/AutomationTest.h"
#include "Misc/OutputDeviceRedirector.h"

FMortonArray::FMortonArray( const TArray< FBounds3f >& InBounds )
	: Bounds( InBounds )
{
	TArray< FSortPair > Unsorted;
	Unsorted.AddUninitialized( Bounds.Num() );
	Sorted.AddUninitialized( Bounds.Num() );
	
	FBounds3f TotalBounds;
	for( int i = 0; i < Bounds.Num(); i++ )
	{
		TotalBounds += Bounds[i].Min + Bounds[i].Max;
	}

	FVector3f Scale = FVector3f( 1.0f ) / ( TotalBounds.Max - TotalBounds.Min );
	FVector3f Bias = -TotalBounds.Min / ( TotalBounds.Max - TotalBounds.Min );

	for( int i = 0; i < Bounds.Num(); i++ )
	{
		FVector3f CenterLocal = ( Bounds[i].Min + Bounds[i].Max ) * Scale + Bias;

		uint32 Morton;
		Morton  = FMath::MortonCode3( CenterLocal.X * 1023 );
		Morton |= FMath::MortonCode3( CenterLocal.Y * 1023 ) << 1;
		Morton |= FMath::MortonCode3( CenterLocal.Z * 1023 ) << 2;

		Unsorted[i].Code = Morton;
		Unsorted[i].Index = i;
	}

	RadixSort32( Sorted.GetData(), Unsorted.GetData(), Unsorted.Num(),
		[&]( FSortPair Pair )
		{
			return Pair.Code;
		} );
}

void FMortonArray::RegenerateCodes( const FRange& Range )
{
	FBounds3f TotalBounds;
	for( int32 i = Range.Begin; i < Range.End; i++ )
	{
		uint32 Index = Sorted[i].Index;
		TotalBounds += Bounds[ Index ].Min + Bounds[ Index ].Max;
	}

	FVector3f Scale = FVector3f( 1.0f ) / ( TotalBounds.Max - TotalBounds.Min );
	FVector3f Bias = -TotalBounds.Min / ( TotalBounds.Max - TotalBounds.Min );

	for( int32 i = Range.Begin; i < Range.End; i++ )
	{
		uint32 Index = Sorted[i].Index;

		FVector3f CenterLocal = ( Bounds[ Index ].Min + Bounds[ Index ].Max ) * Scale + Bias;

		uint32 Morton;
		Morton  = FMath::MortonCode3( CenterLocal.X * 1023 );
		Morton |= FMath::MortonCode3( CenterLocal.Y * 1023 ) << 1;
		Morton |= FMath::MortonCode3( CenterLocal.Z * 1023 ) << 2;

		Sorted[i].Code = Morton;
	}

	Algo::Sort(MakeArrayView(&Sorted[Range.Begin], Range.End - Range.Begin));
}


inline FVector3f RandomVector( float Min, float Max )
{
	FVector3f V;
	V.X = FMath::RandRange( Min, Max );
	V.Y = FMath::RandRange( Min, Max );
	V.Z = FMath::RandRange( Min, Max );
	return V;
}

static FBounds3f RandomBounds( float CenterMin, float CenterMax, float ExtentMin, float ExtentMax )
{
	FVector3f Center = RandomVector( CenterMin, CenterMax );
	FVector3f Extent = RandomVector( ExtentMin, ExtentMax );
	return FBounds3f( { Center - Extent, Center + Extent } );
}

template< uint32 MaxChildren >
static void	RandomBVH( float CenterMin, float CenterMax, float ExtentMin, float ExtentMax, uint32 Num, FDynamicBVH< MaxChildren >& BVH )
{
	for( uint32 i = 0; i < Num; i++ )
	{
		BVH.Add( RandomBounds( CenterMin, CenterMax, ExtentMin,  ExtentMax ), i );
	}
}

void TestBVH_ZeroToZero()
{
	FMath::RandInit( 17 );

	FDynamicBVH<4> BVH;
	
	uint32 Time0 = FPlatformTime::Cycles();

	RandomBVH( -64.0f, 64.0f, 1.0f, 4.0f, 65536, BVH );

	uint32 Num = BVH.GetNumLeaves();
	uint32 Time1 = FPlatformTime::Cycles();

	float Cost = BVH.GetTotalCost();

	for( uint32 i = 0; i < Num; i++ )
	{
		BVH.Remove(i);
	}

	uint32 Time2 = FPlatformTime::Cycles();
	GLog->Logf( TEXT("TestBVH_ZeroToZero add [%.2fms]"), FPlatformTime::ToMilliseconds( Time1 - Time0 ) );
	GLog->Logf( TEXT("TestBVH_ZeroToZero cost %.2f"), Cost );
	GLog->Logf( TEXT("TestBVH_ZeroToZero remove [%.2fms]"), FPlatformTime::ToMilliseconds( Time2 - Time1 ) );
}

void TestBVH_AddRemove()
{
	FMath::RandInit( 17 );

	FDynamicBVH<4> BVH;
	RandomBVH( -64.0f, 64.0f, 1.0f, 4.0f, 8192, BVH );

	uint32 Num = BVH.GetNumLeaves();
	uint32 Time0 = FPlatformTime::Cycles();

	BVH.NumTested = 0;
	
	for( int i = 0; i < 65536; i++ )
	{
		BVH.Add( RandomBounds( -64.0f, 64.0f, 1.0f, 4.0f ), Num );
		BVH.Remove( Num );
	}

	uint32 Time1 = FPlatformTime::Cycles();
	GLog->Logf( TEXT("TestBVH_AddRemove [%.2fms]"), FPlatformTime::ToMilliseconds( Time1 - Time0 ) );
	GLog->Logf( TEXT("TestBVH_AddRemove tested %.2f"), (float)BVH.NumTested / 65536.0f );

	BVH.Check();
}

void TestBVH_AddRemoveOverlapped()
{
	FMath::RandInit( 17 );

	FDynamicBVH<4> BVH;
	RandomBVH( -64.0f, 64.0f, 16.0f, 64.0f, 8192, BVH );

	uint32 Num = BVH.GetNumLeaves();
	uint32 Time0 = FPlatformTime::Cycles();

	BVH.NumTested = 0;
	
	for( int i = 0; i < 65536; i++ )
	{
		BVH.Add( RandomBounds( -64.0f, 64.0f, 16.0f, 64.0f ), Num );
		BVH.Remove( Num );
	}

	uint32 Time1 = FPlatformTime::Cycles();
	GLog->Logf( TEXT("TestBVH_AddRemoveOverlapped [%.2fms]"), FPlatformTime::ToMilliseconds( Time1 - Time0 ) );
	GLog->Logf( TEXT("TestBVH_AddRemoveOverlapped tested %.2f"), (float)BVH.NumTested / 65536.0f );

	BVH.Check();
}

void TestBVH_BatchAddRemove()
{
	FMath::RandInit( 17 );

	FDynamicBVH<4> BVH;
	RandomBVH( -64.0f, 64.0f, 1.0f, 4.0f, 8192, BVH );

	uint32 Num = BVH.GetNumLeaves();
	uint32 Time0 = FPlatformTime::Cycles();

	for( int i = 0; i < 256; i++ )
	{
		for( int j = 0; j < 256; j++ )
			BVH.Add( RandomBounds( -64.0f, 64.0f, 1.0f, 4.0f ), Num + j );

		for( int j = 0; j < 256; j++ )
			BVH.Remove( Num + j );
	}

	uint32 Time1 = FPlatformTime::Cycles();
	GLog->Logf( TEXT("TestBVH_BatchAddRemove [%.2fms]"), FPlatformTime::ToMilliseconds( Time1 - Time0 ) );

	BVH.Check();
}

void TestBVH_UpdateTeleport()
{
	FMath::RandInit( 17 );

	FDynamicBVH<4> BVH;
	RandomBVH( -64.0f, 64.0f, 1.0f, 4.0f, 8192, BVH );

	uint32 Num = BVH.GetNumLeaves();
	uint32 Time0 = FPlatformTime::Cycles();

	for( int i = 0; i < 65536; i++ )
	{
		uint32 Index = FMath::Rand() % Num;
		BVH.Update( RandomBounds( -64.0f, 64.0f, 1.0f, 4.0f ), Index );
	}

	uint32 Time1 = FPlatformTime::Cycles();
	GLog->Logf( TEXT("TestBVH_UpdateTeleport [%.2fms]"), FPlatformTime::ToMilliseconds( Time1 - Time0 ) );

	BVH.Check();
}

void TestBVH_UpdateJitter()
{
	FMath::RandInit( 17 );
	
	FDynamicBVH<4> BVH;
	RandomBVH( -64.0f, 64.0f, 1.0f, 4.0f, 8192, BVH );

	uint32 Num = BVH.GetNumLeaves();
	uint32 Time0 = FPlatformTime::Cycles();

	for( int i = 0; i < 65536; i++ )
	{
		uint32 Index = FMath::Rand() % Num;

		FBounds3f Bounds = BVH.GetBounds( Index );
		FVector3f Jitter = RandomVector( -0.01f, 0.01f );
		Bounds.Min += Jitter;
		Bounds.Max += Jitter;

		BVH.Update( Bounds, Index );
	}

	uint32 Time1 = FPlatformTime::Cycles();
	GLog->Logf( TEXT("TestBVH_UpdateJitter [%.2fms]"), FPlatformTime::ToMilliseconds( Time1 - Time0 ) );

	BVH.Check();
}

void TestBVH_Ordered()
{
	FBounds3f UnitBounds;
	UnitBounds.Min = FVector3f::ZeroVector;
	UnitBounds.Max = FVector3f::OneVector;

	FDynamicBVH<4> BVH;

	for( uint32 i = 0; i < 8192; i++ )
	{
		FBounds3f Bounds = UnitBounds;
		Bounds.Min.X += (float)i;
		Bounds.Max.X += (float)i;
		BVH.Add( Bounds, i );
	}

	GLog->Logf( TEXT("TestBVH_Ordered cost %.2f"), BVH.GetTotalCost() );

	uint32 Num = BVH.GetNumLeaves();

	for( int i = 0; i < 1024; i++ )
	{
		uint32 Index = FMath::Rand() % Num;
		FBounds3f Bounds = BVH.GetBounds( Index );
		BVH.Update( Bounds, Index );
		//BVH.Optimize(1);
	}

	GLog->Logf( TEXT("TestBVH_Ordered cost after %.2f"), BVH.GetTotalCost() );

	for( int i = 0; i < 65536; i++ )
	{
		uint32 Index = FMath::Rand() % Num;
		FBounds3f Bounds = BVH.GetBounds( Index );
		BVH.Update( Bounds, Index );
		//BVH.Optimize(1);
	}

	GLog->Logf( TEXT("TestBVH_Ordered cost after %.2f"), BVH.GetTotalCost() );
}

void TestBVH_Optimize()
{
	FMath::RandInit( 17 );
	
	FDynamicBVH<4> BVH;
	RandomBVH( -64.0f, 64.0f, 1.0f, 4.0f, 8192, BVH );

	GLog->Logf( TEXT("TestBVH_Optimize cost before %.2f"), BVH.GetTotalCost() );

	uint32 Num = BVH.GetNumLeaves();

	for( int i = 0; i < 1024; i++ )
	{
		uint32 Index = FMath::Rand() % Num;
		FBounds3f Bounds = BVH.GetBounds( Index );
		BVH.Update( Bounds, Index );
		//BVH.Optimize(1);
	}

	GLog->Logf( TEXT("TestBVH_Optimize cost after %.2f"), BVH.GetTotalCost() );

	for( int i = 0; i < 65536; i++ )
	{
		uint32 Index = FMath::Rand() % Num;
		FBounds3f Bounds = BVH.GetBounds( Index );
		BVH.Update( Bounds, Index );
		//BVH.Optimize(1);
	}

	GLog->Logf( TEXT("TestBVH_Optimize cost after %.2f"), BVH.GetTotalCost() );
}

void TestBVH_Build()
{
	FMath::RandInit( 17 );

	FDynamicBVH<4> BVH;

	uint32 Num = 65536;

	TArray< FBounds3f > BoundsArray;
	BoundsArray.AddUninitialized( Num );
	
	for( uint32 i = 0; i < Num; i++ )
		BoundsArray[i] = RandomBounds( -64.0f, 64.0f, 1.0f, 4.0f );
	
	uint32 Time0 = FPlatformTime::Cycles();

	BVH.Build( BoundsArray, 0 );

	uint32 Time1 = FPlatformTime::Cycles();

	GLog->Logf( TEXT("TestBVH_Build [%.2fms]"), FPlatformTime::ToMilliseconds( Time1 - Time0 ) );
	GLog->Logf( TEXT("TestBVH_Build cost %.2f"), BVH.GetTotalCost() );
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST( FTestBVH, "System.Renderer.DynamicBVH", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter );
bool FTestBVH::RunTest( const FString& Parameters )
{
	TestBVH_ZeroToZero();
	TestBVH_AddRemove();
	TestBVH_AddRemoveOverlapped();
	TestBVH_BatchAddRemove();
	TestBVH_UpdateTeleport();
	TestBVH_UpdateJitter();
	TestBVH_Ordered();
	TestBVH_Optimize();
	TestBVH_Build();

	return true;
}
