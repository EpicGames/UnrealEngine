// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Math/Bounds.h"

struct FSurfaceAreaHeuristic
{
	float operator()( const FBounds3f& Bounds ) const
	{
		FVector3f Extent = Bounds.Max - Bounds.Min;
		return Extent.X * Extent.Y + Extent.X * Extent.Z + Extent.Y * Extent.Z;
	}
};

struct FIgnoreDirty
{
	int32	Num() const { return 0; }
	void	Add( uint32 Index ) {}
	void	Mark( uint32 Index ) {}

	template< typename FFuncType >
	void	ForAll( const FFuncType& Func ) {}
};

struct FTrackDirty
{
	TBitArray<>			NodeIsDirty;
	TArray< uint32 >	DirtyNodes;

	int32	Num() const { return DirtyNodes.Num(); }

	void	Add( uint32 Index )
	{
		NodeIsDirty.Add( true );
		DirtyNodes.Add( Index );
	}

	void	Mark( uint32 Index )
	{
		if( !NodeIsDirty[ Index ] )
		{
			NodeIsDirty[ Index ] = true;
			DirtyNodes.Add( Index );
		}
	}

	template< typename FFuncType >
	void	ForAll( const FFuncType& Func )
	{
		for( uint32 Index : DirtyNodes )
		{
			Func( Index );
			NodeIsDirty[ Index ] = false;
		}
		DirtyNodes.Reset();
	}
};

struct FSingleRoot
{
	struct FRoot
	{
		FBounds3f		Bounds;
		uint32			FirstChild = ~0u;

		const FVector3f& ToRelative( const FVector3f& Position ) const	{ return Position; }
		const FBounds3f& ToRelative( const FBounds3f& Other ) const		{ return Other; }
		const FBounds3f& ToAbsolute( const FBounds3f& Other ) const		{ return Other; }
	};
	FRoot Root;

	FRoot&	FindOrAdd( const FBounds3f& Bounds )	{ return Root; }
	FRoot&	FindChecked( uint32 RootFirstChild )	{ return Root; }
	void	Remove( uint32 RootFirstChild )			{ Root.FirstChild = ~0u; }

	template< typename FFuncType >
	void ForAll( const FFuncType& Func ) const
	{
		Func( Root );
	}
};

// Supports LWC with a tile grid at the top where each tile points to a BVH in tile relative coordinates.
struct FRootForest
{
	struct FRoot
	{
		FVector3d		Offset;
		FBounds3f		Bounds;
		uint32			FirstChild;

		template< typename T >
		FVector3f ToRelative( const UE::Math::TVector<T>& Position ) const
		{
			return FVector3f( Position - Offset );
		}

		template< typename T >
		FBounds3f ToRelative( const TBounds<T>& Other ) const
		{
			return Other.ToRelative( Offset );
		}

		FBounds3d ToAbsolute( const FBounds3f& Other ) const
		{
			return Other.ToAbsolute( Offset );
		}
	};
	TArray< FRoot > Roots;

	template< typename T >
	FRoot& FindOrAdd( const TBounds<T>& Bounds )
	{
		constexpr double TileSize = 1024.0 * 1024.0;

		UE::Math::TVector<T> RootOffset = Bounds.GetCenter() / TileSize;
		RootOffset.X = FMath::RoundToZero( RootOffset.X );
		RootOffset.Y = FMath::RoundToZero( RootOffset.Y );
		RootOffset.Z = FMath::RoundToZero( RootOffset.Z );
		RootOffset *= TileSize;

		FRoot* Root = Roots.FindByPredicate(
			[ &RootOffset ]( FRoot& Root )
			{
				return Root.Offset == RootOffset;
			} );

		if( Root == nullptr )
		{
			Root = &Roots.AddDefaulted_GetRef();
			Root->Offset = RootOffset;
			Root->FirstChild = ~0u;
		}

		return *Root;
	}

	FRoot& FindChecked( uint32 RootFirstChild )
	{
		FRoot* Root = Roots.FindByPredicate(
			[ RootFirstChild ]( FRoot& Root )
			{
				return Root.FirstChild == RootFirstChild;
			} );
		check( Root );
		return *Root;
	}

	void Remove( uint32 RootFirstChild )
	{
		Roots.RemoveAllSwap(
			[ RootFirstChild ]( FRoot& Root )
			{
				return Root.FirstChild == RootFirstChild;
			},
			EAllowShrinking::No);
	}

	template< typename FFuncType >
	void ForAll( const FFuncType& Func ) const
	{
		for( const FRoot& Root : Roots )
		{
			Func( Root );
		}
	}
};

constexpr uint32 ConstLog2( uint32 x )
{
	return ( x < 2 ) ? 0 : 1 + ConstLog2( x / 2 );
}

class FLowestCostList
{
	using FCandidate = TPair< float, uint32 >;

public:
	void	Reset()
	{
		CandidateHead = 0;
		Candidates.Reset();
		NumZeros = 0;
	}

	void	Add( float NodeCost, uint32 NodeIndex )
	{
		if( NodeCost < UE_KINDA_SMALL_NUMBER && NumZeros < MaxZeros )
		{
			ZeroCostNodes[ NumZeros++ ] = NodeIndex;
		}
		else
		{
			Candidates.Add( FCandidate( NodeCost, NodeIndex ) );
		}
	}

	bool GetNext( float BestCost, float& NodeCost, uint32& NodeIndex )
	{
		if( NumZeros )
		{
			NodeIndex = ZeroCostNodes[ --NumZeros ];
		}
		else
		{
			// Move head up
			int32 Num = Candidates.Num();
			while( CandidateHead < Num && Candidates[ CandidateHead ].Key >= BestCost )
				CandidateHead++;
			if( CandidateHead == Num )
				return false;

			// Find smallest linear search
			float SmallestCost	= Candidates[ CandidateHead ].Key;
			int32 SmallestIndex	= CandidateHead;
			for( int32 i = CandidateHead + 1; i < Num; i++ )
			{
				float Cost = Candidates[i].Key;
				if( Cost < SmallestCost )
				{
					SmallestCost = Cost;
					SmallestIndex = i;
				}
			}

			// Return smallest cost and NodeIndex
			NodeCost = SmallestCost;
			NodeIndex = Candidates[ SmallestIndex ].Value;

			Candidates.RemoveAtSwap( SmallestIndex, EAllowShrinking::No);
		}

		return true;
	}

private:
	TArray< FCandidate >	Candidates;
	int32	CandidateHead;

	static constexpr uint32 MaxZeros = 32;
	uint32	NumZeros = 0;
	uint32	ZeroCostNodes[ MaxZeros ];
};


template<
	uint32 MaxChildren,
	typename FRootPolicy	= FSingleRoot,
	typename FDirtyPolicy	= FIgnoreDirty,
	typename FCostMetric	= FSurfaceAreaHeuristic
>
class FDynamicBVH
{
	using FRoot = typename FRootPolicy::FRoot;
	
	FRootPolicy		Roots;
	FDirtyPolicy	DirtyPolicy;
	FCostMetric		CostMetric;

public:
					FDynamicBVH();

	int32			GetNumNodes() const		{ return Nodes.Num(); }
	int32			GetNumLeaves() const	{ return Leaves.Num(); }
	int32			GetNumDirty() const		{ return DirtyPolicy.Num(); }

					template< typename T  >
	void			Add( const TBounds<T>& Bounds, uint32 Index );
					template< typename T  >
	void			Update( const TBounds<T>& Bounds, uint32 Index );
	void			Remove( uint32 Index );

	bool			IsPresent( uint32 Index ) const	{ return Index < (uint32)Leaves.Num() && Leaves[ Index ] != ~0u; }
	void			AddDefaulted()					{ Leaves.Add( ~0u ); }
	void			SwapIndexes( uint32 Index0, uint32 Index1 );

	void			Build( const TArray< FBounds3f >& BoundsArray, uint32 FirstIndex );

					template< typename T, typename FFuncType >
	void			ForAll( const TBounds<T>& Bounds, const FFuncType& Func ) const;
					template< typename FPredicate, typename FFuncType >
	void			ForAll( const FPredicate& Predicate, const FFuncType& Func ) const;
					template< typename FFuncType >
	void			ForAllDirty( const FFuncType& Func );

					template< typename T, typename FFuncType >
	uint32			FindClosest( const UE::Math::TVector<T>& Position, const FFuncType& LeafDistSqr );

	// Not correct with FRootForest
	const FBounds3f& GetBounds( uint32 Index ) const
	{
		check( Leaves[ Index ] != ~0u );
		uint32 NodeIndex = Leaves[ Index ];
		return GetNode( NodeIndex ).GetBounds( NodeIndex );
	}
	
	float GetTotalCost() const
	{
		float TotalCost = 0.0f;
		for( auto& Node : Nodes )
		{
			for( uint32 i = 0; i < Node.NumChildren; i++ )
			{
				if( ( Node.ChildIndexes[i] & 1 ) == 0 )
					TotalCost += CostMetric( Node.ChildBounds[i] );
			}
		}

		return TotalCost;
	}

	bool Check() const
	{
		for( int32 i = 0; i < Nodes.Num(); i++ )
		{
			for( uint32 j = 0; j < Nodes[i].NumChildren; j++ )
			{
				CheckNode( (i << IndexShift) | j );
			}
		}

		return true;
	}

	uint32 NumTested = 0;

protected:
	static constexpr uint32	IndexShift = ConstLog2( MaxChildren );
	static constexpr uint32	ChildMask = MaxChildren - 1;
	static constexpr uint32 MaxChildren4 = (MaxChildren + 3) / 4;

	struct FNode
	{
		// Index: low bits index child, high bits index FNode
		uint32		ParentIndex;
		uint32		NumChildren;		// Lots of bits for this!
		uint32		ChildIndexes[ MaxChildren ];
		FBounds3f	ChildBounds[ MaxChildren ];
		
		uint32				GetFirstChild( uint32 NodeIndex ) const	{ return ChildIndexes[ NodeIndex & ChildMask ]; }
		const FBounds3f&	GetBounds( uint32 NodeIndex ) const		{ return ChildBounds[ NodeIndex & ChildMask ]; }
		
		bool		IsRoot() const						{ return ParentIndex == ~0u; }
		bool		IsLeaf( uint32 NodeIndex ) const	{ return GetFirstChild( NodeIndex ) & 1; }
		bool		IsFull() const						{ return NumChildren == MaxChildren; }

		FBounds3f	UnionBounds() const
		{
			FBounds3f Bounds;
			for( uint32 i = 0; i < NumChildren; i++ )
			{
				Bounds += ChildBounds[i];
			}
			return Bounds;
		}
	};

	TArray< FNode >		Nodes;
	TArray< uint32 >	Leaves;
	uint32				FreeHead = ~0u;

	FLowestCostList		Candidates;

protected:
	FNode&			GetNode( uint32 NodeIndex )			{ return Nodes[ NodeIndex >> IndexShift ]; }
	const FNode&	GetNode( uint32 NodeIndex ) const	{ return Nodes[ NodeIndex >> IndexShift ]; }

	void	MarkDirty( uint32 NodeIndex )
	{
		DirtyPolicy.Mark( NodeIndex >> IndexShift );
	}

	void	Set( uint32 NodeIndex, const FBounds3f& Bounds, uint32 FirstChild )
	{
		GetNode( NodeIndex ).ChildBounds[ NodeIndex & ChildMask ] = Bounds;
		SetFirstChild( NodeIndex, FirstChild );
	}
	
	void	SetBounds( uint32 NodeIndex, const FBounds3f& Bounds )
	{
		GetNode( NodeIndex ).ChildBounds[ NodeIndex & ChildMask ] = Bounds;
		MarkDirty( NodeIndex );
	}

	void	SetFirstChild( uint32 NodeIndex, uint32 FirstChild )
	{
		GetNode( NodeIndex ).ChildIndexes[ NodeIndex & ChildMask ] = FirstChild;
		MarkDirty( NodeIndex );
		
		if( FirstChild & 1 )
		{
			Leaves[ FirstChild >> 1 ] = NodeIndex;
		}
		else
		{
			GetNode( FirstChild ).ParentIndex = NodeIndex;
			MarkDirty( FirstChild );
		}
	}

	uint32	FindBestInsertion_BranchAndBound( uint32 NodeIndex, const FBounds3f& RESTRICT Bounds );
	uint32	FindBestInsertion_Greedy( uint32 NodeIndex, const FBounds3f& RESTRICT Bounds );

	uint32	Insert( FRoot& RESTRICT Root, const FBounds3f& RESTRICT Bounds, uint32 NodeIndex );
	void	Extract( uint32 NodeIndex );
	void	RemoveAndSwap( uint32 NodeIndex );
	bool	RecursivePromoteChild( uint32 NodeIndex );
	uint32	PromoteChild( uint32 NodeIndex );
	void	Rotate( uint32 NodeIndex );
	
	uint32	AllocNode();
	void	FreeNode( uint32 NodeIndex );
	
	void	CheckNode( uint32 NodeIndex ) const;
};

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::FDynamicBVH()
{
	static_assert( MaxChildren > 1, "Must at least be binary tree." );
	static_assert( ( MaxChildren & (MaxChildren - 1) ) == 0, "MaxChildren must be power of 2" );
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
template< typename T  >
FORCEINLINE void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::Add( const TBounds<T>& Bounds, uint32 Index )
{
	if( Index >= (uint32)Leaves.Num() )
	{
		int32 Count = Index + 1 - Leaves.Num();
		int32 First = Leaves.AddUninitialized( Count );
		FMemory::Memset( &Leaves[ First ], 0xff, Count * sizeof( uint32 ) );
	}

	FRoot& Root = Roots.FindOrAdd( Bounds );

	if( Root.FirstChild == ~0u )
	{
		Root.FirstChild = AllocNode();
		GetNode( Root.FirstChild ).ParentIndex = ~0u;
		GetNode( Root.FirstChild ).NumChildren = 0;
	}

	check( Leaves[ Index ] == ~0u );
	Leaves[ Index ] = Insert( Root, Root.ToRelative( Bounds ), ( Index << 1 ) | 1 );

	//CheckNode( Leaves[ Index ] );
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
template< typename T  >
FORCEINLINE void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::Update( const TBounds<T>& Bounds, uint32 Index )
{
	Remove( Index );
	Add( Bounds, Index );
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
FORCEINLINE void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::Remove( uint32 Index )
{
	check( Leaves[ Index ] != ~0u );
	check( GetNode( Leaves[ Index ] ).GetFirstChild( Leaves[ Index ] ) == ( (Index << 1) | 1 ) );

	Extract( Leaves[ Index ] );
	Leaves[ Index ] = ~0u;
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
FORCEINLINE void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::SwapIndexes( uint32 Index0, uint32 Index1 )
{
	Swap( Leaves[ Index0 ], Leaves[ Index1 ] );
		
	uint32 NodeIndex0 = Leaves[ Index0 ];
	uint32 NodeIndex1 = Leaves[ Index1 ];

	if( NodeIndex0 != ~0u )
	{
		GetNode( NodeIndex0 ).ChildIndexes[ NodeIndex0 & ChildMask ] = (Index0 << 1) | 1;
		MarkDirty( NodeIndex0 );
	}

	if( NodeIndex1 != ~0u )
	{
		GetNode( NodeIndex1 ).ChildIndexes[ NodeIndex1 & ChildMask ] = (Index1 << 1) | 1;
		MarkDirty( NodeIndex1 );
	}
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
uint32 FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::FindBestInsertion_BranchAndBound( uint32 NodeIndex, const FBounds3f& RESTRICT Bounds )
{
	// Uses branch and bound search algorithm outlined in:
	// [ Bittner et al. 2012, "Fast Insertion-Based Optimization of Bounding Volume Hierarchies" ]
	
	// Binary tree nodes besides the root are always full meaning a new level will always be added.
	float MinAddedCost = MaxChildren > 2 ? 0.0f : CostMetric( Bounds );

	// Find best node to merge with.
	float	BestCost = MAX_flt;
	uint32	BestIndex = 0;

	Candidates.Reset();
	
	float InducedCost = 0.0f;

	while( true )
	{
		const FNode& RESTRICT Node = GetNode( NodeIndex );
		NumTested++;

		if( Node.IsFull() )
		{
			for( uint32 i = 0; i < Node.NumChildren; i += 4 )
			{
				FVector4f TotalCost;
				FVector4f ChildCost;
				constexpr uint32 Four = MaxChildren < 4 ? MaxChildren : 4;
				for( uint32 j = 0; j < Four; j++ )
				{
					const FBounds3f& RESTRICT NodeBounds = Node.ChildBounds[ i + j ];

					float DirectCost = CostMetric( Bounds + NodeBounds );
					// Cost if we need to add a level
					TotalCost[j] = InducedCost + DirectCost;
					// Induced cost for children
					ChildCost[j] = TotalCost[j] - CostMetric( NodeBounds );
				}

				for( uint32 j = 0; j < 4 && i + j < Node.NumChildren; j++ )
				{
					if( ChildCost[j] < BestCost )
					{
						if( TotalCost[j] < BestCost )
						{
							BestCost = TotalCost[j];
							BestIndex = NodeIndex + i + j;
						}

						uint32 FirstChild = Node.ChildIndexes[ i + j ];
						bool bIsLeaf = FirstChild & 1;
						if( !bIsLeaf )
						{
							Candidates.Add( ChildCost[j], FirstChild );
						}
					}
				}
			}
		}
		else
		{
			// Don't need to add a level because we can add a child.
			if( InducedCost < BestCost )
			{
				// Can't do better as this was already the smallest from the heap.
				return Node.ParentIndex;
			}
		}
		
		if( !Candidates.GetNext( BestCost, InducedCost, NodeIndex ) )
			break;
		
		if( InducedCost + MinAddedCost >= BestCost )
		{
			// Not possible to reduce cost further.
			break;
		}
	}

	return BestIndex;
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
uint32 FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::FindBestInsertion_Greedy( uint32 NodeIndex, const FBounds3f& RESTRICT Bounds )
{
	// Binary tree nodes besides the root are always full meaning a new level will always be added.
	float MinAddedCost = MaxChildren > 2 ? 0.0f : CostMetric( Bounds );

	// Find best node to merge with.
	float	BestCost = MAX_flt;
	uint32	BestIndex = 0;

	float InducedCost = 0.0f;

	do
	{
		FNode& RESTRICT Node = GetNode( NodeIndex );
		NumTested++;

		if( Node.IsFull() )
		{
			float	BestChildDist = MAX_flt;
			uint32	BestChildIndex = 0;

			for( uint32 i = 0; i < Node.NumChildren; i += 4 )
			{
				FVector4f Dist;
				constexpr uint32 Four = MaxChildren < 4 ? MaxChildren : 4;
				for( uint32 j = 0; j < Four; j++ )
				{
					const FBounds3f& RESTRICT NodeBounds = Node.ChildBounds[ i + j ];

					FVector3f Delta = ( Bounds.Min - NodeBounds.Min ) + ( Bounds.Max - NodeBounds.Max );
					Delta = Delta.GetAbs();
					Dist[j] = Delta.X + Delta.Y + Delta.Z;
				}

				for( uint32 j = 0; j < 4 && i + j < Node.NumChildren; j++ )
				{
					if( Dist[j] < BestChildDist )
					{
						BestChildDist = Dist[j];
						BestChildIndex = i + j;
					}
				}
			}

			const FBounds3f& RESTRICT ClosestBounds = Node.ChildBounds[ BestChildIndex ];

			float DirectCost = CostMetric( Bounds + ClosestBounds );
			// Cost if we need to add a level
			float TotalCost = InducedCost + DirectCost;
			// Induced cost for children
			float ChildCost = TotalCost - CostMetric( ClosestBounds );

			if( ChildCost >= BestCost )
				break;

			if( TotalCost < BestCost )
			{
				BestCost = TotalCost;
				BestIndex = NodeIndex + BestChildIndex;
			}

			uint32 FirstChild = Node.ChildIndexes[ BestChildIndex ];
			bool bIsLeaf = FirstChild & 1;
			if( bIsLeaf )
				break;

			InducedCost	= ChildCost;
			NodeIndex	= FirstChild;
		}
		else
		{
			// Don't need to add a level because we can add a child.
			// Can't do better. Cost is monotonic.
			return Node.ParentIndex;
		}
		
		if( InducedCost + MinAddedCost >= BestCost )
		{
			// Not possible to reduce cost further.
			break;
		}
	}
	while( NodeIndex != ~0u );

	return BestIndex;
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
uint32 FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::Insert( FRoot& RESTRICT Root, const FBounds3f& RESTRICT Bounds, uint32 Index )
{
	FNode& RootNode = GetNode( Root.FirstChild );
	if( !RootNode.IsFull() )
	{
		uint32 NodeIndex = Root.FirstChild + RootNode.NumChildren++;
		Set( NodeIndex, Bounds, Index );
		Root.Bounds += Bounds;
		return NodeIndex;
	}

	//uint32 BestIndex = FindBestInsertion_BranchAndBound( Root.FirstChild, Bounds );
	uint32 BestIndex = FindBestInsertion_Greedy( Root.FirstChild, Bounds );
	
	// Add to BestIndex's children
	uint32 NodeIndex = GetNode( BestIndex ).GetFirstChild( BestIndex );
	bool bIsLeaf = NodeIndex & 1;
	bool bAddLevel = bIsLeaf || GetNode( NodeIndex ).IsFull();
	if( bAddLevel )
	{
		// Create a new node and add NodeIndex as a child.
		uint32 NewNodeIndex = AllocNode();
		FNode& NewNode = GetNode( NewNodeIndex );

		NewNode.NumChildren = 1;
		Set( NewNodeIndex,
			GetNode( BestIndex ).GetBounds( BestIndex ),
			GetNode( BestIndex ).GetFirstChild( BestIndex ) );

		SetFirstChild( BestIndex, NewNodeIndex );

		checkSlow( NewNode.ParentIndex == BestIndex );
		checkSlow( NewNode.ChildIndexes[0] == NodeIndex );

		NodeIndex = NewNodeIndex;
	}
	
	FNode& Children = GetNode( NodeIndex );

	// Add child
	NodeIndex |= Children.NumChildren++;
	Set( NodeIndex, Bounds, Index );

	// Propagate bounds up tree
	FBounds3f PathBounds = Bounds;
	uint32    PathIndex  = BestIndex;
	while( PathIndex != ~0u )
	{
		FNode& PathNode = GetNode( PathIndex );
		SetBounds( PathIndex, PathNode.GetBounds( PathIndex ) + PathBounds );

		Rotate( PathIndex );

		PathBounds	= PathNode.GetBounds( PathIndex );
		PathIndex	= PathNode.ParentIndex;
	}
	Root.Bounds += PathBounds;

	return NodeIndex;
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::Extract( uint32 NodeIndex )
{
	FNode& Node = GetNode( NodeIndex );
	check( Node.IsRoot() || Node.NumChildren > 1 );

	RemoveAndSwap( NodeIndex );
	
	// Propagate bounds up tree
	FBounds3f PathBounds = Node.UnionBounds();
	uint32    PathIndex  = Node.ParentIndex;
	uint32    RootIndex  = NodeIndex;
	while( PathIndex != ~0u )
	{
		RootIndex = PathIndex;

		SetBounds( PathIndex, PathBounds );

		FNode& PathNode = GetNode( PathIndex );
		PathBounds	= PathNode.UnionBounds();
		PathIndex	= PathNode.ParentIndex;
	}
	Roots.FindChecked( RootIndex & ~ChildMask ).Bounds = PathBounds;

	if( !Node.IsRoot() && Node.NumChildren == 1 )
	{
		Set( Node.ParentIndex,
			Node.ChildBounds[0],
			Node.ChildIndexes[0] );
			
		FreeNode( NodeIndex );
	}
	else if( Node.IsRoot() && Node.NumChildren == 0 )
	{
		Roots.Remove( NodeIndex & ~ChildMask );
		FreeNode( NodeIndex );
	}
	else
	{
		RecursivePromoteChild( NodeIndex );
	}
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::RemoveAndSwap( uint32 NodeIndex )
{
	FNode& Node = GetNode( NodeIndex );

	uint32 LastChild = --Node.NumChildren;
	if( ( NodeIndex & ChildMask ) < LastChild )
	{
		// Fill with last
		Set( NodeIndex,
			Node.GetBounds( LastChild ),
			Node.GetFirstChild( LastChild ) );
	}
}

// Recursively promotes children to fill the hole until it reaches a leaf.
// The result of doing this on every Extract is that all inner nodes are guarenteed to be full.
template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
bool FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::RecursivePromoteChild( uint32 NodeIndex )
{
	bool bPromoted = false;

	do
	{
		FNode& PathNode = GetNode( NodeIndex );

		// Find best node to promote a child from.
		float	BestCost = 0.0f;
		uint32	BestIndex = ~0u;
		for( uint32 i = 0; i < PathNode.NumChildren; i++ )
		{
			if( !PathNode.IsLeaf(i) )
			{
				float Cost = CostMetric( PathNode.ChildBounds[i] );
				if( Cost > BestCost )
				{
					BestCost = Cost;
					BestIndex = ( NodeIndex & ~ChildMask ) | i;
				}
			}
		}

		if( BestIndex == ~0u )
			break;

		NodeIndex = PromoteChild( BestIndex );
		bPromoted = true;
	}
	while( NodeIndex != ~0u );

	return bPromoted;
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
uint32 FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::PromoteChild( uint32 NodeIndex )
{
	FNode& Node = GetNode( NodeIndex );
	check( !Node.IsLeaf( NodeIndex ) );
	check( Node.NumChildren < MaxChildren );

	uint32 FirstChild = Node.GetFirstChild( NodeIndex );
	FNode& Children = GetNode( FirstChild );
	
	FBounds3f Excluded[ MaxChildren ];

	// Sweep forward and backward with prefix and postfix sums. Prefix + postfix sums == sum excluding this.
	FBounds3f Forward;
	FBounds3f Back;
	for( uint32 i = 0; i < Children.NumChildren; i++ )
	{
		uint32 j = Children.NumChildren - 1 - i;

		Excluded[i] += Forward;
		Excluded[j] += Back;

		Forward	+= Children.ChildBounds[i];
		Back	+= Children.ChildBounds[j];
	}

	float	BestCost = MAX_flt;
	uint32	BestIndex = ~0u;
	
	for( uint32 i = 0; i < Children.NumChildren; i++ )
	{
		float Cost = CostMetric( Excluded[i] );
		if( Cost < BestCost )
		{
			BestCost = Cost;
			BestIndex = FirstChild | i;
		}
	}
	
	// Promote from child to sibling
	
	// Remove from bounds
	CA_SUPPRESS(6385);
	SetBounds( NodeIndex, Excluded[ BestIndex & ChildMask ] );

	// Add as sibling
	uint32 SiblingIndex = ( NodeIndex & ~ChildMask ) | Node.NumChildren;
	Set( SiblingIndex,
		Children.GetBounds( BestIndex ),
		Children.GetFirstChild( BestIndex ) );
	Node.NumChildren++;

	// Remove from children
	uint32 LastChild = --Children.NumChildren;
	if( Children.NumChildren == 1 )
	{
		uint32 OtherChild = ~BestIndex & 1;

		Set( NodeIndex,
			Children.ChildBounds[ OtherChild ],
			Children.ChildIndexes[ OtherChild ] );
		
		// Delete Children
		FreeNode( BestIndex );
		BestIndex = ~0u;
	}
	else if( ( BestIndex & ChildMask ) != LastChild )
	{
		// Fill with last
		Set( BestIndex,
			Children.GetBounds( LastChild ),
			Children.GetFirstChild( LastChild ) );
	}

	return BestIndex;
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::Rotate( uint32 NodeIndex )
{
	FNode& Node = GetNode( NodeIndex );

	if( Node.IsRoot() )
		return;

	FBounds3f ExcludedBounds;
	for( uint32 i = 0; i < Node.NumChildren; i++ )
	{
		if( i != (NodeIndex & ChildMask) )
			ExcludedBounds += Node.ChildBounds[i];
	}
	
	FNode& ParentNode = GetNode( Node.ParentIndex );

	float	BestCost = CostMetric( ParentNode.GetBounds( Node.ParentIndex ) );
	uint32	BestIndex = ~0u;
	for( uint32 i = 0; i < ParentNode.NumChildren; i++ )
	{
		if( i != (Node.ParentIndex & ChildMask) )
		{
			// Parent's sibling
			float Cost = CostMetric( ExcludedBounds + ParentNode.ChildBounds[i] );
			if( Cost < BestCost )
			{
				BestCost = Cost;
				BestIndex = ( Node.ParentIndex & ~ChildMask ) | i;
			}
		}
	}

	if( BestIndex != ~0u )
	{
		// Swap
		FBounds3f Bounds	= Node.GetBounds( NodeIndex );
		uint32 FirstChild	= Node.GetFirstChild( NodeIndex );

		Set( NodeIndex, ParentNode.GetBounds( BestIndex ), ParentNode.GetFirstChild( BestIndex ) );
		Set( BestIndex, Bounds, FirstChild );
	}
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
FORCEINLINE uint32 FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::AllocNode()
{
	if( FreeHead != ~0u )
	{
		uint32 NodeIndex = FreeHead;
		uint32& NextIndex = GetNode( NodeIndex ).ParentIndex;
		FreeHead = NextIndex;
		NextIndex = ~0u;
		return NodeIndex;
	}
	else
	{
		DirtyPolicy.Add( Nodes.Num() );
		return Nodes.AddUninitialized() << IndexShift;
	}
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
FORCEINLINE void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::FreeNode( uint32 NodeIndex )
{
	// Assumes nothing is still linking to it
	GetNode( NodeIndex ).ParentIndex = FreeHead;
	GetNode( NodeIndex ).NumChildren = 0;
	FreeHead = NodeIndex & ~ChildMask;
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::CheckNode( uint32 NodeIndex ) const
{
	const FNode& Node = GetNode( NodeIndex );

	check( ( NodeIndex & ChildMask ) < Node.NumChildren );
	
	if( !Node.IsRoot() )
	{
		check( Node.NumChildren > 1 );
		check( GetNode( Node.ParentIndex ).GetFirstChild( Node.ParentIndex ) == ( NodeIndex & ~ChildMask ) );
	}

	uint32 FirstChild = Node.GetFirstChild( NodeIndex );
	if( FirstChild & 1 )
	{
		check( Leaves[ FirstChild >> 1 ] == NodeIndex );
	}
	else
	{
		check( ( FirstChild & ChildMask ) == 0 );

		const FNode& Children = GetNode( FirstChild );
		for( uint32 i = 0; i < Children.NumChildren; i++ )
		{
			check( Children.ParentIndex == NodeIndex );
		}
	}
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
template< typename T, typename FFuncType >
void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::ForAll( const TBounds<T>& Bounds, const FFuncType& Func ) const
{
	TArray< uint32, TInlineAllocator<256> > Stack;

	Roots.ForAll(
		[ this, &Stack, &Bounds, &Func ]( const FRoot& Root )
		{
			FBounds3f RelativeBounds = Root.ToRelative( Bounds );

			if( !RelativeBounds.Intersect( Root.Bounds ) )
				return;

			uint32 NodeIndex = Root.FirstChild;

			while( true )
			{
				const FNode& RESTRICT Node = GetNode( NodeIndex );

				for( uint32 i = 0; i < Node.NumChildren; i++ )
				{
					// TODO detect fully contained and stop intersection testing.
					if( RelativeBounds.Intersect( Node.ChildBounds[i] ) )
					{
						uint32 FirstChild = Node.ChildIndexes[i];
						if( FirstChild & 1 )
						{
							// Leaf
							Func( FirstChild >> 1 );
						}
						else
						{
							Stack.Push( FirstChild );
						}
					}
				}

				if( Stack.Num() == 0 )
					break;

				NodeIndex = Stack.Pop( EAllowShrinking::No );
			}
		} );
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
template< typename FPredicate, typename FFuncType >
void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::ForAll( const FPredicate& Predicate, const FFuncType& Func ) const
{
	TArray< uint32, TInlineAllocator<256> > Stack;

	Roots.ForAll(
		[ this, &Stack, &Predicate, &Func ]( const FRoot& Root )
		{
			if( !Predicate( Root.ToAbsolute( Root.Bounds ) ) )
				return;

			uint32 NodeIndex = Root.FirstChild;

			while( true )
			{
				const FNode& RESTRICT Node = GetNode( NodeIndex );

				for( uint32 i = 0; i < Node.NumChildren; i++ )
				{
					if( Predicate( Root.ToAbsolute( Node.ChildBounds[i] ) ) )
					{
						uint32 FirstChild = Node.ChildIndexes[i];
						if( FirstChild & 1 )
						{
							// Leaf
							Func( FirstChild >> 1 );
						}
						else
						{
							Stack.Push( FirstChild );
						}
					}
				}

				if( Stack.Num() == 0 )
					break;

				NodeIndex = Stack.Pop( EAllowShrinking::No );
			}
		} );
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
template< typename FFuncType >
void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::ForAllDirty( const FFuncType& Func )
{
	DirtyPolicy.ForAll(
		[&]( uint32 Index )
		{
			Func( Index, GetNode( Index << IndexShift ) );
		}
	);
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
template< typename T, typename FFuncType >
uint32 FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::FindClosest( const UE::Math::TVector<T>& Position, const FFuncType& LeafDistSqr )
{
	float	ClosestDistSqr = MAX_flt;
	uint32	ClosestIndex = ~0u;

	Candidates.Reset();

	Roots.ForAll(
		[ this, &Position, &LeafDistSqr, &ClosestDistSqr, &ClosestIndex ]( const FRoot& Root )
		{
			FVector3f RelativePosition = Root.ToRelative( Position );

			float	NodeDistSqr	= Root.Bounds.DistSqr( RelativePosition );
			uint32	NodeIndex	= Root.FirstChild;

			while( NodeDistSqr < ClosestDistSqr )
			{
				const FNode& RESTRICT Node = GetNode( NodeIndex );

				for( uint32 i = 0; i < Node.NumChildren; i += 4 )
				{
					FVector4f ChildDistSqr;
					constexpr uint32 Four = MaxChildren < 4 ? MaxChildren : 4;
					for( uint32 j = 0; j < Four; j++ )
					{
						const FBounds3f& RESTRICT NodeBounds = Node.ChildBounds[ i + j ];

						ChildDistSqr[j] = NodeBounds.DistSqr( RelativePosition );
					}

					for( uint32 j = 0; j < 4 && i + j < Node.NumChildren; j++ )
					{
						if( ChildDistSqr[j] < ClosestDistSqr )
						{
							uint32 FirstChild = Node.ChildIndexes[ i + j ];
							if( FirstChild & 1 )
							{
								uint32 Index = FirstChild >> 1;
								float DistSqr = LeafDistSqr( Position, Index );
								if( DistSqr < ClosestDistSqr )
								{
									ClosestDistSqr	= DistSqr;
									ClosestIndex	= Index;
								}
							}
							else
							{
								Candidates.Add( ChildDistSqr[j], FirstChild );
							}
						}
					}
				}
		
				if( !Candidates.GetNext( ClosestDistSqr, NodeDistSqr, NodeIndex ) )
					break;
			}
		} );

	return ClosestIndex;
}

class FMortonArray
{
public:
	struct FRange
	{
		int32	Begin;
		int32	End;

		int32	Num() const { return End - Begin; }
	};
	
public:
			FMortonArray( const TArray< FBounds3f >& InBounds );

	uint32	GetIndex( int32 i ) const { return Sorted[i].Index; }
	uint32	Split( const FRange& Range );
	
private:
	void	RegenerateCodes( const FRange& Range );

	struct FSortPair
	{
		uint32 Code;
		uint32 Index;

		bool operator<( const FSortPair& Other ) const { return Code < Other.Code; }
	};
	TArray< FSortPair >			Sorted;

	const TArray< FBounds3f >&	Bounds;
};

FORCEINLINE uint32 FMortonArray::Split( const FRange& Range )
{
	uint32 Code0 = Sorted[ Range.Begin ].Code;
	uint32 Code1 = Sorted[ Range.End - 1 ].Code;
	uint32 Diff = Code0 ^ Code1;
	if( Diff == 0 )
	{
		RegenerateCodes( Range );

		Code0 = Sorted[ Range.Begin ].Code;
		Code1 = Sorted[ Range.End - 1 ].Code;
		Diff = Code0 ^ Code1;

		if( Diff == 0 )
			return ( Range.Begin + Range.End ) >> 1;
	}

	uint32 HighestBitDiff = FMath::FloorLog2( Diff );
	uint32 Mask = 1 << HighestBitDiff;

	int32 Min = Range.Begin;
	int32 Max = Range.End;
	while( Min + 1 != Max )
	{
		int32 Mid = ( Min + Max ) >> 1;
		if( Sorted[ Mid ].Code & Mask )
			Max = Mid;
		else
			Min = Mid;
	}
	
	return Max;
}

template< uint32 MaxChildren, typename FRootPolicy, typename FDirtyPolicy, typename FCostMetric >
void FDynamicBVH< MaxChildren, FRootPolicy, FDirtyPolicy, FCostMetric >::Build( const TArray< FBounds3f >& BoundsArray, uint32 FirstIndex )
{
	if( FirstIndex + BoundsArray.Num() > (uint32)Leaves.Num() )
	{
		int32 Count = FirstIndex + BoundsArray.Num() - Leaves.Num();
		int32 First = Leaves.AddUninitialized( Count );
		FMemory::Memset( &Leaves[ First ], 0xff, Count * sizeof( uint32 ) );
	}
	
	FMortonArray MortonArray( BoundsArray );

	using FRange = FMortonArray::FRange;

	// TEMP Start empty
	FRoot& Root = Roots.FindOrAdd( FBounds3f( { FVector3f::ZeroVector, FVector3f::ZeroVector } ) );
	Root.FirstChild = 0;

	struct FCreateNode
	{
		uint32	ParentIndex;
		FRange	Range;
	};
	TArray< FCreateNode, TInlineAllocator<32> > Stack;

	uint32 ParentIndex = ~0u;
	FRange Range = { 0, BoundsArray.Num() };

	while( true )
	{
		uint32 NodeIndex = AllocNode();
		FNode& Node = GetNode( NodeIndex );

		Node.ParentIndex = ParentIndex;
		if( ParentIndex != ~0u )
			SetFirstChild( ParentIndex, NodeIndex );

		check( Range.Begin < Range.End );

		int32 NumLeaves = Range.Num();
		if( NumLeaves <= MaxChildren )
		{
			Node.NumChildren = NumLeaves;
			for( int32 i = 0; i < NumLeaves; i++ )
			{
				uint32 Index = MortonArray.GetIndex( Range.Begin + i );
				Set( NodeIndex + i, BoundsArray[ Index ], ( ( FirstIndex + Index ) << 1 ) | 1 );
			}

			// Propagate bounds up tree
			FBounds3f PathBounds = Node.UnionBounds();
			uint32    PathIndex  = Node.ParentIndex;
			while( PathIndex != ~0u )
			{
				SetBounds( PathIndex, PathBounds );

				// Only continue if first child, which signifies the node is complete.
				if( PathIndex & ChildMask )
					break;

				FNode& PathNode = GetNode( PathIndex );
				PathBounds = PathNode.UnionBounds();
				PathIndex  = PathNode.ParentIndex;
			}
			// TODO: RootBounds

			if( Stack.Num() == 0 )
				break;

			ParentIndex	= Stack.Last().ParentIndex;
			Range		= Stack.Last().Range;

			Stack.Pop( EAllowShrinking::No );
		}
		else
		{
			FRange Children[ MaxChildren ];
			Children[0] = Range;

			int32 NumChildren = 1;
			int32 SplitIndex = 0;
			do
			{
				FRange Child = Children[ SplitIndex ];

				uint32 Middle = MortonArray.Split( Child );
				check( Middle != Child.Begin );
				check( Middle != Child.End );

				Children[ SplitIndex ].Begin	= Child.Begin;
				Children[ SplitIndex ].End		= Middle;
				Children[ NumChildren ].Begin	= Middle;
				Children[ NumChildren ].End		= Child.End;
				NumChildren++;

				int32 LargestNum = 0;
				int32 LargestIndex = -1;
				for( int32 i = 0; i < NumChildren; i++ )
				{
					int32 Num = Children[i].Num();
					if( Num <= MaxChildren )
						continue;

					if( Num > LargestNum )
					{
						LargestNum = Num;
						LargestIndex = i;
					}
				}

				SplitIndex = LargestIndex;
			}
			while( NumChildren < MaxChildren && SplitIndex >= 0 );
			
			Node.NumChildren = NumChildren;

			// Move leaves to the back
			for( int32 Front = 0, Back = NumChildren - 1; Front < Back; )
			{
				if( Children[ Front ].Num() == 1 )
					Swap( Children[ Front ], Children[ Back-- ] );
				else
					Front++;
			}

			NumLeaves = 0;
			for( int32 i = NumChildren - 1; i >= 0; i-- )
			{
				bool bIsLeaf = Children[i].Num() == 1;
				if( !bIsLeaf )
					break;
				NumLeaves++;

				uint32 Index = MortonArray.GetIndex( Children[i].Begin );
				Set( NodeIndex + i, BoundsArray[ Index ], ( ( FirstIndex + Index ) << 1 ) | 1 );
			}
			check( NumLeaves < NumChildren );

			int32 Last = NumChildren - NumLeaves - 1;
			for( int32 i = 0; i < Last; i++ )
			{
				Stack.Push( { NodeIndex + i, Children[i] } );
			}

			ParentIndex = NodeIndex + Last;
			Range = Children[ Last ];
		}
	}
}
