// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DynamicPrimitiveDrawing.h: Dynamic primitive drawing definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "SceneManagement.h"
#include "SceneRendering.h"

class FGPUScenePrimitiveCollector;

/** A primitive draw interface which adds the drawn elements to the view's batched elements. */
class FViewElementPDI : public FPrimitiveDrawInterface
{
public:

	FViewElementPDI(FViewInfo* InViewInfo,FHitProxyConsumer* InHitProxyConsumer, FGPUScenePrimitiveCollector *InDynamicPrimitiveCollector);

	// FPrimitiveDrawInterface interface.
	virtual bool IsHitTesting() override;
	virtual void SetHitProxy(HHitProxy* HitProxy) override;
	virtual void RegisterDynamicResource(FDynamicPrimitiveResource* DynamicResource) override;
	virtual void AddReserveLines(uint8 DepthPriorityGroup, int32 NumLines, bool bDepthBiased = false, bool bThickLines = false) override;
	virtual void DrawSprite(
		const FVector& Position,
		float SizeX,
		float SizeY,
		const FTexture* Sprite,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float U,
		float UL,
		float V,
		float VL,
		uint8 BlendMode = SE_BLEND_Masked,
		float OpacityMaskRefVal = .5f
		) override;

	virtual void DrawLine(
		const FVector& Start,
		const FVector& End,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float Thickness = 0.0f,
		float DepthBias = 0.0f,
		bool bScreenSpace = false
		) override;
	virtual void DrawTranslucentLine(
		const FVector& Start,
		const FVector& End,
		const FLinearColor& Color,
		uint8 DepthPriorityGroup,
		float Thickness = 0.0f,
		float DepthBias = 0.0f,
		bool bScreenSpace = false
	) override;
	virtual void DrawPoint(
		const FVector& Position,
		const FLinearColor& Color,
		float PointSize,
		uint8 DepthPriorityGroup
		) override;
	virtual int32 DrawMesh(const FMeshBatch& Mesh) override;

private:
	FViewInfo* ViewInfo;
	TRefCountPtr<HHitProxy> CurrentHitProxy;
	FHitProxyConsumer* HitProxyConsumer;
	FGPUScenePrimitiveCollector* DynamicPrimitiveCollector;

	/** Depending of the DPG we return a different FBatchedElement instance. */
	FBatchedElements& GetElements(uint8 DepthPriorityGroup) const;
};

#include "DynamicPrimitiveDrawing.inl" // IWYU pragma: export

