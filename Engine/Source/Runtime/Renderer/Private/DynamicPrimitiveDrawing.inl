// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DynamicPrimitiveDrawing.inl: Dynamic primitive drawing implementation.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "CoreFwd.h"
#include "Materials/MaterialRenderProxy.h"

class FBatchedElements;
class FDynamicPrimitiveResource;
class FHitProxyConsumer;
class FHitProxyId;
class FParallelCommandListSet;
class FPrimitiveDrawInterface;
class FRenderTask;
class FRHICommandList;
class FTexture;
class FViewElementPDI;
class FViewInfo;
class HHitProxy;
struct FMath;
struct FMeshBatch;
struct FMeshBatchElement;
struct TStatId;
template<typename TTask> class TGraphTask;

inline FViewElementPDI::FViewElementPDI(FViewInfo* InViewInfo,FHitProxyConsumer* InHitProxyConsumer, FGPUScenePrimitiveCollector* InDynamicPrimitiveCollector):
	FPrimitiveDrawInterface(InViewInfo),
	ViewInfo(InViewInfo),
	HitProxyConsumer(InHitProxyConsumer),
	DynamicPrimitiveCollector(InDynamicPrimitiveCollector)
{}

inline bool FViewElementPDI::IsHitTesting()
{
	return HitProxyConsumer != NULL;
}
inline void FViewElementPDI::SetHitProxy(HHitProxy* HitProxy)
{
	// Change the current hit proxy.
	CurrentHitProxy = HitProxy;

	if(HitProxyConsumer && HitProxy)
	{
		// Notify the hit proxy consumer of the new hit proxy.
		HitProxyConsumer->AddHitProxy(HitProxy);
	}
}

inline void FViewElementPDI::RegisterDynamicResource(FDynamicPrimitiveResource* DynamicResource)
{
	if (IsInGameThread())
	{
		// Render thread might be reading the array while we are adding in the game thread
		FViewInfo* InViewInfo = ViewInfo;
		ENQUEUE_RENDER_COMMAND(AddViewInfoDynamicResource)(
			[InViewInfo, DynamicResource](FRHICommandListImmediate& RHICmdList)
			{
				InViewInfo->DynamicResources.Add(DynamicResource);
				DynamicResource->InitPrimitiveResource(RHICmdList);
			});
	}
	else
	{
		ViewInfo->DynamicResources.Add(DynamicResource);
		DynamicResource->InitPrimitiveResource(FRHICommandListImmediate::Get());
	}
}

inline FBatchedElements& FViewElementPDI::GetElements(uint8 DepthPriorityGroup) const
{
	return DepthPriorityGroup ? ViewInfo->TopBatchedViewElements : ViewInfo->BatchedViewElements;
}

inline void FViewElementPDI::DrawSprite(
	const FVector& Position,
	float SizeX,
	float SizeY,
	const FTexture* Sprite,
	const FLinearColor& Color,
	uint8 DepthPriorityGroup,
	float U,
	float UL,
	float V,
	float VL,
	uint8 BlendMode,
	float OpacityMaskRefVal
	)
{
	FBatchedElements &Elements = GetElements(DepthPriorityGroup);

	Elements.AddSprite(
		Position,
		SizeX,
		SizeY,
		Sprite,
		Color,
		CurrentHitProxy ? CurrentHitProxy->Id : FHitProxyId(),
		U, UL, V, VL,
		BlendMode,
		OpacityMaskRefVal
	);
}

inline void FViewElementPDI::AddReserveLines(uint8 DepthPriorityGroup, int32 NumLines, bool bDepthBiased, bool bThickLines)
{
	FBatchedElements& Elements = GetElements(DepthPriorityGroup);

	Elements.AddReserveLines(NumLines, bDepthBiased, bThickLines);
}

inline void FViewElementPDI::DrawLine(
	const FVector& Start,
	const FVector& End,
	const FLinearColor& Color,
	uint8 DepthPriorityGroup,
	float Thickness,
	float DepthBias,
	bool bScreenSpace
	)
{
	FBatchedElements &Elements = GetElements(DepthPriorityGroup);

	Elements.AddLine(
		Start,
		End,
		Color,
		CurrentHitProxy ? CurrentHitProxy->Id : FHitProxyId(),
		Thickness,
		DepthBias,
		bScreenSpace
	);
}

inline void FViewElementPDI::DrawTranslucentLine(
	const FVector& Start,
	const FVector& End,
	const FLinearColor& Color,
	uint8 DepthPriorityGroup,
	float Thickness,
	float DepthBias,
	bool bScreenSpace
)
{
	FBatchedElements& Elements = GetElements(DepthPriorityGroup);

	Elements.AddTranslucentLine(
		Start,
		End,
		Color,
		CurrentHitProxy ? CurrentHitProxy->Id : FHitProxyId(),
		Thickness,
		DepthBias,
		bScreenSpace
	);
}

inline void FViewElementPDI::DrawPoint(
	const FVector& Position,
	const FLinearColor& Color,
	float PointSize,
	uint8 DepthPriorityGroup
	)
{
	float ScaledPointSize = PointSize;

	bool bIsPerspective = (ViewInfo->ViewMatrices.GetViewToClip().M[3][3] < 1.0f) ? true : false;
	if( !bIsPerspective )
	{
		const float ZoomFactor = FMath::Min<float>(View->ViewMatrices.GetViewToClip().M[0][0], View->ViewMatrices.GetViewToClip().M[1][1]);
		ScaledPointSize = ScaledPointSize / ZoomFactor;
	}

	FBatchedElements &Elements = GetElements(DepthPriorityGroup);

	Elements.AddPoint(
		Position,
		ScaledPointSize,
		Color,
		CurrentHitProxy ? CurrentHitProxy->Id : FHitProxyId()
	);
}

inline bool MeshBatchHasPrimitives(const FMeshBatch& Mesh)
{
	bool bHasPrimitives = true;
	const int32 NumElements = Mesh.Elements.Num();
	for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
	{
		const FMeshBatchElement& Element = Mesh.Elements[ElementIndex];
		bHasPrimitives = bHasPrimitives && Element.NumPrimitives > 0 && Element.NumInstances > 0;
	}
	return bHasPrimitives;
}

inline int32 FViewElementPDI::DrawMesh(const FMeshBatch& Mesh)
{
	// Warning: can be called from Game Thread or Rendering Thread.  Be careful what you access.
	check(DynamicPrimitiveCollector);

	if (ensure(MeshBatchHasPrimitives(Mesh)))
	{
		// Keep track of view mesh elements whether that have translucency.
		ViewInfo->bHasTranslucentViewMeshElements |= true;//Mesh.IsTranslucent() ? 1 : 0;

		uint8 DPGIndex = Mesh.DepthPriorityGroup;
		// Get the correct element list based on dpg index
		// Translucent view mesh elements in the foreground dpg are not supported yet
		TIndirectArray<FMeshBatch, SceneRenderingAllocator>& ViewMeshElementList = ( ( DPGIndex == SDPG_Foreground  ) ? ViewInfo->TopViewMeshElements : ViewInfo->ViewMeshElements );

		FMeshBatch* NewMesh = new FMeshBatch(Mesh);
		ViewMeshElementList.Add(NewMesh);
		if( CurrentHitProxy != nullptr )
		{
			NewMesh->BatchHitProxyId = CurrentHitProxy->Id;
		}

		{
			ERHIFeatureLevel::Type FeatureLevel = ViewInfo->GetFeatureLevel();
			EShaderPlatform ShaderPlatform = ViewInfo->GetShaderPlatform();
			ENQUEUE_RENDER_COMMAND(FCopyDynamicPrimitiveShaderData)(
				[NewMesh, DynamicPrimitiveCollectorForRT = DynamicPrimitiveCollector, FeatureLevel, ShaderPlatform](FRHICommandListImmediate& RHICmdList)
				{
					const bool bPrimitiveShaderDataComesFromSceneBuffer = NewMesh->VertexFactory->GetPrimitiveIdStreamIndex(FeatureLevel, EVertexInputStreamType::Default) >= 0;

					for (int32 ElementIndex = 0; ElementIndex < NewMesh->Elements.Num(); ElementIndex++)
					{
						FMeshBatchElement& MeshElement = NewMesh->Elements[ElementIndex];

						if (bPrimitiveShaderDataComesFromSceneBuffer)
						{
							checkf(!NewMesh->Elements[ElementIndex].PrimitiveUniformBuffer,
								TEXT("FMeshBatch was assigned a PrimitiveUniformBuffer even though Vertex Factory %s fetches primitive shader data through a Scene buffer.  The assigned PrimitiveUniformBuffer cannot be respected.  Use PrimitiveUniformBufferResource instead for dynamic primitive data, or leave both null to get FPrimitiveSceneProxy->UniformBuffer."), NewMesh->VertexFactory->GetType()->GetName());
						}

						checkf(bPrimitiveShaderDataComesFromSceneBuffer || NewMesh->Elements[ElementIndex].PrimitiveUniformBufferResource != NULL,
							TEXT("FMeshBatch was not properly setup.  The primitive uniform buffer must be specified."));
					}

					// If we are maintaining primitive scene data on the GPU, copy the primitive uniform buffer data to a unified array so it can be uploaded later
					if (UseGPUScene(ShaderPlatform) && bPrimitiveShaderDataComesFromSceneBuffer)
					{
						for (int32 Index = 0; Index < NewMesh->Elements.Num(); ++Index)
						{
							FMeshBatchElement& Element = NewMesh->Elements[Index];
							const TUniformBuffer<FPrimitiveUniformShaderParameters>* PrimitiveUniformBufferResource = Element.PrimitiveUniformBufferResource;

							if (PrimitiveUniformBufferResource)
							{
								Element.PrimitiveIdMode = PrimID_DynamicPrimitiveShaderData;
								DynamicPrimitiveCollectorForRT->Add(
									Element.DynamicPrimitiveData,
									*reinterpret_cast<const FPrimitiveUniformShaderParameters*>(PrimitiveUniformBufferResource->GetContents()),
									Element.NumInstances,
									Element.DynamicPrimitiveIndex,
									Element.DynamicPrimitiveInstanceSceneDataOffset);
							}
						}
					}

					NewMesh->MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(FeatureLevel);
				});
		}		

		return 1;
	}
	return 0;
}

