// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicShadowMapChannelBindingHelper.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "PrimitiveDrawingUtils.h"

void FDynamicShadowMapChannelBindingHelper::DisableChannel(int32 ChannelIndex)
{
	if (IsChannelEnabled(ChannelIndex))
	{
		Channels[ChannelIndex].bIsAvailable = false;
		Channels[ChannelIndex].Lights.Empty();
		--AvailableChannelCount;
	}
}

// This also works when passing INDEX_NONE.
void FDynamicShadowMapChannelBindingHelper::DisableAllOtherChannels(int32 EnabledChannelIndex)
{
	for (int32 ChannelIndex = 0; ChannelIndex < CHANNEL_COUNT; ++ChannelIndex)
	{
		if (ChannelIndex != EnabledChannelIndex)
		{
			DisableChannel(ChannelIndex);
		}
	}
}

// Update channel usage based on the current enabled channels. Channels with light of MaxPriority or more will be disabled.
void FDynamicShadowMapChannelBindingHelper::UpdateAvailableChannels(const FSceneLightInfoArray& Lights, FLightSceneInfo* LightInfo)
{
	const int32 LightPriority = LightInfo ? GetPriority(LightInfo) : INDEX_NONE;
	if (LightPriority != INDEX_NONE)
	{
		const FSphere LightBounds = LightInfo->Proxy->GetBoundingSphere();
		for (const FLightSceneInfoCompact LightSceneInfoCompact : Lights)
		{
			FLightSceneInfo* OtherLightInfo = LightSceneInfoCompact.LightSceneInfo;
			if (OtherLightInfo && (OtherLightInfo->Proxy->CastsDynamicShadow() || OtherLightInfo->Proxy->GetLightFunctionMaterial()) && OtherLightInfo != LightInfo)
			{
				const int32 OtherLightChannel = OtherLightInfo->GetDynamicShadowMapChannel();
				if (OtherLightChannel != INDEX_NONE && IsChannelEnabled(OtherLightChannel))
				{
					const int32 OtherLightPiority = GetPriority(OtherLightInfo);
					// If the two lights are static shadowing, then the bound intersection test here is invalid and the channels can't be reassigned anyway.
					if (OtherLightPiority != INDEX_NONE && (LightPriority < STATIC_SHADOWING_PRIORITY || OtherLightPiority < STATIC_SHADOWING_PRIORITY) &&  OtherLightInfo->Proxy->AffectsBounds(LightBounds))
					{
						if (OtherLightPiority < LightPriority)
						{
							// If LightInfo gets assigned to this channel, then OtherLightInfo will need another channel.
							AddLight(OtherLightChannel, OtherLightInfo);
						}
						else // Otherwise we are not allowed to insert lights on channels used by lights of same priority.
						{
							DisableChannel(OtherLightChannel);
							if (!HasAnyChannelEnabled())
							{
								break;
							}
						}
					}
				}
			}
		}
	}
	else
	{
		DisableAllOtherChannels(INDEX_NONE);
	}
}

int32 FDynamicShadowMapChannelBindingHelper::GetBestAvailableChannel() const
{
	int32 BestIndex = INDEX_NONE;
	if (HasAnyChannelEnabled())
	{
		// Try to find an index between 0 and Count - 1 (CSM currently have issue with index 3)
		for (int32 ChannelIndex = 0; ChannelIndex < CHANNEL_COUNT; ++ChannelIndex)
		{
			if (Channels[ChannelIndex].bIsAvailable)
			{
				if (BestIndex == INDEX_NONE)
				{
					BestIndex = ChannelIndex;
				}
				// If this channel has less light to remove then the previously chosen channel.
				else if (Channels[BestIndex].Lights.Num() > Channels[ChannelIndex].Lights.Num())
				{
					BestIndex = ChannelIndex;
				}
			}
		}
	}		
	return BestIndex;
}


void FDynamicShadowMapChannelBindingHelper::SortLightByPriority(int32 ChannelIndex)
{
	check(ChannelIndex >= 0 && ChannelIndex < CHANNEL_COUNT);

	Channels[ChannelIndex].Lights.Sort([](const FLightSceneInfo& A, const FLightSceneInfo& B)
	{
		return GetPriority(&A) >= GetPriority(&B);
	});
}

int32 FDynamicShadowMapChannelBindingHelper::GetPriority(const FLightSceneInfo* Light)
{
	check(Light);
	const FLightSceneProxy* Proxy = Light->Proxy;
	if (Proxy && (Proxy->CastsDynamicShadow() || Proxy->GetLightFunctionMaterial()))
	{
		if (Proxy->HasStaticShadowing())
		{
			return STATIC_SHADOWING_PRIORITY;
		}
		else if (Proxy->GetLightType() == LightType_Directional)
		{
			return 1;
		}
		else
		{
			return 0;
		}
	}
	return INDEX_NONE;
}
