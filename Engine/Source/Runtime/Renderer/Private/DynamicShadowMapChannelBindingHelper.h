// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DynamicShadowMapChannelBindingHelper.h: 
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "LightSceneInfo.h"

class FLightSceneInfo;
class FLightSceneInfoCompact;

// This is used in forward only
class FDynamicShadowMapChannelBindingHelper
{
public:

	// The number of valid dynamic shadowmap channels
	static const int32 CHANNEL_COUNT = 4;
	// This is used in GetPriority().
	static const int32 STATIC_SHADOWING_PRIORITY = 2;

	FDynamicShadowMapChannelBindingHelper() : AvailableChannelCount(CHANNEL_COUNT) {}

	FORCEINLINE bool IsChannelEnabled(int32 ChannelIndex)
	{
		check(ChannelIndex >= 0 && ChannelIndex < CHANNEL_COUNT);
		return Channels[ChannelIndex].bIsAvailable;
	}

	FORCEINLINE bool HasAnyChannelEnabled() const
	{
		return AvailableChannelCount > 0;
	}

	FORCEINLINE const TArray<FLightSceneInfo*, TInlineAllocator<8>>& GetLights(int32 ChannelIndex) const
	{
		check(ChannelIndex >= 0 && ChannelIndex < CHANNEL_COUNT);
		return Channels[ChannelIndex].Lights;
	}

	void DisableChannel(int32 ChannelIndex);
	void DisableAllOtherChannels(int32 EnabledChannelIndex);

	// Update channel usage based on the current enabled channels. Channels with light of MaxPriority or more will be disabled.
	void UpdateAvailableChannels(const FSceneLightInfoArray& Lights, FLightSceneInfo* LightInfo);

	int32 GetBestAvailableChannel() const;

	void SortLightByPriority(int32 ChannelIndex);

private:

	struct FChannelInfo
	{
		FChannelInfo() : bIsAvailable(true) {}
		// Lights currently bound at that channel.
		TArray<FLightSceneInfo*, TInlineAllocator<8>> Lights;
		// Whether this channel is available for analysis.
		bool bIsAvailable;
	};

	FChannelInfo Channels[CHANNEL_COUNT];
	int32 AvailableChannelCount;

	FORCEINLINE void AddLight(int32 ChannelIndex, FLightSceneInfo* Light)
	{
		check(IsChannelEnabled(ChannelIndex) && Light);
		Channels[ChannelIndex].Lights.Add(Light);
	}

	/**
 	 * Returns the priority value used when assigning DynamicShadowMapChannel. 
	 * High priority lights punt low priority light in the assignement logic.
	 * @return The assignment priority, or INDEX_NONE if this light does not cast dynamic (or preview) shadows.
	 */
	static int32 GetPriority(const FLightSceneInfo* Light);
};
