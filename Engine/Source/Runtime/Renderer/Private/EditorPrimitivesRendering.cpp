// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorPrimitivesRendering
=============================================================================*/

#include "EditorPrimitivesRendering.h"
#include "BasePassRendering.h"
#include "Engine/EngineTypes.h"
#include "GBufferInfo.h"
#include "GPUProfiler.h"
#include "HAL/Platform.h"
#include "LightMapRendering.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Materials/MaterialRenderProxy.h"
#include "MeshBatch.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "MobileBasePassRendering.h"
#include "PrimitiveSceneProxy.h"
#include "RHI.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "Shader.h"
#include "VertexFactory.h"
#include "TranslucentPassResource.h"
#include "ScenePrivate.h"


DEFINE_GPU_STAT(EditorPrimitives);

FEditorPrimitivesBasePassMeshProcessor::FEditorPrimitivesBasePassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InDrawRenderState, bool bInTranslucentBasePass, FMeshPassDrawListContext* InDrawListContext) 
	: FMeshPassProcessor(EMeshPass::Num, Scene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InDrawRenderState)
	, bTranslucentBasePass(bInTranslucentBasePass)
{}

void FEditorPrimitivesBasePassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (MeshBatch.bUseForMaterial)
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material && Material->GetRenderingThreadShaderMap())
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

bool FEditorPrimitivesBasePassMeshProcessor::TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material)
{
	const bool bIsTranslucent = IsTranslucentBlendMode(Material);

	bool bResult = true;
	if (bIsTranslucent == bTranslucentBasePass
		&& (!PrimitiveSceneProxy || PrimitiveSceneProxy->ShouldRenderInMainPass())
		&& ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain()))
	{
		if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Mobile)
		{
			bResult = ProcessMobileShadingPath(MeshBatch, BatchElementMask, Material, MaterialRenderProxy, PrimitiveSceneProxy, StaticMeshId);
		}
		else
		{
			bResult = ProcessDeferredShadingPath(MeshBatch, BatchElementMask, Material, MaterialRenderProxy, PrimitiveSceneProxy, StaticMeshId);
		}
	}
	return bResult;
}

bool FEditorPrimitivesBasePassMeshProcessor::ProcessDeferredShadingPath(const FMeshBatch& MeshBatch, uint64 BatchElementMask, const FMaterial& Material, const FMaterialRenderProxy& MaterialRenderProxy, const FPrimitiveSceneProxy* PrimitiveSceneProxy, int32 StaticMeshId)
{
	FUniformLightMapPolicy NoLightmapPolicy(LMP_NO_LIGHTMAP);
	typedef FUniformLightMapPolicy LightMapPolicyType;

	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		TBasePassVertexShaderPolicyParamType<LightMapPolicyType>,
		TBasePassPixelShaderPolicyParamType<LightMapPolicyType>> BasePassShaders;

	if (!GetBasePassShaders<LightMapPolicyType>(
		Material,
		VertexFactory->GetType(),
		NoLightmapPolicy,
		FeatureLevel,
		false, // 128bit
		false, // bIsDebug
		GBL_Default,
		nullptr, // Mesh Shader
		&BasePassShaders.VertexShader,
		&BasePassShaders.PixelShader
		))
	{
		return false;
	}

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);

	if (bTranslucentBasePass)
	{
		extern void SetTranslucentRenderState(FMeshPassProcessorRenderState& DrawRenderState, const FMaterial& Material, const EShaderPlatform Platform, ETranslucencyPass::Type InTranslucencyPassType);
		SetTranslucentRenderState(DrawRenderState, Material, Scene->GetShaderPlatform(), ETranslucencyPass::TPT_TranslucencyStandard);
	}

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
	
	TBasePassShaderElementData<LightMapPolicyType> ShaderElementData(nullptr);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(BasePassShaders.VertexShader, BasePassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		Material,
		DrawRenderState,
		BasePassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

bool FEditorPrimitivesBasePassMeshProcessor::ProcessMobileShadingPath(const FMeshBatch& MeshBatch, uint64 BatchElementMask, const FMaterial& Material, const FMaterialRenderProxy& MaterialRenderProxy, const FPrimitiveSceneProxy* PrimitiveSceneProxy, int32 StaticMeshId)
{
	FUniformLightMapPolicy NoLightmapPolicy(LMP_NO_LIGHTMAP);
	typedef FUniformLightMapPolicy LightMapPolicyType;

	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;
	const EMobileLocalLightSetting LocalLightSetting = EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED;

	TMeshProcessorShaders<
		TMobileBasePassVSPolicyParamType<LightMapPolicyType>,
		TMobileBasePassPSPolicyParamType<LightMapPolicyType>> BasePassShaders;
	
	if (!MobileBasePass::GetShaders(
		NoLightmapPolicy,
		LocalLightSetting,
		Material,
		VertexFactory->GetType(),
		BasePassShaders.VertexShader,
		BasePassShaders.PixelShader))
	{
		return false;
	}
	
	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);

	if (bTranslucentBasePass)
	{
		MobileBasePass::SetTranslucentRenderState(DrawRenderState, Material, Material.GetShadingModels());
	}

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
	
	TMobileBasePassShaderElementData<LightMapPolicyType> ShaderElementData(nullptr, false);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(BasePassShaders.VertexShader, BasePassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		Material,
		DrawRenderState,
		BasePassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}