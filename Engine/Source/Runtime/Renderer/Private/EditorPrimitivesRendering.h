// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	EditorPrimitivesRendering
=============================================================================*/

#pragma once

#include "MeshPassProcessor.h"

class FEditorPrimitivesBasePassMeshProcessor : public FMeshPassProcessor
{
public:
	FEditorPrimitivesBasePassMeshProcessor(const FScene* InScene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InDrawRenderState, bool bTranslucentBasePass, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;

	const FMeshPassProcessorRenderState PassDrawRenderState;
	const bool bTranslucentBasePass;

private:
	bool TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material);

	bool ProcessDeferredShadingPath(const FMeshBatch& MeshBatch, uint64 BatchElementMask, const FMaterial& Material, const FMaterialRenderProxy& MaterialRenderProxy, const FPrimitiveSceneProxy* PrimitiveSceneProxy, int32 StaticMeshId);
	bool ProcessMobileShadingPath(const FMeshBatch& MeshBatch, uint64 BatchElementMask, const FMaterial& Material, const FMaterialRenderProxy& MaterialRenderProxy, const FPrimitiveSceneProxy* PrimitiveSceneProxy, int32 StaticMeshId);
};

DECLARE_GPU_STAT_NAMED_EXTERN(EditorPrimitives, TEXT("Editor Primitives"));
