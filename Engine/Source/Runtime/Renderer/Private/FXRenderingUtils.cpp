// Copyright Epic Games, Inc. All Rights Reserved.

#include "FXRenderingUtils.h"
#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "Containers/StridedView.h"
#include "DistanceFieldLightingShared.h"
#include "ExtensibleUniformBuffer.h"
#include "FrontLayerTranslucency.h"
#include "GlobalDistanceFieldParameters.h"
#include "GPUScene.h"
#include "Logging/LogMacros.h"
#include "Lumen/LumenScreenProbeGather.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Misc/LargeWorldCoordinates.h"
#include "PrimitiveSceneProxy.h"
#include "RayTracing/RaytracingOptions.h"
#include "RenderGraphBuilder.h"
#include "RenderResource.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIResourceUtils.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextures.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "Stats/Stats.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"

#if RHI_RAYTRACING
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracing/RayTracingShaderBindingTable.h"
#include "RayTracingMeshDrawCommands.h"
#include "RayTracingDefinitions.h"
#endif

TConstStridedView<FSceneView> UE::FXRenderingUtils::ConvertViewArray(TConstArrayView<FViewInfo> Views)
{
	return MakeStridedViewOfBase<const FSceneView>(Views);
}

FIntRect UE::FXRenderingUtils::GetRawViewRectUnsafe(const FSceneView& View)
{
	check(View.bIsViewInfo);
	return static_cast<const FViewInfo&>(View).ViewRect;
}

bool UE::FXRenderingUtils::CanMaterialRenderBeforeFXPostOpaque(
	const FSceneViewFamily& ViewFamily,
	const FPrimitiveSceneProxy& SceneProxy,
	const FMaterial& Material)
{
	// Opaque materials, none surface materials & translucent that write custom depth will always need to render before FFXSystemInterface::PostOpaqueRender
	if (!IsTranslucentBlendMode(Material) || Material.GetMaterialDomain() != MD_Surface || (SceneProxy.ShouldRenderCustomDepth() && Material.IsTranslucencyWritingCustomDepth()))
	{
		return true;
	}

	// When rendering Lumen, it's possible a translucent material might render in the LumenTranslucencyRadianceCacheMark pass,
	// which happens before PostRenderOpaque.
	const FScene* Scene = SceneProxy.GetScene().GetRenderScene();
	if (Scene 
		&& (CanMaterialRenderInLumenTranslucencyRadianceCacheMarkPass(*Scene, ViewFamily, SceneProxy, Material)
			|| CanMaterialRenderInFrontLayerTranslucencyGBufferPass(*Scene, ViewFamily, SceneProxy, Material)))
	{
		return true;
	}
		
	return false;
}

const FGlobalDistanceFieldParameterData* UE::FXRenderingUtils::GetGlobalDistanceFieldParameterData(TConstStridedView<FSceneView> Views)
{
	return Views.Num() > 0 ? &static_cast<const FViewInfo&>(Views[0]).GlobalDistanceFieldInfo.ParameterData : nullptr;
}

FRDGTextureRef UE::FXRenderingUtils::GetSceneVelocityTexture(const FSceneView& View)
{
	const FViewFamilyInfo* ViewFamily = static_cast<const FViewFamilyInfo*>(View.Family);
	const FSceneTextures* SceneTextures = ViewFamily ? ViewFamily->GetSceneTexturesChecked() : nullptr;
	return SceneTextures ? SceneTextures->Velocity : nullptr;
}

TRDGUniformBufferRef<FSceneTextureUniformParameters> UE::FXRenderingUtils::GetOrCreateSceneTextureUniformBuffer(
	FRDGBuilder& GraphBuilder,
	TConstStridedView<FSceneView> Views,
	ERHIFeatureLevel::Type FeatureLevel,
	ESceneTextureSetupMode SetupMode)
{
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformParams = nullptr;

	const FViewInfo* View = Views.Num() > 0 ? static_cast<const FViewInfo*>(&Views[0]) : nullptr;
	if (View)
	{
		const FViewFamilyInfo& ViewFamily = *static_cast<const FViewFamilyInfo*>(View->Family);

		if (!HasRayTracedOverlay(ViewFamily))
		{
			if (const FSceneTextures* SceneTextures = ViewFamily.GetSceneTexturesChecked())
			{
				SceneTexturesUniformParams = SceneTextures->UniformBuffer;
			}
		}
	}

	if (SceneTexturesUniformParams == nullptr)
	{
		SceneTexturesUniformParams = CreateSceneTextureUniformBuffer(GraphBuilder, nullptr, FeatureLevel, SetupMode);
	}

	return SceneTexturesUniformParams;
}

TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> UE::FXRenderingUtils::GetOrCreateMobileSceneTextureUniformBuffer(
	FRDGBuilder& GraphBuilder,
	TConstStridedView<FSceneView> Views,
	EMobileSceneTextureSetupMode SetupMode)
{
	TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> MobileSceneTexturesUniformParams = nullptr;

	const FViewInfo* View = Views.Num() > 0 ? static_cast<const FViewInfo*>(&Views[0]) : nullptr;
	if (const FSceneTextures* SceneTextures = View ? static_cast<const FViewFamilyInfo*>(View->Family)->GetSceneTexturesChecked() : nullptr)
	{
		MobileSceneTexturesUniformParams = SceneTextures->MobileUniformBuffer;
	}

	if (MobileSceneTexturesUniformParams == nullptr)
	{
		MobileSceneTexturesUniformParams = CreateMobileSceneTextureUniformBuffer(GraphBuilder, nullptr, SetupMode);
	}

	return MobileSceneTexturesUniformParams;
}

TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> UE::FXRenderingUtils::GetSubstratePublicGlobalUniformBuffer(
	FRDGBuilder& GraphBuilder,
	TConstStridedView<FSceneView> Views)
{
	const FSceneTextures* SceneTextures = Views.Num() > 0 ? static_cast<const FViewInfo*>(&Views[0])->GetSceneTexturesChecked() : nullptr;
	return ::Substrate::CreatePublicGlobalUniformBuffer(GraphBuilder, SceneTextures ? SceneTextures->SubstrateSceneData.Get() : nullptr);
}

const FShaderParametersMetadata* UE::FXRenderingUtils::DistanceFields::GetObjectBufferParametersMetadata()
{
	return TShaderParameterStructTypeInfo<FDistanceFieldObjectBufferParameters>::GetStructMetadata();
}

const FShaderParametersMetadata* UE::FXRenderingUtils::DistanceFields::GetAtlasParametersMetadata()
{
	return TShaderParameterStructTypeInfo<FDistanceFieldAtlasParameters>::GetStructMetadata();
}

inline const FDistanceFieldSceneData* GetDistanceFieldSceneData(const FSceneView& View)
{
	FSceneInterface* SceneInterface = View.Family && View.Family->Scene ? View.Family->Scene : nullptr;
	const FScene* Scene = SceneInterface ? SceneInterface->GetRenderScene() : nullptr;
	return  Scene ? &Scene->DistanceFieldSceneData : nullptr;
}

bool UE::FXRenderingUtils::DistanceFields::HasDataToBind(const FSceneView& View)
{
	return GetDistanceFieldSceneData(View) != nullptr;
}

void UE::FXRenderingUtils::DistanceFields::SetupObjectBufferParameters(FRDGBuilder& GraphBuilder, uint8* DestinationData, const FSceneView* View)
{
	if (FDistanceFieldObjectBufferParameters* ObjectBufferParameters = reinterpret_cast<FDistanceFieldObjectBufferParameters*>(DestinationData))
	{
		const FDistanceFieldSceneData* DistanceFieldSceneData = View ? GetDistanceFieldSceneData(*View) : nullptr;

		if (DistanceFieldSceneData && DistanceFieldSceneData->NumObjectsInBuffer > 0)
		{
			*ObjectBufferParameters = DistanceField::SetupObjectBufferParameters(GraphBuilder, *DistanceFieldSceneData);
		}
		else
		{
			FRDGBufferSRVRef DefaultVector4 = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f))));

			ObjectBufferParameters->SceneObjectBounds = DefaultVector4;
			ObjectBufferParameters->SceneObjectData = DefaultVector4;
			ObjectBufferParameters->NumSceneObjects = 0;
			ObjectBufferParameters->SceneHeightfieldObjectBounds = DefaultVector4;
			ObjectBufferParameters->SceneHeightfieldObjectData = DefaultVector4;
			ObjectBufferParameters->NumSceneHeightfieldObjects = 0;
		}
	}
}

class FDFDummyByteAddress : public FRenderResource
{
public:
	TRefCountPtr<FRDGPooledBuffer> PooledBuffer;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1);
		BufferDesc.Usage = BUF_Static | BUF_ShaderResource | BUF_ByteAddressBuffer;

		const FRHIBufferCreateDesc BufferCreateDesc =
			FRHIBufferCreateDesc::CreateStructured(TEXT("FDFDummyByteAddress"), BufferDesc.GetSize(), BufferDesc.BytesPerElement)
			.AddUsage(BufferDesc.Usage)
			.DetermineInitialState();

		FBufferRHIRef RHIBuffer = UE::RHIResourceUtils::CreateBufferZeroed(RHICmdList, BufferCreateDesc);

		PooledBuffer = new FRDGPooledBuffer(RHICmdList, RHIBuffer, BufferDesc, BufferDesc.NumElements, BufferCreateDesc.DebugName);
	}

	virtual void ReleaseRHI() override
	{
		PooledBuffer.SafeRelease();
	}
};

static TGlobalResource<FDFDummyByteAddress> GDFDummyByteAddress;

void UE::FXRenderingUtils::DistanceFields::SetupAtlasParameters(FRDGBuilder& GraphBuilder, uint8* DestinationData, const FSceneView* View)
{
	if (FDistanceFieldAtlasParameters* AtlasParameters = reinterpret_cast<FDistanceFieldAtlasParameters*>(DestinationData))
	{
		const FDistanceFieldSceneData* DistanceFieldSceneData = View ? GetDistanceFieldSceneData(*View) : nullptr;

		if (DistanceFieldSceneData && DistanceFieldSceneData->NumObjectsInBuffer > 0)
		{
			*AtlasParameters = DistanceField::SetupAtlasParameters(GraphBuilder, *DistanceFieldSceneData);
		}
		else
		{
			FRDGBufferSRVRef DefaultVector4 = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f))));
			FRDGBufferSRVRef DefaultUInt32 = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(GraphBuilder.RegisterExternalBuffer(GDFDummyByteAddress.PooledBuffer, ERDGBufferFlags::SkipTracking)));

			const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

			AtlasParameters->SceneDistanceFieldAssetData = DefaultVector4;
			AtlasParameters->DistanceFieldIndirectionTable = DefaultUInt32;
			AtlasParameters->DistanceFieldIndirection2Table = DefaultVector4;
			AtlasParameters->DistanceFieldIndirectionAtlas = SystemTextures.VolumetricBlack;
			AtlasParameters->DistanceFieldBrickTexture = SystemTextures.VolumetricBlack;
			AtlasParameters->DistanceFieldSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
			AtlasParameters->DistanceFieldBrickSize = FVector3f::ZeroVector;
			AtlasParameters->DistanceFieldUniqueDataBrickSize = FVector3f::ZeroVector;
			AtlasParameters->DistanceFieldBrickAtlasSizeInBricks = FIntVector::ZeroValue;
			AtlasParameters->DistanceFieldBrickAtlasMask = FIntVector::ZeroValue;
			AtlasParameters->DistanceFieldBrickAtlasSizeLog2 = FIntVector::ZeroValue;
			AtlasParameters->DistanceFieldBrickAtlasTexelSize = FVector3f::ZeroVector;
			AtlasParameters->DistanceFieldBrickAtlasHalfTexelSize = FVector3f::ZeroVector;
			AtlasParameters->DistanceFieldBrickOffsetToAtlasUVScale = FVector3f::ZeroVector;
			AtlasParameters->DistanceFieldUniqueDataBrickSizeInAtlasTexels = FVector3f::ZeroVector;
		}
	}
}

FSceneUniformBuffer& UE::FXRenderingUtils::CreateSceneUniformBuffer(FRDGBuilder& GraphBuilder, const FSceneInterface* InScene)
{
	FSceneUniformBuffer *Result = GraphBuilder.AllocObject<FSceneUniformBuffer>();
	if (const FScene* Scene = InScene->GetRenderScene())
	{
		Scene->GPUScene.FillSceneUniformBuffer(GraphBuilder, *Result);
	}
	return *Result;
}

TRDGUniformBufferRef<FSceneUniformParameters> UE::FXRenderingUtils::GetSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer &SceneUniformBuffer)
{
	return SceneUniformBuffer.GetBuffer(GraphBuilder);
}

#if RHI_RAYTRACING

bool UE::FXRenderingUtils::RayTracing::HasRayTracingScene(const FSceneInterface& InScene)
{
	if (const FScene* Scene = InScene.GetRenderScene())
	{
		return Scene->RayTracingScene.IsCreated();
	}

	return false;
}

FRHIRayTracingScene* UE::FXRenderingUtils::RayTracing::GetRayTracingScene(const FSceneInterface& InScene, const FSceneView& InView)
{
	if (const FScene* Scene = InScene.GetRenderScene())
	{
		check(InView.bIsViewInfo);
		const FViewInfo& View = static_cast<const FViewInfo&>(InView);
		return View.GetRayTracingSceneChecked(ERayTracingSceneLayer::Base);
	}

	return nullptr;
}

FRHIShaderResourceView* UE::FXRenderingUtils::RayTracing::GetInlineRayTracingBindingDataBuffer(const FSceneInterface& InScene)
{
	FRHIShaderResourceView *InlineRayTracingBindingDataBufferSRV = nullptr;
	if (const FScene* Scene = InScene.GetRenderScene())
	{
		InlineRayTracingBindingDataBufferSRV = Scene->RayTracingSBT.GetPersistentInlineBindingDataBufferSRV(Scene->RayTracingPersistentSBTState.InlineSBT.ID);

	}
	return InlineRayTracingBindingDataBufferSRV;
}

FRHIShaderResourceView* UE::FXRenderingUtils::RayTracing::GetRayTracingSceneView(FRHICommandListBase& RHICmdList, const FSceneInterface& InScene, const FSceneView& InView)
{
	if (const FScene* Scene = InScene.GetRenderScene())
	{
		check(InView.bIsViewInfo);
		const FViewInfo& View = static_cast<const FViewInfo&>(InView);
		return Scene->RayTracingScene.CreateLayerViewRHI(RHICmdList, ERayTracingSceneLayer::Base, View.GetRayTracingSceneViewHandle());
	}

	return nullptr;
}

FRDGBufferSRVRef UE::FXRenderingUtils::RayTracing::GetRayTracingSceneViewRDG(const FSceneInterface& InScene, const FSceneView& InView)
{
	if (const FScene* Scene = InScene.GetRenderScene())
	{
		const FViewInfo& View = static_cast<const FViewInfo&>(InView);
		return Scene->RayTracingScene.GetLayerView(ERayTracingSceneLayer::Base, View.GetRayTracingSceneViewHandle());
	}

	return nullptr;
}

FShaderBindingTableRHIRef UE::FXRenderingUtils::RayTracing::CreateShaderBindingTable(FRHICommandListBase& RHICmdList, const FSceneInterface* InScene, uint32 LocalBindingDataSize)
{
	if (const FScene* Scene = InScene->GetRenderScene())
	{
		return Scene->RayTracingSBT.AllocateTransientRHI(RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Allow, LocalBindingDataSize);
	}

	return nullptr;
}

TConstArrayView<FRayTracingShaderBindingData> UE::FXRenderingUtils::RayTracing::GetVisibleRayTracingShaderBindings(const FSceneView& View)
{
	return static_cast<const FViewInfo&>(View).VisibleRayTracingShaderBindings;
}

const FRayTracingMeshCommandStorage* UE::FXRenderingUtils::RayTracing::GetRayTracingMeshCommands(const FSceneInterface* InScene)
{
	if (const FScene* Scene = InScene->GetRenderScene())
	{
		return &Scene->CachedRayTracingMeshCommands;
	}

	return nullptr;
}

#endif // RHI_RAYTRACING
