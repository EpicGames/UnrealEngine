// Copyright Epic Games, Inc. All Rights Reserved.

#include "FirstPersonSceneExtension.h"
#include "ScenePrivate.h"
#include "RenderUtils.h"
#include "ViewData.h"

IMPLEMENT_SCENE_EXTENSION(FFirstPersonSceneExtension);

bool FFirstPersonSceneExtension::ShouldCreateExtension(FScene& Scene)
{
	// For now, the bounds computed by this extension are only needed for First Person Self-Shadow and Lumen HWRT reflections of
	// FirstPersonWorldSpaceRepresentation primitives. Both these features require the first person gbuffer bit.
	return HasFirstPersonGBufferBit(GetFeatureLevelShaderPlatform(Scene.GetFeatureLevel()));
}

ISceneExtensionUpdater* FFirstPersonSceneExtension::CreateUpdater()
{
	return new FUpdater(*this);
}

ISceneExtensionRenderer* FFirstPersonSceneExtension::CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags)
{
	return new FFirstPersonSceneExtensionRenderer(InSceneRenderer, *this);
}

const TArray<FPrimitiveSceneInfo*>& FFirstPersonSceneExtension::GetFirstPersonPrimitives() const
{
	return FirstPersonPrimitives;
}

const TArray<FPrimitiveSceneInfo*>& FFirstPersonSceneExtension::GetWorldSpaceRepresentationPrimitives() const
{
	return WorldSpaceRepresentationPrimitives;
}

void FFirstPersonSceneExtension::FUpdater::PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
{
	// Iterate over removed primitives and remove them from the respective first person primitive lists.
	for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.RemovedPrimitiveSceneInfos)
	{
		if (PrimitiveSceneInfo->Proxy->IsFirstPerson())
		{
			SceneExtension.FirstPersonPrimitives.RemoveSwap(PrimitiveSceneInfo);
		}
		else if (PrimitiveSceneInfo->Proxy->IsFirstPersonWorldSpaceRepresentation())
		{
			SceneExtension.WorldSpaceRepresentationPrimitives.RemoveSwap(PrimitiveSceneInfo);
		}
	}
}

void FFirstPersonSceneExtension::FUpdater::PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
{
	// Iterate over added primitives and add them to the respective first person primitive lists.
	for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.AddedPrimitiveSceneInfos)
	{
		const FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;
		if (Proxy->IsFirstPerson())
		{
			SceneExtension.FirstPersonPrimitives.Add(PrimitiveSceneInfo);
		}
		else if (Proxy->IsFirstPersonWorldSpaceRepresentation())
		{
			SceneExtension.WorldSpaceRepresentationPrimitives.Add(PrimitiveSceneInfo);
		}
	}
}

void FFirstPersonSceneExtensionRenderer::UpdateViewData(FRDGBuilder& GraphBuilder, const FRendererViewDataManager& ViewDataManager)
{
	// Union of two bounds, taking into account that the ExistingBounds might be zero initialized.
	auto SafeBoundsUnion = [](const FBoxSphereBounds& ExistingBounds, const FBoxSphereBounds& NewBounds)
	{
		const bool bValidExistingBounds = ExistingBounds.SphereRadius != 0.0f;
		return bValidExistingBounds ? ExistingBounds + NewBounds : NewBounds;
	};

	// Ideally we'd compute this per view, but PrimitiveVisibilityMap will be false for these primitives, so we might as well compute
	// a single set of bounds for all the primitives in the scene now and assign it to each view. In practice, all of these primitives
	// are very close together and will mostly be in the frustum anyways, so it doesn't make much of a difference
	// and will most importantly still be conservative.
	const bool bHasWorldSpaceRepresentationPrimitives = !SceneExtension.GetWorldSpaceRepresentationPrimitives().IsEmpty();
	FBoxSphereBounds WorldSpaceRepresentationBounds = FBoxSphereBounds(ForceInit);
	for (const FPrimitiveSceneInfo* PrimitiveSceneInfo : SceneExtension.GetWorldSpaceRepresentationPrimitives())
	{
		const FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;
		const FBoxSphereBounds Bounds = Proxy->GetBounds();
		WorldSpaceRepresentationBounds = SafeBoundsUnion(WorldSpaceRepresentationBounds, Bounds);
	}

	TConstArrayView<FViewInfo*> Views = ViewDataManager.GetRegisteredPrimaryViews(); // This call is valid even if !ViewDataManager.IsEnabled()
	const int32 NumViews = Views.Num();
	ViewBoundsArray.Reset(NumViews);

	// Init bounds info on the views.
	for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
	{
		check(ViewIndex == Views[ViewIndex]->SceneRendererPrimaryViewId); // We rely on SceneRendererPrimaryViewId being an index into RegisteredPrimaryViews on the ViewDataManager
		FFirstPersonViewBounds& ViewBounds = ViewBoundsArray.AddDefaulted_GetRef();
		ViewBounds.bHasFirstPersonPrimitives = false;
		ViewBounds.FirstPersonBounds = FBoxSphereBounds(ForceInit);

		ViewBounds.bHasFirstPersonWorldSpaceRepresentationPrimitives = bHasWorldSpaceRepresentationPrimitives;
		ViewBounds.WorldSpaceRepresentationBounds = WorldSpaceRepresentationBounds;
	}

	// Iterate over all first person primitives, get the bounds and then compute conservative bounds for each view.
	for (const FPrimitiveSceneInfo* PrimitiveSceneInfo : SceneExtension.GetFirstPersonPrimitives())
	{
		const FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;
		const FBoxSphereBounds Bounds = Proxy->GetBounds();
		const int32 PrimitiveIndex = PrimitiveSceneInfo->GetIndex();
		
		for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
		{
			const FViewInfo& View = *Views[ViewIndex];
			if (View.PrimitiveVisibilityMap[PrimitiveIndex])
			{
				// Materials can lerp between first person and world space, so we compute the union of both bounds.
				const FBoxSphereBounds ConservativeBounds = Bounds.TransformBy(View.ViewMatrices.GetWorldToFirstPerson()) + Bounds;
				
				FFirstPersonViewBounds& ViewBounds = ViewBoundsArray[ViewIndex];
				ViewBounds.FirstPersonBounds = SafeBoundsUnion(ViewBounds.FirstPersonBounds, ConservativeBounds);
				ViewBounds.bHasFirstPersonPrimitives = true;
			}
		}
	}
}

FFirstPersonViewBounds FFirstPersonSceneExtensionRenderer::GetFirstPersonViewBounds(const FViewInfo& ViewInfo) const
{
	if (ViewBoundsArray.IsValidIndex(ViewInfo.SceneRendererPrimaryViewId))
	{
		return ViewBoundsArray[ViewInfo.SceneRendererPrimaryViewId];
	}
	return FFirstPersonViewBounds();
}
