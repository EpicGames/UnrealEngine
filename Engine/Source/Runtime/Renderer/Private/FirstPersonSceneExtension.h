// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneExtensions.h"

class FPrimitiveSceneInfo;
class FViewInfo;

// Holds collective bounds of all FirstPerson/WorldSpaceRepresentation primitives for a given view. This assumes that these primitives are all very close together in world space,
// allowing to do a single overlap/intersection test to cover all the primitives.
struct FFirstPersonViewBounds
{
	// Collective bounds for all FirstPerson primitives visible in this view.
	FBoxSphereBounds FirstPersonBounds = FBoxSphereBounds(ForceInit);

	// Collective bounds for all FirstPersonWorldSpaceRepresentation primitives affecting this view.
	FBoxSphereBounds WorldSpaceRepresentationBounds = FBoxSphereBounds(ForceInit);
	
	// Whether the scene has at least one FirstPerson primitive visible in this view.
	uint32 bHasFirstPersonPrimitives : 1 = false;

	// Whether the scene has at least one FirstPersonWorldSpaceRepresentation primitive affecting this view.
	uint32 bHasFirstPersonWorldSpaceRepresentationPrimitives : 1 = false;
};

// This scene extension maintains dense arrays of all FirstPerson and FirstPersonWorldSpaceRepresentation primitives in the scene.
// Every frame, it then computes collective per-view for all the first person relevant primitives affecting each view.
// Currently, these bounds are used for First Person Self-Shadow and Lumen HWRT reflections.
class FFirstPersonSceneExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FFirstPersonSceneExtension);
public:
	using ISceneExtension::ISceneExtension;

	static bool ShouldCreateExtension(FScene& Scene);

	//~ Begin ISceneExtension Interface.
	virtual ISceneExtensionUpdater* CreateUpdater() override;
	virtual ISceneExtensionRenderer* CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags) override;
	//~ End ISceneExtension Interface.

	const TArray<FPrimitiveSceneInfo*>& GetFirstPersonPrimitives() const;
	const TArray<FPrimitiveSceneInfo*>& GetWorldSpaceRepresentationPrimitives() const;

private:

	// Private updater class for handling adding and removal of primitives.
	class FUpdater : public ISceneExtensionUpdater
	{
		DECLARE_SCENE_EXTENSION_UPDATER(FUpdater, FFirstPersonSceneExtension);
	public:
		FUpdater(FFirstPersonSceneExtension& InSceneExtension) : SceneExtension(InSceneExtension) {}

		//~ Begin ISceneExtensionUpdater Interface.
		virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override;
		virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override;
		//~ End ISceneExtensionUpdater Interface.

	private:
		FFirstPersonSceneExtension& SceneExtension;
	};

	TArray<FPrimitiveSceneInfo*> FirstPersonPrimitives;
	TArray<FPrimitiveSceneInfo*> WorldSpaceRepresentationPrimitives;
};

// First person "renderer" class, calculating the per-view bounds when a frame is rendered.
class FFirstPersonSceneExtensionRenderer : public ISceneExtensionRenderer
{
	DECLARE_SCENE_EXTENSION_RENDERER(FRenderer, FFirstPersonSceneExtension);
public:
	FFirstPersonSceneExtensionRenderer(FSceneRendererBase& InSceneRenderer, FFirstPersonSceneExtension& InSceneExtension)
		: ISceneExtensionRenderer(InSceneRenderer), SceneExtension(InSceneExtension) {
	}

	//~ Begin ISceneExtensionRenderer Interface.
	virtual void UpdateViewData(FRDGBuilder& GraphBuilder, const FRendererViewDataManager& ViewDataManager) override;
	//~ End ISceneExtensionRenderer Interface.

	FFirstPersonViewBounds GetFirstPersonViewBounds(const FViewInfo& ViewInfo) const;

private:
	FFirstPersonSceneExtension& SceneExtension;
	TArray<FFirstPersonViewBounds> ViewBoundsArray;
};
