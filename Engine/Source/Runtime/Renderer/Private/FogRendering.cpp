// Copyright Epic Games, Inc. All Rights Reserved.

#include "FogRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DeferredShadingRenderer.h"
#include "LightSceneProxy.h"
#include "ScenePrivate.h"
#include "Engine/TextureCube.h"
#include "PipelineStateCache.h"
#include "SingleLayerWaterRendering.h"
#include "SceneCore.h"
#include "ScreenPass.h"
#include "TextureResource.h"
#include "PostProcess/PostProcessing.h" // IsPostProcessingWithAlphaChannelSupported
#include "EnvironmentComponentsFlags.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "FogSeparateComposition.h"
#include "PostProcess/TemporalAA.h"
#include "SceneViewState.h"


DECLARE_GPU_DRAWCALL_STAT(Fog);

static TAutoConsoleVariable<int32> CVarFog(
	TEXT("r.Fog"),
	1,
	TEXT(" 0: disabled\n")
	TEXT(" 1: enabled (default)"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<bool> CVarFogUseDepthBounds(
	TEXT("r.FogUseDepthBounds"),
	true,
	TEXT("Allows enable depth bounds optimization on fog full screen pass.\n")
	TEXT(" false: disabled\n")
	TEXT(" true: enabled (default)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarUpsampleJitterMultiplier(
	TEXT("r.VolumetricFog.UpsampleJitterMultiplier"),
		0.0f,
	TEXT("Multiplier for random offset value used to jitter the sample position of the 3D fog volume to hide fog pixelization due to sampling from a lower resolution texture."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<bool> CVarUnderwaterFogWhenCameraIsAboveWater(
	TEXT("r.Water.SingleLayer.UnderwaterFogWhenCameraIsAboveWater"), 
	false, 
	TEXT("Renders height fog behind the water surface even when the camera is above water. This avoids artifacts when entering and exiting the water with strong height fog in the scene but causes artifacts when looking at the water surface from a distance."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarOrthoFogHeightAdjustment(
	TEXT("r.Ortho.FogHeightAdjustment"),
	true,
	TEXT("When enabled, uses the Ortho camera height to determine the fog cutoff height"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobilePixelFogQuality(
	TEXT("r.Mobile.PixelFogQuality"),
	1,
	TEXT("Exponentional height fog rendering quality.\n")
	TEXT("0 - basic per-pixel fog")
	TEXT("1 - all per-pixel fog features (second fog, directional inscattering, aerial perspective)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarFogScreenSpaceScatteringTAA(
	TEXT("r.Fog.ScreenSpaceScattering.TAA"),
	1,
	TEXT("Enable temporal anti-aliasing filtering in order to reduce flickering of FSSS blurred scene color."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

bool PlatformUsesBasicFogFeatures(EShaderPlatform ShaderPlatform)
{
	// Disables AP, "SecondFog" and "DirectionalLight Inscattering"
	return IsMobilePlatform(ShaderPlatform) && CVarMobilePixelFogQuality.GetValueOnAnyThread() == 0;
}

static TAutoConsoleVariable<int> CVarVolumetricFogFilteringQuality(
	TEXT("r.VolumetricFog.Filtering.Quality"),
	0,
	TEXT(" 0: Trilinear\n")
	TEXT(" 1: Bicubic\n")
	TEXT(" 2: Tricubic"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FFogUniformParameters, "FogStruct");

static void SetupDummyExponentialHeightFogUniformParameters(FFogUniformParameters& OutParameters)
{
	FMemory::Memzero(OutParameters);
	OutParameters.InscatteringLightDirection.W = -1;
	OutParameters.DirectionalInscatteringColor.W = -1;
	OutParameters.FogInscatteringColorCubemap = GBlackTextureCube->TextureRHI;
	OutParameters.FogInscatteringColorSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
}

// Returns the sky light texture to use in when sky light is present, height fog is present and all that data is there ready.
static FTextureRHIRef IsSkyLightCaptureAffectsHeightFogTextureEnabled(const FViewInfo& View)
{
	FTextureRHIRef SkyLightTextureResource = nullptr;

	const FScene* Scene = View.Family->Scene ? View.Family->Scene->GetRenderScene() : nullptr;
	if (Scene && Scene->HasAnyExponentialHeightFog())
	{
		const FExponentialHeightFogSceneInfo& HFog = Scene->ExponentialFogs[0];

		const bool bApplySkyLight = View.Family->EngineShowFlags.SkyLighting;
		if (bApplySkyLight && Scene && Scene->SkyLight 
			&& SupportsSkyLightCaptureAffectsHeightFogInScattering(View.GetShaderPlatform())
			&& HFog.SkyLightCaptureAffectsHeightFogStrength > 0.0f )
		{
			if ((Scene->SkyLight->ProcessedTexture || Scene->CanSampleSkyLightRealTimeCaptureData())
				&& bApplySkyLight)
			{
				const FSkyLightSceneProxy& SkyLight = *Scene->SkyLight;

				if (Scene->CanSampleSkyLightRealTimeCaptureData())
				{
					// Cannot blend with this capture mode as of today.
					SkyLightTextureResource = Scene->ConvolvedSkyRenderTarget[Scene->ConvolvedSkyRenderTargetReadyIndex]->GetRHI();
				}
				else if (SkyLight.ProcessedTexture)
				{
					SkyLightTextureResource = SkyLight.ProcessedTexture->TextureRHI;
				}
			}
		}
	}
	return SkyLightTextureResource;
}

void SetupFogUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FFogUniformParameters& OutParameters, bool bForRealtimeSkyCapture)
{
	// Exponential Height Fog
	const FScene* Scene = View.Family->Scene ? View.Family->Scene->GetRenderScene() : nullptr;
	if (bForRealtimeSkyCapture && Scene && Scene->HasAnyExponentialHeightFog() && !Scene->ExponentialFogs[0].bVisibleInRealTimeSkyCaptures)
	{
		// If Height Fog is hidden in Real Time Sky capture then setup the FogData with dummy values.
		SetupDummyExponentialHeightFogUniformParameters(OutParameters);
	}
	else
	{
		OutParameters.SkyLightCaptureAffectsHeightFogStrength = 0.0f;
		OutParameters.SkyLightCaptureAffectsHeightFogRoughness = 0.0f;
		OutParameters.SkyLightCaptureAffectsHeightFogMipCount = 0.0f;

		FTextureRHIRef SkyLightTextureResource = IsSkyLightCaptureAffectsHeightFogTextureEnabled(View);
		if (Scene && SkyLightTextureResource != nullptr)
		{
			const FExponentialHeightFogSceneInfo& HFog = Scene->ExponentialFogs[0];

			OutParameters.SkyLightCaptureAffectsHeightFogStrength = HFog.SkyLightCaptureAffectsHeightFogStrength;
			OutParameters.SkyLightCaptureAffectsHeightFogRoughness = HFog.SkyLightCaptureAffectsHeightFogRoughness;

			const int32 CubemapWidth = SkyLightTextureResource->GetDesc().Extent.X;
			const float SkyMipCount = FMath::Log2(static_cast<float>(CubemapWidth)) + 1.0f;
			OutParameters.SkyLightCaptureAffectsHeightFogMipCount = SkyMipCount;
		}
		else if (View.FogInscatteringColorCubemap)
		{
			SkyLightTextureResource = View.FogInscatteringColorCubemap->GetResource()->GetTextureRHI();
		}
		else
		{
			SkyLightTextureResource = GWhiteTextureCube->GetTextureRHI();
		}

		OutParameters.ExponentialFogParameters = View.ExponentialFogParameters;
		OutParameters.ExponentialFogColorParameter = FVector4f(View.ExponentialFogColor, 1.0f - View.FogMaxOpacity);
		OutParameters.ExponentialFogParameters2 = View.ExponentialFogParameters2;
		OutParameters.ExponentialFogParameters3 = View.ExponentialFogParameters3;
		if (OutParameters.SkyLightCaptureAffectsHeightFogStrength > 0.0f)
		{
			OutParameters.ExponentialFogParameters3.Z = 0.0f;	// Disables in scattering texture when it is re-used for sky light in scattering
		}
		OutParameters.SkyAtmosphereAmbientContributionColorScale = View.SkyAtmosphereAmbientContributionColorScale;
		OutParameters.SinCosInscatteringColorCubemapRotation = View.SinCosInscatteringColorCubemapRotation;
		OutParameters.FogInscatteringTextureParameters = (FVector3f)View.FogInscatteringTextureParameters;
		OutParameters.InscatteringLightDirection = (FVector3f)View.InscatteringLightDirection;
		OutParameters.InscatteringLightDirection.W = View.bUseDirectionalInscattering ? FMath::Max(0.f, View.DirectionalInscatteringStartDistance) : -1.f;
		OutParameters.DirectionalInscatteringColor = FVector4f(FVector3f(View.DirectionalInscatteringColor), FMath::Clamp(View.DirectionalInscatteringExponent, 0.000001f, 1000.0f));
		OutParameters.FogInscatteringColorCubemap = SkyLightTextureResource;
		OutParameters.FogInscatteringColorSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		OutParameters.EndDistance = View.FogEndDistance;
	}

	// Volumetric Fog
	{
		if (View.VolumetricFogResources.IntegratedLightScatteringTexture)
		{
			OutParameters.IntegratedLightScattering = View.VolumetricFogResources.IntegratedLightScatteringTexture;
			OutParameters.ApplyVolumetricFog = 1.0f;
		}
		else
		{
			const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
			OutParameters.IntegratedLightScattering = SystemTextures.VolumetricBlackAlphaOne;
			OutParameters.ApplyVolumetricFog = 0.0f;
		}
		OutParameters.IntegratedLightScatteringSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		OutParameters.VolumetricFogStartDistance = View.VolumetricFogStartDistance;
		OutParameters.VolumetricFogNearFadeInDistanceInv = View.VolumetricFogNearFadeInDistanceInv;
		OutParameters.VolumetricFogPhaseG = View.VolumetricFogPhaseG;
		OutParameters.VolumetricFogAlbedo = View.VolumetricFogAlbedo;
	}

	OutParameters.EnableFSSS							= 0.0f;
	OutParameters.FSSSSceneColorScatteringAmountScale	= 0.0f;
	OutParameters.FSSSSceneColorScatteringAmountPower	= 1.0f;
	OutParameters.FSSSSpreadScale						= 0.0f;
	OutParameters.FSSSBlurControl						= 0.0f;
	if(Scene && Scene->HasAnyExponentialHeightFog())
	{
		const FExponentialHeightFogSceneInfo& HFog = Scene->ExponentialFogs[0];
		OutParameters.EnableFSSS							= !HFog.bEnableFSSS ? 0.0f : 1.0f;
		OutParameters.FSSSSceneColorScatteringAmountScale	= !HFog.bEnableFSSS ? 0.0f : HFog.FSSSSceneColorScatteringAmountScale;
		OutParameters.FSSSSceneColorScatteringAmountPower	= !HFog.bEnableFSSS ? 1.0f : HFog.FSSSSceneColorScatteringAmountPower;
		OutParameters.FSSSSpreadScale						= !HFog.bEnableFSSS ? 0.0f : HFog.FSSSSpreadScale;
		OutParameters.FSSSBlurControl						= !HFog.bEnableFSSS ? 0.0f : HFog.FSSSBlurControl;
	}
}

TRDGUniformBufferRef<FFogUniformParameters> CreateFogUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	auto* FogStruct = GraphBuilder.AllocParameters<FFogUniformParameters>();
	SetupFogUniformParameters(GraphBuilder, View, *FogStruct);
	return GraphBuilder.CreateUniformBuffer(FogStruct);
}

/** A vertex shader for rendering height fog. */
class FHeightFogVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHeightFogVS);
	SHADER_USE_PARAMETER_STRUCT(FHeightFogVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHeightFogVS, "/Engine/Private/HeightFogVertexShader.usf", "Main", SF_Vertex);

class FSupportFogInScatteringTexture : SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_FOG_INSCATTERING_TEXTURE");
class FSupportFogDirectionalLightInScattering : SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_FOG_DIRECTIONAL_LIGHT_INSCATTERING");
class FSupportVolumetricFog : SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_VOLUMETRIC_FOG");
class FSupportLocalFogVolume : SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME");
class FSampleSkyAtmosphereOnOpaque : SHADER_PERMUTATION_BOOL("PERMUTATION_SAMPLE_SKYATMOSPHERE_ON_OPAQUE");
class FSampleFogOnClouds : SHADER_PERMUTATION_BOOL("PERMUTATION_SAMPLE_FOG_ON_CLOUDS");
class FVolumetricFogFilteringQuality : SHADER_PERMUTATION_INT("PERMUTATION_VOLUMETRIC_FOG_FILTERING_QUALITY", 3);

BEGIN_SHADER_PARAMETER_STRUCT(FExpFogCommonParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogUniformBuffer)
	SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, OcclusionTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, OcclusionSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WaterDepthTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, WaterDepthSampler)
	SHADER_PARAMETER(float, bOnlyOnRenderedOpaque)
	SHADER_PARAMETER(uint32, bUseWaterDepthTexture)
	SHADER_PARAMETER(uint32, bPropagateAlpha)
	SHADER_PARAMETER(float, UpsampleJitterMultiplier)
	SHADER_PARAMETER(FVector4f, WaterDepthTextureMinMaxUV)
	SHADER_PARAMETER(FVector4f, OcclusionTextureMinMaxUV)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SrcCloudDepthTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SrcCloudDepthSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SrcCloudViewTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SrcCloudViewSampler)
END_SHADER_PARAMETER_STRUCT()

/** A pixel shader for rendering exponential height fog. */
class FExponentialHeightFogPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FExponentialHeightFogPS);
	SHADER_USE_PARAMETER_STRUCT(FExponentialHeightFogPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSupportFogInScatteringTexture, FVolumetricFogFilteringQuality, FSupportFogDirectionalLightInScattering, FSupportVolumetricFog, FSupportLocalFogVolume, FSampleFogOnClouds>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FExpFogCommonParameters, FogCommon)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FExponentialHeightFogPS, "/Engine/Private/HeightFogPixelShader.usf", "ExponentialPixelMain", SF_Pixel);

/** A compute shader for rendering the volumetric layer containing fog, atmosphere and other participating media effects. Potentially at half resolution */
class FRenderSeparateCompositionTexturesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderSeparateCompositionTexturesCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderSeparateCompositionTexturesCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FSupportFogInScatteringTexture, FVolumetricFogFilteringQuality, FSupportFogDirectionalLightInScattering, FSupportVolumetricFog, FSupportLocalFogVolume, FSampleSkyAtmosphereOnOpaque>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FExpFogCommonParameters, FogCommon)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FogSeparateCompositionDepth)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, FogTexture0UAV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float2>, FogTexture1UAV)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, InputSceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSceneColorTextureSampler)
		SHADER_PARAMETER(FVector2f, FogSeparateCompositionViewResolution)
		SHADER_PARAMETER(FVector2f, FogSeparateCompositionViewRectMin)
		SHADER_PARAMETER(float, FogSeparateCompositionResolutionDivider)
	END_SHADER_PARAMETER_STRUCT()

	static const uint32 ThreadGroupSize = 8;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderSeparateCompositionTexturesCS, "/Engine/Private/HeightFogPixelShader.usf", "RenderSeparateCompositionTexturesCS", SF_Compute);

/** The fog vertex declaration resource type. */
class FFogVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	// Destructor
	virtual ~FFogVertexDeclaration() {}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0, VET_Float2, 0, sizeof(FVector2f)));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

/** Vertex declaration for the light function fullscreen 2D quad. */
TGlobalResource<FFogVertexDeclaration> GFogVertexDeclaration;

void FSceneRenderer::InitFogConstants()
{
	const bool bExpFogMatchesVolumetricFog = DoesProjectSupportExpFogMatchesVolumetricFog();

	for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];

		// set fog consts based on height fog components
		if(ShouldRenderFog(*View.Family))
		{
			if (Scene->ExponentialFogs.Num() > 0)
			{
				const FExponentialHeightFogSceneInfo& FogInfo = Scene->ExponentialFogs[0];
				float CollapsedFogParameter[FExponentialHeightFogSceneInfo::NumFogs];
				float MaxObserverHeightDifference = 65536.0f;
				float MaxObserverHeight = FLT_MAX;
				const bool bSimpleFog = PlatformUsesBasicFogFeatures(View.GetShaderPlatform());

				const bool bUsingOrthoHeightAdjustment = !View.IsPerspectiveProjection() && CVarOrthoFogHeightAdjustment.GetValueOnRenderThread();				
				if (bUsingOrthoHeightAdjustment)
				{
					FVector ViewForward = View.ViewMatrices.GetWorldToView().GetColumn(2);
					MaxObserverHeightDifference = FMath::Max(View.ViewMatrices.GetViewOrigin().Z, View.ViewMatrices.GetCameraToViewTarget().Length() * FMath::Abs(ViewForward.Z));
				}

				for (int i = 0; i < FExponentialHeightFogSceneInfo::NumFogs; i++)
				{
					// Only limit the observer height to fog if it has any density
					if (FogInfo.FogData[i].Density > 0.0f)
					{
						MaxObserverHeight = FMath::Min(MaxObserverHeight, FogInfo.FogData[i].Height + MaxObserverHeightDifference);
					}
				}
				
				// Clamping the observer height to avoid numerical precision issues in the height fog equation. The max observer height is relative to the fog height.
				const float ObserverHeight = bUsingOrthoHeightAdjustment ? MaxObserverHeight : FMath::Min<float>(View.ViewMatrices.GetViewOrigin().Z, MaxObserverHeight);

				for (int i = 0; i < FExponentialHeightFogSceneInfo::NumFogs; i++)
				{
					const float CollapsedFogParameterPower = FMath::Clamp(
						-FogInfo.FogData[i].HeightFalloff * (ObserverHeight - FogInfo.FogData[i].Height),
						-126.f + 1.f, // min and max exponent values for IEEE floating points (http://en.wikipedia.org/wiki/IEEE_floating_point)
						+127.f - 1.f
					);

					CollapsedFogParameter[i] = FogInfo.FogData[i].Density * FMath::Pow(2.0f, CollapsedFogParameterPower);
				}

				View.ExponentialFogParameters = FVector4f(CollapsedFogParameter[0], FogInfo.FogData[0].HeightFalloff, MaxObserverHeight, FogInfo.StartDistance);
				if (!bSimpleFog)
				{
					View.ExponentialFogParameters2 = FVector4f(CollapsedFogParameter[1], FogInfo.FogData[1].HeightFalloff, FogInfo.FogData[1].Density, FogInfo.FogData[1].Height);
				}
				View.ExponentialFogColor = FVector3f(FogInfo.FogColor.R, FogInfo.FogColor.G, FogInfo.FogColor.B);
				View.FogMaxOpacity = FogInfo.FogMaxOpacity;
				View.ExponentialFogParameters3 = FVector4f(FogInfo.FogData[0].Density, FogInfo.FogData[0].Height, FogInfo.InscatteringColorCubemap ? 1.0f : 0.0f, FogInfo.FogCutoffDistance);
				View.SinCosInscatteringColorCubemapRotation = FVector2f(FMath::Sin(FogInfo.InscatteringColorCubemapAngle), FMath::Cos(FogInfo.InscatteringColorCubemapAngle));
				View.FogInscatteringColorCubemap = FogInfo.InscatteringColorCubemap;
				const float InvRange = 1.0f / FMath::Max(FogInfo.FullyDirectionalInscatteringColorDistance - FogInfo.NonDirectionalInscatteringColorDistance, .00001f);
				float NumMips = 1.0f;

				View.FogEndDistance = FogInfo.EndDistance;
				View.SkyAtmosphereAmbientContributionColorScale = FogInfo.SkyAtmosphereAmbientContributionColorScale;

				if (FogInfo.InscatteringColorCubemap)
				{
					NumMips = FogInfo.InscatteringColorCubemap->GetNumMips();
				}

				View.FogInscatteringTextureParameters = FVector(InvRange, -FogInfo.NonDirectionalInscatteringColorDistance * InvRange, NumMips);

				View.DirectionalInscatteringExponent = FogInfo.DirectionalInscatteringExponent;
				View.DirectionalInscatteringStartDistance = FogInfo.DirectionalInscatteringStartDistance;
				View.InscatteringLightDirection = FVector(0);
				FLightSceneInfo* SunLight = Scene->AtmosphereLights[0] ? Scene->AtmosphereLights[0] : Scene->SimpleDirectionalLight;	// Fog only takes into account a single atmosphere light with index 0, or the default scene directional light.
				if (SunLight)
				{
					View.InscatteringLightDirection = -SunLight->Proxy->GetDirection();
					View.DirectionalInscatteringColor = FogInfo.DirectionalInscatteringColor * SunLight->Proxy->GetColor().GetLuminance();
				}
				View.bUseDirectionalInscattering = !bSimpleFog && SunLight != nullptr;
				View.bEnableVolumetricFog = FogInfo.bEnableVolumetricFog;
				View.VolumetricFogStartDistance = FogInfo.VolumetricFogStartDistance;
				View.VolumetricFogNearFadeInDistanceInv = FogInfo.VolumetricFogNearFadeInDistance > 0.0f ? (1.0f / FogInfo.VolumetricFogNearFadeInDistance) : 100000000.0f;
				View.VolumetricFogPhaseG = FogInfo.VolumetricFogScatteringDistribution;

				View.VolumetricFogAlbedo = FVector3f::Zero(); // unused by default
				if (bExpFogMatchesVolumetricFog)
				{
					// We make everything we can to get a good match between height fog and volumetric fog
					View.DirectionalInscatteringStartDistance = 0.0f;						// No start distance for ExpFog as for VFog
					View.DirectionalInscatteringExponent = 1.0f;							// Exponent is ununsed in this case
					View.ExponentialFogColor = FVector3f(FogInfo.VolumetricFogEmissive);	// Emisive from ExpFog matches the VFog ExponentialFogColorParameter
					View.VolumetricFogAlbedo = FVector3f(FogInfo.VolumetricFogAlbedo);		// Albedo is now supported by ExpFog and matches the VFog
					View.DirectionalInscatteringColor = FLinearColor::Black;				// Directional scattering is only impacted by the atmospheric light to match
				}
			}
		}
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FFogPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FHeightFogVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FExponentialHeightFogPS::FParameters, PS)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()


static void CreateDefaultCommonFogPassParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	TRDGUniformBufferRef<FFogUniformParameters>& FogUniformBuffer,
	FRDGTextureRef LightShaftOcclusionTexture,
	const FScreenPassTextureViewportParameters& LightShaftParameters,
	FExpFogCommonParameters& OutFogCommon)
{
	extern int32 GVolumetricFogGridPixelSize;

	OutFogCommon.ViewUniformBuffer = GetShaderBinding(View.ViewUniformBuffer);
	OutFogCommon.FogUniformBuffer = FogUniformBuffer;
	OutFogCommon.LFV = View.LocalFogVolumeViewData.UniformParametersStruct;
	OutFogCommon.OcclusionTexture = LightShaftOcclusionTexture != nullptr ? LightShaftOcclusionTexture : GSystemTextures.GetWhiteDummy(GraphBuilder);
	OutFogCommon.OcclusionSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	OutFogCommon.WaterDepthTexture = GSystemTextures.GetDepthDummy(GraphBuilder);
	OutFogCommon.WaterDepthSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	OutFogCommon.OcclusionTextureMinMaxUV = FVector4f(LightShaftParameters.UVViewportBilinearMin, LightShaftParameters.UVViewportBilinearMax);
	OutFogCommon.WaterDepthTextureMinMaxUV = FVector4f::Zero();
	OutFogCommon.UpsampleJitterMultiplier = CVarUpsampleJitterMultiplier.GetValueOnRenderThread() * GVolumetricFogGridPixelSize;
	OutFogCommon.bOnlyOnRenderedOpaque = View.bFogOnlyOnRenderedOpaque;
	OutFogCommon.bUseWaterDepthTexture = false;
	OutFogCommon.bPropagateAlpha = IsPostProcessingWithAlphaChannelSupported();

	OutFogCommon.SrcCloudDepthTexture = GSystemTextures.GetWhiteDummy(GraphBuilder);
	OutFogCommon.SrcCloudDepthSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	OutFogCommon.SrcCloudViewTexture = GSystemTextures.GetWhiteDummy(GraphBuilder);
	OutFogCommon.SrcCloudViewSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
}

static FFogPassParameters* CreateDefaultFogPassParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformbuffer,
	TRDGUniformBufferRef<FFogUniformParameters>& FogUniformBuffer,
	FRDGTextureRef LightShaftOcclusionTexture,
	const FScreenPassTextureViewportParameters& LightShaftParameters)
{
	FFogPassParameters* PassParameters = GraphBuilder.AllocParameters<FFogPassParameters>();
	PassParameters->SceneTextures = SceneTexturesUniformbuffer;
	PassParameters->VS.ViewUniformBuffer = GetShaderBinding(View.ViewUniformBuffer);

	CreateDefaultCommonFogPassParameters(
		GraphBuilder,
		View,
		FogUniformBuffer,
		LightShaftOcclusionTexture,
		LightShaftParameters,
		PassParameters->PS.FogCommon);

	return PassParameters;
}

static bool ShouldRenderFogForView(
	const FScene* Scene,
	const FViewInfo& View)
{
	const bool bViewIsReflectionCapture = View.bIsReflectionCapture;
	bool bFogRendersInReflectionCapture = true;
	if (Scene && Scene->HasAnyExponentialHeightFog())
	{
		bFogRendersInReflectionCapture = Scene->ExponentialFogs[0].bVisibleInReflectionCaptures;
	}
	return !bViewIsReflectionCapture || bFogRendersInReflectionCapture;
}

static void RenderViewFog(
	FRHICommandList& RHICmdList, 
	const FViewInfo& View, 
	FIntRect ViewRect, 
	FFogPassParameters* PassParameters, 
	bool bShouldRenderVolumetricFog,
	bool bFogComposeLocalFogVolumes,
	bool bSampleFogOnClouds = false,
	bool bEnableBlending = true)
{
	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	if (bEnableBlending)
	{
		const bool bSupportsAlpha = IsPostProcessingWithAlphaChannelSupported();
		if (bSupportsAlpha)
		{
			// Coverage is the alpha output of the shader in this case.
			if (IsExponentialFogHoldout(View.CachedViewUniformShaderParameters->EnvironmentComponentsFlags) && View.CachedViewUniformShaderParameters->bPrimitiveAlphaHoldoutEnabled)
			{
				// Alpha holdout: apply only when requested and when not rendering reflections. We want to punch a hole according to the Coverage. (black as throughput=0 should become brighter for see throught)
				// SceneAlpha = Coverage*1 + (1.0-Coverage)*(SceneThroughput)
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI();
			}
			else
			{
				// Same color blending. Alpha blending is kept so that the throughput of height fog can still be applied on the background (for instance when the sky or a mesh is holdout).
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
			}
		}
		else
		{
			// Disable alpha writes in order to preserve scene depth values on PC
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha>::GetRHI();
		}
	}
	else
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	}

	TShaderMapRef<FHeightFogVS> VertexShader(View.ShaderMap);
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFogVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

	const bool bUseFogInscatteringColorCubemap = View.FogInscatteringColorCubemap != nullptr && !IsSkyLightCaptureAffectsHeightFogTextureEnabled(View);
	FExponentialHeightFogPS::FPermutationDomain PsPermutationVector;
	PsPermutationVector.Set<FSupportFogInScatteringTexture>(bUseFogInscatteringColorCubemap);
	PsPermutationVector.Set<FSupportFogDirectionalLightInScattering>(!bUseFogInscatteringColorCubemap && View.bUseDirectionalInscattering);
	PsPermutationVector.Set<FSupportVolumetricFog>(bShouldRenderVolumetricFog);
	PsPermutationVector.Set<FSupportLocalFogVolume>(bFogComposeLocalFogVolumes);
	PsPermutationVector.Set<FSampleFogOnClouds>(bSampleFogOnClouds);

	//Ensure Trilinear for Reflection Capture to avoid needless cost
	int VolumetricFogFilteringQuality = View.CachedViewUniformShaderParameters->RenderingReflectionCaptureMask == 0.0f ? CVarVolumetricFogFilteringQuality.GetValueOnAnyThread() : 0;
	PsPermutationVector.Set<FVolumetricFogFilteringQuality>(FMath::Clamp(VolumetricFogFilteringQuality, 0, 2));

	TShaderMapRef<FExponentialHeightFogPS> PixelShader(View.ShaderMap, PsPermutationVector);
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

	// Setup the depth bound optimization if possible on that platform.
	GraphicsPSOInit.bDepthBounds = GSupportsDepthBoundsTest && CVarFogUseDepthBounds.GetValueOnAnyThread() && !bSampleFogOnClouds;
	if (GraphicsPSOInit.bDepthBounds)
	{
		float FogStartDistance = GetViewFogCommonStartDistance(View, bShouldRenderVolumetricFog, bFogComposeLocalFogVolumes);

		// Here we compute the nearest z value the fog can start
		// to skip shader execution on pixels that are closer.
		// This means with a bigger distance specified more pixels are
		// are culled and don't need to be rendered. This is faster if
		// there is opaque content nearer than the computed z.
		// This optimization is achieved using depth bound tests.
		// Mobile platforms typically does not support that feature 
		// but typically renders the world using forward shading 
		// with height fog evaluated as part of the material vertex or pixel shader.
		FMatrix InvProjectionMatrix = View.ViewMatrices.GetClipToView();
		FVector ViewSpaceCorner = InvProjectionMatrix.TransformFVector4(FVector4(1, 1, 1, 1));
		float Ratio = ViewSpaceCorner.Z / ViewSpaceCorner.Size();
		FVector ViewSpaceStartFogPoint(0.0f, 0.0f, FogStartDistance * Ratio);
		FVector4f ClipSpaceMaxDistance = (FVector4f)View.ViewMatrices.GetViewToClip().TransformPosition(ViewSpaceStartFogPoint); // LWC_TODO: precision loss
		float FogClipSpaceZ = ClipSpaceMaxDistance.Z / ClipSpaceMaxDistance.W;
		FogClipSpaceZ = FMath::Clamp(FogClipSpaceZ, 0.f, 1.f);
		RHICmdList.SetDepthBounds(0.0f, FogClipSpaceZ);
	}

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

	SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

	// Draw a quad covering the view.
	RHICmdList.SetStreamSource(0, GScreenSpaceVertexBuffer.VertexBufferRHI, 0);
	RHICmdList.DrawIndexedPrimitive(GTwoTrianglesIndexBuffer.IndexBufferRHI, 0, 0, 4, 0, 2, 1);
}

void RenderFogOnClouds(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	FRDGTextureRef SrcCloudDepth,
	FRDGTextureRef SrcCloudView,
	FRDGTextureRef DstCloudView,
	const bool bShouldRenderVolumetricFog,
	const bool bUseVolumetricRenderTarget)
{
	if (Scene->ExponentialFogs.Num() > 0 && ShouldRenderFogForView(Scene, View))
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, Fog, "ExponentialHeightFog on Clouds");
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		TRDGUniformBufferRef<FFogUniformParameters> FogUniformBuffer = CreateFogUniformBuffer(GraphBuilder, View);

		// Light shaft is not accounted for in this case
		const FScreenPassTextureViewportParameters LightShaftParameters;
		// Local fog volume are not accounted for in this case
		const bool bFogComposeLocalFogVolumes = false;

		FFogPassParameters* PassParameters = CreateDefaultFogPassParameters(
			GraphBuilder, View, 
			CreateSceneTextureUniformBuffer(GraphBuilder,View, ESceneTextureSetupMode::None),
			FogUniformBuffer, nullptr /*LightShaftOcclusionTexture*/, LightShaftParameters);

		// Patch the pass parameter for it to work on clouds
		PassParameters->VS.ViewUniformBuffer = GetShaderBinding(bUseVolumetricRenderTarget ? View.VolumetricRenderTargetViewUniformBuffer : View.ViewUniformBuffer);
		PassParameters->PS.FogCommon.ViewUniformBuffer = GetShaderBinding(bUseVolumetricRenderTarget ? View.VolumetricRenderTargetViewUniformBuffer : View.ViewUniformBuffer);
		PassParameters->PS.FogCommon.bOnlyOnRenderedOpaque = false;

		PassParameters->PS.FogCommon.SrcCloudDepthTexture = SrcCloudDepth;
		PassParameters->PS.FogCommon.SrcCloudViewTexture = SrcCloudView;

		PassParameters->RenderTargets[0] = FRenderTargetBinding(DstCloudView, ERenderTargetLoadAction::ENoAction);
		// No depth target

		// We enable the blending when volumetric render target is not enabled. Because in this case, the fog pass is compositing directly over the scene.
		const bool bEnableBlending = !bUseVolumetricRenderTarget;

		FIntRect ViewRect(0, 0, SrcCloudView->Desc.Extent.X, SrcCloudView->Desc.Extent.Y);
		GraphBuilder.AddPass(RDG_EVENT_NAME("Fog"), PassParameters, ERDGPassFlags::Raster,
			[&View, ViewRect, PassParameters, bShouldRenderVolumetricFog, bFogComposeLocalFogVolumes, bEnableBlending](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				const bool bSampleFogOnClouds = true;
				RenderViewFog(RHICmdList, View, ViewRect, PassParameters, bShouldRenderVolumetricFog, bFogComposeLocalFogVolumes, bSampleFogOnClouds, bEnableBlending);
			});
}
}

void FDeferredShadingSceneRenderer::RenderFog(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef LightShaftOcclusionTexture,
	bool bFogComposeLocalFogVolumes)
{
	if (ShouldRenderDeferredFog(Scene))
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, Fog, "ExponentialHeightFog");

		const bool bShouldRenderVolumetricFog = ShouldRenderVolumetricFog();

		for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];

			const bool bShouldRenderFogForView = ShouldRenderFogForView(Scene, View);
			if (!bShouldRenderFogForView)
			{
				continue;
			}

			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, bShouldRenderFogForView && Views.Num() > 1, "View%d", ViewIndex);
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			TRDGUniformBufferRef<FFogUniformParameters> FogUniformBuffer = CreateFogUniformBuffer(GraphBuilder, View);

			const FScreenPassTextureViewport SceneViewport(SceneTextures.Config.Extent, View.ViewRect);
			const FScreenPassTextureViewport OutputViewport(GetDownscaledViewport(SceneViewport, GetLightShaftDownsampleFactor()));
			const FScreenPassTextureViewportParameters LightShaftParameters = GetScreenPassTextureViewportParameters(OutputViewport);

			FFogPassParameters* PassParameters = CreateDefaultFogPassParameters(GraphBuilder, View, SceneTextures.UniformBuffer, FogUniformBuffer, LightShaftOcclusionTexture, LightShaftParameters);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);

			GraphBuilder.AddPass(RDG_EVENT_NAME("Fog"), PassParameters, ERDGPassFlags::Raster,
				[this, &View, PassParameters, bShouldRenderVolumetricFog, bFogComposeLocalFogVolumes](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					RenderViewFog(RHICmdList, View, View.ViewRect, PassParameters, bShouldRenderVolumetricFog, bFogComposeLocalFogVolumes);
				});
		}
	}
}

void FDeferredShadingSceneRenderer::RenderFogSeparateCompositionTextures(
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	TArray<FFogSeparateCompositionViewResources>& FogSeparateCompositionViewResources,
	FRDGTextureRef LightShaftOcclusionTexture,
	bool bFogComposeLocalFogVolumes,
	const bool bRenderSkyAtmosphereInLayer)
{
	if (ShouldRenderDeferredFog(Scene))
	{
		check(bFogComposeLocalFogVolumes == true);
		check(SceneTextures.DitheredHalfResDepth);

		const bool bShouldRenderVolumetricFog = ShouldRenderVolumetricFog();
		const bool bSceneRequestsFSSS = SceneRequestsFSSS(Scene);

		for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];

			const bool bShouldRenderFogForView = ShouldRenderFogForView(Scene, View);
			if (!bShouldRenderFogForView)
			{
				continue;
			}

			FFogSeparateCompositionViewResources& FogSeparateCompositionViewResourcesIt = FogSeparateCompositionViewResources[ViewIndex];
			FogSeparateCompositionAllocateResources(GraphBuilder, FogSeparateCompositionViewResourcesIt, View, Scene);
			const FIntPoint FogViewExtent = FogSeparateCompositionViewResourcesIt.ViewRect.Size();

			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, bShouldRenderFogForView && Views.Num() > 1, "View%d - %dx%d", ViewIndex, FogViewExtent.X, FogViewExtent.Y);
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			{
				TRDGUniformBufferRef<FFogUniformParameters> FogUniformBuffer = CreateFogUniformBuffer(GraphBuilder, View);

				const FScreenPassTextureViewport SceneViewport(SceneTextures.Config.Extent, View.ViewRect);
				const FScreenPassTextureViewport OutputViewport(GetDownscaledViewport(SceneViewport, GetLightShaftDownsampleFactor()));
				const FScreenPassTextureViewportParameters LightShaftParameters = GetScreenPassTextureViewportParameters(OutputViewport);

				FRenderSeparateCompositionTexturesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderSeparateCompositionTexturesCS::FParameters>();

				CreateDefaultCommonFogPassParameters(
					GraphBuilder,
					View,
					FogUniformBuffer,
					LightShaftOcclusionTexture,
					LightShaftParameters,
					PassParameters->FogCommon);

				PassParameters->FogSeparateCompositionDepth = FogSeparateCompositionViewResourcesIt.ResolutionDivider == 2 ? SceneTextures.DitheredHalfResDepth : SceneTextures.Depth.Resolve;
				PassParameters->FogTexture0UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(FogSeparateCompositionViewResourcesIt.FogSeparateCompositionTexture0));
				PassParameters->FogTexture1UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(FogSeparateCompositionViewResourcesIt.FogSeparateCompositionTexture1));
				PassParameters->InputSceneColorTexture = SceneTextures.Color.Target;
				PassParameters->InputSceneColorTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->FogSeparateCompositionViewResolution = FVector2f(FogViewExtent.X, FogViewExtent.Y);
				PassParameters->FogSeparateCompositionViewRectMin = FVector2f(FogSeparateCompositionViewResourcesIt.ViewRect.Min.X, FogSeparateCompositionViewResourcesIt.ViewRect.Min.Y);
				PassParameters->FogSeparateCompositionResolutionDivider = float(FogSeparateCompositionViewResourcesIt.ResolutionDivider);
				FIntVector NumGroups = FIntVector::DivideAndRoundUp(FIntVector(FogViewExtent.X, FogViewExtent.Y, 1), FIntVector(FRenderSeparateCompositionTexturesCS::ThreadGroupSize, FRenderSeparateCompositionTexturesCS::ThreadGroupSize, 1));

				using FPermutationDomain = TShaderPermutationDomain<FSupportFogInScatteringTexture, FVolumetricFogFilteringQuality, FSupportFogDirectionalLightInScattering, FSupportVolumetricFog, FSupportLocalFogVolume, FSampleSkyAtmosphereOnOpaque>;

				const bool bUseFogInscatteringColorCubemap = View.FogInscatteringColorCubemap != nullptr;
				FRenderSeparateCompositionTexturesCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FSupportFogInScatteringTexture>(bUseFogInscatteringColorCubemap);

				int VolumetricFogFilteringQuality = View.CachedViewUniformShaderParameters->RenderingReflectionCaptureMask == 0.0f ? CVarVolumetricFogFilteringQuality.GetValueOnAnyThread() : 0;
				PermutationVector.Set<FVolumetricFogFilteringQuality>(FMath::Clamp(VolumetricFogFilteringQuality, 0, 2));

				PermutationVector.Set<FSupportFogDirectionalLightInScattering>(!bUseFogInscatteringColorCubemap && View.bUseDirectionalInscattering);
				PermutationVector.Set<FSupportVolumetricFog>(bShouldRenderVolumetricFog);
				PermutationVector.Set<FSupportLocalFogVolume>(bFogComposeLocalFogVolumes);
				PermutationVector.Set<FSampleSkyAtmosphereOnOpaque>(bRenderSkyAtmosphereInLayer);
				TShaderMapRef<FRenderSeparateCompositionTexturesCS> ComputeShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);

				ERDGPassFlags PassFlag = ERDGPassFlags::Compute;

				// Dispatch with GenerateMips: reading from a slice through SRV and writing into lower mip through UAV.
				ClearUnusedGraphResources(ComputeShader, PassParameters);
				GraphBuilder.AddPass(
					Forward<FRDGEventName>(RDG_EVENT_NAME("Fog::RenderSeparateCompositionTextures")),
					PassParameters,
					PassFlag,
					[PassParameters, ComputeShader, NumGroups](FRDGAsyncTask, FRHICommandList& RHICmdList)
					{
						FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *PassParameters, NumGroups);
					});
			}
		}
	}
}

void FDeferredShadingSceneRenderer::RenderFogScreenSpaceScatteringMipChain( // FSSS
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	TArray<FFogSeparateCompositionViewResources>& FogSeparateCompositionViewResources)
{
	if (ShouldRenderDeferredFog(Scene))
	{
		const bool bShouldRenderVolumetricFog = ShouldRenderVolumetricFog();
		const bool bSceneRequestsFSSS = SceneRequestsFSSS(Scene);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];

			const bool bShouldRenderFogForView = ShouldRenderFogForView(Scene, View);
			if (!bShouldRenderFogForView)
			{
				continue;
			}

			FFogSeparateCompositionViewResources& FogSeparateCompositionViewResourcesIt = FogSeparateCompositionViewResources[ViewIndex];

			bool bMip1HasBeenGenerated = false;
			if (bSceneRequestsFSSS && FogSeparateCompositionViewResourcesIt.FogSeparateCompositionTexture0 && View.ViewState && CVarFogScreenSpaceScatteringTAA.GetValueOnRenderThread() > 0)
			{
				FTAAPassParameters TAASettings(View);
				TAASettings.SceneDepthTexture = SceneTextures.Depth.Resolve;
			
				// We need a valid velocity buffer texture. Use black (no velocity) if it's not produced.
				TAASettings.SceneVelocityTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);	// Cannot use velocity since our buffers are not aligned on view rects.
				TAASettings.Pass = ETAAPassConfig::Main;		// Reusing main config for now.
				TAASettings.SceneColorInput = FogSeparateCompositionViewResourcesIt.FogSeparateCompositionTexture0;
				TAASettings.Quality = ETAAQuality::Low;			// This is enough quality wise and the cost can come down form 0.25ms to 0.16ms on some platforms at 4K resolution.
				TAASettings.bOutputRenderTargetable = true;
				TAASettings.bDownsample = true;
				TAASettings.DownsampleRenderTarget = FogSeparateCompositionViewResourcesIt.FogSeparateCompositionTexture0;
				TAASettings.DownsampleRenderTargetMipLevel = 1;
				TAASettings.CoCBilateralFilterStrength = 0.0f;

				// We cannot use TAASettings.SetupViewRect since our view containing data for FSSS is always at the origin of the FSSS render target.
				TAASettings.InputViewRect  = View.ViewRect - View.ViewRect.Min;
				TAASettings.OutputViewRect = View.ViewRect - View.ViewRect.Min;

				FTAAOutputs TAAOutputs = AddTemporalAAPass(
					GraphBuilder,
					View,
					TAASettings,
					View.PrevViewInfo.FSSSHistory,
					&View.ViewState->PrevFrameViewInfo.FSSSHistory);

				bMip1HasBeenGenerated = true;
			}


			if (bSceneRequestsFSSS && FogSeparateCompositionViewResourcesIt.FogSeparateCompositionTexture0)
			{
				// Now generate the blur mip chain.
				const FExponentialHeightFogSceneInfo& HFog = Scene->ExponentialFogs[0];
				FFogSSSBlurMipChainParameters BlurMipChainParameters;
				BlurMipChainParameters.TextureWithMipChain = FogSeparateCompositionViewResourcesIt.FogSeparateCompositionTexture0;
				BlurMipChainParameters.UpsampleBlurFactor = HFog.FSSSBlurControl;
				BlurMipChainParameters.GaussianBlurScale = View.CachedViewUniformShaderParameters->ViewResolutionFraction;
				BlurMipChainParameters.bMip1HasBeenGenerated = bMip1HasBeenGenerated;
				// BlurMipChainParameters.BlurMaxExposedLuminance
				FogSeparateCompositionViewResourcesIt.FogSeparateCompositionTexture0 = GenerateFSSSBlurMipChain(GraphBuilder, BlurMipChainParameters);
			}
		}
	}
}

void FDeferredShadingSceneRenderer::RenderUnderWaterFog(
	FRDGBuilder& GraphBuilder,
	const FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesWithDepth)
{
	if (ShouldRenderDeferredFog(Scene))
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, Fog, "SLW::ExponentialHeightFog");

		FRDGTextureRef WaterDepthTexture = SceneWithoutWaterTextures.DepthTexture;
		check(WaterDepthTexture);

		const bool bShouldRenderVolumetricFog = ShouldRenderVolumetricFog();

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];

			const bool bShouldRenderFogForView = ShouldRenderFogForView(Scene, View);
			if (!bShouldRenderFogForView)
			{
				continue;
			}

			if (View.IsUnderwater() || CVarUnderwaterFogWhenCameraIsAboveWater.GetValueOnRenderThread())
			{
				RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

				const auto& SceneWithoutWaterView = SceneWithoutWaterTextures.Views[ViewIndex];

				TRDGUniformBufferRef<FFogUniformParameters> FogUniformBuffer = CreateFogUniformBuffer(GraphBuilder, View);

				// TODO add support for occlusion texture on water
				FRDGTextureRef LightShaftOcclusionTexture = nullptr;	
				FScreenPassTextureViewportParameters LightShaftParameters = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(FIntRect(0, 0, 1, 1)));

				FFogPassParameters* PassParameters = CreateDefaultFogPassParameters(GraphBuilder, View, SceneTexturesWithDepth, FogUniformBuffer, LightShaftOcclusionTexture, LightShaftParameters);
				PassParameters->PS.FogCommon.WaterDepthTexture = WaterDepthTexture;
				PassParameters->PS.FogCommon.bUseWaterDepthTexture = true;
				PassParameters->PS.FogCommon.WaterDepthTextureMinMaxUV = SceneWithoutWaterView.MinMaxUV;
				PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneWithoutWaterTextures.ColorTexture, ERenderTargetLoadAction::ELoad);
				// No depth/stencil bound so depth bound clip will not work. If we enable this at some point, we will have to check LocalFogVolume to disable depth bound. Or have a start depth for it.

				const bool bFogComposeLocalFogVolumes = ShouldRenderLocalFogVolume(Scene, ViewFamily); // Always render LFV as part of underwater fog, if present, to see them through the water.
				GraphBuilder.AddPass(RDG_EVENT_NAME("FogBehindWater"), PassParameters, ERDGPassFlags::Raster, [this, &View, SceneWithoutWaterView, PassParameters, bShouldRenderVolumetricFog, bFogComposeLocalFogVolumes](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					RenderViewFog(RHICmdList, View, SceneWithoutWaterView.ViewRect, PassParameters, bShouldRenderVolumetricFog, bFogComposeLocalFogVolumes);
				});
			}
		}
	}
}

bool ShouldRenderFog(const FSceneViewFamily& Family)
{
	const FEngineShowFlags EngineShowFlags = Family.EngineShowFlags;

	return EngineShowFlags.Fog
		&& EngineShowFlags.Materials 
		&& !Family.UseDebugViewPS()
		&& CVarFog.GetValueOnRenderThread() == 1
		&& !EngineShowFlags.StationaryLightOverlap 
		&& !EngineShowFlags.LightMapDensity;
}

bool ShouldRenderDeferredFog(const FScene* Scene)
{
	return Scene && Scene->ExponentialFogs.Num() > 0
		// Fog must be done in the base pass for MSAA to work
		&& !IsForwardShadingEnabled(Scene->GetShaderPlatform());
}

float GetFogDefaultStartDistance()
{
	return 30.0f;
}

float GetViewFogCommonStartDistance(const FViewInfo& View, bool bShouldRenderVolumetricFog, bool bShouldRenderLocalFogVolumes)
{
	float ExpFogStartDistance = View.ExponentialFogParameters.W;
	float VolFogStartDistance = bShouldRenderVolumetricFog ? View.VolumetricFogStartDistance : ExpFogStartDistance;

	// The fog can be set to start at a certain euclidean distance.
	// clamp the value to be behind the near plane z, according to the smallest distance between volumetric fog and height fog (if they are enabled). 
	float FogCommonStartDistance = FMath::Min(ExpFogStartDistance, VolFogStartDistance);

	if (bShouldRenderLocalFogVolumes)
	{
		const float LocalFogVolumeStartDistance = GetLocalFogVolumeViewStartDistance(View);
		FogCommonStartDistance = FMath::Min(LocalFogVolumeStartDistance, FogCommonStartDistance);
	}

	FogCommonStartDistance = FMath::Max(GetFogDefaultStartDistance(), FogCommonStartDistance);

	return FogCommonStartDistance;
}
