// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FogRendering.h: 
=============================================================================*/

#pragma once

#include "ShaderParameterMacros.h"
#include "RenderGraphFwd.h"

class FSceneViewFamily;
class FScene;
class FViewInfo;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FFogUniformParameters,)
	SHADER_PARAMETER(FVector4f, ExponentialFogParameters)
	SHADER_PARAMETER(FVector4f, ExponentialFogParameters2)
	SHADER_PARAMETER(FVector4f, ExponentialFogColorParameter)
	SHADER_PARAMETER(FVector4f, ExponentialFogParameters3)
	SHADER_PARAMETER(FVector4f, SkyAtmosphereAmbientContributionColorScale)
	SHADER_PARAMETER(FVector4f, InscatteringLightDirection) // non negative DirectionalInscatteringStartDistance in .W
	SHADER_PARAMETER(FVector4f, DirectionalInscatteringColor)
	SHADER_PARAMETER(FVector2f, SinCosInscatteringColorCubemapRotation)
	SHADER_PARAMETER(FVector3f, FogInscatteringTextureParameters)
	SHADER_PARAMETER(float, SkyLightCaptureAffectsHeightFogStrength) // When >0 and r.SupportsSkyLightCaptureAffectsHeightFogInScattering>0, uses FogInscatteringColorCubemap to sample the skylight at some roughness.
	SHADER_PARAMETER(float, SkyLightCaptureAffectsHeightFogRoughness)
	SHADER_PARAMETER(float, SkyLightCaptureAffectsHeightFogMipCount)
	SHADER_PARAMETER(float, EndDistance)
	SHADER_PARAMETER(float, ApplyVolumetricFog)
	SHADER_PARAMETER(float, VolumetricFogStartDistance)
	SHADER_PARAMETER(float, VolumetricFogNearFadeInDistanceInv)
	SHADER_PARAMETER(FVector3f, VolumetricFogAlbedo)
	SHADER_PARAMETER(float, VolumetricFogPhaseG)
	SHADER_PARAMETER(float, EnableFSSS)
	SHADER_PARAMETER(float, FSSSSceneColorScatteringAmountScale)
	SHADER_PARAMETER(float, FSSSSceneColorScatteringAmountPower)
	SHADER_PARAMETER(float, FSSSSpreadScale)
	SHADER_PARAMETER(float, FSSSBlurControl)
	SHADER_PARAMETER_TEXTURE(TextureCube, FogInscatteringColorCubemap)
	SHADER_PARAMETER_SAMPLER(SamplerState, FogInscatteringColorSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, IntegratedLightScattering)
	SHADER_PARAMETER_SAMPLER(SamplerState, IntegratedLightScatteringSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

extern void SetupFogUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FFogUniformParameters& OutParameters, bool bForRealtimeSkyCapture = false);
TRDGUniformBufferRef<FFogUniformParameters> CreateFogUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& View);

extern bool ShouldRenderFog(const FSceneViewFamily& Family);
extern bool ShouldRenderDeferredFog(const FScene* Scene);

extern float GetViewFogCommonStartDistance(const FViewInfo& View, bool bShouldRenderVolumetricFog, bool bShouldRenderLocalFogVolumes);
extern float GetFogDefaultStartDistance();

extern void RenderFogOnClouds(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	FRDGTextureRef SrcCloudDepth,
	FRDGTextureRef SrcCloudView,
	FRDGTextureRef DstCloudView,
	const bool bShouldRenderVolumetricFog,
	const bool bUseVolumetricRenderTarget);