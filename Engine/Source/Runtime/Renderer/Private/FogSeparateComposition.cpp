// Copyright Epic Games, Inc. All Rights Reserved.

#include "FogSeparateComposition.h"
#include "GlobalShader.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "SceneRendering.h"
#include "PixelShaderUtils.h"
#include "ScenePrivate.h"
#include "SceneCore.h"


static TAutoConsoleVariable<int32> CVarFogSeparateComposition(
	TEXT("r.Fog.SeparateComposition"),
	-1,
	TEXT("Renders the fog into its own separate texture that can be then upsample on screen. It will contain height fog, volumetric fog as well as local fog volumes contributions. -1: default behavior, 0: force disabled. 1 force enabled."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

//static TAutoConsoleVariable<int32> CVarFogSeparateCompositionHalfResolution(
//	TEXT("r.Fog.SeparateComposition.HalfResolution"),
//	0,
//	TEXT("Renders the fog into its own separate texture that can be then upsample on screen. It will contain height fog, volumetric fog as well as local fog volumes contributions. 0: Fog separate composition runs at full resolution, 1: Fog separate composition runs at half resolution."),
//	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarFogScreenSpaceScattering(
	TEXT("r.Fog.ScreenSpaceScattering"),
	1,
	TEXT("Enabled Fog Screen Space Scattering (FSSS). When 0, it is disabled."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarFogScreenSpaceScatteringMaxExposedLuminance(
	TEXT("r.Fog.ScreenSpaceScattering.MaxExposedLuminance"),
	10.0f,
	TEXT("FSSS maximum luminance. This is too avoid the blur kernel to become too saturated with visible artefact due to high luminance value in the scene color buffer such as the sun disk or other bright elements."),
	ECVF_RenderThreadSafe | ECVF_Scalability);


EPixelFormat GetFogSeparateCompositionSceneColorTransmittanceFormat()
{
	return PF_FloatRGBA;
}

static bool DoesPlatformSupportFSSS()
{
	// Fog screen space scattering requiers typed load through UAV
	EPixelFormat PixelFormat					= GetFogSeparateCompositionSceneColorTransmittanceFormat();
	const static bool bFormatSupported			= GPixelFormats[PixelFormat].Supported && UE::PixelFormat::HasCapabilities(PixelFormat, EPixelFormatCapabilities::TextureFilterable | EPixelFormatCapabilities::UAV);
	const static bool bFormatTypedLoadSupported	= bFormatSupported && UE::PixelFormat::HasCapabilities(PixelFormat, EPixelFormatCapabilities::TypedUAVLoad);
	return bFormatTypedLoadSupported;
}

bool SceneRequestsFSSS(const FScene* Scene)
{
	if (!DoesPlatformSupportFSSS())
	{
		return false;
	}

	if (Scene && Scene->HasAnyExponentialHeightFog() && CVarFogScreenSpaceScattering.GetValueOnAnyThread() > 0)
	{
		const FExponentialHeightFogSceneInfo& FogInfo = Scene->ExponentialFogs[0];
		return FogInfo.bEnableFSSS;
	}

	return false;
}

bool ShouldRenderFogUsingSeparateComposition(const FScene* Scene)
{
	const bool SceneRequestFSSS = SceneRequestsFSSS(Scene);

	const int32 CVarValue = CVarFogSeparateComposition.GetValueOnAnyThread();

	// Enable if enforced, or if default behavior and the exponential height fog component request FSSS enabled.
	return CVarValue == 1 || (CVarValue == -1 && SceneRequestFSSS);
}

bool ShouldRenderFogUsingSeparateCompositionHalfRes()
{
	return false; // Not fully implemented
	//return CVarFogSeparateCompositionHalfResolution.GetValueOnAnyThread() > 0;
}

static bool CanRenderFogScreenSpaceScattering()
{
	// Fog screen space scattering requiers typed load through UAV

	EPixelFormat PixelFormat				= GetFogSeparateCompositionSceneColorTransmittanceFormat();
	const bool bFormatSupported				= GPixelFormats[PixelFormat].Supported && UE::PixelFormat::HasCapabilities(PixelFormat, EPixelFormatCapabilities::TextureFilterable | EPixelFormatCapabilities::UAV);
	const bool bFormatTypedLoadSupported	= bFormatSupported && UE::PixelFormat::HasCapabilities(PixelFormat, EPixelFormatCapabilities::TypedUAVLoad);
	return bFormatTypedLoadSupported;
}


//////////////////////////////////////////////////////////////////////////

class FSSSDownsampleSceneColorCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSSDownsampleSceneColorCS);
	SHADER_USE_PARAMETER_STRUCT(FSSSDownsampleSceneColorCS, FGlobalShader);

	static const uint32 ThreadGroupSize = 8;

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, SrcMipLevel)
		SHADER_PARAMETER(FIntPoint, SrcMipResolution)
		SHADER_PARAMETER(FIntPoint, DstMipResolution)
		SHADER_PARAMETER(float, GaussianBlurScale)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SourceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTextureMipColor)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("DOWNSAMPLECOLORCS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FSSSDownsampleSceneColorCS, "/Engine/Private/FogSeparateComposition.usf", "DownsampleColorCS", SF_Compute);

static void AddDownsampleSceneColorPass(FRDGBuilder& GraphBuilder, FSSSDownsampleSceneColorCS::FParameters* PassParameters, bool bSSFS = false)
{
	FIntVector NumGroups = FIntVector::DivideAndRoundUp(
		FIntVector(PassParameters->DstMipResolution.X, PassParameters->DstMipResolution.Y, 1),
		FIntVector(FSSSDownsampleSceneColorCS::ThreadGroupSize, FSSSDownsampleSceneColorCS::ThreadGroupSize, 1));

	FSSSDownsampleSceneColorCS::FPermutationDomain PermutationVector;
	TShaderMapRef<FSSSDownsampleSceneColorCS> ComputeShader(GetGlobalShaderMap(GMaxRHIShaderPlatform), PermutationVector);

	// Dispatch with GenerateMips: reading from a slice through SRV and writing into lower mip through UAV.
	ClearUnusedGraphResources(ComputeShader, PassParameters);
	check(PassParameters->OutTextureMipColor);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("FSSS::MipGen - MipLevel%d", PassParameters->OutTextureMipColor->Desc.MipLevel), ComputeShader, PassParameters, NumGroups);
}

//////////////////////////////////////////////////////////////////////////

// Must match FogSeparateComposition.ush.
// Default: filter source + blend with previous mip
#define PERMUTATION_UPSAMPLE_MODE_FILTER_AND_BLEND	0
// Copy source to destination without filtering.
#define PERMUTATION_UPSAMPLE_MODE_COPY_ONLY			1
// Filter one texture into another.
#define PERMUTATION_UPSAMPLE_MODE_FILTER_ONLY		2
// Blend with another texture.
#define PERMUTATION_UPSAMPLE_MODE_BLEND_ONLY		3

class FSSSUpsampleSceneColorCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSSSUpsampleSceneColorCS);
	SHADER_USE_PARAMETER_STRUCT(FSSSUpsampleSceneColorCS, FGlobalShader);

	static const uint32 ThreadGroupSize = 8;

	class FUpsampleMode : SHADER_PERMUTATION_INT("PERMUTATION_UPSAMPLE_MODE", 4);
	using FPermutationDomain = TShaderPermutationDomain<FUpsampleMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, GaussianBlurScale)
		SHADER_PARAMETER(float, UpsampleBlurFactor)
		SHADER_PARAMETER(float, BlurMaxExposedLuminance)
		SHADER_PARAMETER(FIntPoint, SourceTextureToFilterMipResolution)
		SHADER_PARAMETER(FVector2f, SourceTextureToFilterMipResolutionInv)
		SHADER_PARAMETER(int32, MipLeveltoBlend0)
		SHADER_PARAMETER(int32, MipLeveltoBlend1)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SourceTextureToFilter)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTextureToBlend)			// Used by MODE_BLEND_ONLY that samples a mip level using parameters and needs full resource for global barrier resulting in compute work overlaping on GPU.
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SourceTextureToBlendSRV)	// Use by mode FILTER_AND_BLEND that needs to pointto a particular SRV of the source to filter.
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTextureMipColor)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("UPSAMPLECOLORCS"), 1);

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads); // Because we are blending into OutTextureMipColor
	}
};
IMPLEMENT_GLOBAL_SHADER(FSSSUpsampleSceneColorCS, "/Engine/Private/FogSeparateComposition.usf", "UpsampleColorCS", SF_Compute);

static void AddUpsampleSceneColorPass(FRDGBuilder& GraphBuilder, FSSSUpsampleSceneColorCS::FParameters* PassParameters, uint8 UpsamplingMode)
{
	FIntVector NumGroups = FIntVector::DivideAndRoundUp(
		FIntVector(PassParameters->SourceTextureToFilterMipResolution.X, PassParameters->SourceTextureToFilterMipResolution.Y, 1),
		FIntVector(FSSSUpsampleSceneColorCS::ThreadGroupSize, FSSSUpsampleSceneColorCS::ThreadGroupSize, 1));

	FSSSUpsampleSceneColorCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FSSSUpsampleSceneColorCS::FUpsampleMode>(UpsamplingMode);
	TShaderMapRef<FSSSUpsampleSceneColorCS> ComputeShader(GetGlobalShaderMap(GMaxRHIShaderPlatform), PermutationVector);

	// Dispatch with GenerateMips: reading from a slice through SRV and writing into lower mip through UAV.
	ClearUnusedGraphResources(ComputeShader, PassParameters);
	check(PassParameters->OutTextureMipColor);
	FComputeShaderUtils::AddPass(GraphBuilder, 
		RDG_EVENT_NAME("FSSS::%s - MipLevel%d", 
			UpsamplingMode == PERMUTATION_UPSAMPLE_MODE_FILTER_AND_BLEND ? TEXT("FilterAndBlend") :
			(UpsamplingMode == PERMUTATION_UPSAMPLE_MODE_COPY_ONLY ? TEXT("CopyOnly") :
			(UpsamplingMode == PERMUTATION_UPSAMPLE_MODE_FILTER_ONLY ? TEXT("FilterOnly") :
			(UpsamplingMode == PERMUTATION_UPSAMPLE_MODE_BLEND_ONLY ? TEXT("BlendOnly") : 
			TEXT("UNKOWN PASS NAME")))),
			PassParameters->OutTextureMipColor->Desc.MipLevel),
		ComputeShader, PassParameters, NumGroups);
}


//////////////////////////////////////////////////////////////////////////


FRDGTextureRef GenerateFSSSBlurMipChain(FRDGBuilder& GraphBuilder, FFogSSSBlurMipChainParameters& Parameters)
{
	RDG_EVENT_SCOPE(GraphBuilder, "FSSS::GenerateMipChain");

	FRDGTextureRef SourceTexture = Parameters.TextureWithMipChain;
	check(SourceTexture);
	const uint32 MipCount = SourceTexture->Desc.NumMips;
	check(MipCount <= 32);
	FIntPoint MipResolutions[32];

	// Generate the mip chain.
	FIntPoint DstMipResolution = FIntPoint::ZeroValue;
	FIntPoint SrcMipResolution = FIntPoint(SourceTexture->Desc.GetSize().X, SourceTexture->Desc.GetSize().Y);

	if (Parameters.bMip1HasBeenGenerated)
	{
		DstMipResolution = FIntPoint(FMath::Max(SrcMipResolution.X / 2, 1), FMath::Max(SrcMipResolution.Y / 2, 1));
		MipResolutions[0] = SrcMipResolution;
		SrcMipResolution = DstMipResolution;
	}

	for (uint32 DstMipLevel = Parameters.bMip1HasBeenGenerated ? 2 : 1; DstMipLevel < MipCount; ++DstMipLevel)
	{
		const uint32 SrcMipLevel = DstMipLevel - 1;
		DstMipResolution = FIntPoint(FMath::Max(SrcMipResolution.X / 2, 1), FMath::Max(SrcMipResolution.Y / 2, 1));

		FSSSDownsampleSceneColorCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSSDownsampleSceneColorCS::FParameters>();
		PassParameters->SrcMipLevel = SrcMipLevel;
		PassParameters->SrcMipResolution = SrcMipResolution;
		PassParameters->DstMipResolution = DstMipResolution;
		PassParameters->GaussianBlurScale = Parameters.GaussianBlurScale;
		PassParameters->SourceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

		PassParameters->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceTexture, SrcMipLevel));
		PassParameters->OutTextureMipColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SourceTexture, DstMipLevel));
		const bool bSSFS = true;
		AddDownsampleSceneColorPass(GraphBuilder, PassParameters, bSSFS);

		MipResolutions[SrcMipLevel] = SrcMipResolution;
		SrcMipResolution = DstMipResolution;
	}
	MipResolutions[MipCount - 1] = SrcMipResolution;


	// Create the blur destination texture.
	FRDGTextureDesc TextureDesc = SourceTexture->Desc;
	TextureDesc.Flags |= ETextureCreateFlags::UAV;
	TextureDesc.ClearValue = FClearValueBinding::None;
	FRDGTextureRef DestinationTextureWithMipChain = GraphBuilder.CreateTexture(TextureDesc, TEXT("FSSSBlurredMipChain"));

	// We use the optimise filtering processing all mips in parallel for each steps with overlaping compute work.
#define OPTIMISED_FSSS_FILTERING 1
#if OPTIMISED_FSSS_FILTERING 

	// First generate all blurred levels without blending with previous mip into a temporary texture.
	// We want to run it in parallel on GPU using overlapping compute work to avoid long chain of tiny low occupancy work.
	// We do not filter the first mip level, so DstMipLevel starts at 1.
	for (uint32 DstMipLevel = 1; DstMipLevel < MipCount; ++DstMipLevel)
	{
		FIntPoint SourceTextureToFilterMipResolution = MipResolutions[DstMipLevel];

		FSSSUpsampleSceneColorCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSSUpsampleSceneColorCS::FParameters>();
		PassParameters->SourceTextureToFilterMipResolution = SourceTextureToFilterMipResolution;
		PassParameters->SourceTextureToFilterMipResolutionInv = FVector2f::One() / FVector2f(SourceTextureToFilterMipResolution);
		PassParameters->SourceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->GaussianBlurScale = Parameters.GaussianBlurScale;
		PassParameters->UpsampleBlurFactor = Parameters.UpsampleBlurFactor;
		PassParameters->BlurMaxExposedLuminance = FMath::Max(0.1f, CVarFogScreenSpaceScatteringMaxExposedLuminance.GetValueOnAnyThread());

		PassParameters->SourceTextureToFilter = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceTexture, DstMipLevel));
		PassParameters->OutTextureMipColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(DestinationTextureWithMipChain, DstMipLevel), ERDGUnorderedAccessViewFlags::SkipBarrier);

		AddUpsampleSceneColorPass(GraphBuilder, PassParameters, PERMUTATION_UPSAMPLE_MODE_FILTER_ONLY);
	}

	// Second run the blending step for all filtered mips back into into the destination SourceTexture.
	// The last mip level is not blended.
	const uint32 LastMipId = MipCount - 1;
	for (uint32 DstMipLevel = 1; DstMipLevel < LastMipId; ++DstMipLevel)
	{
		FIntPoint SourceTextureToFilterMipResolution = MipResolutions[DstMipLevel];

		FSSSUpsampleSceneColorCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSSSUpsampleSceneColorCS::FParameters>();
		PassParameters->SourceTextureToFilterMipResolution = SourceTextureToFilterMipResolution;
		PassParameters->SourceTextureToFilterMipResolutionInv = FVector2f::One() / FVector2f(SourceTextureToFilterMipResolution);
		PassParameters->SourceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->GaussianBlurScale = Parameters.GaussianBlurScale;
		PassParameters->UpsampleBlurFactor = Parameters.UpsampleBlurFactor;
		PassParameters->BlurMaxExposedLuminance = FMath::Max(0.1f, CVarFogScreenSpaceScatteringMaxExposedLuminance.GetValueOnAnyThread());

		PassParameters->SourceTextureToFilter = nullptr;// unused for this mode

		PassParameters->SourceTextureToBlend = DestinationTextureWithMipChain;
		PassParameters->MipLeveltoBlend0 = DstMipLevel;
		PassParameters->MipLeveltoBlend1 = DstMipLevel+1;

		PassParameters->OutTextureMipColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SourceTexture, DstMipLevel), ERDGUnorderedAccessViewFlags::SkipBarrier); // We blend in all mip level in parallel.

		AddUpsampleSceneColorPass(GraphBuilder, PassParameters, PERMUTATION_UPSAMPLE_MODE_BLEND_ONLY);
	}

	return SourceTexture;

#else

	// Generate the blurred scene texture mips.
	// The first upsampling pass for the lowest mip is only a copy.
	// For the next upsampling passeses, we filter the same mip level texture and blend it with the previous mip to get a more accurate result.
	const int32 LastMipId = MipCount - 1;
	for (int32 DstMipLevel = LastMipId; DstMipLevel >= 0; --DstMipLevel)
	{
		const int32 SrcMipLevel = DstMipLevel + 1;
		FIntPoint SourceTextureToFilterMipResolution			= MipResolutions[DstMipLevel];

		FSSSUpsampleSceneColorCS::FParameters* PassParameters	= GraphBuilder.AllocParameters<FSSSUpsampleSceneColorCS::FParameters>();
		PassParameters->SourceTextureToFilterMipResolution = SourceTextureToFilterMipResolution;
		PassParameters->SourceTextureToFilterMipResolutionInv = FVector2f::One() / FVector2f(SourceTextureToFilterMipResolution);
		PassParameters->SourceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->GaussianBlurScale = Parameters.GaussianBlurScale;
		PassParameters->UpsampleBlurFactor = Parameters.UpsampleBlurFactor;
		PassParameters->BlurMaxExposedLuminance = FMath::Max(0.1f, CVarFogScreenSpaceScatteringMaxExposedLuminance.GetValueOnAnyThread());

		if(DstMipLevel == LastMipId)
		{
			// For the first upsample pass in last mip level, we cannot add any previous mip.
			PassParameters->SourceTextureToBlendSRV	= GraphBuilder.CreateSRV(GSystemTextures.GetBlackDummy(GraphBuilder));
		}
		else
		{
			PassParameters->SourceTextureToBlendSRV	= GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(	DestinationTextureWithMipChain,		SrcMipLevel));
		}

		PassParameters->SourceTextureToFilter		= GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(	SourceTexture,						DstMipLevel));
		PassParameters->OutTextureMipColor			= GraphBuilder.CreateUAV(FRDGTextureUAVDesc(					DestinationTextureWithMipChain,		DstMipLevel));

		// Optimisation to avoid the heavy filtering of mip 0 we we can do without.
		// The last mip is also just a copy since there is no previous texture filter and blend with.
		const uint8 UpsamplingMode = DstMipLevel == LastMipId || (DstMipLevel == 0) ? PERMUTATION_UPSAMPLE_MODE_COPY_ONLY : PERMUTATION_UPSAMPLE_MODE_FILTER_AND_BLEND;
		AddUpsampleSceneColorPass(GraphBuilder, PassParameters, UpsamplingMode);
	}

	return DestinationTextureWithMipChain;
#endif
}





IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSSSGlobalParameters, "FSSSGlobals");

TRDGUniformBufferRef<FSSSGlobalParameters> GetDefaultFSSSGlobalParameters(FRDGBuilder& GraphBuilder)
{
	FSSSGlobalParameters* FSSSGlobals = GraphBuilder.AllocParameters<FSSSGlobalParameters>();

	FSSSGlobals->HalfResDepthMinMaxCoord = FIntVector4::ZeroValue;
	FSSSGlobals->TexelCoordToUVs = FVector2f::One();
	FSSSGlobals->FullResToFogResScale = 1.0f;
	FSSSGlobals->FSSSSpreadScale = 1.0f;
	FSSSGlobals->FSSSMaxMip = 1.0f;

	FSSSGlobals->FogSeparateCompositionTexture0 = GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder);
	FSSSGlobals->FogSeparateCompositionTexture1 = GSystemTextures.GetWhiteDummy(GraphBuilder);
	FSSSGlobals->FogSeparateCompositionTexture0Sampler = TStaticSamplerState<SF_Trilinear>::GetRHI();
	FSSSGlobals->FogSeparateCompositionTexture1Sampler = TStaticSamplerState<SF_Trilinear>::GetRHI();

	return GraphBuilder.CreateUniformBuffer(FSSSGlobals);
}


class FUpsampleHalfResTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FUpsampleHalfResTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FUpsampleHalfResTexturePS, FGlobalShader);

	class FUpsampleFullResolution : SHADER_PERMUTATION_BOOL("PERMUTATION_UPSAMPLE_FULL_RESOLUTION");
	using FPermutationDomain = TShaderPermutationDomain<FUpsampleFullResolution>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSSSGlobalParameters, FSSSGlobals)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HalfResDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, HalfResDepthSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FullResDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, FullResDepthSampler)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HALFRESOLUTIONUPSAMPLERPS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FUpsampleHalfResTexturePS, "/Engine/Private/FogSeparateComposition.usf", "HalfResolutionUpsamplerPS", SF_Pixel);





static uint32 GetViewBlurMipChainMipCount(const FIntPoint TextureMip0Resolution)
{
	// We do not use max and do not add 1 to the result, this to not go down to the lowest mip level as this is not required.
	const int32 MipCount = FMath::Max((int32)1, (int32)FMath::CeilLogTwo(FMath::Max((int32)1, FMath::Min(TextureMip0Resolution.X, TextureMip0Resolution.Y))));
	check(MipCount >= 1);
	return uint32(MipCount);
}

void FogSeparateCompositionAllocateResources(
	FRDGBuilder& GraphBuilder,
	FFogSeparateCompositionViewResources& FogSeparateCompositionViewResources,
	const class FViewInfo& View,
	const FScene* Scene)
{
	FogSeparateCompositionViewResources.ResolutionDivider = ShouldRenderFogUsingSeparateCompositionHalfRes() ? 2 : 1;

	FogSeparateCompositionViewResources.ViewRect = GetDownscaledRect(View.ViewRect, FIntPoint(FogSeparateCompositionViewResources.ResolutionDivider, FogSeparateCompositionViewResources.ResolutionDivider));
	const FIntPoint FogViewExtent = FogSeparateCompositionViewResources.ViewRect.Size();

	const bool bSceneRequestsFSSS = SceneRequestsFSSS(Scene);
	const uint32 MipCount = bSceneRequestsFSSS ? GetViewBlurMipChainMipCount(FogViewExtent) : 1;

	EPixelFormat SceneColorTransmittanceFormat		= GetFogSeparateCompositionSceneColorTransmittanceFormat();
	const FRDGTextureDesc FogSeparateCompositionDesc		= FRDGTextureDesc::Create2D(FogViewExtent, SceneColorTransmittanceFormat,	FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable, MipCount);
	const FRDGTextureDesc FogSeparateComposition2Desc		= FRDGTextureDesc::Create2D(FogViewExtent, PF_G16R16F,						FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV | TexCreate_RenderTargetable);
	FogSeparateCompositionViewResources.FogSeparateCompositionTexture0	= GraphBuilder.CreateTexture(FogSeparateCompositionDesc, TEXT("FogSeparateComposition"));
	FogSeparateCompositionViewResources.FogSeparateCompositionTexture1	= GraphBuilder.CreateTexture(FogSeparateComposition2Desc, TEXT("FogSeparateComposition2"));
}




void UpsampleFogSeparateCompositionTextureForView(
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	FFogSeparateCompositionViewResources& FogSeparateCompositionViewResources,
	FViewInfo& View,
	const FScene* Scene)
{
	const bool bShouldRenderFogForView = FogSeparateCompositionViewResources.FogSeparateCompositionTexture0 != nullptr && FogSeparateCompositionViewResources.FogSeparateCompositionTexture1 != nullptr;
	if (!bShouldRenderFogForView)
	{
		return;
	}

	const int32 ResolutionDivider = FogSeparateCompositionViewResources.ResolutionDivider;
	check(ResolutionDivider == 1 || ResolutionDivider == 2); // Upsampling only support those two cases

	check(Scene && Scene->ExponentialFogs.Num() > 0);
	const FExponentialHeightFogSceneInfo& FogInfo = Scene->ExponentialFogs[0];

	FIntRect HalfResViewRect = GetDownscaledRect(View.ViewRect, FIntPoint(ResolutionDivider, ResolutionDivider));
	FScreenPassTextureViewport HalfResViewport = FScreenPassTextureViewport(SceneTextures.DitheredHalfResDepth, HalfResViewRect);
	FScreenPassTextureViewport FullResViewport = FScreenPassTextureViewport(SceneTextures.Color.Target, View.ViewRect);

	const FVector2f HalfResExtentInv = FVector2f(1.0f, 1.0f) / FVector2f(HalfResViewport.Extent);

	FScreenTransform SvPositionToViewportUV = FScreenTransform::SvPositionToViewportUV(FullResViewport.Rect);

	FUpsampleHalfResTexturePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FUpsampleHalfResTexturePS::FParameters>();

	const FVector2f SourceTextureResolution = FVector2f(FogSeparateCompositionViewResources.FogSeparateCompositionTexture0->Desc.Extent.X, FogSeparateCompositionViewResources.FogSeparateCompositionTexture0->Desc.Extent.Y);
	const bool bSceneRequestsFSSS = SceneRequestsFSSS(Scene);

	// Create global shader parameters for fog composition
	FSSSGlobalParameters* FSSSGlobals = GraphBuilder.AllocParameters<FSSSGlobalParameters>();
	FSSSGlobals->HalfResDepthMinMaxCoord = FIntVector4(HalfResViewRect.Min.X, HalfResViewRect.Min.Y, HalfResViewRect.Max.X, HalfResViewRect.Max.Y);

	const FVector2f TextureFullResolution = FVector2f(FogSeparateCompositionViewResources.FogSeparateCompositionTexture0->Desc.Extent);
	const FVector2f TextureViewResolution = FVector2f(View.ViewRect.Size());
	const FVector2f ResolutionRatio = TextureViewResolution / TextureFullResolution;
	const FVector2f TexelSize = FVector2f::One() / TextureFullResolution;
	FSSSGlobals->TexelCoordToUVs = ResolutionRatio * TexelSize;

	FSSSGlobals->FullResToFogResScale = 1.0f / float(ResolutionDivider);
	FSSSGlobals->FSSSSpreadScale = bSceneRequestsFSSS ? FogInfo.FSSSSpreadScale : 0.0f; // Spread of 0 if FSSS is not enabled and mip chain not generated.
	FSSSGlobals->FSSSMaxMip = FogSeparateCompositionViewResources.FogSeparateCompositionTexture0->Desc.NumMips - 1.0f;
	FSSSGlobals->FogSeparateCompositionTexture0 = FogSeparateCompositionViewResources.FogSeparateCompositionTexture0;
	FSSSGlobals->FogSeparateCompositionTexture1 = FogSeparateCompositionViewResources.FogSeparateCompositionTexture1;
	FSSSGlobals->FogSeparateCompositionTexture0Sampler = TStaticSamplerState<SF_Trilinear>::GetRHI();
	FSSSGlobals->FogSeparateCompositionTexture1Sampler = TStaticSamplerState<SF_Trilinear>::GetRHI();

	FogSeparateCompositionViewResources.FSSSGlobalsUniformBuffer = GraphBuilder.CreateUniformBuffer(FSSSGlobals);
	PassParameters->FSSSGlobals = FogSeparateCompositionViewResources.FSSSGlobalsUniformBuffer;

	PassParameters->HalfResDepthTexture = SceneTextures.DitheredHalfResDepth;
	PassParameters->HalfResDepthSampler = TStaticSamplerState<SF_Point>::GetRHI();
	PassParameters->FullResDepthTexture = SceneTextures.Depth.Resolve;
	PassParameters->FullResDepthSampler = TStaticSamplerState<SF_Point>::GetRHI();

	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;

	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);

	FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();

	FUpsampleHalfResTexturePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FUpsampleHalfResTexturePS::FUpsampleFullResolution>(ResolutionDivider == 1);
	TShaderMapRef<FUpsampleHalfResTexturePS> PixelShader(View.ShaderMap, PermutationVector);
	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME("Fog::UpsampleSeparateCompositionTextures"),
		PixelShader,
		PassParameters,
		View.ViewRect,
		BlendState);
}