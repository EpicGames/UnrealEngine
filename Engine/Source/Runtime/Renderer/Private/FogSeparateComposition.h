// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FogSeparateComposition.h: code related to rendering of fog in separate textures.
	The composition can happen at full or half resolution.
	It also contains code for FSSS (Fog Screen Space Scattering) which is a technique 
	used to blur the fog in screen space to approximate multiple scattering.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphFwd.h"
#include "RenderGraphResources.h"


struct FFogSSSBlurMipChainParameters
{
	FRDGTextureRef TextureWithMipChain = nullptr;		// Textures to compute the mip chain for.

	float UpsampleBlurFactor = 0.8f;
	float GaussianBlurScale = 1.0f;
	bool bMip1HasBeenGenerated = false;
};

FRDGTextureRef GenerateFSSSBlurMipChain(FRDGBuilder& GraphBuilder, FFogSSSBlurMipChainParameters& Parameters);


BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSSSGlobalParameters,)
	SHADER_PARAMETER(FIntVector4, HalfResDepthMinMaxCoord)
	SHADER_PARAMETER(FVector2f, TexelCoordToUVs)
	SHADER_PARAMETER(float, FullResToFogResScale)
	SHADER_PARAMETER(float, FSSSSpreadScale)
	SHADER_PARAMETER(float, FSSSMaxMip)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, FogSeparateCompositionTexture0)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, FogSeparateCompositionTexture1)
	SHADER_PARAMETER_SAMPLER(SamplerState,			FogSeparateCompositionTexture0Sampler)
	SHADER_PARAMETER_SAMPLER(SamplerState,			FogSeparateCompositionTexture1Sampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

TRDGUniformBufferRef<FSSSGlobalParameters> GetDefaultFSSSGlobalParameters(FRDGBuilder& GraphBuilder);


class FFogSeparateCompositionViewResources
{
public:

	int32 ResolutionDivider = 1;

	FIntRect ViewRect;

	// Texture for all volumetric effect composited together with mip chain
	FRDGTextureRef FogSeparateCompositionTexture0 = nullptr;

	// Texture for side data used for FSSS (Fog Screen Space Scattering) when enabled
	FRDGTextureRef FogSeparateCompositionTexture1 = nullptr;

	// Uniform buffer containing global fog composition parameters
	TRDGUniformBufferRef<FSSSGlobalParameters> FSSSGlobalsUniformBuffer = nullptr;
};

bool ShouldRenderFogUsingSeparateComposition(
	const class FScene* Scene);

// Maybe scene requests FSSS (Fog Screen Space Scattering) to be enabled, but that will only happen when ShouldRenderFogUsingSeparateComposition is true, as FSSS is only achievable when rendering fog in separate textures.
bool SceneRequestsFSSS(const FScene* Scene);

void FogSeparateCompositionAllocateResources(
	FRDGBuilder& GraphBuilder,
	FFogSeparateCompositionViewResources& FogSeparateCompositionViewResources,
	const class FViewInfo& View,
	const class FScene* Scene);

void UpsampleFogSeparateCompositionTextureForView(
	FRDGBuilder& GraphBuilder,
	struct FSceneTextures& SceneTextures,
	FFogSeparateCompositionViewResources& FogSeparateCompositionViewResources,
	class FViewInfo& View,
	const class FScene* Scene);
