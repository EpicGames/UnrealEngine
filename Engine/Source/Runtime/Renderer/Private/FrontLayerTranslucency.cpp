// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrontLayerTranslucency.h"
#include "ScenePrivate.h"
#include "SceneUtils.h"
#include "DeferredShadingRenderer.h"
#include "PipelineStateCache.h"
#include "ShaderParameterStruct.h"
#include "VolumeLighting.h"
#include "MeshPassProcessor.inl"
#include "PixelShaderUtils.h"
#include "Lumen/LumenSceneData.h"
#include "Lumen/LumenTracingUtils.h"
#include "Lumen/RayTracedTranslucency.h"
#include "TextureFallbacksRDG.h"
#include "SceneViewState.h"

DECLARE_GPU_DRAWCALL_STAT(FrontLayerTranslucencyGBuffer);

static constexpr int32 FRONT_LAYER_TILE_SIZE = 8;

// Whether to enable Front Layer Translucency reflections from scalability
int32 GLumenFrontLayerTranslucencyReflectionsEnabled = 0;
FAutoConsoleVariableRef CVarLumenTranslucencyReflectionsFrontLayerEnabled(
	TEXT("r.Lumen.TranslucencyReflections.FrontLayer.Enable"),
	GLumenFrontLayerTranslucencyReflectionsEnabled,
	TEXT("Whether to render Lumen Reflections on the frontmost layer of Translucent Surfaces.  Other layers will use the lower quality Radiance Cache method that can only produce glossy reflections."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

// Note: Driven by URendererSettings
static TAutoConsoleVariable<int32> CVarLumenFrontLayerTranslucencyReflectionsEnabledForProject(
	TEXT("r.Lumen.TranslucencyReflections.FrontLayer.EnableForProject"),
	0,
	TEXT("Whether to render Lumen Reflections on the frontmost layer of Translucent Surfaces.  Other layers will use the lower quality Radiance Cache method that can only produce glossy reflections."),
	ECVF_RenderThreadSafe
);

// Whether the user setting should be respected based on the current scalability level
int32 GLumenFrontLayerTranslucencyReflectionsAllowed = 1;
FAutoConsoleVariableRef CVarLumenTranslucencyReflectionsFrontLayerAllowed(
	TEXT("r.Lumen.TranslucencyReflections.FrontLayer.Allow"),
	GLumenFrontLayerTranslucencyReflectionsAllowed,
	TEXT("Whether to render Lumen Reflections on the frontmost layer of Translucent Surfaces.  Other layers will use the lower quality Radiance Cache method that can only produce glossy reflections."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarFrontLayerTranslucencyDepthThreshold(
	TEXT("r.FrontLayerTranslucency.DepthThreshold"),
	1024.0f,
	TEXT("Depth test threshold used to determine whether the fragments being rendered match the single layer of translucency for which lighting was calculated in a dedicated pass. In float ULP units."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarFrontLayerTranslucencyIndirectDispatch(
	TEXT("r.FrontLayerTranslucency.IndirectTileClassificationDispatch"),
	true,
	TEXT("Whether to use a compact tile list for the front-layer translucency tile classification pass:\n")
	TEXT("    0 - Dispatch over the full view rect (always processes all tiles)\n")
	TEXT("    1 - Build a compact tile list from the tile mask and dispatch only over valid tiles (default)")
	TEXT("(currently only supported by MegaLights)"),
	ECVF_RenderThreadSafe);

FAutoConsoleVariableDeprecated CVarLumenFrontLayerDepthThreshold(TEXT("r.Lumen.TranslucencyReflections.FrontLayer.DepthThreshold"), TEXT("r.FrontLayerTranslucency.DepthThreshold"), TEXT("5.8"));

bool IsVSMTranslucentHighQualityEnabled();

namespace Lumen
{
	bool UseFrontLayerTranslucencyReflections(const FViewInfo& View)
	{
		return (View.FinalPostProcessSettings.LumenFrontLayerTranslucencyReflections || GLumenFrontLayerTranslucencyReflectionsEnabled)
			&& GLumenFrontLayerTranslucencyReflectionsAllowed != 0 
			&& View.Family->EngineShowFlags.LumenReflections;
	}
}

namespace
{
	bool ShouldRenderInFrontLayerTranslucencyGBufferPass(bool bShouldRenderInMainPass, const FMaterial& Material)
	{
		return bShouldRenderInMainPass
			&& ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain())
			&& Material.IsTranslucencyWritingFrontLayerTransparency()
			&& !Material.IsTranslucencyAfterMotionBlurEnabled(); // see MTP_AfterMotionBlur
	}

	EPixelFormat GetFrontLayerTranslucencyGBufferAFormat()
	{
		return PF_A16B16G16R16; // Need more precision than PF_A2B10G10R10 for mirror reflections
	}

	EPixelFormat GetFrontLayerTranslucencyGBufferBFormat()
	{
		return PF_B8G8R8A8;
	}

	EPixelFormat GetFrontLayerTranslucencyGBufferCFormat()
	{
		return PF_R8G8;
	}
}

bool ShouldRenderFrontLayerTranslucency(const FViewInfo& View)
{
	return (ShouldRenderLumenReflections(View) && Lumen::UseFrontLayerTranslucencyReflections(View))
		|| RayTracedTranslucency::IsEnabled(View)
		|| MegaLights::UseFrontLayerTranslucencyDirectLighting(View)
		// TODO: Should check if VSM is actually enabled for this view
		|| IsVSMTranslucentHighQualityEnabled();
}

class FFrontLayerTranslucencyClearGBufferPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FFrontLayerTranslucencyClearGBufferPS);
	SHADER_USE_PARAMETER_STRUCT(FFrontLayerTranslucencyClearGBufferPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportLumenGI(Parameters.Platform)
			|| ShouldCompileMegaLightsShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		const bool bMegaLightsSupported = ShouldCompileMegaLightsShaders(Parameters.Platform);

		OutEnvironment.SetRenderTargetOutputFormat(0, GetFrontLayerTranslucencyGBufferAFormat());
		if (bMegaLightsSupported)
		{
			OutEnvironment.SetRenderTargetOutputFormat(1, GetFrontLayerTranslucencyGBufferBFormat());
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FFrontLayerTranslucencyClearGBufferPS, "/Engine/Private/FrontLayerTranslucency.usf", "FrontLayerTranslucencyClearGBufferPS", SF_Pixel);


BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FFrontLayerTranslucencyGBufferPassUniformParameters, )
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER(uint32, OutputExtendedGBuffer)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, RWMegaLightsTileMask)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FFrontLayerTranslucencyGBufferPassUniformParameters, "FrontLayerTranslucencyGBufferPass", SceneTextures);

class FFrontLayerTranslucencyGBufferVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FFrontLayerTranslucencyGBufferVS, MeshMaterial);

protected:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportLumenGI(Parameters.Platform) && IsTranslucentBlendMode(Parameters.MaterialParameters) && Parameters.MaterialParameters.bIsTranslucencySurface;
	}

	FFrontLayerTranslucencyGBufferVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}

	FFrontLayerTranslucencyGBufferVS() = default;
};


IMPLEMENT_MATERIAL_SHADER_TYPE(, FFrontLayerTranslucencyGBufferVS, TEXT("/Engine/Private/FrontLayerTranslucency.usf"), TEXT("MainVS"), SF_Vertex);

class FFrontLayerTranslucencyGBufferPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FFrontLayerTranslucencyGBufferPS, MeshMaterial);

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportLumenGI(Parameters.Platform) && IsTranslucentBlendMode(Parameters.MaterialParameters) && Parameters.MaterialParameters.bIsTranslucencySurface;
	}

	static void ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		const bool bMegaLightsSupported = ShouldCompileMegaLightsShaders(Parameters.Platform);

		OutEnvironment.SetRenderTargetOutputFormat(0, GetFrontLayerTranslucencyGBufferAFormat());
		if (bMegaLightsSupported)
		{
			OutEnvironment.SetRenderTargetOutputFormat(1, GetFrontLayerTranslucencyGBufferBFormat());
			OutEnvironment.SetRenderTargetOutputFormat(2, GetFrontLayerTranslucencyGBufferCFormat());
		}

		OutEnvironment.SetDefine(TEXT("FRONT_LAYER_TILE_SIZE"), FRONT_LAYER_TILE_SIZE);
	}

	FFrontLayerTranslucencyGBufferPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}

	FFrontLayerTranslucencyGBufferPS() = default;
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FFrontLayerTranslucencyGBufferPS, TEXT("/Engine/Private/FrontLayerTranslucency.usf"), TEXT("MainPS"), SF_Pixel);

class FFrontLayerTranslucencyGBufferMeshProcessor : public FSceneRenderingAllocatorObject<FFrontLayerTranslucencyGBufferMeshProcessor>, public FMeshPassProcessor
{
public:

	FFrontLayerTranslucencyGBufferMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

	FMeshPassProcessorRenderState PassDrawRenderState;
};

bool GetFrontLayerTranslucencyGBufferShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	TShaderRef<FFrontLayerTranslucencyGBufferVS>& VertexShader,
	TShaderRef<FFrontLayerTranslucencyGBufferPS>& PixelShader)
{
	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<FFrontLayerTranslucencyGBufferVS>();
	ShaderTypes.AddShaderType<FFrontLayerTranslucencyGBufferPS>();

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

bool CanMaterialRenderInFrontLayerTranslucencyGBufferPass(
	const FScene& Scene,
	const FSceneViewFamily& ViewFamily,
	const FPrimitiveSceneProxy& PrimitiveSceneProxy,
	const FMaterial& Material)
{
	const FSceneView* View = ViewFamily.Views[0];
	check(View);
	checkSlow(View->bIsViewInfo);

	return ShouldRenderFrontLayerTranslucency(*(const FViewInfo*)View) && ShouldRenderInFrontLayerTranslucencyGBufferPass(PrimitiveSceneProxy.ShouldRenderInMainPass(), Material);
}

void FFrontLayerTranslucencyGBufferMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	checkSlow(!ViewIfDynamicMeshCommand || ViewIfDynamicMeshCommand->bIsViewInfo);

	if (MeshBatch.bUseForMaterial && PrimitiveSceneProxy && ViewIfDynamicMeshCommand)
	{
		check(ShouldRenderFrontLayerTranslucency(*(const FViewInfo*)ViewIfDynamicMeshCommand));

		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material)
			{
				auto TryAddMeshBatch = [this](const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material) -> bool
				{
					if (ShouldRenderInFrontLayerTranslucencyGBufferPass(PrimitiveSceneProxy->ShouldRenderInMainPass(), Material))
					{
						const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;
						FVertexFactoryType* VertexFactoryType = VertexFactory->GetType();

						TMeshProcessorShaders<
							FFrontLayerTranslucencyGBufferVS,
							FFrontLayerTranslucencyGBufferPS> PassShaders;

						if (!GetFrontLayerTranslucencyGBufferShaders(
							Material,
							VertexFactory->GetType(),
							PassShaders.VertexShader,
							PassShaders.PixelShader))
						{
							return false;
						}

						FMeshMaterialShaderElementData ShaderElementData;
						ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

						const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
						const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
						const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
						const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);

						BuildMeshDrawCommands(
							MeshBatch,
							BatchElementMask,
							PrimitiveSceneProxy,
							MaterialRenderProxy,
							Material,
							PassDrawRenderState,
							PassShaders,
							MeshFillMode,
							MeshCullMode,
							SortKey,
							EMeshPassFeatures::Default,
							ShaderElementData);
					}

					return true;
				};

				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

void FFrontLayerTranslucencyGBufferMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{		
	if (PreCacheParams.bRenderInMainPass && !ShouldRenderInFrontLayerTranslucencyGBufferPass(PreCacheParams.bRenderInMainPass, Material))
	{
		return;
	}

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

	TMeshProcessorShaders<
		FFrontLayerTranslucencyGBufferVS,
		FFrontLayerTranslucencyGBufferPS> PassShaders;

	if (!GetFrontLayerTranslucencyGBufferShaders(
		Material,
		VertexFactoryData.VertexFactoryType,
		PassShaders.VertexShader,
		PassShaders.PixelShader))
	{
		return;
	}

	const bool bMegaLightsSupported = ShouldCompileMegaLightsShaders(SceneTexturesConfig.ShaderPlatform);

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;
	AddRenderTargetInfo(GetFrontLayerTranslucencyGBufferAFormat(), TexCreate_ShaderResource | TexCreate_RenderTargetable, RenderTargetsInfo);
	if (bMegaLightsSupported)
	{
		AddRenderTargetInfo(GetFrontLayerTranslucencyGBufferBFormat(), TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_SRGB, RenderTargetsInfo);
		AddRenderTargetInfo(GetFrontLayerTranslucencyGBufferCFormat(), TexCreate_ShaderResource | TexCreate_RenderTargetable, RenderTargetsInfo);
	}
	ETextureCreateFlags DepthStencilCreateFlags = SceneTexturesConfig.DepthCreateFlags;
	SetupDepthStencilInfo(PF_DepthStencil, DepthStencilCreateFlags, ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop, RenderTargetsInfo);

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		Material,
		PassDrawRenderState,
		RenderTargetsInfo,
		PassShaders,
		MeshFillMode,
		MeshCullMode,
		(EPrimitiveType)PreCacheParams.PrimitiveType,
		EMeshPassFeatures::Default,
		true /*bRequired*/,
		PSOInitializers);
}

FFrontLayerTranslucencyGBufferMeshProcessor::FFrontLayerTranslucencyGBufferMeshProcessor(const FScene* Scene,	ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::FrontLayerTranslucencyGBuffer, Scene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InPassDrawRenderState)
{}

FMeshPassProcessor* CreateFrontLayerTranslucencyGBufferPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassState;

	PassState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());

	PassState.SetBlendState(TStaticBlendState<>::GetRHI());

	return new FFrontLayerTranslucencyGBufferMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, PassState, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(FrontLayerTranslucencyGBufferPass, CreateFrontLayerTranslucencyGBufferPassProcessor, EShadingPath::Deferred, EMeshPass::FrontLayerTranslucencyGBuffer, EMeshPassFlags::MainView);

BEGIN_SHADER_PARAMETER_STRUCT(FFrontLayerTranslucencyGBufferPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFrontLayerTranslucencyGBufferPassUniformParameters, GBufferPass)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FBuildFrontLayerTileListCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildFrontLayerTileListCS)
	SHADER_USE_PARAMETER_STRUCT(FBuildFrontLayerTileListCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, FrontLayerTileMask)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWTileAllocator)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWTileData)
		SHADER_PARAMETER(FIntPoint, TileViewRectMin)
		SHADER_PARAMETER(FIntPoint, TileViewRectSize)
	END_SHADER_PARAMETER_STRUCT()

	static int32 GetGroupSize()
	{
		return 8;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileMegaLightsShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FBuildFrontLayerTileListCS, "/Engine/Private/FrontLayerTranslucency.usf", "BuildFrontLayerTileListCS", SF_Compute);

class FInitFrontLayerTileIndirectArgsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInitFrontLayerTileIndirectArgsCS)
	SHADER_USE_PARAMETER_STRUCT(FInitFrontLayerTileIndirectArgsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileAllocator)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWTileIndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileMegaLightsShaders(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FInitFrontLayerTileIndirectArgsCS, "/Engine/Private/FrontLayerTranslucency.usf", "InitFrontLayerTileIndirectArgsCS", SF_Compute);

static void BuildMegaLightsTileList(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SharedTileMask,
	FMegaLightsTileList& OutTileList)
{
	checkf((View.ViewRect.Min.X % FRONT_LAYER_TILE_SIZE) == 0 && (View.ViewRect.Min.Y % FRONT_LAYER_TILE_SIZE) == 0, TEXT("Viewport rect must be %d-pixel aligned."), FRONT_LAYER_TILE_SIZE);

	const FIntPoint TileViewRectMin = FIntPoint::DivideAndRoundDown(View.ViewRect.Min, FRONT_LAYER_TILE_SIZE);
	const FIntPoint TileViewRectMax = FIntPoint::DivideAndRoundUp(View.ViewRect.Max, FRONT_LAYER_TILE_SIZE);
	const FIntPoint TileViewRectSize = TileViewRectMax - TileViewRectMin;
	const int32 MaxTiles = TileViewRectSize.X * TileViewRectSize.Y;

	OutTileList.TileAllocator = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1),
		TEXT("FrontLayerTranslucency.MegaLightsTileAllocator"));

	OutTileList.TileData = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxTiles),
		TEXT("FrontLayerTranslucency.MegaLightsTileData"));

	OutTileList.TileIndirectArgs = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1),
		TEXT("FrontLayerTranslucency.MegaLightsTileIndirectArgs"));

	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutTileList.TileAllocator), 0u);

	{
		FBuildFrontLayerTileListCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildFrontLayerTileListCS::FParameters>();
		PassParameters->FrontLayerTileMask = SharedTileMask;
		PassParameters->RWTileAllocator = GraphBuilder.CreateUAV(OutTileList.TileAllocator);
		PassParameters->RWTileData = GraphBuilder.CreateUAV(OutTileList.TileData);
		PassParameters->TileViewRectMin = TileViewRectMin;
		PassParameters->TileViewRectSize = TileViewRectSize;

		auto ComputeShader = View.ShaderMap->GetShader<FBuildFrontLayerTileListCS>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("BuildFrontLayerMegaLightsTileList"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(TileViewRectSize, FBuildFrontLayerTileListCS::GetGroupSize()));
	}

	{
		FInitFrontLayerTileIndirectArgsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInitFrontLayerTileIndirectArgsCS::FParameters>();
		PassParameters->TileAllocator = GraphBuilder.CreateSRV(OutTileList.TileAllocator);
		PassParameters->RWTileIndirectArgs = GraphBuilder.CreateUAV(OutTileList.TileIndirectArgs);

		auto ComputeShader = View.ShaderMap->GetShader<FInitFrontLayerTileIndirectArgsCS>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("InitFrontLayerMegaLightsTileIndirectArgs"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1));
	}
}

// -----------------------------------------

void RenderFrontLayerTranslucencyGBuffer(
	FRDGBuilder& GraphBuilder,
	const FSceneRenderer& SceneRenderer,
	FViewInfo& View,
	const FSceneTextures& SceneTextures,
	const FFrontLayerTranslucencyData& FrontLayerTranslucencyData,
	bool bOutputExtendedGBuffer)
{
	// TODO: Implement Nanite support

	const EMeshPass::Type MeshPass = EMeshPass::FrontLayerTranslucencyGBuffer;
	const float ViewportScale = 1.0f;
	FIntRect GBufferViewRect = GetScaledRect(View.ViewRect, ViewportScale);
	auto* Pass = View.ParallelMeshDrawCommandPasses[MeshPass];

	if (!Pass)
	{
		return;
	}

	checkf((GBufferViewRect.Min.X % FRONT_LAYER_TILE_SIZE) == 0 && (GBufferViewRect.Min.Y % FRONT_LAYER_TILE_SIZE) == 0, TEXT("Viewport rect must be %d-pixel aligned."), FRONT_LAYER_TILE_SIZE);

	const bool bMegaLightsSupported = ShouldCompileMegaLightsShaders(SceneRenderer.ShaderPlatform);

	View.BeginRenderView();

	FFrontLayerTranslucencyGBufferPassParameters* PassParameters = GraphBuilder.AllocParameters<FFrontLayerTranslucencyGBufferPassParameters>();

	PassParameters->RenderTargets[0] = FRenderTargetBinding(FrontLayerTranslucencyData.GBufferA, ERenderTargetLoadAction::ELoad, 0);
	if (bMegaLightsSupported)
	{
		PassParameters->RenderTargets[1] = FRenderTargetBinding(FrontLayerTranslucencyData.GBufferB, ERenderTargetLoadAction::ELoad, 0);
		PassParameters->RenderTargets[2] = FRenderTargetBinding(FrontLayerTranslucencyData.GBufferC, ERenderTargetLoadAction::ELoad, 0);
	}
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(FrontLayerTranslucencyData.SceneDepth, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilNop);

	{
		FViewUniformShaderParameters DownsampledTranslucencyViewParameters = *View.CachedViewUniformShaderParameters;

		FViewMatrices ViewMatrices = View.ViewMatrices;
		FViewMatrices PrevViewMatrices = View.PrevViewInfo.ViewMatrices;

		// Update the parts of DownsampledTranslucencyParameters which are dependent on the buffer size and view rect
		View.SetupViewRectUniformBufferParameters(
			DownsampledTranslucencyViewParameters,
			SceneTextures.Config.Extent,
			GBufferViewRect,
			ViewMatrices,
			PrevViewMatrices);

		PassParameters->View.View = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(DownsampledTranslucencyViewParameters, UniformBuffer_SingleFrame);

		if (View.bShouldBindInstancedViewUB)
		{
			FInstancedViewUniformShaderParameters LocalInstancedViewUniformShaderParameters;
			InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, DownsampledTranslucencyViewParameters, 0);

			if (const FViewInfo* InstancedView = View.GetInstancedView())
			{
				InstancedView->SetupViewRectUniformBufferParameters(
					DownsampledTranslucencyViewParameters,
					SceneTextures.Config.Extent,
					GetScaledRect(InstancedView->ViewRect, ViewportScale),
					ViewMatrices,
					PrevViewMatrices);

				InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, DownsampledTranslucencyViewParameters, 1);
			}

			PassParameters->View.InstancedView = TUniformBufferRef<FInstancedViewUniformShaderParameters>::CreateUniformBufferImmediate(
				reinterpret_cast<const FInstancedViewUniformShaderParameters&>(LocalInstancedViewUniformShaderParameters),
				UniformBuffer_SingleFrame);
		}
	}

	{
		FFrontLayerTranslucencyGBufferPassUniformParameters& GBufferPassParameters = *GraphBuilder.AllocParameters<FFrontLayerTranslucencyGBufferPassUniformParameters>();
		SetupSceneTextureUniformParameters(GraphBuilder, &SceneTextures, View.FeatureLevel, ESceneTextureSetupMode::All, GBufferPassParameters.SceneTextures);
		GBufferPassParameters.OutputExtendedGBuffer = bOutputExtendedGBuffer ? 1 : 0;

		// Bind the tile mask UAV through the uniform buffer (matching the LumenTranslucencyRadianceCacheMark pattern).
		// The shader only writes to it under #if PROJECT_SUPPORTS_MEGALIGHTS, so a placeholder suffices otherwise.
		FRDGTextureUAVRef TileMaskUAV = (bOutputExtendedGBuffer && FrontLayerTranslucencyData.MegaLightsTileMask)
			? GraphBuilder.CreateUAV(FrontLayerTranslucencyData.MegaLightsTileMask)
			: nullptr;
		if (!TileMaskUAV)
		{
			FRDGTextureRef PlaceholderTileMask = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_R8_UINT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV),
				TEXT("FrontLayerTranslucency.MegaLightsTileMask.Placeholder"));
			TileMaskUAV = GraphBuilder.CreateUAV(PlaceholderTileMask);
		}
		GBufferPassParameters.RWMegaLightsTileMask = TileMaskUAV;

		PassParameters->GBufferPass = GraphBuilder.CreateUniformBuffer(&GBufferPassParameters);
	}

	Pass->BuildRenderingCommands(GraphBuilder, SceneRenderer.Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("TranslucencyGBuffer"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, Pass, &SceneRenderer, MeshPass, PassParameters, ViewportScale, GBufferViewRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FSceneRenderer::SetStereoViewport(RHICmdList, View, ViewportScale);
			Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
		});
}

bool FDeferredShadingSceneRenderer::IsLumenFrontLayerTranslucencyEnabled(const FViewInfo& View) const
{ 
	return View.ParallelMeshDrawCommandPasses[EMeshPass::FrontLayerTranslucencyGBuffer] != nullptr
		&& ((GetViewPipelineState(View).ReflectionsMethod == EReflectionsMethod::Lumen
		&& Lumen::UseFrontLayerTranslucencyReflections(View))
		|| RayTracedTranslucency::IsEnabled(View));
}

bool FDeferredShadingSceneRenderer::IsMegaLightsFrontLayerTranslucencyEnabled(const FViewInfo& View) const
{
	return View.ParallelMeshDrawCommandPasses[EMeshPass::FrontLayerTranslucencyGBuffer] != nullptr
		&& MegaLights::UseFrontLayerTranslucencyDirectLighting(View);
}

bool IsFrontLayerHistoryValid(const FViewInfo& View)
{ 
	return View.ViewState && 
		View.ViewState->PrevFrameNumber == View.ViewState->StochasticLighting.HistoryFrameIndex &&
		View.ViewState->StochasticLighting.FrontLayerTranslucencyDepthHistory != nullptr &&
		View.ViewState->StochasticLighting.FrontLayerTranslucencyNormalHistory != nullptr;
}

FFrontLayerTranslucencyData FDeferredShadingSceneRenderer::RenderFrontLayerTranslucency(
	FRDGBuilder& GraphBuilder,
	TArray<FViewInfo>& InViews,
	const FSceneTextures& SceneTextures,
	const TSharedPtr<FMegaLightsFrameTemporaries>& MegaLightsFrameTemporaries,
	bool bRenderOnlyForVSMPageMarking)
{
	// TODO: Implement Nanite support

	// bNeedFrontLayerData:
	// 0 : No front layer
	// 1 : Front layer data requested by Lumen reflection
	// 2 : Front layer data requested by VSM page marking
	uint32 bNeedFrontLayerData = 0;
	if (bRenderOnlyForVSMPageMarking)
	{
		if (IsVSMTranslucentHighQualityEnabled())
		{
			// Front layer translucency data can be used from different sources (in priority order):
			// * Lumen/MegaLights front layer History data
			// * Skipped if Lumen/MegaLights will render front layer the same frame (as Translucent 'after opaque' might be rendered, e.g. particle, and if we render the front layer here, they won't be present)
			// * Render local front layer data here
			bool bValidHistory = false;
			bool bWillLumenOrMegaLightsRenderFrontLayer = false;
			bool bAnyViewHasFrontLayerDraws = false;
			for (FViewInfo& View : InViews)
			{
				bValidHistory = bValidHistory || IsFrontLayerHistoryValid(View);
				bWillLumenOrMegaLightsRenderFrontLayer |= IsLumenFrontLayerTranslucencyEnabled(View) || IsMegaLightsFrontLayerTranslucencyEnabled(View);
				bAnyViewHasFrontLayerDraws |= View.ParallelMeshDrawCommandPasses[EMeshPass::FrontLayerTranslucencyGBuffer] != nullptr;
			}

			if (bAnyViewHasFrontLayerDraws && !bValidHistory && !bWillLumenOrMegaLightsRenderFrontLayer)
			{
				bNeedFrontLayerData = 2;
			}
		}
	}
	else
	{
		// Check if any view require translucent front layer data
		for (const FViewInfo& View : InViews)
		{
			if (IsLumenFrontLayerTranslucencyEnabled(View) || IsMegaLightsFrontLayerTranslucencyEnabled(View))
			{
				bNeedFrontLayerData = 1;
				break;
			}	
		}
	}

	FFrontLayerTranslucencyData Out;
	if (bNeedFrontLayerData > 0)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, FrontLayerTranslucencyGBuffer, "FrontLayerTranslucencyGBuffer");

#if DO_CHECK
		bool bAnyViewRendersPass = false;
		for (const FViewInfo& View : InViews)
		{
			if (View.ParallelMeshDrawCommandPasses[EMeshPass::FrontLayerTranslucencyGBuffer] != nullptr)
			{
				bAnyViewRendersPass = true;
				break;
			}
		}

		ensureMsgf(bAnyViewRendersPass, TEXT("FrontLayerTranslucencyGBuffer resources should only be created if any view will render in this pass."));
#endif // DO_CHECK

		const bool bMegaLightsSupported = ShouldCompileMegaLightsShaders(ShaderPlatform);

		// Allocate resources for all views
		{
			{
				EPixelFormat PixelFormat = GetFrontLayerTranslucencyGBufferAFormat();
				FRDGTextureDesc TextureDesc(FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PixelFormat, FClearValueBinding::Transparent, TexCreate_ShaderResource | TexCreate_RenderTargetable));
				Out.GBufferA = GraphBuilder.CreateTexture(TextureDesc, TEXT("FrontLayerTranslucency.GBufferA"));
			}
			
			if (bMegaLightsSupported)
			{
				{
					EPixelFormat PixelFormat = GetFrontLayerTranslucencyGBufferBFormat();
					FRDGTextureDesc TextureDesc(FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PixelFormat, FClearValueBinding::Transparent, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_SRGB));
					Out.GBufferB = GraphBuilder.CreateTexture(TextureDesc, TEXT("FrontLayerTranslucency.GBufferB"));
				}

				{
					EPixelFormat PixelFormat = GetFrontLayerTranslucencyGBufferCFormat();
					FRDGTextureDesc TextureDesc(FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PixelFormat, FClearValueBinding::Transparent, TexCreate_ShaderResource | TexCreate_RenderTargetable));
					Out.GBufferC = GraphBuilder.CreateTexture(TextureDesc, TEXT("FrontLayerTranslucency.GBufferC"));
				}

				if(CVarFrontLayerTranslucencyIndirectDispatch.GetValueOnRenderThread())
				{
					const FIntPoint TileGridSize = FIntPoint::DivideAndRoundUp(SceneTextures.Config.Extent, FRONT_LAYER_TILE_SIZE);
					FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(TileGridSize, PF_R8_UINT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV);
					Out.MegaLightsTileMask = GraphBuilder.CreateTexture(TextureDesc, TEXT("FrontLayerTranslucency.MegaLightsTileMask"));
					
					AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Out.MegaLightsTileMask), 0u);
				}
			}

			Out.SceneDepth = GraphBuilder.CreateTexture(SceneTextures.Depth.Target->Desc, TEXT("FrontLayerTranslucency.SceneDepth"));
		}

		// Clear ViewRect for each view
		{
			TArray<FIntRect, SceneRenderingAllocator>& ViewRectsToClear = GraphBuilder.AllocArray<FIntRect>();
			ViewRectsToClear.Reserve(InViews.Num());

			for (const FViewInfo& View : InViews)
			{
				if (bNeedFrontLayerData > 1 || IsLumenFrontLayerTranslucencyEnabled(View) || IsMegaLightsFrontLayerTranslucencyEnabled(View))
				{
					ViewRectsToClear.Add(View.ViewRect);
				}
			}

			if (!ViewRectsToClear.IsEmpty())
			{
				const FGlobalShaderMap* GlobalShaderMap = InViews[0].ShaderMap;

				// need to clear Depth and GBufferA/B since they contain valid flags
				// don't need to clear GBuffer C since it's only read when the pixel flags on the other buffers are valid

				FFrontLayerTranslucencyClearGBufferPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFrontLayerTranslucencyClearGBufferPS::FParameters>();
				PassParameters->RenderTargets[0] = FRenderTargetBinding(Out.GBufferA, ERenderTargetLoadAction::ENoAction, 0);
				if (bMegaLightsSupported)
				{
					PassParameters->RenderTargets[1] = FRenderTargetBinding(Out.GBufferB, ERenderTargetLoadAction::ENoAction, 0);
				}
				PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Out.SceneDepth, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
				PassParameters->SceneTexturesStruct = SceneTextures.UniformBuffer;

				auto PixelShader = GlobalShaderMap->GetShader<FFrontLayerTranslucencyClearGBufferPS>();

				ClearUnusedGraphResources(PixelShader, PassParameters);

				// Not using FPixelShaderUtils::AddFullscreenPass because want to clear multiple view rects in a single pass
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("ClearTranslucencyGBuffer"),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, GlobalShaderMap, PixelShader, &ViewRectsToClear](FRDGAsyncTask, FRHICommandList& RHICmdList)
					{
						for (const FIntRect& ViewRect : ViewRectsToClear)
						{
							FPixelShaderUtils::DrawFullscreenPixelShader(
								RHICmdList,
								GlobalShaderMap,
								PixelShader,
								*PassParameters,
								ViewRect,
								TStaticBlendState<>::GetRHI(),
								TStaticRasterizerState<FM_Solid, CM_None>::GetRHI(),
								TStaticDepthStencilState<true, CF_Always>::GetRHI());
						}
					});
			}
		}

		// Render front layer data for each view
		for (FViewInfo& View : InViews)
		{
			const bool bIsLumenFrontLayerTranslucencyEnabled = IsLumenFrontLayerTranslucencyEnabled(View);
			const bool bIsMegaLightsFrontLayerTranslucencyEnabled = IsMegaLightsFrontLayerTranslucencyEnabled(View);
			const bool bViewNeedFrontLayer = bNeedFrontLayerData > 1 || bIsLumenFrontLayerTranslucencyEnabled || bIsMegaLightsFrontLayerTranslucencyEnabled;

			if (bViewNeedFrontLayer)
			{
				const bool bOutputExtendedGBuffer = bIsMegaLightsFrontLayerTranslucencyEnabled;
				RenderFrontLayerTranslucencyGBuffer(GraphBuilder, *this, View, SceneTextures, Out, bOutputExtendedGBuffer);
			}

			View.FrontLayerTranslucency.GBufferA = Out.GBufferA;
			View.FrontLayerTranslucency.GBufferB = Out.GBufferB;
			View.FrontLayerTranslucency.GBufferC = Out.GBufferC;
			View.FrontLayerTranslucency.SceneDepth = Out.SceneDepth;

			View.FrontLayerTranslucency.RelativeDepthThreshold = CVarFrontLayerTranslucencyDepthThreshold.GetValueOnRenderThread();
		}

		// Build a per-view compact tile list from the shared tile mask for use as indirect dispatch args in TileClassificationMarkCS.
		if (Out.MegaLightsTileMask)
		{
			Out.PerViewMegaLightsTileLists.SetNum(InViews.Num()); // null-initialized
			for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
			{
				const FViewInfo& View = InViews[ViewIndex];
				const bool bIsMegaLightsFrontLayerEnabled = IsMegaLightsFrontLayerTranslucencyEnabled(View);
				if (bIsMegaLightsFrontLayerEnabled)
				{
					BuildMegaLightsTileList(GraphBuilder, View, Out.MegaLightsTileMask, Out.PerViewMegaLightsTileLists[ViewIndex]);
				}
			}
		}
	}
	return Out;
}

void FDeferredShadingSceneRenderer::RenderLumenFrontLayerTranslucencyReflections(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FLumenSceneFrameTemporaries& LumenFrameTemporaries, 
	const FFrontLayerTranslucencyData& FrontLayerTranslucencyData)
{
	if (!RayTracedTranslucency::IsEnabled(View))
	{
		check(FrontLayerTranslucencyData.IsValid());

		RDG_EVENT_SCOPE(GraphBuilder, "LumenFrontLayerTranslucencyReflections");

		FLumenMeshSDFGridParameters MeshSDFGridParameters;
		LumenRadianceCache::FRadianceCacheInterpolationParameters RadianceCacheParameters;

		FLumenReflectionsConfig LumenReflectionsConfig;
		LumenReflectionsConfig.FrontLayerTranslucencyData = &FrontLayerTranslucencyData;

		FRDGTextureRef ReflectionTexture = RenderLumenReflections(
			GraphBuilder,
			View,
			SceneTextures,
			LumenFrameTemporaries,
			MeshSDFGridParameters,
			RadianceCacheParameters,
			ELumenReflectionPass::FrontLayerTranslucency,
			LumenReflectionsConfig,
			ERDGPassFlags::Compute);

		View.FrontLayerTranslucency.bReflectionEnabled = true;
		View.FrontLayerTranslucency.Reflection = ReflectionTexture;

		extern float GetLumenReflectionSpecularScale();
		extern float GetLumenReflectionContrast();
		View.FrontLayerTranslucency.SpecularScale = GetLumenReflectionSpecularScale();
		View.FrontLayerTranslucency.Contrast = GetLumenReflectionContrast();
	}
}

FFrontLayerTranslucencyGBufferParameters GetFrontLayerTranslucencyGBufferParameters(const FFrontLayerTranslucencyData& FrontLayerTranslucencyData)
{
	FFrontLayerTranslucencyGBufferParameters Parameters;
	Parameters.FrontLayerTranslucencySceneDepth = FrontLayerTranslucencyData.SceneDepth;
	Parameters.FrontLayerTranslucencyGBufferA = FrontLayerTranslucencyData.GBufferA;
	Parameters.FrontLayerTranslucencyGBufferB = FrontLayerTranslucencyData.GBufferB;
	Parameters.FrontLayerTranslucencyGBufferC = FrontLayerTranslucencyData.GBufferC;

	return Parameters;
}

FFrontLayerTranslucencyParameters GetFrontLayerTranslucencyParameters(
	FRDGBuilder& GraphBuilder,
	const FFrontLayerTranslucency& FrontLayerTranslucency
)
{
	FFrontLayerTranslucencyParameters Parameters;
	Parameters.ReflectionEnabled = FrontLayerTranslucency.bReflectionEnabled ? 1 : 0;
	Parameters.DirectLightingEnabled = FrontLayerTranslucency.bDirectLightingEnabled ? 1 : 0;
	Parameters.SpecularOnly = FrontLayerTranslucency.bSpecularOnly ? 1 : 0;
	Parameters.RelativeDepthThreshold = FrontLayerTranslucency.RelativeDepthThreshold;
	Parameters.Reflection = OrBlack2DArrayIfNull(GraphBuilder, FrontLayerTranslucency.Reflection);
	Parameters.DirectLighting = OrBlack2DArrayIfNull(GraphBuilder, FrontLayerTranslucency.DirectLighting);
	Parameters.GBufferA = OrBlack2DIfNull(GraphBuilder, FrontLayerTranslucency.GBufferA);
	Parameters.GBufferB = OrBlack2DIfNull(GraphBuilder, FrontLayerTranslucency.GBufferB);
	Parameters.GBufferC = OrBlack2DIfNull(GraphBuilder, FrontLayerTranslucency.GBufferC);
	Parameters.SceneDepth = OrBlack2DIfNull(GraphBuilder, FrontLayerTranslucency.SceneDepth);
	Parameters.SpecularScale = FrontLayerTranslucency.SpecularScale;
	Parameters.Contrast = FrontLayerTranslucency.Contrast;

	return Parameters;
}
