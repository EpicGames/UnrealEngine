// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StochasticLighting/StochasticLighting.h"
#include "LumenDefinitions.h"
#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"

class FScene;
class FPrimitiveSceneProxy;

// Per-view compact tile list produced from MegaLightsTileMask for indirect dispatch.
struct FMegaLightsTileList
{
	FRDGBufferRef TileAllocator = nullptr;
	FRDGBufferRef TileData = nullptr;
	FRDGBufferRef TileIndirectArgs = nullptr;
};

struct FFrontLayerTranslucencyData
{
	bool IsValid() const { return SceneDepth != nullptr; }
	FRDGTextureRef SceneDepth = nullptr;
	FRDGTextureRef GBufferA = nullptr;
	FRDGTextureRef GBufferB = nullptr;
	FRDGTextureRef GBufferC = nullptr;

	// Single shared tile mask written by all views (each into their own non-overlapping rect).
	FRDGTextureRef MegaLightsTileMask = nullptr;

	// Per-view compact tile lists built from MegaLightsTileMask; indexed by view index.
	// Entry is null for views that do not use MegaLights front-layer direct lighting.
	TArray<FMegaLightsTileList, TInlineAllocator<2>> PerViewMegaLightsTileLists;

	StochasticLighting::FFrameTemporaries StochasticLighting;
};

BEGIN_SHADER_PARAMETER_STRUCT(FFrontLayerTranslucencyGBufferParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FrontLayerTranslucencyGBufferA)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FrontLayerTranslucencyGBufferB)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FrontLayerTranslucencyGBufferC)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FrontLayerTranslucencySceneDepth)
END_SHADER_PARAMETER_STRUCT()

FFrontLayerTranslucencyGBufferParameters GetFrontLayerTranslucencyGBufferParameters(const FFrontLayerTranslucencyData& FrontLayerTranslucencyData);

class FFrontLayerTranslucency
{
public:
	FRDGTextureRef Reflection = nullptr;
	FRDGTextureRef DirectLighting = nullptr;
	FRDGTextureRef GBufferA = nullptr;
	FRDGTextureRef GBufferB = nullptr;
	FRDGTextureRef GBufferC = nullptr;
	FRDGTextureRef SceneDepth = nullptr;
	bool bReflectionEnabled = false;
	bool bDirectLightingEnabled = false;
	bool bSpecularOnly = false;
	float RelativeDepthThreshold = 0.0f;
	float SpecularScale = 0.0f;
	float Contrast = 0.0f;
};

// Used by Translucency Base Pass
BEGIN_SHADER_PARAMETER_STRUCT(FFrontLayerTranslucencyParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, Reflection)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DirectLighting)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferA)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferB)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferC)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepth)
	SHADER_PARAMETER(uint32, ReflectionEnabled)
	SHADER_PARAMETER(uint32, DirectLightingEnabled)
	SHADER_PARAMETER(uint32, SpecularOnly)
	SHADER_PARAMETER(float, RelativeDepthThreshold)
	SHADER_PARAMETER(float, SpecularScale)
	SHADER_PARAMETER(float, Contrast)
END_SHADER_PARAMETER_STRUCT()

FFrontLayerTranslucencyParameters GetFrontLayerTranslucencyParameters(
	FRDGBuilder& GraphBuilder,
	const FFrontLayerTranslucency& FrontLayerTranslucency
);

extern bool CanMaterialRenderInFrontLayerTranslucencyGBufferPass(
	const FScene& Scene,
	const FSceneViewFamily& ViewFamily,
	const FPrimitiveSceneProxy& PrimitiveSceneProxy,
	const FMaterial& Material);

bool ShouldRenderFrontLayerTranslucency(const FViewInfo& View);