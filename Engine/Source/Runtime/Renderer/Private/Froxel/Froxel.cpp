// Copyright Epic Games, Inc. All Rights Reserved.

#include "Froxel.h"
#include "RenderGraphUtils.h"
#include "RendererPrivateUtils.h"
#include "SceneRendering.h"

namespace Froxel
{

FSharedParameters MakeSharedParameters(const FViewInfo& View)
{
	const float FroxelTileSize = float(FRenderer::TileSize);

	FVector2f ViewSize = FVector2f(View.ViewRect.Size());// FVector2f(View.ViewSizeAndInvSize.X, View.ViewSizeAndInvSize.Y);
				
	// How cubeish the froxels should be
	const float DepthStretchFactor = 1.0f;

	const FMatrix& ViewToClip = View.ViewMatrices.GetViewToClip();
	// 2x for, er, probably because clipspace is +-1
	FVector2f AbsClipTileSize = FVector2f(FroxelTileSize * 2.0f) / ViewSize;
	FVector2f ProjScaleXY = FVector2f(static_cast<float>(ViewToClip.M[0][0]), static_cast<float>(ViewToClip.M[1][1]));
	FVector2f RadiusXY = AbsClipTileSize / ProjScaleXY;
	float RadiusScreen = FMath::Min(RadiusXY.X, RadiusXY.Y) * DepthStretchFactor; 


	const bool bIsOrtho = !View.ViewMatrices.IsPerspectiveProjection();

	const float FroxelNear = static_cast<float>(ViewToClip.M[3][3] - ViewToClip.M[3][2]) / (ViewToClip.M[2][2] - ViewToClip.M[2][3]);
	float FarPlane = FroxelNear - 1.0f / static_cast<float>(ViewToClip.M[2][2]);

	float DepthScale = ViewToClip.M[2][3] * RadiusScreen; // + ViewToClip[3].W ignore the ortho fix unsure how to change, needs to somehow move outside the log, probably does factor out
	FSharedParameters FroxelParameters;
	FroxelParameters.FroxelNear = FroxelNear;
	FroxelParameters.FroxelDepthScale1 = DepthScale + 1.0f;
	FroxelParameters.FroxelRecNearScale = 1.0f / FroxelNear;
	FroxelParameters.FroxelRecLog2DepthScale1 = 1.0f / FMath::Log2(1.0f + DepthScale);
	FroxelParameters.FroxelRadius = RadiusScreen;
	FroxelParameters.FroxelInvRadius = 1.0f / RadiusScreen;
	FroxelParameters.bFroxelIsOrtho = bIsOrtho;

	// needs to be modified per view if/when moved to shared buffer.
	FroxelParameters.FroxelArgsStride = FRenderer::ArgsStride;
	FroxelParameters.FroxelArgsOffset = 0;

	if (bIsOrtho)
	{
		FroxelParameters.FroxelViewToClipTransformScale = ViewToClip.M[2][2];
		FroxelParameters.FroxelClipToViewTransformScale = 1.0f / ViewToClip.M[2][2];
		FroxelParameters.FroxelClipToViewTransformBias = -ViewToClip.M[3][2] / ViewToClip.M[2][2]; 
	}
	else
	{
		FroxelParameters.FroxelViewToClipTransformScale = ViewToClip.M[3][2];
		FroxelParameters.FroxelClipToViewTransformScale = 1.0f / ViewToClip.M[3][2];
		FroxelParameters.FroxelClipToViewTransformBias = -ViewToClip.M[2][2] / ViewToClip.M[3][2]; 
	}

	const FVector2f ClipTileSize = FVector2f(AbsClipTileSize.X, -AbsClipTileSize.Y);
	const FVector2f ClipSpaceMin = FVector2f(-1.0f, 1.0f);

	const FMatrix& ClipToView = View.ViewMatrices.GetClipToView();
	const FVector2f ClipToViewScale = FVector2f(ClipToView.M[0][0], ClipToView.M[1][1]);
	FVector2f FroxelToViewScale = ClipTileSize * ClipToViewScale;
	FVector2f FroxelToViewBias = ClipSpaceMin * ClipToViewScale;
	FroxelParameters.FroxelToViewScaleBias = FVector4f(FroxelToViewScale, FroxelToViewBias);

	FroxelParameters.FroxelToClipScaleBias = FVector4f(ClipTileSize, ClipSpaceMin);
	FroxelParameters.FroxelClipToViewScale = ClipToViewScale;

	return FroxelParameters;
}

FRenderer::FRenderer(bool bIsEnabled, FRDGBuilder& GraphBuilder, const TArray<FViewInfo>& InViews)
{
	if (!bIsEnabled)
	{
		return;
	}

	for (const FViewInfo& View : InViews)
	{
		FSharedParameters FroxelParameters = MakeSharedParameters(View);

		// TODO: over-conservative(!) 
		int32 MaxNumFroxels = View.ViewRect.Area();
		FRDGBuffer* FroxelsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FPackedFroxel), MaxNumFroxels), TEXT("r.Froxels"));
		// TODO: make shared buffer & initialize once
		FRDGBuffer* FroxelArgsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(4), TEXT("r.FroxelArgs"));
		AddClearIndirectDispatchArgs1DPass(GraphBuilder, View.GetFeatureLevel(), FroxelArgsRDG, 1u, ArgsStride);

		Views.Emplace(FViewData{this, FroxelsRDG, FroxelArgsRDG, FroxelParameters});
	}
}

FBuilderParameters FViewData::GetBuilderParameters(FRDGBuilder& GraphBuilder) const
{
	FBuilderParameters BuilderParameters;
	BuilderParameters.FroxelParameters = SharedParameters;
	BuilderParameters.OutFroxels = GraphBuilder.CreateUAV(FroxelsRDG);
	BuilderParameters.OutFroxelArgs = GraphBuilder.CreateUAV(FroxelArgsRDG);
	return BuilderParameters;
}

FParameters FViewData::GetShaderParameters(FRDGBuilder& GraphBuilder) const
{
	FParameters Parameters;
	Parameters.FroxelParameters = SharedParameters;
	Parameters.Froxels = GraphBuilder.CreateSRV(FroxelsRDG);
	Parameters.FroxelArgs = GraphBuilder.CreateSRV(FroxelArgsRDG);
	return Parameters;
}

}