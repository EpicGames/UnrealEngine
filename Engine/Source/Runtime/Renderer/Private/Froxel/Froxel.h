// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameterMacros.h"
#include "FroxelDefinitions.h"

class FRDGBuilder;
class FViewInfo;

/**
 * Functions and data structures to manage lists of froxels that represent some geometry, e.g., the depth buffer.
 * The froxels are spaced such that they are as deep as they are wide (at the near plane of the froxel slice). This means they provide good bounds for the samples they represent.
 */
namespace Froxel
{

BEGIN_SHADER_PARAMETER_STRUCT(FSharedParameters, )
	SHADER_PARAMETER(FVector4f, FroxelToViewScaleBias)
	SHADER_PARAMETER(FVector4f, FroxelToClipScaleBias)
	SHADER_PARAMETER(FVector2f, FroxelClipToViewScale)
	SHADER_PARAMETER(float, FroxelRecLog2DepthScale1)
	SHADER_PARAMETER(float, FroxelRecNearScale)
	SHADER_PARAMETER(float, FroxelDepthScale1)
	SHADER_PARAMETER(float, FroxelNear)
	SHADER_PARAMETER(float, FroxelViewToClipTransformScale)
	SHADER_PARAMETER(float, FroxelClipToViewTransformScale)
	SHADER_PARAMETER(float, FroxelClipToViewTransformBias)
	SHADER_PARAMETER(float, FroxelRadius)
	SHADER_PARAMETER(float, FroxelInvRadius)
	SHADER_PARAMETER(uint32, bFroxelIsOrtho)
	SHADER_PARAMETER(uint32, FroxelArgsOffset)
	SHADER_PARAMETER(uint32, FroxelArgsStride)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FBuilderParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FSharedParameters, FroxelParameters)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer< FPackedFroxel >, OutFroxels)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer< uint >, OutFroxelArgs)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FSharedParameters, FroxelParameters)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedFroxel >, Froxels)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer< uint >, FroxelArgs)
END_SHADER_PARAMETER_STRUCT()


FSharedParameters MakeSharedParameters(const FViewInfo &View);

// per view froxel data.
struct FViewData
{
	class FRenderer *FroxelData = nullptr;
	FRDGBuffer* FroxelsRDG = nullptr;
	// Represents an argument to use with an indirect dispacth to perform some processing on the froxels. 
	FRDGBuffer* FroxelArgsRDG = nullptr;
	FSharedParameters SharedParameters;
	int32 ArgsOffset = 0;

	FBuilderParameters GetBuilderParameters(FRDGBuilder& GraphBuilder) const;

	FParameters GetShaderParameters(FRDGBuilder& GraphBuilder) const;

};

class FRenderer
{
public:
	// Stride in the indirect argument buffer, the 4th slot is used to store the atomic counter of individual froxels (rather than the group).
	static constexpr int32 ArgsStride = 4;
	// Work group size that should be used on an indirect dispatch using the argument produced
	static constexpr int32 IndirectWorkGroupSize = FROXEL_INDIRECT_ARG_WORKGROUP_SIZE;
	static constexpr int32 TileSize = FROXEL_TILE_SIZE; // 8x8 tiles

	FRenderer() {}
	FRenderer(bool bIsEnabled, FRDGBuilder& GraphBuilder, const TArray<FViewInfo>& Views);

	const FViewData *GetView(int32 ViewIndex) const { return Views.IsValidIndex(ViewIndex) ? &Views[ViewIndex] : nullptr; }

	bool IsEnabled() const { return !Views.IsEmpty(); }

private:
	TArray<FViewData> Views;
};

} // namespace froxel
