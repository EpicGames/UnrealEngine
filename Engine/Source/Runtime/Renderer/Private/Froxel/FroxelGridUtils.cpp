// Copyright Epic Games, Inc. All Rights Reserved.

#include "FroxelGridUtils.h"
#include "RHIGlobals.h"

FVector2f GetFroxelGridUVMaxForSampling(const FVector2f& ViewRectSize, FIntVector ResourceGridSize, int32 ResourceGridPixelSize)
{
	float ViewRectSizeXSafe = FMath::DivideAndRoundUp<int32>(int32(ViewRectSize.X), ResourceGridPixelSize) * ResourceGridPixelSize - (ResourceGridPixelSize / 2 + 1);
	float ViewRectSizeYSafe = FMath::DivideAndRoundUp<int32>(int32(ViewRectSize.Y), ResourceGridPixelSize) * ResourceGridPixelSize - (ResourceGridPixelSize / 2 + 1);
	return FVector2f(ViewRectSizeXSafe, ViewRectSizeYSafe) / (FVector2f(ResourceGridSize.X, ResourceGridSize.Y) * ResourceGridPixelSize);
}

FVector2f GetFroxelGridPrevUVMaxForTemporalBlend(const FVector2f& ViewRectSize, FIntVector ResourceGridSize, int32 ResourceGridPixelSize)
{
	float ViewRectSizeXSafe = FMath::DivideAndRoundUp<int32>(int32(ViewRectSize.X), ResourceGridPixelSize) * ResourceGridPixelSize;
	float ViewRectSizeYSafe = FMath::DivideAndRoundUp<int32>(int32(ViewRectSize.Y), ResourceGridPixelSize) * ResourceGridPixelSize;
	return FVector2f(ViewRectSizeXSafe, ViewRectSizeYSafe) / (FVector2f(ResourceGridSize.X, ResourceGridSize.Y) * ResourceGridPixelSize);
}

FIntVector GetFroxelGridSize(const FIntPoint& TargetResolution, int32& InOutGridPixelSize, int32 GridSizeZ)
{
	FIntPoint GridSizeXY = FIntPoint::DivideAndRoundUp(TargetResolution, InOutGridPixelSize);
	if (GridSizeXY.X > GMaxVolumeTextureDimensions || GridSizeXY.Y > GMaxVolumeTextureDimensions) //clamp to max volume texture dimensions. only happens for extreme resolutions (~8x2k)
	{
		float PixelSizeX = (float)TargetResolution.X / GMaxVolumeTextureDimensions;
		float PixelSizeY = (float)TargetResolution.Y / GMaxVolumeTextureDimensions;
		InOutGridPixelSize = FMath::Max(FMath::CeilToInt(PixelSizeX), FMath::CeilToInt(PixelSizeY));
		GridSizeXY = FIntPoint::DivideAndRoundUp(TargetResolution, InOutGridPixelSize);
	}
	return FIntVector(GridSizeXY.X, GridSizeXY.Y, GridSizeZ);
}
