// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

extern FVector2f GetFroxelGridUVMaxForSampling(const FVector2f& ViewRectSize, FIntVector ResourceGridSize, int32 ResourceGridPixelSize);
extern FVector2f GetFroxelGridPrevUVMaxForTemporalBlend(const FVector2f& ViewRectSize, FIntVector ResourceGridSize, int32 ResourceGridPixelSize);

extern FIntVector GetFroxelGridSize(const FIntPoint& TargetResolution, int32& InOutGridPixelSize, int32 GridSizeZ);
