// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUBenchmark.cpp: GPUBenchmark to compute performance index to set video options automatically
=============================================================================*/

#include "GPUBenchmark.h"
#include "GenericPlatform/GenericPlatformSurvey.h"
#include "RHI.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderParameters.h"
#include "RenderResource.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "GlobalShader.h"
#include "RenderTargetPool.h"
#include "PostProcess/SceneFilterRendering.h"
#include "GPUProfiler.h"
#include "PipelineStateCache.h"
#include "LongGPUTask.h"
#include "VisualizeTexture.h"
#include "CommonRenderResources.h"
#include "SceneView.h"
#include "RHIResourceUtils.h"

static const uint32 GBenchmarkResolution = 512;
static const uint32 GBenchmarkPrimitives = 200000;
static const uint32 GBenchmarkVertices = GBenchmarkPrimitives * 3;

/** Encapsulates the post processing down sample pixel shader. */
template <uint32 PsMethod>
class FPostProcessBenchmarkPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FPostProcessBenchmarkPS, Global);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) || Parameters.Platform == SP_PCD3D_ES3_1;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("PS_METHOD"), PsMethod);
	}

	/** Default constructor. */
	FPostProcessBenchmarkPS() {}

public:
	LAYOUT_FIELD(FShaderResourceParameter, InputTexture);
	LAYOUT_FIELD(FShaderResourceParameter, InputTextureSampler);

	/** Initialization constructor. */
	FPostProcessBenchmarkPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		InputTexture.Bind(Initializer.ParameterMap,TEXT("InputTexture"));
		InputTextureSampler.Bind(Initializer.ParameterMap,TEXT("InputTextureSampler"));
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, TRefCountPtr<IPooledRenderTarget>& Src)
	{
		FGlobalShader::SetParameters<FViewUniformShaderParameters>(BatchedParameters, View.ViewUniformBuffer);

		SetTextureParameter(BatchedParameters, InputTexture, InputTextureSampler, TStaticSamplerState<>::GetRHI(), Src->GetRHI());
	}

	static const TCHAR* GetSourceFilename()
	{
		return TEXT("/Engine/Private/GPUBenchmark.usf");
	}

	static const TCHAR* GetFunctionName()
	{
		return TEXT("MainPS");
	}
};

// #define avoids a lot of code duplication
#define VARIATION1(A) typedef FPostProcessBenchmarkPS<A> FPostProcessBenchmarkPS##A; \
	IMPLEMENT_SHADER_TYPE2(FPostProcessBenchmarkPS##A, SF_Pixel);

VARIATION1(0)			VARIATION1(1)			VARIATION1(2)			VARIATION1(3)			VARIATION1(4)			VARIATION1(5)
#undef VARIATION1


/** Encapsulates the post processing down sample vertex shader. */
template <uint32 VsMethod>
class FPostProcessBenchmarkVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FPostProcessBenchmarkVS,Global);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("VS_METHOD"), VsMethod);
	}

	/** Default constructor. */
	FPostProcessBenchmarkVS() {}
	
	/** Initialization constructor. */
	FPostProcessBenchmarkVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FGlobalShader(Initializer)
	{
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View)
	{
		FGlobalShader::SetParameters<FViewUniformShaderParameters>(BatchedParameters, View.ViewUniformBuffer);
	}
};

typedef FPostProcessBenchmarkVS<0> FPostProcessBenchmarkVS0;
typedef FPostProcessBenchmarkVS<1> FPostProcessBenchmarkVS1;
typedef FPostProcessBenchmarkVS<2> FPostProcessBenchmarkVS2;

IMPLEMENT_SHADER_TYPE(template<>,FPostProcessBenchmarkVS0,TEXT("/Engine/Private/GPUBenchmark.usf"),TEXT("MainBenchmarkVS"),SF_Vertex);
IMPLEMENT_SHADER_TYPE(template<>,FPostProcessBenchmarkVS1,TEXT("/Engine/Private/GPUBenchmark.usf"),TEXT("MainBenchmarkVS"),SF_Vertex);
IMPLEMENT_SHADER_TYPE(template<>,FPostProcessBenchmarkVS2,TEXT("/Engine/Private/GPUBenchmark.usf"),TEXT("MainBenchmarkVS"),SF_Vertex);

struct FBenchmarkVertex
{
	FVector4f Arg0;
	FVector4f Arg1;
	FVector4f Arg2;
	FVector4f Arg3;
	FVector4f Arg4;

	FBenchmarkVertex(uint32 VertexID)
		: Arg0(VertexID, 0.0f, 0.0f, 0.0f)
		, Arg1()
		, Arg2()
		, Arg3()
		, Arg4()
	{}
};

struct FVertexThroughputDeclaration : public FRenderResource
{
	FVertexDeclarationRHIRef DeclRHI;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements = 
		{
			{ 0, 0 * sizeof(FVector4f), VET_Float4, 0, sizeof(FBenchmarkVertex) },
			{ 0, 1 * sizeof(FVector4f), VET_Float4, 1, sizeof(FBenchmarkVertex) },
			{ 0, 2 * sizeof(FVector4f), VET_Float4, 2, sizeof(FBenchmarkVertex) },
			{ 0, 3 * sizeof(FVector4f), VET_Float4, 3, sizeof(FBenchmarkVertex) },
			{ 0, 4 * sizeof(FVector4f), VET_Float4, 4, sizeof(FBenchmarkVertex) },
		};

		DeclRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI() override
	{
		DeclRHI = nullptr;
	}
};

TGlobalResource<FVertexThroughputDeclaration> GVertexThroughputDeclaration;

template <uint32 VsMethod, uint32 PsMethod>
void RunBenchmarkShader(FRHICommandList& RHICmdList, FRHIBuffer* VertexThroughputBuffer, const FSceneView& View, TRefCountPtr<IPooledRenderTarget>& Src, float WorkScale)
{
	auto ShaderMap = GetGlobalShaderMap(View.GetFeatureLevel());

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

	TShaderMapRef<FPostProcessBenchmarkVS<VsMethod>> VertexShader(ShaderMap);
	TShaderMapRef<FPostProcessBenchmarkPS<PsMethod>> PixelShader(ShaderMap);

	bool bVertexTest = VsMethod != 0;
	FRHIVertexDeclaration* VertexDeclaration = bVertexTest
		? GVertexThroughputDeclaration.DeclRHI
		: GFilterVertexDeclaration.VertexDeclarationRHI;

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = VertexDeclaration;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	SetShaderParametersLegacyPS(RHICmdList, PixelShader, View, Src);
	SetShaderParametersLegacyVS(RHICmdList, VertexShader, View);

	if (bVertexTest)
	{
		// Vertex Tests

		uint32 TotalNumPrimitives = FMath::CeilToInt(GBenchmarkPrimitives * WorkScale);
		uint32 TotalNumVertices = TotalNumPrimitives * 3;

		while (TotalNumVertices != 0)
		{
			uint32 VerticesThisPass = FMath::Min(TotalNumVertices, GBenchmarkVertices);
			uint32 PrimitivesThisPass = VerticesThisPass / 3;

			RHICmdList.SetStreamSource(0, VertexThroughputBuffer, 0);

			RHICmdList.DrawPrimitive(0, PrimitivesThisPass, 1);

			TotalNumVertices -= VerticesThisPass;
		}
	}
	else
	{
		// Pixel Tests

		// single pass was not fine grained enough so we reduce the pass size based on the fractional part of WorkScale
		float TotalHeight = GBenchmarkResolution * WorkScale;

		// rounds up
		uint32 PassCount = (uint32)FMath::CeilToFloat(TotalHeight / GBenchmarkResolution);

		for (uint32 i = 0; i < PassCount; ++i)
		{
			float Top = i * GBenchmarkResolution;
			float Bottom = FMath::Min(Top + GBenchmarkResolution, TotalHeight);
			float LocalHeight = Bottom - Top;

			DrawRectangle(
				RHICmdList,
				0, 0,
				GBenchmarkResolution, LocalHeight,
				0, 0,
				GBenchmarkResolution, LocalHeight,
				FIntPoint(GBenchmarkResolution, GBenchmarkResolution),
				FIntPoint(GBenchmarkResolution, GBenchmarkResolution),
				VertexShader,
				EDRF_Default);
		}
	}
}

void RunBenchmarkShader(FRHICommandListImmediate& RHICmdList, FRHIBuffer* VertexThroughputBuffer, const FSceneView& View, uint32 MethodId, TRefCountPtr<IPooledRenderTarget>& Src, float WorkScale)
{
	SCOPED_DRAW_EVENTF(RHICmdList, Benchmark, TEXT("Benchmark Method:%d"), MethodId);

	switch(MethodId)
	{
		case 0: RunBenchmarkShader<0, 0>(RHICmdList, VertexThroughputBuffer, View, Src, WorkScale); return;
		case 1: RunBenchmarkShader<0, 1>(RHICmdList, VertexThroughputBuffer, View, Src, WorkScale); return;
		case 2: RunBenchmarkShader<0, 2>(RHICmdList, VertexThroughputBuffer, View, Src, WorkScale); return;
		case 3: RunBenchmarkShader<0, 3>(RHICmdList, VertexThroughputBuffer, View, Src, WorkScale); return;
		case 4: RunBenchmarkShader<0, 4>(RHICmdList, VertexThroughputBuffer, View, Src, WorkScale); return;
		case 5: RunBenchmarkShader<1, 5>(RHICmdList, VertexThroughputBuffer, View, Src, WorkScale); return;
		case 6: RunBenchmarkShader<2, 5>(RHICmdList,                nullptr, View, Src, WorkScale); return;
		default:
			check(0);
	}
}

// Many Benchmark timings stored in an array to allow to extract a good value dropping outliers
// We need to get rid of the bad samples.
class FTimingSeries
{
public:
	// @param ArraySize
	void Init(uint32 ArraySize)
	{
		check(ArraySize > 0);

		TimingValues.AddZeroed(ArraySize);
	}

	//
	void SetEntry(uint32 Index, float TimingValue)
	{
		check(Index < (uint32)TimingValues.Num());

		TimingValues[Index] = TimingValue;
	}

	//
	float GetEntry(uint32 Index) const
	{
		check(Index < (uint32)TimingValues.Num());

		return TimingValues[Index];
	}

	// @param OutConfidence
	float ComputeValue(float& OutConfidence) const
	{
		float Ret = 0.0f;

		TArray<float> SortedValues;
		{
			// a lot of values in the beginning are wrong, we cut off some part (1/4 of the samples area)
			uint32 StartIndex = TimingValues.Num() / 3;

			for(uint32 Index = StartIndex; Index < (uint32)TimingValues.Num(); ++Index)
			{
				SortedValues.Add(TimingValues[Index]);
			}
			SortedValues.Sort();
		}

		OutConfidence = 0.0f;
		
		uint32 Passes = 10;

		// slow but simple
		for(uint32 Pass = 0; Pass < Passes; ++Pass)
		{
			// 0..1, 0 not included
			float Alpha = (Pass + 1) / (float)Passes;

			int32 MidIndex = SortedValues.Num() / 2;
			int32 FromIndex = (int32)FMath::Lerp(MidIndex, 0, Alpha); 
			int32 ToIndex = (int32)FMath::Lerp(MidIndex, SortedValues.Num(), Alpha); 

			float Delta = 0.0f;
			float Confidence = 0.0f;

			float TimingValue = ComputeTimingFromSortedRange(FromIndex, ToIndex, SortedValues, Delta, Confidence);

			// aim for 5% delta and some samples
			if(Pass > 0 && Delta > TimingValue * 0.5f)
			{
				// it gets worse, take the best one we had so far
				break;
			}

			OutConfidence = Confidence;
			Ret = TimingValue;
		}

		return Ret;
	}

private:
	// @param FromIndex within SortedValues
	// @param ToIndex within SortedValues
	// @param OutDelta +/- 
	// @param OutConfidence 0..100 0=not at all, 100=fully, meaning how many samples are considered useful
	// @return TimingValue, smaller is better
	static float ComputeTimingFromSortedRange(int32 FromIndex, int32 ToIndex, const TArray<float>& SortedValues, float& OutDelta, float& OutConfidence)
	{
		float ValidSum = 0;
		uint32 ValidCount = 0;
		float Min = FLT_MAX;
		float Max = -FLT_MAX;
		{
			for(int32 Index = FromIndex; Index < ToIndex; ++Index)
			{
				float Value = SortedValues[Index];

				Min = FMath::Min(Min, Value);
				Max = FMath::Max(Max, Value);

				ValidSum += Value;
				++ValidCount;
			}
		}

		if(ValidCount)
		{
			OutDelta = (Max - Min) * 0.5f;

			OutConfidence = 100.0f * ValidCount / (float)SortedValues.Num();

			return ValidSum / ValidCount;
		}
		else
		{
			OutDelta = 0.0f;
			OutConfidence = 0.0f;
			return 0.0f;
		}
	}

	TArray<float> TimingValues;
};

void RendererGPUBenchmark(FRHICommandListImmediate& RHICmdList, FSynthBenchmarkResults& InOut, const FSceneView& View, float WorkScale, bool bDebugOut)
{
	check(IsInRenderingThread());

	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	MeasureLongGPUTaskExecutionTime(RHICmdList);
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);	

	TArray<FBenchmarkVertex> Vertices;
	Vertices.Reserve(GBenchmarkVertices);
	for (uint32 Index = 0; Index < GBenchmarkVertices; ++Index)
	{
		Vertices.Emplace(Index);
	}

	FBufferRHIRef VertexBuffer = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("RendererGPUBenchmark"), EBufferUsageFlags::Static, MakeConstArrayView(Vertices));

	// two RT to ping pong so we force the GPU to flush its pipeline
	TRefCountPtr<IPooledRenderTarget> RTItems[2];
	{
		FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::Create2DDesc(FIntPoint(GBenchmarkResolution, GBenchmarkResolution), PF_B8G8R8A8, FClearValueBinding::None, TexCreate_None, TexCreate_RenderTargetable | TexCreate_ShaderResource, false));

		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, RTItems[0], TEXT("Benchmark0"));
		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, RTItems[1], TEXT("Benchmark1"));
	}

	{
		// larger number means more accuracy but slower, some slower GPUs might timeout with a number to large
		const uint32 IterationCount = 70;
		const uint32 MethodCount = UE_ARRAY_COUNT(InOut.GPUStats);

		enum class EMethodType
		{
			Vertex,
			Pixel
		};

		struct FBenchmarkMethod
		{
			const TCHAR* Desc;
			float IndexNormalizedTime;
			const TCHAR* ValueType;
			float Weight;
			EMethodType Type;
		};
		
		const FBenchmarkMethod Methods[] =
		{
			// e.g. on NV670: Method3 (mostly fill rate )-> 26GP/s (seems realistic)
			// reference: http://en.wikipedia.org/wiki/Comparison_of_Nvidia_graphics_processing_units theoretical: 29.3G/s
			{ TEXT("ALUHeavyNoise"),    1.0f / 4.601f,  TEXT("s/GigaPix"),  1.0f, EMethodType::Pixel  },
			{ TEXT("TexHeavy"),         1.0f / 7.447f,  TEXT("s/GigaPix"),  0.1f, EMethodType::Pixel  },
			{ TEXT("DepTexHeavy"),      1.0f / 3.847f,  TEXT("s/GigaPix"),  0.1f, EMethodType::Pixel  },
			{ TEXT("FillOnly"),         1.0f / 25.463f, TEXT("s/GigaPix"),  3.0f, EMethodType::Pixel  },
			{ TEXT("Bandwidth"),        1.0f / 1.072f,  TEXT("s/GigaPix"),  1.0f, EMethodType::Pixel  },
			{ TEXT("VertThroughPut1"),  1.0f / 1.537f,  TEXT("s/GigaVert"), 0.0f, EMethodType::Vertex }, // TODO: Set weights
			{ TEXT("VertThroughPut2"),  1.0f / 1.767f,  TEXT("s/GigaVert"), 0.0f, EMethodType::Vertex }, // TODO: Set weights
		};

		static_assert(UE_ARRAY_COUNT(Methods) == UE_ARRAY_COUNT(InOut.GPUStats), "Benchmark methods descriptor array lengths should match.");

		// Initialize the GPU benchmark stats
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(Methods); ++Index)
		{
			auto& Method = Methods[Index];
			InOut.GPUStats[Index] = FSynthBenchmarkStat(Method.Desc, Method.IndexNormalizedTime, Method.ValueType, Method.Weight);
		}

		// 0 / 1
		uint32 DestRTIndex = 0;

		if(!GSupportsTimestampRenderQueries)
		{
#if !PLATFORM_MAC
			UE_LOGF(LogSynthBenchmark, Warning, "GPU driver does not support timer queries.");

#else
			// Workaround for Metal not having a timing API
			FTextureMemoryStats MemStats;
			RHIGetTextureMemoryStats(MemStats);
			
			float PerfScale = 1.0f;
			if (MemStats.TotalGraphicsMemory < (2ll * 1024ll * 1024ll * 1024ll))
			{
				// Assume Intel HD 5000, Iris, Iris Pro performance, or low end NVIDIA - low settings
				PerfScale = 6.2f;
			}
			else if (MemStats.TotalGraphicsMemory < (3ll * 1024ll * 1024ll * 1024ll))
			{
				// Assume NVIDIA 6x0 & 7x0 series - medium settings
				PerfScale = 3.0f;
			}

			for (int32 Index = 0; Index < MethodCount; ++Index)
			{
				FSynthBenchmarkStat& Stat = InOut.GPUStats[Index];
				Stat.SetMeasuredTime(FTimeSample(PerfScale, PerfScale * Methods[Index].IndexNormalizedTime));
			}
#endif
			return;
		}

		const uint32 TimerSampleCount = IterationCount * MethodCount + 1;
		FRenderQueryPoolRHIRef TimerQueryPool = RHICreateRenderQueryPool(RQT_AbsoluteTime, TimerSampleCount * 2);
		FRHIPooledRenderQuery TimerQueries[TimerSampleCount];
		float LocalWorkScale[IterationCount];

		for (uint32 i = 0; i < TimerSampleCount; ++i)
		{
			TimerQueries[i] = TimerQueryPool->AllocateQuery();
		}

		IssueScalableLongGPUTask(RHICmdList, -1);
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);

		// TimingValues are in Seconds
		FTimingSeries TimingSeries[MethodCount];
		// in 1/1000000 Seconds
		uint64 TotalTimes[MethodCount];
		
		for(uint32 MethodIterator = 0; MethodIterator < MethodCount; ++MethodIterator)
		{
			TotalTimes[MethodIterator] = 0;
			TimingSeries[MethodIterator].Init(IterationCount);
		}

		RHICmdList.EndRenderQuery(TimerQueries[0].GetQuery());
		
		SCOPED_DRAW_EVENTF(RHICmdList, Benchmark, TEXT("Scale:%f"), WorkScale);

		// multiple iterations to see how trust able the values are
		for(uint32 Iteration = 0; Iteration < IterationCount; ++Iteration)
		{
			if(Iteration % 10 == 0)
			{
				RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
			}
			SCOPED_DRAW_EVENTF(RHICmdList, Benchmark, TEXT("Iteration:%d"), Iteration);

			for(uint32 MethodIterator = 0; MethodIterator < MethodCount; ++MethodIterator)
			{
				// alternate between forward and backward (should give the same number)
				//			uint32 MethodId = (Iteration % 2) ? MethodIterator : (MethodCount - 1 - MethodIterator);
				uint32 MethodId = MethodIterator;

				uint32 QueryIndex = 1 + Iteration * MethodCount + MethodId;

				// 0 / 1
				const uint32 SrcRTIndex = 1 - DestRTIndex;

				GVisualizeTexture.SetCheckPoint(RHICmdList, RTItems[DestRTIndex]);

				// decide how much work we do in this pass
				LocalWorkScale[Iteration] = (Iteration / 10.f + 1.f) * WorkScale;

				FRHIRenderPassInfo RPInfo(RTItems[DestRTIndex]->GetRHI(), ERenderTargetActions::Load_Store);
				RHICmdList.Transition(FRHITransitionInfo(RTItems[DestRTIndex]->GetRHI(), ERHIAccess::Unknown, ERHIAccess::RTV));
				RHICmdList.BeginRenderPass(RPInfo, TEXT("GPUBenchmark"));
				{
					RunBenchmarkShader(RHICmdList, VertexBuffer, View, MethodId, RTItems[SrcRTIndex], LocalWorkScale[Iteration]);
				}
				RHICmdList.EndRenderPass();
				RHICmdList.Transition(FRHITransitionInfo(RTItems[DestRTIndex]->GetRHI(), ERHIAccess::RTV, ERHIAccess::SRVMask));

				RHICmdList.EndRenderQuery(TimerQueries[QueryIndex].GetQuery());

				// ping pong
				DestRTIndex = 1 - DestRTIndex;
			}
		}

		{
			uint64 OldAbsTime = 0;
			// flushes the RHI thread to make sure all RHICmdList.EndRenderQuery() commands got executed.
			RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
			RHIGetRenderQueryResult(TimerQueries[0].GetQuery(), OldAbsTime, true);

			for(uint32 Iteration = 0; Iteration < IterationCount; ++Iteration)
			{
				if(Iteration % 10 == 0)
				{
					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
				}

				uint32 Results[MethodCount];

				for(uint32 MethodId = 0; MethodId < MethodCount; ++MethodId)
				{
					uint32 QueryIndex = 1 + Iteration * MethodCount + MethodId;

					uint64 AbsTime;
					RHIGetRenderQueryResult(TimerQueries[QueryIndex].GetQuery(), AbsTime, true);

					uint64 RelTime = FMath::Max(AbsTime - OldAbsTime, 1ull);

					TotalTimes[MethodId] += RelTime;
					Results[MethodId] = RelTime;

					OldAbsTime = AbsTime;
				}

				for(uint32 MethodId = 0; MethodId < MethodCount; ++MethodId)
				{
					float TimeInSec = Results[MethodId] / 1000000.0f;

					if (Methods[MethodId].Type == EMethodType::Vertex)
					{
						// to normalize from seconds to seconds per GVert
						float SamplesInGVert = LocalWorkScale[Iteration] * GBenchmarkVertices / 1000000000.0f;
						TimingSeries[MethodId].SetEntry(Iteration, TimeInSec / SamplesInGVert);
					}
					else
					{
						check(Methods[MethodId].Type == EMethodType::Pixel);

						// to normalize from seconds to seconds per GPixel
						float SamplesInGPix = LocalWorkScale[Iteration] * GBenchmarkResolution * GBenchmarkResolution / 1000000000.0f;

						// TimingValue in Seconds per GPixel
						TimingSeries[MethodId].SetEntry(Iteration, TimeInSec / SamplesInGPix);
					}
				}
			}

			if(GSupportsTimestampRenderQueries)
			{
				for(uint32 MethodId = 0; MethodId < MethodCount; ++MethodId)
				{
					float Confidence = 0.0f;
					// in seconds per GPixel
					float NormalizedTime = TimingSeries[MethodId].ComputeValue(Confidence);

					if(Confidence > 0)
					{
						FTimeSample TimeSample(TotalTimes[MethodId] / 1000000.0f, NormalizedTime);

						InOut.GPUStats[MethodId].SetMeasuredTime(TimeSample, Confidence);
					}
				}
			}
		}
	}
}
