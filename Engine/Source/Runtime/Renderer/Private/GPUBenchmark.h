// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUBenchmark.h: GPUBenchmark to compute performance index to set video options automatically
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

class FRHICommandListImmediate;
class FSceneView;
struct FSynthBenchmarkResults;

// @param >0, WorkScale 10 for normal precision and runtime of less than a second
// @param bDebugOut has no effect in shipping
void RendererGPUBenchmark(FRHICommandListImmediate& RHICmdList, FSynthBenchmarkResults& InOut, const FSceneView& View, float WorkScale, bool bDebugOut = false);
