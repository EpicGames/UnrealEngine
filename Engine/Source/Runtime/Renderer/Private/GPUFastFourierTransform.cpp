// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPUFastFourierTransform.h"
#include "SceneUtils.h"
#include "GlobalShader.h"
#include "RenderGraph.h" 
#include "ShaderCompilerCore.h"
#include "SystemTextures.h"
#include "DataDrivenShaderPlatformInfo.h"

bool ShouldCompileFullGPUFFT(EShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsFFTBloom(Platform) && !FDataDrivenShaderPlatformInfo::GetIsConsole(Platform);
}

bool ShouldCompileGroupSharedGPUFFT(EShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsFFTBloom(Platform);
}

uint32 GPUFFT::MaxScanLineLength() 
{
	return 4096;
}

bool GPUFFT::IsHorizontal(const FFT_XFORM_TYPE& XFormType)
{
	return (XFormType == FFT_XFORM_TYPE::FORWARD_HORIZONTAL || XFormType == FFT_XFORM_TYPE::INVERSE_HORIZONTAL);
}
bool GPUFFT::IsForward(const FFT_XFORM_TYPE& XFormType)
{
	return (XFormType == FFT_XFORM_TYPE::FORWARD_HORIZONTAL || XFormType == FFT_XFORM_TYPE::FORWARD_VERTICAL);
}

GPUFFT::FFTDescription::FFTDescription(const GPUFFT::FFT_XFORM_TYPE& XForm, const FIntPoint& XFormExtent)
	:XFormType(XForm)
{
	if (GPUFFT::IsHorizontal(XFormType))
	{
		SignalLength = XFormExtent.X;
		NumScanLines = XFormExtent.Y;
	}
	else
	{
		SignalLength = XFormExtent.Y;
		NumScanLines = XFormExtent.X;
	}
}

FIntPoint GPUFFT::FFTDescription::TransformExtent() const
{
	const bool bIsHornizontal = GPUFFT::IsHorizontal(XFormType);

	FIntPoint Extent = (bIsHornizontal) ? FIntPoint(SignalLength, NumScanLines) : FIntPoint(NumScanLines, SignalLength);
	return Extent;
}

bool GPUFFT::FFTDescription::IsHorizontal() const
{
	return GPUFFT::IsHorizontal(XFormType);
}

bool GPUFFT::FFTDescription::IsForward() const
{
	return GPUFFT::IsForward(XFormType);
}

const TCHAR* GPUFFT::FFTDescription::FFT_TypeName() const
{
	if (XFormType == FFT_XFORM_TYPE::FORWARD_HORIZONTAL) return TEXT("Forward Horizontal");
	if (XFormType == FFT_XFORM_TYPE::INVERSE_HORIZONTAL) return TEXT("Inverse Horizontal");
	if (XFormType == FFT_XFORM_TYPE::FORWARD_VERTICAL)   return TEXT("Forward Vertical");
	if (XFormType == FFT_XFORM_TYPE::INVERSE_VERTICAL)   return TEXT("Inverse Vertical");
	unimplemented();
	return TEXT("Error");
}


namespace GPUFFT
{

	// Encode the transform type in the lower two bits
	static uint32 BitEncode(const GPUFFT::FFT_XFORM_TYPE& XFormType)
	{
		// put a 1 in the low bit for Horizontal
		uint32 BitEncodedValue = GPUFFT::IsHorizontal(XFormType) ? 1 : 0;
		// put a 1 in the second lowest bit for forward
		if (GPUFFT::IsForward(XFormType))
		{
			BitEncodedValue |= 2;
		}
		return BitEncodedValue;

	}

	/** 
	* Computes the minimal number of bits required to represent the in number N
	*/
	uint32 BitSize(uint32 N) 
	{
		uint32 Result = 0;
		while (N > 0) {
			N = N >> 1;
			Result++;
		}
		return Result;
	}

	/**
	* Decompose the input PowTwoLenght, as PowTwoLength = N X PowTwoBase X PowTwoBase X .. X PowTwoBase
	* NB: This assumes the PowTwoLength and PowTwoBase are powers of two. 
	*/
	TArray<uint32> GetFactors(const uint32 PowTwoLength, const uint32 PowTwoBase)
	{
		TArray<uint32> FactorList;

		// Early out. 
		if (!FMath::IsPowerOfTwo(PowTwoLength) || !FMath::IsPowerOfTwo(PowTwoBase)) return FactorList;

		const uint32 LogTwoLength = BitSize(PowTwoLength) - 1;
		const uint32 LogTwoBase = BitSize(PowTwoBase) - 1;

		const uint32 RemainderPower = LogTwoLength % LogTwoBase;
		const uint32 BasePower = LogTwoLength / LogTwoBase;

		for (uint32 idx = 0; idx < BasePower; idx++) 
		{
			FactorList.Add(PowTwoBase);
		}

		if (RemainderPower != 0)
		{
			uint32 Factor = 1 << RemainderPower;
			FactorList.Add(Factor);
		}
			
		return FactorList;
	}


	void SwapContents(FRDGTextureRef& TmpBuffer, FRDGTextureRef& DstBuffer)
	{
		// Swap the pointers
		FRDGTextureRef TmpDst = DstBuffer;
		DstBuffer = TmpBuffer;
		TmpBuffer = TmpDst;
	}
}

namespace
{

class FPowRadixSignalLengthDim : SHADER_PERMUTATION_SPARSE_INT("SCAN_LINE_LENGTH", 2, 4, 8, 16, 32, 64, 128, 256, 512, 1024, 2048, 4096);

class FFFTShader : public FGlobalShader
{
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileFullGPUFFT(Parameters.Platform);
	}

	FFFTShader() = default;
	FFFTShader(const CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

class FReorderFFTPassCS : public FFFTShader
{
	DECLARE_GLOBAL_SHADER(FReorderFFTPassCS);
	SHADER_USE_PARAMETER_STRUCT(FReorderFFTPassCS, FFFTShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntRect, SrcRect)
		SHADER_PARAMETER(FIntRect, DstRect)
		SHADER_PARAMETER(int32, TransformType)
		SHADER_PARAMETER(int32, LogTwoLength)
		SHADER_PARAMETER(int32, BitCount)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcSRV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstUAV)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INCLUDE_REORDER_FFT_PASS"), 1);
	}
};


class FGroupShardSubFFTPassCS : public FFFTShader
{
	DECLARE_GLOBAL_SHADER(FGroupShardSubFFTPassCS);
	SHADER_USE_PARAMETER_STRUCT(FGroupShardSubFFTPassCS, FFFTShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntRect, SrcWindow)
		SHADER_PARAMETER(int32, TransformType)
		SHADER_PARAMETER(int32, TransformLength)
		SHADER_PARAMETER(int32, NumSubRegions)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstTexture)
	END_SHADER_PARAMETER_STRUCT()

	static uint32 SubPassLength() { return 2048; }
	static uint32 Radix() { return 2; }

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INCLUDE_GROUP_SHARED_SUB_COMPLEX_FFT"), 1);
		OutEnvironment.SetDefine(TEXT("SCAN_LINE_LENGTH"), FGroupShardSubFFTPassCS::SubPassLength());
		OutEnvironment.SetDefine(TEXT("RADIX"), FGroupShardSubFFTPassCS::Radix());
	}
};

class FComplexFFTPassCS : public FFFTShader
{
	DECLARE_GLOBAL_SHADER(FComplexFFTPassCS);
	SHADER_USE_PARAMETER_STRUCT(FComplexFFTPassCS, FFFTShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntRect, SrcRect)
		SHADER_PARAMETER(FIntRect, DstRect)
		SHADER_PARAMETER(int32, TransformType)
		SHADER_PARAMETER(int32, PowTwoLength)
		SHADER_PARAMETER(int32, BitCount)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcSRV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, DstPostFilterParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstUAV)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INCLUDE_COMPLEX_FFT_PASS"), 1);
	}
};


class FPackTwoForOneFFTPassCS : public FFFTShader
{
	DECLARE_GLOBAL_SHADER(FPackTwoForOneFFTPassCS);
	SHADER_USE_PARAMETER_STRUCT(FPackTwoForOneFFTPassCS, FFFTShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntRect, DstRect)
		SHADER_PARAMETER(int32, TransformType)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcSRV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstUAV)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INCLUDE_PACK_TWOFORONE_FFT_PASS"), 1);
	}
};

class FCopyWindowCS : public FFFTShader
{
	DECLARE_GLOBAL_SHADER(FCopyWindowCS);
	SHADER_USE_PARAMETER_STRUCT(FCopyWindowCS, FFFTShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntRect, SrcRect)
		SHADER_PARAMETER(FIntRect, DstRect)
		SHADER_PARAMETER(FVector3f, BrightPixelGain)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcSRV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstUAV)
	END_SHADER_PARAMETER_STRUCT()

	static FIntPoint ThreadCount() { return FIntPoint(1, 32); }

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FFFTShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INCLUDE_COPY_WINDOW"), 1);
		OutEnvironment.SetDefine(TEXT("X_THREAD_COUNT"), ThreadCount().X);
		OutEnvironment.SetDefine(TEXT("Y_THREAD_COUNT"), ThreadCount().Y);
	}
};

class FComplexMultiplyImagesCS : public FFFTShader
{
	DECLARE_GLOBAL_SHADER(FComplexMultiplyImagesCS);
	SHADER_USE_PARAMETER_STRUCT(FComplexMultiplyImagesCS, FFFTShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntRect, SrcRect)
		SHADER_PARAMETER(int32, DataLayout)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcSRV)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, KnlSRV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstUAV)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INCLUDE_COMPLEX_MULTIPLY_IMAGES"), 1);
	}
};

class FGSComplexTransformCS : public FFFTShader
{
	DECLARE_GLOBAL_SHADER(FGSComplexTransformCS);
	SHADER_USE_PARAMETER_STRUCT(FGSComplexTransformCS, FFFTShader);

	using FPermutationDomain = TShaderPermutationDomain<FPowRadixSignalLengthDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, SrcRectMin)
		SHADER_PARAMETER(FIntPoint, SrcRectMax)
		SHADER_PARAMETER(FIntPoint, DstExtent)
		SHADER_PARAMETER(FIntRect, DstRect)
		SHADER_PARAMETER(FVector3f, BrightPixelGain)
		SHADER_PARAMETER(int32, TransformType)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileGroupSharedGPUFFT(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		int32 PowRadixSignalLength = PermutationVector.Get<FPowRadixSignalLengthDim>();

		FFFTShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INCLUDE_GROUP_SHARED_COMPLEX_FFT"), 1);
		if (PowRadixSignalLength > 8)
		{
			OutEnvironment.SetDefine(TEXT("MIXED_RADIX"), 1);
		}
	}
};

class FGSTwoForOneTransformCS : public FFFTShader
{
	DECLARE_GLOBAL_SHADER(FGSTwoForOneTransformCS);
	SHADER_USE_PARAMETER_STRUCT(FGSTwoForOneTransformCS, FFFTShader);

	using FPermutationDomain = TShaderPermutationDomain<FPowRadixSignalLengthDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, SrcRectMin)
		SHADER_PARAMETER(FIntPoint, SrcRectMax)
		SHADER_PARAMETER(FIntPoint, DstExtent)
		SHADER_PARAMETER(FIntRect, DstRect)
		SHADER_PARAMETER(FVector3f, BrightPixelGain)
		SHADER_PARAMETER(int32, TransformType)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, DstPostFilterParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileGroupSharedGPUFFT(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		int32 PowRadixSignalLength = PermutationVector.Get<FPowRadixSignalLengthDim>();

		FFFTShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INCLUDE_GROUP_SHARED_TWO_FOR_ONE_FFT"), 1);
		if (PowRadixSignalLength > 8)
		{
			OutEnvironment.SetDefine(TEXT("MIXED_RADIX"), 1);
		}
	}
};

class FGSConvolutionWithTextureCS : public FFFTShader
{
	DECLARE_GLOBAL_SHADER(FGSConvolutionWithTextureCS);
	SHADER_USE_PARAMETER_STRUCT(FGSConvolutionWithTextureCS, FFFTShader);

	using FPermutationDomain = TShaderPermutationDomain<FPowRadixSignalLengthDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, SrcRectMin)
		SHADER_PARAMETER(FIntPoint, SrcRectMax)
		SHADER_PARAMETER(FIntPoint, DstExtent)
		SHADER_PARAMETER(int32, TransformType)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SrcTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FilterTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DstTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileGroupSharedGPUFFT(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		int32 PowRadixSignalLength = PermutationVector.Get<FPowRadixSignalLengthDim>();

		FFFTShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INCLUDE_GROUP_SHARED_CONVOLUTION_WITH_TEXTURE"), 1);
		if (PowRadixSignalLength > 8)
		{
			OutEnvironment.SetDefine(TEXT("MIXED_RADIX"), 1);
		}
	}
};


IMPLEMENT_GLOBAL_SHADER(FReorderFFTPassCS,           "/Engine/Private/GPUFastFourierTransform.usf", "ReorderFFTPassCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FGroupShardSubFFTPassCS,     "/Engine/Private/GPUFastFourierTransform.usf", "GroupSharedSubComplexFFTCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FComplexFFTPassCS,           "/Engine/Private/GPUFastFourierTransform.usf", "ComplexFFTPassCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FPackTwoForOneFFTPassCS,     "/Engine/Private/GPUFastFourierTransform.usf", "PackTwoForOneFFTPassCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FCopyWindowCS,               "/Engine/Private/GPUFastFourierTransform.usf", "CopyWindowCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FComplexMultiplyImagesCS,    "/Engine/Private/GPUFastFourierTransform.usf", "ComplexMultiplyImagesCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FGSComplexTransformCS,       "/Engine/Private/GPUFastFourierTransform.usf", "GroupSharedComplexFFTCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FGSTwoForOneTransformCS,     "/Engine/Private/GPUFastFourierTransform.usf", "GroupSharedTwoForOneFFTCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FGSConvolutionWithTextureCS, "/Engine/Private/GPUFastFourierTransform.usf", "GSConvolutionWithTextureCS", SF_Compute);


/**
 * Single Pass of a multi-pass complex FFT that just reorders data for
 * a group shared subpass to consume.
 *
 * Assumes the dst buffer is large enough to hold the result.
 * The Src float4 data is interpt as a pair of independent complex numbers.
 *
 * @param FFTDesc    - Metadata that describes the underlying complex FFT
 * @param SrcRct     - The region in the Src buffer where the image to transform lives.
 * @param SrcTexture - SRV the source image buffer
 * @param DstRect    - The target Window.
 * @param DstUAV     - UAV, the destination buffer that will hold the result of the single pass
 *
 */
void DispatchReorderFFTPassCS(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const GPUFFT::FFTDescription& FFTDesc,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
	FRDGTextureRef DstTexture, const FIntRect& DstWindow,
	const bool bScrubNaNs = false)
{
	// Using multiple radix two passes.
	const uint32 Radix = 2;

	uint32 TransformLength = FFTDesc.SignalLength;

	// Breaks the data into the correct number of sub-transforms for the later group-shared pass.
	uint32 PowTwoSubLengthCount = TransformLength / FGroupShardSubFFTPassCS::SubPassLength();

	FReorderFFTPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FReorderFFTPassCS::FParameters>();
	PassParameters->SrcRect = SrcWindow;
	PassParameters->DstRect = DstWindow;
	{
		PassParameters->TransformType = GPUFFT::BitEncode(FFTDesc.XFormType);
		if (bScrubNaNs)
		{
			PassParameters->TransformType |= 4;
		}
	}
	PassParameters->LogTwoLength = GPUFFT::BitSize(FFTDesc.SignalLength) - 1;
	PassParameters->BitCount = GPUFFT::BitSize(PowTwoSubLengthCount) - 1;

	PassParameters->SrcSRV = SrcTexture;
	PassParameters->DstUAV = GraphBuilder.CreateUAV(DstTexture);

	TShaderMapRef<FReorderFFTPassCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FFT Multipass: Complex %s Reorder pass of size %d", FFTDesc.FFT_TypeName(), TransformLength),
		FFTDesc.ComputePassFlags,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, TransformLength / Radix));
}


/**
* A Group Shared Single Pass of a multi-pass complex FFT.
* Assumes the dst buffer is large enough to hold the result.
* The Src float4 data is interpt as a pair of independent complex numbers.
*
* @param FFTDesc    - Metadata that describes the underlying complex FFT
* @param SrcTexture - SRV the source image buffer
* @param SrcRct     - The region in the Src buffer where the image to transform lives.
* @param DstUAV     - UAV, the destination buffer that will hold the result of the single pass
*
*/
void DispatchGSSubComplexFFTPassCS(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const GPUFFT::FFTDescription& FFTDesc,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
	FRDGTextureRef DstTexture)
{
	const uint32 TransformLength = FFTDesc.SignalLength;

	const uint32 NumSubRegions = TransformLength / FGroupShardSubFFTPassCS::SubPassLength();

	// The window on which a single transform acts.
	FIntRect SubPassWindow = SrcWindow;
	if (FFTDesc.IsHorizontal())
	{
		SubPassWindow.Max.X = SubPassWindow.Min.X + FGroupShardSubFFTPassCS::SubPassLength();
	}
	else
	{
		SubPassWindow.Max.Y = SubPassWindow.Min.Y + FGroupShardSubFFTPassCS::SubPassLength();
	}

	const uint32 NumSignals = FFTDesc.IsHorizontal() ? SubPassWindow.Size().Y : SubPassWindow.Size().X;

	FGroupShardSubFFTPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGroupShardSubFFTPassCS::FParameters>();
	PassParameters->SrcWindow = SrcWindow;
	PassParameters->TransformType = GPUFFT::BitEncode(FFTDesc.XFormType);
	PassParameters->TransformLength = TransformLength;
	PassParameters->NumSubRegions = NumSubRegions;

	PassParameters->SrcTexture = SrcTexture;
	PassParameters->DstTexture = GraphBuilder.CreateUAV(DstTexture);

	TShaderMapRef<FGroupShardSubFFTPassCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FFT Multipass: %d GS Subpasses Complex %s of size %d",
			NumSubRegions, FFTDesc.FFT_TypeName(), FGroupShardSubFFTPassCS::SubPassLength()),
		FFTDesc.ComputePassFlags,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, NumSignals));
}

/**
* Single Pass of a multi-pass complex FFT.
* Assumes the dst buffer is large enough to hold the result.
* The Src float4 data is interpt as a pair of independent complex numbers.
*
* @param FFTDesc    - Metadata that describes the underlying complex FFT
* @param PassStage  - The Depth at which this FFT pass lives. 
* @param SrcTexture - SRV the source image buffer
* @param SrcRct     - The region in the Src buffer where the image to transform lives.
* @param DstUAV     - UAV, the destination buffer that will hold the result of the single pass
*
*/
void DispatchComplexFFTPassCS(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const GPUFFT::FFTDescription& FFTDesc,
	const uint32 PassLength,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
	FRDGBufferRef DstPostFilterParameters,
	FRDGTextureRef DstTexture)
{
	// Using multiple radix two passes.
	const uint32 Radix = 2;

	const uint32 TransformLength = FFTDesc.SignalLength;

	// Translate the transform type. 
	uint32 TransformTypeValue = GPUFFT::BitEncode(FFTDesc.XFormType);
	if (!DstPostFilterParameters)
	{
		DstPostFilterParameters = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f), FVector4f(1.0f, 1.0f, 1.0f, 1.0f));
	}

	FComplexFFTPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComplexFFTPassCS::FParameters>();
	PassParameters->SrcRect = SrcWindow;
	PassParameters->DstRect = FIntRect(FIntPoint(0, 0), FFTDesc.TransformExtent());
	PassParameters->TransformType = TransformTypeValue;
	PassParameters->PowTwoLength = PassLength;
	PassParameters->BitCount = GPUFFT::BitSize(TransformLength);

	PassParameters->SrcSRV = SrcTexture;
	PassParameters->DstPostFilterParameters = GraphBuilder.CreateSRV(DstPostFilterParameters);
	PassParameters->DstUAV = GraphBuilder.CreateUAV(DstTexture);

	TShaderMapRef<FComplexFFTPassCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FFT Multipass: Pass %d of Complex %s of size %d", PassLength, FFTDesc.FFT_TypeName(), TransformLength),
		FFTDesc.ComputePassFlags,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, TransformLength / Radix));
}

/**
* Single Pass that separates or merges the transform of four real signals from 
* viewed as the transform of two complex  signals.  
*
* Assumes the dst buffer is large enough to hold the result.
* The Src float4 data is interprets  as a pair of independent complex numbers.
*
* @param FFTDesc    - Metadata that describes the underlying complex FFT
* @param SrcTexture - SRV the source image buffer
* @param DstUAV     - UAV, the destination buffer that will hold the result of the single pass
*
* The data in the SrcTexture and DstUAV is aligned with (0,0).
* FFTDesc.IsHorizontal() indicates the transform direction in the buffer that needs to be spit/merged
* FFTDesc.IsForward() indicates spit (true) vs merge (false).
*/
void DispatchPackTwoForOneFFTPassCS(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const GPUFFT::FFTDescription& FFTDesc,
	FRDGTextureSRVRef SrcTexture,
	FRDGTextureRef DstTexture)
{
	const uint32 TransformLength = FFTDesc.SignalLength;

	// A real signal of length 'TransformLenght' requires  only TransformLength / 2 + 1  complex coefficients, 
	const uint32 RealTransformLength = ( TransformLength / 2 ) + 1;

	// The splitting into two real signals (isForward) or joinging back into a single signal
	const uint32 ResultingLength = (FFTDesc.IsForward()) ? 2 * RealTransformLength : TransformLength;

	FIntPoint DstExtent = FFTDesc.TransformExtent();
	if (FFTDesc.IsHorizontal())
	{
		DstExtent.X = ResultingLength;
	}	
	else
	{
		DstExtent.Y = ResultingLength;
	}

	FPackTwoForOneFFTPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPackTwoForOneFFTPassCS::FParameters>();
	PassParameters->DstRect = FIntRect(FIntPoint(0, 0), DstExtent);
	PassParameters->TransformType = GPUFFT::BitEncode(FFTDesc.XFormType);

	PassParameters->SrcSRV = SrcTexture;
	PassParameters->DstUAV = GraphBuilder.CreateUAV(DstTexture);

	TShaderMapRef<FPackTwoForOneFFTPassCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FFT Multipass: TwoForOne Combine/split result of %s of size %d", FFTDesc.FFT_TypeName(), TransformLength),
		FFTDesc.ComputePassFlags,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, RealTransformLength));
}

/**
 * Single Pass that Copies a sub-region of a buffer and potentially 
 * boosts the intensity of select pixels.
 * 
 * Assumes the dst buffer is large enough to hold the result.
 * The Src float4 data is interpreted as a pair of independent complex numbers.
 * The Knl float4 data is interpreted as a pair of independent complex numbers
 *
 * @param SrcWindow  - The region of interest to copy.
 * @param SrcTexture - SRV the source image buffer.
 * @param DstWindow  - The target location for the images.
 *
 * @param DstUAV     - UAV, the destination buffer that will hold the result of the single pass
 * @param Prefilter  - Optional filter to boost selected pixels.
 */
void DispatchCopyWindowCS(
	FRDGBuilder& GraphBuilder,
	ERDGPassFlags ComputePassFlags,
	const FGlobalShaderMap* ShaderMap,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
	FRDGTextureRef DstTexture, const FIntRect& DstWindow,
	const GPUFFT::FPreFilter& PreFilter = GPUFFT::FPreFilter(TNumericLimits<float>::Max(), TNumericLimits<float>::Lowest(), 0.f))
{
	FCopyWindowCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyWindowCS::FParameters>();
	PassParameters->SrcRect = SrcWindow;
	PassParameters->DstRect = DstWindow;
	PassParameters->BrightPixelGain = PreFilter;

	PassParameters->SrcSRV = SrcTexture;
	PassParameters->DstUAV = GraphBuilder.CreateUAV(DstTexture);

	TShaderMapRef<FCopyWindowCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FFT Multipass: Copy Subwindow"),
		ComputePassFlags,
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(DstWindow.Size(), FCopyWindowCS::ThreadCount()));
}

/**
 * 
 * Single Pass that computes the Frequency Space convolution of two buffers
 * that have already been transformed into frequency space.
 *
 * This is really means the complex product of two buffers divided
 * by the correct values to "normalize" the effect of the kernel buffer.
 *
 * In this case, each float4 is viewed as a pair of complex numbers,
 * and the product of float4 Src, Knl is computed
 * as float4(ComplexMult(Src.xy, Knl.xy) / Na, ComplexMult(Src.zw, Knl.zw)/Nb)
 * where Na and Nb are related to the sums of the weights in the kernel buffer. 
 *
 * Assumes the dst buffer is large enough to hold the result.
 * The Src float4 data is interpreted as a pair of independent complex numbers.
 * The Knl float4 data is interpreted as a pair of independent complex numbers  
 *
 * @param bHorizontalFirst - Describes the layout of the transformed data.
 *                           bHorizontalFirst == true for data that was transformed as 
 *                           Vertical::ComplexFFT following Horizontal::TwoForOneRealFFT
 *                           bHorizontalFirst == false for data that was transformed as
 *                           Horizontal::ComplexFFT following Vertical::TwoForOneRealFFT
 * @param SrcTexture - SRV the source image buffer.
 * @param KnlTexture - SRV the kernel image buffer.
 *
 * @param DstUAV     - UAV, the destination buffer that will hold the result of the single pass
 *
 */
void DispatchComplexMultiplyImagesCS(
	FRDGBuilder& GraphBuilder,
	ERDGPassFlags ComputePassFlags,
	const FGlobalShaderMap* ShaderMap,
	const bool bHorizontalScanlines,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
	FRDGTextureRef KnlTexure,
	FRDGTextureRef DstTexture)
{
	// The size of the dst
	const FIntPoint DstExtent = SrcWindow.Size();

	// Align the scanlines in the direction of the first transform.
	const uint32 NumScanLines = (bHorizontalScanlines)  ? DstExtent.Y : DstExtent.X;
	const uint32 SignalLength = (!bHorizontalScanlines) ? DstExtent.Y : DstExtent.X;
	
	FComplexMultiplyImagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComplexMultiplyImagesCS::FParameters>();
	PassParameters->SrcRect = SrcWindow;
	PassParameters->DataLayout = (bHorizontalScanlines) ? 1 : 0;

	PassParameters->SrcSRV = SrcTexture;
	PassParameters->KnlSRV = KnlTexure;
	PassParameters->DstUAV = GraphBuilder.CreateUAV(DstTexture);

	TShaderMapRef<FComplexMultiplyImagesCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FFT Multipass: Convolution in freq-space"),
		ComputePassFlags,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, NumScanLines));
}

/**
 * Complex 1D FFT of two independent complex signals.
 * Assumes the dst buffer is large enough to hold the result.
 * The Src float4 data is interpt as a pair of independent complex numbers.
 *
 * @param Context    - container for RHI and ShanderMap
 * @param FFTDesc    - Metadata that describes the underlying complex FFT
 * @param SrcTexture - SRV the source image buffer
 * @param SrcRct     - The region in the Src buffer where the image to transform lives.
 * @param DstUAV     - UAV, the destination buffer that will hold the result of the 1d complex FFT
 *
 */
void DispatchGSComplexFFTCS(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const GPUFFT::FFTDescription& FFTDesc,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcRect,
	FRDGTextureRef DstTexture)
{
	const uint32 TransformLength = FFTDesc.SignalLength;
	const FIntPoint DstExtent    = FFTDesc.TransformExtent();

	// The number of signals to transform simultaneously (i.e. number of scan lines)
	const uint32 NumSignals = FFTDesc.IsHorizontal() ? SrcRect.Size().Y : SrcRect.Size().X;

	FGSComplexTransformCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGSComplexTransformCS::FParameters>();
	PassParameters->SrcRectMin = SrcRect.Min;
	PassParameters->SrcRectMax = SrcRect.Max;
	PassParameters->DstExtent = DstExtent;
	PassParameters->DstRect = FIntRect(FIntPoint(0, 0), DstExtent);
	PassParameters->BrightPixelGain = GPUFFT::FPreFilter(TNumericLimits<float>::Max(), TNumericLimits<float>::Lowest(), 0.f);
	{
		// Translate the transform type. 
		PassParameters->TransformType = GPUFFT::BitEncode(FFTDesc.XFormType);

		// We have a valid prefilter if the min is less than the max
		if (PassParameters->BrightPixelGain.Component(0) < PassParameters->BrightPixelGain.Component(1))
		{
			// Encode a bool to turn on the pre-filter.
			PassParameters->TransformType |= 4;
		}
	}

	PassParameters->SrcTexture = SrcTexture;
	PassParameters->DstTexture = GraphBuilder.CreateUAV(DstTexture);

	FGSComplexTransformCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FPowRadixSignalLengthDim>(TransformLength);

	TShaderMapRef<FGSComplexTransformCS> ComputeShader(ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FFT: Complex %s of size %d", FFTDesc.FFT_TypeName(), TransformLength),
		FFTDesc.ComputePassFlags,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, NumSignals));
}


/**
* Real 1D FFT of four independent real signals.   
* Assumes the dst buffer is large enough to hold the result.
* The Src float4 data is interptd as 4 independent real numbers.
* The Dst float4 data will be two complex numbers.
*
* @param Context    - container for RHI and ShanderMap
* @param FFTDesc    - Metadata that describes the underlying complex FFT
* @param SrcTexture - SRV the source image buffer
* @param SrcRct     - The region in the Src buffer where the image to transform lives.
* @param DstUAV     - UAV, the destination buffer that will hold the result of the 1d complex FFT
* @param DstRect    - Where to write the tranform data in the Dst buffer
* @param PreFilter  - Used to boost the intensity of already bright pixels.
*
*/
void DispatchGSTwoForOneFFTCS(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const GPUFFT::FFTDescription& FFTDesc,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcRect,
	FRDGTextureRef DstTexture, const FIntRect& DstRect,
	const GPUFFT::FPreFilter& PreFilter,
	FRDGBufferRef DstPostFilterParameters)
{
	const uint32 TransformLength = FFTDesc.SignalLength;
	
	// The number of signals to transform simultaneously (i.e. number of scan lines)
	const uint32 NumScanLines = (FFTDesc.IsHorizontal()) ? SrcRect.Size().Y : SrcRect.Size().X;

	if (!DstPostFilterParameters)
	{
		DstPostFilterParameters = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f), FVector4f(1.0f, 1.0f, 1.0f, 1.0f));
	}

	FGSTwoForOneTransformCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGSTwoForOneTransformCS::FParameters>();
	PassParameters->SrcRectMin = SrcRect.Min;
	PassParameters->SrcRectMax = SrcRect.Max;
	PassParameters->DstExtent = DstRect.Size();
	PassParameters->DstRect = DstRect;
	PassParameters->BrightPixelGain = PreFilter;
	{
		// Translate the transform type. 
		PassParameters->TransformType = GPUFFT::BitEncode(FFTDesc.XFormType);

		// We have a valid prefilter if the min is less than the max
		if (PassParameters->BrightPixelGain.Component(0) < PassParameters->BrightPixelGain.Component(1))
		{
			// Encode a bool to turn on the pre-filter.
			PassParameters->TransformType |= 4;
		}
	}

	PassParameters->SrcTexture = SrcTexture;
	PassParameters->DstPostFilterParameters = GraphBuilder.CreateSRV(DstPostFilterParameters);
	PassParameters->DstTexture = GraphBuilder.CreateUAV(DstTexture);

	FGSTwoForOneTransformCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FPowRadixSignalLengthDim>(TransformLength);

	TShaderMapRef<FGSTwoForOneTransformCS> ComputeShader(ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FFT: Two-For-One %s of size %d of buffer %d x %d", FFTDesc.FFT_TypeName(), TransformLength, SrcRect.Size().X, SrcRect.Size().Y),
		FFTDesc.ComputePassFlags,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, NumScanLines));
}


/**
 * Complex 1D FFT followed by multiplication in with kernel and inverse transform.
 *
 * @param Context    - container for RHI and ShanderMap
 * @param FFTDesc    - Metadata that describes the underlying complex FFT
 * @param PreTransformedKernel - Pre-transformed kernel used in the convolution.
 * @param SrcTexture - SRV the source image buffer
 * @param SrcRct     - The region in the Src buffer where the image to transform lives.
 * @param DstUAV     - UAV, the destination buffer that will hold the result of the 1d complex FFT
 * @param DstRect    - Where to write the tranform data in the Dst buffer
 *
 */
void DispatchGSConvolutionWithTextureCS(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const GPUFFT::FFTDescription& FFTDesc,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcRect,
	FRDGTextureRef PreTransformedKernel,
	FRDGTextureRef DstTexture)
{
	const uint32 SignalLength   = FFTDesc.SignalLength;
	const bool bIsHornizontal   = FFTDesc.IsHorizontal();

	const FIntPoint& SrcRectSize = SrcRect.Size();
	// The number of signals to transform simultaneously (i.e. number of scan lines)
	// NB: This may be different from the FFTDesc.NumScanlines.
	const uint32 NumSignals = (bIsHornizontal) ? SrcRectSize.Y : SrcRectSize.X;

	FGSConvolutionWithTextureCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGSConvolutionWithTextureCS::FParameters>();
	PassParameters->SrcRectMin = SrcRect.Min;
	PassParameters->SrcRectMax = SrcRect.Max;
	PassParameters->DstExtent = SrcRect.Size();
	{
		PassParameters->TransformType = GPUFFT::BitEncode(FFTDesc.XFormType);

		const bool bUseAlpha = true;
		if (bUseAlpha)
		{
			PassParameters->TransformType |= 8;
		}
	}

	PassParameters->SrcTexture = SrcTexture;
	PassParameters->FilterTexture = PreTransformedKernel;
	PassParameters->DstTexture = GraphBuilder.CreateUAV(DstTexture);

	FGSConvolutionWithTextureCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FPowRadixSignalLengthDim>(SignalLength);

	TShaderMapRef<FGSConvolutionWithTextureCS> ComputeShader(ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FFT: Apply %s Transform, Multiply Texture, and InverseTransform size %d of buffer %d x %d", FFTDesc.FFT_TypeName(), SignalLength, SrcRectSize.X, SrcRectSize.Y),
		FFTDesc.ComputePassFlags,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, NumSignals));
}

bool FitsInGroupSharedMemory(const uint32& Length)
{
	const bool bFitsInGroupSharedMemory = !(GPUFFT::MaxScanLineLength() < Length);
	return bFitsInGroupSharedMemory;
}

bool FitsInGroupSharedMemory(const GPUFFT::FFTDescription& FFTDesc)
{
	return FitsInGroupSharedMemory(FFTDesc.SignalLength);
}


} // namespace


void GPUFFT::CopyImage2D(
	FRDGBuilder& GraphBuilder,
	ERDGPassFlags ComputePassFlags,
	const FGlobalShaderMap* ShaderMap,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
	FRDGTextureRef DstTexture, const FIntRect& DstWindow,
	const FPreFilter& PreFilter)
{
	DispatchCopyWindowCS(GraphBuilder, ComputePassFlags, ShaderMap, SrcTexture, SrcWindow, DstTexture, DstWindow, PreFilter);
}

void GPUFFT::ComplexFFTImage1D::MultiPass(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const FFTDescription& FFTDesc,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
	FRDGTextureRef DstTexture,
	FRDGBufferRef PostFilterParameters,
	const bool bScrubNaNs)
{
	if (FitsInGroupSharedMemory(FFTDesc))
	{
		return DispatchGSComplexFFTCS(
			GraphBuilder, ShaderMap,
			FFTDesc,
			SrcTexture, SrcWindow,
			DstTexture);
	}

	RDG_EVENT_SCOPE(GraphBuilder, "ComplexFFTImage1D");

	const uint32 TransformLength = FFTDesc.SignalLength;
	const FFT_XFORM_TYPE XFormType = FFTDesc.XFormType;

	// The direction of the transform must be a power of two.

	check(FMath::IsPowerOfTwo(TransformLength));

	// The number of iterations required.
	const uint32 Log2TransformLength = BitSize(TransformLength) - 1;

	FIntPoint DstExtent = FFTDesc.TransformExtent();

	const FIntRect  XFormWindow(FIntPoint(0, 0), DstExtent);

	// Testing code branch: Breaks the transform into Log2(TransformLength) number of passes.
	// this is the slowest algorithm, and uses no group-shared storage.
	#if 0
	{
		DispatchComplexFFTPassCS(Context, FFTDesc, 1, SrcTexture, Window, Targets.DstTarget().UAV, bScrubNaNs);

		for (uint32 Ns = 2; Ns < TransformLength; Ns *= 2)
		{
			Targets.Swap();

			auto HasValidTargets = [&Targets, &DstExtent]()->bool
			{
				// Verify that the buffers being used are big enough.  Note that we are checking the "src" buffer, but due
				// to the double buffering we will end up testing both buffers.
				FIntPoint SrcBufferSize = Targets.SrcTarget().ShaderResourceTexture->GetTexture2D()->GetSizeXY();
				bool Fits = !(SrcBufferSize.X < DstExtent.X) && !(SrcBufferSize.Y < DstExtent.Y);
				return Fits;
			};

			checkf(HasValidTargets(), TEXT("FFT: Allocated Buffers too small."));

			DispatchComplexFFTPassCS(Context, FFTDesc, Ns, Targets.SrcTarget().ShaderResourceTexture, XFormWindow, Targets.DstTarget().UAV);
		}

		// If this transform requires an even number of passes
		// this swap will insure that the "DstBuffer" is filled last.
		if (Log2TransformLength % 2 == 0)
		{
			SwapContents(TmpBuffer, DstBuffer);
		}
	}
	#endif
	// Reorder, followed by a High-level group-shared pass followed by Log2(TransformLength / SubPassLength() ) simple passes.
	// In total 2 + Log2(TransformLength / SubPassLength() )  passes.   This will be on the order of 3 or 4 passes 
	// compared with 12 or more ..
	{
		FRDGTextureDesc SpectralDesc = FFTDesc.IsForward() ? DstTexture->Desc : SrcTexture->Desc.Texture->Desc;

		// Re-order the data so we can do a pass of group-shared transforms
		FRDGTextureRef SpectralTexture = GraphBuilder.CreateTexture(SpectralDesc, TEXT("FFT.Spectral"));
		DispatchReorderFFTPassCS(
			GraphBuilder, ShaderMap, FFTDesc,
			/* SrcTexture = */ SrcTexture, SrcWindow,
			/* DstTexture = */ SpectralTexture, XFormWindow);

		FRDGTextureRef PrevSpectralTexture = GraphBuilder.CreateTexture(SpectralDesc, TEXT("FFT.Spectral"));
		DispatchGSSubComplexFFTPassCS(
			GraphBuilder, ShaderMap, FFTDesc,
			/* SrcTexture = */ GraphBuilder.CreateSRV(FRDGTextureSRVDesc(SpectralTexture)), XFormWindow,
			/* DstTexture = */ PrevSpectralTexture);
		
		for (uint32 Ns = FGroupShardSubFFTPassCS::SubPassLength(); Ns < TransformLength; Ns *= 2)
		{
			FRDGTextureRef OpDstTexture;
			FRDGBufferRef DstPostFilterParameters = nullptr;
			if (Ns * 2 < TransformLength)
			{
				FIntPoint SrcBufferSize = PrevSpectralTexture->Desc.Extent;
				checkf(!(SrcBufferSize.X < DstExtent.X) && !(SrcBufferSize.Y < DstExtent.Y), TEXT("FFT: Allocated Buffers too small."));

				OpDstTexture = GraphBuilder.CreateTexture(SpectralDesc, TEXT("FFT.Spectral"));
			}
			else
			{
				OpDstTexture = DstTexture;
				DstPostFilterParameters = PostFilterParameters;
			}

			DispatchComplexFFTPassCS(
				GraphBuilder, ShaderMap, FFTDesc,
				Ns,
				/* SrcTexture = */ GraphBuilder.CreateSRV(FRDGTextureSRVDesc(PrevSpectralTexture)), XFormWindow,
				/* DstPostFilterParameters = */ DstPostFilterParameters,
				/* DstTexture = */ OpDstTexture);

			PrevSpectralTexture = OpDstTexture;
		}

		check(PrevSpectralTexture == DstTexture);
	}
}

void GPUFFT::TwoForOneRealFFTImage1D::MultiPass(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const FFTDescription& FFTDesc,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
	FRDGTextureRef DstTexture, const FIntRect& DstWindow,
	const FPreFilter& PreFilter,
	FRDGBufferRef PostFilterParameters)
{
	if (FitsInGroupSharedMemory(FFTDesc))
	{
		DispatchGSTwoForOneFFTCS(
			GraphBuilder, ShaderMap, FFTDesc,
			SrcTexture, SrcWindow,
			DstTexture, DstWindow,
			PreFilter,
			PostFilterParameters);
	}
	else if (FFTDesc.IsForward())
	{
		RDG_EVENT_SCOPE(GraphBuilder, "TwoForOneRealFFTImage1D(Forward)");

		// Filter only on forward transform through image copy pass.
		if (IsActive(PreFilter))
		{
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				SrcTexture->Desc.Texture->Desc.Extent,
				SrcTexture->Desc.Texture->Desc.Format,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_UAV);

			FRDGTextureRef PreFilteredTexture = GraphBuilder.CreateTexture(Desc, TEXT("FFT.PreFilter"));

			CopyImage2D(
				GraphBuilder, FFTDesc.ComputePassFlags, ShaderMap,
				/* SrcTexture = */ SrcTexture, SrcWindow,
				/* DstTexture = */ PreFilteredTexture, SrcWindow,
				PreFilter);

			SrcTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(PreFilteredTexture));
		}

		FRDGTextureRef SpectralTexture = GraphBuilder.CreateTexture(DstTexture->Desc, TEXT("FFT.Spectral"));
		ComplexFFTImage1D::MultiPass(
			GraphBuilder, ShaderMap, FFTDesc,
			/* SrcTexture = */ SrcTexture, SrcWindow,
			/* DstTexture = */ SpectralTexture,
			/* PostFilterParameters = */ nullptr,
			/* bScrubNaNs = */ true);
		
		// Unpack the complex transform into transform of real data
		ensure(DstWindow.Min == FIntPoint::ZeroValue);
		DispatchPackTwoForOneFFTPassCS(
			GraphBuilder, ShaderMap, FFTDesc,
			/* SrcTexture = */ GraphBuilder.CreateSRV(FRDGTextureSRVDesc(SpectralTexture)),
			/* DstTexture = */ DstTexture);
	}
	else  // Inverse transform.
	{
		RDG_EVENT_SCOPE(GraphBuilder, "TwoForOneRealFFTImage1D(Inverse)");

		// Pack the 4 transforms of real data as 2 transforms of complex data 
		FRDGTextureRef SpectralTexture = GraphBuilder.CreateTexture(SrcTexture->Desc.Texture->Desc, TEXT("FFT.Spectral"));
		ensure(SrcWindow.Min == FIntPoint::ZeroValue);
		DispatchPackTwoForOneFFTPassCS(
			GraphBuilder, ShaderMap, FFTDesc,
			/* SrcTexture = */ SrcTexture,
			/* DstTexture = */ SpectralTexture);

		// Transform as complex data
		ensure(DstWindow.Min == FIntPoint::ZeroValue);
		ComplexFFTImage1D::MultiPass(
			GraphBuilder, ShaderMap, FFTDesc,
			/* SrcTexture = */ GraphBuilder.CreateSRV(FRDGTextureSRVDesc(SpectralTexture)), SrcWindow,
			/* DstTexture = */ DstTexture,
			/* PostFilterParameters = */ PostFilterParameters,
			/* bScrubNaNs = */ false);
	}
}

void GPUFFT::FFTImage2D(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const FIntPoint& FrequencySize, bool bHorizontalFirst,
	FRDGTextureSRVRef SrcTexture, const FIntRect& ROIRect,
	FRDGTextureRef DstTexture)
{
	using GPUFFT::FFTDescription;

	// This will be for the actual output.
	const FIntRect FFTResultRect = ROIRect;
	const FIntPoint ImageSpaceExtent = ROIRect.Size();

	// Set up the transform descriptions
	FFTDescription TwoForOneFFTDesc = (bHorizontalFirst) ? FFTDescription(FFT_XFORM_TYPE::FORWARD_HORIZONTAL, FrequencySize) : FFTDescription(FFT_XFORM_TYPE::FORWARD_VERTICAL, FrequencySize);
	FFTDescription ComplexFFTDesc = (bHorizontalFirst) ? FFTDescription(FFT_XFORM_TYPE::FORWARD_VERTICAL, FrequencySize) : FFTDescription(FFT_XFORM_TYPE::FORWARD_HORIZONTAL, FrequencySize);


	// The two-for-one transform data storage has two additional elements.
	const uint32 FrequencyPadding = 2;
	// This additional elements translate to two additional scanlines that need to be transformed by the complex fft
	ComplexFFTDesc.NumScanLines += FrequencyPadding;

	// Temp Double Buffers
	const FIntPoint TmpExent = (bHorizontalFirst) ? FIntPoint(FrequencySize.X + FrequencyPadding, ImageSpaceExtent.Y) : FIntPoint(ImageSpaceExtent.X, FrequencySize.Y + FrequencyPadding);

	const FIntRect TmpRect = FIntRect(FIntPoint(0, 0), TmpExent);

	// Two-for-one transform: SrcTexture fills the TmpBuffer
	FRDGTextureRef TempTexture = GraphBuilder.CreateTexture(DstTexture->Desc, DstTexture->Name);
	TwoForOneRealFFTImage1D::MultiPass(
		GraphBuilder, ShaderMap,
		TwoForOneFFTDesc,
		SrcTexture, ROIRect,
		TempTexture, TmpRect);

	ComplexFFTImage1D::MultiPass(
		GraphBuilder, ShaderMap,
		ComplexFFTDesc,
		GraphBuilder.CreateSRV(FRDGTextureSRVDesc(TempTexture)), TmpRect,
		DstTexture);
}

void GPUFFT::ConvolutionWithTextureImage1D::MultiPass(
	FRDGBuilder& GraphBuilder,
	const FGlobalShaderMap* ShaderMap,
	const FFTDescription& FFTDesc,
	FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
	FRDGTextureRef TransformedKernel,
	FRDGTextureRef DstTexture)
{
	if (FitsInGroupSharedMemory(FFTDesc))
	{
		DispatchGSConvolutionWithTextureCS(
			GraphBuilder, ShaderMap, FFTDesc,
			SrcTexture, SrcWindow, TransformedKernel, DstTexture);
	}
	else
	{
		RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionWithTextureImage1D");

		// Frequency Space size.
		const FIntPoint TargetExtent = FFTDesc.TransformExtent();
		const FIntRect  TargetRect(FIntPoint(0, 0), TargetExtent);

		// Forward transform: Results in DstBuffer
		FRDGTextureRef SpectralTexture0 = GraphBuilder.CreateTexture(SrcTexture->Desc.Texture->Desc, TEXT("FFT.Spectral"));
		ComplexFFTImage1D::MultiPass(
			GraphBuilder, ShaderMap, FFTDesc,
			/* SrcTexture = */ SrcTexture, SrcWindow,
			/* DstTexture = */ SpectralTexture0);

		// Apply convolution kernel.
		FRDGTextureRef SpectralTexture1 = GraphBuilder.CreateTexture(SrcTexture->Desc.Texture->Desc, TEXT("FFT.Spectral"));
		DispatchComplexMultiplyImagesCS(
			GraphBuilder, FFTDesc.ComputePassFlags, ShaderMap, FFTDesc.IsHorizontal(),
			/* SrcTexture = */ GraphBuilder.CreateSRV(FRDGTextureSRVDesc(SpectralTexture0)), TargetRect,
			TransformedKernel, 
			/* DstTexture = */ SpectralTexture1);

		// Backward transform.
		FFTDescription InvFFTDesc = FFTDesc;
		InvFFTDesc.XFormType = GetInverseOfXForm(FFTDesc.XFormType);

		ComplexFFTImage1D::MultiPass(
			GraphBuilder, ShaderMap, InvFFTDesc,
			/* SrcTexture = */ GraphBuilder.CreateSRV(FRDGTextureSRVDesc(SpectralTexture1)), TargetRect,
			/* DstTexture = */ DstTexture);
	}
}

FIntPoint GPUFFT::Convolution2DBufferSize(const FIntPoint& FrequencySize, const bool bHorizontalFirst, const FIntPoint& ROIExtent)
{
	using GPUFFT::FFTDescription;
	using GPUFFT::FFT_XFORM_TYPE;

	FFTDescription TwoForOneFFTDesc;
	TwoForOneFFTDesc.XFormType    = bHorizontalFirst ? FFT_XFORM_TYPE::FORWARD_HORIZONTAL : FFT_XFORM_TYPE::FORWARD_VERTICAL;
	TwoForOneFFTDesc.SignalLength = bHorizontalFirst ? FrequencySize.X : FrequencySize.Y;
	TwoForOneFFTDesc.NumScanLines = (bHorizontalFirst) ? ROIExtent.Y : ROIExtent.X;


	FFTDescription ConvolutionFFTDesc;
	ConvolutionFFTDesc.XFormType = bHorizontalFirst ? FFT_XFORM_TYPE::FORWARD_VERTICAL : FFT_XFORM_TYPE::FORWARD_HORIZONTAL;
	ConvolutionFFTDesc.SignalLength = bHorizontalFirst ? FrequencySize.Y : FrequencySize.X;
	ConvolutionFFTDesc.NumScanLines = TwoForOneFFTDesc.SignalLength + 2;

	FFTDescription TwoForOneIvnFFTDesc = TwoForOneFFTDesc;
	TwoForOneIvnFFTDesc.XFormType = GetInverseOfXForm(TwoForOneFFTDesc.XFormType);
	FIntPoint BufferSize;

	if (FitsInGroupSharedMemory(ConvolutionFFTDesc))
	{
		FFTDescription TmpDesc = TwoForOneFFTDesc;
		TmpDesc.SignalLength += 2;
		BufferSize = TmpDesc.TransformExtent();
	}
	else
	{
		// a larger buffer is needed when the convolution can't be done in group-shared
		BufferSize = ConvolutionFFTDesc.TransformExtent();
	}
	return BufferSize;
}

FRDGTextureDesc CreateFrequencyDesc(const FIntPoint& FrequencySize, const bool bHorizontalFirst, const FIntPoint& SrcExtent)
{
	const FIntPoint FrequencyExtent = GPUFFT::Convolution2DBufferSize(
		FrequencySize, bHorizontalFirst, SrcExtent);

	return FRDGTextureDesc::Create2D(
		FrequencyExtent,
		GPUFFT::PixelFormat(),
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV);
}

void GPUFFT::ConvolutionWithTextureImage2D(
	FRDGBuilder& GraphBuilder,
	ERDGPassFlags ComputePassFlags,
	const FGlobalShaderMap* ShaderMap,
	const FIntPoint& FrequencySize, bool bHorizontalFirst,
	FRDGTextureRef TransformedKernel,
	FRDGTextureSRVRef SrcTexture, const FIntRect& ROIRect,
	FRDGTextureRef DstTexture, const FIntRect& DstRect,
	const FPreFilter& PreFilter,
	FRDGBufferRef PostFilterParameters,
	ETextureCreateFlags AdditionalTextureCreateFlags)
{
	FRDGTextureDesc SpectralDesc = CreateFrequencyDesc(FrequencySize, bHorizontalFirst, ROIRect.Size());
	SpectralDesc.Flags |= AdditionalTextureCreateFlags;

	// This will be for the actual output.
	const FIntRect FFTResultRect = ROIRect;
	const FIntPoint ROISize = ROIRect.Size();

	// Set up the transform descriptions
	FFTDescription TwoForOneFFTDesc;
	TwoForOneFFTDesc.XFormType = bHorizontalFirst ? FFT_XFORM_TYPE::FORWARD_HORIZONTAL : FFT_XFORM_TYPE::FORWARD_VERTICAL;
	TwoForOneFFTDesc.SignalLength = bHorizontalFirst ? FrequencySize.X : FrequencySize.Y;
	TwoForOneFFTDesc.NumScanLines = (bHorizontalFirst) ? ROISize.Y : ROISize.X;
	TwoForOneFFTDesc.ComputePassFlags = ComputePassFlags;

	// The output has two more elements
	FIntRect SpectralOutputRect;
	{
		FFTDescription TmpDesc = TwoForOneFFTDesc;
		TmpDesc.SignalLength += 2;
		SpectralOutputRect = FIntRect(FIntPoint(0, 0), TmpDesc.TransformExtent());
	}

	FFTDescription ConvolutionFFTDesc;
	ConvolutionFFTDesc.XFormType = bHorizontalFirst ? FFT_XFORM_TYPE::FORWARD_VERTICAL : FFT_XFORM_TYPE::FORWARD_HORIZONTAL;
	ConvolutionFFTDesc.SignalLength = bHorizontalFirst ? FrequencySize.Y : FrequencySize.X;
	ConvolutionFFTDesc.NumScanLines = TwoForOneFFTDesc.SignalLength + 2; // two-for-one transform generated two additional elements
	ConvolutionFFTDesc.ComputePassFlags = ComputePassFlags;

	FFTDescription TwoForOneIvnFFTDesc = TwoForOneFFTDesc;
	TwoForOneIvnFFTDesc.XFormType = GetInverseOfXForm(TwoForOneFFTDesc.XFormType);
	TwoForOneIvnFFTDesc.ComputePassFlags = ComputePassFlags;
	
	

	// ---- Two For One Transform --- 

	FRDGTextureRef TwoForOneFFTOutput = GraphBuilder.CreateTexture(SpectralDesc, TEXT("FFT.TwoForOne"));
	TwoForOneRealFFTImage1D::MultiPass(
		GraphBuilder, ShaderMap, TwoForOneFFTDesc,
		SrcTexture, ROIRect,
		TwoForOneFFTOutput, SpectralOutputRect,
		PreFilter,
		/* PostFilterParameters = */ nullptr);

	// ---- 1 D Convolution --- 

	FRDGTextureRef ConvolutionOutput = GraphBuilder.CreateTexture(SpectralDesc, TEXT("FFT.Convolution"));
	ConvolutionWithTextureImage1D::MultiPass(
		GraphBuilder, ShaderMap,
		ConvolutionFFTDesc,
		GraphBuilder.CreateSRV(FRDGTextureSRVDesc(TwoForOneFFTOutput)), SpectralOutputRect,
		TransformedKernel,
		ConvolutionOutput);


	// ---- Inverse Two For One ---

	TwoForOneRealFFTImage1D::MultiPass(
		GraphBuilder, ShaderMap,
		TwoForOneIvnFFTDesc,
		GraphBuilder.CreateSRV(FRDGTextureSRVDesc(ConvolutionOutput)), SpectralOutputRect,
		DstTexture, DstRect,
		PreFilter,
		PostFilterParameters);
}
