// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
GPUFastFourierTransform.h: Interface for Fast Fourier Transform (FFT) on GPU.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Math/NumericLimits.h"
#include "GlobalShader.h"
#include "RendererInterface.h"
#include "RHI.h"
#include "RHIStaticStates.h"
#include "Shader.h"
#include "ShaderParameterUtils.h"
#include "RenderGraphDefinitions.h"

namespace GPUFFT
{

	/**
	* Enumerate the FFT direction and type for up to two dimensions.
	*/
	enum class FFT_XFORM_TYPE : int
	{
		FORWARD_HORIZONTAL = 0,
		FORWARD_VERTICAL   = 1,
		
		INVERSE_HORIZONTAL = 2,
		INVERSE_VERTICAL = 3
	};

	/**
	* Utility to get the inverse of a transform.
	*/
	static FFT_XFORM_TYPE GetInverseOfXForm(const FFT_XFORM_TYPE& XForm) 
	{
		return  static_cast<FFT_XFORM_TYPE> ((static_cast<int> (XForm) + 2) % 4);
	}

	/**
	* The direction of the associated transform type
	*/
	bool IsHorizontal(const FFT_XFORM_TYPE& XFormType);
	
	/**
	* To determine if the transform is inverse or forward
	*/
	bool IsForward(const FFT_XFORM_TYPE& XFormType);

	/**
	* Encapsulation of the transform type and transform extent.
	*/
	class FFTDescription
	{
	public:
		FFTDescription(const FFT_XFORM_TYPE& XForm, const FIntPoint& XFormExtent);
		FFTDescription() {};

		// The Transform extent used to construct the description 
		FIntPoint TransformExtent() const;
		bool IsHorizontal() const;
		bool IsForward() const;
		const TCHAR* FFT_TypeName() const;
		
		// member data public
		FFT_XFORM_TYPE  XFormType = FFT_XFORM_TYPE::FORWARD_HORIZONTAL;
		uint32 SignalLength = 0;
		uint32 NumScanLines = 0;
		ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute;
	};

	/**
	* The largest power of two length scan line that can
	* be FFT'd with group shared memory.  Larger sizes require
	* a much slower (@todo) code path
	*/
	uint32 MaxScanLineLength();// { return 4096; }

	/**
	*  The pixel format required for transforming rgba buffer.
	*/
	static EPixelFormat PixelFormat() { return PF_A32B32G32R32F; }

	/**
	* Define a prefilter to be applied to the pixel Luma, used when forward transforming image data.
	* Used to boost the incoming pixel luma of bright pixels - useful to accentuate the influence
	* of bright pixels during a convolution.
	*
	* PreFilter.x = MinLuma
	* PreFilter.y = MaxLuma
	* PreFilter.z = Multiplier
	*
	* When used in TwoForOneRealFFTImage, pixels Luma is limited by MaxLuma
	* and Luma greater than MinLuma is scaled as Multiplier * (Luma - MinLuma) + MinLuma;
	*/
	typedef FVector3f  FPreFilter;
	static bool IsActive(const FPreFilter& Filter) { return (Filter.X < Filter.Y); }


	struct ComplexFFTImage1D
	{
		/**
		* One Dimensional Fast Fourier Transform of two signals in a two dimensional buffer.
		* The each scanline of the float4 buffer is interpreted as two complex signals (x+iy and z + iw).
		*
		* On input a scanline (lenght N) contains 2 interlaced signals float4(r,g,b,a) = {z_n, q_n}.
		* where z_n = r_n + i g_n  and q_n = b_n + i a_n
		*
		* On output a scanline will be N  elements long with the following data layout
		* Entries k = [0,     N-1] hold  float4(Z_k, Q_k)
		* where the array {Z_k} is the transform of the signal {z_n}
		*
		* @param Context     - RHI and ShadersMap
		* @param FFTDesc     - Defines the scanline direction and count and transform type
		* @param SrcWindow   - Location of the data in the SrcTexture to be transformed. Zero values pad
		*                      the data to fit the  FFTDesc TrasformExtent.
		* @param SrcTexture  - The two dimensional float4 input buffer to transform
		* @param DstBuffer   - The buffer to be populated by the result of the transform.
		* @param TmpBuffer   - A temporary buffer used when ping-ponging between iterations.
		* @param bScrubNaNs  - A flag to replace NaNs and negative values with zero.
		*                      This is for use when reading in real data as the first step in two-for-one multipass
		* @return            - True on success. Will only fail if the scanline length is not a power of two.
		*                     Not required to fit in group shared memory.
		*
		* NB: This function does not transition resources.  That must be done by the calling scope.
		*/
		static void MultiPass(
			FRDGBuilder& GraphBuilder,
			const FGlobalShaderMap* ShaderMap,
			const FFTDescription& FFTDesc,
			FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
			FRDGTextureRef DstTexture,
			FRDGBufferRef PostFilterParameters = nullptr,
			const bool bScrubNaNs = false);

	};
	
	

	struct TwoForOneRealFFTImage1D
	{
		/**
		* One Dimensional Fast Fourier Transform of four real signals (rgba) in a two dimensional buffer.
		* The each scanline of the float4 buffer is interpreted four distinct real signals.
		*
		* On input a scanline (lenght N) contains 4 interlaced signals {r_n, g_n, b_n, a_n}.
		* On output a scanline will be N + 2 elements long with the following data layout
		* Entries i = [0,     N/2] hold  float4(R_i, B_i).
		* Entries i = [N/2+1, N-1] hold  float4(G_i, A_i)
		* Entries i = N            hold  float4(G_N/2, A_N/2)
		* Entry   i = N + 1        hold  float4(G_0, A_0)
		*
		* Here R_k, G_k ,B_k, A_k are complex (float2) values representing the transforms of the real signals r,g,b,a
		*
		* Note: Due to the fact that r,g,b,a are real signals, R_0, B_0, G_0, A_0 are real values.
		* and will equal the sum of the signals , i.e. R_0 = sum(r_i, i=0..N-1).
		*
		* Note if {x_n} is a real signal of length N, then the transfromed signal has the symmetry X_k = ComplexConjugate(X_{N-k})
		*
		* @param Context    - RHI and ShadersMap
		* @param FFTDesc    - Defines the scanline direction and power-of-two length and transform type
		* @param SrcWindow  - Location of the data in the SrcTexture to be transformed. Zero values pad
		*                     the data to fit the  FFTDesc TrasformExtent.
		* @param SrcTexture - The two dimensional float4 input buffer to transform
		*
		* @param DstBuffer  - The buffer to be populated by the result of the transform.
		* @param TmpBuffer  - A temporary target used in the multi-pass to ping-pong data.
		* @param PreFilter  - Values use to boost the intensity of any pixels.  Default value deactivates prefilter
		*                     The prefilter is only applied in the forward direction.
		*
		* @return           - True on success.  Will only fail if the buffer is too large for supported transform method.
		*
		* Requirements: On a forward transform, the DstBuffer must accommodate Context.SignalLength + 2 in the transform direction.
		*               On inverse transform the DstBuffer must accommodate Context.SignalLength in the transform direction.
		* NB: This function does not transition resources.  That must be done by the calling scope
		*/
		static void MultiPass(
			FRDGBuilder& GraphBuilder,
			const FGlobalShaderMap* ShaderMap,
			const FFTDescription& FFTDesc,
			FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
			FRDGTextureRef DstTexture, const FIntRect& DstWindow,
			const FPreFilter& PreFilter = FPreFilter(TNumericLimits<float>::Max(), TNumericLimits<float>::Lowest(), 0.f),
			FRDGBufferRef PostFilterParameters = nullptr);

	};

	struct ConvolutionWithTextureImage1D
	{
		/**
		* One Dimensional convolution against a pre-convolved texture: does two complex signals at once.
		*
		* The each scanline of the float4 buffer is interpreted as two complex signals (x+iy and z + iw).
		*
		* ------------------Breakdown of the convolution------------------------------------
		* On input:  Each scanline (length N) of the SrcTexture contains 2 interlaced signals
		* float4(a,b,c,d) = {z_n, q_n}  where z_n = a_n + i b_n  and q_n = c_n + i d_n
		*
		*            Each scanline (length N) of the TransformedKernel contains 2 interlaced, pre-transformed
		*            signals  float4( TK1, TK2)
		*
		* This function transforms the Src scanline to give two interlaced frequency signals float4(Z_k, Q_k)
		*
		* The convolution is affected as a multiplication in frequency space:
		*       float4( ComplexMult(Z_k, TK1_k), ComplexMult(Q_k, TK2_k))
		*
		* And inverse transform is applied to bring the result back to the input space.
		* ---------------------------------------------------------------------------------
		*
		*
		* @param Context    - RHI and ShadersMap
		* @param FFTDesc    - Defines the scanline direction and count and transform type
		* @param TransformedKernel - A texture that contains a transformed convolution kernel.
		* @param SrcTexture - The two dimensional float4 input buffer to transform
		* @param SrcWindow  - Location of the data in the SrcTexture to be transformed. Zero values pad
		*                     the data to fit the  FFTDesc TrasformExtent.
		* @param DstBuffer  - The buffer to be populated by the result of the transform.
		* @param TmpBuffer  - A temporary buffer used in the multi-pass formulation.
		*
		* @return           - True on success.  Will only fail if the buffer is too large for supported transform method
		*
		* NB: This function does not transition resources.  That must be done by the calling scope.
		*/
		static void MultiPass(
			FRDGBuilder& GraphBuilder,
			const FGlobalShaderMap* ShaderMap,
			const FFTDescription& FFTDesc,
			FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
			FRDGTextureRef TransformedKernel,
			FRDGTextureRef DstTexture);
	};


	/**
	* Two Dimensional transform of an image
	*
	* @param FrequencySize     - The nominal count of frequencies in both directions.
	* @param bHornizontalFirst - Defines the data layout of the 2d transform in spectral space.
	*
	* @param ROIRect     - The Region Of Interest: Defines the src data in the SrcTexture and the location of the result in the ResultUAV
	* @param SrcTexture  - Input float4 Image.  Each channel is a seperate signal to be convolved with the corresponding channel in the Kernel
	* @param ResultBuffer   - UAV buffer that will hold the result.
	* @param TmpBuffer0 - Temp float4 buffer with pixelformat PF_A32B32G32R32F
	*
	* NB: TmpBuffers must be (bHorizontalFirst) ? FIntPoint(FrequencySize .X + 2, ROIRect.Size().Y) : FIntPoint(ROIRect.Size().X, FrequencySize .Y + 2);
	*
	* NB: This function does not transition resources on the Src or Target.  That must be done by the calling scope.
	*
	*/
	void FFTImage2D(
		FRDGBuilder& GraphBuilder,
		const FGlobalShaderMap* ShaderMap,
		const FIntPoint& FrequencySize, bool bHorizontalFirst,
		FRDGTextureSRVRef SrcTexture, const FIntRect& ROIRect,
		FRDGTextureRef ResultTexture);



	

	/**
	* Two Dimensional convolution of an image with a pre-convolved kernel.
	* 
	* @param FrequencySize     - The nominal count of frequencies in both directions.
	* @param bHornizontalFirst - Defines the data layout of the 2d transform in spectral space.
	* @param TransformedKernel - The result of a two dimensional transform of the kernel.  Note, the transform order in constructing the 
	*                            TransformedKernel (e.g. bHorizontal first) must match the call here to have compatible data layout. 
	* @param ROIRect     - The Region Of Interest: Defines the src data in the SrcTexture and the location of the result in the ResultUAV
	* @param SrcTexture  - Input float4 Image.  Each channel is a seperate signal to be convolved with the corresponding channel in the Kernel
	* @param ResultUAV   - UAV buffer that will hold the result.
	* @param TmpBuffer0 - Temp float4 buffer with pixelformat PF_A32B32G32R32F 
	* @param TmpBuffer1 - Temp float4 buffer with pixelformat PF_A32B32G32R32F
	*
	* NB: The TransformedKernel should have the same frequencies and layout as transforming the SrcTexture in two dimensions 
	*     ie. FrequencySize + (bHorizontalFirst) ? FIntPoint(2, 0) : (0, 2);
	*
	* NB: both TmpBuffers must be (bHorizontalFirst) ? FIntPoint(FrequencySize .X + 2, ROIRect.Size().Y) : FIntPoint(ROIRect.Size().X, FrequencySize .Y + 2);
	*
	* NB: This function does not transition resources on the Src or Target.  That must be done by the calling scope.
	*
	* The Src and Dst textures can be re-used as the tmp buffers (Dst->TmpBuffer0, Src->TmpBuffer1) provided they are large enough.
	*/
	void ConvolutionWithTextureImage2D(
		FRDGBuilder& GraphBuilder,
		ERDGPassFlags ComputePassFlags,
		const FGlobalShaderMap* ShaderMap,
		const FIntPoint& FrequencySize,
		bool bHorizontalFirst,
		FRDGTextureRef TransformedKernel,
		FRDGTextureSRVRef SrcTexture, const FIntRect& ROIRect,
		FRDGTextureRef DstTexture, const FIntRect& DstRect,
		const FPreFilter& PreFilter,
		FRDGBufferRef PostFilterParameters,
		ETextureCreateFlags AdditionalTextureCreateFlags);

	FIntPoint Convolution2DBufferSize(const FIntPoint& FrequencySize, const bool bHorizontalFirst, const FIntPoint& SrcExtent);

	/**
	* Copy a float4 image and possibly amplify the intensity of select pixels.  
	*
	* @param Context    - RHI and ShadersMap
	* @param SrcWindow  - Location of the data in the SrcTexture to be copied.
	* @param SrcTexture - The two dimensional float4 input buffer to transform
	* @param DstWindow  - Window that images will be copied.
	* @param DstUVA     - The destination buffer
	* @param PreFilter  - Values use to boost the intensity of any pixels.  Default value deactivates prefilter
	*
	* Data in the SrcWindow is copied into the DstWindow with no-rescaling. If the SrcWindow
	* is smaller then the DstWindow, zeros will pad the result.  If the SrcWindow is larger 
	* then the DstWindow, data will be lost.

	* NB: This function does not transition resources.  That must be done by the calling scope.
	*/
	void CopyImage2D(
		FRDGBuilder& GraphBuilder,
		ERDGPassFlags ComputePassFlags,
		const FGlobalShaderMap* ShaderMap,
		FRDGTextureSRVRef SrcTexture, const FIntRect& SrcWindow,
		FRDGTextureRef DstTexture, const FIntRect& DstWindow,
		const FPreFilter& PreFilter = FPreFilter(TNumericLimits<float>::Max(), TNumericLimits<float>::Lowest(), 0.f));
};
