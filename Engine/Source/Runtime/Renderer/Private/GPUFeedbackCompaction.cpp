// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPUFeedbackCompaction.h"

#include "RenderUtils.h"

static bool PlatformSupportsFeedbackCompaction(EShaderPlatform Platform)
{
	// Enable for supported features that use feedback compaction.
	return DoesPlatformSupportLumenGI(Platform) || UseVirtualTexturing(Platform);
}

bool FBuildFeedbackHashTableIndirectArgsCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters & Parameters)
{
	return PlatformSupportsFeedbackCompaction(Parameters.Platform);
}

void FBuildFeedbackHashTableIndirectArgsCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), 1);
}

IMPLEMENT_GLOBAL_SHADER(FBuildFeedbackHashTableIndirectArgsCS, "/Engine/Private/GPUFeedbackCompaction.usf", "BuildFeedbackHashTableIndirectArgsCS", SF_Compute);

bool FBuildFeedbackHashTableCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return PlatformSupportsFeedbackCompaction(Parameters.Platform);
}

void FBuildFeedbackHashTableCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
}

IMPLEMENT_GLOBAL_SHADER(FBuildFeedbackHashTableCS, "/Engine/Private/GPUFeedbackCompaction.usf", "BuildFeedbackHashTableCS", SF_Compute);

bool FCompactFeedbackHashTableCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return PlatformSupportsFeedbackCompaction(Parameters.Platform);
}

void FCompactFeedbackHashTableCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
}

IMPLEMENT_GLOBAL_SHADER(FCompactFeedbackHashTableCS, "/Engine/Private/GPUFeedbackCompaction.usf", "CompactFeedbackHashTableCS", SF_Compute);
