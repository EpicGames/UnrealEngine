// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterStruct.h"

// Setups FBuildFeedbackHashTableCS arguments to run one lane per feedback element
class FBuildFeedbackHashTableIndirectArgsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildFeedbackHashTableIndirectArgsCS);
	SHADER_USE_PARAMETER_STRUCT(FBuildFeedbackHashTableIndirectArgsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWBuildHashTableIndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, FeedbackBufferAllocator)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, FeedbackBuffer)
		SHADER_PARAMETER(uint32, FeedbackBufferSize)
	END_SHADER_PARAMETER_STRUCT()

	class FFeedbackBufferStride : SHADER_PERMUTATION_RANGE_INT("FEEDBACK_BUFFER_STRIDE", 1, 2);
	using FPermutationDomain = TShaderPermutationDomain<FFeedbackBufferStride>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

// Takes a list of feedback elements and builds a hash table with element counts
class FBuildFeedbackHashTableCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildFeedbackHashTableCS)
	SHADER_USE_PARAMETER_STRUCT(FBuildFeedbackHashTableCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		RDG_BUFFER_ACCESS(BuildHashTableIndirectArgs, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWHashTableKeys)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWHashTableElementIndices)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWHashTableElementCounts)
		SHADER_PARAMETER(uint32, HashTableSize)
		SHADER_PARAMETER(uint32, HashTableIndexWrapMask)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, FeedbackBufferAllocator)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, FeedbackBuffer)
		SHADER_PARAMETER(uint32, FeedbackBufferSize)
	END_SHADER_PARAMETER_STRUCT()

	class FFeedbackBufferStride : SHADER_PERMUTATION_RANGE_INT("FEEDBACK_BUFFER_STRIDE", 1, 2);
	using FPermutationDomain = TShaderPermutationDomain<FFeedbackBufferStride>;

	static uint32 GetGroupSize() { return 64; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

// Compacts feedback element hash table into a unique and tightly packed array of feedback elements with counts
class FCompactFeedbackHashTableCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompactFeedbackHashTableCS)
	SHADER_USE_PARAMETER_STRUCT(FCompactFeedbackHashTableCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint2>, RWCompactedFeedbackBuffer)
		SHADER_PARAMETER(uint32, CompactedFeedbackBufferSize)
		SHADER_PARAMETER(uint32, CompactedFeedbackCountShiftBits)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, HashTableElementIndices)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, HashTableElementCounts)
		SHADER_PARAMETER(uint32, HashTableSize)
		SHADER_PARAMETER(uint32, HashTableIndexWrapMask)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, FeedbackBufferAllocator)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, FeedbackBuffer)
		SHADER_PARAMETER(uint32, FeedbackBufferSize)
	END_SHADER_PARAMETER_STRUCT()

	class FFeedbackBufferStride : SHADER_PERMUTATION_RANGE_INT("FEEDBACK_BUFFER_STRIDE", 1, 2);
	using FPermutationDomain = TShaderPermutationDomain<FFeedbackBufferStride>;

	static uint32 GetGroupSize() { return 64; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};
