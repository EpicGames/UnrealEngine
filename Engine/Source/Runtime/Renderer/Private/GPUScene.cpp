// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GPUScene.cpp
=============================================================================*/

#include "GPUScene.h"
#include "Skinning/SkinningSceneExtension.h"
#include "CoreMinimal.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHI.h"
#include "SceneUtils.h"
#include "ScenePrivate.h"
#include "LogRenderer.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Rendering/NaniteResources.h"
#include "Async/ParallelFor.h"
#include "VirtualShadowMaps/VirtualShadowMapCacheManager.h"
#include "NaniteSceneProxy.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "InstanceUniformShaderParameters.h"
#include "ShaderPrint.h"
#include "RenderCore.h"
#include "LightSceneData.h"
#include "LightSceneProxy.h"
#include "SystemTextures.h"
#include "SceneDefinitions.h"
#include "PrimitiveSceneShaderData.h"
#include "RendererOnScreenNotification.h"
#include "InstanceCulling/InstanceCullingOcclusionQuery.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "InstanceDataSceneProxy.h"
#include "SceneRendererInterface.h"
#include "UnrealEngine.h"

// Useful for debugging
#define FORCEINLINE_GPUSCENE FORCEINLINE
//#define FORCEINLINE_GPUSCENE FORCENOINLINE

static_assert(
	sizeof(FLightSceneData) == SIZEOF_LIGHT_SCENE_DATA,
	"FLightSceneData has changed without update to SIZEOF_LIGHT_SCENE_DATA. "
	"This macro is required because sizeof() is not available in FXC and therefore not available for PCD3D_SM5 targets."
);

// Defaults to being disabled, enable using the command line argument: -CsvCategory GPUScene
CSV_DEFINE_CATEGORY(GPUScene, false);

DEFINE_GPU_STAT(GPUSceneUpdate);

#define LOG_INSTANCE_ALLOCATIONS 0

static void ConstructDefault(FGPUSceneResourceParameters& GPUScene, FRDGBuilder& GraphBuilder)
{
	FRDGBufferRef DummyBufferVec4 = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f));
	GPUScene.GPUSceneInstanceSceneData = GraphBuilder.CreateSRV(DummyBufferVec4);
	GPUScene.GPUSceneInstancePayloadData = GraphBuilder.CreateSRV(DummyBufferVec4);
	GPUScene.GPUScenePrimitiveSceneData = GraphBuilder.CreateSRV(DummyBufferVec4);
	GPUScene.GPUSceneLightmapData = GraphBuilder.CreateSRV(DummyBufferVec4);
	FRDGBufferRef DummyBufferLight = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FLightSceneData));
	GPUScene.GPUSceneLightData = GraphBuilder.CreateSRV(DummyBufferLight);
	GPUScene.CommonParameters.GPUSceneFrameNumber = 0u;
	GPUScene.CommonParameters.GPUSceneMaxAllocatedInstanceId = 0;
	GPUScene.CommonParameters.GPUSceneMaxPersistentPrimitiveIndex = 0;
	GPUScene.CommonParameters.GPUSceneNumLightmapDataItems = 0;
	GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeLog2 = 0u;
	GPUScene.CommonParameters.GPUSceneInstanceDataTileSizeMask = 0u;
	GPUScene.CommonParameters.GPUSceneInstanceDataTileStride = 0u;
}

IMPLEMENT_SCENE_UB_STRUCT(FGPUSceneResourceParameters, GPUScene, ConstructDefault);

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FGPUSceneWriterUniformParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneCommonParameters, CommonParameters)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, GPUSceneInstanceSceneDataRW)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, GPUSceneInstancePayloadDataRW)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, GPUScenePrimitiveSceneDataRW)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FGPUSceneWriterUniformParameters, "GPUSceneWriter");

// Max buffer size is 2GB and also largest POT in 32 bits. 2GB / sizeof(float4) <=> 2 << (30 - 4)
constexpr uint64 MaxPayloadBufferSizeInFloat4 = uint64(2) << (30 - 4);

static TAutoConsoleVariable<int32> CVarGPUSceneUploadEveryFrame(
	TEXT("r.GPUScene.UploadEveryFrame"),
	0,
	TEXT("Whether to upload the entire scene's primitive data every frame.  Useful for debugging."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarGPUSceneMaxPooledUploadBufferSize(
	TEXT("r.GPUScene.MaxPooledUploadBufferSize"),
	256000,
	TEXT("Maximum size of GPU Scene upload buffer size to pool."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarGPUSceneParallelUpdate(
	TEXT("r.GPUScene.ParallelUpdate"),
	2048,
	TEXT("0  - disables parallel for updates of GPU scene primitives and instances.")
	TEXT(">0 - enables parallel for updates if the number of items exceeds N."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarGPUSceneDebugMode(
	TEXT("r.GPUScene.DebugMode"),
	0,
	TEXT("Debug Rendering Mode:\n")
	TEXT("0 - (show nothing, decault)\n")
	TEXT(" 1 - Draw All\n")
	TEXT(" 2 - Draw Selected (in the editor)\n")
	TEXT(" 3 - Draw Updated (updated this frame)\n")
	TEXT("You can use r.GPUScene.DebugDrawRange to limit the range\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarGPUSceneDebugDrawRange(
	TEXT("r.GPUScene.DebugDrawRange"),
	-1.0f,
	TEXT("Maximum distance the to draw instance bounds, the default is -1.0 <=> infinite range."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarGPUSceneUseGrowOnlyAllocationPolicy(
	TEXT("r.GPUScene.UseGrowOnlyAllocationPolicy"),
	0,
	TEXT("Deprecated 5.3. If set to 1 the allocators used for GPU-scene instances and similar will use a grow-only allocation policy to mimic the behavior in 5.2 and earlier.\n")
	TEXT("  Disabled by default, which means that the buffers can shrink as well as grow."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarGPUSceneLightsAsyncSetup(
	TEXT("r.GPUScene.Lights.AsyncSetup"),
	1,
	TEXT("Whether to run GPU Scene Lights setup in an async task."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarGPUSceneInstanceDataTileSizeLog2(
	TEXT("r.GPUScene.InstanceDataTileSizeLog2"),
	12,
	TEXT("Log2 of tile size used for the instance data in number of instances. \n")
	TEXT("  <0 indicates no tiling and disables reserved resources. \n")
	TEXT("  >0 (default) will use the given tile-size and attempt to use reserved resources, but falls back to a regular buffer if not available."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarGPUSceneUseReservedResources(
	TEXT("r.GPUScene.UseReservedResources"),
	true,
	TEXT("Turning this on makes all GPU-Scene buffers default to using reserved resource allocations."),
	ECVF_ReadOnly);

static TAutoConsoleVariable<bool> CVarGPUSceneViewDistanceScaleWorkaround(
	TEXT("r.GPUScene.ViewDistanceScaleWorkaround"),
	true,
	TEXT("If true (default) the whole GPU scene is reuploaded whenever the view distance scale is changed to work around the fact that the scale is baked into some primitive data."),
	ECVF_RenderThreadSafe
);
LLM_DECLARE_TAG_API(GPUScene, RENDERER_API);
DECLARE_LLM_MEMORY_STAT(TEXT("GPUScene"), STAT_GPUSceneLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("GPUScene"), STAT_GPUSceneSummaryLLM, STATGROUP_LLM);
LLM_DEFINE_TAG(GPUScene, NAME_None, NAME_None, GET_STATFNAME(STAT_GPUSceneLLM), GET_STATFNAME(STAT_GPUSceneSummaryLLM));

struct FParallelUpdateRange
{
	int32 ItemStart;
	int32 ItemCount;
};

struct FParallelUpdateRanges
{
	FParallelUpdateRange Range[4];
};

// TODO: Improve and move to shared utility location.
static int32 PartitionUpdateRanges(FParallelUpdateRanges& Ranges, int32 ItemCount, bool bAllowParallel)
{
	if (ItemCount < 256 || !bAllowParallel)
	{
		Ranges.Range[0].ItemStart = 0;
		Ranges.Range[0].ItemCount = ItemCount;
		return 1;
	}

	const int32 RangeCount = Align(ItemCount, 4) >> 2;

	Ranges.Range[0].ItemCount = RangeCount;
	Ranges.Range[1].ItemCount = RangeCount;
	Ranges.Range[2].ItemCount = RangeCount;

	Ranges.Range[0].ItemStart = 0;
	Ranges.Range[1].ItemStart = RangeCount;
	Ranges.Range[2].ItemStart = RangeCount * 2;
	Ranges.Range[3].ItemStart = RangeCount * 3;
	Ranges.Range[3].ItemCount = ItemCount - Ranges.Range[3].ItemStart;

	return Ranges.Range[3].ItemCount > 0 ? 4 : 3;
}

namespace UE::Renderer::Private
{
FORCEINLINE_GPUSCENE uint32 PackInstanceDataFlags(FInstanceDataFlags Flags)
{
	uint32 PackedFlags = 0x0;
	PackedFlags |= Flags.bHasPerInstanceRandom ? INSTANCE_SCENE_DATA_FLAG_HAS_RANDOM : 0u;
	PackedFlags |= Flags.bHasPerInstanceCustomData ? INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA : 0u;
	PackedFlags |= Flags.bHasPerInstanceDynamicData ? INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA : 0u;
	PackedFlags |= Flags.bHasPerInstanceSkinningData ? INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA : 0u;
	PackedFlags |= Flags.bHasPerInstanceLMSMUVBias ? INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS : 0u;
	PackedFlags |= Flags.bHasPerInstanceHierarchyOffset ? INSTANCE_SCENE_DATA_FLAG_HAS_HIERARCHY_OFFSET : 0u;
	PackedFlags |= Flags.bHasPerInstanceLocalBounds ? INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS : 0u;
	PackedFlags |= Flags.bHasPerInstancePayloadExtension ? INSTANCE_SCENE_DATA_FLAG_HAS_PAYLOAD_EXTENSION : 0u;
#if WITH_EDITOR
	PackedFlags |= Flags.bHasPerInstanceEditorData ? INSTANCE_SCENE_DATA_FLAG_HAS_EDITOR_DATA : 0u;
#endif

	return PackedFlags;
}
}

void FGPUScenePrimitiveCollector::Add(
	const FMeshBatchDynamicPrimitiveData* MeshBatchData,
	const FPrimitiveUniformShaderParameters& PrimitiveShaderParams,
	uint32 NumInstances,
	uint32& OutPrimitiveIndex,
	uint32& OutInstanceSceneDataOffset)
{
	check(GPUSceneDynamicContext != nullptr);
	check(!bCommitted);
	

	// Lazy allocation of the upload data to not waste space and processing if none was needed.
	if (UploadData == nullptr)
	{
		UploadData = AllocateUploadData();
	}

	const int32 PrimitiveIndex = UploadData->PrimitiveData.Num();
	FPrimitiveData& PrimitiveData = UploadData->PrimitiveData.AddDefaulted_GetRef();		

	if (MeshBatchData != nullptr)
	{
		// make sure the source data is appropriately structured
		MeshBatchData->Validate(NumInstances);
		PrimitiveData.SourceData = *MeshBatchData;
	}
	
	const int32 PayloadFloat4Stride = PrimitiveData.SourceData.GetPayloadFloat4Stride();
	
	PrimitiveData.ShaderParams = &PrimitiveShaderParams;
	PrimitiveData.NumInstances = NumInstances;
	PrimitiveData.LocalInstanceSceneDataOffset = UploadData->TotalInstanceCount;
	PrimitiveData.LocalPayloadDataOffset = PayloadFloat4Stride > 0 ? UploadData->InstancePayloadDataFloat4Count : INDEX_NONE;

	UploadData->TotalInstanceCount += NumInstances;	
	UploadData->InstancePayloadDataFloat4Count += PayloadFloat4Stride * NumInstances;

	if (PrimitiveData.SourceData.DataWriterGPU.IsBound())
	{
		// Enqueue this primitive data to be executed (either upon upload or deferred to a later GPU write pass)
		UploadData->GPUWritePrimitives.Add(PrimitiveIndex);
	}

	// Set the output data offsets
	OutPrimitiveIndex = PrimitiveIndex;
	OutInstanceSceneDataOffset = PrimitiveData.LocalInstanceSceneDataOffset;
}

const FPrimitiveUniformShaderParameters* FGPUScenePrimitiveCollector::GetPrimitiveShaderParameters(int32 DrawPrimitiveId) const
{
	if (UploadData != nullptr && (DrawPrimitiveId & GPrimIDDynamicFlag) != 0)
	{
		int32 DynamicPrimitiveIndex = DrawPrimitiveId & (~GPrimIDDynamicFlag);
		if (UploadData->PrimitiveData.IsValidIndex(DynamicPrimitiveIndex))
		{
			return UploadData->PrimitiveData[DynamicPrimitiveIndex].ShaderParams;
		}
	}
	return nullptr;
}

#if DO_CHECK

void FGPUScenePrimitiveCollector::CheckPrimitiveProcessed(uint32 PrimitiveIndex, const FGPUScene& GPUScene) const
{
	checkf(UploadData != nullptr && bCommitted, TEXT("Dynamic Primitive index %u has not been fully processed. The collector hasn't collected anything or hasn't been uploaded."), PrimitiveIndex);

	if (UploadData != nullptr)
	{
		checkf(PrimitiveIndex < uint32(UploadData->PrimitiveData.Num()), TEXT("Dynamic Primitive index %u has not been fully processed. The specified index is out of range [0,%d)."), PrimitiveIndex, UploadData->PrimitiveData.Num());

		// Early out to avoid the HasPendingGPUWrite check for cases where it cannot happen anyway.
		const FMeshBatchDynamicPrimitiveData& SourceData = UploadData->PrimitiveData[PrimitiveIndex].SourceData;
		if (!SourceData.DataWriterGPU.IsBound() || SourceData.DataWriterGPUPass == EGPUSceneGPUWritePass::None)
		{
			return;
		}
	}

	// If the GPU scene still has a pending deferred write for the primitive, then it has not been fully processed yet
	const uint32 PrimitiveId = GetPrimitiveIdRange().GetLowerBoundValue() + PrimitiveIndex;
	checkf(!GPUScene.HasPendingGPUWrite(PrimitiveId), TEXT("Dynamic Primitive index %u has not been fully processed. The GPU scene still has a pending deferred write for the primitive, it has not been fully processed yet."), PrimitiveIndex);
}

#endif // DO_CHECK

void FGPUScenePrimitiveCollector::Commit()
{
	ensure(!bCommitted);
	if (UploadData)
	{
		PrimitiveIdRange = GPUSceneDynamicContext->GPUScene.CommitPrimitiveCollector(*this);
	}
	bCommitted = true;
}

FGPUScenePrimitiveCollector::FUploadData* FGPUScenePrimitiveCollector::AllocateUploadData()
{
	return GPUSceneDynamicContext->AllocateDynamicPrimitiveData();
}

/**
 * Info needed by the uploader to prepare to upload a primitive.
 */
struct FPrimitiveUploadInfoHeader
{
	uint32 PrimitiveID = INVALID_PRIMITIVE_ID;

	/** Optional */
	int32 InstanceSceneDataOffset = INDEX_NONE;
	int32 NumInstanceUploads = 0;
	int32 InstancePayloadDataOffset = 0;
	int32 InstancePayloadDataStride = 0;
	int32 LightmapUploadCount = 0;
	// Derived: Number of instances to upload payload for, may be clamped to fit in the buffer.
	int32 NumPayloadUploadInstances = 0;
	/** NaniteSceneProxy must be set if the proxy is a Nanite proxy */
	const Nanite::FSceneProxyBase* NaniteSceneProxy = nullptr;
	const FPrimitiveSceneInfo* PrimitiveSceneInfo = nullptr;
};

#if DO_CHECK
void ValidateInstanceUploadInfo(const FInstanceUploadInfo& UploadInfo, FRDGBuffer* InstancePayloadDataBuffer)
{
	const bool bHasRandomID				= (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_RANDOM) != 0u;
	const bool bHasCustomData			= (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA) != 0u;
	const bool bHasDynamicData			= (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA) != 0u;
	const bool bHasLightShadowUVBias	= (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS) != 0u;
	const bool bHasSkinningData			= (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA) != 0u;
	const bool bHasHierarchyOffset		= (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_HIERARCHY_OFFSET) != 0u;
	const bool bHasLocalBounds			= (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS) != 0u;
	const bool bHasPayloadExtension		= (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_PAYLOAD_EXTENSION) != 0u;
#if WITH_EDITOR
	const bool bHasEditorData			= (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_EDITOR_DATA) != 0u;
#endif

	check(!bHasRandomID || UploadInfo.InstanceRandomID.Num() == UploadInfo.NumInstances);
	check(UploadInfo.InstanceLightShadowUVBias.Num()	== (bHasLightShadowUVBias ? UploadInfo.NumInstances : 0));
	check(UploadInfo.InstanceSkinningData.Num()			== (bHasSkinningData ? UploadInfo.NumInstances : 0));
	check(UploadInfo.InstanceHierarchyOffset.Num()		== (bHasHierarchyOffset	? UploadInfo.NumInstances : 0));
#if WITH_EDITOR
	check(UploadInfo.InstanceEditorData.Num()			== (bHasEditorData ? UploadInfo.NumInstances : 0));
#endif

	if (bHasCustomData)
	{
		//check(UploadInfo.InstanceCustomDataCount > 0);
		check(UploadInfo.InstanceCustomDataCount * UploadInfo.NumInstances == UploadInfo.InstanceCustomData.Num());
	}
	else
	{
		check(UploadInfo.InstanceCustomDataCount == 0);
	}

	if (bHasPayloadExtension)
	{
		check(UploadInfo.InstancePayloadExtensionCount > 0);
		check(UploadInfo.InstancePayloadExtensionCount <= PRIMITIVE_SCENE_DATA_MAX_PAYLOAD_EXTENSION_SIZE);
		check(UploadInfo.InstancePayloadExtensionCount * UploadInfo.NumInstances == UploadInfo.InstancePayloadExtension.Num());
	}
	else
	{
		check(UploadInfo.InstancePayloadExtensionCount == 0 && UploadInfo.InstancePayloadExtension.Num() == 0);
	}

	if (UploadInfo.InstanceSceneDataBuffers != nullptr)
	{
		if (UploadInfo.InstanceSceneDataBuffers->GetNumInstances() > 0 && !ensure(UploadInfo.InstanceSceneDataBuffers->GetPrimitiveToRelativeWorld().Equals(UploadInfo.PrimitiveToWorld)))
		{
			UE_LOGF(LogTemp, Warning, "Mismatched Primitive transform! primitive ID %d", UploadInfo.PrimitiveID);
		}
		check(UploadInfo.InstanceSceneDataBuffers->GetReadView().InstanceToPrimitiveRelative.Num() == UploadInfo.NumInstances);

		// TODO: we may alias with the InstanceToPrimitiveRelative if no per-instance transforms have been requested...
		check(UploadInfo.InstanceSceneDataBuffers->GetReadView().PrevInstanceToPrimitiveRelative.Num() == (bHasDynamicData ? UploadInfo.NumInstances : 0));
	}
	else
	{
		check(UploadInfo.InstanceDynamicData.Num()	== (bHasDynamicData	? UploadInfo.NumInstances : 0));
		check(UploadInfo.NumInstances == UploadInfo.PrimitiveInstances.Num()  || UploadInfo.NumInstances == 1 && UploadInfo.PrimitiveInstances.IsEmpty());
	}

	if (UploadInfo.PrimitiveSceneInfo)
	{
		check(UploadInfo.PrimitiveSceneInfo->GetNumInstanceSceneDataEntries() == UploadInfo.NumInstances);
	}

}
#else
FORCEINLINE void ValidateInstanceUploadInfo(const FInstanceUploadInfo& , FRDGBuffer* ) {}
#endif

/**
 * Info required by the uploader to update the lightmap data for a primitive.
 */
struct FLightMapUploadInfo
{
	FPrimitiveSceneProxy::FLCIArray LCIs;
	int32 LightmapDataOffset = 0;
};

/**
 * Implements a thin data abstraction such that the UploadGeneral function can upload primitive data from
 * both scene primitives and dynamic primitives (which are not stored in the same way). 
 */
struct FUploadDataSourceAdapterScenePrimitives
{
	FUploadDataSourceAdapterScenePrimitives(FScene& InScene, uint32 InSceneFrameNumber, TArray<FPersistentPrimitiveIndex> InPrimitivesToUpdate, TArray<EPrimitiveDirtyState> InPrimitiveDirtyState)
		: Scene(InScene)
		, SceneFrameNumber(InSceneFrameNumber)
		, PrimitivesToUpdate(MoveTemp(InPrimitivesToUpdate))
		, PrimitiveDirtyState(MoveTemp(InPrimitiveDirtyState))
	{}

	/**
	 * Return the number of primitives to upload N, GetPrimitiveInfoHeader/GetPrimitiveShaderData will be called with ItemIndex in [0,N).
	 */
	FORCEINLINE_GPUSCENE int32 NumPrimitivesToUpload() const 
	{ 
		return PrimitivesToUpdate.Num(); 
	}

	FORCEINLINE_GPUSCENE TArrayView<const uint32> GetItemPrimitiveIds() const
	{
		return TArrayView<const uint32>(reinterpret_cast<const uint32 *>(PrimitivesToUpdate.GetData()), PrimitivesToUpdate.Num());
	}

	/**
	 * Populate the primitive info for a given item index.
	 * 
	 */
	FORCEINLINE_GPUSCENE void GetPrimitiveInfoHeader(int32 ItemIndex, FPrimitiveUploadInfoHeader& PrimitiveUploadInfo) const
	{
		const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitivesToUpdate[ItemIndex];
		const int32 PrimitiveID = Scene.GetPrimitiveIndex(PersistentPrimitiveIndex);
		check(Scene.PrimitiveSceneProxies.IsValidIndex(PrimitiveID));

		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy = Scene.PrimitiveSceneProxies[PrimitiveID];
		const FPrimitiveSceneInfo* RESTRICT PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();

		PrimitiveUploadInfo.PrimitiveID = uint32(PersistentPrimitiveIndex.Index);
		PrimitiveUploadInfo.LightmapUploadCount = PrimitiveSceneInfo->GetNumLightmapDataEntries();
		PrimitiveUploadInfo.NaniteSceneProxy = PrimitiveSceneProxy->IsNaniteMesh() ? static_cast<const Nanite::FSceneProxyBase*>(PrimitiveSceneProxy) : nullptr;
		PrimitiveUploadInfo.PrimitiveSceneInfo = PrimitiveSceneInfo;

		PrimitiveUploadInfo.InstancePayloadDataOffset = PrimitiveSceneInfo->GetInstancePayloadDataOffset();
		PrimitiveUploadInfo.InstancePayloadDataStride = PrimitiveSceneInfo->GetInstancePayloadDataStride();

		PrimitiveUploadInfo.InstanceSceneDataOffset = PrimitiveSceneInfo->GetInstanceSceneDataOffset();
		// TODO: Somehow associate the procedural update with the update entry such that we can cross validate the existence of a writer here?
		if (PrimitiveSceneProxy->IsInstanceDataGPUOnly())
		{
			PrimitiveUploadInfo.NumInstanceUploads = 0;
		}
		else
		{
			PrimitiveUploadInfo.NumInstanceUploads = PrimitiveSceneInfo->GetNumInstanceSceneDataEntries();
		}
		check(PrimitiveUploadInfo.NumInstanceUploads == 0 || PrimitiveUploadInfo.InstanceSceneDataOffset >= 0);
	}

	/**
	 * Populate the primitive info for a given item index.
	 * 
	 */
	FORCEINLINE_GPUSCENE void GetPrimitiveShaderData(int32 ItemIndex, FVector4f* RESTRICT OutData) const
	{
		const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitivesToUpdate[ItemIndex];
		const int32 PrimitiveID = Scene.GetPrimitiveIndex(PersistentPrimitiveIndex);
		if (ensure(Scene.PrimitiveSceneProxies.IsValidIndex(PrimitiveID)))
		{
			const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy = Scene.PrimitiveSceneProxies[PrimitiveID];
			FPrimitiveSceneShaderData::BuildDataFromProxy(PrimitiveSceneProxy, OutData);
		}
	}

	FORCEINLINE_GPUSCENE void GetInstanceInfo(int32 ItemIndex, FInstanceUploadInfo& InstanceUploadInfo) const
	{
		const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitivesToUpdate[ItemIndex];
		const int32 PrimitiveID = Scene.GetPrimitiveIndex(PersistentPrimitiveIndex);

		check(Scene.PrimitiveSceneProxies.IsValidIndex(PrimitiveID));
		check(PrimitiveDirtyState[PersistentPrimitiveIndex.Index] != EPrimitiveDirtyState::Removed);

		FPrimitiveSceneProxy* PrimitiveSceneProxy = Scene.PrimitiveSceneProxies[PrimitiveID];
		InstanceUploadInfo.SetPrimitiveData(PrimitiveSceneProxy, PersistentPrimitiveIndex, SceneFrameNumber);
	}

	FORCEINLINE_GPUSCENE bool GetLightMapInfo(int32 ItemIndex, FLightMapUploadInfo &UploadInfo) const
	{
		const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitivesToUpdate[ItemIndex];
		const int32 PrimitiveID = Scene.GetPrimitiveIndex(PersistentPrimitiveIndex);
		if (Scene.PrimitiveSceneProxies.IsValidIndex(PrimitiveID))
		{
			FPrimitiveSceneProxy* PrimitiveSceneProxy = Scene.PrimitiveSceneProxies[PrimitiveID];

			PrimitiveSceneProxy->GetLCIs(UploadInfo.LCIs);
			check(UploadInfo.LCIs.Num() == PrimitiveSceneProxy->GetPrimitiveSceneInfo()->GetNumLightmapDataEntries());
			UploadInfo.LightmapDataOffset = PrimitiveSceneProxy->GetPrimitiveSceneInfo()->GetLightmapDataOffset();
			return true;
		}

		return false;
	}

	FScene& Scene;
	const uint32 SceneFrameNumber;
	TArray<FPersistentPrimitiveIndex> PrimitivesToUpdate;
	TArray<EPrimitiveDirtyState> PrimitiveDirtyState;
};

void FGPUScene::SetEnabled(EShaderPlatform InShaderPlatform)
{
	ShaderPlatform = InShaderPlatform;
	bIsEnabled = UseGPUScene(ShaderPlatform);
}

FGPUScene::FGPUScene(FScene &InScene)
	: bUpdateAllPrimitives(false)
	, InstancePayloadDataAllocator(CVarGPUSceneUseGrowOnlyAllocationPolicy.GetValueOnAnyThread() != 0)
	, LightmapDataAllocator(CVarGPUSceneUseGrowOnlyAllocationPolicy.GetValueOnAnyThread() != 0)
	, LightDataBuffer(InitialBufferSize, TEXT("GPUScene.LightData"))
	, Scene(InScene)
	, InstanceSceneDataAllocator(CVarGPUSceneUseGrowOnlyAllocationPolicy.GetValueOnAnyThread() != 0)
{
#if !UE_BUILD_SHIPPING
	ScreenMessageDelegate = FRendererOnScreenNotification::Get().AddLambda([this](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText >& OutMessages)
	{
		if (MaxInstancesDuringPrevUpdate >= MAX_INSTANCE_ID)
		{
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("GPU-Scene Instance data overflow detected, reduce instance count to avoid rendering artifacts"))));
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("  Max allocated ID %d, max instance buffer size: %dM"), MaxInstancesDuringPrevUpdate, MAX_INSTANCE_ID >> 20)));
			if (!bLoggedInstanceOverflow )
			{
				UE_LOGF(LogRenderer, Warning, "GPU-Scene Instance data overflow detected, reduce instance count to avoid rendering artifacts.\n"
					" Max allocated ID %d (%0.3fM), instance buffer size: %dM", MaxInstancesDuringPrevUpdate, double(MaxInstancesDuringPrevUpdate) / (1024.0 * 1024.0),  MAX_INSTANCE_ID >> 20);
				bLoggedInstanceOverflow = true;
			}
		}
		else
		{
			bLoggedInstanceOverflow = false;
		}

		if (InstancePayloadDataSizeReserveUnclamped  >= uint64(MaxPayloadBufferSizeInFloat4))
		{
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("GPU-Scene Payload data overflow detected, reduce instance count and/or e.g, use of custom data to avoid rendering artifacts and potentially crashes (depending on the use of payload data)"))));
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("  Buffer size requested %dM, max instance buffer size (in float4s): %lldM"), uint32(InstancePayloadDataSizeReserveUnclamped >> 20), MaxPayloadBufferSizeInFloat4 >> 20)));
			if (!bLoggedPayloadBufferOverflow )
			{
				UE_LOGF(LogRenderer, Warning, "GPU-Scene Payload data overflow detected, reduce instance count and/or e.g, use of custom data to avoid rendering artifacts and potentially crashes (depending on the use of payload data).\n"
					" Attempted allocation (POT) %dM, instance buffer size: %lldM", uint32(InstancePayloadDataSizeReserveUnclamped >> 20),  MaxPayloadBufferSizeInFloat4 >> 20);
				bLoggedPayloadBufferOverflow = true;
			}
		}
		else
		{
			bLoggedPayloadBufferOverflow = false;
		}
		if (NumGPULights < MaxLightsDuringPrevUpdate)
		{
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("GPU-Scene Light data overflow detected, reduce light count to avoid rendering artifacts"))));
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("  Max allocated ID %d, light buffer capacity: %d"), MaxLightsDuringPrevUpdate, NumGPULights)));
			if (!bLoggedLightOverflow)
			{
				UE_LOGF(LogRenderer, Warning, "GPU-Scene Light data overflow detected, reduce light count to avoid rendering artifacts.\n"
					" Max allocated ID %d, light buffer capacity: %d", MaxLightsDuringPrevUpdate, NumGPULights);
				bLoggedLightOverflow = true;
			}
		}
		else
		{
			bLoggedLightOverflow = false;
		}
	});
#endif
}


FGPUScene::~FGPUScene()
{
#if !UE_BUILD_SHIPPING
	FRendererOnScreenNotification::Get().Remove(ScreenMessageDelegate);
#endif
}

void FGPUScene::BeginRender(FRDGBuilder& GraphBuilder, FGPUSceneDynamicContext &GPUSceneDynamicContext)
{
	ensure(!bInBeginEndBlock);
	ensure(CurrentDynamicContext == nullptr);

	LastDeferredGPUWritePass = EGPUSceneGPUWritePass::None;
	CurrentDynamicContext = &GPUSceneDynamicContext;
	bInBeginEndBlock = true;
	check(!CachedRegisteredBuffers.IsValid());
	if (bIsEnabled)
	{
		// Should always be reset to this as the neutral state.
		check(DynamicPrimitivesOffset == Scene.GetMaxPersistentPrimitiveIndex());
		// Do it anyway for old times sake
		DynamicPrimitivesOffset = Scene.GetMaxPersistentPrimitiveIndex();

		CachedRegisteredBuffers = RegisterBuffers(GraphBuilder);
	}
}

void FGPUScene::EndRender()
{
	ensure(bInBeginEndBlock);
	ensure(CurrentDynamicContext != nullptr);

	// record the previous max primitives to avoid oscillating allocations.
	PrevDynamicPrimitivesMaxSize = DynamicPrimitivesOffset;

	// record number of dynamic allocs for next frame to avoid oscillations
	PrevDynamicPrimitivePayloadMaxSize = InstancePayloadDataAllocator.GetMaxSize();
	PrevDynamicPrimitiveInstanceMaxSize = InstanceSceneDataAllocator.GetMaxSize();

	// Pop all dynamic primitives off the stack
	DynamicPrimitivesOffset = Scene.GetMaxPersistentPrimitiveIndex();
	bInBeginEndBlock = false;
	CurrentDynamicContext = nullptr;
	CachedRegisteredBuffers = {};
}

void FGPUScene::UpdateGPULights(FRDGBuilder& GraphBuilder, const UE::Tasks::FTask& PrerequisiteTask, const FLightSceneChangeSet& LightsPostUpdateData)
{
#if !UE_BUILD_SHIPPING
	MaxLightsDuringPrevUpdate = uint32(Scene.Lights.GetMaxIndex());
#endif // UE_BUILD_SHIPPING

	// Clamp buffer to be smaller than the MAX_LIGHT_ID.
	NumGPULights = FMath::Min(int32(MAX_LIGHT_ID), Scene.Lights.GetMaxIndex());

#if DO_CHECK
	LightDataArray.SetNum(NumGPULights);
#endif

	auto* LightDataUploader = TAsyncByteAddressBufferScatterUploader<FLightSceneData>::Allocate(GraphBuilder, LightsPostUpdateData.SceneLightInfoUpdates.NumCommands());

	const bool bAsyncSetup = CVarGPUSceneLightsAsyncSetup.GetValueOnRenderThread() != 0;

	LightDataSetupTask = GraphBuilder.AddSetupTask([this, LightDataUploader, ChangeSet = LightsPostUpdateData]
		{
			SCOPED_NAMED_EVENT(UpdateGPUScene_Lights, FColor::Green);
			const bool bAllowStaticLighting = IsStaticLightingAllowed();
			const uint32 Flags = RenderRectLightsAsSpotLights(FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(ShaderPlatform)) ? ELightShaderParameterFlags::RectAsSpotLight : 0u;

			ChangeSet.SceneLightInfoUpdates.ForEachCommand(ESceneUpdateCommandFilter::AddedUpdated, [&](const FUpdateLightCommand& UpdateLightCommand)
				{
					FLightSceneId Index = UpdateLightCommand.GetSceneInfo()->Id;

					if (!ensureMsgf(Index.GetIndex() < NumGPULights, TEXT("Index is out-of-range. Either Light Id = %d is incorrect or MAX_LIGHT_ID was exceeded."), Index.GetIndex()))
					{
						return;
					}

					const FLightSceneInfoCompact& LightInfoCompact = Scene.Lights[Index];

					FLightSceneData LightData;
					InitLightData(LightInfoCompact, bAllowStaticLighting, Flags, LightData);

					LightDataUploader->Add(LightData, Index.GetIndex());

#if DO_CHECK
					LightDataArray[Index.GetIndex()] = LightData;
#endif
				});

#if DO_CHECK
			for (int32 Index = 0; Index < NumGPULights; ++Index)
			{
				FLightSceneData LightData;

				FLightSceneId PersistentLightSceneId { Index };
				if (Scene.Lights.IsValidIndex(PersistentLightSceneId))
				{
					const FLightSceneInfoCompact& LightInfoCompact = Scene.Lights[PersistentLightSceneId];

					InitLightData(LightInfoCompact, bAllowStaticLighting, Flags, LightData);

					ensureMsgf(FMemory::Memcmp(&LightData, &LightDataArray[Index], sizeof(LightData)) == 0, TEXT("Data for Id = %d in GPU Scene Lights is stale."), Index);
				}
				else
				{
					LightData.WorldPosition = FDFVector3{};
					LightData.InvRadius = 0.0f;
				}

				LightDataArray[Index] = LightData;
			}
#endif
		},
		MakeArrayView({PrerequisiteTask, LightDataSetupTask}), // chain with previous setup task to prevent race condition accessing LightDataArray
		UE::Tasks::ETaskPriority::Normal,
		bAsyncSetup);

	if (!LightsPostUpdateData.SceneLightInfoUpdates.IsEmpty())
	{
		if (bAsyncSetup)
		{
			LightDataUploader->ResizeAndUploadToAsync(GraphBuilder, LightDataBuffer, NumGPULights);
		}
		else
		{
			LightDataUploader->ResizeAndUploadTo(GraphBuilder, LightDataBuffer, NumGPULights);
		}
	}
	else
	{
		LightDataBuffer.ResizeBufferIfNeeded(GraphBuilder, NumGPULights);
	}
}

void FGPUScene::InitLightData(const FLightSceneInfoCompact& LightInfoCompact, bool bAllowStaticLighting, uint32 LightShaderParameterFlags, FLightSceneData& DataOut)
{
	const FLightSceneInfo& LightInfo = *LightInfoCompact.LightSceneInfo;
	const FLightSceneProxy& LightProxy = *LightInfo.Proxy;

	FLightRenderParameters LightParams;
	LightProxy.GetLightShaderParameters(LightParams, LightShaderParameterFlags);

	if (LightProxy.IsInverseSquared())
	{
		LightParams.FalloffExponent = 0;
	}

	// FLightRenderParameters fields
	DataOut.WorldPosition = FDFVector3{ LightParams.WorldPosition };
	DataOut.InvRadius = LightParams.InvRadius;
	DataOut.FalloffExponent = LightParams.FalloffExponent;
	DataOut.Direction = LightParams.Direction;
	DataOut.SpecularScale = LightParams.SpecularScale;
	DataOut.DiffuseScale = LightParams.DiffuseScale;
	DataOut.Tangent = LightParams.Tangent;
	DataOut.SourceRadius = LightParams.SourceRadius;
	DataOut.SpotAngles = LightParams.SpotAngles;
	DataOut.SoftSourceRadius = LightParams.SoftSourceRadius;
	DataOut.SourceLength = LightParams.SourceLength;
	DataOut.RectLightBarnCosAngle = LightParams.RectLightBarnCosAngle;
	DataOut.RectLightBarnLength = LightParams.RectLightBarnLength;
	DataOut.InverseExposureBlend = LightParams.InverseExposureBlend;
}

void FGPUScene::UpdateInternal(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue, const UE::Tasks::FTask& UpdateTaskPrerequisites, const FUpdateFromComputeCommands& UpdatesFromCompute, FSceneExtensionsUpdaters& SceneExtensionsUpdaters)
{
	LLM_SCOPE_BYTAG(GPUScene);

	check(bIsEnabled == UseGPUScene(Scene.GetShaderPlatform()));
	
	check(DynamicPrimitivesOffset >= Scene.Primitives.Num());

	CSV_CUSTOM_STAT(GPUScene, InstanceAllocMaxSize, InstanceSceneDataAllocator.GetMaxSize(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(GPUScene, InstanceAllocUsedSize, InstanceSceneDataAllocator.GetSparselyAllocatedSize(), ECsvCustomStatOp::Set);

	CSV_CUSTOM_STAT(GPUScene, InstancePayloadAllocMaxSize, InstancePayloadDataAllocator.GetMaxSize(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(GPUScene, InstancePayloadAllocUsedSize, InstancePayloadDataAllocator.GetSparselyAllocatedSize(), ECsvCustomStatOp::Set);

	CSV_CUSTOM_STAT(GPUScene, LightmapDataAllocMaxSize, LightmapDataAllocator.GetMaxSize(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(GPUScene, LightmapDataAllocUsedSize, LightmapDataAllocator.GetSparselyAllocatedSize(), ECsvCustomStatOp::Set);

	RDG_EVENT_SCOPE(GraphBuilder, "GPUScene.Update");

	// Do this stat separately since it should always be on.
	CSV_CUSTOM_STAT_GLOBAL(GPUSceneInstanceCount, float(InstanceSceneDataAllocator.GetSparselyAllocatedSize()), ECsvCustomStatOp::Set);

#if !UE_BUILD_SHIPPING
	MaxInstancesDuringPrevUpdate = uint32(InstanceSceneDataAllocator.GetMaxSize());
#endif // UE_BUILD_SHIPPING

	if (CVarGPUSceneViewDistanceScaleWorkaround.GetValueOnRenderThread())
	{
		if (GetCachedScalabilityCVars().ViewDistanceScale != TrackedViewDistanceScle)
		{
			bUpdateAllPrimitives = true;
		}
		TrackedViewDistanceScle = GetCachedScalabilityCVars().ViewDistanceScale;
	}
	else
	{
		TrackedViewDistanceScle = -1.0f;
	}

	bool bPrevUseTiledInstanceDataLayout = bUseTiledInstanceDataLayout;
	bUseTiledInstanceDataLayout = CVarGPUSceneInstanceDataTileSizeLog2.GetValueOnRenderThread() >= 0 && GRHIGlobals.ReservedResources.Supported;
	int32 NewTileSizeLog2 = FMath::Max(0, CVarGPUSceneInstanceDataTileSizeLog2.GetValueOnRenderThread());
	// if it toggled tiled<->SOA OR the tile size changed, reupload everything 
	if (bPrevUseTiledInstanceDataLayout != bUseTiledInstanceDataLayout)
	{
		// reallocate the buffer on switch to/from reserved.
		// Alywas drop the old buffer - this simplifies the logic here and the tile mode should only be changed as a testing thing anyway, probably make read only before submit.
		InstanceSceneDataBuffer.SafeRelease();

		// Trigger full upload, the tile size has changed!
		bUpdateAllPrimitives = true;
	}
	
	// if the tile size changed we need to do a full reupload. 
	if (bUseTiledInstanceDataLayout && (NewTileSizeLog2 != InstanceDataTileSizeLog2))
	{
		InstanceDataTileSizeLog2 = NewTileSizeLog2;
		bUpdateAllPrimitives = true;
	}

	// For the reserved case, we reserve a "max-sized" buffer up front, otherwise leave it to the resize.
	if (bUseTiledInstanceDataLayout && !InstanceSceneDataBuffer.IsValid())
	{
		// allocate ~2GB of address space
		const uint64 MaxSizeInBytes = uint64(2048) << 20;
		FRDGBufferDesc InstanceDataBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FVector4f), uint32(MaxSizeInBytes / sizeof(FVector4f)));
		InstanceDataBufferDesc.Usage |= EBufferUsageFlags::ReservedResource;
		InstanceSceneDataBuffer = AllocatePooledBuffer(InstanceDataBufferDesc, TEXT("GPUScene.InstanceSceneData"));
	}

	if ((CVarGPUSceneUploadEveryFrame.GetValueOnRenderThread() != 0) || bUpdateAllPrimitives)
	{
		PrimitivesToUpdate.Reset();
		ResizeDirtyState(Scene.GetMaxPersistentPrimitiveIndex());
		InstanceRangesToClear.Reset(Scene.Primitives.Num());
		for (FPrimitiveSceneInfo *PrimitiveSceneInfo : Scene.Primitives)
		{
			PrimitiveDirtyState[PrimitiveSceneInfo->GetPersistentIndex().Index] |= EPrimitiveDirtyState::ChangedAll;
			PrimitivesToUpdate.Add(PrimitiveSceneInfo->GetPersistentIndex());
			InstanceRangesToClear.Add(FInstanceRange{
				PrimitiveSceneInfo->GetPersistentIndex(),
				(uint32)PrimitiveSceneInfo->GetInstanceSceneDataOffset(),
				(uint32)PrimitiveSceneInfo->GetNumInstanceSceneDataEntries() });
		}

		bUpdateAllPrimitives = false;
	}

	// Store in GPU-scene to enable validation that update has been carried out.
	SceneFrameNumber = Scene.GetFrameNumber();

	// Strip all out-of-range ID's (left over because of deletes) so we don't need to check later
	for (int32 Index = 0; Index < PrimitivesToUpdate.Num();)
	{
		const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitivesToUpdate[Index];
		if (!PrimitiveDirtyState.IsValidIndex(PersistentPrimitiveIndex.Index))
		{
			PrimitivesToUpdate.RemoveAtSwap(Index, EAllowShrinking::No);
		}
		else
		{
			// If it was removed (and not readded)
			if (EnumHasAnyFlags(PrimitiveDirtyState[PersistentPrimitiveIndex.Index], EPrimitiveDirtyState::Removed) 
				&& !EnumHasAnyFlags(PrimitiveDirtyState[PersistentPrimitiveIndex.Index], EPrimitiveDirtyState::Added))
			{
				PrimitivesToUpdate.RemoveAtSwap(Index, EAllowShrinking::No);
			}
			else
			{
				// Should only have valid, current, primitives left
				check(Scene.GetPrimitiveIndex(PersistentPrimitiveIndex) != INDEX_NONE);
				++Index;
			}
		}
	}

	FUploadDataSourceAdapterScenePrimitives& Adapter = *GraphBuilder.AllocObject<FUploadDataSourceAdapterScenePrimitives>(Scene, SceneFrameNumber, MoveTemp(PrimitivesToUpdate), MoveTemp(PrimitiveDirtyState));
	
	FRegisteredBuffers BufferState = UpdateBufferAllocations(GraphBuilder);

	// Run a pass that clears (Sets ID to invalid) any instances that need it
	AddClearInstancesPass(GraphBuilder, Scene.InstanceCullingOcclusionQueryRenderer);

	// The adapter copies the IDs of primitives to update such that any that are (incorrectly) marked for update after are not lost.
	PrimitivesToUpdate.Reset();
	PrimitiveDirtyState.Init(EPrimitiveDirtyState::None, PrimitiveDirtyState.Num());

	{
		SCOPED_NAMED_EVENT(UpdateGPUScene, FColor::Green);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateGPUScene);
		SCOPE_CYCLE_COUNTER(STAT_UpdateGPUSceneTime);

		UploadGeneral<FUploadDataSourceAdapterScenePrimitives>(GraphBuilder, BufferState, &ExternalAccessQueue, Adapter, UpdateTaskPrerequisites);
	}

	// Update instance data using GPU compute.
	auto SkinningUpdater = SceneExtensionsUpdaters.GetUpdaterPtr<FSkinningSceneExtension::FUpdater>();

	if (!UpdatesFromCompute.IsEmpty() || SkinningUpdater)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "UpdateGPUScene UpdateInstancesFromCompute");

		FGPUSceneWriteDelegateParams Params;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Params.GPUSceneWriterUB = GetWriteParameters(GraphBuilder, &Params.GPUWriteParams);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		for (const auto& Command : UpdatesFromCompute)
		{
			Params.PersistentPrimitiveId = (uint32)Command.SceneInfo->GetPersistentIndex().Index;
			Params.InstanceSceneDataOffset = Command.SceneInfo->GetInstanceSceneDataOffset();

			const FInstanceSceneDataBuffers* InstanceSceneDataBuffers = Command.SceneInfo->GetInstanceSceneDataBuffers();
			check(InstanceSceneDataBuffers);
			Params.NumCustomDataFloats = InstanceSceneDataBuffers->GetNumCustomDataFloats();
			Params.PackedInstanceSceneDataFlags = UE::Renderer::Private::PackInstanceDataFlags(InstanceSceneDataBuffers->GetFlags());

			Command.Payload.GPUSceneWriter.Execute(GraphBuilder, Params);
		}

		if (SkinningUpdater)
		{
			SkinningUpdater->ResolveAttachments(GraphBuilder, Params);
		}
	}
}

FGPUScene::FRegisteredBuffers FGPUScene::UpdateBufferAllocations(FRDGBuilder& GraphBuilder)
{
	LLM_SCOPE_BYTAG(GPUScene);

	check(bIsEnabled == UseGPUScene(Scene.GetShaderPlatform()));

	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	FRegisteredBuffers BufferState;

	bool bUseReservedResources = CVarGPUSceneUseReservedResources.GetValueOnRenderThread() && GRHIGlobals.ReservedResources.Supported;

	auto AllocOrResizeBufferIfNeeded = [bUseReservedResources](FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer>& Buffer, uint32 NumBytes, const TCHAR* DebugName)
	{
		if (bUseReservedResources && !Buffer.IsValid())
		{
			constexpr uint64 ReservedAllocationSize = uint64(2048) << 20;
			const uint32 BytesPerElement = 16;
			FRDGBufferDesc BufferDescReserved = FRDGBufferDesc::CreateStructuredDesc(BytesPerElement, uint32(ReservedAllocationSize / uint64(BytesPerElement)));
			BufferDescReserved.Usage |= EBufferUsageFlags::ReservedResource;
			Buffer = AllocatePooledBuffer(BufferDescReserved, DebugName);
		}
		return ResizeStructuredBufferIfNeeded(GraphBuilder, Buffer, NumBytes, DebugName);
	};


	const uint32 SizeReserve = FMath::RoundUpToPowerOfTwo(FMath::Max(PrevDynamicPrimitivesMaxSize, FMath::Max(DynamicPrimitivesOffset, InitialBufferSize)));
	BufferState.PrimitiveBuffer = AllocOrResizeBufferIfNeeded(GraphBuilder, PrimitiveBuffer, SizeReserve * sizeof(FPrimitiveSceneShaderData::Data), TEXT("GPUScene.PrimitiveData"));
	
	// clamped instance count / high watermark (any overflowing will not render!)
	const uint32 ClampedMaxInstanceId = FMath::Min(MAX_INSTANCE_ID, uint32(FMath::Max(PrevDynamicPrimitiveInstanceMaxSize, FMath::Max(InstanceSceneDataAllocator.GetMaxSize(), InitialBufferSize))));

	// emulate the old functioning, deriving the stride from the instance count
	if (!bUseTiledInstanceDataLayout)
	{
		// Round to next POT
		const uint32 InstanceSceneDataSizeReserve = FMath::RoundUpToPowerOfTwo(ClampedMaxInstanceId);

		FResizeResourceSOAParams ResizeParams;
		ResizeParams.NumBytes = InstanceSceneDataSizeReserve * FInstanceSceneShaderData::GetEffectiveNumBytes();
		ResizeParams.NumArrays = FInstanceSceneShaderData::GetDataStrideInFloat4s();

		BufferState.InstanceSceneDataBuffer = ResizeStructuredBufferSOAIfNeeded(GraphBuilder, InstanceSceneDataBuffer, ResizeParams, TEXT("GPUScene.InstanceSceneData"));
		check(FMath::IsPowerOfTwo(InstanceSceneDataSizeReserve));
		InstanceDataTileSizeLog2 = FMath::FloorLog2(InstanceSceneDataSizeReserve);
	}
	else
	{
		// Figure out the size required for the buffer when rounded to number of tiles.
		const uint32 TileSizeInBytes = sizeof(FVector4f) * GetInstanceDataTileStride();
		const uint32 NumTiles = FMath::DivideAndRoundUp(ClampedMaxInstanceId, GetInstanceDataTileSize());

		// Grow/shrink in 16MB chunks to reduce the number of page table updates or resizes
		const uint32 AllocationStrideInBytes = 16u << 20;
		const uint32 InstanceSceneDataSizeReserve = FMath::DivideAndRoundUp(NumTiles * TileSizeInBytes, AllocationStrideInBytes) * AllocationStrideInBytes;
		BufferState.InstanceSceneDataBuffer = ResizeStructuredBufferIfNeeded(GraphBuilder, InstanceSceneDataBuffer, InstanceSceneDataSizeReserve, TEXT("GPUScene.InstanceSceneData"));
	}

	const uint32 PayloadFloat4Count = FMath::Max(PrevDynamicPrimitivePayloadMaxSize, FMath::Max(InstancePayloadDataAllocator.GetMaxSize(), InitialBufferSize));
	InstancePayloadDataSizeReserveUnclamped = FMath::RoundUpToPowerOfTwo(uint64(PayloadFloat4Count));
	uint32 InstancePayloadDataSizeReserve = uint32(InstancePayloadDataSizeReserveUnclamped * sizeof(FVector4f));

	if (InstancePayloadDataSizeReserveUnclamped > MaxPayloadBufferSizeInFloat4)
	{
		InstancePayloadDataSizeReserve = MaxPayloadBufferSizeInFloat4 * sizeof(FVector4f); 
	}
	BufferState.InstancePayloadDataBuffer = AllocOrResizeBufferIfNeeded(GraphBuilder, InstancePayloadDataBuffer, InstancePayloadDataSizeReserve, TEXT("GPUScene.InstancePayloadData"));

	const uint32 LightMapDataBufferSize = FMath::RoundUpToPowerOfTwo(FMath::Max(LightmapDataAllocator.GetMaxSize(), InitialBufferSize));
	BufferState.LightmapDataBuffer = AllocOrResizeBufferIfNeeded(GraphBuilder, LightmapDataBuffer, LightMapDataBufferSize * sizeof(FLightmapSceneShaderData::Data), TEXT("GPUScene.LightmapData"));

	BufferState.LightDataBuffer = GraphBuilder.RegisterExternalBuffer(LightDataBuffer.GetPooledBuffer());

	// Set up with no tiling, i.e, behaves as before.
	CommonParameters.GPUSceneInstanceDataTileSizeLog2 = GetInstanceDataTileSizeLog2();
	CommonParameters.GPUSceneInstanceDataTileSizeMask =  GetInstanceDataTileSizeMask();
	CommonParameters.GPUSceneInstanceDataTileStride = GetInstanceDataTileStride();
	CommonParameters.GPUSceneFrameNumber = GetSceneFrameNumber();
	CommonParameters.GPUSceneMaxAllocatedInstanceId = InstanceSceneDataAllocator.GetMaxSize();
	CommonParameters.GPUSceneMaxPersistentPrimitiveIndex = Scene.GetMaxPersistentPrimitiveIndex();
	CommonParameters.GPUSceneNumLightmapDataItems = GetNumLightmapDataItems();

	if (bInBeginEndBlock)
	{
		CachedRegisteredBuffers = BufferState;
	}
	else
	{
		check(!CachedRegisteredBuffers.IsValid());
	}

	return BufferState;
}


FGPUScene::FRegisteredBuffers FGPUScene::RegisterBuffers(FRDGBuilder& GraphBuilder) const
{
	FRegisteredBuffers Result;

	Result.PrimitiveBuffer = GraphBuilder.RegisterExternalBuffer(PrimitiveBuffer);
	Result.InstanceSceneDataBuffer = GraphBuilder.RegisterExternalBuffer(InstanceSceneDataBuffer);
	Result.InstancePayloadDataBuffer = GraphBuilder.RegisterExternalBuffer(InstancePayloadDataBuffer);
	Result.LightmapDataBuffer = GraphBuilder.RegisterExternalBuffer(LightmapDataBuffer);
	Result.LightDataBuffer = GraphBuilder.RegisterExternalBuffer(LightDataBuffer.GetPooledBuffer());

	return Result;
}

/**
 * Used to queue up load-balanced chunks of instance upload work such that it can be spread over a large number of cores.
 */
struct FInstanceUploadBatch
{
	struct FItem
	{
		int32 ItemIndex;
		int32 FirstInstance;
		int32 NumInstances;
	};

	int32 FirstItem = 0;
	int32 NumItems = 0;
};

struct FInstanceBatcher
{
	int32 MaxItems = 64;
	int32 MaxCost = MaxItems * 2; // Selected to allow filling the array when 1:1 primitive / instances

	struct FPrimitiveItemInfo
	{
		int32 InstanceSceneDataUploadOffset;
		int32 InstancePayloadDataUploadOffset;
	};

	FInstanceUploadBatch* CurrentBatch = nullptr;
	TArray<FInstanceUploadBatch, TInlineAllocator<1, SceneRenderingAllocator> > UpdateBatches;
	TArray<FInstanceUploadBatch::FItem, TInlineAllocator<256, SceneRenderingAllocator> > UpdateBatchItems;
	TArray<FPrimitiveItemInfo, SceneRenderingAllocator> PerPrimitiveItemInfo;

	int32 CurrentBatchCost = 0;

	int32 InstanceSceneDataUploadOffset = 0;
	int32 InstancePayloadDataUploadOffset = 0; // Count of float4s

	FInstanceBatcher(bool bExecuteInParallel, int32 NumPrimitiveDataUploads)
	{
		if (bExecuteInParallel)
		{
			MaxItems = 64;
			MaxCost = MaxItems * 2;
		}
		else
		{
			// funnel  all items into a single batch with one item per primitive.
			MaxItems = MAX_int32;
			MaxCost = MAX_int32;
			UpdateBatchItems.Reserve(NumPrimitiveDataUploads);
		}
		CurrentBatch = &UpdateBatches.AddDefaulted_GetRef();
	}

	void QueueInstances(const FPrimitiveUploadInfoHeader &UploadInfo, int32 ItemIndex, const FPrimitiveItemInfo & PrimitiveItemInfo)
	{
		PerPrimitiveItemInfo[ItemIndex] = PrimitiveItemInfo;
		const int32 NumInstances = UploadInfo.NumInstanceUploads;
		int32 InstancesAdded = 0;
		while (InstancesAdded < NumInstances)
		{
			// Populate the last batch until full. Max items/batch = 64, for balance use cost estimate of 1:1 for primitive:instance
			// Fill to minimum cost (1

			// Can add one less to account for primitive cost
			int32 MaxInstancesThisBatch = MaxCost - CurrentBatchCost - 1;

			if (MaxInstancesThisBatch > 0)
			{
				const int NumInstancesThisItem = FMath::Min(MaxInstancesThisBatch, NumInstances - InstancesAdded);
				UpdateBatchItems.Add(FInstanceUploadBatch::FItem{ ItemIndex, InstancesAdded, NumInstancesThisItem });
				CurrentBatch->NumItems += 1;
				InstancesAdded += NumInstancesThisItem;
				CurrentBatchCost += NumInstancesThisItem + 1;
			}

			// Flush batch if it is not possible to add any more items (for one of the reasons)
			if (MaxInstancesThisBatch <= 0 || CurrentBatchCost > MaxCost - 1 || CurrentBatch->NumItems >= MaxItems)
			{
				CurrentBatchCost = 0;
				CurrentBatch = &UpdateBatches.AddDefaulted_GetRef();
				CurrentBatch->FirstItem = UpdateBatchItems.Num();
			}
		}
	}
};

namespace
{
	struct FGPUSceneUploadTaskContext
	{
		TArray<FPrimitiveUploadInfoHeader, FSceneRenderingArrayAllocator> PrimitiveUploadInfos;

		FRDGScatterUploader* PrimitiveUploader = nullptr;
		FRDGScatterUploader* InstancePayloadUploader = nullptr;
		FRDGScatterUploader* InstanceSceneUploader = nullptr;
		FRDGScatterUploader* LightmapUploader = nullptr;

		int32 NumPrimitiveDataUploads = 0;
		int32 NumLightmapDataUploads = 0;
		int32 NumInstanceSceneDataUploads = 0;
		int32 NumInstancePayloadDataUploads = 0; // Count of float4s
		uint32 InstanceDataTileSizeLog2 = 0u;
		uint32 InstanceDataTileSizeMask = 0u;
		uint32 InstanceDataTileStride = 0u;

		FORCEINLINE_GPUSCENE uint32 CalcInstanceDataIndex(uint32 InstanceId, uint32 ArrayIndex)
		{
			uint32 TileId = InstanceId >> InstanceDataTileSizeLog2;
			uint32 IdInTile = InstanceId & InstanceDataTileSizeMask;
			return InstanceDataTileStride * TileId + (ArrayIndex << InstanceDataTileSizeLog2) + IdInTile;
		}
	};

};

template<typename FUploadDataSourceAdapter>
void FGPUScene::UploadGeneral(FRDGBuilder& GraphBuilder, const FRegisteredBuffers &BufferState, FRDGExternalAccessQueue* ExternalAccessQueue, const FUploadDataSourceAdapter& UploadDataSourceAdapter, const UE::Tasks::FTask& PrerequisiteTask)
{
	LLM_SCOPE_BYTAG(GPUScene);

	ensure(bIsEnabled == UseGPUScene(Scene.GetShaderPlatform()));

	const int32 NumPrimitiveDataUploads = UploadDataSourceAdapter.NumPrimitivesToUpload();

	if (!NumPrimitiveDataUploads)
	{
		return;
	}

	const int32 ParallelExecuteThreshold = FApp::ShouldUseThreadingForPerformance() ? CVarGPUSceneParallelUpdate.GetValueOnRenderThread() : 0;

	SCOPED_NAMED_EVENT(UpdateGPUScene, FColor::Green);

	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());
	RDG_EVENT_SCOPE(GraphBuilder, "UpdateGPUScene NumPrimitiveDataUploads %u", NumPrimitiveDataUploads);

	FGPUSceneUploadTaskContext& TaskContext = *GraphBuilder.AllocObject<FGPUSceneUploadTaskContext>();

	TaskContext.NumPrimitiveDataUploads = NumPrimitiveDataUploads;
	TaskContext.InstanceDataTileSizeLog2 = GetInstanceDataTileSizeLog2();
	TaskContext.InstanceDataTileSizeMask = GetInstanceDataTileSizeMask();
	TaskContext.InstanceDataTileStride   = GetInstanceDataTileStride();
	TaskContext.PrimitiveUploadInfos.SetNumUninitialized(NumPrimitiveDataUploads);

	// Max offset to scatter to.
	const uint32 MaxInstancePayloadOffset = InstancePayloadDataBuffer->Desc.NumElements;

	for (int32 ItemIndex = 0; ItemIndex < NumPrimitiveDataUploads; ++ItemIndex)
	{
		FPrimitiveUploadInfoHeader UploadInfo;
		UploadDataSourceAdapter.GetPrimitiveInfoHeader(ItemIndex, UploadInfo);

		if (UploadInfo.InstanceSceneDataOffset + UploadInfo.NumInstanceUploads >= int32(MAX_INSTANCE_ID))
		{
			// clamp the uploads such that we don't try to scatter any that will land outside the buffer anyway.
			UploadInfo.NumInstanceUploads = FMath::Max(0, int32(MAX_INSTANCE_ID) - UploadInfo.InstanceSceneDataOffset);
		}

		UploadInfo.NumPayloadUploadInstances = UploadInfo.InstancePayloadDataStride ? UploadInfo.NumInstanceUploads : 0;
		// Figure out how much space we actually have left.
		int32 PayloadSpaceLeft = FMath::Max(int32(0), int32(MaxInstancePayloadOffset) - int32(UploadInfo.InstancePayloadDataOffset));
		// If we overflow, clamp the number of instances to upload payload data for to fit inside the buffer.
		if (UploadInfo.NumPayloadUploadInstances * UploadInfo.InstancePayloadDataStride > PayloadSpaceLeft)
		{
			UploadInfo.NumPayloadUploadInstances = PayloadSpaceLeft / UploadInfo.InstancePayloadDataStride;
		}
		TaskContext.NumLightmapDataUploads += UploadInfo.LightmapUploadCount; // Not thread safe
		TaskContext.NumInstanceSceneDataUploads += UploadInfo.NumInstanceUploads; // Not thread safe
		TaskContext.NumInstancePayloadDataUploads += UploadInfo.NumPayloadUploadInstances * UploadInfo.InstancePayloadDataStride; // Not thread safe

		TaskContext.PrimitiveUploadInfos[ItemIndex] = UploadInfo;
	}

	TaskContext.PrimitiveUploader = PrimitiveUploadBuffer.Begin(GraphBuilder, BufferState.PrimitiveBuffer, UploadDataSourceAdapter.GetItemPrimitiveIds().Num(), sizeof(FPrimitiveSceneShaderData::Data), TEXT("PrimitiveUploadBuffer"));

	if (TaskContext.NumInstancePayloadDataUploads > 0)
	{
		TaskContext.InstancePayloadUploader = InstancePayloadUploadBuffer.BeginPreSized(GraphBuilder, BufferState.InstancePayloadDataBuffer, TaskContext.NumInstancePayloadDataUploads, sizeof(FVector4f), TEXT("InstancePayloadUploadBuffer"));
	}

	if (TaskContext.NumInstanceSceneDataUploads > 0)
	{
		TaskContext.InstanceSceneUploader = InstanceSceneUploadBuffer.BeginPreSized(GraphBuilder, BufferState.InstanceSceneDataBuffer, TaskContext.NumInstanceSceneDataUploads * FInstanceSceneShaderData::GetDataStrideInFloat4s(), sizeof(FVector4f), TEXT("InstanceSceneUploadBuffer"));
	}

	if (TaskContext.NumLightmapDataUploads > 0)
	{
		TaskContext.LightmapUploader = LightmapUploadBuffer.Begin(GraphBuilder, BufferState.LightmapDataBuffer, TaskContext.NumLightmapDataUploads, sizeof(FLightmapSceneShaderData::Data), TEXT("LightmapUploadBuffer"));
	}

	const bool bExecuteInParallel = ParallelExecuteThreshold > 0 && (TaskContext.NumInstancePayloadDataUploads > ParallelExecuteThreshold || TaskContext.NumPrimitiveDataUploads > ParallelExecuteThreshold);

	GraphBuilder.AddCommandListSetupTask([&TaskContext, &UploadDataSourceAdapter, bExecuteInParallel, FeatureLevel = FDataDrivenShaderPlatformInfo::GetMaxFeatureLevel(ShaderPlatform)](FRHICommandListBase& RHICmdList)
	{
		SCOPED_NAMED_EVENT(UpdateGPUScene_Primitives, FColor::Green);

		LockIfValid(RHICmdList, TaskContext.PrimitiveUploader);
		LockIfValid(RHICmdList, TaskContext.InstancePayloadUploader);
		LockIfValid(RHICmdList, TaskContext.InstanceSceneUploader);
		LockIfValid(RHICmdList, TaskContext.LightmapUploader);

		FInstanceBatcher InstanceUpdates(bExecuteInParallel, TaskContext.NumPrimitiveDataUploads);

		{
			SCOPED_NAMED_EVENT(Primitives, FColor::Green);

			InstanceUpdates.PerPrimitiveItemInfo.SetNumUninitialized(TaskContext.NumPrimitiveDataUploads);

			// Recalculated to determine offsets.
			int32 NumInstanceSceneDataUploads = 0;
			int32 NumInstancePayloadDataUploads = 0;

			for (int32 ItemIndex = 0; ItemIndex < TaskContext.NumPrimitiveDataUploads; ++ItemIndex)
			{
				const FPrimitiveUploadInfoHeader& UploadInfo = TaskContext.PrimitiveUploadInfos[ItemIndex];

				FInstanceBatcher::FPrimitiveItemInfo PrimitiveItemInfo;
				PrimitiveItemInfo.InstanceSceneDataUploadOffset = NumInstanceSceneDataUploads;
				PrimitiveItemInfo.InstancePayloadDataUploadOffset = NumInstancePayloadDataUploads;
				InstanceUpdates.QueueInstances(UploadInfo, ItemIndex, PrimitiveItemInfo);

				NumInstanceSceneDataUploads += UploadInfo.NumInstanceUploads;
				NumInstancePayloadDataUploads += UploadInfo.NumPayloadUploadInstances * UploadInfo.InstancePayloadDataStride;
			}

			TaskContext.PrimitiveUploader->Add(UploadDataSourceAdapter.GetItemPrimitiveIds());

			ParallelForTemplate(TEXT("GPUScene Upload Primitives Task"), TaskContext.NumPrimitiveDataUploads, 1, [&TaskContext, &UploadDataSourceAdapter](int32 ItemIndex)
			{
				FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);

				FVector4f* DstData = static_cast<FVector4f*>(TaskContext.PrimitiveUploader->GetRef(ItemIndex));
				UploadDataSourceAdapter.GetPrimitiveShaderData(ItemIndex, DstData);

			}, bExecuteInParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		}

		if (TaskContext.NumInstanceSceneDataUploads > 0 && !InstanceUpdates.UpdateBatches.IsEmpty())
		{
			SCOPED_NAMED_EVENT(Instances, FColor::Green);

			ParallelForTemplate(TEXT("GPUScene Upload Instances Task"), InstanceUpdates.UpdateBatches.Num(), 1, [&TaskContext, &InstanceUpdates, &UploadDataSourceAdapter](int32 BatchIndex)
			{
				const FInstanceUploadBatch Batch = InstanceUpdates.UpdateBatches[BatchIndex];
				const uint32 InstanceDataStrideInFloat4s = FInstanceSceneShaderData::GetDataStrideInFloat4s();
				const bool bSupportsCompressedTransforms = FInstanceSceneShaderData::SupportsCompressedTransforms();

				for (int32 BatchItemIndex = 0; BatchItemIndex < Batch.NumItems; ++BatchItemIndex)
				{
					const FInstanceUploadBatch::FItem Item = InstanceUpdates.UpdateBatchItems[Batch.FirstItem + BatchItemIndex];

					const int32 ItemIndex = Item.ItemIndex;
					FInstanceUploadInfo UploadInfo;
					UploadDataSourceAdapter.GetInstanceInfo(ItemIndex, UploadInfo);
					ValidateInstanceUploadInfo(UploadInfo, TaskContext.InstancePayloadUploader ? GetAsBuffer(TaskContext.InstancePayloadUploader->GetDstResource()) : nullptr);
					FInstanceBatcher::FPrimitiveItemInfo PrimitiveItemInfo = InstanceUpdates.PerPrimitiveItemInfo[ItemIndex];
					FPrimitiveUploadInfoHeader PrimitiveHeader = TaskContext.PrimitiveUploadInfos[ItemIndex];

					check(TaskContext.NumInstancePayloadDataUploads > 0 || UploadInfo.InstancePayloadDataStride == 0); // Sanity check

					for (int32 BatchInstanceIndex = 0; BatchInstanceIndex < Item.NumInstances; ++BatchInstanceIndex)
					{
						int32 InstanceIndex = Item.FirstInstance + BatchInstanceIndex;

						// Directly embedded in instance scene data
						const float RandomID = (UploadInfo.InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_RANDOM) ? UploadInfo.InstanceRandomID[InstanceIndex] : 0.0f;

						FInstanceSceneShaderData InstanceSceneData;
						if (UploadInfo.InstanceSceneDataBuffers != nullptr)
						{
							InstanceSceneData.BuildInternal(
								UploadInfo.PrimitiveID,
								InstanceIndex,
								UploadInfo.InstanceFlags,
								UploadInfo.LastUpdateSceneFrameNumber,
								UploadInfo.InstanceCustomDataCount,
								RandomID,
								UploadInfo.InstanceSceneDataBuffers->GetInstanceToPrimitiveRelative(InstanceIndex),
								!UploadInfo.bIsPrimitiveForceHidden && UploadInfo.InstanceSceneDataBuffers->GetInstanceVisible(InstanceIndex),
								bSupportsCompressedTransforms
							);
						}
						else if (UploadInfo.PrimitiveInstances.IsEmpty())
						{
							// This path should only be taken for uninstanced primitives
							check(UploadInfo.NumInstances == 1 && InstanceIndex == 0);
							InstanceSceneData.BuildInternal(
								UploadInfo.PrimitiveID,
								InstanceIndex,
								UploadInfo.InstanceFlags,
								UploadInfo.LastUpdateSceneFrameNumber,
								UploadInfo.InstanceCustomDataCount,
								RandomID,
								UploadInfo.PrimitiveToWorld,
								!UploadInfo.bIsPrimitiveForceHidden,
								bSupportsCompressedTransforms
							);
						}
						else
						{
							const FInstanceSceneData& SceneData = UploadInfo.PrimitiveInstances[InstanceIndex];
							InstanceSceneData.Build(
								UploadInfo.PrimitiveID,
								InstanceIndex,
								UploadInfo.InstanceFlags,
								UploadInfo.LastUpdateSceneFrameNumber,
								UploadInfo.InstanceCustomDataCount,
								RandomID,
								SceneData.LocalToPrimitive,
								UploadInfo.PrimitiveToWorld,
								!UploadInfo.bIsPrimitiveForceHidden
							);
						}
						const uint32 UploadInstanceItemOffset = (PrimitiveItemInfo.InstanceSceneDataUploadOffset + InstanceIndex) * InstanceDataStrideInFloat4s;

						for (uint32 RefIndex = 0; RefIndex < InstanceDataStrideInFloat4s; ++RefIndex)
						{
							uint32 ScatterOffset = TaskContext.CalcInstanceDataIndex(PrimitiveHeader.InstanceSceneDataOffset + InstanceIndex, RefIndex);
							FVector4f* DstVector = static_cast<FVector4f*>(TaskContext.InstanceSceneUploader->Set_GetRef(UploadInstanceItemOffset + RefIndex, ScatterOffset));
							*DstVector = InstanceSceneData.Data[RefIndex];
						}

						// Skip uploading if overflowing the buffer
						if (InstanceIndex < PrimitiveHeader.NumPayloadUploadInstances)
						{
							const uint32 UploadPayloadItemOffset = PrimitiveItemInfo.InstancePayloadDataUploadOffset + InstanceIndex * UploadInfo.InstancePayloadDataStride;

							int32 PayloadDataStart = UploadInfo.InstancePayloadDataOffset + (InstanceIndex * UploadInfo.InstancePayloadDataStride);
							FVector4f* DstPayloadData = static_cast<FVector4f*>(TaskContext.InstancePayloadUploader->Set_GetRef(UploadPayloadItemOffset, PayloadDataStart, UploadInfo.InstancePayloadDataStride));
							TArrayView<FVector4f> InstancePayloadData = TArrayView<FVector4f>(DstPayloadData, UploadInfo.InstancePayloadDataStride);

							UploadInfo.WritePayloadData(InstancePayloadData, InstanceIndex);
						}
					}
				}

			}, bExecuteInParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		}

		if (TaskContext.LightmapUploader)
		{
			for (int32 ItemIndex = 0; ItemIndex < TaskContext.NumPrimitiveDataUploads; ++ItemIndex)
			{
				FLightMapUploadInfo UploadInfo;
				if (UploadDataSourceAdapter.GetLightMapInfo(ItemIndex, UploadInfo))
				{
					for (int32 LCIIndex = 0; LCIIndex < UploadInfo.LCIs.Num(); LCIIndex++)
					{
						FLightmapSceneShaderData LightmapSceneData(UploadInfo.LCIs[LCIIndex], FeatureLevel);
						TaskContext.LightmapUploader->Add(UploadInfo.LightmapDataOffset + LCIIndex, &LightmapSceneData.Data[0]);
					}
				}
			}
		}

		UnlockIfValid(RHICmdList, TaskContext.PrimitiveUploader);
		UnlockIfValid(RHICmdList, TaskContext.InstancePayloadUploader);
		UnlockIfValid(RHICmdList, TaskContext.InstanceSceneUploader);
		UnlockIfValid(RHICmdList, TaskContext.LightmapUploader);

	}, PrerequisiteTask);

	PrimitiveUploadBuffer.End(GraphBuilder, TaskContext.PrimitiveUploader);

	if (TaskContext.InstancePayloadUploader)
	{
		InstancePayloadUploadBuffer.End(GraphBuilder, TaskContext.InstancePayloadUploader);
	}

	if (TaskContext.InstanceSceneUploader)
	{
		InstanceSceneUploadBuffer.End(GraphBuilder, TaskContext.InstanceSceneUploader);
	}

	if (TaskContext.LightmapUploader)
	{
		LightmapUploadBuffer.End(GraphBuilder, TaskContext.LightmapUploader);
	}

	const uint32 MaxPooledSize = uint32(CVarGPUSceneMaxPooledUploadBufferSize.GetValueOnRenderThread());
	if (PrimitiveUploadBuffer.GetNumBytes() > MaxPooledSize)
	{
		PrimitiveUploadBuffer.Release();
	}

	if (InstanceSceneUploadBuffer.GetNumBytes() > MaxPooledSize)
	{
		InstanceSceneUploadBuffer.Release();
	}

	if (InstancePayloadUploadBuffer.GetNumBytes() > MaxPooledSize)
	{
		InstancePayloadUploadBuffer.Release();
	}

	if (LightmapUploadBuffer.GetNumBytes() > MaxPooledSize)
	{
		LightmapUploadBuffer.Release();
	}
}

struct FUploadDataSourceAdapterDynamicPrimitives
{
	FUploadDataSourceAdapterDynamicPrimitives(
		const TArray<FGPUScenePrimitiveCollector::FPrimitiveData, TInlineAllocator<8>>& InPrimitiveData,
		int32 InPrimitiveIDStartOffset,
		int32 InInstanceIDStartOffset,
		int32 InPayloadStartOffset,
		uint32 InSceneFrameNumber)
		: PrimitiveData(InPrimitiveData)
		, PrimitiveIDStartOffset(InPrimitiveIDStartOffset)
		, InstanceIDStartOffset(InInstanceIDStartOffset)
		, PayloadStartOffset(InPayloadStartOffset)
		, SceneFrameNumber(InSceneFrameNumber)
	{
		// Need to create this explicit for optimizing the common path
		PrimitivesIds.SetNumUninitialized(PrimitiveData.Num());
		for (int32 Index = 0; Index < PrimitivesIds.Num(); ++Index)
		{
			PrimitivesIds[Index] = uint32(PrimitiveIDStartOffset + Index);
		}
	}

	FORCEINLINE int32 NumPrimitivesToUpload() const
	{ 
		return PrimitiveData.Num();
	}


	FORCEINLINE_GPUSCENE TArrayView<const uint32> GetItemPrimitiveIds() const
	{
		return TArrayView<const uint32>(PrimitivesIds.GetData(), PrimitivesIds.Num());
	}


	FORCEINLINE_GPUSCENE void GetPrimitiveInfoHeader(int32 ItemIndex, FPrimitiveUploadInfoHeader& PrimitiveUploadInfo) const
	{
		PrimitiveUploadInfo.LightmapUploadCount = 0;
		PrimitiveUploadInfo.NaniteSceneProxy = nullptr;
		PrimitiveUploadInfo.PrimitiveSceneInfo = nullptr;

		check(ItemIndex < PrimitiveData.Num());

		PrimitiveUploadInfo.PrimitiveID = PrimitiveIDStartOffset + ItemIndex;

		const FGPUScenePrimitiveCollector::FPrimitiveData& PrimData = PrimitiveData[ItemIndex];
		PrimitiveUploadInfo.NumInstanceUploads = PrimData.NumInstances;
		if (PrimData.LocalPayloadDataOffset != INDEX_NONE)
		{
			PrimitiveUploadInfo.InstancePayloadDataOffset = PayloadStartOffset + PrimData.LocalPayloadDataOffset;
			PrimitiveUploadInfo.InstancePayloadDataStride = PrimData.SourceData.GetPayloadFloat4Stride();
		}
		else
		{
			PrimitiveUploadInfo.InstancePayloadDataOffset = INDEX_NONE;
			PrimitiveUploadInfo.InstancePayloadDataStride = 0;
		}
		PrimitiveUploadInfo.InstanceSceneDataOffset		= InstanceIDStartOffset + PrimData.LocalInstanceSceneDataOffset;
		if (PrimData.SourceData.DataWriterGPU.IsBound())
		{
			// Only upload if we have data, otherwise expect the delegate to handle missing data
			PrimitiveUploadInfo.NumInstanceUploads = PrimData.SourceData.InstanceSceneData.Num();
			if (PrimData.SourceData.InstanceCustomData.Num() == 0)
			{
				PrimitiveUploadInfo.InstancePayloadDataStride = 0;
			}
		}
	}

	FORCEINLINE_GPUSCENE void GetPrimitiveShaderData(int32 ItemIndex, FVector4f* RESTRICT OutData) const
	{
		// Needed to ensure the link back to instance list is up to date
		const FGPUScenePrimitiveCollector::FPrimitiveData& PrimData = PrimitiveData[ItemIndex];		
		FPrimitiveUniformShaderParameters Tmp = *PrimData.ShaderParams;
		Tmp.InstanceSceneDataOffset = InstanceIDStartOffset + PrimData.LocalInstanceSceneDataOffset;
		Tmp.NumInstanceSceneDataEntries = PrimData.NumInstances;
		if (PrimData.LocalPayloadDataOffset != INDEX_NONE)
		{
			Tmp.InstancePayloadDataOffset = PayloadStartOffset + PrimData.LocalPayloadDataOffset;
			Tmp.InstancePayloadDataStride = PrimData.SourceData.GetPayloadFloat4Stride();
		}
		else
		{
			Tmp.InstancePayloadDataOffset = INDEX_NONE;
			Tmp.InstancePayloadDataStride = 0;
		}

		FPrimitiveSceneShaderData::Setup(Tmp, OutData);
	}

	FORCEINLINE_GPUSCENE bool GetInstanceInfo(int32 ItemIndex, FInstanceUploadInfo& InstanceUploadInfo) const
	{
		if (ItemIndex < PrimitiveData.Num())
		{
			const FGPUScenePrimitiveCollector::FPrimitiveData& PrimData = PrimitiveData[ItemIndex];
			const FPrimitiveUniformShaderParameters& ShaderParams = *PrimData.ShaderParams;

			InstanceUploadInfo.PrimitiveID					= PrimitiveIDStartOffset + ItemIndex;
			InstanceUploadInfo.PrimitiveToWorld				= ShaderParams.LocalToRelativeWorld;
			InstanceUploadInfo.PrevPrimitiveToWorld			= ShaderParams.PreviousLocalToRelativeWorld;
			InstanceUploadInfo.InstancePayloadDataOffset	= PrimData.LocalPayloadDataOffset == INDEX_NONE ? INDEX_NONE : PayloadStartOffset + PrimData.LocalPayloadDataOffset;
			InstanceUploadInfo.InstancePayloadDataStride	= PrimData.SourceData.GetPayloadFloat4Stride();
			InstanceUploadInfo.InstanceCustomDataCount		= PrimData.SourceData.NumInstanceCustomDataFloats;
			InstanceUploadInfo.InstanceFlags                = PrimData.SourceData.PayloadDataFlags;
			InstanceUploadInfo.PrimitiveInstances 			= PrimData.SourceData.InstanceSceneData;
			InstanceUploadInfo.InstanceDynamicData 			= PrimData.SourceData.InstanceDynamicData;
			InstanceUploadInfo.InstanceLocalBounds 			= PrimData.SourceData.InstanceLocalBounds;
			InstanceUploadInfo.InstanceCustomData 			= PrimData.SourceData.InstanceCustomData;
			InstanceUploadInfo.InstanceRandomID				= TConstArrayView<float>();
			InstanceUploadInfo.InstanceHierarchyOffset		= TConstArrayView<uint32>();
			InstanceUploadInfo.InstanceLightShadowUVBias	= TConstArrayView<FVector4f>();
#if WITH_EDITOR
			InstanceUploadInfo.InstanceEditorData			= TConstArrayView<uint32>();
#endif
			InstanceUploadInfo.NumInstances = PrimData.SourceData.InstanceSceneData.Num();
			InstanceUploadInfo.bIsPrimitiveForceHidden = false;
			// Use primitive to world as instance transform:
			if (InstanceUploadInfo.PrimitiveInstances.Num() == 0)
			{
				InstanceUploadInfo.NumInstances = 1;
			}

			return true;
		}

		return false;
	}

	FORCEINLINE_GPUSCENE bool GetLightMapInfo(int32 ItemIndex, FLightMapUploadInfo& UploadInfo) const
	{
		return false;
	}

	const TArray<FGPUScenePrimitiveCollector::FPrimitiveData, TInlineAllocator<8>> &PrimitiveData;
	const int32 PrimitiveIDStartOffset;
	const int32 InstanceIDStartOffset;
	const int32 PayloadStartOffset;
	const uint32 SceneFrameNumber;
	TArray<uint32, SceneRenderingAllocator> PrimitivesIds;
};

void FGPUScene::UploadDynamicPrimitiveShaderDataForViewInternal(FRDGBuilder& GraphBuilder, FViewInfo& View, bool bRayTracing, UE::Renderer::Private::IShadowInvalidatingInstances *ShadowInvalidatingInstances)
{
	LLM_SCOPE_BYTAG(GPUScene);

	RDG_EVENT_SCOPE(GraphBuilder, "GPUScene.UploadDynamicPrimitiveShaderDataForView");
	SCOPED_NAMED_EVENT(FGPUScene_UploadDynamicPrimitiveShaderDataForView, FColor::Green);

	ensure(bInBeginEndBlock);
	ensure(DynamicPrimitivesOffset >= Scene.GetMaxPersistentPrimitiveIndex());

	FGPUScenePrimitiveCollector& Collector = bRayTracing ? View.RayTracingDynamicPrimitiveCollector : View.DynamicPrimitiveCollector;

	// Auto-commit if not done (should usually not be done, but sometimes the UploadDynamicPrimitiveShaderDataForViewInternal is called to ensure the 
	// CachedViewUniformShaderParameters is set on the view.
	if (!Collector.bCommitted)
	{
		Collector.Commit();
	}

	const int32 NumPrimitiveDataUploads = Collector.Num();
	ensure(Collector.GetPrimitiveIdRange().Size<int32>() == NumPrimitiveDataUploads);

	// Make sure we are not trying to upload data that lives in a different context.
	ensure(Collector.UploadData == nullptr || CurrentDynamicContext->DymamicPrimitiveUploadData.Find(Collector.UploadData) != INDEX_NONE);

	// Skip uploading empty & already uploaded data
	const bool bNeedsUpload = Collector.UploadData != nullptr && NumPrimitiveDataUploads > 0 && !Collector.UploadData->bIsUploaded;
	if (bNeedsUpload)
	{
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, UploadDynamicPrimitiveShaderData);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UploadDynamicPrimitiveShaderData);

		Collector.UploadData->bIsUploaded = true;

		const int32 UploadIdStart = Collector.GetPrimitiveIdRange().GetLowerBoundValue();
		const int32 InstanceIdStart = Collector.UploadData->InstanceSceneDataOffset;
		ensure(UploadIdStart < DynamicPrimitivesOffset);
		ensure(InstanceIdStart != INDEX_NONE);

		if (ShadowInvalidatingInstances)
		{
			// Enqueue cache invalidations for all dynamic primitives' instances, as they will be removed this frame and are not associated
			// with any particular FPrimitiveSceneInfo. Will occur on the next call to UpdateAllPrimitiveSceneInfos
			for (const FGPUScenePrimitiveCollector::FPrimitiveData& PrimitiveData : Collector.UploadData->PrimitiveData)
			{
				check(PrimitiveData.LocalInstanceSceneDataOffset != INDEX_NONE);
				ShadowInvalidatingInstances->AddInstanceRange(FPersistentPrimitiveIndex(),
					PrimitiveData.LocalInstanceSceneDataOffset + uint32(InstanceIdStart), PrimitiveData.NumInstances);
			}
		}

		FUploadDataSourceAdapterDynamicPrimitives& UploadAdapter = *GraphBuilder.AllocObject<FUploadDataSourceAdapterDynamicPrimitives>(
			Collector.UploadData->PrimitiveData,
			UploadIdStart,
			InstanceIdStart,
			Collector.UploadData->InstancePayloadDataOffset,
			SceneFrameNumber);

		FRegisteredBuffers BufferState = UpdateBufferAllocations(GraphBuilder);

		// Run a pass that clears (Sets ID to invalid) any instances that need it.
		AddClearInstancesPass(GraphBuilder, Scene.InstanceCullingOcclusionQueryRenderer);

		UploadGeneral<FUploadDataSourceAdapterDynamicPrimitives>(GraphBuilder, BufferState, nullptr, UploadAdapter, UE::Tasks::FTask{});
	}

	FSceneUniformBuffer& SceneUniforms = View.GetSceneUniforms();
	FillSceneUniformBuffer(GraphBuilder, SceneUniforms);

	// Execute any instance data GPU writer callbacks. (Note: Done after the UB update, in case the user requires it)
	if (bNeedsUpload) 
	{
		const uint32 PrimitiveIdStart = Collector.GetPrimitiveIdRange().GetLowerBoundValue();
		const uint32 InstanceIdStart = Collector.UploadData->InstanceSceneDataOffset;

		// Determine if we have any GPU data writers this frame and simultaneously defer any writes that must happen later in the frame
		TArray<uint32, SceneRenderingAllocator> ImmediateWrites;
		ImmediateWrites.Reserve(Collector.UploadData->GPUWritePrimitives.Num());
		for (uint32 PrimitiveIndex : Collector.UploadData->GPUWritePrimitives)
		{
			const FGPUScenePrimitiveCollector::FPrimitiveData& PrimData = Collector.UploadData->PrimitiveData[PrimitiveIndex];
			const EGPUSceneGPUWritePass GPUWritePass = PrimData.SourceData.DataWriterGPUPass;
			
			// We're going to immediately execute any GPU writers whose write pass is immediate or has already happened this frame
			if (GPUWritePass == EGPUSceneGPUWritePass::None || GPUWritePass <= LastDeferredGPUWritePass)
			{
				ImmediateWrites.Add(PrimitiveIndex);
			}
			else
			{
				// Defer this write to a later GPU write pass
				FDeferredGPUWrite DeferredWrite;
				DeferredWrite.DataWriterGPU = PrimData.SourceData.DataWriterGPU;
				DeferredWrite.ViewId = View.SceneRendererPrimaryViewId;
				DeferredWrite.PrimitiveId = PrimitiveIdStart + PrimitiveIndex;
				DeferredWrite.InstanceSceneDataOffset = InstanceIdStart + PrimData.LocalInstanceSceneDataOffset;
				DeferredWrite.NumCustomDataFloats = PrimData.SourceData.NumInstanceCustomDataFloats;
				DeferredWrite.PackedInstanceSceneDataFlags = PrimData.SourceData.PayloadDataFlags;

				uint32 PassIndex = uint32(PrimData.SourceData.DataWriterGPUPass);
				DeferredGPUWritePassDelegates[PassIndex].Add(DeferredWrite);
			}
		}

		if (ImmediateWrites.Num() > 0)
		{
			// Execute writes that should execute immediately
			RDG_EVENT_SCOPE(GraphBuilder, "GPU Writer Delegates");

			FGPUSceneWriteDelegateParams Params;
			Params.View = &View;
			Params.GPUWritePass = EGPUSceneGPUWritePass::None;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Params.GPUSceneWriterUB = GetWriteParameters(GraphBuilder, &Params.GPUWriteParams);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			for (uint32 PrimitiveIndex : ImmediateWrites)
			{
				const FGPUScenePrimitiveCollector::FPrimitiveData& PrimData = Collector.UploadData->PrimitiveData[PrimitiveIndex];
				Params.PersistentPrimitiveId = PrimitiveIdStart + PrimitiveIndex;
				Params.InstanceSceneDataOffset = InstanceIdStart + PrimData.LocalInstanceSceneDataOffset;
				Params.NumCustomDataFloats = PrimData.SourceData.NumInstanceCustomDataFloats;
				Params.PackedInstanceSceneDataFlags = PrimData.SourceData.PayloadDataFlags;

				PrimData.SourceData.DataWriterGPU->Execute(GraphBuilder, Params);
			}
		}
	}
}

bool FGPUScene::FillSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUB) const
{
	if (!bIsEnabled)
	{
		return false;
	}

	if (PrimitiveBuffer != nullptr)
	{
		return SceneUB.Set(SceneUB::GPUScene, GetShaderParameters(GraphBuilder));
	}
	else
	{
		// leave the dummy data in place - the gpu scene is not yet populated
		return false;
	}
}

FGPUSceneResourceParameters FGPUScene::GetShaderParameters(FRDGBuilder& GraphBuilder) const
{
	FRegisteredBuffers BufferState = CachedRegisteredBuffers;
	// If we're not in a begin/end block we need to register the buffers here and now.
	if (!BufferState.IsValid())
	{
		BufferState = RegisterBuffers(GraphBuilder);
	}

	FGPUSceneResourceParameters TmpParameters;
	TmpParameters.CommonParameters = GetCommonParameters();
	// Not in an active rendering context, must register the buffers and fill in the data structure.
	TmpParameters.GPUSceneInstanceSceneData = GraphBuilder.CreateSRV(BufferState.InstanceSceneDataBuffer);
	TmpParameters.GPUSceneInstancePayloadData = GraphBuilder.CreateSRV(BufferState.InstancePayloadDataBuffer);
	TmpParameters.GPUScenePrimitiveSceneData = GraphBuilder.CreateSRV(BufferState.PrimitiveBuffer);
	TmpParameters.GPUSceneLightmapData = GraphBuilder.CreateSRV(BufferState.LightmapDataBuffer);
	TmpParameters.GPUSceneLightData = GraphBuilder.CreateSRV(BufferState.LightDataBuffer);

	return TmpParameters;
}

void FGPUScene::AddPrimitiveToUpdate(FPersistentPrimitiveIndex PersistentPrimitiveIndex, EPrimitiveDirtyState DirtyState)
{
	LLM_SCOPE_BYTAG(GPUScene);

	if (bIsEnabled && PersistentPrimitiveIndex.IsValid())
	{
		ResizeDirtyState(PersistentPrimitiveIndex.Index + 1);

		// Make sure we aren't updating same primitive multiple times.
		if (PrimitiveDirtyState[PersistentPrimitiveIndex.Index] == EPrimitiveDirtyState::None)
		{
			PrimitivesToUpdate.Add(PersistentPrimitiveIndex);
		}

		EPrimitiveDirtyState NewState = PrimitiveDirtyState[PersistentPrimitiveIndex.Index] | DirtyState;

		// When a primitive is removed, we clear pending "added" state since it is no longer being added
		// Thus if the state is both added & removed, we know this is a current primitive, where the slot has been reused.
		if (EnumHasAnyFlags(DirtyState, EPrimitiveDirtyState::Removed))
		{
			EnumRemoveFlags(NewState, EPrimitiveDirtyState::Added);
		}
		PrimitiveDirtyState[PersistentPrimitiveIndex.Index] = NewState;
	}
}


void FGPUScene::Update(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue, const FUpdateFromComputeCommands& UpdatesFromCompute, const UE::Tasks::FTask& UpdateTaskPrerequisites, FSceneExtensionsUpdaters& SceneExtensionsUpdaters)
{
	if (!bIsEnabled)
	{
		return;
	}

	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	ensure(!bInBeginEndBlock);

	// Default state when updated (no "dynamic primitives" pushed)
	DynamicPrimitivesOffset = Scene.GetMaxPersistentPrimitiveIndex();

	UpdateInternal(GraphBuilder, ExternalAccessQueue, UpdateTaskPrerequisites, UpdatesFromCompute, SceneExtensionsUpdaters);
}

void FGPUScene::UploadDynamicPrimitiveShaderDataForView(FRDGBuilder& GraphBuilder, FViewInfo& View, bool bRayTracing, UE::Renderer::Private::IShadowInvalidatingInstances *ShadowInvalidatingInstances)
{
	if (bIsEnabled)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

		UploadDynamicPrimitiveShaderDataForViewInternal(GraphBuilder, View, bRayTracing, ShadowInvalidatingInstances);
	}
}

// Grow the last entry in the Output using the given Range if trivial merge is possible (typical case)
inline void AddOrMergeInstanceRange(TArray<FGPUSceneInstanceRange>& Output, FGPUSceneInstanceRange Range)
{
	if (!Output.IsEmpty())
	{
		FGPUSceneInstanceRange& Last = Output.Last();
		if (Range.InstanceSceneDataOffset == (Last.InstanceSceneDataOffset + Last.NumInstanceSceneDataEntries) &&
			Range.Primitive == Last.Primitive)
		{
			Last.NumInstanceSceneDataEntries += Range.NumInstanceSceneDataEntries;
			return;
		}
	}
	Output.Add(Range);
}

int32 FGPUScene::AllocateInstanceSceneDataSlots(FPersistentPrimitiveIndex PersistentPrimitiveIndex, int32 NumInstanceSceneDataEntries)
{
	LLM_SCOPE_BYTAG(GPUScene);

	if (bIsEnabled)
	{
		if (NumInstanceSceneDataEntries > 0)
		{
			const int32 InstanceSceneDataOffset = InstanceSceneDataAllocator.Allocate(NumInstanceSceneDataEntries);
			AddOrMergeInstanceRange(InstanceRangesToClear, FInstanceRange{ PersistentPrimitiveIndex, uint32(InstanceSceneDataOffset), uint32(NumInstanceSceneDataEntries) });
#if LOG_INSTANCE_ALLOCATIONS
			UE_LOGF(LogTemp, Warning, "AllocateInstanceSceneDataSlots: [%6d,%6d)", InstanceSceneDataOffset, InstanceSceneDataOffset + NumInstanceSceneDataEntries);
#endif

			return InstanceSceneDataOffset;
		}
	}
	return INDEX_NONE;
}


void FGPUScene::FreeInstanceSceneDataSlots(int32 InstanceSceneDataOffset, int32 NumInstanceSceneDataEntries)
{
	LLM_SCOPE_BYTAG(GPUScene);

	if (bIsEnabled)
	{
		InstanceSceneDataAllocator.Free(InstanceSceneDataOffset, NumInstanceSceneDataEntries);
		AddOrMergeInstanceRange(InstanceRangesToClear, FInstanceRange{ {}, uint32(InstanceSceneDataOffset), uint32(NumInstanceSceneDataEntries) });
#if LOG_INSTANCE_ALLOCATIONS
		UE_LOGF(LogTemp, Warning, "FreeInstanceSceneDataSlots: [%6d,%6d)", InstanceSceneDataOffset, InstanceSceneDataOffset + NumInstanceSceneDataEntries);
#endif
	}
}

int32 FGPUScene::AllocateInstancePayloadDataSlots(int32 NumInstancePayloadFloat4Entries)
{
	LLM_SCOPE_BYTAG(GPUScene);

	if (bIsEnabled)
	{
		if (NumInstancePayloadFloat4Entries > 0)
		{
			int32 InstancePayloadDataOffset = InstancePayloadDataAllocator.Allocate(NumInstancePayloadFloat4Entries);
			return InstancePayloadDataOffset;
		}
	}
	return INDEX_NONE;
}

void FGPUScene::FreeInstancePayloadDataSlots(int32 InstancePayloadDataOffset, int32 NumInstancePayloadFloat4Entries)
{
	LLM_SCOPE_BYTAG(GPUScene);

	if (bIsEnabled)
	{
		InstancePayloadDataAllocator.Free(InstancePayloadDataOffset, NumInstancePayloadFloat4Entries);
	}
}

uint32 FGPUScene::GetInstanceIdUpperBoundGPU() const 
{ 
	return FMath::Min( uint32(InstanceSceneDataAllocator.GetMaxSize()), MAX_INSTANCE_ID); 
}

struct FPrimitiveSceneDebugNameInfo
{
	uint32 PrimitiveID;
	uint16 Offset;
	uint8  Length;
	uint8  Pad0;
};

class FGPUSceneDebugRenderCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGPUSceneDebugRenderCS);
	SHADER_USE_PARAMETER_STRUCT(FGPUSceneDebugRenderCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER(int32, bDrawAll)
		SHADER_PARAMETER(int32, bDrawUpdatedOnly)
		SHADER_PARAMETER(int32, SelectedNameInfoCount)
		SHADER_PARAMETER(int32, SelectedNameCharacterCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, SelectedPrimitiveFlags)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, SelectedPrimitiveNameInfos)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint8>, SelectedPrimitiveNames)
		SHADER_PARAMETER(FVector3f, PickingRayStart)
		SHADER_PARAMETER(FVector3f, PickingRayEnd)
		SHADER_PARAMETER(float, DrawRange)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint32>, RWDrawCounter)
	END_SHADER_PARAMETER_STRUCT()

public:
	static constexpr uint32 NumThreadsPerGroup = 128U;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return UseGPUScene(Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform);;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);

		// Skip optimization for avoiding long compilation time due to large UAV writes
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGPUSceneDebugRenderCS, "/Engine/Private/GPUSceneDebugRender.usf", "GPUSceneDebugRenderCS", SF_Compute);

void FGPUScene::DebugRender(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer, FViewInfo& View)
{
	int32 DebugMode = CVarGPUSceneDebugMode.GetValueOnRenderThread();
	if (DebugMode > 0)
	{
		// Force ShaderPrint on.
		ShaderPrint::SetEnabled(true); 

		int32 NumInstances = InstanceSceneDataAllocator.GetMaxSize();
		if (ShaderPrint::IsEnabled(View.ShaderPrintData) && NumInstances > 0)
		{
			// This lags by one frame, so may miss some in one frame, also overallocates since we will cull a lot.
			ShaderPrint::RequestSpaceForLines(NumInstances * 12);

			FRDGBufferRef DrawCounterBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, 1), TEXT("GPUScene.DebugCounter"));
			FRDGBufferUAVRef DrawCounterUAV = GraphBuilder.CreateUAV(DrawCounterBuffer, PF_R32_UINT);
			AddClearUAVPass(GraphBuilder, DrawCounterUAV, 0u);

			const uint32 MaxPrimitiveNameCount = 128u;
			check(sizeof(FPrimitiveSceneDebugNameInfo) == 8);
			TArray<FPrimitiveSceneDebugNameInfo> SelectedNameInfos;
			TArray<uint8> SelectedNames;
			SelectedNames.Reserve(MaxPrimitiveNameCount * 30u);

			uint32 SelectedCount = 0;
			TArray<uint32> SelectedPrimitiveFlags;
			const int32 BitsPerWord = (sizeof(uint32) * 8U);
			SelectedPrimitiveFlags.Init(0U, FMath::DivideAndRoundUp(Scene.GetMaxPersistentPrimitiveIndex(), BitsPerWord));
			for (int32 PackedIndex = 0; PackedIndex < Scene.PrimitiveSceneProxies.Num(); ++PackedIndex)
			{
				if (Scene.PrimitiveSceneProxies[PackedIndex]->IsSelected())
				{
					FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene.Primitives[PackedIndex];
					FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitiveSceneInfo->GetPersistentIndex();

					SelectedPrimitiveFlags[PersistentPrimitiveIndex.Index / BitsPerWord] |= 1U << uint32(PersistentPrimitiveIndex.Index % BitsPerWord);

					// Collect Names
					if (SelectedNameInfos.Num() < MaxPrimitiveNameCount)
					{
						const FString OwnerName = PrimitiveSceneInfo->GetFullnameForDebuggingOnly();
						const uint32 NameOffset = SelectedNames.Num();
						const uint32 NameLength = OwnerName.Len();
						for (TCHAR C : OwnerName)
						{
							SelectedNames.Add(uint8(C));
						}

						FPrimitiveSceneDebugNameInfo& NameInfo = SelectedNameInfos.AddDefaulted_GetRef();
						NameInfo.PrimitiveID= PersistentPrimitiveIndex.Index;
						NameInfo.Length		= NameLength;
						NameInfo.Offset		= NameOffset;
						++SelectedCount;
					}
				}
			}

			if (SelectedNameInfos.IsEmpty())
			{
				FPrimitiveSceneDebugNameInfo& NameInfo = SelectedNameInfos.AddDefaulted_GetRef();
				NameInfo.PrimitiveID= ~0;
				NameInfo.Length		= 4;
				NameInfo.Offset		= 0;
				SelectedNames.Add(uint8('N'));
				SelectedNames.Add(uint8('o'));
				SelectedNames.Add(uint8('n'));
				SelectedNames.Add(uint8('e'));
			}

			// Request more characters for printing if needed
			ShaderPrint::RequestSpaceForCharacters(SelectedNames.Num() + SelectedCount * 48u);

			FRDGBufferRef SelectedPrimitiveNames	 = CreateVertexBuffer(GraphBuilder, TEXT("GPUScene.Debug.SelectedPrimitiveNames"), FRDGBufferDesc::CreateBufferDesc(1, SelectedNames.Num()), SelectedNames.GetData(), SelectedNames.Num());
			FRDGBufferRef SelectedPrimitiveNameInfos = CreateStructuredBuffer(GraphBuilder, TEXT("GPUScene.Debug.SelectedPrimitiveNameInfos"), SelectedNameInfos);
			FRDGBufferRef SelectedPrimitiveFlagsRDG  = CreateStructuredBuffer(GraphBuilder, TEXT("GPUScene.Debug.SelectedPrimitiveFlags"), SelectedPrimitiveFlags);

			FGPUSceneDebugRenderCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGPUSceneDebugRenderCS::FParameters>();
			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintUniformBuffer);
			PassParameters->Scene = SceneUniformBuffer.GetBuffer(GraphBuilder);
			PassParameters->bDrawUpdatedOnly = DebugMode == 3;
			PassParameters->bDrawAll = DebugMode != 2;
			PassParameters->SelectedNameInfoCount = SelectedCount;
			PassParameters->SelectedNameCharacterCount = SelectedCount > 0 ? SelectedNames.Num() : 0;
			PassParameters->SelectedPrimitiveFlags = GraphBuilder.CreateSRV(SelectedPrimitiveFlagsRDG);
			PassParameters->SelectedPrimitiveNameInfos = GraphBuilder.CreateSRV(SelectedPrimitiveNameInfos);
			PassParameters->SelectedPrimitiveNames = GraphBuilder.CreateSRV(SelectedPrimitiveNames, PF_R8_UINT);
			PassParameters->DrawRange = CVarGPUSceneDebugDrawRange.GetValueOnRenderThread();
			PassParameters->RWDrawCounter = DrawCounterUAV;

			FVector PickingRayStart(ForceInit);
			FVector PickingRayDir(ForceInit);
			View.DeprojectFVector2D(View.CursorPos, PickingRayStart, PickingRayDir);

			PassParameters->PickingRayStart = (FVector3f)PickingRayStart;
			PassParameters->PickingRayEnd = FVector3f(PickingRayStart + PickingRayDir * WORLD_MAX);

			auto ComputeShader = View.ShaderMap->GetShader<FGPUSceneDebugRenderCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("GPUScene::DebugRender"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(NumInstances, FGPUSceneDebugRenderCS::NumThreadsPerGroup)
			);
		}
	}
}

void FGPUScene::ConsolidateInstanceDataAllocations()
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ConsolidateInstanceDataAllocations);

	InstanceSceneDataAllocator.Consolidate();
	InstancePayloadDataAllocator.Consolidate();
	LightmapDataAllocator.Consolidate();
		SCOPED_NAMED_EVENT(FGPUScene_EndDeferAllocatorMerges, FColor::Green);
}


TRange<int32> FGPUScene::CommitPrimitiveCollector(FGPUScenePrimitiveCollector& PrimitiveCollector)
{
	ensure(bInBeginEndBlock);
	ensure(CurrentDynamicContext != nullptr);

	// Make sure we are not trying to commit data that lives in a different context.
	ensure(CurrentDynamicContext == nullptr || CurrentDynamicContext->DymamicPrimitiveUploadData.Find(PrimitiveCollector.UploadData) != INDEX_NONE);

	int32 StartOffset = DynamicPrimitivesOffset;
	DynamicPrimitivesOffset += PrimitiveCollector.Num();

	PrimitiveCollector.UploadData->InstanceSceneDataOffset = AllocateInstanceSceneDataSlots({}, PrimitiveCollector.NumInstances());
	PrimitiveCollector.UploadData->InstancePayloadDataOffset = AllocateInstancePayloadDataSlots(PrimitiveCollector.NumPayloadDataSlots());

	return TRange<int32>(StartOffset, DynamicPrimitivesOffset);
}


bool FGPUScene::ExecuteDeferredGPUWritePass(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views, EGPUSceneGPUWritePass GPUWritePass)
{
	check(GPUWritePass != EGPUSceneGPUWritePass::None && GPUWritePass < EGPUSceneGPUWritePass::Num);
	check(LastDeferredGPUWritePass < GPUWritePass);

	if (!bIsEnabled)
	{
		return false;
	}

	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	// Mark this pass as having executed for the frame
	LastDeferredGPUWritePass = GPUWritePass;

	const uint32 PassIndex = uint32(GPUWritePass);
	if (DeferredGPUWritePassDelegates[PassIndex].Num() == 0)
	{
		// No deferred writes to make for this pass this frame
		return false;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "GPUScene.DeferredGPUWrites - Pass %u", uint32(GPUWritePass));

	FGPUSceneWriteDelegateParams Params;
	Params.GPUWritePass = GPUWritePass;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Params.GPUSceneWriterUB = GetWriteParameters(GraphBuilder, &Params.GPUWriteParams);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (const FDeferredGPUWrite& DeferredWrite : DeferredGPUWritePassDelegates[PassIndex])
	{
		FViewInfo* View = Views.FindByPredicate([&DeferredWrite](const FViewInfo& V){ return V.SceneRendererPrimaryViewId == DeferredWrite.ViewId; });
		checkf(View != nullptr, TEXT("Deferred GPU Write found with no matching view in the view family"));
		
		Params.View = View;
		Params.PersistentPrimitiveId = DeferredWrite.PrimitiveId;
		Params.InstanceSceneDataOffset = DeferredWrite.InstanceSceneDataOffset;
		Params.NumCustomDataFloats = DeferredWrite.NumCustomDataFloats;
		Params.PackedInstanceSceneDataFlags = DeferredWrite.PackedInstanceSceneDataFlags;

		DeferredWrite.DataWriterGPU->Execute(GraphBuilder, Params);
	}

	DeferredGPUWritePassDelegates[PassIndex].Reset();
	return true;
}


bool FGPUScene::HasPendingGPUWrite(uint32 PrimitiveId) const
{
	for (uint32 PassIndex = uint32(LastDeferredGPUWritePass) + 1; PassIndex < uint32(EGPUSceneGPUWritePass::Num); ++PassIndex) //-V1008 Disable single-iteration loop warning as this is by design in case EGPUSceneGPUWritePass is extended later.
	{
		if (DeferredGPUWritePassDelegates[PassIndex].FindByPredicate(
			[PrimitiveId](const FDeferredGPUWrite& Write)
			{
				return Write.PrimitiveId == PrimitiveId;
			}))
		{
			return true;
		}
	}

	return false;
}

void FGPUScene::OnPreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ScenePreUpdateData)
{
	ScenePreUpdateData.PrimitiveUpdates.ForEachUpdateCommand(ESceneUpdateCommandFilter::Updated | ESceneUpdateCommandFilter::Deleted, EPrimitiveUpdateDirtyFlags::GPUState, 
	[&](const FPrimitiveUpdateCommand& Cmd)
	{
		AddPrimitiveToUpdate(Cmd.GetPersistentId(), Cmd.IsDelete() ? EPrimitiveDirtyState::Removed : EPrimitiveDirtyState::ChangedAll );
	});
}

void FGPUScene::OnPostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ScenePostUpdateData)
{
	for (FPersistentPrimitiveIndex PersistentPrimitiveIndex  : ScenePostUpdateData.AddedPrimitiveIds)
	{
		AddPrimitiveToUpdate(PersistentPrimitiveIndex, EPrimitiveDirtyState::AddedMask);
	}
}

void FGPUScene::OnPostLightSceneInfoUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightsPostUpdateData)
{
	UpdateGPULights(GraphBuilder, UE::Tasks::FTask{}, LightsPostUpdateData);
}

SIZE_T FGPUScene::GetAllocatedSize() const
{
	return PrimitivesToUpdate.GetAllocatedSize()
		+ InstanceRangesToClear.GetAllocatedSize()
		+ PrimitiveDirtyState.GetAllocatedSize()
		+ DeferredGPUWritePassDelegates[uint32(EGPUSceneGPUWritePass::PostOpaqueRendering)].GetAllocatedSize();
}

FGPUSceneResourceParametersRHI FGPUScene::GetShaderParametersRHI() const
{
	FGPUSceneResourceParametersRHI Result;
	Result.CommonParameters = GetCommonParameters();

	Result.GPUSceneInstanceSceneData = InstanceSceneDataBuffer->GetSRV();
	Result.GPUSceneInstancePayloadData = InstancePayloadDataBuffer->GetSRV();
	Result.GPUScenePrimitiveSceneData = PrimitiveBuffer->GetSRV();
	Result.GPUSceneLightmapData = LightmapDataBuffer->GetSRV();
	Result.GPUSceneLightData = LightDataBuffer.GetPooledBuffer()->GetSRV();

	return Result;
}

FGPUSceneCommonParameters FGPUScene::GetCommonParameters() const
{
	return CommonParameters;
}

FGPUSceneDynamicContext::~FGPUSceneDynamicContext()
{
	Release();
}

void FGPUSceneDynamicContext::Release()
{
	for (auto UploadData : DymamicPrimitiveUploadData)
	{
		if (UploadData->InstanceSceneDataOffset != INDEX_NONE)
		{
			GPUScene.FreeInstanceSceneDataSlots(UploadData->InstanceSceneDataOffset, UploadData->TotalInstanceCount);
		}
		if (UploadData->InstancePayloadDataOffset != INDEX_NONE)
		{
			GPUScene.FreeInstancePayloadDataSlots(UploadData->InstancePayloadDataOffset, UploadData->InstancePayloadDataFloat4Count);
		}
		delete UploadData;
	}
	DymamicPrimitiveUploadData.Empty();
}

FGPUScenePrimitiveCollector::FUploadData* FGPUSceneDynamicContext::AllocateDynamicPrimitiveData()
{
	LLM_SCOPE_BYTAG(GPUScene);

	FGPUScenePrimitiveCollector::FUploadData* UploadData = new FGPUScenePrimitiveCollector::FUploadData;
	UE::TScopeLock ScopeLock(DymamicPrimitiveUploadDataMutex);
	DymamicPrimitiveUploadData.Add(UploadData);
	return UploadData;
}

TRDGUniformBufferRef<FGPUSceneWriterUniformParameters> FGPUScene::GetWriteParameters(FRDGBuilder& GraphBuilder, FGPUSceneWriterParameters* LegacyParameters, bool bSkipBarriers)
{
	FRegisteredBuffers BufferState = CachedRegisteredBuffers;
	// If we're not in a begin/end block we need to register the buffers here and now.
	if (!BufferState.IsValid())
	{
		BufferState = RegisterBuffers(GraphBuilder);
	}
	FGPUSceneWriterUniformParameters* UBParams = GraphBuilder.AllocParameters<FGPUSceneWriterUniformParameters>();
	UBParams->CommonParameters = CommonParameters;
	UBParams->GPUSceneInstanceSceneDataRW = GraphBuilder.CreateUAV(BufferState.InstanceSceneDataBuffer,		bSkipBarriers ? ERDGUnorderedAccessViewFlags::SkipBarrier : ERDGUnorderedAccessViewFlags::None);
	UBParams->GPUSceneInstancePayloadDataRW = GraphBuilder.CreateUAV(BufferState.InstancePayloadDataBuffer, bSkipBarriers ? ERDGUnorderedAccessViewFlags::SkipBarrier : ERDGUnorderedAccessViewFlags::None);
	UBParams->GPUScenePrimitiveSceneDataRW = GraphBuilder.CreateUAV(BufferState.PrimitiveBuffer,			bSkipBarriers ? ERDGUnorderedAccessViewFlags::SkipBarrier : ERDGUnorderedAccessViewFlags::None);
	TRDGUniformBufferRef<FGPUSceneWriterUniformParameters> Result = GraphBuilder.CreateUniformBuffer(UBParams);

	if (LegacyParameters != nullptr)
	{
		// Note, this is only correct if the tiling is disabled, but the parameter is only there because it is exposed in public API (and that only an accident, really), it is not used internally.
		LegacyParameters->GPUSceneInstanceSceneDataSOAStride = 1u << InstanceDataTileSizeLog2;
		LegacyParameters->GPUSceneNumAllocatedInstances = UBParams->CommonParameters.GPUSceneMaxAllocatedInstanceId;
		LegacyParameters->GPUSceneNumAllocatedPrimitives = UBParams->CommonParameters.GPUSceneMaxPersistentPrimitiveIndex;
		LegacyParameters->GPUSceneInstanceSceneDataRW = UBParams->GPUSceneInstanceSceneDataRW;
		LegacyParameters->GPUSceneInstancePayloadDataRW = UBParams->GPUSceneInstancePayloadDataRW;
		LegacyParameters->GPUScenePrimitiveSceneDataRW = UBParams->GPUScenePrimitiveSceneDataRW;
		LegacyParameters->GPUSceneWriterUB = Result;
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
		LegacyParameters->CommonParameters = CommonParameters;
#else
		LegacyParameters->GPUSceneFrameNumber = UBParams->CommonParameters.GPUSceneFrameNumber;
#endif
	}
	return Result;
}

uint32 FGPUScene::GetInstanceDataTileSizeMask() const 
{ 
	return (1u << InstanceDataTileSizeLog2) - 1u; 
}

uint32 FGPUScene::GetInstanceDataTileStride() const 
{ 
	return FInstanceSceneShaderData::GetDataStrideInFloat4s() << InstanceDataTileSizeLog2; 
}

class FGPUSceneSetInstancePrimitiveIdCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGPUSceneSetInstancePrimitiveIdCS);
	SHADER_USE_PARAMETER_STRUCT(FGPUSceneSetInstancePrimitiveIdCS, FGlobalShader)
public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUScene::FInstanceGPULoadBalancer::FShaderParameters, BatcherParameters)
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneWriterParameters, GPUSceneWriterParameters)
#else
		// TODO: Go back to using the UB. Reverted to use loose parameters due to issues with  DX11 RHI.
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FGPUSceneWriterUniformParameters, GPUSceneWriterUB)
#endif
	END_SHADER_PARAMETER_STRUCT()

	static constexpr int32 NumThreadsPerGroup = FGPUScene::FInstanceGPULoadBalancer::ThreadGroupSize;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return UseGPUScene(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		FGPUScene::FInstanceGPULoadBalancer::SetShaderDefines(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FGPUSceneSetInstancePrimitiveIdCS, "/Engine/Private/GPUScene/GPUSceneDataManagement.usf", "GPUSceneSetInstancePrimitiveIdCS", SF_Compute);


void FGPUScene::AddUpdatePrimitiveIdsPass(FRDGBuilder& GraphBuilder, FInstanceGPULoadBalancer& IdOnlyUpdateItems)
{
	if (!IdOnlyUpdateItems.IsEmpty())
	{
		FGPUSceneSetInstancePrimitiveIdCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGPUSceneSetInstancePrimitiveIdCS::FParameters>();

		IdOnlyUpdateItems.Upload(GraphBuilder).GetShaderParameters(GraphBuilder, PassParameters->BatcherParameters);
		
#if ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GetWriteParameters(GraphBuilder, &PassParameters->GPUSceneWriterParameters, false);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
		PassParameters->GPUSceneWriterUB = GetWriteParameters(GraphBuilder, nullptr, false);
#endif
		auto ComputeShader = GetGlobalShaderMap(ShaderPlatform)->GetShader<FGPUSceneSetInstancePrimitiveIdCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("GPUScene::SetInstancePrimitiveIdCS"),
			ComputeShader,
			PassParameters,
			IdOnlyUpdateItems.GetWrappedCsGroupCount()
		);
	}
}

FBatchedPrimitiveShaderData::FBatchedPrimitiveShaderData(const FPrimitiveSceneProxy* Proxy)
	: Data(InPlace, NoInit)
{
	FPrimitiveUniformShaderParametersBuilder Builder = FPrimitiveUniformShaderParametersBuilder{};
	Proxy->BuildUniformShaderParameters(Builder);
	Setup(Builder.Build());
}

void FBatchedPrimitiveShaderData::Emplace(FBatchedPrimitiveShaderData* Dest, const FPrimitiveUniformShaderParameters& ShaderParams)
{
	new (Dest) FBatchedPrimitiveShaderData(ShaderParams);
}

void FBatchedPrimitiveShaderData::Emplace(FBatchedPrimitiveShaderData* Dest, const FPrimitiveSceneProxy* Proxy)
{
	new (Dest) FBatchedPrimitiveShaderData(Proxy);
}

void FBatchedPrimitiveShaderData::Setup(const FPrimitiveUniformShaderParameters& PrimitiveUniformShaderParameters)
{
	// Note: layout must match LoadPrimitiveDataUBO in SceneDataMobileLoader.ush
	int32 i = 0;

	const bool bAllowStaticLighting = IsStaticLightingAllowed();
	if (bAllowStaticLighting)
	{
		FVector4f LightMapUVScaleBias = FVector4f(1, 1, 0, 0);
		FVector4f ShadowMapUVScaleBias = FVector4f(1, 1, 0, 0);
		uint32 LightMapDataIdx = 0;

		// FIXME: valid lightmap data
		Data[i+0] = LightMapUVScaleBias;
		Data[i+1] = ShadowMapUVScaleBias;
		Data[i+2] = FVector4f(FMath::AsFloat(LightMapDataIdx), 0.f, 0.f, 0.f);
		i+=3;
	}

	// PositionHigh, Flags
	{
		Data[i].X = PrimitiveUniformShaderParameters.PositionHigh.X;
		Data[i].Y = PrimitiveUniformShaderParameters.PositionHigh.Y;
		Data[i].Z = PrimitiveUniformShaderParameters.PositionHigh.Z;
		Data[i].W = FMath::AsFloat(PrimitiveUniformShaderParameters.Flags);
		i+=1;
	}
	
	// LocalToWorld
	{
		FMatrix44f LocalToRelativeWorldTranspose = PrimitiveUniformShaderParameters.LocalToRelativeWorld.GetTransposed();
		Data[i+0] = *(const FVector4f*)&LocalToRelativeWorldTranspose.M[0][0];
		Data[i+1] = *(const FVector4f*)&LocalToRelativeWorldTranspose.M[1][0];
		Data[i+2] = *(const FVector4f*)&LocalToRelativeWorldTranspose.M[2][0];
		i+=3;
	}

	// InvNonUniformScale, TODO .w
	{
		Data[i+0] = FVector4f(PrimitiveUniformShaderParameters.InvNonUniformScale, 0.0f);
		i+=1;
	}

	// ObjectWorldPosition, Radius
	{
		Data[i+0] = PrimitiveUniformShaderParameters.ObjectWorldPositionHighAndRadius;
		Data[i+1] = FVector4f(PrimitiveUniformShaderParameters.ObjectWorldPositionLow, 0.0f);
		i+=2;
	}

	// ActorWorldPosition, TODO .w
	{
		Data[i+0] = FVector4f(PrimitiveUniformShaderParameters.ActorWorldPositionHigh, 0.0f);
		Data[i+1] = FVector4f(PrimitiveUniformShaderParameters.ActorWorldPositionLow, 0.0f);
		i+=2;
	}

	// ObjectOrientation, ObjectBoundsX
	{
		Data[i+0] = FVector4f(PrimitiveUniformShaderParameters.ObjectOrientation, PrimitiveUniformShaderParameters.ObjectBoundsX);
		i+=1;
	}

	// LocalObjectBoundsMin, ObjectBoundsY
	{
		Data[i+0] = FVector4f(PrimitiveUniformShaderParameters.LocalObjectBoundsMin, PrimitiveUniformShaderParameters.ObjectBoundsY);
		i+=1;
	}

	// LocalObjectBoundsMax, ObjectBoundsZ
	{
		Data[i+0] = FVector4f(PrimitiveUniformShaderParameters.LocalObjectBoundsMax, PrimitiveUniformShaderParameters.ObjectBoundsZ);
		i+=1;
	}

	// WorldToLocal
	{
		FMatrix44f RelativeWorldToLocalTranspose = PrimitiveUniformShaderParameters.RelativeWorldToLocal.GetTransposed();
		Data[i+0] = *(const FVector4f*)&RelativeWorldToLocalTranspose.M[0][0];
		Data[i+1] = *(const FVector4f*)&RelativeWorldToLocalTranspose.M[1][0];
		Data[i+2] = *(const FVector4f*)&RelativeWorldToLocalTranspose.M[2][0];
		i+=3;
	}

	// PreviousLocalToWorld
	{
		FMatrix44f PreviousLocalToRelativeWorldTranspose = PrimitiveUniformShaderParameters.PreviousLocalToRelativeWorld.GetTransposed();
		Data[i+0] = *(const FVector4f*)&PreviousLocalToRelativeWorldTranspose.M[0][0];
		Data[i+1] = *(const FVector4f*)&PreviousLocalToRelativeWorldTranspose.M[1][0];
		Data[i+2] = *(const FVector4f*)&PreviousLocalToRelativeWorldTranspose.M[2][0];
		i+=3;
	}

	// PreviousWorldToLocal
	{
		FMatrix44f PreviousRelativeWorldToLocalTranspose = PrimitiveUniformShaderParameters.PreviousRelativeWorldToLocal.GetTransposed();
		Data[i+0] = *(const FVector4f*)&PreviousRelativeWorldToLocalTranspose.M[0][0];
		Data[i+1] = *(const FVector4f*)&PreviousRelativeWorldToLocalTranspose.M[1][0];
		Data[i+2] = *(const FVector4f*)&PreviousRelativeWorldToLocalTranspose.M[2][0];
		i+=3;
	}

	// Set all the custom primitive data float4. This matches the loop in SceneData.ush
	int32 NumCustomData = FMath::Min<int32>(FCustomPrimitiveData::NumCustomPrimitiveDataFloat4s, DataStrideInFloat4s - i);
	for (int32 DataIndex = 0; DataIndex < NumCustomData; ++DataIndex)
	{
		Data[i + DataIndex] = PrimitiveUniformShaderParameters.CustomPrimitiveData[DataIndex];
	}
}

void FGPUScene::AddClearInstancesPass(FRDGBuilder& GraphBuilder, FInstanceCullingOcclusionQueryRenderer* OcclusionQueryRenderer)
{
	// TODO: pre-size these & probably move work to async task.
	FInstanceGPULoadBalancer ClearIdData;
	FInstanceGPULoadBalancer InitializeIdData;
#if LOG_INSTANCE_ALLOCATIONS
	FString ClearRangesStr;
	FString InitializeRangesStr;
#endif
	for (FInstanceRange Range : InstanceRangesToClear)
	{
		// Clamp the instance range to the used instances.
		int32 RangeEnd = FMath::Min(int32(Range.InstanceSceneDataOffset + Range.NumInstanceSceneDataEntries), int32(GetInstanceIdUpperBoundGPU()));
		// Clamp to zero to avoid overflow since the start of the range may also be outside the valid instances
		Range.NumInstanceSceneDataEntries = uint32(FMath::Max(0, RangeEnd - int32(Range.InstanceSceneDataOffset)));

		if (Range.NumInstanceSceneDataEntries > 0u)
		{
			const uint32 PrimitiveID = (Range.Primitive.Index != INDEX_NONE) ? uint32(Range.Primitive.Index) : INVALID_PRIMITIVE_ID;

			if (PrimitiveID == INVALID_PRIMITIVE_ID)
			{
				ClearIdData.Add(Range.InstanceSceneDataOffset, Range.NumInstanceSceneDataEntries, PrimitiveID);
#if LOG_INSTANCE_ALLOCATIONS
				ClearRangesStr.Appendf(TEXT("[%6d, %6d), "), Range.InstanceSceneDataOffset, Range.InstanceSceneDataOffset + Range.NumInstanceSceneDataEntries);
#endif
			}
			else
			{
				InitializeIdData.Add(Range.InstanceSceneDataOffset, Range.NumInstanceSceneDataEntries, PrimitiveID);
#if LOG_INSTANCE_ALLOCATIONS
				InitializeRangesStr.Appendf(TEXT("[%6d, %6d), "), Range.InstanceSceneDataOffset, Range.InstanceSceneDataOffset + Range.NumInstanceSceneDataEntries);
#endif
			}
		}
	}

	if (OcclusionQueryRenderer)
	{
		// Invalidate last frame occlusion query slots
		OcclusionQueryRenderer->MarkInstancesVisible(GraphBuilder, InstanceRangesToClear);
	}

#if LOG_INSTANCE_ALLOCATIONS
	UE_LOGF(LogTemp, Warning, "AddClearInstancesPass(ClearIdData): \n%s",  *ClearRangesStr);
	UE_LOGF(LogTemp, Warning, "AddClearInstancesPass(InitializeIdData): \n%s",  *InitializeRangesStr);
#endif
	AddUpdatePrimitiveIdsPass(GraphBuilder, ClearIdData);
	AddUpdatePrimitiveIdsPass(GraphBuilder, InitializeIdData);
	InstanceRangesToClear.Empty();
}

FGPUSceneWriteDelegate::FGPUSceneWriteDelegate(FGPUSceneWriteDelegateImpl&& GPUSceneWriteDelegateImpl)
	: Delegate(MoveTemp(GPUSceneWriteDelegateImpl))
{
}

void FGPUSceneWriteDelegate::Execute(FRDGBuilder& GraphBuilder, const FGPUSceneWriteDelegateParams& Params) const
{
	check(Delegate.IsBound());
	Delegate.Execute(GraphBuilder, Params);
}

FGPUSceneWriteDelegateRef FGPUSceneWriteDelegate::CreateInternal(FMeshElementCollector* MeshElementCollector, FGPUSceneWriteDelegateImpl&& DelegateImpl)
{
	FGPUSceneWriteDelegateRef Result;
	Result.GPUSceneWriteDelegate = &MeshElementCollector->AllocateOneFrameResource<FGPUSceneWriteDelegate>(MoveTemp(DelegateImpl));
	return Result;
}

void FInstanceUploadInfo::SetPrimitiveData(FPrimitiveSceneProxy* InPrimitiveSceneProxy, const FPersistentPrimitiveIndex& PrimitiveIndex, uint32 SceneFrameNumber)
{
	const FPrimitiveSceneInfo* LocalPrimitiveSceneInfo = InPrimitiveSceneProxy->GetPrimitiveSceneInfo();
#if DO_CHECK
	PrimitiveSceneInfo = LocalPrimitiveSceneInfo;
#endif

	InstancePayloadDataOffset = LocalPrimitiveSceneInfo->GetInstancePayloadDataOffset();
	InstancePayloadDataStride = LocalPrimitiveSceneInfo->GetInstancePayloadDataStride();
	LastUpdateSceneFrameNumber = SceneFrameNumber;
	PrimitiveID = PrimitiveIndex.Index;
	// HACK: ignoring IsForceHidden for non-Nanite due to issues that cropped up with water rendering.
	// TODO: Remove the IsNaniteMesh() check
	bIsPrimitiveForceHidden = InPrimitiveSceneProxy->IsNaniteMesh() && InPrimitiveSceneProxy->IsForceHidden();

	const FInstanceSceneDataBuffers* InstanceSceneDataBuffersPtr = LocalPrimitiveSceneInfo->GetInstanceSceneDataBuffers();
	SetInstancePayloadData(InstanceSceneDataBuffersPtr);

#if DO_CHECK
	const bool bSetPrimitiveToWorld = true;
#else
	const bool bSetPrimitiveToWorld = !InstanceSceneDataBuffers;
#endif

	// If the primitive has isntancing data then this is already precomputed in the InstanceSceneDataBuffers and we don't need to do it again here, except for validation purposes
	// If the primitive does not have instancing data, the primitive to world is required
	// TODO: this validation should (also?) move elsewhere and validate that the transform on RT matches that on GT
	if (bSetPrimitiveToWorld)
	{
		const FMatrix LocalToWorld = InPrimitiveSceneProxy->GetLocalToWorld();
		const FDFVector3 AbsoluteOrigin(LocalToWorld.GetOrigin());
		PrimitiveToWorld = FDFMatrix::MakeToRelativeWorldMatrix(AbsoluteOrigin.High, LocalToWorld).M;
	}
}

void FInstanceUploadInfo::SetInstancePayloadData(const FInstanceSceneDataBuffers* InInstanceSceneDataBuffers)
{
	InstanceSceneDataBuffers = InInstanceSceneDataBuffers;
	if (InstanceSceneDataBuffers)
	{
		const FInstanceSceneDataBuffers::FReadView ReadView = InstanceSceneDataBuffers->GetReadView();
		NumInstances = InstanceSceneDataBuffers->GetNumInstances();
		InstanceFlags = UE::Renderer::Private::PackInstanceDataFlags(ReadView.Flags);
		InstanceLightShadowUVBias = ReadView.InstanceLightShadowUVBias;
		InstanceCustomData = ReadView.InstanceCustomData;
		InstanceRandomID = ReadView.InstanceRandomIDs;
		InstanceSkinningData = ReadView.InstanceSkinningData;
		InstanceHierarchyOffset = ReadView.InstanceHierarchyOffset;
		InstancePayloadExtension = ReadView.InstancePayloadExtension;
		InstanceLocalBounds = ReadView.InstanceLocalBounds;
#if WITH_EDITOR
		InstanceEditorData = ReadView.InstanceEditorData;
#endif
	}
	else
	{
		InstanceFlags = 0u;
		check(InstancePayloadDataOffset == INDEX_NONE && InstancePayloadDataStride == 0);

		// empty array signals that we should use the PrimitiveToWorld as the instance transform (and that it is therefore pre-transformed).
		PrimitiveInstances = TConstArrayView<FInstanceSceneData>();
		InstanceDynamicData = TConstArrayView<FInstanceDynamicData>();

#if 0
		// NOTE: We only need this if not using the InstanceSceneDataBuffers and if the old path were to support dynamic data
		{
			bool bHasPrecomputedVolumetricLightmap{};
			bool bOutputVelocity{};
			int32 SingleCaptureIndex{};

			FMatrix PreviousLocalToWorld;
			Scene.GetPrimitiveUniformShaderParameters_RenderThread(PrimitiveSceneInfo, bHasPrecomputedVolumetricLightmap, PreviousLocalToWorld, SingleCaptureIndex, bOutputVelocity);
			PrevPrimitiveToWorld = FDFMatrix::MakeClampedToRelativeWorldMatrix(AbsoluteOrigin.High, PreviousLocalToWorld).M;
		}
#endif
		InstanceLightShadowUVBias = TConstArrayView<FVector4f>();
		InstanceCustomData = TConstArrayView<float>();
		InstanceRandomID = TConstArrayView<float>();
		InstanceSkinningData = TConstArrayView<uint32>();
		InstanceHierarchyOffset = TConstArrayView<uint32>();
		InstancePayloadExtension = TConstArrayView<FVector4f>();
		NumInstances = 1;
#if WITH_EDITOR
		InstanceEditorData = TConstArrayView<uint32>();
#endif
	}

	InstancePayloadExtensionCount = 0;
	if (InstancePayloadExtension.Num() > 0)
	{
		InstancePayloadExtensionCount = InstancePayloadExtension.Num() / NumInstances;
	}

	InstanceCustomDataCount = 0;
	if (InstanceCustomData.Num() > 0)
	{
		InstanceCustomDataCount = InstanceCustomData.Num() / NumInstances;
	}

	// Only trigger upload if this primitive has instances
	check(NumInstances > 0);
}

uint32 FInstanceUploadInfo::CalculatePayloadStrideFloat4() const
{
	const bool bSupportsCompressedTransforms = FInstanceSceneShaderData::SupportsCompressedTransforms();
	int32 PayloadPosition = 0;

	if (InstanceFlags & (INSTANCE_SCENE_DATA_FLAG_HAS_HIERARCHY_OFFSET | INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS | INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA))
	{
		PayloadPosition += (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS) ? 2 : 1;
	}
	
	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA)
	{
		if (bSupportsCompressedTransforms)
		{
			PayloadPosition += 2;
		}
		else
		{
			PayloadPosition += 3;
		}
	}

#if WITH_EDITOR
	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_EDITOR_DATA)
	{
		check(InstanceEditorData.Num() == NumInstances);
		PayloadPosition += 1;
	}
#endif

	// TODO: Skip all this if static lighting is disabled
	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS)
	{
		check(InstanceLightShadowUVBias.Num() == NumInstances);
		PayloadPosition += 1;
	}

	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_PAYLOAD_EXTENSION)
	{
		check(InstancePayloadExtension.Num() / InstancePayloadExtensionCount == NumInstances);
		PayloadPosition += InstancePayloadExtensionCount;
	}

	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA && InstanceCustomDataCount > 0)
	{
		PayloadPosition += FMath::DivideAndRoundUp(InstanceCustomDataCount, 4);
	}
	return PayloadPosition;
}

void FInstanceUploadInfo::WritePayloadData(TArrayView<FVector4f> InstancePayloadData, int32 InstanceIndex)
{
	const bool bSupportsCompressedTransforms = FInstanceSceneShaderData::SupportsCompressedTransforms();
	int32 PayloadPosition = 0;

	if (InstanceFlags & (INSTANCE_SCENE_DATA_FLAG_HAS_HIERARCHY_OFFSET | INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS | INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA))
	{
		const uint32 InstanceHierarchyOffsetValue = (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_HIERARCHY_OFFSET) ? InstanceHierarchyOffset[InstanceIndex] : 0;
		InstancePayloadData[PayloadPosition].X = *(const float*)&InstanceHierarchyOffsetValue;

		const uint32 InstanceSkinningDataValue = (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA) ? InstanceSkinningData[InstanceIndex] : 0;
		InstancePayloadData[PayloadPosition].Y = *(const float*)&InstanceSkinningDataValue;

		if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS)
		{
			check(InstanceLocalBounds.Num() == NumInstances);
			const FRenderBounds& InstanceLocalBoundsValue = InstanceLocalBounds[InstanceIndex];
			const FVector3f BoundsOrigin = InstanceLocalBoundsValue.GetCenter();
			const FVector3f BoundsExtent = InstanceLocalBoundsValue.GetExtent();

			InstancePayloadData[PayloadPosition + 0].Z = *(const float*)&BoundsOrigin.X;
			InstancePayloadData[PayloadPosition + 0].W = *(const float*)&BoundsOrigin.Y;

			InstancePayloadData[PayloadPosition + 1].X = *(const float*)&BoundsOrigin.Z;
			InstancePayloadData[PayloadPosition + 1].Y = *(const float*)&BoundsExtent.X;
			InstancePayloadData[PayloadPosition + 1].Z = *(const float*)&BoundsExtent.Y;
			InstancePayloadData[PayloadPosition + 1].W = *(const float*)&BoundsExtent.Z;
		}

		PayloadPosition += (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS) ? 2 : 1;
	}

	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA)
	{
		const FRenderTransform PrevLocalToWorld = InstanceSceneDataBuffers != nullptr ?
			InstanceSceneDataBuffers->GetPrevInstanceToPrimitiveRelative(InstanceIndex) :
			InstanceDynamicData[InstanceIndex].ComputePrevLocalToWorld(PrevPrimitiveToWorld);

		if (bSupportsCompressedTransforms)
		{
			check(PayloadPosition + 1 < InstancePayloadData.Num()); // Sanity check
			FCompressedTransform CompressedPrevLocalToWorld(PrevLocalToWorld);
			InstancePayloadData[PayloadPosition + 0] = *(const FVector4f*)&CompressedPrevLocalToWorld.Rotation[0];
			InstancePayloadData[PayloadPosition + 1] = *(const FVector3f*)&CompressedPrevLocalToWorld.Translation;
			PayloadPosition += 2;
		}
		else
		{
			// Note: writes 3x float4s
			check(PayloadPosition + 2 < InstancePayloadData.Num()); // Sanity check
			PrevLocalToWorld.To3x4MatrixTranspose((float*)&InstancePayloadData[PayloadPosition]);
			PayloadPosition += 3;
		}
	}

#if WITH_EDITOR
	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_EDITOR_DATA)
	{
		check(InstanceEditorData.Num() == NumInstances);
		InstancePayloadData[PayloadPosition].X = *(const float*)&InstanceEditorData[InstanceIndex];
		InstancePayloadData[PayloadPosition].Y = 0.0f; // Unused
		InstancePayloadData[PayloadPosition].Z = 0.0f; // Unused
		InstancePayloadData[PayloadPosition].W = 0.0f; // Unused
		PayloadPosition += 1;
	}
#endif

	// TODO: Skip all this if static lighting is disabled
	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS)
	{
		check(InstanceLightShadowUVBias.Num() == NumInstances);
		InstancePayloadData[PayloadPosition] = InstanceLightShadowUVBias[InstanceIndex];
		PayloadPosition += 1;
	}

	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_PAYLOAD_EXTENSION)
	{
		check(InstancePayloadExtension.Num() / InstancePayloadExtensionCount == NumInstances);
		const int32 SrcOffset = InstanceIndex * InstancePayloadExtensionCount;
		for (int32 Idx = 0; Idx < InstancePayloadExtensionCount; ++Idx)
		{
			InstancePayloadData[PayloadPosition++] = InstancePayloadExtension[SrcOffset + Idx];
		}
	}

	if (InstanceFlags & INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA && InstanceCustomDataCount > 0)
	{
		check((PayloadPosition + FMath::DivideAndRoundUp(InstanceCustomDataCount, 4)) <= InstancePayloadData.Num());
		const float* CustomDataGetPtr = &InstanceCustomData[(InstanceIndex * InstanceCustomDataCount)];
		float* CustomDataPutPtr = (float*)&InstancePayloadData[PayloadPosition];
		for (uint32 FloatIndex = 0; FloatIndex < uint32(InstanceCustomDataCount); ++FloatIndex)
		{
			CustomDataPutPtr[FloatIndex] = CustomDataGetPtr[FloatIndex];
		}
	}
}
