// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"
#include "RenderTransform.h"
#include "RendererInterface.h"
#include "PrimitiveUniformShaderParameters.h"
#include "PrimitiveSceneInfo.h"
#include "SpanAllocator.h"
#include "GrowOnlySpanAllocator.h"
#include "InstanceCulling/InstanceCullingLoadBalancer.h"
#include "MeshBatch.h"
#include "LightSceneData.h"
#include "SceneUniformBuffer.h"
#include "UnifiedBuffer.h"
#include "ScenePrimitiveUpdates.h"
#include "GPUSceneWriter.h"
#include "RendererPrivateUtils.h"

class FRDGExternalAccessQueue;
class FRHICommandList;
class FScene;
class FViewInfo;
class FLightSceneInfoCompact;
class FGPUScene;
class FGPUSceneDynamicContext;
class FViewUniformShaderParameters;
class FInstanceCullingOcclusionQueryRenderer;
class FScenePreUpdateChangeSet;
class FScenePostUpdateChangeSet;
class FPrimitiveSceneProxy;
struct FLightSceneChangeSet;
class FGPUSceneWriterParameters;
class FSceneExtensionsUpdaters;

namespace UE::Renderer::Private
{
	class IShadowInvalidatingInstances;
}

DECLARE_GPU_STAT_NAMED_EXTERN(GPUSceneUpdate, TEXT("GPUSceneUpdate"));

#if !ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND
// Note: this is declared in GPUSceneWriter.h (a public header) until such a time as the UB path works.
// Parameter sub struct that is common to all parameter use-cases
BEGIN_SHADER_PARAMETER_STRUCT(FGPUSceneCommonParameters, RENDERER_API)
	SHADER_PARAMETER(uint32, GPUSceneInstanceDataSOAStride)
	SHADER_PARAMETER(uint32, GPUSceneFrameNumber)
	SHADER_PARAMETER(int32, GPUSceneMaxAllocatedInstanceId)
	SHADER_PARAMETER(int32, GPUSceneMaxPersistentPrimitiveIndex)
END_SHADER_PARAMETER_STRUCT()

#endif

BEGIN_SHADER_PARAMETER_STRUCT(FGPUSceneResourceParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneCommonParameters, CommonParameters)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, GPUSceneInstanceSceneData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, GPUSceneInstancePayloadData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, GPUScenePrimitiveSceneData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, GPUSceneLightmapData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, GPUSceneLightData)
END_SHADER_PARAMETER_STRUCT()

//Non-RDG parameters, untracked, use with care (or preferably, don't)
BEGIN_SHADER_PARAMETER_STRUCT(FGPUSceneResourceParametersRHI, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneCommonParameters, CommonParameters)
	SHADER_PARAMETER_SRV(StructuredBuffer<float4>, GPUSceneInstanceSceneData)
	SHADER_PARAMETER_SRV(StructuredBuffer<float4>, GPUSceneInstancePayloadData)
	SHADER_PARAMETER_SRV(StructuredBuffer<float4>, GPUScenePrimitiveSceneData)
	SHADER_PARAMETER_SRV(StructuredBuffer<float4>, GPUSceneLightmapData)
	SHADER_PARAMETER_SRV(ByteAddressBuffer, GPUSceneLightData)
END_SHADER_PARAMETER_STRUCT()

DECLARE_SCENE_UB_STRUCT(FGPUSceneResourceParameters, GPUScene, RENDERER_API)

/**
 * Used to manage dynamic primitives for a given view, during InitViews the data is collected and then can be committed to the GPU-Scene. 
 * Once committed the range of indices are valid and can be used to calculate the PrimitiveIds.
 */
class FGPUScenePrimitiveCollector
{
public:
	FGPUScenePrimitiveCollector(FGPUSceneDynamicContext* InGPUSceneDynamicContext = nullptr) :
		GPUSceneDynamicContext(InGPUSceneDynamicContext)
	{};

	/**
	 * Add data for a primitive with a number of instances.
	 * May be called outside (before) a FGPUScene::Begin/EndRender block.
	 * Note: needs to be virtual to prevent a linker error
	 */
	virtual void Add(
		const FMeshBatchDynamicPrimitiveData* MeshBatchData,
		const FPrimitiveUniformShaderParameters& PrimitiveShaderParams,
		uint32 NumInstances,
		uint32& OutPrimitiveIndex,
		uint32& OutInstanceSceneDataOffset);

	/**
	 * Allocates the range in GPUScene and queues the data for upload. 
	 * After this is called no more calls to Add are allowed.
	 * Only allowed inside a FGPUScene::Begin/EndRender block.
	 */
	RENDERER_API void Commit();

	/**
	 * Get the range of Primitive IDs in GPU-Scene for this batch of dynamic primitives, only valid to call after commit.
	 */
	FORCEINLINE const TRange<int32>& GetPrimitiveIdRange() const
	{
		check(bCommitted || UploadData == nullptr);
		return PrimitiveIdRange;
	}

	FORCEINLINE int32 GetInstanceSceneDataOffset() const
	{
		check(bCommitted || UploadData == nullptr);

		return UploadData != nullptr ? UploadData->InstanceSceneDataOffset : 0;
	}

	FORCEINLINE int32 GetInstancePayloadDataOffset() const
	{
		check(bCommitted || UploadData == nullptr);

		return UploadData != nullptr ? UploadData->InstancePayloadDataOffset : 0;
	}

	int32 Num() const {	return UploadData != nullptr ? UploadData->PrimitiveData.Num() : 0; }
	int32 NumInstances() const { return UploadData != nullptr ? UploadData->TotalInstanceCount : 0; }
	int32 NumPayloadDataSlots() const { return UploadData != nullptr ? UploadData->InstancePayloadDataFloat4Count : 0; }
	const FPrimitiveUniformShaderParameters* GetPrimitiveShaderParameters(int32 PrimitiveId) const;

#if DO_CHECK
	/**
	 * Determines if the specified primitive has been sufficiently processed and its data can be read
	 */
	void CheckPrimitiveProcessed(uint32 PrimitiveIndex, const FGPUScene& GPUScene) const;
#endif // DO_CHECK

private:

	friend class FGPUScene;
	friend class FGPUSceneDynamicContext;
	friend struct FUploadDataSourceAdapterDynamicPrimitives;

	struct FPrimitiveData
	{
		FMeshBatchDynamicPrimitiveData SourceData;
		const FPrimitiveUniformShaderParameters* ShaderParams = nullptr;
		uint32 NumInstances = 0;
		uint32 LocalInstanceSceneDataOffset = INDEX_NONE;
		uint32 LocalPayloadDataOffset = INDEX_NONE;
	};

	struct FUploadData
	{
		TArray<FPrimitiveData, TInlineAllocator<8>> PrimitiveData;
		TArray<uint32> GPUWritePrimitives;

		uint32 InstanceSceneDataOffset = INDEX_NONE;
		uint32 TotalInstanceCount = 0;
		uint32 InstancePayloadDataOffset = INDEX_NONE;
		uint32 InstancePayloadDataFloat4Count = 0;
		bool bIsUploaded = false;
	};

	FUploadData* AllocateUploadData();

	/**
	 * Range in GPU scene allocated to the dynamic primitives.
	 */
	TRange<int32> PrimitiveIdRange = TRange<int32>::Empty();
	FUploadData* UploadData = nullptr; // Owned by FGPUSceneDynamicContext
	bool bCommitted = false;
	FGPUSceneDynamicContext* GPUSceneDynamicContext = nullptr;
};

// TODO: move to own header
class FInstanceProcessingGPULoadBalancer;

/**
 * Contains and controls the lifetime of any dynamic primitive data collected for the scene rendering.
 * Typically shares life-time with the SceneRenderer. 
 */
class FGPUSceneDynamicContext
{
public:
	FGPUSceneDynamicContext(FGPUScene& InGPUScene) : GPUScene(InGPUScene) {}
	~FGPUSceneDynamicContext();

	void Release();

private:
	friend class FGPUScene;
	friend class FGPUScenePrimitiveCollector;

	FGPUScenePrimitiveCollector::FUploadData* AllocateDynamicPrimitiveData();
	UE::FMutex DymamicPrimitiveUploadDataMutex;
	TArray<FGPUScenePrimitiveCollector::FUploadData*, TInlineAllocator<128, SceneRenderingAllocator> > DymamicPrimitiveUploadData;
	FGPUScene& GPUScene;
};

struct FGPUSceneInstanceRange
{
	FGPUSceneInstanceRange(FPersistentPrimitiveIndex InPrimitive, uint32 InInstanceSceneDataOffset, uint32 InNumInstanceSceneDataEntries)
		: Primitive(InPrimitive)
		, InstanceSceneDataOffset(InInstanceSceneDataOffset)
		, NumInstanceSceneDataEntries(InNumInstanceSceneDataEntries)
	{}

	FPersistentPrimitiveIndex Primitive;
	uint32 InstanceSceneDataOffset;
	uint32 NumInstanceSceneDataEntries;
};

class FGPUScene
{
public:
	FGPUScene(FScene &InScene);
	~FGPUScene();

	using FUpdateFromComputeCommands = FScenePrimitiveUpdates::TPayloadRangeView<FUpdateInstanceFromComputeCommand>;

	void SetEnabled(EShaderPlatform InShaderPlatform);
	bool IsEnabled() const { return bIsEnabled; }
	/**
	 * Call at start of rendering (but after scene primitives are updated) to let GPU-Scene record scene primitive count 
	 * and prepare for dynamic primitive allocations.
	 * Scene may be NULL which means there are zero scene primitives (but there may be dynamic ones added later).
	 */
	void BeginRender(FRDGBuilder& GraphBuilder, FGPUSceneDynamicContext &GPUSceneDynamicContext);
	inline bool IsRendering() const { return bInBeginEndBlock; }
	void EndRender();

	EShaderPlatform GetShaderPlatform() const { return ShaderPlatform; }

	/**
	 * Allocates a range of space in the instance scene data buffer for the required number of instances, 
	 * returns the offset to the first instance or INDEX_NONE if either the allocation failed or NumInstanceSceneDataEntries was zero.
	 * Marks the instances as requiring update (actual update is handled later).
	 */
	int32 AllocateInstanceSceneDataSlots(FPersistentPrimitiveIndex PersistentPrimitiveIndex, int32 NumInstanceSceneDataEntries);
	
	/**
	 * Free the instance data slots for reuse.
	 */
	void FreeInstanceSceneDataSlots(int32 InstanceSceneDataOffset, int32 NumInstanceSceneDataEntries);

	int32 AllocateInstancePayloadDataSlots(int32 NumInstancePayloadFloat4Entries);
	void FreeInstancePayloadDataSlots(int32 InstancePayloadDataOffset, int32 NumInstancePayloadFloat4Entries);

	/**
	 * Upload primitives from View.DynamicPrimitiveCollector.
	 */
	void UploadDynamicPrimitiveShaderDataForView(FRDGBuilder& GraphBuilder, FViewInfo& View, bool bRayTracing = false, UE::Renderer::Private::IShadowInvalidatingInstances *ShadowInvalidatingInstances = nullptr);

	/**
	 * Modifies the GPUScene specific scene UB parameters to the current versions. Returns true if any of the parameters changed.
	 */
	bool FillSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUB) const;

	/**
	 * Pull all pending updates from Scene and upload primitive & instance data.
	 */
	void Update(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue, const FUpdateFromComputeCommands& UpdatesFromCompute, const UE::Tasks::FTask& UpdateTaskPrerequisites, FSceneExtensionsUpdaters& SceneExtensionsUpdaters);

	/**
	 * Queue the given primitive for upload to GPU at next call to Update.
	 * May be called multiple times, dirty-flags are cumulative.
	 */
	void RENDERER_API AddPrimitiveToUpdate(FPersistentPrimitiveIndex PersistentPrimitiveIndex, EPrimitiveDirtyState DirtyState = EPrimitiveDirtyState::ChangedAll);

	FORCEINLINE EPrimitiveDirtyState GetPrimitiveDirtyState(FPersistentPrimitiveIndex PersistentPrimitiveIndex) const 
	{ 
		if (!PrimitiveDirtyState.IsValidIndex(PersistentPrimitiveIndex.Index))
		{
			return EPrimitiveDirtyState::None;
		}
		return PrimitiveDirtyState[PersistentPrimitiveIndex.Index]; 
	}

	uint32 GetSceneFrameNumber() const { return SceneFrameNumber; }

	int32 GetNumInstances() const { return InstanceSceneDataAllocator.GetMaxSize(); }
	int32 GetNumPrimitives() const { return DynamicPrimitivesOffset; }
	int32 GetMaxLightId() const { return NumGPULights; }
	int32 GetNumLightmapDataItems() const { return LightmapDataAllocator.GetMaxSize(); }
	FGPUSceneDynamicContext* GetCurrentDynamicContext() const { return CurrentDynamicContext; }
	/**
	 * Returns the highest instance ID that is represented in the GPU scene (which may be lower than the host allocated IDs due to various limits)
	 * Never larger than MAX_INSTANCE_ID, see Engine\Shaders\Shared\SceneDefinitions.h
	 */
	uint32 GetInstanceIdUpperBoundGPU() const;

	const FSpanAllocator& GetInstanceSceneDataAllocator() const { return InstanceSceneDataAllocator; }

	/**
	 * Return the GPU scene resource
	 */
	RENDERER_API FGPUSceneResourceParameters GetShaderParameters(FRDGBuilder& GraphBuilder) const;

	/**
	 * Draw GPU-Scene debug info, such as bounding boxes. Call once per view at some point in the frame after GPU scene has been updated fully.
	 * What is drawn is controlled by the CVar: r.GPUScene.DebugMode. Enabling this cvar causes ShaderDraw to be being active (if supported). 
	 */
	void DebugRender(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer, FViewInfo& View);

	/**
	 * Manually trigger an allocator consolidate (will otherwise be done when an item is allocated).
	 */
	void ConsolidateInstanceDataAllocations();

	/**
	 * Executes GPUScene writes that were deferred until a later point in scene rendering
	 **/
	bool ExecuteDeferredGPUWritePass(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views, EGPUSceneGPUWritePass Pass);

	/** Returns whether or not a GPU Write is pending for the specified primitive */
	bool HasPendingGPUWrite(uint32 PrimitiveId) const;

	/**
	 * Called by FScene::UpdateAllPrimimitiveSceneInfos before the scene is udated.
	 */
	void OnPreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ScenePreUpdateData);

	/**
	 * Called by FScene::UpdateAllPrimimitiveSceneInfos after the scene is udated.
	 */
	void OnPostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ScenePostUpdateData);

	/**
	 */
	void OnPostLightSceneInfoUpdate(FRDGBuilder& OnPostLightSceneInfoUpdate, const FLightSceneChangeSet& LightsPostUpdateData);

	bool bUpdateAllPrimitives;

	/** GPU mirror of Primitives */
	TRefCountPtr<FRDGPooledBuffer> PrimitiveBuffer;
	FRDGAsyncScatterUploadBuffer   PrimitiveUploadBuffer;

	/** GPU primitive instance list */
	TRefCountPtr<FRDGPooledBuffer> InstanceSceneDataBuffer;
	FRDGAsyncScatterUploadBuffer   InstanceSceneUploadBuffer;

	FSpanAllocator                 InstancePayloadDataAllocator;
	TRefCountPtr<FRDGPooledBuffer> InstancePayloadDataBuffer;
	FRDGAsyncScatterUploadBuffer   InstancePayloadUploadBuffer;

	/** GPU light map data */
	FSpanAllocator                 LightmapDataAllocator;
	TRefCountPtr<FRDGPooledBuffer> LightmapDataBuffer;
	FRDGAsyncScatterUploadBuffer   LightmapUploadBuffer;

	using FInstanceRange = FGPUSceneInstanceRange;
	using FInstanceGPULoadBalancer = TInstanceCullingLoadBalancer<SceneRenderingAllocator>;

	inline const FScene &GetScene() const { return Scene; }

	SIZE_T GetAllocatedSize() const;

	FGPUSceneResourceParametersRHI GetShaderParametersRHI() const;

private:
	/**
	 * Fills in the FGPUSceneWriterParameters to use for read/write access to the GPU Scene.
	 */
	TRDGUniformBufferRef<FGPUSceneWriterUniformParameters> GetWriteParameters(FRDGBuilder& GraphBuilder, FGPUSceneWriterParameters* LegacyParameters = nullptr, bool bSkipBarriers = true);

	// set to true if the requested (cvar) tile size is >0, otherwise the tile size is re-derived from the instance count
	bool bUseTiledInstanceDataLayout = true;

	// log2(TileSize) in number of instances, thus the tile-size in memory is #float4s per instance * tilesize
	int32 InstanceDataTileSizeLog2 = -1;

	uint32 GetInstanceDataTileSizeLog2() const { return InstanceDataTileSizeLog2; }
	uint32 GetInstanceDataTileSize() const { return 1u << InstanceDataTileSizeLog2; }
	uint32 GetInstanceDataTileSizeMask() const;
	uint32 GetInstanceDataTileStride() const;

	static constexpr int32 InitialBufferSize = 256;

	TPersistentByteAddressBuffer<FLightSceneData> LightDataBuffer;
	int32 NumGPULights = 0;
	UE::Tasks::FTask LightDataSetupTask;

#if DO_CHECK
	TArray<FLightSceneData> LightDataArray;
#endif

	FGPUSceneCommonParameters CommonParameters;

	FGPUSceneCommonParameters GetCommonParameters() const;


	// Buffers used by GPU-Scene, since they can be resized during updates AND the render passes must retain the 
	// right copy (this is chiefly because the init of shadow views after pre-pass means we need to be able to set 
	// up GPU-Scene before pre-pass, but then may discover new primitives etc. As there is no way to know how many
	// dynamic primitives will turn up after Pre-pass, we can't guarantee a resize won't happen).
	struct FRegisteredBuffers
	{
		bool IsValid() const { return PrimitiveBuffer != nullptr; }

		FRDGBuffer* PrimitiveBuffer = nullptr;
		FRDGBuffer* InstanceSceneDataBuffer = nullptr;
		FRDGBuffer* InstancePayloadDataBuffer = nullptr;
		FRDGBuffer* LightmapDataBuffer = nullptr;
		FRDGBuffer* LightDataBuffer = nullptr;
	};



	FScene &Scene;
	FSpanAllocator		           InstanceSceneDataAllocator;

	FORCEINLINE void ResizeDirtyState(int32 NewSizeIn)
	{
		if (IsEnabled() && NewSizeIn > PrimitiveDirtyState.Num())
		{
			const int32 NewSize = Align(NewSizeIn, 64);
			static_assert(static_cast<uint32>(EPrimitiveDirtyState::None) == 0U, "Using AddZeroed to ensure efficent add, requires None == 0");
			PrimitiveDirtyState.AddZeroed(NewSize - PrimitiveDirtyState.Num());
		}
	}

	/** Indices of primitives that need to be updated in GPU Scene */
	TArray<FPersistentPrimitiveIndex> PrimitivesToUpdate;

	FRegisteredBuffers CachedRegisteredBuffers;

	TArray<EPrimitiveDirtyState> PrimitiveDirtyState;

	TArray<FInstanceRange> InstanceRangesToClear;

	struct FDeferredGPUWrite
	{
		FGPUSceneWriteDelegateRef DataWriterGPU;
		int32 ViewId = INDEX_NONE;
		uint32 PrimitiveId = INDEX_NONE;
		uint32 InstanceSceneDataOffset = INDEX_NONE;
		uint32 NumCustomDataFloats = INDEX_NONE;
		uint32 PackedInstanceSceneDataFlags = 0u;
	};

	static constexpr uint32 NumDeferredGPUWritePasses = uint32(EGPUSceneGPUWritePass::Num);
	TArray<FDeferredGPUWrite> DeferredGPUWritePassDelegates[NumDeferredGPUWritePasses];
	EGPUSceneGPUWritePass LastDeferredGPUWritePass = EGPUSceneGPUWritePass::None;

	TRange<int32> CommitPrimitiveCollector(FGPUScenePrimitiveCollector& PrimitiveCollector);


	friend class FGPUScenePrimitiveCollector;

	/** 
	 * Stores a copy of the Scene.GetFrameNumber() when updated. Used to track which primitives/instances are updated.
	 * When using GPU-Scene for rendering it should ought to be the same as that stored in the Scene (otherwise they are not in sync).
	 */
	uint32 SceneFrameNumber = 0xFFFFFFFF;

	int32 DynamicPrimitivesOffset = 0;
	
	int32 PrevDynamicPrimitivesMaxSize = 0;
	int32 PrevDynamicPrimitivePayloadMaxSize = 0;
	int32 PrevDynamicPrimitiveInstanceMaxSize = 0;

	bool bIsEnabled = false;
	bool bInBeginEndBlock = false;
	FGPUSceneDynamicContext* CurrentDynamicContext = nullptr;

	EShaderPlatform ShaderPlatform;

	FRegisteredBuffers UpdateBufferAllocations(FRDGBuilder& GraphBuilder);

	/**
	 * Register the external buffers with the graphbuilder.
	 */
	FRegisteredBuffers RegisterBuffers(FRDGBuilder& GraphBuilder) const;

	/**
	 * Generalized upload that uses an adapter to abstract the data souce. Enables uploading scene primitives & dynamic primitives using a single path.
	 * @parameter Scene may be null, as it is only needed for the Nanite material table update (which is coupled to the Scene at the moment).
	 */
	template<typename FUploadDataSourceAdapter>
	void UploadGeneral(FRDGBuilder& GraphBuilder, const FRegisteredBuffers& BufferState, FRDGExternalAccessQueue* ExternalAccessQueue, const FUploadDataSourceAdapter& UploadDataSourceAdapter, const UE::Tasks::FTask& PrerequisiteTask);

	/**
	 * Upload scene light data to gpu
	 */
	void UpdateGPULights(FRDGBuilder& GraphBuilder, const UE::Tasks::FTask& PrerequisiteTask, const FLightSceneChangeSet& LightsPostUpdateData);

	static void InitLightData(const FLightSceneInfoCompact& LightInfoCompact, bool bAllowStaticLighting, uint32 LightShaderParameterFlags, FLightSceneData& DataOut);

	void UploadDynamicPrimitiveShaderDataForViewInternal(FRDGBuilder& GraphBuilder, FViewInfo& View, bool bRayTracing, UE::Renderer::Private::IShadowInvalidatingInstances *ShadowInvalidatingInstances);

	void UpdateInternal(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue, const UE::Tasks::FTask& UpdateTaskPrerequisites, const FUpdateFromComputeCommands& UpdatesFromCompute, FSceneExtensionsUpdaters& SceneExtensionsUpdaters);

	void AddUpdatePrimitiveIdsPass(FRDGBuilder& GraphBuilder, FInstanceGPULoadBalancer& IdOnlyUpdateItems);

	void AddClearInstancesPass(FRDGBuilder& GraphBuilder, FInstanceCullingOcclusionQueryRenderer* OcclusionQueryRenderer = nullptr);


#if !UE_BUILD_SHIPPING
	FDelegateHandle ScreenMessageDelegate;
	bool bLoggedInstanceOverflow = false;
	uint32 MaxInstancesDuringPrevUpdate = 0;
	bool bLoggedLightOverflow = false;
	int32 MaxLightsDuringPrevUpdate = 0;
	bool bLoggedPayloadBufferOverflow = false;
#endif // UE_BUILD_SHIPPING
	uint64 InstancePayloadDataSizeReserveUnclamped = 0;
	// Used to track the GetCachedScalabilityCVars().ViewDistanceScale such that we can trigger a full upload if it changes, this is required as the value is baked into some values such as in FPrimitiveUniformShaderParametersBuilder::InstanceWorldPositionOffsetDisableDistance
	float TrackedViewDistanceScle = -1.0f;
};

class FGPUSceneScopeBeginEndHelper
{
public:
	FGPUSceneScopeBeginEndHelper(FRDGBuilder& GraphBuilder, FGPUScene& InGPUScene, FGPUSceneDynamicContext &GPUSceneDynamicContext) :
		GPUScene(InGPUScene)
	{
		GPUScene.BeginRender(GraphBuilder, GPUSceneDynamicContext);
	}

	~FGPUSceneScopeBeginEndHelper()
	{
		GPUScene.EndRender();
	}

private:
	FGPUSceneScopeBeginEndHelper() = delete;
	FGPUSceneScopeBeginEndHelper(const FGPUSceneScopeBeginEndHelper&) = delete;
	FGPUScene& GPUScene;
};

struct FBatchedPrimitiveShaderData
{
	static const uint32 DataStrideInFloat4s = BATCHED_PRIMITIVE_DATA_STRIDE_FLOAT4;

	TStaticArray<FVector4f, DataStrideInFloat4s> Data;

	FBatchedPrimitiveShaderData()
		: Data(InPlace, NoInit)
	{
		Setup(GetIdentityPrimitiveParameters());
	}

	explicit FBatchedPrimitiveShaderData(const FPrimitiveUniformShaderParameters& PrimitiveUniformShaderParameters)
		: Data(InPlace, NoInit)
	{
		Setup(PrimitiveUniformShaderParameters);
	}

	explicit FBatchedPrimitiveShaderData(const FPrimitiveSceneProxy* Proxy);
	
	static void Emplace(FBatchedPrimitiveShaderData* Dest, const FPrimitiveUniformShaderParameters& ShaderParams);
	static void Emplace(FBatchedPrimitiveShaderData* Dest, const FPrimitiveSceneProxy* Proxy);

private:
	void Setup(const FPrimitiveUniformShaderParameters& PrimitiveUniformShaderParameters);
};


/**
 * Info required by the uploader to update the instances that belong to a primitive.
 */
struct FInstanceUploadInfo
{
	const FInstanceSceneDataBuffers* InstanceSceneDataBuffers = nullptr;
	TConstArrayView<FInstanceSceneData> PrimitiveInstances;
	int32 InstancePayloadDataOffset = INDEX_NONE;
	int32 InstancePayloadDataStride = 0;
	int32 InstancePayloadExtensionCount = 0;
	int32 InstanceCustomDataCount = 0;

	// Optional per-instance data views
	TConstArrayView<FInstanceDynamicData> InstanceDynamicData;
	TConstArrayView<uint32> InstanceSkinningData;
	TConstArrayView<FVector4f> InstanceLightShadowUVBias;
	TConstArrayView<float> InstanceCustomData;
	TConstArrayView<float> InstanceRandomID;
	TConstArrayView<uint32> InstanceHierarchyOffset;
	TConstArrayView<FRenderBounds> InstanceLocalBounds;
	TConstArrayView<FVector4f> InstancePayloadExtension;
#if WITH_EDITOR
	TConstArrayView<uint32> InstanceEditorData;
#endif

	uint32 InstanceFlags = 0x0;

	FRenderTransform PrimitiveToWorld;
	FRenderTransform PrevPrimitiveToWorld;
	int32 PrimitiveID = INDEX_NONE;
	uint32 LastUpdateSceneFrameNumber = ~uint32(0);
	int32 NumInstances = 0;
	bool bIsPrimitiveForceHidden = false;
#if DO_CHECK
	const FPrimitiveSceneInfo* PrimitiveSceneInfo = nullptr;
#endif

	void SetPrimitiveData(FPrimitiveSceneProxy* InPrimitiveSceneProxy, const FPersistentPrimitiveIndex& PrimitiveIndex, uint32 SceneFrameNumber);
	void SetInstancePayloadData(const FInstanceSceneDataBuffers* InInstanceSceneDataBuffers);
	void WritePayloadData(TArrayView<FVector4f> InstancePayloadData, int32 InstanceIndex);
	uint32 CalculatePayloadStrideFloat4() const;
};
