// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPUWorkGroupLoadBalancer.h"


void FGPUWorkGroupLoadBalancer::GetParametersAsync(FRDGBuilder& GraphBuilder, FShaderParameters& OutShaderParameters)
{
	FRDGBufferRef WorkGroupInfosRDG = CreateStructuredBuffer(GraphBuilder, TEXT("GPUWorkGroupLoadBalancer.WorkGroupInfos"), [this]() -> auto& 
		{ 
			return WorkGroupInfos; 
		});
	FRDGBufferRef ItemsRDG = CreateStructuredBuffer(GraphBuilder, TEXT("GPUWorkGroupLoadBalancer.Items"), [this]() -> auto& 
		{ 
			return Items; 
		});
	OutShaderParameters.WorkGroupInfoBuffer = GraphBuilder.CreateSRV(WorkGroupInfosRDG);
	OutShaderParameters.ItemBuffer = GraphBuilder.CreateSRV(ItemsRDG);
	OutShaderParameters.NumWorkGroupInfos = ~0u;
	OutShaderParameters.NumItems = ~0u;
}

void FGPUWorkGroupLoadBalancer::FinalizeParametersAsync(FShaderParameters& OutShaderParameters)
{
	check(CurrentWorkGroupNumItems == 0);
	OutShaderParameters.NumWorkGroupInfos = WorkGroupInfos.Num();
	OutShaderParameters.NumItems = Items.Num();
}

FIntVector FGPUWorkGroupLoadBalancer::GetWrappedCsGroupCount() const
{
	return FComputeShaderUtils::GetGroupCountWrapped(WorkGroupInfos.Num());
}

void FGPUWorkGroupLoadBalancer::SetShaderDefines(FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("WGLB_ENABLE"), 1);
	OutEnvironment.SetDefine(TEXT("WGLB_NUM_THREADS_PER_GROUP"), ThreadGroupSize);
	OutEnvironment.SetDefine(TEXT("WGLB_NUM_ITEM_BITS"), NumItemBits);
	OutEnvironment.SetDefine(TEXT("WGLB_NUM_ITEM_MASK"), NumItemMask);
	OutEnvironment.SetDefine(TEXT("WGLB_PREFIX_BITS"), PrefixBits);
	OutEnvironment.SetDefine(TEXT("WGLB_PREFIX_BIT_MASK"), PrefixBitMask);
}

