// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphDefinitions.h"
#include "ShaderParameterMacros.h"
#include "RendererInterface.h"
#include "RenderGraphUtils.h"

struct FShaderCompilerEnvironment;

/*
 */
class FGPUWorkGroupLoadBalancer
{
public:
	static constexpr uint32 ThreadGroupSizeLog2 = 6U;
	static constexpr uint32 ThreadGroupSize = 1u << ThreadGroupSizeLog2;
	static constexpr uint32 PrefixBits = ThreadGroupSizeLog2;
	static constexpr uint32 PrefixBitMask = (1U << PrefixBits) - 1U;
	static constexpr uint32 NumItemBits = PrefixBits;
	static constexpr uint32 NumItemMask = (1U << NumItemBits) - 1U;
	static constexpr uint32 PayLoadBits = 32u - (1 + NumItemBits + PrefixBits);

	/**
	 * Describes the work for a workgroup (64 threads), a workgroup may use anything in the range [1,64] items
	 */
	struct FWorkGroupInfo
	{
		FUint32Point WorkGroupWorkBoundary;
		uint32 FirstItem;
		uint32 NumItems; // Note: NumItems = countbits(WorkGroupWorkBoundary)
		uint32 CarryOverStartOffset;
		uint32 Payload; // aribitrary 32-bit payload for each workgroup
	};

	FWorkGroupInfo PackWorkGroupInfo(uint32 FirstItem, uint32 NumItems, uint32 Payload, uint32 CarryOverStartOffset, uint64 WorkGroupWorkBoundary)
	{
		checkSlow(NumItems > 0);
		checkSlow(NumItems - 1 < (1U << NumItemBits));
		checkSlow(FirstItem < (1U << (32U - NumItemBits)));

		return FWorkGroupInfo{ { uint32(WorkGroupWorkBoundary), uint32(WorkGroupWorkBoundary >> 32u) }, FirstItem, NumItems, CarryOverStartOffset, Payload };
	}


	/**
	 * An item represents a subset of children that fits inside a workgroup. This allows packing the data tightly.
	 */
	struct FPackedItem
	{
		uint32 Packed;
	};

	FPackedItem PackItem(bool bHasCarryOver, uint32 NumChildren, uint32 Payload, uint32 BatchPrefixSum)
	{
		checkSlow(NumChildren > 0);
		checkSlow(NumChildren - 1 < (1U << NumItemBits));
		//checkSlow(ParentWorkItemOffset < (1U << (32U - NumItemBits)));
		checkSlow(BatchPrefixSum < (1U << PrefixBits));
		// arbitrary per-item payload encoded in as many bits as we happen to have left over
		checkSlow(Payload < (1U << PayLoadBits));
		uint32 Packed = (bHasCarryOver ? 1u : 0u) 
			| ((NumChildren - 1u) << 1u) 
			| (BatchPrefixSum << (1u + NumItemBits))
			| (Payload << (1u + NumItemBits + PrefixBits));
		return FPackedItem { Packed	};
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FWorkGroupInfo >, WorkGroupInfoBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedItem >, ItemBuffer)
		SHADER_PARAMETER(uint32, NumWorkGroupInfos)
		SHADER_PARAMETER(uint32, NumItems)
	END_SHADER_PARAMETER_STRUCT()

	void ReserveStorage(int32 NumBatches, int32 NumItems, int32 NumWork)
	{
		WorkGroupInfos.Empty(NumBatches);
		Items.Empty(NumItems);
	}

	/**
	 * Add parent work item and child count to be processed.
	 */
	template <typename PayloadGeneratorType>
	void Add(PayloadGeneratorType& PayloadGenerator, uint32 NumChildren)
	{
		uint32 NumAdded = 0u;
		while (NumAdded < NumChildren)
		{
			uint32 MaxChildrenThisBatch = ThreadGroupSize - CurrentWorkGroupPrefixSum;

			if (MaxChildrenThisBatch > 0)
			{
				const uint32 NumChildrenThisItem = FMath::Min(MaxChildrenThisBatch, NumChildren - NumAdded);

				Items.Add(PackItem(NumAdded != 0u, NumChildrenThisItem, PayloadGenerator.GetItemPayload(), CurrentWorkGroupPrefixSum));
				CurrentWorkGroupNumItems += 1U;
				NumAdded += NumChildrenThisItem;
				CurrentWorkGroupPrefixSum += NumChildrenThisItem;
				CurrentWorkGroupWorkBoundary |= 1ull << (CurrentWorkGroupPrefixSum - 1u);
			}

			// Flush batch if it is not possible to add any more items (for one of the reasons)
			if (MaxChildrenThisBatch <= 0U || CurrentWorkGroupPrefixSum >= ThreadGroupSize)
			{
				WorkGroupInfos.Add(PackWorkGroupInfo(CurrentWorkGroupFirstItem, CurrentWorkGroupNumItems, PayloadGenerator.GetWorkGroupPayload(), CurrentWorkGroupCarryOver, CurrentWorkGroupWorkBoundary));
				CurrentWorkGroupFirstItem = uint32(Items.Num());
				CurrentWorkGroupPrefixSum = 0u;
				CurrentWorkGroupNumItems = 0U;
				CurrentWorkGroupCarryOver = NumAdded;
				CurrentWorkGroupWorkBoundary = 0ull;
			}
		}
		TotalChildren += NumAdded;
	}

	bool IsEmpty() const
	{
		return Items.IsEmpty();
	}

	/**
	 * Call when finished adding work items to the balancer to flush any in-progress batches.
	 */
	template <typename PayloadGeneratorType>
	void FinalizeBatches(PayloadGeneratorType& PayloadGenerator)
	{
		if (CurrentWorkGroupNumItems != 0)
		{
			WorkGroupInfos.Add(PackWorkGroupInfo(CurrentWorkGroupFirstItem, CurrentWorkGroupNumItems, PayloadGenerator.GetWorkGroupPayload(), CurrentWorkGroupCarryOver, CurrentWorkGroupWorkBoundary));
			CurrentWorkGroupNumItems = 0;
		}
	}

	void GetParametersAsync(FRDGBuilder& GraphBuilder, FShaderParameters& OutShaderParameters);

	void FinalizeParametersAsync(FShaderParameters& OutShaderParameters);

	/**
	 * Returns a 3D group count that is large enough to generate one group per batch using FComputeShaderUtils::GetGroupCountWrapped.
	 * Use GetUnWrappedDispatchGroupId in the shader to retrieve the linear index.
	 */
	FIntVector GetWrappedCsGroupCount() const;

	/*
	* Publish constants to a shader implementing a kernel using the load balancer.
	* Call from ModifyCompilationEnvironment
	*/
	static void SetShaderDefines(FShaderCompilerEnvironment& OutEnvironment);

	uint32 GetTotalChildren() const { return TotalChildren; }

protected:
	TArray<FWorkGroupInfo, SceneRenderingAllocator>  WorkGroupInfos;
	TArray<FPackedItem, SceneRenderingAllocator>  Items;

	uint32 CurrentWorkGroupPrefixSum = 0u;
	uint32 CurrentWorkGroupNumItems = 0U;
	uint32 CurrentWorkGroupFirstItem = 0U;
	uint32 TotalChildren = 0U;
	uint32 CurrentWorkGroupCarryOver = 0u;
	uint64 CurrentWorkGroupWorkBoundary = 0ull;
};


/*
 */
template <typename ParentInfoType>
class TGPUWorkGroupLoadBalancer : public FGPUWorkGroupLoadBalancer
{
private:
	struct FPayloadGenerator;
public:
	using FParentInfo = ParentInfoType;

	void ReserveStorage(int32 NumBatches, int32 NumItems, int32 NumParents)
	{
		FGPUWorkGroupLoadBalancer::ReserveStorage(NumBatches, NumItems);
		ParentInfos.Empty(NumParents);
	}

	/**
	 * Add a span of instances to be processed.
	 */
	void Add(FParentInfo&& ParentInfo, uint32 NumChildren)
	{
		PayloadGenerator.ParentItemOffset = uint32(ParentInfos.Num());
		ParentInfos.Emplace(MoveTemp(ParentInfo));
		FGPUWorkGroupLoadBalancer::Add(PayloadGenerator, NumChildren);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUWorkGroupLoadBalancer::FShaderParameters, BaseParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FParentInfo >, ParentInfoBuffer)
		SHADER_PARAMETER(uint32, NumParentInfos)
	END_SHADER_PARAMETER_STRUCT()

	void GetParametersAsync(FRDGBuilder& GraphBuilder, FShaderParameters& OutShaderParameters)
	{
		FGPUWorkGroupLoadBalancer::GetParametersAsync(GraphBuilder, OutShaderParameters.BaseParameters);

		FRDGBufferRef ParentInfosRDG = CreateStructuredBuffer(GraphBuilder, TEXT("GPUWorkGroupLoadBalancer.ParentInfos"), [&]() -> auto& { return ParentInfos; });
		OutShaderParameters.ParentInfoBuffer = GraphBuilder.CreateSRV(ParentInfosRDG);
		OutShaderParameters.NumParentInfos = ~0u;
	}

	void FinalizeParametersAsync(FShaderParameters& OutShaderParameters)
	{
		FGPUWorkGroupLoadBalancer::FinalizeParametersAsync(OutShaderParameters.BaseParameters);
		OutShaderParameters.NumParentInfos = ParentInfos.Num();
	}

	/**
	 * Call when finished adding work items to the balancer to flush any in-progress batches.
	 */
	void FinalizeBatches()
	{
		FGPUWorkGroupLoadBalancer::FinalizeBatches(PayloadGenerator);
	}

private:
	struct FPayloadGenerator
	{
		uint32 CurrentWorkGroupFirstParentItem = 0u;
		uint32 ParentItemOffset = 0u;

		uint32 GetItemPayload() const
		{
			return ParentItemOffset - CurrentWorkGroupFirstParentItem;
		}

		uint32 GetWorkGroupPayload()
		{
			uint32 Tmp = CurrentWorkGroupFirstParentItem;
			CurrentWorkGroupFirstParentItem = ParentItemOffset;
			return Tmp;
		}
	};

	TArray<FParentInfo, SceneRenderingAllocator> ParentInfos;
	FPayloadGenerator PayloadGenerator;
};
