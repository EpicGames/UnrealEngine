// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenerateConservativeDepth.cpp
=============================================================================*/

#include "GenerateConservativeDepthBuffer.h"
#include "SceneRendering.h"
#include "DataDrivenShaderPlatformInfo.h"

class FGenerateConservativeDepthBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGenerateConservativeDepthBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FGenerateConservativeDepthBufferCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, ConservativeDepthTextureUAV)
		SHADER_PARAMETER(FIntPoint, ConservativeDepthTextureSize)
		SHADER_PARAMETER(int32, DestinationPixelSizeAtFullRes)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGenerateConservativeDepthBufferCS, "/Engine/Private/GenerateConservativeDepth.usf", "GenerateConservativeDepthBufferCS", SF_Compute);



void AddGenerateConservativeDepthBufferPass(FViewInfo& View, FRDGBuilder& GraphBuilder, FRDGTextureRef ConservativeDepthTexture, int32 DestinationPixelSizeAtFullRes)
{
	if (View.HZB)
	{
		FGenerateConservativeDepthBufferCS::FPermutationDomain Permutation;
		TShaderMapRef<FGenerateConservativeDepthBufferCS> ComputeShader(GetGlobalShaderMap(View.GetFeatureLevel()), Permutation);

		FIntVector ConservativeDepthTextureSize3D = ConservativeDepthTexture->Desc.GetSize();
		FIntPoint ConservativeDepthTextureSize = FIntPoint(ConservativeDepthTextureSize3D.X, ConservativeDepthTextureSize3D.Y);

		FGenerateConservativeDepthBufferCS::FParameters* Parameters = GraphBuilder.AllocParameters<FGenerateConservativeDepthBufferCS::FParameters>();
		Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
		Parameters->ConservativeDepthTextureUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(ConservativeDepthTexture));
		Parameters->ConservativeDepthTextureSize = ConservativeDepthTextureSize;
		Parameters->DestinationPixelSizeAtFullRes = DestinationPixelSizeAtFullRes;
		Parameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);

		const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(ConservativeDepthTextureSize.X, ConservativeDepthTextureSize.Y, 1), FIntVector(8, 8, 1));
		FComputeShaderUtils::AddPass(
			GraphBuilder, RDG_EVENT_NAME("GenerateConservativeDepthBuffer"),
			ComputeShader, Parameters, DispatchCount);
	}
	else
	{
		// Clear to far distance
		AddClearRenderTargetPass(GraphBuilder, ConservativeDepthTexture, FLinearColor::Black);
	}
}


