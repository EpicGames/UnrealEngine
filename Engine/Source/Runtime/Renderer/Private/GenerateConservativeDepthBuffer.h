// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GenerateConservativeDepth.h
=============================================================================*/

#pragma once

#include "RenderGraphResources.h"

class FViewInfo;
class FRDGBuilder;


void AddGenerateConservativeDepthBufferPass(FViewInfo& View, FRDGBuilder& GraphBuilder, FRDGTextureRef ConservativeDepthTexture, int32 DestinationPixelSizeAtFullRes);

