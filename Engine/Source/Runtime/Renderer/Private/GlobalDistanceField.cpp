// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalDistanceField.h"
#include "CoreMinimal.h"
#include "Async/TaskGraphInterfaces.h"
#include "CanvasTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "DistanceFieldLightingShared.h"
#include "DynamicPrimitiveDrawing.h"
#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "Engine/Texture2D.h"
#include "GameTime.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "GlobalDistanceFieldConstants.h"
#include "GlobalDistanceFieldHeightfields.h"
#include "GlobalDistanceFieldParameters.h"
#include "GlobalDistanceFieldReadback.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "HeightfieldLighting.h"
#include "Logging/LogMacros.h"
#include "Lumen/Lumen.h"
#include "Math/Color.h"
#include "Math/DoubleFloat.h"
#include "Misc/LargeWorldCoordinates.h"
#include "MultiGPU.h"
#include "PrimitiveDrawingUtils.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "RenderCommandTag.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "RenderResource.h"
#include "RenderTargetPool.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIGPUReadback.h"
#include "RHIPipeline.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderPrintParameters.h"
#include "Stats/Stats.h"
#include "TextureFallbacks.h"
#include "TextureResource.h"
#include "SceneViewState.h"

DECLARE_GPU_STAT(GlobalDistanceFieldUpdate);

extern int32 GDistanceFieldOffsetDataStructure;

int32 GAOGlobalDistanceField = 1;
FAutoConsoleVariableRef CVarAOGlobalDistanceField(
	TEXT("r.AOGlobalDistanceField"), 
	GAOGlobalDistanceField,
	TEXT("Whether to use a global distance field to optimize occlusion cone traces.\n")
	TEXT("The global distance field is created by compositing object distance fields into clipmaps as the viewer moves through the level."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GGlobalDistanceFieldOccupancyRatio = 0.3f;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldOccupancyRatio(
	TEXT("r.AOGlobalDistanceField.OccupancyRatio"),
	GGlobalDistanceFieldOccupancyRatio,
	TEXT("Expected sparse global distacne field occupancy for the page atlas allocation. 0.25 means 25% - filled and 75% - empty."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GAOGlobalDistanceFieldNumClipmaps = 4;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldNumClipmaps(
	TEXT("r.AOGlobalDistanceField.NumClipmaps"), 
	GAOGlobalDistanceFieldNumClipmaps,
	TEXT("Num clipmaps in the global distance field.  Setting this to anything other than 4 is currently only supported by Lumen."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GAOGlobalDistanceFieldHeightfield = 1;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldHeightfield(
	TEXT("r.AOGlobalDistanceField.Heightfield"),
	GAOGlobalDistanceFieldHeightfield,
	TEXT("Whether to voxelize Heightfield into the global distance field.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GAOUpdateGlobalDistanceField = 1;
FAutoConsoleVariableRef CVarAOUpdateGlobalDistanceField(
	TEXT("r.AOUpdateGlobalDistanceField"),
	GAOUpdateGlobalDistanceField,
	TEXT("Whether to update the global distance field, useful for debugging."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GAOGlobalDistanceFieldCacheMostlyStaticSeparately = 1;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldCacheMostlyStaticSeparately(
	TEXT("r.AOGlobalDistanceFieldCacheMostlyStaticSeparately"),
	GAOGlobalDistanceFieldCacheMostlyStaticSeparately,
	TEXT("Whether to cache mostly static primitives separately from movable primitives, which reduces global DF update cost when a movable primitive is modified.  Adds another 12Mb of volume textures."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GAOGlobalDistanceFieldPartialUpdates = 1;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldPartialUpdates(
	TEXT("r.AOGlobalDistanceFieldPartialUpdates"),
	GAOGlobalDistanceFieldPartialUpdates,
	TEXT("Whether to allow partial updates of the global distance field.  When profiling it's useful to disable this and get the worst case composition time that happens on camera cuts."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GAOGlobalDistanceFieldStaggeredUpdates = 1;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldStaggeredUpdatess(
	TEXT("r.AOGlobalDistanceFieldStaggeredUpdates"),
	GAOGlobalDistanceFieldStaggeredUpdates,
	TEXT("Whether to allow the larger clipmaps to be updated less frequently."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GAOGlobalDistanceFieldClipmapUpdatesPerFrame = 2;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldClipmapUpdatesPerFrame(
	TEXT("r.AOGlobalDistanceFieldClipmapUpdatesPerFrame"),
	GAOGlobalDistanceFieldClipmapUpdatesPerFrame,
	TEXT("How many clipmaps to update each frame.  With values less than 2, the first clipmap is only updated every other frame, which can cause incorrect self occlusion during movement."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GAOGlobalDistanceFieldForceFullUpdate = 0;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldForceFullUpdate(
	TEXT("r.AOGlobalDistanceFieldForceFullUpdate"),
	GAOGlobalDistanceFieldForceFullUpdate,
	TEXT("Whether to force full global distance field update every frame."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GAOGlobalDistanceFieldForceUpdateOnce = 0;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldForceUpdateOnce(
	TEXT("r.AOGlobalDistanceFieldForceUpdateOnce"),
	GAOGlobalDistanceFieldForceUpdateOnce,
	TEXT("Whether to force full global distance field once."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GAOGlobalDFClipmapDistanceExponent = 2;
FAutoConsoleVariableRef CVarAOGlobalDFClipmapDistanceExponent(
	TEXT("r.AOGlobalDFClipmapDistanceExponent"),
	GAOGlobalDFClipmapDistanceExponent,
	TEXT("Exponent used to derive each clipmap's size, together with r.AOInnerGlobalDFClipmapDistance."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GAOGlobalDFResolution = 128;
FAutoConsoleVariableRef CVarAOGlobalDFResolution(
	TEXT("r.AOGlobalDFResolution"),
	GAOGlobalDFResolution,
	TEXT("Resolution of the global distance field.  Higher values increase fidelity but also increase memory and composition cost."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GAOGlobalDFStartDistance = 100;
FAutoConsoleVariableRef CVarAOGlobalDFStartDistance(
	TEXT("r.AOGlobalDFStartDistance"),
	GAOGlobalDFStartDistance,
	TEXT("World space distance along a cone trace to switch to using the global distance field instead of the object distance fields.\n")
	TEXT("This has to be large enough to hide the low res nature of the global distance field, but smaller values result in faster cone tracing."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GAOGlobalDistanceFieldRepresentHeightfields = 1;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldRepresentHeightfields(
	TEXT("r.AOGlobalDistanceFieldRepresentHeightfields"),
	GAOGlobalDistanceFieldRepresentHeightfields,
	TEXT("Whether to put landscape in the global distance field.  Changing this won't propagate until the global distance field gets recached (fly away and back)."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GGlobalDistanceFieldHeightFieldThicknessScale = 4.0f;
FAutoConsoleVariableRef CVarGlobalDistanceFieldHeightFieldThicknessScale(
	TEXT("r.GlobalDistanceFieldHeightFieldThicknessScale"),
	GGlobalDistanceFieldHeightFieldThicknessScale,
	TEXT("Thickness of the height field when it's entered into the global distance field, measured in distance field voxels. Defaults to 4 which means 4x the voxel size as thickness."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GAOGlobalDistanceFieldMinMeshSDFRadius = 20;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldMinMeshSDFRadius(
	TEXT("r.AOGlobalDistanceField.MinMeshSDFRadius"),
	GAOGlobalDistanceFieldMinMeshSDFRadius,
	TEXT("Meshes with a smaller world space radius than this are culled from the global SDF."),
	ECVF_RenderThreadSafe
	);

float GAOGlobalDistanceFieldMinMeshSDFRadiusInVoxels = .5f;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldMinMeshSDFRadiusInVoxels(
	TEXT("r.AOGlobalDistanceField.MinMeshSDFRadiusInVoxels"),
	GAOGlobalDistanceFieldMinMeshSDFRadiusInVoxels,
	TEXT("Meshes with a smaller radius than this number of voxels are culled from the global SDF."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GAOGlobalDistanceFieldCameraPositionVelocityOffsetDecay = .7f;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldCameraPositionVelocityOffsetDecay(
	TEXT("r.AOGlobalDistanceField.CameraPositionVelocityOffsetDecay"),
	GAOGlobalDistanceFieldCameraPositionVelocityOffsetDecay,
	TEXT(""),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GAOGlobalDistanceFieldFastCameraMode = 0;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldFastCameraMode(
	TEXT("r.AOGlobalDistanceField.FastCameraMode"),
	GAOGlobalDistanceFieldFastCameraMode,
	TEXT("Whether to update the Global SDF for fast camera movement - lower quality, faster updates so lighting can keep up with the camera."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAOGlobalDistanceFieldAverageCulledObjectsPerCell(
	TEXT("r.AOGlobalDistanceField.AverageCulledObjectsPerCell"),
	512,
	TEXT("Average expected number of objects per cull grid cell, used to preallocate memory for the cull grid."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GAOGlobalDistanceFieldMipFactor = 4;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldMipFactor(
	TEXT("r.AOGlobalDistanceField.MipFactor"),
	GAOGlobalDistanceFieldMipFactor,
	TEXT("Resolution divider for the mip map of a distance field clipmap."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GAOGlobalDistanceFieldRecacheClipmapsWithPendingStreaming = 1;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldRecacheClipmapsWithPendingStreaming(
	TEXT("r.AOGlobalDistanceField.RecacheClipmapsWithPendingStreaming"),
	GAOGlobalDistanceFieldRecacheClipmapsWithPendingStreaming,
	TEXT("Whether to readback clipmaps cached with incomplete Mesh SDFs due to streaming and recache them on subsequent frames. Fixes innaccurate Global SDF around the camera after teleporting or loading a new level."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GAOGlobalDistanceFieldForceRecacheForStreaming = 0;
FAutoConsoleVariableRef CVarAOGlobalDistanceFieldForceRecacheForStreaming(
	TEXT("r.AOGlobalDistanceField.ForceRecacheForStreaming"),
	GAOGlobalDistanceFieldForceRecacheForStreaming,
	TEXT("Useful for debugging or profiling full clipmap updates that happen when a clipmap is detected to have pending streaming."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GLumenSceneGlobalSDFCoveredExpandSurfaceScale = 1.0f;
FAutoConsoleVariableRef CVarLumenSceneGlobalSDFCoveredExpandSurfaceScale(
	TEXT("r.LumenScene.GlobalSDF.CoveredExpandSurfaceScale"),
	GLumenSceneGlobalSDFCoveredExpandSurfaceScale,
	TEXT("Scales the half voxel SDF expand used by the Global SDF to reconstruct surfaces that are thinner than the distance between two voxels, erring on the side of over-occlusion."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GLumenSceneGlobalSDFNotCoveredExpandSurfaceScale = .6f;
FAutoConsoleVariableRef CVarLumenScenGlobalSDFNotCoveredExpandSurfaceScale(
	TEXT("r.LumenScene.GlobalSDF.NotCoveredExpandSurfaceScale"),
	GLumenSceneGlobalSDFNotCoveredExpandSurfaceScale,
	TEXT("Scales the half voxel SDF expand used by the Global SDF to reconstruct surfaces that are thinner than the distance between two voxels, for regions of space that only contain Two Sided Mesh SDFs."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GLumenSceneGlobalSDFSimpleCoverageBasedExpand = 0;
FAutoConsoleVariableRef CVarLumenSceneGlobalSDFSimpleCoverageBasedExpand(
	TEXT("r.LumenScene.GlobalSDF.SimpleCoverageBasedExpand"),
	GLumenSceneGlobalSDFSimpleCoverageBasedExpand,
	TEXT("Whether to use simple coverage based surface expansion. Less accurate but does not sample the coverage texture."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

float GLumenSceneGlobalSDFNotCoveredMinStepScale = 4.0f;
FAutoConsoleVariableRef CVarLumenScenGlobalSDFNotCoveredMinStepScale(
	TEXT("r.LumenScene.GlobalSDF.NotCoveredMinStepScale"),
	GLumenSceneGlobalSDFNotCoveredMinStepScale,
	TEXT("Scales the min step size to improve performance, for regions of space that only contain Two Sided Mesh SDFs."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GLumenSceneGlobalSDFDitheredTransparencyStepThreshold = .5f;
FAutoConsoleVariableRef CVarLumenSceneGlobalSDFDitheredTransparencyStepThreshold(
	TEXT("r.LumenScene.GlobalSDF.DitheredTransparencyStepThreshold"),
	GLumenSceneGlobalSDFDitheredTransparencyStepThreshold,
	TEXT("Per-step stochastic semi-transparency threshold, for tracing users that have dithered transparency enabled, for regions of space that only contain Two Sided Mesh SDFs."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GLumenSceneGlobalSDFDitheredTransparencyTraceThreshold = .9f;
FAutoConsoleVariableRef CVarLumenSceneGlobalSDFDitheredTransparencyTraceThreshold(
	TEXT("r.LumenScene.GlobalSDF.DitheredTransparencyTraceThreshold"),
	GLumenSceneGlobalSDFDitheredTransparencyTraceThreshold,
	TEXT("Per-trace stochastic semi-transparency threshold, for tracing users that have dithered transparency enabled, for regions of space that only contain Two Sided Mesh SDFs.  Anything less than 1 causes leaking."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarGlobalDistanceFieldDebugShowStats(
	TEXT("r.GlobalDistanceField.Debug.ShowStats"),
	0,
	TEXT("Debug drawing for the Global Distance Field."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GGlobalDistanceFieldDebugForceMovementUpdate = 0;
FAutoConsoleVariableRef CVarGlobalDistanceFieldDebugForceMovementUpdate(
	TEXT("r.GlobalDistanceField.Debug.ForceMovementUpdate"),
	GGlobalDistanceFieldDebugForceMovementUpdate,
	TEXT("Whether to force N texel border on X, Y and Z update each frame."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GGlobalDistanceFieldDebugLogModifiedPrimitives = 0;
FAutoConsoleVariableRef CVarGlobalDistanceFieldDebugLogModifiedPrimitives(
	TEXT("r.GlobalDistanceField.Debug.LogModifiedPrimitives"),
	GGlobalDistanceFieldDebugLogModifiedPrimitives,
	TEXT("Whether to log primitive modifications (add, remove, updatetransform) that caused an update of the global distance field.\n")
	TEXT("This can be useful for tracking down why updating the global distance field is always costing a lot, since it should be mostly cached.\n")
	TEXT("Pass 2 to log only non movable object updates."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GGlobalDistanceFieldDebugDrawModifiedPrimitives = 0;
FAutoConsoleVariableRef CVarGlobalDistanceFieldDebugDrawModifiedPrimitives(
	TEXT("r.GlobalDistanceField.Debug.DrawModifiedPrimitives"),
	GGlobalDistanceFieldDebugDrawModifiedPrimitives,
	TEXT("Whether to draw primitive modifications (add, remove, updatetransform) that caused an update of the global distance field.\n")
	TEXT("This can be useful for tracking down why updating the global distance field is always costing a lot, since it should be mostly cached."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

bool UseGlobalDistanceField()
{
	return GAOGlobalDistanceField != 0;
}

bool UseGlobalDistanceField(const FDistanceFieldAOParameters& Parameters)
{
	return UseGlobalDistanceField() && Parameters.GlobalMaxOcclusionDistance > 0;
}

static bool SupportsGlobalDistanceFieldCoverage(EShaderPlatform Platform)
{
	return RHISupportsVolumeTextureAtomics(Platform);
}

FGlobalDistanceFieldParameters2 SetupGlobalDistanceFieldParameters(const FGlobalDistanceFieldParameterData& ParameterData)
{
	FGlobalDistanceFieldParameters2 ShaderParameters;

	ShaderParameters.GlobalDistanceFieldPageAtlasTexture = OrBlack3DIfNull(ParameterData.PageAtlasTexture);
	ShaderParameters.GlobalDistanceFieldCoverageAtlasTexture = OrBlack3DIfNull(ParameterData.CoverageAtlasTexture);
	ShaderParameters.GlobalDistanceFieldPageTableTexture = OrBlack3DUintIfNull(ParameterData.PageTableTexture);
	ShaderParameters.GlobalDistanceFieldMipTexture = OrBlack3DIfNull(ParameterData.MipTexture);

	for (int32 Index = 0; Index < GlobalDistanceField::MaxClipmaps; Index++)
	{
		ShaderParameters.GlobalVolumeTranslatedCenterAndExtent[Index] = ParameterData.TranslatedCenterAndExtent[Index];
		ShaderParameters.GlobalVolumeTranslatedWorldToUVAddAndMul[Index] = ParameterData.TranslatedWorldToUVAddAndMul[Index];
		ShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVScale[Index] = ParameterData.MipTranslatedWorldToUVScale[Index];
		ShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVBias[Index] = ParameterData.MipTranslatedWorldToUVBias[Index];
	}

	ShaderParameters.GlobalDistanceFieldMipFactor = ParameterData.MipFactor;
	ShaderParameters.GlobalDistanceFieldMipTransition = ParameterData.MipTransition;
	ShaderParameters.GlobalDistanceFieldClipmapSizeInPages = ParameterData.ClipmapSizeInPages;
	ShaderParameters.GlobalDistanceFieldInvPageAtlasSize = (FVector3f)ParameterData.InvPageAtlasSize;
	ShaderParameters.GlobalDistanceFieldInvCoverageAtlasSize = (FVector3f)ParameterData.InvCoverageAtlasSize;
	ShaderParameters.GlobalVolumeDimension = ParameterData.GlobalDFResolution;
	ShaderParameters.GlobalVolumeTexelSize = 1.0f / ParameterData.GlobalDFResolution;
	ShaderParameters.MaxGlobalDFAOConeDistance = ParameterData.MaxDFAOConeDistance;
	ShaderParameters.NumGlobalSDFClipmaps = ParameterData.NumGlobalSDFClipmaps;

	ShaderParameters.CoveredExpandSurfaceScale = GLumenSceneGlobalSDFCoveredExpandSurfaceScale;
	ShaderParameters.NotCoveredExpandSurfaceScale = GLumenSceneGlobalSDFNotCoveredExpandSurfaceScale;
	ShaderParameters.NotCoveredMinStepScale = GLumenSceneGlobalSDFNotCoveredMinStepScale;
	ShaderParameters.DitheredTransparencyStepThreshold = GLumenSceneGlobalSDFDitheredTransparencyStepThreshold;
	ShaderParameters.DitheredTransparencyTraceThreshold = GLumenSceneGlobalSDFDitheredTransparencyTraceThreshold;
	ShaderParameters.GlobalDistanceFieldCoverageAtlasTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	ShaderParameters.GlobalDistanceFieldPageAtlasTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	ShaderParameters.GlobalDistanceFieldMipTextureSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	return ShaderParameters;
}

const TCHAR* GetRecaptureReasonString(EGlobalSDFFullRecaptureReason Reason)
{
	switch (Reason)
	{
	case EGlobalSDFFullRecaptureReason::None:
		return TEXT("");
	case EGlobalSDFFullRecaptureReason::TooManyUpdateBounds:
		return TEXT("FULL UPDATE: TooManyUpdateBounds");
	case EGlobalSDFFullRecaptureReason::HeightfieldStreaming:
		return TEXT("FULL UPDATE: HeightfieldStreaming");
	case EGlobalSDFFullRecaptureReason::MeshSDFStreaming:
		return TEXT("FULL UPDATE: MeshSDFStreaming");
	case EGlobalSDFFullRecaptureReason::NoViewState:
		return TEXT("FULL UPDATE: NoViewState");
	default:
		return nullptr;
	}
}

float GetMinMeshSDFRadius(float VoxelWorldSize)
{
	float MinRadius = GAOGlobalDistanceFieldMinMeshSDFRadius * (GAOGlobalDistanceFieldFastCameraMode ? 10.0f : 1.0f);
	float MinVoxelRadius = GAOGlobalDistanceFieldMinMeshSDFRadiusInVoxels * VoxelWorldSize * (GAOGlobalDistanceFieldFastCameraMode ? 5.0f : 1.0f);

	return FMath::Max(MinRadius, MinVoxelRadius);
}

int32 GetNumClipmapUpdatesPerFrame()
{
	return GAOGlobalDistanceFieldFastCameraMode ? 1 : GAOGlobalDistanceFieldClipmapUpdatesPerFrame;
}

int32 GlobalDistanceField::GetNumGlobalDistanceFieldClipmaps(bool bLumenEnabled, float LumenSceneViewDistance)
{
	int32 WantedClipmaps = GAOGlobalDistanceFieldNumClipmaps;

	if (bLumenEnabled)
	{
		if (GlobalDistanceField::GetClipmapExtent(WantedClipmaps + 1, nullptr, true) <= LumenSceneViewDistance)
		{
			WantedClipmaps += 2;
		}
		else if (GlobalDistanceField::GetClipmapExtent(WantedClipmaps, nullptr, true) <= LumenSceneViewDistance)
		{
			WantedClipmaps += 1;
		}
	}

	if (GAOGlobalDistanceFieldFastCameraMode)
	{
		WantedClipmaps++;
	}
	return FMath::Clamp<int32>(WantedClipmaps, 0, GlobalDistanceField::MaxClipmaps);
}

// Global Distance Field Pages
// Must match GlobalDistanceFieldShared.ush
namespace GlobalDistanceField
{
	const int32 CullGridFactor = 4;
	const int32 PageAtlasSizeInPagesX = 128;
	const int32 PageAtlasSizeInPagesY = 128;

	// Distance field object grid
	// Every distance field page stores 4^3 object grid cells with 4 * uint32 elements per cell
	const int32 ObjectGridPageBufferStride = 4 * sizeof(uint32);
	const int32 ObjectGridPageBufferNumElementsPerPage = 4 * 4 * 4;
}
const int32 GGlobalDistanceFieldPageResolutionInAtlas = 8; // Includes 0.5 texel trilinear filter margin
const int32 GGlobalDistanceFieldCoveragePageResolutionInAtlas = 4; // Includes 0.5 texel trilinear filter margin
const int32 GGlobalDistanceFieldPageResolution = GGlobalDistanceFieldPageResolutionInAtlas - 1;
const int32 GGlobalDistanceFieldInfluenceRangeInVoxels = 4;

int32 GlobalDistanceField::GetClipmapResolution(bool bLumenEnabled)
{
	int32 DFResolution = GAOGlobalDFResolution;

	if (bLumenEnabled)
	{
		DFResolution = Lumen::GetGlobalDFResolution();
	}

	return FMath::DivideAndRoundUp(DFResolution, 4 * GGlobalDistanceFieldPageResolution) * 4 * GGlobalDistanceFieldPageResolution;
}

int32 GlobalDistanceField::GetMipFactor()
{
	return FMath::Clamp(GAOGlobalDistanceFieldMipFactor, 1, 8);
}

int32 GlobalDistanceField::GetClipmapMipResolution(bool bLumenEnabled)
{
	return FMath::DivideAndRoundUp(GlobalDistanceField::GetClipmapResolution(bLumenEnabled), GetMipFactor());
}

float GlobalDistanceField::GetClipmapExtent(int32 ClipmapIndex, const FScene* Scene, bool bLumenEnabled)
{
	if (bLumenEnabled) 
	{
		return Lumen::GetGlobalDFClipmapExtent(ClipmapIndex);
	}
	else
	{
		const float InnerClipmapDistance = Scene->GlobalDistanceFieldViewDistance / FMath::Pow(GAOGlobalDFClipmapDistanceExponent, 3);
		return InnerClipmapDistance * FMath::Pow(GAOGlobalDFClipmapDistanceExponent, ClipmapIndex);
	}
}

uint32 GlobalDistanceField::GetPageTableClipmapResolution(bool bLumenEnabled)
{
	return FMath::DivideAndRoundUp(GlobalDistanceField::GetClipmapResolution(bLumenEnabled), GGlobalDistanceFieldPageResolution);
}

FIntVector GlobalDistanceField::GetPageTableTextureResolution(bool bLumenEnabled, float LumenSceneViewDistance)
{
	const int32 NumClipmaps = GetNumGlobalDistanceFieldClipmaps(bLumenEnabled, LumenSceneViewDistance);
	const uint32 PageTableClipmapResolution = GetPageTableClipmapResolution(bLumenEnabled);

	const FIntVector PageTableTextureResolution = FIntVector(
		PageTableClipmapResolution, 
		PageTableClipmapResolution, 
		PageTableClipmapResolution * NumClipmaps);

	return PageTableTextureResolution;
}

FIntVector GlobalDistanceField::GetPageAtlasSizeInPages(bool bLumenEnabled, float LumenSceneViewDistance)
{
	const FIntVector PageTableTextureResolution = GetPageTableTextureResolution(bLumenEnabled, LumenSceneViewDistance);

	const int32 RequiredNumberOfPages = FMath::CeilToInt(
		PageTableTextureResolution.X * PageTableTextureResolution.Y * PageTableTextureResolution.Z 
		* (GAOGlobalDistanceFieldCacheMostlyStaticSeparately ? 2 : 1)
		* FMath::Clamp(GGlobalDistanceFieldOccupancyRatio, 0.1f, 1.0f));

	const int32 RequiredNumberOfPagesInZ = FMath::DivideAndRoundUp(RequiredNumberOfPages, GlobalDistanceField::PageAtlasSizeInPagesX * GlobalDistanceField::PageAtlasSizeInPagesY);

	const FIntVector PageAtlasTextureSizeInPages = FIntVector(
		GlobalDistanceField::PageAtlasSizeInPagesX,
		GlobalDistanceField::PageAtlasSizeInPagesY,
		RequiredNumberOfPagesInZ);

	return PageAtlasTextureSizeInPages;
}

FIntVector GlobalDistanceField::GetPageAtlasSize(bool bLumenEnabled, float LumenSceneViewDistance)
{
	const FIntVector PageAtlasTextureSizeInPages = GlobalDistanceField::GetPageAtlasSizeInPages(bLumenEnabled, LumenSceneViewDistance);
	return PageAtlasTextureSizeInPages * GGlobalDistanceFieldPageResolutionInAtlas;
}

FIntVector GlobalDistanceField::GetCoverageAtlasSize(bool bLumenEnabled, float LumenSceneViewDistance)
{
	const FIntVector PageAtlasTextureSizeInPages = GlobalDistanceField::GetPageAtlasSizeInPages(bLumenEnabled, LumenSceneViewDistance);
	return PageAtlasTextureSizeInPages * GGlobalDistanceFieldCoveragePageResolutionInAtlas;
}

int32 GlobalDistanceField::GetMaxPageNum(bool bLumenEnabled, float LumenSceneViewDistance)
{
	const FIntVector PageAtlasTextureSizeInPages = GlobalDistanceField::GetPageAtlasSizeInPages(bLumenEnabled, LumenSceneViewDistance);
	int32 MaxPageNum = PageAtlasTextureSizeInPages.X * PageAtlasTextureSizeInPages.Y * PageAtlasTextureSizeInPages.Z;
	return MaxPageNum;
}

// For reading back the distance field data
static FGlobalDistanceFieldReadback* GDFReadbackRequest = nullptr;

void RequestGlobalDistanceFieldReadback(FGlobalDistanceFieldReadback* Readback)
{
	if (ensure(GDFReadbackRequest == nullptr))
	{
		ensure(Readback->ReadbackComplete.IsBound());
		ensure(Readback->CallbackThread != ENamedThreads::UnusedAnchor);
		GDFReadbackRequest = Readback;
	}
}

void RequestGlobalDistanceFieldReadback_GameThread(FGlobalDistanceFieldReadback* Readback)
{
	ENQUEUE_RENDER_COMMAND(RequestGlobalDistanceFieldReadback)(
		[Readback](FRHICommandListImmediate& RHICmdList) {
			RequestGlobalDistanceFieldReadback(Readback);
		});
}

void FGlobalDistanceFieldInfo::UpdateParameterData(float MaxOcclusionDistance, bool bLumenEnabled, float LumenSceneViewDistance, FVector PreViewTranslation)
{
	ParameterData.PageTableTexture = nullptr;
	ParameterData.PageAtlasTexture = nullptr;
	ParameterData.PageObjectGridBuffer = nullptr;
	ParameterData.CoverageAtlasTexture = nullptr;
	ParameterData.MipTexture = nullptr;
	ParameterData.MaxPageNum = GlobalDistanceField::GetMaxPageNum(bLumenEnabled, LumenSceneViewDistance);

	if (Clipmaps.Num() > 0)
	{
		if (PageAtlasTexture)
		{
			ParameterData.PageAtlasTexture = PageAtlasTexture->GetRHI();
		}

		if (CoverageAtlasTexture)
		{
			ParameterData.CoverageAtlasTexture = CoverageAtlasTexture->GetRHI();
		}

		if (PageObjectGridBuffer)
		{
			ParameterData.PageObjectGridBuffer = PageObjectGridBuffer;
		}

		if (PageTableCombinedTexture)
		{
			ensureMsgf(GAOGlobalDistanceFieldCacheMostlyStaticSeparately, TEXT("PageTableCombinedTexture should only be allocated when caching mostly static objects separately."));
			ParameterData.PageTableTexture = PageTableCombinedTexture->GetRHI();
		}
		else if (PageTableLayerTextures[GDF_Full])
		{
			ensureMsgf(!GAOGlobalDistanceFieldCacheMostlyStaticSeparately, TEXT("PageTableCombinedTexture should be allocated when caching mostly static objects separately."));
			ParameterData.PageTableTexture = PageTableLayerTextures[GDF_Full]->GetRHI();
		}

		FIntVector MipTextureResolution(1, 1, 1);
		if (MipTexture)
		{
			ParameterData.MipTexture = MipTexture->GetRHI();
			MipTextureResolution.X = MipTexture->GetDesc().Extent.X;
			MipTextureResolution.Y = MipTexture->GetDesc().Extent.Y;
			MipTextureResolution.Z = MipTexture->GetDesc().Depth;
		}

		for (int32 ClipmapIndex = 0; ClipmapIndex < GlobalDistanceField::MaxClipmaps; ClipmapIndex++)
		{
			if (ClipmapIndex < Clipmaps.Num())
			{
				const FGlobalDistanceFieldClipmap& Clipmap = Clipmaps[ClipmapIndex];

				const FBox TranslatedBounds = Clipmap.Bounds.ShiftBy(PreViewTranslation);

				ParameterData.TranslatedCenterAndExtent[ClipmapIndex] = FVector4f((FVector3f)TranslatedBounds.GetCenter(), TranslatedBounds.GetExtent().X);

				// GlobalUV = (TranslatedWorldPosition - GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz + GlobalVolumeScollOffset[ClipmapIndex].xyz) / (GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * 2) + .5f;
				// TranslatedWorldToUVMul = 1.0f / (GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * 2)
				// TranslatedWorldToUVAdd = (GlobalVolumeScollOffset[ClipmapIndex].xyz - GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz) / (GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * 2) + .5f
				const FVector TranslatedWorldToUVAdd = (Clipmap.ScrollOffset - TranslatedBounds.GetCenter()) / TranslatedBounds.GetSize().X + FVector(.5f);
				ParameterData.TranslatedWorldToUVAddAndMul[ClipmapIndex] = FVector4f((FVector3f)TranslatedWorldToUVAdd, 1.0f / TranslatedBounds.GetSize().X);

				ParameterData.MipTranslatedWorldToUVScale[ClipmapIndex] = FVector3f(FVector(1.0f) / TranslatedBounds.GetSize());
				ParameterData.MipTranslatedWorldToUVBias[ClipmapIndex] = FVector3f((-TranslatedBounds.Min) / TranslatedBounds.GetSize());

				ParameterData.MipTranslatedWorldToUVScale[ClipmapIndex].Z = ParameterData.MipTranslatedWorldToUVScale[ClipmapIndex].Z / Clipmaps.Num();
				ParameterData.MipTranslatedWorldToUVBias[ClipmapIndex].Z = (ParameterData.MipTranslatedWorldToUVBias[ClipmapIndex].Z + ClipmapIndex) / Clipmaps.Num();

				// MipUV.z min max for correct bilinear filtering
				const int32 ClipmapMipResolution = GlobalDistanceField::GetClipmapMipResolution(bLumenEnabled);
				const float MipUVMinZ = (ClipmapIndex * ClipmapMipResolution + 0.5f) / MipTextureResolution.Z;
				const float MipUVMaxZ = (ClipmapIndex * ClipmapMipResolution + ClipmapMipResolution - 0.5f) / MipTextureResolution.Z;
				ParameterData.MipTranslatedWorldToUVScale[ClipmapIndex].W = MipUVMinZ;
				ParameterData.MipTranslatedWorldToUVBias[ClipmapIndex].W = MipUVMaxZ;
			}
			else
			{
				ParameterData.TranslatedCenterAndExtent[ClipmapIndex] = FVector4f(0);
				ParameterData.TranslatedWorldToUVAddAndMul[ClipmapIndex] = FVector4f(0);
				ParameterData.MipTranslatedWorldToUVScale[ClipmapIndex] = FVector4f(0);
				ParameterData.MipTranslatedWorldToUVBias[ClipmapIndex] = FVector4f(0);
			}
		}

		ParameterData.MipFactor = GlobalDistanceField::GetMipFactor();
		ParameterData.MipTransition = (GGlobalDistanceFieldInfluenceRangeInVoxels + ParameterData.MipFactor / GGlobalDistanceFieldInfluenceRangeInVoxels) / (2.0f * GGlobalDistanceFieldInfluenceRangeInVoxels);
		ParameterData.ClipmapSizeInPages = GlobalDistanceField::GetPageTableTextureResolution(bLumenEnabled, LumenSceneViewDistance).X;
		ParameterData.InvPageAtlasSize = FVector(1.0f) / FVector(GlobalDistanceField::GetPageAtlasSize(bLumenEnabled, LumenSceneViewDistance));
		ParameterData.InvCoverageAtlasSize = FVector(1.0f) / FVector(GlobalDistanceField::GetCoverageAtlasSize(bLumenEnabled, LumenSceneViewDistance));
		ParameterData.GlobalDFResolution = GlobalDistanceField::GetClipmapResolution(bLumenEnabled);

		extern float GAOConeHalfAngle;
		const float MaxClipmapSizeX = Clipmaps[Clipmaps.Num() - 1].Bounds.GetSize().X;
		const float MaxClipmapVoxelSize =  MaxClipmapSizeX / GlobalDistanceField::GetClipmapResolution(bLumenEnabled);
		float MaxClipmapInfluenceRadius = GGlobalDistanceFieldInfluenceRangeInVoxels * MaxClipmapVoxelSize;
		const float GlobalMaxSphereQueryRadius = FMath::Min(MaxOcclusionDistance / (1.0f + FMath::Tan(GAOConeHalfAngle)), MaxClipmapInfluenceRadius);
		ParameterData.MaxDFAOConeDistance = GlobalMaxSphereQueryRadius;
		ParameterData.NumGlobalSDFClipmaps = Clipmaps.Num();
	}
	else
	{
		FPlatformMemory::Memzero(&ParameterData, sizeof(ParameterData));
	}

	bInitialized = true;
}

/** Constructs and adds an update region based on camera movement for the given axis. */
static void AddUpdateBoundsForAxis(FInt64Vector MovementInPages,
	const FBox& ClipmapBounds,
	float ClipmapPageSize,
	int32 ComponentIndex, 
	TArray<FClipmapUpdateBounds, TInlineAllocator<64>>& UpdateBounds)
{
	FBox AxisUpdateBounds = ClipmapBounds;

	if (MovementInPages[ComponentIndex] > 0)
	{
		// Positive axis movement, set the min of that axis to contain the newly exposed area
		AxisUpdateBounds.Min[ComponentIndex] = FMath::Max(ClipmapBounds.Max[ComponentIndex] - MovementInPages[ComponentIndex] * ClipmapPageSize, ClipmapBounds.Min[ComponentIndex]);
	}
	else if (MovementInPages[ComponentIndex] < 0)
	{
		// Negative axis movement, set the max of that axis to contain the newly exposed area
		AxisUpdateBounds.Max[ComponentIndex] = FMath::Min(ClipmapBounds.Min[ComponentIndex] - MovementInPages[ComponentIndex] * ClipmapPageSize, ClipmapBounds.Max[ComponentIndex]);
	}

	if (FMath::Abs(MovementInPages[ComponentIndex]) > 0)
	{
		const FVector CellCenterAndBilinearFootprintBias = FVector((1.0f - 0.5f) * ClipmapPageSize);
		UpdateBounds.Add(FClipmapUpdateBounds(AxisUpdateBounds.GetCenter(), AxisUpdateBounds.GetExtent() + CellCenterAndBilinearFootprintBias, false));
	}
}

static void GetUpdateFrequencyForClipmap(int32 ClipmapIndex, int32 NumClipmaps, int32& OutFrequency, int32& OutPhase)
{
	const int32 NumClipmapUpdatesPerFrame = GAOGlobalDistanceFieldStaggeredUpdates ? FMath::Min(GetNumClipmapUpdatesPerFrame(), NumClipmaps) : NumClipmaps;

	if (ClipmapIndex < NumClipmapUpdatesPerFrame - 1)
	{
		// update the first N-1 clipmaps every frame
		OutFrequency = 1;
		OutPhase = 0;
		return;
	}

	// remaining clipmaps update at different frequencies so that only one is updated each frame

	const int32 RemIndex = ClipmapIndex - (NumClipmapUpdatesPerFrame - 1);

	if (ClipmapIndex == NumClipmaps - 1)
	{
		// last clipmap updates at same frequency as previous
		OutFrequency = 1 << RemIndex;
	}
	else
	{
		// each clipmap updates with half frequency of previous
		OutFrequency = 2 << RemIndex;
	}
		
	// every clipmap uses a different phase
	OutPhase = (1 << RemIndex) - 1; // 2^n - 1 sequence
}

/** Staggers clipmap updates so there are only 2 per frame */
static bool ShouldUpdateClipmapThisFrame(int32 ClipmapIndex, int32 NumClipmaps, int32 GlobalDistanceFieldUpdateIndex)
{
	int32 Frequency;
	int32 Phase;
	GetUpdateFrequencyForClipmap(ClipmapIndex, NumClipmaps, Frequency, Phase);

	return GlobalDistanceFieldUpdateIndex % Frequency == Phase;
}

FVector ClampCameraVelocityOffset(FVector CameraVelocityOffset, int32 ClipmapIndex, const FScene* Scene, bool bLumenEnabled)
{
	// Clamp the view origin to stay inside the current clipmap extents
	const float ClipmapExtent = GlobalDistanceField::GetClipmapExtent(ClipmapIndex, Scene, bLumenEnabled);
	const float MaxCameraDriftFraction = .75f;
	CameraVelocityOffset.X = FMath::Clamp<float>(CameraVelocityOffset.X, -ClipmapExtent * MaxCameraDriftFraction, ClipmapExtent * MaxCameraDriftFraction);
	CameraVelocityOffset.Y = FMath::Clamp<float>(CameraVelocityOffset.Y, -ClipmapExtent * MaxCameraDriftFraction, ClipmapExtent * MaxCameraDriftFraction);
	CameraVelocityOffset.Z = FMath::Clamp<float>(CameraVelocityOffset.Z, -ClipmapExtent * MaxCameraDriftFraction, ClipmapExtent * MaxCameraDriftFraction);

	return CameraVelocityOffset;
}

static void UpdateGlobalDistanceFieldViewOrigin(const FViewInfo& View, bool bLumenEnabled)
{
	// Don't update origin if it has already been updated this frame
	if (View.ViewState && (View.ViewState->GlobalDistanceFieldData->bFirstFrame || View.ViewState->GlobalDistanceFieldData->UpdateFrame != View.Family->FrameNumber))
	{
		FVector& CameraVelocityOffset = View.ViewState->GlobalDistanceFieldData->CameraVelocityOffset;

		if (GAOGlobalDistanceFieldFastCameraMode != 0)
		{
			const FVector CameraVelocity = View.ViewMatrices.GetViewOrigin() - View.PrevViewInfo.ViewMatrices.GetViewOrigin();
			// Framerate independent decay
			CameraVelocityOffset = CameraVelocityOffset * FMath::Pow(GAOGlobalDistanceFieldCameraPositionVelocityOffsetDecay, View.Family->Time.GetDeltaWorldTimeSeconds()) + CameraVelocity;

			const FScene* Scene = (const FScene*)View.Family->Scene;
			const int32 NumClipmaps = GlobalDistanceField::GetNumGlobalDistanceFieldClipmaps(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance);

			if (Scene && NumClipmaps > 0)
			{
				// clamp based on largest voxel clipmap extent
				CameraVelocityOffset = ClampCameraVelocityOffset(CameraVelocityOffset, NumClipmaps - 1, Scene, bLumenEnabled);
			}
		}
		else
		{
			CameraVelocityOffset = FVector(0.0f, 0.0f, 0.0f);
		}
	}
}

void UpdateGlobalDistanceFieldViewOrigin(const FSceneRenderUpdateInputs& SceneUpdateInputs)
{
	SceneUpdateInputs.ForEachView([&](const FSceneRenderer* Renderer, FViewInfo& View)
		{
			const bool bUseLumenGlobalDistanceFieldSettings = Renderer->IsLumenEnabled(View) && Lumen::IsUsingGlobalSDF(*View.Family);
			UpdateGlobalDistanceFieldViewOrigin(View, bUseLumenGlobalDistanceFieldSettings);
			return true;
		});
}

FVector GetGlobalDistanceFieldViewOrigin(const FViewInfo& View, int32 ClipmapIndex, bool bLumenEnabled)
{
	FVector CameraOrigin = View.ViewMatrices.GetViewOrigin();

	if (View.ViewState)
	{
		FVector CameraVelocityOffset = View.ViewState->GlobalDistanceFieldData->CameraVelocityOffset;

		const FScene* Scene = (const FScene*)View.Family->Scene;

		if (Scene)
		{
			CameraVelocityOffset = ClampCameraVelocityOffset(CameraVelocityOffset, ClipmapIndex, Scene, bLumenEnabled);
		}

		CameraOrigin += CameraVelocityOffset;

		if (!View.ViewState->GlobalDistanceFieldData->bUpdateViewOrigin)
		{
			CameraOrigin = View.ViewState->GlobalDistanceFieldData->LastViewOrigin;
		}
	}

	return CameraOrigin;
}

static void ComputeUpdateRegionsAndUpdateViewState(
	FRHICommandListImmediate& RHICmdList, 
	FViewInfo& View, 
	const FScene* Scene, 
	FGlobalDistanceFieldInfo& GlobalDistanceFieldInfo, 
	int32 NumClipmaps, 
	float MaxOcclusionDistance,
	bool bLumenEnabled)
{
	GlobalDistanceFieldInfo.Clipmaps.AddZeroed(NumClipmaps);
	GlobalDistanceFieldInfo.MostlyStaticClipmaps.AddZeroed(NumClipmaps);

	// Cache the heightfields update region boxes for fast reuse for each clip region.
	TArray<FBox> PendingStreamingHeightfieldBoxes;
	for (const FPrimitiveSceneInfo* HeightfieldPrimitive : Scene->DistanceFieldSceneData.HeightfieldPrimitives)
	{
		if (HeightfieldPrimitive->Proxy->HeightfieldHasPendingStreaming())
		{
			PendingStreamingHeightfieldBoxes.Add(HeightfieldPrimitive->Proxy->GetBounds().GetBox());
		}
	}

	if (View.ViewState)
	{
		FSceneViewState& ViewState = *View.ViewState;
		FPersistentGlobalDistanceFieldData& GlobalDistanceFieldData = *ViewState.GlobalDistanceFieldData;

		bool bUpdatedThisFrame = !GlobalDistanceFieldData.bFirstFrame && GlobalDistanceFieldData.UpdateFrame == View.Family->FrameNumber;

		// Don't advance update counter if we've already been updated
		if (!bUpdatedThisFrame)
		{
			GlobalDistanceFieldData.UpdateIndex++;

			if (GlobalDistanceFieldData.UpdateIndex > 128)
			{
				GlobalDistanceFieldData.UpdateIndex = 0;
			}
		}

		// Check if GPU mask changed
		bool bForceFullUpdateMGPU = false;
#if WITH_MGPU
		if (GlobalDistanceFieldData.LastGPUMask != RHICmdList.GetGPUMask())
		{
			bForceFullUpdateMGPU = true;
			GlobalDistanceFieldData.LastGPUMask = RHICmdList.GetGPUMask();
		}
#endif

		int32 NumClipmapUpdateRequests = 0;

		FViewElementPDI ViewPDI(&View, nullptr, &View.DynamicPrimitiveCollector);

		bool bSharedDataReallocated = false;

		GlobalDistanceFieldInfo.PageFreeListAllocatorBuffer = nullptr;
		GlobalDistanceFieldInfo.PageFreeListBuffer = nullptr;
		GlobalDistanceFieldInfo.PageAtlasTexture = nullptr;
		GlobalDistanceFieldInfo.PageObjectGridBuffer = nullptr;
		GlobalDistanceFieldInfo.CoverageAtlasTexture = nullptr;

		{
			const int32 MaxPageNum = GlobalDistanceField::GetMaxPageNum(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance);
			const FIntVector PageAtlasTextureSize = GlobalDistanceField::GetPageAtlasSize(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance);
			const int32 PageObjectGridBufferSize = MaxPageNum * GlobalDistanceField::ObjectGridPageBufferNumElementsPerPage;

			if (!GlobalDistanceFieldData.PageFreeListAllocatorBuffer)
			{
				GlobalDistanceFieldData.PageFreeListAllocatorBuffer = AllocatePooledBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("GlobalDistanceField.PageFreeListAllocator"));
			}

			if (!GlobalDistanceFieldData.PageFreeListBuffer
				|| GlobalDistanceFieldData.PageFreeListBuffer->Desc.NumElements != MaxPageNum)
			{
				GlobalDistanceFieldData.PageFreeListBuffer = AllocatePooledBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxPageNum), TEXT("GlobalDistanceField.PageFreeList"));
			}

			if (bLumenEnabled && Lumen::UseGlobalSDFObjectGrid(*View.Family))
			{
				if (!GlobalDistanceFieldData.PageObjectGridBuffer
					|| GlobalDistanceFieldData.PageObjectGridBuffer->Desc.NumElements != PageObjectGridBufferSize)
				{
					GlobalDistanceFieldData.PageObjectGridBuffer = AllocatePooledBuffer(FRDGBufferDesc::CreateStructuredDesc(GlobalDistanceField::ObjectGridPageBufferStride, PageObjectGridBufferSize), TEXT("GlobalDistanceField.PageObjectGridBuffer"));
					bSharedDataReallocated = true;
				}
			}
			else
			{
				GlobalDistanceFieldData.PageObjectGridBuffer = nullptr;
			}

			if (!GlobalDistanceFieldData.PageAtlasTexture
				|| GlobalDistanceFieldData.PageAtlasTexture->GetDesc().Extent.X != PageAtlasTextureSize.X
				|| GlobalDistanceFieldData.PageAtlasTexture->GetDesc().Extent.Y != PageAtlasTextureSize.Y
				|| GlobalDistanceFieldData.PageAtlasTexture->GetDesc().Depth != PageAtlasTextureSize.Z)
			{
				FPooledRenderTargetDesc VolumeDesc = FPooledRenderTargetDesc(FPooledRenderTargetDesc::CreateVolumeDesc(
					PageAtlasTextureSize.X,
					PageAtlasTextureSize.Y,
					PageAtlasTextureSize.Z,
					PF_R8,
					FClearValueBinding::None,
					TexCreate_None,
					// TexCreate_ReduceMemoryWithTilingMode used because 128^3 texture comes out 4x bigger on PS4 with recommended volume texture tiling modes
					TexCreate_ShaderResource | TexCreate_UAV | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling,
					false));

				GRenderTargetPool.FindFreeElement(
					RHICmdList,
					VolumeDesc,
					GlobalDistanceFieldData.PageAtlasTexture,
					TEXT("GlobalDistanceField.PageAtlas")
				);

				bSharedDataReallocated = true;
			}

			const FIntVector CoverageAtlasTextureSize = GlobalDistanceField::GetCoverageAtlasSize(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance);

			if (bLumenEnabled && SupportsGlobalDistanceFieldCoverage(View.GetShaderPlatform()))

			{
				if (!GlobalDistanceFieldData.CoverageAtlasTexture
					|| GlobalDistanceFieldData.CoverageAtlasTexture->GetDesc().Extent.X != CoverageAtlasTextureSize.X
					|| GlobalDistanceFieldData.CoverageAtlasTexture->GetDesc().Extent.Y != CoverageAtlasTextureSize.Y
					|| GlobalDistanceFieldData.CoverageAtlasTexture->GetDesc().Depth != CoverageAtlasTextureSize.Z)
				{
					FPooledRenderTargetDesc VolumeDesc = FPooledRenderTargetDesc(FPooledRenderTargetDesc::CreateVolumeDesc(
						CoverageAtlasTextureSize.X,
						CoverageAtlasTextureSize.Y,
						CoverageAtlasTextureSize.Z,
						PF_R8,
						FClearValueBinding::None,
						TexCreate_None,
						TexCreate_ShaderResource | TexCreate_UAV | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling,
						false));

					GRenderTargetPool.FindFreeElement(
						RHICmdList,
						VolumeDesc,
						GlobalDistanceFieldData.CoverageAtlasTexture,
						TEXT("GlobalDistanceField.CoverageAtlas")
					);

					bSharedDataReallocated = true;
				}
			}
			else
			{
				if (GlobalDistanceFieldData.CoverageAtlasTexture != nullptr)
				{
					bSharedDataReallocated = true;
				}
				GlobalDistanceFieldData.CoverageAtlasTexture = nullptr;
			}

			GlobalDistanceFieldInfo.PageFreeListAllocatorBuffer = GlobalDistanceFieldData.PageFreeListAllocatorBuffer;
			GlobalDistanceFieldInfo.PageFreeListBuffer = GlobalDistanceFieldData.PageFreeListBuffer;
			GlobalDistanceFieldInfo.PageObjectGridBuffer = GlobalDistanceFieldData.PageObjectGridBuffer;
			GlobalDistanceFieldInfo.PageAtlasTexture = GlobalDistanceFieldData.PageAtlasTexture;
			GlobalDistanceFieldInfo.CoverageAtlasTexture = GlobalDistanceFieldData.CoverageAtlasTexture;
		}

		if(GAOGlobalDistanceFieldCacheMostlyStaticSeparately)
		{
			const FIntVector PageTableTextureResolution = GlobalDistanceField::GetPageTableTextureResolution(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance);
			TRefCountPtr<IPooledRenderTarget>& PageTableTexture = GlobalDistanceFieldData.PageTableCombinedTexture;

			if (!PageTableTexture
					|| PageTableTexture->GetDesc().Extent.X != PageTableTextureResolution.X
					|| PageTableTexture->GetDesc().Extent.Y != PageTableTextureResolution.Y
					|| PageTableTexture->GetDesc().Depth != PageTableTextureResolution.Z)
			{
					FPooledRenderTargetDesc VolumeDesc = FPooledRenderTargetDesc(FPooledRenderTargetDesc::CreateVolumeDesc(
						PageTableTextureResolution.X,
						PageTableTextureResolution.Y,
						PageTableTextureResolution.Z,
						PF_R32_UINT,
						FClearValueBinding::None,
						TexCreate_None,
						TexCreate_ShaderResource | TexCreate_UAV | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling,
						false));

					GRenderTargetPool.FindFreeElement(
						RHICmdList,
						VolumeDesc,
						PageTableTexture,
						TEXT("GlobalDistanceField.PageTableCombined")
					);

					bSharedDataReallocated = true;
			}

			GlobalDistanceFieldInfo.PageTableCombinedTexture = PageTableTexture;
		}

		{
			const int32 ClipmapMipResolution = GlobalDistanceField::GetClipmapMipResolution(bLumenEnabled);
			const FIntVector MipTextureResolution = FIntVector(ClipmapMipResolution, ClipmapMipResolution, ClipmapMipResolution * GlobalDistanceField::GetNumGlobalDistanceFieldClipmaps(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance));
			TRefCountPtr<IPooledRenderTarget>& MipTexture = GlobalDistanceFieldData.MipTexture;

			if (!MipTexture
				|| MipTexture->GetDesc().Extent.X != MipTextureResolution.X
				|| MipTexture->GetDesc().Extent.Y != MipTextureResolution.Y
				|| MipTexture->GetDesc().Depth != MipTextureResolution.Z)
			{
				FPooledRenderTargetDesc VolumeDesc = FPooledRenderTargetDesc(FPooledRenderTargetDesc::CreateVolumeDesc(
					MipTextureResolution.X,
					MipTextureResolution.Y,
					MipTextureResolution.Z,
					PF_R8,
					FClearValueBinding::None,
					TexCreate_None,
					TexCreate_ShaderResource | TexCreate_UAV | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling,
					false));

				GRenderTargetPool.FindFreeElement(
					RHICmdList,
					VolumeDesc,
					MipTexture,
					TEXT("GlobalDistanceField.MipTexture")
				);

				bSharedDataReallocated = true;
			}

			GlobalDistanceFieldInfo.MipTexture = MipTexture;
		}

		for (uint32 CacheType = 0; CacheType < GDF_Num; CacheType++)
		{
			const FIntVector PageTableTextureResolution = GlobalDistanceField::GetPageTableTextureResolution(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance);
			TRefCountPtr<IPooledRenderTarget>& PageTableTexture = GlobalDistanceFieldData.PageTableLayerTextures[CacheType];

			if (CacheType == GDF_Full || GAOGlobalDistanceFieldCacheMostlyStaticSeparately)
			{
				if (!PageTableTexture
					|| PageTableTexture->GetDesc().Extent.X != PageTableTextureResolution.X
					|| PageTableTexture->GetDesc().Extent.Y != PageTableTextureResolution.Y
					|| PageTableTexture->GetDesc().Depth != PageTableTextureResolution.Z)
				{
					FPooledRenderTargetDesc VolumeDesc = FPooledRenderTargetDesc(FPooledRenderTargetDesc::CreateVolumeDesc(
						PageTableTextureResolution.X,
						PageTableTextureResolution.Y,
						PageTableTextureResolution.Z,
						PF_R32_UINT,
						FClearValueBinding::None,
						TexCreate_None,
						TexCreate_ShaderResource | TexCreate_UAV | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling,
						false));

					GRenderTargetPool.FindFreeElement(
						RHICmdList,
						VolumeDesc,
						PageTableTexture,
						CacheType == GDF_MostlyStatic ? TEXT("GlobalDistanceField.PageTableStationaryLayer") : TEXT("GlobalDistanceField.PageTableMovableLayer")
					);

					bSharedDataReallocated = true;
				}
			}

			GlobalDistanceFieldInfo.PageTableLayerTextures[CacheType] = PageTableTexture;
		}

		// Process mesh SDF streaming readback in order to trigger full clipmap update when streaming is done
		for (uint32 CacheType = 0; CacheType < GDF_Num; CacheType++)
		{
			FGlobalDistanceFieldStreamingReadback& StreamingReadback = GlobalDistanceFieldData.StreamingReadback[CacheType];

			FRHIGPUBufferReadback* LatestReadbackBuffer = nullptr;

			// Find latest buffer that is ready
			while (StreamingReadback.ReadbackBuffersNumPending > 0)
			{
				uint32 Index = (StreamingReadback.ReadbackBuffersWriteIndex + StreamingReadback.MaxPendingStreamingReadbackBuffers - StreamingReadback.ReadbackBuffersNumPending) % StreamingReadback.MaxPendingStreamingReadbackBuffers;
				if (StreamingReadback.PendingStreamingReadbackBuffers[Index]->IsReady())
				{
					StreamingReadback.ReadbackBuffersNumPending--;
					LatestReadbackBuffer = StreamingReadback.PendingStreamingReadbackBuffers[Index].Get();
				}
				else
				{
					break;
				}
			}

			// Readback whether the last CullObjectsToClipmap for this clipmap detected Mesh SDFs that have not streamed in (NumMips == 1)
			if (LatestReadbackBuffer)
			{
				const uint32* LatestReadbackBufferPtr = (const uint32*)LatestReadbackBuffer->Lock(GlobalDistanceField::MaxClipmaps * sizeof(uint32));

				for (uint32 ClipmapIndex = 0; ClipmapIndex < GlobalDistanceField::MaxClipmaps; ++ClipmapIndex)
				{
					if (LatestReadbackBufferPtr[ClipmapIndex] != 0)
					{
						// Add a new deferred update for when the streaming is finished
						GlobalDistanceFieldData.DeferredUpdatesForMeshSDFStreaming[CacheType].AddUnique(ClipmapIndex);
					}
				}

				LatestReadbackBuffer->Unlock();
			}
		}

		for (int32 ClipmapIndex = 0; ClipmapIndex < NumClipmaps; ClipmapIndex++)
		{
			FGlobalDistanceFieldClipmapState& ClipmapViewState = GlobalDistanceFieldData.ClipmapState[ClipmapIndex];

			const int32 ClipmapResolution = GlobalDistanceField::GetClipmapResolution(bLumenEnabled);
			const float ClipmapExtent = GlobalDistanceField::GetClipmapExtent(ClipmapIndex, Scene, bLumenEnabled);
			const float ClipmapVoxelSize = (2.0f * ClipmapExtent) / ClipmapResolution;
			const float ClipmapPageSize = GGlobalDistanceFieldPageResolution * ClipmapVoxelSize;
			const float ClipmapInfluenceRadius = GGlobalDistanceFieldInfluenceRangeInVoxels * ClipmapVoxelSize;
			const int64 ClipmapSizeInPages = GlobalDistanceField::GetPageTableClipmapResolution(bLumenEnabled);

			// Accumulate primitive modifications in the viewstate in case we don't update the clipmap this frame
			for (uint32 CacheType = 0; CacheType < GDF_Num; CacheType++)
			{
				const uint32 DestCacheType = GAOGlobalDistanceFieldCacheMostlyStaticSeparately ? CacheType : GDF_Full;
				ClipmapViewState.Cache[DestCacheType].PrimitiveModifiedBounds.Append(Scene->DistanceFieldSceneData.PrimitiveModifiedBounds[CacheType]);
			}

			const bool bForceFullUpdate = bSharedDataReallocated
				|| bForceFullUpdateMGPU
				|| !GlobalDistanceFieldData.bInitializedOrigins
				// Detect when max occlusion distance has changed
				|| ClipmapViewState.CachedClipmapExtent != ClipmapExtent
				|| ClipmapViewState.CacheMostlyStaticSeparately != GAOGlobalDistanceFieldCacheMostlyStaticSeparately
				|| ClipmapViewState.LastUsedSceneDataForFullUpdate != &Scene->DistanceFieldSceneData
				|| GAOGlobalDistanceFieldForceFullUpdate
				|| GAOGlobalDistanceFieldForceUpdateOnce
				|| GDFReadbackRequest != nullptr;

			const bool bUpdateRequested = GAOUpdateGlobalDistanceField != 0 && ShouldUpdateClipmapThisFrame(ClipmapIndex, NumClipmaps, GlobalDistanceFieldData.UpdateIndex);

			if (bUpdateRequested && !bUpdatedThisFrame)
			{
				NumClipmapUpdateRequests++;
			}

			if ((bUpdateRequested || bForceFullUpdate) && !bUpdatedThisFrame)
			{
				const FVector GlobalDistanceFieldViewOrigin = GetGlobalDistanceFieldViewOrigin(View, ClipmapIndex, bLumenEnabled);

				// Snap to the global distance field page's size
				FInt64Vector PageGridCenter;
				PageGridCenter.X = FMath::RoundToInt(GlobalDistanceFieldViewOrigin.X / ClipmapPageSize);
				PageGridCenter.Y = FMath::RoundToInt(GlobalDistanceFieldViewOrigin.Y / ClipmapPageSize);
				PageGridCenter.Z = FMath::RoundToInt(GlobalDistanceFieldViewOrigin.Z / ClipmapPageSize);

				const FVector SnappedCenter = FVector(PageGridCenter) * ClipmapPageSize;
				const FBox ClipmapBounds(SnappedCenter - ClipmapExtent, SnappedCenter + ClipmapExtent);

				const bool bUsePartialUpdates = GAOGlobalDistanceFieldPartialUpdates && !bForceFullUpdate;

				if (!bUsePartialUpdates)
				{
					// Store the location of the full update
					ClipmapViewState.FullUpdateOriginInPages = PageGridCenter;
					GlobalDistanceFieldData.bInitializedOrigins = true;
					GlobalDistanceFieldData.bPendingReset = true;
					ClipmapViewState.LastUsedSceneDataForFullUpdate = &Scene->DistanceFieldSceneData;
				}

				const FGlobalDFCacheType StartCacheType = GAOGlobalDistanceFieldCacheMostlyStaticSeparately ? GDF_MostlyStatic : GDF_Full;

				for (uint32 CacheType = StartCacheType; CacheType < GDF_Num; CacheType++)
				{
					FGlobalDistanceFieldClipmap& Clipmap = *(CacheType == GDF_MostlyStatic
						? &GlobalDistanceFieldInfo.MostlyStaticClipmaps[ClipmapIndex]
						: &GlobalDistanceFieldInfo.Clipmaps[ClipmapIndex]);

					const TArray<FBox>& PrimitiveModifiedBounds = ClipmapViewState.Cache[CacheType].PrimitiveModifiedBounds;

					uint32 NumCulledPrimitiveModifiedBounds = 0;

					Clipmap.UpdateBounds.Empty(PrimitiveModifiedBounds.Num() / 2);

					for (int32 BoundsIndex = 0; BoundsIndex < PrimitiveModifiedBounds.Num(); BoundsIndex++)
					{
						const FBox ModifiedBounds = PrimitiveModifiedBounds[BoundsIndex];

						if (ModifiedBounds.ComputeSquaredDistanceToBox(ClipmapBounds) < ClipmapInfluenceRadius * ClipmapInfluenceRadius)
						{
							++NumCulledPrimitiveModifiedBounds;

							Clipmap.UpdateBounds.Add(FClipmapUpdateBounds(ModifiedBounds.GetCenter(), ModifiedBounds.GetExtent(), true));
							
							if (GGlobalDistanceFieldDebugDrawModifiedPrimitives)
							{
								const uint8 MarkerHue = ((ClipmapIndex * 10 + BoundsIndex) * 10) & 0xFF;
								const uint8 MarkerSaturation = 0xFF;
								const uint8 MarkerValue = 0xFF;

								FLinearColor MarkerColor = FLinearColor::MakeFromHSV8(MarkerHue, MarkerSaturation, MarkerValue);
								MarkerColor.A = 0.5f;
	
								DrawWireBox(&ViewPDI, ModifiedBounds, MarkerColor, SDPG_World);
							}
						}
					}

					if (bUsePartialUpdates)
					{
						FInt64Vector MovementInPages = PageGridCenter - ClipmapViewState.LastPartialUpdateOriginInPages;

						if (GGlobalDistanceFieldDebugForceMovementUpdate != 0)
						{
							MovementInPages = FInt64Vector(GGlobalDistanceFieldDebugForceMovementUpdate, GGlobalDistanceFieldDebugForceMovementUpdate, GGlobalDistanceFieldDebugForceMovementUpdate);
						}

						if (CacheType == GDF_MostlyStatic || !GAOGlobalDistanceFieldCacheMostlyStaticSeparately)
						{
							// Add an update region for each potential axis of camera movement
							AddUpdateBoundsForAxis(MovementInPages, ClipmapBounds, ClipmapPageSize, 0, Clipmap.UpdateBounds);
							AddUpdateBoundsForAxis(MovementInPages, ClipmapBounds, ClipmapPageSize, 1, Clipmap.UpdateBounds);
							AddUpdateBoundsForAxis(MovementInPages, ClipmapBounds, ClipmapPageSize, 2, Clipmap.UpdateBounds);
						}
						else
						{
							// Inherit from parent
							Clipmap.UpdateBounds.Append(GlobalDistanceFieldInfo.MostlyStaticClipmaps[ClipmapIndex].UpdateBounds);
						}
					}

					// Only use partial updates with small numbers of primitive modifications
					bool bUsePartialUpdatesForUpdateBounds = bUsePartialUpdates && NumCulledPrimitiveModifiedBounds < 1024;

					if (!bUsePartialUpdatesForUpdateBounds)
					{
						Clipmap.UpdateBounds.Reset();
						Clipmap.UpdateBounds.Add(FClipmapUpdateBounds(ClipmapBounds.GetCenter(), ClipmapBounds.GetExtent(), false));
						Clipmap.FullRecaptureReason = EGlobalSDFFullRecaptureReason::TooManyUpdateBounds;
					}

					// Check if the clipmap intersects with a pending update region
					bool bHasPendingStreaming = false;
					for (const FBox& HeightfieldBox : PendingStreamingHeightfieldBoxes)
					{
						if (ClipmapBounds.Intersect(HeightfieldBox))
						{
							bHasPendingStreaming = true;
							break;
						}
					}

					// If some of the height fields has pending streaming regions, postpone a full update.
					if (bHasPendingStreaming && GAOGlobalDistanceFieldForceRecacheForStreaming == 0)
					{
						// Mark a pending update for this height field. It will get processed when all pending texture streaming affecting it will be completed.
						GlobalDistanceFieldData.DeferredUpdates[CacheType].AddUnique(ClipmapIndex);
					}
					else if (GlobalDistanceFieldData.DeferredUpdates[CacheType].Remove(ClipmapIndex) > 0 || GAOGlobalDistanceFieldForceRecacheForStreaming != 0)
					{
						// Push full update
						Clipmap.UpdateBounds.Reset();
						Clipmap.UpdateBounds.Add(FClipmapUpdateBounds(ClipmapBounds.GetCenter(), ClipmapBounds.GetExtent(), false));
						Clipmap.FullRecaptureReason = EGlobalSDFFullRecaptureReason::HeightfieldStreaming;
					}

					// Push full update when mesh SDF streaming is done
					if (!Scene->DistanceFieldSceneData.HasPendingStreaming() && GlobalDistanceFieldData.DeferredUpdatesForMeshSDFStreaming[CacheType].Remove(ClipmapIndex))
					{
						// Push full update
						Clipmap.UpdateBounds.Reset();
						Clipmap.UpdateBounds.Add(FClipmapUpdateBounds(ClipmapBounds.GetCenter(), ClipmapBounds.GetExtent(), false));
						Clipmap.FullRecaptureReason = EGlobalSDFFullRecaptureReason::MeshSDFStreaming;
					}

					ClipmapViewState.Cache[CacheType].PrimitiveModifiedBounds.Empty(DistanceField::MinPrimitiveModifiedBoundsAllocation);
				}

				ClipmapViewState.LastPartialUpdateOriginInPages = PageGridCenter;
			}

			const FVector SnappedCenter = FVector(ClipmapViewState.LastPartialUpdateOriginInPages) * ClipmapPageSize;
			const FGlobalDFCacheType StartCacheType = GAOGlobalDistanceFieldCacheMostlyStaticSeparately ? GDF_MostlyStatic : GDF_Full;

			for (uint32 CacheType = StartCacheType; CacheType < GDF_Num; CacheType++)
			{
				FGlobalDistanceFieldClipmap& Clipmap = *(CacheType == GDF_MostlyStatic 
					? &GlobalDistanceFieldInfo.MostlyStaticClipmaps[ClipmapIndex] 
					: &GlobalDistanceFieldInfo.Clipmaps[ClipmapIndex]);

				// Setup clipmap properties from view state exclusively, so we can skip updating on some frames
				Clipmap.Bounds = FBox(SnappedCenter - ClipmapExtent, SnappedCenter + ClipmapExtent);

				// Scroll offset so the contents of the global distance field don't have to be moved as the camera moves around, only updated in slabs
				FInt64Vector ScrollOffsetInPages = ClipmapViewState.LastPartialUpdateOriginInPages - ClipmapViewState.FullUpdateOriginInPages;
				ScrollOffsetInPages %= ClipmapSizeInPages; // prevent floating point precision issues
				Clipmap.ScrollOffset = FVector(ScrollOffsetInPages) * ClipmapPageSize;
			}

			ClipmapViewState.CachedClipmapCenter = (FVector3f)SnappedCenter;
			ClipmapViewState.CachedClipmapExtent = ClipmapExtent;
			ClipmapViewState.CacheClipmapInfluenceRadius = ClipmapInfluenceRadius;
			ClipmapViewState.CacheMostlyStaticSeparately = GAOGlobalDistanceFieldCacheMostlyStaticSeparately;
		}
		GAOGlobalDistanceFieldForceUpdateOnce = 0;

		ensureMsgf(!GAOGlobalDistanceFieldStaggeredUpdates || NumClipmapUpdateRequests <= GetNumClipmapUpdatesPerFrame(), TEXT("ShouldUpdateClipmapThisFrame needs to be adjusted for the NumClipmaps to even out the work distribution"));
	}
	else
	{
		for (int32 ClipmapIndex = 0; ClipmapIndex < NumClipmaps; ClipmapIndex++)
		{
			const FGlobalDFCacheType StartCacheType = GAOGlobalDistanceFieldCacheMostlyStaticSeparately ? GDF_MostlyStatic : GDF_Full;

			for (uint32 CacheType = StartCacheType; CacheType < GDF_Num; CacheType++)
			{
				FGlobalDistanceFieldClipmap& Clipmap = *(CacheType == GDF_MostlyStatic
					? &GlobalDistanceFieldInfo.MostlyStaticClipmaps[ClipmapIndex]
					: &GlobalDistanceFieldInfo.Clipmaps[ClipmapIndex]);

				Clipmap.ScrollOffset = FVector(0);

				const int32 ClipmapResolution = GlobalDistanceField::GetClipmapResolution(bLumenEnabled);
				const float Extent = GlobalDistanceField::GetClipmapExtent(ClipmapIndex, Scene, bLumenEnabled);
				const float ClipmapVoxelSize = (2.0f * Extent) / ClipmapResolution;
				const float ClipmapPageSize = GGlobalDistanceFieldPageResolution * ClipmapVoxelSize;
				const FVector GlobalDistanceFieldViewOrigin = GetGlobalDistanceFieldViewOrigin(View, ClipmapIndex, bLumenEnabled);

				FIntVector PageGridCenter;
				PageGridCenter.X = FMath::RoundToInt(GlobalDistanceFieldViewOrigin.X / ClipmapPageSize);
				PageGridCenter.Y = FMath::RoundToInt(GlobalDistanceFieldViewOrigin.Y / ClipmapPageSize);
				PageGridCenter.Z = FMath::RoundToInt(GlobalDistanceFieldViewOrigin.Z / ClipmapPageSize);

				FVector Center = FVector(PageGridCenter) * ClipmapPageSize;

				FBox ClipmapBounds(Center - Extent, Center + Extent);
				Clipmap.Bounds = ClipmapBounds;

				Clipmap.UpdateBounds.Reset();
				Clipmap.UpdateBounds.Add(FClipmapUpdateBounds(ClipmapBounds.GetCenter(), ClipmapBounds.GetExtent(), false));
				Clipmap.FullRecaptureReason = EGlobalSDFFullRecaptureReason::NoViewState;
			}
		}
	}

	GlobalDistanceFieldInfo.UpdateParameterData(MaxOcclusionDistance, bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance, View.ViewMatrices.GetPreViewTranslation());
}

void FViewInfo::SetupDefaultGlobalDistanceFieldUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters) const
{
	if (GlobalDistanceFieldInfo.bInitialized)
	{
		return;
	}

	// Initialize global distance field members to defaults, because View.GlobalDistanceFieldInfo is not valid yet
	for (int32 Index = 0; Index < GlobalDistanceField::MaxClipmaps; Index++)
	{
		ViewUniformShaderParameters.GlobalVolumeTranslatedCenterAndExtent[Index] = FVector4f(0);
		ViewUniformShaderParameters.GlobalVolumeTranslatedWorldToUVAddAndMul[Index] = FVector4f(0);
		ViewUniformShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVScale[Index] = FVector4f(0);
		ViewUniformShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVBias[Index] = FVector4f(0);
	}
	ViewUniformShaderParameters.GlobalDistanceFieldMipFactor = 1.0f;
	ViewUniformShaderParameters.GlobalDistanceFieldMipTransition = 0.0f;
	ViewUniformShaderParameters.GlobalDistanceFieldClipmapSizeInPages = 1;
	ViewUniformShaderParameters.GlobalDistanceFieldInvPageAtlasSize = FVector3f::OneVector;
	ViewUniformShaderParameters.GlobalDistanceFieldInvCoverageAtlasSize = FVector3f::OneVector;
	ViewUniformShaderParameters.GlobalVolumeDimension = 0.0f;
	ViewUniformShaderParameters.GlobalVolumeTexelSize = 0.0f;
	ViewUniformShaderParameters.MaxGlobalDFAOConeDistance = 0.0f;
	ViewUniformShaderParameters.NumGlobalSDFClipmaps = 0;

	ViewUniformShaderParameters.GlobalDistanceFieldPageAtlasTexture = OrBlack3DIfNull(GBlackVolumeTexture->TextureRHI.GetReference());
	ViewUniformShaderParameters.GlobalDistanceFieldCoverageAtlasTexture = OrBlack3DIfNull(GBlackVolumeTexture->TextureRHI.GetReference());
	ViewUniformShaderParameters.GlobalDistanceFieldPageTableTexture = OrBlack3DUintIfNull(GBlackUintVolumeTexture->TextureRHI.GetReference());
	ViewUniformShaderParameters.GlobalDistanceFieldMipTexture = OrBlack3DIfNull(GBlackVolumeTexture->TextureRHI.GetReference());
}

void FViewInfo::SetupGlobalDistanceFieldUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters) const
{
	check(GlobalDistanceFieldInfo.bInitialized);

	for (int32 Index = 0; Index < GlobalDistanceField::MaxClipmaps; Index++)
	{
		ViewUniformShaderParameters.GlobalVolumeTranslatedCenterAndExtent[Index] = GlobalDistanceFieldInfo.ParameterData.TranslatedCenterAndExtent[Index];
		ViewUniformShaderParameters.GlobalVolumeTranslatedWorldToUVAddAndMul[Index] = GlobalDistanceFieldInfo.ParameterData.TranslatedWorldToUVAddAndMul[Index];
		ViewUniformShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVScale[Index] = GlobalDistanceFieldInfo.ParameterData.MipTranslatedWorldToUVScale[Index];
		ViewUniformShaderParameters.GlobalDistanceFieldMipTranslatedWorldToUVBias[Index] = GlobalDistanceFieldInfo.ParameterData.MipTranslatedWorldToUVBias[Index];
	}
	ViewUniformShaderParameters.GlobalDistanceFieldMipFactor = GlobalDistanceFieldInfo.ParameterData.MipFactor;
	ViewUniformShaderParameters.GlobalDistanceFieldMipTransition = GlobalDistanceFieldInfo.ParameterData.MipTransition;
	ViewUniformShaderParameters.GlobalDistanceFieldClipmapSizeInPages = GlobalDistanceFieldInfo.ParameterData.ClipmapSizeInPages;
	ViewUniformShaderParameters.GlobalDistanceFieldInvPageAtlasSize = (FVector3f)GlobalDistanceFieldInfo.ParameterData.InvPageAtlasSize;
	ViewUniformShaderParameters.GlobalDistanceFieldInvCoverageAtlasSize = (FVector3f)GlobalDistanceFieldInfo.ParameterData.InvCoverageAtlasSize;
	ViewUniformShaderParameters.GlobalVolumeDimension = GlobalDistanceFieldInfo.ParameterData.GlobalDFResolution;
	ViewUniformShaderParameters.GlobalVolumeTexelSize = 1.0f / GlobalDistanceFieldInfo.ParameterData.GlobalDFResolution;
	ViewUniformShaderParameters.MaxGlobalDFAOConeDistance = GlobalDistanceFieldInfo.ParameterData.MaxDFAOConeDistance;
	ViewUniformShaderParameters.NumGlobalSDFClipmaps = GlobalDistanceFieldInfo.ParameterData.NumGlobalSDFClipmaps;

	ViewUniformShaderParameters.GlobalDistanceFieldPageAtlasTexture = OrBlack3DIfNull(GlobalDistanceFieldInfo.ParameterData.PageAtlasTexture);
	ViewUniformShaderParameters.GlobalDistanceFieldCoverageAtlasTexture = OrBlack3DIfNull(GlobalDistanceFieldInfo.ParameterData.CoverageAtlasTexture);
	ViewUniformShaderParameters.GlobalDistanceFieldPageTableTexture = OrBlack3DUintIfNull(GlobalDistanceFieldInfo.ParameterData.PageTableTexture);
	ViewUniformShaderParameters.GlobalDistanceFieldMipTexture = OrBlack3DIfNull(GlobalDistanceFieldInfo.ParameterData.MipTexture);

	ViewUniformShaderParameters.CoveredExpandSurfaceScale = GLumenSceneGlobalSDFCoveredExpandSurfaceScale;
	ViewUniformShaderParameters.NotCoveredExpandSurfaceScale = GLumenSceneGlobalSDFNotCoveredExpandSurfaceScale;
	ViewUniformShaderParameters.NotCoveredMinStepScale = GLumenSceneGlobalSDFNotCoveredMinStepScale;
	ViewUniformShaderParameters.DitheredTransparencyStepThreshold = GLumenSceneGlobalSDFDitheredTransparencyStepThreshold;
	ViewUniformShaderParameters.DitheredTransparencyTraceThreshold = GLumenSceneGlobalSDFDitheredTransparencyTraceThreshold;
}

void ReadbackDistanceFieldClipmap(FRHICommandListImmediate& RHICmdList, FGlobalDistanceFieldInfo& GlobalDistanceFieldInfo)
{
	FGlobalDistanceFieldReadback* Readback = GDFReadbackRequest;
	GDFReadbackRequest = nullptr;

	//FGlobalDistanceFieldClipmap& ClipMap = GlobalDistanceFieldInfo.Clipmaps[0];
	//FTextureRHIRef SourceTexture = ClipMap.RenderTarget->GetRHI();
	//FIntVector Size = SourceTexture->GetSizeXYZ();
	
	//RHICmdList.Read3DSurfaceFloatData(SourceTexture, FIntRect(0, 0, Size.X, Size.Y), FIntPoint(0, Size.Z), Readback->ReadbackData);
	//Readback->Bounds = ClipMap.Bounds;
	//Readback->Size = Size;

	ensureMsgf(false, TEXT("#todo: Global DF readback requires a rewrite as global distance field is no longer stored in a continuous memory"));

	Readback->Bounds = FBox(FVector(0.0f), FVector(0.0f));
	Readback->Size = FIntVector(0);
	
	// Fire the callback to notify that the request is complete
	DECLARE_CYCLE_STAT(TEXT("FSimpleDelegateGraphTask.DistanceFieldReadbackDelegate"), STAT_FSimpleDelegateGraphTask_DistanceFieldReadbackDelegate, STATGROUP_TaskGraphTasks);
	FSimpleDelegateGraphTask::CreateAndDispatchWhenReady(
		Readback->ReadbackComplete,
		GET_STATID(STAT_FSimpleDelegateGraphTask_DistanceFieldReadbackDelegate),
		nullptr,
		Readback->CallbackThread
		);	
}

class FCullObjectsToClipmapCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCullObjectsToClipmapCS);
	SHADER_USE_PARAMETER_STRUCT(FCullObjectsToClipmapCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWObjectIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWObjectIndexNumBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWHasPendingStreaming)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlasParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, PackedClipmapBuffer)
		SHADER_PARAMETER(uint32, AcceptOftenMovingObjectsOnly)
		SHADER_PARAMETER(uint32, NumPackedClipmaps)
		SHADER_PARAMETER(uint32, ObjectIndexBufferStride)
		SHADER_PARAMETER(FVector3f, PreViewTranslationHigh)
		SHADER_PARAMETER(FVector3f, PreViewTranslationLow)
	END_SHADER_PARAMETER_STRUCT()

	class FReadbackHasPendingStreaming : SHADER_PERMUTATION_BOOL("READBACK_HAS_PENDING_STREAMING");
	using FPermutationDomain = TShaderPermutationDomain<FReadbackHasPendingStreaming>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 64;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("CULLOBJECTS_THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FCullObjectsToClipmapCS, "/Engine/Private/DistanceField/GlobalDistanceField.usf", "CullObjectsToClipmapCS", SF_Compute);

class FClearIndirectArgBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearIndirectArgBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FClearIndirectArgBufferCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWPageUpdateIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWCullGridUpdateIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWPageComposeIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWBuildObjectGridIndirectArgBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 64;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FClearIndirectArgBufferCS, "/Engine/Private/DistanceField/GlobalDistanceField.usf", "ClearIndirectArgBufferCS", SF_Compute);

class FBuildGridTilesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildGridTilesCS);
	SHADER_USE_PARAMETER_STRUCT(FBuildGridTilesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageTileBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCullGridTileBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCullGridIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, UpdateBoundsBuffer)
		SHADER_PARAMETER(uint32, NumUpdateBounds)
		SHADER_PARAMETER(float, InfluenceRadiusSq)
		SHADER_PARAMETER(FIntVector, PageGridResolution)
		SHADER_PARAMETER(FVector3f, PageGridCoordToTranslatedWorldCenterScale)
		SHADER_PARAMETER(FVector3f, PageGridCoordToTranslatedWorldCenterBias)
		SHADER_PARAMETER(FVector3f, PageGridTileWorldExtent)
		SHADER_PARAMETER(FIntVector, CullGridResolution)
		SHADER_PARAMETER(FVector3f, CullGridCoordToTranslatedWorldCenterScale)
		SHADER_PARAMETER(FVector3f, CullGridCoordToTranslatedWorldCenterBias)
		SHADER_PARAMETER(FVector3f, CullGridTileWorldExtent)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		static_assert(GlobalDistanceField::CullGridFactor == 4, "Shader is hard coded for CullGridFactor=4");
		return 4;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FBuildGridTilesCS, "/Engine/Private/DistanceField/GlobalDistanceField.usf", "BuildGridTilesCS", SF_Compute);

class FCullObjectsToGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCullObjectsToGridCS);
	SHADER_USE_PARAMETER_STRUCT(FCullObjectsToGridCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCullGridAllocator)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCullGridObjectHeader)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCullGridObjectArray)
		RDG_BUFFER_ACCESS(CullGridIndirectArgBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CullGridTileBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ObjectIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ObjectIndexNumBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER(FIntVector, CullGridResolution)
		SHADER_PARAMETER(FVector3f, CullGridCoordToTranslatedWorldCenterScale)
		SHADER_PARAMETER(FVector3f, CullGridCoordToTranslatedWorldCenterBias)
		SHADER_PARAMETER(FVector3f, CullTileWorldExtent)
		SHADER_PARAMETER(float, InfluenceRadiusSq)
		SHADER_PARAMETER(FVector3f, PreViewTranslationHigh)
		SHADER_PARAMETER(FVector3f, PreViewTranslationLow)
		SHADER_PARAMETER(uint32, PackedClipmapIndex)
		SHADER_PARAMETER(uint32, ObjectIndexBufferStride)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCullObjectsToGridCS, "/Engine/Private/DistanceField/GlobalDistanceField.usf", "CullObjectsToGridCS", SF_Compute);

BEGIN_SHADER_PARAMETER_STRUCT(FGlobalDistanceFieldUpdateParameters, )
	SHADER_PARAMETER(float, ClipmapVoxelExtent)
	SHADER_PARAMETER(float, InfluenceRadius)
	SHADER_PARAMETER(float, InfluenceRadiusSq)
	SHADER_PARAMETER(FIntVector, CullGridResolution)
	SHADER_PARAMETER(FVector3f, GlobalDistanceFieldInvPageAtlasSize)
	SHADER_PARAMETER(FVector3f, InvPageGridResolution)
	SHADER_PARAMETER(FIntVector, PageGridResolution)
	SHADER_PARAMETER(FIntVector, ClipmapResolution)
	SHADER_PARAMETER(FVector3f, PageCoordToVoxelTranslatedCenterScale)
	SHADER_PARAMETER(FVector3f, PageCoordToVoxelTranslatedCenterBias)
	SHADER_PARAMETER(FVector3f, PageCoordToPageTranslatedWorldCenterScale)
	SHADER_PARAMETER(FVector3f, PageCoordToPageTranslatedWorldCenterBias)
	SHADER_PARAMETER(FVector4f, ClipmapVolumeTranslatedWorldToUVAddAndMul)
	SHADER_PARAMETER(FVector3f, ComposeTileWorldExtent)
	SHADER_PARAMETER(FVector3f, ClipmapMinBounds)
	SHADER_PARAMETER(uint32, PageTableClipmapOffsetZ)
	SHADER_PARAMETER(FVector3f, PreViewTranslationHigh)
	SHADER_PARAMETER(FVector3f, PreViewTranslationLow)
END_SHADER_PARAMETER_STRUCT()

class FCompositeObjectsIntoObjectGridPagesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositeObjectsIntoObjectGridPagesCS);
	SHADER_USE_PARAMETER_STRUCT(FCompositeObjectsIntoObjectGridPagesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint4>, RWPageObjectGridBuffer)
		RDG_BUFFER_ACCESS(IndirectArgBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<uint>, PageTableLayerTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<uint>, ParentPageTableLayerTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGlobalDistanceFieldUpdateParameters, GlobalDistanceFieldUpdateParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ComposeTileBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CullGridObjectHeader)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CullGridObjectArray)
	END_SHADER_PARAMETER_STRUCT()

	class FComposeParentDistanceField : SHADER_PERMUTATION_BOOL("COMPOSE_PARENT_DISTANCE_FIELD");
	class FProcessDistanceFields : SHADER_PERMUTATION_BOOL("PROCESS_DISTANCE_FIELDS");
	class FOffsetDataStructure : SHADER_PERMUTATION_INT("OFFSET_DATA_STRUCT", 3);
	using FPermutationDomain = TShaderPermutationDomain<FComposeParentDistanceField, FProcessDistanceFields, FOffsetDataStructure>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(4, 4, 4);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize().X);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCompositeObjectsIntoObjectGridPagesCS, "/Engine/Private/DistanceField/GlobalDistanceFieldCompositeObjects.usf", "CompositeObjectsIntoObjectGridPagesCS", SF_Compute);

class FCompositeObjectsIntoPagesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositeObjectsIntoPagesCS);
	SHADER_USE_PARAMETER_STRUCT(FCompositeObjectsIntoPagesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<UNORM float>, RWPageAtlasTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<UNORM float>, RWCoverageAtlasTexture)
		RDG_BUFFER_ACCESS(ComposeIndirectArgBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ComposeTileBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, HeightfieldMarkedPageBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, RWPageTableCombinedTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<uint>, ParentPageTableLayerTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CullGridObjectHeader)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CullGridObjectArray)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGlobalDistanceFieldUpdateParameters, GlobalDistanceFieldUpdateParameters)
	END_SHADER_PARAMETER_STRUCT()

	class FComposeParentDistanceField : SHADER_PERMUTATION_BOOL("COMPOSE_PARENT_DISTANCE_FIELD");
	class FProcessDistanceFields : SHADER_PERMUTATION_BOOL("PROCESS_DISTANCE_FIELDS");
	class FCompositeCoverageAtlas : SHADER_PERMUTATION_BOOL("COMPOSITE_COVERAGE_ATLAS");
	class FOffsetDataStructure : SHADER_PERMUTATION_INT("OFFSET_DATA_STRUCT", 3);
	using FPermutationDomain = TShaderPermutationDomain<FComposeParentDistanceField, FProcessDistanceFields, FCompositeCoverageAtlas, FOffsetDataStructure>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(4, 4, 4);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize().X);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCompositeObjectsIntoPagesCS, "/Engine/Private/DistanceField/GlobalDistanceFieldCompositeObjects.usf", "CompositeObjectsIntoPagesCS", SF_Compute);

class FInitPageFreeListCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInitPageFreeListCS);
	SHADER_USE_PARAMETER_STRUCT(FInitPageFreeListCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageFreeListBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int, RWPageFreeListAllocatorBuffer)
		SHADER_PARAMETER(uint32, GlobalDistanceFieldMaxPageNum)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static uint32 GetGroupSize()
	{
		return 64;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FInitPageFreeListCS, "/Engine/Private/DistanceField/GlobalDistanceField.usf", "InitPageFreeListCS", SF_Compute);

class FAllocatePagesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAllocatePagesCS);
	SHADER_USE_PARAMETER_STRUCT(FAllocatePagesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		RDG_BUFFER_ACCESS(PageUpdateIndirectArgBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PageUpdateTileBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, MarkedHeightfieldPageBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, RWPageTableCombinedTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, RWPageTableLayerTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int>, RWPageFreeListAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PageFreeListBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageFreeListReturnAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageFreeListReturnBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageComposeTileBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWPageComposeIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWBuildObjectGridIndirectArgBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<uint>, ParentPageTableLayerTexture)
		SHADER_PARAMETER(FVector3f, InvPageGridResolution)
		SHADER_PARAMETER(FIntVector, PageGridResolution)
		SHADER_PARAMETER(uint32, GlobalDistanceFieldMaxPageNum)
		SHADER_PARAMETER(uint32, PageTableClipmapOffsetZ)
		SHADER_PARAMETER(FVector3f, PageWorldExtent)
		SHADER_PARAMETER(float, PageWorldRadius)
		SHADER_PARAMETER(float, ClipmapInfluenceRadius)
		SHADER_PARAMETER(FVector3f, PageCoordToPageTranslatedWorldCenterScale)
		SHADER_PARAMETER(FVector3f, PageCoordToPageTranslatedWorldCenterBias)
		SHADER_PARAMETER(FVector4f, ClipmapVolumeTranslatedWorldToUVAddAndMul)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CullGridObjectHeader)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CullGridObjectArray)
		SHADER_PARAMETER(FIntVector, CullGridResolution)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldObjectBufferParameters, DistanceFieldObjectBuffers)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDistanceFieldAtlasParameters, DistanceFieldAtlas)
		SHADER_PARAMETER(FVector3f, PreViewTranslationHigh)
		SHADER_PARAMETER(FVector3f, PreViewTranslationLow)
	END_SHADER_PARAMETER_STRUCT()

	class FProcessDistanceFields : SHADER_PERMUTATION_BOOL("PROCESS_DISTANCE_FIELDS");
	class FMarkedHeightfieldPageBuffer : SHADER_PERMUTATION_BOOL("MARKED_HEIGHTFIELD_PAGE_BUFFER");
	class FComposeParentDistanceField : SHADER_PERMUTATION_BOOL("COMPOSE_PARENT_DISTANCE_FIELD");
	class FOffsetDataStructure : SHADER_PERMUTATION_INT("OFFSET_DATA_STRUCT", 3);
	using FPermutationDomain = TShaderPermutationDomain<FProcessDistanceFields, FMarkedHeightfieldPageBuffer, FComposeParentDistanceField, FOffsetDataStructure>;
	
	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FComposeParentDistanceField>())
		{
			PermutationVector.Set<FMarkedHeightfieldPageBuffer>(false);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}

		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(64, 1, 1);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GetGroupSize().X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GetGroupSize().Y);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), GetGroupSize().Z);
	}
};

IMPLEMENT_GLOBAL_SHADER(FAllocatePagesCS, "/Engine/Private/DistanceField/GlobalDistanceField.usf", "AllocatePagesCS", SF_Compute);

class FPageFreeListReturnIndirectArgBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPageFreeListReturnIndirectArgBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FPageFreeListReturnIndirectArgBufferCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWFreeListReturnIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int, RWPageFreeListAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PageFreeListReturnAllocatorBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FPageFreeListReturnIndirectArgBufferCS, "/Engine/Private/DistanceField/GlobalDistanceField.usf", "PageFreeListReturnIndirectArgBufferCS", SF_Compute);

class FPageFreeListReturnCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPageFreeListReturnCS);
	SHADER_USE_PARAMETER_STRUCT(FPageFreeListReturnCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		RDG_BUFFER_ACCESS(FreeListReturnIndirectArgBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int, RWPageFreeListAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageFreeListBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PageFreeListReturnAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PageFreeListReturnBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static uint32 GetGroupSize()
	{
		return 64;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GetGroupSize());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FPageFreeListReturnCS, "/Engine/Private/DistanceField/GlobalDistanceField.usf", "PageFreeListReturnCS", SF_Compute);

class FPropagateMipDistanceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPropagateMipDistanceCS);
	SHADER_USE_PARAMETER_STRUCT(FPropagateMipDistanceCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, RWMipTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float>, PrevMipTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<uint>, PageTableTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float>, PageAtlasTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistanceFieldSampler)
		SHADER_PARAMETER(FVector3f, GlobalDistanceFieldInvPageAtlasSize)
		SHADER_PARAMETER(uint32, GlobalDistanceFieldClipmapSizeInPages)
		SHADER_PARAMETER(uint32, ClipmapMipResolution)
		SHADER_PARAMETER(float, OneOverClipmapMipResolution)
		SHADER_PARAMETER(uint32, ClipmapIndex)
		SHADER_PARAMETER(uint32, PrevClipmapOffsetZ)
		SHADER_PARAMETER(uint32, ClipmapOffsetZ)
		SHADER_PARAMETER(FVector3f, ClipmapUVScrollOffset)
		SHADER_PARAMETER(float, CoarseDistanceFieldValueScale)
		SHADER_PARAMETER(float, CoarseDistanceFieldValueBias)
	END_SHADER_PARAMETER_STRUCT()

	class FReadPages : SHADER_PERMUTATION_BOOL("READ_PAGES");
	using FPermutationDomain = TShaderPermutationDomain<FReadPages>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(4, 4, 4);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GetGroupSize().X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GetGroupSize().Y);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), GetGroupSize().Z);
	}
};

IMPLEMENT_GLOBAL_SHADER(FPropagateMipDistanceCS, "/Engine/Private/DistanceField/GlobalDistanceFieldMip.usf", "PropagateMipDistanceCS", SF_Compute);

class FGlobalDistanceFieldAccumulateUpdatedPagesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGlobalDistanceFieldAccumulateUpdatedPagesCS)
	SHADER_USE_PARAMETER_STRUCT(FGlobalDistanceFieldAccumulateUpdatedPagesCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageStatsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PageComposeIndirectArgBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGlobalDistanceFieldAccumulateUpdatedPagesCS, "/Engine/Private/DistanceField/GlobalDistanceFieldDebug.usf", "GlobalDistanceFieldAccumulateUpdatedPagesCS", SF_Compute);

class FGlobalDistanceFieldPageStatsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGlobalDistanceFieldPageStatsCS)
	SHADER_USE_PARAMETER_STRUCT(FGlobalDistanceFieldPageStatsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageStatsBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<uint>, PageTableCombinedTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<float>, PageAtlasTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistanceFieldSampler)
		SHADER_PARAMETER(uint32, ClipmapSizeInPages)
		SHADER_PARAMETER(uint32, ClipmapIndex)
		SHADER_PARAMETER(FVector3f, InvPageAtlasSize)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static int32 GetGroupSize()
	{
		return 4;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FGlobalDistanceFieldPageStatsCS, "/Engine/Private/DistanceField/GlobalDistanceFieldDebug.usf", "GlobalDistanceFieldPageStatsCS", SF_Compute);

class FGlobalDistanceFieldDebugCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGlobalDistanceFieldDebugCS)
	SHADER_USE_PARAMETER_STRUCT(FGlobalDistanceFieldDebugCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, GlobalDistanceFieldPageFreeListAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PageStatsBuffer)
		SHADER_PARAMETER(uint32, GlobalDistanceFieldMaxPageNum)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}

	static int32 GetGroupSize()
	{
		return 64;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FGlobalDistanceFieldDebugCS, "/Engine/Private/DistanceField/GlobalDistanceFieldDebug.usf", "GlobalDistanceFieldDebugCS", SF_Compute);

struct FGlobalDistanceFieldInfoRDG
{
	FRDGBufferRef PageFreeListAllocatorBuffer;
	FRDGBufferRef PageFreeListBuffer;
	FRDGTextureRef PageAtlasTexture;
	FRDGTextureRef CoverageAtlasTexture;
	FRDGBufferRef PageObjectGridBuffer;
	FRDGTextureRef PageTableCombinedTexture;
	FRDGTextureRef MipTexture;
	FRDGTextureRef PageTableLayerTextures[GDF_Num];
};

static void RegisterGlobalDistanceFieldExternalResources(
	FRDGBuilder& GraphBuilder,
	const FGlobalDistanceFieldInfo& GlobalDistanceFieldInfo,
	const FGlobalDFCacheType StartCacheType,
	FGlobalDistanceFieldInfoRDG& Out)
{
	Out.PageFreeListAllocatorBuffer = nullptr;
	if (GlobalDistanceFieldInfo.PageFreeListAllocatorBuffer)
	{
		Out.PageFreeListAllocatorBuffer = GraphBuilder.RegisterExternalBuffer(GlobalDistanceFieldInfo.PageFreeListAllocatorBuffer, TEXT("GlobalDistanceField.PageFreeListAllocator"));
	}

	Out.PageFreeListBuffer = nullptr;
	if (GlobalDistanceFieldInfo.PageFreeListBuffer)
	{
		Out.PageFreeListBuffer = GraphBuilder.RegisterExternalBuffer(GlobalDistanceFieldInfo.PageFreeListBuffer, TEXT("GlobalDistanceField.PageFreeList"));
	}

	Out.PageAtlasTexture = nullptr;
	if (GlobalDistanceFieldInfo.PageAtlasTexture)
	{
		Out.PageAtlasTexture = GraphBuilder.RegisterExternalTexture(GlobalDistanceFieldInfo.PageAtlasTexture, TEXT("GlobalDistanceField.PageAtlas"));
	}

	Out.CoverageAtlasTexture = nullptr;
	if (GlobalDistanceFieldInfo.CoverageAtlasTexture)
	{
		Out.CoverageAtlasTexture = GraphBuilder.RegisterExternalTexture(GlobalDistanceFieldInfo.CoverageAtlasTexture, TEXT("GlobalDistanceField.CoverageAtlas"));
	}

	Out.PageObjectGridBuffer = nullptr;
	if (GlobalDistanceFieldInfo.PageObjectGridBuffer)
	{
		Out.PageObjectGridBuffer = GraphBuilder.RegisterExternalBuffer(GlobalDistanceFieldInfo.PageObjectGridBuffer, TEXT("GlobalDistanceField.PageObjectGridBuffer"));
	}

	Out.PageTableCombinedTexture = nullptr;
	if (GlobalDistanceFieldInfo.PageTableCombinedTexture)
	{
		Out.PageTableCombinedTexture = GraphBuilder.RegisterExternalTexture(GlobalDistanceFieldInfo.PageTableCombinedTexture, TEXT("GlobalDistanceField.PageTableCombined"));
	}

	Out.MipTexture = nullptr;
	if (GlobalDistanceFieldInfo.MipTexture)
	{
		Out.MipTexture = GraphBuilder.RegisterExternalTexture(GlobalDistanceFieldInfo.MipTexture, TEXT("GlobalDistanceField.SDFMips"));
	}

	for (int32 CacheType = StartCacheType; CacheType < GDF_Num; CacheType++)
	{
		Out.PageTableLayerTextures[CacheType] = nullptr;
		if (GlobalDistanceFieldInfo.PageTableLayerTextures[CacheType])
		{
			Out.PageTableLayerTextures[CacheType] = GraphBuilder.RegisterExternalTexture(GlobalDistanceFieldInfo.PageTableLayerTextures[CacheType], TEXT("GlobalDistanceFieldPageTableLayer"));
		}
	}
}

static void FinalizeGlobalDistanceFieldExternalResourceAccess(
	FRDGBuilder& GraphBuilder,
	FRDGExternalAccessQueue& ExternalAccessQueue,
	FGlobalDistanceFieldInfo& GlobalDistanceFieldInfo,
	const FGlobalDFCacheType StartCacheType,
	const FGlobalDistanceFieldInfoRDG& In)
{
	for (int32 CacheType = StartCacheType; CacheType < GDF_Num; CacheType++)
	{
		if (In.PageTableLayerTextures[CacheType])
		{
			GlobalDistanceFieldInfo.PageTableLayerTextures[CacheType] = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, In.PageTableLayerTextures[CacheType]);
		}
	}

	if (In.PageFreeListAllocatorBuffer)
	{
		GlobalDistanceFieldInfo.PageFreeListAllocatorBuffer = ConvertToExternalAccessBuffer(GraphBuilder, ExternalAccessQueue, In.PageFreeListAllocatorBuffer);
	}

	if (In.PageFreeListBuffer)
	{
		GlobalDistanceFieldInfo.PageFreeListBuffer = ConvertToExternalAccessBuffer(GraphBuilder, ExternalAccessQueue, In.PageFreeListBuffer);
	}

	if (In.PageAtlasTexture)
	{
		GlobalDistanceFieldInfo.PageAtlasTexture = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, In.PageAtlasTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}

	if (In.CoverageAtlasTexture)
	{
		GlobalDistanceFieldInfo.CoverageAtlasTexture = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, In.CoverageAtlasTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}

	if (In.PageObjectGridBuffer)
	{
		GlobalDistanceFieldInfo.PageObjectGridBuffer = ConvertToExternalAccessBuffer(GraphBuilder, ExternalAccessQueue, In.PageObjectGridBuffer, ERHIAccess::SRVMask, ERHIPipeline::All);
	}

	if (In.PageTableCombinedTexture)
	{
		GlobalDistanceFieldInfo.PageTableCombinedTexture = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, In.PageTableCombinedTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}

	if (In.MipTexture)
	{
		GlobalDistanceFieldInfo.MipTexture = ConvertToExternalAccessTexture(GraphBuilder, ExternalAccessQueue, In.MipTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}
}

struct FGlobalDistanceFieldPackedClipmap
{
	int32 Index = -1;

	int32 Resolution;
	FVector Size;
	FVector VoxelSize;
	FVector VoxelExtent;
	FDFVector3 PreViewTranslation;
	FBox TranslatedBounds;
	float InfluenceRadius;
	FVector4f VolumeTranslatedWorldToUVAddAndMul;

	uint32 PageGridSize;
	FIntVector PageGridResolution;
	FVector PageTileWorldExtent;
	FVector PageTileWorldExtentWithoutBorders;
	FVector PageGridCoordToTranslatedWorldCenterScale;
	FVector PageGridCoordToTranslatedWorldCenterBias;
	FVector PageCoordToVoxelTranslatedCenterScale;
	FVector PageCoordToVoxelTranslatedCenterBias;

	uint32 CullGridSize;
	FIntVector CullGridResolution;

	FVector CullTileWorldExtent;
	FVector CullGridCoordToTranslatedWorldCenterScale;
	FVector CullGridCoordToTranslatedWorldCenterBias;

	int32 NumUpdateBounds = 0;
	FRDGBufferRef UpdateBoundsBuffer = nullptr;
	FHeightfieldDescription UpdateRegionHeightfield;

	bool bRecacheClipmapsWithPendingStreaming = false;

	FRDGBufferRef PageUpdateIndirectArgBuffer = nullptr;
	FRDGBufferRef CullGridUpdateIndirectArgBuffer = nullptr;
	FRDGBufferRef PageComposeIndirectArgBuffer = nullptr;
	FRDGBufferRef PageComposeHeightfieldIndirectArgBuffer = nullptr;
	FRDGBufferRef BuildObjectGridIndirectArgBuffer = nullptr;

	FRDGBufferRef PageUpdateTileBuffer = nullptr;
	FRDGBufferRef PageComposeTileBuffer = nullptr;
	FRDGBufferRef PageComposeHeightfieldTileBuffer = nullptr;
	FRDGBufferRef CullGridUpdateTileBuffer = nullptr;

	FRDGBufferRef CullGridAllocator = nullptr;
	FRDGBufferRef CullGridObjectHeader = nullptr;
	FRDGBufferRef CullGridObjectArray = nullptr;

	FRDGBufferRef BuildHeightfieldComposeTilesIndirectArgBuffer = nullptr;
	FRDGBufferRef MarkedHeightfieldPageBuffer = nullptr;

	FRDGTextureRef TempMipTexture = nullptr;

	FGlobalDistanceFieldUpdateParameters UpdateParameters;
};

/**
 * Shader logic for updating modified Global Distance Field regions.
 * Dispatches are done per clipmap in order to overlap as much work as possible.
 **/
void UpdateGlobalDistanceFieldCache(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FScene* Scene,
	int32 CacheType,
	TArray<FGlobalDistanceFieldClipmap>& Clipmaps,
	FRDGBufferRef PageStatsBuffer,
	FRDGBufferRef PageObjectGridBuffer,
	FRDGTextureRef PageTableLayerTexture,
	FRDGTextureRef ParentPageTableLayerTexture,
	FRDGTextureRef PageTableCombinedTexture,
	FRDGTextureRef PageAtlasTexture,
	FRDGTextureRef CoverageAtlasTexture,
	FRDGTextureRef MipTexture,
	FRDGBufferRef PageFreeListAllocatorBuffer,
	FRDGBufferRef PageFreeListBuffer,
	bool bLumenEnabled)
{
	RDG_EVENT_SCOPE(GraphBuilder, "Update %s", CacheType == GDF_MostlyStatic ? TEXT("MostlyStatic") : TEXT("Movable"));

	const FDistanceFieldSceneData& DistanceFieldSceneData = Scene->DistanceFieldSceneData;
	TArray<FGlobalDistanceFieldPackedClipmap, TInlineAllocator<GlobalDistanceField::MaxClipmaps>> PackedClipmaps;

	// Gather clipmaps to update
	for (int32 ClipmapIndex = 0; ClipmapIndex < Clipmaps.Num(); ++ClipmapIndex)
	{
		const FGlobalDistanceFieldClipmap& Clipmap = Clipmaps[ClipmapIndex];

		if (Clipmap.UpdateBounds.Num() > 0)
		{
			FGlobalDistanceFieldPackedClipmap& PackedClipmap = PackedClipmaps.AddDefaulted_GetRef();
			PackedClipmap.Index = ClipmapIndex;

			const FVector ClipmapWorldCenter = Clipmap.Bounds.GetCenter();

			const FVector PreViewTranslation = -ClipmapWorldCenter;
			const FDFVector3 PreViewTranslationDF(PreViewTranslation);

			PackedClipmap.PreViewTranslation = PreViewTranslationDF;
			PackedClipmap.TranslatedBounds = Clipmap.Bounds.ShiftBy(PreViewTranslation);

			PackedClipmap.Resolution = GlobalDistanceField::GetClipmapResolution(bLumenEnabled);
			PackedClipmap.Size = PackedClipmap.TranslatedBounds.GetSize();
			PackedClipmap.VoxelSize = PackedClipmap.Size / FVector(PackedClipmap.Resolution);
			PackedClipmap.VoxelExtent = 0.5f * PackedClipmap.VoxelSize;
			PackedClipmap.InfluenceRadius = (GGlobalDistanceFieldInfluenceRangeInVoxels * PackedClipmap.Size.X) / PackedClipmap.Resolution;

			const FVector TranslatedWorldToUVAdd = (Clipmap.ScrollOffset - PackedClipmap.TranslatedBounds.GetCenter()) / PackedClipmap.TranslatedBounds.GetSize().X + FVector(0.5f);
			PackedClipmap.VolumeTranslatedWorldToUVAddAndMul = FVector4f((FVector3f)TranslatedWorldToUVAdd, 1.0f / PackedClipmap.TranslatedBounds.GetSize().X);

			const uint32 PageGridDim = FMath::DivideAndRoundUp(PackedClipmap.Resolution, GGlobalDistanceFieldPageResolution);
			PackedClipmap.PageGridSize = PageGridDim * PageGridDim * PageGridDim;
			PackedClipmap.PageGridResolution = FIntVector(PageGridDim, PageGridDim, PageGridDim);

			PackedClipmap.PageTileWorldExtent = PackedClipmap.VoxelExtent * GGlobalDistanceFieldPageResolutionInAtlas;
			PackedClipmap.PageTileWorldExtentWithoutBorders = PackedClipmap.VoxelExtent * GGlobalDistanceFieldPageResolution;
			PackedClipmap.PageGridCoordToTranslatedWorldCenterScale = PackedClipmap.Size / FVector(PackedClipmap.PageGridResolution);
			PackedClipmap.PageGridCoordToTranslatedWorldCenterBias = PackedClipmap.TranslatedBounds.Min + 0.5f * PackedClipmap.PageGridCoordToTranslatedWorldCenterScale;

			PackedClipmap.CullGridResolution = FComputeShaderUtils::GetGroupCount(PackedClipmap.PageGridResolution, GlobalDistanceField::CullGridFactor);
			PackedClipmap.CullGridSize = PackedClipmap.CullGridResolution.X * PackedClipmap.CullGridResolution.Y * PackedClipmap.CullGridResolution.Z;
			PackedClipmap.CullTileWorldExtent = PackedClipmap.VoxelExtent * GGlobalDistanceFieldPageResolutionInAtlas * GlobalDistanceField::CullGridFactor;
			PackedClipmap.CullGridCoordToTranslatedWorldCenterScale = PackedClipmap.Size / FVector(PackedClipmap.CullGridResolution);
			PackedClipmap.CullGridCoordToTranslatedWorldCenterBias = PackedClipmap.TranslatedBounds.Min + 0.5 * PackedClipmap.CullGridCoordToTranslatedWorldCenterScale;

			const FVector PageVoxelExtent = 0.5f * PackedClipmap.Size / FVector(PackedClipmap.Resolution);
			PackedClipmap.PageCoordToVoxelTranslatedCenterScale = PackedClipmap.Size / FVector(PackedClipmap.Resolution);
			PackedClipmap.PageCoordToVoxelTranslatedCenterBias = PackedClipmap.TranslatedBounds.Min + PageVoxelExtent;

			const uint32 PageComposeTileSize = 4;
			const FVector PageComposeTileWorldExtent = PackedClipmap.VoxelExtent * PageComposeTileSize;

			PackedClipmap.UpdateParameters.InfluenceRadius = PackedClipmap.InfluenceRadius;
			PackedClipmap.UpdateParameters.InfluenceRadiusSq = PackedClipmap.InfluenceRadius * PackedClipmap.InfluenceRadius;
			PackedClipmap.UpdateParameters.ClipmapVoxelExtent = PackedClipmap.VoxelExtent.X;
			PackedClipmap.UpdateParameters.CullGridResolution = PackedClipmap.CullGridResolution;
			PackedClipmap.UpdateParameters.PageGridResolution = PackedClipmap.PageGridResolution;
			PackedClipmap.UpdateParameters.InvPageGridResolution = FVector3f::OneVector / (FVector3f)PackedClipmap.PageGridResolution;
			PackedClipmap.UpdateParameters.ClipmapResolution = FIntVector(PackedClipmap.Resolution);
			PackedClipmap.UpdateParameters.PageCoordToVoxelTranslatedCenterScale = (FVector3f)PackedClipmap.PageCoordToVoxelTranslatedCenterScale;
			PackedClipmap.UpdateParameters.PageCoordToVoxelTranslatedCenterBias = (FVector3f)PackedClipmap.PageCoordToVoxelTranslatedCenterBias;
			PackedClipmap.UpdateParameters.ComposeTileWorldExtent = (FVector3f)PageComposeTileWorldExtent;
			PackedClipmap.UpdateParameters.ClipmapMinBounds = (FVector3f)Clipmap.Bounds.Min;
			PackedClipmap.UpdateParameters.PageCoordToPageTranslatedWorldCenterScale = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterScale;
			PackedClipmap.UpdateParameters.PageCoordToPageTranslatedWorldCenterBias = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterBias;
			PackedClipmap.UpdateParameters.ClipmapVolumeTranslatedWorldToUVAddAndMul = PackedClipmap.VolumeTranslatedWorldToUVAddAndMul;
			PackedClipmap.UpdateParameters.PageTableClipmapOffsetZ = PackedClipmap.Index * PackedClipmap.PageGridResolution.Z;
			PackedClipmap.UpdateParameters.PreViewTranslationHigh = PackedClipmap.PreViewTranslation.High;
			PackedClipmap.UpdateParameters.PreViewTranslationLow = PackedClipmap.PreViewTranslation.Low;


			// Upload update bounds data
			PackedClipmap.UpdateBoundsBuffer = nullptr;
			PackedClipmap.NumUpdateBounds = 0;

			const uint32 BufferStrideInFloat4 = 2;
			const uint32 BufferStride = BufferStrideInFloat4 * sizeof(FVector4f);

			FRDGUploadData<FVector4f> UpdateBoundsData(GraphBuilder, BufferStrideInFloat4 * Clipmap.UpdateBounds.Num());

			for (int32 UpdateBoundsIndex = 0; UpdateBoundsIndex < Clipmap.UpdateBounds.Num(); ++UpdateBoundsIndex)
			{
				const FClipmapUpdateBounds& UpdateBounds = Clipmap.UpdateBounds[UpdateBoundsIndex];

				UpdateBoundsData[PackedClipmap.NumUpdateBounds * BufferStrideInFloat4 + 0] = FVector4f((FVector3f)(UpdateBounds.Center + PreViewTranslation), UpdateBounds.bExpandByInfluenceRadius ? 1.0f : 0.0f);
				UpdateBoundsData[PackedClipmap.NumUpdateBounds * BufferStrideInFloat4 + 1] = FVector4f((FVector3f)UpdateBounds.Extent, 0.0f);
				++PackedClipmap.NumUpdateBounds;
			}

			check(UpdateBoundsData.Num() % BufferStrideInFloat4 == 0);

			PackedClipmap.UpdateBoundsBuffer =
				CreateUploadBuffer(GraphBuilder, TEXT("GlobalDistanceField.UpdateBoundsBuffer"),
					sizeof(FVector4f), FMath::RoundUpToPowerOfTwo(FMath::Max(UpdateBoundsData.Num(), 2)),
					UpdateBoundsData);
		}
	}

	// Update heightfield descriptors
	for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
	{
		const FGlobalDistanceFieldClipmap& Clipmap = Clipmaps[PackedClipmap.Index];

		const int32 NumHeightfieldPrimitives = DistanceFieldSceneData.HeightfieldPrimitives.Num();
		if ((CacheType == GDF_MostlyStatic || !GAOGlobalDistanceFieldCacheMostlyStaticSeparately)
			&& NumHeightfieldPrimitives > 0
			&& GAOGlobalDistanceFieldRepresentHeightfields
			&& !IsVulkanMobileSM5Platform(Scene->GetShaderPlatform()))
		{
			for (int32 HeightfieldPrimitiveIndex = 0; HeightfieldPrimitiveIndex < NumHeightfieldPrimitives; HeightfieldPrimitiveIndex++)
			{
				const FPrimitiveSceneInfo* HeightfieldPrimitiveSceneInfo = Scene->DistanceFieldSceneData.HeightfieldPrimitives[HeightfieldPrimitiveIndex];
				const FPrimitiveSceneProxy* HeightfieldPrimitiveProxy = HeightfieldPrimitiveSceneInfo->Proxy;
				const FBoxSphereBounds& PrimitiveBounds = HeightfieldPrimitiveProxy->GetBounds();
				const uint32 GPUSceneInstanceIndex = HeightfieldPrimitiveSceneInfo->GetInstanceSceneDataOffset();

				if (HeightfieldPrimitiveProxy->HeightfieldHasPendingStreaming())
				{
					continue;
				}

				// Expand bounding box by a SDF max influence distance (only in local Z axis, as distance is computed from a top down projected heightmap point).
				const FVector QueryInfluenceExpand = HeightfieldPrimitiveProxy->GetLocalToWorld().GetUnitAxis(EAxis::Z) * FVector(0.0f, 0.0f, PackedClipmap.InfluenceRadius);
				const FBox HeightfieldInfluenceBox = PrimitiveBounds.GetBox().ExpandBy(QueryInfluenceExpand, QueryInfluenceExpand);

				if (Clipmap.Bounds.Intersect(HeightfieldInfluenceBox))
				{
					UTexture2D* HeightfieldTexture = nullptr;
					UTexture2D* VisibilityTexture = nullptr;
					FHeightfieldComponentDescription NewComponentDescription(HeightfieldPrimitiveProxy->GetLocalToWorld(), GPUSceneInstanceIndex);
					HeightfieldPrimitiveProxy->GetHeightfieldRepresentation(HeightfieldTexture, VisibilityTexture, NewComponentDescription);

					if (HeightfieldTexture && HeightfieldTexture->GetResource() && HeightfieldTexture->GetResource()->TextureRHI)
					{
						TArray<FHeightfieldComponentDescription>& ComponentDescriptions = PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions.FindOrAdd(FHeightfieldComponentTextures(HeightfieldTexture, VisibilityTexture));
						ComponentDescriptions.Add(NewComponentDescription);
					}
				}
			}
		}
	}

	if (PageAtlasTexture)
	{
		FDistanceFieldObjectBufferParameters DistanceFieldObjectBuffers = DistanceField::SetupObjectBufferParameters(GraphBuilder, DistanceFieldSceneData);
		FDistanceFieldAtlasParameters DistanceFieldAtlas = DistanceField::SetupAtlasParameters(GraphBuilder, DistanceFieldSceneData);

		// Allocate buffers for objects culled to clipmaps
		const uint32 ObjectIndexBufferStride = FMath::RoundUpToPowerOfTwo(DistanceFieldSceneData.NumObjectsInBuffer);
		FRDGBufferRef ObjectIndexBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::Max(PackedClipmaps.Num(), 1) * ObjectIndexBufferStride), TEXT("GlobalDistanceField.ObjectIndices"));
		FRDGBufferRef ObjectIndexNumBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::Max(PackedClipmaps.Num(), 1)), TEXT("GlobalDistanceField.ObjectIndexNum"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(ObjectIndexNumBuffer, PF_R32_UINT), 0);

		// Prepare re-cache buffers
		bool bAnyClipmapHasPendingStreamingReadback = false;
		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			if (Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0)
			{
				PackedClipmap.bRecacheClipmapsWithPendingStreaming =
					GAOGlobalDistanceFieldRecacheClipmapsWithPendingStreaming != 0 &&
					View.ViewState &&
					CacheType == GDF_MostlyStatic &&
					PackedClipmap.Index < 2;

				if (PackedClipmap.bRecacheClipmapsWithPendingStreaming)
				{
					const FGlobalDistanceFieldStreamingReadback& StreamingReadback = View.ViewState->GlobalDistanceFieldData->StreamingReadback[CacheType];

					// It is not safe to EnqueueCopy on a buffer that already has a pending copy
					PackedClipmap.bRecacheClipmapsWithPendingStreaming = PackedClipmap.bRecacheClipmapsWithPendingStreaming 
						&& StreamingReadback.ReadbackBuffersNumPending < StreamingReadback.MaxPendingStreamingReadbackBuffers;

					if (PackedClipmap.bRecacheClipmapsWithPendingStreaming)
					{
						bAnyClipmapHasPendingStreamingReadback = true;
					}
				}
			}
		}

		// Prepare pending mesh SDF streaming readback buffer
		FRDGBufferRef PendingStreamingReadbackBuffer = nullptr;
		if (bAnyClipmapHasPendingStreamingReadback)
		{
			FRDGBufferDesc HasPendingStreamingReadbackDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), GlobalDistanceField::MaxClipmaps);
			HasPendingStreamingReadbackDesc.Usage = EBufferUsageFlags(HasPendingStreamingReadbackDesc.Usage | BUF_SourceCopy);
			PendingStreamingReadbackBuffer = GraphBuilder.CreateBuffer(HasPendingStreamingReadbackDesc, TEXT("GlobalDistanceField.HasPendingStreamingReadback"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PendingStreamingReadbackBuffer, PF_R32_UINT), 0);
		}

		// Upload packed clipmap data to GPU
		FRDGBufferRef PackedClipmapBuffer = nullptr;
		{
			// Must match FPackedClipmapData in GlobalDistanceField.usf
			struct FPackedClipmapData
			{
				FVector3f TranslatedWorldCenter; // Camera-centered world space
				float MeshSDFRadiusThreshold;

				FVector3f WorldExtent;
				float InfluenceRadiusSq;

				uint32 ClipmapIndex;
				uint32 Flags;
				uint32 Padding0;
				uint32 Padding1;
			};

			TArray<FPackedClipmapData> UploadData;
			UploadData.SetNum(FMath::Max(PackedClipmaps.Num(), 1));

			for (int32 PackedClipmapIndex = 0; PackedClipmapIndex < PackedClipmaps.Num(); ++PackedClipmapIndex)
			{
				const FGlobalDistanceFieldPackedClipmap& PackedClipmap = PackedClipmaps[PackedClipmapIndex];

				const float RadiusThresholdScale = bLumenEnabled ? 1.0f / FMath::Clamp(View.FinalPostProcessSettings.LumenSceneDetail, .01f, 100.0f) : 1.0f;
				const float MeshSDFRadiusThreshold = GetMinMeshSDFRadius(PackedClipmap.VoxelSize.X) * RadiusThresholdScale;

				// Camera-centered world space
				const FGlobalDistanceFieldClipmap& Clipmap = Clipmaps[PackedClipmap.Index];
				const FVector TranslatedWorldCenter = Clipmap.Bounds.GetCenter() + View.ViewMatrices.GetPreViewTranslation();

				FPackedClipmapData& ClipmapData = UploadData[PackedClipmapIndex];

				ClipmapData.TranslatedWorldCenter = (FVector3f)TranslatedWorldCenter;
				ClipmapData.MeshSDFRadiusThreshold = MeshSDFRadiusThreshold;

				ClipmapData.WorldExtent = (FVector3f)PackedClipmap.TranslatedBounds.GetExtent();
				ClipmapData.InfluenceRadiusSq = PackedClipmap.InfluenceRadius * PackedClipmap.InfluenceRadius;

				ClipmapData.ClipmapIndex = PackedClipmap.Index;
				ClipmapData.Flags = PackedClipmap.bRecacheClipmapsWithPendingStreaming ? 1u : 0u;
				ClipmapData.Padding0 = 0;
				ClipmapData.Padding1 = 0;
			}

			PackedClipmapBuffer =
				CreateStructuredUploadBuffer(GraphBuilder,
					TEXT("GlobalDistanceField.PackedClipmapBuffer"),
					UploadData);
		}

		// Cull the global objects to the update regions
		if (Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0)
		{
			uint32 AcceptOftenMovingObjectsOnlyValue = 0;

			if (!GAOGlobalDistanceFieldCacheMostlyStaticSeparately)
			{
				AcceptOftenMovingObjectsOnlyValue = 2;
			}
			else if (CacheType == GDF_Full)
			{
				// First cache is for mostly static, second contains both, inheriting static objects distance fields with a lookup
				// So only composite often moving objects into the full global distance field
				AcceptOftenMovingObjectsOnlyValue = 1;
			}

			FCullObjectsToClipmapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCullObjectsToClipmapCS::FParameters>();
			PassParameters->RWObjectIndexBuffer = GraphBuilder.CreateUAV(ObjectIndexBuffer, PF_R32_UINT);
			PassParameters->RWObjectIndexNumBuffer = GraphBuilder.CreateUAV(ObjectIndexNumBuffer, PF_R32_UINT);
			PassParameters->RWHasPendingStreaming = PendingStreamingReadbackBuffer ? GraphBuilder.CreateUAV(PendingStreamingReadbackBuffer, PF_R32_UINT) : nullptr;
			PassParameters->PackedClipmapBuffer = GraphBuilder.CreateSRV(PackedClipmapBuffer);
			PassParameters->AcceptOftenMovingObjectsOnly = AcceptOftenMovingObjectsOnlyValue;
			PassParameters->NumPackedClipmaps = PackedClipmaps.Num();
			PassParameters->ObjectIndexBufferStride = ObjectIndexBufferStride;
			PassParameters->DistanceFieldObjectBuffers = DistanceFieldObjectBuffers;
			PassParameters->DistanceFieldAtlasParameters = DistanceFieldAtlas;


			FDFVector3 PreViewTranslation(View.ViewMatrices.GetPreViewTranslation());
			PassParameters->PreViewTranslationHigh = PreViewTranslation.High;
			PassParameters->PreViewTranslationLow = PreViewTranslation.Low;
			
			FCullObjectsToClipmapCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCullObjectsToClipmapCS::FReadbackHasPendingStreaming>(bAnyClipmapHasPendingStreamingReadback);
			auto ComputeShader = View.ShaderMap->GetShader<FCullObjectsToClipmapCS>(PermutationVector);

			const FIntVector GroupCount = FComputeShaderUtils::GetGroupCountWrapped(DistanceFieldSceneData.NumObjectsInBuffer, FCullObjectsToClipmapCS::GetGroupSize());

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("CullToClipmaps"),
				ComputeShader,
				PassParameters,
				GroupCount);
		}

		// Readback re-cache requests
		if (PendingStreamingReadbackBuffer)
		{
			FGlobalDistanceFieldStreamingReadback& StreamingReadback = View.ViewState->GlobalDistanceFieldData->StreamingReadback[CacheType];

			if (!StreamingReadback.PendingStreamingReadbackBuffers[StreamingReadback.ReadbackBuffersWriteIndex].IsValid())
			{
				StreamingReadback.PendingStreamingReadbackBuffers[StreamingReadback.ReadbackBuffersWriteIndex] =
					MakeUnique<FRHIGPUBufferReadback>(TEXT("GlobalDistanceField.PendingStreamingReadback"));
			}

			FRHIGPUBufferReadback* ReadbackBuffer = StreamingReadback.PendingStreamingReadbackBuffers[StreamingReadback.ReadbackBuffersWriteIndex].Get();

			AddReadbackBufferPass(GraphBuilder, RDG_EVENT_NAME("GlobalDistanceField.HasPendingStreamingReadback"), PendingStreamingReadbackBuffer,
				[ReadbackBuffer, PendingStreamingReadbackBuffer = PendingStreamingReadbackBuffer](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					ReadbackBuffer->EnqueueCopy(RHICmdList, PendingStreamingReadbackBuffer->GetRHI(), 0u);
				});

			StreamingReadback.ReadbackBuffersWriteIndex = (StreamingReadback.ReadbackBuffersWriteIndex + 1u) % StreamingReadback.MaxPendingStreamingReadbackBuffers;
			StreamingReadback.ReadbackBuffersNumPending = FMath::Min(StreamingReadback.ReadbackBuffersNumPending + 1u, StreamingReadback.MaxPendingStreamingReadbackBuffers);
		}

		// Clear indirect dispatch arguments
		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			PackedClipmap.PageUpdateIndirectArgBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("GlobalDistanceField.PageUpdateIndirectArgs"));
			PackedClipmap.CullGridUpdateIndirectArgBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("GlobalDistanceField.CullGridUpdateIndirectArgs"));
			PackedClipmap.PageComposeIndirectArgBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("GlobalDistanceField.PageComposeIndirectArgs"));
			PackedClipmap.PageComposeHeightfieldIndirectArgBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("GlobalDistanceField.PageComposeHeightfieldIndirectArgs"));
			PackedClipmap.BuildObjectGridIndirectArgBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("GlobalDistanceField.BuildObjectGridIndirectArgs"));

			FClearIndirectArgBufferCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FClearIndirectArgBufferCS::FParameters>();
			PassParameters->RWPageUpdateIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.PageUpdateIndirectArgBuffer, PF_R32_UINT);
			PassParameters->RWCullGridUpdateIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.CullGridUpdateIndirectArgBuffer, PF_R32_UINT);
			PassParameters->RWPageComposeIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.PageComposeIndirectArgBuffer, PF_R32_UINT);
			PassParameters->RWBuildObjectGridIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.BuildObjectGridIndirectArgBuffer, PF_R32_UINT);

			auto ComputeShader = View.ShaderMap->GetShader<FClearIndirectArgBufferCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ClearIndirectArgBuffer"),
				ComputeShader,
				PassParameters,
				FIntVector(1, 1, 1));
		}

		// Prepare page tiles which need to be updated for update regions
		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			const FGlobalDistanceFieldClipmap& Clipmap = Clipmaps[PackedClipmap.Index];

			PackedClipmap.PageUpdateTileBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), PackedClipmap.PageGridSize), TEXT("GlobalDistanceField.PageUpdateTiles"));
			PackedClipmap.PageComposeTileBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), PackedClipmap.PageGridSize), TEXT("GlobalDistanceField.PageComposeTiles"));
			PackedClipmap.PageComposeHeightfieldTileBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), PackedClipmap.PageGridSize), TEXT("GlobalDistanceField.PageComposeHeightfieldTiles"));
			PackedClipmap.CullGridUpdateTileBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), PackedClipmap.CullGridSize), TEXT("GlobalDistanceField.CullGridUpdateTiles"));

			FBuildGridTilesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildGridTilesCS::FParameters>();
			PassParameters->RWPageTileBuffer = GraphBuilder.CreateUAV(PackedClipmap.PageUpdateTileBuffer, PF_R32_UINT);
			PassParameters->RWPageIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.PageUpdateIndirectArgBuffer, PF_R32_UINT);
			PassParameters->RWCullGridTileBuffer = GraphBuilder.CreateUAV(PackedClipmap.CullGridUpdateTileBuffer, PF_R32_UINT);
			PassParameters->RWCullGridIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.CullGridUpdateIndirectArgBuffer, PF_R32_UINT);
			PassParameters->UpdateBoundsBuffer = GraphBuilder.CreateSRV(PackedClipmap.UpdateBoundsBuffer, PF_A32B32G32R32F);
			PassParameters->NumUpdateBounds = PackedClipmap.NumUpdateBounds;
			PassParameters->InfluenceRadiusSq = PackedClipmap.InfluenceRadius * PackedClipmap.InfluenceRadius;
			// Page grid
			PassParameters->PageGridResolution = PackedClipmap.PageGridResolution;
			PassParameters->PageGridCoordToTranslatedWorldCenterScale = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterScale;
			PassParameters->PageGridCoordToTranslatedWorldCenterBias = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterBias;
			PassParameters->PageGridTileWorldExtent = (FVector3f)PackedClipmap.PageTileWorldExtent;
			// Cull grid
			PassParameters->CullGridResolution = PackedClipmap.CullGridResolution;
			PassParameters->CullGridCoordToTranslatedWorldCenterScale = (FVector3f)PackedClipmap.CullGridCoordToTranslatedWorldCenterScale;
			PassParameters->CullGridCoordToTranslatedWorldCenterBias = (FVector3f)PackedClipmap.CullGridCoordToTranslatedWorldCenterBias;
			PassParameters->CullGridTileWorldExtent = (FVector3f)PackedClipmap.CullTileWorldExtent;

			auto ComputeShader = View.ShaderMap->GetShader<FBuildGridTilesCS>();

			const FIntVector GroupSize = FComputeShaderUtils::GetGroupCount(PackedClipmap.PageGridResolution, FBuildGridTilesCS::GetGroupSize());

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("BuildPageUpdateTiles UpdateBounds:%d Clipmap:%d %s", PackedClipmap.NumUpdateBounds, PackedClipmap.Index, GetRecaptureReasonString(Clipmap.FullRecaptureReason)),
				ComputeShader,
				PassParameters,
				GroupSize);
		}

		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			if (PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions.Num() > 0)
			{
				PackedClipmap.MarkedHeightfieldPageBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), PackedClipmap.PageGridSize), TEXT("GlobalDistanceField.MarkedHeightfieldPages"));
				AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PackedClipmap.MarkedHeightfieldPageBuffer, PF_R32_UINT), 0);
			}
		}

		// Mark pages which contain a heightfield
		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			if (PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions.Num() > 0)
			{
				for (TMap<FHeightfieldComponentTextures, TArray<FHeightfieldComponentDescription>>::TConstIterator It(PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions); It; ++It)
				{
					const TArray<FHeightfieldComponentDescription>& HeightfieldDescriptions = It.Value();

					if (HeightfieldDescriptions.Num() > 0)
					{
						FRDGBufferRef HeightfieldDescriptionBuffer = UploadHeightfieldDescriptions(GraphBuilder, HeightfieldDescriptions);

						UTexture2D* HeightfieldTexture = It.Key().HeightAndNormal;
						UTexture2D* VisibilityTexture = It.Key().Visibility;

						FMarkHeightfieldPagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMarkHeightfieldPagesCS::FParameters>();
						PassParameters->RWMarkedHeightfieldPageBuffer = GraphBuilder.CreateUAV(PackedClipmap.MarkedHeightfieldPageBuffer, PF_R32_UINT);
						PassParameters->PageUpdateIndirectArgBuffer = PackedClipmap.PageUpdateIndirectArgBuffer;
						PassParameters->PageUpdateTileBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageUpdateTileBuffer, PF_R32_UINT);
						PassParameters->PageCoordToVoxelTranslatedCenterScale = (FVector3f)PackedClipmap.PageCoordToVoxelTranslatedCenterScale;
						PassParameters->PageCoordToVoxelTranslatedCenterBias = (FVector3f)PackedClipmap.PageCoordToVoxelTranslatedCenterBias;
						PassParameters->PageWorldExtent = (FVector3f)PackedClipmap.PageTileWorldExtentWithoutBorders;
						PassParameters->ClipmapVoxelExtent = PackedClipmap.VoxelExtent.X;
						PassParameters->PageGridResolution = PackedClipmap.PageGridResolution;
						PassParameters->NumHeightfields = HeightfieldDescriptions.Num();
						PassParameters->InfluenceRadius = PackedClipmap.InfluenceRadius;
						PassParameters->HeightfieldThickness = PackedClipmap.VoxelSize.X * GGlobalDistanceFieldHeightFieldThicknessScale;
						PassParameters->HeightfieldTexture = HeightfieldTexture->GetResource()->TextureRHI;
						PassParameters->HeightfieldSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
						PassParameters->VisibilityTexture = VisibilityTexture ? VisibilityTexture->GetResource()->TextureRHI : GBlackTexture->TextureRHI;
						PassParameters->VisibilitySampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
						PassParameters->HeightfieldDescriptions = GraphBuilder.CreateSRV(HeightfieldDescriptionBuffer, EPixelFormat::PF_A32B32G32R32F);

						PassParameters->PreViewTranslationHigh = PackedClipmap.PreViewTranslation.High;
						PassParameters->PreViewTranslationLow = PackedClipmap.PreViewTranslation.Low;

						auto ComputeShader = View.ShaderMap->GetShader<FMarkHeightfieldPagesCS>();

						FComputeShaderUtils::AddPass(
							GraphBuilder,
							RDG_EVENT_NAME("MarkHeightfieldPages Clipmap:%d", PackedClipmap.Index),
							ComputeShader,
							PassParameters,
							PackedClipmap.PageUpdateIndirectArgBuffer,
							0);
					}
				}
			}
		}

		// Prepare for building heightfield page compose tile buffer
		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			if (PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions.Num() > 0)
			{
				PackedClipmap.BuildHeightfieldComposeTilesIndirectArgBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("GlobalDistanceField.BuildHeightfieldComposeTilesIndirectArgs"));

				FBuildHeightfieldComposeTilesIndirectArgBufferCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildHeightfieldComposeTilesIndirectArgBufferCS::FParameters>();
				PassParameters->RWBuildHeightfieldComposeTilesIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.BuildHeightfieldComposeTilesIndirectArgBuffer, PF_R32_UINT);
				PassParameters->RWPageComposeHeightfieldIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.PageComposeHeightfieldIndirectArgBuffer, PF_R32_UINT);
				PassParameters->PageUpdateIndirectArgBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageUpdateIndirectArgBuffer, PF_R32_UINT);

				auto ComputeShader = View.ShaderMap->GetShader<FBuildHeightfieldComposeTilesIndirectArgBufferCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("BuildHeightfieldComposeTilesIndirectArgs"),
					ComputeShader,
					PassParameters,
					FIntVector(1, 1, 1));
			}
		}

		// Build heightfield page compose tile buffer
		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			if (PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions.Num() > 0)
			{
				FBuildHeightfieldComposeTilesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildHeightfieldComposeTilesCS::FParameters>();
				PassParameters->RWPageComposeHeightfieldIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.PageComposeHeightfieldIndirectArgBuffer, PF_R32_UINT);
				PassParameters->RWPageComposeHeightfieldTileBuffer = GraphBuilder.CreateUAV(PackedClipmap.PageComposeHeightfieldTileBuffer, PF_R32_UINT);;
				PassParameters->PageUpdateTileBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageUpdateTileBuffer, PF_R32_UINT);
				PassParameters->MarkedHeightfieldPageBuffer = GraphBuilder.CreateSRV(PackedClipmap.MarkedHeightfieldPageBuffer, PF_R32_UINT);
				PassParameters->PageUpdateIndirectArgBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageUpdateIndirectArgBuffer, PF_R32_UINT);
				PassParameters->BuildHeightfieldComposeTilesIndirectArgBuffer = PackedClipmap.BuildHeightfieldComposeTilesIndirectArgBuffer;

				auto ComputeShader = View.ShaderMap->GetShader<FBuildHeightfieldComposeTilesCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("BuildHeightfieldComposeTiles"),
					ComputeShader,
					PassParameters,
					PackedClipmap.BuildHeightfieldComposeTilesIndirectArgBuffer,
					0);
			}
		}

		// Clear cull grid resources
		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			const uint32 AverageCulledObjectsPerPage = FMath::Clamp(CVarAOGlobalDistanceFieldAverageCulledObjectsPerCell.GetValueOnRenderThread(), 1, 8192);
			PackedClipmap.CullGridAllocator = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("GlobalDistanceField.CullGridAllocator"));
			PackedClipmap.CullGridObjectHeader = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 2 * PackedClipmap.CullGridSize), TEXT("GlobalDistanceField.CullGridObjectHeader"));
			PackedClipmap.CullGridObjectArray = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), PackedClipmap.CullGridSize * AverageCulledObjectsPerPage), TEXT("GlobalDistanceField.CullGridObjectArray"));

			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PackedClipmap.CullGridAllocator, PF_R32_UINT), 0);
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PackedClipmap.CullGridObjectHeader, PF_R32_UINT), 0);
		}

		// Cull objects into a cull grid
		if (Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0)
		{
			for (int32 PackedClipmapIndex = 0; PackedClipmapIndex < PackedClipmaps.Num(); ++PackedClipmapIndex)
			{
				const FGlobalDistanceFieldPackedClipmap& PackedClipmap = PackedClipmaps[PackedClipmapIndex];

				FCullObjectsToGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCullObjectsToGridCS::FParameters>();
				PassParameters->RWCullGridAllocator = GraphBuilder.CreateUAV(PackedClipmap.CullGridAllocator, PF_R32_UINT);
				PassParameters->RWCullGridObjectHeader = GraphBuilder.CreateUAV(PackedClipmap.CullGridObjectHeader, PF_R32_UINT);
				PassParameters->RWCullGridObjectArray = GraphBuilder.CreateUAV(PackedClipmap.CullGridObjectArray, PF_R32_UINT);
				PassParameters->CullGridIndirectArgBuffer = PackedClipmap.CullGridUpdateIndirectArgBuffer;
				PassParameters->CullGridTileBuffer = GraphBuilder.CreateSRV(PackedClipmap.CullGridUpdateTileBuffer, PF_R32_UINT);
				PassParameters->ObjectIndexBuffer = GraphBuilder.CreateSRV(ObjectIndexBuffer, PF_R32_UINT);
				PassParameters->ObjectIndexNumBuffer = GraphBuilder.CreateSRV(ObjectIndexNumBuffer, PF_R32_UINT);
				PassParameters->PackedClipmapIndex = PackedClipmapIndex;
				PassParameters->ObjectIndexBufferStride = ObjectIndexBufferStride;
				PassParameters->DistanceFieldObjectBuffers = DistanceFieldObjectBuffers;
				PassParameters->CullGridResolution = PackedClipmap.CullGridResolution;
				PassParameters->CullGridCoordToTranslatedWorldCenterScale = (FVector3f)PackedClipmap.CullGridCoordToTranslatedWorldCenterScale;
				PassParameters->CullGridCoordToTranslatedWorldCenterBias = (FVector3f)PackedClipmap.CullGridCoordToTranslatedWorldCenterBias;
				PassParameters->CullTileWorldExtent = (FVector3f)PackedClipmap.CullTileWorldExtent;
				PassParameters->InfluenceRadiusSq = PackedClipmap.InfluenceRadius * PackedClipmap.InfluenceRadius;
				PassParameters->PreViewTranslationHigh = PackedClipmap.PreViewTranslation.High;
				PassParameters->PreViewTranslationLow = PackedClipmap.PreViewTranslation.Low;

				auto ComputeShader = View.ShaderMap->GetShader<FCullObjectsToGridCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("CullObjectsToGrid Clipmap:%d", PackedClipmap.Index),
					ComputeShader,
					PassParameters,
					PackedClipmap.CullGridUpdateIndirectArgBuffer,
					0);
			}
		}

		const uint32 GGlobalDistanceFieldMaxPageNum = GlobalDistanceField::GetMaxPageNum(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance);
		FRDGBufferRef PageFreeListReturnAllocatorBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("GlobalDistanceField.PageFreeListReturnAllocator"));
		FRDGBufferRef PageFreeListReturnBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), GGlobalDistanceFieldMaxPageNum), TEXT("GlobalDistanceField.PageFreeListReturn"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PageFreeListReturnAllocatorBuffer, PF_R32_UINT), 0);

		// TODO: atlas PageComposeTileBuffer and use SkipBarrier in order to allow overlap in this pass.
		// Allocate pages for objects and build page lists
		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			FAllocatePagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FAllocatePagesCS::FParameters>();
			PassParameters->PageUpdateIndirectArgBuffer = PackedClipmap.PageUpdateIndirectArgBuffer;
			PassParameters->PageUpdateTileBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageUpdateTileBuffer, PF_R32_UINT);
			PassParameters->MarkedHeightfieldPageBuffer = PackedClipmap.MarkedHeightfieldPageBuffer ? GraphBuilder.CreateSRV(PackedClipmap.MarkedHeightfieldPageBuffer, PF_R32_UINT) : nullptr;

			PassParameters->RWPageTableCombinedTexture = PageTableCombinedTexture ? GraphBuilder.CreateUAV(PageTableCombinedTexture) : nullptr;
			PassParameters->RWPageTableLayerTexture = GraphBuilder.CreateUAV(PageTableLayerTexture);
			PassParameters->RWPageFreeListAllocatorBuffer = GraphBuilder.CreateUAV(PageFreeListAllocatorBuffer, PF_R32_SINT);
			PassParameters->PageFreeListBuffer = GraphBuilder.CreateSRV(PageFreeListBuffer, PF_R32_UINT);
			PassParameters->RWPageFreeListReturnAllocatorBuffer = GraphBuilder.CreateUAV(PageFreeListReturnAllocatorBuffer, PF_R32_UINT);
			PassParameters->RWPageFreeListReturnBuffer = GraphBuilder.CreateUAV(PageFreeListReturnBuffer, PF_R32_UINT);
			PassParameters->RWPageComposeTileBuffer = GraphBuilder.CreateUAV(PackedClipmap.PageComposeTileBuffer, PF_R32_UINT);
			PassParameters->RWPageComposeIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.PageComposeIndirectArgBuffer, PF_R32_UINT);
			PassParameters->RWBuildObjectGridIndirectArgBuffer = GraphBuilder.CreateUAV(PackedClipmap.BuildObjectGridIndirectArgBuffer, PF_R32_UINT);

			PassParameters->ParentPageTableLayerTexture = ParentPageTableLayerTexture;
			PassParameters->PageWorldExtent = (FVector3f)PackedClipmap.PageTileWorldExtentWithoutBorders;
			PassParameters->PageWorldRadius = PackedClipmap.PageTileWorldExtentWithoutBorders.Size();
			PassParameters->ClipmapInfluenceRadius = PackedClipmap.InfluenceRadius;
			PassParameters->PageGridResolution = PackedClipmap.PageGridResolution;
			PassParameters->InvPageGridResolution = FVector3f::OneVector / (FVector3f)PackedClipmap.PageGridResolution;
			PassParameters->GlobalDistanceFieldMaxPageNum = GGlobalDistanceFieldMaxPageNum;
			PassParameters->PageCoordToPageTranslatedWorldCenterScale = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterScale;
			PassParameters->PageCoordToPageTranslatedWorldCenterBias = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterBias;
			PassParameters->ClipmapVolumeTranslatedWorldToUVAddAndMul = PackedClipmap.VolumeTranslatedWorldToUVAddAndMul;
			PassParameters->PageTableClipmapOffsetZ = PackedClipmap.Index * PackedClipmap.PageGridResolution.Z;

			PassParameters->CullGridObjectHeader = GraphBuilder.CreateSRV(PackedClipmap.CullGridObjectHeader, PF_R32_UINT);
			PassParameters->CullGridObjectArray = GraphBuilder.CreateSRV(PackedClipmap.CullGridObjectArray, PF_R32_UINT);
			PassParameters->CullGridResolution = PackedClipmap.CullGridResolution;

			PassParameters->DistanceFieldObjectBuffers = DistanceFieldObjectBuffers;
			PassParameters->DistanceFieldAtlas = DistanceFieldAtlas;

			PassParameters->PreViewTranslationHigh = PackedClipmap.PreViewTranslation.High;
			PassParameters->PreViewTranslationLow = PackedClipmap.PreViewTranslation.Low;

			FAllocatePagesCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FAllocatePagesCS::FProcessDistanceFields>(Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0);
			PermutationVector.Set<FAllocatePagesCS::FMarkedHeightfieldPageBuffer>(PackedClipmap.MarkedHeightfieldPageBuffer != nullptr);
			PermutationVector.Set<FAllocatePagesCS::FComposeParentDistanceField>(ParentPageTableLayerTexture != nullptr);
			PermutationVector.Set<FAllocatePagesCS::FOffsetDataStructure>(GDistanceFieldOffsetDataStructure);
			auto ComputeShader = View.ShaderMap->GetShader<FAllocatePagesCS>(PermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("AllocatePages Clipmap:%d", PackedClipmap.Index),
				ComputeShader,
				PassParameters,
				PackedClipmap.PageUpdateIndirectArgBuffer,
				0);
		}

		// Return to the free list
		if (PackedClipmaps.Num() > 0)
		{
			FRDGBufferRef FreeListReturnIndirectArgBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("GlobalDistanceField.FreeListReturnIndirectArgs"));

			// Setup free list return indirect dispatch arguments
			{
				FPageFreeListReturnIndirectArgBufferCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPageFreeListReturnIndirectArgBufferCS::FParameters>();
				PassParameters->RWFreeListReturnIndirectArgBuffer = GraphBuilder.CreateUAV(FreeListReturnIndirectArgBuffer, PF_R32_UINT);
				PassParameters->RWPageFreeListAllocatorBuffer = GraphBuilder.CreateUAV(PageFreeListAllocatorBuffer, PF_R32_SINT);
				PassParameters->PageFreeListReturnAllocatorBuffer = GraphBuilder.CreateSRV(PageFreeListReturnAllocatorBuffer, PF_R32_UINT);

				auto ComputeShader = View.ShaderMap->GetShader<FPageFreeListReturnIndirectArgBufferCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SetupPageFreeListReturnIndirectArgs"),
					ComputeShader,
					PassParameters,
					FIntVector(1, 1, 1));
			}

			// Return to the free list
			{
				FPageFreeListReturnCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPageFreeListReturnCS::FParameters>();
				PassParameters->FreeListReturnIndirectArgBuffer = FreeListReturnIndirectArgBuffer;
				PassParameters->RWPageFreeListAllocatorBuffer = GraphBuilder.CreateUAV(PageFreeListAllocatorBuffer, PF_R32_SINT);
				PassParameters->RWPageFreeListBuffer = GraphBuilder.CreateUAV(PageFreeListBuffer, PF_R32_UINT);
				PassParameters->PageFreeListReturnAllocatorBuffer = GraphBuilder.CreateSRV(PageFreeListReturnAllocatorBuffer, PF_R32_UINT);
				PassParameters->PageFreeListReturnBuffer = GraphBuilder.CreateSRV(PageFreeListReturnBuffer, PF_R32_UINT);

				auto ComputeShader = View.ShaderMap->GetShader<FPageFreeListReturnCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("ReturnToPageFreeList"),
					ComputeShader,
					PassParameters,
					FreeListReturnIndirectArgBuffer,
					0);
			}
		}

		if (CVarGlobalDistanceFieldDebugShowStats.GetValueOnRenderThread())
		{
			for (const FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
			{
				FGlobalDistanceFieldAccumulateUpdatedPagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGlobalDistanceFieldAccumulateUpdatedPagesCS::FParameters>();
					PassParameters->PageComposeIndirectArgBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageComposeIndirectArgBuffer);
					PassParameters->RWPageStatsBuffer = GraphBuilder.CreateUAV(PageStatsBuffer);

				auto ComputeShader = View.ShaderMap->GetShader<FGlobalDistanceFieldAccumulateUpdatedPagesCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("AccumulateUpdatedPages (Debug)"),
					ComputeShader,
					PassParameters,
					FIntVector(1, 1, 1));
			}
		}

		// Composite mesh SDFs into allocated global distance field pages
		{
			const FRDGTextureUAVRef PageAtlasTextureUAV = PageAtlasTexture ? GraphBuilder.CreateUAV(PageAtlasTexture, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
			const FRDGTextureUAVRef CoverageAtlasTextureUAV = CoverageAtlasTexture ? GraphBuilder.CreateUAV(CoverageAtlasTexture, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
			const FRDGTextureUAVRef PageTableCombinedTextureUAV = GraphBuilder.CreateUAV(PageTableCombinedTexture && ParentPageTableLayerTexture ? PageTableCombinedTexture : PageTableLayerTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);

			for (const FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
			{
				if (Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0 || PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions.Num() > 0)
				{
					FCompositeObjectsIntoPagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositeObjectsIntoPagesCS::FParameters>();
					PassParameters->RWPageAtlasTexture = PageAtlasTextureUAV;
					PassParameters->RWCoverageAtlasTexture = CoverageAtlasTextureUAV;
					PassParameters->RWPageTableCombinedTexture = PageTableCombinedTextureUAV;
					PassParameters->ComposeIndirectArgBuffer = PackedClipmap.PageComposeIndirectArgBuffer;
					PassParameters->ComposeTileBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageComposeTileBuffer, PF_R32_UINT);
					PassParameters->ParentPageTableLayerTexture = ParentPageTableLayerTexture;
					PassParameters->CullGridObjectHeader = GraphBuilder.CreateSRV(PackedClipmap.CullGridObjectHeader, PF_R32_UINT);
					PassParameters->CullGridObjectArray = GraphBuilder.CreateSRV(PackedClipmap.CullGridObjectArray, PF_R32_UINT);
					PassParameters->DistanceFieldObjectBuffers = DistanceFieldObjectBuffers;
					PassParameters->DistanceFieldAtlas = DistanceFieldAtlas;
					PassParameters->GlobalDistanceFieldUpdateParameters = PackedClipmap.UpdateParameters;

					FCompositeObjectsIntoPagesCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FCompositeObjectsIntoPagesCS::FComposeParentDistanceField>(ParentPageTableLayerTexture != nullptr);
					PermutationVector.Set<FCompositeObjectsIntoPagesCS::FProcessDistanceFields>(Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0);
					PermutationVector.Set<FCompositeObjectsIntoPagesCS::FCompositeCoverageAtlas>(CoverageAtlasTexture != nullptr);
					PermutationVector.Set<FCompositeObjectsIntoPagesCS::FOffsetDataStructure>(GDistanceFieldOffsetDataStructure);
					auto ComputeShader = View.ShaderMap->GetShader<FCompositeObjectsIntoPagesCS>(PermutationVector);

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("CompositeObjectsIntoPages Clipmap:%d", PackedClipmap.Index),
						ComputeShader,
						PassParameters,
						PackedClipmap.PageComposeIndirectArgBuffer,
						0);
				}
			}
		}

		// Composite mesh SDFs into allocated object grid pages
		if (PageObjectGridBuffer)
		{
			const FRDGBufferUAVRef PageObjectGridBufferUAV = GraphBuilder.CreateUAV(PageObjectGridBuffer, ERDGUnorderedAccessViewFlags::SkipBarrier);

			for (const FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
			{
				if (Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0 || PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions.Num() > 0)
				{
					FCompositeObjectsIntoObjectGridPagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositeObjectsIntoObjectGridPagesCS::FParameters>();
					PassParameters->RWPageObjectGridBuffer = PageObjectGridBufferUAV;
					PassParameters->IndirectArgBuffer = PackedClipmap.BuildObjectGridIndirectArgBuffer;
					PassParameters->DistanceFieldObjectBuffers = DistanceField::SetupObjectBufferParameters(GraphBuilder, DistanceFieldSceneData);
					PassParameters->DistanceFieldAtlas = DistanceFieldAtlas;
					PassParameters->GlobalDistanceFieldUpdateParameters = PackedClipmap.UpdateParameters;
					PassParameters->ComposeTileBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageComposeTileBuffer, PF_R32_UINT);
					PassParameters->PageTableLayerTexture = PageTableLayerTexture;
					PassParameters->ParentPageTableLayerTexture = ParentPageTableLayerTexture;
					PassParameters->CullGridObjectHeader = GraphBuilder.CreateSRV(PackedClipmap.CullGridObjectHeader, PF_R32_UINT);
					PassParameters->CullGridObjectArray = GraphBuilder.CreateSRV(PackedClipmap.CullGridObjectArray, PF_R32_UINT);

					FCompositeObjectsIntoObjectGridPagesCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FCompositeObjectsIntoObjectGridPagesCS::FComposeParentDistanceField>(ParentPageTableLayerTexture != nullptr);
					PermutationVector.Set<FCompositeObjectsIntoObjectGridPagesCS::FProcessDistanceFields>(Scene->DistanceFieldSceneData.NumObjectsInBuffer > 0);
					PermutationVector.Set<FCompositeObjectsIntoObjectGridPagesCS::FOffsetDataStructure>(GDistanceFieldOffsetDataStructure);
					auto ComputeShader = View.ShaderMap->GetShader<FCompositeObjectsIntoObjectGridPagesCS>(PermutationVector);

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("CompositeObjectsIntoObjectGridPages Clipmap:%d", PackedClipmap.Index),
						ComputeShader,
						PassParameters,
						PackedClipmap.BuildObjectGridIndirectArgBuffer,
						0);
				}
			}
		}

		// Composite Heightfields into pages
		if (GAOGlobalDistanceFieldHeightfield != 0)
		{
			const FRDGTextureUAVRef PageAtlasTextureUAV = PageAtlasTexture ? GraphBuilder.CreateUAV(PageAtlasTexture, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;
			const FRDGTextureUAVRef CoverageAtlasTextureUAV = CoverageAtlasTexture ? GraphBuilder.CreateUAV(CoverageAtlasTexture, ERDGUnorderedAccessViewFlags::SkipBarrier) : nullptr;

			for (const FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
			{
				if (PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions.Num() > 0)
				{
					for (TMap<FHeightfieldComponentTextures, TArray<FHeightfieldComponentDescription>>::TConstIterator It(PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions); It; ++It)
					{
						const TArray<FHeightfieldComponentDescription>& HeightfieldDescriptions = It.Value();

						if (HeightfieldDescriptions.Num() > 0)
						{
							FRDGBufferRef HeightfieldDescriptionBuffer = UploadHeightfieldDescriptions(GraphBuilder, HeightfieldDescriptions);

							UTexture2D* HeightfieldTexture = It.Key().HeightAndNormal;
							UTexture2D* VisibilityTexture = It.Key().Visibility;

							FComposeHeightfieldsIntoPagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComposeHeightfieldsIntoPagesCS::FParameters>();
							PassParameters->RWPageAtlasTexture = PageAtlasTextureUAV;
							PassParameters->RWCoverageAtlasTexture = CoverageAtlasTextureUAV;
							PassParameters->ComposeIndirectArgBuffer = PackedClipmap.PageComposeHeightfieldIndirectArgBuffer;
							PassParameters->ComposeTileBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageComposeHeightfieldTileBuffer, PF_R32_UINT);
							PassParameters->PageTableLayerTexture = PageTableLayerTexture;
							PassParameters->InfluenceRadius = PackedClipmap.InfluenceRadius;
							PassParameters->PageCoordToVoxelTranslatedCenterScale = (FVector3f)PackedClipmap.PageCoordToVoxelTranslatedCenterScale;
							PassParameters->PageCoordToVoxelTranslatedCenterBias = (FVector3f)PackedClipmap.PageCoordToVoxelTranslatedCenterBias;
							PassParameters->ClipmapVoxelExtent = PackedClipmap.VoxelExtent.X;
							PassParameters->PageGridResolution = PackedClipmap.PageGridResolution;
							PassParameters->InvPageGridResolution = FVector3f::OneVector / (FVector3f)PackedClipmap.PageGridResolution;
							PassParameters->PageCoordToPageTranslatedWorldCenterScale = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterScale;
							PassParameters->PageCoordToPageTranslatedWorldCenterBias = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterBias;
							PassParameters->ClipmapVolumeTranslatedWorldToUVAddAndMul = PackedClipmap.VolumeTranslatedWorldToUVAddAndMul;
							PassParameters->PageTableClipmapOffsetZ = PackedClipmap.Index * PackedClipmap.PageGridResolution.Z;
							PassParameters->NumHeightfields = HeightfieldDescriptions.Num();
							PassParameters->InfluenceRadius = PackedClipmap.InfluenceRadius;
							PassParameters->HeightfieldThickness = PackedClipmap.VoxelSize.X * GGlobalDistanceFieldHeightFieldThicknessScale;
							PassParameters->HeightfieldTexture = HeightfieldTexture->GetResource()->TextureRHI;
							PassParameters->HeightfieldSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
							PassParameters->VisibilityTexture = VisibilityTexture ? VisibilityTexture->GetResource()->TextureRHI : GBlackTexture->TextureRHI;
							PassParameters->VisibilitySampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
							PassParameters->HeightfieldDescriptions = GraphBuilder.CreateSRV(HeightfieldDescriptionBuffer, EPixelFormat::PF_A32B32G32R32F);
							PassParameters->PreViewTranslationHigh = PackedClipmap.PreViewTranslation.High;
							PassParameters->PreViewTranslationLow = PackedClipmap.PreViewTranslation.Low;

							FComposeHeightfieldsIntoPagesCS::FPermutationDomain PermutationVector;
							PermutationVector.Set<FComposeHeightfieldsIntoPagesCS::FCompositeCoverageAtlas>(CoverageAtlasTexture != nullptr);
							auto ComputeShader = View.ShaderMap->GetShader<FComposeHeightfieldsIntoPagesCS>(PermutationVector);

							FComputeShaderUtils::AddPass(
								GraphBuilder,
								RDG_EVENT_NAME("CompositeHeightfield Clipmap:%d", PackedClipmap.Index),
								ComputeShader,
								PassParameters,
								PackedClipmap.PageComposeHeightfieldIndirectArgBuffer,
								0);
						}
					}
				}
			}
		}

		// Composite heightfields into distance field object grid
		if (GAOGlobalDistanceFieldHeightfield != 0 && PageObjectGridBuffer)
		{
			const FRDGBufferUAVRef PageObjectGridBufferUAV = GraphBuilder.CreateUAV(PageObjectGridBuffer, ERDGUnorderedAccessViewFlags::SkipBarrier);

			for (const FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
			{
				if (PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions.Num() > 0)
				{
					for (TMap<FHeightfieldComponentTextures, TArray<FHeightfieldComponentDescription>>::TConstIterator It(PackedClipmap.UpdateRegionHeightfield.ComponentDescriptions); It; ++It)
					{
						const TArray<FHeightfieldComponentDescription>& HeightfieldDescriptions = It.Value();

						if (HeightfieldDescriptions.Num() > 0)
						{
							FRDGBufferRef HeightfieldDescriptionBuffer = UploadHeightfieldDescriptions(GraphBuilder, HeightfieldDescriptions);

							UTexture2D* HeightfieldTexture = It.Key().HeightAndNormal;
							UTexture2D* VisibilityTexture = It.Key().Visibility;

							FCompositeHeightfieldsIntoObjectGridPagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositeHeightfieldsIntoObjectGridPagesCS::FParameters>();
							PassParameters->RWPageObjectGridBuffer = PageObjectGridBufferUAV;
							PassParameters->ComposeIndirectArgBuffer = PackedClipmap.PageComposeHeightfieldIndirectArgBuffer;
							PassParameters->ComposeTileBuffer = GraphBuilder.CreateSRV(PackedClipmap.PageComposeHeightfieldTileBuffer, PF_R32_UINT);
							PassParameters->PageTableLayerTexture = PageTableLayerTexture;
							PassParameters->InfluenceRadius = PackedClipmap.InfluenceRadius;
							PassParameters->PageCoordToVoxelTranslatedCenterScale = (FVector3f)PackedClipmap.PageCoordToVoxelTranslatedCenterScale;
							PassParameters->PageCoordToVoxelTranslatedCenterBias = (FVector3f)PackedClipmap.PageCoordToVoxelTranslatedCenterBias;
							PassParameters->ClipmapVoxelExtent = PackedClipmap.VoxelExtent.X;
							PassParameters->PageGridResolution = PackedClipmap.PageGridResolution;
							PassParameters->InvPageGridResolution = FVector3f::OneVector / (FVector3f)PackedClipmap.PageGridResolution;
							PassParameters->PageCoordToPageTranslatedWorldCenterScale = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterScale;
							PassParameters->PageCoordToPageTranslatedWorldCenterBias = (FVector3f)PackedClipmap.PageGridCoordToTranslatedWorldCenterBias;
							PassParameters->ClipmapVolumeTranslatedWorldToUVAddAndMul = PackedClipmap.VolumeTranslatedWorldToUVAddAndMul;
							PassParameters->PageTableClipmapOffsetZ = PackedClipmap.Index * PackedClipmap.PageGridResolution.Z;
							PassParameters->NumHeightfields = HeightfieldDescriptions.Num();
							PassParameters->InfluenceRadius = PackedClipmap.InfluenceRadius;
							PassParameters->HeightfieldThickness = PackedClipmap.VoxelSize.X * GGlobalDistanceFieldHeightFieldThicknessScale;
							PassParameters->HeightfieldTexture = HeightfieldTexture->GetResource()->TextureRHI;
							PassParameters->HeightfieldSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
							PassParameters->VisibilityTexture = VisibilityTexture ? VisibilityTexture->GetResource()->TextureRHI : GBlackTexture->TextureRHI;
							PassParameters->VisibilitySampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
							PassParameters->HeightfieldDescriptions = GraphBuilder.CreateSRV(HeightfieldDescriptionBuffer, EPixelFormat::PF_A32B32G32R32F);
							PassParameters->PreViewTranslationHigh = PackedClipmap.PreViewTranslation.High;
							PassParameters->PreViewTranslationLow = PackedClipmap.PreViewTranslation.Low;

							auto ComputeShader = View.ShaderMap->GetShader<FCompositeHeightfieldsIntoObjectGridPagesCS>();

							FComputeShaderUtils::AddPass(
								GraphBuilder,
								RDG_EVENT_NAME("CompositeHeightfieldsIntoObjectGridPages Clipmap:%d", PackedClipmap.Index),
								ComputeShader,
								PassParameters,
								PackedClipmap.PageComposeHeightfieldIndirectArgBuffer,
								0);
						}
					}
				}
			}
		}
	}

	// Coarse Clipmap
	if (PageAtlasTexture && MipTexture && CacheType == GDF_Full)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "Coarse Clipmap");

		const int32 ClipmapMipResolution = GlobalDistanceField::GetClipmapMipResolution(bLumenEnabled);

		// Allocate temporary textures for distance propagation
		for (FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
		{
			PackedClipmap.TempMipTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create3D(
				FIntVector(ClipmapMipResolution),
				PF_R8,
				FClearValueBinding::Black,
				TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling),
				TEXT("GlobalDistanceField.TempMip"));
		}

		// Propagate distance field
		const int32 NumPropagationSteps = 5;
		for (int32 StepIndex = 0; StepIndex < NumPropagationSteps; ++StepIndex)
		{
			const FRDGTextureUAVRef MipTextureUAV = GraphBuilder.CreateUAV(MipTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);

			for (const FGlobalDistanceFieldPackedClipmap& PackedClipmap : PackedClipmaps)
			{
				const FGlobalDistanceFieldClipmap& Clipmap = Clipmaps[PackedClipmap.Index];

				FRDGTextureRef PrevTexture = PackedClipmap.TempMipTexture;
				FRDGTextureUAVRef NextTextureUAV = MipTextureUAV;
				uint32 PrevClipmapOffsetZ = 0;
				uint32 NextClipmapOffsetZ = ClipmapMipResolution * PackedClipmap.Index;

				if (StepIndex % 2 == NumPropagationSteps % 2)
				{
					PrevTexture = MipTexture;
					NextTextureUAV = GraphBuilder.CreateUAV(PackedClipmap.TempMipTexture);

					Swap(PrevClipmapOffsetZ, NextClipmapOffsetZ);
				}

				FPropagateMipDistanceCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPropagateMipDistanceCS::FParameters>();
				PassParameters->RWMipTexture = NextTextureUAV;
				PassParameters->PageTableTexture = GAOGlobalDistanceFieldCacheMostlyStaticSeparately ? PageTableCombinedTexture : PageTableLayerTexture;
				PassParameters->PageAtlasTexture = PageAtlasTexture;
				PassParameters->DistanceFieldSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
				PassParameters->GlobalDistanceFieldInvPageAtlasSize = FVector3f::OneVector / FVector3f(GlobalDistanceField::GetPageAtlasSize(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance));
				PassParameters->GlobalDistanceFieldClipmapSizeInPages = GlobalDistanceField::GetPageTableTextureResolution(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance).X;
				PassParameters->PrevMipTexture = PrevTexture;
				PassParameters->ClipmapMipResolution = ClipmapMipResolution;
				PassParameters->OneOverClipmapMipResolution = 1.0f / ClipmapMipResolution;
				PassParameters->ClipmapIndex = PackedClipmap.Index;
				PassParameters->PrevClipmapOffsetZ = PrevClipmapOffsetZ;
				PassParameters->ClipmapOffsetZ = NextClipmapOffsetZ;
				PassParameters->ClipmapUVScrollOffset = (FVector3f)Clipmap.ScrollOffset / (FVector3f)PackedClipmap.Size;
				PassParameters->CoarseDistanceFieldValueScale = 1.0f / GlobalDistanceField::GetMipFactor();
				PassParameters->CoarseDistanceFieldValueBias = 0.5f - 0.5f / GlobalDistanceField::GetMipFactor();

				FPropagateMipDistanceCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FPropagateMipDistanceCS::FReadPages>(StepIndex == 0);
				auto ComputeShader = View.ShaderMap->GetShader<FPropagateMipDistanceCS>(PermutationVector);

				const FIntVector GroupSize = FComputeShaderUtils::GetGroupCount(FIntVector(ClipmapMipResolution), FPropagateMipDistanceCS::GetGroupSize());

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("Propagate Clipmap:%d Step %d", PackedClipmap.Index, StepIndex),
					ComputeShader,
					PassParameters,
					GroupSize);
			}
		}
	}
}

/** 
 * Updates the global distance field for a view.  
 * Typically issues updates for just the newly exposed regions of the volume due to camera movement.
 * In the worst case of a camera cut or large distance field scene changes, a full update of the global distance field will be done.
 **/
void UpdateGlobalDistanceFieldVolume(
	FRDGBuilder& GraphBuilder,
	FRDGExternalAccessQueue& ExternalAccessQueue,
	FViewInfo& View,
	FScene* Scene,
	float MaxOcclusionDistance,
	bool bLumenEnabled,
	FGlobalDistanceFieldInfo& GlobalDistanceFieldInfo)
{
	RDG_RHI_EVENT_SCOPE_STAT(GraphBuilder, GlobalDistanceFieldUpdate, "GlobalDistanceFieldUpdate");

	const FDistanceFieldSceneData& DistanceFieldSceneData = Scene->DistanceFieldSceneData;
	bool bNeedsFinalizeAccess = true;

	if (DistanceFieldSceneData.NumObjectsInBuffer > 0 || DistanceFieldSceneData.HeightfieldPrimitives.Num() > 0)
	{
		const int32 NumClipmaps = FMath::Clamp<int32>(GlobalDistanceField::GetNumGlobalDistanceFieldClipmaps(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance), 0, GlobalDistanceField::MaxClipmaps);
		ComputeUpdateRegionsAndUpdateViewState(GraphBuilder.RHICmdList, View, Scene, GlobalDistanceFieldInfo, NumClipmaps, MaxOcclusionDistance, bLumenEnabled);

		if (!View.CachedViewUniformShaderParameters)
		{
			View.CachedViewUniformShaderParameters = MakeUnique<FViewUniformShaderParameters>();
		}

		// Recreate the view uniform buffer now that we have updated GlobalDistanceFieldInfo
		View.SetupGlobalDistanceFieldUniformBufferParameters(*View.CachedViewUniformShaderParameters);

		FRDGBufferRef PageStatsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 64), TEXT("GlobalDistanceField.PageStatsBuffer"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PageStatsBuffer), 0);

		bool bHasUpdateBounds = false;

		for (int32 ClipmapIndex = 0; ClipmapIndex < GlobalDistanceFieldInfo.Clipmaps.Num(); ClipmapIndex++)
		{
			bHasUpdateBounds = bHasUpdateBounds || GlobalDistanceFieldInfo.Clipmaps[ClipmapIndex].UpdateBounds.Num() > 0;
		}

		for (int32 ClipmapIndex = 0; ClipmapIndex < GlobalDistanceFieldInfo.MostlyStaticClipmaps.Num(); ClipmapIndex++)
		{
			bHasUpdateBounds = bHasUpdateBounds || GlobalDistanceFieldInfo.MostlyStaticClipmaps[ClipmapIndex].UpdateBounds.Num() > 0;
		}

		if (bHasUpdateBounds)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "UpdateGlobalDistanceField");

			bNeedsFinalizeAccess = false;
			const FGlobalDFCacheType StartCacheType = GAOGlobalDistanceFieldCacheMostlyStaticSeparately ? GDF_MostlyStatic : GDF_Full;
			FGlobalDistanceFieldInfoRDG GlobalDistanceFieldInfoRDG;

			RegisterGlobalDistanceFieldExternalResources(GraphBuilder, GlobalDistanceFieldInfo, StartCacheType, GlobalDistanceFieldInfoRDG);

			FRDGBufferRef PageFreeListAllocatorBuffer = GlobalDistanceFieldInfoRDG.PageFreeListAllocatorBuffer;
			FRDGBufferRef PageFreeListBuffer = GlobalDistanceFieldInfoRDG.PageFreeListBuffer;
			FRDGTextureRef PageAtlasTexture = GlobalDistanceFieldInfoRDG.PageAtlasTexture;
			FRDGTextureRef CoverageAtlasTexture = GlobalDistanceFieldInfoRDG.CoverageAtlasTexture;
			FRDGBufferRef PageObjectGridBuffer = GlobalDistanceFieldInfoRDG.PageObjectGridBuffer;
			FRDGTextureRef PageTableCombinedTexture = GlobalDistanceFieldInfoRDG.PageTableCombinedTexture;
			FRDGTextureRef MipTexture = GlobalDistanceFieldInfoRDG.MipTexture;
			FRDGTextureRef (&PageTableLayerTextures)[GDF_Num] = GlobalDistanceFieldInfoRDG.PageTableLayerTextures;

			if (View.ViewState && View.ViewState->GlobalDistanceFieldData->bPendingReset)
			{
				// Reset all allocators to default

				const uint32 PageTableClearValue[4] = { 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF };

				if (PageTableCombinedTexture)
				{
					AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PageTableCombinedTexture), PageTableClearValue);
				}

				for (int32 CacheType = StartCacheType; CacheType < GDF_Num; ++CacheType)
				{
					if (PageTableLayerTextures[CacheType])
					{
						AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PageTableLayerTextures[CacheType]), PageTableClearValue);
					}
				}

				const int32 MaxPageNum = GlobalDistanceField::GetMaxPageNum(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance);

				if (PageFreeListAllocatorBuffer)
				{
					FInitPageFreeListCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInitPageFreeListCS::FParameters>();
					PassParameters->RWPageFreeListBuffer = GraphBuilder.CreateUAV(PageFreeListBuffer, PF_R32_UINT);
					PassParameters->RWPageFreeListAllocatorBuffer = GraphBuilder.CreateUAV(PageFreeListAllocatorBuffer, PF_R32_SINT);
					PassParameters->GlobalDistanceFieldMaxPageNum = MaxPageNum;

					auto ComputeShader = View.ShaderMap->GetShader<FInitPageFreeListCS>();

					const FIntVector GroupSize = FComputeShaderUtils::GetGroupCount(MaxPageNum, FInitPageFreeListCS::GetGroupSize());

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("InitPageFreeList"),
						ComputeShader,
						PassParameters,
						GroupSize);
				}

				View.ViewState->GlobalDistanceFieldData->bPendingReset = false;
			}

			for (int32 CacheType = StartCacheType; CacheType < GDF_Num; CacheType++)
			{
				FRDGTextureRef PageTableLayerTexture = PageTableLayerTextures[CacheType];
				FRDGTextureRef ParentPageTableLayerTexture = nullptr;

				if (CacheType == GDF_Full && GAOGlobalDistanceFieldCacheMostlyStaticSeparately && PageTableLayerTextures[GDF_MostlyStatic])
				{
					ParentPageTableLayerTexture = PageTableLayerTextures[GDF_MostlyStatic];
				}

				TArray<FGlobalDistanceFieldClipmap>& Clipmaps = CacheType == GDF_MostlyStatic
					? GlobalDistanceFieldInfo.MostlyStaticClipmaps
					: GlobalDistanceFieldInfo.Clipmaps;

				UpdateGlobalDistanceFieldCache(
					GraphBuilder,
					View,
					Scene,
					CacheType,
					Clipmaps,
					PageStatsBuffer,
					PageObjectGridBuffer,
					PageTableLayerTexture,
					ParentPageTableLayerTexture,
					PageTableCombinedTexture,
					PageAtlasTexture,
					CoverageAtlasTexture,
					MipTexture,
					PageFreeListAllocatorBuffer,
					PageFreeListBuffer,
					bLumenEnabled);
			}

			FinalizeGlobalDistanceFieldExternalResourceAccess(GraphBuilder, ExternalAccessQueue, GlobalDistanceFieldInfo, StartCacheType, GlobalDistanceFieldInfoRDG);
		}

		if (CVarGlobalDistanceFieldDebugShowStats.GetValueOnRenderThread()
			&& GlobalDistanceFieldInfo.PageFreeListAllocatorBuffer 
			&& GlobalDistanceFieldInfo.PageAtlasTexture)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "GlobalDistanceFieldDebug");

			ShaderPrint::SetEnabled(true);

			FRDGTextureRef PageTableCombinedTexture = GraphBuilder.RegisterExternalTexture(GAOGlobalDistanceFieldCacheMostlyStaticSeparately ? GlobalDistanceFieldInfo.PageTableCombinedTexture : GlobalDistanceFieldInfo.PageTableLayerTextures[0], TEXT("GlobalDistanceField.PageTableCombined"));
			FRDGTextureRef PageAtlasTexture = GraphBuilder.RegisterExternalTexture(GlobalDistanceFieldInfo.PageAtlasTexture, TEXT("GlobalDistanceField.PageAtlasTexture"));

			for (int32 ClipmapIndex = 0; ClipmapIndex < GlobalDistanceFieldInfo.ParameterData.NumGlobalSDFClipmaps; ClipmapIndex++)
			{
				FGlobalDistanceFieldPageStatsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGlobalDistanceFieldPageStatsCS::FParameters>();
				PassParameters->RWPageStatsBuffer = GraphBuilder.CreateUAV(PageStatsBuffer);
				PassParameters->PageTableCombinedTexture = PageTableCombinedTexture;
				PassParameters->PageAtlasTexture = PageAtlasTexture;
				PassParameters->DistanceFieldSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
				PassParameters->ClipmapSizeInPages = GlobalDistanceField::GetPageTableTextureResolution(bLumenEnabled, View.FinalPostProcessSettings.LumenSceneViewDistance).X;
				PassParameters->ClipmapIndex = ClipmapIndex;
				PassParameters->InvPageAtlasSize = FVector3f(GlobalDistanceFieldInfo.ParameterData.InvPageAtlasSize);

				auto ComputeShader = View.ShaderMap->GetShader<FGlobalDistanceFieldPageStatsCS>();

				const FIntVector GroupSize = FComputeShaderUtils::GetGroupCount(FIntVector(PassParameters->ClipmapSizeInPages), FGlobalDistanceFieldPageStatsCS::GetGroupSize());

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("PageStats Clipmap:%d", ClipmapIndex),
					ComputeShader,
					PassParameters,
					GroupSize);
			}

			if (ShaderPrint::IsEnabled(View.ShaderPrintData))
			{
				FRDGBufferRef PageFreeListAllocatorBuffer = GraphBuilder.RegisterExternalBuffer(GlobalDistanceFieldInfo.PageFreeListAllocatorBuffer);

				FGlobalDistanceFieldDebugCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGlobalDistanceFieldDebugCS::FParameters>();
				ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintUniformBuffer);
				PassParameters->GlobalDistanceFieldPageFreeListAllocatorBuffer = GraphBuilder.CreateSRV(PageFreeListAllocatorBuffer, PF_R32_UINT);
				PassParameters->GlobalDistanceFieldMaxPageNum = View.GlobalDistanceFieldInfo.ParameterData.MaxPageNum;
				PassParameters->PageStatsBuffer = GraphBuilder.CreateSRV(PageStatsBuffer);

				auto ComputeShader = View.ShaderMap->GetShader<FGlobalDistanceFieldDebugCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("Print Stats"),
					ComputeShader,
					PassParameters,
					FIntVector(1, 1, 1));
			}
		}
	}

#if ENABLE_RHI_VALIDATION
	if (bNeedsFinalizeAccess)
	{
		const FGlobalDFCacheType StartCacheType = GAOGlobalDistanceFieldCacheMostlyStaticSeparately ? GDF_MostlyStatic : GDF_Full;
		FGlobalDistanceFieldInfoRDG GlobalDistanceFieldInfoRDG;

		RegisterGlobalDistanceFieldExternalResources(GraphBuilder, GlobalDistanceFieldInfo, StartCacheType, GlobalDistanceFieldInfoRDG);
		FinalizeGlobalDistanceFieldExternalResourceAccess(GraphBuilder, ExternalAccessQueue, GlobalDistanceFieldInfo, StartCacheType, GlobalDistanceFieldInfoRDG);
	}
#endif

	if (GDFReadbackRequest && GlobalDistanceFieldInfo.Clipmaps.Num() > 0)
	{
		// Read back a clipmap
		ReadbackDistanceFieldClipmap(GraphBuilder.RHICmdList, GlobalDistanceFieldInfo);
	}

	if (GDFReadbackRequest && GlobalDistanceFieldInfo.Clipmaps.Num() > 0)
	{
		// Read back a clipmap
		ReadbackDistanceFieldClipmap(GraphBuilder.RHICmdList, GlobalDistanceFieldInfo);
	}

	// Mark that we've update this global distance field data this frame
	if (View.ViewState)
	{
		View.ViewState->GlobalDistanceFieldData->bFirstFrame = false;
		View.ViewState->GlobalDistanceFieldData->UpdateFrame = View.Family->FrameNumber;
	}
}
