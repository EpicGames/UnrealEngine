// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/IntVector.h"

class FDistanceFieldAOParameters;
class FGlobalDistanceFieldInfo;
class FRDGBuilder;
class FRDGExternalAccessQueue;
class FScene;
class FSceneViewFamily;
class FViewInfo;

struct FSceneRenderUpdateInputs;

enum FGlobalDFCacheType
{
	GDF_MostlyStatic,
	GDF_Full,
	GDF_Num
};

extern int32 GAOGlobalDistanceField;

bool UseGlobalDistanceField();
bool UseGlobalDistanceField(const FDistanceFieldAOParameters& Parameters);

namespace GlobalDistanceField
{
	int32 GetClipmapResolution(bool bLumenEnabled);
	int32 GetMipFactor();
	int32 GetClipmapMipResolution(bool bLumenEnabled);
	float GetClipmapExtent(int32 ClipmapIndex, const FScene* Scene, bool bLumenEnabled);
	int32 GetNumGlobalDistanceFieldClipmaps(bool bLumenEnabled, float LumenSceneViewDistance);

	FIntVector GetPageAtlasSizeInPages(bool bLumenEnabled, float LumenSceneViewDistance);
	FIntVector GetPageAtlasSize(bool bLumenEnabled, float LumenSceneViewDistance);
	FIntVector GetCoverageAtlasSize(bool bLumenEnabled, float LumenSceneViewDistance);
	uint32 GetPageTableClipmapResolution(bool bLumenEnabled);
	FIntVector GetPageTableTextureResolution(bool bLumenEnabled, float LumenSceneViewDistance);
	int32 GetMaxPageNum(bool bLumenEnabled, float LumenSceneViewDistance);
};

/** 
 * Updates the global distance field for a view.  
 * Typically issues updates for just the newly exposed regions of the volume due to camera movement.
 * In the worst case of a camera cut or large distance field scene changes, a full update of the global distance field will be done.
 **/
extern void UpdateGlobalDistanceFieldVolume(
	FRDGBuilder& GraphBuilder,
	FRDGExternalAccessQueue& ExternalAccessQueue,
	FViewInfo& View, 
	FScene* Scene, 
	float MaxOcclusionDistance, 
	bool bLumenEnabled,
	FGlobalDistanceFieldInfo& Info);