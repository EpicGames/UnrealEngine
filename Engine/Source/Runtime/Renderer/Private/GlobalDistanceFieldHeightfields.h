// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GlobalDistanceFieldHeightfields.h
=============================================================================*/

#pragma once

// HEADER_UNIT_SKIP - Internal

class FHeightfieldComponentDescription;

class FMarkHeightfieldPagesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMarkHeightfieldPagesCS);
	SHADER_USE_PARAMETER_STRUCT(FMarkHeightfieldPagesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWMarkedHeightfieldPageBuffer)
		RDG_BUFFER_ACCESS(PageUpdateIndirectArgBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PageUpdateTileBuffer)
		SHADER_PARAMETER(FVector3f, PageCoordToVoxelTranslatedCenterScale)
		SHADER_PARAMETER(FVector3f, PageCoordToVoxelTranslatedCenterBias)
		SHADER_PARAMETER(FVector3f, PageWorldExtent)
		SHADER_PARAMETER(FVector3f, InvPageGridResolution)
		SHADER_PARAMETER(FIntVector, PageGridResolution)
		SHADER_PARAMETER(float, ClipmapVoxelExtent)
		SHADER_PARAMETER(float, InfluenceRadius)
		SHADER_PARAMETER_TEXTURE(Texture2D, HeightfieldTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, HeightfieldSampler)
		SHADER_PARAMETER_TEXTURE(Texture2D, VisibilityTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, VisibilitySampler)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, HeightfieldDescriptions)
		SHADER_PARAMETER(uint32, NumHeightfields)
		SHADER_PARAMETER(float, HeightfieldThickness)

		SHADER_PARAMETER(FVector3f, PreViewTranslationHigh)
		SHADER_PARAMETER(FVector3f, PreViewTranslationLow)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(8, 8, 1);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize().X);
	}												   
};

IMPLEMENT_GLOBAL_SHADER(FMarkHeightfieldPagesCS, "/Engine/Private/DistanceField/GlobalDistanceFieldHeightfields.usf", "MarkHeightfieldPagesCS", SF_Compute);

class FBuildHeightfieldComposeTilesIndirectArgBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildHeightfieldComposeTilesIndirectArgBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FBuildHeightfieldComposeTilesIndirectArgBufferCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWBuildHeightfieldComposeTilesIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWPageComposeHeightfieldIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PageUpdateIndirectArgBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FBuildHeightfieldComposeTilesIndirectArgBufferCS, "/Engine/Private/DistanceField/GlobalDistanceFieldHeightfields.usf", "BuildHeightfieldComposeTilesIndirectArgBufferCS", SF_Compute);

class FBuildHeightfieldComposeTilesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildHeightfieldComposeTilesCS);
	SHADER_USE_PARAMETER_STRUCT(FBuildHeightfieldComposeTilesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWPageComposeHeightfieldIndirectArgBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWPageComposeHeightfieldTileBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, PageUpdateTileBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, MarkedHeightfieldPageBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PageUpdateIndirectArgBuffer)
		RDG_BUFFER_ACCESS(BuildHeightfieldComposeTilesIndirectArgBuffer, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(64, 1, 1);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize().X);
	}												   
};

IMPLEMENT_GLOBAL_SHADER(FBuildHeightfieldComposeTilesCS, "/Engine/Private/DistanceField/GlobalDistanceFieldHeightfields.usf", "BuildHeightfieldComposeTilesCS", SF_Compute);

class FComposeHeightfieldsIntoPagesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComposeHeightfieldsIntoPagesCS);
	SHADER_USE_PARAMETER_STRUCT(FComposeHeightfieldsIntoPagesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, RWPageAtlasTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<UNORM float>, RWCoverageAtlasTexture)
		RDG_BUFFER_ACCESS(ComposeIndirectArgBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ComposeTileBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<uint>, PageTableLayerTexture)
		SHADER_PARAMETER(FVector3f, InvPageGridResolution)
		SHADER_PARAMETER(FIntVector, PageGridResolution)
		SHADER_PARAMETER(FVector3f, PageCoordToVoxelTranslatedCenterScale)
		SHADER_PARAMETER(FVector3f, PageCoordToVoxelTranslatedCenterBias)
		SHADER_PARAMETER(FVector3f, PageCoordToPageTranslatedWorldCenterScale)
		SHADER_PARAMETER(FVector3f, PageCoordToPageTranslatedWorldCenterBias)
		SHADER_PARAMETER(FVector4f, ClipmapVolumeTranslatedWorldToUVAddAndMul)
		SHADER_PARAMETER(float, ClipmapVoxelExtent)
		SHADER_PARAMETER(float, InfluenceRadius)
		SHADER_PARAMETER(uint32, PageTableClipmapOffsetZ)
		SHADER_PARAMETER_TEXTURE(Texture2D, HeightfieldTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, HeightfieldSampler)
		SHADER_PARAMETER_TEXTURE(Texture2D, VisibilityTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, VisibilitySampler)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, HeightfieldDescriptions)
		SHADER_PARAMETER(uint32, NumHeightfields)
		SHADER_PARAMETER(float, HeightfieldThickness)
		SHADER_PARAMETER(FVector3f, PreViewTranslationHigh)
		SHADER_PARAMETER(FVector3f, PreViewTranslationLow)
	END_SHADER_PARAMETER_STRUCT()

	class FCompositeCoverageAtlas : SHADER_PERMUTATION_BOOL("COMPOSITE_COVERAGE_ATLAS");
	using FPermutationDomain = TShaderPermutationDomain<FCompositeCoverageAtlas>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(8, 8, 1);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize().X);
	}												   
};

IMPLEMENT_GLOBAL_SHADER(FComposeHeightfieldsIntoPagesCS, "/Engine/Private/DistanceField/GlobalDistanceFieldHeightfields.usf", "ComposeHeightfieldsIntoPagesCS", SF_Compute);

class FCompositeHeightfieldsIntoObjectGridPagesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositeHeightfieldsIntoObjectGridPagesCS);
	SHADER_USE_PARAMETER_STRUCT(FCompositeHeightfieldsIntoObjectGridPagesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint4>, RWPageObjectGridBuffer)
		RDG_BUFFER_ACCESS(ComposeIndirectArgBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ComposeTileBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D<uint>, PageTableLayerTexture)
		SHADER_PARAMETER(FVector3f, InvPageGridResolution)
		SHADER_PARAMETER(FIntVector, PageGridResolution)
		SHADER_PARAMETER(FVector3f, PageCoordToVoxelTranslatedCenterScale)
		SHADER_PARAMETER(FVector3f, PageCoordToVoxelTranslatedCenterBias)
		SHADER_PARAMETER(FVector3f, PageCoordToPageTranslatedWorldCenterScale)
		SHADER_PARAMETER(FVector3f, PageCoordToPageTranslatedWorldCenterBias)
		SHADER_PARAMETER(FVector4f, ClipmapVolumeTranslatedWorldToUVAddAndMul)
		SHADER_PARAMETER(float, ClipmapVoxelExtent)
		SHADER_PARAMETER(float, InfluenceRadius)
		SHADER_PARAMETER(uint32, PageTableClipmapOffsetZ)
		SHADER_PARAMETER_TEXTURE(Texture2D, HeightfieldTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, HeightfieldSampler)
		SHADER_PARAMETER_TEXTURE(Texture2D, VisibilityTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, VisibilitySampler)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, HeightfieldDescriptions)
		SHADER_PARAMETER(uint32, NumHeightfields)
		SHADER_PARAMETER(float, HeightfieldThickness)
		SHADER_PARAMETER(FVector3f, PreViewTranslationHigh)
		SHADER_PARAMETER(FVector3f, PreViewTranslationLow)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileDistanceFieldAOShaders(Parameters.Platform);
	}

	static FIntVector GetGroupSize()
	{
		return FIntVector(4, 4, 4);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize().X);
	}												   
};

IMPLEMENT_GLOBAL_SHADER(FCompositeHeightfieldsIntoObjectGridPagesCS, "/Engine/Private/DistanceField/GlobalDistanceFieldHeightfields.usf", "CompositeHeightfieldsIntoObjectGridPagesCS", SF_Compute);

extern FRDGBufferRef UploadHeightfieldDescriptions(FRDGBuilder& GraphBuilder, const TArray<FHeightfieldComponentDescription>& HeightfieldDescriptions);