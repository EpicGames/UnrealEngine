// Copyright Epic Games, Inc. All Rights Reserved.

#include "HZB.h"
#include "SystemTextures.h"
#include "RenderGraphUtils.h"
#include "SceneRendering.h"

bool operator&(EHZBType A, EHZBType B)
{
	return (uint32(A) & uint32(B)) != 0;
}

static FHZBParameters InitHZBCommonParameter(FRDGBuilder& GraphBuilder, const FViewInfo* View, const FIntRect& InViewRect, const FIntPoint& InHZBTextureExtent)
{
	// ViewportUV to HZBBufferUV
	const FIntPoint HZBMipmap0Size = InHZBTextureExtent;
	const FVector2D HZBUvFactor(float(InViewRect.Width()) / float(2 * HZBMipmap0Size.X), float(InViewRect.Height()) / float(2 * HZBMipmap0Size.Y));
	const FVector4f ScreenPositionScaleBias = View ? View->GetScreenPositionScaleBias(View->GetSceneTextures().Config.Extent, InViewRect) : FVector4f(1.f, 1.f, 0.f, 0.f);
	const FVector2f HZBUVToScreenUVScale = FVector2f(1.0f / HZBUvFactor.X, 1.0f / HZBUvFactor.Y) * FVector2f(2.0f, -2.0f) * FVector2f(ScreenPositionScaleBias.X, ScreenPositionScaleBias.Y);
	const FVector2f HZBUVToScreenUVBias = FVector2f(-1.0f, 1.0f) * FVector2f(ScreenPositionScaleBias.X, ScreenPositionScaleBias.Y) + FVector2f(ScreenPositionScaleBias.W, ScreenPositionScaleBias.Z);
	
	FRDGTextureRef DummyTexture = GSystemTextures.GetBlackDummy(GraphBuilder);

	FHZBParameters Out;
	Out.HZBSize = FVector2f(HZBMipmap0Size);
	Out.HZBViewSize = FVector2f(InViewRect.Size());
	Out.HZBViewRect = FIntRect(0, 0, InViewRect.Width(), InViewRect.Height());
	Out.HZBBaseTexelSize = FVector2f(1.0f / InHZBTextureExtent.X, 1.0f / InHZBTextureExtent.Y);
	Out.HZBUVToScreenUVScaleBias = FVector4f(HZBUVToScreenUVScale, HZBUVToScreenUVBias);
	Out.HZBUvFactorAndInvFactor = FVector4f(HZBUvFactor.X, HZBUvFactor.Y, 1.0f / HZBUvFactor.X, 1.0f / HZBUvFactor.Y);
	Out.ViewportUVToHZBBufferUV = FVector2f(HZBUvFactor.X, HZBUvFactor.Y);
	Out.SamplePixelToHZBUV = FVector2f(
		0.5f / float(InHZBTextureExtent.X),
		0.5f / float(InHZBTextureExtent.Y));
	Out.ScreenPosToHZBUVScaleBias = FVector4f::Zero();

	Out.bIsHZBValid = 0;
	Out.bIsFurthestHZBValid = 0;
	Out.bIsClosestHZBValid = 0;

	Out.HZBTexture = DummyTexture;
	Out.FurthestHZBTexture = DummyTexture;
	Out.ClosestHZBTexture = DummyTexture;

	Out.HZBSampler = TStaticSamplerState<SF_Point>::GetRHI();
	Out.HZBTextureSampler = Out.HZBSampler;
	Out.FurthestHZBTextureSampler = Out.HZBSampler;
	Out.ClosestHZBTextureSampler = Out.HZBSampler;	

	return Out;
}

FHZBParameters GetHZBParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, EHZBType InHZBTypes, FRDGTextureRef InClosestHZB, FRDGTextureRef InFurthestHZB)
{
	FIntPoint TextureExtent;
	if (InHZBTypes & EHZBType::FurthestHZB)
	{
		TextureExtent = InFurthestHZB ? InFurthestHZB->Desc.Extent : FIntPoint(1,1);
	}
	else
	{
		TextureExtent = InClosestHZB ? InClosestHZB->Desc.Extent : FIntPoint(1,1);
	}

	FHZBParameters Out = InitHZBCommonParameter(GraphBuilder, &View, View.ViewRect, TextureExtent);

	if ((InHZBTypes & EHZBType::FurthestHZB) && InFurthestHZB)
	{
		Out.bIsHZBValid = 1u;
		Out.HZBTexture = InFurthestHZB;
	
		check(InFurthestHZB && HasBeenProduced(InFurthestHZB));
		Out.bIsFurthestHZBValid = 1u;
		Out.FurthestHZBTexture = InFurthestHZB;
	}

	if ((InHZBTypes & EHZBType::ClosestHZB) && InClosestHZB)
	{
		check(InClosestHZB && HasBeenProduced(InClosestHZB));
		Out.bIsClosestHZBValid = 1;
		Out.ClosestHZBTexture = InClosestHZB;
	}

	// Sanity check
	//check(ClosestHZB.Desc.Extent == FurthestHZB.Desc.Extent);
	return Out;
}

FHZBParameters GetHZBParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, EHZBType InHZBTypes)
{
	return GetHZBParameters(GraphBuilder, View, InHZBTypes, View.ClosestHZB, View.HZB);
}

FHZBParameters GetHZBParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, bool bUsePreviousHZBAsFallback)
{
	const bool bUseCurrent = View.HZB != nullptr;
	const FRDGTextureRef HZB = bUseCurrent ? View.HZB : GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.HZB);
	const FIntRect ViewRect = bUseCurrent ? View.ViewRect : View.PrevViewInfo.ViewRect;

	FHZBParameters Out = InitHZBCommonParameter(GraphBuilder, &View, ViewRect, HZB ? HZB->Desc.Extent : FIntPoint(1,1));
	if (HZB)
	{
		Out.HZBTexture = HZB;
		Out.FurthestHZBTexture = HZB;
		Out.bIsHZBValid = 1u;
		Out.bIsFurthestHZBValid = 1u;
	}
	return Out;
}

FHZBParameters GetDummyHZBParameters(FRDGBuilder& GraphBuilder)
{
	return InitHZBCommonParameter(GraphBuilder, nullptr, FIntRect(0,0,1,1), FIntPoint(1,1));
}

bool IsHZBValid(const FViewInfo& View, EHZBType InHZBTypes, bool bCheckIfProduced)
{
	bool bOut = true;

	if (InHZBTypes & EHZBType::FurthestHZB)
	{
		bOut = bOut && View.HZB && (bCheckIfProduced ? HasBeenProduced(View.HZB) : true);
	}

	if (InHZBTypes & EHZBType::ClosestHZB)
	{
		bOut = bOut && View.ClosestHZB && (bCheckIfProduced ? HasBeenProduced(View.ClosestHZB) : true);
	}

	return bOut;
}

bool IsPreviousHZBValid(const FViewInfo& View, EHZBType InHZBTypes)
{
	bool bOut = true;

	if (InHZBTypes & EHZBType::FurthestHZB)
	{
		bOut = bOut && View.PrevViewInfo.HZB;
	}

	if (InHZBTypes & EHZBType::ClosestHZB)
	{
		bOut = false;
	}

	return bOut;

}

FRDGTextureRef GetHZBTexture(const FViewInfo& View, EHZBType InHZBTypes)
{
	checkf(InHZBTypes == EHZBType::FurthestHZB || InHZBTypes == EHZBType::ClosestHZB, TEXT("HZB texture can only be requested with ClosestHZB or FurthestHZB value"));

	if (InHZBTypes & EHZBType::FurthestHZB)
	{
		return View.HZB;
	}

	if (InHZBTypes & EHZBType::ClosestHZB)
	{
		return View.ClosestHZB;
	}

	return nullptr;
}