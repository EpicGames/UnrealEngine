// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "ScreenPass.h"

class FViewInfo;

// Include "HZB.ush" on the shader side to declare and use these shader parameters

BEGIN_SHADER_PARAMETER_STRUCT(FHZBParameters, )
	// Default HZB (furthest depth)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HZBTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, HZBSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, HZBTextureSampler)

	// Closest depth HZB
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ClosestHZBTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ClosestHZBTextureSampler)

	// Furthest depth HZB
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FurthestHZBTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, FurthestHZBTextureSampler)

	// Common parameters
	SHADER_PARAMETER(FVector2f, HZBSize)
	SHADER_PARAMETER(FVector2f, HZBViewSize)
	SHADER_PARAMETER(FIntRect,  HZBViewRect)
	SHADER_PARAMETER(FVector2f, ViewportUVToHZBBufferUV)
	SHADER_PARAMETER(FVector4f, HZBUvFactorAndInvFactor)
	SHADER_PARAMETER(FVector4f, HZBUVToScreenUVScaleBias)
	SHADER_PARAMETER(FVector2f, HZBBaseTexelSize)
	SHADER_PARAMETER(FVector2f, SamplePixelToHZBUV)
	SHADER_PARAMETER(uint32,    bIsHZBValid)
	SHADER_PARAMETER(uint32,    bIsFurthestHZBValid)
	SHADER_PARAMETER(uint32,    bIsClosestHZBValid)
	SHADER_PARAMETER(FVector4f, ScreenPosToHZBUVScaleBias)
	SHADER_PARAMETER(FVector2f, DummyHZB)
END_SHADER_PARAMETER_STRUCT()

enum class EHZBType : uint8
{
	Dummy = 0,
	ClosestHZB = 1,
	FurthestHZB = 2,
	All = ClosestHZB | FurthestHZB
};

FHZBParameters GetHZBParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, EHZBType InHZBTypes, FRDGTextureRef InClosestHZB, FRDGTextureRef InFurthestHZB);
FHZBParameters GetHZBParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, EHZBType InHZBTypes);
FHZBParameters GetHZBParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, bool bUsePreviousHZBAsFallback);
FHZBParameters GetDummyHZBParameters(FRDGBuilder& GraphBuilder);

bool IsHZBValid(const FViewInfo& View, EHZBType InHZBTypes, bool bCheckIfProduced=false);
bool IsPreviousHZBValid(const FViewInfo& View, EHZBType InHZBTypes);

FRDGTextureRef GetHZBTexture(const FViewInfo& View, EHZBType InHZBTypes);