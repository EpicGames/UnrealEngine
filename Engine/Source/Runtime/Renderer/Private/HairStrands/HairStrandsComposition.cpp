// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsComposition.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterMacros.h"
#include "RenderGraphUtils.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneRenderTargets.h"
#include "HairStrandsRendering.h"
#include "HairStrandsTile.h"
#include "FogRendering.h"
#include "PostProcess/TemporalAA.h"
#include "HairStrandsForwardRaster.h"
#include "SingleLayerWaterRendering.h"

/////////////////////////////////////////////////////////////////////////////////////////

DECLARE_GPU_STAT(HairStrandsComposition);

static int32 GHairFastResolveVelocityThreshold = 1;
static FAutoConsoleVariableRef CVarHairFastResolveVelocityThreshold(TEXT("r.HairStrands.VelocityThreshold"), GHairFastResolveVelocityThreshold, TEXT("Threshold value (in pixel) above which a pixel is forced to be resolve with responsive AA (in order to avoid smearing). Default is 3."));

static int32 GHairWriteGBufferData = 1;
static FAutoConsoleVariableRef CVarHairWriteGBufferData(TEXT("r.HairStrands.WriteGBufferData"), GHairWriteGBufferData, TEXT("Write hair hair material data into GBuffer before post processing run. 0: no write, 1: dummy write into GBuffer A/B (Normal/ShadingModel), 2: write into GBuffer A/B (Normal/ShadingModel). 3: Write entire GBuffer data. (default 1)."));

static int32 GHairStrandsHoldoutMode = 1;
static FAutoConsoleVariableRef CVarHairStrandsHoldoutMode(TEXT("r.HairStrands.HoldoutMode"), GHairStrandsHoldoutMode, TEXT("Change how sample are merged when rendering with holdout."));

/////////////////////////////////////////////////////////////////////////////////////////

float GetHairFastResolveVelocityThreshold(const FIntPoint& Resolution)
{
	FVector2f PixelVelocity(1.f / (Resolution.X * 2), 1.f / (Resolution.Y * 2));
	const float VelocityThreshold = FMath::Clamp(GHairFastResolveVelocityThreshold, 0, 512) * FMath::Min(PixelVelocity.X, PixelVelocity.Y);
	return VelocityThreshold;
}

enum EHairStrandsCommonPassType
{
	Composition,
	DOF,
	TAAFastResolve,
	GBuffer,
	Blit,
	HoldoutScene,
	HoldoutHair,
};

enum EHairSLWCompositionType
{
	All,
	BehindWater,
	InFrontOfWater,
};

template<typename TPassParameter, typename TPixelShader>
void InternalCommonDrawPass(
	FRDGBuilder& GraphBuilder, 
	FRDGEventName&& EventName,
	const FViewInfo& View,
	const FIntPoint Resolution,
	const EHairStrandsCommonPassType Type,
	const bool bWriteDepth,
	const bool bHasHoldout,
	const FHairStrandsTiles& TileData,
	TPixelShader& PixelShader,
	TPassParameter* PassParameters)
{
	//ClearUnusedGraphResources(PixelShader, PassParameters);

	const FIntRect Viewport = View.ViewRect;
	TShaderMapRef<FHairStrandsTilePassVS> TileVertexShader(View.ShaderMap);

	const FHairStrandsTiles::ETileType TileType = FHairStrandsTiles::ETileType::HairAll;
	check(TileData.IsValid());
	PassParameters->TileData = GetHairStrandsTileParameters(View, TileData, TileType);

	GraphBuilder.AddPass(
		Forward<FRDGEventName>(EventName),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, TileVertexShader, PixelShader, Viewport, Resolution, Type, bWriteDepth, bHasHoldout, TileType](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FHairStrandsTilePassVS::FParameters ParametersVS = PassParameters->TileData;

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		if (Type == EHairStrandsCommonPassType::Composition)
		{
			if (bHasHoldout)
			{
				// EHairStrandsCommonPassType::Composition and bHasHoldout, only color is composed. The alpha value is composed into a separate pass.
				// Two modes:
				// * 0: the background pixels are *not* weighted by the hair coverage
				// * 1: the background pixels are       weighted by the hair coverage
				if (GHairStrandsHoldoutMode == 0)
				{
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_One>::GetRHI();
				}
				else
				{
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
				}
			}
			else
			{
				// Alpha usage/output is controlled with r.PostProcessing.PropagateAlpha. The value are:
				// 0: disabled(default);
				// 1: enabled
				//
				// When enabled, the alpha value means:
				// 0: valid pixel
				// 1: invalid pixel (background)
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
			}
		}
		else if (Type == EHairStrandsCommonPassType::Blit) //used to write data into the temporal input buffer
		{
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		}
		else if (Type == EHairStrandsCommonPassType::HoldoutScene)
		{
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_ALPHA, BO_Add, BF_Zero, BF_One, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
		}
		else if (Type == EHairStrandsCommonPassType::HoldoutHair)
		{
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_ALPHA, BO_Add, BF_Zero, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
		}
		else
		{
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Max, BF_One, BF_One>::GetRHI();
		}

		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

		if (Type == EHairStrandsCommonPassType::Composition || Type == EHairStrandsCommonPassType::HoldoutScene || Type == EHairStrandsCommonPassType::HoldoutHair)
		{
			if (bWriteDepth)
			{
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
			}
			else
			{
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			}
		}
		else if (Type == EHairStrandsCommonPassType::DOF)
		{
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		}
		else if (Type == EHairStrandsCommonPassType::TAAFastResolve)
		{
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
				false, CF_Always,
				true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
				false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
				STENCIL_TEMPORAL_RESPONSIVE_AA_MASK, STENCIL_TEMPORAL_RESPONSIVE_AA_MASK>::GetRHI();
		}
		else if ((Type == EHairStrandsCommonPassType::GBuffer) || (Type == EHairStrandsCommonPassType::Blit))
		{
			if (bWriteDepth)
			{
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_Always>::GetRHI();
			}
			else
			{
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			}
		}

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = TileVertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PassParameters->TileData.bRectPrimitive > 0 ? PT_RectList : PT_TriangleList;
		
		const uint32 StencilRef = (Type == EHairStrandsCommonPassType::TAAFastResolve) ? STENCIL_TEMPORAL_RESPONSIVE_AA_MASK : 0;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
		RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);

		SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), ParametersVS);
		RHICmdList.SetStreamSource(0, nullptr, 0);
		RHICmdList.DrawPrimitiveIndirect(PassParameters->TileData.TileIndirectBuffer->GetRHI(), FHairStrandsTiles::GetIndirectDrawArgOffset(TileType));
	});
}

/////////////////////////////////////////////////////////////////////////////////////////

class FHairHoldoutPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairHoldoutPS);
	SHADER_USE_PARAMETER_STRUCT(FHairHoldoutPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileData)
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(uint32, bComposeDofDepth)
		SHADER_PARAMETER(uint32, PassType)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogStruct)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairDOFDepthTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_HOLDOUT"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairHoldoutPS, "/Engine/Private/HairStrands/HairStrandsComposition.usf", "HoldoutPS", SF_Pixel);

static void AddHairHoldoutPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsVisibilityData& VisibilityData,
	const FRDGTextureRef& HairDOFDepthTexture,
	FRDGTextureRef& OutColorTexture,
	FRDGTextureRef& OutDepthTexture)
{
	const bool bDOFEnable = HairDOFDepthTexture != nullptr ? 1 : 0;

	TRDGUniformBufferRef<FFogUniformParameters> FogBuffer = CreateFogUniformBuffer(GraphBuilder, View);

	// Attenuate original scene's pixel alpha value
	{
		FHairHoldoutPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairHoldoutPS::FParameters>();
		Parameters->bComposeDofDepth = bDOFEnable ? 1 : 0;
		Parameters->PassType = 0;
		Parameters->HairDOFDepthTexture = bDOFEnable ? HairDOFDepthTexture : GSystemTextures.GetBlackDummy(GraphBuilder);
		Parameters->OutputResolution = OutColorTexture->Desc.Extent;
		Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
		Parameters->HairStrands = View.HairStrandsViewData.UniformBuffer;
		Parameters->FogStruct = FogBuffer;
		Parameters->RenderTargets[0] = FRenderTargetBinding(OutColorTexture, ERenderTargetLoadAction::ELoad);

		FHairHoldoutPS::FPermutationDomain PermutationVector;
		TShaderMapRef<FHairHoldoutPS> PixelShader(View.ShaderMap, PermutationVector);
		InternalCommonDrawPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::Holdout(Scene)"),
			View,
			OutColorTexture->Desc.Extent,
			EHairStrandsCommonPassType::HoldoutScene,
			false /*bWriteDepth*/,
			true /*bHasHoldout*/,
			VisibilityData.TileData,
			PixelShader,
			Parameters);
	}

	// Add hair's fog contribution
	{
		FHairHoldoutPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairHoldoutPS::FParameters>();
		Parameters->bComposeDofDepth = bDOFEnable ? 1 : 0;
		Parameters->PassType = 1;
		Parameters->HairDOFDepthTexture = bDOFEnable ? HairDOFDepthTexture : GSystemTextures.GetBlackDummy(GraphBuilder);
		Parameters->OutputResolution = OutColorTexture->Desc.Extent;
		Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
		Parameters->HairStrands = View.HairStrandsViewData.UniformBuffer;
		Parameters->FogStruct = FogBuffer;
		Parameters->RenderTargets[0] = FRenderTargetBinding(OutColorTexture, ERenderTargetLoadAction::ELoad);

		FHairHoldoutPS::FPermutationDomain PermutationVector;
		TShaderMapRef<FHairHoldoutPS> PixelShader(View.ShaderMap, PermutationVector);
		InternalCommonDrawPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::Holdout(Hair)"),
			View,
			OutColorTexture->Desc.Extent,
			EHairStrandsCommonPassType::HoldoutHair,
			false /*bWriteDepth*/,
			true /*bHasHoldout*/,
			VisibilityData.TileData,
			PixelShader,
			Parameters);
	}
}

/////////////////////////////////////////////////////////////////////////////////////////

class FHairVisibilityComposeSamplePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairVisibilityComposeSamplePS);
	SHADER_USE_PARAMETER_STRUCT(FHairVisibilityComposeSamplePS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	class FReadSLWDepth : SHADER_PERMUTATION_BOOL("PERMUTATION_READ_SLW_DEPTH");
	class FWriteDepth : SHADER_PERMUTATION_BOOL("PERMUTATION_WRITE_DEPTH");
	using FPermutationDomain = TShaderPermutationDomain<FDebug, FReadSLWDepth, FWriteDepth>;
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileData)
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(uint32, bComposeDofDepth)
		SHADER_PARAMETER(uint32, bHasHoldout)
		SHADER_PARAMETER(uint32, SLWComposeType)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairLightingSampleBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairDOFDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SLWDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutMetaTexture)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogStruct)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_COMPOSE_SAMPLE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairVisibilityComposeSamplePS, "/Engine/Private/HairStrands/HairStrandsComposition.usf", "ComposeSamplePS", SF_Pixel);

static void AddHairVisibilityComposeSamplePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsVisibilityData& VisibilityData,
	const bool bHasHoldout,
	const EHairSLWCompositionType SLWComposeType,
	const FRDGTextureRef& HairDOFDepthTexture,
	const FRDGTextureRef SLWDepthTexture,
	FRDGTextureRef& OutColorTexture,
	FRDGTextureRef& OutDepthTexture)
{
	check(VisibilityData.SampleLightingTexture);
	const bool bDOFEnable = HairDOFDepthTexture != nullptr ? 1 : 0;

	TRDGUniformBufferRef<FFogUniformParameters> FogBuffer = CreateFogUniformBuffer(GraphBuilder, View);

	// When bReadSLWDepth == true, the SLW copy pass has been executed, and thus the hair depth has already be written into the scene depth (no need to rewrite it again)
	const bool bReadSLWDepth = SLWDepthTexture && SLWComposeType != EHairSLWCompositionType::All;
	const bool bWriteDepth = !bReadSLWDepth;

	FHairVisibilityComposeSamplePS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairVisibilityComposeSamplePS::FParameters>();
	Parameters->bComposeDofDepth = bDOFEnable ? 1 : 0;
	Parameters->SLWComposeType = 0;
	Parameters->HairLightingSampleBuffer = VisibilityData.SampleLightingTexture;
	Parameters->HairDOFDepthTexture = bDOFEnable ? HairDOFDepthTexture : GSystemTextures.GetBlackDummy(GraphBuilder);
	Parameters->OutputResolution = OutColorTexture->Desc.Extent;
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->HairStrands = View.HairStrandsViewData.UniformBuffer;
	Parameters->FogStruct = FogBuffer;
	Parameters->bHasHoldout = bHasHoldout ? 1u : 0u;
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutColorTexture, ERenderTargetLoadAction::ELoad);

	if (bWriteDepth)
	{
		Parameters->RenderTargets.DepthStencil = FDepthStencilBinding(OutDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilRead);
	}

	if (bReadSLWDepth)
	{
		Parameters->SLWComposeType = SLWComposeType;
		Parameters->SLWDepthTexture = SLWDepthTexture;
	}

	const bool bDebugComposition = View.Family->EngineShowFlags.LODColoration;
	FHairVisibilityComposeSamplePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairVisibilityComposeSamplePS::FDebug>(bDebugComposition);
	PermutationVector.Set<FHairVisibilityComposeSamplePS::FReadSLWDepth>(bReadSLWDepth);
	PermutationVector.Set<FHairVisibilityComposeSamplePS::FWriteDepth>(bWriteDepth);
	TShaderMapRef<FHairVisibilityComposeSamplePS> PixelShader(View.ShaderMap, PermutationVector);
	InternalCommonDrawPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::ComposeSample(WriteDepth:%s)", bWriteDepth?TEXT("1"):TEXT("0"), bReadSLWDepth?TEXT(",ReadSLW"):TEXT("")),
		View,
		OutColorTexture->Desc.Extent,
		EHairStrandsCommonPassType::Composition,
		bWriteDepth,
		bHasHoldout,
		VisibilityData.TileData,
		PixelShader,
		Parameters);
}

/////////////////////////////////////////////////////////////////////////////////////////

class FHairDOFDepthPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairDOFDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FHairDOFDepthPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileData)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairLightingSampleBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_DOFDEPTH"), 1);
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32_FLOAT); 
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairDOFDepthPS, "/Engine/Private/HairStrands/HairStrandsComposition.usf", "DOFDepthPS", SF_Pixel);

static FRDGTextureRef AddHairDOFDepthPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsVisibilityData& VisibilityData,
	const FRDGTextureRef& CategorizationTexture,
	const FRDGTextureRef& InColorTexture,
	const FRDGTextureRef& InDepthTexture)
{
	check(VisibilityData.SampleLightingTexture);
	FIntPoint OutputResolution = InColorTexture->Desc.Extent;

	FRDGTextureRef OutDOFDepthTexture = nullptr;
	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(OutputResolution, PF_R32_FLOAT, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource, 1);
		OutDOFDepthTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.DOFDepth"));
	}

	FHairDOFDepthPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairDOFDepthPS::FParameters>();
	Parameters->HairStrands = View.HairStrandsViewData.UniformBuffer;
	Parameters->HairLightingSampleBuffer = VisibilityData.SampleLightingTexture;
	Parameters->SceneColorTexture = InColorTexture;
	Parameters->SceneDepthTexture = InDepthTexture;
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutDOFDepthTexture, ERenderTargetLoadAction::ENoAction);

	TShaderMapRef<FHairDOFDepthPS> PixelShader(View.ShaderMap);
	InternalCommonDrawPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::DOFDepth"),
		View,
		OutputResolution,
		EHairStrandsCommonPassType::DOF,
		false /*bWriteDepth*/,
		false /*bHasHoldout*/,
		VisibilityData.TileData,
		PixelShader,
		Parameters);

	return OutDOFDepthTexture;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairVisibilityFastResolveMaskPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairVisibilityFastResolveMaskPS);
	SHADER_USE_PARAMETER_STRUCT(FHairVisibilityFastResolveMaskPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileData)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ResolveMaskTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_FASTRESOLVE_MASK"), 1);
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_R8G8B8A8);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairVisibilityFastResolveMaskPS, "/Engine/Private/HairStrands/HairStrandsComposition.usf", "FastResolvePS", SF_Pixel);

static void AddHairVisibilityFastResolveMaskPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef& HairResolveMaskTexture,
	const FHairStrandsTiles& TileData,
	FRDGTextureRef& OutDepthTexture)
{
	const FIntPoint Resolution = OutDepthTexture->Desc.Extent;
	FRDGTextureRef DummyTexture;
	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Resolution, PF_R8G8B8A8, FClearValueBinding::Black, TexCreate_RenderTargetable);
		Desc.NumSamples = OutDepthTexture->Desc.NumSamples;
		DummyTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.DummyTexture"));
	}

	FHairVisibilityFastResolveMaskPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairVisibilityFastResolveMaskPS::FParameters>();
	Parameters->ResolveMaskTexture = HairResolveMaskTexture;
	Parameters->RenderTargets[0] = FRenderTargetBinding(DummyTexture, ERenderTargetLoadAction::ENoAction);
	Parameters->RenderTargets.DepthStencil = FDepthStencilBinding(
		OutDepthTexture,
		ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad,
		FExclusiveDepthStencil::DepthRead_StencilWrite);

	TShaderMapRef<FHairVisibilityFastResolveMaskPS> PixelShader(View.ShaderMap);
	InternalCommonDrawPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::MarkTAAFastResolve"),
		View,
		Resolution,
		EHairStrandsCommonPassType::TAAFastResolve,
		false /*bWriteDepth*/,
		false /*bHasHoldout*/,
		TileData,
		PixelShader,
		Parameters);
}


///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairVisibilityGBufferWritePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairVisibilityGBufferWritePS);
	SHADER_USE_PARAMETER_STRUCT(FHairVisibilityGBufferWritePS, FGlobalShader);

	class FOutputType : SHADER_PERMUTATION_INT("PERMUTATION_OUTPUT_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FOutputType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileData)
		SHADER_PARAMETER(uint32, bWriteDummyData)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		// Substrate
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateCommonParameters, Substrate)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, RWMaterialTextureArray)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWTopLayerTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_B8G8R8A8);
		OutEnvironment.SetRenderTargetOutputFormat(1, PF_FloatRGBA);
		OutEnvironment.SetDefine(TEXT("SHADER_WRITE_GBUFFER"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairVisibilityGBufferWritePS, "/Engine/Private/HairStrands/HairStrandsComposition.usf", "MainPS", SF_Pixel);

namespace Substrate
{
	FSubstrateCommonParameters GetSubstrateCommonParameter(const FSubstrateSceneData& In);
}

static void AddHairVisibilityGBufferWritePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const bool bWriteDummyData,
	const FHairStrandsTiles& TileData,
	FSubstrateSceneData* SubstrateSceneData,
	FRDGTextureRef OutDepthTexture)
{
	check(SubstrateSceneData);
	check(SubstrateSceneData->MaterialTextureArray);
	check(Substrate::IsSubstrateEnabled());

	const bool bWriteDepth = OutDepthTexture != nullptr;
	const FIntPoint Resolution = OutDepthTexture ? OutDepthTexture->Desc.Extent : SubstrateSceneData->MaterialTextureArray->Desc.Extent;
	FHairVisibilityGBufferWritePS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairVisibilityGBufferWritePS::FParameters>();
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->bWriteDummyData = bWriteDummyData ? 1 : 0;
	Parameters->HairStrands = View.HairStrandsViewData.UniformBuffer;
	Parameters->Substrate = Substrate::GetSubstrateCommonParameter(*SubstrateSceneData);
	Parameters->RWMaterialTextureArray = SubstrateSceneData->MaterialTextureArrayUAV;
	Parameters->RWTopLayerTexture = SubstrateSceneData->TopLayerTextureUAV;
	if (bWriteDepth)
	{
		Parameters->RenderTargets.DepthStencil = FDepthStencilBinding(
			OutDepthTexture,
			ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad,
			FExclusiveDepthStencil::DepthWrite_StencilNop);
	}

	FHairVisibilityGBufferWritePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairVisibilityGBufferWritePS::FOutputType>(bWriteDepth ? 1 : 0);
	TShaderMapRef<FHairVisibilityGBufferWritePS> PixelShader(View.ShaderMap, PermutationVector);
	InternalCommonDrawPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::GBufferOverride"),
		View,
		Resolution,
		EHairStrandsCommonPassType::GBuffer,
		bWriteDepth,
		false /*bHasHoldout*/,
		TileData,
		PixelShader,
		Parameters);
}

static void AddHairVisibilityGBufferWritePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const bool bWriteDummyData,
	const FHairStrandsTiles& TileData,
	FRDGTextureRef OutGBufferATexture,
	FRDGTextureRef OutGBufferBTexture,
	FRDGTextureRef OutGBufferCTexture,
	FRDGTextureRef OutGBufferDTexture,
	FRDGTextureRef OutGBufferETexture,
	FRDGTextureRef OutDepthTexture)
{
	const bool bWriteFullGBuffer = OutGBufferCTexture != nullptr;
	const bool bWriteDepth = OutDepthTexture != nullptr;

	if (!OutGBufferATexture || !OutGBufferBTexture)
	{
		return;
	}

	if (bWriteFullGBuffer && (!OutGBufferCTexture || !OutDepthTexture))
	{
		return;
	}

	FHairVisibilityGBufferWritePS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairVisibilityGBufferWritePS::FParameters>();
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->bWriteDummyData = bWriteDummyData ? 1 : 0;
	Parameters->HairStrands = View.HairStrandsViewData.UniformBuffer;
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutGBufferATexture, ERenderTargetLoadAction::ELoad);
	Parameters->RenderTargets[1] = FRenderTargetBinding(OutGBufferBTexture, ERenderTargetLoadAction::ELoad);	
	if (bWriteFullGBuffer)
	{
		Parameters->RenderTargets[2] = FRenderTargetBinding(OutGBufferCTexture, ERenderTargetLoadAction::ELoad);
		if (OutGBufferDTexture)
		{
			Parameters->RenderTargets[3] = FRenderTargetBinding(OutGBufferDTexture, ERenderTargetLoadAction::ELoad);
		}
		if (OutGBufferETexture)
		{
			Parameters->RenderTargets[4] = FRenderTargetBinding(OutGBufferETexture, ERenderTargetLoadAction::ELoad);
		}
	}
	if (bWriteDepth)
	{
		Parameters->RenderTargets.DepthStencil = FDepthStencilBinding(
			OutDepthTexture,
			ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad,
			FExclusiveDepthStencil::DepthWrite_StencilNop);
	}

	FHairVisibilityGBufferWritePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairVisibilityGBufferWritePS::FOutputType>(bWriteFullGBuffer ? 1 : 0);
	TShaderMapRef<FHairVisibilityGBufferWritePS> PixelShader(View.ShaderMap, PermutationVector);
	InternalCommonDrawPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::GBufferOverride"),
		View,
		OutGBufferATexture->Desc.Extent,
		EHairStrandsCommonPassType::GBuffer,
		bWriteDepth,
		false /*bHasHoldout*/,
		TileData,
		PixelShader,
		Parameters);
}

static void InternalRenderHairComposition(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef SceneVelocityTexture,
	FRDGTextureRef SLWDepthTexture)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsComposition, "HairStrandsComposition");
	SCOPED_NAMED_EVENT(HairStrandsComposition, FColor::Emerald);

	if (View.HairStrandsViewData.VisibilityData.RasterizedInstanceCount > 0)
	{
		AddHairStrandsForwardRasterPass(
			GraphBuilder,
			View,
			SceneDepthTexture->Desc.Extent,
			View.HairStrandsViewData.VisibilityData,
			SceneDepthTexture,
			SceneColorTexture,
			SceneVelocityTexture);
	}
	else
	{
		const FHairStrandsVisibilityData& VisibilityData = View.HairStrandsViewData.VisibilityData;

		if (!VisibilityData.CoverageTexture)
		{
			return; // Automatically skip for any view not rendering hair
		}

		FRDGTextureRef DOFDepth = nullptr;
		if (View.FinalPostProcessSettings.DepthOfFieldUseHairDepth)
		{
			DOFDepth = AddHairDOFDepthPass(
				GraphBuilder,
				View,
				VisibilityData,
				VisibilityData.CoverageTexture,
				SceneColorTexture,
				SceneDepthTexture);
		}

		const bool bHasHoldout = HasHairFlags(View.HairStrandsViewData.Flags, HAIR_FLAGS_HOLDOUT) && View.Family->EngineShowFlags.AllowPrimitiveAlphaHoldout;
		const bool bTemporalLayeringEnabled = false; // HAIR_TODO: remove

		AddHairVisibilityComposeSamplePass(
			GraphBuilder,
			View,
			VisibilityData,
			bHasHoldout,
			EHairSLWCompositionType::InFrontOfWater,
			DOFDepth,
			SLWDepthTexture,
			SceneColorTexture,
			SceneDepthTexture);

		if (bHasHoldout)
		{
			AddHairHoldoutPass(
				GraphBuilder,
				View,
				VisibilityData,
				DOFDepth,
				SceneColorTexture,
				SceneDepthTexture);
		}
		if (VisibilityData.ResolveMaskTexture)
		{
			AddHairVisibilityFastResolveMaskPass(
				GraphBuilder,
				View,
				VisibilityData.ResolveMaskTexture,
				VisibilityData.TileData,
				SceneDepthTexture);
		}

		const bool bWriteDummyData		= !(View.Family->ViewMode == VMI_VisualizeBuffer || View.Family->ViewMode == VMI_VisualizeSubstrate) && GHairWriteGBufferData == 1;
		const bool bWritePartialGBuffer =  (View.Family->ViewMode == VMI_VisualizeBuffer || View.Family->ViewMode == VMI_VisualizeSubstrate) && (GHairWriteGBufferData == 1 || GHairWriteGBufferData == 2);
		const bool bWriteFullGBuffer	=  (View.Family->ViewMode == VMI_VisualizeBuffer || View.Family->ViewMode == VMI_VisualizeSubstrate) || (GHairWriteGBufferData == 3);
		if (bWriteFullGBuffer || bWritePartialGBuffer)
		{
			if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(View.GetShaderPlatform()))
			{
				AddHairVisibilityGBufferWritePass(
					GraphBuilder,
					View,
					bWriteDummyData,
					VisibilityData.TileData,
					View.SubstrateViewData.SceneData, 
					SceneDepthTexture);
			}
			else
			{
				const FSceneTextures& SceneTextures = View.GetSceneTextures();
				const FRDGTextureRef GBufferATexture = SceneTextures.GBufferA;
				const FRDGTextureRef GBufferBTexture = SceneTextures.GBufferB;
				const FRDGTextureRef GBufferCTexture = SceneTextures.GBufferC;
				const FRDGTextureRef GBufferDTexture = SceneTextures.GBufferD;
				const FRDGTextureRef GBufferETexture = SceneTextures.GBufferE;
				if (bWritePartialGBuffer && GBufferATexture && GBufferBTexture)
				{
					AddHairVisibilityGBufferWritePass(
						GraphBuilder,
						View,
						bWriteDummyData,
						VisibilityData.TileData,
						GBufferATexture,
						GBufferBTexture,
						nullptr,
						nullptr,
						nullptr,
						nullptr);
				}
				else if (bWriteFullGBuffer && GBufferATexture && GBufferBTexture && GBufferCTexture && SceneDepthTexture)
				{
					AddHairVisibilityGBufferWritePass(
						GraphBuilder,
						View,
						bWriteDummyData,
						VisibilityData.TileData,
						GBufferATexture,
						GBufferBTexture,
						GBufferCTexture,
						GBufferDTexture,
						GBufferETexture,
						SceneDepthTexture);
				}
			}
		}
	}
}

void RenderHairCompositionForSLW(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	FRDGTextureRef SLWDepthTexture,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture)
{
	for (const FViewInfo& View : Views)
	{		
		if (View.Family && HairStrands::HasViewHairStrandsData(View))
		{
			const FHairStrandsVisibilityData& VisibilityData = View.HairStrandsViewData.VisibilityData;
			const bool bHasHoldout = HasHairFlags(View.HairStrandsViewData.Flags, HAIR_FLAGS_HOLDOUT) && View.Family->EngineShowFlags.AllowPrimitiveAlphaHoldout;
			if (!VisibilityData.CoverageTexture || bHasHoldout)
			{
				continue; // Automatically skip for any view not rendering hair
			}

			// When rendering hair for SLW prior to the scene color copy, compose all hair samples (in front & behind) to accomodate 
			// UV distortion which can require more data than what is stricly needed.
			AddHairVisibilityComposeSamplePass(
				GraphBuilder,
				View,
				VisibilityData,
				false /*bHasHoldout*/,
				EHairSLWCompositionType::All,
				nullptr /*DOFDepth*/,
				SLWDepthTexture,
				SceneColorTexture,
				SceneDepthTexture);
		}
	}
}

void RenderHairComposition(
	FRDGBuilder& GraphBuilder,
	const TArray<FViewInfo>& Views,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef SceneVelocityTexture,
	FTranslucencyPassResourcesMap& TranslucencyResourceMap,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult)
{
	uint32 ViewIndex = 0;
	for (const FViewInfo& View : Views)
	{		
		if (View.Family && HairStrands::HasViewHairStrandsData(View))
		{
			FTranslucencyPassResources& TranslucencyPassResources = TranslucencyResourceMap.Get(ViewIndex, ETranslucencyPass::TPT_TranslucencyStandard);
			FRDGTextureRef SceneColorTarget = TranslucencyPassResources.ColorTexture.IsValid() ? TranslucencyPassResources.ColorTexture.Target : SceneColorTexture;

			InternalRenderHairComposition(
				GraphBuilder,
				View,
				SceneColorTarget,
				SceneDepthTexture,
				SceneVelocityTexture,
				SingleLayerWaterPrePassResult ? SingleLayerWaterPrePassResult->DepthPrepassTexture.Resolve : nullptr);
		}
		ViewIndex++;
	}
}

void RenderHairComposition(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef SceneVelocityTexture,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult)
{
	if (View.Family && HairStrands::HasViewHairStrandsData(View))
	{
		InternalRenderHairComposition(
			GraphBuilder,
			View,
			SceneColorTexture,
			SceneDepthTexture,
			SceneVelocityTexture,
			SingleLayerWaterPrePassResult ? SingleLayerWaterPrePassResult->DepthPrepassTexture.Resolve : nullptr);
	}
}
