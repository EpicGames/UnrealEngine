// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsComposition.h: Hair strands pixel composition implementation.
=============================================================================*/

#pragma once

#include "Containers/ContainersFwd.h"
#include "RenderGraphFwd.h"

class FViewInfo;
struct FSingleLayerWaterPrePassResult;

void RenderHairComposition(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture, 
	FRDGTextureRef VelocityTexture,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult = nullptr);

void RenderHairComposition(
	FRDGBuilder& GraphBuilder, 
	const TArray<FViewInfo>& Views,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef VelocityTexture,
	struct FTranslucencyPassResourcesMap& TranslucencyResourceMap,
	const FSingleLayerWaterPrePassResult* SingleLayerWaterPrePassResult = nullptr);

void RenderHairCompositionForSLW(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	FRDGTextureRef SLWDepthTexture,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture);
