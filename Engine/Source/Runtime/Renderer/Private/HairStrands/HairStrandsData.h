// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairRendering.h: Hair rendering implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "RHIGPUReadback.h"
#include "Shader.h"
#include "ConvexVolume.h"
#include "HairStrandsDefinitions.h"
#include "HairStrandsInterface.h"
#include "ScenePrivateBase.h"

class FLightSceneInfo;
class FPrimitiveSceneProxy;
class FViewInfo;
class FScene;
class FInstanceCullingManager;
class FHairGroupPublicData;
struct FMeshBatch;
struct FMeshBatchAndRelevance;

////////////////////////////////////////////////////////////////////////////////////
// HairStrands uniform buffer

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FHairStrandsViewUniformParameters, )	
	SHADER_PARAMETER(FIntPoint, HairTileCountXY)										// Tile count in X/Y
	SHADER_PARAMETER(uint32, MaxSamplePerPixelCount)									// Max number of sample per pixel
	SHADER_PARAMETER(float, HairDualScatteringRoughnessOverride)						// Override the roughness used for dual scattering (for hack/test purpose only)
	SHADER_PARAMETER(FIntPoint, HairSampleViewportResolution)							// Maximum viewport resolution of the sample space
	SHADER_PARAMETER(uint32, bHairTileValid)											// True if tile data are valid
	SHADER_PARAMETER(FVector4f, HairOnlyDepthHZBParameters)								// HZB parameters
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, HairCoverageTexture)					// Hair pixel's coverage
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairOnlyDepthTexture)						// Depth texture containing only hair depth (not strongly typed for legacy shader code reason)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairOnlyDepthClosestHZBTexture)				// HZB closest depth texture containing only hair depth (not strongly typed for legacy shader code reason)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairOnlyDepthFurthestHZBTexture)			// HZB furthest depth texture containing only hair depth (not strongly typed for legacy shader code reason)
	SHADER_PARAMETER_SAMPLER(SamplerState, HairOnlyDepthHZBSampler)						// HZB sampler
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, HairSampleOffset)						// Offset & count, for accessing pixel's samples, based on screen pixel position
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, HairSampleCount)			// Total count of hair sample, in sample space
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedHairSample>, HairSampleData)// Sample data (coverage, tangent, base color, ...), in sample space // HAIRSTRANDS_TODO: change this to be a uint4 so that we don't have to include the type for generated contant buffer
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint2>, HairSampleCoords)					// Screen pixel coordinate of each sample, in sample space
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint2>, HairTileData)						// Tile coords (RG16F)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>,  HairTileCount)						// Tile total count (actual number of tiles)
END_GLOBAL_SHADER_PARAMETER_STRUCT()


void GetHairStrandsInstanceCommon(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, const FHairGroupPublicData* HairGroupPublicData, FHairStrandsInstanceCommonParameters& OutCommon);
void GetHairStrandsInstanceResources(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, const FHairGroupPublicData* HairGroupPublicData, bool bForceRegister, FHairStrandsInstanceResourceParameters& OutResources);
void GetHairStrandsInstanceCulling(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, const FHairGroupPublicData* HairGroupPublicData, bool bCullingEnable, FHairStrandsInstanceCullingParameters& OutCulling);
FHairStrandsInstanceParameters GetHairStrandsInstanceParameters(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, const FHairGroupPublicData* HairGroupPublicData, bool bCullingEnable, bool bForceRegister);

////////////////////////////////////////////////////////////////////////////////////
// Tile data

struct FHairStrandsTiles
{
	enum class ETileType : uint8 { HairAll, HairFull, HairPartial, Other, Count };
	static const uint32 TileTypeCount = uint32(ETileType::Count);

	FIntPoint			BufferResolution = FIntPoint(0, 0);
	static const uint32 GroupSize = 64;
	static const uint32	TileSize = 8;
	static const uint32	TilePerThread_GroupSize = 64;
	uint32				TileCount = 0;
	FIntPoint			TileCountXY = FIntPoint(0, 0);
	bool				bRectPrimitive = false;

	// Buffers per tile types
	FRDGBufferSRVRef	TileDataSRV[TileTypeCount] = { nullptr, nullptr };
	FRDGBufferRef		TileDataBuffer[TileTypeCount] = { nullptr, nullptr };

	// Buffer shared for all tile types 
	FRDGBufferSRVRef	TileCountSRV = nullptr;
	FRDGBufferRef		TileCountBuffer = nullptr;

	FRDGBufferRef		TileIndirectDrawBuffer = nullptr;
	FRDGBufferRef		TileIndirectDispatchBuffer = nullptr;
	FRDGBufferRef		TileIndirectRayDispatchBuffer = nullptr;
	FRDGBufferRef		TilePerThreadIndirectDispatchBuffer = nullptr;

	static FORCEINLINE uint32 GetIndirectDrawArgOffset(ETileType Type)			{ return uint32(Type) * sizeof(FRHIDrawIndirectParameters);	}
	static FORCEINLINE uint32 GetIndirectDispatchArgOffset(ETileType Type)		{ return uint32(Type) * sizeof(FRHIDispatchIndirectParameters); } 
	static FORCEINLINE uint32 GetIndirectRayDispatchArgOffset(ETileType Type)	{ return uint32(Type) * sizeof(FRHIDispatchIndirectParameters); }

	bool IsValid() const										{ return TileCount > 0 && TileDataBuffer[uint32(ETileType::HairAll)] != nullptr; }
	FRDGBufferRef GetTileBuffer(ETileType Type) const			{ const uint32 Index = uint32(Type); check(TileDataBuffer[Index] != nullptr); return TileDataBuffer[Index];}
	FRDGBufferSRVRef GetTileBufferSRV(ETileType Type) const		{ const uint32 Index = uint32(Type); check(TileDataSRV[Index] != nullptr); return TileDataSRV[Index];}
};
FORCEINLINE uint32 ToIndex(FHairStrandsTiles::ETileType Type) { return uint32(Type); }
const TCHAR* ToString(FHairStrandsTiles::ETileType Type);

////////////////////////////////////////////////////////////////////////////////////
// Visibility Data

struct FHairStrandsVisibilityData
{
	FRDGTextureRef VelocityTexture = nullptr;
	FRDGTextureRef ResolveMaskTexture = nullptr;
	FRDGTextureRef CoverageTexture = nullptr;
	FRDGTextureRef ViewHairCountTexture = nullptr;
	FRDGTextureRef ViewHairCountUintTexture = nullptr;
	FRDGTextureRef HairOnlyDepthTexture = nullptr;

	FRDGTextureRef HairOnlyDepthClosestHZBTexture = nullptr;
	FRDGTextureRef HairOnlyDepthFurthestHZBTexture = nullptr;

	FRDGTextureRef LightChannelMaskTexture = nullptr;

	uint32			MaxSampleCount = 8; // Sample count per pixel
	uint32			MaxNodeCount = 0;	// Total sample count
	FRDGBufferRef	NodeCount = nullptr;
	FRDGTextureRef	NodeIndex = nullptr;
	FRDGBufferRef	NodeData = nullptr;
	FRDGBufferRef	NodeVisData = nullptr;
	FRDGBufferRef	NodeCoord = nullptr;
	FRDGBufferRef	NodeIndirectArg = nullptr;
	uint32			NodeGroupSize = 0;
	FVector4f		HairOnlyDepthHZBParameters = FVector4f::Zero();

	uint32				RasterizedInstanceCount = 0;
	uint32				MaxControlPointCount = 0;
	FRDGBufferSRVRef	ControlPointsSRV = nullptr;
	FRDGBufferSRVRef	ControlPointCount = nullptr;
	FRDGBufferSRVRef	ControlPointVelocitySRV = nullptr;

	FHairStrandsTiles TileData;

	const static EPixelFormat NodeCoordFormat = PF_R16G16_UINT;
	const static EPixelFormat CoverageFormat = PF_R16F;

	// Hair lighting is accumulated within this buffer
	// Allocated conservatively
	// User indirect dispatch for accumulating contribution
	FIntPoint SampleLightingViewportResolution;
	FRDGTextureRef SampleLightingTexture = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////
// Voxel data

struct FPackedVirtualVoxelNodeDesc
{
	// This is just a placeholder having the correct size. The actual definition is in HairStradsNVoxelPageCommon.ush
	const static EPixelFormat Format = PF_R32G32B32A32_UINT;
	const static uint32 ComponentCount = 2;

	// Shader View is struct { uint4; uint4; }
	FVector3f	MinAABB;
	uint32		PackedPageIndexResolution;
	FVector3f	MaxAABB;
	uint32		PageIndexOffset;
};

// PixelRadiusAtDepth1 shouldn't be stored into this structure should be view independent, 
// but is put here for convenience at the moment since multiple views are not supported 
// at the moment
BEGIN_SHADER_PARAMETER_STRUCT(FHairStrandsVoxelCommonParameters, )
	SHADER_PARAMETER(FIntVector, PageCountResolution)
	SHADER_PARAMETER(float, CPUMinVoxelWorldSize)
	SHADER_PARAMETER(FIntVector, PageTextureResolution)
	SHADER_PARAMETER(uint32, PageCount)
	SHADER_PARAMETER(uint32, PageResolution)
	SHADER_PARAMETER(uint32, PageResolutionLog2)
	SHADER_PARAMETER(uint32, PageIndexCount)
	SHADER_PARAMETER(uint32, IndirectDispatchGroupSize)
	SHADER_PARAMETER(uint32, NodeDescCount)
	SHADER_PARAMETER(uint32, JitterMode)

	SHADER_PARAMETER(float, DensityScale)
	SHADER_PARAMETER(float, DensityScale_AO)
	SHADER_PARAMETER(float, DensityScale_Shadow)
	SHADER_PARAMETER(float, DensityScale_Transmittance)
	SHADER_PARAMETER(float, DensityScale_Environment)
	SHADER_PARAMETER(float, DensityScale_Raytracing)

	SHADER_PARAMETER(float, DepthBiasScale_Shadow)
	SHADER_PARAMETER(float, DepthBiasScale_Transmittance)
	SHADER_PARAMETER(float, DepthBiasScale_Environment)

	SHADER_PARAMETER(float, SteppingScale_Shadow)
	SHADER_PARAMETER(float, SteppingScale_Transmittance)
	SHADER_PARAMETER(float, SteppingScale_Environment)
	SHADER_PARAMETER(float, SteppingScale_Raytracing)

	SHADER_PARAMETER(float, HairCoveragePixelRadiusAtDepth1)
	SHADER_PARAMETER(float, Raytracing_ShadowOcclusionThreshold)
	SHADER_PARAMETER(float, Raytracing_SkyOcclusionThreshold)

	SHADER_PARAMETER(FVector3f, TranslatedWorldOffset) // For debug purpose
	SHADER_PARAMETER(FVector3f, TranslatedWorldOffsetStereoCorrection) // PreViewTranslation correction between View0 & View1 when rendering stereo

	SHADER_PARAMETER(uint32, AllocationFeedbackEnable)

	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, AllocatedPageCountBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PageIndexBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PageIndexCoordBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedVirtualVoxelNodeDesc>, NodeDescBuffer) // Packed into 2 x uint4

	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, CurrGPUMinVoxelSize)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, NextGPUMinVoxelSize)

END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVirtualVoxelParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsVoxelCommonParameters, Common)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D<uint>, PageTexture)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

struct FHairStrandsVoxelResources
{
	FVirtualVoxelParameters	Parameters;
	TRDGUniformBufferRef<FVirtualVoxelParameters> UniformBuffer;
	FRDGTextureRef PageTexture = nullptr;
	FRDGBufferRef PageIndexBuffer = nullptr;
	FRDGBufferRef NodeDescBuffer = nullptr;
	FRDGBufferRef PageIndexCoordBuffer = nullptr;
	FRDGBufferRef IndirectArgsBuffer = nullptr;
	FRDGBufferRef PageIndexGlobalCounter = nullptr;

	const bool IsValid() const { return UniformBuffer != nullptr && PageTexture != nullptr && NodeDescBuffer != nullptr; }
};


////////////////////////////////////////////////////////////////////////////////////
// Deep shadow data

struct FMinHairRadiusAtDepth1
{
	float Primary = 1;
	float Velocity = 1;
	float Stable = 1;
};

/// Hold deep shadow information for a given light.
struct FHairStrandsDeepShadowData
{
	FMinHairRadiusAtDepth1 CPU_MinStrandRadiusAtDepth1;
	uint32 MacroGroupId = ~0;
	uint32 AtlasSlotIndex = 0;

	FIntPoint AtlasResolution = FIntPoint::ZeroValue;
	FLightSceneId LightId;
	bool bIsLightDirectional = false;
	FVector3f LightDirection;
	FVector3f TranslatedLightPosition;
	FLinearColor LightLuminance;
	float LayerDistribution;

	FBoxSphereBounds Bounds;
};

typedef TArray<FHairStrandsDeepShadowData, SceneRenderingAllocator> FHairStrandsDeepShadowDatas;

struct FHairStrandsDeepShadowResources
{
	// Limit the number of atlas slot to 32, in order to create the view info per slot in single compute
	// This limitation can be alleviate, and is just here for convenience (see FDeepShadowCreateViewInfoCS)
	static const uint32 MaxAtlasSlotCount = 32u;

	uint32 TotalAtlasSlotCount = 0;
	FIntPoint AtlasSlotResolution;

	FRDGTextureRef DepthAtlasTexture = nullptr;
	FRDGTextureRef LayersAtlasTexture = nullptr;
	FRDGBufferRef DeepShadowViewInfoBuffer = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////
// Cluster data

// A groom component contains one or several HairGroup. These hair group are send to the 
// render as mesh batches. These meshes batches are filtered/culled per view, and regroup 
// into HairMacroGroup for computing voxelization/DOM data, ...
//
// The hierarchy of the data structure is as follow:
//  * HairMacroGroup
//  * HairGroup
//  * HairCluster

struct FHairStrandsMacroGroupResources
{
	uint32 MacroGroupCount = 0;
	FRDGBufferRef MacroGroupAABBsBuffer = nullptr; // Tight bounding boxes
	FRDGBufferRef MacroGroupVoxelAlignedAABBsBuffer = nullptr; // Contents of MacroGroupAABBsBuffer, but snapped to the voxel page boundaries
	FRDGBufferRef MacroGroupVoxelSizeBuffer = nullptr;
};

class FHairGroupPublicData;

/// Hair macro group infos
struct FHairStrandsMacroGroupData
{
	static const uint32 MaxMacroGroupCount = 16u; // Needs to be kept consistent with MAX_HAIR_MACROGROUP_COUNT (HairStrandsVisibilityCommon.ush)

	// List of primitive/mesh batch within an instance group
	struct PrimitiveInfo
	{
		const FMeshBatch* Mesh = nullptr;
		const FPrimitiveSceneProxy* PrimitiveSceneProxy = nullptr;
		uint32 MaterialId = 0;
		uint32 ResourceId = 0;
		uint32 GroupIndex = 0;
		uint32 Flags = 0; // Hair instance flags
		FHairGroupPublicData* PublicDataPtr = nullptr;
		bool IsCullingEnable() const;
	};
	typedef TArray<PrimitiveInfo, SceneRenderingAllocator> TPrimitiveInfos;

	FHairStrandsDeepShadowDatas DeepShadowDatas;
	TPrimitiveInfos PrimitivesInfos;
	FBoxSphereBounds Bounds;
	FIntRect ScreenRect;
	uint32 MacroGroupId = 0;
	uint32 Flags = 0; // Aggregated flags for all instances from the group

	bool bSupportVoxelization = false; // true if at least one of the Primitive requires voxelization
};

////////////////////////////////////////////////////////////////////////////////////
// Debug data

struct FHairStrandsDebugData
{

	// PPLL debug data
	struct FPPLLData
	{
		FRDGTextureRef	NodeCounterTexture = nullptr;
		FRDGTextureRef	NodeIndexTexture = nullptr;
		FRDGBufferRef	NodeDataBuffer = nullptr;
	} PPLLData;

	bool IsPPLLDataValid() const 
	{ 
		return PPLLData.NodeCounterTexture && PPLLData.NodeIndexTexture && PPLLData.NodeDataBuffer;
	}

	// Instance cull data
	struct FCullData
	{
		struct FBound
		{
			FVector3f Min;
			FVector3f Max;
		};

		struct FLight
		{
			FMatrix WorldToLight;
			FMatrix LightToWorld;
			FVector3f Center;
			FVector3f Extent;
			FConvexVolume ViewFrustumInLightSpace;
			TArray<FBound> InstanceBoundInLightSpace;
			TArray<FBound> InstanceBoundInWorldSpace;
			TArray<uint32> InstanceIntersection;
		};

		bool bIsValid = false;
		TArray<FLight> DirectionalLights;
		FConvexVolume ViewFrustum;
	} CullData;

	struct FCommon
	{
		FRDGTextureRef SceneDepthTextureBeforeCompsition = nullptr;
	} Common;

};

typedef TArray<FHairStrandsMacroGroupData, SceneRenderingAllocator> FHairStrandsMacroGroupDatas;

////////////////////////////////////////////////////////////////////////////////////
// View Data
struct FHairStrandsViewData
{
	TRDGUniformBufferRef<FHairStrandsViewUniformParameters> UniformBuffer{};
	bool bIsValid = false;

	// Internal data
	FHairStrandsVisibilityData VisibilityData;
	FHairStrandsMacroGroupDatas MacroGroupDatas;
	FHairStrandsDeepShadowResources DeepShadowResources;
	FHairStrandsVoxelResources VirtualVoxelResources;
	FHairStrandsMacroGroupResources MacroGroupResources;
	FHairStrandsDebugData DebugData;
	uint32 Flags = 0;

	// Transient: store all light visible in primary view(s)
	struct FDirectionalLightCullData { const FLightSceneInfo* LightInfo = nullptr; FConvexVolume ViewFrustumInLightSpace; };
	TArray<const FLightSceneInfo*> VisibleShadowCastingLights;
	TArray<FDirectionalLightCullData> VisibleShadowCastingDirectionalLights;
};

// View State data (i.e., persistent across frame)
struct FHairStrandsViewStateData
{
	bool IsInit() const { return PositionsChangedDatas.Num() > 0; }
	void Init();
	void Release();

	// Buffer used for reading back the number of allocated voxels on the GPU
	TRefCountPtr<FRDGPooledBuffer> VoxelFeedbackBuffer = nullptr;

	// Buffer used for reading back hair strands position changed on the GPU
	struct FPositionChangedData
	{
		FRHIGPUBufferReadback* ReadbackBuffer = nullptr;
		bool bHasPendingReadback = false;
	};
	void EnqueuePositionsChanged(FRDGBuilder& GraphBuilder, FRDGBufferRef InBuffer);
	bool ReadPositionsChanged();
	TArray<FPositionChangedData> PositionsChangedDatas;
};

namespace HairStrands
{
	TRDGUniformBufferRef<FHairStrandsViewUniformParameters> CreateDefaultHairStrandsViewUniformBuffer(FRDGBuilder& GraphBuilder);
	TRDGUniformBufferRef<FHairStrandsViewUniformParameters> BindHairStrandsViewUniformParameters(const FViewInfo& View);

	TRDGUniformBufferRef<FVirtualVoxelParameters> CreateDefaultHairStrandsVoxelUniformBuffer(FRDGBuilder& GraphBuilder);
	TRDGUniformBufferRef<FVirtualVoxelParameters> BindHairStrandsVoxelUniformParameters(const FViewInfo& View);

	bool HasViewHairStrandsData(const FViewInfo& View);
	bool HasViewHairStrandsData(const TArray<FViewInfo>& Views);
	bool HasViewHairStrandsVoxelData(const FViewInfo& View);

	bool HasPositionsChanged(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View);
	void DrawHitProxies(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View, FInstanceCullingManager& InstanceCullingManager, FRDGTextureRef HitProxyTexture, FRDGTextureRef HitProxyDepthTexture);
	void DrawEditorSelection(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FIntRect& ViewportRect, FRDGTextureRef SelectionDepthTexture);
	void DrawCustomDepth(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FIntRect& ViewportRect, FRDGTextureRef CustomDepthTexture);

	// Mesh batch helpers
	FHairGroupPublicData* GetHairData(const FMeshBatch* In);
	bool IsHairCompatible(const FMeshBatch* Mesh);
	bool IsHairStrandsVF(const FMeshBatch* Mesh);
	bool IsHairCardsVF(const FMeshBatch* Mesh);
	bool IsHairVisible(const FMeshBatchAndRelevance& MeshBatch, bool bCheckLengthScale);

	// Hair helpers
	bool HasHairInstanceInScene(const FScene& Scene);
	bool HasHairCardsVisible(const TArray<FViewInfo>& Views);
	bool HasHairStrandsVisible(TArrayView<const FViewInfo> Views);
	bool IsHairStrandsVF(const FVertexFactoryType* InVFType);

	void AddVisibleShadowCastingLight(const FScene& Scene, TArray<FViewInfo>& Views, const FLightSceneInfo* LightSceneInfo);

	void PostRender(FScene& Scene);
}