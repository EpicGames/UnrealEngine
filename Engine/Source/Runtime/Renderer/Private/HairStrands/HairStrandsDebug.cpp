// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsDebug.h"
#include "CoreMinimal.h"
#include "CanvasTypes.h"
#include "CommonRenderResources.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "ExtensibleUniformBuffer.h"
#include "GlobalShader.h"
#include "GroomVisualizationData.h"
#include "HairStrands/HairStrandsData.h"
#include "HairStrands/HairStrandsRendering.h"
#include "HairStrands/HairStrandsTile.h"
#include "HairStrands/HairStrandsUtils.h"
#include "HairStrandsInterface.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "Math/UnrealPlatformMathSSE.h"
#include "Misc/LargeWorldCoordinates.h"
#include "PipelineStateCache.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "SceneTextures.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "ScreenRendering.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderPrintParameters.h"
#include "Stats/Stats.h"
#include "SystemTextures.h"
#include "UnrealEngine.h"

static int32 GDeepShadowDebugIndex = 0;
static float GDeepShadowDebugScale = 20;

static FAutoConsoleVariableRef CVarDeepShadowDebugDomIndex(TEXT("r.HairStrands.DeepShadow.DebugDOMIndex"), GDeepShadowDebugIndex, TEXT("Index of the DOM texture to draw"));
static FAutoConsoleVariableRef CVarDeepShadowDebugDomScale(TEXT("r.HairStrands.DeepShadow.DebugDOMScale"), GDeepShadowDebugScale, TEXT("Scaling value for the DeepOpacityMap when drawing the deep shadow stats"));

static int32 GHairVirtualVoxel_DebugTraversalType = 0;
static FAutoConsoleVariableRef CVarHairVirtualVoxel_DebugTraversalType(TEXT("r.HairStrands.Voxelization.Virtual.DebugTraversalType"), GHairVirtualVoxel_DebugTraversalType, TEXT("Traversal mode (0:linear, 1:mip) for debug voxel visualization."));

static bool TryEnableShaderDrawAndShaderPrint(const FViewInfo& View, uint32 ResquestedShaderDrawElements, uint32 RequestedShaderPrintElements)
{
	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);

	ShaderPrint::RequestSpaceForCharacters(RequestedShaderPrintElements);
	ShaderPrint::RequestSpaceForLines(ResquestedShaderDrawElements);

	return ShaderPrint::IsEnabled(View.ShaderPrintData);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairPrintLODInfoCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairPrintLODInfoCS);
	SHADER_USE_PARAMETER_STRUCT(FHairPrintLODInfoCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, MaxResolution)
		SHADER_PARAMETER(FVector3f, GroupColor)
		SHADER_PARAMETER(uint32, GroupIndex)
		SHADER_PARAMETER(uint32, GeometryType)
		SHADER_PARAMETER(uint32, CurveCount)
		SHADER_PARAMETER(uint32, PointCount)
		SHADER_PARAMETER(float, CoverageScale)
		SHADER_PARAMETER(float, ScreenSize)
		SHADER_PARAMETER(float, LOD)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::All, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform);
	}
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Skip optimization for avoiding long compilation time due to large UAV writes
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_LOD_INFO"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairPrintLODInfoCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

static void AddPrintLODInfoPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairGroupPublicData* Data)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForCharacters(2000);

	if (!ShaderPrint::IsEnabled(View.ShaderPrintData))
	{
		return;
	}

	const uint32 GroupIndex = Data->GetGroupIndex();
	const FLinearColor GroupColor = Data->DebugGroupColor;
	const uint32 IntLODIndex = Data->LODIndex;

	FHairPrintLODInfoCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairPrintLODInfoCS::FParameters>();
	Parameters->MaxResolution = FIntPoint(View.ViewRect.Width(), View.ViewRect.Height());
	Parameters->GroupIndex = GroupIndex;
	Parameters->LOD = Data->LODIndex;
	Parameters->GroupColor = FVector3f(GroupColor.R, GroupColor.G, GroupColor.B);
	Parameters->ScreenSize = Data->DebugScreenSize;
	Parameters->CurveCount = Data->GetActiveStrandsCurveCount();
	Parameters->PointCount = Data->GetActiveStrandsPointCount();
	Parameters->CoverageScale = Data->GetActiveStrandsCoverageScale();
	switch (Data->VFInput.GeometryType)
	{
	case EHairGeometryType::Strands: Parameters->GeometryType = 0; break;
	case EHairGeometryType::Cards  : Parameters->GeometryType = 1; break;
	case EHairGeometryType::Meshes : Parameters->GeometryType = 2; break;
	}
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintUniformBuffer);
	TShaderMapRef<FHairPrintLODInfoCS> ComputeShader(View.ShaderMap);

	ClearUnusedGraphResources(ComputeShader, Parameters);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::PrintLODInfo(%d/%d)", Parameters->GroupIndex, Parameters->GroupIndex),
		ComputeShader,
		Parameters,
		FIntVector(1, 1, 1));
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairDebugPrintCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairDebugPrintCS);
	SHADER_USE_PARAMETER_STRUCT(FHairDebugPrintCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, GroupSize)
		SHADER_PARAMETER(FIntPoint, MaxResolution)
		SHADER_PARAMETER(uint32, FastResolveMask)
		SHADER_PARAMETER(uint32, HairMacroGroupCount)
		SHADER_PARAMETER(uint32, HairVisibilityNodeGroupSize)
		SHADER_PARAMETER(uint32, AllocatedSampleCount)
		SHADER_PARAMETER(uint32, HairInstanceCount)
		SHADER_PARAMETER(float, ResolutionScale)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HairInstanceDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, InstanceAABBBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairCountTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairCountUintTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HairVisibilityIndirectArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, HairMacroGroupAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, HairMacroGroupVoxelAlignedAABBBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, StencilTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Skip optimization for avoiding long compilation time due to large UAV writes
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_PRINT"), 1);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairDebugPrintCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

static void AddDebugHairPrintPass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* View,
	const EGroomViewMode ViewMode,
	const FHairStrandsVisibilityData& VisibilityData,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FHairStrandsMacroGroupResources& MacroGroupResources,
	FRDGTextureSRVRef InStencilTexture)
{
	if (!Scene || !View || !View->HairStrandsViewData.UniformBuffer || !InStencilTexture) return;

	if (!ShaderPrint::IsSupported(View->GetShaderPlatform()))
	{
		return;
	}

	if (!TryEnableShaderDrawAndShaderPrint(*View, MacroGroupResources.MacroGroupCount * 32u, 8192u))
	{
		return;
	}

	struct FData
	{
		uint32 PrimitiveID = ~0;
		uint32 RegisteredIndex = ~0;
		uint32 GeometryType = 0;
		uint32 Pad0 = 0;
		FVector4f InstanceScreenSphereBound = FVector4f::Zero();
	};

	// Build mapping Instance -> PrimitiveID to fetch primitive data (i.e., transform & co)
	const uint32 MacroGroupCount = MacroGroupDatas.Num();
	TArray<FData> InstanceDatas;
	InstanceDatas.Reserve(MacroGroupCount * 4u);
	for (uint32 MacroGroupIndex = 0; MacroGroupIndex < MacroGroupCount; ++MacroGroupIndex)
	{
		const FHairStrandsMacroGroupData& MacroGroup = MacroGroupDatas[MacroGroupIndex];
		for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : MacroGroup.PrimitivesInfos)
		{
			if (PrimitiveInfo.PrimitiveSceneProxy)
			{
				FData& InstanceData = InstanceDatas.AddDefaulted_GetRef();
				InstanceData.PrimitiveID = ~0u;
				if (const FPrimitiveSceneInfo* SceneInfo = PrimitiveInfo.PrimitiveSceneProxy->GetPrimitiveSceneInfo())
				{
					InstanceData.PrimitiveID = SceneInfo->GetIndex();
				}

				FHairStrandsInstance* Instance = PrimitiveInfo.PublicDataPtr->Instance;
				check(Instance);

				const float MaxRectSizeInPixels = FMath::Min(View->UnscaledViewRect.Height(), View->UnscaledViewRect.Width());
				const float ContinousLODRadius = PrimitiveInfo.PublicDataPtr->ContinuousLODScreenSize * MaxRectSizeInPixels * 0.5f; // Diameter->Radius

				InstanceData.GeometryType 				= Instance->GetHairGeometry();
				InstanceData.RegisteredIndex			= Instance->RegisteredIndex;
				InstanceData.InstanceScreenSphereBound 	= FVector4f(PrimitiveInfo.PublicDataPtr->ContinuousLODScreenPos.X, PrimitiveInfo.PublicDataPtr->ContinuousLODScreenPos.Y, 0.f, ContinousLODRadius);
			}
			else
			{
				InstanceDatas.AddDefaulted();
			}
		}
	}
	FRDGBufferRef InstanceDataBuffer = CreateVertexBuffer(GraphBuilder, TEXT("Hair.Debug.InstanceDatas"), FRDGBufferDesc::CreateBufferDesc(sizeof(FData), InstanceDatas.Num()), InstanceDatas.GetData(), sizeof(FData) * InstanceDatas.Num());

	FRDGTextureRef ViewHairCountTexture = VisibilityData.ViewHairCountTexture ? VisibilityData.ViewHairCountTexture : GSystemTextures.GetBlackDummy(GraphBuilder);
	FRDGTextureRef ViewHairCountUintTexture = VisibilityData.ViewHairCountUintTexture ? VisibilityData.ViewHairCountUintTexture : GSystemTextures.GetBlackDummy(GraphBuilder);

	const FIntRect Viewport = View->ViewRect;
	const FIntPoint Resolution(Viewport.Width(), Viewport.Height());

	FHairDebugPrintCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairDebugPrintCS::FParameters>();
	Parameters->ResolutionScale = float(View->ViewRect.Width()) / float(View->UnscaledViewRect.Width());
	Parameters->Scene = View->GetSceneUniforms().GetBuffer(GraphBuilder);
	Parameters->HairInstanceCount = InstanceDatas.Num();
	Parameters->HairInstanceDataBuffer = GraphBuilder.CreateSRV(InstanceDataBuffer, PF_R32_UINT);
	Parameters->InstanceAABBBuffer = Scene->HairStrandsSceneData.TransientResources->GroupAABBSRV;
	Parameters->GroupSize = GetVendorOptimalGroupSize2D();
	Parameters->ViewUniformBuffer = View->ViewUniformBuffer;
	Parameters->MaxResolution = VisibilityData.CoverageTexture ? VisibilityData.CoverageTexture->Desc.Extent : FIntPoint(0,0);
	Parameters->AllocatedSampleCount = VisibilityData.MaxNodeCount;
	Parameters->FastResolveMask = STENCIL_TEMPORAL_RESPONSIVE_AA_MASK;
	Parameters->HairCountTexture = ViewHairCountTexture;
	Parameters->HairCountUintTexture = ViewHairCountUintTexture;
	Parameters->HairVisibilityIndirectArgsBuffer = GraphBuilder.CreateSRV(VisibilityData.NodeIndirectArg, PF_R32_UINT);
	Parameters->HairVisibilityNodeGroupSize = VisibilityData.NodeGroupSize;
	Parameters->StencilTexture = InStencilTexture;
	Parameters->LinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters->HairMacroGroupCount = MacroGroupResources.MacroGroupCount;
	Parameters->HairStrands = View->HairStrandsViewData.UniformBuffer;
	Parameters->HairMacroGroupAABBBuffer = GraphBuilder.CreateSRV(MacroGroupResources.MacroGroupAABBsBuffer, PF_R32_SINT);
	Parameters->HairMacroGroupVoxelAlignedAABBBuffer = GraphBuilder.CreateSRV(MacroGroupResources.MacroGroupVoxelAlignedAABBsBuffer, PF_R32_SINT);
	ShaderPrint::SetParameters(GraphBuilder, View->ShaderPrintData, Parameters->ShaderPrintUniformBuffer);
	TShaderMapRef<FHairDebugPrintCS> ComputeShader(View->ShaderMap);

	ClearUnusedGraphResources(ComputeShader, Parameters);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::DebugPrint"),
		ComputeShader,
		Parameters,
		FIntVector(1, 1, 1));
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairDebugShadowCullingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairDebugShadowCullingCS);
	SHADER_USE_PARAMETER_STRUCT(FHairDebugShadowCullingCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InstanceCount)
		SHADER_PARAMETER(FVector3f, LightCenter)
		SHADER_PARAMETER(FVector3f, LightExtent)
		SHADER_PARAMETER(FMatrix44f, LightToWorld)
		SHADER_PARAMETER(FMatrix44f, ViewWorldToProj)
		SHADER_PARAMETER(FMatrix44f, ViewProjToWorld)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InstanceBoundInWorldSpace)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InstanceBoundInLightSpace)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InstanceIntersection)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Skip optimization for avoiding long compilation time due to large UAV writes
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_SHADOW_CULLING"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairDebugShadowCullingCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

static void AddDebugHairShadowCullingPass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* View)
{
	if (!Scene || !View || !View->HairStrandsViewData.UniformBuffer) return;

	if (!ShaderPrint::IsSupported(View->GetShaderPlatform()))
	{
		return;
	}

	const uint32 LightCount = View->HairStrandsViewData.DebugData.CullData.DirectionalLights.Num();
	uint32 InstanceCount = 0;
	for (auto It : View->HairStrandsViewData.DebugData.CullData.DirectionalLights)
	{
		InstanceCount += It.InstanceBoundInLightSpace.Num();
	}

	if (LightCount == 0 || InstanceCount == 0)
	{
		return;
	}

	if (!TryEnableShaderDrawAndShaderPrint(*View, 2000u, 2000u))
	{
		return;
	}

	auto CreateBoundBuffer = [&](const TArray<FHairStrandsDebugData::FCullData::FBound>& In)
	{
		TArray<float> Bound; Bound.Reserve(In.Num() * 6);
		for (auto& B : In)
		{
			Bound.Add(B.Min.X);
			Bound.Add(B.Min.Y);
			Bound.Add(B.Min.Z);
			Bound.Add(B.Max.X);
			Bound.Add(B.Max.Y);
			Bound.Add(B.Max.Z);
		}
		return CreateVertexBuffer(GraphBuilder, TEXT("Hair.Debug.InstanceBounds"), FRDGBufferDesc::CreateBufferDesc(4, Bound.Num()), Bound.GetData(), 4u * Bound.Num());
	};

	uint32 LightIndex = 0;
	for (auto It : View->HairStrandsViewData.DebugData.CullData.DirectionalLights)
	{
		FRDGBufferRef InstanceBoundInLightSpaceBuffer = CreateBoundBuffer(It.InstanceBoundInLightSpace);
		FRDGBufferRef InstanceBoundInWorldSpaceBuffer = CreateBoundBuffer(It.InstanceBoundInWorldSpace);
		FRDGBufferRef InstanceIntersectionBuffer = CreateVertexBuffer(GraphBuilder, TEXT("Hair.Debug.InstanceIntersection"), FRDGBufferDesc::CreateBufferDesc(4, It.InstanceIntersection.Num()), It.InstanceIntersection.GetData(), 4u * It.InstanceIntersection.Num());

		FHairDebugShadowCullingCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairDebugShadowCullingCS::FParameters>();
		Parameters->InstanceCount = InstanceCount;
		Parameters->LightCenter = It.Center;
		Parameters->LightExtent = It.Extent;
		Parameters->LightToWorld = FMatrix44f(It.LightToWorld);
		Parameters->ViewWorldToProj = FMatrix44f(View->ViewMatrices.GetWorldToClip());
		Parameters->ViewProjToWorld = FMatrix44f(View->ViewMatrices.GetClipToWorld());
		Parameters->InstanceBoundInLightSpace = GraphBuilder.CreateSRV(InstanceBoundInLightSpaceBuffer, PF_R32_FLOAT);
		Parameters->InstanceBoundInWorldSpace = GraphBuilder.CreateSRV(InstanceBoundInWorldSpaceBuffer, PF_R32_FLOAT);
		Parameters->InstanceIntersection = GraphBuilder.CreateSRV(InstanceIntersectionBuffer, PF_R32_UINT);
		Parameters->ViewUniformBuffer = View->ViewUniformBuffer;
		ShaderPrint::SetParameters(GraphBuilder, View->ShaderPrintData, Parameters->ShaderPrintUniformBuffer);
		Parameters->HairStrands = View->HairStrandsViewData.UniformBuffer;
		TShaderMapRef<FHairDebugShadowCullingCS> ComputeShader(View->ShaderMap);

		ClearUnusedGraphResources(ComputeShader, Parameters);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::DebugShadowCulling(%d/%d)", LightIndex++, LightCount),
			ComputeShader,
			Parameters,
			FIntVector(1, 1, 1));
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairDebugPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairDebugPS);
	SHADER_USE_PARAMETER_STRUCT(FHairDebugPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, OutputResolution)
		SHADER_PARAMETER(uint32, FastResolveMask)
		SHADER_PARAMETER(uint32, DebugMode)
		SHADER_PARAMETER(int32, SampleIndex)
		SHADER_PARAMETER(uint32, MaxSampleCount)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairCountTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairCountUintTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, DepthStencilTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_DEBUG_MODE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairDebugPS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainPS", SF_Pixel);

static void AddDebugHairPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo* View,
	const EGroomViewMode InDebugMode,
	const FHairStrandsVisibilityData& VisibilityData,
	FRDGTextureSRVRef InDepthStencilTexture,
	FRDGTextureRef& OutTarget)
{
	check(OutTarget);
	check(InDebugMode == EGroomViewMode::TAAResolveType || 
		InDebugMode == EGroomViewMode::SamplePerPixel || 
		InDebugMode == EGroomViewMode::CoverageType || 
		InDebugMode == EGroomViewMode::Coverage ||
		InDebugMode == EGroomViewMode::MaterialDepth ||
		InDebugMode == EGroomViewMode::MaterialBaseColor ||
		InDebugMode == EGroomViewMode::MaterialRoughness ||
		InDebugMode == EGroomViewMode::MaterialSpecular ||
		InDebugMode == EGroomViewMode::MaterialTangent);

	if (!VisibilityData.CoverageTexture || !VisibilityData.NodeIndex || !VisibilityData.NodeData) return;
	if (InDebugMode == EGroomViewMode::TAAResolveType && !InDepthStencilTexture) return;

	const FIntRect Viewport = View->ViewRect;
	const FIntPoint Resolution(Viewport.Width(), Viewport.Height());

	uint32 InternalDebugMode = 0;
	switch (InDebugMode)
	{
		case EGroomViewMode::SamplePerPixel:	InternalDebugMode = 0; break;
		case EGroomViewMode::CoverageType:		InternalDebugMode = 1; break;
		case EGroomViewMode::TAAResolveType:	InternalDebugMode = 2; break;
		case EGroomViewMode::Coverage:			InternalDebugMode = 3; break;
		case EGroomViewMode::MaterialDepth:		InternalDebugMode = 4; break;
		case EGroomViewMode::MaterialBaseColor:	InternalDebugMode = 5; break;
		case EGroomViewMode::MaterialRoughness:	InternalDebugMode = 6; break;
		case EGroomViewMode::MaterialSpecular:	InternalDebugMode = 7; break;
		case EGroomViewMode::MaterialTangent:	InternalDebugMode = 8; break;
	};

	FHairDebugPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairDebugPS::FParameters>();
	Parameters->ViewUniformBuffer = View->ViewUniformBuffer;
	Parameters->OutputResolution = Resolution;
	Parameters->FastResolveMask = STENCIL_TEMPORAL_RESPONSIVE_AA_MASK;
	Parameters->HairStrands = View->HairStrandsViewData.UniformBuffer;
	Parameters->DepthStencilTexture = InDepthStencilTexture;
	Parameters->LinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters->DebugMode = InternalDebugMode;
	Parameters->SampleIndex = 0;
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutTarget, ERenderTargetLoadAction::ELoad, 0);
	TShaderMapRef<FScreenVS> VertexShader(View->ShaderMap);

	TShaderMapRef<FHairDebugPS> PixelShader(View->ShaderMap);

	ClearUnusedGraphResources(PixelShader, Parameters);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::DebugMode(%s)", GetGroomViewModeName(InDebugMode)),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, VertexShader, PixelShader, Viewport, Resolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_One, BF_Zero>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			Viewport.Width(), Viewport.Height(),
			Viewport.Min.X, Viewport.Min.Y,
			Viewport.Width(), Viewport.Height(),
			Viewport.Size(),
			Resolution,
			VertexShader,
			EDRF_UseTriangleOptimization);
	});
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FDeepShadowVisualizePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDeepShadowVisualizePS);
	SHADER_USE_PARAMETER_STRUCT(FDeepShadowVisualizePS, FGlobalShader);

	class FOutputType : SHADER_PERMUTATION_INT("PERMUTATION_OUTPUT_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FOutputType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, DomScale)
		SHADER_PARAMETER(FVector2f, OutputResolution)
		SHADER_PARAMETER(FVector2f, InvOutputResolution)
		SHADER_PARAMETER(FIntVector4, HairViewRect)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DeepShadowDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DeepShadowLayerTexture)

		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_VISUALIZEDOM"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeepShadowVisualizePS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "VisualizeDomPS", SF_Pixel);

static void AddDebugDeepShadowTexturePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo* View,
	const FIntRect& HairViewRect,
	const FHairStrandsDeepShadowData* ShadowData,
	const FHairStrandsDeepShadowResources* Resources,
	FRDGTextureRef& OutTarget)
{
	check(OutTarget);

	const FIntRect Viewport = View->ViewRect;
	const FIntPoint Resolution(Viewport.Width(), Viewport.Height());

	FDeepShadowVisualizePS::FParameters* Parameters = GraphBuilder.AllocParameters<FDeepShadowVisualizePS::FParameters>();
	Parameters->DomScale = GDeepShadowDebugScale;
	Parameters->OutputResolution = Resolution;
	Parameters->InvOutputResolution = FVector2f(1.f / Resolution.X, 1.f / Resolution.Y);
	Parameters->DeepShadowDepthTexture = Resources ? Resources->DepthAtlasTexture : nullptr;
	Parameters->DeepShadowLayerTexture = Resources ? Resources->LayersAtlasTexture : nullptr;
	Parameters->LinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters->HairViewRect = FIntVector4(HairViewRect.Min.X, HairViewRect.Min.Y, HairViewRect.Width(), HairViewRect.Height());
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutTarget, ERenderTargetLoadAction::ELoad, 0);
	TShaderMapRef<FScreenVS> VertexShader(View->ShaderMap);
	FDeepShadowVisualizePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FDeepShadowVisualizePS::FOutputType>(ShadowData ? 0 : 1);
	TShaderMapRef<FDeepShadowVisualizePS> PixelShader(View->ShaderMap, PermutationVector);

	ClearUnusedGraphResources(PixelShader, Parameters);

	GraphBuilder.AddPass(
		ShadowData ? RDG_EVENT_NAME("DebugDeepShadowTexture") : RDG_EVENT_NAME("DebugHairViewRect"),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, VertexShader, PixelShader, Viewport, Resolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			Viewport.Width(), Viewport.Height(),
			Viewport.Min.X, Viewport.Min.Y,
			Viewport.Width(), Viewport.Height(),
			Viewport.Size(),
			Resolution,
			VertexShader,
			EDRF_UseTriangleOptimization);
	});
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FDeepShadowInfoCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDeepShadowInfoCS);
	SHADER_USE_PARAMETER_STRUCT(FDeepShadowInfoCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER(FVector2f, OutputResolution)
		SHADER_PARAMETER(uint32, AllocatedSlotCount)
		SHADER_PARAMETER(uint32, MacroGroupCount)
		SHADER_PARAMETER(uint32, bViewRectOptimizeEnabled)
		SHADER_PARAMETER(uint32, bVoxelizationEnabled)
		SHADER_PARAMETER(FIntPoint, AtlasResolution)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, MacroGroupAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, ShadowViewInfoBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_DOMINFO"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeepShadowInfoCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

static void AddDeepShadowInfoPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsDeepShadowResources& DeepShadowResources,
	const FHairStrandsMacroGroupResources& MacroGroupResources,
	FRDGTextureRef& OutputTexture)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	if (DeepShadowResources.TotalAtlasSlotCount == 0)
	{
		return;
	}

	if (!TryEnableShaderDrawAndShaderPrint(View, DeepShadowResources.TotalAtlasSlotCount * 64, 2000))
	{
		return;
	}

	FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

	const FIntPoint Resolution(OutputTexture->Desc.Extent);
	FDeepShadowInfoCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDeepShadowInfoCS::FParameters>();
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->OutputResolution = Resolution;
	Parameters->AllocatedSlotCount = DeepShadowResources.TotalAtlasSlotCount;
	Parameters->MacroGroupCount = MacroGroupResources.MacroGroupCount;
	Parameters->SceneTextures = SceneTextures;
	Parameters->bViewRectOptimizeEnabled = IsHairStrandsViewRectOptimEnable() ? 1u : 0u;
	Parameters->bVoxelizationEnabled = View.HairStrandsViewData.VirtualVoxelResources.IsValid() ? 1u : 0u;
	Parameters->AtlasResolution = View.HairStrandsViewData.DeepShadowResources.DepthAtlasTexture->Desc.Extent;
	Parameters->MacroGroupAABBBuffer = GraphBuilder.CreateSRV(MacroGroupResources.MacroGroupAABBsBuffer, PF_R32_SINT);
	Parameters->ShadowViewInfoBuffer = GraphBuilder.CreateSRV(DeepShadowResources.DeepShadowViewInfoBuffer);
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);
	Parameters->OutputTexture = GraphBuilder.CreateUAV(OutputTexture);

	TShaderMapRef<FDeepShadowInfoCS> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::DeepShadowDebugInfo"), ComputeShader, Parameters, FIntVector(1, 1, 1));
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FVoxelVirtualRaymarchingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelVirtualRaymarchingCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelVirtualRaymarchingCS, FGlobalShader);

	class FTraversalType : SHADER_PERMUTATION_INT("PERMUTATION_TRAVERSAL", 2);
	using FPermutationDomain = TShaderPermutationDomain<FTraversalType>;
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER(FVector2f, OutputResolution)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(uint32, MacroGroupCount)
		SHADER_PARAMETER(uint32, MaxTotalPageIndexCount)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthBeforeCompositionTexture)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualVoxelParameters, VirtualVoxel)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, TotalValidPageCounter)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Skip optimization for avoiding long compilation time due to large UAV writes
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("SHADER_VOXEL_DEBUG"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVoxelVirtualRaymarchingCS, "/Engine/Private/HairStrands/HairStrandsVoxelDebug.usf", "MainCS", SF_Compute);

static void AddVoxelPageRaymarchingPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FHairStrandsVoxelResources& VoxelResources,
	FRDGTextureRef& OutputTexture)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	if (!TryEnableShaderDrawAndShaderPrint(View, 4000, 2000))
	{
		return;
	}

	FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

	const FIntPoint Resolution(OutputTexture->Desc.Extent);
	for (const FHairStrandsMacroGroupData& MacroGroupData : MacroGroupDatas)
	{
		FVoxelVirtualRaymarchingCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVoxelVirtualRaymarchingCS::FParameters>();
		Parameters->ViewUniformBuffer		= View.ViewUniformBuffer;
		Parameters->OutputResolution		= Resolution;
		Parameters->SceneTextures			= SceneTextures;
		Parameters->MacroGroupId			= MacroGroupData.MacroGroupId;
		Parameters->MacroGroupCount			= MacroGroupDatas.Num();
		Parameters->MaxTotalPageIndexCount  = VoxelResources.Parameters.Common.PageIndexCount;
		Parameters->VirtualVoxel			= VoxelResources.UniformBuffer;
		Parameters->TotalValidPageCounter	= GraphBuilder.CreateSRV(VoxelResources.PageIndexGlobalCounter, PF_R32_UINT);
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);
		Parameters->OutputTexture			= GraphBuilder.CreateUAV(OutputTexture);
		Parameters->SceneDepthBeforeCompositionTexture = View.HairStrandsViewData.DebugData.Common.SceneDepthTextureBeforeCompsition;
		if (Parameters->SceneDepthBeforeCompositionTexture == nullptr)
		{
			Parameters->SceneDepthBeforeCompositionTexture = SceneTextures.SceneDepthTexture;
		}

		FVoxelVirtualRaymarchingCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FVoxelVirtualRaymarchingCS::FTraversalType>(GHairVirtualVoxel_DebugTraversalType > 0 ? 1 : 0);
		TShaderMapRef<FVoxelVirtualRaymarchingCS> ComputeShader(View.ShaderMap, PermutationVector);

		const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(OutputTexture->Desc.Extent.X, OutputTexture->Desc.Extent.Y, 1), FIntVector(8, 8, 1));
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VoxelVirtualRaymarching"), ComputeShader, Parameters, DispatchCount);
	}
}
///////////////////////////////////////////////////////////////////////////////////////////////////

class FDebugHairTangentCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDebugHairTangentCS);
	SHADER_USE_PARAMETER_STRUCT(FDebugHairTangentCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrint)
		SHADER_PARAMETER(FVector2f, OutputResolution)
		SHADER_PARAMETER(FIntPoint, TileCount)
		SHADER_PARAMETER(uint32, TileSize)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_SAMPLER(SamplerState, BilinearTextureSampler)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TANGENT"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDebugHairTangentCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "MainCS", SF_Compute);

static void AddDebugHairTangentPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef& OutputTexture)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	if (!TryEnableShaderDrawAndShaderPrint(View, 64000u, 1000u))
	{
		return;
	}

	FDebugHairTangentCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDebugHairTangentCS::FParameters>();
	Parameters->ViewUniformBuffer		= View.ViewUniformBuffer;
	Parameters->HairStrands				= View.HairStrandsViewData.UniformBuffer;
	Parameters->OutputResolution		= OutputTexture->Desc.Extent;
	Parameters->TileSize				= 8;
	Parameters->TileCount				= FIntPoint(FMath::FloorToInt(Parameters->OutputResolution.X / Parameters->TileSize), FMath::FloorToInt(Parameters->OutputResolution.X / Parameters->TileSize));
	Parameters->SceneTextures			= SceneTextures.UniformBuffer;
	Parameters->BilinearTextureSampler	= TStaticSamplerState<SF_Bilinear>::GetRHI();
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrint);

	const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(OutputTexture->Desc.Extent.X, OutputTexture->Desc.Extent.Y, 1), FIntVector(8, 8, 1));
	ShaderPrint::RequestSpaceForLines(DispatchCount.X * DispatchCount.Y);

	TShaderMapRef<FDebugHairTangentCS> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::DebugTangentCS"), ComputeShader, Parameters, DispatchCount);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairVisibilityDebugPPLLCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairVisibilityDebugPPLLCS);
	SHADER_USE_PARAMETER_STRUCT(FHairVisibilityDebugPPLLCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(float, PPLLMeanListElementCountPerPixel)
		SHADER_PARAMETER(float, PPLLMaxTotalListElementCount)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PPLLCounter)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PPLLNodeIndex)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, PPLLNodeData)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, SceneColorTextureUAV)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

		static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Tool, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) { return EShaderPermutationPrecacheRequest::NotPrecached; }

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_PPLL_DEBUG"), 1);
		// Skip optimization for avoiding long compilation time due to large UAV writes
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
	}
};
IMPLEMENT_GLOBAL_SHADER(FHairVisibilityDebugPPLLCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "VisibilityDebugPPLLCS", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////
uint32 GetHairStrandsMeanSamplePerPixel(EShaderPlatform In);
static void InternalRenderHairStrandsDebugInfo(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	FViewInfo& View,
	FHairStrandsBookmarkParameters& Params)
{
	if (!ShaderPrint::IsSupported(View.GetShaderPlatform()))
	{
		return;
	}

	if (!Params.HasInstances())
	{
		return;
	}

	const float YStep = 14;
	const float ColumnWidth = 200;

	RDG_EVENT_SCOPE(GraphBuilder, "HairStrandsDebug");

	// Only render debug information for the main view
	const FSceneTextures& SceneTextures = View.GetSceneTextures();

	// Bookmark for calling debug rendering from the plugin
	{
		const FSceneView* View0 = Params.View;
		FShaderPrintData* View0ShaderPrintData = Params.ShaderPrintData;
		Params.View = &View;
		Params.ShaderPrintData = &View.ShaderPrintData;
		RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessDebug, Params);
		Params.View = View0;
		Params.ShaderPrintData = View0ShaderPrintData;
	}

	const EGroomViewMode ViewMode = GetGroomViewMode(View);

	// Display tangent vector for strands/cards/meshes
	if (ViewMode == EGroomViewMode::Tangent)
		{
			AddDebugHairTangentPass(GraphBuilder, View, SceneTextures, Params.SceneColorTexture);
		}

	// Draw LOD info 
	for (const FMeshBatchAndRelevance& Mesh : View.HairStrandsMeshElements)
	{
		const FHairGroupPublicData* GroupData = HairStrands::GetHairData(Mesh.Mesh);
		if (GroupData->bDebugDrawLODInfo)
		{
			AddPrintLODInfoPass(GraphBuilder, View, GroupData);
		}
	}
	for (const FMeshBatchAndRelevance& Mesh : View.HairCardsMeshElements)
	{
		const FHairGroupPublicData* GroupData = HairStrands::GetHairData(Mesh.Mesh);
		if (GroupData->bDebugDrawLODInfo)
		{
			AddPrintLODInfoPass(GraphBuilder, View, GroupData);
		}
	}

	// Pass this point, all debug rendering concern only hair strands data
	if (!HairStrands::HasViewHairStrandsData(View))
	{
		return;
	}

	const FScreenPassRenderTarget SceneColor(Params.SceneColorTexture, View.ViewRect, ERenderTargetLoadAction::ELoad);

	const FHairStrandsViewData& HairData = View.HairStrandsViewData;

	float ClusterY = 38;

	if (View.HairStrandsViewData.DebugData.CullData.bIsValid)
	{
		AddDebugHairShadowCullingPass(GraphBuilder, Scene, &View);
	}

	if (ViewMode == EGroomViewMode::MacroGroups)
	{
		AddDebugHairPrintPass(GraphBuilder, Scene, &View, ViewMode, HairData.VisibilityData, HairData.MacroGroupDatas, HairData.MacroGroupResources, SceneTextures.Stencil);
	}

	if (ViewMode == EGroomViewMode::DeepOpacityMaps)
	{
		for (const FHairStrandsMacroGroupData& MacroGroup : HairData.MacroGroupDatas)
		{
			if (!HairData.DeepShadowResources.DepthAtlasTexture || !HairData.DeepShadowResources.LayersAtlasTexture)
			{
				continue;
			}

			for (const FHairStrandsDeepShadowData& DeepShadowData : MacroGroup.DeepShadowDatas)
			{
				const uint32 DomIndex = GDeepShadowDebugIndex;
				if (DeepShadowData.AtlasSlotIndex != DomIndex)
					continue;

				AddDebugDeepShadowTexturePass(GraphBuilder, &View, FIntRect(), &DeepShadowData, &HairData.DeepShadowResources, Params.SceneColorTexture);
			}
		}
	}

	// View Rect
	if (IsHairStrandsViewRectOptimEnable() && ViewMode == EGroomViewMode::MacroGroupScreenRect)
	{
		for (const FHairStrandsMacroGroupData& MacroGroupData : HairData.MacroGroupDatas)
		{
			AddDebugDeepShadowTexturePass(GraphBuilder, &View, MacroGroupData.ScreenRect, nullptr, nullptr, Params.SceneColorTexture);
		}

		const FIntRect TotalRect = ComputeVisibleHairStrandsMacroGroupsRect(View, View.ViewRect, HairData.MacroGroupDatas);
		AddDebugDeepShadowTexturePass(GraphBuilder, &View, TotalRect, nullptr, nullptr, Params.SceneColorTexture);
	}
	

	// Render Frustum for all lights & macro groups 
	{
		if ((ViewMode == EGroomViewMode::LightBounds || ViewMode == EGroomViewMode::DeepOpacityMaps))
		{
			AddDeepShadowInfoPass(GraphBuilder, View, HairData.DeepShadowResources, HairData.MacroGroupResources, Params.SceneColorTexture);
		}
	}
	
	const bool bRunDebugPass =
		ViewMode == EGroomViewMode::TAAResolveType ||
		ViewMode == EGroomViewMode::SamplePerPixel ||
		ViewMode == EGroomViewMode::CoverageType ||
		ViewMode == EGroomViewMode::Coverage ||
		ViewMode == EGroomViewMode::MaterialDepth ||
		ViewMode == EGroomViewMode::MaterialBaseColor ||
		ViewMode == EGroomViewMode::MaterialRoughness ||
		ViewMode == EGroomViewMode::MaterialSpecular ||
		ViewMode == EGroomViewMode::MaterialTangent;
	if (bRunDebugPass)
	{
		AddDebugHairPass(GraphBuilder, &View, ViewMode, HairData.VisibilityData, SceneTextures.Stencil, Params.SceneColorTexture);
		AddDebugHairPrintPass(GraphBuilder, Scene, &View, ViewMode, HairData.VisibilityData, HairData.MacroGroupDatas, HairData.MacroGroupResources, SceneTextures.Stencil);
	}
	else if (ViewMode == EGroomViewMode::Tile)
	{
		check(HairData.VisibilityData.TileData.IsValid());
		AddHairStrandsDebugTilePass(GraphBuilder, View, Params.SceneColorTexture, HairData.VisibilityData.TileData);
	}

	const bool bIsVoxelMode = ViewMode == EGroomViewMode::VoxelsDensity;
	if (bIsVoxelMode)
	{
		if (HairData.VirtualVoxelResources.IsValid())
		{
			AddVoxelPageRaymarchingPass(GraphBuilder, View, HairData.MacroGroupDatas, HairData.VirtualVoxelResources, Params.SceneColorTexture);
		}
	}

	if (HairData.DebugData.IsPPLLDataValid()) // Check if PPLL rendering is used and its debug view is enabled.
	{
		// Force ShaderPrint on.
		ShaderPrint::SetEnabled(true);
		ShaderPrint::RequestSpaceForCharacters(256);

		const FIntPoint PPLLResolution = HairData.DebugData.PPLLData.NodeIndexTexture->Desc.Extent;
		FHairVisibilityDebugPPLLCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairVisibilityDebugPPLLCS::FParameters>();
		PassParameters->PPLLMeanListElementCountPerPixel = GetHairStrandsMeanSamplePerPixel(View.GetShaderPlatform());
		PassParameters->PPLLMaxTotalListElementCount = HairData.DebugData.PPLLData.NodeDataBuffer->Desc.NumElements;
		PassParameters->PPLLCounter = HairData.DebugData.PPLLData.NodeCounterTexture;
		PassParameters->PPLLNodeIndex = HairData.DebugData.PPLLData.NodeIndexTexture;
		PassParameters->PPLLNodeData = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(HairData.DebugData.PPLLData.NodeDataBuffer));
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->SceneColorTextureUAV = GraphBuilder.CreateUAV(Params.SceneColorTexture);
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintParameters);

		FHairVisibilityDebugPPLLCS::FPermutationDomain PermutationVector;
		TShaderMapRef<FHairVisibilityDebugPPLLCS> ComputeShader(View.ShaderMap, PermutationVector);
		FIntVector TextureSize = Params.SceneColorTexture->Desc.GetSize(); TextureSize.Z = 1;
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::PPLLDebug"), ComputeShader, PassParameters, FIntVector::DivideAndRoundUp(TextureSize, FIntVector(8, 8, 1)));
	}

	if (ViewMode != EGroomViewMode::None)
	{
		AddDrawCanvasPass(GraphBuilder, {}, View, SceneColor, [&View, YStep, ViewMode](FCanvas& Canvas)
		{
			float X = 40;
			float Y = View.ViewRect.Height() - YStep * 3.f;
			FString Line;
			if (ViewMode != EGroomViewMode::None)
			{
				Line = FString::Printf(TEXT("Hair Debug mode - %s"), GetGroomViewModeName(ViewMode));
			}

			Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 0));
		});
	}
}

void RenderHairStrandsDebugInfo(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	TArrayView<FViewInfo> Views,
	FHairStrandsBookmarkParameters& Parameters)
{
	bool bHasHairData = false;
	for (FViewInfo& View : Views)
	{
		InternalRenderHairStrandsDebugInfo(GraphBuilder, Scene, View, Parameters);
	}
}
