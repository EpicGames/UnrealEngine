// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsDebug.h: Hair strands debug display.
=============================================================================*/

#pragma once

#include "RenderGraph.h"
#include "SceneRendering.h"

void RenderHairStrandsDebugInfo(
	FRDGBuilder& GraphBuilder,
	FScene* Scene, 
	TArrayView<FViewInfo> Views,
	FHairStrandsBookmarkParameters& Parameters);
