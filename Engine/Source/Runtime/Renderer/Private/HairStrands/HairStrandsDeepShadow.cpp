// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsDeepShadow.h"
#include "HairStrandsRasterCommon.h"
#include "HairStrandsUtils.h"
#include "HairStrandsData.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "ScenePrivate.h"
#include "SystemTextures.h"

// this is temporary until we split the voxelize and DOM path
static int32 GDeepShadowResolution = 2048;
static FAutoConsoleVariableRef CVarDeepShadowResolution(TEXT("r.HairStrands.DeepShadow.Resolution"), GDeepShadowResolution, TEXT("Shadow resolution for Deep Opacity Map rendering. (default = 2048)"));

static int32 GDeepShadowMinResolution = 64;
static FAutoConsoleVariableRef CVarDeepShadowMinResolution(TEXT("r.HairStrands.DeepShadow.MinResolution"), GDeepShadowMinResolution, TEXT("Minimum shadow resolution for shadow atlas tiles for Deep Opacity Map rendering. (default = 64)"));

static int32 GDeepShadowInjectVoxelDepth = 0;
static FAutoConsoleVariableRef CVarDeepShadowInjectVoxelDepth(TEXT("r.HairStrands.DeepShadow.InjectVoxelDepth"), GDeepShadowInjectVoxelDepth, TEXT("Inject voxel content to generate the deep shadow map instead of rasterizing groom. This is an experimental path"));

DECLARE_GPU_STAT(HairStrandsDeepShadow);
DECLARE_GPU_STAT(HairStrandsDeepShadowFrontDepth);
DECLARE_GPU_STAT(HairStrandsDeepShadowLayers);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Inject voxel structure into shadow map to amortize the tracing, and rely on look up kernel to 
// filter limited resolution
BEGIN_SHADER_PARAMETER_STRUCT(FHairStransShadowDepthInjectionParameters, )
	SHADER_PARAMETER(FVector2f, OutputResolution)
	SHADER_PARAMETER(uint32, AtlasSlotIndex)

	SHADER_PARAMETER(FVector3f, LightDirection)
	SHADER_PARAMETER(uint32, MacroGroupId)

	SHADER_PARAMETER(FVector3f, TranslatedLightPosition)
	SHADER_PARAMETER(uint32, bIsDirectional)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FDeepShadowViewInfo>, DeepShadowViewInfoBuffer)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualVoxelParameters, VirtualVoxel)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FHairStrandsShadowDepthInjection : public FGlobalShader
{
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_DEPTH_INJECTION"), 1);
	}

	FHairStrandsShadowDepthInjection() = default;
	FHairStrandsShadowDepthInjection(const CompiledShaderInitializerType& Initializer) : FGlobalShader(Initializer) {}
};

class FHairStrandsShadowDepthInjectionVS : public FHairStrandsShadowDepthInjection
{
	DECLARE_GLOBAL_SHADER(FHairStrandsShadowDepthInjectionVS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsShadowDepthInjectionVS, FHairStrandsShadowDepthInjection);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStransShadowDepthInjectionParameters, Pass)
		END_SHADER_PARAMETER_STRUCT()
};

class FHairStrandsShadowDepthInjectionPS : public FHairStrandsShadowDepthInjection
{
	DECLARE_GLOBAL_SHADER(FHairStrandsShadowDepthInjectionPS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsShadowDepthInjectionPS, FHairStrandsShadowDepthInjection);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStransShadowDepthInjectionParameters, Pass)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsShadowDepthInjectionPS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FHairStrandsShadowDepthInjectionVS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "MainVS", SF_Vertex);

void AddInjectHairVoxelShadowCaster(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const bool bClear,
	const FHairStrandsDeepShadowData& DomData,
	FHairStrandsVoxelResources& VoxelResources,
	FRDGBufferSRVRef DeepShadowViewInfoBufferSRV,
	FRDGTextureRef OutDepthTexture)
{
	const FIntPoint AtlasResolution = DomData.AtlasResolution;

	FHairStransShadowDepthInjectionParameters* Parameters = GraphBuilder.AllocParameters<FHairStransShadowDepthInjectionParameters>();
	Parameters->OutputResolution = AtlasResolution;
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->RenderTargets.DepthStencil = FDepthStencilBinding(OutDepthTexture, bClear ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
	Parameters->VirtualVoxel = VoxelResources.UniformBuffer;
	Parameters->LightDirection = DomData.LightDirection;
	Parameters->TranslatedLightPosition = DomData.TranslatedLightPosition;
	Parameters->bIsDirectional = DomData.bIsLightDirectional ? 1 : 0;
	Parameters->MacroGroupId = DomData.MacroGroupId;
	Parameters->DeepShadowViewInfoBuffer = DeepShadowViewInfoBufferSRV;
	Parameters->AtlasSlotIndex = DomData.AtlasSlotIndex;

	TShaderMapRef<FHairStrandsShadowDepthInjectionVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FHairStrandsShadowDepthInjectionPS> PixelShader(View.ShaderMap);
	FHairStrandsShadowDepthInjectionVS::FParameters ParametersVS;
	FHairStrandsShadowDepthInjectionPS::FParameters ParametersPS;
	ParametersVS.Pass = *Parameters;
	ParametersPS.Pass = *Parameters;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrandsShadowDepthInjection"),
		Parameters,
		ERDGPassFlags::Raster,
		[ParametersVS, ParametersPS, VertexShader, PixelShader, AtlasResolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{

			// Apply additive blending pipeline state.
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Max, BF_SourceColor, BF_DestColor, BO_Max, BF_One, BF_One>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_Greater>::GetRHI();
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), ParametersPS);

			// Emit an instanced quad draw call on the order of the number of pixels on the screen.	
			RHICmdList.SetViewport(0, 0, 0.0f, AtlasResolution.X, AtlasResolution.Y, 1.0f);
			RHICmdList.DrawPrimitive(0, 12, 1);
		});
}


///////////////////////////////////////////////////////////////////////////////////////////////////

typedef TArray<const FLightSceneInfo*, SceneRenderingAllocator> FLightSceneInfos;

static FLightSceneInfos GetVisibleDeepShadowLights(const FScene* Scene, const FViewInfo& View)
{
	// Collect all visible lights for the current view
	FLightSceneInfos Out;
	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
	{
		const FLightSceneInfo* const LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;

		if (!LightSceneInfo->ShouldRenderLightViewIndependent())
			continue;

		// Check if the light is visible in any of the views.
		{
			const bool bIsCompatible = LightSceneInfo->ShouldRenderLight(View) && LightSceneInfo->Proxy->CastsHairStrandsDeepShadow();
			if (!bIsCompatible)
				continue;

			Out.Add(LightSceneInfo);
		}
	}

	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FDeepShadowCreateViewInfoCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDeepShadowCreateViewInfoCS);
	SHADER_USE_PARAMETER_STRUCT(FDeepShadowCreateViewInfoCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )

		SHADER_PARAMETER(float, RasterizationScale)

		SHADER_PARAMETER(FIntPoint, SlotResolution)
		SHADER_PARAMETER(uint32, SlotIndexCount)
		SHADER_PARAMETER(uint32, MacroGroupCount)

		SHADER_PARAMETER(float, AABBScale)
		SHADER_PARAMETER(float, MaxHafFovInRad)

		SHADER_PARAMETER(FUintVector2, AtlasResolution)
		SHADER_PARAMETER(FVector2f, AtlasTexelSize)
		SHADER_PARAMETER(uint32, MinAtlasTileResolution)
		SHADER_PARAMETER(uint32, MinAtlasTileResolutionLog2)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, LightDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, MacroGroupAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FDeepShadowViewInfo>, OutShadowViewInfoBuffer)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)

	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ALLOCATE"), 1);
		OutEnvironment.SetDefine(TEXT("MAX_SLOT_COUNT"), FHairStrandsDeepShadowResources::MaxAtlasSlotCount);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeepShadowCreateViewInfoCS, "/Engine/Private/HairStrands/HairStrandsDeepShadowAllocation.usf", "CreateViewInfo", SF_Compute);
///////////////////////////////////////////////////////////////////////////////////////////////////

bool IsHairStrandsForVoxelTransmittanceAndShadowEnable(EShaderPlatform ShaderPlatform);
float GetDeepShadowMaxFovAngle();
float GetDeepShadowRasterizationScale();
float GetDeepShadowAABBScale();
FVector4f ComputeDeepShadowLayerDepths(float LayerDistribution);

void RenderHairStrandsDeepShadows(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	FViewInfo& View,
	FInstanceCullingManager& InstanceCullingManager)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CLM_RenderDeepShadow);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsDeepShadow, "HairStrandsDeepShadow");

	const FLightSceneInfos VisibleLights = GetVisibleDeepShadowLights(Scene, View);
	FHairStrandsMacroGroupDatas& MacroGroupDatas = View.HairStrandsViewData.MacroGroupDatas;
	FHairStrandsMacroGroupResources MacroGroupResources = View.HairStrandsViewData.MacroGroupResources;
	FHairStrandsDeepShadowResources& DeepShadowResources = View.HairStrandsViewData.DeepShadowResources;
	FHairStrandsVoxelResources VirtualVoxelResources = View.HairStrandsViewData.VirtualVoxelResources;

	// Reset view data
	for (FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
	{
		MacroGroup.DeepShadowDatas.Empty();
	}
	DeepShadowResources = FHairStrandsDeepShadowResources();

	{
		if (!View.Family)
		{
			return;
		}

		if (MacroGroupDatas.Num() == 0 || 
			VisibleLights.Num() == 0 ||
			IsHairStrandsForVoxelTransmittanceAndShadowEnable(View.GetShaderPlatform())) 
		{
			return; 
		}

		// 0. Compute the number of DOM which need to be created and insert default value
		uint32 DOMSlotCount = 0;
		for (const FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
		{			
			const FBoxSphereBounds MacroGroupBounds = MacroGroup.Bounds;
			for (const FLightSceneInfo* LightInfo : VisibleLights)
			{
				const FLightSceneProxy* LightProxy = LightInfo->Proxy;
				if (!LightProxy->AffectsBounds(MacroGroupBounds))
				{
					continue;
				}

				// Run out of atlas slot
				if (DOMSlotCount >= FHairStrandsDeepShadowResources::MaxAtlasSlotCount)
				{
					continue;
				}

				DOMSlotCount++;
			}
		}

		if (DOMSlotCount == 0)
			return;

		const uint32 AtlasSlotX = FGenericPlatformMath::CeilToInt(FMath::Sqrt(static_cast<float>(DOMSlotCount)));
		const FIntPoint AtlasSlotResolution(GDeepShadowResolution, GDeepShadowResolution);
		const FIntPoint AtlasResolution = FIntPoint(GDeepShadowResolution, GDeepShadowResolution);
	
		DeepShadowResources.TotalAtlasSlotCount = 0;

		// Create Atlas resources for DOM. It is shared for all lights, across all views
		bool bClear = true;
		FRDGTextureRef FrontDepthAtlasTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(AtlasResolution, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource), TEXT("Hair.ShadowDepth"));
		FRDGTextureRef DeepShadowLayersAtlasTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(AtlasResolution, PF_FloatRGBA, FClearValueBinding::Transparent, TexCreate_RenderTargetable | TexCreate_ShaderResource), TEXT("Hair.DeepShadowLayers"));

		const FVector& TranslatedWorldOffset = View.ViewMatrices.GetPreViewTranslation();

		// 1. Cull light per macro groups
		// TODO add support for multiple view: need to deduplicate light which are visible accross several views
		// Allocate atlas CPU slot
		uint32 TotalAtlasSlotIndex = 0;
		for (FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
		{
			// List of all the light in the scene.
			for (const FLightSceneInfo* LightInfo : VisibleLights)
			{
				FBoxSphereBounds MacroGroupBounds = MacroGroup.Bounds;

				const FLightSceneProxy* LightProxy = LightInfo->Proxy;
				if (!LightProxy->AffectsBounds(MacroGroupBounds))
				{
					continue;
				}
					
				if (TotalAtlasSlotIndex >= FHairStrandsDeepShadowResources::MaxAtlasSlotCount)
				{
					continue;
				}

				const ELightComponentType LightType = (ELightComponentType)LightProxy->GetLightType();
				const bool bIsDirectional = LightType == ELightComponentType::LightType_Directional;
				FMinHairRadiusAtDepth1 MinStrandRadiusAtDepth1;

				// Note: LightPosition.W is used in the transmittance mask shader to differentiate between directional and local lights.
				FHairStrandsDeepShadowData& DomData = MacroGroup.DeepShadowDatas.AddZeroed_GetRef();
				FMatrix CPU_TranslatedWorldToLightTransform;
				ComputeTranslatedWorldToLightClip(TranslatedWorldOffset, CPU_TranslatedWorldToLightTransform, MinStrandRadiusAtDepth1, MacroGroupBounds, *LightProxy, LightType, AtlasSlotResolution);
				DomData.LightDirection = (FVector3f)LightProxy->GetDirection();
				DomData.TranslatedLightPosition = FVector4f(FVector3f((FVector4f)LightProxy->GetPosition() + (FVector3f)TranslatedWorldOffset), bIsDirectional ? 0 : 1);
				DomData.LightLuminance = LightProxy->GetColor();
				DomData.LayerDistribution = LightProxy->GetDeepShadowLayerDistribution();
				DomData.bIsLightDirectional = bIsDirectional;
				DomData.LightId = LightInfo->Id;
				DomData.AtlasResolution = AtlasResolution;
				DomData.Bounds = MacroGroupBounds;
				DomData.MacroGroupId = MacroGroup.MacroGroupId;
				DomData.CPU_MinStrandRadiusAtDepth1 = MinStrandRadiusAtDepth1;
				DomData.AtlasSlotIndex = TotalAtlasSlotIndex;
				TotalAtlasSlotIndex++;
			}
		}

		// Sanity check
		check(DOMSlotCount == TotalAtlasSlotIndex); 

		DeepShadowResources.TotalAtlasSlotCount = TotalAtlasSlotIndex;
		DeepShadowResources.AtlasSlotResolution = AtlasSlotResolution;

		FRDGBufferRef DeepShadowViewInfoBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc((16 + 16 + 4 + 3 + 1) * sizeof(float), FMath::Max(1u, TotalAtlasSlotIndex)), TEXT("Hair.DeepShadowViewInfo"));
		FRDGBufferSRVRef DeepShadowViewInfoBufferSRV = GraphBuilder.CreateSRV(DeepShadowViewInfoBuffer);

		// 2. Allocate slots
		{
			// Allocate and create projection matrix and Min radius
			// Stored FDeepShadowViewInfo structs
			// See HairStrandsDeepShadowCommonStruct.ush for more details
			FDeepShadowCreateViewInfoCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDeepShadowCreateViewInfoCS::FParameters>();

			struct FLightData
			{
				FVector3f LightDirection;
				uint32 MacroGroupId;
				FVector3f TranslatedLightPosition;
				uint32 bIsLightDirectional;
			};
			static_assert(sizeof(FLightData) == 32u);
			TArray<FLightData> LightData;
			LightData.Reserve(DOMSlotCount);
			for (FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
			{
				for (FHairStrandsDeepShadowData& DomData : MacroGroup.DeepShadowDatas)
				{
					FLightData& Data = LightData.AddDefaulted_GetRef();
					Data.LightDirection				= DomData.LightDirection;
					Data.TranslatedLightPosition	= DomData.TranslatedLightPosition;
					Data.MacroGroupId				= DomData.MacroGroupId;
					Data.bIsLightDirectional		= DomData.bIsLightDirectional ? 1 : 0;
				}
			}

			FRDGBufferRef LightDataBuffer= CreateStructuredBuffer(GraphBuilder, TEXT("Hair.DeepShadow.LightData"), sizeof(FLightData), LightData.Num(), LightData.GetData(), sizeof(FLightData) * LightData.Num());

			Parameters->LightDataBuffer = GraphBuilder.CreateSRV(LightDataBuffer);
			Parameters->SlotResolution = DeepShadowResources.AtlasSlotResolution;
			Parameters->SlotIndexCount = DeepShadowResources.TotalAtlasSlotCount;
			Parameters->MacroGroupCount = MacroGroupDatas.Num();
			Parameters->MacroGroupAABBBuffer = GraphBuilder.CreateSRV(MacroGroupResources.MacroGroupAABBsBuffer, PF_R32_SINT);
			Parameters->OutShadowViewInfoBuffer = GraphBuilder.CreateUAV(DeepShadowViewInfoBuffer);

			Parameters->MaxHafFovInRad = 0.5f * FMath::DegreesToRadians(GetDeepShadowMaxFovAngle());
			Parameters->AABBScale = GetDeepShadowAABBScale();
			Parameters->RasterizationScale = GetDeepShadowRasterizationScale();
			Parameters->AtlasResolution = FUintVector2(AtlasResolution.X, AtlasResolution.Y);
			Parameters->AtlasTexelSize = FVector2f(1.0f / AtlasResolution.X, 1.0f / AtlasResolution.Y);
			Parameters->MinAtlasTileResolutionLog2 = FMath::FloorLog2(FMath::Clamp(GDeepShadowMinResolution, 16u, GDeepShadowResolution)); // A shadow map resolution of less than 16x16 is highly unlikely to ever be needed.
			Parameters->MinAtlasTileResolution = 1u << Parameters->MinAtlasTileResolutionLog2;
			Parameters->ViewUniformBuffer = View.ViewUniformBuffer;

			// Currently support only 32 instance group at max
			TShaderMapRef<FDeepShadowCreateViewInfoCS> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("HairStrandsDeepShadowAllocate"),
				ComputeShader,
				Parameters,
				FIntVector(1, 1, 1));
		}

		// 3. Render deep shadows
		for (FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
		{
			for (FHairStrandsDeepShadowData& DomData : MacroGroup.DeepShadowDatas)
			{
				const bool bIsOrtho = DomData.bIsLightDirectional;
				const FVector4f HairRenderInfo = PackHairRenderInfo(DomData.CPU_MinStrandRadiusAtDepth1.Primary, DomData.CPU_MinStrandRadiusAtDepth1.Stable, DomData.CPU_MinStrandRadiusAtDepth1.Primary, 1);
				const uint32 HairRenderInfoBits = PackHairRenderInfoBits(bIsOrtho, true /*bIsGPUDriven*/);
					
				const bool bDeepShadow = GDeepShadowInjectVoxelDepth == 0;
				// Inject voxel result into the deep shadow
				if (!bDeepShadow)
				{
					RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsDeepShadowFrontDepth, "HairStrandsDeepShadowFrontDepth");

					AddInjectHairVoxelShadowCaster(
						GraphBuilder,
						View,
						bClear,
						DomData,
						VirtualVoxelResources,
						DeepShadowViewInfoBufferSRV,
						FrontDepthAtlasTexture);

					if (bClear)
					{
						AddClearRenderTargetPass(GraphBuilder, DeepShadowLayersAtlasTexture);
					}
				}
					
				const FVector4f LayerDepths = ComputeDeepShadowLayerDepths(DomData.LayerDistribution);
				// Front depth
				if (bDeepShadow)
				{
					const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

					RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsDeepShadowFrontDepth, "HairStrandsDeepShadowFrontDepth");

					FHairDeepShadowRasterPassParameters* PassParameters = GraphBuilder.AllocParameters<FHairDeepShadowRasterPassParameters>();

					{
						FHairDeepShadowRasterUniformParameters* UniformParameters = GraphBuilder.AllocParameters<FHairDeepShadowRasterUniformParameters>();
						UniformParameters->AtlasSlotIndex = DomData.AtlasSlotIndex;
						UniformParameters->LayerDepths = LayerDepths;
						UniformParameters->FrontDepthTexture = SystemTextures.DepthDummy;
						UniformParameters->DeepShadowViewInfoBuffer = DeepShadowViewInfoBufferSRV;

						PassParameters->UniformBuffer = GraphBuilder.CreateUniformBuffer(UniformParameters);
					}

					PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(FrontDepthAtlasTexture, bClear ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);

					AddHairDeepShadowRasterPass(
						GraphBuilder,
						Scene,
						&View,
						MacroGroup.PrimitivesInfos,
						EHairStrandsRasterPassType::FrontDepth,
						DomData.AtlasResolution,
						HairRenderInfo,
						HairRenderInfoBits,
						DomData.LightDirection,
						PassParameters,
						InstanceCullingManager);
				}

				// Deep layers
				if (bDeepShadow)
				{
					RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsDeepShadowLayers, "HairStrandsDeepShadowLayers");

					FHairDeepShadowRasterPassParameters* PassParameters = GraphBuilder.AllocParameters<FHairDeepShadowRasterPassParameters>();

					{
						FHairDeepShadowRasterUniformParameters* UniformParameters = GraphBuilder.AllocParameters<FHairDeepShadowRasterUniformParameters>();
						UniformParameters->AtlasSlotIndex = DomData.AtlasSlotIndex;
						UniformParameters->LayerDepths = LayerDepths;
						UniformParameters->FrontDepthTexture = FrontDepthAtlasTexture;
						UniformParameters->DeepShadowViewInfoBuffer = DeepShadowViewInfoBufferSRV;

						PassParameters->UniformBuffer = GraphBuilder.CreateUniformBuffer(UniformParameters);
					}

					PassParameters->RenderTargets[0] = FRenderTargetBinding(DeepShadowLayersAtlasTexture, bClear ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad, 0);

					AddHairDeepShadowRasterPass(
						GraphBuilder,
						Scene,
						&View,
						MacroGroup.PrimitivesInfos,
						EHairStrandsRasterPassType::DeepOpacityMap,
						DomData.AtlasResolution,
						HairRenderInfo,
						HairRenderInfoBits,
						DomData.LightDirection,
						PassParameters,
						InstanceCullingManager);
				}
				bClear = false;
			}
		}
		DeepShadowResources.DepthAtlasTexture = FrontDepthAtlasTexture;
		DeepShadowResources.LayersAtlasTexture = DeepShadowLayersAtlasTexture;
		DeepShadowResources.DeepShadowViewInfoBuffer = DeepShadowViewInfoBuffer;
	}
}
