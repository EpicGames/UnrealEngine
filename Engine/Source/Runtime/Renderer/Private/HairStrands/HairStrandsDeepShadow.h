// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsDeepShadow.h: Hair strands deep shadow implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "HairStrandsInterface.h"
#include "RenderGraphResources.h"
#include "SceneTypes.h"
#include "SceneRendering.h"

void RenderHairStrandsDeepShadows(
	FRDGBuilder& GraphBuilder,
	const class FScene* Scene,
	FViewInfo& View,
	FInstanceCullingManager& InstanceCullingManager);
