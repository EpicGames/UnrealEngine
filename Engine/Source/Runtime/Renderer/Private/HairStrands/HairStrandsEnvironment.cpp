// Copyright Epic Games, Inc. All Rights Reserved.


#include "HairStrandsEnvironment.h"
#include "BasePassRendering.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "SceneTextureParameters.h"
#include "SceneRenderTargetParameters.h"
#include "RenderGraph.h"
#include "RenderGraphUtils.h"
#include "SceneRendering.h"
#include "PixelShaderUtils.h"
#include "SystemTextures.h"
#include "HairStrandsRendering.h"
#include "ReflectionEnvironment.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "RenderGraphEvent.h"
#include "PostProcess/PostProcessing.h"
#include "ShaderPrint.h"
#include "Lumen/LumenRadianceCache.h"
#include "Lumen/LumenScreenProbeGather.h"
#include "Lumen/LumenTracingUtils.h"
#include "IndirectLightRendering.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "RenderGraphEvent.h"
#include "SceneViewState.h"

///////////////////////////////////////////////////////////////////////////////////////////////////

static int32 GHairScatterSceneLighting = 0;
static FAutoConsoleVariableRef CVarHairScatterSceneLighting(TEXT("r.HairStrands.ScatterSceneLighting"), GHairScatterSceneLighting, TEXT("Enable scene color lighting scattering into hair (valid for short hair only), disabled by default."));

static int32 GHairSkylightingEnable = 1;
static FAutoConsoleVariableRef CVarHairSkylightingEnable(TEXT("r.HairStrands.SkyLighting"), GHairSkylightingEnable, TEXT("Enable sky lighting on hair."));

static int32 GHairSkyAOEnable = 1;
static FAutoConsoleVariableRef CVarHairSkyAOEnable(TEXT("r.HairStrands.SkyAO"), GHairSkyAOEnable, TEXT("Enable (sky) AO on hair."));

static float GHairSkylightingConeAngle = 3;
static FAutoConsoleVariableRef CVarHairSkylightingConeAngle(TEXT("r.HairStrands.SkyLighting.ConeAngle"), GHairSkylightingConeAngle, TEXT("Cone angle for tracing sky lighting on hair."));

static int32 GHairStrandsSkyLightingSampleCount = 16;
static FAutoConsoleVariableRef CVarHairStrandsSkyLightingSampleCount(TEXT("r.HairStrands.SkyLighting.SampleCount"), GHairStrandsSkyLightingSampleCount, TEXT("Number of samples used for evaluating multiple scattering and visible area (default is set to 16)."), ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GHairStrandsSkyAOSampleCount = 4;
static FAutoConsoleVariableRef CVarHairStrandsSkyAOSampleCount(TEXT("r.HairStrands.SkyAO.SampleCount"), GHairStrandsSkyAOSampleCount, TEXT("Number of samples used for evaluating hair AO (default is set to 16)."), ECVF_Scalability | ECVF_RenderThreadSafe);

static float GHairStrandsTransmissionDensityScaleFactor = 10;
static FAutoConsoleVariableRef CVarHairStrandsTransmissionDensityScaleFactor(TEXT("r.HairStrands.SkyLighting.TransmissionDensityScale"), GHairStrandsTransmissionDensityScaleFactor, TEXT("Density scale for controlling how much sky lighting is transmitted."), ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GHairStrandsSkyLightingUseHairCountTexture = 1;
static FAutoConsoleVariableRef CVarHairStrandsSkyLightingUseHairCountTexture(TEXT("r.HairStrands.SkyLighting.UseViewHairCount"), GHairStrandsSkyLightingUseHairCountTexture, TEXT("Use the view hair count texture for estimating background transmitted light (enabled by default)."), ECVF_Scalability | ECVF_RenderThreadSafe);

static float GHairStrandsSkyAODistanceThreshold = 10;
static float GHairStrandsSkyLightingDistanceThreshold = 10;
static FAutoConsoleVariableRef CVarHairStrandsSkyAOThreshold(TEXT("r.HairStrands.SkyAO.DistanceThreshold"), GHairStrandsSkyAODistanceThreshold, TEXT("Max distance for occlusion search."), ECVF_Scalability | ECVF_RenderThreadSafe);
static FAutoConsoleVariableRef CVarHairStrandsSkyLightingDistanceThreshold(TEXT("r.HairStrands.SkyLighting.DistanceThreshold"), GHairStrandsSkyLightingDistanceThreshold, TEXT("Max distance for occlusion search."), ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GHairStrandsSkyLighting_IntegrationType = 2;
static FAutoConsoleVariableRef CVarHairStrandsSkyLighting_IntegrationType(TEXT("r.HairStrands.SkyLighting.IntegrationType"), GHairStrandsSkyLighting_IntegrationType, TEXT("Hair env. lighting integration type (0:Adhoc, 1:Uniform."), ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GHairStrandsSkyLighting_ScreenTraceOcclusion = 0;
static FAutoConsoleVariableRef CVarHairStrandsSkyLighting_ScreenTraceOcclusion(TEXT("r.HairStrands.SkyLighting.ScreenTraceOcclusion"), GHairStrandsSkyLighting_ScreenTraceOcclusion, TEXT("Enable screen trace occlusion for during hair env. lighting integration"), ECVF_Scalability | ECVF_RenderThreadSafe);

///////////////////////////////////////////////////////////////////////////////////////////////////

DECLARE_GPU_STAT_NAMED(HairSkyLighting, TEXT("Hair Sky lighting"));

///////////////////////////////////////////////////////////////////////////////////////////////////

enum class EHairLightingSourceType : uint8
{
	SceneColor = 0,
	ReflectionProbe = 1,
	LumenScreenProbeGather = 2,
	LumenIrradianceFieldGather = 3,
	Count
};

enum class EHairLightingIntegrationType : uint8
{
	SceneColor = 0,
	AdHoc = 1,
	Uniform = 2,
	SH = 3,
	Count
};

bool GetHairStrandsSkyLightingEnable()				{ return GHairSkylightingEnable > 0; }
static bool GetHairStrandsSkyAOEnable()				{ return GHairSkyAOEnable > 0; }
static float GetHairStrandsSkyLightingConeAngle()	{ return FMath::Max(0.f, GHairSkylightingConeAngle); }
DECLARE_GPU_STAT_NAMED(HairStrandsReflectionEnvironment, TEXT("Hair Strands Reflection Environment"));

///////////////////////////////////////////////////////////////////////////////////////////////////
// AO
class FHairEnvironmentAO : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairEnvironmentAO);
	SHADER_USE_PARAMETER_STRUCT(FHairEnvironmentAO, FGlobalShader)

	class FSampleSet : SHADER_PERMUTATION_INT("PERMUTATION_SAMPLESET", 2);
	using FPermutationDomain = TShaderPermutationDomain<FSampleSet>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )

		SHADER_PARAMETER(uint32, Voxel_MacroGroupId)
		SHADER_PARAMETER(float, Voxel_TanConeAngle)
		SHADER_PARAMETER(float, AO_Power)
		SHADER_PARAMETER(float, AO_Intensity)
		SHADER_PARAMETER(uint32, AO_SampleCount)
		SHADER_PARAMETER(float, AO_DistanceThreshold)
		SHADER_PARAMETER(uint32, Output_bHalfRes)
		SHADER_PARAMETER(FVector2f, Output_InvResolution)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualVoxelParameters, VirtualVoxel)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FHairEnvironmentAO, "/Engine/Private/HairStrands/HairStrandsEnvironmentAO.usf", "MainPS", SF_Pixel);

static void AddHairStrandsEnvironmentAOPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsVoxelResources& VoxelResources,
	const FHairStrandsMacroGroupData& MacroGroupData,
	FRDGTextureRef Output)
{
	check(Output);
	FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

	const FIntRect Viewport = View.ViewRect;
	const FIntRect HalfResViewport = FIntRect::DivideAndRoundUp(Viewport, 2);
	const bool bHalfRes = Output->Desc.Extent.X == HalfResViewport.Width();

	FHairEnvironmentAO::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairEnvironmentAO::FParameters>();
	PassParameters->Voxel_MacroGroupId = MacroGroupData.MacroGroupId;
	PassParameters->Voxel_TanConeAngle = View.IsPerspectiveProjection() ? FMath::Tan(FMath::DegreesToRadians(GetHairStrandsSkyLightingConeAngle())) : 1.0f;
	PassParameters->SceneTextures = SceneTextures;
	PassParameters->VirtualVoxel = VoxelResources.UniformBuffer;

	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	const FFinalPostProcessSettings& Settings = View.FinalPostProcessSettings;
	PassParameters->AO_Power = Settings.AmbientOcclusionPower;
	PassParameters->AO_Intensity = Settings.AmbientOcclusionIntensity;
	PassParameters->AO_SampleCount = FMath::Max(uint32(GHairStrandsSkyAOSampleCount), 1u);
	PassParameters->AO_DistanceThreshold = FMath::Max(GHairStrandsSkyAODistanceThreshold, 1.f);
	PassParameters->RenderTargets[0] = FRenderTargetBinding(Output, ERenderTargetLoadAction::ELoad);
	PassParameters->Output_bHalfRes = bHalfRes;
	PassParameters->Output_InvResolution = FVector2f(1.f/Output->Desc.Extent.X, 1.f/Output->Desc.Extent.Y);

	FIntRect ViewRect;
	if (bHalfRes)
	{
		ViewRect.Min.X = 0;
		ViewRect.Min.Y = 0;
		ViewRect.Max = Output->Desc.Extent;
	}
	else
	{
		ViewRect = View.ViewRect;
	}

	if (ShaderPrint::IsValid(View.ShaderPrintData))
	{
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintParameters);
	}

	FHairEnvironmentAO::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairEnvironmentAO::FSampleSet>(PassParameters->AO_SampleCount <= 16 ? 0 : 1);

	TShaderMapRef<FHairEnvironmentAO> PixelShader(View.ShaderMap, PermutationVector);
	ClearUnusedGraphResources(PixelShader, PassParameters);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::AO %dx%d", ViewRect.Width(), ViewRect.Height()),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, &View, PixelShader, ViewRect](FRDGAsyncTask, FRHICommandList& InRHICmdList)
	{
		InRHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		FPixelShaderUtils::InitFullscreenPipelineState(InRHICmdList, View.ShaderMap, PixelShader, GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Min, BF_SourceColor, BF_DestColor, BO_Add, BF_Zero, BF_DestAlpha>::GetRHI();
		SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);
		SetShaderParameters(InRHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
		FPixelShaderUtils::DrawFullscreenTriangle(InRHICmdList);
	});
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairEnvironmentLightingVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairEnvironmentLightingVS);
	SHADER_USE_PARAMETER_STRUCT(FHairEnvironmentLightingVS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("LIGHTING_VS"), 1);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}
};

class FHairEnvironmentLightingPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairEnvironmentLightingPS);
	SHADER_USE_PARAMETER_STRUCT(FHairEnvironmentLightingPS, FGlobalShader)

	class FIntegrationType	: SHADER_PERMUTATION_INT("PERMUTATION_INTEGRATION_TYPE", uint32(EHairLightingIntegrationType::Count));
	class FLumen			: SHADER_PERMUTATION_INT("PERMUTATION_LUMEN", 3); 
	class FStaticLighting	: SHADER_PERMUTATION_BOOL("PERMUTATION_STATIC_LIGHTING"); 
	using FPermutationDomain = TShaderPermutationDomain<FIntegrationType, FLumen, FStaticLighting>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSkyDiffuseLightingParameters, SkyDiffuseLighting)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(LumenRadianceCache::FRadianceCacheInterpolationParameters, RadianceCache)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)

		SHADER_PARAMETER(uint32, bDynamicSkyLight)
		SHADER_PARAMETER(uint32, bHasStaticLighting)
		SHADER_PARAMETER(uint32, MultipleScatterSampleCount)
		SHADER_PARAMETER(float,  HairDualScatteringRoughnessOverride)
		SHADER_PARAMETER(float,  TransmissionDensityScaleFactor)
		SHADER_PARAMETER(float,  HairDistanceThreshold)
		SHADER_PARAMETER(uint32, bHairUseViewHairCount)

		SHADER_PARAMETER(uint32, bScreenTrace)
		SHADER_PARAMETER(float, ScreenTraceLength)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)

		SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
		SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairEnergyLUTTexture)

		SHADER_PARAMETER(float, ProbeOcclusionBias)	
		SHADER_PARAMETER(float, MaxRoughnessToEvaluateRoughSpecular)	
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ProbeValid)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FReflectionUniformParameters, ReflectionsParameters)
		SHADER_PARAMETER_STRUCT_REF(FReflectionCaptureShaderData, ReflectionCaptureData)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualVoxelParameters, VirtualVoxel)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutLightingBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("LIGHTING_PS"), 1);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// If using Lumen, no static lighting
		if (PermutationVector.Get<FStaticLighting>() && PermutationVector.Get<FLumen>()>0)
		{
			return false;
		}

		// If using another integrator than SH integrator, no static lighting
		if (PermutationVector.Get<FStaticLighting>() && PermutationVector.Get<FIntegrationType>() != int32(EHairLightingIntegrationType::SH))
		{
			return false;
		}

		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairEnvironmentLightingPS, "/Engine/Private/HairStrands/HairStrandsEnvironmentLighting.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FHairEnvironmentLightingVS, "/Engine/Private/HairStrands/HairStrandsEnvironmentLighting.usf", "MainVS", SF_Vertex);

static void AddHairStrandsEnvironmentLightingPassPS(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FHairStrandsVisibilityData& VisibilityData,
	const FHairStrandsVoxelResources& VirtualVoxelResources,
	const FRDGTextureRef SceneColorTexture,
	const EHairLightingSourceType LightingType)
{
	if (!Scene)
	{
		return;
	}

	EHairLightingIntegrationType IntegrationType = EHairLightingIntegrationType::AdHoc;
	if (LightingType == EHairLightingSourceType::SceneColor)
	{
		check(SceneColorTexture != nullptr);
		IntegrationType = EHairLightingIntegrationType::SceneColor;
	}
	else
	{
		switch (GHairStrandsSkyLighting_IntegrationType)
		{
			case 0: IntegrationType = EHairLightingIntegrationType::AdHoc; break;
			case 1: IntegrationType = EHairLightingIntegrationType::Uniform; break;
			case 2: IntegrationType = EHairLightingIntegrationType::SH; break;
		}
	}

	// The specular sky light contribution is also needed by RT Reflections as a fallback.
	const bool bSkyLight = Scene->SkyLight && !Scene->SkyLight->bHasStaticLighting;

	const bool bDynamicSkyLight = ShouldRenderDeferredDynamicSkyLight(Scene, *View.Family);
	const bool bReflectionEnv = ShouldDoReflectionEnvironment(Scene, *View.Family);

	// Only support static lighting with SH integrator at the moment
	const bool bUseVolumetricLightmap = Scene && Scene->VolumetricLightmapSceneData.HasData();
	const bool bLumenActive = ShouldRenderLumenDiffuseGI(Scene, View);
	const bool bHasStaticLighting = IsStaticLightingAllowed() && bUseVolumetricLightmap && !bLumenActive && IntegrationType == EHairLightingIntegrationType::SH;

	// Sanity check
	if (bHasStaticLighting) { check(LightingType != EHairLightingSourceType::LumenScreenProbeGather && LightingType != EHairLightingSourceType::LumenIrradianceFieldGather); }

	// Early out if there is no static lighting, no sky lighting, nor reflection probes, nor Lumen
	if (!bSkyLight && !bDynamicSkyLight && !bHasStaticLighting && !bReflectionEnv && !bLumenActive && IntegrationType != EHairLightingIntegrationType::SceneColor)
		return;

	FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

	check(VirtualVoxelResources.IsValid());

	// Render the reflection environment with tiled deferred culling
	const bool bHasBoxCaptures = (View.NumBoxReflectionCaptures > 0);
	const bool bHasSphereCaptures = (View.NumSphereReflectionCaptures > 0);
	FHairEnvironmentLightingPS::FParameters* ParametersPS = GraphBuilder.AllocParameters<FHairEnvironmentLightingPS::FParameters>();

	ParametersPS->SceneColorTexture = SceneColorTexture;
	ParametersPS->HairEnergyLUTTexture = GetHairLUT(GraphBuilder, View, HairLUTType_MeanEnergy);
	ParametersPS->bDynamicSkyLight = bDynamicSkyLight ? 1 : 0;
	ParametersPS->bHasStaticLighting = bHasStaticLighting ? 1 : 0;
	ParametersPS->SkyDiffuseLighting = GetSkyDiffuseLightingParameters(Scene->SkyLight, 1.0f /*DynamicBentNormalAO*/);
	ParametersPS->HairDistanceThreshold = FMath::Max(GHairStrandsSkyLightingDistanceThreshold, 1.f);
	ParametersPS->bHairUseViewHairCount = VisibilityData.ViewHairCountTexture && GHairStrandsSkyLightingUseHairCountTexture ? 1 : 0;
	ParametersPS->MultipleScatterSampleCount = FMath::Max(uint32(GHairStrandsSkyLightingSampleCount), 1u);
	ParametersPS->HairDualScatteringRoughnessOverride = GetHairDualScatteringRoughnessOverride();
	ParametersPS->TransmissionDensityScaleFactor = FMath::Max(0.f, GHairStrandsTransmissionDensityScaleFactor);
	ParametersPS->PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
	ParametersPS->PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	ParametersPS->SceneTextures = SceneTextures;
	ParametersPS->VirtualVoxel = VirtualVoxelResources.UniformBuffer;
	ParametersPS->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
	ParametersPS->ViewUniformBuffer = View.ViewUniformBuffer;
	ParametersPS->ReflectionCaptureData = View.ReflectionCaptureUniformBuffer;
	ParametersPS->ReflectionsParameters = CreateReflectionUniformBuffer(GraphBuilder, View);
	
	FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
	ParametersPS->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

	const bool bScreenTrace = IsHZBValid(View, EHZBType::FurthestHZB) && GHairStrandsSkyLighting_ScreenTraceOcclusion && IntegrationType != EHairLightingIntegrationType::SceneColor;
	ParametersPS->bScreenTrace = bScreenTrace ? 1u : 0u;
	if (bScreenTrace)
	{
		// Default values, need more more testing/tweaking
		ParametersPS->ScreenTraceLength = 128;
		ParametersPS->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
	}
	else
	{
		ParametersPS->HZBParameters = GetDummyHZBParameters(GraphBuilder);
	}

	ParametersPS->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	ParametersPS->OutLightingBuffer = nullptr;

	if (ShaderPrint::IsValid(View.ShaderPrintData))
	{
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, ParametersPS->ShaderPrintParameters);
	}

	const TCHAR* LightingTypeName = TEXT("Unknown");
	uint32 LumenPermutation = 0;
	switch (LightingType)
	{
		case EHairLightingSourceType::SceneColor: 
		{
			LightingTypeName = TEXT("SceneScatter") ; 
		}
		break;
		case EHairLightingSourceType::LumenScreenProbeGather : 
		{
			LightingTypeName = TEXT("LumenScreenProbe"); 
			LumenPermutation = 1; 

			const FRadianceCacheState& RadianceCacheState = View.ViewState->Lumen.RadianceCacheState;
			const LumenRadianceCache::FRadianceCacheInputs RadianceCacheInputs = LumenScreenProbeGatherRadianceCache::SetupRadianceCacheInputs(View);
			LumenRadianceCache::GetInterpolationParameters(View, GraphBuilder, RadianceCacheState, RadianceCacheInputs, ParametersPS->RadianceCache);
		}
		break;
		case EHairLightingSourceType::LumenIrradianceFieldGather : 
		{
			LightingTypeName = TEXT("LumenIrradianceField"); 
			LumenPermutation = 2; 

			const FRadianceCacheState& RadianceCacheState = View.ViewState->Lumen.RadianceCacheState;
			LumenRadianceCache::FRadianceCacheInputs RadianceCacheInputs = LumenIrradianceFieldGather::SetupRadianceCacheInputs();
			RadianceCacheInputs.UseAdaptiveProbes = 0;
			LumenRadianceCache::GetInterpolationParameters(View, GraphBuilder, RadianceCacheState, RadianceCacheInputs, ParametersPS->RadianceCache);
			ParametersPS->ProbeValid = LumenRadianceCache::GetProbeValidSRV(GraphBuilder, GraphBuilder.RegisterExternalBuffer(RadianceCacheState.ProbeValid));
			ParametersPS->ProbeOcclusionBias = LumenIrradianceFieldGather::GetProbeOcclusionBias();
			ParametersPS->MaxRoughnessToEvaluateRoughSpecular = LumenIrradianceFieldGather::GetMaxRoughnessToEvaluateRoughSpecular();
		}
		break;
		case EHairLightingSourceType::ReflectionProbe : 
		{
			LightingTypeName = TEXT("ReflectionProbe"); 
		}
		break;
	};

	FHairEnvironmentLightingPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairEnvironmentLightingPS::FIntegrationType>(uint32(IntegrationType));
	PermutationVector.Set<FHairEnvironmentLightingPS::FLumen>(LumenPermutation);
	PermutationVector.Set<FHairEnvironmentLightingPS::FStaticLighting>(bHasStaticLighting);
	PermutationVector = FHairEnvironmentLightingPS::RemapPermutation(PermutationVector);

	FIntPoint ViewportResolution = VisibilityData.SampleLightingViewportResolution;
	const FViewInfo* CapturedView = &View;
	TShaderMapRef<FHairEnvironmentLightingVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FHairEnvironmentLightingPS> PixelShader(View.ShaderMap, PermutationVector);

	check(VisibilityData.SampleLightingTexture);
	ParametersPS->RenderTargets[0] = FRenderTargetBinding(VisibilityData.SampleLightingTexture, ERenderTargetLoadAction::ELoad);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::EnvLightingPS(%s)", LightingTypeName),
		ParametersPS,
		ERDGPassFlags::Raster,
		[ParametersPS, VertexShader, PixelShader, ViewportResolution, CapturedView](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FHairEnvironmentLightingVS::FParameters ParametersVS;
		ParametersVS.HairStrands = ParametersPS->HairStrands;

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Max, BF_One, BF_One>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *ParametersPS);

		RHICmdList.SetViewport(0, 0, 0.0f, ViewportResolution.X, ViewportResolution.Y, 1.0f);
		RHICmdList.SetStreamSource(0, nullptr, 0);
		RHICmdList.DrawPrimitive(0, 1, 1);
	});
}

///////////////////////////////////////////////////////////////////////////////////////////////////

void RenderHairStrandsSceneColorScattering(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef SceneColorTexture,
	const FScene* Scene,
	TArrayView<const FViewInfo> Views)
{
	if (Views.Num() == 0 || GHairScatterSceneLighting <= 0)
		return;

	for (const FViewInfo& View : Views)
	{
		if (!HairStrands::HasViewHairStrandsData(View))
		{
			continue;
		}

		const FHairStrandsVisibilityData& VisibilityData = View.HairStrandsViewData.VisibilityData;
		const FHairStrandsVoxelResources& VoxelResources = View.HairStrandsViewData.VirtualVoxelResources;
		check(VoxelResources.IsValid());

		if (HasHairFlags(View.HairStrandsViewData.Flags, HAIR_FLAGS_SCATTER_SCENE_LIGHT))
		{
			AddHairStrandsEnvironmentLightingPassPS(GraphBuilder, Scene, View, VisibilityData, VoxelResources, SceneColorTexture, EHairLightingSourceType::SceneColor);
		}
	}
}

static void InternalRenderHairStrandsEnvironmentLighting(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	EHairLightingSourceType LightingType)
{
	if (!GetHairStrandsSkyLightingEnable() || !HairStrands::HasViewHairStrandsData(View))
		return;

	const FHairStrandsVisibilityData& VisibilityData = View.HairStrandsViewData.VisibilityData;
	const FHairStrandsVoxelResources& VoxelResources = View.HairStrandsViewData.VirtualVoxelResources;
	const bool bRenderHairLighting = VoxelResources.IsValid() && VisibilityData.CoverageTexture;
	if (!bRenderHairLighting)
	{
		return;
	}
	
	AddHairStrandsEnvironmentLightingPassPS(GraphBuilder, Scene, View, VisibilityData, VoxelResources, nullptr, LightingType);
}

void RenderHairStrandsLumenLighting(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View)
{
	const ELumenFinalGatherMethod FinalGatherMethod = Lumen::GetFinalGatherMethod(*View.Family, View.GetShaderPlatform());
	const EHairLightingSourceType SourceType = FinalGatherMethod == ELumenFinalGatherMethod::IrradianceFieldGather ? EHairLightingSourceType::LumenIrradianceFieldGather : EHairLightingSourceType::LumenScreenProbeGather;
	InternalRenderHairStrandsEnvironmentLighting(GraphBuilder, Scene, View, SourceType);
}

void RenderHairStrandsEnvironmentLighting(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairSkyLighting, "HairSkyLighting");

	InternalRenderHairStrandsEnvironmentLighting(GraphBuilder, Scene, View, EHairLightingSourceType::ReflectionProbe);
}

void RenderHairStrandsAmbientOcclusion(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef& InAOTexture)
{
	if (!GetHairStrandsSkyAOEnable() || !HairStrands::HasViewHairStrandsData(View) || !InAOTexture)
		return;

	const FHairStrandsVisibilityData& VisibilityData = View.HairStrandsViewData.VisibilityData;
	const FHairStrandsVoxelResources& VoxelResources = View.HairStrandsViewData.VirtualVoxelResources;
	const bool bRenderHairLighting = VoxelResources.IsValid() && VisibilityData.CoverageTexture;
	check(bRenderHairLighting);

	for (const FHairStrandsMacroGroupData& MacroGroupData : View.HairStrandsViewData.MacroGroupDatas)
	{
		AddHairStrandsEnvironmentAOPass(GraphBuilder, View, VoxelResources, MacroGroupData, InAOTexture);
	}
}
