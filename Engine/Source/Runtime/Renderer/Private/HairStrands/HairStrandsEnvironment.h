// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsEnvironment.h: Hair strands environment lighting.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"

class FScene;
class FViewInfo;

void RenderHairStrandsAmbientOcclusion(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef& InAOTexture);

void RenderHairStrandsLumenLighting(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View);

void RenderHairStrandsEnvironmentLighting(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View);

void RenderHairStrandsSceneColorScattering(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef SceneColorTexture,
	const FScene* Scene,
	TArrayView<const FViewInfo> Views);
