// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsForwardRaster.h"

#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "SceneTextureParameters.h"
#include "RenderGraphUtils.h"
#include "HairStrandsInterface.h"
#include "SceneRendering.h"
#include "DataDrivenShaderPlatformInfo.h"

extern int32 GHairVisibilityComputeRaster_Culling;
extern int32 GHairVisibilityComputeRaster_MaxTiles;
extern int32 GHairVisibilityComputeRaster_TileSize;
extern int32 GHairVisibility_NumClassifiers;
extern int32 GHairVisibilityComputeRaster_NumBinners;
extern int32 GHairVisibilityComputeRaster_NumRasterizers;
extern int32 GHairVisibilityComputeRaster_NumRasterizersNaive;
extern int32 GHairVisibilityComputeRaster_Debug;

void SetUpViewHairRenderInfo(const FViewInfo& ViewInfo, FVector4f& OutHairRenderInfo, uint32& OutHairRenderInfoBits, uint32& OutHairComponents, float FinalResolutionScale);
void SetUpViewHairRenderInfo(const FViewInfo& ViewInfo, bool bEnableMSAA, FVector4f& OutHairRenderInfo, uint32& OutHairRenderInfoBits, uint32& OutHairComponents, float FinalResolutionScale);
FMinHairRadiusAtDepth1 ComputeMinStrandRadiusAtDepth1(const FIntPoint& Resolution, const float FOV, const uint32 SampleCount, const float OverrideStrandHairRasterizationScale, const float OrthoWidth, const float FinalResolutionScale);

inline bool IsHairStrandsForwardRasterSupported(EShaderPlatform In) 
{ 
	return IsFeatureLevelSupported(In, ERHIFeatureLevel::SM6) && IsHairStrandsSupported(EHairStrandsShaderType::Strands, In) && 
		!IsVulkanPlatform(In) && !IsMetalPlatform(In); // :todo-jn: fix SPIR-V error during compilation (Mac and Vulkan)
}

/////////////////////////////////////////////////////////////////////////////////////////
// Culling pass

class FHairCullSegmentCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairCullSegmentCS);
	SHADER_USE_PARAMETER_STRUCT(FHairCullSegmentCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, OutputResolution)
		SHADER_PARAMETER(uint32, HairMaterialId)
		SHADER_PARAMETER(uint32, ControlPointCount)
		SHADER_PARAMETER(uint32, CurveCount)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceParameters, HairInstance)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPackedHairVis>, RWHairVis)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint2>, RWCoord)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCounter)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, RWPoints)

		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsForwardRasterSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_CULL"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairCullSegmentCS, "/Engine/Private/HairStrands/HairStrandsForwardCulling.usf", "CSMain", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Raster compute common parameters

BEGIN_SHADER_PARAMETER_STRUCT(FRasterComputeForwardCommonParameters, )
	SHADER_PARAMETER(FIntPoint, OutputResolution)
	SHADER_PARAMETER(FVector2f, OutputResolutionf)

	SHADER_PARAMETER(FIntPoint, BinTileRes)
	SHADER_PARAMETER(FIntPoint, RasterTileRes)

	SHADER_PARAMETER(uint32,    NumRasterizers)
	SHADER_PARAMETER(float,     RcpNumRasterizers)

	SHADER_PARAMETER(uint32,    NumBinners)
	SHADER_PARAMETER(float,     RcpNumBinners)

	SHADER_PARAMETER(float,     RadiusAtDepth1)
	SHADER_PARAMETER(uint32,    MaxControlPointCount)

	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, ControlPoints)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ControlPointCount)
END_SHADER_PARAMETER_STRUCT()

///////////////////////////////////////////////////////////////////////////////////////////////////
// Compute depth tile data based on scene data

class FHairStrandsForwardRasterPrepareDepthGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsForwardRasterPrepareDepthGridCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsForwardRasterPrepareDepthGridCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeForwardCommonParameters, Common)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutVisTileDepthGrid)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsForwardRasterSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_DEPTH_GRID"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsForwardRasterPrepareDepthGridCS, "/Engine/Private/HairStrands/HairStrandsForwardRaster.usf", "PrepareDepthGridCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////
// Bin strands segments

class FHairStrandsForwardRasterBinningCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsForwardRasterBinningCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsForwardRasterBinningCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeForwardCommonParameters, Common)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, VisTileBinningGrid)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutVisTilePrims)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutVisTilePrimDepths)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, OutVisTileBinningGrid)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, OutVisTileBinningGridMinZ)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, OutVisTileBinningGridMaxZ)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VisTileDepthGrid)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutVisTileArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutVisTileData)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RDG_TEXTURE_ACCESS(VisTileBinningGridTex, ERHIAccess::UAVCompute)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsForwardRasterSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_BINNING"), 1);
	}
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDebug>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}
		return FGlobalShader::ShouldPrecachePermutation(Parameters);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsForwardRasterBinningCS, "/Engine/Private/HairStrands/HairStrandsForwardRaster.usf", "BinningCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////
// Compacted binned segments into contiguous list 

class FHairStrandsForwardRasterCompactionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsForwardRasterCompactionCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsForwardRasterCompactionCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeForwardCommonParameters, Common)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, InData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InPrims)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InDepths)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutPrims)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutWork)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutDataCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutWorkCount)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsForwardRasterSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters &Parameters, FShaderCompilerEnvironment &OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_COMPACTION"), 1);
	}
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDebug>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}
		return FGlobalShader::ShouldPrecachePermutation(Parameters);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsForwardRasterCompactionCS, "/Engine/Private/HairStrands/HairStrandsForwardRaster.usf", "CompactionCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////
// Rasterized binned & shaded segements

class FHairStrandsForwardRasterRasterizeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsForwardRasterRasterizeCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsForwardRasterRasterizeCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeForwardCommonParameters, Common)
		SHADER_PARAMETER(FIntPoint, SampleLightingViewportResolution)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SampleLightingTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, SampleVelocityBuffer)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTilePrims)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTileArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, VisTileWork)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, VisTileWorkCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, VisTileData)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutHairCountTexture_ForDebug)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutHairPixelCountPerTile_ForDebug)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutSceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutSceneVelocityTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, RWWorkCounter)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsForwardRasterSupported(Parameters.Platform); }
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDebug>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}

		return EShaderPermutationPrecacheRequest::Precached;
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_RASTER"), 1);

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsForwardRasterRasterizeCS, "/Engine/Private/HairStrands/HairStrandsForwardRaster.usf", "RasterCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////
// Debug pass

class FHairStrandsForwardRasterDebugCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsForwardRasterDebugCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsForwardRasterDebugCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeForwardCommonParameters, Common)
		SHADER_PARAMETER(uint32, InstanceCount)
		SHADER_PARAMETER(uint32, CPUAllocatedTileCount)
		SHADER_PARAMETER(uint32, CPUAllocatedCompactedTileCount)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairCountTexture_ForDebug)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairPixelCountPerTile_ForDebug)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTileArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, VisTileData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, CompactedVisTileArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CompactedVisTileData)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsForwardRasterSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_DEBUG"), 1);
	}
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsForwardRasterDebugCS, "/Engine/Private/HairStrands/HairStrandsForwardRaster.usf", "MainCS", SF_Compute);


/////////////////////////////////////////////////////////////////////////////////////////
// Culling pass

FRasterForwardCullingOutput AddHairStrandsForwardCullingPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& ViewInfo,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FIntPoint& InResolution,
	const FRDGTextureRef SceneDepthTexture,
	bool bSupportCulling,
	bool bForceRegister)
{	
	if (!IsHairStrandsForwardRasterSupported(ViewInfo.GetShaderPlatform()))
	{
		return FRasterForwardCullingOutput();
	}

	// Compute maximum number of PrimIDs
	uint32 MaxNumPrimIDs = 0;
	for (const FHairStrandsMacroGroupData &MacroGroup : MacroGroupDatas)
	{
		for (const FHairStrandsMacroGroupData::PrimitiveInfo &PrimitiveInfo : MacroGroup.PrimitivesInfos)
		{
			// If a groom is not visible in primary view, but visible in shadow view, its PrimitiveInfo.Mesh will be null.
			if (PrimitiveInfo.Mesh == nullptr || PrimitiveInfo.Mesh->Elements.Num() == 0)
			{
				continue;
			}

			const FHairGroupPublicData* HairGroupPublicData = reinterpret_cast<const FHairGroupPublicData*>(PrimitiveInfo.Mesh->Elements[0].VertexFactoryUserData);
			check(HairGroupPublicData);

			const uint32 PointCount = HairGroupPublicData->GetActiveStrandsPointCount();
			// Sanity check
			check(PointCount == HairGroupPublicData->VFInput.Strands.Common.PointCount);

			MaxNumPrimIDs += PointCount;
		}
	}

	if (MaxNumPrimIDs == 0)
	{
		return FRasterForwardCullingOutput();
	}
	const uint32 NodeVisInBytes = 8; // See HairStrandsVisibilityInternal::NodeVis in HairStrandsVisibility.cpp
	const EPixelFormat ControlPointFormat = PF_A32B32G32R32F;

	FRasterForwardCullingOutput Out;
	Out.Resolution		= InResolution;
	Out.NodeIndex		= GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(Out.Resolution, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource), TEXT("Hair.VisibilityCompactNodeIndex"));
	Out.NodeVis			= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(NodeVisInBytes, MaxNumPrimIDs), TEXT("Hair.VisibilityNodeVis"));
	Out.NodeCoord		= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MaxNumPrimIDs), TEXT("Hair.VisibilityNodeCoord"));
	Out.Points			= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32)*4u, MaxNumPrimIDs), TEXT("Hair.ControlPoints"));
	Out.PointsSRV		= GraphBuilder.CreateSRV(Out.Points, ControlPointFormat);
	Out.PointCount		= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("Hair.ControlPointCount"));
	Out.RasterizedInstanceCount = 0;

	FRDGBufferUAVRef PointsUAV = GraphBuilder.CreateUAV(Out.Points, ControlPointFormat);

	const uint32 ClearValues[4] = { 0u,0u,0u,0u };
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Out.PointCount), 0u);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Out.NodeIndex), ClearValues);

	// Create and set the uniform buffer
	const bool bEnableMSAA = false;
	TUniformBufferRef<FViewUniformShaderParameters> ViewUniformShaderParameters;
	SetUpViewHairRenderInfo(ViewInfo, bEnableMSAA, ViewInfo.CachedViewUniformShaderParameters->HairRenderInfo, ViewInfo.CachedViewUniformShaderParameters->HairRenderInfoBits, ViewInfo.CachedViewUniformShaderParameters->HairComponents, ViewInfo.Family->FinalResolutionScale);
	ViewUniformShaderParameters = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(*ViewInfo.CachedViewUniformShaderParameters, UniformBuffer_SingleFrame);

	TShaderMapRef<FHairCullSegmentCS> CullShader(ViewInfo.ShaderMap);
	for (const FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
	{
		const FHairStrandsMacroGroupData::TPrimitiveInfos& PrimitiveSceneInfos = MacroGroup.PrimitivesInfos;

		for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : PrimitiveSceneInfos)
		{
			// If a groom is not visible in primary view, but visible in shadow view, its PrimitiveInfo.Mesh will be null.
			if (PrimitiveInfo.Mesh == nullptr || PrimitiveInfo.Mesh->Elements.Num() == 0)
			{
				continue;
			}

			const FHairGroupPublicData* HairGroupPublicData = reinterpret_cast<const FHairGroupPublicData*>(PrimitiveInfo.Mesh->Elements[0].VertexFactoryUserData);
			check(HairGroupPublicData);

			const bool bCullingEnable = false; //bSupportCulling && GHairVisibilityComputeRaster_Culling ? HairGroupPublicData->GetCullingResultAvailable() : false;
			
			// calculate current view screen size - which can result in fewer strands rasterized in current view
			const uint32 CurveCount  = HairGroupPublicData->GetActiveStrandsCurveCount();
			const uint32 PointCount = HairGroupPublicData->GetActiveStrandsPointCount();

			// Curve version
			#if 1
			FHairCullSegmentCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairCullSegmentCS::FParameters>();
			Parameters->OutputResolution		= Out.Resolution;
			Parameters->HairMaterialId			= PrimitiveInfo.MaterialId;
			Parameters->ControlPointCount		= PointCount;
			Parameters->CurveCount				= CurveCount;
			Parameters->RWHairVis				= GraphBuilder.CreateUAV(Out.NodeVis);
			Parameters->RWCoord					= GraphBuilder.CreateUAV(Out.NodeCoord, PF_R16G16_UINT);
			Parameters->RWCounter				= GraphBuilder.CreateUAV(Out.PointCount);
			Parameters->RWPoints				= PointsUAV;
			Parameters->HairInstance			= GetHairStrandsInstanceParameters(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnable, bForceRegister);
			Parameters->ViewUniformBuffer		= ViewUniformShaderParameters;
			Parameters->SceneDepthTexture		= SceneDepthTexture;

			FIntVector DispatchCount = FComputeShaderUtils::GetGroupCount(CurveCount, 1024);
			FComputeShaderUtils::AddPass(
				GraphBuilder, 
				RDG_EVENT_NAME("HairStrands::CullSegment(Curves)"), 
				CullShader,
				Parameters, 
				DispatchCount);
			#endif

			// Vertex version
			#if 0
			FHairCullSegmentCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairCullSegmentCS::FParameters>();
			Parameters->OutputResolution		= Out.Resolution;
			Parameters->HairMaterialId			= PrimitiveInfo.MaterialId;
			Parameters->ControlPointCount		= VertexCount;
			Parameters->CurveCount				= CurveCount;
			Parameters->RWHairVis				= GraphBuilder.CreateUAV(Out.NodeVis);
			Parameters->RWCoord					= GraphBuilder.CreateUAV(Out.NodeCoord, PF_R16G16_UINT);
			Parameters->RWCounter				= GraphBuilder.CreateUAV(Out.PointCount);
			Parameters->RWPoints				= GraphBuilder.CreateUAV(Out.Points);
			Parameters->HairInstance			= GetHairStrandsInstanceParameters(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnable, bForceRegister);
			Parameters->ViewUniformBuffer		= ViewUniformShaderParameters;
			Parameters->SceneDepthTexture		= SceneDepthTexture;

			FIntVector DispatchCount = FComputeShaderUtils::GetGroupCount(VertexCount, 1024);
			FComputeShaderUtils::AddPass(
				GraphBuilder, 
				RDG_EVENT_NAME("HairStrands::CullSegment(Points)"), 
				CullShader,
				Parameters, 
				DispatchCount);
			#endif

			++Out.RasterizedInstanceCount;
		}
	}

	return Out;
}

/////////////////////////////////////////////////////////////////////////////////////////
// Raster pass 

void AddHairStrandsForwardRasterPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& ViewInfo,
	const FIntPoint& InResolution,
	const FHairStrandsVisibilityData& InData,
	const FRDGTextureRef SceneDepthTexture,
	const FRDGTextureRef SceneColorTexture,
	const FRDGTextureRef SceneVelocityTexture)
{	
	if (!IsHairStrandsForwardRasterSupported(ViewInfo.GetShaderPlatform()))
	{
		return;
	}

	const FIntPoint OutResolution = InResolution;

	// See the comment on the GHairVisibilityComputeRaster_MaxTiles declaration for an explanation for the large upper bound.
	const uint32 NumBinners = FMath::Min(FMath::Max(GHairVisibilityComputeRaster_NumBinners, 1), 256);
	const uint32 NumRasterizers = FMath::Min(FMath::Max(GHairVisibilityComputeRaster_NumRasterizers, 1), 1024);

	const uint32 BinTileSize = 32;
	const uint32 RasterTileSize = 8;

	const FIntPoint BinTileRes = FIntPoint(FMath::DivideAndRoundUp(uint32(InResolution.X), BinTileSize), FMath::DivideAndRoundUp(uint32(InResolution.Y), BinTileSize));
	const FIntPoint RasterTileRes = FIntPoint(FMath::DivideAndRoundUp(uint32(InResolution.X), RasterTileSize), FMath::DivideAndRoundUp(uint32(InResolution.Y), RasterTileSize));

	struct FBinningData
	{
		// Binned segments
		FRDGBufferRef VisTilePrims;
		FRDGBufferSRVRef VisTilePrimsSRV;

		FRDGBufferRef VisTilePrimDepths;
		FRDGBufferSRVRef VisTilePrimDepthsSRV;

		FRDGBufferRef VisTileData;
		FRDGBufferSRVRef VisTileDataSRV;

		FRDGBufferRef VisTileArgs;
		FRDGBufferUAVRef VisTileArgsUAV;
		FRDGBufferSRVRef VisTileArgsSRV;

		FRDGTextureRef VisTileBinningGrid;
		FRDGTextureUAVRef VisTileBinningGridUAV;

		// BinZ Min/Max
		FRDGTextureRef VisTileBinningGridMinZ;
		FRDGTextureRef VisTileBinningGridMaxZ;
		FRDGTextureUAVRef VisTileBinningGridMinZUAV;
		FRDGTextureUAVRef VisTileBinningGridMaxZUAV;

		// Binned & compacted segments
		FRDGBufferRef CompactedVisTilePrims;
		FRDGBufferRef CompactedVisTileData;
		FRDGBufferRef CompactedVisTileArgs;
		FRDGBufferRef CompactedVisTileWork;
		FRDGBufferRef CompactedVisTileDataCount;
		FRDGBufferRef CompactedVisTileWorkCount;
	};

	const uint32 EntryCount = 5; // See VT_XXX in .usf: PrimOffset / PrimCount / Coord / MaxIndex / MinMaxDepth

	FBinningData BinData;
	{
		const uint32 MaxPrimPerTile = 1024u;
		const uint32 BinMaxTiles = FMath::Clamp(GHairVisibilityComputeRaster_MaxTiles, MaxPrimPerTile, 262144); //?
		const uint32 MaxTotalPrims = BinMaxTiles * MaxPrimPerTile; // Number of 'binned' primitives. This number can/needs to be higher than the actual number of primitives, since a primitive can be binned into several bins

		BinData.VisTilePrims = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MaxTotalPrims), TEXT("Hair.VisTilePrims"));
		BinData.VisTilePrimsSRV = GraphBuilder.CreateSRV(BinData.VisTilePrims, PF_R32_UINT);

		BinData.VisTilePrimDepths = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MaxTotalPrims), TEXT("Hair.VisTilePrimDepths"));
		BinData.VisTilePrimDepthsSRV = GraphBuilder.CreateSRV(BinData.VisTilePrimDepths, PF_R32_UINT);

		FRDGBufferDesc DescTileData = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), BinMaxTiles * EntryCount);
		DescTileData.Usage = EBufferUsageFlags(DescTileData.Usage | BUF_ByteAddressBuffer);
		BinData.VisTileData = GraphBuilder.CreateBuffer(DescTileData, TEXT("Hair.VisTileData"));
		BinData.VisTileDataSRV = GraphBuilder.CreateSRV(BinData.VisTileData);

		FRDGBufferDesc VisTileArgsDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1);
		BinData.VisTileArgs = GraphBuilder.CreateBuffer(VisTileArgsDesc, TEXT("Hair.VisTileArgs"));
		BinData.VisTileArgsUAV = GraphBuilder.CreateUAV(BinData.VisTileArgs, PF_R32_UINT);
		BinData.VisTileArgsSRV = GraphBuilder.CreateSRV(BinData.VisTileArgs, PF_R32_UINT);

		BinData.VisTileBinningGrid = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2DArray(BinTileRes, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource, NumBinners * 3), TEXT("Hair.VisTileBinningGrid"));
		BinData.VisTileBinningGridUAV = GraphBuilder.CreateUAV(BinData.VisTileBinningGrid);

		// BinZ Min/Max
		BinData.VisTileBinningGridMinZ = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2DArray(BinTileRes, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource, NumBinners * 3), TEXT("Hair.VisTileBinningGridMinZ"));
		BinData.VisTileBinningGridMaxZ = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2DArray(BinTileRes, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource, NumBinners * 3), TEXT("Hair.VisTileBinningGridMaxZ"));
		BinData.VisTileBinningGridMinZUAV = GraphBuilder.CreateUAV(BinData.VisTileBinningGridMinZ);
		BinData.VisTileBinningGridMaxZUAV = GraphBuilder.CreateUAV(BinData.VisTileBinningGridMaxZ);

		BinData.CompactedVisTilePrims = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MaxTotalPrims), TEXT("Hair.CompactedVisTilePrims"));
		BinData.CompactedVisTileData = GraphBuilder.CreateBuffer(DescTileData, TEXT("Hair.CompactedVisTileData"));
		BinData.CompactedVisTileArgs = GraphBuilder.CreateBuffer(VisTileArgsDesc, TEXT("Hair.CompactedVisTileArgs"));
		BinData.CompactedVisTileWork		= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxTotalPrims), TEXT("Hair.CompactedVisTileWork"));
		BinData.CompactedVisTileDataCount	= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("Hair.CompactedVisTileDataCount"));
		BinData.CompactedVisTileWorkCount	= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("Hair.CompactedVisTileWorkCount"));
	}

	// Create and set the uniform buffer
	TUniformBufferRef<FViewUniformShaderParameters> ViewUniformShaderParameters;
	{
		const bool bEnableMSAA = false;
		SetUpViewHairRenderInfo(ViewInfo, bEnableMSAA, ViewInfo.CachedViewUniformShaderParameters->HairRenderInfo, ViewInfo.CachedViewUniformShaderParameters->HairRenderInfoBits, ViewInfo.CachedViewUniformShaderParameters->HairComponents, ViewInfo.Family->FinalResolutionScale);
		ViewUniformShaderParameters = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(*ViewInfo.CachedViewUniformShaderParameters, UniformBuffer_SingleFrame);
	}

	// Common parameters
	FRasterComputeForwardCommonParameters Common;
	{		
		Common.OutputResolution = OutResolution;
		Common.OutputResolutionf = FVector2f(OutResolution.X, OutResolution.Y);

		Common.RasterTileRes = RasterTileRes;
		Common.NumRasterizers = NumRasterizers;
		Common.RcpNumRasterizers = 1.0 / NumRasterizers;

		Common.BinTileRes = BinTileRes;
		Common.NumBinners = NumBinners;
		Common.RcpNumBinners = 1.0 / NumBinners;

		Common.MaxControlPointCount = InData.MaxControlPointCount;
		Common.ControlPoints = InData.ControlPointsSRV;
		Common.ControlPointCount = InData.ControlPointCount;

		Common.RadiusAtDepth1 = ComputeMinStrandRadiusAtDepth1(FIntPoint(ViewInfo.UnconstrainedViewRect.Width(), ViewInfo.UnconstrainedViewRect.Height()), ViewInfo.FOV, 1 /*SampleCount*/, 0 /*ScaleOverride*/, ViewInfo.ViewMatrices.GetOrthoDimensions().X, ViewInfo.Family->FinalResolutionScale).Primary;
	}

	// Fill in tile depth
	FRDGTextureRef VisTileDepthGrid = nullptr;
	{
		VisTileDepthGrid = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(BinTileRes, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource), TEXT("Hair.VisTileDepthGrid"));

		TShaderMapRef<FHairStrandsForwardRasterPrepareDepthGridCS> ComputeShaderPrepareDepthGrid(ViewInfo.ShaderMap);
		FHairStrandsForwardRasterPrepareDepthGridCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsForwardRasterPrepareDepthGridCS::FParameters>();		
		Parameters->Common = Common;
		Parameters->SceneDepthTexture = SceneDepthTexture;
		Parameters->OutVisTileDepthGrid = GraphBuilder.CreateUAV(VisTileDepthGrid);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterPrepDepthGrid"), ComputeShaderPrepareDepthGrid, Parameters, FIntVector(BinTileRes.X, BinTileRes.Y, 1));
	}

	const bool bDebugEnabled = GHairVisibilityComputeRaster_Debug > 0;
	uint32 TotalPrimitiveInfoCount = 1; // TODO = total number of groups

	if (bDebugEnabled)
	{
		ShaderPrint::SetEnabled(true);
		ShaderPrint::RequestSpaceForTriangles(2 * NumBinners * Common.RasterTileRes.X * Common.RasterTileRes.Y + 2 * NumBinners * 10);
		ShaderPrint::RequestSpaceForLines(4 * 2 * NumBinners * 128);
	}

	// Reset buffers
	{
		// BinZ Min/Max
		uint32 IndexGridClearMinZ[4] = { 0xFFFFFFFF,0xFFFFFFFF,0xFFFFFFFF,0xFFFFFFFF };
		uint32 IndexGridClearMaxZ[4] = { 0x0,0x0,0x0,0x0 };
		AddClearUAVPass(GraphBuilder, BinData.VisTileBinningGridMinZUAV, IndexGridClearMinZ);
		AddClearUAVPass(GraphBuilder, BinData.VisTileBinningGridMaxZUAV, IndexGridClearMaxZ);

		uint32 IndexGridClearValues[4] = { 0x0,0x0,0x0,0x0 };
		AddClearUAVPass(GraphBuilder, BinData.VisTileBinningGridUAV, IndexGridClearValues);
		AddClearUAVPass(GraphBuilder, BinData.VisTileArgsUAV, 0u);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BinData.CompactedVisTileArgs, PF_R32_UINT), 0u);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BinData.CompactedVisTileDataCount, PF_R32_UINT), 0u);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BinData.CompactedVisTileWorkCount, PF_R32_UINT), 0u);
	}

	// Binning pass
	{
		FHairStrandsForwardRasterBinningCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsForwardRasterBinningCS::FParameters>();
		Parameters->Common = Common;
		Parameters->ViewUniformBuffer = ViewUniformShaderParameters;
		Parameters->OutVisTilePrims = GraphBuilder.CreateUAV(BinData.VisTilePrims, PF_R32_UINT);
		Parameters->OutVisTilePrimDepths = GraphBuilder.CreateUAV(BinData.VisTilePrimDepths, PF_R32_UINT);
		Parameters->OutVisTileBinningGrid = BinData.VisTileBinningGridUAV;
		Parameters->OutVisTileBinningGridMinZ = BinData.VisTileBinningGridMinZUAV;
		Parameters->OutVisTileBinningGridMaxZ = BinData.VisTileBinningGridMaxZUAV;
		Parameters->VisTileDepthGrid = VisTileDepthGrid;
		Parameters->OutVisTileArgs = BinData.VisTileArgsUAV;
		Parameters->OutVisTileData = GraphBuilder.CreateUAV(BinData.VisTileData);
		Parameters->VisTileBinningGridTex = BinData.VisTileBinningGrid;
		ShaderPrint::SetParameters(GraphBuilder, ViewInfo.ShaderPrintData, Parameters->ShaderPrintParameters);

		FHairStrandsForwardRasterBinningCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FHairStrandsForwardRasterBinningCS::FDebug>(bDebugEnabled);
		TShaderMapRef<FHairStrandsForwardRasterBinningCS> ComputeShaderBinning(ViewInfo.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder, 
			RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterBinning"),
			ComputeShaderBinning,
			Parameters, 
			FIntVector(NumBinners, 1, 1));
	}

	// Compaction
	{
		FHairStrandsForwardRasterCompactionCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsForwardRasterCompactionCS::FParameters>();
		Parameters->Common				= Common;
		Parameters->ViewUniformBuffer	= ViewUniformShaderParameters;
		Parameters->InData				= BinData.VisTileDataSRV;
		Parameters->InPrims				= BinData.VisTilePrimsSRV;
		Parameters->InDepths			= BinData.VisTilePrimDepthsSRV;
		Parameters->InArgs				= BinData.VisTileArgsSRV;
		Parameters->OutData				= GraphBuilder.CreateUAV(BinData.CompactedVisTileData);
		Parameters->OutPrims			= GraphBuilder.CreateUAV(BinData.CompactedVisTilePrims, PF_R32_UINT);
		Parameters->OutArgs 			= GraphBuilder.CreateUAV(BinData.CompactedVisTileArgs, PF_R32_UINT);
		Parameters->OutWork				= GraphBuilder.CreateUAV(BinData.CompactedVisTileWork, PF_R32_UINT);
		Parameters->OutDataCount		= GraphBuilder.CreateUAV(BinData.CompactedVisTileDataCount, PF_R32_UINT);
		Parameters->OutWorkCount		= GraphBuilder.CreateUAV(BinData.CompactedVisTileWorkCount, PF_R32_UINT);

		ShaderPrint::SetParameters(GraphBuilder, ViewInfo.ShaderPrintData, Parameters->ShaderPrintParameters);

		FHairStrandsForwardRasterCompactionCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FHairStrandsForwardRasterCompactionCS::FDebug>(bDebugEnabled);
		TShaderMapRef<FHairStrandsForwardRasterCompactionCS> ComputeShaderCompaction(ViewInfo.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterCompaction"), ComputeShaderCompaction, Parameters, FIntVector(BinTileRes.X, BinTileRes.Y, 1));
	}

	// Debug data
	FRDGTextureRef HairPixelCountPerTile_ForDebug = nullptr;
	FRDGTextureRef HairCountTexture_ForDebug = nullptr;
	if (bDebugEnabled)
	{
		HairPixelCountPerTile_ForDebug = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(RasterTileRes, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable), TEXT("Hair.HairPixelTileTexture_ForDebug"));
		HairCountTexture_ForDebug = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutResolution, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable), TEXT("Hair.HairCountTexture_ForDebug"));

		uint32 ClearValues[4] = { 0,0,0,0 };
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HairCountTexture_ForDebug), ClearValues);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HairPixelCountPerTile_ForDebug), ClearValues);
	}

	// Raster pass
	{
		FRDGBufferRef WorkCounter = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("Hair.RasterWorkCounter"));
		FRDGBufferUAVRef WorkCounterUAV = GraphBuilder.CreateUAV(WorkCounter, PF_R32_UINT);
		AddClearUAVPass(GraphBuilder, WorkCounterUAV, 0);

		FHairStrandsForwardRasterRasterizeCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsForwardRasterRasterizeCS::FParameters>();
		Parameters->Common = Common;
		Parameters->ViewUniformBuffer = ViewUniformShaderParameters;
		Parameters->SceneDepthTexture = SceneDepthTexture;
		Parameters->OutHairCountTexture_ForDebug = HairCountTexture_ForDebug ? GraphBuilder.CreateUAV(HairCountTexture_ForDebug) : nullptr;
		Parameters->OutHairPixelCountPerTile_ForDebug = HairPixelCountPerTile_ForDebug ? GraphBuilder.CreateUAV(HairPixelCountPerTile_ForDebug) : nullptr;
		Parameters->SampleLightingViewportResolution = InData.SampleLightingViewportResolution;
		Parameters->SampleLightingTexture = InData.SampleLightingTexture;
		Parameters->SampleVelocityBuffer = InData.ControlPointVelocitySRV;
		Parameters->VisTilePrims = GraphBuilder.CreateSRV(BinData.CompactedVisTilePrims, PF_R32_UINT);
		Parameters->VisTileArgs = GraphBuilder.CreateSRV(BinData.CompactedVisTileArgs, PF_R32_UINT);
		Parameters->VisTileWork = GraphBuilder.CreateSRV(BinData.CompactedVisTileWork, PF_R32_UINT);
		Parameters->VisTileWorkCount = GraphBuilder.CreateSRV(BinData.CompactedVisTileWorkCount, PF_R32_UINT);
		Parameters->VisTileData = GraphBuilder.CreateSRV(BinData.CompactedVisTileData);
		Parameters->OutSceneColorTexture = GraphBuilder.CreateUAV(SceneColorTexture);
		Parameters->OutSceneVelocityTexture = GraphBuilder.CreateUAV(SceneVelocityTexture);
		Parameters->RWWorkCounter = WorkCounterUAV;
		ShaderPrint::SetParameters(GraphBuilder, ViewInfo.ShaderPrintData, Parameters->ShaderPrintParameters);

		FHairStrandsForwardRasterRasterizeCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FHairStrandsForwardRasterRasterizeCS::FDebug>(bDebugEnabled);
		TShaderMapRef<FHairStrandsForwardRasterRasterizeCS> ComputeShaderRaster(ViewInfo.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterRaster(tiled)"), ComputeShaderRaster, Parameters, FIntVector(NumRasterizers, 1, 1));
	}

	if (bDebugEnabled)
	{
		FHairStrandsForwardRasterDebugCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsForwardRasterDebugCS::FParameters>();
		Parameters->Common = Common;
		Parameters->InstanceCount = InData.RasterizedInstanceCount;
		Parameters->CPUAllocatedTileCount = BinData.VisTileData->Desc.NumElements / EntryCount;
		Parameters->CPUAllocatedCompactedTileCount = BinData.CompactedVisTileData->Desc.NumElements / EntryCount;
		Parameters->VisTileData = GraphBuilder.CreateSRV(BinData.CompactedVisTileData);
		Parameters->VisTileArgs = BinData.VisTileArgsSRV;
		Parameters->CompactedVisTileData = GraphBuilder.CreateSRV(BinData.CompactedVisTileData);
		Parameters->CompactedVisTileArgs = GraphBuilder.CreateSRV(BinData.CompactedVisTileArgs, PF_R32_UINT);
		Parameters->HairCountTexture_ForDebug = HairCountTexture_ForDebug;
		Parameters->HairPixelCountPerTile_ForDebug = HairPixelCountPerTile_ForDebug;
		ShaderPrint::SetParameters(GraphBuilder, ViewInfo.ShaderPrintData, Parameters->ShaderPrintParameters);

		TShaderMapRef<FHairStrandsForwardRasterDebugCS> DebugComputeShader(ViewInfo.ShaderMap);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VisibilityComputeRaster(Debug)"), DebugComputeShader, Parameters, FIntVector(FMath::DivideAndRoundUp(Common.BinTileRes.X, 8), FMath::DivideAndRoundUp(Common.BinTileRes.Y, 8), 1));
	}
}
