// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairVisibilityRendering.h: Hair strands visibility buffer implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "HairStrandsData.h"

class FViewInfo;
class FInstanceCullingManager;

struct FRasterForwardCullingOutput
{
	FIntPoint			Resolution;
	uint32				RasterizedInstanceCount = 0;
	FRDGTextureRef		NodeIndex = nullptr;
	FRDGBufferRef		NodeCoord = nullptr;
	FRDGBufferRef		NodeVis = nullptr;
	FRDGBufferRef		PointCount = nullptr;
	FRDGBufferRef		Points = nullptr;
	FRDGBufferSRVRef	PointsSRV = nullptr;
};

FRasterForwardCullingOutput AddHairStrandsForwardCullingPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& ViewInfo,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FIntPoint& InResolution,
	const FRDGTextureRef SceneDepthTexture,
	bool bSupportCulling,
	bool bForceRegister);

void AddHairStrandsForwardRasterPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& ViewInfo,
	const FIntPoint& InResolution,
	const FHairStrandsVisibilityData& InData,
	const FRDGTextureRef SceneDepthTexture,
	const FRDGTextureRef SceneColorTexture,
	const FRDGTextureRef SceneVelocityColorTexture);