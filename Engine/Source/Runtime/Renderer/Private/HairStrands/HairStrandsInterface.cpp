// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsInterface.h: Hair manager implementation.
=============================================================================*/

#include "HairStrandsInterface.h"
#include "HairStrandsRendering.h"
#include "HairStrandsData.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "LightSceneProxy.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "CommonRenderResources.h"
#include "Components/SkeletalMeshComponent.h"
#include "SkeletalRenderPublic.h"
#include "SceneRendering.h"
#include "SystemTextures.h"
#include "ShaderPrint.h"
#include "ScenePrivate.h"
#include "SceneViewState.h"
#include "GroomVisualizationData.h"

DEFINE_LOG_CATEGORY_STATIC(LogHairRendering, Log, All);

static TAutoConsoleVariable<int32> CVarHairStrandsRaytracingEnable(
	TEXT("r.HairStrands.Raytracing"), 1,
	TEXT("Enable/Disable hair strands raytracing geometry. This is anopt-in option per groom asset/groom instance."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarHairStrandsEnable(
	TEXT("r.HairStrands.Strands"), 1,
	TEXT("Enable/Disable hair strands rendering"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarHairCardsEnable(
	TEXT("r.HairStrands.Cards"), 1,
	TEXT("Enable/Disable hair cards rendering. This variable needs to be turned on when the engine starts."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarHairMeshesEnable(
	TEXT("r.HairStrands.Meshes"), 1,
	TEXT("Enable/Disable hair meshes rendering. This variable needs to be turned on when the engine starts."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarHairStrandsBinding(
	TEXT("r.HairStrands.Binding"), 1,
	TEXT("Enable/Disable hair binding, i.e., hair attached to skeletal meshes."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarHairStrandsSimulation(
	TEXT("r.HairStrands.Simulation"), 1,
	TEXT("Enable/disable hair simulation"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarHairStrandsNonVisibleShadowCasting(
	TEXT("r.HairStrands.Shadow.CastShadowWhenNonVisible"), 1,
	TEXT("Enable shadow casting for hair strands even when culled out from the primary view"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarHairStrandsNonVisibleShadowCasting_CullDistance(
	TEXT("r.HairStrands.Visibility.NonVisibleShadowCasting.CullDistance"), 2000,
	TEXT("Cull distance at which shadow casting starts to be disabled for non-visible hair strands instances."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHairStrandsNonVisibleShadowCasting_Debug(
	TEXT("r.HairStrands.Visibility.NonVisibleShadowCasting.Debug"), 0,
	TEXT("Enable debug rendering for non-visible hair strands instance, casting shadow."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHairStrandsContinuousDecimationReordering(
	TEXT("r.HairStrands.ContinuousDecimationReordering"), 0,
	TEXT("Enable strand reordering to allow Continuous LOD. Experimental"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarHairStrandsVisibilityComputeRaster(
	TEXT("r.HairStrands.Visibility.ComputeRaster"), 0,
	TEXT("Hair Visiblity uses raster compute. Experimental"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHairStrandsVisibilityComputeRaster_ContinuousLOD(
	TEXT("r.HairStrands.Visibility.ComputeRaster.ContinuousLOD"), 1,
	TEXT("Enable Continuos LOD when using compute rasterization. Experimental"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHairStrandsRaytracingEnableLinearSweptSpheres(
	TEXT("r.HairStrands.Raytracing.EnableLinearSweptSpheres"), 1,
	TEXT("Enable Linear Swept Spheres for hair strand raytracing (if the current hardware supports it)."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Import/export utils function for hair resources
void FRDGExternalBuffer::Release()
{
	Buffer = nullptr;
	SRV = nullptr;
	UAV = nullptr;
}

FRDGImportedBuffer Register(FRDGBuilder& GraphBuilder, const FRDGExternalBuffer& In, ERDGImportedBufferFlags Flags, ERDGUnorderedAccessViewFlags UAVFlags)
{
	FRDGImportedBuffer Out;
	if (!In.Buffer)
	{
		return Out;
	}
	const uint32 uFlags = uint32(Flags);
	Out.Buffer = GraphBuilder.RegisterExternalBuffer(In.Buffer);
	if (In.Format != PF_Unknown)
	{
		if (uFlags & uint32(ERDGImportedBufferFlags::CreateSRV)) { Out.SRV = GraphBuilder.CreateSRV(Out.Buffer, In.Format); }
		if (uFlags & uint32(ERDGImportedBufferFlags::CreateUAV)) { Out.UAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Out.Buffer, In.Format), UAVFlags); }
	}
	else
	{
		if (uFlags & uint32(ERDGImportedBufferFlags::CreateSRV)) { Out.SRV = GraphBuilder.CreateSRV(Out.Buffer); }
		if (uFlags & uint32(ERDGImportedBufferFlags::CreateUAV)) { Out.UAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Out.Buffer),  UAVFlags); }
	}
	return Out;
}

FRDGBufferSRVRef RegisterAsSRV(FRDGBuilder& GraphBuilder, const FRDGExternalBuffer& In)
{
	if (!In.Buffer)
	{
		return nullptr;
	}

	FRDGBufferSRVRef Out = nullptr;
	FRDGBufferRef Buffer = GraphBuilder.RegisterExternalBuffer(In.Buffer);
	if (In.Format != PF_Unknown)
	{
		Out = GraphBuilder.CreateSRV(Buffer, In.Format);
	}
	else
	{
		Out = GraphBuilder.CreateSRV(Buffer);
	}
	return Out;
}

FRDGBufferUAVRef RegisterAsUAV(FRDGBuilder& GraphBuilder, const FRDGExternalBuffer& In, ERDGUnorderedAccessViewFlags Flags)
{
	if (!In.Buffer)
	{
		return nullptr;
	}

	FRDGBufferUAVRef Out = nullptr;
	FRDGBufferRef Buffer = GraphBuilder.RegisterExternalBuffer(In.Buffer);
	if (In.Format != PF_Unknown)
	{
		Out = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Buffer, In.Format), Flags);
	}
	else
	{
		Out = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Buffer), Flags);
	}
	return Out;
}

bool IsHairRayTracingEnabled()
{
	if (GIsRHIInitialized && !IsRunningCookCommandlet())
	{
		return IsRayTracingEnabled() && CVarHairStrandsRaytracingEnable.GetValueOnAnyThread();
	}
	else
	{
		return false;
	}
}

bool IsHairStrandsSupported(EHairStrandsShaderType Type, EShaderPlatform Platform)
{
	if (!IsGroomEnabled()) return false;

	const bool Cards_Meshes_All = true;

	switch (Type)
	{
	case EHairStrandsShaderType::Strands: return IsHairStrandsGeometrySupported(Platform);
	case EHairStrandsShaderType::Cards:	  return Cards_Meshes_All;
	case EHairStrandsShaderType::Meshes:  return Cards_Meshes_All;
	case EHairStrandsShaderType::Tool:	  return (IsD3DPlatform(Platform) || IsVulkanPlatform(Platform)) && IsPCPlatform(Platform) && IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5);
	case EHairStrandsShaderType::All:	  return Cards_Meshes_All;
	}
	return false;
}

bool IsHairStrandsEnabled(EHairStrandsShaderType Type, EShaderPlatform Platform)
{
	const bool HairStrandsGlobalEnable = IsGroomEnabled();
	if (!HairStrandsGlobalEnable) return false;

	const int32 HairStrandsEnable = CVarHairStrandsEnable.GetValueOnAnyThread();
	const int32 HairCardsEnable   = CVarHairCardsEnable.GetValueOnAnyThread();
	const int32 HairMeshesEnable  = CVarHairMeshesEnable.GetValueOnAnyThread();
	switch (Type)
	{
	case EHairStrandsShaderType::Strands:	return HairStrandsEnable > 0 && (Platform != EShaderPlatform::SP_NumPlatforms ? IsHairStrandsGeometrySupported(Platform) : true);
	case EHairStrandsShaderType::Cards:		return HairCardsEnable > 0;
	case EHairStrandsShaderType::Meshes:	return HairMeshesEnable > 0;
#if PLATFORM_DESKTOP && PLATFORM_WINDOWS
	case EHairStrandsShaderType::Tool:		return (HairCardsEnable > 0 || HairMeshesEnable > 0 || HairStrandsEnable > 0);
#else
	case EHairStrandsShaderType::Tool:		return false;
#endif
	case EHairStrandsShaderType::All :		return HairStrandsGlobalEnable && (HairCardsEnable > 0 || HairMeshesEnable > 0 || HairStrandsEnable > 0);
	}
	return false;
}

bool IsHairStrandsBindingEnable()
{
	return CVarHairStrandsBinding.GetValueOnAnyThread() > 0;
}

bool IsHairStrandsSimulationEnable()
{
	return CVarHairStrandsSimulation.GetValueOnAnyThread() > 0;
}

bool IsHairStrandsLinearSweptSpheresEnabled()
{
	// Are LSS primitives supported on the currently running device and are they enable for hair strands in this project?
	return GRHIGlobals.RayTracing.SupportsLinearSweptSpheres && CVarHairStrandsRaytracingEnableLinearSweptSpheres.GetValueOnAnyThread() != 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FHairTransientResources

FVector3f FHairTransientResources::GetTranslatedWorldOffsetCorrection(const FViewInfo& View) const
{ 
	return FVector3f(View.ViewMatrices.GetPreViewTranslation() - TranslatedWorldOffsetView0); 
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FHairInstanceCullingResults

void FHairInstanceCullingResults::InitVisibility(uint32 InstanceCount)
{
	InstancesVisibilityType.Init(0u, InstanceCount);
}

void FHairInstanceCullingResults::Add(const FViewInfo& View, const FHairStrandsInstance& Instance, EHairInstanceVisibilityType Flag)
{
	if (InstancesVisibilityType.IsValidIndex(Instance.RegisteredIndex))
	{
		const uint32 ViewIndex = FMath::Min(View.SceneRendererPrimaryViewId, 7);
		const uint32 ViewMask  = 0xF << (ViewIndex * 4u);
		const uint32 BitsToAdd = uint32(Flag) << (ViewIndex * 4u);
		InstancesVisibilityType[Instance.RegisteredIndex] &= ~ViewMask;
		InstancesVisibilityType[Instance.RegisteredIndex] |=  BitsToAdd;
	}
}

EHairInstanceVisibilityType FHairInstanceCullingResults::Is(const FViewInfo& View, const FHairStrandsInstance& Instance) const
{
	EHairInstanceVisibilityType Out = EHairInstanceVisibilityType::NotVisible;
	if (InstancesVisibilityType.IsValidIndex(Instance.RegisteredIndex))
	{
		const uint32 ViewIndex = FMath::Min(View.SceneRendererPrimaryViewId, 7);
		const uint32 ViewBits  = (InstancesVisibilityType[Instance.RegisteredIndex] >> (4u * ViewIndex)) & 0xF;
//		const uint32 BitIndex = 31u - FMath::CountLeadingZeros(ViewBits);
		Out = EHairInstanceVisibilityType(ViewBits);
	}
	return Out;
}

EHairInstanceVisibilityType FHairInstanceCullingResults::Is(const FSceneView& View, const FHairStrandsInstance& Instance) const
{
	return Is(*(FViewInfo*)(&View), Instance);
}

bool FHairInstanceCullingResults::Is(const FViewInfo& View, const FHairStrandsInstance& Instance, EHairInstanceVisibilityType Flag) const
{
	return Is(View, Instance) == Flag;
}

bool FHairInstanceCullingResults::IsVisibleInAnyViews(const FHairStrandsInstance& Instance) const
{
	return InstancesVisibilityType.IsValidIndex(Instance.RegisteredIndex) && InstancesVisibilityType[Instance.RegisteredIndex] != 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FHairGroupPublicData::FHairGroupPublicData(uint32 InGroupIndex, const FName& InOwnerName)
{
	GroupIndex = InGroupIndex;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

bool IsHairStrandsNonVisibleShadowCastingEnable()
{
	return CVarHairStrandsNonVisibleShadowCasting.GetValueOnAnyThread() > 0;
}

bool IsHairStrandsVisibleInShadows(const FViewInfo& View, const FHairStrandsInstance& Instance)
{
	const bool bDebugEnable = CVarHairStrandsNonVisibleShadowCasting_Debug.GetValueOnRenderThread() > 0;
	FHairStrandsDebugData::FCullData* CullingData = nullptr;
	if (bDebugEnable)
	{
		CullingData = const_cast<FHairStrandsDebugData::FCullData*>(&View.HairStrandsViewData.DebugData.CullData);
		CullingData->bIsValid = true;
		CullingData->ViewFrustum = View.ViewFrustum;
	}

	bool bIsVisibleInShadow = false;
	if (const FHairGroupPublicData* HairData = Instance.GetHairData())
	{
		// Run simulation if the instance is either Strands geometry, have simulation, or has global interpolation enabled.
		// This ensures that the groom is correctly updated if visible in shadows
		const bool bNeedUpdate = 
			HairData->LODIndex >= 0 && 
			(Instance.GetHairGeometry() == EHairGeometryType::Strands || 
			 HairData->IsSimulationEnable(HairData->LODIndex) || 
			 HairData->IsGlobalInterpolationEnable(HairData->LODIndex));
		if (!bNeedUpdate)
		{
			return false;
		}

		const FBoxSphereBounds& Bounds = Instance.GetBounds();
		{
			// Local lights
			for (const FLightSceneInfo* LightInfo : View.HairStrandsViewData.VisibleShadowCastingLights)
			{
				if (LightInfo->Proxy->AffectsBounds(Bounds))
				{
					bIsVisibleInShadow = true;
					break;
				}
			}

			// Directional lights
			if (!bIsVisibleInShadow)
			{
				const float CullDistance = FMath::Max(0.f, CVarHairStrandsNonVisibleShadowCasting_CullDistance.GetValueOnAnyThread());
				for (const FHairStrandsViewData::FDirectionalLightCullData& CullData : View.HairStrandsViewData.VisibleShadowCastingDirectionalLights)
				{
					// Transform frustum into light space
					const FMatrix& WorldToLight = CullData.LightInfo->Proxy->GetWorldToLight();
					
					// Transform groom bound into light space, and extend it along the light direction
					FBoxSphereBounds BoundsInLightSpace = Bounds.TransformBy(WorldToLight);
					BoundsInLightSpace.BoxExtent.X += CullDistance;

					const bool bIntersect = CullData.ViewFrustumInLightSpace.IntersectBox(BoundsInLightSpace.Origin, BoundsInLightSpace.BoxExtent);
					if (bDebugEnable)
					{
						FHairStrandsDebugData::FCullData::FLight& LightData = CullingData->DirectionalLights.AddDefaulted_GetRef();
						LightData.WorldToLight = WorldToLight;
						LightData.LightToWorld = CullData.LightInfo->Proxy->GetLightToWorld();
						LightData.Center = FVector3f(CullData.LightInfo->Proxy->GetBoundingSphere().Center);
						LightData.Extent = FVector3f(CullData.LightInfo->Proxy->GetBoundingSphere().W);
						LightData.ViewFrustumInLightSpace = CullData.ViewFrustumInLightSpace;
						LightData.InstanceBoundInLightSpace.Add({FVector3f(BoundsInLightSpace.GetBox().Min), FVector3f(BoundsInLightSpace.GetBox().Max)});
						LightData.InstanceBoundInWorldSpace.Add({ FVector3f(Bounds.GetBox().Min), FVector3f(Bounds.GetBox().Max)});
						LightData.InstanceIntersection.Add(bIntersect ? 1u : 0u);
					}

					// Ensure the extended groom bound interest the frustum
					if (bIntersect)
					{
						bIsVisibleInShadow = true;
						if (!bDebugEnable)
						{
							break;
						}
					}
				}
			}
		}
	}
	return bIsVisibleInShadow;
}

bool IsHairStrandContinuousDecimationReorderingEnabled()
{
	//NB this is a readonly cvar - and causes changes in platform data and runtime allocations 
	return CVarHairStrandsContinuousDecimationReordering.GetValueOnAnyThread() > 0;
}

bool IsHairVisibilityComputeRasterEnabled()
{
	return CVarHairStrandsVisibilityComputeRaster.GetValueOnAnyThread() == 1;
}

bool IsHairVisibilityComputeRasterForwardEnabled(EShaderPlatform InPlatform)
{
	return IsFeatureLevelSupported(InPlatform, ERHIFeatureLevel::SM6) && CVarHairStrandsVisibilityComputeRaster.GetValueOnAnyThread() == 2;
}

bool IsHairVisibilityComputeRasterContinuousLODEnabled()
{
	return IsHairStrandContinuousDecimationReorderingEnabled() && IsHairVisibilityComputeRasterEnabled() && (CVarHairStrandsVisibilityComputeRaster_ContinuousLOD.GetValueOnAnyThread() > 0);
}

uint32 FHairGroupPublicData::GetActiveStrandsPointCount(bool bPrevious) const
{
	return bPrevious ? ContinuousLODPreviousPointCount : ContinuousLODPointCount;
}

uint32 FHairGroupPublicData::GetActiveStrandsCurveCount(bool bPrevious) const
{
	return bPrevious ? ContinuousLODPreviousCurveCount : ContinuousLODCurveCount;
}

float FHairGroupPublicData::GetActiveStrandsCoverageScale() const
{
	return ContinuousLODCoverageScale; 
}

float FHairGroupPublicData::GetActiveStrandsRadiusScale() const
{
	return ContinuousLODRadiusScale; 
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Bookmark API
THairStrandsBookmarkFunction  GHairStrandsBookmarkFunction = nullptr;
void RegisterBookmarkFunction(THairStrandsBookmarkFunction Bookmark)
{
	if (Bookmark)
	{
		GHairStrandsBookmarkFunction = Bookmark;
	}
}

void RunHairStrandsBookmark(FRDGBuilder& GraphBuilder, EHairStrandsBookmark Bookmark, FHairStrandsBookmarkParameters& Parameters)
{
	if (GHairStrandsBookmarkFunction)
	{
		GHairStrandsBookmarkFunction(&GraphBuilder, Bookmark, Parameters);
	}
}

void RunHairStrandsBookmark(EHairStrandsBookmark Bookmark, FHairStrandsBookmarkParameters& Parameters)
{
	if (GHairStrandsBookmarkFunction)
	{
		GHairStrandsBookmarkFunction(nullptr, Bookmark, Parameters);
	}
}

void CullHairInstance(FScene* Scene, FViewInfo& View, FHairInstanceCullingResults& Out)
{
	// Only compute visible instances when required as this is expensive.
	{
		const int32 ActiveInstanceCount = Scene->HairStrandsSceneData.RegisteredProxies.Num();
		Out.InitVisibility(ActiveInstanceCount);
	
		// 1. Strands - Add all visible strands instances
		Out.VisibleInAnyViews_Strands.Reserve(View.HairStrandsMeshElements.Num());
		for (const FMeshBatchAndRelevance& MeshBatch : View.HairStrandsMeshElements)
		{
			check(MeshBatch.PrimitiveSceneProxy && MeshBatch.PrimitiveSceneProxy->ShouldRenderInMainPass());
			if (MeshBatch.Mesh && MeshBatch.Mesh->Elements.Num() > 0)
			{
				FHairGroupPublicData* HairData = HairStrands::GetHairData(MeshBatch.Mesh);
				if (HairData && HairData->Instance)
				{
					if (!Out.IsVisibleInAnyViews(*HairData->Instance))
					{
						Out.VisibleInAnyViews_Strands.Add(HairData->Instance);
					}
					Out.Add(View, *HairData->Instance, EHairInstanceVisibilityType::StrandsPrimaryView);
				}
			}
		}
	
		// 2. Cards/Meshes - Add all visible cards instances
		Out.VisibleInAnyViews_CardsOrMeshes_Primary.Reserve(View.HairCardsMeshElements.Num());
		for (const FMeshBatchAndRelevance& MeshBatch : View.HairCardsMeshElements)
		{
			check(MeshBatch.PrimitiveSceneProxy && MeshBatch.PrimitiveSceneProxy->ShouldRenderInMainPass());
			if (MeshBatch.Mesh && MeshBatch.Mesh->Elements.Num() > 0)
			{
				FHairGroupPublicData* HairData = HairStrands::GetHairData(MeshBatch.Mesh);
				if (HairData && HairData->Instance)
				{
					if (!Out.IsVisibleInAnyViews(*HairData->Instance))
					{
						Out.VisibleInAnyViews_CardsOrMeshes_Primary.Add(HairData->Instance);
					}
					Out.Add(View, *HairData->Instance, EHairInstanceVisibilityType::CardsOrMeshesPrimaryView);
				}
			}
		}
	} 
}

void CullHairInstanceShadow(FScene* Scene, TArray<FViewInfo>& Views, FHairInstanceCullingResults& Out)
{
	const int32 ActiveInstanceCount = Scene->HairStrandsSceneData.RegisteredProxies.Num();
	for (FHairStrandsInstance* Instance : Scene->HairStrandsSceneData.RegisteredProxies)
	{
		// 2. Strands - Add all instances non-visible primary view(s) but visible in shadow view(s)
		const bool bStrands = Instance->GetHairGeometry() == EHairGeometryType::Strands;
		const bool bCardsOrMeshes = Instance->GetHairGeometry() == EHairGeometryType::Cards || Instance->GetHairGeometry() == EHairGeometryType::Meshes;
		const bool bCompatible = bStrands || bCardsOrMeshes;

		if (bCompatible)
		{
			for (const FViewInfo& View : Views)
			{
				if (Out.Is(View, *Instance) == EHairInstanceVisibilityType::NotVisible && IsHairStrandsVisibleInShadows(View, *Instance))
				{
					if (bStrands)
					{ 
						if (!Out.IsVisibleInAnyViews(*Instance))
						{
							Out.VisibleInAnyViews_Strands.Add(Instance);
						}
						Out.Add(View, *Instance, EHairInstanceVisibilityType::StrandsShadowView);
					}
					else if (bCardsOrMeshes) 
					{ 
						if (!Out.IsVisibleInAnyViews(*Instance))
						{
							Out.VisibleInAnyViews_CardsOrMeshes_Shadow.Add(Instance);
						}
						Out.Add(View, *Instance, EHairInstanceVisibilityType::CardsOrMeshesShadowView);
					}
				}
			}
		}
	}
}

void CreateHairStrandsBookmarkParameters(FScene* Scene, FViewInfo& View, FHairStrandsBookmarkParameters& Out, bool bComputeVisibleInstances)
{
	// Only compute visible instances when required as this is expensive.
	if (bComputeVisibleInstances)
	{
		CullHairInstance(Scene, View, Out.CullingResults);
	}

	Out.ShaderPrintData			= ShaderPrint::IsEnabled(View.ShaderPrintData) ? &View.ShaderPrintData : nullptr;
	Out.ShaderMap				= View.ShaderMap;
	Out.Instances				= &Scene->HairStrandsSceneData.RegisteredProxies;
	Out.View					= &View;
	Out.ViewRect				= View.ViewRect;
	Out.ViewUniqueID			= View.ViewState ? View.ViewState->UniqueID : ~0;
	Out.SceneColorTexture		= nullptr;
	Out.SceneDepthTexture		= nullptr;
	Out.Scene					= Scene;
	Out.TransientResources		= Scene->HairStrandsSceneData.TransientResources;
}

void UpdateHairStrandsBookmarkParameters(FScene* Scene, TArray<FViewInfo>& Views, FHairStrandsBookmarkParameters& Out)
{
	if (Views.Num() == 0)
	{
		return;
	}

	if (IsHairStrandsNonVisibleShadowCastingEnable())
	{
		CullHairInstanceShadow(Scene, Views, Out.CullingResults);
	}
}

void CreateHairStrandsBookmarkParameters(FScene* Scene, TArray<FViewInfo>& Views, TConstArrayView<FViewInfo*> AllViews, FHairStrandsBookmarkParameters& Out, bool bComputeVisibleInstances)
{
	CreateHairStrandsBookmarkParameters(Scene, Views[0], Out, bComputeVisibleInstances);
	Out.AllViews.Reserve(AllViews.Num());
	for (FViewInfo* View : AllViews)
	{
		Out.AllViews.Emplace(View);
	}
}

namespace HairStrands
{

bool IsHairStrandsVF(const FMeshBatch* Mesh)
{
	if (Mesh)
	{
		return IsHairStrandsVF(Mesh->VertexFactory->GetType());
	}
	return false;
}

bool IsHairCardsVF(const FMeshBatch* Mesh)
{
	if (Mesh)
	{
		static const FHashedName& VFTypeRef = FVertexFactoryType::GetVFByName(TEXT("FHairCardsVertexFactory"))->GetHashedName();
		const FHashedName& VFType = Mesh->VertexFactory->GetType()->GetHashedName();
		return VFType == VFTypeRef;
	}
	return false;
}

bool IsHairCompatible(const FMeshBatch* Mesh)
{
	return IsHairStrandsVF(Mesh) || IsHairCardsVF(Mesh);
}

bool IsHairVisible(const FMeshBatchAndRelevance& MeshBatch, bool bCheckLengthScale)
{
	if (MeshBatch.Mesh && MeshBatch.PrimitiveSceneProxy && MeshBatch.PrimitiveSceneProxy->ShouldRenderInMainPass())
	{
		const FHairGroupPublicData* Data = HairStrands::GetHairData(MeshBatch.Mesh);
		switch (Data->VFInput.GeometryType)
		{
		case EHairGeometryType::Strands: return bCheckLengthScale ? Data->VFInput.Strands.Common.LengthScale > 0 : true;
		case EHairGeometryType::Cards: return true;
		case EHairGeometryType::Meshes: return true;
		}
	}
	return false;
}

bool IsHairStrandsVF(const FVertexFactoryType* InVFType)
{
	// NOTE: The VF is defined in a different module, so we must search for it by name
	static const FVertexFactoryType* VFTypeA = FVertexFactoryType::GetVFByName(TEXT("FHairStrandsVertexFactory"));
	static const FVertexFactoryType* VFTypeB = FVertexFactoryType::GetVFByName(TEXT("FLinearSweptSpheresHairStrandsVertexFactory"));
	#if !UE_BUILD_SHIPPING
	if (IsGroomEnabled())
	{
		check(VFTypeA != nullptr);
		check(VFTypeB != nullptr);
	}
	#endif
	return InVFType == VFTypeA || InVFType == VFTypeB;
}

FHairGroupPublicData* GetHairData(const FMeshBatch* Mesh)
{
	return reinterpret_cast<FHairGroupPublicData*>(Mesh->Elements[0].VertexFactoryUserData);
}

void AddVisibleShadowCastingLight(const FScene& Scene, TArray<FViewInfo>& Views, const FLightSceneInfo* LightSceneInfo)
{
	const uint8 LightType = LightSceneInfo->Proxy->GetLightType();
	for (FViewInfo& View : Views)
	{
		// If any hair data are registered, track which lights are visible so that hair strands can cast shadow even if not visible in primary view
		// 
		// Actual intersection test is done in IsHairStrandsVisibleInShadows():
		// * For local lights, shadow casting is determined based on light influence radius
		// * For directional light, view frustum and hair instance bounds are transformed in light space to determined potential intersections
		if (Scene.HairStrandsSceneData.RegisteredProxies.Num() > 0)
		{
			if (LightType == ELightComponentType::LightType_Directional)
			{
				FHairStrandsViewData::FDirectionalLightCullData& Data = View.HairStrandsViewData.VisibleShadowCastingDirectionalLights.AddDefaulted_GetRef();

				// Transform view frustum into light space
				const FMatrix& WorldToLight = LightSceneInfo->Proxy->GetWorldToLight();
				const uint32 PlaneCount = View.ViewFrustum.Planes.Num();
				Data.ViewFrustumInLightSpace.Planes.SetNum(PlaneCount);
				for (uint32 PlaneIt = 0; PlaneIt < PlaneCount; ++PlaneIt)
				{
					Data.ViewFrustumInLightSpace.Planes[PlaneIt] = View.ViewFrustum.Planes[PlaneIt].TransformBy(WorldToLight);
				}
				Data.ViewFrustumInLightSpace.Init();
				Data.LightInfo = LightSceneInfo;
			}
			else
			{
				View.HairStrandsViewData.VisibleShadowCastingLights.Add(LightSceneInfo);
			}
		}
	}
}

} // namespace HairStrands
