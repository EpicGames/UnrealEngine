// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsLUT.h"
#include "HairStrandsUtils.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "RenderGraphUtils.h"
#include "SceneRendering.h"
#include "SystemTextures.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/VolumeTexture.h"
#include "TextureResource.h"
#include "Engine/Engine.h"

static int32 GHairLUTIncidentAngleCount = 64;
static int32 GHairLUTRoughnessCount = 64;
static int32 GHairLUTAbsorptionCount = 16;
static int32 GHairLUTSampleCountScale = 1;
static FAutoConsoleVariableRef CVarHairLUTIncidentAngleCount(TEXT("r.HairStrands.HairLUT.IncidentAngleCount"), GHairLUTIncidentAngleCount, TEXT("Change the number of slices of the hair LUT for the incident angle axis"));
static FAutoConsoleVariableRef CVarHairLUTRoughnessCount(TEXT("r.HairStrands.HairLUT.RoughnessCount"), GHairLUTRoughnessCount, TEXT("Change the number of slices of the hair LUT for the roughness axis"));
static FAutoConsoleVariableRef CVarHairLUTAbsorptionCount(TEXT("r.HairStrands.HairLUT.AbsorptionCount"), GHairLUTAbsorptionCount, TEXT("Change the number of slices of the hair LUT for the absorption axis"));
static FAutoConsoleVariableRef CVarHairLUTSampleCount(TEXT("r.HairStrands.HairLUT.SampleCountScale"), GHairLUTSampleCountScale, TEXT("Change the number of sample used for computing the hair LUT. This is a multiplier, default is 1."));

static bool IsHairMobilePlatform(EShaderPlatform Platform)
{
	return IsMobilePlatform(Platform);
}

/////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR

class FHairLUTCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairLUTCS);
	SHADER_USE_PARAMETER_STRUCT(FHairLUTCS, FGlobalShader);

	class FLUTType : SHADER_PERMUTATION_INT("PERMUTATION_LUT_TYPE", 2); // Permutation only for HairLUTType_DualScattering and HairLUTType_MeanEnergy
	using FPermutationDomain = TShaderPermutationDomain<FLUTType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, AbsorptionCount)
		SHADER_PARAMETER(uint32, RoughnessCount)
		SHADER_PARAMETER(uint32, ThetaCount)
		SHADER_PARAMETER(uint32, SampleCountScale)
		SHADER_PARAMETER(FIntVector, OutputResolution)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, Out3D)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, Out2D)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return !IsHairMobilePlatform(Parameters.Platform) && IsHairStrandsSupported(EHairStrandsShaderType::Cards, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_HAIRLUT"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairLUTCS, "/Engine/Private/HairStrands/HairStrandsLUT.usf", "MainCS", SF_Compute);

static FRDGTextureRef GenerateHairLUTPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairLUTType LUTType)
{
	// Ensure HairLUTAbsorptionCount is a square so we can output a tiled version (i.e. Out2DTexture) 
	// for saving and imported the 3D texture as a 2D texture
	const uint32 HairLUTAbsorptionCountSqrt = FMath::Max(1u, FMath::Floor(FMath::Sqrt(float(GHairLUTAbsorptionCount))));
	const uint32 HairLUTAbsorptionCount = HairLUTAbsorptionCountSqrt * HairLUTAbsorptionCountSqrt;
	const FIntVector OutResolution(GHairLUTIncidentAngleCount, GHairLUTRoughnessCount, HairLUTAbsorptionCount);

	// 3D LUT output
	FRDGTextureRef Out3DTexture = nullptr;
	{
		FRDGTextureDesc Desc;
		Desc.Dimension = ETextureDimension::Texture3D;
		Desc.Extent.X = OutResolution.X;
		Desc.Extent.Y = OutResolution.Y;
		Desc.Depth = OutResolution.Z;
		Desc.Format = PF_FloatRGBA;
		Desc.NumMips = 1;
		Desc.Flags = TexCreate_ShaderResource | TexCreate_UAV;
		Out3DTexture = GraphBuilder.CreateTexture(Desc, 
			 LUTType == FHairLUTType::HairLUTType_DualScattering ? TEXT("Hair.LUT(DualScattering)") : 
			(LUTType == FHairLUTType::HairLUTType_MeanEnergy ? TEXT("Hair.LUT(DirectionalAlbedo)") : TEXT("Hair.LUT(Coverage)")));
	}

	// 2D output - Tiled version of the 3D texture for exporting purpose
	FRDGTextureRef Out2DTexture = nullptr;
	{
		const uint32 TileScale = HairLUTAbsorptionCountSqrt;
		FRDGTextureDesc Desc;
		Desc.Dimension = ETextureDimension::Texture2D;
		Desc.Extent.X = OutResolution.X * TileScale;
		Desc.Extent.Y = OutResolution.Y * TileScale;
		Desc.Depth = 1;
		Desc.Format = PF_FloatRGBA;
		Desc.NumMips = 1;
		Desc.Flags = TexCreate_ShaderResource | TexCreate_UAV;
		Out2DTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.TransientOutput"));
	}

	FHairLUTCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairLUTCS::FParameters>();
	Parameters->Out3D = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Out3DTexture, 0));
	Parameters->Out2D = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Out2DTexture, 0));
	Parameters->ThetaCount = OutResolution.X;
	Parameters->RoughnessCount = OutResolution.Y;
	Parameters->AbsorptionCount = OutResolution.Z;
	Parameters->SampleCountScale = GHairLUTSampleCountScale;
	Parameters->OutputResolution = OutResolution;

	FHairLUTCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairLUTCS::FLUTType>(LUTType);

	TShaderMapRef<FHairLUTCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrandsLUT"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ComputeShader,
		Parameters,
		FComputeShaderUtils::GetGroupCount(OutResolution, FIntVector(FComputeShaderUtils::kGolden2DGroupSize)));

	return Out3DTexture;
}
#endif // WITH_EDITOR

/////////////////////////////////////////////////////////////////////////////////////////

struct FHairCountToCoverageData
{
	// The LUT data have been computed by rasterizing in 1D segments with various radius size 
	// (normalized), which are aligned with the rasterization plane and ortho-projected. 
	// The segments position are randomly position within this volume. The number represent the 
	// coverage of these segments over (the result is averaged over multiple iteration)
	//
	//
	//  | --.--				 |
	//  |			--.--	 |
	//  |	--.--			 |
	//  |				--.--| <- segment with radius 2
	//  |					 |
	//	 ====================  <- Projection plane
	//    -------   ---------  <- Projected coverage (18/20 = 0.9)
	//
	// LUT data
	//
	// Data are layout as:
	// .----> Radius [0..0.5]
	// :
	// :
	// v
	// Count [0..64]
	static const uint32 HairCount = 64;			// Hair count
	static const uint32 HairRadiusCount = 64;	// Normalized hair radius [0,1]
	float Data[HairCount * HairRadiusCount] =
	{
		0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f, 0.000000f,
		0.008789f, 0.024406f, 0.039734f, 0.055405f, 0.069511f, 0.084885f, 0.098946f, 0.113274f, 0.129799f, 0.144028f, 0.155937f, 0.173019f, 0.188591f, 0.196022f, 0.218147f, 0.228256f, 0.238892f, 0.258514f, 0.264328f, 0.280075f, 0.295296f, 0.314209f, 0.324478f, 0.327744f, 0.347076f, 0.360077f, 0.379913f, 0.376045f, 0.393951f, 0.407997f, 0.412193f, 0.434975f, 0.440971f, 0.461090f, 0.466393f, 0.484055f, 0.489891f, 0.498116f, 0.525528f, 0.526619f, 0.542175f, 0.557365f, 0.571144f, 0.565247f, 0.571953f, 0.580391f, 0.578880f, 0.617622f, 0.622833f, 0.630913f, 0.652977f, 0.645233f, 0.666428f, 0.660080f, 0.683022f, 0.682678f, 0.685478f, 0.693871f, 0.684959f, 0.705963f, 0.709587f, 0.712509f, 0.727493f, 0.751579f,
		0.017426f, 0.048172f, 0.077583f, 0.107742f, 0.135368f, 0.161079f, 0.192635f, 0.210968f, 0.246330f, 0.274132f, 0.303452f, 0.310791f, 0.336220f, 0.358101f, 0.382088f, 0.398293f, 0.406044f, 0.438766f, 0.461700f, 0.480713f, 0.508865f, 0.533913f, 0.542992f, 0.545395f, 0.567390f, 0.563477f, 0.583435f, 0.612846f, 0.631836f, 0.647995f, 0.648209f, 0.685837f, 0.669106f, 0.712608f, 0.713882f, 0.719688f, 0.757088f, 0.747147f, 0.718300f, 0.774094f, 0.784523f, 0.760803f, 0.800415f, 0.786049f, 0.776390f, 0.820137f, 0.835327f, 0.832344f, 0.836143f, 0.810715f, 0.842072f, 0.853645f, 0.857590f, 0.865463f, 0.896072f, 0.855072f, 0.894402f, 0.885559f, 0.901695f, 0.874512f, 0.921165f, 0.907143f, 0.906357f, 0.942085f,
		0.026024f, 0.070976f, 0.112885f, 0.156082f, 0.196938f, 0.235626f, 0.270355f, 0.312592f, 0.337318f, 0.370491f, 0.405411f, 0.425171f, 0.459076f, 0.488503f, 0.512268f, 0.544518f, 0.556007f, 0.568535f, 0.610794f, 0.621742f, 0.631241f, 0.653442f, 0.674744f, 0.688759f, 0.732086f, 0.735016f, 0.750565f, 0.761841f, 0.775375f, 0.784073f, 0.799828f, 0.823380f, 0.803802f, 0.826637f, 0.844254f, 0.845901f, 0.847824f, 0.867767f, 0.860481f, 0.865143f, 0.896805f, 0.895027f, 0.894897f, 0.899597f, 0.900177f, 0.921333f, 0.917160f, 0.923805f, 0.911125f, 0.926506f, 0.922195f, 0.931145f, 0.951881f, 0.946098f, 0.936646f, 0.940208f, 0.951546f, 0.954811f, 0.957283f, 0.958916f, 0.971268f, 0.953644f, 0.970245f, 0.966476f,
		0.034500f, 0.092972f, 0.149261f, 0.204109f, 0.255180f, 0.294800f, 0.338127f, 0.383827f, 0.427025f, 0.466690f, 0.490837f, 0.531883f, 0.552689f, 0.579475f, 0.609520f, 0.636734f, 0.656677f, 0.711418f, 0.730705f, 0.725983f, 0.742622f, 0.763206f, 0.774139f, 0.806419f, 0.820213f, 0.828415f, 0.839287f, 0.824165f, 0.864479f, 0.863770f, 0.903656f, 0.863655f, 0.894279f, 0.894775f, 0.914398f, 0.906357f, 0.934486f, 0.918411f, 0.923599f, 0.927643f, 0.938988f, 0.951210f, 0.947090f, 0.936371f, 0.950104f, 0.939949f, 0.967262f, 0.967094f, 0.963509f, 0.950974f, 0.947487f, 0.964249f, 0.975464f, 0.979202f, 0.968735f, 0.970848f, 0.984901f, 0.993584f, 0.988457f, 0.978539f, 0.984863f, 0.986694f, 0.987686f, 0.988327f,
		0.043159f, 0.115349f, 0.185425f, 0.246895f, 0.301567f, 0.363075f, 0.408554f, 0.460640f, 0.496216f, 0.541985f, 0.575050f, 0.627769f, 0.626396f, 0.643623f, 0.706795f, 0.727127f, 0.741158f, 0.776749f, 0.780037f, 0.780907f, 0.823433f, 0.832558f, 0.837196f, 0.873184f, 0.878357f, 0.907158f, 0.891731f, 0.902962f, 0.907890f, 0.919731f, 0.917488f, 0.904564f, 0.938148f, 0.946938f, 0.947891f, 0.944351f, 0.930428f, 0.960114f, 0.954109f, 0.964043f, 0.957581f, 0.974121f, 0.967110f, 0.966339f, 0.972649f, 0.978516f, 0.973984f, 0.978958f, 0.978470f, 0.976807f, 0.980011f, 0.988541f, 0.988998f, 0.985252f, 0.992378f, 0.989326f, 0.994057f, 0.993500f, 0.992233f, 0.997513f, 0.994102f, 0.993172f, 0.991951f, 0.994698f,
		0.051559f, 0.136993f, 0.217186f, 0.290283f, 0.352890f, 0.410606f, 0.464027f, 0.511902f, 0.551926f, 0.609207f, 0.658470f, 0.676750f, 0.696190f, 0.728073f, 0.766769f, 0.786621f, 0.781738f, 0.822853f, 0.826515f, 0.854538f, 0.868111f, 0.889374f, 0.890587f, 0.899826f, 0.894936f, 0.929871f, 0.924667f, 0.925766f, 0.950134f, 0.953720f, 0.950790f, 0.963669f, 0.956429f, 0.958801f, 0.968628f, 0.969856f, 0.971626f, 0.966431f, 0.981903f, 0.980042f, 0.986427f, 0.976883f, 0.980614f, 0.992188f, 0.995102f, 0.987427f, 0.990036f, 0.987900f, 0.984207f, 0.995285f, 0.995262f, 0.994095f, 0.994446f, 0.997803f, 0.991959f, 0.997414f, 0.998039f, 0.995407f, 0.996544f, 0.998749f, 0.998421f, 0.997673f, 0.993515f, 0.998276f,
		0.060028f, 0.157425f, 0.246254f, 0.323997f, 0.392830f, 0.466003f, 0.523483f, 0.579788f, 0.612083f, 0.654938f, 0.696167f, 0.738136f, 0.759682f, 0.806519f, 0.823616f, 0.823822f, 0.859291f, 0.867531f, 0.874245f, 0.888222f, 0.904663f, 0.910927f, 0.927429f, 0.932625f, 0.934631f, 0.947792f, 0.953156f, 0.957588f, 0.963486f, 0.967110f, 0.963135f, 0.970177f, 0.972160f, 0.983017f, 0.973801f, 0.984840f, 0.977005f, 0.984856f, 0.980446f, 0.984871f, 0.986580f, 0.989845f, 0.993851f, 0.990486f, 0.991814f, 0.994095f, 0.992905f, 0.994278f, 0.998276f, 0.992569f, 0.998299f, 0.998459f, 0.999046f, 0.998894f, 0.996979f, 0.998032f, 0.996536f, 0.999657f, 0.999542f, 0.998299f, 1.000000f, 0.998856f, 0.998085f, 1.000000f,
		0.067612f, 0.178909f, 0.279297f, 0.365265f, 0.445526f, 0.513519f, 0.570251f, 0.630669f, 0.675842f, 0.696297f, 0.751038f, 0.777313f, 0.800484f, 0.814667f, 0.850967f, 0.869514f, 0.885628f, 0.888557f, 0.894508f, 0.897949f, 0.944176f, 0.945366f, 0.944557f, 0.956909f, 0.947495f, 0.957817f, 0.968575f, 0.976051f, 0.972198f, 0.966736f, 0.983101f, 0.978447f, 0.987679f, 0.986870f, 0.983665f, 0.985641f, 0.992165f, 0.994568f, 0.992073f, 0.992020f, 0.992538f, 0.994095f, 0.993568f, 0.995605f, 0.993912f, 0.994537f, 0.998459f, 0.996910f, 0.996323f, 0.996185f, 0.994881f, 0.999306f, 0.998672f, 0.998795f, 0.998474f, 1.000000f, 0.997963f, 0.998917f, 0.998825f, 0.998131f, 1.000000f, 0.999641f, 0.998283f, 0.999916f,
		0.075523f, 0.196907f, 0.304184f, 0.402954f, 0.484756f, 0.550385f, 0.614304f, 0.653305f, 0.704628f, 0.741997f, 0.785645f, 0.810913f, 0.837151f, 0.861092f, 0.883461f, 0.902954f, 0.922089f, 0.931786f, 0.929649f, 0.937462f, 0.949234f, 0.953613f, 0.958870f, 0.967575f, 0.969231f, 0.971031f, 0.977249f, 0.979080f, 0.984818f, 0.981041f, 0.988510f, 0.982941f, 0.993843f, 0.993423f, 0.988708f, 0.993393f, 0.993721f, 0.989815f, 0.997269f, 0.994308f, 0.994835f, 0.998421f, 0.998520f, 0.995613f, 0.998856f, 0.997902f, 0.997162f, 1.000000f, 0.999886f, 0.997650f, 0.995758f, 0.997231f, 0.999870f, 0.999611f, 0.998886f, 1.000000f, 1.000000f, 0.999306f, 0.999840f, 0.999802f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.083893f, 0.218582f, 0.339401f, 0.433067f, 0.513351f, 0.586296f, 0.637558f, 0.701775f, 0.758751f, 0.779373f, 0.829193f, 0.852005f, 0.873390f, 0.879280f, 0.915291f, 0.924088f, 0.933380f, 0.944077f, 0.937752f, 0.967621f, 0.956108f, 0.970253f, 0.978577f, 0.972061f, 0.975395f, 0.978745f, 0.985992f, 0.979378f, 0.984177f, 0.986565f, 0.986961f, 0.994408f, 0.991814f, 0.994125f, 0.990974f, 0.998482f, 0.996368f, 0.995483f, 0.998840f, 0.998703f, 0.997612f, 0.998749f, 0.999237f, 0.999069f, 0.996582f, 0.999664f, 0.999733f, 0.999947f, 0.999634f, 0.998520f, 0.999161f, 0.999008f, 1.000000f, 0.999626f, 1.000000f, 0.999992f, 1.000000f, 1.000000f, 0.999954f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.092308f, 0.235756f, 0.361153f, 0.458038f, 0.553986f, 0.618820f, 0.681404f, 0.744438f, 0.788002f, 0.830200f, 0.856842f, 0.866829f, 0.894180f, 0.909943f, 0.937607f, 0.934853f, 0.951210f, 0.956596f, 0.957680f, 0.968117f, 0.969849f, 0.978058f, 0.975967f, 0.981163f, 0.987625f, 0.986732f, 0.990028f, 0.990814f, 0.993111f, 0.994026f, 0.993919f, 0.996422f, 0.992676f, 0.995613f, 0.998428f, 0.998375f, 0.996201f, 0.997986f, 0.998680f, 0.995148f, 0.998497f, 0.998375f, 0.999336f, 0.998276f, 1.000000f, 0.997009f, 0.999931f, 0.999451f, 0.999916f, 1.000000f, 1.000000f, 1.000000f, 0.999527f, 0.999489f, 0.999458f, 0.999664f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999069f, 1.000000f, 0.999931f, 1.000000f,
		0.100403f, 0.251717f, 0.386551f, 0.494720f, 0.576874f, 0.652550f, 0.717285f, 0.760880f, 0.794640f, 0.848831f, 0.862274f, 0.888161f, 0.913094f, 0.928772f, 0.926506f, 0.944870f, 0.962143f, 0.966675f, 0.968651f, 0.972496f, 0.976944f, 0.984314f, 0.979530f, 0.989098f, 0.986214f, 0.988693f, 0.992043f, 0.993332f, 0.991219f, 0.991920f, 0.994682f, 0.997849f, 0.998085f, 0.995010f, 0.999527f, 0.999237f, 0.995590f, 0.996895f, 0.997597f, 0.998405f, 1.000000f, 0.996521f, 0.999916f, 0.999939f, 0.999237f, 0.999702f, 1.000000f, 0.999779f, 0.999641f, 1.000000f, 1.000000f, 0.999588f, 1.000000f, 1.000000f, 0.998657f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999809f, 1.000000f,
		0.108910f, 0.270073f, 0.401291f, 0.516602f, 0.611908f, 0.686852f, 0.741570f, 0.801125f, 0.828384f, 0.862450f, 0.885727f, 0.914024f, 0.921753f, 0.933525f, 0.954437f, 0.963829f, 0.969299f, 0.970062f, 0.972473f, 0.983170f, 0.986092f, 0.987579f, 0.990578f, 0.992264f, 0.997375f, 0.995148f, 0.994690f, 0.995209f, 0.996483f, 0.994431f, 0.995171f, 0.997261f, 0.992523f, 0.997787f, 0.999664f, 0.998795f, 0.998055f, 0.999107f, 0.999626f, 1.000000f, 0.998718f, 1.000000f, 0.999802f, 0.999992f, 1.000000f, 0.999733f, 1.000000f, 1.000000f, 0.999374f, 1.000000f, 0.999901f, 0.999657f, 0.999672f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.116379f, 0.289742f, 0.434738f, 0.544319f, 0.636292f, 0.704086f, 0.775749f, 0.819443f, 0.862495f, 0.880333f, 0.912170f, 0.930916f, 0.939827f, 0.939926f, 0.969421f, 0.967003f, 0.970039f, 0.980370f, 0.979713f, 0.987335f, 0.988701f, 0.991798f, 0.991356f, 0.995628f, 0.991928f, 0.993896f, 0.995262f, 0.995811f, 0.996529f, 0.996834f, 0.997292f, 0.998085f, 0.997940f, 0.996895f, 0.998947f, 0.998672f, 1.000000f, 0.998657f, 1.000000f, 0.999893f, 0.999268f, 1.000000f, 0.999504f, 1.000000f, 1.000000f, 0.999878f, 1.000000f, 0.999908f, 0.999901f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999542f, 1.000000f, 1.000000f, 0.999847f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.124184f, 0.307930f, 0.453300f, 0.571358f, 0.658180f, 0.735107f, 0.783073f, 0.835258f, 0.890739f, 0.901199f, 0.924019f, 0.952148f, 0.941734f, 0.961433f, 0.961937f, 0.975883f, 0.973068f, 0.981674f, 0.982170f, 0.988525f, 0.991249f, 0.989670f, 0.994225f, 0.995537f, 0.995682f, 0.994553f, 0.997116f, 0.997818f, 0.997185f, 0.998962f, 0.999329f, 0.998863f, 0.999138f, 0.999298f, 0.999107f, 0.998711f, 0.999695f, 1.000000f, 0.999275f, 0.999374f, 0.998672f, 0.999222f, 0.999580f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999992f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.131897f, 0.325531f, 0.474838f, 0.594193f, 0.684105f, 0.756439f, 0.811806f, 0.852737f, 0.885254f, 0.912521f, 0.926079f, 0.946075f, 0.958748f, 0.973618f, 0.970711f, 0.979477f, 0.980881f, 0.984039f, 0.992569f, 0.989113f, 0.994568f, 0.996231f, 0.993401f, 0.996964f, 0.997414f, 0.999512f, 0.996246f, 0.997818f, 0.996475f, 0.999901f, 0.998894f, 0.999962f, 1.000000f, 0.999260f, 0.998810f, 0.998993f, 1.000000f, 1.000000f, 0.998962f, 1.000000f, 0.999878f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.138802f, 0.342453f, 0.497086f, 0.612640f, 0.699142f, 0.781097f, 0.830269f, 0.874092f, 0.911591f, 0.924973f, 0.944786f, 0.950035f, 0.967682f, 0.976898f, 0.978294f, 0.979156f, 0.984772f, 0.992920f, 0.991875f, 0.993591f, 0.994789f, 0.995239f, 0.998413f, 0.996101f, 0.994453f, 0.997665f, 0.999481f, 0.998207f, 0.998192f, 0.998344f, 0.999496f, 0.999420f, 0.999779f, 1.000000f, 0.999390f, 1.000000f, 0.999901f, 0.998154f, 1.000000f, 0.999870f, 1.000000f, 1.000000f, 0.999802f, 1.000000f, 0.999718f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.146057f, 0.357552f, 0.514435f, 0.646202f, 0.726608f, 0.796219f, 0.838158f, 0.883415f, 0.902046f, 0.934616f, 0.953339f, 0.954117f, 0.971527f, 0.981773f, 0.983421f, 0.984383f, 0.987785f, 0.987968f, 0.994308f, 0.992142f, 0.992188f, 0.996742f, 0.995972f, 0.998962f, 0.999557f, 0.999008f, 0.998917f, 0.998589f, 0.999290f, 0.998520f, 0.999802f, 0.999344f, 0.999916f, 0.998840f, 0.999565f, 1.000000f, 1.000000f, 1.000000f, 0.999466f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.154175f, 0.375061f, 0.531952f, 0.655937f, 0.745850f, 0.821587f, 0.854744f, 0.899902f, 0.919884f, 0.937317f, 0.957794f, 0.961128f, 0.976089f, 0.984970f, 0.982155f, 0.986389f, 0.990921f, 0.996063f, 0.997536f, 0.995163f, 0.997627f, 0.997665f, 0.997032f, 0.998360f, 0.997452f, 0.998573f, 0.999863f, 0.999153f, 0.998703f, 0.999336f, 0.999397f, 0.998558f, 0.999046f, 0.999496f, 0.999794f, 1.000000f, 1.000000f, 0.999779f, 0.998978f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.161049f, 0.389206f, 0.548698f, 0.685417f, 0.767311f, 0.835144f, 0.874847f, 0.916588f, 0.929390f, 0.954712f, 0.966225f, 0.980965f, 0.981232f, 0.979637f, 0.985573f, 0.990486f, 0.994980f, 0.996101f, 0.996887f, 0.996147f, 0.997635f, 0.998535f, 0.997871f, 0.999275f, 0.998100f, 0.999962f, 0.999397f, 1.000000f, 1.000000f, 1.000000f, 0.999374f, 1.000000f, 0.998657f, 0.999962f, 0.999962f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999809f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.170219f, 0.404900f, 0.569893f, 0.692253f, 0.783989f, 0.854202f, 0.883682f, 0.922493f, 0.950554f, 0.952759f, 0.965172f, 0.977585f, 0.981270f, 0.987717f, 0.991150f, 0.993927f, 0.991531f, 0.996193f, 0.992485f, 0.994942f, 0.996277f, 0.998131f, 0.999123f, 0.998474f, 0.998230f, 0.998543f, 0.999771f, 0.999451f, 0.999916f, 0.999901f, 0.999443f, 1.000000f, 0.999512f, 1.000000f, 0.999863f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.175995f, 0.417778f, 0.587364f, 0.704094f, 0.794579f, 0.848862f, 0.906929f, 0.925674f, 0.950180f, 0.960602f, 0.969231f, 0.975098f, 0.984589f, 0.989143f, 0.993950f, 0.993095f, 0.994751f, 0.996964f, 0.995728f, 0.997673f, 0.997772f, 0.996094f, 0.998779f, 0.998352f, 0.998810f, 1.000000f, 0.999397f, 0.999718f, 0.999466f, 0.999428f, 0.999542f, 1.000000f, 0.999413f, 1.000000f, 0.999809f, 1.000000f, 1.000000f, 0.999870f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.183105f, 0.431953f, 0.608543f, 0.722977f, 0.815285f, 0.879860f, 0.906868f, 0.940750f, 0.962158f, 0.968269f, 0.977051f, 0.986664f, 0.987892f, 0.992203f, 0.997589f, 0.994934f, 0.998116f, 0.998703f, 0.996826f, 0.999458f, 0.997658f, 0.999382f, 0.998772f, 0.997490f, 0.999878f, 1.000000f, 0.999550f, 1.000000f, 0.999062f, 1.000000f, 1.000000f, 1.000000f, 0.999657f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.189987f, 0.440308f, 0.616745f, 0.747749f, 0.818283f, 0.876892f, 0.920494f, 0.950851f, 0.962852f, 0.966843f, 0.973465f, 0.990219f, 0.986603f, 0.993317f, 0.995186f, 0.996986f, 0.996811f, 0.998154f, 0.997940f, 0.998734f, 0.998146f, 0.998367f, 0.999504f, 0.999908f, 0.999779f, 0.997971f, 0.998894f, 0.999939f, 1.000000f, 1.000000f, 0.999893f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999817f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.198090f, 0.458580f, 0.638443f, 0.752754f, 0.839699f, 0.888031f, 0.927704f, 0.946678f, 0.959824f, 0.978127f, 0.979744f, 0.987892f, 0.987404f, 0.995247f, 0.992355f, 0.995598f, 0.997238f, 0.998955f, 0.997292f, 0.999100f, 0.999382f, 0.999924f, 0.999161f, 0.999992f, 0.999466f, 1.000000f, 0.999870f, 1.000000f, 1.000000f, 1.000000f, 0.999863f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.204880f, 0.471451f, 0.648201f, 0.774055f, 0.847694f, 0.896393f, 0.931526f, 0.954346f, 0.968887f, 0.976952f, 0.983536f, 0.989174f, 0.995110f, 0.995262f, 0.995323f, 0.997826f, 0.997688f, 0.997101f, 0.998741f, 0.999519f, 0.999809f, 0.999985f, 0.999489f, 1.000000f, 1.000000f, 0.999626f, 0.999977f, 1.000000f, 1.000000f, 0.999924f, 0.999374f, 1.000000f, 1.000000f, 0.999901f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.211052f, 0.481041f, 0.659256f, 0.774529f, 0.850517f, 0.902763f, 0.947754f, 0.962341f, 0.973381f, 0.981850f, 0.984558f, 0.990578f, 0.992073f, 0.991371f, 0.996262f, 0.998383f, 0.995682f, 0.999008f, 0.999786f, 0.998871f, 0.999123f, 0.999809f, 0.999931f, 0.999832f, 1.000000f, 0.999466f, 0.999947f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999954f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.220139f, 0.503319f, 0.671234f, 0.787834f, 0.860901f, 0.918205f, 0.943840f, 0.962631f, 0.979546f, 0.988197f, 0.988914f, 0.992943f, 0.995140f, 0.996895f, 0.997475f, 0.999001f, 0.998734f, 0.999107f, 0.998596f, 0.999985f, 0.999710f, 0.999886f, 0.999573f, 1.000000f, 1.000000f, 1.000000f, 0.999725f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.225166f, 0.509087f, 0.683983f, 0.804581f, 0.873917f, 0.912231f, 0.947807f, 0.968803f, 0.978722f, 0.985580f, 0.992889f, 0.996399f, 0.994324f, 0.998238f, 0.997505f, 0.997810f, 0.999580f, 0.999031f, 0.999031f, 1.000000f, 0.999405f, 0.999565f, 0.999954f, 1.000000f, 0.999962f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.230301f, 0.524612f, 0.698334f, 0.822609f, 0.886528f, 0.937378f, 0.956963f, 0.971664f, 0.981796f, 0.988419f, 0.992538f, 0.995132f, 0.994629f, 0.998329f, 0.999405f, 0.999649f, 0.999252f, 0.999107f, 0.999504f, 0.999763f, 0.999802f, 0.999733f, 0.999535f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999992f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999954f, 1.000000f, 1.000000f, 0.999969f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.238495f, 0.530701f, 0.707237f, 0.817604f, 0.888786f, 0.931213f, 0.962540f, 0.976921f, 0.982521f, 0.982918f, 0.990417f, 0.996971f, 0.997513f, 0.997192f, 0.998779f, 0.999001f, 0.999176f, 0.999374f, 0.999413f, 1.000000f, 0.998802f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999992f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.245865f, 0.542244f, 0.719765f, 0.835083f, 0.905663f, 0.943886f, 0.955551f, 0.968971f, 0.987480f, 0.991135f, 0.992340f, 0.997025f, 0.996613f, 0.998848f, 0.999878f, 0.999641f, 0.998367f, 0.999413f, 1.000000f, 0.999435f, 0.999901f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.250404f, 0.554688f, 0.736572f, 0.845070f, 0.908997f, 0.945251f, 0.969360f, 0.978600f, 0.987839f, 0.993118f, 0.991402f, 0.995766f, 0.995193f, 0.998695f, 0.996956f, 0.999435f, 0.998276f, 0.999557f, 1.000000f, 1.000000f, 1.000000f, 0.999527f, 0.999947f, 0.999733f, 0.999512f, 0.999924f, 1.000000f, 1.000000f, 0.999908f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.257126f, 0.566643f, 0.750778f, 0.854897f, 0.914001f, 0.953949f, 0.969147f, 0.984016f, 0.991066f, 0.991371f, 0.995728f, 0.995651f, 0.998466f, 0.999046f, 0.999199f, 0.998924f, 0.999657f, 1.000000f, 0.999847f, 0.999718f, 0.999863f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999924f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.266174f, 0.571640f, 0.755798f, 0.860291f, 0.915726f, 0.956024f, 0.974861f, 0.982788f, 0.987846f, 0.995201f, 0.996902f, 0.996552f, 0.998238f, 0.998566f, 0.999435f, 0.999893f, 0.998993f, 0.997871f, 0.999435f, 1.000000f, 1.000000f, 0.999130f, 1.000000f, 0.999939f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.269890f, 0.594246f, 0.772141f, 0.867409f, 0.922417f, 0.958870f, 0.971848f, 0.984039f, 0.991684f, 0.994217f, 0.995880f, 0.998108f, 0.998093f, 0.998947f, 0.999718f, 0.999611f, 0.998413f, 1.000000f, 1.000000f, 0.999840f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.278290f, 0.597427f, 0.779610f, 0.880524f, 0.927666f, 0.957581f, 0.975914f, 0.985031f, 0.989151f, 0.993179f, 0.999023f, 0.999153f, 0.998840f, 0.997696f, 0.999672f, 1.000000f, 0.999443f, 0.999702f, 0.999924f, 1.000000f, 1.000000f, 1.000000f, 0.999825f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.284424f, 0.607811f, 0.780205f, 0.878990f, 0.933708f, 0.958366f, 0.982422f, 0.990944f, 0.995102f, 0.995644f, 0.997368f, 0.998718f, 0.999268f, 0.999268f, 0.999992f, 0.999901f, 0.999962f, 1.000000f, 1.000000f, 0.999863f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.290627f, 0.613869f, 0.785736f, 0.891724f, 0.937843f, 0.969086f, 0.979057f, 0.990097f, 0.994537f, 0.994766f, 0.996559f, 0.998047f, 0.998680f, 0.999443f, 0.999252f, 0.999977f, 1.000000f, 0.999466f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999794f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.297386f, 0.629547f, 0.803345f, 0.891632f, 0.949585f, 0.971718f, 0.984802f, 0.989922f, 0.995056f, 0.998642f, 0.998802f, 0.999039f, 0.999535f, 0.999481f, 1.000000f, 0.999619f, 0.999481f, 0.999924f, 1.000000f, 0.999840f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.303810f, 0.631767f, 0.804535f, 0.902519f, 0.948914f, 0.969879f, 0.988220f, 0.991173f, 0.997070f, 0.996880f, 0.997566f, 0.999237f, 0.999451f, 0.999138f, 1.000000f, 0.999733f, 0.999611f, 1.000000f, 1.000000f, 0.999939f, 0.999878f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999939f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.312042f, 0.638069f, 0.817337f, 0.912064f, 0.948082f, 0.976067f, 0.986984f, 0.991264f, 0.997993f, 0.998634f, 0.998901f, 0.999413f, 0.999786f, 0.999802f, 0.999916f, 1.000000f, 0.999939f, 0.999672f, 1.000000f, 0.999969f, 0.999908f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999985f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.317795f, 0.651794f, 0.820450f, 0.910469f, 0.957016f, 0.979805f, 0.990822f, 0.991936f, 0.995750f, 0.997276f, 0.998398f, 0.999001f, 0.999855f, 0.999199f, 0.999992f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.321625f, 0.660835f, 0.828522f, 0.910446f, 0.960503f, 0.975853f, 0.987556f, 0.993469f, 0.996925f, 0.998535f, 0.998871f, 0.999443f, 0.999527f, 0.999268f, 0.999954f, 0.999924f, 1.000000f, 0.999969f, 0.999802f, 0.999725f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.324356f, 0.663864f, 0.845032f, 0.921822f, 0.960854f, 0.980316f, 0.989243f, 0.993690f, 0.998207f, 0.998024f, 0.997673f, 0.998962f, 0.999588f, 1.000000f, 0.999870f, 1.000000f, 0.999649f, 0.999878f, 1.000000f, 1.000000f, 1.000000f, 0.999931f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.330910f, 0.676758f, 0.841492f, 0.921738f, 0.962257f, 0.980797f, 0.990211f, 0.994637f, 0.997414f, 0.999016f, 0.998611f, 0.999596f, 0.999596f, 0.999794f, 0.999954f, 1.000000f, 0.999786f, 1.000000f, 1.000000f, 1.000000f, 0.999596f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.340332f, 0.687988f, 0.851479f, 0.919609f, 0.966499f, 0.984642f, 0.986450f, 0.996460f, 0.997650f, 0.997688f, 0.999130f, 1.000000f, 0.999603f, 1.000000f, 0.999710f, 0.999611f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.348244f, 0.686165f, 0.859581f, 0.931076f, 0.967926f, 0.983055f, 0.993011f, 0.993179f, 0.998367f, 0.998245f, 0.998032f, 0.998596f, 0.999916f, 1.000000f, 0.999962f, 0.999252f, 0.999870f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.350342f, 0.698196f, 0.854889f, 0.928741f, 0.965134f, 0.986572f, 0.994316f, 0.996613f, 0.996811f, 0.997177f, 0.999741f, 0.999481f, 0.999802f, 0.999901f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999954f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.356522f, 0.709854f, 0.865700f, 0.937141f, 0.976524f, 0.983276f, 0.993027f, 0.996117f, 0.999123f, 0.998611f, 0.999359f, 0.999947f, 0.999680f, 0.999969f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.360924f, 0.714699f, 0.870094f, 0.942703f, 0.973259f, 0.986771f, 0.993820f, 0.997528f, 0.998154f, 0.999390f, 0.999428f, 0.999550f, 0.999924f, 0.999939f, 0.999695f, 1.000000f, 0.999908f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.369286f, 0.722252f, 0.877327f, 0.944992f, 0.977333f, 0.990089f, 0.995255f, 0.997139f, 0.998589f, 0.999512f, 0.998917f, 0.999886f, 1.000000f, 0.999901f, 1.000000f, 0.999886f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.374062f, 0.725288f, 0.884613f, 0.952583f, 0.975601f, 0.989159f, 0.996315f, 0.998009f, 0.998070f, 0.999146f, 0.999863f, 0.999985f, 0.999855f, 0.999939f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.379021f, 0.733742f, 0.890366f, 0.949318f, 0.983437f, 0.993240f, 0.993996f, 0.997612f, 0.999283f, 0.999596f, 0.999496f, 0.999977f, 0.999809f, 0.999924f, 0.999931f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.382401f, 0.740753f, 0.896332f, 0.950905f, 0.975594f, 0.990028f, 0.995827f, 0.998184f, 0.999268f, 0.999916f, 0.999756f, 0.999832f, 0.999939f, 1.000000f, 1.000000f, 0.999893f, 1.000000f, 1.000000f, 0.999901f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.393326f, 0.748840f, 0.893021f, 0.960419f, 0.982246f, 0.990952f, 0.997612f, 0.998741f, 0.999229f, 0.999809f, 0.999718f, 0.999817f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.391319f, 0.747688f, 0.894875f, 0.959045f, 0.983551f, 0.993164f, 0.996681f, 0.998161f, 0.999359f, 0.999626f, 0.998711f, 0.999969f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.397568f, 0.760132f, 0.901192f, 0.959579f, 0.985405f, 0.992844f, 0.996971f, 0.999138f, 0.998741f, 0.999344f, 0.999733f, 0.999954f, 0.999672f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999855f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.405067f, 0.760406f, 0.905258f, 0.962410f, 0.987137f, 0.993011f, 0.997681f, 0.999596f, 0.998993f, 0.999146f, 0.999565f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.408173f, 0.773369f, 0.905670f, 0.963364f, 0.987007f, 0.994469f, 0.997124f, 0.998253f, 0.999489f, 0.999954f, 0.999748f, 0.999748f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.418167f, 0.771477f, 0.912422f, 0.965050f, 0.988014f, 0.995186f, 0.998009f, 0.999237f, 0.999825f, 0.999664f, 1.000000f, 0.999962f, 0.999817f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 0.999481f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.419846f, 0.781548f, 0.921494f, 0.965538f, 0.985680f, 0.996445f, 0.997757f, 0.999626f, 0.999504f, 0.999771f, 1.000000f, 0.999985f, 1.000000f, 1.000000f, 0.999771f, 1.000000f, 1.000000f, 0.999718f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
		0.426842f, 0.782837f, 0.919655f, 0.968567f, 0.988304f, 0.995544f, 0.998573f, 0.998650f, 0.999329f, 0.999252f, 1.000000f, 0.999916f, 0.999992f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f, 1.000000f,
	};

	static uint32 ToLinearIndex(uint32 HairCountIndex, uint32 HairRadiusIndex)
	{
		check(HairCountIndex < FHairCountToCoverageData::HairCount);
		check(HairRadiusIndex < FHairCountToCoverageData::HairRadiusCount);
		return HairRadiusIndex + HairCountIndex * FHairCountToCoverageData::HairRadiusCount;
	}
}; // FHairCountToCoverageData

static FHairCountToCoverageData g_CPUHairLUT;
float GetHairCoverage(uint32 InCount, float InAvgRadius)
{
	const float AvgWidth = FMath::Clamp(InAvgRadius, 0.f, 1.f);

	const float RadiusIndex = FMath::FloorToFloat(AvgWidth * FHairCountToCoverageData::HairRadiusCount);
	const float S = FMath::Clamp(AvgWidth * FHairCountToCoverageData::HairRadiusCount - RadiusIndex, 0.f, 1.f);

	const uint32 CountIndex   = FMath::Clamp(InCount,				 0u, FHairCountToCoverageData::HairCount - 1u);
	const uint32 RadiusIndex0 = FMath::Clamp(uint32(RadiusIndex),    0u, FHairCountToCoverageData::HairRadiusCount - 1u);
	const uint32 RadiusIndex1 = FMath::Clamp(uint32(RadiusIndex)+1u, 0u, FHairCountToCoverageData::HairRadiusCount - 1u);

	const uint32 Index0 = FHairCountToCoverageData::ToLinearIndex(CountIndex, RadiusIndex0);
	const uint32 Index1 = FHairCountToCoverageData::ToLinearIndex(CountIndex, RadiusIndex1);

	return FMath::Lerp(float(g_CPUHairLUT.Data[Index0]), float(g_CPUHairLUT.Data[Index1]), S);
}

float GetHairAvgRadius(uint32 InCount, float InCoverage)
{
	InCoverage = FMath::Clamp(InCoverage, 0.f, 1.f);

	const uint32 CountIndex = FMath::Clamp(InCount, 0u, FHairCountToCoverageData::HairCount - 1u);

	const uint32 Offset = CountIndex * FHairCountToCoverageData::HairCount;
	uint32 RadiusIndex = 0;
	while (
		RadiusIndex + 1 < FHairCountToCoverageData::HairRadiusCount && 
		!(InCoverage >= g_CPUHairLUT.Data[Offset + RadiusIndex] &&
		  InCoverage <= g_CPUHairLUT.Data[Offset + RadiusIndex+1]))
	{
		++RadiusIndex;
	}

	const uint32 RadiusIndex0 = FMath::Clamp(RadiusIndex,   0u, FHairCountToCoverageData::HairRadiusCount - 1u);
	const uint32 RadiusIndex1 = FMath::Clamp(RadiusIndex+1, 0u, FHairCountToCoverageData::HairRadiusCount - 1u);

	const float Coverage0 = g_CPUHairLUT.Data[Offset + RadiusIndex0];
	const float Coverage1 = g_CPUHairLUT.Data[Offset + RadiusIndex1];
	const float Delta = FMath::Abs(Coverage1 - Coverage0);

	const float S =  FMath::Clamp((InCoverage - Coverage0) / FMath::Max(0.001f, Delta), 0.f, 1.f);

	return FMath::Lerp(
		RadiusIndex0 / float(FHairCountToCoverageData::HairRadiusCount - 1u), 
		RadiusIndex1 / float(FHairCountToCoverageData::HairRadiusCount - 1u), 
		S);
}

/////////////////////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
class FHairCoverageLUTCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairCoverageLUTCS);
	SHADER_USE_PARAMETER_STRUCT(FHairCoverageLUTCS, FGlobalShader); 

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InputBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FUploadParameters, )
		RDG_BUFFER_ACCESS(UploadBuffer, ERHIAccess::CopyDest)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return !IsHairMobilePlatform(Parameters.Platform) && IsHairStrandsSupported(EHairStrandsShaderType::Cards, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_HAIRCOVERAGE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairCoverageLUTCS, "/Engine/Private/HairStrands/HairStrandsLUT.usf", "MainCS", SF_Compute);

static FRDGTextureRef GenerateHairCoverageLUTPass(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	// Create buffer with upload data
	FIntPoint OutputResolution(FHairCountToCoverageData::HairCount, FHairCountToCoverageData::HairRadiusCount);
	FRDGBufferRef InputBuffer = nullptr;

	const uint32 ElementCount = FHairCountToCoverageData::HairCount * FHairCountToCoverageData::HairRadiusCount;
	const uint32 SizeInBytes = ElementCount * sizeof(float);
	FRDGBufferRef UploadBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(float), ElementCount), TEXT("HairCoverageData"));
	FHairCoverageLUTCS::FUploadParameters* UploadParameters = GraphBuilder.AllocParameters<FHairCoverageLUTCS::FUploadParameters>();
	UploadParameters->UploadBuffer = UploadBuffer;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("UploadHairCoverageBuffer"),
		UploadParameters,
		ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
		[UploadParameters, SizeInBytes](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FHairCountToCoverageData Source;
		
		void* Dest = RHICmdList.LockBuffer(UploadParameters->UploadBuffer->GetRHI(), 0, SizeInBytes, RLM_WriteOnly);
		FPlatformMemory::Memcpy(Dest, Source.Data, SizeInBytes);
		RHICmdList.UnlockBuffer(UploadParameters->UploadBuffer->GetRHI());
	});


	FRDGTextureDesc OutputDesc;
	OutputDesc.Dimension = ETextureDimension::Texture2D;
	OutputDesc.Extent.X = OutputResolution.X;
	OutputDesc.Extent.Y = OutputResolution.Y;
	OutputDesc.Format = PF_R32_FLOAT;
	OutputDesc.NumMips = 1;
	OutputDesc.Flags = TexCreate_UAV | TexCreate_ShaderResource;
	FRDGTextureRef HairLUTTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("Hair.CoverageLUT"));

	FHairCoverageLUTCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairCoverageLUTCS::FParameters>();
	Parameters->OutputResolution = OutputResolution;
	Parameters->InputBuffer = GraphBuilder.CreateSRV(UploadBuffer, PF_R32_FLOAT);
	Parameters->OutputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(HairLUTTexture, 0));

	TShaderMapRef<FHairCoverageLUTCS> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrandsCoverageLUT"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ComputeShader,
		Parameters,
		FComputeShaderUtils::GetGroupCount(OutputResolution, FIntPoint(8,8)));

	return HairLUTTexture;
}
#endif // WITH_EDITOR

// Returns the RHI texture from a UVolumeTexture if it's loaded and has a valid resource.
static FRHITexture* GetVolumeTextureRHI(UVolumeTexture* Texture)
{
	if (Texture)
	{
		if (const FTextureResource* Resource = Texture->GetResource())
		{
			if (Resource->TextureRHI)
			{
				return Resource->TextureRHI;
			}
		}
	}
	return nullptr;
}

// Returns the RHI texture from a UTexture2D if it's loaded and has a valid resource.
static FRHITexture* GetTexture2DRHI(UTexture2D* Texture)
{
	if (Texture)
	{
		if (const FTextureResource* Resource = Texture->GetResource())
		{
			if (Resource->TextureRHI)
			{
				return Resource->TextureRHI;
			}
		}
	}
	return nullptr;
}

static FRDGTextureRef InternalGetHairLUT(FRDGBuilder& GraphBuilder, const FViewInfo& View, FHairLUTType Type, bool bRegister)
{
	FRDGTextureRef Out = nullptr;

	const EShaderPlatform Platform = View.GetShaderPlatform();
	const bool bIsCompatible = !IsHairMobilePlatform(Platform) && IsHairStrandsSupported(EHairStrandsShaderType::Cards, Platform);
	if (!bIsCompatible)
	{
		return Out;
	}

	#if WITH_EDITOR
	static bool bForceGPUGeneration = false;
	if (bForceGPUGeneration)
	{
		FRDGTextureRef HairLUT0 = GenerateHairLUTPass(GraphBuilder, View, HairLUTType_DualScattering);
		FRDGTextureRef HairLUT1 = GenerateHairLUTPass(GraphBuilder, View, HairLUTType_MeanEnergy);
		FRDGTextureRef HairLUT2 = GenerateHairCoverageLUTPass(GraphBuilder, View);

		GSystemTextures.HairLUT0 = ConvertToExternalAccessTexture(GraphBuilder, HairLUT0);
		GSystemTextures.HairLUT1 = ConvertToExternalAccessTexture(GraphBuilder, HairLUT1);
		GSystemTextures.HairLUT2 = ConvertToExternalAccessTexture(GraphBuilder, HairLUT2);
	}
	#endif

	if (Type == FHairLUTType::HairLUTType_DualScattering)
	{
		if (GEngine)
		{
			if (FRHITexture* AssetRHI = GetVolumeTextureRHI(GEngine->HairDualScatteringTexture))
			{
				if (GSystemTextures.HairLUT0 == nullptr || GSystemTextures.HairLUT0->GetRHI() != AssetRHI)
				{
					GSystemTextures.HairLUT0 = CreateRenderTarget(AssetRHI, TEXT("Hair.LUT(DualScattering)"));
				}
			}
		}
		if (bRegister)
		{
			Out = GSystemTextures.HairLUT0 ? GraphBuilder.RegisterExternalTexture(GSystemTextures.HairLUT0, ERDGTextureFlags::SkipTracking) : GSystemTextures.GetBlackDummy(GraphBuilder);
		}
		
		return Out;
	}
	else if (Type == FHairLUTType::HairLUTType_MeanEnergy)
	{
		if (GEngine)
		{
			if (FRHITexture* AssetRHI = GetVolumeTextureRHI(GEngine->HairDirectionalAlbedoTexture))
			{
				if (GSystemTextures.HairLUT1 == nullptr || GSystemTextures.HairLUT1->GetRHI() != AssetRHI)
				{
					GSystemTextures.HairLUT1 = CreateRenderTarget(AssetRHI, TEXT("Hair.LUT(DirectionalAlbedo)"));
				}
			}
		}
		if (bRegister)
		{
			Out = GSystemTextures.HairLUT1 ? GraphBuilder.RegisterExternalTexture(GSystemTextures.HairLUT1, ERDGTextureFlags::SkipTracking) : GSystemTextures.GetBlackDummy(GraphBuilder);
		}

		return Out;
	}
	else if (Type == FHairLUTType::HairLUTType_Coverage)
	{
		if (GEngine)
		{
			if (FRHITexture* AssetRHI = GetTexture2DRHI(GEngine->HairCoverageTexture))
			{
				if (GSystemTextures.HairLUT2 == nullptr || GSystemTextures.HairLUT2->GetRHI() != AssetRHI)
				{
					GSystemTextures.HairLUT2 = CreateRenderTarget(AssetRHI, TEXT("Hair.LUT(Coverage)"));
				}
			}
		}
		if (bRegister)
		{
			Out = GSystemTextures.HairLUT2 ? GraphBuilder.RegisterExternalTexture(GSystemTextures.HairLUT2, ERDGTextureFlags::SkipTracking) : GSystemTextures.GetBlackDummy(GraphBuilder);
		}

		return Out;
	}

	return Out;
}

FRDGTextureRef GetHairLUT(FRDGBuilder& GraphBuilder, const FViewInfo& View, FHairLUTType Type)
{
	return InternalGetHairLUT(GraphBuilder, View, Type, true);
}

void UpdateHairResources(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	InternalGetHairLUT(GraphBuilder, View, FHairLUTType::HairLUTType_DualScattering, false);
}