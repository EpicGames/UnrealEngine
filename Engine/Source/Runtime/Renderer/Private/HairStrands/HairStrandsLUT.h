// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairRendering.h: Hair strands LUT generation.
=============================================================================*/

#pragma once

#include "RenderGraphFwd.h"

class FViewInfo;

enum FHairLUTType
{
	HairLUTType_DualScattering,
	HairLUTType_MeanEnergy,
	HairLUTType_Coverage,
	HairLUTTypeCount
};

/// Returns Hair LUTs. LUTs are generated on demand.
FRDGTextureRef GetHairLUT(FRDGBuilder& GraphBuilder, const FViewInfo& View, FHairLUTType Type);
