// Copyright Epic Games, Inc. All Rights Reserved.


#include "HairStrandsMacroGroup.h"
#include "HairStrandsUtils.h"
#include "HairStrandsData.h"
#include "SceneRendering.h"
#include "PrimitiveDrawingUtils.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "ScenePrivate.h"

static int32 GHairVirtualVoxel_NumPixelPerVoxel = 1;
static FAutoConsoleVariableRef CVarHairVirtualVoxel_NumPixelPerVoxel(TEXT("r.HairStrands.Voxelization.VoxelSizeInPixel"), GHairVirtualVoxel_NumPixelPerVoxel, TEXT("Target size of voxel size in pixels"), ECVF_RenderThreadSafe);

DECLARE_GPU_STAT(HairStrandsAABB);

class FHairMacroGroupAABBCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairMacroGroupAABBCS);
	SHADER_USE_PARAMETER_STRUCT(FHairMacroGroupAABBCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, PassCount)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(uint32, MacroGroupCount)

		SHADER_PARAMETER(float, PixelSizeAtDepth1)
		SHADER_PARAMETER(float, NumPixelPerVoxel)
		SHADER_PARAMETER(uint32, VoxelPageResolution)
		SHADER_PARAMETER(FVector3f, TranslatedWorldOffsetCorrection)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, RegisteredIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, InGroupAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutMacroGroupAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutMacroGroupVoxelSizeBuffer)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 32; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_AABBUPDATE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairMacroGroupAABBCS, "/Engine/Private/HairStrands/HairStrandsAABB.usf", "Main", SF_Compute);

void GetVoxelPageResolution(uint32& OutPageResolution, uint32& OutPageResolutionLog2);

static void AddHairMacroGroupAABBPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, 
	FHairTransientResources& TransientResources,
	FHairStrandsMacroGroupData& MacroGroup,
	FRDGBufferUAVRef& OutHairMacroGroupAABBBufferUAV,
	FRDGBufferUAVRef& OutMacroGroupVoxelSizeBufferUAV)
{
	const uint32 PrimitiveCount = MacroGroup.PrimitivesInfos.Num();
	if (PrimitiveCount == 0)
		return;
	
	// Compute the average pixel size at a distance of 1 units
	const FIntPoint Resolution(View.ViewRect.Width(), View.ViewRect.Height());
	const float vFOV = FMath::DegreesToRadians(View.FOV);
	const float PixelSizeAtDepth1 = (View.IsPerspectiveProjection() ? FMath::Tan(vFOV * 0.5f) / (0.5f * Resolution.Y) : FMath::Clamp(Resolution.X / View.ViewMatrices.GetOrthoDimensions().X, 0.0f, 1.0f));
	const uint32 MacroGroupCount = MacroGroup.PrimitivesInfos.Num();
	
	uint32 VoxelPageResolution = 0;
	uint32 VoxelPageResolutionLog2 = 1;
	GetVoxelPageResolution(VoxelPageResolution, VoxelPageResolutionLog2);

	TArray<uint32> RegisteredIndices;
	RegisteredIndices.Reserve(MacroGroupCount);
	for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : MacroGroup.PrimitivesInfos)
	{
		RegisteredIndices.Add(PrimitiveInfo.PublicDataPtr->Instance->RegisteredIndex);
	}
	FRDGBufferRef RegisteredIndexBuffer = CreateVertexBuffer(GraphBuilder, TEXT("Hair.RegisteredIndexBuffer"), FRDGBufferDesc::CreateBufferDesc(4, RegisteredIndices.Num()), RegisteredIndices.GetData(), 4u * RegisteredIndices.Num());

	const float NumPixelPerVoxel = FMath::Clamp(GHairVirtualVoxel_NumPixelPerVoxel, 1.f, 50.f);

	FHairMacroGroupAABBCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairMacroGroupAABBCS::FParameters>();
	Parameters->MacroGroupId 					= MacroGroup.MacroGroupId;
	Parameters->RegisteredIndexBuffer			= GraphBuilder.CreateSRV(RegisteredIndexBuffer, PF_R32_UINT);
	Parameters->InGroupAABBBuffer				= TransientResources.GroupAABBSRV;
	Parameters->OutMacroGroupAABBBuffer 		= OutHairMacroGroupAABBBufferUAV;
	Parameters->OutMacroGroupVoxelSizeBuffer 	= OutMacroGroupVoxelSizeBufferUAV;
	Parameters->PixelSizeAtDepth1   			= PixelSizeAtDepth1;
	Parameters->NumPixelPerVoxel    			= NumPixelPerVoxel;
	Parameters->VoxelPageResolution 			= VoxelPageResolution;
	Parameters->View			    			= View.ViewUniformBuffer;
	Parameters->MacroGroupCount					= MacroGroupCount;
	Parameters->PassCount						= FMath::DivideAndRoundUp(MacroGroupCount, FHairMacroGroupAABBCS::GetGroupSize());
	Parameters->TranslatedWorldOffsetCorrection = TransientResources.GetTranslatedWorldOffsetCorrection(View);

	TShaderMapRef<FHairMacroGroupAABBCS> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::MacroGroupAABBUpdate"),
		ComputeShader,
		Parameters,
		FIntVector(1, 1, 1));
}

static bool DoesGroupExists(uint32 ResourceId, uint32 GroupIndex, const FHairStrandsMacroGroupData::TPrimitiveInfos& PrimitivesGroups)
{
	// Simple linear search as the expected number of groups is supposed to be low (<16, see FHairStrandsMacroGroupData::MaxMacroGroupCount)
	for (const FHairStrandsMacroGroupData::PrimitiveInfo& Group : PrimitivesGroups)
	{
		if (Group.GroupIndex == GroupIndex && Group.ResourceId == ResourceId)
		{
			return true;
		}
	}
	return false;
}

bool IsHairStrandsNonVisibleShadowCastingEnable();

static void InternalUpdateMacroGroup(FHairStrandsMacroGroupData& MacroGroup, int32& MaterialId, FHairGroupPublicData* HairData, const FMeshBatch* Mesh, const FPrimitiveSceneProxy* Proxy)
{
	check(HairData);

	// Track if any instance in the current group needs some features (scatter light scene, holdout, ...)
	MacroGroup.Flags |= HairData->VFInput.Strands.Common.Flags;

	FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo = MacroGroup.PrimitivesInfos.AddZeroed_GetRef();
	PrimitiveInfo.Mesh = Mesh;
	PrimitiveInfo.PrimitiveSceneProxy = Proxy;
	PrimitiveInfo.MaterialId = MaterialId++;
	PrimitiveInfo.ResourceId = Mesh ? reinterpret_cast<uint64>(Mesh->Elements[0].UserData) : ~0u;
	PrimitiveInfo.GroupIndex = HairData->GetGroupIndex();
	PrimitiveInfo.Flags = HairData->VFInput.Strands.Common.Flags;
	PrimitiveInfo.PublicDataPtr = HairData;

	if (HairData->DoesSupportVoxelization())
	{
		MacroGroup.bSupportVoxelization = true;
	}
}

void CreateHairStrandsMacroGroups(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View, 
	const FHairInstanceCullingResults& CullingResults,
	FHairStrandsViewData& OutHairStrandsViewData,
	bool bBuildGPUAABB)
{
	const bool bHasHairStrandsElements = View.HairStrandsMeshElements.Num() != 0 || Scene->HairStrandsSceneData.RegisteredProxies.Num() != 0;
	if (!View.Family || !bHasHairStrandsElements || View.bIsReflectionCapture)
	{
		return;
	}

	TArray<FHairStrandsMacroGroupData, SceneRenderingAllocator>& MacroGroups = OutHairStrandsViewData.MacroGroupDatas;

	int32 MaterialId = 0;

	// Aggregate all hair primitives within the same area into macro groups, for allocating/rendering DOM/voxel
	uint32 MacroGroupId = 0;
	auto UpdateMacroGroup = [&MacroGroups, &MacroGroupId, &MaterialId](FHairGroupPublicData* HairData, const FMeshBatch* Mesh,  const FPrimitiveSceneProxy* Proxy, const FBoxSphereBounds& Bounds)
	{
		check(HairData);

		// Ensure that the element has been initialized
		const bool bIsValid = HairData->VFInput.Strands.PositionBufferRHISRV != nullptr;
		if (!bIsValid)
			return;

		const FBoxSphereBounds& OriginalPrimitiveBounds = Proxy ? Proxy->GetBounds() : Bounds;

		// Expand hair bound by (half) the groom's max hair-length, to be sure that the bounds are large enough. 
		// This is important as the primary visibility memory allocation is based on the screen-projection of this CPU bound.
		// If the bound is too small, the allocation won't be enough, resulting in tile artifacts.
		FBoxSphereBounds PrimitiveBounds = OriginalPrimitiveBounds.ExpandBy(HairData->VFInput.Strands.Common.Length * 0.5f);

		bool bFound = false;
		float MinDistance = FLT_MAX;
		uint32 ClosestMacroGroupId = ~0u;
		for (FHairStrandsMacroGroupData& MacroGroup : MacroGroups)
		{
			const FSphere MacroSphere = MacroGroup.Bounds.GetSphere();
			const FSphere PrimSphere = PrimitiveBounds.GetSphere();

			const float DistCenters = (MacroSphere.Center - PrimSphere.Center).Size();
			const float AccumRadius = FMath::Max(0.f, MacroSphere.W + PrimSphere.W);
			const bool bIntersect = DistCenters <= AccumRadius;
			
			if (bIntersect)
			{
				MacroGroup.Bounds = Union(MacroGroup.Bounds, PrimitiveBounds);

				InternalUpdateMacroGroup(MacroGroup, MaterialId, HairData, Mesh, Proxy);
				bFound = true;
				break;
			}

			const float MacroToPrimDistance = DistCenters - AccumRadius;
			if (MacroToPrimDistance < MinDistance)
			{
				MinDistance = MacroToPrimDistance;
				ClosestMacroGroupId = MacroGroup.MacroGroupId;
			}
		}

		if (!bFound)
		{
			// If we have reached the max number of macro group (MAX_HAIR_MACROGROUP_COUNT), then merge the current one with the closest one.
			if (MacroGroups.Num() == FHairStrandsMacroGroupData::MaxMacroGroupCount)
			{
				check(ClosestMacroGroupId != ~0u);
				FHairStrandsMacroGroupData& MacroGroup = MacroGroups[ClosestMacroGroupId];
				check(MacroGroup.MacroGroupId == ClosestMacroGroupId);
				MacroGroup.Bounds = Union(MacroGroup.Bounds, PrimitiveBounds);
				InternalUpdateMacroGroup(MacroGroup, MaterialId, HairData, Mesh, Proxy);
			}
			else
			{
				FHairStrandsMacroGroupData MacroGroup;
				MacroGroup.MacroGroupId = MacroGroupId++;
				InternalUpdateMacroGroup(MacroGroup, MaterialId, HairData, Mesh, Proxy);
				MacroGroup.Bounds = PrimitiveBounds;
				MacroGroups.Add(MacroGroup);
			}
		}
	};

	// 0. Pre-sort visible instances by register index to get stable macro-groups
	struct FVisibleBatch
	{
		const FMeshBatchAndRelevance* Batch = nullptr;
		FHairGroupPublicData* HairData = nullptr;
	};
	TArray<FVisibleBatch> VisibleBatches;
	VisibleBatches.Reserve(View.HairStrandsMeshElements.Num());
	for (const FMeshBatchAndRelevance& MeshBatchAndRelevance : View.HairStrandsMeshElements)
	{
		if (HairStrands::IsHairStrandsVF(MeshBatchAndRelevance.Mesh))
		{
			if (FHairGroupPublicData* HairData = HairStrands::GetHairData(MeshBatchAndRelevance.Mesh))
			{
				VisibleBatches.Add( { &MeshBatchAndRelevance, HairData });
			}
		}
	}
	VisibleBatches.Sort([](const FVisibleBatch& A, const FVisibleBatch& B) { return A.HairData->Instance->RegisteredIndex < B.HairData->Instance->RegisteredIndex; });

	// 1. Add all visible hair-strands instances
	static FBoxSphereBounds EmptyBound(ForceInit);
	for (FVisibleBatch& VisibleBatch : VisibleBatches)
	{
		UpdateMacroGroup(VisibleBatch.HairData, VisibleBatch.Batch->Mesh, VisibleBatch.Batch->PrimitiveSceneProxy, EmptyBound);
	}

	// 2. Add all hair-strands instances which are non-visible in primary view(s) but visible in shadow view(s)
	if (IsHairStrandsNonVisibleShadowCastingEnable())
	{
		for (FHairStrandsInstance* Instance : Scene->HairStrandsSceneData.RegisteredProxies)
		{
			if (Instance && CullingResults.Is(View, *Instance, EHairInstanceVisibilityType::StrandsShadowView))
			{
				UpdateMacroGroup(const_cast<FHairGroupPublicData*>(Instance->GetHairData()), nullptr, nullptr, Instance->GetBounds());
			}
		}
	}

	// Compute the screen size of macro group projection, for allocation purpose
	for (FHairStrandsMacroGroupData& MacroGroup : MacroGroups)
	{
		MacroGroup.ScreenRect = ComputeProjectedScreenRect(MacroGroup.Bounds.GetBox(), View);
	}
	// Sanity check
	check(MacroGroups.Num() <= FHairStrandsMacroGroupData::MaxMacroGroupCount);
	check(Scene->HairStrandsSceneData.TransientResources);

	// Build hair macro group AABBB
	FHairStrandsMacroGroupResources& MacroGroupResources = OutHairStrandsViewData.MacroGroupResources;
	const uint32 MacroGroupCount = MacroGroups.Num();
	if (MacroGroupCount > 0 && bBuildGPUAABB)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsAABB, "HairStrandsAABB");

		MacroGroupResources.MacroGroupAABBsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, 6 * MacroGroupCount), TEXT("Hair.MacroGroupAABBBuffer"));
		MacroGroupResources.MacroGroupVoxelAlignedAABBsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, 6 * MacroGroupCount), TEXT("Hair.MacroGroupVoxelAlignedAABBBuffer"));
		MacroGroupResources.MacroGroupVoxelSizeBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(2, MacroGroupCount), TEXT("Hair.MacroGroupVoxelSize"));
		FRDGBufferUAVRef MacroGroupAABBBufferUAV = GraphBuilder.CreateUAV(MacroGroupResources.MacroGroupAABBsBuffer, PF_R32_SINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
		FRDGBufferUAVRef MacroGroupVoxelSizeBufferUAV = GraphBuilder.CreateUAV(MacroGroupResources.MacroGroupVoxelSizeBuffer, PF_R16F, ERDGUnorderedAccessViewFlags::SkipBarrier);
		for (FHairStrandsMacroGroupData& MacroGroup : MacroGroups)
		{				
			AddHairMacroGroupAABBPass(GraphBuilder, View, *Scene->HairStrandsSceneData.TransientResources, MacroGroup, MacroGroupAABBBufferUAV, MacroGroupVoxelSizeBufferUAV);
		}
		MacroGroupResources.MacroGroupCount = MacroGroups.Num();
	}

	// Aggregate flags accross all instances/macrogroups
	OutHairStrandsViewData.Flags = 0;
	for (const FHairStrandsMacroGroupData& MacroGroupData : View.HairStrandsViewData.MacroGroupDatas)
	{			
		OutHairStrandsViewData.Flags |= MacroGroupData.Flags;
	}
}

bool FHairStrandsMacroGroupData::PrimitiveInfo::IsCullingEnable() const
{
	const FHairGroupPublicData* HairData = HairStrands::GetHairData(Mesh);
	return HairData->GetCullingResultAvailable();
}
