// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsCluster.h: Hair strands macro group computation implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "HairStrandsInterface.h"

class FScene;
class FViewInfo;
struct FHairStrandsViewData;

void CreateHairStrandsMacroGroups(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View, 
	const FHairInstanceCullingResults& CullingResults,
	FHairStrandsViewData& OutHairStrandsViewData,
	bool bBuildGPUAABB=true);