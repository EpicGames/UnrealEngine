// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsRasterCommon.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HairStrandsUtils.h"
#include "PrimitiveSceneProxy.h"
#include "Shader.h"
#include "MeshMaterialShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "ScenePrivate.h"
#include "RenderCore.h"
#include "SimpleMeshDrawCommandPass.h"

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FHairDeepShadowRasterUniformParameters, "DeepRasterPass", SceneTextures);

/////////////////////////////////////////////////////////////////////////////////////////

class FDeepShadowDepthMeshVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FDeepShadowDepthMeshVS, MeshMaterial);

protected:

	FDeepShadowDepthMeshVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel((EShaderPlatform)Initializer.Target.Platform);
		check(FSceneInterface::GetShadingPath(FeatureLevel) != EShadingPath::Mobile);
	}

	FDeepShadowDepthMeshVS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsCompatibleWithHairStrands(Parameters.Platform, Parameters.MaterialParameters) && HairStrands::IsHairStrandsVF(Parameters.VertexFactoryType);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MESH_RENDER_MODE"), 0);
		OutEnvironment.SetDefine(TEXT("USE_CULLED_CLUSTER"), 1);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FDeepShadowDepthMeshVS, TEXT("/Engine/Private/HairStrands/HairStrandsDeepShadowVS.usf"), TEXT("Main"), SF_Vertex);

/////////////////////////////////////////////////////////////////////////////////////////

class FDeepShadowDomMeshVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FDeepShadowDomMeshVS, MeshMaterial);

protected:

	FDeepShadowDomMeshVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel((EShaderPlatform)Initializer.Target.Platform);
		check(FSceneInterface::GetShadingPath(FeatureLevel) != EShadingPath::Mobile);
	}

	FDeepShadowDomMeshVS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsCompatibleWithHairStrands(Parameters.Platform, Parameters.MaterialParameters) && HairStrands::IsHairStrandsVF(Parameters.VertexFactoryType);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MESH_RENDER_MODE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_CULLED_CLUSTER"), 1);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FDeepShadowDomMeshVS, TEXT("/Engine/Private/HairStrands/HairStrandsDeepShadowVS.usf"), TEXT("Main"), SF_Vertex);

/////////////////////////////////////////////////////////////////////////////////////////

class FDeepShadowDepthMeshPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FDeepShadowDepthMeshPS, MeshMaterial);

public:

	FDeepShadowDepthMeshPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel((EShaderPlatform)Initializer.Target.Platform);
		check(FSceneInterface::GetShadingPath(FeatureLevel) != EShadingPath::Mobile);
	}

	FDeepShadowDepthMeshPS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsCompatibleWithHairStrands(Parameters.Platform, Parameters.MaterialParameters) && HairStrands::IsHairStrandsVF(Parameters.VertexFactoryType);
	}
};
IMPLEMENT_MATERIAL_SHADER_TYPE(, FDeepShadowDepthMeshPS, TEXT("/Engine/Private/HairStrands/HairStrandsDeepShadowPS.usf"), TEXT("MainDepth"), SF_Pixel);

/////////////////////////////////////////////////////////////////////////////////////////

class FDeepShadowDomMeshPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FDeepShadowDomMeshPS, MeshMaterial);

public:

	FDeepShadowDomMeshPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel((EShaderPlatform)Initializer.Target.Platform);
		check(FSceneInterface::GetShadingPath(FeatureLevel) != EShadingPath::Mobile);
	}

	FDeepShadowDomMeshPS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsCompatibleWithHairStrands(Parameters.Platform, Parameters.MaterialParameters) && HairStrands::IsHairStrandsVF(Parameters.VertexFactoryType);
	}
};
IMPLEMENT_MATERIAL_SHADER_TYPE(, FDeepShadowDomMeshPS, TEXT("/Engine/Private/HairStrands/HairStrandsDeepShadowPS.usf"), TEXT("MainDom"), SF_Pixel);

/////////////////////////////////////////////////////////////////////////////////////////

class FHairRasterMeshProcessor : public FMeshPassProcessor
{
public:

	FHairRasterMeshProcessor(
		const FScene* Scene,
		const FSceneView* InViewIfDynamicMeshCommand,
		const FMeshPassProcessorRenderState& InPassDrawRenderState,
		FDynamicPassMeshDrawListContext* InDrawListContext,
		const EHairStrandsRasterPassType PType);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
	{
		AddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, false);
	}

	void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, bool bCullingEnable);

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	template<typename VertexShaderType, typename PixelShaderType>
	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	EHairStrandsRasterPassType RasterPassType;
	FMeshPassProcessorRenderState PassDrawRenderState;
};

void FHairRasterMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, bool bCullingEnable)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

bool FHairRasterMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	// Determine the mesh's material and blend mode.
	bool bIsCompatible = IsCompatibleWithHairStrands(&Material, FeatureLevel);
	if (bIsCompatible && PrimitiveSceneProxy && (RasterPassType == EHairStrandsRasterPassType::FrontDepth || RasterPassType == EHairStrandsRasterPassType::DeepOpacityMap))
	{
		bIsCompatible = PrimitiveSceneProxy->CastsDynamicShadow();
	}

	if (bIsCompatible
		&& (!PrimitiveSceneProxy || PrimitiveSceneProxy->ShouldRenderInMainPass())
		&& ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain()))
	{
		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = RasterPassType == EHairStrandsRasterPassType::FrontDepth ? ComputeMeshCullMode(Material, OverrideSettings) : CM_None;

		if (RasterPassType == EHairStrandsRasterPassType::FrontDepth)
			return Process<FDeepShadowDepthMeshVS, FDeepShadowDepthMeshPS>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, MeshFillMode, MeshCullMode);
		else if (RasterPassType == EHairStrandsRasterPassType::DeepOpacityMap)
			return Process<FDeepShadowDomMeshVS, FDeepShadowDomMeshPS>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, MeshFillMode, MeshCullMode);
	}

	return true;
}

// Vertex is either FDeepShadowDepthMeshVS or FDeepShadowDomMeshVS
// Pixel  is either FDeepShadowDepthMeshPS or FDeepShadowDomMeshPS
template<typename VertexShaderType, typename PixelShaderType>
bool FHairRasterMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<VertexShaderType>();
	ShaderTypes.AddShaderType<PixelShaderType>();

	FMaterialShaders Shaders;
	if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
	{
		return false;
	}

	TMeshProcessorShaders<
		VertexShaderType,
		PixelShaderType> PassShaders;
	{
		FVertexFactoryType* VertexFactoryType = VertexFactory->GetType();
		const bool bIsHairStrandsFactory = HairStrands::IsHairStrandsVF(MeshBatch.VertexFactory->GetType());
		if (!bIsHairStrandsFactory)
			return true;	// Skip adding this batch if the VF type does not match.

		Shaders.TryGetVertexShader(PassShaders.VertexShader);
		Shaders.TryGetPixelShader(PassShaders.PixelShader);
	}

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		DrawRenderState,
		PassShaders,
		MeshFillMode,
		MeshCullMode,
		FMeshDrawCommandSortKey::Default,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

FHairRasterMeshProcessor::FHairRasterMeshProcessor(
	const FScene* Scene,
	const FSceneView* InViewIfDynamicMeshCommand,
	const FMeshPassProcessorRenderState& InPassDrawRenderState,
	FDynamicPassMeshDrawListContext* InDrawListContext,
	const EHairStrandsRasterPassType PType)
	: FMeshPassProcessor(EMeshPass::Num, Scene, Scene->GetFeatureLevel(), InViewIfDynamicMeshCommand, InDrawListContext)
	, RasterPassType(PType)
	, PassDrawRenderState(InPassDrawRenderState)
{
}

/////////////////////////////////////////////////////////////////////////////////////////

template<typename TPassParameter>
void AddHairStrandsRasterPass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* ViewInfo,
	const FHairStrandsMacroGroupData::TPrimitiveInfos& PrimitiveSceneInfos,
	const EHairStrandsRasterPassType RasterPassType,
	const FIntRect& ViewportRect,
	const FVector4f& HairRenderInfo,
	const uint32 HairRenderInfoBits,
	const FVector3f& RasterDirection,
	TPassParameter* PassParameters,
	FInstanceCullingManager& InstanceCullingManager)
{
	auto GetPassName = [](EHairStrandsRasterPassType Type)
	{
		switch (Type)
		{
		case EHairStrandsRasterPassType::DeepOpacityMap:		return RDG_EVENT_NAME("HairStrandsRasterDeepOpacityMap");
		case EHairStrandsRasterPassType::FrontDepth:			return RDG_EVENT_NAME("HairStrandsRasterFrontDepth");
		default:												return RDG_EVENT_NAME("Noname");
		}
	};

	{
		TUniformBufferRef<FViewUniformShaderParameters> ViewUniformShaderParameters;
		ViewInfo->CachedViewUniformShaderParameters->HairRenderInfo = HairRenderInfo;
		ViewInfo->CachedViewUniformShaderParameters->HairRenderInfoBits = HairRenderInfoBits;

		const FVector3f SavedViewForward = ViewInfo->CachedViewUniformShaderParameters->ViewForward;
		ViewInfo->CachedViewUniformShaderParameters->ViewForward = RasterDirection;
		PassParameters->View = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(*ViewInfo->CachedViewUniformShaderParameters, UniformBuffer_SingleFrame);
		ViewInfo->CachedViewUniformShaderParameters->ViewForward = SavedViewForward;
	}

	AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, *ViewInfo, &InstanceCullingManager, GetPassName(RasterPassType), ViewportRect, false /*bAllowOverrideIndirectArgs*/,
		[PassParameters, Scene = Scene, ViewInfo, RasterPassType, &PrimitiveSceneInfos, HairRenderInfo, HairRenderInfoBits, RasterDirection](FDynamicPassMeshDrawListContext* ShadowContext)
	{
		SCOPE_CYCLE_COUNTER(STAT_RenderPerObjectShadowDepthsTime);

		FMeshPassProcessorRenderState DrawRenderState;

		if (RasterPassType == EHairStrandsRasterPassType::DeepOpacityMap)
		{
			DrawRenderState.SetBlendState(TStaticBlendState<
				CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
				CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI());
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
		}
		else if (RasterPassType == EHairStrandsRasterPassType::FrontDepth)
		{
			DrawRenderState.SetBlendState(TStaticBlendState<
				CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
				CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI());
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
		}

		FHairRasterMeshProcessor HairRasterMeshProcessor(Scene, ViewInfo /* is a SceneView */, DrawRenderState, ShadowContext, RasterPassType);

		for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : PrimitiveSceneInfos)
		{
			// Ensure that we submit only primitive with valid MeshBatch data. Non-visible groom casting 
			// deep shadow map, will be skipped. This will be fixed once we remove mesh processor to replace 
			// it with a global shader
			if (PrimitiveInfo.Mesh != nullptr)
			{
				const bool bCullingEnable = PrimitiveInfo.IsCullingEnable();
				const FMeshBatch& MeshBatch = *PrimitiveInfo.Mesh;
				const uint64 BatchElementMask = ~0ull;
				HairRasterMeshProcessor.AddMeshBatch(MeshBatch, BatchElementMask, PrimitiveInfo.PrimitiveSceneProxy, -1 , bCullingEnable);
			}
		}
	});
}

void AddHairDeepShadowRasterPass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* ViewInfo,
	const FHairStrandsMacroGroupData::TPrimitiveInfos& PrimitiveSceneInfos,
	const EHairStrandsRasterPassType PassType,
	const FIntPoint& AtlasResolution,
	const FVector4f& HairRenderInfo,
	const uint32 HairRenderInfoBits,
	const FVector3f& LightDirection,
	FHairDeepShadowRasterPassParameters* PassParameters,
	FInstanceCullingManager& InstanceCullingManager)
{
	check(PassType == EHairStrandsRasterPassType::FrontDepth || PassType == EHairStrandsRasterPassType::DeepOpacityMap);

	AddHairStrandsRasterPass<FHairDeepShadowRasterPassParameters>(
		GraphBuilder, 
		Scene, 
		ViewInfo, 
		PrimitiveSceneInfos, 
		PassType, 
		FIntRect(FIntPoint::ZeroValue, AtlasResolution), 
		HairRenderInfo, 
		HairRenderInfoBits,
		LightDirection,
		PassParameters,
		InstanceCullingManager);
}