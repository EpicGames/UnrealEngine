// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairRendering.h: Hair rendering implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "HairStrands/HairStrandsData.h"
#include "SceneView.h"

class FViewInfo;
class FInstanceCullingManager;

enum class EHairStrandsRasterPassType : uint8
{
	FrontDepth,
	DeepOpacityMap
};

// ////////////////////////////////////////////////////////////////
// Deep shadow raster pass

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FHairDeepShadowRasterUniformParameters, RENDERER_API)
	SHADER_PARAMETER(uint32, AtlasSlotIndex)
	SHADER_PARAMETER(FVector4f, LayerDepths)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, FrontDepthTexture)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FDeepShadowViewInfo>, DeepShadowViewInfoBuffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FHairDeepShadowRasterPassParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairDeepShadowRasterUniformParameters, UniformBuffer)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void AddHairDeepShadowRasterPass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* ViewInfo,
	const FHairStrandsMacroGroupData::TPrimitiveInfos& PrimitiveSceneInfos,
	const EHairStrandsRasterPassType ShadowPassType,
	const FIntPoint& ViewportRect,
	const FVector4f& HairRenderInfo,
	const uint32 HairRenderInfoBits,
	const FVector3f& LightDirection,
	FHairDeepShadowRasterPassParameters* PassParameters,
	FInstanceCullingManager& InstanceCullingManager);
