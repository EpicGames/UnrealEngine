// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsRendering.h"
#include "HairStrandsData.h"
#include "SceneRendering.h"
#include "ScenePrivate.h"
#include "SystemTextures.h"
#include "RenderGraphUtils.h"
#include "HairStrands/HairStrandsMacroGroup.h"
#include "RenderCurveSceneExtension.h"
#include "MegaLights/MegaLightsInternal.h"
#include "SceneViewState.h"

/////////////////////////////////////////////////////////////////////////////////////////

void AddRenderCurveRasterPipeline(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const TArray<FViewInfo>& Views,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture);

bool IsHairVisibilityUpdateSceneHZBEnabled();

/////////////////////////////////////////////////////////////////////////////////////////

static TRDGUniformBufferRef<FHairStrandsViewUniformParameters> InternalCreateHairStrandsViewUniformBuffer(
	FRDGBuilder& GraphBuilder, 
	FHairStrandsVisibilityData* In)
{
	FHairStrandsViewUniformParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsViewUniformParameters>();
	Parameters->HairDualScatteringRoughnessOverride = GetHairDualScatteringRoughnessOverride();
	if (In && In->CoverageTexture)
	{
		Parameters->HairCoverageTexture = In->CoverageTexture;
		Parameters->HairOnlyDepthTexture = In->HairOnlyDepthTexture;
		Parameters->HairOnlyDepthHZBParameters = In->HairOnlyDepthHZBParameters;
		Parameters->HairOnlyDepthClosestHZBTexture = In->HairOnlyDepthClosestHZBTexture;
		Parameters->HairOnlyDepthFurthestHZBTexture = In->HairOnlyDepthFurthestHZBTexture;
		Parameters->HairOnlyDepthHZBSampler = TStaticSamplerState<SF_Point>::GetRHI();
		Parameters->HairSampleOffset = In->NodeIndex;
		Parameters->HairSampleData = GraphBuilder.CreateSRV(In->NodeData);
		Parameters->HairSampleCoords = GraphBuilder.CreateSRV(In->NodeCoord, FHairStrandsVisibilityData::NodeCoordFormat);
		Parameters->HairSampleCount = GraphBuilder.CreateSRV(In->NodeCount);
		Parameters->HairSampleViewportResolution = In->SampleLightingViewportResolution;
		Parameters->MaxSamplePerPixelCount = In->MaxSampleCount;

		check(In->TileData.IsValid())
		Parameters->HairTileData = In->TileData.GetTileBufferSRV(FHairStrandsTiles::ETileType::HairAll);
		Parameters->HairTileCount = In->TileData.TileCountSRV;
		Parameters->HairTileCountXY = In->TileData.TileCountXY;
		Parameters->bHairTileValid = true;

		if (!Parameters->HairOnlyDepthFurthestHZBTexture)
		{
			FRDGTextureRef BlackTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
			Parameters->HairOnlyDepthHZBParameters = FVector4f::Zero();
			Parameters->HairOnlyDepthFurthestHZBTexture = BlackTexture;
			Parameters->HairOnlyDepthClosestHZBTexture = BlackTexture;
		}
	}
	else
	{
		FRDGBufferRef DummyBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, 4);
		FRDGBufferRef DummyNodeBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 20);

		FRDGTextureRef BlackTexture 	= GSystemTextures.GetBlackDummy(GraphBuilder);
		FRDGTextureRef ZeroR32_UINT 	= GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, 0u);
		FRDGTextureRef ZeroRGBA16_UINT 	= GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R16G16_UINT, 0u);
		FRDGTextureRef FarDepth 		= GSystemTextures.GetDepthDummy(GraphBuilder);

		FRDGBufferSRVRef DummyBufferR32SRV = GraphBuilder.CreateSRV(DummyBuffer, PF_R32_UINT);
		FRDGBufferSRVRef DummyBufferRG16SRV = GraphBuilder.CreateSRV(DummyBuffer, PF_R16G16_UINT);

		Parameters->HairOnlyDepthTexture = FarDepth;
		Parameters->HairOnlyDepthHZBParameters = FVector4f::Zero();
		Parameters->HairOnlyDepthFurthestHZBTexture = BlackTexture;
		Parameters->HairOnlyDepthClosestHZBTexture = Parameters->HairOnlyDepthFurthestHZBTexture;
		Parameters->HairOnlyDepthHZBSampler = TStaticSamplerState<SF_Point>::GetRHI();
		Parameters->HairCoverageTexture = BlackTexture;
		Parameters->HairSampleCount = DummyBufferR32SRV;
		Parameters->HairSampleOffset = ZeroR32_UINT;
		Parameters->HairSampleCoords = DummyBufferRG16SRV;
		Parameters->HairSampleData	 = GraphBuilder.CreateSRV(DummyNodeBuffer);
		Parameters->HairSampleViewportResolution = FIntPoint(0, 0);
		Parameters->MaxSamplePerPixelCount = 0u;

		Parameters->HairTileData = DummyBufferR32SRV;
		Parameters->HairTileCount = DummyBufferRG16SRV;
		Parameters->HairTileCountXY = FIntPoint(0, 0);
	}

	return GraphBuilder.CreateUniformBuffer(Parameters);
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FHairStrandsViewUniformParameters, "HairStrands");

void AddMeshDrawTransitionPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& ViewInfo,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas);

FHairTransientResources* AllocateHairTransientResources(FRDGBuilder& GraphBuilder, FScene* Scene, const TArray<FViewInfo>& Views)
{
	if (Scene->HairStrandsSceneData.TransientResources == nullptr)
	{
		Scene->HairStrandsSceneData.TransientResources = new FHairTransientResources();
	}

	const uint32 InstanceCount = Scene->HairStrandsSceneData.RegisteredProxies.Num();
	FHairTransientResources* Out = Scene->HairStrandsSceneData.TransientResources;
	*Out = FHairTransientResources();
	Out->TranslatedWorldOffsetView0 = Views.Num() > 0 ? Views[0].ViewMatrices.GetPreViewTranslation() : FVector3d::ZeroVector;
	if (InstanceCount > 0)
	{
		Out->bIsGroupAABBValid.Init(false, InstanceCount);

		// Change this into a structure buffer
		Out->GroupAABBBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, 6 * InstanceCount), TEXT("Hair.Transient.GroupAABB"));
		Out->GroupAABBSRV = GraphBuilder.CreateSRV(Out->GroupAABBBuffer, PF_R32_SINT);

		// *4u* elements for storing DispatchCount.xyz and .w which contains the original number of items
		Out->IndirectDispatchArgsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(4u * InstanceCount), TEXT("Hair.Transient.IndirectDispatchArgs"));
		Out->IndirectDispatchArgsSRV = GraphBuilder.CreateSRV(Out->IndirectDispatchArgsBuffer);
	}
	return Out;
}

void RenderHairPrePass(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const FSceneTextures& SceneTextures,
	TArray<FViewInfo>& Views,
	FInstanceCullingManager& InstanceCullingManager,
	const FHairInstanceCullingResults& CullingResults)
{
	// Experimental curve raster pipeline. The function early outs when it is not enabled.
	AddRenderCurveRasterPipeline(GraphBuilder, Scene, Views, SceneTextures.Color.Target, SceneTextures.Depth.Target);

	TArray<FViewInfo*> ActiveViews;
	ActiveViews.Reserve(Views.Num());
	for (FViewInfo& View : Views)
	{
		const bool bIsViewCompatible = IsHairStrandsEnabled(EHairStrandsShaderType::Strands, View.GetShaderPlatform());
		if (!View.Family || !bIsViewCompatible || !View.Family->EngineShowFlags.Hair)
			continue;

		// For stereo rendering, hair groups/voxelization/deep-shadow are only produced once
		FVector PreViewStereoCorrection = FVector::ZeroVector;
		if (IStereoRendering::IsStereoEyeView(View))
		{
			// nDisplay uses StereoRendering code path with a mono-view
			if (Views.Num() >= 2)
			{
				PreViewStereoCorrection = Views[0].ViewMatrices.GetPreViewTranslation() - Views[1].ViewMatrices.GetPreViewTranslation();
			}
			if (IStereoRendering::IsASecondaryView(View))
			{
				// No need to copy the view state (i.e., HairStrandsViewStateData) as it is only used for 
				// voxelization feedback (only done for the first view in stereo) and Path-Tracer invalidation 
				// (not supporting stereo)
				Views[1].HairStrandsViewData = Views[0].HairStrandsViewData;

				// Render DeepShadow for the second view, as for now the computations are view dependent. 
				// This needs to be view independent to share result between eyes.
				RenderHairStrandsDeepShadows(GraphBuilder, Scene, View, InstanceCullingManager);
				return;
			}
		}

		ActiveViews.Add(&View);

		// Allocate state/feedback data if needed
		if (View.ViewState && !View.ViewState->HairStrandsViewStateData.IsInit())
		{
			View.ViewState->HairStrandsViewStateData.Init();
		}

		CreateHairStrandsMacroGroups(GraphBuilder, Scene, View, CullingResults, View.HairStrandsViewData);

		// Voxelization and Deep Opacity Maps
		VoxelizeHairStrands(GraphBuilder, Scene, View, InstanceCullingManager, PreViewStereoCorrection);
	}

	// Transit resources to be used by draw passes (e.g., primary/shadow draws). 
	// Note: this will reset references to instances's imported buffers
	for (FViewInfo* View : ActiveViews)
	{
		if (View->HairStrandsViewData.MacroGroupDatas.Num() > 0)
		{
			AddMeshDrawTransitionPass(GraphBuilder, *View, View->HairStrandsViewData.MacroGroupDatas);
		}
	}

	for (FViewInfo* View : ActiveViews)
	{
		RenderHairStrandsDeepShadows(GraphBuilder, Scene, *View, InstanceCullingManager);
	}
}

void RenderHairBasePass(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const FSceneTextures& SceneTextures,
	TArray<FViewInfo>& Views,
	FInstanceCullingManager& InstanceCullingManager,
	TSharedPtr<FMegaLightsFrameTemporaries> MegaLightsContext)
{
	for (FViewInfo& View : Views)
	{
		const bool bIsViewCompatible = IsHairStrandsEnabled(EHairStrandsShaderType::Strands, View.GetShaderPlatform());
		if (View.Family && View.Family->EngineShowFlags.Hair && bIsViewCompatible && View.HairStrandsViewData.MacroGroupDatas.Num() > 0)
		{
			RenderHairStrandsVisibilityBuffer(
				GraphBuilder, 
				Scene, 
				View, 
				SceneTextures.GBufferA,
				SceneTextures.GBufferB,
				SceneTextures.GBufferC,
				SceneTextures.GBufferD,
				SceneTextures.GBufferE,
				SceneTextures.Color.Resolve,
				SceneTextures.Depth.Resolve,
				SceneTextures.Velocity,
				InstanceCullingManager);
		}
		
		if (View.HairStrandsViewData.VisibilityData.CoverageTexture)
		{			
			View.HairStrandsViewData.UniformBuffer = InternalCreateHairStrandsViewUniformBuffer(GraphBuilder, &View.HairStrandsViewData.VisibilityData);
			View.HairStrandsViewData.bIsValid = true;
		}
		else
		{
			View.HairStrandsViewData.UniformBuffer = InternalCreateHairStrandsViewUniformBuffer(GraphBuilder, nullptr);
			View.HairStrandsViewData.bIsValid = false;
		}

		// Late composition of hair into to custom depth (avoid extra rasterization by reusing primary raster outputs)
		if (SceneTextures.CustomDepth.Depth && EnumHasAllFlags(SceneTextures.SetupMode, ESceneTextureSetupMode::CustomDepth))
		{
			if (HairStrands::HasViewHairStrandsData(View))
			{
				HairStrands::DrawCustomDepth(GraphBuilder, View, View.ViewRect, SceneTextures.CustomDepth.Depth);
			}
		}
	}

	// Megalights registers early Scene.HZB / Scene.ClosestHZB into FMegaLightsParameters. When IsHairVisibilityUpdateSceneHZBEnabled() is enabled, 
	// new scene HZB textures are created. The Megalights context needs to be patched with these new HZB texture
	if (MegaLightsContext && IsHairVisibilityUpdateSceneHZBEnabled())
	{
		MegaLightsContext->UpdateHZB();
	}
}

void FHairStrandsViewStateData::Init()
{
	// Voxel adaptive sizing
	VoxelFeedbackBuffer = nullptr;
	
	// Track if hair strands positions has changed
	PositionsChangedDatas.SetNum(4);
	for (FPositionChangedData& Data : PositionsChangedDatas)
	{
		Data.ReadbackBuffer = new FRHIGPUBufferReadback(TEXT("Hair.PositionsChangedReadback"));
		Data.bHasPendingReadback = false;
	}
}

void FHairStrandsViewStateData::Release()
{
	// Voxel adaptive sizing
	VoxelFeedbackBuffer = nullptr;

	// Track if hair strands positions has changed
	for (FPositionChangedData& Data : PositionsChangedDatas)
	{
		delete Data.ReadbackBuffer;
		Data.ReadbackBuffer = nullptr;
		Data.bHasPendingReadback = false;
	}
	PositionsChangedDatas.Empty();
}

void FHairStrandsViewStateData::EnqueuePositionsChanged(FRDGBuilder& GraphBuilder, FRDGBufferRef InBuffer)
{
	for (FPositionChangedData& Data : PositionsChangedDatas)
	{
		if (!Data.bHasPendingReadback)
		{
			AddEnqueueCopyPass(GraphBuilder, Data.ReadbackBuffer, InBuffer, 4u);
			Data.bHasPendingReadback = true;
			break;
		}
	}
}

bool FHairStrandsViewStateData::ReadPositionsChanged()
{
	for (FPositionChangedData& Data : PositionsChangedDatas)
	{
		if (Data.bHasPendingReadback && Data.ReadbackBuffer->IsReady())
		{
			const uint32 ReadData = *(uint32*)(Data.ReadbackBuffer->Lock(sizeof(uint32)));
			Data.ReadbackBuffer->Unlock();
			Data.bHasPendingReadback = false;
			return ReadData > 0;
		}
	}
	return false;
}

namespace HairStrands
{

TRDGUniformBufferRef<FHairStrandsViewUniformParameters> CreateDefaultHairStrandsViewUniformBuffer(FRDGBuilder& GraphBuilder)
{
	return InternalCreateHairStrandsViewUniformBuffer(GraphBuilder, nullptr);
}

TRDGUniformBufferRef<FHairStrandsViewUniformParameters> BindHairStrandsViewUniformParameters(const FViewInfo& View)
{
	return View.HairStrandsViewData.UniformBuffer;
}

TRDGUniformBufferRef<FVirtualVoxelParameters> CreateDefaultHairStrandsVoxelUniformBuffer(FRDGBuilder& GraphBuilder)
{
	FRDGBufferSRVRef DummyBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 4u, 0u), PF_R32_UINT);
	FRDGBufferSRVRef DummyNodeDescBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FPackedVirtualVoxelNodeDesc), 0u));

	FVirtualVoxelParameters* Out = GraphBuilder.AllocParameters<FVirtualVoxelParameters>();
	Out->Common.PageCountResolution = FIntVector::ZeroValue;
	Out->Common.CPUMinVoxelWorldSize = 0.f;
	Out->Common.PageTextureResolution = FIntVector::ZeroValue;
	Out->Common.PageCount = 0;
	Out->Common.PageResolution = 0;
	Out->Common.PageResolutionLog2 = 0;
	Out->Common.PageIndexCount = 0;
	Out->Common.IndirectDispatchGroupSize = 0;
	Out->Common.NodeDescCount = 0;
	Out->Common.JitterMode = 0;
	Out->Common.DensityScale = 0;
	Out->Common.DensityScale_AO = 0;
	Out->Common.DensityScale_Shadow = 0;
	Out->Common.DensityScale_Transmittance = 0;
	Out->Common.DensityScale_Environment = 0;
	Out->Common.DensityScale_Raytracing = 0;
	Out->Common.DepthBiasScale_Shadow = 0;
	Out->Common.DepthBiasScale_Transmittance = 0;
	Out->Common.DepthBiasScale_Environment = 0;
	Out->Common.SteppingScale_Shadow = 0;
	Out->Common.SteppingScale_Transmittance = 0;
	Out->Common.SteppingScale_Environment = 0;
	Out->Common.SteppingScale_Raytracing = 0;
	Out->Common.HairCoveragePixelRadiusAtDepth1 = 0;
	Out->Common.Raytracing_ShadowOcclusionThreshold = 0;
	Out->Common.Raytracing_SkyOcclusionThreshold = 0;
	Out->Common.TranslatedWorldOffset = FVector3f::ZeroVector;
	Out->Common.TranslatedWorldOffsetStereoCorrection = FVector3f::ZeroVector;
	Out->Common.AllocationFeedbackEnable = 0;
	Out->Common.AllocatedPageCountBuffer = DummyBuffer;
	Out->Common.PageIndexBuffer = DummyBuffer;
	Out->Common.PageIndexCoordBuffer = DummyBuffer;
	Out->Common.NodeDescBuffer = DummyNodeDescBuffer;
	Out->Common.CurrGPUMinVoxelSize = DummyBuffer;
	Out->Common.NextGPUMinVoxelSize = DummyBuffer;
	Out->PageTexture = GSystemTextures.GetVolumetricBlackDummy(GraphBuilder);
	return GraphBuilder.CreateUniformBuffer(Out);
}

TRDGUniformBufferRef<FVirtualVoxelParameters> BindHairStrandsVoxelUniformParameters(const FViewInfo& View)
{
	// Voxel uniform buffer exist only if the view has hair strands data
	check(View.HairStrandsViewData.bIsValid && View.HairStrandsViewData.VirtualVoxelResources.IsValid());
	return View.HairStrandsViewData.VirtualVoxelResources.UniformBuffer;
}

bool HasViewHairStrandsData(const FViewInfo& View)
{
	return View.HairStrandsViewData.bIsValid;
}

bool HasViewHairStrandsVoxelData(const FViewInfo& View)
{
	return View.HairStrandsViewData.bIsValid && View.HairStrandsViewData.VirtualVoxelResources.IsValid();
}

bool HasViewHairStrandsData(const TArray<FViewInfo>& Views)
{
	for (const FViewInfo& View : Views)
	{
		if (View.HairStrandsViewData.bIsValid)
		{
			return true;
		}
	}
	return false;
}

bool HasHairStrandsVisible(TArrayView<const FViewInfo> Views)
{
	for (const FViewInfo& View : Views)
	{
		if (View.HairStrandsMeshElements.Num() > 0)
		{
			return true;
		}
	}
	return false;
}

bool HasHairCardsVisible(const TArray<FViewInfo>& Views)
{
	for (const FViewInfo& View : Views)
	{
		if (View.HairCardsMeshElements.Num() > 0)
		{
			return true;
		}
	}
	return false;
}

bool HasHairInstanceInScene(const FScene& Scene)
{	
	return Scene.HairStrandsSceneData.RegisteredProxies.Num() > 0;
}

void PostRender(FScene& Scene)
{
	// Dellocate transient resourcse
	if (Scene.HairStrandsSceneData.TransientResources)
	{
		*Scene.HairStrandsSceneData.TransientResources = FHairTransientResources();
	}
}

} // HairStrands
