// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairRendering.h: Hair rendering implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "HairStrandsUtils.h"
#include "HairStrandsLUT.h"
#include "HairStrandsDeepShadow.h"
#include "HairStrandsVoxelization.h"
#include "HairStrandsVisibility.h"
#include "HairStrandsTransmittance.h"
#include "HairStrandsEnvironment.h"
#include "HairStrandsComposition.h"
#include "HairStrandsDebug.h"
#include "HairStrandsInterface.h"
#include "HairStrandsData.h"

struct FMegaLightsFrameTemporaries;

FHairTransientResources* AllocateHairTransientResources(FRDGBuilder& GraphBuilder, FScene* Scene, const TArray<FViewInfo>& Views);

void RenderHairPrePass(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const FSceneTextures& SceneTextures,
	TArray<FViewInfo>& Views,
	FInstanceCullingManager& InstanceCullingManager,
	const FHairInstanceCullingResults& CullingResults);

void RenderHairBasePass(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const FSceneTextures& SceneTextures,
	TArray<FViewInfo>& Views,
	FInstanceCullingManager& InstanceCullingManager,
	TSharedPtr<FMegaLightsFrameTemporaries> MegaLightsContext);

void RunHairStrandsBookmark(
	FRDGBuilder& GraphBuilder, 
	EHairStrandsBookmark Bookmark, 
	FHairStrandsBookmarkParameters& Parameters);

void RunHairStrandsBookmark(
	EHairStrandsBookmark Bookmark,
	FHairStrandsBookmarkParameters& Parameters);

void CreateHairStrandsBookmarkParameters(FScene* Scene, FViewInfo& View, FHairStrandsBookmarkParameters& Out, bool bComputeVisibleInstances=true);
void CreateHairStrandsBookmarkParameters(FScene* Scene, TArray<FViewInfo>& Views, TConstArrayView<FViewInfo*> AllViews, FHairStrandsBookmarkParameters& Out, bool bComputeVisibleInstances=true);
void UpdateHairStrandsBookmarkParameters(FScene* Scene, TArray<FViewInfo>& Views, FHairStrandsBookmarkParameters& Out);
