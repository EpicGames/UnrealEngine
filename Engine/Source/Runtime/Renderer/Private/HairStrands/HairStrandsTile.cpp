// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsTile.h"
#include "HairStrandsUtils.h"
#include "HairStrandsInterface.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "SceneTextureParameters.h"
#include "RenderGraphUtils.h"
#include "PostProcess/PostProcessing.h"
#include "MeshPassProcessor.inl"
#include "ShaderPrintParameters.h"
#include "ShaderPrint.h"
#include "SceneRendering.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

const TCHAR* ToString(FHairStrandsTiles::ETileType Type)
{
	switch (Type)
	{
		case FHairStrandsTiles::ETileType::HairAll:			return TEXT("Hair(All)");
		case FHairStrandsTiles::ETileType::HairFull:		return TEXT("Hair(Full)");
		case FHairStrandsTiles::ETileType::HairPartial:		return TEXT("Hair(Partial)");
		case FHairStrandsTiles::ETileType::Other:			return TEXT("Other");
		default:											return TEXT("Unknown");
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FHairStrandsTilePassVS::FParameters GetHairStrandsTileParameters(const FViewInfo& InView, const FHairStrandsTiles& InTile, FHairStrandsTiles::ETileType TileType)
{
	FHairStrandsTilePassVS::FParameters Out;
	Out.TileType				= uint32(TileType);
	Out.bRectPrimitive			= InTile.bRectPrimitive ? 1 : 0;
	Out.ViewMin					= InView.ViewRect.Min;
	Out.ViewInvSize				= FVector2f(1.f / InView.ViewRect.Width(), 1.f / InView.ViewRect.Height());
	Out.TileDataBuffer			= InTile.IsValid() ? InTile.GetTileBufferSRV(TileType) : nullptr;
	Out.TileIndirectBuffer		= InTile.TileIndirectDrawBuffer;
	return Out;
}

static ERHIFeatureSupport HairSupportsWaveOps(EShaderPlatform Platform)
{
	// D3D11 / SM5 or preview do not support, or work well with, wave-ops by default (or SM5 preview has issues with wave intrinsics too), that fixes classification and black/wrong tiling.
	if (Platform == SP_PCD3D_SM5 || FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Platform))
	{
		return ERHIFeatureSupport::Unsupported;
	}

	return FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(Platform);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Generate indirect draw and indirect dispatch buffers

class FHairStrandsTileCopyArgsPassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsTileCopyArgsPassCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsTileCopyArgsPassCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(FIntPoint, TileCountXY)
		SHADER_PARAMETER(uint32, TilePerThread_GroupSize)
		SHADER_PARAMETER(uint32, bRectPrimitive)
		SHADER_PARAMETER(uint32, TileSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, TileCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileIndirectDrawBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileIndirectDispatchBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileIndirectRayDispatchBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TilePerThreadIndirectDispatchBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TILE_COPY_ARGS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsTileCopyArgsPassCS, "/Engine/Private/HairStrands/HairStrandsVisibilityTile.usf", "MainCS", SF_Compute);

void AddHairStrandsCopyArgsTilesPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FHairStrandsTiles& TileData)
{
	TShaderMapRef<FHairStrandsTileCopyArgsPassCS> ComputeShader(View.ShaderMap);
	FHairStrandsTileCopyArgsPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairStrandsTileCopyArgsPassCS::FParameters>();
	PassParameters->TileSize = TileData.TileSize;
	PassParameters->TileCountXY = TileData.TileCountXY;
	PassParameters->TilePerThread_GroupSize = TileData.TilePerThread_GroupSize;
	PassParameters->bRectPrimitive = TileData.bRectPrimitive ? 1 : 0;
	PassParameters->TileCountBuffer = TileData.TileCountSRV;
	PassParameters->TileIndirectDrawBuffer = GraphBuilder.CreateUAV(TileData.TileIndirectDrawBuffer, PF_R32_UINT);
	PassParameters->TileIndirectDispatchBuffer = GraphBuilder.CreateUAV(TileData.TileIndirectDispatchBuffer, PF_R32_UINT);
	PassParameters->TileIndirectRayDispatchBuffer = GraphBuilder.CreateUAV(TileData.TileIndirectRayDispatchBuffer, PF_R32_UINT);
	PassParameters->TilePerThreadIndirectDispatchBuffer = GraphBuilder.CreateUAV(TileData.TilePerThreadIndirectDispatchBuffer, PF_R32_UINT);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::TileCopyArgs"),
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, 1));
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Generate tiles data based on input texture (hair pixel coverage)

class FHairStrandsTileGenerationPassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsTileGenerationPassCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsTileGenerationPassCS, FGlobalShader);

	class FInputType : SHADER_PERMUTATION_INT("PERMUTATION_INPUT_TYPE", 2);
	class FWaveOps : SHADER_PERMUTATION_BOOL("PERMUTATION_WAVEOPS");
	using FPermutationDomain = TShaderPermutationDomain<FInputType, FWaveOps>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(FIntPoint, BufferResolution)
		SHADER_PARAMETER(uint32, bForceOutputAllTiles)
		SHADER_PARAMETER(float, TransmittanceThreshold)
		SHADER_PARAMETER(uint32, IntCoverageThreshold)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, InputFloatTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>,  InputUintTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileHairAllBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileHairFullBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileHairPartialBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileOtherBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, TileClearBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveOps>() && HairSupportsWaveOps(Parameters.Platform) == ERHIFeatureSupport::Unsupported)
		{
			return false;
		}
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TILE_GENERATION"), 1);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FWaveOps>())
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_WaveOperations);
		}
	}
};
IMPLEMENT_GLOBAL_SHADER(FHairStrandsTileGenerationPassCS, "/Engine/Private/HairStrands/HairStrandsVisibilityTile.usf", "TileMainCS", SF_Compute);

float GetHairStrandsFullCoverageThreshold();
uint32 GetHairStrandsIntCoverageThreshold();

static FHairStrandsTiles AddHairStrandsGenerateTilesPass_Internal(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FIntPoint& InputResolution,
	const FRDGTextureRef& InputTexture)
{
	const bool bHasValidInput = InputTexture != nullptr;
	const bool bUintTexture   = InputTexture && InputTexture->Desc.Format == PF_R32_UINT;
	const bool bWaveOps       = GRHISupportsWaveOperations&& GRHIMaximumWaveSize >= 64 && HairSupportsWaveOps(View.GetShaderPlatform()) != ERHIFeatureSupport::Unsupported;

	FHairStrandsTiles Out;

	check(FHairStrandsTiles::TilePerThread_GroupSize == 64); // If this value change, we need to update the shaders using 
	check(FHairStrandsTiles::TileSize == 8); // only size supported for now
	Out.TileCountXY = FIntPoint(FMath::CeilToInt(InputResolution.X / float(FHairStrandsTiles::TileSize)), FMath::CeilToInt(InputResolution.Y / float(FHairStrandsTiles::TileSize)));
	Out.TileCount = Out.TileCountXY.X * Out.TileCountXY.Y;
	Out.BufferResolution = InputResolution;
	Out.bRectPrimitive = GRHISupportsRectTopology;
	Out.TileCountBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, FHairStrandsTiles::TileTypeCount), TEXT("Hair.TileCountBuffer"));
	Out.TileIndirectDrawBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndirectParameters>(FHairStrandsTiles::TileTypeCount), TEXT("Hair.TileIndirectDrawBuffer"));
	Out.TileIndirectDispatchBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(FHairStrandsTiles::TileTypeCount), TEXT("Hair.TileIndirectDispatchBuffer"));
	Out.TileIndirectRayDispatchBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(FHairStrandsTiles::TileTypeCount), TEXT("Hair.TileIndirectRayDispatchBuffer"));
	Out.TilePerThreadIndirectDispatchBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(FHairStrandsTiles::TileTypeCount), TEXT("Hair.TilePerThreadIndirectDispatchBuffer"));
	Out.TileDataBuffer[ToIndex(FHairStrandsTiles::ETileType::HairAll)] = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Out.TileCount), TEXT("Hair.TileDataBuffer(HairAll)"));
	Out.TileDataBuffer[ToIndex(FHairStrandsTiles::ETileType::HairFull)] = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Out.TileCount), TEXT("Hair.TileDataBuffer(HairFull)"));
	Out.TileDataBuffer[ToIndex(FHairStrandsTiles::ETileType::HairPartial)] = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Out.TileCount), TEXT("Hair.TileDataBuffer(HairPartial)"));
	Out.TileDataBuffer[ToIndex(FHairStrandsTiles::ETileType::Other)] = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Out.TileCount), TEXT("Hair.TileDataBuffer(Other)"));

	FRDGBufferUAVRef TileCountUAV = GraphBuilder.CreateUAV(Out.TileCountBuffer, PF_R32_UINT);
	AddClearUAVPass(GraphBuilder, TileCountUAV, 0u);

	FHairStrandsTileGenerationPassCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairStrandsTileGenerationPassCS::FInputType>(bUintTexture ? 1u : 0u);
	PermutationVector.Set<FHairStrandsTileGenerationPassCS::FWaveOps>(bWaveOps);

	TShaderMapRef<FHairStrandsTileGenerationPassCS> ComputeShader(View.ShaderMap, PermutationVector);
	FHairStrandsTileGenerationPassCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairStrandsTileGenerationPassCS::FParameters>();
	PassParameters->ViewUniformBuffer		= View.ViewUniformBuffer;
	PassParameters->BufferResolution		= View.ViewRect.Size();
	PassParameters->bForceOutputAllTiles	= bHasValidInput ? 0 : 1;
	PassParameters->TransmittanceThreshold	= 1.f - GetHairStrandsFullCoverageThreshold();	
	PassParameters->IntCoverageThreshold 	= GetHairStrandsIntCoverageThreshold();
	PassParameters->InputFloatTexture		= GSystemTextures.GetBlackDummy(GraphBuilder);
	PassParameters->InputUintTexture		= GSystemTextures.GetZeroUIntDummy(GraphBuilder);
	if (bHasValidInput && bUintTexture)
	{
		PassParameters->InputUintTexture	= InputTexture;
	}
	else if (bHasValidInput && !bUintTexture)
	{
		PassParameters->InputFloatTexture	= InputTexture;
	}
	PassParameters->TileHairAllBuffer		= GraphBuilder.CreateUAV(Out.TileDataBuffer[ToIndex(FHairStrandsTiles::ETileType::HairAll)], PF_R16G16_UINT);
	PassParameters->TileHairFullBuffer		= GraphBuilder.CreateUAV(Out.TileDataBuffer[ToIndex(FHairStrandsTiles::ETileType::HairFull)], PF_R16G16_UINT);
	PassParameters->TileHairPartialBuffer	= GraphBuilder.CreateUAV(Out.TileDataBuffer[ToIndex(FHairStrandsTiles::ETileType::HairPartial)], PF_R16G16_UINT);
	PassParameters->TileOtherBuffer			= GraphBuilder.CreateUAV(Out.TileDataBuffer[ToIndex(FHairStrandsTiles::ETileType::Other)], PF_R16G16_UINT);
	PassParameters->TileCountBuffer			= TileCountUAV;

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::TileClassification"),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(InputResolution, FHairStrandsTiles::TileSize));

	Out.TileCountSRV = GraphBuilder.CreateSRV(Out.TileCountBuffer, PF_R32_UINT);
	for (uint32 BufferIt = 0; BufferIt < FHairStrandsTiles::TileTypeCount; ++BufferIt)
	{
		if (Out.TileDataBuffer[BufferIt])
		{
			Out.TileDataSRV[BufferIt] = GraphBuilder.CreateSRV(Out.TileDataBuffer[BufferIt], PF_R16G16_UINT);
		}
	}

	// Initialize indirect dispatch buffer, based on the indirect draw bugger
	AddHairStrandsCopyArgsTilesPass(GraphBuilder, View, Out);

	return Out;
}

FHairStrandsTiles AddHairStrandsGenerateTilesPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FRDGTextureRef& InputTexture)
{
	return AddHairStrandsGenerateTilesPass_Internal(GraphBuilder, View, InputTexture->Desc.Extent, InputTexture);
}

FHairStrandsTiles AddHairStrandsGenerateTilesPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FIntPoint& Resolution)
{
	return AddHairStrandsGenerateTilesPass_Internal(GraphBuilder, View, Resolution, nullptr);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool FHairStrandsTilePassVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
}

void FHairStrandsTilePassVS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("SHADER_TILE_VS"), 1);
}

IMPLEMENT_GLOBAL_SHADER(FHairStrandsTilePassVS, "/Engine/Private/HairStrands/HairStrandsVisibilityTile.usf", "MainVS", SF_Vertex);


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FHairStrandsTileDebugPrintPassCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsTileDebugPrintPassCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsTileDebugPrintPassCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, MaxResolution)
		SHADER_PARAMETER(uint32, TileGroupSize)
		SHADER_PARAMETER(uint32, TileSize)
		SHADER_PARAMETER(uint32, TileCount)
		SHADER_PARAMETER(uint32, TileType)
		SHADER_PARAMETER(FIntPoint, TileCountXY)
		SHADER_PARAMETER(uint32, bRectPrimitive)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TILE_DEBUG_PRINT"), 1);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsTileDebugPrintPassCS, "/Engine/Private/HairStrands/HairStrandsVisibilityTile.usf", "MainCS", SF_Compute);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FHairStrandsTileDebugPassPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsTileDebugPassPS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsTileDebugPassPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileParameters)
		SHADER_PARAMETER(FVector3f, DebugColor)
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5; //TODO
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TILE_DEBUG"), 1); 
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsTileDebugPassPS, "/Engine/Private/HairStrands/HairStrandsVisibilityTile.usf", "MainPS", SF_Pixel);

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void AddHairStrandsDebugTilePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef& ColorTexture,
	const FHairStrandsTiles& TileData)
{	
	const FIntRect Viewport = View.ViewRect;

	auto DrawDebugTile = [&](FHairStrandsTiles::ETileType TileType)
	{
		FHairStrandsTileDebugPassPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairStrandsTileDebugPassPS::FParameters>();
		PassParameters->TileParameters = GetHairStrandsTileParameters(View, TileData, TileType);
		PassParameters->DebugColor = FVector3f(1.f, 0.f, 1.f);
		switch (TileType)
		{
		case FHairStrandsTiles::ETileType::HairAll:		PassParameters->DebugColor = FVector3f(1.f, 1.f, 0.f); break;
		case FHairStrandsTiles::ETileType::HairFull:	PassParameters->DebugColor = FVector3f(1.f, 0.f, 0.f); break;
		case FHairStrandsTiles::ETileType::HairPartial: PassParameters->DebugColor = FVector3f(0.f, 1.f, 0.f); break;
		case FHairStrandsTiles::ETileType::Other:		PassParameters->DebugColor = FVector3f(0.6f, 0.6f, 0.6f); break;
		}

		TShaderMapRef<FHairStrandsTilePassVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FHairStrandsTileDebugPassPS> PixelShader(View.ShaderMap);

		PassParameters->RenderTargets[0] = FRenderTargetBinding(ColorTexture, ERenderTargetLoadAction::ELoad);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HairStrands::TileDebugPass(%s)", ToString(TileType)),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, VertexShader, PixelShader, Viewport, TileType](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FHairStrandsTilePassVS::FParameters ParametersVS = PassParameters->TileParameters;

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Max, BF_One, BF_One>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PassParameters->TileParameters.bRectPrimitive > 0 ? PT_RectList : PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

				RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
				RHICmdList.SetStreamSource(0, nullptr, 0);
				RHICmdList.DrawPrimitiveIndirect(PassParameters->TileParameters.TileIndirectBuffer->GetRHI(), FHairStrandsTiles::GetIndirectDrawArgOffset(TileType));
			});
	};

	DrawDebugTile(FHairStrandsTiles::ETileType::HairFull);
	DrawDebugTile(FHairStrandsTiles::ETileType::HairPartial);
	DrawDebugTile(FHairStrandsTiles::ETileType::Other);

	if (View.HairStrandsViewData.UniformBuffer || ShaderPrint::IsEnabled(View.ShaderPrintData))
	{
		static FHairStrandsTiles::ETileType TileType = FHairStrandsTiles::ETileType::HairAll;

		FHairStrandsTileDebugPrintPassCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsTileDebugPrintPassCS::FParameters>();
		Parameters->MaxResolution = FIntPoint(Viewport.Width(), Viewport.Height());
		Parameters->TileType = uint32(TileType);
		Parameters->TileGroupSize = TileData.GroupSize;
		Parameters->TileSize = TileData.TileSize;
		Parameters->TileCount = TileData.TileCount;
		Parameters->TileCountXY = TileData.TileCountXY;
		Parameters->bRectPrimitive = TileData.bRectPrimitive ? 1u : 0u;
		Parameters->HairStrands = View.HairStrandsViewData.UniformBuffer;
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintUniformBuffer);

		TShaderMapRef<FHairStrandsTileDebugPrintPassCS> ComputeShader(View.ShaderMap);
		ClearUnusedGraphResources(ComputeShader, Parameters);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::TileDebugPrint(%s)", ToString(TileType)),
			ComputeShader,
			Parameters,
			FIntVector(1, 1, 1));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FHairTileClearCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairTileClearCS);
	SHADER_USE_PARAMETER_STRUCT(FHairTileClearCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER(FIntPoint, TileCountXY)
		SHADER_PARAMETER(FIntPoint, ViewRectMin)
		SHADER_PARAMETER(uint32, TileSize)
		SHADER_PARAMETER(uint32, TileType)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint2>, TileDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HairTileCount)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTexture)
		RDG_BUFFER_ACCESS(TileIndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TILE_CLEAR"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairTileClearCS, "/Engine/Private/HairStrands/HairStrandsVisibilityTile.usf", "TileMainCS", SF_Compute);

void AddHairStrandsTileClearPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsTiles& TileData,
	FHairStrandsTiles::ETileType TileType,
	FRDGTextureRef OutTexture)
{
	if (!OutTexture || !TileData.IsValid() || TileType == FHairStrandsTiles::ETileType::Count)
	{
		return;
	}

	FHairTileClearCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairTileClearCS::FParameters>();
	PassParameters->Resolution = OutTexture->Desc.Extent;
	PassParameters->TileCountXY = TileData.TileCountXY;
	PassParameters->ViewRectMin = View.ViewRect.Min;
	PassParameters->TileSize = TileData.TileSize;
	PassParameters->TileType = ToIndex(TileType);
	PassParameters->TileCountBuffer = TileData.TileCountSRV;
	PassParameters->TileDataBuffer = TileData.GetTileBufferSRV(TileType);
	PassParameters->TileIndirectArgs = TileData.TileIndirectDispatchBuffer;
	PassParameters->OutTexture = GraphBuilder.CreateUAV(OutTexture);

	FHairTileClearCS::FPermutationDomain PermutationVector;
	TShaderMapRef<FHairTileClearCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::TileClear(%s)", ToString(TileType)),
		ComputeShader,
		PassParameters,
		PassParameters->TileIndirectArgs, TileData.GetIndirectDispatchArgOffset(TileType));
}
