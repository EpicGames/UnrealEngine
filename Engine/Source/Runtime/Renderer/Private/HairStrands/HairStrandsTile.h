// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairVisibilityRendering.h: Hair strands visibility buffer implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "GlobalShader.h"
#include "ShaderParameterStruct.h"
#include "HairStrandsData.h"

class FViewInfo;

class FHairStrandsTilePassVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsTilePassVS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsTilePassVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, ViewMin)
		SHADER_PARAMETER(int32, bRectPrimitive)
		SHADER_PARAMETER(uint32, TileType)
		SHADER_PARAMETER(FVector2f, ViewInvSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, TileDataBuffer)
		RDG_BUFFER_ACCESS(TileIndirectBuffer, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

FHairStrandsTilePassVS::FParameters GetHairStrandsTileParameters(const FViewInfo& InView, const FHairStrandsTiles& In, FHairStrandsTiles::ETileType TileType);

FHairStrandsTiles AddHairStrandsGenerateTilesPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef& InputTexture);

FHairStrandsTiles AddHairStrandsGenerateTilesPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FIntPoint& Extent);

void AddHairStrandsTileClearPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsTiles& TileData,
	FHairStrandsTiles::ETileType TileType,
	FRDGTextureRef OutTexture);

void AddHairStrandsDebugTilePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef& ColorTexture,
	const FHairStrandsTiles& TileData);