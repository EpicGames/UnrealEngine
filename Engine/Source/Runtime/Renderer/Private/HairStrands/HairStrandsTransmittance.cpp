// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsTransmittance.h"
#include "HairStrandsLUT.h"
#include "HairStrandsDeepShadow.h"
#include "HairStrandsVoxelization.h"
#include "HairStrandsRendering.h"

#include "BasePassRendering.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "SceneTextureParameters.h"
#include "RenderGraphUtils.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "ShaderPrintParameters.h"
#include "LightSceneInfo.h"
#include "ShaderPrint.h"

///////////////////////////////////////////////////////////////////////////////////////////////////

static int32 GDeepShadowDebugMode = 0;
static FAutoConsoleVariableRef CVarDeepShadowDebugMode(TEXT("r.HairStrands.DeepShadow.DebugMode"), GDeepShadowDebugMode, TEXT("Color debug mode for deep shadow"));
static uint32 GetDeepShadowDebugMode() { return uint32(FMath::Max(0, GDeepShadowDebugMode)); }

static int32 GDeepShadowKernelType = 2; // 0:linear, 1:PCF_2x2, 2: PCF_6x4, 3:PCSS
static float GDeepShadowKernelAperture = 1;
static FAutoConsoleVariableRef CVarDeepShadowKernelType(TEXT("r.HairStrands.DeepShadow.KernelType"), GDeepShadowKernelType, TEXT("Set the type of kernel used for evaluating hair transmittance, 0:linear, 1:PCF_2x2, 2: PCF_6x4, 3:PCSS, 4:PCF_6x6_Accurate"));
static FAutoConsoleVariableRef CVarDeepShadowKernelAperture(TEXT("r.HairStrands.DeepShadow.KernelAperture"), GDeepShadowKernelAperture, TEXT("Set the aperture angle, in degree, used by the kernel for evaluating the hair transmittance when using PCSS kernel"));

static uint32 GetDeepShadowKernelType() { return uint32(FMath::Max(0, GDeepShadowKernelType)); }
static float GetDeepShadowKernelAperture() { return GDeepShadowKernelAperture; }

static int32 GStrandHairShadowMaskKernelType = 4;
static FAutoConsoleVariableRef GVarDeepShadowShadowMaskKernelType(TEXT("r.HairStrands.DeepShadow.ShadowMaskKernelType"), GStrandHairShadowMaskKernelType, TEXT("Set the kernel type for filtering shadow cast by hair on opaque geometry (0:2x2, 1:4x4, 2:Gaussian8, 3:Gaussian16, 4:Gaussian8 with transmittance. Default is 4"));

static float GDeepShadowDensityScale = 2;	// Default is arbitrary, based on Mike asset
static float GDeepShadowDepthBiasScale = 0.05f;
static FAutoConsoleVariableRef CVarDeepShadowDensityScale(TEXT("r.HairStrands.DeepShadow.DensityScale"), GDeepShadowDensityScale, TEXT("Set density scale for compensating the lack of hair fiber in an asset"));
static FAutoConsoleVariableRef CVarDeepShadowDepthBiasScale(TEXT("r.HairStrands.DeepShadow.DepthBiasScale"), GDeepShadowDepthBiasScale, TEXT("Set depth bias scale for transmittance computation"));

static int32 GHairStrandsTransmittanceSuperSampling = 0;
static FAutoConsoleVariableRef CVarHairStrandsTransmittanceSuperSampling(TEXT("r.HairStrands.DeepShadow.SuperSampling"), GHairStrandsTransmittanceSuperSampling, TEXT("Evaluate transmittance with supersampling. This is expensive and intended to be used only in cine mode."), ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GHairStrandsTransmittanceMaskUseMipTraversal = 1;
static FAutoConsoleVariableRef CVarHairStrandsTransmittanceMaskUseMipTraversal(TEXT("r.HairStrands.DeepShadow.MipTraversal"), GHairStrandsTransmittanceMaskUseMipTraversal, TEXT("Evaluate transmittance using mip-map traversal (faster)."), ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GHairStrandsShadowRandomTraversalType = 2;
static FAutoConsoleVariableRef CVarHairStrandsShadowRandomTraversalType(TEXT("r.HairStrands.DeepShadow.RandomType"), GHairStrandsShadowRandomTraversalType, TEXT("Change how traversal jittering is initialized. Valid value are 0, 1, and 2. Each type makes different type of tradeoff."), ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GHairStrandsShadow_ShadowMaskPassType = 1;
static FAutoConsoleVariableRef CVarHairStrandsShadow_ShadowMaskPassType(TEXT("r.HairStrands.DeepShadow.ShadowMaskPassType"), GHairStrandsShadow_ShadowMaskPassType, TEXT("Change how shadow mask from hair onto opaque geometry is generated. 0: one pass per hair group, 1: one pass for all groups."), ECVF_Scalability | ECVF_RenderThreadSafe);

static int32 GHairStrandsShadow_UseScreenTrace = 1;
static FAutoConsoleVariableRef CVarHairStrandsShadow_HZBType(TEXT("r.HairStrands.DeepShadow.UseScreenTrace"), GHairStrandsShadow_UseScreenTrace, TEXT("Use screen trace for adding fine occlusion to hair transmission."), ECVF_Scalability | ECVF_RenderThreadSafe);

static float GHairStrandsShadow_ScreenTraceLength = 10.f;
static FAutoConsoleVariableRef CVarHairStrandsShadow_ScreenTraceLength(TEXT("r.HairStrands.DeepShadow.ScreenTraceLength"), GHairStrandsShadow_ScreenTraceLength, TEXT("Screen trace length for hair transmission."), ECVF_Scalability | ECVF_RenderThreadSafe);

static float GetDeepShadowDensityScale() { return FMath::Max(0.0f, GDeepShadowDensityScale); }
static float GetDeepShadowDepthBiasScale() { return FMath::Max(0.0f, GDeepShadowDepthBiasScale); }
///////////////////////////////////////////////////////////////////////////////////////////////////

enum class EHairTransmittancePassType : uint8
{
	PerLight,
	OnePass
};

static bool HasDeepShadowData(const FLightSceneInfo* LightSceneInfo, const FHairStrandsMacroGroupDatas& InDatas)
{
	for (const FHairStrandsMacroGroupData& MacroGroupData : InDatas)
	{
		for (const FHairStrandsDeepShadowData& DomData : MacroGroupData.DeepShadowDatas)
		{
			if (DomData.LightId == LightSceneInfo->Id)
				return true;
		}
	}

	return false;
}

FVector4f ComputeDeepShadowLayerDepths(float LayerDistribution)
{
	// LayerDistribution in [0..1]
	// Exponent in [1 .. 6.2]
	// Default LayerDistribution is 0.5, which is mapped onto exponent=3.1, making the last layer at depth 0.5f in clip space
	// Within this range the last layer's depth goes from 1 to 0.25 in clip space (prior to inverse Z)
	const float Exponent = FMath::Clamp(LayerDistribution, 0.f, 1.f) * 5.2f + 1;
	FVector4f Depths;
	Depths.X = FMath::Pow(0.2f, Exponent);
	Depths.Y = FMath::Pow(0.4f, Exponent);
	Depths.Z = FMath::Pow(0.6f, Exponent);
	Depths.W = FMath::Pow(0.8f, Exponent);
	return Depths;
}

static FVector4f GetLightTranslatedWorldPositionAndDirection(const FViewInfo& View, const FLightSceneInfo* LightSceneInfo)
{
	const FVector& TranslatedWorldOffset = View.ViewMatrices.GetPreViewTranslation();
	if (LightSceneInfo->Proxy->GetLightType() == ELightComponentType::LightType_Directional)
	{
		return FVector4f((FVector3f)LightSceneInfo->Proxy->GetDirection(), 0.f);
	}
	else
	{
		return FVector4f(FVector4f(LightSceneInfo->Proxy->GetPosition() + TranslatedWorldOffset), 1.0f); // LWC_TODO: precision loss
	}
}

struct FHairStrandsTransmittanceLightParams
{
	FVector4f TranslatedLightPosition_LightDirection = FVector4f(0, 0, 0, 0);
	uint32 LightChannelMask = 0;
	uint32 ShadowChannelMask = 0;
	float LightRadius = 0;
};

///////////////////////////////////////////////////////////////////////////////////////////////////
// Clear transmittance Mask

class FHairStrandsClearTransmittanceMaskCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsClearTransmittanceMaskCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsClearTransmittanceMaskCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, ElementCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutputMask)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLEAR"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsClearTransmittanceMaskCS, "/Engine/Private/HairStrands/HairStrandsDeepTransmittanceMask.usf", "MainCS", SF_Compute);

static void AddHairStrandsClearTransmittanceMaskPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	FRDGBufferRef OutTransmittanceMask)
{
	FHairStrandsClearTransmittanceMaskCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsClearTransmittanceMaskCS::FParameters>();
	Parameters->ElementCount = OutTransmittanceMask->Desc.NumElements;
	Parameters->OutputMask = GraphBuilder.CreateUAV(OutTransmittanceMask, FHairStrandsTransmittanceMaskData::Format);

	FHairStrandsClearTransmittanceMaskCS::FPermutationDomain PermutationVector;
	TShaderMapRef<FHairStrandsClearTransmittanceMaskCS> ComputeShader(ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::ClearTransmittanceMask"),
		ComputeShader,
		Parameters,
		FComputeShaderUtils::GetGroupCount(Parameters->ElementCount, 64));
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Transmittance buffer

static FRDGBufferRef CreateHairStrandsTransmittanceMaskBuffer(FRDGBuilder& GraphBuilder, uint32 NumElements)
{
	return GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(
		sizeof(uint32),
		NumElements),
		TEXT("Hair.TransmittanceNodeData"));
}

FHairStrandsTransmittanceMaskData CreateDummyHairStrandsTransmittanceMaskData(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap)
{
	FHairStrandsTransmittanceMaskData Out;
	Out.TransmittanceMask = CreateHairStrandsTransmittanceMaskBuffer(GraphBuilder, 1);
	AddHairStrandsClearTransmittanceMaskPass(GraphBuilder, ShaderMap, Out.TransmittanceMask);
	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Transmittance Mask from voxel

class FHairStrandsVoxelTransmittanceMaskCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsVoxelTransmittanceMaskCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsVoxelTransmittanceMaskCS, FGlobalShader);

	class FTransmittanceGroupSize : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_GROUP_SIZE", 32, 64);
	class FSuperSampling : SHADER_PERMUTATION_INT("PERMUTATION_SUPERSAMPLING", 2);
	class FTraversal : SHADER_PERMUTATION_INT("PERMUTATION_TRAVERSAL", 2);
	class FOnePass : SHADER_PERMUTATION_BOOL("PERMUTATION_ONE_PASS");
	using FPermutationDomain = TShaderPermutationDomain<FTransmittanceGroupSize, FSuperSampling, FTraversal, FOnePass>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualVoxelParameters, VirtualVoxel)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMap)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER(uint32, bUseScreenTrace)
		SHADER_PARAMETER(float, ScreenTraceLength)
		SHADER_PARAMETER(FVector4f, TranslatedLightPosition_LightDirection)
		SHADER_PARAMETER(FVector4f, ShadowChannelMask)
		SHADER_PARAMETER(uint32, LightChannelMask)
		SHADER_PARAMETER(float, LightRadius)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, ShadowMaskBitsTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RayMarchMaskTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutTransmittanceMask)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)

		RDG_BUFFER_ACCESS(IndirectArgsBuffer, ERHIAccess::IndirectArgs)

	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TRANSMITTANCE_VOXEL"), 1);
		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsVoxelTransmittanceMaskCS, "/Engine/Private/HairStrands/HairStrandsDeepTransmittanceMask.usf", "MainCS", SF_Compute);

// Transmittance mask using voxel volume
static FRDGBufferRef AddHairStrandsVoxelTransmittanceMaskPass(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfo& View, int32 ViewIndex,
	const EHairTransmittancePassType PassType,
	const FHairStrandsTransmittanceLightParams& Params,
	const uint32 NodeGroupSize,
	FRDGBufferRef IndirectArgsBuffer,
	FRDGTextureRef ShadowMaskTexture,
	FVirtualShadowMapArray* VirtualShadowMapArray = nullptr)
{
	check(HairStrands::HasViewHairStrandsVoxelData(View));
	check(NodeGroupSize == 64 || NodeGroupSize == 32);

	const uint32 MaxLightPerPass = 10u; // HAIR_TODO: Need to match the virtual shadow mask bits encoding
	const uint32 AverageLightPerPixel = PassType == EHairTransmittancePassType::OnePass ? MaxLightPerPass : 1u;
	FRDGBufferRef OutBuffer = CreateHairStrandsTransmittanceMaskBuffer(GraphBuilder, View.HairStrandsViewData.VisibilityData.MaxNodeCount * AverageLightPerPixel);

	FHairStrandsVoxelTransmittanceMaskCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsVoxelTransmittanceMaskCS::FParameters>();
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->SceneTextures = SceneTextures;
	Parameters->OutTransmittanceMask = GraphBuilder.CreateUAV(OutBuffer, FHairStrandsTransmittanceMaskData::Format);
	if (PassType == EHairTransmittancePassType::OnePass)
	{
		check(VirtualShadowMapArray != nullptr);
		Parameters->VirtualShadowMap = VirtualShadowMapArray->GetSamplingParameters(GraphBuilder, ViewIndex);
		Parameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		Parameters->RayMarchMaskTexture = nullptr;
		Parameters->ShadowMaskBitsTexture = ShadowMaskTexture ? ShadowMaskTexture : GSystemTextures.GetZeroUIntDummy(GraphBuilder);
	}
	else
	{
		Parameters->TranslatedLightPosition_LightDirection = Params.TranslatedLightPosition_LightDirection;
		Parameters->LightRadius = Params.LightRadius;
		Parameters->RayMarchMaskTexture = ShadowMaskTexture ? ShadowMaskTexture : GSystemTextures.GetWhiteDummy(GraphBuilder);
		Parameters->ShadowMaskBitsTexture = nullptr;
	}

	Parameters->ShadowChannelMask = FVector4f(0, 0, 0, 0);
	Parameters->ShadowChannelMask[FMath::Clamp<uint32>(Params.ShadowChannelMask, 0, 3)] = 1.0f;
	Parameters->LightChannelMask = Params.LightChannelMask;
	Parameters->IndirectArgsBuffer = IndirectArgsBuffer;
	Parameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
	Parameters->VirtualVoxel = HairStrands::BindHairStrandsVoxelUniformParameters(View);

	FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
	Parameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);

	const bool bIsSuperSampled = GHairStrandsTransmittanceSuperSampling > 0;
	const bool bIsMipTraversal = GHairStrandsTransmittanceMaskUseMipTraversal > 0;

	// Screen trace
	Parameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::ClosestHZB, View.HairStrandsViewData.VisibilityData.HairOnlyDepthFurthestHZBTexture, View.HairStrandsViewData.VisibilityData.HairOnlyDepthClosestHZBTexture);
	Parameters->bUseScreenTrace = GHairStrandsShadow_UseScreenTrace > 0? 1u : 0u;
	Parameters->ScreenTraceLength = FMath::Max(GHairStrandsShadow_ScreenTraceLength, 0.f);

	FHairStrandsVoxelTransmittanceMaskCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairStrandsVoxelTransmittanceMaskCS::FTransmittanceGroupSize>(NodeGroupSize);
	PermutationVector.Set<FHairStrandsVoxelTransmittanceMaskCS::FSuperSampling>(bIsSuperSampled ? 1 : 0);
	PermutationVector.Set<FHairStrandsVoxelTransmittanceMaskCS::FTraversal>(bIsMipTraversal ? 1 : 0);
	PermutationVector.Set<FHairStrandsVoxelTransmittanceMaskCS::FOnePass>(PassType == EHairTransmittancePassType::OnePass);
	TShaderMapRef<FHairStrandsVoxelTransmittanceMaskCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::TransmittanceMask(Voxel,%s)", PassType == EHairTransmittancePassType::OnePass ? TEXT("OnePass") : TEXT("PerLight")),
		ComputeShader,
		Parameters,
		IndirectArgsBuffer,
		0);

	return OutBuffer;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Transmittance Mask from deep shadow

class FHairStrandsDeepShadowTransmittanceMaskCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsDeepShadowTransmittanceMaskCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsDeepShadowTransmittanceMaskCS, FGlobalShader);

	class FTransmittanceGroupSize : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_GROUP_SIZE", 32, 64);
	using FPermutationDomain = TShaderPermutationDomain<FTransmittanceGroupSize>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)

		SHADER_PARAMETER(FIntPoint, DeepShadow_AtlasResolution)
		SHADER_PARAMETER(float, LightRadius)
		SHADER_PARAMETER(FVector4f, TranslatedLightPosition_LightDirection)
		SHADER_PARAMETER(uint32, LightChannelMask)
		SHADER_PARAMETER(FVector4f, ShadowChannelMask)
		SHADER_PARAMETER(FVector4f, DeepShadow_LayerDepths)
		SHADER_PARAMETER(float, DeepShadow_DepthBiasScale)
		SHADER_PARAMETER(float, DeepShadow_DensityScale)
		SHADER_PARAMETER(float, DeepShadow_KernelAperture)
		SHADER_PARAMETER(uint32, DeepShadow_KernelType)
		SHADER_PARAMETER(uint32, DeepShadow_DebugMode)
		SHADER_PARAMETER(FMatrix44f, DeepShadow_ShadowToWorld)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RayMarchMaskTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DeepShadow_FrontDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DeepShadow_DomTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, DeepShadow_ViewInfoBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, DeepShadow_AtlasSlotIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutTransmittanceMask)

		RDG_BUFFER_ACCESS(IndirectArgsBuffer, ERHIAccess::IndirectArgs)

	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_TRANSMITTANCE_DEEPSHADOW"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsDeepShadowTransmittanceMaskCS, "/Engine/Private/HairStrands/HairStrandsDeepTransmittanceMask.usf", "MainCS", SF_Compute);

struct FHairStrandsDeepShadowTransmittanceLightParams : FHairStrandsTransmittanceLightParams
{
	//FIntVector4 DeepShadow_AtlasSlotOffsets_AtlasSlotIndex[FHairStrandsDeepShadowData::MaxMacroGroupCount];
	FRDGBufferSRVRef DeepShadow_ViewInfoBuffer = nullptr;
	FRDGBufferSRVRef DeepShadow_AtlasSlotIndexBuffer = nullptr;
	FIntPoint DeepShadow_AtlasResolution = FIntPoint(0, 0);
	FVector4f DeepShadow_LayerDepths = FVector4f(0, 0, 0, 0);
	float DeepShadow_DepthBiasScale = 0;
	float DeepShadow_DensityScale = 0;
	FMatrix DeepShadow_ShadowToWorld = FMatrix::Identity;
	uint32 OutputChannel = ~0;
	uint32 ShadowChannelMask = 0;

	FRDGTextureRef DeepShadow_FrontDepthTexture = nullptr;
	FRDGTextureRef DeepShadow_DomTexture = nullptr;
};

// Transmittance mask using deep shadow
static FRDGBufferRef AddHairStrandsDeepShadowTransmittanceMaskPass(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfo& View,
	const FHairStrandsDeepShadowTransmittanceLightParams& Params,
	const uint32 NodeGroupSize,
	FRDGBufferRef IndirectArgsBuffer,
	FRDGTextureRef ScreenShadowMaskSubPixelTexture)
{
	FRDGBufferRef OutBuffer = CreateHairStrandsTransmittanceMaskBuffer(GraphBuilder, View.HairStrandsViewData.VisibilityData.MaxNodeCount);

	FHairStrandsDeepShadowTransmittanceMaskCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsDeepShadowTransmittanceMaskCS::FParameters>();
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->SceneTextures = SceneTextures;
	Parameters->DeepShadow_FrontDepthTexture = Params.DeepShadow_FrontDepthTexture;
	Parameters->DeepShadow_DomTexture = Params.DeepShadow_DomTexture;
	Parameters->OutTransmittanceMask = GraphBuilder.CreateUAV(OutBuffer, FHairStrandsTransmittanceMaskData::Format);
	Parameters->LightChannelMask = Params.LightChannelMask;
	Parameters->DeepShadow_AtlasResolution = Params.DeepShadow_AtlasResolution;
	Parameters->TranslatedLightPosition_LightDirection = Params.TranslatedLightPosition_LightDirection;
	Parameters->LightRadius = Params.LightRadius;
	Parameters->DeepShadow_DepthBiasScale = Params.DeepShadow_DepthBiasScale;
	Parameters->DeepShadow_DensityScale = Params.DeepShadow_DensityScale;
	Parameters->DeepShadow_KernelAperture = GetDeepShadowKernelAperture();
	Parameters->DeepShadow_KernelType = GetDeepShadowKernelType();
	Parameters->DeepShadow_DebugMode = GetDeepShadowDebugMode();
	Parameters->DeepShadow_LayerDepths = Params.DeepShadow_LayerDepths;
	Parameters->DeepShadow_ShadowToWorld = FMatrix44f(Params.DeepShadow_ShadowToWorld);		// LWC_TODO: Precision loss
	Parameters->IndirectArgsBuffer = IndirectArgsBuffer;
	Parameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
	Parameters->DeepShadow_ViewInfoBuffer = Params.DeepShadow_ViewInfoBuffer;
	Parameters->DeepShadow_AtlasSlotIndexBuffer = Params.DeepShadow_AtlasSlotIndexBuffer;
	Parameters->RayMarchMaskTexture = ScreenShadowMaskSubPixelTexture ? ScreenShadowMaskSubPixelTexture : GraphBuilder.RegisterExternalTexture(GSystemTextures.WhiteDummy);
	Parameters->ShadowChannelMask = FVector4f(0, 0, 0, 0);
	Parameters->ShadowChannelMask[FMath::Clamp<uint32>(Params.ShadowChannelMask, 0, 3)] = 1.0f;
	
	check(NodeGroupSize == 64 || NodeGroupSize == 32);
	FHairStrandsDeepShadowTransmittanceMaskCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairStrandsDeepShadowTransmittanceMaskCS::FTransmittanceGroupSize>(NodeGroupSize);

	TShaderMapRef<FHairStrandsDeepShadowTransmittanceMaskCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::TransmittanceMask(DeepShadow)"),
		ComputeShader,
		Parameters,
		IndirectArgsBuffer,
		0);

	return OutBuffer;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Opaque Mask from voxel volume

class FHairStrandsVoxelShadowMaskPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsVoxelShadowMaskPS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsVoxelShadowMaskPS, FGlobalShader);

	class FOnePass : SHADER_PERMUTATION_BOOL("PERMUTATION_USE_ONEPASS");
	using FPermutationDomain = TShaderPermutationDomain<FOnePass>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		
		SHADER_PARAMETER(FVector4f, Voxel_TranslatedLightPosition_LightDirection)
		SHADER_PARAMETER(uint32, Voxel_MacroGroupCount)
		SHADER_PARAMETER(uint32, Voxel_MacroGroupId)
		SHADER_PARAMETER(uint32, Voxel_RandomType)
		SHADER_PARAMETER(uint32, EncodingType)
		SHADER_PARAMETER(float, FadeAlpha)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RayMarchMaskTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowSampler)

		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualVoxelParameters, VirtualVoxel)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_B8G8R8A8);
		OutEnvironment.SetDefine(TEXT("SHADER_SHADOWMASK_VOXEL"), 1);
		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsVoxelShadowMaskPS, "/Engine/Private/HairStrands/HairStrandsDeepShadowMask.usf", "MainPS", SF_Pixel);

// Opaque mask from voxels
static void AddHairStrandsVoxelShadowMaskPass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef SceneDepthTexture,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	const float FadeAlpha,
	const uint32 EncodingType,
	const bool bProjectingForForwardShading,
	FRDGTextureRef& OutShadowMask)
{
	check(LightSceneInfo);
	check(OutShadowMask);
	check(HairStrands::HasViewHairStrandsVoxelData(View));

	// Copy the shadow mask texture to read its content, and early out voxel traversal
	FRDGTextureRef RayMarchMask = nullptr;
	{
		FRDGTextureDesc Desc = OutShadowMask->Desc;
		Desc.Flags |= TexCreate_ShaderResource;
		RayMarchMask = GraphBuilder.CreateTexture(Desc, TEXT("Hair.RayMarchMask"));
		FRHICopyTextureInfo CopyInfo;
		CopyInfo.Size = OutShadowMask->Desc.GetSize();
		AddCopyTexturePass(GraphBuilder, OutShadowMask, RayMarchMask, CopyInfo);
	}

	const bool bOnePass = GHairStrandsShadow_ShadowMaskPassType > 0 && View.HairStrandsViewData.MacroGroupDatas.Num() > 1;
	const FIntPoint OutputResolution = SceneDepthTexture->Desc.Extent;
	FHairStrandsVoxelShadowMaskPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairStrandsVoxelShadowMaskPS::FOnePass>(bOnePass);

	TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FHairStrandsVoxelShadowMaskPS> PixelShader(View.ShaderMap, PermutationVector);
	const FGlobalShaderMap* GlobalShaderMap = View.ShaderMap;
	const FIntRect Viewport = View.ViewRect;
	const uint32 OutputChannel = bProjectingForForwardShading ? LightSceneInfo->GetDynamicShadowMapChannel() : ~0;
	const FIntPoint Resolution = OutShadowMask->Desc.Extent;
	
	FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
	for (int32 GroupIt=0, GroupCount=View.HairStrandsViewData.MacroGroupDatas.Num(); GroupIt < GroupCount; ++GroupIt)
	{
		if (bOnePass && GroupIt > 0)
		{
			return;
		}

		const FHairStrandsMacroGroupData& MacroGroupData = View.HairStrandsViewData.MacroGroupDatas[GroupIt];

		FHairStrandsVoxelShadowMaskPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsVoxelShadowMaskPS::FParameters>();
		Parameters->FadeAlpha = FadeAlpha;
		Parameters->EncodingType = EncodingType;
		Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
		Parameters->SceneDepthTexture = SceneDepthTexture;	
		Parameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
		Parameters->VirtualVoxel = HairStrands::BindHairStrandsVoxelUniformParameters(View);
		Parameters->LinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Parameters->ShadowSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Parameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);
		Parameters->Voxel_TranslatedLightPosition_LightDirection = GetLightTranslatedWorldPositionAndDirection(View, LightSceneInfo);
		Parameters->Voxel_MacroGroupId	= MacroGroupData.MacroGroupId;
		Parameters->Voxel_MacroGroupCount = GroupCount;
		Parameters->Voxel_RandomType = FMath::Clamp(GHairStrandsShadowRandomTraversalType, 0, 2);	
		Parameters->RenderTargets[0] = FRenderTargetBinding(OutShadowMask, ERenderTargetLoadAction::ELoad);
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);
		Parameters->RayMarchMaskTexture = RayMarchMask;

		ClearUnusedGraphResources(PixelShader, Parameters);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HairStrands::ShadowMask(Voxel,%s)", bOnePass ? TEXT("OnePass") : TEXT("PerGroup")),
			Parameters,
			ERDGPassFlags::Raster,
			[Parameters, VertexShader, PixelShader, Viewport, Resolution, OutputChannel](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			switch (OutputChannel)
			{
			case 0:  GraphicsPSOInit.BlendState = TStaticBlendState<CW_RED,   BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
			case 1:  GraphicsPSOInit.BlendState = TStaticBlendState<CW_GREEN, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
			case 2:  GraphicsPSOInit.BlendState = TStaticBlendState<CW_BLUE,  BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
			case 3:  GraphicsPSOInit.BlendState = TStaticBlendState<CW_ALPHA, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
			default: GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA,  BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
			}
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);

			DrawRectangle(
				RHICmdList,
				0, 0,
				Viewport.Width(), Viewport.Height(),
				Viewport.Min.X, Viewport.Min.Y,
				Viewport.Width(), Viewport.Height(),
				Viewport.Size(),
				Resolution,
				VertexShader,
				EDRF_UseTriangleOptimization);
		});
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Opaque Mask from deep shadow

class FHairStrandsDeepShadowMaskPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsDeepShadowMaskPS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsDeepShadowMaskPS, FGlobalShader);

	class FKernelType : SHADER_PERMUTATION_INT("PERMUTATION_KERNEL_TYPE", 5);
	using FPermutationDomain = TShaderPermutationDomain<FKernelType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		
		SHADER_PARAMETER(FIntPoint, DeepShadow_AtlasResolution)
		SHADER_PARAMETER(float, DeepShadow_DepthBiasScale)
		SHADER_PARAMETER(float, DeepShadow_DensityScale)
		SHADER_PARAMETER(FVector4f, DeepShadow_LayerDepths)
		SHADER_PARAMETER(float, FadeAlpha)
		SHADER_PARAMETER(uint32, EncodingType)
		SHADER_PARAMETER(uint32, EffectiveAtlasSlotCount)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, RayMarchMaskTexture)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, DeepShadow_ViewInfoBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, DeepShadow_AtlasSlotIndexBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DeepShadow_FrontDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DeepShadow_DomTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, ShadowSampler)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_B8G8R8A8);
		OutEnvironment.SetDefine(TEXT("SHADER_SHADOWMASK_DEEPSHADOW"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsDeepShadowMaskPS, "/Engine/Private/HairStrands/HairStrandsDeepShadowMask.usf", "MainPS", SF_Pixel);


struct FHairStrandsDeepShadowParams
{
	uint32			OutputChannel = ~0;
	FRDGBufferSRVRef DeepShadow_ViewInfoBuffer = nullptr;
	FRDGBufferSRVRef DeepShadow_AtlasSlotIndexBuffer = nullptr;
	FIntPoint		DeepShadow_AtlasResolution;
	FRDGTextureRef	DeepShadow_FrontDepthTexture = nullptr;
	FRDGTextureRef	DeepShadow_LayerTexture = nullptr;
	float			DeepShadow_DepthBiasScale = 1;
	float			DeepShadow_DensityScale = 1;
	FVector4f		DeepShadow_LayerDepths = FVector4f(0, 0, 0, 0);
};

// Opaque mask with deep shadow
static void AddHairStrandsDeepShadowMaskPass(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef SceneDepthTexture,
	const FViewInfo& View,
	const FHairStrandsDeepShadowParams& Params,
	const float FadeAlpha,
	const uint32 EncodingType,
	const uint32 EffectiveAtlasSlotCount,
	FRDGTextureRef& OutShadowMask)
{
	check(OutShadowMask);

	FHairStrandsDeepShadowMaskPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsDeepShadowMaskPS::FParameters>();
	Parameters->FadeAlpha = FadeAlpha;
	Parameters->EncodingType = EncodingType;
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->SceneDepthTexture = SceneDepthTexture;
	Parameters->DeepShadow_FrontDepthTexture = Params.DeepShadow_FrontDepthTexture;
	Parameters->DeepShadow_DomTexture = Params.DeepShadow_LayerTexture;
	Parameters->LinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters->ShadowSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters->DeepShadow_DepthBiasScale = Params.DeepShadow_DepthBiasScale;
	Parameters->DeepShadow_DensityScale = Params.DeepShadow_DensityScale;
	Parameters->DeepShadow_LayerDepths = Params.DeepShadow_LayerDepths;
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutShadowMask, ERenderTargetLoadAction::ELoad);
	Parameters->DeepShadow_AtlasResolution = Params.DeepShadow_AtlasResolution;
	Parameters->EffectiveAtlasSlotCount = EffectiveAtlasSlotCount;
	Parameters->DeepShadow_AtlasSlotIndexBuffer = Params.DeepShadow_AtlasSlotIndexBuffer;
	Parameters->DeepShadow_ViewInfoBuffer = Params.DeepShadow_ViewInfoBuffer;
	Parameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
	if (ShaderPrint::IsValid(View.ShaderPrintData))
	{
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);
	}

	FRDGTextureRef RayMarchMask = nullptr;
	{
		FRDGTextureDesc Desc = OutShadowMask->Desc;
		Desc.Flags |= TexCreate_ShaderResource;
		RayMarchMask = GraphBuilder.CreateTexture(Desc, TEXT("Hair.RayMarchMask"));
		FRHICopyTextureInfo CopyInfo;
		CopyInfo.Size = OutShadowMask->Desc.GetSize();
		AddCopyTexturePass(GraphBuilder, OutShadowMask, RayMarchMask, CopyInfo);
	}
	Parameters->RayMarchMaskTexture = RayMarchMask;

	FHairStrandsDeepShadowMaskPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairStrandsDeepShadowMaskPS::FKernelType>(FMath::Clamp(GStrandHairShadowMaskKernelType, 0, 4));

	const FIntPoint OutputResolution = SceneDepthTexture->Desc.Extent;
	TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FHairStrandsDeepShadowMaskPS> PixelShader(View.ShaderMap, PermutationVector);
	const FGlobalShaderMap* GlobalShaderMap = View.ShaderMap;
	const FIntRect Viewport = View.ViewRect;
	const uint32 OutputChannel = Params.OutputChannel;

	ClearUnusedGraphResources(PixelShader, Parameters);
	FIntPoint Resolution = OutShadowMask->Desc.Extent;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::ShadowMask"),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, VertexShader, PixelShader, Viewport, Resolution, OutputChannel](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		switch (OutputChannel)
		{
		case 0:  GraphicsPSOInit.BlendState = TStaticBlendState<CW_RED, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
		case 1:  GraphicsPSOInit.BlendState = TStaticBlendState<CW_GREEN, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
		case 2:  GraphicsPSOInit.BlendState = TStaticBlendState<CW_BLUE, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
		case 3:  GraphicsPSOInit.BlendState = TStaticBlendState<CW_ALPHA, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
		default: GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Min, BF_One, BF_One, BO_Min, BF_One, BF_One>::GetRHI(); break; // Min Operator
		}
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			Viewport.Width(), Viewport.Height(),
			Viewport.Min.X, Viewport.Min.Y,
			Viewport.Width(), Viewport.Height(),
			Viewport.Size(),
			Resolution,
			VertexShader,
			EDRF_UseTriangleOptimization);
	});
}

///////////////////////////////////////////////////////////////////////////////////////////////////

static FHairStrandsTransmittanceMaskData InternalRenderHairStrandsTransmittanceMask(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, int32 ViewIndex,
	const FLightSceneInfo* LightSceneInfo,
	const FHairStrandsVisibilityData& VisibilityData,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FHairStrandsDeepShadowResources& DeepShadowResources,
	const FHairStrandsVoxelResources& VoxelResources,
	const bool bProjectingForForwardShading,
	FRDGTextureRef ScreenShadowMaskSubPixelTexture)
{
	FHairStrandsTransmittanceMaskData Out;
	if (MacroGroupDatas.Num() == 0)
		return Out;

	if (!HasDeepShadowData(LightSceneInfo, MacroGroupDatas) && !IsHairStrandsVoxelizationEnable(View.GetShaderPlatform()))
		return Out;

	DECLARE_GPU_STAT(HairStrandsTransmittanceMask);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsTransmittanceMask, "HairStrands::TransmittanceMask");

	// Note: GbufferB.a store the shading model on the 4 lower bits (MATERIAL_SHADINGMODEL_HAIR)
	FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

	bool bHasFoundLight = false;
	if (!IsHairStrandsForVoxelTransmittanceAndShadowEnable(View.GetShaderPlatform()))
	{
		FHairStrandsDeepShadowTransmittanceLightParams Params;
		Params.DeepShadow_DensityScale = GetDeepShadowDensityScale();
		Params.DeepShadow_DepthBiasScale = GetDeepShadowDepthBiasScale();

		TArray<uint32> AtlasSlotIndexBuffer;
		AtlasSlotIndexBuffer.Reserve(MacroGroupDatas.Num());
		for (const FHairStrandsMacroGroupData& MacroGroupData : MacroGroupDatas)
		{
			uint32 AtlasSlotIndex = 0;
			for (const FHairStrandsDeepShadowData& DeepShadowData : MacroGroupData.DeepShadowDatas)
			{
				if (DeepShadowData.LightId == LightSceneInfo->Id)
				{
					bHasFoundLight = true;
					Params.TranslatedLightPosition_LightDirection = GetLightTranslatedWorldPositionAndDirection(View, LightSceneInfo);
					Params.DeepShadow_FrontDepthTexture = DeepShadowResources.DepthAtlasTexture;
					Params.DeepShadow_DomTexture = DeepShadowResources.LayersAtlasTexture;
					Params.DeepShadow_AtlasResolution = DeepShadowData.AtlasResolution;
					Params.LightRadius = 0;
					Params.LightChannelMask = LightSceneInfo->Proxy->GetLightingChannelMask();
					Params.ShadowChannelMask = bProjectingForForwardShading ? LightSceneInfo->GetDynamicShadowMapChannel() : 0;
					Params.DeepShadow_LayerDepths = ComputeDeepShadowLayerDepths(DeepShadowData.LayerDistribution);
					AtlasSlotIndex = DeepShadowData.AtlasSlotIndex;
				}
			}
			AtlasSlotIndexBuffer.Add(AtlasSlotIndex);
		}

		if (bHasFoundLight)
		{
			check(Params.DeepShadow_FrontDepthTexture);
			check(Params.DeepShadow_DomTexture);

			FRDGBufferRef DeepShadowAtlasSlotIndexBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("Hair.DeepShadow.AtlasSlotIndexBuffer"), 4u, AtlasSlotIndexBuffer.Num(), AtlasSlotIndexBuffer.GetData(), 4u * AtlasSlotIndexBuffer.Num());			
			Params.DeepShadow_ViewInfoBuffer = GraphBuilder.CreateSRV(DeepShadowResources.DeepShadowViewInfoBuffer);
			Params.DeepShadow_AtlasSlotIndexBuffer = GraphBuilder.CreateSRV(DeepShadowAtlasSlotIndexBuffer);

			Out.TransmittanceMask = AddHairStrandsDeepShadowTransmittanceMaskPass(
				GraphBuilder,
				SceneTextures,
				View,
				Params,
				VisibilityData.NodeGroupSize,
				VisibilityData.NodeIndirectArg,
				ScreenShadowMaskSubPixelTexture);
		}
	}

	if (!bHasFoundLight && VoxelResources.IsValid())
	{
		FLightRenderParameters LightParameters;
		LightSceneInfo->Proxy->GetLightShaderParameters(LightParameters);

		FHairStrandsTransmittanceLightParams Params;
		Params.TranslatedLightPosition_LightDirection = GetLightTranslatedWorldPositionAndDirection(View, LightSceneInfo);
		Params.LightChannelMask = LightSceneInfo->Proxy->GetLightingChannelMask();
		Params.ShadowChannelMask = bProjectingForForwardShading ? LightSceneInfo->GetDynamicShadowMapChannel() : 0;
		Params.LightRadius = FMath::Max(LightParameters.SourceLength, LightParameters.SourceRadius);

		Out.TransmittanceMask = AddHairStrandsVoxelTransmittanceMaskPass(
			GraphBuilder,
			SceneTextures,
			View, ViewIndex,
			EHairTransmittancePassType::PerLight,
			Params,
			VisibilityData.NodeGroupSize,
			VisibilityData.NodeIndirectArg,
			ScreenShadowMaskSubPixelTexture);
	}

	return Out;
}
	
FHairStrandsTransmittanceMaskData RenderHairStrandsOnePassTransmittanceMask(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, int32 ViewIndex,
	FRDGTextureRef ShadowMaskBits,
	FVirtualShadowMapArray& VirtualShadowMapArray)
{
	FHairStrandsTransmittanceMaskData Out;
	if (HairStrands::HasViewHairStrandsData(View) && View.HairStrandsViewData.MacroGroupDatas.Num() > 0)
	{
		DECLARE_GPU_STAT(HairStrandsOnePassTransmittanceMask);
		RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsOnePassTransmittanceMask, "HairStrands::TransmittanceMask(OnePass)");

		if (HairStrands::HasViewHairStrandsVoxelData(View))
		{
			// Note: GbufferB.a store the shading model on the 4 lower bits (MATERIAL_SHADINGMODEL_HAIR)
			FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);
			FHairStrandsTransmittanceLightParams DummyParams;

			Out.TransmittanceMask = AddHairStrandsVoxelTransmittanceMaskPass(
				GraphBuilder,
				SceneTextures,
				View, ViewIndex,
				EHairTransmittancePassType::OnePass,
				DummyParams,
				View.HairStrandsViewData.VisibilityData.NodeGroupSize,
				View.HairStrandsViewData.VisibilityData.NodeIndirectArg,
				ShadowMaskBits,
				&VirtualShadowMapArray);
		}
	}
	return Out;
}

FHairStrandsTransmittanceMaskData RenderHairStrandsTransmittanceMask(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, int32 ViewIndex,
	const FLightSceneInfo* LightSceneInfo,
	const bool bProjectingForForwardShading,
	FRDGTextureRef ScreenShadowMaskSubPixelTexture)
{
	FHairStrandsTransmittanceMaskData TransmittanceMaskData;
	if (HairStrands::HasViewHairStrandsData(View))
	{
		TransmittanceMaskData = InternalRenderHairStrandsTransmittanceMask(
			GraphBuilder, 
			View, ViewIndex,
			LightSceneInfo, 
			View.HairStrandsViewData.VisibilityData,
			View.HairStrandsViewData.MacroGroupDatas,
			View.HairStrandsViewData.DeepShadowResources,
			View.HairStrandsViewData.VirtualVoxelResources,
			bProjectingForForwardShading,
			ScreenShadowMaskSubPixelTexture);
	}
	return TransmittanceMaskData;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

static void InternalRenderHairStrandsShadowMask(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const uint32 ViewIndex,
	const FLightSceneInfo* LightSceneInfo,
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FHairStrandsVisibilityData& InVisibilityData,
	const FHairStrandsMacroGroupDatas& InMacroGroupDatas,
	const FHairStrandsDeepShadowResources& DeepShadowResources,
	const FHairStrandsVoxelResources& VoxelResources,
	const bool bProjectingForForwardShading, 
	FRDGTextureRef OutShadowMask)
{
	if (InMacroGroupDatas.Num() == 0)
		return;

	if (!HasDeepShadowData(LightSceneInfo, InMacroGroupDatas) && !IsHairStrandsVoxelizationEnable(View.GetShaderPlatform()))
		return;

	DECLARE_GPU_STAT(HairStrandsOpaqueMask);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsOpaqueMask, "HairStrands::OpaqueShadowMask");

	const FMinimalSceneTextures& SceneTextures = View.GetSceneTextures();

	const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ShadowInfos = VisibleLightInfos[LightSceneInfo->Id].AllProjectedShadows;
	float FadeAlpha = 1.0f; 
	bool bIsWholeSceneDirectionalLightShadow = false;
	bool bIsWholeSceneLocalLightShadow = false;
	if (ShadowInfos.Num() > 0)
	{
		FadeAlpha = ShadowInfos[0]->FadeAlphas[ViewIndex];
		bIsWholeSceneDirectionalLightShadow = ShadowInfos[0]->IsWholeSceneDirectionalShadow();
		bIsWholeSceneLocalLightShadow = ShadowInfos[0]->IsWholeScenePointLightShadow();
	}

	uint32 EncodingType = 0;
	if (bIsWholeSceneDirectionalLightShadow)
	{
		EncodingType = 1;
	}
	else if (bIsWholeSceneLocalLightShadow)
	{
		EncodingType = 2;
	}

	// 1. Build list of all the macrogroup affected by this light
	bool bHasDeepShadow = false;
	TArray<uint32> AtlasSlotIndexData;
	AtlasSlotIndexData.Reserve(InMacroGroupDatas.Num());
	float LayerDistribution = 0;
	FIntPoint AtlasResolution = FIntPoint::ZeroValue;
	if (!IsHairStrandsForVoxelTransmittanceAndShadowEnable(View.GetShaderPlatform()))
	{
		for (const FHairStrandsMacroGroupData& MacroGroupData : InMacroGroupDatas)
		{
			uint32 AtlasSlotIndex = 0;
			for (const FHairStrandsDeepShadowData& DomData : MacroGroupData.DeepShadowDatas)
			{
				if (DomData.LightId == LightSceneInfo->Id)
				{
					bHasDeepShadow = true;
					LayerDistribution = DomData.LayerDistribution;
					AtlasSlotIndexData.Add(DomData.AtlasSlotIndex);
					AtlasResolution = DomData.AtlasResolution;
				}
			}
		}
	}

	// 2. Render deep shadow mask if any
	if (bHasDeepShadow)
	{
		FRDGBufferRef AtlasSlotIndexBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("Hair.DeepShadow.AtlasSlotIndexBuffer"), 4u, AtlasSlotIndexData.Num(), AtlasSlotIndexData.GetData(), 4u * AtlasSlotIndexData.Num());			

		FHairStrandsDeepShadowParams Params;
		Params.DeepShadow_AtlasSlotIndexBuffer = GraphBuilder.CreateSRV(AtlasSlotIndexBuffer);
		Params.DeepShadow_ViewInfoBuffer = GraphBuilder.CreateSRV(DeepShadowResources.DeepShadowViewInfoBuffer);
		Params.DeepShadow_AtlasResolution = AtlasResolution;
		Params.DeepShadow_FrontDepthTexture = DeepShadowResources.DepthAtlasTexture;
		Params.DeepShadow_LayerTexture = DeepShadowResources.LayersAtlasTexture;
		Params.DeepShadow_DepthBiasScale = GetDeepShadowDepthBiasScale();
		Params.DeepShadow_DensityScale = GetDeepShadowDensityScale();
		Params.DeepShadow_LayerDepths = ComputeDeepShadowLayerDepths(LayerDistribution);
		Params.OutputChannel = bProjectingForForwardShading ? LightSceneInfo->GetDynamicShadowMapChannel() : ~0;

		AddHairStrandsDeepShadowMaskPass(
			GraphBuilder,
			SceneTextures.Depth.Resolve,
			View,
			Params,
			FadeAlpha,
			EncodingType,
			AtlasSlotIndexData.Num() /* EffectiveAtlasSlotCount */,
			OutShadowMask);
	}

	// 3. If there is no deep shadow for this light, fallback on the voxel representation
	if (!bHasDeepShadow && HairStrands::HasViewHairStrandsVoxelData(View))
	{
		AddHairStrandsVoxelShadowMaskPass(
			GraphBuilder,
			SceneTextures.Depth.Resolve,
			View,
			LightSceneInfo,
			FadeAlpha,
			EncodingType,
			bProjectingForForwardShading,
			OutShadowMask);
	}
}

void RenderHairStrandsShadowMask(
	FRDGBuilder& GraphBuilder,
	const TArray<FViewInfo>& Views,
	const FLightSceneInfo* LightSceneInfo,
	const FVisibleLightInfoArray& VisibleLightInfos,
	const bool bProjectingForForwardShading,
	FRDGTextureRef OutShadowMask)
{
	if (Views.Num() == 0 || OutShadowMask == nullptr)
	{
		return;
	}

	uint32 ViewIndex = 0;
	for (const FViewInfo& View : Views)
	{
		if (HairStrands::HasViewHairStrandsData(View))
		{
			check(View.HairStrandsViewData.VisibilityData.CoverageTexture);
			InternalRenderHairStrandsShadowMask(
				GraphBuilder,
				View,
				ViewIndex++,
				LightSceneInfo,
				VisibleLightInfos,
				View.HairStrandsViewData.VisibilityData,
				View.HairStrandsViewData.MacroGroupDatas,
				View.HairStrandsViewData.DeepShadowResources,
				View.HairStrandsViewData.VirtualVoxelResources,
				bProjectingForForwardShading,
				OutShadowMask);
		}
	}
}

void RenderHairStrandsDeepShadowMask(
	FRDGBuilder& GraphBuilder,
	const TArray<FViewInfo>& Views,
	const FLightSceneInfo* LightSceneInfo,
	const FVisibleLightInfoArray& VisibleLightInfos,
	FRDGTextureRef OutShadowMask)
{
	if (Views.Num() == 0 || OutShadowMask == nullptr)
	{
		return;
	}

	// Render only light with deep shadow
	if (!LightSceneInfo || !LightSceneInfo->Proxy->CastsHairStrandsDeepShadow())
	{
		return;
	}

	uint32 ViewIndex = 0;
	for (const FViewInfo& View : Views)
	{
		if (HairStrands::HasViewHairStrandsData(View))
		{
			check(View.HairStrandsViewData.VisibilityData.CoverageTexture);
			InternalRenderHairStrandsShadowMask(
				GraphBuilder, 
				View, 
				ViewIndex++,
				LightSceneInfo, 
				VisibleLightInfos,
				View.HairStrandsViewData.VisibilityData,
				View.HairStrandsViewData.MacroGroupDatas,
				View.HairStrandsViewData.DeepShadowResources,
				View.HairStrandsViewData.VirtualVoxelResources,
				false,
				OutShadowMask);
		}
	}
}
