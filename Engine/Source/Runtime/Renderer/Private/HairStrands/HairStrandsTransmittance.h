// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsTransmittance.h: Hair strands transmittance evaluation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "ScenePrivateBase.h"

class FLightSceneInfo;
class FViewInfo;
class FVirtualShadowMapArray;
class FVisibleLightInfo;

struct FHairStrandsTransmittanceMaskData
{
	static const EPixelFormat Format = PF_R32_UINT;
	FRDGBufferRef TransmittanceMask = nullptr;
};

/// Write opaque hair shadow onto screen shadow mask to have fine hair details cast onto opaque geometries
void RenderHairStrandsShadowMask(
	FRDGBuilder& GraphBuilder,
	const TArray<FViewInfo>& Views,
	const FLightSceneInfo* LightSceneInfo,
	const FVisibleLightInfoArray& VisibleLightInfos,
	const bool bProjectingForForwardShading,
	FRDGTextureRef ScreenShadowMaskTexture); 

/// Write opaque hair shadow onto screen shadow mask to have fine hair details cast onto opaque geometries (deep shadow caster only)
void RenderHairStrandsDeepShadowMask(
	FRDGBuilder& GraphBuilder,
	const TArray<FViewInfo>& Views,
	const FLightSceneInfo* LightSceneInfo,
	const FVisibleLightInfoArray& VisibleLightInfos,
	FRDGTextureRef OutShadowMask);

/// Output hair transmittance per hair sample for a given light
FHairStrandsTransmittanceMaskData RenderHairStrandsTransmittanceMask(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, int32 ViewIndex,
	const class FLightSceneInfo* LightSceneInfo,
	const bool bProjectingForForwardShading,
	FRDGTextureRef ScreenShadowMaskSubPixelTexture);

/// Output hair transmittance per hair sample for all lights using the forward cluster lights
FHairStrandsTransmittanceMaskData RenderHairStrandsOnePassTransmittanceMask(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, int32 ViewIndex,
	FRDGTextureRef ShadowMaskBits,
	FVirtualShadowMapArray& VirtualShadowMapArray);