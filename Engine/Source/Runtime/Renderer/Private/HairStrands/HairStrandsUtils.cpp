// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsUtils.h"
#include "LightSceneProxy.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "HairStrandsData.h"
#include "SystemTextures.h"
#include "SceneRendering.h"

static float GHairR = 1;
static float GHairTT = 1;
static float GHairTRT = 1;
static float GHairGlobalScattering = 1;
static float GHairLocalScattering = 1;
static int32 GHairTTModel = 0;
static FAutoConsoleVariableRef CVarHairR(TEXT("r.HairStrands.Components.R"), GHairR, TEXT("Enable/disable hair BSDF component R"));
static FAutoConsoleVariableRef CVarHairTT(TEXT("r.HairStrands.Components.TT"), GHairTT, TEXT("Enable/disable hair BSDF component TT"));
static FAutoConsoleVariableRef CVarHairTRT(TEXT("r.HairStrands.Components.TRT"), GHairTRT, TEXT("Enable/disable hair BSDF component TRT"));
static FAutoConsoleVariableRef CVarHairGlobalScattering(TEXT("r.HairStrands.Components.GlobalScattering"), GHairGlobalScattering, TEXT("Enable/disable hair BSDF component global scattering"));
static FAutoConsoleVariableRef CVarHairLocalScattering(TEXT("r.HairStrands.Components.LocalScattering"), GHairLocalScattering, TEXT("Enable/disable hair BSDF component local scattering"));
static FAutoConsoleVariableRef CVarHairTTModel(TEXT("r.HairStrands.Components.TTModel"), GHairTTModel, TEXT("Select hair TT model"));

static float GStrandHairRasterizationScale = 0.5f; // For no AA without TAA, a good value is: 1.325f (Empirical)
static FAutoConsoleVariableRef CVarStrandHairRasterizationScale(TEXT("r.HairStrands.RasterizationScale"), GStrandHairRasterizationScale, TEXT("Rasterization scale to snap strand to pixel"), ECVF_Scalability | ECVF_RenderThreadSafe);

static float GStrandHairStableRasterizationScale = 1.0f; // For no AA without TAA, a good value is: 1.325f (Empirical)
static FAutoConsoleVariableRef CVarStrandHairStableRasterizationScale(TEXT("r.HairStrands.StableRasterizationScale"), GStrandHairStableRasterizationScale, TEXT("Rasterization scale to snap strand to pixel for 'stable' hair option. This value can't go below 1."), ECVF_Scalability | ECVF_RenderThreadSafe);

static float GStrandHairVelocityRasterizationScale = 1.5f; // Tuned based on heavy motion example (e.g., head shaking)
static FAutoConsoleVariableRef CVarStrandHairVelocityRasterizationScale(TEXT("r.HairStrands.VelocityRasterizationScale"), GStrandHairVelocityRasterizationScale, TEXT("Rasterization scale to snap strand to pixel under high velocity"), ECVF_Scalability | ECVF_RenderThreadSafe);

static float GStrandHairShadowRasterizationScale = 1.0f;
static FAutoConsoleVariableRef CVarStrandHairShadowRasterizationScale(TEXT("r.HairStrands.ShadowRasterizationScale"), GStrandHairShadowRasterizationScale, TEXT("Rasterization scale to snap strand to pixel in shadow view"));

static float GDeepShadowAABBScale = 1.0f;
static FAutoConsoleVariableRef CVarDeepShadowAABBScale(TEXT("r.HairStrands.DeepShadow.AABBScale"), GDeepShadowAABBScale, TEXT("Scaling value for loosing/tighting deep shadow bounding volume"));

static int32 GHairVisibilityRectOptimEnable = 1;
static FAutoConsoleVariableRef CVarHairVisibilityRectOptimEnable(TEXT("r.HairStrands.RectLightingOptim"), GHairVisibilityRectOptimEnable, TEXT("Hair Visibility use projected view rect to light only relevant pixels"));

static int32 GHairStrandsComposeAfterTranslucency = 1;
static FAutoConsoleVariableRef CVarHairStrandsComposeAfterTranslucency(TEXT("r.HairStrands.ComposeAfterTranslucency"), GHairStrandsComposeAfterTranslucency, TEXT("0: Compose hair before translucent objects. 1: Compose hair after translucent objects, but before separate translucent objects. 2: Compose hair after all/seperate translucent objects, 3: Compose hair after translucent objects but before translucent render after DOF (which allows depth testing against hair depth)"));

static float GHairDualScatteringRoughnessOverride = 0;
static FAutoConsoleVariableRef CVarHairDualScatteringRoughnessOverride(TEXT("r.HairStrands.DualScatteringRoughness"), GHairDualScatteringRoughnessOverride, TEXT("Override all roughness for the dual scattering evaluation. 0 means no override. Default:0"));

static float GHairStrandsDeepShadowMaxAngle = 90.f;
static FAutoConsoleVariableRef CVarHairStrandsDeepShadowMaxAngle(TEXT("r.HairStrands.DeepShadow.MaxFrustumAngle"), GHairStrandsDeepShadowMaxAngle, TEXT("Max deep shadow frustum angle to avoid strong deformation. Default:90"));

float GetDeepShadowMaxFovAngle()
{
	return FMath::Clamp(GHairStrandsDeepShadowMaxAngle, 10.f, 170.f);
}

float GetHairDualScatteringRoughnessOverride()
{
	return GHairDualScatteringRoughnessOverride;
}

EHairStrandsCompositionType GetHairStrandsComposition()
{
	switch (GHairStrandsComposeAfterTranslucency)
	{
	case 0	: return EHairStrandsCompositionType::BeforeTranslucent;
	case 1	: return EHairStrandsCompositionType::AfterTranslucent;
	case 2	: return EHairStrandsCompositionType::AfterSeparateTranslucent;
	case 3	: return EHairStrandsCompositionType::AfterTranslucentBeforeTranslucentAfterDOF;
	default	: return EHairStrandsCompositionType::BeforeTranslucent;
	}
}

float GetDeepShadowAABBScale()
{
	return FMath::Max(0.f, GDeepShadowAABBScale);
}

float SampleCountToSubPixelSize(uint32 SamplePerPixelCount)
{
	float Scale = 1;
	switch (SamplePerPixelCount)
	{
	case 1: Scale = 1.f; break;
	case 2: Scale = 8.f / 16.f; break;
	case 4: Scale = 8.f / 16.f; break;
	case 8: Scale = 4.f / 16.f; break;
	}
	return Scale;
}
FHairComponent GetHairComponents()
{
	FHairComponent Out;
	Out.R = GHairR > 0;
	Out.TT = GHairTT > 0;
	Out.TRT = GHairTRT > 0;
	Out.LocalScattering = GHairLocalScattering > 0;
	Out.GlobalScattering = GHairGlobalScattering > 0;
	Out.TTModel = GHairTTModel > 0 ? 1 : 0;
	return Out;
}

uint32 ToBitfield(const FHairComponent& C)
{
	return
		(C.R				? 1u : 0u)      |
		(C.TT				? 1u : 0u) << 1 |
		(C.TRT				? 1u : 0u) << 2 |
		(C.LocalScattering  ? 1u : 0u) << 3 |
		(C.GlobalScattering ? 1u : 0u) << 4 |
		// Multiple scattering         << 5 (used only within shader)
		(C.TTModel			? 1u : 0u) << 6;
}

float GetPrimiartyRasterizationScale()
{
	return FMath::Max(0.f, GStrandHairRasterizationScale);
}

float GetDeepShadowRasterizationScale()
{
	return FMath::Max(0.f, GStrandHairShadowRasterizationScale ? GStrandHairShadowRasterizationScale : GStrandHairRasterizationScale);
}

FMinHairRadiusAtDepth1 ComputeMinStrandRadiusAtDepth1(
	const FIntPoint& InResolution,
	const float FOV,
	const uint32 SampleCount,
	const float OverrideStrandHairRasterizationScale,
	const float OrthoWidth,
	const float FinalResolutionScale)
{
	FIntPoint Resolution;
	Resolution.X = FMath::CeilToInt32(InResolution.X * FinalResolutionScale);
	Resolution.Y = FMath::CeilToInt32(InResolution.Y * FinalResolutionScale);

	auto InternalMinRadiusAtDepth1 = [Resolution, FOV, SampleCount, OrthoWidth](float RasterizationScale)
	{
		const float DiameterToRadius = 0.5f;
		float StrandDiameterAtDepth1 = SampleCountToSubPixelSize(SampleCount); //SubPixelScale
		if (OrthoWidth >= 1.0f)
		{
			StrandDiameterAtDepth1 *= FMath::Clamp(Resolution.X / OrthoWidth, 0.0f, 1.0f);
		}
		else
		{
			const float vFOV = FMath::DegreesToRadians(FOV);
			StrandDiameterAtDepth1 *= FMath::Tan(vFOV * 0.5f) / (0.5f * Resolution.Y);
		}
		return DiameterToRadius * RasterizationScale * StrandDiameterAtDepth1;
	};

	FMinHairRadiusAtDepth1 Out;

	// Scales strand to covers a bit more than a pixel and insure at least one sample point is hit
	const float PrimaryRasterizationScale = OverrideStrandHairRasterizationScale > 0 ? OverrideStrandHairRasterizationScale : GStrandHairRasterizationScale;
	const float VelocityRasterizationScale = OverrideStrandHairRasterizationScale > 0 ? OverrideStrandHairRasterizationScale : GStrandHairVelocityRasterizationScale;
	const float StableRasterizationScale = FMath::Max(1.f, GStrandHairStableRasterizationScale);
	Out.Primary = InternalMinRadiusAtDepth1(PrimaryRasterizationScale);
	Out.Velocity = InternalMinRadiusAtDepth1(VelocityRasterizationScale);
	Out.Stable = InternalMinRadiusAtDepth1(StableRasterizationScale);

	return Out;
}

void ComputeTranslatedWorldToLightClip(
	const FVector& TranslatedWorldOffset,
	FMatrix& OutTranslatedWorldToClipTransform,
	FMinHairRadiusAtDepth1& OutMinStrandRadiusAtDepth1,
	const FBoxSphereBounds& PrimitivesBounds,
	const FLightSceneProxy& LightProxy,
	const ELightComponentType LightType,
	const FIntPoint& ShadowResolution)
{
	// Translated SphereBound & translated light position
	const FSphere TranslatedSphereBound = FSphere(PrimitivesBounds.GetSphere().Center + TranslatedWorldOffset, PrimitivesBounds.GetSphere().W);
	const float SphereRadius = TranslatedSphereBound.W * GetDeepShadowAABBScale();
	const FVector3f TranslatedLightPosition = FVector3f(FVector(LightProxy.GetPosition()) + TranslatedWorldOffset); // LWC_TODO: precision loss

	const float MinNear = 1.0f; // 1cm, lower value than this cause precision issue. Similar value than in HairStrandsDeepShadowAllocation.usf
	const float MinZ = FMath::Max(MinNear, FMath::Max(0.1f, FVector::Distance((FVector)TranslatedLightPosition, TranslatedSphereBound.Center)) - TranslatedSphereBound.W);
	const float MaxZ = FMath::Max(0.2f, FVector::Distance((FVector)TranslatedLightPosition, TranslatedSphereBound.Center)) + TranslatedSphereBound.W;
	const float MaxDeepShadowFrustumHalfAngleInRad = 0.5f * FMath::DegreesToRadians(GetDeepShadowMaxFovAngle());

	const float StrandHairRasterizationScale = GetDeepShadowRasterizationScale();
	const float StrandHairStableRasterizationScale = FMath::Max(GStrandHairStableRasterizationScale, 1.0f);
	OutMinStrandRadiusAtDepth1 = FMinHairRadiusAtDepth1();
	OutTranslatedWorldToClipTransform = FMatrix::Identity;
	if (LightType == LightType_Directional)
	{
		const FVector& LightDirection = LightProxy.GetDirection();
		FReversedZOrthoMatrix OrthoMatrix(SphereRadius, SphereRadius, 1.f / (2 * SphereRadius), 0);
		FLookAtMatrix LookAt(TranslatedSphereBound.Center - LightDirection * SphereRadius, TranslatedSphereBound.Center, FVector(0, 0, 1));
		OutTranslatedWorldToClipTransform = LookAt * OrthoMatrix;

		const float RadiusAtDepth1 = SphereRadius / FMath::Min(ShadowResolution.X, ShadowResolution.Y);
		OutMinStrandRadiusAtDepth1.Stable = RadiusAtDepth1 * StrandHairStableRasterizationScale;
		OutMinStrandRadiusAtDepth1.Primary = RadiusAtDepth1 * StrandHairRasterizationScale;
		OutMinStrandRadiusAtDepth1.Velocity = OutMinStrandRadiusAtDepth1.Primary;
	}
	else if (LightType == LightType_Spot || LightType == LightType_Point)
	{
		const float SphereDistance = FVector3f::Distance(TranslatedLightPosition, (FVector3f)TranslatedSphereBound.Center);
		float HalfFov = asin(SphereRadius / SphereDistance);
		HalfFov = FMath::Min(HalfFov, MaxDeepShadowFrustumHalfAngleInRad);

		FReversedZPerspectiveMatrix ProjMatrix(HalfFov, 1, 1, MinZ, MaxZ);
		FLookAtMatrix TranslatedWorldToLight((FVector)TranslatedLightPosition, TranslatedSphereBound.Center, FVector(0, 0, 1));
		OutTranslatedWorldToClipTransform = TranslatedWorldToLight * ProjMatrix;
		OutMinStrandRadiusAtDepth1 = ComputeMinStrandRadiusAtDepth1(ShadowResolution, 2 * HalfFov, 1, StrandHairRasterizationScale); //Light propagation so use perspective not ortho
	}
	else if (LightType == LightType_Rect)
	{
		const float SphereDistance = FVector3f::Distance(TranslatedLightPosition, (FVector3f)TranslatedSphereBound.Center);
		float HalfFov = asin(SphereRadius / SphereDistance);
		HalfFov = FMath::Min(HalfFov, MaxDeepShadowFrustumHalfAngleInRad);

		FReversedZPerspectiveMatrix ProjMatrix(HalfFov, 1, 1, MinZ, MaxZ);
		FLookAtMatrix TranslatedWorldToLight((FVector)TranslatedLightPosition, TranslatedSphereBound.Center, FVector(0, 0, 1));
		OutTranslatedWorldToClipTransform = TranslatedWorldToLight * ProjMatrix;
		OutMinStrandRadiusAtDepth1 = ComputeMinStrandRadiusAtDepth1(ShadowResolution, 2 * HalfFov, 1, StrandHairRasterizationScale); //Light propagation so use perspective not ortho
	}
}

FIntRect ComputeProjectedScreenRect(const FBox& B, const FViewInfo& View)
{
	FVector2D MinP( FLT_MAX,  FLT_MAX);
	FVector2D MaxP(-FLT_MAX, -FLT_MAX);
	FVector Vertices[8] =
	{
		FVector(B.Min),
		FVector(B.Min.X, B.Min.Y, B.Max.Z),
		FVector(B.Min.X, B.Max.Y, B.Min.Z),
		FVector(B.Max.X, B.Min.Y, B.Min.Z),
		FVector(B.Max.X, B.Max.Y, B.Min.Z),
		FVector(B.Max.X, B.Min.Y, B.Max.Z),
		FVector(B.Min.X, B.Max.Y, B.Max.Z),
		FVector(B.Max)
	};

	// Compute the MinP/MaxP in pixel coord, relative to View.ViewRect.Min
	const FMatrix& WorldToView	= View.ViewMatrices.GetWorldToView();
	const FMatrix& ViewToProj	= View.ViewMatrices.GetViewToClip();
	const float NearClippingDistance = View.NearClippingDistance + SMALL_NUMBER;
	for (uint32 i = 0; i < 8; ++i)
	{
		// Clamp position on the near plane to get valid rect even if bounds' points are behind the camera
		FPlane P_View = WorldToView.TransformFVector4(FVector4(Vertices[i], 1.f));
		if (P_View.Z <= NearClippingDistance)
		{
			P_View.Z = NearClippingDistance;
		}

		// Project from view to projective space
		FVector2D P;
		if (FSceneView::ProjectWorldToScreen(P_View, View.ViewRect, ViewToProj, P))
		{
			MinP.X = FMath::Min(MinP.X, P.X);
			MinP.Y = FMath::Min(MinP.Y, P.Y);
			MaxP.X = FMath::Max(MaxP.X, P.X);
			MaxP.Y = FMath::Max(MaxP.Y, P.Y);
		}
	}

	// Clamp to pixel border
	FIntRect OutRect;
	OutRect.Min = FIntPoint(FMath::FloorToInt(MinP.X), FMath::FloorToInt(MinP.Y));
	OutRect.Max = FIntPoint(FMath::CeilToInt(MaxP.X),  FMath::CeilToInt(MaxP.Y));

	// Clamp to screen rect
	OutRect.Min.X = FMath::Clamp(OutRect.Min.X, View.ViewRect.Min.X, View.ViewRect.Max.X);
	OutRect.Max.X = FMath::Clamp(OutRect.Max.X, View.ViewRect.Min.X, View.ViewRect.Max.X);
	OutRect.Min.Y = FMath::Clamp(OutRect.Min.Y, View.ViewRect.Min.Y, View.ViewRect.Max.Y);
	OutRect.Max.Y = FMath::Clamp(OutRect.Max.Y, View.ViewRect.Min.Y, View.ViewRect.Max.Y);

	return OutRect;
}


FIntRect ComputeVisibleHairStrandsMacroGroupsRect(const FViewInfo& View, const FIntRect& ViewRect, const FHairStrandsMacroGroupDatas& Datas)
{
	FIntRect TotalRect(INT_MAX, INT_MAX, -INT_MAX, -INT_MAX);
	if (IsHairStrandsViewRectOptimEnable())
	{
		for (const FHairStrandsMacroGroupData& Data : Datas)
		{
			TotalRect.Union(Data.ScreenRect);
		}

		// For stereo rendering we don't compute FHairStrandsMacroGroupDatas for each view, but reuse MacroGroup data from View0 to View1. 
		// See RenderHairPrePass() in Renderer\Private\HairStrands\HairStrandsRendering.cpp for more details.
		// In order to correct the view rect for the secondary view, we add the view.min value.
		if (IStereoRendering::IsASecondaryView(View))
		{
			TotalRect.Min += View.ViewRect.Min;
			TotalRect.Max += View.ViewRect.Min;
		}

		// In case bounds are not initialized correct for some reason, return view rect
		if (TotalRect.Min.X >= TotalRect.Max.X || TotalRect.Min.Y >= TotalRect.Max.Y) TotalRect = ViewRect;
	}
	else
	{
		TotalRect = ViewRect;
	}

	return TotalRect;
}

bool IsHairStrandsViewRectOptimEnable()
{
	return GHairVisibilityRectOptimEnable > 0;
}

enum EHairVisibilityVendor
{
	HairVisibilityVendor_AMD,
	HairVisibilityVendor_NVIDIA,
	HairVisibilityVendor_INTEL,
	HairVisibilityVendorCount
};

inline EHairVisibilityVendor GetVendor()
{
	return IsRHIDeviceAMD() ? HairVisibilityVendor_AMD : (IsRHIDeviceNVIDIA() ? HairVisibilityVendor_NVIDIA : HairVisibilityVendor_INTEL);
}

uint32 GetVendorOptimalGroupSize1D()
{
	switch (GetVendor())
	{
	case HairVisibilityVendor_AMD:		return 64;
	case HairVisibilityVendor_NVIDIA:	return 32;
	case HairVisibilityVendor_INTEL:	return 64;
	default:							return 64;
	}
}

FIntPoint GetVendorOptimalGroupSize2D()
{
	switch (GetVendor())
	{
	case HairVisibilityVendor_AMD:		return FIntPoint(8, 8);
	case HairVisibilityVendor_NVIDIA:	return FIntPoint(8, 4);
	case HairVisibilityVendor_INTEL:	return FIntPoint(8, 8);
	default:							return FIntPoint(8, 8);
	}
}

FVector4f PackHairRenderInfo(
	float PrimaryRadiusAtDepth1,
	float StableRadiusAtDepth1,
	float VelocityRadiusAtDepth1,
	float VelocityMagnitudeScale)
{
	FVector4f Out;
	Out.X = PrimaryRadiusAtDepth1;
	Out.Y = StableRadiusAtDepth1;
	Out.Z = VelocityRadiusAtDepth1;
	Out.W = VelocityMagnitudeScale;
	return Out;
}

uint32 PackHairRenderInfoBits(
	bool  bIsOrtho,
	bool  bIsGPUDriven)
{
	uint32 BitField = 0;
	BitField |= bIsOrtho ? 0x1 : 0;
	BitField |= bIsGPUDriven ? 0x2 : 0;
	return BitField;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairResourceTransitionPass : public FGlobalShader
{
public:
	static const int32 MaxBufferCount = 16;

private:
	DECLARE_GLOBAL_SHADER(FHairResourceTransitionPass);
	SHADER_USE_PARAMETER_STRUCT(FHairResourceTransitionPass, FGlobalShader);

	class FBufferType : SHADER_PERMUTATION_INT("PERMUTATION_BUFFER_TYPE", 5);
	using FPermutationDomain = TShaderPermutationDomain<FBufferType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, DummyValue)
		SHADER_PARAMETER_RDG_BUFFER_SRV_ARRAY(Buffer<uint>, VertexUIntBuffers, [MaxBufferCount])
		SHADER_PARAMETER_RDG_BUFFER_SRV_ARRAY(Buffer<uint4>, VertexUInt4Buffers, [MaxBufferCount])
		SHADER_PARAMETER_RDG_BUFFER_SRV_ARRAY(Buffer<float4>, VertexFloat4Buffers, [MaxBufferCount])
		SHADER_PARAMETER_RDG_BUFFER_SRV_ARRAY(StructuredBuffer<float4>, StructuredBuffers, [MaxBufferCount])
		SHADER_PARAMETER_RDG_BUFFER_SRV_ARRAY(ByteAddressBuffer, ByteAddressBuffers, [MaxBufferCount])
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, DummyOutput)
	END_SHADER_PARAMETER_STRUCT()

public:
	// 0 : StructuredBuffer<float4>
	// 1 : Buffer<uint>
	// 2 : Buffer<uint4>
	// 3 : Buffer<float4>
	// 4 : ByteAdressBuffer
	static const int32 PermutationCount = 5;
	static int32 GetPermutationIndex(bool bStructured, bool bByteAddressBuffer, bool bInteger, int32 NumComponents)
	{
		if (bByteAddressBuffer)
		{
			return 4;
		}
		else if (bStructured)
		{
			return 0;
		}
		else if (bInteger)
		{
			return (NumComponents == 1) ? 1 : 2;
		}
		else
		{
			return 3;
		}
	}

	static bool IsSupported(EShaderPlatform InPlatform)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, InPlatform);
	}
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsSupported(Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RESOURCE_TRANSITION"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairResourceTransitionPass, "/Engine/Private/HairStrands/HairStrandsMesh.usf", "MainCS", SF_Compute);

void AddTransitionPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	EShaderPlatform InPlatform,
	const TArray<FRDGBufferSRVRef>& Transitions)
{
	const int32 ResourceCount = Transitions.Num();
	if (ResourceCount == 0 || !FHairResourceTransitionPass::IsSupported(InPlatform))
	{
		return;
	}

	FRDGBufferRef DummyOutput = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(4, 1), TEXT("DummyOutput"));
	FRDGBufferUAVRef DummyOutputUAV = GraphBuilder.CreateUAV(DummyOutput, PF_R32_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);

	TStaticArray<FRDGBufferSRVRef, FHairResourceTransitionPass::MaxBufferCount> SortedTransitions[FHairResourceTransitionPass::PermutationCount];

	FRDGBufferSRVRef DummyInputs[FHairResourceTransitionPass::PermutationCount];
	DummyInputs[0] = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 16u));
	DummyInputs[1] = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 4u, 1u), PF_R32_UINT);
	DummyInputs[2] = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 16u, 1u), PF_R32G32B32A32_UINT);
	DummyInputs[3] = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 16u, 1u), PF_A32B32G32R32F);
	DummyInputs[4] = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, 4u));

	int32 ArrayCounts[FHairResourceTransitionPass::PermutationCount];
	ArrayCounts[0] = 0;
	ArrayCounts[1] = 0;
	ArrayCounts[2] = 0;
	ArrayCounts[3] = 0;
	ArrayCounts[4] = 0;

	auto FlushArray = [&](int32 PermutationIndex, int32 TransitionCount)
	{
		FHairResourceTransitionPass::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairResourceTransitionPass::FParameters>();
		PassParameters->DummyValue = 0;
		PassParameters->DummyOutput = DummyOutputUAV;

		TShaderResourceParameterArray<FRDGBufferSRV*, FHairResourceTransitionPass::MaxBufferCount>* ParamArray = nullptr;
		switch (PermutationIndex)
		{
		case 0: ParamArray = &PassParameters->StructuredBuffers; break;
		case 1: ParamArray = &PassParameters->VertexUIntBuffers; break;
		case 2: ParamArray = &PassParameters->VertexUInt4Buffers; break;
		case 3: ParamArray = &PassParameters->VertexFloat4Buffers; break;
		case 4: ParamArray = &PassParameters->ByteAddressBuffers; break;
		default: checkNoEntry();
		};

		for (int32 ResourceIndex = 0; ResourceIndex < TransitionCount; ++ResourceIndex)
		{
			(*ParamArray)[ResourceIndex] = SortedTransitions[PermutationIndex][ResourceIndex];
		}

		for (int32 ResourceIndex = TransitionCount; ResourceIndex < FHairResourceTransitionPass::MaxBufferCount; ++ResourceIndex)
		{
			(*ParamArray)[ResourceIndex] = DummyInputs[PermutationIndex];
		}

		FHairResourceTransitionPass::FPermutationDomain PermutationVector;
		PermutationVector.Set<FHairResourceTransitionPass::FBufferType>(PermutationIndex);
		TShaderMapRef<FHairResourceTransitionPass> ComputeShader(ShaderMap, PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::ResourceTransitions(P=%d)", PermutationIndex),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1));

		ArrayCounts[PermutationIndex] = 0;
	};

	// Sort the transitions into each of the arrays based on the shader permutation required to transition them
	for (int32 TransitionIndex = 0; TransitionIndex < Transitions.Num(); ++TransitionIndex)
	{
		const bool bStructuredBuffer = EnumHasAnyFlags(Transitions[TransitionIndex]->Desc.Buffer->Desc.Usage, EBufferUsageFlags::StructuredBuffer);
		const bool bByteAddressBuffer = EnumHasAnyFlags(Transitions[TransitionIndex]->Desc.Buffer->Desc.Usage, EBufferUsageFlags::ByteAddressBuffer);
		const EPixelFormat PixelFormat = Transitions[TransitionIndex]->Desc.Format;
		const bool bIsIntegerFormat = IsInteger(PixelFormat);
		const int32 NumComponents = GPixelFormats[PixelFormat].NumComponents;

		const int32 PermutationIndex = FHairResourceTransitionPass::GetPermutationIndex(bStructuredBuffer, bByteAddressBuffer, bIsIntegerFormat, NumComponents);

		SortedTransitions[PermutationIndex][ArrayCounts[PermutationIndex]++] = Transitions[TransitionIndex];

		if (ArrayCounts[PermutationIndex] >= FHairResourceTransitionPass::MaxBufferCount)
		{
			FlushArray(PermutationIndex, FHairResourceTransitionPass::MaxBufferCount);
		}
	}

	for (int32 PermutationIndex = 0; PermutationIndex < FHairResourceTransitionPass::PermutationCount; ++PermutationIndex)
	{
		if (ArrayCounts[PermutationIndex] > 0)
		{
			FlushArray(PermutationIndex, ArrayCounts[PermutationIndex]);
		}
	}
}

FPointPerCurveDispatchInfo GetPointPerCurveDispatchInfo(uint32 InAssetMaxPointPerCurve, uint32 InAssetCurveCount, uint32 InGroupSize)
{
	FPointPerCurveDispatchInfo Out;
	Out.SourcePoinPerCurve = InAssetMaxPointPerCurve;
	Out.SourceCurveCount = InAssetCurveCount;
	Out.GroupSize = InGroupSize;

	// Compute the rounded point-per-curve count, based on the asset and the shader's requirement
	Out.PointPerCurve = FMath::Clamp(uint32(FMath::Pow(2u, FMath::RoundFromZero(FMath::Log2(float(InAssetMaxPointPerCurve))))), 4u, Out.GroupSize);
	check(FMath::IsPowerOfTwo(Out.PointPerCurve));

	// Compute the number of curve per group
	Out.CurvePerGroup = Out.GroupSize / Out.PointPerCurve;

	// Compute dispatch count
	const uint32 LinearGroupCount = FMath::DivideAndRoundUp(Out.SourceCurveCount, Out.CurvePerGroup);
	Out.DispatchCount = FIntVector(LinearGroupCount, 1, 1);
	if (Out.DispatchCount.X > 0xFFFFu)
	{
		Out.DispatchCount.X = 64;
		Out.DispatchCount.Y = FMath::DivideAndRoundUp(LinearGroupCount, uint32(Out.DispatchCount.X));
	}
	check(Out.DispatchCount.X <= 0xFFFFu);
	check(Out.DispatchCount.Y <= 0xFFFFu);

	return Out;
}