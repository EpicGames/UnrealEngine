// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsUtils.h: Hair strands utils.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "HairStrandsInterface.h"
#include "SceneTypes.h"
#include "HairStrandsData.h"

class FViewInfo;

FMinHairRadiusAtDepth1 ComputeMinStrandRadiusAtDepth1(
	const FIntPoint& Resolution,
	const float FOV,
	const uint32 SampleCount,
	const float OverrideStrandHairRasterizationScale,
	const float OrthoWidth = 0.0f,
	const float FinalResolutionScale = 1.0f);

FIntRect ComputeProjectedScreenRect(const FBox& B, const FViewInfo& View);

void ComputeTranslatedWorldToLightClip(
	const FVector& TranslatedWorldOffset,
	FMatrix& OutTranslatedWorldToClipTransform,
	FMinHairRadiusAtDepth1& OutMinStrandRadiusAtDepth1,
	const FBoxSphereBounds& PrimitivesBounds,
	const class FLightSceneProxy& LightProxy,
	const ELightComponentType LightType,
	const FIntPoint& ShadowResolution);

struct FHairComponent
{
	bool R = true;
	bool TT = true;
	bool TRT = true;
	bool GlobalScattering = true;
	bool LocalScattering = true;
	bool TTModel = false;
};
FHairComponent GetHairComponents();
uint32 ToBitfield(const FHairComponent& Component);

float GetHairDualScatteringRoughnessOverride();

float SampleCountToSubPixelSize(uint32 SamplePerPixelCount);

FIntRect ComputeVisibleHairStrandsMacroGroupsRect(const FViewInfo& View, const FIntRect& ViewRect, const FHairStrandsMacroGroupDatas& Datas);

bool IsHairStrandsViewRectOptimEnable();

uint32 GetVendorOptimalGroupSize1D();
FIntPoint GetVendorOptimalGroupSize2D();

enum class  EHairStrandsCompositionType : uint8
{
	BeforeTranslucent,
	AfterTranslucent,
	AfterSeparateTranslucent,
	AfterTranslucentBeforeTranslucentAfterDOF
};

EHairStrandsCompositionType GetHairStrandsComposition();

FVector4f PackHairRenderInfo(
	float PrimaryRadiusAtDepth1,
	float StableRadiusAtDepth1,
	float VelocityRadiusAtDepth1,
	float VelocityScale);

uint32 PackHairRenderInfoBits(
	bool bIsOrtho,
	bool bIsGPUDriven);