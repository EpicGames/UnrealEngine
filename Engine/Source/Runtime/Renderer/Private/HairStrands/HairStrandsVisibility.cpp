// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsVisibility.h"
#include "CoreMinimal.h"
#include "CommonRenderResources.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DynamicRHI.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "GroomVisualizationData.h"
#include "HairStrands/HairStrandsData.h"
#include "HairStrands/HairStrandsForwardRaster.h"
#include "HairStrands/HairStrandsLUT.h"
#include "HairStrands/HairStrandsMacroGroup.h"
#include "HairStrands/HairStrandsTile.h"
#include "HairStrands/HairStrandsUtils.h"
#include "HairStrandsInterface.h"
#include "HAL/IConsoleManager.h"
#include "HZB.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "Logging/LogMacros.h"
#include "MaterialDomain.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "Math/TransformVectorized.h"
#include "MeshBatch.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "Misc/LargeWorldCoordinates.h"
#include "PipelineStateCache.h"
#include "PixelShaderUtils.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "PSOPrecache.h"
#include "PSOPrecacheMaterial.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTextureReductions.h"
#include "SceneTexturesConfig.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "ScreenRendering.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShaderPrintParameters.h"
#include "SimpleMeshDrawCommandPass.h"
#include "Stats/Stats.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "VelocityRendering.h"
#include "VertexFactory.h"
#include "SceneViewState.h"


DECLARE_GPU_STAT(HairStrandsVisibility);

/////////////////////////////////////////////////////////////////////////////////////////

static float GHairStrandsMaterialCompactionDepthThreshold = 1.f;
static float GHairStrandsMaterialCompactionTangentThreshold = 10.f;
static FAutoConsoleVariableRef CVarHairStrandsMaterialCompactionDepthThreshold(TEXT("r.HairStrands.MaterialCompaction.DepthThreshold"), GHairStrandsMaterialCompactionDepthThreshold, TEXT("Compaction threshold for depth value for material compaction (in centimeters). Default 1 cm."));
static FAutoConsoleVariableRef CVarHairStrandsMaterialCompactionTangentThreshold(TEXT("r.HairStrands.MaterialCompaction.TangentThreshold"), GHairStrandsMaterialCompactionTangentThreshold, TEXT("Compaciton threshold for tangent value for material compaction (in degrees). Default 10 deg."));

static int32 GHairVisibilityMSAA_MaxSamplePerPixel = 8;
static float GHairVisibilityMSAA_MeanSamplePerPixel = 0.75f;
static FAutoConsoleVariableRef CVarHairVisibilityMSAA_MaxSamplePerPixel(TEXT("r.HairStrands.Visibility.MSAA.SamplePerPixel"), GHairVisibilityMSAA_MaxSamplePerPixel, TEXT("Hair strands visibility sample count (2, 4, or 8)"), ECVF_Scalability | ECVF_RenderThreadSafe);
static FAutoConsoleVariableRef CVarHairVisibilityMSAA_MeanSamplePerPixel(TEXT("r.HairStrands.Visibility.MSAA.MeanSamplePerPixel"), GHairVisibilityMSAA_MeanSamplePerPixel, TEXT("Scale the numer of sampler per pixel for limiting memory allocation (0..1, default 0.5f)"));

static int32 GHairVisibilityCompute_MaxSamplePerPixel = 1;
static float GHairVisibilityCompute_MeanSamplePerPixel = 0.75f;
static FAutoConsoleVariableRef CVarHairVisibilityCompute_MaxSamplePerPixel(TEXT("r.HairStrands.Visibility.Compute.SamplePerPixel"), GHairVisibilityCompute_MaxSamplePerPixel, TEXT("Hair strands visibility sample count (2, 4, or 8)"), ECVF_Scalability | ECVF_RenderThreadSafe);
static FAutoConsoleVariableRef CVarHairVisibilityCompute_MeanSamplePerPixel(TEXT("r.HairStrands.Visibility.Compute.MeanSamplePerPixel"), GHairVisibilityCompute_MeanSamplePerPixel, TEXT("Scale the numer of sampler per pixel for limiting memory allocation (0..1, default 0.5f)"));

static int32 GHairClearVisibilityBuffer = 0;
static FAutoConsoleVariableRef CVarHairClearVisibilityBuffer(TEXT("r.HairStrands.Visibility.Clear"), GHairClearVisibilityBuffer, TEXT("Clear hair strands visibility buffer"));

static int32 GHairResolutionScaleAware = 1;
static FAutoConsoleVariableRef CVarHairResolutionScaleAware(TEXT("r.HairStrands.Visibility.ResolutionScaleAware"), GHairResolutionScaleAware, TEXT("Scale hair strands based on resolution scale"));

static TAutoConsoleVariable<int32> CVarHairVelocityMagnitudeScale(
	TEXT("r.HairStrands.VelocityMagnitudeScale"),
	100,  // Tuned by eye, based on heavy motion (strong head shack)
	TEXT("Velocity magnitude (in pixel) at which a hair will reach its pic velocity-rasterization-scale under motion to reduce aliasing. Default is 100."));

static int32 GHairVelocityType = 1; // default is 
static FAutoConsoleVariableRef CVarHairVelocityType(TEXT("r.HairStrands.VelocityType"), GHairVelocityType, TEXT("Type of velocity filtering (0:avg, 1:closest, 2:max). Default is 1."));

static int32 GHairVisibilityPPLL = 0;
static int32 GHairVisibilityPPLL_MaxSamplePerPixel = 16;
static float GHairVisibilityPPLL_MeanSamplePerPixel = 1;
static FAutoConsoleVariableRef CVarGHairVisibilityPPLL(TEXT("r.HairStrands.Visibility.PPLL"), GHairVisibilityPPLL, TEXT("Hair Visibility uses per pixel linked list"), ECVF_Scalability | ECVF_RenderThreadSafe);
static FAutoConsoleVariableRef CVarGHairVisibilityPPLL_MeanNodeCountPerPixel(TEXT("r.HairStrands.Visibility.PPLL.SamplePerPixel"), GHairVisibilityPPLL_MaxSamplePerPixel, TEXT("The maximum number of node allowed to be independently shaded and composited per pixel. Total amount of node will be width*height*VisibilityPPLLMaxRenderNodePerPixel. The last node is used to aggregate all furthest strands to shade into a single one."));
static FAutoConsoleVariableRef CVarGHairVisibilityPPLL_MeanSamplePerPixel(TEXT("r.HairStrands.Visibility.PPLL.MeanSamplePerPixel"), GHairVisibilityPPLL_MeanSamplePerPixel, TEXT("Scale the maximum number of node allowed for all linked list element (0..1, default 1). It will be width*height*SamplerPerPixel*Scale."));

static float GHairStrandsViewHairCountDepthDistanceThreshold = 30.f;
static FAutoConsoleVariableRef CVarHairStrandsViewHairCountDepthDistanceThreshold(TEXT("r.HairStrands.Visibility.HairCount.DistanceThreshold"), GHairStrandsViewHairCountDepthDistanceThreshold, TEXT("Distance threshold defining if opaque depth get injected into the 'view-hair-count' buffer."));

int32 GHairVisibilityComputeRaster_Culling = 0;
// This value was previously at 8192 but extending the binning algorithm to support segments longer than three tiles revealed a problem: Each binner allocates and writes its own tiles,
// which often leads to lots of partially filled tiles for the same position in the tile grid. Without increasing the tile limit, we quickly run out of tiles to allocate when we get close
// to a groom and the strand count and segment length increase. This new value was found empirically.
// TODO: Find an efficient alternative algorithm where the binners output into shared tiles.
int32 GHairVisibilityComputeRaster_MaxTiles = 65536;
int32 GHairVisibilityComputeRaster_TileSize = 32;
int32 GHairVisibility_NumClassifiers = 32;
int32 GHairVisibilityComputeRaster_NumBinners = 32;
int32 GHairVisibilityComputeRaster_NumRasterizers = 256;
int32 GHairVisibilityComputeRaster_NumRasterizersNaive = 256;
int32 GHairVisibilityComputeRaster_Debug = 0;

static FAutoConsoleVariableRef CVarHairVisibilityComputeRaster_Culling(TEXT("r.HairStrands.Visibility.ComputeRaster.Culling"), GHairVisibilityComputeRaster_Culling, TEXT("Use culling buffers with compute rasterization."));
static FAutoConsoleVariableRef CVarHairVisibilityComputeRaster_MaxTiles(TEXT("r.HairStrands.Visibility.ComputeRaster.MaxTiles"), GHairVisibilityComputeRaster_MaxTiles, TEXT("Maximum number of tiles used for compute rasterization. 8192 is default"));
static FAutoConsoleVariableRef CVarHairVisibilityComputeRaster_TileSize(TEXT("r.HairStrands.Visibility.ComputeRaster.TileSize"), GHairVisibilityComputeRaster_TileSize, TEXT("Tile size used for compute rasterization. Experimental - only size of 32 currently supported"));
static FAutoConsoleVariableRef CVarHairVisibility_NumClassifiers(TEXT("r.HairStrands.Visibility.NumClassifiers"), GHairVisibility_NumClassifiers, TEXT("Number of workgroups used in hair segment classification pass. 32 is default"));
static FAutoConsoleVariableRef CVarHairVisibilityComputeRaster_NumBinners(TEXT("r.HairStrands.Visibility.ComputeRaster.NumBinners"), GHairVisibilityComputeRaster_NumBinners, TEXT("Number of Binners used in Binning compute rasterization pass. 32 is default"));
static FAutoConsoleVariableRef CVarHairVisibilityComputeRaster_NumRasterizers(TEXT("r.HairStrands.Visibility.ComputeRaster.NumRasterizers"), GHairVisibilityComputeRaster_NumRasterizers, TEXT("Number of Rasterizers used compute rasterization. 256 is default"));
static FAutoConsoleVariableRef CVarHairVisibilityComputeRaster_NumRasterizersNaive(TEXT("r.HairStrands.Visibility.ComputeRaster.NumRasterizersNaive"), GHairVisibilityComputeRaster_NumRasterizersNaive, TEXT("Number of Rasterizers used in naive compute rasterization. 256 is default"));
static FAutoConsoleVariableRef CVarHairVisibilityComputeRaster_Debug(TEXT("r.HairStrands.Visibility.ComputeRaster.Debug"), GHairVisibilityComputeRaster_Debug, TEXT("Debug compute raster output"));

static float GHairStrandsFullCoverageThreshold = 0.98f;
static FAutoConsoleVariableRef CVarHairStrandsFullCoverageThreshold(TEXT("r.HairStrands.Visibility.FullCoverageThreshold"), GHairStrandsFullCoverageThreshold, TEXT("Define the coverage threshold at which a pixel is considered fully covered."));

static float GHairStrandsWriteVelocityCoverageThreshold = 0.f;
static FAutoConsoleVariableRef CVarHairStrandsWriteVelocityCoverageThreshold(TEXT("r.HairStrands.Visibility.WriteVelocityCoverageThreshold"), GHairStrandsWriteVelocityCoverageThreshold, TEXT("Define the coverage threshold at which a pixel write its hair velocity (default: 0, i.e., write for all pixel)"));

static int32 GHairStrandsSortHairSampleByDepth = 0;
static FAutoConsoleVariableRef CVarHairStrandsSortHairSampleByDepth(TEXT("r.HairStrands.Visibility.SortByDepth"), GHairStrandsSortHairSampleByDepth, TEXT("Sort hair fragment by depth and update their coverage based on ordered transmittance."));

static int32 GHairStrandsHairCountToTransmittance = 0;
static FAutoConsoleVariableRef CVarHairStrandsHairCountToTransmittance(TEXT("r.HairStrands.Visibility.UseCoverageMappping"), GHairStrandsHairCountToTransmittance, TEXT("Use hair count to coverage transfer function."));

static int32 GHairStrandsDebugPPLL = 0;
static FAutoConsoleVariableRef CVarHairStrandsDebugPPLL(TEXT("r.HairStrands.Visibility.PPLL.Debug"), GHairStrandsDebugPPLL, TEXT("Draw debug per pixel light list rendering."));

static int32 GHairStrandsLightSampleFormat = 1;
static FAutoConsoleVariableRef CVarHairStrandsLightSampleFormat(TEXT("r.HairStrands.LightSampleFormat"), GHairStrandsLightSampleFormat, TEXT("Define the format used for storing the lighting of hair samples (0: RGBA-16bits, 1: RGB-11.11.10bits)"));

static float GHairStrands_InvalidationPosition_Threshold = 0.05f;
static FAutoConsoleVariableRef CVarHairStrands_InvalidationPosition_Threshold(TEXT("r.HairStrands.PathTracing.InvalidationThreshold"), GHairStrands_InvalidationPosition_Threshold, TEXT("Define the minimal distance to invalidate path tracer output when groom changes (in cm, default: 0.5mm)\nSet to a negative value to disable this feature"));

static int32 GHairStrands_InvalidationPosition_Debug = 0;
static FAutoConsoleVariableRef CVarHairStrands_InvalidationPosition_Debug(TEXT("r.HairStrands.PathTracing.InvalidationDebug"), GHairStrands_InvalidationPosition_Debug, TEXT("Enable bounding box drawing for groom element causing path tracer invalidation"));

static float GHairStrands_Selection_CoverageThreshold = 0.0f;
static FAutoConsoleVariableRef CVarHairStrands_Selection_CoverageThreshold(TEXT("r.HairStrands.Selection.CoverageThreshold"), GHairStrands_Selection_CoverageThreshold, TEXT("Coverage threshold for making hair strands outline selection and custom depth finer"));

static int32 GHairStrandsUseHWRaster = 0;
static FAutoConsoleVariableRef CVarHairStrandsUseHWRaster(TEXT("r.HairStrands.Visibility.UseHWRaster"), GHairStrandsUseHWRaster, TEXT("Toggles the hardware rasterizer for hair strands visibility rendering."));

static int32 GHairStrandsUseNaiveSWRaster = 0;
static FAutoConsoleVariableRef CVarHairStrandsUseNaiveSWRaster(TEXT("r.HairStrands.Visibility.UseNaiveSWRaster"), GHairStrandsUseNaiveSWRaster, TEXT("Toggles a naive version of the software rasterizer for hair strands visibility rendering."));

static int32 GHairStrandsTileCompaction = 0;
static FAutoConsoleVariableRef CVarHairStrandsTileCompaction(TEXT("r.HairStrands.Visibility.TileCompaction"), GHairStrandsTileCompaction, TEXT("Enables a compaction pass to run on the output of the binning pass of the hair software rasterizer."));

static int32 GHairStrandsHWSWClassifaction = 0;
static FAutoConsoleVariableRef CVarHairStrandsHWSWClassifaction(TEXT("r.HairStrands.Visibility.HWSWClassifaction"), GHairStrandsHWSWClassifaction, TEXT("Enables classifying hair segments to be rasterized with a hardware rasterizer or a software rasterizer."));

int32 GHairVisibilityUpdateSceneHZB = 0;
static FAutoConsoleVariableRef CVarHairVisibilityUpdateSceneHZB(TEXT("r.HairStrands.Visibility.UpdateSceneHZB"), GHairVisibilityUpdateSceneHZB, TEXT("Update scene HZB with hair data (experimental)."));

/////////////////////////////////////////////////////////////////////////////////////////

namespace HairStrandsVisibilityInternal
{
	struct NodeData
	{
		uint32 Depth;
		uint32 ControlPointId_MacroGroupId;
		uint32 Tangent_Coverage;
		uint32 BaseColor_Roughness;
		uint32 Specular;
	};

	// 64 bit alignment
	struct NodeVis
	{
		uint32 Depth_Coverage;
		uint32 ControlPointId_MacroGroupId;
	};
}

enum EHairVisibilityRenderMode
{
	HairVisibilityRenderMode_Transmittance,
	HairVisibilityRenderMode_PPLL,
	HairVisibilityRenderMode_MSAA_Visibility,
	HairVisibilityRenderMode_TransmittanceAndHairCount,
	HairVisibilityRenderMode_ComputeRaster,
	HairVisibilityRenderMode_ComputeRasterForward,
	HairVisibilityRenderModeCount
};

bool IsHairVisibilityComputeRasterEnabled();
bool IsHairVisibilityComputeRasterForwardEnabled(EShaderPlatform InPlatform);

inline EHairVisibilityRenderMode GetHairVisibilityRenderMode(EShaderPlatform InPlatform)
{
	if (GHairVisibilityPPLL > 0)
	{
		return HairVisibilityRenderMode_PPLL;
	}
	else if (IsHairVisibilityComputeRasterEnabled())
	{
		return HairVisibilityRenderMode_ComputeRaster;
	}
	else if (IsHairVisibilityComputeRasterForwardEnabled(InPlatform))
	{
		return HairVisibilityRenderMode_ComputeRasterForward;
	}
	else
	{
		return HairVisibilityRenderMode_MSAA_Visibility;
	}
}

inline bool IsMsaaEnabled(EShaderPlatform InPlatform)
{
	const EHairVisibilityRenderMode Mode = GetHairVisibilityRenderMode(InPlatform);
	return Mode == HairVisibilityRenderMode_MSAA_Visibility;
}

static uint32 GetMaxSamplePerPixel(EShaderPlatform InPlatform)
{
	switch (GetHairVisibilityRenderMode(InPlatform))
	{
		case HairVisibilityRenderMode_ComputeRaster:
		{
			if (GHairVisibilityCompute_MaxSamplePerPixel <= 1)
			{
				return 1;
			}
			else if (GHairVisibilityCompute_MaxSamplePerPixel == 2)
			{
				return 2;
			}
			else if (GHairVisibilityCompute_MaxSamplePerPixel <= 4)
			{
				return 4;
			}
			else
			{
				return 8;
			}
		}
		case HairVisibilityRenderMode_MSAA_Visibility:
		{
			// Mac platfoms don't properly support R32_UINT format for for MSAA render target. To alleviate this, force sample count to 1.
			#if PLATFORM_MAC
			{
				return 1;
			}
			#else
			if (GHairVisibilityMSAA_MaxSamplePerPixel <= 1)
			{
				return 1;
			}
			else if (GHairVisibilityMSAA_MaxSamplePerPixel == 2)
			{
				return 2;
			}
			else if (GHairVisibilityMSAA_MaxSamplePerPixel <= 4)
			{
				return 4;
			}
			else
			{
				return 8;
			}
			#endif
		}
		case HairVisibilityRenderMode_PPLL:
		{
			// The following must match the FPPLL permutation of FHairVisibilityPrimitiveIdCompactionCS.
			if (GHairVisibilityPPLL_MaxSamplePerPixel == 0)
			{
				return 0;
			}
			else if (GHairVisibilityPPLL_MaxSamplePerPixel <= 8)
			{
				return 8;
			}
			else if (GHairVisibilityPPLL_MaxSamplePerPixel <= 16)
			{
				return 16;
			}
			else //if (GHairVisibilityPPLL_MaxSamplePerPixel <= 32)
			{
				return 32;
			}
			// If more is needed: please check out EncodeNodeDesc from HairStrandsVisibilityCommon.ush to verify node count representation limitations.
		}
	}
	return 1;
}

inline uint32 GetMeanSamplePerPixel(EShaderPlatform InPlatform)
{
	const uint32 SamplePerPixel = GetMaxSamplePerPixel(InPlatform);
	switch (GetHairVisibilityRenderMode(InPlatform))
	{
	case HairVisibilityRenderMode_ComputeRasterForward:
	case HairVisibilityRenderMode_ComputeRaster:
		return FMath::Max(1, FMath::FloorToInt(SamplePerPixel * FMath::Clamp(GHairVisibilityCompute_MeanSamplePerPixel, 0.f, 1.f)));
	case HairVisibilityRenderMode_MSAA_Visibility:
		return FMath::Max(1, FMath::FloorToInt(SamplePerPixel * FMath::Clamp(GHairVisibilityMSAA_MeanSamplePerPixel, 0.f, 1.f)));
	case HairVisibilityRenderMode_PPLL:
		return FMath::Max(1, FMath::FloorToInt(SamplePerPixel * FMath::Clamp(GHairVisibilityPPLL_MeanSamplePerPixel, 0.f, 10.f)));
	case HairVisibilityRenderMode_Transmittance:
	case HairVisibilityRenderMode_TransmittanceAndHairCount:
		return 1;
	}
	return 1;
}

uint32 GetHairStrandsMeanSamplePerPixel(EShaderPlatform InPlatform)
{
	return GetMeanSamplePerPixel(InPlatform);
}

struct FRasterComputeOutput
{
	FIntPoint Resolution;

	FRDGTextureRef HairCountTexture = nullptr;
	FRDGTextureRef DepthTexture = nullptr;

	FRDGTextureRef DepthCovTexture = nullptr;
	FRDGTextureRef PrimMatTexture = nullptr;
};

static uint32 GetTotalSampleCountForAllocation(FIntPoint Resolution, EShaderPlatform InPlatform)
{
	return Resolution.X * Resolution.Y * GetMeanSamplePerPixel(InPlatform);
}

void SetUpViewHairRenderInfo(const FViewInfo& ViewInfo, bool bEnableMSAA, FVector4f& OutHairRenderInfo, uint32& OutHairRenderInfoBits, uint32& OutHairComponents, float FinalResolutionScale)
{
	FVector2f PixelVelocity(1.f / (ViewInfo.ViewRect.Width() * 2), 1.f / (ViewInfo.ViewRect.Height() * 2));
	const float VelocityMagnitudeScale = FMath::Clamp(CVarHairVelocityMagnitudeScale.GetValueOnAnyThread(), 0, 512) * FMath::Min(PixelVelocity.X, PixelVelocity.Y);

	// In the case we render coverage, we need to override some view uniform shader parameters to account for the change in MSAA sample count.
	const uint32 HairVisibilitySampleCount = bEnableMSAA ? GetMaxSamplePerPixel(ViewInfo.GetShaderPlatform()) : 1;	// The coverage pass does not use MSAA
	const float RasterizationScaleOverride = 0.0f;	// no override

	const FIntRect EffectiveViewRect = GHairResolutionScaleAware > 0 ? ViewInfo.ViewRect : ViewInfo.UnconstrainedViewRect;
	const FMinHairRadiusAtDepth1 MinHairRadius = ComputeMinStrandRadiusAtDepth1(
		FIntPoint(EffectiveViewRect.Width(), EffectiveViewRect.Height()), ViewInfo.FOV, HairVisibilitySampleCount, RasterizationScaleOverride, ViewInfo.ViewMatrices.GetOrthoDimensions().X, FinalResolutionScale);

	OutHairRenderInfo = PackHairRenderInfo(MinHairRadius.Primary, MinHairRadius.Stable, MinHairRadius.Velocity, VelocityMagnitudeScale);
	OutHairRenderInfoBits = PackHairRenderInfoBits(!ViewInfo.IsPerspectiveProjection(), false);
	OutHairComponents = ToBitfield(GetHairComponents());
}

void SetUpViewHairRenderInfo(const FViewInfo& ViewInfo, FVector4f& OutHairRenderInfo, uint32& OutHairRenderInfoBits, uint32& OutHairComponents, float FinalResolutionScale)
{
	SetUpViewHairRenderInfo(ViewInfo, IsMsaaEnabled(ViewInfo.GetShaderPlatform()), OutHairRenderInfo, OutHairRenderInfoBits, OutHairComponents, FinalResolutionScale);
}

static bool IsCompatibleWithHairVisibility(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return IsCompatibleWithHairStrands(Parameters.Platform, Parameters.MaterialParameters);
}

float GetHairWriteVelocityCoverageThreshold()
{
	return FMath::Clamp(GHairStrandsWriteVelocityCoverageThreshold, 0.f, 1.f);
}

float GetHairStrandsFullCoverageThreshold()
{
	return FMath::Clamp(GHairStrandsFullCoverageThreshold, 0.1f, 1.f);
}

uint32 GetHairStrandsIntCoverageThreshold()
{
	return 1000u;
}

bool IsHairVisibilityUpdateSceneHZBEnabled()
{
	return GHairVisibilityUpdateSceneHZB > 0;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairLightSampleClearVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairLightSampleClearVS);
	SHADER_USE_PARAMETER_STRUCT(FHairLightSampleClearVS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, MaxViewportResolution)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, HairNodeCountBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_VERTEX"), 1);
	}
};

class FHairLightSampleClearPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairLightSampleClearPS);
	SHADER_USE_PARAMETER_STRUCT(FHairLightSampleClearPS, FGlobalShader)

	class FOutputFormat : SHADER_PERMUTATION_INT("PERMUTATION_OUTPUT_FORMAT", 2);
	using FPermutationDomain = TShaderPermutationDomain<FOutputFormat>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, MaxViewportResolution)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, HairNodeCountBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static EPixelFormat GetHairLightSampleFormat()
	{
		EPixelFormat Format = PF_FloatRGBA;
		if (GHairStrandsLightSampleFormat > 0 && GPixelFormats[PF_FloatR11G11B10].Supported)
		{
			Format = PF_FloatR11G11B10;
		}
		return Format;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLEAR"), 1);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FOutputFormat>() == 0)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_FloatRGBA);
		}
		else if (PermutationVector.Get<FOutputFormat>() == 1)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_FloatR11G11B10);
		}
	}
};

//Used by DeferredLightGlobalPSOCollector
EPixelFormat GetHairLightSampleFormat()
{
	return FHairLightSampleClearPS::GetHairLightSampleFormat();
}

IMPLEMENT_GLOBAL_SHADER(FHairLightSampleClearVS, "/Engine/Private/HairStrands/HairStrandsLightSample.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FHairLightSampleClearPS, "/Engine/Private/HairStrands/HairStrandsLightSample.usf", "ClearPS", SF_Pixel);

static FRDGTextureRef AddClearLightSamplePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo* View,
	const uint32 MaxNodeCount,
	const FRDGBufferSRVRef NodeCounterBuffer)
{	
	const EPixelFormat Format = FHairLightSampleClearPS::GetHairLightSampleFormat();

	// Compute the target texture resolution and round it up 128
	uint32 SampleTextureResolution = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(MaxNodeCount)));
	SampleTextureResolution = FMath::DivideAndRoundUp(SampleTextureResolution, 128u) * 128u;

	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(FIntPoint(SampleTextureResolution, SampleTextureResolution), Format, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable);
	FRDGTextureRef Output = GraphBuilder.CreateTexture(Desc, TEXT("Hair.LightSample"));

	FHairLightSampleClearPS::FParameters* ParametersPS = GraphBuilder.AllocParameters<FHairLightSampleClearPS::FParameters>();
	ParametersPS->MaxViewportResolution = Desc.Extent;
	ParametersPS->HairNodeCountBuffer = NodeCounterBuffer;
	
	FHairLightSampleClearPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairLightSampleClearPS::FOutputFormat>(Format == PF_FloatR11G11B10 ? 1 : 0);

	const FIntPoint ViewportResolution = Desc.Extent;
	TShaderMapRef<FHairLightSampleClearVS> VertexShader(View->ShaderMap);
	TShaderMapRef<FHairLightSampleClearPS> PixelShader(View->ShaderMap, PermutationVector);

	ParametersPS->RenderTargets[0] = FRenderTargetBinding(Output, ERenderTargetLoadAction::ENoAction);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::LightSampleClearPS"),
		ParametersPS,
		ERDGPassFlags::Raster,
		[ParametersPS, VertexShader, PixelShader, ViewportResolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FHairLightSampleClearVS::FParameters ParametersVS;
		ParametersVS.MaxViewportResolution = ParametersPS->MaxViewportResolution;
		ParametersVS.HairNodeCountBuffer = ParametersPS->HairNodeCountBuffer;

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *ParametersPS);

		RHICmdList.SetViewport(0, 0, 0.0f, ViewportResolution.X, ViewportResolution.Y, 1.0f);
		RHICmdList.SetStreamSource(0, nullptr, 0);
		RHICmdList.DrawPrimitive(0, 1, 1);
	});

	return Output;
}

/////////////////////////////////////////////////////////////////////////////////////////

class FHairMaterialVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FHairMaterialVS, MeshMaterial);

protected:
	FHairMaterialVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel((EShaderPlatform)Initializer.Target.Platform);
		check(FSceneInterface::GetShadingPath(FeatureLevel) != EShadingPath::Mobile);
	}

	FHairMaterialVS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsCompatibleWithHairVisibility(Parameters) && HairStrands::IsHairStrandsVF(Parameters.VertexFactoryType);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};
IMPLEMENT_MATERIAL_SHADER_TYPE(, FHairMaterialVS, TEXT("/Engine/Private/HairStrands/HairStrandsMaterialVS.usf"), TEXT("Main"), SF_Vertex);

/////////////////////////////////////////////////////////////////////////////////////////

class FHairMaterialShaderElementData : public FMeshMaterialShaderElementData
{
public:
	FHairMaterialShaderElementData(
		int32 MacroGroupId, 
		int32 MaterialId, 
		int32 PrimitiveId, 
		uint32 LightChannelMask, 
		uint32 Flags, 
		float HairCoverageScale) 
	: MaterialPass_MacroGroupId(MacroGroupId)
	, MaterialPass_MaterialId(MaterialId)
	, MaterialPass_PrimitiveId(PrimitiveId)
	, MaterialPass_LightChannelMask(LightChannelMask)
	, MaterialPass_Flags(Flags)
	, MaterialPass_HairCoverageScale(HairCoverageScale) 
	{ }

	uint32 MaterialPass_MacroGroupId;
	uint32 MaterialPass_MaterialId;
	uint32 MaterialPass_PrimitiveId;
	uint32 MaterialPass_LightChannelMask;
	uint32 MaterialPass_Flags;
	uint32 MaterialPass_HairCoverageScale;
};

#define HAIR_MATERIAL_DEBUG_OUTPUT 0
static bool IsPlatformRequiringRenderTargetForMaterialPass(EShaderPlatform Platform)
{
	return HAIR_MATERIAL_DEBUG_OUTPUT || FDataDrivenShaderPlatformInfo::GetRequiresRenderTargetDuringRaster(Platform); //#hair_todo: change to a proper RHI(Platform) function
}

class FHairMaterialPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FHairMaterialPS, MeshMaterial);

public:
	FHairMaterialPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel((EShaderPlatform)Initializer.Target.Platform);
		check(FSceneInterface::GetShadingPath(FeatureLevel) != EShadingPath::Mobile);
		MaterialPass_MacroGroupId.Bind(Initializer.ParameterMap, TEXT("MaterialPass_MacroGroupId"));
		MaterialPass_MaterialId.Bind(Initializer.ParameterMap, TEXT("MaterialPass_MaterialId"));
		MaterialPass_PrimitiveId.Bind(Initializer.ParameterMap, TEXT("MaterialPass_PrimitiveId"));
		MaterialPass_LightChannelMask.Bind(Initializer.ParameterMap, TEXT("MaterialPass_LightChannelMask"));
		MaterialPass_Flags.Bind(Initializer.ParameterMap, TEXT("MaterialPass_Flags"));
		MaterialPass_HairCoverageScale.Bind(Initializer.ParameterMap, TEXT("MaterialPass_HairCoverageScale"));
	}

	FHairMaterialPS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsCompatibleWithHairStrands(Parameters.Platform, Parameters.MaterialParameters) && HairStrands::IsHairStrandsVF(Parameters.VertexFactoryType);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const bool bPlatformRequireRenderTarget = IsPlatformRequiringRenderTargetForMaterialPass(Parameters.Platform);
		const bool bHasEmissiveConnected = Parameters.MaterialParameters.bHasEmissiveColorConnected;
		OutEnvironment.SetDefine(TEXT("HAIR_MATERIAL_EMISSIVE_OUTPUT"), (bHasEmissiveConnected || bPlatformRequireRenderTarget) ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("HAIRSTRANDS_HAS_NORMAL_CONNECTED"), Parameters.MaterialParameters.bHasNormalConnected ? 1 : 0);

		const EPixelFormat Format = FHairLightSampleClearPS::GetHairLightSampleFormat();
		OutEnvironment.SetRenderTargetOutputFormat(0, Format);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FHairMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
		ShaderBindings.Add(MaterialPass_MacroGroupId, ShaderElementData.MaterialPass_MacroGroupId);
		ShaderBindings.Add(MaterialPass_MaterialId, ShaderElementData.MaterialPass_MaterialId);
		ShaderBindings.Add(MaterialPass_PrimitiveId, ShaderElementData.MaterialPass_PrimitiveId);
		ShaderBindings.Add(MaterialPass_LightChannelMask, ShaderElementData.MaterialPass_LightChannelMask);
		ShaderBindings.Add(MaterialPass_Flags, ShaderElementData.MaterialPass_Flags);
		ShaderBindings.Add(MaterialPass_HairCoverageScale, ShaderElementData.MaterialPass_HairCoverageScale);
	}

private:
	LAYOUT_FIELD(FShaderParameter, MaterialPass_MacroGroupId);
	LAYOUT_FIELD(FShaderParameter, MaterialPass_MaterialId);
	LAYOUT_FIELD(FShaderParameter, MaterialPass_PrimitiveId);
	LAYOUT_FIELD(FShaderParameter, MaterialPass_LightChannelMask);
	LAYOUT_FIELD(FShaderParameter, MaterialPass_Flags);
	LAYOUT_FIELD(FShaderParameter, MaterialPass_HairCoverageScale);
};
IMPLEMENT_MATERIAL_SHADER_TYPE(, FHairMaterialPS, TEXT("/Engine/Private/HairStrands/HairStrandsMaterialPS.usf"), TEXT("Main"), SF_Pixel);

/////////////////////////////////////////////////////////////////////////////////////////

enum class EHairMaterialPassFilter : uint8
{
	All,
	EmissiveOnly,
	NonEmissiveOnly
};

static bool ShouldRenderHairStrands(ERHIFeatureLevel::Type FeatureLevel, const FMaterial& Material, const FVertexFactoryType* VFType, bool bPrimitiveRenderInPass)
{
	// Determine the mesh's material and blend mode.
	const bool bIsCompatible = IsCompatibleWithHairStrands(&Material, FeatureLevel);
	const bool bIsHairStrandsFactory = HairStrands::IsHairStrandsVF(VFType);

	return bPrimitiveRenderInPass && bIsCompatible && bIsHairStrandsFactory && ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain());
}

class FHairMaterialProcessor : public FMeshPassProcessor
{
public:
	FHairMaterialProcessor(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand,				
		FDynamicPassMeshDrawListContext* InDrawListContext,
		EHairMaterialPassFilter InFilter);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, int32 MacroGroupId, int32 HairMaterialId, uint32 HairFlags, float HairCoverageScale);

	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		uint32 MacroGroupId,
		uint32 HairMaterialId,
		uint32 HairFlags,
		float HairCoverageScale,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		const int32 MacroGroupId,
		const int32 HairMaterialId,
		const int32 HairControlPointId,
		const uint32 HairPrimitiveLightChannelMask,
		const uint32 HairFlags,
		const float HairCoverageScale);

	void SetupDrawRenderState(EHairMaterialPassFilter InFilter);

	FMeshPassProcessorRenderState PassDrawRenderState;
	EHairMaterialPassFilter Filter;
};

void FHairMaterialProcessor::SetupDrawRenderState(EHairMaterialPassFilter InFilter)
{
	if (InFilter == EHairMaterialPassFilter::All || InFilter == EHairMaterialPassFilter::EmissiveOnly)
	{
		PassDrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_Zero>::GetRHI());
	}
	else
	{
		PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	}
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState <false, CF_Always> ::GetRHI());
}

void FHairMaterialProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	AddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, 0/*MacroGroupId*/, 0/*HairMaterialId*/, 0/*HairFlags*/, 1.f/*HairCoverageScale*/);
}

void FHairMaterialProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, int32 MacroGroupId, int32 HairMaterialId, uint32 HairFlags, float HairCoverageScale)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MacroGroupId, HairMaterialId, HairFlags, HairCoverageScale, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

bool FHairMaterialProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	uint32 MacroGroupId,
	uint32 HairMaterialId,
	uint32 HairFlags,
	float HairCoverageScale,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	const bool bPrimitiveRenderInPass = (!PrimitiveSceneProxy && MeshBatch.Elements.Num() > 0) || (PrimitiveSceneProxy && PrimitiveSceneProxy->ShouldRenderInMainPass());
	const bool bShouldRender = ShouldRenderHairStrands(FeatureLevel, Material, MeshBatch.VertexFactory->GetType(), bPrimitiveRenderInPass);
	if (bShouldRender)
	{
		// For the mesh patch to be rendered a single triangle triangle to spawn the necessary amount of thread
		FMeshBatch MeshBatchCopy = MeshBatch;
		for (uint32 ElementIt = 0, ElementCount = uint32(MeshBatch.Elements.Num()); ElementIt < ElementCount; ++ElementIt)
		{
			MeshBatchCopy.Elements[ElementIt].FirstIndex = 0;
			MeshBatchCopy.Elements[ElementIt].NumPrimitives = 1;
			MeshBatchCopy.Elements[ElementIt].NumInstances = 1;
			MeshBatchCopy.Elements[ElementIt].IndirectArgsBuffer = nullptr;
			MeshBatchCopy.Elements[ElementIt].IndirectArgsOffset = 0;
		}

		FPrimitiveSceneInfo* SceneInfo = PrimitiveSceneProxy ? PrimitiveSceneProxy->GetPrimitiveSceneInfo() : nullptr;
		FMeshDrawCommandPrimitiveIdInfo IdInfo = GetDrawCommandPrimitiveId(SceneInfo, MeshBatch.Elements[0]);
		uint32 LightChannelMask = PrimitiveSceneProxy ? PrimitiveSceneProxy->GetLightingChannelMask() : 0;

		return Process(MeshBatchCopy, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, MacroGroupId, HairMaterialId, IdInfo.DrawPrimitiveId, LightChannelMask, HairFlags, HairCoverageScale);
	}

	return true;
}

bool FHairMaterialProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	const int32 MacroGroupId,
	const int32 HairMaterialId,
	const int32 HairControlPointId,
	const uint32 HairPrimitiveLightChannelMask,
	const uint32 HairFlags,
	const float HairCoverageScale)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		FHairMaterialVS,
		FHairMaterialPS> PassShaders;
	{
		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FHairMaterialVS>();
		ShaderTypes.AddShaderType<FHairMaterialPS>();

		FVertexFactoryType* VertexFactoryType = VertexFactory->GetType();

		FMaterialShaders Shaders;
		if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
		{
			return false;
		}

		Shaders.TryGetVertexShader(PassShaders.VertexShader);
		Shaders.TryGetPixelShader(PassShaders.PixelShader);
	}

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);
	FHairMaterialShaderElementData ShaderElementData(MacroGroupId, HairMaterialId, HairControlPointId, HairPrimitiveLightChannelMask, HairFlags, HairCoverageScale);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const bool bReverseCulling = ViewIfDynamicMeshCommand ? ViewIfDynamicMeshCommand->bIsPlanarReflection : false;
	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		DrawRenderState,
		PassShaders,
		ERasterizerFillMode::FM_Solid,
		bReverseCulling ? ERasterizerCullMode::CM_CW : ERasterizerCullMode::CM_CCW,
		FMeshDrawCommandSortKey::Default,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

void FHairMaterialProcessor::CollectPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig, 
	const FMaterial& Material, 
	const FPSOPrecacheVertexFactoryData& VertexFactoryData, 
	const FPSOPrecacheParams& PreCacheParams, 
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	const bool bShouldRender = ShouldRenderHairStrands(FeatureLevel, Material, VertexFactoryData.VertexFactoryType, PreCacheParams.bRenderInMainPass);
	if (!bShouldRender)
	{
		return;
	}

	TMeshProcessorShaders<
		FHairMaterialVS,
		FHairMaterialPS> PassShaders;
	{
		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FHairMaterialVS>();
		ShaderTypes.AddShaderType<FHairMaterialPS>();

		FMaterialShaders Shaders;
		if (!Material.TryGetShaders(ShaderTypes, VertexFactoryData.VertexFactoryType, Shaders))
		{
			return;
		}

		Shaders.TryGetVertexShader(PassShaders.VertexShader);
		Shaders.TryGetPixelShader(PassShaders.PixelShader);
	}
		
	const auto AddPSOInitializer = [&](EHairMaterialPassFilter InFilter)
	{
		FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
		switch (InFilter)
		{
		case EHairMaterialPassFilter::All:
		case EHairMaterialPassFilter::EmissiveOnly:
		{
			RenderTargetsInfo.NumSamples = 1;
			const EPixelFormat Format = FHairLightSampleClearPS::GetHairLightSampleFormat();
			AddRenderTargetInfo(Format, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable, RenderTargetsInfo);
			break;
		}
		case EHairMaterialPassFilter::NonEmissiveOnly:
		{
			// No render targets
			break;
		}
		}

		SetupDrawRenderState(InFilter);

		// Generate version for planar reflections as well?
		bool bReverseCulling = false;
		AddGraphicsPipelineStateInitializer(
			VertexFactoryData,
			Material,
			PassDrawRenderState,
			RenderTargetsInfo,
			PassShaders,
			ERasterizerFillMode::FM_Solid,
			bReverseCulling ? ERasterizerCullMode::CM_CW : ERasterizerCullMode::CM_CCW,
			(EPrimitiveType)PreCacheParams.PrimitiveType,
			EMeshPassFeatures::Default,
			true /*bRequired*/,
			PSOInitializers);
	};

	AddPSOInitializer(EHairMaterialPassFilter::All);
	AddPSOInitializer(EHairMaterialPassFilter::EmissiveOnly);
	AddPSOInitializer(EHairMaterialPassFilter::NonEmissiveOnly);
}

static const TCHAR* HairMaterialPassName = TEXT("HairMaterial");

FHairMaterialProcessor::FHairMaterialProcessor(
	const FScene* Scene,
	ERHIFeatureLevel::Type FeatureLevel,
	const FSceneView* InViewIfDynamicMeshCommand,
	FDynamicPassMeshDrawListContext* InDrawListContext,
	EHairMaterialPassFilter InFilter)
	: FMeshPassProcessor(HairMaterialPassName, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, Filter(InFilter)
{
	SetupDrawRenderState(Filter);
}

IPSOCollector* CreatePSOCollectorHairMaterial(ERHIFeatureLevel::Type FeatureLevel)
{
	return new FHairMaterialProcessor(nullptr, FeatureLevel, nullptr, nullptr, EHairMaterialPassFilter::All);
}
FRegisterPSOCollectorCreateFunction RegisterPSOCollectorHairMaterial(&CreatePSOCollectorHairMaterial, EShadingPath::Deferred, HairMaterialPassName);

/////////////////////////////////////////////////////////////////////////////////////////

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVisibilityMaterialPassUniformParameters, )
	SHADER_PARAMETER(FIntPoint, MaxResolution)
	SHADER_PARAMETER(uint32, MaxSampleCount)
	SHADER_PARAMETER(uint32, NodeGroupSize)
	SHADER_PARAMETER(uint32, bUpdateSampleCoverage)
	SHADER_PARAMETER(uint32, bInterpolationEnabled)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, NodeIndex)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TotalNodeCounter)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint2>, NodeCoord)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedHairVis>, NodeVis)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, IndirectArgs)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPackedHairSample>, OutNodeData)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, OutNodeVelocity)
END_GLOBAL_SHADER_PARAMETER_STRUCT()
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FVisibilityMaterialPassUniformParameters, "MaterialPassParameters", SceneTextures);

BEGIN_SHADER_PARAMETER_STRUCT(FVisibilityMaterialPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVisibilityMaterialPassUniformParameters, UniformBuffer)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

///////////////////////////////////////////////////////////////////////////////////////////////////
// Patch sample coverage
class FUpdateSampleCoverageCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FUpdateSampleCoverageCS);
	SHADER_USE_PARAMETER_STRUCT(FUpdateSampleCoverageCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, NodeIndexAndOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedHairSample>,  InNodeDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPackedHairSample>, OutNodeDataBuffer)
	END_SHADER_PARAMETER_STRUCT()
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
};

IMPLEMENT_GLOBAL_SHADER(FUpdateSampleCoverageCS, "/Engine/Private/HairStrands/HairStrandsVisibilityComputeSampleCoverage.usf", "MainCS", SF_Compute);

static FRDGBufferRef AddUpdateSampleCoveragePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo* View,
	const FRDGTextureRef NodeIndexAndOffset,
	const FRDGBufferRef InNodeDataBuffer)
{
	FRDGBufferRef OutNodeDataBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(InNodeDataBuffer->Desc.BytesPerElement, InNodeDataBuffer->Desc.NumElements), TEXT("Hair.CompactNodeData"));

	FUpdateSampleCoverageCS::FParameters* Parameters = GraphBuilder.AllocParameters<FUpdateSampleCoverageCS::FParameters>();
	Parameters->Resolution = NodeIndexAndOffset->Desc.Extent;
	Parameters->NodeIndexAndOffset = NodeIndexAndOffset;
	Parameters->InNodeDataBuffer = GraphBuilder.CreateSRV(InNodeDataBuffer);
	Parameters->OutNodeDataBuffer = GraphBuilder.CreateUAV(OutNodeDataBuffer);

	TShaderMapRef<FUpdateSampleCoverageCS> ComputeShader(View->ShaderMap);

	// Add 64 threads permutation
	const uint32 GroupSizeX = 8;
	const uint32 GroupSizeY = 4;
	const FIntVector DispatchCount = FIntVector(
		(Parameters->Resolution.X + GroupSizeX-1) / GroupSizeX, 
		(Parameters->Resolution.Y + GroupSizeY-1) / GroupSizeY, 
		1);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::VisbilityUpdateCoverage"),
		ComputeShader,
		Parameters,
		DispatchCount);

	return OutNodeDataBuffer;
}
///////////////////////////////////////////////////////////////////////////////////////////////////
struct FMaterialPassOutput
{
	FRDGBufferRef NodeData = nullptr;
	FRDGBufferRef NodeVelocity = nullptr;
	FRDGBufferSRVRef NodeVelocitySRV = nullptr;
	FRDGTextureRef SampleLightingTexture = nullptr;
};

static void AddAttributeDebugPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FHairStrandsMacroGroupDatas& MacroGroupDatas, const FRDGTextureRef& NodeIndexTexture, const FRDGBufferSRVRef& NodeVisBuffer);

static FMaterialPassOutput AddHairMaterialPass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* ViewInfo,
	const bool bUpdateSampleCoverage,
	const bool bInteroplationEnabled,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	FInstanceCullingManager& InstanceCullingManager,
	const uint32 NodeGroupSize,
	FRDGTextureRef CompactNodeIndex,
	FRDGBufferRef CompactNodeVis,
	FRDGBufferRef CompactNodeCoord,
	FRDGBufferRef CompactNodeCounter,
	FRDGBufferRef IndirectArgBuffer)
{
	if (!CompactNodeVis || !CompactNodeIndex)
		return FMaterialPassOutput();

	const uint32 MaxNodeCount = CompactNodeVis->Desc.NumElements;

	// Sanity check
	const EPixelFormat VelocityFormat = FVelocityRendering::GetFormat(ViewInfo->GetShaderPlatform());
	check(VelocityFormat == PF_A16B16G16R16 || VelocityFormat == PF_G16R16);
		
	FRDGBufferSRVRef CompactNodeCounterSRV = GraphBuilder.CreateSRV(CompactNodeCounter);

	FMaterialPassOutput Output;
	Output.NodeData				 = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(HairStrandsVisibilityInternal::NodeData), MaxNodeCount), TEXT("Hair.CompactNodeData"));
	Output.NodeVelocity			 = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(VelocityFormat == PF_G16R16 ? 4 : 8, CompactNodeVis->Desc.NumElements), TEXT("Hair.CompactNodeVelocity"));
	Output.SampleLightingTexture = AddClearLightSamplePass(GraphBuilder, ViewInfo, MaxNodeCount, CompactNodeCounterSRV);
	Output.NodeVelocitySRV		 = GraphBuilder.CreateSRV(Output.NodeVelocity, VelocityFormat);

	const uint32 ResolutionDim = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(MaxNodeCount)));
	const FIntPoint Resolution(ResolutionDim, ResolutionDim);

	const ERHIFeatureLevel::Type FeatureLevel = ViewInfo->FeatureLevel;

	// Find among the mesh batch, if any of them emit emissive data
	bool bHasEmissiveMaterial = false;
	for (const FHairStrandsMacroGroupData& MacroGroupData : MacroGroupDatas)
	{
		for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : MacroGroupData.PrimitivesInfos)
		{
			if (const FMeshBatch* MeshBatch = PrimitiveInfo.Mesh)
			{
				if (MeshBatch->MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel).HasEmissiveColorConnected())
				{
					bHasEmissiveMaterial = true;
					break;
				}
			}
		}
	}

	// Generic material pass dispatch
	auto MaterialPass = [&](FRDGTextureRef RenderTarget, EHairMaterialPassFilter Filter)
	{
		// Add resources reference to the pass parameters, in order to get the resource lifetime extended to this pass
		FVisibilityMaterialPassParameters* PassParameters = GraphBuilder.AllocParameters<FVisibilityMaterialPassParameters>();

		{
			FVisibilityMaterialPassUniformParameters* UniformParameters = GraphBuilder.AllocParameters<FVisibilityMaterialPassUniformParameters>();

			UniformParameters->bUpdateSampleCoverage = bUpdateSampleCoverage ? 1 : 0;
			UniformParameters->bInterpolationEnabled = bInteroplationEnabled ? 1 : 0;
			UniformParameters->MaxResolution = Resolution;
			UniformParameters->NodeGroupSize = NodeGroupSize;
			UniformParameters->MaxSampleCount = MaxNodeCount;
			UniformParameters->TotalNodeCounter = CompactNodeCounterSRV;
			UniformParameters->NodeIndex = CompactNodeIndex;
			UniformParameters->NodeVis = GraphBuilder.CreateSRV(CompactNodeVis);
			UniformParameters->NodeCoord = GraphBuilder.CreateSRV(CompactNodeCoord, FHairStrandsVisibilityData::NodeCoordFormat);
			UniformParameters->IndirectArgs = GraphBuilder.CreateSRV(IndirectArgBuffer);
			UniformParameters->OutNodeData = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.NodeData));
			UniformParameters->OutNodeVelocity = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Output.NodeVelocity, VelocityFormat));

			PassParameters->UniformBuffer = GraphBuilder.CreateUniformBuffer(UniformParameters);
		}

		{
			const bool bEnableMSAA = false;
			SetUpViewHairRenderInfo(*ViewInfo, bEnableMSAA, ViewInfo->CachedViewUniformShaderParameters->HairRenderInfo, ViewInfo->CachedViewUniformShaderParameters->HairRenderInfoBits, ViewInfo->CachedViewUniformShaderParameters->HairComponents, ViewInfo->Family->FinalResolutionScale);
			PassParameters->View = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(*ViewInfo->CachedViewUniformShaderParameters, UniformBuffer_SingleFrame);
		}

		if (RenderTarget)
		{
			PassParameters->RenderTargets[0] = FRenderTargetBinding(RenderTarget, ERenderTargetLoadAction::EClear, 0);
		}

		AddSimpleMeshPass(
			GraphBuilder, 
			PassParameters, 
			Scene, 
			*ViewInfo, 
			&InstanceCullingManager, 
			RDG_EVENT_NAME("HairStrands::MaterialPass(Emissive=%s)", Filter == EHairMaterialPassFilter::All ? TEXT("On/Off") : (Filter == EHairMaterialPassFilter::EmissiveOnly ? TEXT("On") : TEXT("Off"))),
			FIntRect(0, 0, Resolution.X, Resolution.Y),
			false /*bAllowOverrideIndirectArgs*/,
		[PassParameters, Scene = Scene, ViewInfo, &MacroGroupDatas, MaxNodeCount, Resolution, NodeGroupSize, bUpdateSampleCoverage, Filter, FeatureLevel](FDynamicPassMeshDrawListContext* ShadowContext)
		{			
			FHairMaterialProcessor MeshProcessor(Scene, Scene->GetFeatureLevel(), ViewInfo, ShadowContext, Filter);
			for (const FHairStrandsMacroGroupData& MacroGroupData : MacroGroupDatas)
			{
				for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : MacroGroupData.PrimitivesInfos)
				{
					if (const FMeshBatch* MeshBatch = PrimitiveInfo.Mesh)
					{
						const uint64 BatchElementMask = ~0ull;
						bool bIsCompatible = true;
						if (Filter != EHairMaterialPassFilter::All)
						{
							const bool bHasEmissive = MeshBatch->MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel).HasEmissiveColorConnected();
							bIsCompatible = (bHasEmissive && Filter == EHairMaterialPassFilter::EmissiveOnly) || (!bHasEmissive && Filter == EHairMaterialPassFilter::NonEmissiveOnly);
						}

						if (bIsCompatible)
						{
							MeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveInfo.PrimitiveSceneProxy, -1, MacroGroupData.MacroGroupId, PrimitiveInfo.MaterialId, PrimitiveInfo.Flags, PrimitiveInfo.PublicDataPtr->GetActiveStrandsCoverageScale());
						}
					}
				}
			}
		});
	};

	const bool bIsPlatformRequireRenderTarget = IsPlatformRequiringRenderTargetForMaterialPass(Scene->GetShaderPlatform()) || GRHIRequiresRenderTargetForPixelShaderUAVs;

	// Output:
	// 1. Single pass: when the platform require an RT as output, render both emissive & non-emissive in a single pass
	// 2. Two passes : one pass for emissive material with an RT, one pass for regular/non-emissive material without an RT
	// 3. Single pass: when there is no emissive material, and platform does not require an RT
	if (bIsPlatformRequireRenderTarget)
	{
		MaterialPass(Output.SampleLightingTexture, EHairMaterialPassFilter::All);
	}
	else if (bHasEmissiveMaterial)
	{
		MaterialPass(Output.SampleLightingTexture, EHairMaterialPassFilter::EmissiveOnly);
		MaterialPass(nullptr, EHairMaterialPassFilter::NonEmissiveOnly);
	}
	else
	{
		MaterialPass(nullptr, EHairMaterialPassFilter::NonEmissiveOnly);
	}

	// Add debug attribute pass
	const EGroomViewMode ViewMode = GetGroomViewMode(*ViewInfo);
	const bool bDebugEnabled = 
		ViewMode == EGroomViewMode::UV
		|| ViewMode == EGroomViewMode::RootUV
		|| ViewMode == EGroomViewMode::RootUDIM
		|| ViewMode == EGroomViewMode::Seed
		|| ViewMode == EGroomViewMode::Dimension
		|| ViewMode == EGroomViewMode::RadiusVariation
		|| ViewMode == EGroomViewMode::Tangent
		|| ViewMode == EGroomViewMode::Color
		|| ViewMode == EGroomViewMode::Roughness
		|| ViewMode == EGroomViewMode::AO
		|| ViewMode == EGroomViewMode::ClumpID;
	if (bDebugEnabled)
	{	
		AddAttributeDebugPass(GraphBuilder, *ViewInfo, MacroGroupDatas, CompactNodeIndex, GraphBuilder.CreateSRV(CompactNodeVis));
	}
	
	return Output;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairAttributeDebugCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairAttributeDebugCS);
	SHADER_USE_PARAMETER_STRUCT(FHairAttributeDebugCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FHairInstance, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceCommonParameters, Common)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceResourceParameters, Resources)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MaterialId)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, NodeIndexTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedHairVis>, NodeVisBuffer)
		SHADER_PARAMETER_STRUCT(FHairInstance, HairInstance)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()


public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters &Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters &Parameters, FShaderCompilerEnvironment &OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ATTRIBUTE_DEBUG"), 1);
	}
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairAttributeDebugCS, "/Engine/Private/HairStrands/HairStrandsDebug.usf", "CSMain", SF_Compute);

static void AddAttributeDebugPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FHairStrandsMacroGroupDatas& MacroGroupDatas, const FRDGTextureRef& NodeIndexTexture, const FRDGBufferSRVRef& NodeVisBuffer)
{
	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForCharacters(2048);
	ShaderPrint::RequestSpaceForLines(256);
	if (!ShaderPrint::IsEnabled(View.ShaderPrintData))
	{
		return;
	}

	TShaderMapRef<FHairAttributeDebugCS> ComputeShader(View.ShaderMap);

	for (const FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
	{
		const FHairStrandsMacroGroupData::TPrimitiveInfos& PrimitiveSceneInfos = MacroGroup.PrimitivesInfos;
		for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : PrimitiveSceneInfos)
		{
			// If a groom is not visible in primary view, but visible in shadow view, its PrimitiveInfo.Mesh will be null.
			if (PrimitiveInfo.Mesh == nullptr || PrimitiveInfo.Mesh->Elements.Num() == 0)
			{
				continue;
			}

			const FHairGroupPublicData* HairGroupPublicData = reinterpret_cast<const FHairGroupPublicData*>(PrimitiveInfo.Mesh->Elements[0].VertexFactoryUserData);
			check(HairGroupPublicData);

			const bool bCullingEnable = HairGroupPublicData->GetCullingResultAvailable();

			FHairAttributeDebugCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairAttributeDebugCS::FParameters>();
			Parameters->MaterialId = PrimitiveInfo.MaterialId;
			Parameters->NodeIndexTexture = NodeIndexTexture;
			Parameters->NodeVisBuffer = NodeVisBuffer;
			GetHairStrandsInstanceCommon(GraphBuilder, View, HairGroupPublicData, Parameters->HairInstance.Common);
			GetHairStrandsInstanceResources(GraphBuilder, View, HairGroupPublicData, true/*bForceRegister*/, Parameters->HairInstance.Resources);
//			Parameters->HairInstance = GetHairStrandsInstanceParameters(GraphBuilder, View, HairGroupPublicData, bCullingEnable, true );
			Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintUniformBuffer);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("HairStrands::AttributeDebug"),
				ComputeShader,
				Parameters,
				FIntVector(1, 1, 1));
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairVelocityCS: public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairVelocityCS);
	SHADER_USE_PARAMETER_STRUCT(FHairVelocityCS, FGlobalShader);

	class FVelocity : SHADER_PERMUTATION_INT("PERMUTATION_VELOCITY", 4);
	class FOuputFormat : SHADER_PERMUTATION_INT("PERMUTATION_OUTPUT_FORMAT", 2);
	using FPermutationDomain = TShaderPermutationDomain<FVelocity, FOuputFormat>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )

		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER(FIntPoint, ResolutionOffset)
		SHADER_PARAMETER(float, VelocityThreshold)
		SHADER_PARAMETER(float, CoverageThreshold)
		SHADER_PARAMETER(uint32, bNeedClear)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CoverageTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, NodeIndex)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, NodeVelocity)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedHairVis>, NodeVis)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutVelocityTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutResolveMaskTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)

		SHADER_PARAMETER(FIntPoint, TileCountXY)
		SHADER_PARAMETER(uint32, TileSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint2>, TileDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HairTileCount)
		RDG_BUFFER_ACCESS(TileIndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairVelocityCS, "/Engine/Private/HairStrands/HairStrandsVelocity.usf", "MainCS", SF_Compute);

float GetHairFastResolveVelocityThreshold(const FIntPoint& Resolution);

static void AddHairVelocityPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FHairStrandsTiles& TileData,
	FRDGTextureRef& CoverageTexture,
	FRDGTextureRef& NodeIndex,
	FRDGBufferRef& NodeVis,
	FRDGBufferRef& NodeVelocity,
	FRDGTextureRef& OutVelocityTexture,
	FRDGTextureRef& OutResolveMaskTexture)
{
	const bool bWriteOutVelocity = OutVelocityTexture != nullptr;
	if (!bWriteOutVelocity)
		return;

	check(TileData.IsValid());

	// If velocity texture has not been created by the base-pass, clear it here
	const bool bNeedClear = !HasBeenProduced(OutVelocityTexture);
	if (bNeedClear)
	{
		AddHairStrandsTileClearPass(GraphBuilder, View, TileData, FHairStrandsTiles::ETileType::Other, OutVelocityTexture);
	}

	const FIntPoint Resolution = OutVelocityTexture->Desc.Extent;
	OutResolveMaskTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(Resolution, PF_R8_UINT, FClearValueBinding::None, TexCreate_UAV), TEXT("Hair.VelocityResolveMaskTexture"));

	const EPixelFormat VelocityFormat = FVelocityRendering::GetFormat(View.GetShaderPlatform());
	check(OutVelocityTexture->Desc.Format == PF_G16R16 || OutVelocityTexture->Desc.Format == PF_A16B16G16R16);
	const bool bTwoChannelsOutput = OutVelocityTexture->Desc.Format == PF_G16R16;

	FHairVelocityCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairVelocityCS::FVelocity>(bWriteOutVelocity ? FMath::Clamp(GHairVelocityType + 1, 0, 3) : 0);
	PermutationVector.Set<FHairVelocityCS::FOuputFormat>(bTwoChannelsOutput ? 0 : 1);

	FHairVelocityCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairVelocityCS::FParameters>();
	PassParameters->bNeedClear = bNeedClear ? 1u : 0u;
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->VelocityThreshold = GetHairFastResolveVelocityThreshold(Resolution);
	PassParameters->CoverageThreshold = GetHairWriteVelocityCoverageThreshold();
	PassParameters->NodeIndex = NodeIndex;
	PassParameters->NodeVis = GraphBuilder.CreateSRV(NodeVis);
	PassParameters->NodeVelocity = GraphBuilder.CreateSRV(NodeVelocity, VelocityFormat);
	PassParameters->CoverageTexture = CoverageTexture;
	PassParameters->OutVelocityTexture = GraphBuilder.CreateUAV(OutVelocityTexture);
	PassParameters->OutResolveMaskTexture = GraphBuilder.CreateUAV(OutResolveMaskTexture);

	const FHairStrandsTiles::ETileType TileType = FHairStrandsTiles::ETileType::HairAll;

	PassParameters->ResolutionOffset	= FIntPoint(0,0);
	PassParameters->Resolution			= Resolution;
	PassParameters->TileCountXY			= TileData.TileCountXY;
	PassParameters->TileSize			= TileData.TileSize;
	PassParameters->TileCountBuffer		= TileData.TileCountSRV;
	PassParameters->TileDataBuffer		= TileData.GetTileBufferSRV(TileType);
	PassParameters->TileIndirectArgs	= TileData.TileIndirectDispatchBuffer;

	TShaderMapRef<FHairVelocityCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::Velocity"),
		ComputeShader,
		PassParameters,
		PassParameters->TileIndirectArgs, TileData.GetIndirectDispatchArgOffset(TileType));
}

/////////////////////////////////////////////////////////////////////////////////////////
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FVisibilityPassUniformParameters, )
	SHADER_PARAMETER(uint32, MaxPPLLNodeCount)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, PPLLCounter)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, PPLLNodeIndex)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPackedHairVisPPLL>, PPLLNodeData)
END_GLOBAL_SHADER_PARAMETER_STRUCT()
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FVisibilityPassUniformParameters, "HairVisibilityPass", SceneTextures);

BEGIN_SHADER_PARAMETER_STRUCT(FVisibilityPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVisibilityPassUniformParameters, UniformBuffer)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

// Example: 12bytes * 8spp = 96bytes per pixel = 192Mb @ 1080p
struct FPackedHairVisPPLL
{
	uint32 Depth;
	uint32 ControlPointId_MacroGroupId;
	uint32 NextNodeIndex;
};

TRDGUniformBufferRef<FVisibilityPassUniformParameters> CreatePassDummyTextures(FRDGBuilder& GraphBuilder)
{
	FVisibilityPassUniformParameters* UniformParameters = GraphBuilder.AllocParameters<FVisibilityPassUniformParameters>();

	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(FIntPoint(1,1), PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource);
	UniformParameters->PPLLCounter		= GraphBuilder.CreateUAV(GraphBuilder.CreateTexture(Desc, TEXT("Hair.VisibilityPPLLNodeCounter")));
	UniformParameters->PPLLNodeIndex	= GraphBuilder.CreateUAV(GraphBuilder.CreateTexture(Desc, TEXT("Hair.VisibilityPPLLNodeIndex")));
	UniformParameters->PPLLNodeData		= GraphBuilder.CreateUAV(GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FPackedHairVisPPLL), 1), TEXT("Hair.DummyPPLLNodeData")));

	return GraphBuilder.CreateUniformBuffer(UniformParameters);
}

template<EHairVisibilityRenderMode RenderMode, bool bCullingEnable>
class FHairVisibilityVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FHairVisibilityVS, MeshMaterial);

protected:

	FHairVisibilityVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel((EShaderPlatform)Initializer.Target.Platform);
		check(FSceneInterface::GetShadingPath(FeatureLevel) != EShadingPath::Mobile);
	}

	FHairVisibilityVS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsCompatibleWithHairStrands(Parameters.Platform, Parameters.MaterialParameters) && HairStrands::IsHairStrandsVF(Parameters.VertexFactoryType);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const uint32 RenderModeValue = uint32(RenderMode);
		OutEnvironment.SetDefine(TEXT("HAIR_RENDER_MODE"), RenderModeValue);
		OutEnvironment.SetDefine(TEXT("USE_CULLED_CLUSTER"), bCullingEnable ? 1 : 0);
	}
};

typedef FHairVisibilityVS<HairVisibilityRenderMode_MSAA_Visibility, false >				THairVisiblityVS_MSAAVisibility_NoCulling;
typedef FHairVisibilityVS<HairVisibilityRenderMode_MSAA_Visibility, true >				THairVisiblityVS_MSAAVisibility_Culling;
typedef FHairVisibilityVS<HairVisibilityRenderMode_Transmittance, true >				THairVisiblityVS_Transmittance;
typedef FHairVisibilityVS<HairVisibilityRenderMode_TransmittanceAndHairCount, true >	THairVisiblityVS_TransmittanceAndHairCount;
typedef FHairVisibilityVS<HairVisibilityRenderMode_PPLL, true >							THairVisiblityVS_PPLL;

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, THairVisiblityVS_MSAAVisibility_NoCulling,	TEXT("/Engine/Private/HairStrands/HairStrandsVisibilityVS.usf"), TEXT("Main"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, THairVisiblityVS_MSAAVisibility_Culling,		TEXT("/Engine/Private/HairStrands/HairStrandsVisibilityVS.usf"), TEXT("Main"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, THairVisiblityVS_Transmittance,				TEXT("/Engine/Private/HairStrands/HairStrandsVisibilityVS.usf"), TEXT("Main"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, THairVisiblityVS_TransmittanceAndHairCount,	TEXT("/Engine/Private/HairStrands/HairStrandsVisibilityVS.usf"), TEXT("Main"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, THairVisiblityVS_PPLL,						TEXT("/Engine/Private/HairStrands/HairStrandsVisibilityVS.usf"), TEXT("Main"), SF_Vertex);

/////////////////////////////////////////////////////////////////////////////////////////

class FHairVisibilityShaderElementData : public FMeshMaterialShaderElementData
{
public:
	FHairVisibilityShaderElementData(uint32 InHairMacroGroupId, uint32 InHairMaterialId, uint32 InLightChannelMask, float InHairCoverageScale) : HairMacroGroupId(InHairMacroGroupId), HairMaterialId(InHairMaterialId), LightChannelMask(InLightChannelMask), HairCoverageScale(InHairCoverageScale) { }
	uint32 HairMacroGroupId;
	uint32 HairMaterialId;
	uint32 LightChannelMask;
	float HairCoverageScale;
};

template<EHairVisibilityRenderMode RenderMode>
class FHairVisibilityPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FHairVisibilityPS, MeshMaterial);

public:

	FHairVisibilityPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		:	FMeshMaterialShader(Initializer)
	{
		ERHIFeatureLevel::Type FeatureLevel = GetMaxSupportedFeatureLevel((EShaderPlatform)Initializer.Target.Platform);
		check(FSceneInterface::GetShadingPath(FeatureLevel) != EShadingPath::Mobile);
		HairVisibilityPass_HairMacroGroupIndex.Bind(Initializer.ParameterMap, TEXT("HairVisibilityPass_HairMacroGroupIndex"));
		HairVisibilityPass_HairMaterialId.Bind(Initializer.ParameterMap, TEXT("HairVisibilityPass_HairMaterialId"));
		HairVisibilityPass_LightChannelMask.Bind(Initializer.ParameterMap, TEXT("HairVisibilityPass_LightChannelMask"));
		HairVisibilityPass_HairCoverageScale.Bind(Initializer.ParameterMap, TEXT("HairVisibilityPass_HairCoverageScale"));
	}

	FHairVisibilityPS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		if (!HairStrands::IsHairStrandsVF(Parameters.VertexFactoryType))
		{
			return false;
		}

		// Disable PPLL rendering for non-PC platform
		if (RenderMode == HairVisibilityRenderMode_PPLL)
		{
			return IsCompatibleWithHairStrands(Parameters.Platform, Parameters.MaterialParameters) && IsPCPlatform(Parameters.Platform) && !IsMobilePlatform(Parameters.Platform);
		}
		else
		{
			return IsCompatibleWithHairStrands(Parameters.Platform, Parameters.MaterialParameters);
		}
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)	
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const uint32 RenderModeValue = uint32(RenderMode);
		OutEnvironment.SetDefine(TEXT("HAIR_RENDER_MODE"), RenderModeValue);

		if (RenderMode == HairVisibilityRenderMode_MSAA_Visibility)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32_UINT);
		}
		else if (RenderMode == HairVisibilityRenderMode_Transmittance)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32_FLOAT);
		}
		else if (RenderMode == HairVisibilityRenderMode_TransmittanceAndHairCount)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32_FLOAT);
			OutEnvironment.SetRenderTargetOutputFormat(1, PF_R32G32_UINT);
		}
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FHairVisibilityShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
		ShaderBindings.Add(HairVisibilityPass_HairMacroGroupIndex, ShaderElementData.HairMacroGroupId);
		ShaderBindings.Add(HairVisibilityPass_HairMaterialId, ShaderElementData.HairMaterialId);
		ShaderBindings.Add(HairVisibilityPass_LightChannelMask, ShaderElementData.LightChannelMask);
		ShaderBindings.Add(HairVisibilityPass_HairCoverageScale, ShaderElementData.HairCoverageScale);
	}

	LAYOUT_FIELD(FShaderParameter, HairVisibilityPass_HairMacroGroupIndex);
	LAYOUT_FIELD(FShaderParameter, HairVisibilityPass_HairMaterialId);
	LAYOUT_FIELD(FShaderParameter, HairVisibilityPass_LightChannelMask);
	LAYOUT_FIELD(FShaderParameter, HairVisibilityPass_HairCoverageScale);
};
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FHairVisibilityPS<HairVisibilityRenderMode_MSAA_Visibility>, TEXT("/Engine/Private/HairStrands/HairStrandsVisibilityPS.usf"), TEXT("MainVisibility"), SF_Pixel);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FHairVisibilityPS<HairVisibilityRenderMode_Transmittance>, TEXT("/Engine/Private/HairStrands/HairStrandsVisibilityPS.usf"), TEXT("MainVisibility"), SF_Pixel);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FHairVisibilityPS<HairVisibilityRenderMode_TransmittanceAndHairCount>, TEXT("/Engine/Private/HairStrands/HairStrandsVisibilityPS.usf"), TEXT("MainVisibility"), SF_Pixel);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FHairVisibilityPS<HairVisibilityRenderMode_PPLL>, TEXT("/Engine/Private/HairStrands/HairStrandsVisibilityPS.usf"), TEXT("MainVisibility"), SF_Pixel);

/////////////////////////////////////////////////////////////////////////////////////////

class FHairVisibilityProcessor : public FMeshPassProcessor
{
public:
	FHairVisibilityProcessor(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FSceneView* InViewIfDynamicMeshCommand,
		const EHairVisibilityRenderMode InRenderMode,
		FDynamicPassMeshDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, uint32 HairMacroGroupId, uint32 HairMaterialId, float HairCoverageScale, bool bCullingEnable);

	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		uint32 HairMacroGroupId,
		uint32 HairMaterialId,
		float HairCoverageScale,
		bool bCullingEnable,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	template<EHairVisibilityRenderMode RenderMode, bool bCullingEnable=true>
	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		const uint32 HairMacroGroupId,
		const uint32 HairMaterialId,
		const uint32 LightChannelMask,
		const float HairCoverageScale,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	template<EHairVisibilityRenderMode RenderMode, bool bCullingEnable = true>
	void AddPSOInitializer(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material, 
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		TArray<FPSOPrecacheData>& PSOInitializers);

	void SetupDrawRenderState(EHairVisibilityRenderMode InRenderMode);

	const EHairVisibilityRenderMode RenderMode;
	FMeshPassProcessorRenderState PassDrawRenderState;
};

void FHairVisibilityProcessor::SetupDrawRenderState(EHairVisibilityRenderMode InRenderMode)
{
	if (InRenderMode == HairVisibilityRenderMode_MSAA_Visibility)
	{
		PassDrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI());
		PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
	}
	else if (InRenderMode == HairVisibilityRenderMode_Transmittance)
	{
		PassDrawRenderState.SetBlendState(TStaticBlendState<CW_RED, BO_Add, BF_DestColor, BF_Zero, BO_Add, BF_Zero, BF_Zero>::GetRHI());
		PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
	}
	else if (InRenderMode == HairVisibilityRenderMode_TransmittanceAndHairCount)
	{
		PassDrawRenderState.SetBlendState(TStaticBlendState<
			CW_RED, BO_Add, BF_DestColor, BF_Zero, BO_Add, BF_Zero, BF_Zero,
			CW_RG, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_Zero>::GetRHI());
		PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
	}
	else if (InRenderMode == HairVisibilityRenderMode_PPLL)
	{
		PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
		PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
	}
}

void FHairVisibilityProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	AddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, 0, 0, 1.f, false);
}

void FHairVisibilityProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, uint32 HairMacroGroupId, uint32 HairMaterialId, float HairCoverageScale, bool bCullingEnable)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, HairMacroGroupId, HairMaterialId, HairCoverageScale, bCullingEnable, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

bool FHairVisibilityProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	uint32 HairMacroGroupId,
	uint32 HairMaterialId,
	float HairCoverageScale,
	bool bCullingEnable,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	const bool bPrimitiveRenderInPass = (!PrimitiveSceneProxy && MeshBatch.Elements.Num() > 0) || (PrimitiveSceneProxy && PrimitiveSceneProxy->ShouldRenderInMainPass());
	const bool bShouldRender = ShouldRenderHairStrands(FeatureLevel, Material, MeshBatch.VertexFactory->GetType(), bPrimitiveRenderInPass);
	if (bShouldRender)
	{
		const uint32 LightChannelMask = PrimitiveSceneProxy ? PrimitiveSceneProxy->GetLightingChannelMask() : 0;

		const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
		const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
		const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
		if (RenderMode == HairVisibilityRenderMode_MSAA_Visibility && bCullingEnable)
			return Process<HairVisibilityRenderMode_MSAA_Visibility, true>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, HairMacroGroupId, HairMaterialId, LightChannelMask, HairCoverageScale, MeshFillMode, MeshCullMode);
		else if (RenderMode == HairVisibilityRenderMode_MSAA_Visibility && !bCullingEnable)
			return Process<HairVisibilityRenderMode_MSAA_Visibility, false>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, HairMacroGroupId, HairMaterialId, LightChannelMask, HairCoverageScale, MeshFillMode, MeshCullMode);
		else if (RenderMode == HairVisibilityRenderMode_Transmittance)
			return Process<HairVisibilityRenderMode_Transmittance>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, HairMacroGroupId, HairMaterialId, LightChannelMask, HairCoverageScale, MeshFillMode, MeshCullMode);
		else if (RenderMode == HairVisibilityRenderMode_TransmittanceAndHairCount)
			return Process<HairVisibilityRenderMode_TransmittanceAndHairCount>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, HairMacroGroupId, HairMaterialId, LightChannelMask, HairCoverageScale, MeshFillMode, MeshCullMode);
		else if (RenderMode == HairVisibilityRenderMode_PPLL)
			return Process<HairVisibilityRenderMode_PPLL>(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, HairMacroGroupId, HairMaterialId, LightChannelMask, HairCoverageScale, MeshFillMode, MeshCullMode);
	}

	return true;
}

template<EHairVisibilityRenderMode TRenderMode, bool bCullingEnable>
bool FHairVisibilityProcessor::Process(
	const FMeshBatch& MeshBatch, 
	uint64 BatchElementMask, 
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	const uint32 HairMacroGroupId,
	const uint32 HairMaterialId,
	const uint32 LightChannelMask,
	const float HairCoverageScale,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		FHairVisibilityVS<TRenderMode, bCullingEnable>,
		FHairVisibilityPS<TRenderMode>> PassShaders;
	{
		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FHairVisibilityVS<TRenderMode, bCullingEnable>>();
		ShaderTypes.AddShaderType<FHairVisibilityPS<TRenderMode>>();

		FVertexFactoryType* VertexFactoryType = VertexFactory->GetType();

		FMaterialShaders Shaders;
		if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
		{
			return false;
		}

		Shaders.TryGetVertexShader(PassShaders.VertexShader);
		Shaders.TryGetPixelShader(PassShaders.PixelShader);
	}

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);
	FHairVisibilityShaderElementData ShaderElementData(HairMacroGroupId, HairMaterialId, LightChannelMask, HairCoverageScale);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		DrawRenderState,
		PassShaders,
		MeshFillMode,
		MeshCullMode,
		FMeshDrawCommandSortKey::Default,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

void FHairVisibilityProcessor::CollectPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig, 
	const FMaterial& Material, 
	const FPSOPrecacheVertexFactoryData& VertexFactoryData, 
	const FPSOPrecacheParams& PreCacheParams, 
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	const bool bShouldRender = ShouldRenderHairStrands(FeatureLevel, Material, VertexFactoryData.VertexFactoryType, PreCacheParams.bRenderInMainPass);
	if (!bShouldRender)
	{
		return;
	}

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);

	AddPSOInitializer<HairVisibilityRenderMode_MSAA_Visibility, true>(SceneTexturesConfig, Material, VertexFactoryData, PreCacheParams, MeshFillMode, MeshCullMode, PSOInitializers);
	AddPSOInitializer<HairVisibilityRenderMode_MSAA_Visibility, false>(SceneTexturesConfig, Material, VertexFactoryData, PreCacheParams, MeshFillMode, MeshCullMode, PSOInitializers);
	AddPSOInitializer<HairVisibilityRenderMode_Transmittance>(SceneTexturesConfig, Material, VertexFactoryData, PreCacheParams, MeshFillMode, MeshCullMode, PSOInitializers);
	AddPSOInitializer<HairVisibilityRenderMode_TransmittanceAndHairCount>(SceneTexturesConfig, Material, VertexFactoryData, PreCacheParams, MeshFillMode, MeshCullMode, PSOInitializers);
	AddPSOInitializer<HairVisibilityRenderMode_PPLL>(SceneTexturesConfig, Material, VertexFactoryData, PreCacheParams, MeshFillMode, MeshCullMode, PSOInitializers);
}

template<EHairVisibilityRenderMode TRenderMode, bool bCullingEnable>
void FHairVisibilityProcessor::AddPSOInitializer(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	SetupDrawRenderState(TRenderMode);

	TMeshProcessorShaders<
		FHairVisibilityVS<TRenderMode, bCullingEnable>,
		FHairVisibilityPS<TRenderMode>> PassShaders;
	{
		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FHairVisibilityVS<TRenderMode, bCullingEnable>>();
		ShaderTypes.AddShaderType<FHairVisibilityPS<TRenderMode>>();

		FMaterialShaders Shaders;
		if (!Material.TryGetShaders(ShaderTypes, VertexFactoryData.VertexFactoryType, Shaders))
		{
			return;
		}

		Shaders.TryGetVertexShader(PassShaders.VertexShader);
		Shaders.TryGetPixelShader(PassShaders.PixelShader);
	}

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;

	if (TRenderMode == HairVisibilityRenderMode_MSAA_Visibility)
	{
		const uint32 MSAASampleCount = GetMaxSamplePerPixel(GetFeatureLevelShaderPlatform(FeatureLevel));
		RenderTargetsInfo.NumSamples = MSAASampleCount;

		AddRenderTargetInfo(PF_R32_UINT, TexCreate_NoFastClear | TexCreate_RenderTargetable | TexCreate_ShaderResource, RenderTargetsInfo);
		SetupDepthStencilInfo(PF_D24, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop, RenderTargetsInfo);
	}
	else if (TRenderMode == HairVisibilityRenderMode_Transmittance || TRenderMode == HairVisibilityRenderMode_TransmittanceAndHairCount)
	{
		AddRenderTargetInfo(PF_R32_FLOAT, TexCreate_RenderTargetable | TexCreate_ShaderResource, RenderTargetsInfo);
		if (TRenderMode == HairVisibilityRenderMode_TransmittanceAndHairCount)
		{
			AddRenderTargetInfo(PF_G32R32F, TexCreate_RenderTargetable | TexCreate_ShaderResource, RenderTargetsInfo);
		}

		SetupDepthStencilInfo(PF_DepthStencil, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead, RenderTargetsInfo);
	}
	else if (TRenderMode == HairVisibilityRenderMode_PPLL)
	{
		SetupDepthStencilInfo(PF_DepthStencil, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead, RenderTargetsInfo);
	}

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		Material,
		PassDrawRenderState,
		RenderTargetsInfo,
		PassShaders,
		MeshFillMode,
		MeshCullMode,
		(EPrimitiveType)PreCacheParams.PrimitiveType,
		EMeshPassFeatures::Default,
		true /*bRequired*/,
		PSOInitializers);
}

static const TCHAR* HairVisibilityPassName = TEXT("HairVisibility");

FHairVisibilityProcessor::FHairVisibilityProcessor(
	const FScene* Scene,
	ERHIFeatureLevel::Type FeatureLevel,
	const FSceneView* InViewIfDynamicMeshCommand,
	const EHairVisibilityRenderMode InRenderMode,
	FDynamicPassMeshDrawListContext* InDrawListContext)
	: FMeshPassProcessor(HairVisibilityPassName, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, RenderMode(InRenderMode)
{
	SetupDrawRenderState(RenderMode);
}

IPSOCollector* CreatePSOCollectorHairVisibility(ERHIFeatureLevel::Type FeatureLevel)
{ 
	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	EHairVisibilityRenderMode RenderMode = GetHairVisibilityRenderMode(ShaderPlatform);
	return new FHairVisibilityProcessor(nullptr, FeatureLevel, nullptr, RenderMode, nullptr);
} 
FRegisterPSOCollectorCreateFunction RegisterPSOCollectorHairVisibility(&CreatePSOCollectorHairVisibility, EShadingPath::Deferred, HairVisibilityPassName);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Clear uint texture
class FClearUIntGraphicPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearUIntGraphicPS);
	SHADER_USE_PARAMETER_STRUCT(FClearUIntGraphicPS, FGlobalShader);

	class FOutputFormat : SHADER_PERMUTATION_INT("PERMUTATION_OUTPUT_FORMAT", 3);
	using FPermutationDomain = TShaderPermutationDomain<FOutputFormat>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, uClearValue)
		SHADER_PARAMETER(float, fClearValue)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileData)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FOutputFormat>() == 0)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32_UINT);
		}
		else if (PermutationVector.Get<FOutputFormat>() == 1)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32G32_UINT);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FClearUIntGraphicPS, "/Engine/Private/HairStrands/HairStrandsVisibilityClearPS.usf", "ClearPS", SF_Pixel);

static void AddClearGraphicPass(
	FRDGBuilder& GraphBuilder,
	FRDGEventName&& PassName,
	const FViewInfo* View,
	const uint32 ClearValue,
	const FHairStrandsTiles& TileData,
	FRDGTextureRef& OutTarget)
{
	check(OutTarget);
	check(TileData.IsValid());

	const FHairStrandsTiles::ETileType TileType = FHairStrandsTiles::ETileType::HairAll;

	FClearUIntGraphicPS::FParameters* Parameters = GraphBuilder.AllocParameters<FClearUIntGraphicPS::FParameters>();
	Parameters->uClearValue = ClearValue;
	Parameters->fClearValue = ClearValue;
	Parameters->TileData = GetHairStrandsTileParameters(*View, TileData, TileType);
	Parameters->RenderTargets[0] = FRenderTargetBinding(OutTarget, ERenderTargetLoadAction::ENoAction, 0);

	FClearUIntGraphicPS::FPermutationDomain PermutationVector;
	if (OutTarget->Desc.Format == PF_R32_UINT)
	{
		PermutationVector.Set<FClearUIntGraphicPS::FOutputFormat>(0);
	}
	else if (OutTarget->Desc.Format == PF_R32G32_UINT)
	{
		PermutationVector.Set<FClearUIntGraphicPS::FOutputFormat>(1);
	}
	else
	{
		PermutationVector.Set<FClearUIntGraphicPS::FOutputFormat>(2);
	}

	TShaderMapRef<FHairStrandsTilePassVS> TileVertexShader(View->ShaderMap);
	TShaderMapRef<FClearUIntGraphicPS> PixelShader(View->ShaderMap, PermutationVector);
	const FIntRect Viewport = View->ViewRect;
	const FIntPoint Resolution = OutTarget->Desc.Extent;

	//ClearUnusedGraphResources(PixelShader, Parameters);

	GraphBuilder.AddPass(
		Forward<FRDGEventName>(PassName),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, TileVertexShader, PixelShader, Viewport, Resolution, TileType](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FHairStrandsTilePassVS::FParameters ParametersVS = Parameters->TileData;

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = TileVertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = Parameters->TileData.bRectPrimitive > 0 ? PT_RectList : PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);
		SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), ParametersVS);
		RHICmdList.SetStreamSource(0, nullptr, 0);
		RHICmdList.DrawPrimitiveIndirect(Parameters->TileData.TileIndirectBuffer->GetRHI(), FHairStrandsTiles::GetIndirectDrawArgOffset(TileType));
	});
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Copy dispatch count into an indirect buffer 
class FCopyIndirectBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyIndirectBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FCopyIndirectBufferCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, ThreadGroupSize)
		SHADER_PARAMETER(uint32, ItemCountPerGroup)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CounterBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutArgBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
};

IMPLEMENT_GLOBAL_SHADER(FCopyIndirectBufferCS, "/Engine/Private/HairStrands/HairStrandsVisibilityCopyIndirectArg.usf", "CopyCS", SF_Compute);

static FRDGBufferRef AddCopyIndirectArgPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo* View,
	const uint32 ThreadGroupSize,
	const uint32 ItemCountPerGroup,
	FRDGBufferSRVRef CounterBuffer)
{
	check(CounterBuffer);

	FRDGBufferRef OutBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(), TEXT("Hair.VisibilityIndirectArgBuffer"));

	FCopyIndirectBufferCS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyIndirectBufferCS::FParameters>();
	Parameters->ThreadGroupSize = ThreadGroupSize;
	Parameters->ItemCountPerGroup = ItemCountPerGroup;
	Parameters->CounterBuffer = CounterBuffer;
	Parameters->OutArgBuffer = GraphBuilder.CreateUAV(OutBuffer);

	TShaderMapRef<FCopyIndirectBufferCS> ComputeShader(View->ShaderMap);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::VisbilityCopyIndirectArgs"),
		ComputeShader,
		Parameters,
		FIntVector(1,1,1));

	return OutBuffer;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairVisibilityControlPointIdCompactionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairVisibilityControlPointIdCompactionCS);
	SHADER_USE_PARAMETER_STRUCT(FHairVisibilityControlPointIdCompactionCS, FGlobalShader);

	class FGroupSize	: SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_GROUPSIZE", 32, 64);
	class FPPLL 		: SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_PPLL", 0, 8, 16, 32); // See GetPPLLMaxRenderNodePerPixel
	class FMSAACount 	: SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_MSAACOUNT", 1, 2, 4, 8);
	using FPermutationDomain = TShaderPermutationDomain<FGroupSize, FPPLL, FMSAACount>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(FIntPoint, ResolutionOffset)
		SHADER_PARAMETER(uint32, MaxNodeCount)
		SHADER_PARAMETER(uint32, bSortSampleByDepth)
		SHADER_PARAMETER(float, DepthTheshold)
		SHADER_PARAMETER(float, CosTangentThreshold)
		SHADER_PARAMETER(float, CoverageThreshold)

		SHADER_PARAMETER(FIntPoint, TileCountXY)
		SHADER_PARAMETER(uint32, TileSize)

		// Available for the MSAA path
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, MSAA_DepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, MSAA_IDTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, MSAA_MaterialTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, MSAA_AttributeTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, MSAA_VelocityTexture)
		// Available for the PPLL path
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PPLLCounter)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PPLLNodeIndex)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, PPLLNodeData)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ViewTransmittanceTexture)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutCompactNodeCounter)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCompactNodeIndex)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCoverageTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutCompactNodeVis)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutCompactNodeCoord)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutVelocityTexture)
		
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint2>, TileDataBuffer)						// Tile coords (RG16)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileCountBuffer)						// Tile total count (actual number of tiles)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RDG_BUFFER_ACCESS(IndirectBufferArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

public:

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FPPLL>() > 0)
		{
			PermutationVector.Set<FMSAACount>(1);
		}
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FPPLL>() > 0 && PermutationVector.Get<FMSAACount>() != 1)
		{
			return false;
		}
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairVisibilityControlPointIdCompactionCS, "/Engine/Private/HairStrands/HairStrandsVisibilityCompaction.usf", "MainCS", SF_Compute);

static void AddHairVisibilityControlPointIdCompactionPass(
	const bool bUsePPLL,
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef& SceneDepthTexture,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const uint32 NodeGroupSize,
	const FHairStrandsTiles& TileData,
	FHairVisibilityControlPointIdCompactionCS::FParameters* PassParameters,
	FRDGBufferRef& OutCompactCounterBuffer,
	FRDGTextureRef& OutCompactNodeIndex,
	FRDGBufferRef& OutCompactNodeVis,
	FRDGBufferRef& OutCompactNodeCoord,
	FRDGTextureRef& OutCoverageTexture,
	FRDGBufferRef& OutIndirectArgsBuffer,
	uint32& OutMaxRenderNodeCount)
{
	const uint32 MaxSamplePerPixel = GetMaxSamplePerPixel(View.GetShaderPlatform());
	FIntPoint Resolution;
	if (bUsePPLL)
	{
		check(PassParameters->PPLLCounter);
		check(PassParameters->PPLLNodeIndex);
		check(PassParameters->PPLLNodeData);
		Resolution = PassParameters->PPLLNodeIndex->Desc.Extent;
	}
	else
	{
		check(PassParameters->MSAA_DepthTexture->Desc.NumSamples == MaxSamplePerPixel);
		check(PassParameters->MSAA_DepthTexture);
		check(PassParameters->MSAA_IDTexture);
		Resolution = PassParameters->MSAA_DepthTexture->Desc.Extent;
	}

	{
		OutCompactCounterBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("Hair.VisibilityCompactCounter"));
	}

	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Resolution, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable);
		OutCompactNodeIndex = GraphBuilder.CreateTexture(Desc, TEXT("Hair.VisibilityCompactNodeIndex"));
	}

	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Resolution, FHairStrandsVisibilityData::CoverageFormat, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable);
		OutCoverageTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.CoverageTexture"));
	}

	const uint32 ClearValues[4] = { 0u,0u,0u,0u };
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutCompactCounterBuffer), 0u);
	AddClearGraphicPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::NodeOffsetAndCount"), &View, 0, TileData, OutCompactNodeIndex);
	AddClearGraphicPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::CoverageTexture"), &View, 0, TileData, OutCoverageTexture);

	// Adapt the buffer allocation based on the bounding box of the hair macro groups. This allows to reduce the overall allocation size
	const FIntRect HairRect = ComputeVisibleHairStrandsMacroGroupsRect(View, View.ViewRect, MacroGroupDatas);
	const FIntPoint EffectiveResolution = bUsePPLL ? FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()) : FIntPoint(HairRect.Width(), HairRect.Height());

	// Select render node count according to current mode
	const uint32 MSAASampleCount = GetHairVisibilityRenderMode(View.GetShaderPlatform()) == HairVisibilityRenderMode_MSAA_Visibility ? MaxSamplePerPixel : 1;
	const uint32 PPLLMaxRenderNodePerPixel = MaxSamplePerPixel;
	const uint32 MaxRenderNodeCount = GetTotalSampleCountForAllocation(EffectiveResolution, View.GetShaderPlatform());
	check(TileData.IsValid());

	OutCompactNodeVis = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(HairStrandsVisibilityInternal::NodeVis), MaxRenderNodeCount), TEXT("Hair.VisibilityNodeVis"));
	// Pixel coord of the node. Stored as 2*R16_UINT
	OutCompactNodeCoord = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MaxRenderNodeCount), TEXT("Hair.VisibilityNodeCoord"));

	FHairVisibilityControlPointIdCompactionCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairVisibilityControlPointIdCompactionCS::FGroupSize>(FHairStrandsTiles::GroupSize);
	PermutationVector.Set<FHairVisibilityControlPointIdCompactionCS::FPPLL>(bUsePPLL ? PPLLMaxRenderNodePerPixel : 0);
	PermutationVector.Set<FHairVisibilityControlPointIdCompactionCS::FMSAACount>(MSAASampleCount);
	PermutationVector = FHairVisibilityControlPointIdCompactionCS::RemapPermutation(PermutationVector);

	PassParameters->ResolutionOffset = FIntPoint(0,0);
	PassParameters->OutputResolution = Resolution;
	PassParameters->MaxNodeCount = MaxRenderNodeCount;
	PassParameters->bSortSampleByDepth = GHairStrandsSortHairSampleByDepth > 0 ? 1 : 0;
	PassParameters->CoverageThreshold = GetHairStrandsFullCoverageThreshold();
	PassParameters->DepthTheshold = FMath::Clamp(GHairStrandsMaterialCompactionDepthThreshold, 0.f, 100.f);
	PassParameters->CosTangentThreshold = FMath::Cos(FMath::DegreesToRadians(FMath::Clamp(GHairStrandsMaterialCompactionTangentThreshold, 0.f, 90.f)));
	PassParameters->SceneDepthTexture = SceneDepthTexture;
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->OutCompactNodeCounter = GraphBuilder.CreateUAV(OutCompactCounterBuffer);
	PassParameters->OutCompactNodeIndex = GraphBuilder.CreateUAV(OutCompactNodeIndex);
	PassParameters->OutCompactNodeVis = GraphBuilder.CreateUAV(OutCompactNodeVis);
	PassParameters->OutCompactNodeCoord = GraphBuilder.CreateUAV(OutCompactNodeCoord, FHairStrandsVisibilityData::NodeCoordFormat);
	PassParameters->OutCoverageTexture = GraphBuilder.CreateUAV(OutCoverageTexture);

	const FHairStrandsTiles::ETileType TileType = FHairStrandsTiles::ETileType::HairAll;
	PassParameters->TileCountXY			= TileData.TileCountXY;
	PassParameters->TileSize			= FHairStrandsTiles::TileSize;
	PassParameters->TileCountBuffer		= GraphBuilder.CreateSRV(TileData.TileCountBuffer, PF_R32_UINT);
	PassParameters->TileDataBuffer		= TileData.GetTileBufferSRV(TileType);
	PassParameters->IndirectBufferArgs	= TileData.TileIndirectDispatchBuffer;
 
	TShaderMapRef<FHairVisibilityControlPointIdCompactionCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::VisibilityCompaction"),
		ComputeShader,
		PassParameters,
		TileData.TileIndirectDispatchBuffer,
		FHairStrandsTiles::GetIndirectDispatchArgOffset(TileType));

	OutIndirectArgsBuffer = AddCopyIndirectArgPass(GraphBuilder, &View, NodeGroupSize, 1, GraphBuilder.CreateSRV(OutCompactCounterBuffer));
	OutMaxRenderNodeCount = MaxRenderNodeCount;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairVisibilityCompactionComputeRasterCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairVisibilityCompactionComputeRasterCS);
	SHADER_USE_PARAMETER_STRUCT(FHairVisibilityCompactionComputeRasterCS, FGlobalShader);

	class FGroupSize : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_GROUPSIZE", 32, 64);
	class FSampleCount : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_MULTI_SAMPLE_COUNT", 1, 2, 4, 8);
	using FPermutationDomain = TShaderPermutationDomain<FGroupSize, FSampleCount>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(uint32, MaxNodeCount)
		SHADER_PARAMETER(uint32, SamplerPerPixel)
		SHADER_PARAMETER(float, CoverageThreshold)
		SHADER_PARAMETER(uint32, bSortSampleByDepth)

		SHADER_PARAMETER(FIntPoint, TileCountXY)
		SHADER_PARAMETER(uint32, TileSize)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, DepthCovTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, PrimMatTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, HairCountTexture)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutCompactNodeCounter)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCompactNodeIndex)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutCoverageTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutCompactNodeVis)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutCompactNodeCoord)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint2>, TileDataBuffer)						// Tile coords (RG16)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileCountBuffer)						// Tile total count (actual number of tiles)

//		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RDG_BUFFER_ACCESS(IndirectBufferArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

public:

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);	
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairVisibilityCompactionComputeRasterCS, "/Engine/Private/HairStrands/HairStrandsVisibilityCompactionComputeRaster.usf", "MainCS", SF_Compute);

static void AddHairVisibilityCompactionComputeRasterPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const uint32 NodeGroupSize,
	const uint32 SamplesPerPixel,
	const FRasterComputeOutput& RasterComputeData,
	const FHairStrandsTiles& TileData,
	FRDGBufferRef& OutCompactCounter,
	FRDGTextureRef& OutCompactNodeIndex,
	FRDGBufferRef&  OutCompactNodeVis,
	FRDGBufferRef&  OutCompactNodeCoord,
	FRDGTextureRef& OutCoverageTexture,
	FRDGBufferRef&  OutIndirectArgsBuffer,
	uint32& OutMaxRenderNodeCount)
{	
	FIntPoint Resolution = RasterComputeData.DepthCovTexture->Desc.Extent;

	{
		OutCompactCounter = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32),1), TEXT("Hair.VisibilityCompactCounter"));
	}

	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Resolution, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable);
		OutCompactNodeIndex = GraphBuilder.CreateTexture(Desc, TEXT("Hair.VisibilityCompactNodeIndex"));
	}

	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Resolution, FHairStrandsVisibilityData::CoverageFormat, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable);
		OutCoverageTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.CoverageTexture"));
	}

	const uint32 ClearValues[4] = { 0u,0u,0u,0u };
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutCompactCounter), 0u);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutCompactNodeIndex), ClearValues);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutCoverageTexture), 0.f);

	// Select render node count according to current mode
	const uint32 MaxSamplePerPixel = GetMaxSamplePerPixel(View.GetShaderPlatform());
	check(TileData.IsValid());
	const FHairStrandsTiles::ETileType TileType = FHairStrandsTiles::ETileType::HairAll;
	const uint32 MSAASampleCount = GetHairVisibilityRenderMode(View.GetShaderPlatform()) == HairVisibilityRenderMode_MSAA_Visibility ? MaxSamplePerPixel : 1;
	const uint32 PPLLMaxRenderNodePerPixel = MaxSamplePerPixel;
	const uint32 MaxRenderNodeCount = GetTotalSampleCountForAllocation(Resolution, View.GetShaderPlatform());
	OutCompactNodeVis = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(HairStrandsVisibilityInternal::NodeVis), MaxRenderNodeCount), TEXT("Hair.VisibilityControlPointIdCompactNodeData"));
	OutCompactNodeCoord = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MaxRenderNodeCount), TEXT("Hair.VisibilityPrimitiveIdCompactNodeCoord"));

	FRDGTextureRef DefaultTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
	FHairVisibilityCompactionComputeRasterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairVisibilityCompactionComputeRasterCS::FParameters>();

	PassParameters->HairCountTexture = RasterComputeData.HairCountTexture;
	PassParameters->DepthCovTexture = RasterComputeData.DepthCovTexture;
	PassParameters->PrimMatTexture = RasterComputeData.PrimMatTexture;

	PassParameters->SamplerPerPixel			= SamplesPerPixel;
	PassParameters->OutputResolution		= Resolution;
	PassParameters->MaxNodeCount			= MaxRenderNodeCount;
	PassParameters->CoverageThreshold		= GetHairStrandsFullCoverageThreshold();
	PassParameters->bSortSampleByDepth		= GHairStrandsSortHairSampleByDepth > 0 ? 1 : 0;
	PassParameters->ViewUniformBuffer		= View.ViewUniformBuffer;
//	PassParameters->SceneTexturesStruct		= CreateSceneTextureUniformBuffer(GraphBuilder, View.GetSceneTexturesChecked(), View.FeatureLevel);
	PassParameters->OutCompactNodeCounter	= GraphBuilder.CreateUAV(OutCompactCounter);
	PassParameters->OutCompactNodeIndex		= GraphBuilder.CreateUAV(OutCompactNodeIndex);
	PassParameters->OutCompactNodeVis		= GraphBuilder.CreateUAV(OutCompactNodeVis);
	PassParameters->OutCompactNodeCoord		= GraphBuilder.CreateUAV(OutCompactNodeCoord, FHairStrandsVisibilityData::NodeCoordFormat);
	PassParameters->OutCoverageTexture		= GraphBuilder.CreateUAV(OutCoverageTexture);
	PassParameters->TileCountXY = TileData.TileCountXY;
	PassParameters->TileSize = FHairStrandsTiles::TileSize;
	PassParameters->TileCountBuffer = GraphBuilder.CreateSRV(TileData.TileCountBuffer, PF_R32_UINT);
	PassParameters->TileDataBuffer = TileData.GetTileBufferSRV(TileType);
	PassParameters->IndirectBufferArgs = TileData.TileIndirectDispatchBuffer;

	const FIntPoint GroupSize = GetVendorOptimalGroupSize2D();
	FHairVisibilityCompactionComputeRasterCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairVisibilityCompactionComputeRasterCS::FGroupSize>(FHairStrandsTiles::GroupSize);
	PermutationVector.Set<FHairVisibilityCompactionComputeRasterCS::FSampleCount>(SamplesPerPixel);
	TShaderMapRef<FHairVisibilityCompactionComputeRasterCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::VisibilityCompaction"),
		ComputeShader,
		PassParameters,
		TileData.TileIndirectDispatchBuffer,
		FHairStrandsTiles::GetIndirectDispatchArgOffset(TileType));

	OutIndirectArgsBuffer = AddCopyIndirectArgPass(GraphBuilder, &View, NodeGroupSize, 1, GraphBuilder.CreateSRV(OutCompactCounter));
	OutMaxRenderNodeCount = MaxRenderNodeCount;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairVisibilityFillOpaqueDepthPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairVisibilityFillOpaqueDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FHairVisibilityFillOpaqueDepthPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VisibilityDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VisibilityIDTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileData)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
};

IMPLEMENT_GLOBAL_SHADER(FHairVisibilityFillOpaqueDepthPS, "/Engine/Private/HairStrands/HairStrandsVisibilityFillOpaqueDepthPS.usf", "MainPS", SF_Pixel);

static FRDGTextureRef AddHairVisibilityFillOpaqueDepth(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FIntPoint& Resolution,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FHairStrandsTiles& TileData,
	const FRDGTextureRef& SceneDepthTexture)
{
	check(GetHairVisibilityRenderMode(View.GetShaderPlatform()) == HairVisibilityRenderMode_MSAA_Visibility || GetHairVisibilityRenderMode(View.GetShaderPlatform()) == HairVisibilityRenderMode_ComputeRaster);
	check(TileData.IsValid());

	const FHairStrandsTiles::ETileType TileType = FHairStrandsTiles::ETileType::HairAll;
	FRDGTextureRef OutVisibilityDepthTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(Resolution, PF_D24, FClearValueBinding::DepthFar, TexCreate_NoFastClear | TexCreate_DepthStencilTargetable | TexCreate_ShaderResource, 1, GetMaxSamplePerPixel(View.GetShaderPlatform())), TEXT("Hair.VisibilityDepthTexture"));

	FHairVisibilityFillOpaqueDepthPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairVisibilityFillOpaqueDepthPS::FParameters>();
	Parameters->TileData = GetHairStrandsTileParameters(View, TileData, TileType);
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->SceneDepthTexture = SceneDepthTexture;
	Parameters->RenderTargets.DepthStencil = FDepthStencilBinding(
		OutVisibilityDepthTexture,
		ERenderTargetLoadAction::ENoAction,
		ERenderTargetLoadAction::ENoAction,
		FExclusiveDepthStencil::DepthWrite_StencilNop);

	TShaderMapRef<FHairVisibilityFillOpaqueDepthPS> PixelShader(View.ShaderMap);
	
	const FIntRect Viewport = View.ViewRect;
	TShaderMapRef<FHairStrandsTilePassVS> TileVertexShader(View.ShaderMap);
	//ClearUnusedGraphResources(PixelShader, Parameters);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::FillVisibilityDepth(Tile)"),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, TileVertexShader, PixelShader, Viewport, TileType](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FHairStrandsTilePassVS::FParameters ParametersVS = Parameters->TileData;

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_Always>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = TileVertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = Parameters->TileData.bRectPrimitive > 0 ? PT_RectList : PT_TriangleList;
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);

			RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
			SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), ParametersVS);
			RHICmdList.SetStreamSource(0, nullptr, 0);
			RHICmdList.DrawPrimitiveIndirect(Parameters->TileData.TileIndirectBuffer->GetRHI(), FHairStrandsTiles::GetIndirectDrawArgOffset(TileType));
		});

	// Ensure HTile is valid after manually feeding the scene depth value
	if (GRHISupportsResummarizeHTile)
	{
		AddResummarizeHTilePass(GraphBuilder, OutVisibilityDepthTexture);
	}

	return OutVisibilityDepthTexture;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

static void AddHairVisibilityCommonPass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* ViewInfo,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const EHairVisibilityRenderMode RenderMode,
	FVisibilityPassParameters* PassParameters,
	FInstanceCullingManager& InstanceCullingManager)
{
	auto GetPassName = [RenderMode]()
	{
		switch (RenderMode)
		{
		case HairVisibilityRenderMode_PPLL:						return RDG_EVENT_NAME("HairStrands::VisibilityPPLLPass");
		case HairVisibilityRenderMode_MSAA_Visibility:			return RDG_EVENT_NAME("HairStrands::VisibilityMSAAVisPass");
		case HairVisibilityRenderMode_Transmittance:			return RDG_EVENT_NAME("HairStrands::TransmittancePass");
		case HairVisibilityRenderMode_TransmittanceAndHairCount:return RDG_EVENT_NAME("HairStrands::TransmittanceAndHairCountPass");
		default:												return RDG_EVENT_NAME("Noname");
		}
	};

	// Note: this reference needs to persistent until SubmitMeshDrawCommands() is called, as DrawRenderState does not ref count 
	// the view uniform buffer (raw pointer). It is only within the MeshProcessor that the uniform buffer get reference
	TUniformBufferRef<FViewUniformShaderParameters> ViewUniformShaderParameters;
	if (RenderMode == HairVisibilityRenderMode_Transmittance || RenderMode == HairVisibilityRenderMode_TransmittanceAndHairCount || RenderMode == HairVisibilityRenderMode_PPLL)
	{
		const bool bEnableMSAA = false;
		SetUpViewHairRenderInfo(*ViewInfo, bEnableMSAA, ViewInfo->CachedViewUniformShaderParameters->HairRenderInfo, ViewInfo->CachedViewUniformShaderParameters->HairRenderInfoBits, ViewInfo->CachedViewUniformShaderParameters->HairComponents, ViewInfo->Family->FinalResolutionScale);

		// Create and set the uniform buffer
		PassParameters->View = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(*ViewInfo->CachedViewUniformShaderParameters, UniformBuffer_SingleFrame);
	}
	else
	{
		PassParameters->View = ViewInfo->ViewUniformBuffer;
	}

	AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, *ViewInfo, &InstanceCullingManager, GetPassName(), ViewInfo->ViewRect, false /*bAllowOverrideIndirectArgs*/,
		[PassParameters, Scene = Scene, ViewInfo, &MacroGroupDatas, RenderMode](FDynamicPassMeshDrawListContext* ShadowContext)
	{
		check(IsInRenderingThread());

		FHairVisibilityProcessor MeshProcessor(Scene, Scene->GetFeatureLevel(),  ViewInfo, RenderMode, ShadowContext);
		for (const FHairStrandsMacroGroupData& MacroGroupData : MacroGroupDatas)
		{
			for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : MacroGroupData.PrimitivesInfos)
			{
				if (const FMeshBatch* MeshBatch = PrimitiveInfo.Mesh)
				{
					const uint64 BatchElementMask = ~0ull;
					const float HairCoverageScale = PrimitiveInfo.PublicDataPtr->GetActiveStrandsCoverageScale();
					MeshProcessor.AddMeshBatch(*MeshBatch, BatchElementMask, PrimitiveInfo.PrimitiveSceneProxy, -1, MacroGroupData.MacroGroupId, PrimitiveInfo.MaterialId, HairCoverageScale, PrimitiveInfo.IsCullingEnable());
				}
			}
		}
	});
}

static void AddHairVisibilityMSAAPass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* ViewInfo,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FIntPoint& Resolution,
	const FHairStrandsTiles& TileData,
	FInstanceCullingManager& InstanceCullingManager,
	FRDGTextureRef& OutVisibilityIdTexture,
	FRDGTextureRef& OutVisibilityDepthTexture)
{
	const uint32 MSAASampleCount = GetMaxSamplePerPixel(ViewInfo->GetShaderPlatform());
	{
		{
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Resolution, PF_R32_UINT, FClearValueBinding(EClearBinding::ENoneBound), TexCreate_NoFastClear | TexCreate_RenderTargetable | TexCreate_ShaderResource, 1, MSAASampleCount);
			OutVisibilityIdTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.VisibilityIDTexture"));
		}

		AddClearGraphicPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::ClearVisibilityMSAAIdTexture(%s)", TileData.IsValid()? TEXT("Tile") : TEXT("Screen")), ViewInfo, 0xFFFFFFFF, TileData, OutVisibilityIdTexture);

		FVisibilityPassParameters* PassParameters = GraphBuilder.AllocParameters<FVisibilityPassParameters>();
		PassParameters->UniformBuffer = CreatePassDummyTextures(GraphBuilder);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutVisibilityIdTexture, ERenderTargetLoadAction::ELoad, 0);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
			OutVisibilityDepthTexture,
			ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ENoAction,
			FExclusiveDepthStencil::DepthWrite_StencilNop);
		AddHairVisibilityCommonPass(GraphBuilder, Scene, ViewInfo, MacroGroupDatas, HairVisibilityRenderMode_MSAA_Visibility, PassParameters, InstanceCullingManager);
	}
}

static void AddHairVisibilityPPLLPass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* ViewInfo,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FIntPoint& Resolution,
	FInstanceCullingManager& InstanceCullingManager,
	FRDGTextureRef& InViewZDepthTexture,
	FRDGTextureRef& OutVisibilityPPLLNodeCounter,
	FRDGTextureRef& OutVisibilityPPLLNodeIndex,
	FRDGBufferRef&  OutVisibilityPPLLNodeData)
{
	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(FIntPoint(1,1), PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource);
		OutVisibilityPPLLNodeCounter = GraphBuilder.CreateTexture(Desc, TEXT("Hair.VisibilityPPLLCounter"));
	}

	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Resolution, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource);
		OutVisibilityPPLLNodeIndex = GraphBuilder.CreateTexture(Desc, TEXT("Hair.VisibilityPPLLNodeIndex"));
	}

	// Don't use CPU bounds projection to drive PPLL buffer allocation (ComputeVisibleHairStrandsMacroGroupsRect), 
	// as it often underestimate allocation needs and causes flickering
	const FIntRect HairRect = ViewInfo->ViewRect;
	const FIntPoint EffectiveResolution(HairRect.Width(), HairRect.Height());

	const uint32 PPLLMaxTotalListElementCount = GetTotalSampleCountForAllocation(EffectiveResolution, ViewInfo->GetShaderPlatform());
	{
		OutVisibilityPPLLNodeData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FPackedHairVisPPLL), PPLLMaxTotalListElementCount), TEXT("Hair.VisibilityPPLLNodeData"));
	}
	const uint32 ClearValue0[4] = { 0,0,0,0 };
	const uint32 ClearValueInvalid[4] = { 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF };
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutVisibilityPPLLNodeCounter), ClearValue0);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutVisibilityPPLLNodeIndex), ClearValueInvalid);

	FVisibilityPassParameters* PassParameters = GraphBuilder.AllocParameters<FVisibilityPassParameters>();

	{
		FVisibilityPassUniformParameters* UniformParameters = GraphBuilder.AllocParameters<FVisibilityPassUniformParameters>();

		UniformParameters->PPLLCounter = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutVisibilityPPLLNodeCounter, 0));
		UniformParameters->PPLLNodeIndex = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutVisibilityPPLLNodeIndex, 0));
		UniformParameters->PPLLNodeData = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutVisibilityPPLLNodeData));
		UniformParameters->MaxPPLLNodeCount = PPLLMaxTotalListElementCount;

		PassParameters->UniformBuffer = GraphBuilder.CreateUniformBuffer(UniformParameters);
	}

	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(InViewZDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthRead_StencilNop);
	AddHairVisibilityCommonPass(GraphBuilder, Scene, ViewInfo, MacroGroupDatas, HairVisibilityRenderMode_PPLL, PassParameters, InstanceCullingManager);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

struct FHairPrimaryTransmittance
{
	FRDGTextureRef TransmittanceTexture = nullptr;
	FRDGTextureRef HairCountTexture = nullptr;

	FRDGTextureRef HairCountTextureUint = nullptr;
	FRDGTextureRef DepthTextureUint = nullptr;
};

static FHairPrimaryTransmittance AddHairViewTransmittancePass(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo* ViewInfo,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FIntPoint& Resolution,
	const bool bOutputHairCount,
	FRDGTextureRef SceneDepthTexture,
	FInstanceCullingManager& InstanceCullingManager)
{
	check(SceneDepthTexture->Desc.Extent == Resolution);
	const EHairVisibilityRenderMode RenderMode = bOutputHairCount ? HairVisibilityRenderMode_TransmittanceAndHairCount : HairVisibilityRenderMode_Transmittance;

	// Clear to transmittance 1
	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Resolution, PF_R32_FLOAT, FClearValueBinding(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f)), TexCreate_RenderTargetable | TexCreate_ShaderResource);
	FVisibilityPassParameters* PassParameters = GraphBuilder.AllocParameters<FVisibilityPassParameters>();
	PassParameters->UniformBuffer = CreatePassDummyTextures(GraphBuilder);
	FHairPrimaryTransmittance Out;

	Out.TransmittanceTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.ViewTransmittanceTexture"));
	PassParameters->RenderTargets[0] = FRenderTargetBinding(Out.TransmittanceTexture, ERenderTargetLoadAction::EClear, 0);

	if (RenderMode == HairVisibilityRenderMode_TransmittanceAndHairCount)
	{
		Desc.Format = PF_G32R32F;
		Desc.ClearValue = FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
		Out.HairCountTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.ViewHairCountTexture"));
		PassParameters->RenderTargets[1] = FRenderTargetBinding(Out.HairCountTexture, ERenderTargetLoadAction::EClear, 0);
	}

	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
		SceneDepthTexture,
		ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad,
		FExclusiveDepthStencil::DepthRead_StencilRead);
	AddHairVisibilityCommonPass(GraphBuilder, Scene, ViewInfo, MacroGroupDatas, RenderMode, PassParameters, InstanceCullingManager);

	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Inject depth information into the view hair count texture, to block opaque occluder
class FHairViewTransmittanceDepthPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairViewTransmittanceDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FHairViewTransmittanceDepthPS, FGlobalShader);

	class FOutputFormat : SHADER_PERMUTATION_INT("PERMUTATION_OUTPUT_FORMAT", 2);
	using FPermutationDomain = TShaderPermutationDomain<FOutputFormat>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, DistanceThreshold)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CoverageTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FOutputFormat>() == 0)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32_FLOAT);
		}
		else if (PermutationVector.Get<FOutputFormat>() == 1)
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_G32R32F);
		}

	}
};

IMPLEMENT_GLOBAL_SHADER(FHairViewTransmittanceDepthPS, "/Engine/Private/HairStrands/HairStrandsVisibilityTransmittanceDepthPS.usf", "MainPS", SF_Pixel);

static void AddHairViewTransmittanceDepthPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureRef& CoverageTexture,
	const FRDGTextureRef& SceneDepthTexture,
	FRDGTextureRef& HairCountTexture)
{
	FHairViewTransmittanceDepthPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairViewTransmittanceDepthPS::FParameters>();
	Parameters->DistanceThreshold = FMath::Max(1.f, GHairStrandsViewHairCountDepthDistanceThreshold);
	Parameters->CoverageTexture = CoverageTexture;
	Parameters->SceneDepthTexture = SceneDepthTexture;
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->RenderTargets[0] = FRenderTargetBinding(HairCountTexture, ERenderTargetLoadAction::ELoad);

	FHairViewTransmittanceDepthPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairViewTransmittanceDepthPS::FOutputFormat>(HairCountTexture->Desc.Format == PF_G32R32F ? 1 : 0);

	TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FHairViewTransmittanceDepthPS> PixelShader(View.ShaderMap, PermutationVector);
	const FGlobalShaderMap* GlobalShaderMap = View.ShaderMap;
	const FIntRect Viewport = View.ViewRect;
	const FIntPoint Resolution = HairCountTexture->Desc.Extent;
	ClearUnusedGraphResources(PixelShader, Parameters);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::ViewTransmittanceDepth"),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, VertexShader, PixelShader, Viewport, Resolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_Zero>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);
		DrawRectangle(
			RHICmdList,
			0, 0,
			Viewport.Width(), Viewport.Height(),
			Viewport.Min.X, Viewport.Min.Y,
			Viewport.Width(), Viewport.Height(),
			Viewport.Size(),
			Resolution,
			VertexShader,
			EDRF_UseTriangleOptimization);
	});
}


///////////////////////////////////////////////////////////////////////////////////////////////////
class FHairVisibilityDepthPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairVisibilityDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FHairVisibilityDepthPS, FGlobalShader);

	class FOutputType : SHADER_PERMUTATION_INT("PERMUTATION_OUTPUT_TYPE", 4);
	using FPermutationDomain = TShaderPermutationDomain<FOutputType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileData)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(uint32, bClear)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, CoverageTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, HairSampleOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedHairSample>, HairSampleData)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutLightChannelMaskTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetRenderTargetOutputFormat(0, Substrate::IsSubstrateEnabled() ? PF_R32_UINT : PF_B8G8R8A8);
		OutEnvironment.SetRenderTargetOutputFormat(1, Substrate::IsSubstrateEnabled() ? PF_R32_UINT : PF_B8G8R8A8);
		OutEnvironment.SetRenderTargetOutputFormat(2, PF_FloatRGBA);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairVisibilityDepthPS, "/Engine/Private/HairStrands/HairStrandsVisibilityDepthPS.usf", "MainPS", SF_Pixel);

enum class EHairAuxilaryPassType
{
	MaterialData,
	MaterialData_LightChannelMask,
	LightChannelMask,
	DepthPatch,
	DepthClear
};

static void AddHairAuxilaryPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsTiles& TileData,
	const EHairAuxilaryPassType PassType,
	const FRDGTextureRef& CoverageTexture,
	const FRDGTextureRef& HairSampleOffset,
	const FRDGBufferRef& HairSampleData,
	FRDGTextureRef Output0,
	FRDGTextureRef Output1,
	FRDGTextureRef Output2,
	FRDGTextureRef OutColorTexture,
	FRDGTextureRef OutDepthTexture,
	FRDGTextureRef OutLightChannelMaskTexture)
{
	const FHairStrandsTiles::ETileType TileType = (PassType == EHairAuxilaryPassType::DepthClear) ? FHairStrandsTiles::ETileType::Other : FHairStrandsTiles::ETileType::HairAll;

	FHairVisibilityDepthPS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairVisibilityDepthPS::FParameters>();
	Parameters->bClear = PassType == EHairAuxilaryPassType::DepthClear ? 1u : 0u;
	Parameters->TileData = GetHairStrandsTileParameters(View, TileData, TileType);
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->CoverageTexture = CoverageTexture;
	Parameters->HairSampleOffset = HairSampleOffset;
	Parameters->HairSampleData = GraphBuilder.CreateSRV(HairSampleData);

	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();

	const bool bDepthTested = PassType != EHairAuxilaryPassType::LightChannelMask;
	if (bDepthTested)
	{
		Parameters->RenderTargets.DepthStencil = FDepthStencilBinding(
			OutDepthTexture,
			ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad,
			FExclusiveDepthStencil::DepthWrite_StencilNop);
	}

	if (PassType == EHairAuxilaryPassType::MaterialData || PassType == EHairAuxilaryPassType::MaterialData_LightChannelMask)
	{
		check(Output0 && Output1 && OutColorTexture);
		Parameters->RenderTargets[0] = FRenderTargetBinding(Output0, ERenderTargetLoadAction::ELoad, 0 /*Mip-0*/, 0/*First slice, which contains state/counter*/);
		Parameters->RenderTargets[1] = FRenderTargetBinding(Output1, ERenderTargetLoadAction::ELoad);
		Parameters->RenderTargets[2] = FRenderTargetBinding(OutColorTexture, ERenderTargetLoadAction::ELoad);
		if (!bSubstrateEnabled)
		{
			Parameters->RenderTargets[3] = FRenderTargetBinding(Output2, ERenderTargetLoadAction::ELoad);
		}
	}

	if (PassType == EHairAuxilaryPassType::MaterialData_LightChannelMask || PassType == EHairAuxilaryPassType::LightChannelMask)
	{
		check(OutLightChannelMaskTexture);
		Parameters->OutLightChannelMaskTexture = GraphBuilder.CreateUAV(OutLightChannelMaskTexture);
	}

	uint32 OutputType = 0;
	const TCHAR* Method = nullptr;
	switch (PassType)
	{
		case EHairAuxilaryPassType::DepthPatch:						OutputType = 0; Method = TEXT("HairOnlyDepth"); break;
		case EHairAuxilaryPassType::DepthClear:						OutputType = 0; Method = TEXT("HairOnlyDepth:Clear"); break;
		case EHairAuxilaryPassType::MaterialData:					OutputType = 1; Method = TEXT("MaterialData"); break;
		case EHairAuxilaryPassType::LightChannelMask:				OutputType = 2; Method = TEXT("LightChannel"); break;
		case EHairAuxilaryPassType::MaterialData_LightChannelMask:	OutputType = 3; Method = TEXT("MaterialData, LightChannel"); break;
		default: 													OutputType = 0; Method = TEXT("Unknown"); break;
	};

	TShaderMapRef<FHairStrandsTilePassVS> TileVertexShader(View.ShaderMap);

	FHairVisibilityDepthPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairVisibilityDepthPS::FOutputType>(OutputType);
	TShaderMapRef<FHairVisibilityDepthPS> PixelShader(View.ShaderMap, PermutationVector);
	const FIntRect Viewport = View.ViewRect;
	const FIntPoint Resolution = OutDepthTexture->Desc.Extent;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HairStrands::AuxilaryPass(%s)", Method),
		Parameters,
		ERDGPassFlags::Raster,
		[Parameters, TileVertexShader, PixelShader, Viewport, Resolution, bDepthTested, TileType](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FHairStrandsTilePassVS::FParameters ParametersVS = Parameters->TileData;

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = bDepthTested ? 
			TStaticDepthStencilState<true, CF_Always>::GetRHI() : 
			TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = TileVertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = Parameters->TileData.bRectPrimitive > 0 ? PT_RectList : PT_TriangleList;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);

		RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
		SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), ParametersVS);
		RHICmdList.SetStreamSource(0, nullptr, 0);
		RHICmdList.DrawPrimitiveIndirect(Parameters->TileData.TileIndirectBuffer->GetRHI(), FHairStrandsTiles::GetIndirectDrawArgOffset(TileType));
	});
}

#if RHI_RAYTRACING
static FRDGTextureRef CreateLightChannelMaskTexture(FRDGBuilder& GraphBuilder, const FIntPoint& Resolution)
{
	return GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(Resolution, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource), TEXT("Hair.LightChannelMask"));
}

static FRDGTextureRef AddHairLightChannelMaskPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsTiles& TileData,
	const FRDGTextureRef& CoverageTexture,
	const FRDGTextureRef& HairSampleOffset,
	const FRDGBufferRef& HairSampleData,
	const FRDGTextureRef& SceneDepthTexture)
{
	check(IsRayTracingEnabled());
	FRDGTextureRef OutLightChannelMask = CreateLightChannelMaskTexture(GraphBuilder, View.ViewRect.Size());

	AddHairAuxilaryPass(
		GraphBuilder,
		View,
		TileData,
		EHairAuxilaryPassType::LightChannelMask,
		CoverageTexture,
		HairSampleOffset,
		HairSampleData,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		SceneDepthTexture,
		OutLightChannelMask);
	return OutLightChannelMask;
}
#endif


static void AddHairMaterialDataPatchPass(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	const FHairStrandsTiles& TileData,
	const FRDGTextureRef& CoverageTexture,
	const FRDGTextureRef& HairSampleOffset,
	const FRDGBufferRef& HairSampleData,
	FRDGTextureRef& OutMaterial0,
	FRDGTextureRef& OutMaterial1,
	FRDGTextureRef& OutMaterial2,
	FRDGTextureRef& OutColorTexture,
	FRDGTextureRef& OutDepthTexture,
	FRDGTextureRef& OutLightChannelMask)
{
	if (!OutMaterial0 || !OutMaterial1 || !OutColorTexture || !OutDepthTexture)
	{
		return;
	}

#if RHI_RAYTRACING
	const bool bLightingChannel = IsRayTracingEnabled() && View.IsRayTracingAllowedForView() && OutLightChannelMask == nullptr;
	if (bLightingChannel)
	{
		OutLightChannelMask = CreateLightChannelMaskTexture(GraphBuilder, View.ViewRect.Size());
	}
#else
	const bool bLightingChannel = false;
#endif

	if (GetGroomViewMode(View) == EGroomViewMode::VoxelsDensity && OutDepthTexture)
	{
		FHairStrandsDebugData* DebugData = const_cast<FHairStrandsDebugData*>(&View.HairStrandsViewData.DebugData);
		DebugData->Common.SceneDepthTextureBeforeCompsition = GraphBuilder.CreateTexture(OutDepthTexture->Desc, TEXT("Hair.SceneDepthBeforeCompositionForDebug"));
		AddCopyTexturePass(GraphBuilder, OutDepthTexture, DebugData->Common.SceneDepthTextureBeforeCompsition);
	}

	AddHairAuxilaryPass(
		GraphBuilder,
		View,
		TileData,
		bLightingChannel ? EHairAuxilaryPassType::MaterialData_LightChannelMask : EHairAuxilaryPassType::MaterialData,
		CoverageTexture,
		HairSampleOffset,
		HairSampleData,
		OutMaterial0,
		OutMaterial1,
		OutMaterial2,
		OutColorTexture,
		OutDepthTexture,
		OutLightChannelMask);

	if (IsHairVisibilityUpdateSceneHZBEnabled())
	{
		BuildHZB(
			GraphBuilder,
			OutDepthTexture,
			/* VisBufferTexture = */ nullptr,
			View.ViewRect,
			View.GetFeatureLevel(),
			View.GetShaderPlatform(),
			TEXT("HZBHairClosest"),
			&View.ClosestHZB,
			TEXT("HZBHairFurthest"),
			&View.HZB
		);
	}
}

static void AddHairOnlyDepthPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsTiles& TileData,
	const FRDGTextureRef& CoverageTexture,
	const FRDGTextureRef& HairSampleOffset,
	const FRDGBufferRef& HairSampleData,
	FRDGTextureRef& OutDepthTexture)
{
	if (!OutDepthTexture)
	{
		return;
	}

	// If tile data are available, we dispatch a complementary set of tile to clear non-hair tile
	// If tile data are not available, then the clearly is done prior to that.
	if (TileData.IsValid())
	{
		AddHairAuxilaryPass(
			GraphBuilder,
			View,
			TileData,
			EHairAuxilaryPassType::DepthClear,
			CoverageTexture,
			HairSampleOffset,
			HairSampleData,
			nullptr,
			nullptr,
			nullptr,
			nullptr,
			OutDepthTexture,
			nullptr);
	}

	// Depth value
	AddHairAuxilaryPass(
		GraphBuilder,
		View,
		TileData,
		EHairAuxilaryPassType::DepthPatch,
		CoverageTexture,
		HairSampleOffset,
		HairSampleData,
		nullptr,
		nullptr,
		nullptr,
		nullptr,
		OutDepthTexture,
		nullptr);
}

static void AddHairOnlyHZBPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef HairDepthTexture,
	FVector4f& OutHZBUvFactorAndInvFactor,
	FRDGTextureRef& OutClosestHZBTexture,
	FRDGTextureRef& OutFurthestHZBTexture)
{
	FHZBParameters HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
	OutHZBUvFactorAndInvFactor = HZBParameters.HZBUvFactorAndInvFactor;

	BuildHZB(
		GraphBuilder,
		HairDepthTexture,
		/* VisBufferTexture = */ nullptr,
		FSceneRendererBase::GetHZBViewRect(View),
		View.GetFeatureLevel(),
		View.GetShaderPlatform(),
		TEXT("HZBHairClosest"),
		&OutClosestHZBTexture,
		TEXT("HZBHairFurthest"),
		&OutFurthestHZBTexture);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FHairCountToCoverageCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairCountToCoverageCS);
	SHADER_USE_PARAMETER_STRUCT(FHairCountToCoverageCS, FGlobalShader);

	class FInputType : SHADER_PERMUTATION_INT("PERMUTATION_INPUT_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FInputType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(float, LUT_HairCount)
		SHADER_PARAMETER(float, LUT_HairRadiusCount)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairCoverageLUT)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairCountTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairCountToCoverageCS, "/Engine/Private/HairStrands/HairStrandsCoverage.usf", "MainCS", SF_Compute);

static FRDGTextureRef AddHairHairCountToTransmittancePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& ViewInfo,
	const FRDGTextureRef HairCountTexture)
{
	const FIntPoint OutputResolution = HairCountTexture->Desc.Extent;

	check(HairCountTexture->Desc.Format == PF_R32_UINT || HairCountTexture->Desc.Format == PF_G32R32F)
	const bool bUseOneChannel = HairCountTexture->Desc.Format == PF_R32_UINT;

	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(OutputResolution, PF_R32_FLOAT, FClearValueBinding(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f)), TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable);
	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.VisibilityTexture"));
	FRDGTextureRef HairCoverageLUT = GetHairLUT(GraphBuilder, ViewInfo, HairLUTType_Coverage);

	FHairCountToCoverageCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairCountToCoverageCS::FParameters>();
	PassParameters->LUT_HairCount = HairCoverageLUT->Desc.Extent.X;
	PassParameters->LUT_HairRadiusCount = HairCoverageLUT->Desc.Extent.Y;
	PassParameters->OutputResolution = OutputResolution;
	PassParameters->HairCoverageLUT = HairCoverageLUT;
	PassParameters->HairCountTexture = HairCountTexture;
	PassParameters->LinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->OutputTexture = GraphBuilder.CreateUAV(OutputTexture);

	FHairCountToCoverageCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairCountToCoverageCS::FInputType>(bUseOneChannel ? 1 : 0);
	TShaderMapRef<FHairCountToCoverageCS> ComputeShader(ViewInfo.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::HairCountToTransmittancePass"), ComputeShader, PassParameters, FComputeShaderUtils::GetGroupCount(OutputResolution, FIntPoint(8,8)));

	return OutputTexture;
}

// Transit resources used during the MeshDraw passes
void AddMeshDrawTransitionPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& ViewInfo,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas)
{
	FRDGExternalAccessQueue ExternalAccessQueue;

	for (const FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
	{
		for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : MacroGroup.PrimitivesInfos)
		{
			if (PrimitiveInfo.Mesh == nullptr)
			{
				continue;
			}

			FHairGroupPublicData* HairGroupPublicData = PrimitiveInfo.PublicDataPtr;
			check(HairGroupPublicData);

			FHairGroupPublicData::FVertexFactoryInput& VFInput = HairGroupPublicData->VFInput;
			ExternalAccessQueue.Add(VFInput.Strands.PositionBuffer.Buffer);
			ExternalAccessQueue.Add(VFInput.Strands.PrevPositionBuffer.Buffer);
			ExternalAccessQueue.Add(VFInput.Strands.TangentBuffer.Buffer);
			ExternalAccessQueue.Add(VFInput.Strands.CurveAttributeBuffer.Buffer);
			ExternalAccessQueue.Add(VFInput.Strands.PointAttributeBuffer.Buffer);
			ExternalAccessQueue.Add(VFInput.Strands.PointToCurveBuffer.Buffer);
			ExternalAccessQueue.Add(VFInput.Strands.PositionOffsetBuffer.Buffer);
			ExternalAccessQueue.Add(VFInput.Strands.PrevPositionOffsetBuffer.Buffer);

			FRDGBufferRef CulledCurveBuffer = Register(GraphBuilder, HairGroupPublicData->GetCulledCurveBuffer(), ERDGImportedBufferFlags::None).Buffer;
			FRDGBufferRef CulledVertexIdBuffer = Register(GraphBuilder, HairGroupPublicData->GetCulledVertexIdBuffer(), ERDGImportedBufferFlags::None).Buffer;
			FRDGBufferRef DrawIndirectBuffer = Register(GraphBuilder, HairGroupPublicData->GetDrawIndirectBuffer(), ERDGImportedBufferFlags::None).Buffer;
			ExternalAccessQueue.Add(CulledCurveBuffer);
			ExternalAccessQueue.Add(CulledVertexIdBuffer);
			ExternalAccessQueue.Add(DrawIndirectBuffer, ERHIAccess::IndirectArgs);

			const EHairVisibilityRenderMode RasterMode = GetHairVisibilityRenderMode(ViewInfo.GetShaderPlatform());
			if (RasterMode != HairVisibilityRenderMode_ComputeRaster && RasterMode != HairVisibilityRenderMode_ComputeRasterForward)
			{
				VFInput.Strands.PositionBuffer				= FRDGImportedBuffer();
				VFInput.Strands.PrevPositionBuffer			= FRDGImportedBuffer();
				VFInput.Strands.TangentBuffer				= FRDGImportedBuffer();
				VFInput.Strands.CurveAttributeBuffer		= FRDGImportedBuffer();
				VFInput.Strands.PointAttributeBuffer		= FRDGImportedBuffer();
				VFInput.Strands.PointToCurveBuffer			= FRDGImportedBuffer();
				VFInput.Strands.PositionOffsetBuffer		= FRDGImportedBuffer();
				VFInput.Strands.PrevPositionOffsetBuffer	= FRDGImportedBuffer();
			}
		}
	}

	ExternalAccessQueue.Submit(GraphBuilder);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Raster compute common parameters

BEGIN_SHADER_PARAMETER_STRUCT(FRasterComputeCommonParameters, )
	SHADER_PARAMETER(uint32, MaxRasterCount)
	SHADER_PARAMETER(uint32, FrameIdMod8)
	SHADER_PARAMETER(uint32, ResolutionMultiplier)
	SHADER_PARAMETER(FIntPoint, OutputResolution)
	SHADER_PARAMETER(FVector2f, OutputResolutionf)
	SHADER_PARAMETER(uint32, TileSize)
	SHADER_PARAMETER(float, RcpTileSize)
	SHADER_PARAMETER(uint32, TileSizeAsShift)
	SHADER_PARAMETER(uint32, SqrTileSize)
	SHADER_PARAMETER(uint32, HalfTileSize)
	SHADER_PARAMETER(float, RcpHalfTileSize)
	SHADER_PARAMETER(uint32, SqrHalfTileSize)
	SHADER_PARAMETER(uint32, NumRasterizers)
	SHADER_PARAMETER(float, RcpNumRasterizers)
	SHADER_PARAMETER(FIntPoint, TileRes)
	SHADER_PARAMETER(uint32, NumBinners)
	SHADER_PARAMETER(float, RcpNumBinners)
END_SHADER_PARAMETER_STRUCT()

///////////////////////////////////////////////////////////////////////////////////////////////////

class FVisiblityRasterComputePrepareDepthGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterComputePrepareDepthGridCS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterComputePrepareDepthGridCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeCommonParameters, Common)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutVisTileDepthGrid)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutDepthCovTexture)
		SHADER_PARAMETER(uint32, NumSamples)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_DEPTH_GRID"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterComputePrepareDepthGridCS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterCompute.usf", "PrepareDepthGridCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

void GetHairStrandsInstanceCommon(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, const FHairGroupPublicData* HairGroupPublicData, FHairStrandsInstanceCommonParameters& OutCommon)
{
	check(HairGroupPublicData);
	const FHairGroupPublicData::FVertexFactoryInput& VFInput = HairGroupPublicData->VFInput;

	OutCommon = VFInput.Strands.Common;

	// Absolute local to world
	OutCommon.LocalToWorldPrimitiveTransform	= FMatrix44f(VFInput.LocalToWorldTransform.ToMatrixWithScale()); // LWC_TODO: Precision loss, remove usage for this

	// Translated local to world
	const FVector& TranslatedWorldOffset = ViewInfo.ViewMatrices.GetPreViewTranslation();
	FTransform LocalToTranslatedWorldTransform = VFInput.LocalToWorldTransform;
	LocalToTranslatedWorldTransform.AddToTranslation(TranslatedWorldOffset);
	OutCommon.LocalToTranslatedWorldPrimitiveTransform = FMatrix44f(LocalToTranslatedWorldTransform.ToMatrixWithScale());
}

void GetHairStrandsInstanceResources(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, const FHairGroupPublicData* HairGroupPublicData, bool bForceRegister, FHairStrandsInstanceResourceParameters& OutResources)
{
	check(HairGroupPublicData);
	const FHairGroupPublicData::FVertexFactoryInput& VFInput = HairGroupPublicData->VFInput;

	if (bForceRegister)
	{
		OutResources.PositionBuffer			= Register(GraphBuilder, VFInput.Strands.PositionBufferExternal, ERDGImportedBufferFlags::CreateSRV).SRV;
		OutResources.PositionOffsetBuffer	= Register(GraphBuilder, VFInput.Strands.PositionOffsetBufferExternal, ERDGImportedBufferFlags::CreateSRV).SRV;
		OutResources.CurveBuffer			= Register(GraphBuilder, VFInput.Strands.CurveBufferExternal, ERDGImportedBufferFlags::CreateSRV).SRV;
		OutResources.PointToCurveBuffer		= Register(GraphBuilder, VFInput.Strands.PointToCurveBufferExternal, ERDGImportedBufferFlags::CreateSRV).SRV;
		OutResources.CurveAttributeBuffer	= Register(GraphBuilder, VFInput.Strands.CurveAttributeBufferExternal, ERDGImportedBufferFlags::CreateSRV).SRV;
		OutResources.PointAttributeBuffer	= Register(GraphBuilder, VFInput.Strands.PointAttributeBufferExternal, ERDGImportedBufferFlags::CreateSRV).SRV;
		OutResources.CurveMappingBuffer		= Register(GraphBuilder, VFInput.Strands.CurveMappingBufferExternal, ERDGImportedBufferFlags::CreateSRV).SRV;
		OutResources.PointMappingBuffer		= Register(GraphBuilder, VFInput.Strands.PointMappingBufferExternal, ERDGImportedBufferFlags::CreateSRV).SRV;
	}
	else
	{
		OutResources.PositionBuffer			= VFInput.Strands.PositionBuffer.SRV;
		OutResources.PositionOffsetBuffer	= VFInput.Strands.PositionOffsetBuffer.SRV;
		OutResources.CurveBuffer			= VFInput.Strands.CurveBuffer.SRV;
		OutResources.PointToCurveBuffer		= VFInput.Strands.PointToCurveBuffer.SRV;
		OutResources.CurveAttributeBuffer	= VFInput.Strands.CurveAttributeBuffer.SRV;
		OutResources.PointAttributeBuffer	= VFInput.Strands.PointAttributeBuffer.SRV;
		OutResources.CurveMappingBuffer 	= VFInput.Strands.CurveMappingBuffer.SRV;
		OutResources.PointMappingBuffer 	= VFInput.Strands.PointMappingBuffer.SRV;
	}
}

void GetHairStrandsInstanceCulling(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, const FHairGroupPublicData* HairGroupPublicData, bool bCullingEnable, FHairStrandsInstanceCullingParameters& OutCulling)
{
	check(HairGroupPublicData);
	const FHairGroupPublicData::FVertexFactoryInput& VFInput = HairGroupPublicData->VFInput;

	OutCulling.bCullingEnable = 0;
	if (bCullingEnable)
	{
		FRDGImportedBuffer CullingIndirectBuffer = Register(GraphBuilder, HairGroupPublicData->GetDrawIndirectRasterComputeBuffer(), ERDGImportedBufferFlags::CreateSRV);
		OutCulling.bCullingEnable = 1;
		OutCulling.CullingIndirectBuffer = CullingIndirectBuffer.SRV;
		OutCulling.CullingIndexBuffer = RegisterAsSRV(GraphBuilder, HairGroupPublicData->GetCulledVertexIdBuffer());
		OutCulling.CullingIndirectBufferArgs = CullingIndirectBuffer.Buffer;
	}
}

FHairStrandsInstanceParameters GetHairStrandsInstanceParameters(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo, const FHairGroupPublicData* HairGroupPublicData, bool bCullingEnable, bool bForceRegister)
{
	FHairStrandsInstanceParameters Output;
	GetHairStrandsInstanceCommon(GraphBuilder, ViewInfo, HairGroupPublicData, Output.HairStrandsVF.Common);
	GetHairStrandsInstanceResources(GraphBuilder, ViewInfo, HairGroupPublicData, bForceRegister, Output.HairStrandsVF.Resources);
	GetHairStrandsInstanceCulling(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnable, Output.HairStrandsVF.Culling);
	return Output;
}

class FVisiblityRasterClassificationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterClassificationCS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterClassificationCS, FGlobalShader);

	class FCulling : SHADER_PERMUTATION_BOOL("PERMUTATION_CULLING");
	using FPermutationDomain = TShaderPermutationDomain<FCulling>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeCommonParameters, Common)
		SHADER_PARAMETER(uint32, ControlPointCount)
		SHADER_PARAMETER(uint32, PrimIDsBufferSize)
		SHADER_PARAMETER(uint32, NumWorkGroups)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceParameters, HairInstance)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutPrimIDs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutDrawIndexedArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutSWRasterPrimitiveCount)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters &Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters &Parameters, FShaderCompilerEnvironment &OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterClassificationCS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterClassification.usf", "CSMain", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

class FVisiblityRasterComputeBinningCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterComputeBinningCS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterComputeBinningCS, FGlobalShader);

	class FCulling : SHADER_PERMUTATION_BOOL("PERMUTATION_CULLING");
	class FIndirectPrimIDs : SHADER_PERMUTATION_BOOL("PERMUTATION_INDIRECT_PRIM_IDS");
	using FPermutationDomain = TShaderPermutationDomain<FCulling, FIndirectPrimIDs>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeCommonParameters, Common)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(uint32, HairMaterialId)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceParameters, HairInstance)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, VisTileBinningGrid)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutVisTilePrims)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, OutVisTileBinningGrid)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VisTileDepthGrid)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutVisTileArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutVisTileData)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RDG_TEXTURE_ACCESS(VisTileBinningGridTex, ERHIAccess::UAVCompute)
		SHADER_PARAMETER(uint32, VertexCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, IndirectPrimIDCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, IndirectPrimIDs)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_BINNING"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterComputeBinningCS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterCompute.usf", "BinningCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

class FVisiblityRasterComputeCompactionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterComputeCompactionCS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterComputeCompactionCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeCommonParameters, Common)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceParameters, HairInstance)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, VisTileData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTilePrims)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTileArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, OutCompactedVisTileData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutCompactedVisTilePrims)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutCompactedVisTileArgs)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters &Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters &Parameters, FShaderCompilerEnvironment &OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_COMPACTION"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterComputeCompactionCS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterCompute.usf", "CompactionCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

class FVisiblityRasterComputeRasterizeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterComputeRasterizeCS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterComputeRasterizeCS, FGlobalShader);

	class FGroupSize : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_GROUP_SIZE", 32, 64);
	using FPermutationDomain = TShaderPermutationDomain<FGroupSize>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeCommonParameters, Common)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(uint32, HairMaterialId)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceParameters, HairInstance)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER(float, CoverageScale)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTilePrims)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTileArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, VisTileData)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutHairCountTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutDepthCovTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutPrimMatTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_RASTER"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterComputeRasterizeCS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterCompute.usf", "RasterCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

class FVisiblityRasterComputeRasterizeMultiSampleCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterComputeRasterizeMultiSampleCS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterComputeRasterizeMultiSampleCS, FGlobalShader);

	class FSampleCount : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_MULTI_SAMPLE_COUNT", 1, 2, 4, 8);
	class FGroupSize : SHADER_PERMUTATION_SPARSE_INT("PERMUTATION_GROUP_SIZE", 32, 64);
	using FPermutationDomain = TShaderPermutationDomain<FGroupSize, FSampleCount>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeCommonParameters, Common)
		SHADER_PARAMETER(uint32, MacroGroupId)
			SHADER_PARAMETER(uint32, HairMaterialId)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceParameters, HairInstance)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER(float, CoverageScale)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTilePrims)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTileArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, VisTileData)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, OutHairCountTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, OutDepthCovTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, OutPrimMatTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_RASTER_MULTI_SAMPLE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterComputeRasterizeMultiSampleCS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterCompute.usf", "RasterMultiSampleCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

class FVisiblityRasterComputeDebugCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterComputeDebugCS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterComputeDebugCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRasterComputeCommonParameters, Common)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(uint32, PrimitiveInfoIndex)
		SHADER_PARAMETER(uint32, TotalPrimitiveInfoCount)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VisTileDepthGrid)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, VisTileBinningGrid)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, VisTileArgs)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_DEBUG"), 1);
	}
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterComputeDebugCS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterCompute.usf", "MainCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

class FVisiblityRasterHWVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterHWVS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterHWVS, FGlobalShader);

	class FDrawIndirect : SHADER_PERMUTATION_BOOL("PERMUTATION_DRAW_INDIRECT");
	class FCulling : SHADER_PERMUTATION_BOOL("PERMUTATION_CULLING");
	using FPermutationDomain = TShaderPermutationDomain<FDrawIndirect, FCulling>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, RasterPrimIDs)
		SHADER_PARAMETER(FVector3f, ViewDir)
		SHADER_PARAMETER(uint32, HairMaterialId)
		SHADER_PARAMETER(FVector3f, CameraOrigin)
		SHADER_PARAMETER(float, RadiusAtDepth1)
		SHADER_PARAMETER(uint32, PrimIDsBufferSize)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceParameters, HairInstance)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RDG_BUFFER_ACCESS(IndirectBufferArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTER_HW_VS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterHWVS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterHW.usf", "VSMain", SF_Vertex);

/////////////////////////////////////////////////////////////////////////////////////////

class FVisiblityRasterHWPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterHWPS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterHWPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutHairCountTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutDepthCovTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutPrimMatTexture)
		SHADER_PARAMETER(float, CoverageScale)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTER_HW_PS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterHWPS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterHW.usf", "PSMain", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FVisiblityRasterHWParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FVisiblityRasterHWVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FVisiblityRasterHWPS::FParameters, PS)
END_SHADER_PARAMETER_STRUCT()

/////////////////////////////////////////////////////////////////////////////////////////

class FVisiblityRasterComputeNaiveCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisiblityRasterComputeNaiveCS);
	SHADER_USE_PARAMETER_STRUCT(FVisiblityRasterComputeNaiveCS, FGlobalShader);

	class FIndirectPrimIDs : SHADER_PERMUTATION_BOOL("PERMUTATION_INDIRECT_PRIM_IDS");
	class FCulling : SHADER_PERMUTATION_BOOL("PERMUTATION_CULLING");
	using FPermutationDomain = TShaderPermutationDomain<FIndirectPrimIDs, FCulling>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, OutputResolution)
		SHADER_PARAMETER(uint32, HairMaterialId)
		SHADER_PARAMETER(uint32, ControlPointCount)
		SHADER_PARAMETER(float, CoverageScale)
		SHADER_PARAMETER(uint32, NumWorkGroups)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceParameters, HairInstance)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutHairCountTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutDepthCovTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutPrimMatTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, IndirectPrimIDCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, IndirectPrimIDs)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE_RASTER"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisiblityRasterComputeNaiveCS, "/Engine/Private/HairStrands/HairStrandsVisibilityRasterComputeNaive.usf", "CSMain", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

bool IsHairStrandContinuousDecimationReorderingEnabled();

static FRasterComputeOutput AddVisibilityComputeRasterPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& ViewInfo,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const FIntPoint& InResolution,
	const uint32 SamplePerPixelCount,
	const FRDGTextureRef SceneDepthTexture,
	bool bSupportCulling,
	bool bForceRegister)
{	
	FRasterComputeOutput Out;
	Out.Resolution = InResolution;

	FRDGTextureDesc DescCount = FRDGTextureDesc::Create2D(Out.Resolution, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable);
	FRDGTextureDesc DescVis = FRDGTextureDesc::Create2DArray(Out.Resolution, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable, SamplePerPixelCount);

	Out.HairCountTexture = GraphBuilder.CreateTexture(DescCount, TEXT("Hair.HairCountTexture"));
	Out.DepthCovTexture = GraphBuilder.CreateTexture(DescVis, TEXT("Hair.DepthCovTexture"));
	Out.PrimMatTexture = GraphBuilder.CreateTexture(DescVis, TEXT("Hair.PrimMatTexture"));

	FRDGTextureUAVRef HairCountTextureUAV = GraphBuilder.CreateUAV(Out.HairCountTexture);
	FRDGTextureUAVRef DepthCovTextureUAV = GraphBuilder.CreateUAV(Out.DepthCovTexture);
	FRDGTextureUAVRef PrimMatTextureUAV = GraphBuilder.CreateUAV(Out.PrimMatTexture);

	uint32 ClearValues[4] = { 0,0,0,0 };
	AddClearUAVPass(GraphBuilder, DepthCovTextureUAV, ClearValues);
	AddClearUAVPass(GraphBuilder, HairCountTextureUAV, ClearValues);

	// Set up buffers for binning and raster.

	// See the comment on the GHairVisibilityComputeRaster_MaxTiles declaration for an explanation for the large upper bound.
	const uint32 MaxTiles = FMath::Min(FMath::Max(GHairVisibilityComputeRaster_MaxTiles, 1024), 262144);
	const uint32 TileSize = FMath::Min(FMath::Max(GHairVisibilityComputeRaster_TileSize, 8),32);
	const uint32 NumClassifiers = FMath::Min(FMath::Max(GHairVisibility_NumClassifiers, 1), 256);
	const uint32 NumBinners = FMath::Min(FMath::Max(GHairVisibilityComputeRaster_NumBinners, 1), 256);
	const uint32 NumRasterizers = FMath::Min(FMath::Max(GHairVisibilityComputeRaster_NumRasterizers, 1), 1024);
	const uint32 NumRasterizersNaive = FMath::Min(FMath::Max(GHairVisibilityComputeRaster_NumRasterizersNaive, 1), 1024);

	const FIntPoint TileGridRes = FIntPoint((InResolution.X + (TileSize-1)) / TileSize, ((InResolution.Y + (TileSize - 1)) / TileSize)*1);

	// Compute maximum number of PrimIDs
	uint32 MaxNumPrimIDs = 0;
	for (const FHairStrandsMacroGroupData &MacroGroup : MacroGroupDatas)
	{
		for (const FHairStrandsMacroGroupData::PrimitiveInfo &PrimitiveInfo : MacroGroup.PrimitivesInfos)
		{
			// If a groom is not visible in primary view, but visible in shadow view, its PrimitiveInfo.Mesh will be null.
			if (PrimitiveInfo.Mesh == nullptr || PrimitiveInfo.Mesh->Elements.Num() == 0)
			{
				continue;
			}

			const FHairGroupPublicData* HairGroupPublicData = reinterpret_cast<const FHairGroupPublicData*>(PrimitiveInfo.Mesh->Elements[0].VertexFactoryUserData);
			check(HairGroupPublicData);

			const uint32 PointCount = HairGroupPublicData->GetActiveStrandsPointCount();
			// Sanity check
			check(HairGroupPublicData->VFInput.Strands.Common.PointCount == PointCount);

			MaxNumPrimIDs = FMath::Max(MaxNumPrimIDs, PointCount);
		}
	}

	FRDGBufferRef VisTilePrims = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MaxTiles * 1024), TEXT("Hair.VisTilePrims"));
	FRDGBufferSRVRef VisTilePrimsSRV = GraphBuilder.CreateSRV(VisTilePrims, PF_R32_UINT);
	
	FRDGBufferRef CompactedVisTilePrims = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MaxTiles * 1024), TEXT("Hair.CompactedVisTilePrims"));

	FRDGTextureDesc DescBinningGrid = FRDGTextureDesc::Create2DArray(TileGridRes, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource, NumBinners * 3);
	FRDGTextureRef VisTileBinningGrid = GraphBuilder.CreateTexture(DescBinningGrid, TEXT("Hair.VisTileBinningGrid"));
	FRDGTextureUAVRef VisTileBinningGridUAV = GraphBuilder.CreateUAV(VisTileBinningGrid);

	FRDGTextureDesc DescDepthGrid = FRDGTextureDesc::Create2D(TileGridRes, PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource);
	FRDGTextureRef VisTileDepthGrid = GraphBuilder.CreateTexture(DescDepthGrid, TEXT("Hair.VisTileDepthGrid"));

	FRDGBufferDesc VisTileArgsDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 4 + 16 + (((TileGridRes.X * TileGridRes.Y) + 31) / 32));
	FRDGBufferRef VisTileArgs = GraphBuilder.CreateBuffer(VisTileArgsDesc, TEXT("Hair.VisTileArgs"));
	FRDGBufferUAVRef VisTileArgsUAV = GraphBuilder.CreateUAV(VisTileArgs, PF_R32_UINT);
	FRDGBufferSRVRef VisTileArgsSRV = GraphBuilder.CreateSRV(VisTileArgs, PF_R32_UINT);
	
	FRDGBufferRef CompactedVisTileArgs = GraphBuilder.CreateBuffer(VisTileArgsDesc, TEXT("Hair.CompactedVisTileArgs"));

	FRDGBufferDesc DescTileData = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxTiles * 4 * 2);
	DescTileData.Usage = EBufferUsageFlags(DescTileData.Usage | BUF_ByteAddressBuffer);
	FRDGBufferRef VisTileData = GraphBuilder.CreateBuffer(DescTileData, TEXT("Hair.VisTileData"));
	FRDGBufferSRVRef VisTileDataSRV = GraphBuilder.CreateSRV(VisTileData);
	
	FRDGBufferRef CompactedVisTileData = GraphBuilder.CreateBuffer(DescTileData, TEXT("Hair.CompactedVisTileData"));

	FRDGBufferRef RasterizerPrimIDs = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FMath::Max(MaxNumPrimIDs, 1u)), TEXT("Hair.RasterizerPrimIDs"));
	FRDGBufferSRVRef RasterizerPrimIDsSRV = GraphBuilder.CreateSRV(RasterizerPrimIDs, PF_R32_UINT);

	FRDGBufferRef DrawIndexedIndirectArgs = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndexedIndirectParameters>(1), TEXT("Hair.DrawIndexedIndirectArgs"));
	FRDGBufferUAVRef DrawIndexedIndirectArgsUAV = GraphBuilder.CreateUAV(DrawIndexedIndirectArgs, PF_R32_UINT);

	FRDGBufferRef SWRasterPrimIDCount = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1), TEXT("Hair.SWRasterPrimIDCount"));
	FRDGBufferUAVRef SWRasterPrimIDCountUAV = GraphBuilder.CreateUAV(SWRasterPrimIDCount, PF_R32_UINT);
	FRDGBufferSRVRef SWRasterPrimIDCountSRV = GraphBuilder.CreateSRV(SWRasterPrimIDCount, PF_R32_UINT);

	// Create and set the uniform buffer
	const bool bEnableMSAA = false;
	TUniformBufferRef<FViewUniformShaderParameters> ViewUniformShaderParameters;
	SetUpViewHairRenderInfo(ViewInfo, bEnableMSAA, ViewInfo.CachedViewUniformShaderParameters->HairRenderInfo, ViewInfo.CachedViewUniformShaderParameters->HairRenderInfoBits, ViewInfo.CachedViewUniformShaderParameters->HairComponents, ViewInfo.Family->FinalResolutionScale);
	ViewUniformShaderParameters = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(*ViewInfo.CachedViewUniformShaderParameters, UniformBuffer_SingleFrame);

	const bool bClassification = GHairStrandsHWSWClassifaction != 0 && SamplePerPixelCount <= 1;
	const bool bUseHWRaster = GHairStrandsUseHWRaster != 0 && SamplePerPixelCount <= 1;
	const bool bUseNaiveSWRaster = GHairStrandsUseNaiveSWRaster != 0 && SamplePerPixelCount <= 1;
	const bool bTileCompactionPass = GHairStrandsTileCompaction != 0;
	const bool bMultiSampleSWRaster = SamplePerPixelCount > 1;

	const uint32 GroupSize = GetVendorOptimalGroupSize1D();

	FVisiblityRasterClassificationCS::FPermutationDomain ClassificationPermutationVector_CullingOff;
	ClassificationPermutationVector_CullingOff.Set<FVisiblityRasterClassificationCS::FCulling>(false);
	TShaderMapRef<FVisiblityRasterClassificationCS> ComputeShaderClassification_CullingOff(ViewInfo.ShaderMap, ClassificationPermutationVector_CullingOff);
	
	FVisiblityRasterClassificationCS::FPermutationDomain ClassificationPermutationVector_CullingOn;
	ClassificationPermutationVector_CullingOn.Set<FVisiblityRasterClassificationCS::FCulling>(true);
	TShaderMapRef<FVisiblityRasterClassificationCS> ComputeShaderClassification_CullingOn(ViewInfo.ShaderMap, ClassificationPermutationVector_CullingOn);
	
	FVisiblityRasterComputeBinningCS::FPermutationDomain BinningPermutationVector_CullingOff;
	BinningPermutationVector_CullingOff.Set<FVisiblityRasterComputeBinningCS::FCulling>(false);
	BinningPermutationVector_CullingOff.Set<FVisiblityRasterComputeBinningCS::FIndirectPrimIDs>(bClassification);
	TShaderMapRef<FVisiblityRasterComputeBinningCS> ComputeShaderBinning_CullingOff(ViewInfo.ShaderMap, BinningPermutationVector_CullingOff);

	FVisiblityRasterComputeBinningCS::FPermutationDomain BinningPermutationVector_CullingOn;
	BinningPermutationVector_CullingOn.Set<FVisiblityRasterComputeBinningCS::FCulling>(true);
	BinningPermutationVector_CullingOn.Set<FVisiblityRasterComputeBinningCS::FIndirectPrimIDs>(bClassification);
	TShaderMapRef<FVisiblityRasterComputeBinningCS> ComputeShaderBinning_CullingOn(ViewInfo.ShaderMap, BinningPermutationVector_CullingOn);

	FVisiblityRasterComputeCompactionCS::FPermutationDomain CompactionPermutationVector;
	TShaderMapRef<FVisiblityRasterComputeCompactionCS> ComputeShaderCompaction(ViewInfo.ShaderMap, CompactionPermutationVector);

	FVisiblityRasterComputeRasterizeCS::FPermutationDomain RasterPermutationVector;
	RasterPermutationVector.Set<FVisiblityRasterComputeRasterizeCS::FGroupSize>(GroupSize);
	TShaderMapRef<FVisiblityRasterComputeRasterizeCS> ComputeShaderRaster(ViewInfo.ShaderMap, RasterPermutationVector);

	FVisiblityRasterComputeRasterizeMultiSampleCS::FPermutationDomain RasterMultiSamplePermutationVector;
	RasterMultiSamplePermutationVector.Set<FVisiblityRasterComputeRasterizeMultiSampleCS::FGroupSize>(GroupSize);
	RasterMultiSamplePermutationVector.Set<FVisiblityRasterComputeRasterizeMultiSampleCS::FSampleCount>(SamplePerPixelCount);
	TShaderMapRef<FVisiblityRasterComputeRasterizeMultiSampleCS> ComputeShaderRasterMultiSample(ViewInfo.ShaderMap, RasterMultiSamplePermutationVector);

	FVisiblityRasterHWVS::FPermutationDomain VSRasterPermutationVector_CullingOff;
	VSRasterPermutationVector_CullingOff.Set<FVisiblityRasterHWVS::FDrawIndirect>(bClassification);
	VSRasterPermutationVector_CullingOff.Set<FVisiblityRasterHWVS::FCulling>(false);
	TShaderMapRef<FVisiblityRasterHWVS> VertexShaderRaster_CullingOff(ViewInfo.ShaderMap, VSRasterPermutationVector_CullingOff);

	FVisiblityRasterHWVS::FPermutationDomain VSRasterPermutationVector_CullingOn;
	VSRasterPermutationVector_CullingOn.Set<FVisiblityRasterHWVS::FDrawIndirect>(bClassification);
	VSRasterPermutationVector_CullingOn.Set<FVisiblityRasterHWVS::FCulling>(true);
	TShaderMapRef<FVisiblityRasterHWVS> VertexShaderRaster_CullingOn(ViewInfo.ShaderMap, VSRasterPermutationVector_CullingOn);
	
	TShaderMapRef<FVisiblityRasterHWPS> PixelShaderRaster(ViewInfo.ShaderMap);

	FVisiblityRasterComputeNaiveCS::FPermutationDomain RasterNaivePermutationVector_CullingOff;
	RasterNaivePermutationVector_CullingOff.Set<FVisiblityRasterComputeNaiveCS::FIndirectPrimIDs>(bClassification);
	RasterNaivePermutationVector_CullingOff.Set<FVisiblityRasterComputeNaiveCS::FCulling>(false);
	TShaderMapRef<FVisiblityRasterComputeNaiveCS> ComputeShaderRasterNaive_CullingOff(ViewInfo.ShaderMap, RasterNaivePermutationVector_CullingOff);

	FVisiblityRasterComputeNaiveCS::FPermutationDomain RasterNaivePermutationVector_CullingOn;
	RasterNaivePermutationVector_CullingOn.Set<FVisiblityRasterComputeNaiveCS::FIndirectPrimIDs>(bClassification);
	RasterNaivePermutationVector_CullingOn.Set<FVisiblityRasterComputeNaiveCS::FCulling>(true);
	TShaderMapRef<FVisiblityRasterComputeNaiveCS> ComputeShaderRasterNaive_CullingOn(ViewInfo.ShaderMap, RasterNaivePermutationVector_CullingOn);

	// Common parameters
	FRasterComputeCommonParameters Common;
	{		
		Common.MaxRasterCount = TileSize;
		Common.FrameIdMod8 =
		Common.ResolutionMultiplier = 1;
		Common.OutputResolution = Out.Resolution;
		Common.OutputResolutionf = FVector2f(Out.Resolution.X, Out.Resolution.Y);
		Common.TileSize = TileSize;
		Common.TileSizeAsShift = uint32(FMath::Log2(float(TileSize)));
		Common.RcpTileSize = 1.0 / TileSize;
		Common.SqrTileSize = TileSize * TileSize;
		Common.HalfTileSize = TileSize / 2;
		Common.RcpHalfTileSize = 1.0f / Common.HalfTileSize;
		Common.SqrHalfTileSize = Common.HalfTileSize * Common.HalfTileSize;
		Common.NumRasterizers = NumRasterizers;
		Common.RcpNumRasterizers = 1.0 / NumRasterizers;
		Common.TileRes = TileGridRes;
		Common.NumBinners = NumBinners;
		Common.RcpNumBinners = 1.0 / NumBinners;
	}

	// Fill in tile depth
	if (!bUseNaiveSWRaster && (bClassification || !bUseHWRaster))
	{
		TShaderMapRef<FVisiblityRasterComputePrepareDepthGridCS> ComputeShaderPrepareDepthGrid(ViewInfo.ShaderMap);
		FVisiblityRasterComputePrepareDepthGridCS::FParameters* PrepDepthGridParameters = GraphBuilder.AllocParameters<FVisiblityRasterComputePrepareDepthGridCS::FParameters>();
		
		PrepDepthGridParameters->Common = Common;
		PrepDepthGridParameters->SceneDepthTexture = SceneDepthTexture;
		PrepDepthGridParameters->OutVisTileDepthGrid = GraphBuilder.CreateUAV(VisTileDepthGrid);
		PrepDepthGridParameters->OutDepthCovTexture = DepthCovTextureUAV;
		PrepDepthGridParameters->NumSamples = SamplePerPixelCount;
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterPrepDepthGrid"), ComputeShaderPrepareDepthGrid, PrepDepthGridParameters, FIntVector(TileGridRes.X, TileGridRes.Y, 1));
	}

	const bool bDebugEnabled = GHairVisibilityComputeRaster_Debug > 0;
	uint32 TotalPrimitiveInfoCount = 0;
	if (bDebugEnabled && !bUseHWRaster && !bUseNaiveSWRaster)
	{
		ShaderPrint::SetEnabled(true);
		ShaderPrint::RequestSpaceForTriangles(2 * NumBinners * Common.TileRes.X * Common.TileRes.Y + 2 * NumBinners * 10);
		ShaderPrint::RequestSpaceForLines(4 * 2 * NumBinners * 10);
		for (const FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
		{
			const FHairStrandsMacroGroupData::TPrimitiveInfos& PrimitiveSceneInfos = MacroGroup.PrimitivesInfos;
			TotalPrimitiveInfoCount += PrimitiveSceneInfos.Num();
		}
	}

	bool bAnyHairRasterized = false;
	uint32 PrimitiveInfoIndex = 0;
	for (const FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
	{
		const FHairStrandsMacroGroupData::TPrimitiveInfos& PrimitiveSceneInfos = MacroGroup.PrimitivesInfos;

		for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : PrimitiveSceneInfos)
		{
			// If a groom is not visible in primary view, but visible in shadow view, its PrimitiveInfo.Mesh will be null.
			if (PrimitiveInfo.Mesh == nullptr || PrimitiveInfo.Mesh->Elements.Num() == 0)
			{
				continue;
			}

			const FHairGroupPublicData* HairGroupPublicData = reinterpret_cast<const FHairGroupPublicData*>(PrimitiveInfo.Mesh->Elements[0].VertexFactoryUserData);
			check(HairGroupPublicData);

			const bool bCullingEnable = bSupportCulling && GHairVisibilityComputeRaster_Culling ? HairGroupPublicData->GetCullingResultAvailable() : false;
			const bool bCullingEnableInRasterizers = bCullingEnable && !bClassification;

			const uint32 PointCount = HairGroupPublicData->GetActiveStrandsPointCount();
			check(PointCount == HairGroupPublicData->VFInput.Strands.Common.PointCount);
			const float CoverageScale = HairGroupPublicData->GetActiveStrandsCoverageScale();

			// HW/SW classification
			if (bClassification)
			{
				// Reset buffers
				{
					AddClearUAVPass(GraphBuilder, DrawIndexedIndirectArgsUAV, 0u);
					AddClearUAVPass(GraphBuilder, SWRasterPrimIDCountUAV, 0u);
				}

				const uint32 NumWorkGroups = NumClassifiers;

				FVisiblityRasterClassificationCS::FParameters* ClassificationParameters = GraphBuilder.AllocParameters<FVisiblityRasterClassificationCS::FParameters>();
				ClassificationParameters->Common = Common;
				ClassificationParameters->ControlPointCount = PointCount;
				ClassificationParameters->PrimIDsBufferSize = MaxNumPrimIDs;
				ClassificationParameters->NumWorkGroups = NumWorkGroups;
				ClassificationParameters->HairInstance = GetHairStrandsInstanceParameters(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnable, bForceRegister);
				ClassificationParameters->ViewUniformBuffer = ViewUniformShaderParameters;
				ClassificationParameters->OutPrimIDs = GraphBuilder.CreateUAV(RasterizerPrimIDs, PF_R32_UINT);
				ClassificationParameters->OutDrawIndexedArgs = DrawIndexedIndirectArgsUAV;
				ClassificationParameters->OutSWRasterPrimitiveCount = SWRasterPrimIDCountUAV;

				FComputeShaderUtils::AddPass(
					GraphBuilder, 
					RDG_EVENT_NAME("HairStrands::VisibilityRasterClassification"), 
					bCullingEnable ? ComputeShaderClassification_CullingOn : ComputeShaderClassification_CullingOff, 
					ClassificationParameters, 
					FIntVector(NumWorkGroups, 1, 1));
			}

			if (bClassification || bUseHWRaster)
			{
				const FVector ViewDirection = ViewInfo.GetViewDirection();
				const FVector CameraOrigin = ViewInfo.ViewMatrices.GetViewOrigin();
				const FIntRect Viewport = ViewInfo.ViewRect;
				const FIntPoint Resolution = SceneDepthTexture->Desc.Extent;

				FVisiblityRasterHWParameters* Parameters = GraphBuilder.AllocParameters<FVisiblityRasterHWParameters>();
				Parameters->VS.RasterPrimIDs = bClassification ? RasterizerPrimIDsSRV : nullptr;
				Parameters->VS.ViewDir = FVector3f(ViewDirection.X, ViewDirection.Y, ViewDirection.Z);
				Parameters->VS.HairMaterialId = PrimitiveInfo.MaterialId;
				Parameters->VS.CameraOrigin = FVector3f(CameraOrigin.X, CameraOrigin.Y, CameraOrigin.Z);
				Parameters->VS.RadiusAtDepth1 = ComputeMinStrandRadiusAtDepth1(Resolution, ViewInfo.FOV, 1, -1.0f, ViewInfo.ViewMatrices.GetOrthoDimensions().X, ViewInfo.Family->FinalResolutionScale).Primary;
				Parameters->VS.PrimIDsBufferSize = MaxNumPrimIDs;
				Parameters->VS.HairInstance = GetHairStrandsInstanceParameters(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnable, bForceRegister);
				Parameters->VS.ViewUniformBuffer = ViewUniformShaderParameters;
				Parameters->VS.IndirectBufferArgs = bClassification ? DrawIndexedIndirectArgs : nullptr;
				Parameters->PS.OutHairCountTexture = HairCountTextureUAV;
				Parameters->PS.OutDepthCovTexture = DepthCovTextureUAV;
				Parameters->PS.OutPrimMatTexture = PrimMatTextureUAV;
				Parameters->PS.CoverageScale = CoverageScale;
				Parameters->PS.RenderTargets.DepthStencil = FDepthStencilBinding(
					SceneDepthTexture,
					ERenderTargetLoadAction::ELoad,
					ERenderTargetLoadAction::ENoAction,
					FExclusiveDepthStencil::DepthRead_StencilNop);

				TShaderMapRef<FVisiblityRasterHWVS> VertexShaderRaster = bCullingEnableInRasterizers ? VertexShaderRaster_CullingOn : VertexShaderRaster_CullingOff;

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("HairStrands::VisibilityRasterHW"),
					Parameters,
					ERDGPassFlags::Raster,
					[Parameters, VertexShaderRaster, bClassification, PixelShaderRaster, Viewport, Resolution, PointCount, DrawIndexedIndirectArgs](FRDGAsyncTask, FRHICommandList &RHICmdList)
					{
						FGraphicsPipelineStateInitializer GraphicsPSOInit;
						RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
						GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
						GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
						GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNear>::GetRHI();
						GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
						GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShaderRaster.GetVertexShader();
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShaderRaster.GetPixelShader();
						GraphicsPSOInit.PrimitiveType = PT_TriangleList;
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

						SetShaderParameters(RHICmdList, VertexShaderRaster, VertexShaderRaster.GetVertexShader(), Parameters->VS);
						SetShaderParameters(RHICmdList, PixelShaderRaster, PixelShaderRaster.GetPixelShader(), Parameters->PS);

						RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
						RHICmdList.SetStreamSource(0, nullptr, 0);
						if (bClassification)
						{
							RHICmdList.DrawIndexedPrimitiveIndirect(GTwoTrianglesIndexBuffer.IndexBufferRHI, DrawIndexedIndirectArgs->GetIndirectRHICallBuffer(), 0);
						}
						else
						{
							const uint32 NumInstances = PointCount;
							RHICmdList.DrawIndexedPrimitive(GTwoTrianglesIndexBuffer.IndexBufferRHI, 0, 0, 6, 0, 2, NumInstances);
						}
					});
			}

			if (bMultiSampleSWRaster || bClassification || !bUseHWRaster)
			{
				if (bUseNaiveSWRaster)
				{
					const uint32 NumWorkGroups = NumRasterizersNaive;

					FVisiblityRasterComputeNaiveCS::FParameters* RasterParameters = GraphBuilder.AllocParameters<FVisiblityRasterComputeNaiveCS::FParameters>();
					RasterParameters->OutputResolution = Common.OutputResolutionf;
					RasterParameters->HairMaterialId = PrimitiveInfo.MaterialId;
					RasterParameters->ControlPointCount = PointCount;
					RasterParameters->CoverageScale = CoverageScale;
					RasterParameters->NumWorkGroups = NumWorkGroups;
					RasterParameters->HairInstance = GetHairStrandsInstanceParameters(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnableInRasterizers, bForceRegister);
					RasterParameters->ViewUniformBuffer = ViewUniformShaderParameters;
					RasterParameters->OutHairCountTexture = HairCountTextureUAV;
					RasterParameters->OutDepthCovTexture = DepthCovTextureUAV;
					RasterParameters->OutPrimMatTexture = PrimMatTextureUAV;
					RasterParameters->SceneDepthTexture = SceneDepthTexture;
					RasterParameters->IndirectPrimIDCount = bClassification ? SWRasterPrimIDCountSRV : nullptr;
					RasterParameters->IndirectPrimIDs = bClassification ? RasterizerPrimIDsSRV : nullptr;

					FComputeShaderUtils::AddPass(
						GraphBuilder, 
						RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterNaive"), 
						bCullingEnableInRasterizers ? ComputeShaderRasterNaive_CullingOn : ComputeShaderRasterNaive_CullingOff,
						RasterParameters, 
						FIntVector(NumWorkGroups, 1, 1));
				}
				else
				{
					// Reset buffers
					{
						uint32 IndexGridClearValues[4] = { 0x0,0x0,0x0,0x0 };
						AddClearUAVPass(GraphBuilder, VisTileBinningGridUAV, IndexGridClearValues);
						AddClearUAVPass(GraphBuilder, VisTileArgsUAV, 0u);
						if (bTileCompactionPass)
						{
							AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(CompactedVisTileArgs, PF_R32_UINT), 0u);
						}
					}

					// Binning pass
					{
						FVisiblityRasterComputeBinningCS::FParameters* BinningParameters = GraphBuilder.AllocParameters<FVisiblityRasterComputeBinningCS::FParameters>();
						BinningParameters->Common = Common;
						BinningParameters->MacroGroupId = MacroGroup.MacroGroupId;
						BinningParameters->HairMaterialId = PrimitiveInfo.MaterialId;
						BinningParameters->ViewUniformBuffer = ViewUniformShaderParameters;
						BinningParameters->SceneDepthTexture = SceneDepthTexture;
						BinningParameters->OutVisTilePrims = GraphBuilder.CreateUAV(VisTilePrims, PF_R32_UINT);
						BinningParameters->OutVisTileBinningGrid = VisTileBinningGridUAV;
						BinningParameters->VisTileDepthGrid = VisTileDepthGrid;
						BinningParameters->OutVisTileArgs = VisTileArgsUAV;
						BinningParameters->OutVisTileData = GraphBuilder.CreateUAV(VisTileData);
						BinningParameters->HairInstance = GetHairStrandsInstanceParameters(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnableInRasterizers, bForceRegister);
						BinningParameters->VertexCount = PointCount;
						BinningParameters->VisTileBinningGridTex = VisTileBinningGrid;
						BinningParameters->IndirectPrimIDCount = bClassification ? SWRasterPrimIDCountSRV : nullptr;
						BinningParameters->IndirectPrimIDs = bClassification ? RasterizerPrimIDsSRV : nullptr;

						FComputeShaderUtils::AddPass(
							GraphBuilder, 
							RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterBinning(culling=%s)", bCullingEnableInRasterizers ? TEXT("On") : TEXT("Off")),
							bCullingEnableInRasterizers ? ComputeShaderBinning_CullingOn : ComputeShaderBinning_CullingOff,
							BinningParameters, 
							FIntVector(NumBinners, 1, 1));
					}

					if (bTileCompactionPass)
					{
						FVisiblityRasterComputeCompactionCS::FParameters* CompactionParameters = GraphBuilder.AllocParameters<FVisiblityRasterComputeCompactionCS::FParameters>();
						CompactionParameters->Common = Common;
						CompactionParameters->ViewUniformBuffer = ViewUniformShaderParameters;
						CompactionParameters->HairInstance = GetHairStrandsInstanceParameters(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnableInRasterizers, bForceRegister);
						CompactionParameters->VisTileData = VisTileDataSRV;
						CompactionParameters->VisTilePrims = VisTilePrimsSRV;
						CompactionParameters->VisTileArgs = VisTileArgsSRV;
						CompactionParameters->OutCompactedVisTileData = GraphBuilder.CreateUAV(CompactedVisTileData);
						CompactionParameters->OutCompactedVisTilePrims = GraphBuilder.CreateUAV(CompactedVisTilePrims, PF_R32_UINT);
						CompactionParameters->OutCompactedVisTileArgs = GraphBuilder.CreateUAV(CompactedVisTileArgs, PF_R32_UINT);

						FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterCompaction"), ComputeShaderCompaction, CompactionParameters, FIntVector(TileGridRes.X, TileGridRes.Y, 1));
					}

					// Raster pass
					if (bMultiSampleSWRaster)
					{
						FVisiblityRasterComputeRasterizeMultiSampleCS::FParameters* RasterParameters = GraphBuilder.AllocParameters<FVisiblityRasterComputeRasterizeMultiSampleCS::FParameters>();
						RasterParameters->Common = Common;
						RasterParameters->MacroGroupId = MacroGroup.MacroGroupId;
						RasterParameters->HairMaterialId = PrimitiveInfo.MaterialId;
						RasterParameters->ViewUniformBuffer = ViewUniformShaderParameters;
						RasterParameters->SceneDepthTexture = SceneDepthTexture;
						RasterParameters->OutHairCountTexture = HairCountTextureUAV;
						RasterParameters->OutDepthCovTexture = DepthCovTextureUAV;
						RasterParameters->OutPrimMatTexture = PrimMatTextureUAV;
						RasterParameters->CoverageScale = CoverageScale;
						RasterParameters->HairInstance = GetHairStrandsInstanceParameters(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnableInRasterizers, bForceRegister);
						RasterParameters->VisTilePrims = bTileCompactionPass ? GraphBuilder.CreateSRV(CompactedVisTilePrims, PF_R32_UINT) : VisTilePrimsSRV;
						RasterParameters->VisTileArgs = bTileCompactionPass ? GraphBuilder.CreateSRV(CompactedVisTileArgs, PF_R32_UINT) : VisTileArgsSRV;
						RasterParameters->VisTileData = bTileCompactionPass ? GraphBuilder.CreateSRV(CompactedVisTileData) : VisTileDataSRV;

						FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterRaster(tiled)"), ComputeShaderRasterMultiSample, RasterParameters, FIntVector(NumRasterizers, 1, 1));
					}
					else
					{
						FVisiblityRasterComputeRasterizeCS::FParameters* RasterParameters = GraphBuilder.AllocParameters<FVisiblityRasterComputeRasterizeCS::FParameters>();
						RasterParameters->Common = Common;
						RasterParameters->MacroGroupId = MacroGroup.MacroGroupId;
						RasterParameters->HairMaterialId = PrimitiveInfo.MaterialId;
						RasterParameters->ViewUniformBuffer = ViewUniformShaderParameters;
						RasterParameters->SceneDepthTexture = SceneDepthTexture;
						RasterParameters->OutHairCountTexture = HairCountTextureUAV;
						RasterParameters->OutDepthCovTexture = DepthCovTextureUAV;
						RasterParameters->OutPrimMatTexture = PrimMatTextureUAV;
						RasterParameters->CoverageScale = CoverageScale;
						RasterParameters->HairInstance = GetHairStrandsInstanceParameters(GraphBuilder, ViewInfo, HairGroupPublicData, bCullingEnableInRasterizers, bForceRegister);
						RasterParameters->VisTilePrims = bTileCompactionPass ? GraphBuilder.CreateSRV(CompactedVisTilePrims, PF_R32_UINT) : VisTilePrimsSRV;
						RasterParameters->VisTileArgs = bTileCompactionPass ? GraphBuilder.CreateSRV(CompactedVisTileArgs, PF_R32_UINT) : VisTileArgsSRV;
						RasterParameters->VisTileData = bTileCompactionPass ? GraphBuilder.CreateSRV(CompactedVisTileData) : VisTileDataSRV;

						FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VisibilityComputeRasterRaster(tiled)"), ComputeShaderRaster, RasterParameters, FIntVector(NumRasterizers, 1, 1));
					}

					if (bDebugEnabled)
					{
						FVisiblityRasterComputeDebugCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVisiblityRasterComputeDebugCS::FParameters>();
						Parameters->MacroGroupId = 0;
						Parameters->Common = Common;
						Parameters->PrimitiveInfoIndex = PrimitiveInfoIndex;
						Parameters->TotalPrimitiveInfoCount = TotalPrimitiveInfoCount;
						Parameters->VisTileDepthGrid = VisTileDepthGrid;
						Parameters->VisTileBinningGrid = VisTileBinningGrid;
						Parameters->VisTileArgs = VisTileArgsSRV;
						ShaderPrint::SetParameters(GraphBuilder, ViewInfo.ShaderPrintData, Parameters->ShaderPrintParameters);

						TShaderMapRef<FVisiblityRasterComputeDebugCS> DebugComputeShader(ViewInfo.ShaderMap);
						FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VisibilityComputeRaster(Debug)"), DebugComputeShader, Parameters, FIntVector(FMath::DivideAndRoundUp(Common.TileRes.X, 8), FMath::DivideAndRoundUp(Common.TileRes.Y, 8), 1));
					}
				}
			}

			++PrimitiveInfoIndex;
			bAnyHairRasterized = true;
		}
	}

	// If no groom was rasterized, clear the PrimMat texture to invalid values so that RDG doesn't complain about unwritten resources.
	// Usually we don't have to clear this texture because it is only accessed if the corresponding texels in the hair count texture indicate valid samples.
	if (!bAnyHairRasterized)
	{
		uint32 PrimMatClearValues[4] = { 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF, 0xFFFFFFFF };
		AddClearUAVPass(GraphBuilder, PrimMatTextureUAV, PrimMatClearValues);
	}

	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Selection outline, HitProxyId, CustomDepth

class FHairStrandsEmitDepthPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsEmitDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsEmitDepthPS, FGlobalShader);

	class FColor : SHADER_PERMUTATION_BOOL("PERMUTATION_COLOR");
	class FTile : SHADER_PERMUTATION_BOOL("PERMUTATION_TILE");
	using FPermutationDomain = TShaderPermutationDomain<FColor, FTile>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, CoverageThreshold)
		SHADER_PARAMETER(FVector2f, InvViewportResolution)
		SHADER_PARAMETER(uint32, MaxMaterialCount)
		SHADER_PARAMETER(uint32, bHasValidHairOnlyDepth)
		SHADER_PARAMETER(uint32, bDiscardIfZero)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VisNodeIndex)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, CoverageTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, HairOnlyDepthTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedHairVis>, VisNodeData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ActiveMaterialIdBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsTilePassVS::FParameters, TileData)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_EMIT_DEPTH"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FHairStrandsEmitDepthPS, "/Engine/Private/HairStrands/HairStrandsHitProxy.usf", "EmitPS", SF_Pixel);

enum class EHairStrandsEmitDepthMode : uint8
{
	Selection,
	HitProxy,
	CustomDepth,
};

static void AddHairStrandsEmitDepthPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FIntRect& ViewportRect,
	const FHairStrandsMacroGroupDatas& MacroGroupDatas,
	const EHairStrandsEmitDepthMode Mode,
	const FHairStrandsTiles& TileData,
	FRDGTextureRef VisNodeIndex,
	FRDGBufferRef VisNodeData,
	FRDGTextureRef CoverageTexture,
	FRDGTextureRef HairOnlyDepthTexture,
	FRDGTextureRef OutHitProxyTexture,
	FRDGTextureRef OutEmitDepthTexture)
{
	if (View.HairStrandsMeshElements.Num() == 0 || !VisNodeData)
	{
		return;
	}

	// Create mapping table between MaterialId and BatchId
	TArray<uint32> ActiveMaterialId;
	ActiveMaterialId.Init(0u, View.HairStrandsMeshElements.Num());
	for (const FHairStrandsMacroGroupData& MacroGroupData : MacroGroupDatas)
	{
		for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : MacroGroupData.PrimitivesInfos)
		{
			if (ActiveMaterialId.IsValidIndex(PrimitiveInfo.MaterialId) && PrimitiveInfo.PrimitiveSceneProxy)
			{
				uint32 Value = 0u;
				switch (Mode)
				{
					case EHairStrandsEmitDepthMode::Selection   : Value = PrimitiveInfo.PrimitiveSceneProxy->IsSelected() ? 1u : 0u; break;
					case EHairStrandsEmitDepthMode::CustomDepth : Value = PrimitiveInfo.PrimitiveSceneProxy->ShouldRenderCustomDepth() ? 1u : 0u; break;
					case EHairStrandsEmitDepthMode::HitProxy    : Value = PrimitiveInfo.Mesh ? PrimitiveInfo.Mesh->BatchHitProxyId.GetColor().DWColor() : 0u; break;
				}
				ActiveMaterialId[PrimitiveInfo.MaterialId] = Value;
			}
		}
	}
	
	FRDGBufferRef ActiveMaterialIdBuffer = CreateUploadBuffer(GraphBuilder, TEXT("Hair.ActiveMaterialIdBuffer"), sizeof(uint32), ActiveMaterialId.Num(), ActiveMaterialId.GetData(), sizeof(uint32) * ActiveMaterialId.Num());
	auto* PassParameters = GraphBuilder.AllocParameters<FHairStrandsEmitDepthPS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->CoverageThreshold = FMath::Clamp(GHairStrands_Selection_CoverageThreshold, 0.f, 1.f);
	PassParameters->MaxMaterialCount = ActiveMaterialId.Num();
	PassParameters->InvViewportResolution = FVector2f(1.f/ViewportRect.Width(), 1.f/ViewportRect.Height());
	PassParameters->bDiscardIfZero = Mode != EHairStrandsEmitDepthMode::HitProxy ? 1u : 0u;
	PassParameters->VisNodeIndex = VisNodeIndex;
	PassParameters->VisNodeData = GraphBuilder.CreateSRV(VisNodeData);
	PassParameters->CoverageTexture = CoverageTexture;
	PassParameters->HairOnlyDepthTexture = HairOnlyDepthTexture;
	PassParameters->ActiveMaterialIdBuffer = GraphBuilder.CreateSRV(ActiveMaterialIdBuffer, PF_R32_UINT);
	if (PassParameters->HairOnlyDepthTexture == nullptr)
	{
		PassParameters->HairOnlyDepthTexture = GSystemTextures.GetDepthDummy(GraphBuilder);
	}
	
	bool bUseTile = false;
	FRHIDepthStencilState* DepthStencilState = nullptr;
	uint32 StencilRef = 0;
	const TCHAR* PassName = TEXT("Unknown");
	switch (Mode)
	{
		case EHairStrandsEmitDepthMode::Selection:
		{
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(OutEmitDepthTexture, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);
			DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Keep, SO_Replace>::GetRHI(); 
			StencilRef = 3;
			PassName = TEXT("Selection");
			bUseTile = false;
		}
		break;
		case EHairStrandsEmitDepthMode::CustomDepth:
		{
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(OutEmitDepthTexture, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilNop);
			DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual, false>::GetRHI(); 
			StencilRef = 0xFF;
			PassName = TEXT("CustomDepth");
			bUseTile = true;
		}
		break;
		case EHairStrandsEmitDepthMode::HitProxy:
		{
			PassParameters->RenderTargets[0] = FRenderTargetBinding(OutHitProxyTexture, ERenderTargetLoadAction::ELoad);
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(OutEmitDepthTexture, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);
			DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
			StencilRef = 0xFF;
			PassName = TEXT("HitProxy");
			bUseTile = true;
		}
		break;
		default: check(false);
	}

	const FHairStrandsTiles::ETileType TileType = FHairStrandsTiles::ETileType::HairAll;
	if (bUseTile)
	{
		PassParameters->TileData = GetHairStrandsTileParameters(View, TileData, TileType);
		check(TileData.IsValid());
	}

	FHairStrandsEmitDepthPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FHairStrandsEmitDepthPS::FColor>(Mode == EHairStrandsEmitDepthMode::HitProxy);
	PermutationVector.Set<FHairStrandsEmitDepthPS::FTile>(bUseTile);
	auto PixelShader = View.ShaderMap->GetShader<FHairStrandsEmitDepthPS>(PermutationVector);
	if (!bUseTile)
	{
		// We don't use tile rendering for hair selection, because the outline buffer is unscaled, and does not match the visibility buffer which is unscaled
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("HairStrands::EmitDepth(%s)", PassName),
			PixelShader,
			PassParameters,
			ViewportRect,
			TStaticBlendState<>::GetRHI(),
			TStaticRasterizerState<>::GetRHI(),
			DepthStencilState,
			StencilRef);
	}
	else
	{
		const FIntRect Viewport = View.ViewRect;
		TShaderMapRef<FHairStrandsTilePassVS> TileVertexShader(View.ShaderMap);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HairStrands::EmitDepth(%s)", PassName),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, TileVertexShader, PixelShader, Viewport, TileType, DepthStencilState](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FHairStrandsTilePassVS::FParameters ParametersVS = PassParameters->TileData;

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = DepthStencilState;

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = TileVertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PassParameters->TileData.bRectPrimitive > 0 ? PT_RectList : PT_TriangleList;
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
			SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), ParametersVS);
			RHICmdList.SetStreamSource(0, nullptr, 0);
			RHICmdList.DrawPrimitiveIndirect(PassParameters->TileData.TileIndirectBuffer->GetRHI(), FHairStrandsTiles::GetIndirectDrawArgOffset(TileType));
		});
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Groom comparison
class FHairStrandsPositionChangedCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHairStrandsPositionChangedCS);
	SHADER_USE_PARAMETER_STRUCT(FHairStrandsPositionChangedCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InstanceResgisteredIndex)
		SHADER_PARAMETER(uint32, PointCount)
		SHADER_PARAMETER(float, PositionThreshold2)
		SHADER_PARAMETER(uint32, HairStrandsVF_bCullingEnable)
		SHADER_PARAMETER(uint32, bDrawInvalidElement)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HairStrandsVF_CullingIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HairStrandsVF_CullingIndirectBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, CurrPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, PrevPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, GroupAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, InvalidationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, InvalidationPrintCounter)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	END_SHADER_PARAMETER_STRUCT()
public:
	static uint32 GetGroupSize() { return HAIR_VERTEXCOUNT_GROUP_SIZE; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SHADER_POSITION_CHANGED"), TEXT("1"));
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FHairStrandsPositionChangedCS, "/Engine/Private/HairStrands/HairStrandsRaytracingGeometry.usf", "MainCS", SF_Compute);

static void AddHairStrandsHasPositionChangedPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo* View,
	const FHairGroupPublicData* HairGroupPublicData,
	FHairTransientResources* TransientResources,
	FRDGBufferUAVRef InvalidationBuffer)
{
	check(TransientResources);

	const uint32 PointCount = HairGroupPublicData->GetActiveStrandsPointCount();

	FRDGBufferRef InvalidationPrintCounter = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1), TEXT("Hair.InvalidationPrintCounter"));
	FRDGBufferUAVRef InvalidationPrintCounterUAV = GraphBuilder.CreateUAV(InvalidationPrintCounter, PF_R32_UINT);
	AddClearUAVPass(GraphBuilder, InvalidationPrintCounterUAV, 0u);

	FHairStrandsPositionChangedCS::FParameters* Parameters = GraphBuilder.AllocParameters<FHairStrandsPositionChangedCS::FParameters>();
	Parameters->InstanceResgisteredIndex = HairGroupPublicData->Instance->RegisteredIndex;
	Parameters->PointCount = PointCount;
	Parameters->PositionThreshold2 = FMath::Square(GHairStrands_InvalidationPosition_Threshold);
	Parameters->bDrawInvalidElement = GHairStrands_InvalidationPosition_Debug > 0 ? 1u : 0u;
	Parameters->HairStrandsVF_bCullingEnable = 0u;
	Parameters->HairStrandsVF_CullingIndexBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 4, 0u), PF_R32_UINT);
	Parameters->HairStrandsVF_CullingIndirectBuffer = Parameters->HairStrandsVF_CullingIndexBuffer;
	Parameters->CurrPositionBuffer = HairGroupPublicData->VFInput.Strands.PositionBuffer.SRV;
	Parameters->PrevPositionBuffer = HairGroupPublicData->VFInput.Strands.PrevPositionBuffer.SRV;
	Parameters->GroupAABBBuffer = TransientResources->GroupAABBSRV;
	Parameters->InvalidationBuffer = InvalidationBuffer;
	Parameters->InvalidationPrintCounter = InvalidationPrintCounterUAV;
	ShaderPrint::SetParameters(GraphBuilder, View->ShaderPrintData, Parameters->ShaderPrintParameters);
	if (HairGroupPublicData->GetCullingResultAvailable())
	{
		Parameters->HairStrandsVF_CullingIndexBuffer = Register(GraphBuilder, HairGroupPublicData->GetCulledVertexIdBuffer(), ERDGImportedBufferFlags::CreateSRV).SRV;
		Parameters->HairStrandsVF_CullingIndirectBuffer = Register(GraphBuilder, HairGroupPublicData->GetDrawIndirectRasterComputeBuffer(), ERDGImportedBufferFlags::CreateSRV).SRV;
		Parameters->HairStrandsVF_bCullingEnable = 1;
	}

	const FIntVector DispatchCount(FMath::DivideAndRoundUp(PointCount, FHairStrandsPositionChangedCS::GetGroupSize()), 1, 1);
	TShaderMapRef<FHairStrandsPositionChangedCS> ComputeShader(View->ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::HasPositionChanged"),
		ComputeShader,
		Parameters,
		DispatchCount);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

namespace HairStrands
{

// Draw hair strands custom depth value
void DrawCustomDepth(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FIntRect& ViewportRect, FRDGTextureRef CustomDepthTexture)
{
	AddHairStrandsEmitDepthPass(
		GraphBuilder,
		View,
		ViewportRect,
		View.HairStrandsViewData.MacroGroupDatas,
		EHairStrandsEmitDepthMode::CustomDepth,
		View.HairStrandsViewData.VisibilityData.TileData,
		View.HairStrandsViewData.VisibilityData.NodeIndex,
		View.HairStrandsViewData.VisibilityData.NodeVisData,
		View.HairStrandsViewData.VisibilityData.CoverageTexture,
		View.HairStrandsViewData.VisibilityData.HairOnlyDepthTexture,
		nullptr,
		CustomDepthTexture);
}

// Draw hair strands depth value for outline selection
void DrawEditorSelection(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FIntRect& ViewportRect, FRDGTextureRef SelectionDepthTexture)
{
	#if WITH_EDITOR
	AddHairStrandsEmitDepthPass(
		GraphBuilder,
		View,
		ViewportRect,
		View.HairStrandsViewData.MacroGroupDatas,
		EHairStrandsEmitDepthMode::Selection,
		View.HairStrandsViewData.VisibilityData.TileData,
		View.HairStrandsViewData.VisibilityData.NodeIndex,
		View.HairStrandsViewData.VisibilityData.NodeVisData,
		View.HairStrandsViewData.VisibilityData.CoverageTexture,
		View.HairStrandsViewData.VisibilityData.HairOnlyDepthTexture,
		nullptr,
		SelectionDepthTexture);
	#endif
}

// Draw hair strands hit proxy values
void DrawHitProxies(
	FRDGBuilder& GraphBuilder,
	const FScene& Scene,
	const FViewInfo& View,
	FInstanceCullingManager& InstanceCullingManager,
	FRDGTextureRef HitProxyTexture,
	FRDGTextureRef HitProxyDepthTexture)
{
	// Proxy rendering is only supported/compatible with MSAA-visibility rendering. 
	// PPLL is not supported, but it is supposed to be used only for final render.
	const EHairVisibilityRenderMode RenderMode = GetHairVisibilityRenderMode(View.GetShaderPlatform());
	if (RenderMode != HairVisibilityRenderMode_MSAA_Visibility && 
		RenderMode != HairVisibilityRenderMode_PPLL && 
		RenderMode != HairVisibilityRenderMode_ComputeRaster && 
		RenderMode != HairVisibilityRenderMode_ComputeRasterForward)
	{
		return;
	}

	// TransientResources haven't been allocated by the main path. Skipping hit proxy rendering until then.
	if (Scene.HairStrandsSceneData.TransientResources == nullptr)
	{
		return;
	}

	// The hit proxy view reuse data generated by regular view. This means it assumes LOD selection, simulation, and interpolation has run. 
	// Geometry won't be updated for proxy view
	const FIntPoint Resolution = HitProxyTexture->Desc.Extent;
	FHairStrandsViewData HairStrandsViewData;
	FHairInstanceCullingResults EmptyCullingResults;
	CreateHairStrandsMacroGroups(GraphBuilder, &Scene, View, EmptyCullingResults, HairStrandsViewData, false /*bBuildGPUAABB*/);

	// We don't compute the transmittance texture as there is no need for picking.
	FRDGTextureRef DummyTransmittanceTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(Resolution, PF_R32_FLOAT, FClearValueBinding::White, ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV), TEXT("Hair.DummyTransmittanceTextureForHitProxyId"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(DummyTransmittanceTexture), 1.0f);

	FHairStrandsTiles TileData;
	const FHairStrandsMacroGroupDatas& MacroGroupDatas = HairStrandsViewData.MacroGroupDatas;

	FRDGTextureRef SceneDepthTexture = HitProxyDepthTexture;

	const uint32 NodeGroupSize = GetVendorOptimalGroupSize1D();
	const uint32 MaxSampleCount = 4;
	FRDGBufferRef  NodeCounter = nullptr;
	FRDGTextureRef VisNodeIndex = nullptr;
	FRDGBufferRef  VisNodeData = nullptr;
	FRDGBufferRef  VisNodeCoord = nullptr;
	FRDGTextureRef CoverageTexture = nullptr;
	FRDGBufferRef  IndirectArgsBuffer = nullptr;
	uint32 OutMaxNodeCount = 0;
	if (RenderMode == HairVisibilityRenderMode_ComputeRaster)
	{
		FRasterComputeOutput RasterOutput = AddVisibilityComputeRasterPass(
			GraphBuilder,
			View,
			MacroGroupDatas,
			Resolution,
			MaxSampleCount,
			SceneDepthTexture,
			false,
			true);

		// Generate Tile data
		check(RasterOutput.PrimMatTexture);
		TileData = AddHairStrandsGenerateTilesPass(GraphBuilder, View, RasterOutput.HairCountTexture);

		AddHairVisibilityCompactionComputeRasterPass(
			GraphBuilder,
			View,
			NodeGroupSize,
			MaxSampleCount,
			RasterOutput,
			TileData,
			NodeCounter,
			VisNodeIndex,
			VisNodeData,
			VisNodeCoord,
			CoverageTexture,
			IndirectArgsBuffer,
			OutMaxNodeCount);

	}
	else if (RenderMode == HairVisibilityRenderMode_MSAA_Visibility)
	{
		FHairPrimaryTransmittance ViewTransmittance = AddHairViewTransmittancePass(
			GraphBuilder,
			&Scene,
			&View,
			MacroGroupDatas,
			Resolution,
			false /*bOutputHairCount*/,
			SceneDepthTexture,
			InstanceCullingManager);

		// Generate Tile data
		check (ViewTransmittance.TransmittanceTexture);
		TileData = AddHairStrandsGenerateTilesPass(GraphBuilder, View, ViewTransmittance.TransmittanceTexture);

		FRDGTextureRef VisDepthTexture = AddHairVisibilityFillOpaqueDepth(
			GraphBuilder,
			View,
			Resolution,
			MacroGroupDatas,
			TileData,
			SceneDepthTexture);

		FRDGTextureRef VisIdTexture = nullptr;
		AddHairVisibilityMSAAPass(
			GraphBuilder,
			&Scene,
			&View,
			MacroGroupDatas,
			Resolution,
			TileData,
			InstanceCullingManager,
			VisIdTexture,
			VisDepthTexture);

		FHairVisibilityControlPointIdCompactionCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairVisibilityControlPointIdCompactionCS::FParameters>();
		PassParameters->MSAA_DepthTexture = VisDepthTexture;
		PassParameters->MSAA_IDTexture = VisIdTexture;
		PassParameters->ViewTransmittanceTexture = ViewTransmittance.TransmittanceTexture;

		AddHairVisibilityControlPointIdCompactionPass(
			false, // bUsePPLL
			GraphBuilder,
			View,
			SceneDepthTexture,
			MacroGroupDatas,
			NodeGroupSize,
			TileData,
			PassParameters,
			NodeCounter,
			VisNodeIndex,
			VisNodeData,
			VisNodeCoord,
			CoverageTexture,
			IndirectArgsBuffer,
			OutMaxNodeCount);
	}
	else if (RenderMode == HairVisibilityRenderMode_PPLL)
	{
		// Linked list generation pass
		FRDGTextureRef PPLLNodeCounterTexture;
		FRDGTextureRef PPLLNodeIndexTexture;
		FRDGBufferRef PPLLNodeDataBuffer;
		AddHairVisibilityPPLLPass(GraphBuilder, &Scene, &View, MacroGroupDatas, Resolution, InstanceCullingManager, SceneDepthTexture, PPLLNodeCounterTexture, PPLLNodeIndexTexture, PPLLNodeDataBuffer);

		// Generate Tile data
		check(PPLLNodeIndexTexture)
		TileData = AddHairStrandsGenerateTilesPass(GraphBuilder, View, PPLLNodeIndexTexture);

		// Linked list sorting pass and compaction into common representation
		FHairVisibilityControlPointIdCompactionCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairVisibilityControlPointIdCompactionCS::FParameters>();
		PassParameters->PPLLCounter  = PPLLNodeCounterTexture;
		PassParameters->PPLLNodeIndex= PPLLNodeIndexTexture;
		PassParameters->PPLLNodeData = GraphBuilder.CreateSRV(PPLLNodeDataBuffer);
		PassParameters->ViewTransmittanceTexture = nullptr;

		FRDGTextureRef ResolveMaskTexture = nullptr;
		AddHairVisibilityControlPointIdCompactionPass(
			true, // bUsePPLL
			GraphBuilder,
			View,
			SceneDepthTexture,
			MacroGroupDatas,
			NodeGroupSize,
			TileData,
			PassParameters,
			NodeCounter,
			VisNodeIndex,
			VisNodeData,
			VisNodeCoord,
			CoverageTexture,
			IndirectArgsBuffer,
			OutMaxNodeCount);
	}

	// Make this tile dispatch
	AddHairStrandsEmitDepthPass(
		GraphBuilder,
		View,
		View.ViewRect,
		MacroGroupDatas,
		EHairStrandsEmitDepthMode::HitProxy,
		TileData,
		VisNodeIndex,
		VisNodeData,
		CoverageTexture,
		View.HairStrandsViewData.VisibilityData.HairOnlyDepthTexture,
		HitProxyTexture,
		HitProxyDepthTexture);
}

// Check if any simulated/skinned-bound groom has its positions updated (e.g. for invalidating the path-tracer accumulation)
bool HasPositionsChanged(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View)
{
	if (View.HairStrandsMeshElements.IsEmpty())
	{
		// there are no hair strands in the scene
		return false;
	}

	if (GHairStrands_InvalidationPosition_Threshold < 0)
	{
		return false;
	}

	FHairStrandsViewStateData* HairStrandsViewStateData = const_cast<FHairStrandsViewStateData*>(&View.ViewState->HairStrandsViewStateData);
	if (!HairStrandsViewStateData->IsInit())
	{
		HairStrandsViewStateData->Init();
	}

	TArray<const FHairGroupPublicData*> GroupDatas;
	for (const FMeshBatchAndRelevance& Batch : View.HairStrandsMeshElements)
	{
		FHairGroupPublicData* HairGroupPublicData = GetHairData(Batch.Mesh);
		check(HairGroupPublicData);
		const int32 LODIndex = HairGroupPublicData->GetIntLODIndex();
		const bool bHasSimulationOrSkinning = 
			HairGroupPublicData->GetGeometryType(LODIndex) == EHairGeometryType::Strands &&
			(HairGroupPublicData->IsSimulationEnable(LODIndex) || HairGroupPublicData->bIsDeformationEnable || HairGroupPublicData->GetBindingType(LODIndex) == EHairBindingType::Skinning);
		if (bHasSimulationOrSkinning)
		{
			GroupDatas.Add(HairGroupPublicData);
		}
	}
	if (GroupDatas.IsEmpty())
	{
		// there are no strands currently being simulated or skinned
		return false;
	}

	FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1);
	Desc.Usage |= BUF_SourceCopy;
	FRDGBufferRef InvalidationBuffer = GraphBuilder.CreateBuffer(Desc, TEXT("Hair.HasSimulationRunningBuffer"));
	FRDGBufferUAVRef InvalidationUAV = GraphBuilder.CreateUAV(InvalidationBuffer, PF_R32_UINT);
	AddClearUAVPass(GraphBuilder, InvalidationUAV, 0u);

	// Compare current/previous and enqueue aggregated comparison
	for (const FHairGroupPublicData* GroupData : GroupDatas)
	{
		AddHairStrandsHasPositionChangedPass(GraphBuilder, &View, GroupData, Scene.HairStrandsSceneData.TransientResources, InvalidationUAV);
	}

	// Pull a 'ready' previous frame value
	bool bHasPositionChanged = HairStrandsViewStateData->ReadPositionsChanged();

	// Enqueue new readback request
	HairStrandsViewStateData->EnqueuePositionsChanged(GraphBuilder, InvalidationBuffer);


	return bHasPositionChanged;
}

}

///////////////////////////////////////////////////////////////////////////////////////////////////
bool GetHairStrandsSkyLightingEnable();

void RenderHairStrandsVisibilityBuffer(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	FViewInfo& View,
	FRDGTextureRef SceneGBufferATexture,
	FRDGTextureRef SceneGBufferBTexture,
	FRDGTextureRef SceneGBufferCTexture,
	FRDGTextureRef SceneGBufferDTexture,
	FRDGTextureRef SceneGBufferETexture,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef SceneVelocityTexture,
	FInstanceCullingManager& InstanceCullingManager)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CLM_RenderHairStrandsVisibility);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsVisibility, "HairStrandsVisibility");

	FHairStrandsMacroGroupDatas& MacroGroupDatas = View.HairStrandsViewData.MacroGroupDatas;
	check(View.Family);
	check(MacroGroupDatas.Num() > 0);

	const FIntRect HairRect = ComputeVisibleHairStrandsMacroGroupsRect(View, View.ViewRect, MacroGroupDatas);
	const int32 HairPixelCount = HairRect.Width() * HairRect.Height();
	if (HairPixelCount <= 0)
	{
		View.HairStrandsViewData.VisibilityData = FHairStrandsVisibilityData();
		return;
	}

	FRDGTextureRef SceneMaterial0 = nullptr;
	FRDGTextureRef SceneMaterial1 = nullptr;
	FRDGTextureRef SceneMaterial2 = nullptr;
	if (Substrate::IsSubstrateEnabled() && !Substrate::IsSubstrateBlendableGBufferEnabled(View.GetShaderPlatform()))
	{
		SceneMaterial0 = View.SubstrateViewData.SceneData->MaterialTextureArray;
		SceneMaterial1 = View.SubstrateViewData.SceneData->TopLayerTexture;
		SceneMaterial2 = nullptr;
	}
	else
	{
		SceneMaterial0 = SceneGBufferBTexture;
		SceneMaterial1 = SceneGBufferCTexture;
		SceneMaterial2 = SceneGBufferATexture;
	}

	{
		
		{
			FHairStrandsVisibilityData& VisibilityData = View.HairStrandsViewData.VisibilityData;
			VisibilityData.NodeGroupSize = GetVendorOptimalGroupSize1D();
			VisibilityData.MaxSampleCount = GetMaxSamplePerPixel(View.GetShaderPlatform());

			// Use the scene color for computing target resolution as the View.ViewRect, 
			// doesn't include the actual resolution padding which make buffer size 
			// mismatch, and create artifact (e.g. velocity computation)
			check(SceneDepthTexture);
			const FIntPoint Resolution = SceneDepthTexture->Desc.Extent;

			const EHairVisibilityRenderMode RenderMode = GetHairVisibilityRenderMode(View.GetShaderPlatform());
			check(
				RenderMode == HairVisibilityRenderMode_MSAA_Visibility || 
				RenderMode == HairVisibilityRenderMode_PPLL || 
				RenderMode == HairVisibilityRenderMode_ComputeRaster ||
				RenderMode == HairVisibilityRenderMode_ComputeRasterForward);

			const bool bRunColorAndDepthPatching = SceneMaterial0 != nullptr && SceneMaterial1 != nullptr && SceneColorTexture != nullptr && RenderMode != HairVisibilityRenderMode_ComputeRaster && RenderMode != HairVisibilityRenderMode_ComputeRasterForward;

			FRDGTextureRef HairOnlyDepthTexture = GraphBuilder.CreateTexture(SceneDepthTexture->Desc, TEXT("Hair.HairOnlyDepthTexture"));
			FRDGTextureRef CoverageTexture = nullptr;
			FRDGTextureRef CompactNodeIndex = nullptr;
			FRDGBufferRef  CompactNodeData = nullptr;
			FRDGBufferRef  CompactNodeVis = nullptr;
			FRDGBufferRef  NodeCounter = nullptr;

			if (RenderMode == HairVisibilityRenderMode_ComputeRasterForward)
			{
				FRasterForwardCullingOutput Out = AddHairStrandsForwardCullingPass(
					GraphBuilder,
					View,
					MacroGroupDatas,
					Resolution,
					SceneDepthTexture,
					true,
					false);

				if (Out.NodeCoord == nullptr)
				{
					return;
				}
				VisibilityData.MaxNodeCount = Out.NodeCoord->Desc.NumElements;

				VisibilityData.MaxControlPointCount = Out.NodeCoord->Desc.NumElements;
				VisibilityData.ControlPointsSRV = Out.PointsSRV;
				VisibilityData.ControlPointCount = GraphBuilder.CreateSRV(Out.PointCount);
				VisibilityData.RasterizedInstanceCount = Out.RasterizedInstanceCount;

				CompactNodeIndex = Out.NodeIndex;
				NodeCounter = Out.PointCount;
				CompactNodeVis = Out.NodeVis;
				FRDGBufferRef CompactNodeCoord = Out.NodeCoord;

				FRDGBufferRef IndirectArgsBuffer = AddCopyIndirectArgPass(GraphBuilder, &View, VisibilityData.NodeGroupSize, 1, GraphBuilder.CreateSRV(Out.PointCount));

				// Generate Tile data
				VisibilityData.TileData = AddHairStrandsGenerateTilesPass(GraphBuilder, View,Out.Resolution); // Tile for all pixels == no tile accelaration

				// Evaluate material based on the visibility pass result
				// Output both complete sample data + per-sample velocity
				FMaterialPassOutput PassOutput = AddHairMaterialPass(
					GraphBuilder,
					Scene,
					&View,
					false,
					false,
					MacroGroupDatas,
					InstanceCullingManager,
					VisibilityData.NodeGroupSize,
					CompactNodeIndex,
					CompactNodeVis,
					CompactNodeCoord,
					NodeCounter,
					IndirectArgsBuffer);

				// Dummy coverage texture
				CoverageTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(Resolution, FHairStrandsVisibilityData::CoverageFormat, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource), TEXT("Hair.CoverageTexture"));

				CompactNodeData = PassOutput.NodeData;

				VisibilityData.SampleLightingViewportResolution = PassOutput.SampleLightingTexture->Desc.Extent;
				VisibilityData.SampleLightingTexture = PassOutput.SampleLightingTexture;
				VisibilityData.NodeIndex = CompactNodeIndex;
				VisibilityData.CoverageTexture = CoverageTexture;
				VisibilityData.HairOnlyDepthTexture = HairOnlyDepthTexture;
				VisibilityData.NodeData = CompactNodeData;
				VisibilityData.NodeVisData = CompactNodeVis;
				VisibilityData.NodeCoord = CompactNodeCoord;
				VisibilityData.NodeIndirectArg = IndirectArgsBuffer;
				VisibilityData.NodeCount = NodeCounter;
				VisibilityData.ResolveMaskTexture = nullptr;
				VisibilityData.ControlPointVelocitySRV = PassOutput.NodeVelocitySRV;

				// For fully covered pixels, write: 
				// * black color into the scene color
				// * closest depth
				// * unlit shading model ID 
				if (bRunColorAndDepthPatching)
				{
					AddHairMaterialDataPatchPass(
						GraphBuilder,
						View,
						VisibilityData.TileData,
						CoverageTexture,
						CompactNodeIndex,
						CompactNodeData,
						SceneMaterial0,
						SceneMaterial1,
						SceneMaterial2,
						SceneColorTexture,
						SceneDepthTexture,
						VisibilityData.LightChannelMaskTexture);
				}

				AddHairOnlyDepthPass(
					GraphBuilder,
					View,
					VisibilityData.TileData,
					CoverageTexture,
					CompactNodeIndex,
					CompactNodeData,
					HairOnlyDepthTexture);

				AddHairOnlyHZBPass(
					GraphBuilder,
					View,
					HairOnlyDepthTexture,
					VisibilityData.HairOnlyDepthHZBParameters,
					VisibilityData.HairOnlyDepthClosestHZBTexture,
					VisibilityData.HairOnlyDepthFurthestHZBTexture);
			}
			else if (RenderMode == HairVisibilityRenderMode_ComputeRaster)
			{
				FRasterComputeOutput RasterOutput = AddVisibilityComputeRasterPass(
					GraphBuilder,
					View,
					MacroGroupDatas,
					Resolution,
					VisibilityData.MaxSampleCount,
					SceneDepthTexture,
					true,
					false);

				// Generate Tile data
				check(RasterOutput.PrimMatTexture);
				VisibilityData.TileData = AddHairStrandsGenerateTilesPass(GraphBuilder, View, RasterOutput.HairCountTexture);

				FRDGBufferRef CompactNodeCoord;
				FRDGBufferRef IndirectArgsBuffer;
				FRDGTextureRef ResolveMaskTexture = nullptr;
				AddHairVisibilityCompactionComputeRasterPass(
					GraphBuilder,
					View,
					VisibilityData.NodeGroupSize,
					VisibilityData.MaxSampleCount,
					RasterOutput,
					VisibilityData.TileData,
					NodeCounter,
					CompactNodeIndex,
					CompactNodeVis,
					CompactNodeCoord,
					CoverageTexture,
					IndirectArgsBuffer,
					VisibilityData.MaxNodeCount);


				// Evaluate material based on the visiblity pass result
				// Output both complete sample data + per-sample velocity
				FMaterialPassOutput PassOutput = AddHairMaterialPass(
					GraphBuilder,
					Scene,
					&View,
					false,
					true,
					MacroGroupDatas,
					InstanceCullingManager,
					VisibilityData.NodeGroupSize,
					CompactNodeIndex,
					CompactNodeVis,
					CompactNodeCoord,
					NodeCounter,
					IndirectArgsBuffer);

				// Merge per-sample velocity into the scene velocity buffer
				AddHairVelocityPass(
					GraphBuilder,
					View,
					MacroGroupDatas,
					VisibilityData.TileData,
					CoverageTexture,
					CompactNodeIndex,
					CompactNodeVis,
					PassOutput.NodeVelocity,
					SceneVelocityTexture,
					ResolveMaskTexture);

				CompactNodeData = PassOutput.NodeData;

				VisibilityData.SampleLightingViewportResolution = PassOutput.SampleLightingTexture->Desc.Extent;
				VisibilityData.SampleLightingTexture = PassOutput.SampleLightingTexture;
				VisibilityData.NodeIndex = CompactNodeIndex;
				VisibilityData.CoverageTexture = CoverageTexture;
				VisibilityData.HairOnlyDepthTexture = HairOnlyDepthTexture;
				VisibilityData.NodeData = CompactNodeData;
				VisibilityData.NodeVisData = CompactNodeVis;
				VisibilityData.NodeCoord = CompactNodeCoord;
				VisibilityData.NodeIndirectArg = IndirectArgsBuffer;
				VisibilityData.NodeCount = NodeCounter;
				VisibilityData.ResolveMaskTexture = ResolveMaskTexture;	

				// For fully covered pixels, write: 
				// * black color into the scene color
				// * closest depth
				// * unlit shading model ID 
				if (bRunColorAndDepthPatching)
				{
					AddHairMaterialDataPatchPass(
						GraphBuilder,
						View,
						VisibilityData.TileData,
						CoverageTexture,
						CompactNodeIndex,
						CompactNodeData,
						SceneMaterial0,
						SceneMaterial1,
						SceneMaterial2,
						SceneColorTexture,
						SceneDepthTexture,
						VisibilityData.LightChannelMaskTexture);
				}

				AddHairOnlyDepthPass(
					GraphBuilder,
					View,
					VisibilityData.TileData,
					CoverageTexture,
					CompactNodeIndex,
					CompactNodeData,
					HairOnlyDepthTexture);

				AddHairOnlyHZBPass(
					GraphBuilder,
					View,
					HairOnlyDepthTexture,
					VisibilityData.HairOnlyDepthHZBParameters,
					VisibilityData.HairOnlyDepthClosestHZBTexture,
					VisibilityData.HairOnlyDepthFurthestHZBTexture);
			}
			else if (RenderMode == HairVisibilityRenderMode_MSAA_Visibility)
			{
				// Run the view transmittance pass if needed (not in PPLL mode that is already a high quality render path)
				FHairPrimaryTransmittance ViewTransmittance;
				{
					// Note: Hair count is required for the sky lighting at the moment as it is used for the TT term
					// TT sampling is disable in hair sky lighting integrator 0. So the GetHairStrandsSkyLightingEnable() check is no longer needed
					const bool bOutputHairCount = GHairStrandsHairCountToTransmittance > 0;
					ViewTransmittance = AddHairViewTransmittancePass(
						GraphBuilder,
						Scene,
						&View,
						MacroGroupDatas,
						Resolution,
						bOutputHairCount,
						SceneDepthTexture,
						InstanceCullingManager);

					const bool bHairCountToTransmittance = GHairStrandsHairCountToTransmittance > 0;
					if (bHairCountToTransmittance)
					{
						ViewTransmittance.TransmittanceTexture = AddHairHairCountToTransmittancePass(
							GraphBuilder,
							View,
							ViewTransmittance.HairCountTexture);
					}

				}

				// Generate Tile data
				check (ViewTransmittance.TransmittanceTexture);
				VisibilityData.TileData = AddHairStrandsGenerateTilesPass(GraphBuilder, View, ViewTransmittance.TransmittanceTexture);

				struct FRDGMsaaVisibilityResources
				{
					FRDGTextureRef DepthTexture;
					FRDGTextureRef IdTexture;
				} MsaaVisibilityResources;

				MsaaVisibilityResources.DepthTexture = AddHairVisibilityFillOpaqueDepth(
					GraphBuilder,
					View,
					Resolution,
					MacroGroupDatas,
					VisibilityData.TileData,
					SceneDepthTexture);

				AddHairVisibilityMSAAPass(
					GraphBuilder,
					Scene,
					&View,
					MacroGroupDatas,
					Resolution,
					VisibilityData.TileData,
					InstanceCullingManager,
					MsaaVisibilityResources.IdTexture,
					MsaaVisibilityResources.DepthTexture);

				// This is used when compaction is not enabled.
				VisibilityData.MaxSampleCount = MsaaVisibilityResources.IdTexture->Desc.NumSamples;
				VisibilityData.HairOnlyDepthTexture = HairOnlyDepthTexture;
				
				{
					FHairVisibilityControlPointIdCompactionCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairVisibilityControlPointIdCompactionCS::FParameters>();
					PassParameters->MSAA_DepthTexture = MsaaVisibilityResources.DepthTexture;
					PassParameters->MSAA_IDTexture = MsaaVisibilityResources.IdTexture;
					PassParameters->ViewTransmittanceTexture = ViewTransmittance.TransmittanceTexture;

					FRDGBufferRef CompactNodeCoord;
					FRDGBufferRef IndirectArgsBuffer;
					FRDGTextureRef ResolveMaskTexture = nullptr;
					AddHairVisibilityControlPointIdCompactionPass(
						false, // bUsePPLL
						GraphBuilder,
						View,
						SceneDepthTexture,
						MacroGroupDatas,
						VisibilityData.NodeGroupSize,
						VisibilityData.TileData,
						PassParameters,
						NodeCounter,
						CompactNodeIndex,
						CompactNodeVis,
						CompactNodeCoord,
						CoverageTexture,
						IndirectArgsBuffer,
						VisibilityData.MaxNodeCount);


					{
						const bool bUpdateSampleCoverage = GHairStrandsSortHairSampleByDepth > 0;

						// Evaluate material based on the visiblity pass result
						// Output both complete sample data + per-sample velocity
						FMaterialPassOutput PassOutput = AddHairMaterialPass(
							GraphBuilder,
							Scene,
							&View,
							bUpdateSampleCoverage,
							true,
							MacroGroupDatas,
							InstanceCullingManager,
							VisibilityData.NodeGroupSize,
							CompactNodeIndex,
							CompactNodeVis,
							CompactNodeCoord,
							NodeCounter,
							IndirectArgsBuffer);

						// Merge per-sample velocity into the scene velocity buffer
						AddHairVelocityPass(
							GraphBuilder,
							View,
							MacroGroupDatas,
							VisibilityData.TileData,
							CoverageTexture,
							CompactNodeIndex,
							CompactNodeVis,
							PassOutput.NodeVelocity,
							SceneVelocityTexture,
							ResolveMaskTexture);

						if (bUpdateSampleCoverage)
						{
							PassOutput.NodeData = AddUpdateSampleCoveragePass(
								GraphBuilder,
								&View,
								CompactNodeIndex,
								PassOutput.NodeData);
						}

						CompactNodeData = PassOutput.NodeData;

						VisibilityData.SampleLightingViewportResolution = PassOutput.SampleLightingTexture->Desc.Extent;
						VisibilityData.SampleLightingTexture			= PassOutput.SampleLightingTexture;
					}

					VisibilityData.NodeIndex			= CompactNodeIndex;
					VisibilityData.CoverageTexture		= CoverageTexture;
					VisibilityData.HairOnlyDepthTexture	= HairOnlyDepthTexture;
					VisibilityData.NodeData				= CompactNodeData;
					VisibilityData.NodeVisData			= CompactNodeVis;
					VisibilityData.NodeCoord			= CompactNodeCoord;
					VisibilityData.NodeIndirectArg		= IndirectArgsBuffer;
					VisibilityData.NodeCount			= NodeCounter;
					VisibilityData.ResolveMaskTexture	= ResolveMaskTexture;

					// View transmittance depth test needs to happen before the scene depth is patched with the hair depth (for fully-covered-by-hair pixels)
					if (ViewTransmittance.HairCountTexture)
					{
						AddHairViewTransmittanceDepthPass(
							GraphBuilder,
							View,
							CoverageTexture,
							SceneDepthTexture,
							ViewTransmittance.HairCountTexture);
						VisibilityData.ViewHairCountTexture = ViewTransmittance.HairCountTexture;
					}

					// For fully covered pixels, write: 
					// * black color into the scene color
					// * closest depth
					// * unlit shading model ID 
					if (bRunColorAndDepthPatching)
					{
						AddHairMaterialDataPatchPass(
							GraphBuilder,
							View,
							VisibilityData.TileData,
							CoverageTexture,
							CompactNodeIndex,
							CompactNodeData,
							SceneMaterial0,
							SceneMaterial1,
							SceneMaterial2,
							SceneColorTexture,
							SceneDepthTexture,
							VisibilityData.LightChannelMaskTexture);
					}

					AddHairOnlyDepthPass(
						GraphBuilder,
						View,
						VisibilityData.TileData,
						CoverageTexture,
						CompactNodeIndex,
						CompactNodeData,
						HairOnlyDepthTexture);

					AddHairOnlyHZBPass(
						GraphBuilder,
						View,
						HairOnlyDepthTexture,
						VisibilityData.HairOnlyDepthHZBParameters,
						VisibilityData.HairOnlyDepthClosestHZBTexture,
						VisibilityData.HairOnlyDepthFurthestHZBTexture);
				}
			}
			else if (RenderMode == HairVisibilityRenderMode_PPLL)
			{
				// In this pas we reuse the scene depth buffer to cull hair pixels out.
				// Pixel data is accumulated in buffer containing data organized in a linked list with node scattered in memory according to pixel shader execution. 
				// This with up to width * height * GHairVisibilityPPLLGlobalMaxPixelNodeCount node total maximum.
				// After we have that a node sorting pass happening and we finally output all the data once into the common compaction node list.

				FRDGTextureRef PPLLNodeCounterTexture;
				FRDGTextureRef PPLLNodeIndexTexture;
				FRDGBufferRef PPLLNodeDataBuffer;
				FRDGTextureRef ViewZDepthTexture = SceneDepthTexture;

				// Linked list generation pass
				AddHairVisibilityPPLLPass(GraphBuilder, Scene, &View, MacroGroupDatas, Resolution, InstanceCullingManager, ViewZDepthTexture, PPLLNodeCounterTexture, PPLLNodeIndexTexture, PPLLNodeDataBuffer);

				// Generate Tile data
				check(PPLLNodeIndexTexture)
				VisibilityData.TileData = AddHairStrandsGenerateTilesPass(GraphBuilder, View, PPLLNodeIndexTexture);

				// Linked list sorting pass and compaction into common representation
				{
					FHairVisibilityControlPointIdCompactionCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHairVisibilityControlPointIdCompactionCS::FParameters>();
					PassParameters->PPLLCounter  = PPLLNodeCounterTexture;
					PassParameters->PPLLNodeIndex= PPLLNodeIndexTexture;
					PassParameters->PPLLNodeData = GraphBuilder.CreateSRV(PPLLNodeDataBuffer);
					PassParameters->ViewTransmittanceTexture = nullptr;

					FRDGBufferRef CompactNodeCoord;
					FRDGBufferRef IndirectArgsBuffer;
					FRDGTextureRef ResolveMaskTexture = nullptr;
					AddHairVisibilityControlPointIdCompactionPass(
						true, // bUsePPLL
						GraphBuilder,
						View,
						SceneDepthTexture,
						MacroGroupDatas,
						VisibilityData.NodeGroupSize,
						VisibilityData.TileData,
						PassParameters,
						NodeCounter,
						CompactNodeIndex,
						CompactNodeVis,
						CompactNodeCoord,
						CoverageTexture,
						IndirectArgsBuffer,
						VisibilityData.MaxNodeCount);

					{
						const bool bUpdateSampleCoverage = GHairStrandsSortHairSampleByDepth > 0;

						// Evaluate material based on the visiblity pass result
						// Output both complete sample data + per-sample velocity
						FMaterialPassOutput PassOutput = AddHairMaterialPass(
							GraphBuilder,
							Scene,
							&View,
							bUpdateSampleCoverage,
							true,
							MacroGroupDatas,
							InstanceCullingManager,
							VisibilityData.NodeGroupSize,
							CompactNodeIndex,
							CompactNodeVis,
							CompactNodeCoord,
							NodeCounter,
							IndirectArgsBuffer);

						// Merge per-sample velocity into the scene velocity buffer
						AddHairVelocityPass(
							GraphBuilder,
							View,
							MacroGroupDatas,
							VisibilityData.TileData,
							CoverageTexture,
							CompactNodeIndex,
							CompactNodeVis,
							PassOutput.NodeVelocity,
							SceneVelocityTexture,
							ResolveMaskTexture);

						if (bUpdateSampleCoverage)
						{
							PassOutput.NodeData = AddUpdateSampleCoveragePass(
								GraphBuilder,
								&View,
								CompactNodeIndex,
								PassOutput.NodeData);
						}

						CompactNodeData = PassOutput.NodeData;

						VisibilityData.SampleLightingViewportResolution = PassOutput.SampleLightingTexture->Desc.Extent;
						VisibilityData.SampleLightingTexture			= PassOutput.SampleLightingTexture;
					}

					VisibilityData.MaxSampleCount = GetMaxSamplePerPixel(View.GetShaderPlatform());
					VisibilityData.NodeIndex = CompactNodeIndex;
					VisibilityData.CoverageTexture = CoverageTexture;
					VisibilityData.HairOnlyDepthTexture = HairOnlyDepthTexture;
					VisibilityData.NodeVisData = CompactNodeVis;
					VisibilityData.NodeData = CompactNodeData;
					VisibilityData.NodeCoord = CompactNodeCoord;
					VisibilityData.NodeIndirectArg = IndirectArgsBuffer;
					VisibilityData.NodeCount = NodeCounter;
				}

				if (bRunColorAndDepthPatching)
				{
					AddHairMaterialDataPatchPass(
						GraphBuilder,
						View,
						VisibilityData.TileData,
						CoverageTexture,
						CompactNodeIndex,
						CompactNodeData,
						SceneMaterial0,
						SceneMaterial1,
						SceneMaterial2,
						SceneColorTexture,
						SceneDepthTexture,
						VisibilityData.LightChannelMaskTexture);
				}

				AddHairOnlyDepthPass(
					GraphBuilder,
					View,
					VisibilityData.TileData,
					CoverageTexture,
					CompactNodeIndex,
					CompactNodeData,
					HairOnlyDepthTexture);

				AddHairOnlyHZBPass(
					GraphBuilder,
					View,
					HairOnlyDepthTexture,
					VisibilityData.HairOnlyDepthHZBParameters,
					VisibilityData.HairOnlyDepthClosestHZBTexture,
					VisibilityData.HairOnlyDepthFurthestHZBTexture);

			#if WITH_EDITOR
				// Extract texture for debug visualization
				if (GHairStrandsDebugPPLL > 0)
				{
					View.HairStrandsViewData.DebugData.PPLLData.NodeCounterTexture = PPLLNodeCounterTexture;
					View.HairStrandsViewData.DebugData.PPLLData.NodeIndexTexture = PPLLNodeIndexTexture;
					View.HairStrandsViewData.DebugData.PPLLData.NodeDataBuffer = PPLLNodeDataBuffer;
				}
			#endif
			}

		#if RHI_RAYTRACING
			if (IsRayTracingEnabled() && View.IsRayTracingAllowedForView() && VisibilityData.LightChannelMaskTexture == nullptr)
			{
				VisibilityData.LightChannelMaskTexture = AddHairLightChannelMaskPass(
					GraphBuilder,
					View,
					VisibilityData.TileData,
					CoverageTexture,
					CompactNodeIndex,
					CompactNodeData,
					SceneDepthTexture);
			}
		#endif

			const bool bIsMSAAForwardEnabled = IsForwardShadingEnabled(View.GetShaderPlatform()) && SceneColorTexture && SceneColorTexture->Desc.NumSamples > 1;
			if (bIsMSAAForwardEnabled)
			{		
				VisibilityData.ResolveMaskTexture = nullptr;
			}
		}
	}
}
