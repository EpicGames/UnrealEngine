// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairVisibilityRendering.h: Hair strands visibility buffer implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"

class FViewInfo;
class FInstanceCullingManager;

void RenderHairStrandsVisibilityBuffer(
	FRDGBuilder& GraphBuilder,
	const class FScene* Scene,
	FViewInfo& View,
	FRDGTextureRef SceneGBufferATexture,
	FRDGTextureRef SceneGBufferBTexture,
	FRDGTextureRef SceneGBufferCTexture,
	FRDGTextureRef SceneGBufferDTexture,
	FRDGTextureRef SceneGBufferETexture,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef SceneVelocityTexture,
	FInstanceCullingManager& InstanceCullingManager);

EPixelFormat GetHairLightSampleFormat();
void SetUpViewHairRenderInfo(const FViewInfo& ViewInfo, FVector4f& OutHairRenderInfo, uint32& OutHairRenderInfoBits, uint32& OutHairComponents, float FinalResolutionScale = 1.0f);

