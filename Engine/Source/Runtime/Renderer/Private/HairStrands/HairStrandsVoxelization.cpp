// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsVoxelization.h"
#include "HairStrandsRasterCommon.h"
#include "HairStrandsUtils.h"
#include "HairStrandsLUT.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "SceneTextureParameters.h"
#include "ShaderPrintParameters.h"
#include "RenderGraphUtils.h"
#include "PostProcess/PostProcessing.h"
#include "ScenePrivate.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneViewState.h"

DECLARE_GPU_STAT(HairStrandsIndVoxelPageClear);
DECLARE_GPU_STAT(HairStrandsVoxelize);
DECLARE_GPU_STAT(HairStrandsDensityMipGen);

// Common threading group layout used in voxelization code by the following pass
// * FVoxelIndPageClearCS
// * FVirtualVoxelInjectOpaqueCS
// * FVirtualVoxelPatchPageIndexWithMipDataCS
// * FVirtualVoxelGenerateMipCS
// 
// This threading layout allows to process up to 1M allocated pages (16 * 65k), and process 4x4x4 voxels within the same page.
// Notes: 1024 threads is the limit per work-group 
static const FIntVector GHairVoxel_GroupSize_NumVoxelPerPage_1_NumAllocatedPage(64u, 0u, 16u);

static int32 GHairVoxelizationEnable = 1;
static FAutoConsoleVariableRef CVarGHairVoxelizationEnable(TEXT("r.HairStrands.Voxelization"), GHairVoxelizationEnable, TEXT("Enable hair voxelization for transmittance evaluation"));

static float GHairVoxelizationAABBScale = 1.0f;
static FAutoConsoleVariableRef CVarHairVoxelizationAABBScale(TEXT("r.HairStrands.Voxelization.AABBScale"), GHairVoxelizationAABBScale, TEXT("Scale the hair macro group bounding box"));

static float GHairVoxelizationDensityScale = 2.0f;
static float GHairVoxelizationDensityScale_AO = -1;
static float GHairVoxelizationDensityScale_Shadow = -1;
static float GHairVoxelizationDensityScale_Transmittance = -1;
static float GHairVoxelizationDensityScale_Environment = -1;
static float GHairVoxelizationDensityScale_Raytracing = -1; 
static FAutoConsoleVariableRef CVarHairVoxelizationDensityScale(TEXT("r.HairStrands.Voxelization.DensityScale"), GHairVoxelizationDensityScale, TEXT("Scale the hair density when computing voxel transmittance. Default value is 2 (arbitraty)"));
static FAutoConsoleVariableRef CVarHairVoxelizationDensityScale_AO(TEXT("r.HairStrands.Voxelization.DensityScale.AO"), GHairVoxelizationDensityScale_AO, TEXT("Scale the hair density when computing voxel AO. (Default:-1, it will use the global density scale"));
static FAutoConsoleVariableRef CVarHairVoxelizationDensityScale_Shadow(TEXT("r.HairStrands.Voxelization.DensityScale.Shadow"), GHairVoxelizationDensityScale_Shadow, TEXT("Scale the hair density when computing voxel shadow. (Default:-1, it will use the global density scale"));
static FAutoConsoleVariableRef CVarHairVoxelizationDensityScale_Transmittance(TEXT("r.HairStrands.Voxelization.DensityScale.Transmittance"), GHairVoxelizationDensityScale_Transmittance, TEXT("Scale the hair density when computing voxel transmittance. (Default:-1, it will use the global density scale"));
static FAutoConsoleVariableRef CVarHairVoxelizationDensityScale_Environment(TEXT("r.HairStrands.Voxelization.DensityScale.Environment"), GHairVoxelizationDensityScale_Environment, TEXT("Scale the hair density when computing voxel environment. (Default:-1, it will use the global density scale"));
static FAutoConsoleVariableRef CVarHairVoxelizationDensityScale_Raytracing(TEXT("r.HairStrands.Voxelization.DensityScale.Raytracing"), GHairVoxelizationDensityScale_Raytracing, TEXT("Scale the hair density when computing voxel raytracing. (Default:-1, it will use the global density scale"));

static float GetHairStrandsVoxelizationDensityScale() { return FMath::Max(0.0f, GHairVoxelizationDensityScale); }
static float GetHairStrandsVoxelizationDensityScale_AO() { return GHairVoxelizationDensityScale_AO >= 0 ? FMath::Max(0.0f, GHairVoxelizationDensityScale_AO) : GetHairStrandsVoxelizationDensityScale();  }
static float GetHairStrandsVoxelizationDensityScale_Shadow() { return GHairVoxelizationDensityScale_Shadow >= 0 ? FMath::Max(0.0f, GHairVoxelizationDensityScale_Shadow) : GetHairStrandsVoxelizationDensityScale(); }
static float GetHairStrandsVoxelizationDensityScale_Transmittance() { return GHairVoxelizationDensityScale_Transmittance >= 0 ? FMath::Max(0.0f, GHairVoxelizationDensityScale_Transmittance) : GetHairStrandsVoxelizationDensityScale(); }
static float GetHairStrandsVoxelizationDensityScale_Environment() { return GHairVoxelizationDensityScale_Environment >= 0 ? FMath::Max(0.0f, GHairVoxelizationDensityScale_Environment) : GetHairStrandsVoxelizationDensityScale(); }
static float GetHairStrandsVoxelizationDensityScale_Raytracing() { return GHairVoxelizationDensityScale_Raytracing >= 0 ? FMath::Max(0.0f, GHairVoxelizationDensityScale_Raytracing) : GetHairStrandsVoxelizationDensityScale(); }

static float GHairVoxelizationDepthBiasScale_Shadow = 2.0f;
static float GHairVoxelizationDepthBiasScale_Transmittance = 3.0f;
static float GHairVoxelizationDepthBiasScale_Environment = 1.8f;
static FAutoConsoleVariableRef CVarHairVoxelizationDepthBiasScale_Shadow(TEXT("r.HairStrands.Voxelization.DepthBiasScale.Shadow"), GHairVoxelizationDepthBiasScale_Shadow, TEXT("Set depth bias for voxel ray marching for analyticaly light. Offset the origin position towards the light for shadow computation"));
static FAutoConsoleVariableRef CVarHairVoxelizationDepthBiasScale_Light(TEXT("r.HairStrands.Voxelization.DepthBiasScale.Light"), GHairVoxelizationDepthBiasScale_Transmittance, TEXT("Set depth bias for voxel ray marching for analyticaly light. Offset the origin position towards the light for transmittance computation"));
static FAutoConsoleVariableRef CVarHairVoxelizationDepthBiasScale_Transmittance(TEXT("r.HairStrands.Voxelization.DepthBiasScale.Transmittance"), GHairVoxelizationDepthBiasScale_Transmittance, TEXT("Set depth bias for voxel ray marching for analyticaly light. Offset the origin position towards the light for transmittance computation"));
static FAutoConsoleVariableRef CVarHairVoxelizationDepthBiasScale_Environment(TEXT("r.HairStrands.Voxelization.DepthBiasScale.Environment"), GHairVoxelizationDepthBiasScale_Environment, TEXT("Set depth bias for voxel ray marching for environement lights. Offset the origin position towards the light"));

static int32 GHairVoxelInjectOpaqueDepthEnable = 1;
static FAutoConsoleVariableRef CVarHairVoxelInjectOpaqueDepthEnable(TEXT("r.HairStrands.Voxelization.InjectOpaqueDepth"), GHairVoxelInjectOpaqueDepthEnable, TEXT("Inject opaque geometry depth into the voxel volume for acting as occluder."));

static int32 GHairStransVoxelInjectOpaqueBiasCount = 3;
static int32 GHairStransVoxelInjectOpaqueMarkCount = 6;
static FAutoConsoleVariableRef CVarHairStransVoxelInjectOpaqueBiasCount(TEXT("r.HairStrands.Voxelization.InjectOpaque.BiasCount"), GHairStransVoxelInjectOpaqueBiasCount, TEXT("Bias, in number of voxel, at which opaque depth is injected."));
static FAutoConsoleVariableRef CVarHairStransVoxelInjectOpaqueMarkCount(TEXT("r.HairStrands.Voxelization.InjectOpaque.MarkCount"), GHairStransVoxelInjectOpaqueMarkCount, TEXT("Number of voxel marked as opaque starting along the view direction beneath the opaque surface."));

static float GHairStransVoxelRaymarchingSteppingScale = 1.15f;
static float GHairStransVoxelRaymarchingSteppingScale_Shadow = -1;
static float GHairStransVoxelRaymarchingSteppingScale_Transmittance = -1;
static float GHairStransVoxelRaymarchingSteppingScale_Environment = -1;
static float GHairStransVoxelRaymarchingSteppingScale_Raytracing = -1;
static FAutoConsoleVariableRef CVarHairStransVoxelRaymarchingSteppingScale(TEXT("r.HairStrands.Voxelization.Raymarching.SteppingScale"), GHairStransVoxelRaymarchingSteppingScale, TEXT("Stepping scale used for raymarching the voxel structure for shadow."));
static FAutoConsoleVariableRef CVarHairStransVoxelRaymarchingSteppingScale_Shadow(TEXT("r.HairStrands.Voxelization.Raymarching.SteppingScale.Shadow"), GHairStransVoxelRaymarchingSteppingScale_Shadow, TEXT("Stepping scale used for raymarching the voxel structure, override scale for shadow (default -1)."));
static FAutoConsoleVariableRef CVarHairStransVoxelRaymarchingSteppingScale_Transmittance(TEXT("r.HairStrands.Voxelization.Raymarching.SteppingScale.Transmission"), GHairStransVoxelRaymarchingSteppingScale_Transmittance, TEXT("Stepping scale used for raymarching the voxel structure, override scale for transmittance (default -1)."));
static FAutoConsoleVariableRef CVarHairStransVoxelRaymarchingSteppingScale_Environment(TEXT("r.HairStrands.Voxelization.Raymarching.SteppingScale.Environment"), GHairStransVoxelRaymarchingSteppingScale_Environment, TEXT("Stepping scale used for raymarching the voxel structure, override scale for env. lighting (default -1)."));
static FAutoConsoleVariableRef CVarHairStransVoxelRaymarchingSteppingScale_Raytracing(TEXT("r.HairStrands.Voxelization.Raymarching.SteppingScale.Raytracing"), GHairStransVoxelRaymarchingSteppingScale_Raytracing, TEXT("Stepping scale used for raymarching the voxel structure, override scale for raytracing (default -1)."));

static float GetHairStrandsVoxelizationDepthBiasScale_Shadow() { return FMath::Max(0.0f, GHairVoxelizationDepthBiasScale_Shadow); }
static float GetHairStrandsVoxelizationDepthBiasScale_Transmittance() { return FMath::Max(0.0f, GHairVoxelizationDepthBiasScale_Transmittance); }
static float GetHairStrandsVoxelizationDepthBiasScale_Environment() { return FMath::Max(0.0f, GHairVoxelizationDepthBiasScale_Environment); }

static int32 GHairForVoxelTransmittanceAndShadow = 0;
static FAutoConsoleVariableRef CVarHairForVoxelTransmittanceAndShadow(TEXT("r.HairStrands.Voxelization.ForceTransmittanceAndShadow"), GHairForVoxelTransmittanceAndShadow, TEXT("For transmittance and shadow to be computed with density volume. This requires voxelization is enabled."));

static int32 GHairVirtualVoxel = 1;
static float GHairVirtualVoxel_VoxelWorldSize = 0.3f; // 3.0mm
static int32 GHairVirtualVoxel_PageResolution = 32;
static int32 GHairVirtualVoxel_PageCountPerDim = 14;
static FAutoConsoleVariableRef CVarHairVirtualVoxel(TEXT("r.HairStrands.Voxelization.Virtual"), GHairVirtualVoxel, TEXT("Enable the two voxel hierarchy."));
static FAutoConsoleVariableRef CVarHairVirtualVoxel_VoxelWorldSize(TEXT("r.HairStrands.Voxelization.Virtual.VoxelWorldSize"), GHairVirtualVoxel_VoxelWorldSize, TEXT("World size of a voxel in cm."));
static FAutoConsoleVariableRef CVarHairVirtualVoxel_VoxelPageResolution(TEXT("r.HairStrands.Voxelization.Virtual.VoxelPageResolution"), GHairVirtualVoxel_PageResolution, TEXT("Resolution of a voxel page."));
static FAutoConsoleVariableRef CVarHairVirtualVoxel_VoxelPageCountPerDim(TEXT("r.HairStrands.Voxelization.Virtual.VoxelPageCountPerDim"), GHairVirtualVoxel_PageCountPerDim, TEXT("Number of voxel pages per texture dimension. The voxel page memory is allocated with a 3D texture. This value provide the resolution of this texture."));

static int32 GHairVirtualVoxelGPUDriven = 1;
static int32 GHairVirtualVoxelGPUDrivenMinPageIndexRes = 32;
static int32 GHairVirtualVoxelGPUDrivenMaxPageIndexRes = 64;
static FAutoConsoleVariableRef CVarHairVirtualVoxelGPUDriven(TEXT("r.HairStrands.Voxelization.GPUDriven"), GHairVirtualVoxelGPUDriven, TEXT("Enable GPU driven voxelization."));
static FAutoConsoleVariableRef CVarHairVirtualVoxelGPUDrivenMinPageIndexRes(TEXT("r.HairStrands.Voxelization.GPUDriven.MinPageIndexResolution"), GHairVirtualVoxelGPUDrivenMinPageIndexRes, TEXT("Min resolution of the page index. This is used for allocating a conservative page index buffer when GPU driven allocation is enabled."));
static FAutoConsoleVariableRef CVarHairVirtualVoxelGPUDrivenMaxPageIndexRes(TEXT("r.HairStrands.Voxelization.GPUDriven.MaxPageIndexResolution"), GHairVirtualVoxelGPUDrivenMaxPageIndexRes, TEXT("Max resolution of the page index. This is used for allocating a conservative page index buffer when GPU driven allocation is enabled."));

static const FIntPoint GPUDrivenViewportResolution = FIntPoint(4096, 4096);

static int32 GHairVirtualVoxelInvalidEmptyPageIndex = 1;
static FAutoConsoleVariableRef CVarHairVirtualVoxelInvalidEmptyPageIndex(TEXT("r.HairStrands.Voxelization.Virtual.InvalidateEmptyPageIndex"), GHairVirtualVoxelInvalidEmptyPageIndex, TEXT("Invalid voxel page index which does not contain any voxelized data."));

static int32 GHairStrandsVoxelComputeRasterMaxVoxelCount = 32;
static FAutoConsoleVariableRef CVarHairStrandsVoxelComputeRasterMaxVoxelCount(TEXT("r.HairStrands.Voxelization.Virtual.ComputeRasterMaxVoxelCount"), GHairStrandsVoxelComputeRasterMaxVoxelCount, TEXT("Max number of voxel which are rasterized for a given hair segment. This is for debug purpose only."));

static float GHairVirtualVoxelRaytracing_ShadowOcclusionThreshold= 1;
static float GHairVirtualVoxelRaytracing_SkyOcclusionThreshold = 1;
static FAutoConsoleVariableRef CVarHairVirtualVoxelRaytracing_ShadowOcclusionThreshold(TEXT("r.RayTracing.Shadows.HairOcclusionThreshold"), GHairVirtualVoxelRaytracing_ShadowOcclusionThreshold, TEXT("Define the number of hair that need to be crossed, before casting occlusion (default = 1)"), ECVF_RenderThreadSafe);
static FAutoConsoleVariableRef CVarHairVirtualVoxelRaytracing_SkyOcclusionThreshold(TEXT("r.RayTracing.Sky.HairOcclusionThreshold"), GHairVirtualVoxelRaytracing_SkyOcclusionThreshold, TEXT("Define the number of hair that need to be crossed, before casting occlusion (default = 1)"), ECVF_RenderThreadSafe);

static int32 GHairVirtualVoxelAdaptive_Enable = 1;
static float GHairVirtualVoxelAdaptive_CorrectionSpeed = 0.1f;
static float GHairVirtualVoxelAdaptive_CorrectionThreshold = 0.90f;
static FAutoConsoleVariableRef CVarHairVirtualVoxelAdaptive_Enable(TEXT("r.HairStrands.Voxelization.Virtual.Adaptive"), GHairVirtualVoxelAdaptive_Enable, TEXT("Enable adaptive voxel allocation (default = 1)"), ECVF_RenderThreadSafe);
static FAutoConsoleVariableRef CVarHairVirtualVoxelAdaptive_CorrectionSpeed(TEXT("r.HairStrands.Voxelization.Virtual.Adaptive.CorrectionSpeed"), GHairVirtualVoxelAdaptive_CorrectionSpeed, TEXT("Define the speed at which allocation adaption runs (value in 0..1, default = 0.25). A higher number means faster adaptation, but with a risk of oscillation i.e. over and under allocation"), ECVF_RenderThreadSafe);
static FAutoConsoleVariableRef CVarHairVirtualVoxelAdaptive_CorrectionThreshold(TEXT("r.HairStrands.Voxelization.Virtual.Adaptive.CorrectionThreshold"), GHairVirtualVoxelAdaptive_CorrectionThreshold, TEXT("Define the allocation margin to limit over allocation (value in 0..1, default = 0.95)"), ECVF_RenderThreadSafe);

static int32 GHairVirtualVoxel_JitterMode = 1;
static FAutoConsoleVariableRef CVarHairVirtualVoxel_JitterMode(TEXT("r.HairStrands.Voxelization.Virtual.Jitter"), GHairVirtualVoxel_JitterMode, TEXT("Change jittered for voxelization/traversal. 0: No jitter 1: Regular randomized jitter: 2: Constant Jitter (default = 1)"), ECVF_RenderThreadSafe);

void GetVoxelPageResolution(uint32& OutPageResolution, uint32& OutPageResolutionLog2)
{
	OutPageResolutionLog2 = FMath::CeilLogTwo(FMath::Clamp(GHairVirtualVoxel_PageResolution, 2, 256));
	OutPageResolution = (1u << OutPageResolutionLog2);
}

static bool IsHairStrandsAdaptiveVoxelAllocationEnable()
{
	return GHairVirtualVoxelAdaptive_Enable > 0;
}

bool IsHairStrandsVoxelizationEnable(EShaderPlatform ShaderPlatform)
{
	return RHISupportsVolumeTextureAtomics(ShaderPlatform) && GHairVoxelizationEnable > 0;
}

bool IsHairStrandsForVoxelTransmittanceAndShadowEnable(EShaderPlatform ShaderPlatform)
{
	return IsHairStrandsVoxelizationEnable(ShaderPlatform) && GHairForVoxelTransmittanceAndShadow > 0;
}

bool IsHairStrandsGPUDrivenVoxelizationEnabled()
{
	return GHairVirtualVoxelGPUDriven > 0; 
}


///////////////////////////////////////////////////////////////////////////////////////////////////

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FVirtualVoxelParameters, "VirtualVoxel");

///////////////////////////////////////////////////////////////////////////////////////////////////
class FVirtualVoxelInjectOpaqueCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVirtualVoxelInjectOpaqueCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualVoxelInjectOpaqueCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT(FHairStrandsVoxelCommonParameters, VirtualVoxelParams)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(FVector2f, SceneDepthResolution)
		SHADER_PARAMETER(uint32, VoxelBiasCount)
		SHADER_PARAMETER(uint32, VoxelMarkCount)
		RDG_BUFFER_ACCESS(IndirectDispatchArgs, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, OutPageTexture)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static FIntVector GetGroupSize() { return GHairVoxel_GroupSize_NumVoxelPerPage_1_NumAllocatedPage; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_INJECTOPAQUE_VIRTUALVOXEL"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE_X"), GetGroupSize().X);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE_Y"), GetGroupSize().Y);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE_Z"), GetGroupSize().Z);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVirtualVoxelInjectOpaqueCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "MainCS", SF_Compute);

static void AddVirtualVoxelInjectOpaquePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHairStrandsVoxelResources& VoxelResources,
	const uint32& MacroGroupId,
	FRDGTextureUAVRef OutTexturePage)
{
	FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

	const uint32 TotalPageCount = VoxelResources.Parameters.Common.PageIndexCount;
	const uint32 PageResolution = VoxelResources.Parameters.Common.PageResolution;

	const uint32 SideSlotCount = FMath::CeilToInt(FMath::Pow(TotalPageCount, 1.f / 3.f));
	const uint32 SideVoxelCount= SideSlotCount * PageResolution;

	FVirtualVoxelInjectOpaqueCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVirtualVoxelInjectOpaqueCS::FParameters>();
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->VirtualVoxelParams = VoxelResources.Parameters.Common;
	Parameters->VoxelBiasCount = FMath::Max(0, GHairStransVoxelInjectOpaqueBiasCount);
	Parameters->VoxelMarkCount = FMath::Max(0, GHairStransVoxelInjectOpaqueMarkCount);
	Parameters->SceneDepthResolution = SceneTextures.SceneDepthTexture->Desc.Extent;
	Parameters->SceneTextures = SceneTextures;
	Parameters->MacroGroupId = MacroGroupId;
	Parameters->OutPageTexture = OutTexturePage;
	Parameters->IndirectDispatchArgs = VoxelResources.IndirectArgsBuffer;
	TShaderMapRef<FVirtualVoxelInjectOpaqueCS> ComputeShader(View.ShaderMap);
	const FGlobalShaderMap* GlobalShaderMap = View.ShaderMap;

	const uint32 ArgsOffset = sizeof(FRHIDispatchIndirectParameters) * MacroGroupId;

	FComputeShaderUtils::AddPass(
		GraphBuilder, 
		RDG_EVENT_NAME("HairStrands::InjectOpaqueDepthInVoxel"), 
		ComputeShader, 
		Parameters, 
		VoxelResources.IndirectArgsBuffer,
		ArgsOffset);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FVoxelAllocatePageIndexCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelAllocatePageIndexCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelAllocatePageIndexCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, CPUPageWorldSize)
		SHADER_PARAMETER(float, CPUVoxelWorldSize)
		SHADER_PARAMETER(uint32, bUseCPUVoxelWorldSize)
		SHADER_PARAMETER(uint32, TotalPageIndexCount)
		SHADER_PARAMETER(uint32, PageResolution)
		SHADER_PARAMETER(uint32, MacroGroupCount)
		SHADER_PARAMETER(uint32, IndirectDispatchGroupSize)
		SHADER_PARAMETER(uint32, bDoesMacroGroupSupportVoxelization)

		SHADER_PARAMETER_ARRAY(FVector4f, CPU_TranslatedWorldMinAABB, [MAX_HAIR_MACROGROUP_COUNT])
		SHADER_PARAMETER_ARRAY(FVector4f, CPU_TranslatedWorldMaxAABB, [MAX_HAIR_MACROGROUP_COUNT])
		SHADER_PARAMETER_ARRAY(FIntVector4, CPU_PageIndexResolution, [MAX_HAIR_MACROGROUP_COUNT])
		SHADER_PARAMETER(uint32, CPU_bUseCPUData)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, GPUVoxelWorldSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, MacroGroupVoxelSizeBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, MacroGroupAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>, MacroGroupVoxelAlignedAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint4>, OutPageIndexResolutionAndOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutPageIndexAllocationIndirectBufferArgs)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return MAX_HAIR_MACROGROUP_COUNT; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ALLOCATEPAGEINDEX"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

class FVoxelMarkValidPageIndex_PrepareCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelMarkValidPageIndex_PrepareCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelMarkValidPageIndex_PrepareCS, FGlobalShader);

	class FUseCluster : SHADER_PERMUTATION_BOOL("PERMUTATION_USE_CLUSTER");
	using FPermutationDomain = TShaderPermutationDomain<FUseCluster>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InstanceRegisteredIndex)
		SHADER_PARAMETER(uint32, ClusterOffset)
		SHADER_PARAMETER(uint32, ClusterCount)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(uint32, bUseMacroGroupBoundCPU)
		SHADER_PARAMETER(FVector3f, MacroGroupBoundCPU_TranslatedWorldMinAABB)
		SHADER_PARAMETER(FVector3f, MacroGroupBoundCPU_TranslatedWorldMaxAABB)
		SHADER_PARAMETER(FVector3f, TranslatedWorldOffsetCorrection)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, GroupAABBsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, ClusterAABBsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, MacroGroupVoxelAlignedAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, PageIndexResolutionAndOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutValidPageIndexBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize(bool bUseCluster) { return bUseCluster ? 256u : 8u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		const bool bUseCluster = PermutationVector.Get<FUseCluster>() != 0;

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_MARKVALID_PREPARE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize(bUseCluster));
	}
};

class FVoxelAllocateVoxelPageCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelAllocateVoxelPageCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelAllocateVoxelPageCS, FGlobalShader);

	class FGPUDriven : SHADER_PERMUTATION_INT("PERMUTATION_GPU_DRIVEN", 2);
	using FPermutationDomain = TShaderPermutationDomain<FGPUDriven>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, CPU_PageIndexResolution)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(uint32, PageCount)
		SHADER_PARAMETER(uint32, CPU_PageIndexCount)
		SHADER_PARAMETER(uint32, CPU_PageIndexOffset)
		RDG_BUFFER_ACCESS(IndirectBufferArgs, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, PageIndexResolutionAndOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, RWPageIndexGlobalCounter)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, RWPageIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, RWPageToPageIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, RWPageIndexCoordBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 256u; /* Allow to handle up to 16M pages (= 65k * 256) */ }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ALLOCATE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

class FVoxelAddNodeDescCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelAddNodeDescCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelAddNodeDescCS, FGlobalShader);

	class FGPUDriven : SHADER_PERMUTATION_INT("PERMUTATION_GPU_DRIVEN", 2);
	using FPermutationDomain = TShaderPermutationDomain<FGPUDriven>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector3f, CPU_TranslatedWorldMinAABB)
		SHADER_PARAMETER(uint32, CPU_PageIndexOffset)
		SHADER_PARAMETER(FVector3f, CPU_TranslatedWorldMaxAABB)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(FIntVector, CPU_PageIndexResolution)
		SHADER_PARAMETER(float, CPU_VoxelWorldSize)
		SHADER_PARAMETER(uint32, bUseCPUVoxelWorldSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, GPU_VoxelWorldSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, MacroGroupVoxelAlignedAABBBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, MacroGroupVoxelSizeBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, PageIndexResolutionAndOffsetBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutNodeDescBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 1; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ADDDESC"), 1);
	}
};

class FVoxelAddIndirectBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelAddIndirectBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelAddIndirectBufferCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(uint32, MacroGroupCount)
		SHADER_PARAMETER(uint32, PageResolution)
		SHADER_PARAMETER(FIntVector, IndirectGroupSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, PageIndexGlobalCounter)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutIndirectArgsBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 32; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ADDINDIRECTBUFFER"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

class FVoxelIndPageClearCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelIndPageClearCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelIndPageClearCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FHairStrandsVoxelCommonParameters, VirtualVoxelParams)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, PageIndexGlobalCounter)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, OutPageTexture)
		RDG_BUFFER_ACCESS(IndirectDispatchBuffer, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

public:
	static FIntVector GetGroupSize() { return GHairVoxel_GroupSize_NumVoxelPerPage_1_NumAllocatedPage; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_INDPAGECLEAR"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE_X"), GetGroupSize().X);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE_Y"), GetGroupSize().Y);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE_Z"), GetGroupSize().Z);
	}
};

class FVoxelAdaptiveFeedbackCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelAdaptiveFeedbackCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelAdaptiveFeedbackCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, CPUAllocatedPageCount)
		SHADER_PARAMETER(float,  CPUMinVoxelWorldSize)
		SHADER_PARAMETER(float,  AdaptiveCorrectionThreshold)
		SHADER_PARAMETER(float,  AdaptiveCorrectionSpeed)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, PageIndexGlobalCounter)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, CurrGPUMinVoxelWorldSize)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, NextGPUMinVoxelWorldSize)

		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 1; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ADAPTIVE_FEEDBACK"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVoxelMarkValidPageIndex_PrepareCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "MarkValid_PrepareCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FVoxelAllocatePageIndexCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "AllocatePageIndex", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FVoxelAllocateVoxelPageCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "AllocateCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FVoxelAddNodeDescCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "AddDescCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FVoxelAddIndirectBufferCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "AddIndirectBufferCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FVoxelIndPageClearCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "VoxelIndPageClearCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FVoxelAdaptiveFeedbackCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "FeedbackCS", SF_Compute);

inline FIntVector CeilToInt(const FVector& V)
{
	return FIntVector(FMath::CeilToInt(V.X), FMath::CeilToInt(V.Y), FMath::CeilToInt(V.Z));
}

static void AddAllocateVoxelPagesPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FHairTransientResources& TransientResources,
	FHairStrandsMacroGroupDatas& MacroGroupDatas,
	FHairStrandsMacroGroupResources& MacroGroupResources,
	const FIntVector PageCountResolution,
	const uint32 PageCount,
	const float CPUMinVoxelWorldSize,
	const uint32 PageResolution,
	const FIntVector PageTextureResolution,
	uint32& OutTotalPageIndexCount,
	FRDGBufferRef& OutPageIndexBuffer,
	FRDGBufferRef& OutPageToPageIndexBuffer,
	FRDGBufferRef& OutPageIndexCoordBuffer,
	FRDGBufferRef& OutNodeDescBuffer,
	FRDGBufferRef& OutIndirectArgsBuffer,
	FRDGBufferRef& OutPageIndexGlobalCounter,
	FRDGBufferRef& CurrGPUMinVoxelSize,
	FRDGBufferRef& NextGPUMinVoxelSize)
{
	const bool bIsGPUDriven = IsHairStrandsGPUDrivenVoxelizationEnabled();
	const uint32 MacroGroupCount = MacroGroupDatas.Num();
	if (MacroGroupCount == 0)
		return;

	struct FCPUMacroGroupAllocation
	{
		FVector		TranslatedMinAABB;
		FVector		TranslatedMaxAABB;
		FIntVector	PageIndexResolution;
		uint32		PageIndexCount;
		uint32		PageIndexOffset;
		uint32		MacroGroupId;
	};
	const float CPUPageWorldSize = PageResolution * CPUMinVoxelWorldSize;

	OutTotalPageIndexCount = 0;
	TArray<FCPUMacroGroupAllocation> CPUAllocationDescs;	
	for (FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
	{
		// Snap the max AABB to the voxel size
		// Scale the bounding box in place of proper GPU driven AABB for now
		const float Scale = FMath::Clamp(GHairVoxelizationAABBScale, 0.01f, 10.0f);
		const FVector BoxCenter = MacroGroup.Bounds.GetBox().GetCenter();
		FVector TranslatedMinAABB = (MacroGroup.Bounds.GetBox().Min - BoxCenter) * Scale + BoxCenter + View.ViewMatrices.GetPreViewTranslation();
		FVector TranslatedMaxAABB = (MacroGroup.Bounds.GetBox().Max - BoxCenter) * Scale + BoxCenter + View.ViewMatrices.GetPreViewTranslation();

		// Allocate enough pages to cover the AABB, where page (0,0,0) origin sit on MinAABB.
		FVector MacroGroupSize = TranslatedMaxAABB - TranslatedMinAABB;
		const FIntVector PageIndexResolution = CeilToInt(MacroGroupSize / CPUPageWorldSize);
		MacroGroupSize = FVector(PageIndexResolution) * CPUPageWorldSize;
		TranslatedMaxAABB = MacroGroupSize + TranslatedMinAABB;

		FCPUMacroGroupAllocation& Out = CPUAllocationDescs.AddDefaulted_GetRef();
		Out.MacroGroupId = MacroGroup.MacroGroupId;
		Out.TranslatedMinAABB = TranslatedMinAABB; // >> these should actually be computed on the GPU ... 
		Out.TranslatedMaxAABB = TranslatedMaxAABB; // >> these should actually be computed on the GPU ... 
		Out.PageIndexResolution = PageIndexResolution;
		Out.PageIndexCount = Out.PageIndexResolution.X * Out.PageIndexResolution.Y * Out.PageIndexResolution.Z;
		Out.PageIndexOffset = OutTotalPageIndexCount;

		OutTotalPageIndexCount += Out.PageIndexCount;
	}

	// Over-allocation (upper bound)
	if (bIsGPUDriven)
	{
		// Use the max between the estimated size on CPU and a pseudo-conservative side driven by settings. The CPU estimation is no necessarily correct as the bounds are not reliable on skel. mesh.
		const uint32 MinPageIndexCount = GHairVirtualVoxelGPUDrivenMinPageIndexRes * GHairVirtualVoxelGPUDrivenMinPageIndexRes * GHairVirtualVoxelGPUDrivenMinPageIndexRes;
		const uint32 MaxPageIndexCount = GHairVirtualVoxelGPUDrivenMaxPageIndexRes * GHairVirtualVoxelGPUDrivenMaxPageIndexRes * GHairVirtualVoxelGPUDrivenMaxPageIndexRes;
		OutTotalPageIndexCount = FMath::Clamp(OutTotalPageIndexCount, MinPageIndexCount, MaxPageIndexCount);
	}
	check(OutTotalPageIndexCount > 0);
	
	FRDGBufferRef PageIndexBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), OutTotalPageIndexCount), TEXT("Hair.PageIndexBuffer"));
	FRDGBufferRef PageIndexCoordBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), OutTotalPageIndexCount), TEXT("Hair.PageIndexCoordBuffer"));
	FRDGBufferRef PageIndexGlobalCounter = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), MacroGroupCount + 1), TEXT("Hair.PageIndexGlobalCounter")); // First entry store global count
	FRDGBufferRef NodeDescBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FPackedVirtualVoxelNodeDesc), MacroGroupCount), TEXT("Hair.VirtualVoxelNodeDescBuffer"));
	FRDGBufferRef IndirectArgsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(MacroGroupCount+1), TEXT("Hair.VirtualVoxelIndirectArgsBuffer")); // First entry store global args

	const uint32 TotalPageCount = PageCountResolution.X * PageCountResolution.Y * PageCountResolution.Z;
	FRDGBufferRef PageToPageIndexBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), TotalPageCount), TEXT("Hair.PageToPageIndexBuffer"));

	FRDGBufferRef ReadBackBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1), TEXT("ReadBackAllocations"));

	const uint32 PageIndexBufferSizeInBytes = 4 * sizeof(uint32) * OutTotalPageIndexCount;
	const uint32 TotalPageIndexBufferSizeInBytes = PageIndexBufferSizeInBytes * PageIndexBufferSizeInBytes;

	FRDGBufferRef PageIndexResolutionBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(MacroGroupCount * 4 * sizeof(uint32), OutTotalPageIndexCount), TEXT("Hair.PageIndexResolutionBuffer"));
	FRDGBufferRef PageIndexAllocationIndirectBufferArgs = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(MacroGroupCount+1), TEXT("Hair.PageIndexAllocationIndirectBufferArgs"));
	
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PageIndexBuffer, PF_R32_UINT), 0u);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(PageIndexGlobalCounter, PF_R32_UINT), 0u);

	const FRDGBufferSRVRef CurrGPUMinVoxelSizeSRV = GraphBuilder.CreateSRV(CurrGPUMinVoxelSize, PF_R16F);
	const bool bAdaptiveVoxelEnable = IsHairStrandsAdaptiveVoxelAllocationEnable();

	// Allocate page index for all instance group
	{
		uint32 bDoesMacroGroupSupportVoxelization = 0;
		for (const FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
		{
			if (MacroGroup.bSupportVoxelization)
			{
				bDoesMacroGroupSupportVoxelization |= 1u << MacroGroup.MacroGroupId;
			}
		}

		FVoxelAllocatePageIndexCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVoxelAllocatePageIndexCS::FParameters>();
		Parameters->CPUPageWorldSize = CPUPageWorldSize;
		Parameters->CPUVoxelWorldSize = CPUMinVoxelWorldSize;
		Parameters->bUseCPUVoxelWorldSize = bIsGPUDriven && bAdaptiveVoxelEnable ? 0u : 1u;
		Parameters->TotalPageIndexCount = OutTotalPageIndexCount;
		Parameters->PageResolution = PageResolution;
		Parameters->MacroGroupCount = MacroGroupCount;
		Parameters->bDoesMacroGroupSupportVoxelization = bDoesMacroGroupSupportVoxelization;
		Parameters->MacroGroupVoxelSizeBuffer = GraphBuilder.CreateSRV(MacroGroupResources.MacroGroupVoxelSizeBuffer, PF_R16F);
		Parameters->MacroGroupAABBBuffer = GraphBuilder.CreateSRV(MacroGroupResources.MacroGroupAABBsBuffer, PF_R32_SINT);
		Parameters->MacroGroupVoxelAlignedAABBBuffer = GraphBuilder.CreateUAV(MacroGroupResources.MacroGroupVoxelAlignedAABBsBuffer, PF_R32_SINT);
		Parameters->IndirectDispatchGroupSize = FVoxelAllocateVoxelPageCS::GetGroupSize();
		Parameters->OutPageIndexResolutionAndOffsetBuffer = GraphBuilder.CreateUAV(PageIndexResolutionBuffer, PF_R32G32B32A32_UINT);
		Parameters->OutPageIndexAllocationIndirectBufferArgs = GraphBuilder.CreateUAV(PageIndexAllocationIndirectBufferArgs);
		Parameters->CPU_bUseCPUData = bIsGPUDriven ? 0 : 1;
		Parameters->GPUVoxelWorldSize = CurrGPUMinVoxelSizeSRV;

		for (int32 It=0; It < MAX_HAIR_MACROGROUP_COUNT; ++It)
		{
			if (It < CPUAllocationDescs.Num())
			{
				const FCPUMacroGroupAllocation& Desc = CPUAllocationDescs[It];
				Parameters->CPU_TranslatedWorldMinAABB[It] = FVector4f(Desc.TranslatedMinAABB.X, Desc.TranslatedMinAABB.Y, Desc.TranslatedMinAABB.Z, 0);
				Parameters->CPU_TranslatedWorldMaxAABB[It] = FVector4f(Desc.TranslatedMaxAABB.X, Desc.TranslatedMaxAABB.Y, Desc.TranslatedMaxAABB.Z, 0);
				Parameters->CPU_PageIndexResolution[It]    = FIntVector4(Desc.PageIndexResolution, 0.f);
			}
			else
			{
				Parameters->CPU_TranslatedWorldMinAABB[It] = FVector4f(0, 0, 0, 0);
				Parameters->CPU_TranslatedWorldMaxAABB[It] = FVector4f(0, 0, 0, 0);
				Parameters->CPU_PageIndexResolution[It]    = FIntVector4(0, 0, 0, 0);
			}
		}

		// Currently support only 32 instance group at max
		check(Parameters->MacroGroupCount <= FHairStrandsMacroGroupData::MaxMacroGroupCount);
		TShaderMapRef<FVoxelAllocatePageIndexCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HairStrands::AllocatePageIndex"),
			ComputeShader,
			Parameters,
			FIntVector(1,1,1));
	}
	FRDGBufferSRVRef PageIndexResolutionAndOffsetBufferSRV = GraphBuilder.CreateSRV(PageIndexResolutionBuffer, PF_R32G32B32A32_UINT);

	// Mark valid page index
	{
		FRDGBufferUAVRef PageIndexBufferUAVSkipBarrier = GraphBuilder.CreateUAV(PageIndexBuffer, PF_R32_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
		for (uint32 MacroGroupIt = 0; MacroGroupIt < MacroGroupCount; ++MacroGroupIt)
		{
			const FHairStrandsMacroGroupData& MacroGroup = MacroGroupDatas[MacroGroupIt];
			FCPUMacroGroupAllocation& CPUAllocationDesc = CPUAllocationDescs[MacroGroupIt];
			for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : MacroGroup.PrimitivesInfos)
			{
				FHairGroupPublicData* HairGroupData = PrimitiveInfo.PublicDataPtr;

				// Among the macro group, some primitive might not support/require voxelization. So skip them.
				if (!HairGroupData->DoesSupportVoxelization())
					continue;

				// For allocating/marking the used pages, a primitive/group can:
				// * Either use its group AABB + cluster AABB (for static groom)
				// * Or use its clusters AABBs (for dynamic groom)
				// * Or use its CPU AABB (for CLOD)
				// Even if the bGroupAABBValid is invalid, it has been reset to invalid AABB, so the code below won't mark any page as used.
				const bool bUseClusterAABB = TransientResources.IsClusterAABBValid(HairGroupData->Instance->RegisteredIndex) && bIsGPUDriven;
				
				FVoxelMarkValidPageIndex_PrepareCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVoxelMarkValidPageIndex_PrepareCS::FParameters>();
				Parameters->InstanceRegisteredIndex = HairGroupData->Instance->RegisteredIndex;
				Parameters->ClusterOffset = TransientResources.GetClusterOffset(HairGroupData->Instance->RegisteredIndex);
				Parameters->ClusterCount = TransientResources.GetClusterCount(HairGroupData->Instance->RegisteredIndex);
				Parameters->MacroGroupId = MacroGroup.MacroGroupId;
				Parameters->GroupAABBsBuffer = TransientResources.GroupAABBSRV;
				Parameters->ClusterAABBsBuffer = TransientResources.ClusterAABBSRV;
				Parameters->MacroGroupVoxelAlignedAABBBuffer = GraphBuilder.CreateSRV(MacroGroupResources.MacroGroupVoxelAlignedAABBsBuffer, PF_R32_SINT);
				Parameters->PageIndexResolutionAndOffsetBuffer = PageIndexResolutionAndOffsetBufferSRV;
				Parameters->bUseMacroGroupBoundCPU = bIsGPUDriven ? 0 : 1;
				Parameters->MacroGroupBoundCPU_TranslatedWorldMinAABB = (FVector3f)CPUAllocationDesc.TranslatedMinAABB;
				Parameters->MacroGroupBoundCPU_TranslatedWorldMaxAABB = (FVector3f)CPUAllocationDesc.TranslatedMaxAABB;
				Parameters->TranslatedWorldOffsetCorrection = TransientResources.GetTranslatedWorldOffsetCorrection(View);
				Parameters->OutValidPageIndexBuffer = PageIndexBufferUAVSkipBarrier;

				FIntVector DispatchCount = bUseClusterAABB ?
					FIntVector(
						FMath::DivideAndRoundUp(Parameters->ClusterCount, FVoxelMarkValidPageIndex_PrepareCS::GetGroupSize(true)),
						1,
						1) :
					FIntVector(
						FMath::DivideAndRoundUp(uint32(CPUAllocationDesc.PageIndexResolution.X), FVoxelMarkValidPageIndex_PrepareCS::GetGroupSize(false)),
						FMath::DivideAndRoundUp(uint32(CPUAllocationDesc.PageIndexResolution.Y), FVoxelMarkValidPageIndex_PrepareCS::GetGroupSize(false)),
						FMath::DivideAndRoundUp(uint32(CPUAllocationDesc.PageIndexResolution.Z), FVoxelMarkValidPageIndex_PrepareCS::GetGroupSize(false)));

				check(DispatchCount.X <= GRHIMaxDispatchThreadGroupsPerDimension.X);
				check(DispatchCount.Y <= GRHIMaxDispatchThreadGroupsPerDimension.Y);
				check(DispatchCount.Z <= GRHIMaxDispatchThreadGroupsPerDimension.Z);

				FVoxelMarkValidPageIndex_PrepareCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FVoxelMarkValidPageIndex_PrepareCS::FUseCluster>(bUseClusterAABB ? 1 : 0);
				TShaderMapRef<FVoxelMarkValidPageIndex_PrepareCS> ComputeShader(View.ShaderMap, PermutationVector);
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("HairStrands::MarkValidPageIndex(%s)", bUseClusterAABB ? TEXT("Cluster") : (bIsGPUDriven ? TEXT("Group") : TEXT("CPUGroup"))),
					ComputeShader,
					Parameters,
					DispatchCount);
			}
		}
	}

	// Fill in hair-macro-group information.
	{
		FRDGBufferUAVRef NodeDescUAVSkipBarrier = GraphBuilder.CreateUAV(NodeDescBuffer, ERDGUnorderedAccessViewFlags::SkipBarrier);
		for (uint32 MacroGroupIt = 0; MacroGroupIt < MacroGroupCount; ++MacroGroupIt)
		{
			const FHairStrandsMacroGroupData& MacroGroup = MacroGroupDatas[MacroGroupIt];
			FCPUMacroGroupAllocation& CPUAllocationDesc = CPUAllocationDescs[MacroGroupIt];
	
			// Note: This need to happen before the allocation as we copy the index global count. This global index is 
			// used as an offset, and thus refers to the previous pass
			{
				check(MacroGroup.MacroGroupId < MacroGroupCount);
	
				FVoxelAddNodeDescCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVoxelAddNodeDescCS::FParameters>();
				Parameters->MacroGroupId = MacroGroup.MacroGroupId;
				Parameters->CPU_TranslatedWorldMinAABB = (FVector3f)CPUAllocationDesc.TranslatedMinAABB;
				Parameters->CPU_TranslatedWorldMaxAABB = (FVector3f)CPUAllocationDesc.TranslatedMaxAABB;
				Parameters->CPU_PageIndexResolution = CPUAllocationDesc.PageIndexResolution;
				Parameters->CPU_PageIndexOffset = CPUAllocationDesc.PageIndexOffset;
				Parameters->CPU_VoxelWorldSize = CPUMinVoxelWorldSize;
				Parameters->GPU_VoxelWorldSize = CurrGPUMinVoxelSizeSRV;
				Parameters->bUseCPUVoxelWorldSize = bAdaptiveVoxelEnable ? 0u : 1u;
				Parameters->MacroGroupVoxelSizeBuffer = GraphBuilder.CreateSRV(MacroGroupResources.MacroGroupVoxelSizeBuffer, PF_R16F);
				Parameters->OutNodeDescBuffer = NodeDescUAVSkipBarrier;
	
				if (bIsGPUDriven)
				{
					Parameters->MacroGroupVoxelAlignedAABBBuffer = GraphBuilder.CreateSRV(MacroGroupResources.MacroGroupVoxelAlignedAABBsBuffer, PF_R32_SINT);
					Parameters->PageIndexResolutionAndOffsetBuffer = PageIndexResolutionAndOffsetBufferSRV;
				}
	
				FVoxelAddNodeDescCS::FPermutationDomain PermutationVector;
				PermutationVector.Set<FVoxelAddNodeDescCS::FGPUDriven>(bIsGPUDriven ? 1 : 0);
	
				const FIntVector DispatchCount(1, 1, 1);
				TShaderMapRef<FVoxelAddNodeDescCS> ComputeShader(View.ShaderMap, PermutationVector);
				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::AddNodeDesc"), ComputeShader, Parameters, DispatchCount);
			}
		}
	}

	// Allocate pages
	{
		FRDGBufferUAVRef PageIndexGlobalCounterUAVSkipBarrier = GraphBuilder.CreateUAV(PageIndexGlobalCounter, PF_R32_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
		FRDGBufferUAVRef PageToPageIndexUAVSkipBarrier = GraphBuilder.CreateUAV(PageToPageIndexBuffer, PF_R32_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
		FRDGBufferUAVRef PageIndexCoordUAVSkipBarrier = GraphBuilder.CreateUAV(PageIndexCoordBuffer, PF_R8G8B8A8_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
		FRDGBufferUAVRef PageIndexBufferUAVSkipBarrier = GraphBuilder.CreateUAV(PageIndexBuffer, PF_R32_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);

		for (uint32 MacroGroupIt = 0; MacroGroupIt < MacroGroupCount; ++MacroGroupIt)
		{
			const FHairStrandsMacroGroupData& MacroGroup = MacroGroupDatas[MacroGroupIt];
			FCPUMacroGroupAllocation& CPUAllocationDesc = CPUAllocationDescs[MacroGroupIt];

			FVoxelAllocateVoxelPageCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVoxelAllocateVoxelPageCS::FParameters>();
			Parameters->MacroGroupId = MacroGroup.MacroGroupId;
			Parameters->PageCount = PageCount;
			Parameters->CPU_PageIndexCount = CPUAllocationDesc.PageIndexCount;
			Parameters->CPU_PageIndexResolution = CPUAllocationDesc.PageIndexResolution;
			Parameters->CPU_PageIndexOffset = CPUAllocationDesc.PageIndexOffset;
			Parameters->RWPageIndexGlobalCounter = PageIndexGlobalCounterUAVSkipBarrier;
			Parameters->RWPageIndexBuffer = PageIndexBufferUAVSkipBarrier;
			Parameters->RWPageToPageIndexBuffer = PageToPageIndexUAVSkipBarrier;
			Parameters->RWPageIndexCoordBuffer = PageIndexCoordUAVSkipBarrier;

			FVoxelAllocateVoxelPageCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FVoxelAllocateVoxelPageCS::FGPUDriven>(bIsGPUDriven ? 1 : 0);
			TShaderMapRef<FVoxelAllocateVoxelPageCS> ComputeShader(View.ShaderMap, PermutationVector);

			if (bIsGPUDriven)
			{
				Parameters->PageIndexResolutionAndOffsetBuffer = PageIndexResolutionAndOffsetBufferSRV;
				Parameters->IndirectBufferArgs = PageIndexAllocationIndirectBufferArgs;

				const uint32 ArgsOffset = sizeof(FRHIDispatchIndirectParameters) * MacroGroup.MacroGroupId;

				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::AllocateVoxelPage"), ComputeShader, Parameters,
					PageIndexAllocationIndirectBufferArgs,
					ArgsOffset);
			}
			else
			{
				const FIntVector DispatchCount(FMath::DivideAndRoundUp(CPUAllocationDesc.PageIndexCount, FVoxelAllocateVoxelPageCS::GetGroupSize()), 1, 1);
				check(DispatchCount.X <= GRHIMaxDispatchThreadGroupsPerDimension.X);
				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::AllocateVoxelPage"), ComputeShader, Parameters, DispatchCount);
			}
		}
	}

	// Prepare indirect dispatch buffers for all pages, and for each macro group
	{
		FVoxelAddIndirectBufferCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVoxelAddIndirectBufferCS::FParameters>();
		Parameters->MacroGroupCount = MacroGroupCount;
		Parameters->PageResolution = PageResolution;
		Parameters->IndirectGroupSize = FVirtualVoxelInjectOpaqueCS::GetGroupSize();
		Parameters->PageIndexGlobalCounter = GraphBuilder.CreateSRV(PageIndexGlobalCounter, PF_R32_UINT);
		Parameters->OutIndirectArgsBuffer = GraphBuilder.CreateUAV(IndirectArgsBuffer);

		TShaderMapRef<FVoxelAddIndirectBufferCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::BuildVoxelIndirectArgs"), ComputeShader, Parameters, FIntVector(FMath::DivideAndRoundUp(MacroGroupCount, FVoxelAddIndirectBufferCS::GetGroupSize()), 1, 1));
	}

	// Feedback allocation for next frame
	if (bAdaptiveVoxelEnable)
	{
		// Store the total requested page allocation (for feedback purpose)
		const EPixelFormat Format = PF_R16F;
		NextGPUMinVoxelSize = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FFloat16), 1), TEXT("Hair.VoxelPageAllocationFeedbackBuffer"));

		FVoxelAdaptiveFeedbackCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVoxelAdaptiveFeedbackCS::FParameters>();
		Parameters->CPUMinVoxelWorldSize = CPUMinVoxelWorldSize;
		Parameters->CPUAllocatedPageCount = TotalPageCount;
		Parameters->AdaptiveCorrectionThreshold = FMath::Clamp(GHairVirtualVoxelAdaptive_CorrectionThreshold, 0.f, 1.f);
		Parameters->AdaptiveCorrectionSpeed = FMath::Clamp(GHairVirtualVoxelAdaptive_CorrectionSpeed, 0.f, 1.f);
		Parameters->PageIndexGlobalCounter = GraphBuilder.CreateSRV(PageIndexGlobalCounter, PF_R32_UINT);
		Parameters->CurrGPUMinVoxelWorldSize = CurrGPUMinVoxelSizeSRV;
		Parameters->NextGPUMinVoxelWorldSize = GraphBuilder.CreateUAV(NextGPUMinVoxelSize, Format);
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintUniformBuffer);

		TShaderMapRef<FVoxelAdaptiveFeedbackCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::RunVoxelSizeFeedback"), ComputeShader, Parameters, FIntVector(1, 1, 1));
	}

	OutPageIndexBuffer = PageIndexBuffer;
	OutPageToPageIndexBuffer = PageToPageIndexBuffer;
	OutPageIndexCoordBuffer = PageIndexCoordBuffer;
	OutNodeDescBuffer = NodeDescBuffer;
	OutIndirectArgsBuffer = IndirectArgsBuffer;
	OutPageIndexGlobalCounter = PageIndexGlobalCounter;
}

static float RoundHairVoxeliSize(float In)
{
	// Round voxel size to 0.01f to avoid oscillation issue
	return FMath::RoundToFloat(In * 1000.f) * 0.001f;
}

static FHairStrandsVoxelResources AllocateVirtualVoxelResources(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FVector& PreViewStereoCorrection,
	FHairTransientResources& TransientResources,
	FHairStrandsMacroGroupDatas& MacroGroupDatas,
	FHairStrandsMacroGroupResources& MacroGroupResources,
	FRDGBufferRef& PageToPageIndexBuffer,
	FHairStrandsViewStateData* OutViewStateData)
{
	DECLARE_GPU_STAT(HairStrandsVoxelPageAllocation);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsVoxelPageAllocation, "HairStrandsVoxelPageAllocation");

	// Init. default page table size and voxel size
	FIntVector PageCountResolution = FIntVector(GHairVirtualVoxel_PageCountPerDim, GHairVirtualVoxel_PageCountPerDim, GHairVirtualVoxel_PageCountPerDim);
	uint32 PageCount = PageCountResolution.X * PageCountResolution.Y * PageCountResolution.Z;
	const float CPUMinVoxelWorldSize = RoundHairVoxeliSize(FMath::Clamp(GHairVirtualVoxel_VoxelWorldSize, HAIR_MIN_VOXEL_SIZE, HAIR_MAX_VOXEL_SIZE));
	const bool bVoxelAllocationFeedbackEnable = OutViewStateData && IsHairStrandsAdaptiveVoxelAllocationEnable();

	FHairStrandsVoxelResources Out;

	GetVoxelPageResolution(Out.Parameters.Common.PageResolution, Out.Parameters.Common.PageResolutionLog2);

	Out.Parameters.Common.PageCountResolution		= PageCountResolution;
	Out.Parameters.Common.PageCount					= PageCount;
	Out.Parameters.Common.CPUMinVoxelWorldSize		= CPUMinVoxelWorldSize;
	Out.Parameters.Common.PageTextureResolution		= Out.Parameters.Common.PageCountResolution * Out.Parameters.Common.PageResolution;
	Out.Parameters.Common.JitterMode				= FMath::Clamp(GHairVirtualVoxel_JitterMode, 0, 2);

	Out.Parameters.Common.DensityScale				= GetHairStrandsVoxelizationDensityScale();
	Out.Parameters.Common.DensityScale_AO			= GetHairStrandsVoxelizationDensityScale_AO();
	Out.Parameters.Common.DensityScale_Shadow		= GetHairStrandsVoxelizationDensityScale_Shadow();
	Out.Parameters.Common.DensityScale_Transmittance= GetHairStrandsVoxelizationDensityScale_Transmittance();
	Out.Parameters.Common.DensityScale_Environment	= GetHairStrandsVoxelizationDensityScale_Environment();
	Out.Parameters.Common.DensityScale_Raytracing   = GetHairStrandsVoxelizationDensityScale_Raytracing();	

	Out.Parameters.Common.DepthBiasScale_Shadow		= GetHairStrandsVoxelizationDepthBiasScale_Shadow();
	Out.Parameters.Common.DepthBiasScale_Transmittance= GetHairStrandsVoxelizationDepthBiasScale_Transmittance();
	Out.Parameters.Common.DepthBiasScale_Environment= GetHairStrandsVoxelizationDepthBiasScale_Environment();

	Out.Parameters.Common.SteppingScale_Shadow		  = FMath::Clamp(GHairStransVoxelRaymarchingSteppingScale_Shadow			>= 0 ? GHairStransVoxelRaymarchingSteppingScale_Shadow			: GHairStransVoxelRaymarchingSteppingScale, 1.f, 10.f);
	Out.Parameters.Common.SteppingScale_Transmittance = FMath::Clamp(GHairStransVoxelRaymarchingSteppingScale_Transmittance		>= 0 ? GHairStransVoxelRaymarchingSteppingScale_Transmittance	: GHairStransVoxelRaymarchingSteppingScale, 1.f, 10.f);
	Out.Parameters.Common.SteppingScale_Environment	  = FMath::Clamp(GHairStransVoxelRaymarchingSteppingScale_Environment		>= 0 ? GHairStransVoxelRaymarchingSteppingScale_Environment		: GHairStransVoxelRaymarchingSteppingScale, 1.f, 10.f);
	Out.Parameters.Common.SteppingScale_Raytracing	  = FMath::Clamp(GHairStransVoxelRaymarchingSteppingScale_Raytracing		>= 0 ? GHairStransVoxelRaymarchingSteppingScale_Raytracing		: GHairStransVoxelRaymarchingSteppingScale, 1.f, 10.f);

	Out.Parameters.Common.NodeDescCount							= MacroGroupDatas.Num();
	Out.Parameters.Common.Raytracing_ShadowOcclusionThreshold	= FMath::Max(0.f, GHairVirtualVoxelRaytracing_ShadowOcclusionThreshold);
	Out.Parameters.Common.Raytracing_SkyOcclusionThreshold		= FMath::Max(0.f, GHairVirtualVoxelRaytracing_SkyOcclusionThreshold);
	Out.Parameters.Common.HairCoveragePixelRadiusAtDepth1		= ComputeMinStrandRadiusAtDepth1(FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()), View.FOV, 1/*SampleCount*/, 1/*RasterizationScale*/, View.ViewMatrices.GetOrthoDimensions().X, View.Family->FinalResolutionScale).Primary;
	Out.Parameters.Common.TranslatedWorldOffset					= FVector3f(View.ViewMatrices.GetPreViewTranslation());
	Out.Parameters.Common.TranslatedWorldOffsetStereoCorrection = FVector3f(PreViewStereoCorrection);

	// For debug purpose
	Out.Parameters.Common.AllocationFeedbackEnable = bVoxelAllocationFeedbackEnable ? 1u : 0u;

	FRDGBufferRef CurrGPUMinVoxelSize = nullptr;
	FRDGBufferRef NextGPUMinVoxelSize = nullptr;
	if (bVoxelAllocationFeedbackEnable && OutViewStateData->VoxelFeedbackBuffer.IsValid())
	{
		CurrGPUMinVoxelSize = GraphBuilder.RegisterExternalBuffer(OutViewStateData->VoxelFeedbackBuffer);
	}
	if (CurrGPUMinVoxelSize == nullptr) { CurrGPUMinVoxelSize = GSystemTextures.GetDefaultBuffer(GraphBuilder, 2, FFloat16(CPUMinVoxelWorldSize)); }

	AddAllocateVoxelPagesPass(
		GraphBuilder, 
		View, 
		TransientResources,
		MacroGroupDatas,
		MacroGroupResources,
		Out.Parameters.Common.PageCountResolution,
		Out.Parameters.Common.PageCount,
		Out.Parameters.Common.CPUMinVoxelWorldSize,
		Out.Parameters.Common.PageResolution,
		Out.Parameters.Common.PageTextureResolution,
		Out.Parameters.Common.PageIndexCount,
		Out.PageIndexBuffer,
		PageToPageIndexBuffer,
		Out.PageIndexCoordBuffer,
		Out.NodeDescBuffer,
		Out.IndirectArgsBuffer,
		Out.PageIndexGlobalCounter,
		CurrGPUMinVoxelSize,
		NextGPUMinVoxelSize);

	if (bVoxelAllocationFeedbackEnable && NextGPUMinVoxelSize)
	{
		GraphBuilder.QueueBufferExtraction(NextGPUMinVoxelSize, &OutViewStateData->VoxelFeedbackBuffer);
	}

	{
		// Allocation should be conservative
		// TODO: do a partial clear with indirect call: we know how many texture page will be touched, so we know how much thread we need to launch to clear what is relevant
		check(FMath::IsPowerOfTwo(Out.Parameters.Common.PageResolution));
		const uint32 MipCount = FMath::Log2(static_cast<float>(Out.Parameters.Common.PageResolution)) + 1;

		FRDGTextureDesc Desc = FRDGTextureDesc::Create3D(
			Out.Parameters.Common.PageTextureResolution,
			PF_R32_UINT, 
			FClearValueBinding::Black, 
			TexCreate_UAV | TexCreate_ShaderResource | TexCreate_NoFastClear | TexCreate_3DTiling | TexCreate_AtomicCompatible,
			MipCount);
		Out.PageTexture = GraphBuilder.CreateTexture(Desc, TEXT("Hair.VoxelPageTexture"));
	}

	Out.Parameters.Common.PageIndexBuffer			= GraphBuilder.CreateSRV(Out.PageIndexBuffer, PF_R32_UINT);
	Out.Parameters.Common.PageIndexCoordBuffer		= GraphBuilder.CreateSRV(Out.PageIndexCoordBuffer, PF_R8G8B8A8_UINT);
	Out.Parameters.Common.NodeDescBuffer			= GraphBuilder.CreateSRV(Out.NodeDescBuffer); 
	Out.Parameters.Common.AllocatedPageCountBuffer  = GraphBuilder.CreateSRV(Out.PageIndexGlobalCounter, PF_R32_UINT);

	Out.Parameters.Common.CurrGPUMinVoxelSize		= GraphBuilder.CreateSRV(CurrGPUMinVoxelSize, PF_R16F);
	Out.Parameters.Common.NextGPUMinVoxelSize		= GraphBuilder.CreateSRV(NextGPUMinVoxelSize ? NextGPUMinVoxelSize : CurrGPUMinVoxelSize, PF_R16F);

	Out.Parameters.PageTexture						= Out.PageTexture;

	if (Out.PageIndexBuffer && Out.NodeDescBuffer)
	{
		FVirtualVoxelParameters* Parameters = GraphBuilder.AllocParameters<FVirtualVoxelParameters>();
		*Parameters = Out.Parameters;
		Out.UniformBuffer = GraphBuilder.CreateUniformBuffer(Parameters);
	}

	return Out;
}

static FHairStrandsVoxelResources AllocateDummyVirtualVoxelResources(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FHairStrandsMacroGroupDatas& MacroGroupDatas)
{
	FHairStrandsVoxelResources Out;

	Out.Parameters.Common.PageCountResolution		= FIntVector(0,0,0);
	Out.Parameters.Common.PageCount					= 0;
	Out.Parameters.Common.CPUMinVoxelWorldSize		= 1;
	Out.Parameters.Common.PageResolution			= 1;
	Out.Parameters.Common.PageTextureResolution		= FIntVector(1, 1, 1);

	Out.Parameters.Common.DensityScale				= 0;
	Out.Parameters.Common.DensityScale_AO			= 0;
	Out.Parameters.Common.DensityScale_Shadow		= 0;
	Out.Parameters.Common.DensityScale_Transmittance= 0;
	Out.Parameters.Common.DensityScale_Environment	= 0;
	Out.Parameters.Common.DensityScale_Raytracing   = 0;

	Out.Parameters.Common.DepthBiasScale_Shadow			= 0;
	Out.Parameters.Common.DepthBiasScale_Transmittance	= 0;
	Out.Parameters.Common.DepthBiasScale_Environment	= 0;

	Out.Parameters.Common.SteppingScale_Shadow		  = 1;
	Out.Parameters.Common.SteppingScale_Transmittance = 1;
	Out.Parameters.Common.SteppingScale_Environment	  = 1;
	Out.Parameters.Common.SteppingScale_Raytracing	  = 1;

	Out.Parameters.Common.NodeDescCount				= 0;
	Out.Parameters.Common.Raytracing_ShadowOcclusionThreshold	= 1;
	Out.Parameters.Common.Raytracing_SkyOcclusionThreshold		= 1;

	Out.Parameters.Common.TranslatedWorldOffset = FVector3f(View.ViewMatrices.GetPreViewTranslation());
	Out.Parameters.Common.HairCoveragePixelRadiusAtDepth1	= ComputeMinStrandRadiusAtDepth1(FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()), View.FOV, 1/*SampleCount*/, 1/*RasterizationScale*/, View.ViewMatrices.GetOrthoDimensions().X, View.Family->FinalResolutionScale).Primary;
	Out.Parameters.Common.PageIndexCount = 0;

	FRDGBufferRef Dummy2BytesBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, 2, 0u);
	FRDGBufferRef Dummy4BytesBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, 4, 0u);
	FRDGBufferRef DummyStructBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FPackedVirtualVoxelNodeDesc), 0u);

	Out.PageIndexBuffer				= Dummy4BytesBuffer;
	Out.PageIndexCoordBuffer		= Dummy4BytesBuffer;
	Out.NodeDescBuffer				= DummyStructBuffer;
	Out.IndirectArgsBuffer			= Dummy4BytesBuffer;
	Out.PageIndexGlobalCounter		= Dummy4BytesBuffer;
	Out.PageTexture					= GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture3D, PF_R32_UINT, 0u);

	Out.Parameters.Common.PageIndexBuffer			= GraphBuilder.CreateSRV(Out.PageIndexBuffer, PF_R32_UINT);
	Out.Parameters.Common.PageIndexCoordBuffer		= GraphBuilder.CreateSRV(Out.PageIndexCoordBuffer, PF_R8G8B8A8_UINT);
	Out.Parameters.Common.NodeDescBuffer			= GraphBuilder.CreateSRV(Out.NodeDescBuffer); 
	Out.Parameters.Common.AllocatedPageCountBuffer  = GraphBuilder.CreateSRV(Out.PageIndexGlobalCounter, PF_R32_UINT);
	Out.Parameters.Common.CurrGPUMinVoxelSize		= GraphBuilder.CreateSRV(Dummy2BytesBuffer, PF_R16F);
	Out.Parameters.Common.NextGPUMinVoxelSize		= Out.Parameters.Common.CurrGPUMinVoxelSize;
	Out.Parameters.PageTexture						= Out.PageTexture;
	
	if (Out.PageIndexBuffer && Out.NodeDescBuffer)
	{
		FVirtualVoxelParameters* Parameters = GraphBuilder.AllocParameters<FVirtualVoxelParameters>();
		*Parameters = Out.Parameters;
		Out.UniformBuffer = GraphBuilder.CreateUniformBuffer(Parameters);
	}

	return Out;
}

static void IndirectVoxelPageClear(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& ViewInfo,
	FHairStrandsVoxelResources& VoxelResources)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsIndVoxelPageClear, "HairStrandsIndVoxelPageClear");

	FVoxelIndPageClearCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVoxelIndPageClearCS::FParameters>();
	Parameters->VirtualVoxelParams = VoxelResources.Parameters.Common;
	Parameters->OutPageTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(VoxelResources.PageTexture));
	Parameters->IndirectDispatchBuffer = VoxelResources.IndirectArgsBuffer;
	Parameters->PageIndexGlobalCounter = GraphBuilder.CreateSRV(VoxelResources.PageIndexGlobalCounter, PF_R32_UINT);

	TShaderMapRef<FVoxelIndPageClearCS> ComputeShader(ViewInfo.ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("HairStrands::VoxelIndPageClearCS"),
		ComputeShader,
		Parameters,
		Parameters->IndirectDispatchBuffer,
		0);
}

///////////////////////////////////////////////////////////////////////////////////////////////////

class FVoxelRasterComputeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVoxelRasterComputeCS);
	SHADER_USE_PARAMETER_STRUCT(FVoxelRasterComputeCS, FGlobalShader);

	class FCulling : SHADER_PERMUTATION_BOOL("PERMUTATION_CULLING");
	using FPermutationDomain = TShaderPermutationDomain<FCulling>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FHairStrandsVoxelCommonParameters, VirtualVoxelParams)
		SHADER_PARAMETER(uint32, MacroGroupId)
		SHADER_PARAMETER(uint32, MaxRasterCount)
		SHADER_PARAMETER(uint32, FrameIdMod8)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHairStrandsInstanceParameters, HairInstance)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, OutPageTexture)
		SHADER_PARAMETER(uint32, VertexCount)
		SHADER_PARAMETER(float, CoverageScale)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return HAIR_VERTEXCOUNT_GROUP_SIZE; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RASTERCOMPUTE"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FVoxelRasterComputeCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "MainCS", SF_Compute);

bool IsHairStrandContinuousDecimationReorderingEnabled();

struct FInstanceData
{
	uint32 MacroGroupId = 0;
	const FHairGroupPublicData* Data = nullptr;
};

static void AddVirtualVoxelizationRasterPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo* ViewInfo,
	const FHairStrandsVoxelResources& VoxelResources,
	const TArray<FInstanceData>& InstanceDatas)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsVoxelize, "HairStrandsVoxelize");

	if (ViewInfo)
	{
		FRDGTextureUAVRef PageTextureUAV = GraphBuilder.CreateUAV(VoxelResources.PageTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);

		const uint32 FrameIdMode8 = ViewInfo->ViewState ? (ViewInfo->ViewState->GetFrameIndex() % 8) : 0;
		const FVector& TranslatedWorldOffset = ViewInfo->ViewMatrices.GetPreViewTranslation();

		FVoxelRasterComputeCS::FPermutationDomain PermutationVector_Off;
		FVoxelRasterComputeCS::FPermutationDomain PermutationVector_On;
		PermutationVector_Off.Set<FVoxelRasterComputeCS::FCulling>(false);
		PermutationVector_On.Set<FVoxelRasterComputeCS::FCulling>(true);

		TShaderMapRef<FVoxelRasterComputeCS> ComputeShader_CullingOff(ViewInfo->ShaderMap, PermutationVector_Off);
		TShaderMapRef<FVoxelRasterComputeCS> ComputeShader_CullingOn(ViewInfo->ShaderMap, PermutationVector_On);
		for (const FInstanceData& InstanceData : InstanceDatas)
		{
			const uint32 PointCount = InstanceData.Data->GetActiveStrandsPointCount();
			const float CoverageScale = InstanceData.Data->GetActiveStrandsCoverageScale();

			FVoxelRasterComputeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVoxelRasterComputeCS::FParameters>();
			PassParameters->MaxRasterCount = FMath::Clamp(GHairStrandsVoxelComputeRasterMaxVoxelCount, 1, 256);
			PassParameters->VirtualVoxelParams = VoxelResources.Parameters.Common;
			PassParameters->MacroGroupId = InstanceData.MacroGroupId;
			PassParameters->OutPageTexture = PageTextureUAV;
			PassParameters->FrameIdMod8 = FrameIdMode8;
			PassParameters->VertexCount = PointCount;
			PassParameters->CoverageScale = CoverageScale;
		
			const bool bCullingEnable = InstanceData.Data->GetCullingResultAvailable();

			PassParameters->HairInstance = GetHairStrandsInstanceParameters(GraphBuilder, *ViewInfo, InstanceData.Data, bCullingEnable, true);

			if (bCullingEnable)
			{
				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VoxelComputeRaster(culling=on)"), ComputeShader_CullingOn, PassParameters, PassParameters->HairInstance.HairStrandsVF.Culling.CullingIndirectBufferArgs, 0);
			}
			else
			{
				const FIntVector DispatchCount(FMath::DivideAndRoundUp(PassParameters->VertexCount, FVoxelRasterComputeCS::GetGroupSize()), 1, 1);
				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::VoxelComputeRaster(culling=off)"), ComputeShader_CullingOff, PassParameters, DispatchCount);
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
class FVirtualVoxelGenerateMipCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVirtualVoxelGenerateMipCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualVoxelGenerateMipCS, FGlobalShader);
	
	class FAggregate : SHADER_PERMUTATION_BOOL("PERMUTATION_MIP_AGGREGATE");
	using FPermutationDomain = TShaderPermutationDomain<FAggregate>;	

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, AllocatedPageCountBuffer)
		SHADER_PARAMETER(FIntVector, PageCountResolution)
		SHADER_PARAMETER(uint32, PageResolution)
		SHADER_PARAMETER(uint32, SourceMip)
		SHADER_PARAMETER(uint32, TargetMip)
		SHADER_PARAMETER(uint32, bPatchEmptyPage)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PageToPageIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutPageIndexBuffer)

		RDG_BUFFER_ACCESS(IndirectDispatchArgs, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, InDensityTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, OutDensityTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, OutDensityTexture2)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D, OutDensityTexture1)

	END_SHADER_PARAMETER_STRUCT()

public:
	static FIntVector GetGroupSize() { return GHairVoxel_GroupSize_NumVoxelPerPage_1_NumAllocatedPage; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_MIP_VIRTUALVOXEL"), 1);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE_X"), GetGroupSize().X);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE_Y"), GetGroupSize().Y);
		OutEnvironment.SetDefine(TEXT("GROUP_SIZE_Z"), GetGroupSize().Z);
	}
};

class FVirtualVoxelIndirectArgMipCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVirtualVoxelIndirectArgMipCS);
	SHADER_USE_PARAMETER_STRUCT(FVirtualVoxelIndirectArgMipCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, PageResolution)
		SHADER_PARAMETER(uint32, TargetMipIndex)
		SHADER_PARAMETER(FIntVector, DispatchGroupSize)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, InIndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer, OutIndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_MIP_INDIRECTARGS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVirtualVoxelGenerateMipCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FVirtualVoxelIndirectArgMipCS, "/Engine/Private/HairStrands/HairStrandsVoxelRasterCompute.usf", "MainCS", SF_Compute);


static void AddVirtualVoxelGenerateMipPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FHairStrandsVoxelResources& VoxelResources,
	FRDGBufferRef InPageToPageIndexBuffer)
{
	if (!VoxelResources.IsValid())
		return;

	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsDensityMipGen, "HairStrandsDensityMipGen");

	const uint32 MipCount = VoxelResources.PageTexture->Desc.NumMips;

	// Prepare indirect dispatch for all the pages this frame (allocated linearly in 3D DensityTexture)
	TArray<FRDGBufferRef> MipIndirectArgsBuffers;
	for (uint32 MipIt = 0; MipIt < MipCount - 1; ++MipIt)
	{
		const uint32 TargetMipIndex = MipIt + 1;
		FRDGBufferRef MipIndirectArgs = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(), TEXT("Hair.VirtualVoxelMipIndirectArgsBuffer"));
		MipIndirectArgsBuffers.Add(MipIndirectArgs);

		FVirtualVoxelIndirectArgMipCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVirtualVoxelIndirectArgMipCS::FParameters>();
		Parameters->PageResolution		= VoxelResources.Parameters.Common.PageResolution;
		Parameters->TargetMipIndex		= TargetMipIndex;
		Parameters->DispatchGroupSize	= FVirtualVoxelGenerateMipCS::GetGroupSize();
		Parameters->InIndirectArgs		= GraphBuilder.CreateSRV(VoxelResources.IndirectArgsBuffer);
		Parameters->OutIndirectArgs		= GraphBuilder.CreateUAV(MipIndirectArgs);

		TShaderMapRef<FVirtualVoxelIndirectArgMipCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("HairStrands::BuildVoxelMipIndirectArgs"), ComputeShader, Parameters, FIntVector(1, 1, 1));
	}

	// Patch the page index buffer with page whose voxels are empty after the voxelization is done
	const bool bAMDPC = IsPCPlatform(View.GetShaderPlatform()) && IsRHIDeviceAMD();
	const bool bPatchEmptyPage = GHairVirtualVoxelInvalidEmptyPageIndex > 0 && !bAMDPC;
	FRDGBufferSRVRef PageToPageIndexBufferSRV = GraphBuilder.CreateSRV(InPageToPageIndexBuffer, PF_R32_UINT);
	FRDGBufferUAVRef PageIndexBufferUAV = GraphBuilder.CreateUAV(VoxelResources.PageIndexBuffer, PF_R32_UINT);

	// Generate MIP level (in one go for all allocated pages)
	const uint32 LastMipMinus3 = MipCount - 3;
	for (uint32 MipIt = 0; MipIt < LastMipMinus3; ++MipIt)
	{
		const uint32 SourceMipIndex = MipIt;
		const uint32 TargetMipIndex = MipIt + 1;

		FVirtualVoxelGenerateMipCS::FParameters* Parameters = GraphBuilder.AllocParameters<FVirtualVoxelGenerateMipCS::FParameters>();
		Parameters->InDensityTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(VoxelResources.PageTexture, MipIt));
		Parameters->OutDensityTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(VoxelResources.PageTexture, MipIt + 1));
		Parameters->PageResolution = VoxelResources.Parameters.Common.PageResolution;
		Parameters->PageCountResolution = VoxelResources.Parameters.Common.PageCountResolution;
		Parameters->SourceMip = SourceMipIndex;
		Parameters->TargetMip = TargetMipIndex;
		Parameters->AllocatedPageCountBuffer = VoxelResources.Parameters.Common.AllocatedPageCountBuffer;
		Parameters->IndirectDispatchArgs = MipIndirectArgsBuffers[MipIt];
		Parameters->bPatchEmptyPage = bPatchEmptyPage;

		const uint32 TargetPageResolution = Parameters->PageResolution >> TargetMipIndex;
		const bool bAggregate = TargetPageResolution <= 4;
		if (bAggregate)
		{
			Parameters->PageToPageIndexBuffer = PageToPageIndexBufferSRV;
			Parameters->OutPageIndexBuffer = PageIndexBufferUAV;

			Parameters->OutDensityTexture2 = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(VoxelResources.PageTexture, MipCount-2));
			Parameters->OutDensityTexture1 = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(VoxelResources.PageTexture, MipCount-1));
		}

		FVirtualVoxelGenerateMipCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FVirtualVoxelGenerateMipCS::FAggregate>(bAggregate);
		TShaderMapRef<FVirtualVoxelGenerateMipCS> ComputeShader(View.ShaderMap, PermutationVector);
		ClearUnusedGraphResources(ComputeShader, Parameters);
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HairStrands::ComputeVoxelMip"),
			Parameters,
			ERDGPassFlags::Compute,
			[Parameters, ComputeShader](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FComputeShaderUtils::DispatchIndirect(RHICmdList, ComputeShader, *Parameters, Parameters->IndirectDispatchArgs->GetIndirectRHICallBuffer(), 0);
		});
	}
}
/////////////////////////////////////////////////////////////////////////////////////////

void VoxelizeHairStrands(
	FRDGBuilder& GraphBuilder, 
	const FScene* Scene, 
	FViewInfo& View,
	FInstanceCullingManager& InstanceCullingManager,
	const FVector& PreViewStereoCorrection)
{	
	FHairStrandsMacroGroupDatas& MacroGroupDatas = View.HairStrandsViewData.MacroGroupDatas;
	FHairStrandsVoxelResources& VirtualVoxelResources = View.HairStrandsViewData.VirtualVoxelResources;
	FHairStrandsMacroGroupResources& MacroGroupResources = View.HairStrandsViewData.MacroGroupResources;
	FHairTransientResources* TransientResources = Scene->HairStrandsSceneData.TransientResources;

	// Simple early out to check if voxelization is needed
	if (!IsHairStrandsVoxelizationEnable(View.GetShaderPlatform()) || MacroGroupDatas.Num() == 0)
	{
		VirtualVoxelResources = AllocateDummyVirtualVoxelResources(GraphBuilder, View, MacroGroupDatas);
		return;
	}

	// Detailed early out to check if voxelization is needed
	bool bHasValidElementToVoxelize = false;
	for (const FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
	{
		if (MacroGroup.bSupportVoxelization)
		{
			bHasValidElementToVoxelize = true;
			break;
		}
	}

	if (!bHasValidElementToVoxelize)
	{
		VirtualVoxelResources = AllocateDummyVirtualVoxelResources(GraphBuilder, View, MacroGroupDatas);
		return;
	}

	DECLARE_GPU_STAT(HairStrandsVoxelization);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HairStrandsVoxelization, "HairStrandsVoxelization");

	check(TransientResources);

	// Gather all instances which needs to be voxelized
	uint32 TotalInstanceCount = 0;
	TArray<FInstanceData> InstanceDatas;
	InstanceDatas.Reserve(Scene->HairStrandsSceneData.RegisteredProxies.Num());
	TArray<uint32> ValidMacroGroupIDs;
	ValidMacroGroupIDs.Reserve(MacroGroupDatas.Num());
	TArray<FUintVector2> MacroGroupInstanceOffsetAndCount;
	MacroGroupInstanceOffsetAndCount.Init(FUintVector2::ZeroValue, MacroGroupDatas.Num());
	for (FHairStrandsMacroGroupData& MacroGroup : MacroGroupDatas)
	{
		if (MacroGroup.bSupportVoxelization)
		{
			ValidMacroGroupIDs.Add(MacroGroup.MacroGroupId);
			MacroGroupInstanceOffsetAndCount[MacroGroup.MacroGroupId].X = TotalInstanceCount;
			for (const FHairStrandsMacroGroupData::PrimitiveInfo& PrimitiveInfo : MacroGroup.PrimitivesInfos)
			{
				check(PrimitiveInfo.PublicDataPtr);
				const FHairGroupPublicData* HairGroupPublicData = PrimitiveInfo.PublicDataPtr;
				const FHairGroupPublicData::FVertexFactoryInput& VFInput = HairGroupPublicData->VFInput;
				if (HairGroupPublicData->VFInput.Strands.PositionBuffer.Buffer == nullptr)
				{
					continue;
				}

				if (HairGroupPublicData->DoesSupportVoxelization())
				{
					FInstanceData& InstanceData = InstanceDatas.AddDefaulted_GetRef();
					InstanceData.MacroGroupId = MacroGroup.MacroGroupId;
					InstanceData.Data = HairGroupPublicData;
					++TotalInstanceCount;
					++MacroGroupInstanceOffsetAndCount[MacroGroup.MacroGroupId].Y;
				}
			}
		}
	}

	// Force transition to ensure passes to be batched
	TArray<FRDGBufferSRVRef> Transitions;
	Transitions.Reserve(InstanceDatas.Num() * 3u);
	for (const FInstanceData& InstanceData : InstanceDatas)
	{
		if (InstanceData.Data->GetCullingResultAvailable())
		{
			Transitions.Add(RegisterAsSRV(GraphBuilder, InstanceData.Data->GetDrawIndirectRasterComputeBuffer()));
			Transitions.Add(RegisterAsSRV(GraphBuilder, InstanceData.Data->GetCulledVertexIdBuffer()));
		}
	}
	AddTransitionPass(GraphBuilder, View.ShaderMap, View.GetShaderPlatform(), Transitions);

	FRDGBufferRef PageToPageIndexBuffer = nullptr;
	FHairStrandsViewStateData* HairStrandsViewStateData = View.ViewState ? &View.ViewState->HairStrandsViewStateData : nullptr;
	VirtualVoxelResources = AllocateVirtualVoxelResources(GraphBuilder, View, PreViewStereoCorrection, *TransientResources, MacroGroupDatas, MacroGroupResources, PageToPageIndexBuffer, HairStrandsViewStateData);

	IndirectVoxelPageClear(GraphBuilder, View, VirtualVoxelResources);
	AddVirtualVoxelizationRasterPass(GraphBuilder, &View, VirtualVoxelResources, InstanceDatas);

	if (GHairVoxelInjectOpaqueDepthEnable > 0)
	{
		FRDGTextureUAVRef PageTextureUAV = GraphBuilder.CreateUAV(VirtualVoxelResources.PageTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		for (uint32 MacroGroupId : ValidMacroGroupIDs)
		{
			AddVirtualVoxelInjectOpaquePass(GraphBuilder, View, VirtualVoxelResources, MacroGroupId, PageTextureUAV);
		}
	}

	AddVirtualVoxelGenerateMipPass(GraphBuilder, View, VirtualVoxelResources, PageToPageIndexBuffer);
}
