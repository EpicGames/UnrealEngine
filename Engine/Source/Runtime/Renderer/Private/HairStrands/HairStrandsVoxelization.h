// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HairStrandsVoxelization.h: Hair voxelization implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "SceneRendering.h"


/// Global enable/disable for hair voxelization
bool IsHairStrandsVoxelizationEnable(EShaderPlatform ShaderPlatform);
bool IsHairStrandsForVoxelTransmittanceAndShadowEnable(EShaderPlatform ShaderPlatform);

void VoxelizeHairStrands(
	FRDGBuilder& GraphBuilder,
	const class FScene* Scene,
	FViewInfo& View,
	FInstanceCullingManager& InstanceCullingManager, 
	const FVector& PreViewStereoCorrection);
