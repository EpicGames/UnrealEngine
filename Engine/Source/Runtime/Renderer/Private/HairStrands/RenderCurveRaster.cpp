// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneRendering.h"
#include "ScenePrivate.h"
#include "RenderGraphUtils.h"
#include "RenderCurveSceneExtension.h"

DECLARE_GPU_STAT(CurveRasterPipeline);

static TAutoConsoleVariable<int32> CVarRenderCurve(
	TEXT("r.RenderCurve"),
	0,
	TEXT("Enable render curve raster pipeline. Read-only variable. (experimental)."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarRenderCurveDebug(
	TEXT("r.RenderCurve.Debug"),
	0,
	TEXT("Enable render curve pipeline debug visualization."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarRenderCurveMinCoverage(
	TEXT("r.RenderCurve.Raster.MinCoverage"),
	0.05f,
	TEXT("Render curve min. coverage before exiting raster work."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRenderCurveNumBinners(
	TEXT("r.RenderCurve.Raster.NumBinners"),
	32,
	TEXT("Number of Binners used in Binning pass by the  render curve pipeline. 32 is default."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRenderCurveNumRasterizers(
	TEXT("r.RenderCurve.Raster.NumRasterizers"),
	256,
	TEXT("Number of Rasterizers used in Raster pass by the render curve pipeline. 256 is default."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRenderCurveSWRaster(
	TEXT("r.RenderCurve.SWRaster"),
	1,
	TEXT("Enable curve software rasterizer."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRenderCurveHWRaster(
	TEXT("r.RenderCurve.HWRaster"),
	0,
	TEXT("Enable curve hardware rasterizer."),
	ECVF_RenderThreadSafe
);

/////////////////////////////////////////////////////////////////////////////////////////
// FPackedSegment

#define FPackedSegmentType uint4
#define FPackedSegmentSizeInBytes (sizeof(uint32) * 4)

/////////////////////////////////////////////////////////////////////////////////////////

bool IsRenderCurveEnabled()
{
	return CVarRenderCurve.GetValueOnAnyThread() > 0;
}

inline bool IsRenderCurveSupported(EShaderPlatform In) 
{ 
	return 
		IsRenderCurveEnabled() &&
		IsFeatureLevelSupported(In, ERHIFeatureLevel::SM6) && 
		FDataDrivenShaderPlatformInfo::GetSupportsMeshShadersTier1(In) && 
		IsHairStrandsSupported(EHairStrandsShaderType::Strands, In) && 
		!IsVulkanPlatform(In) && 
		!IsMetalPlatform(In);
}

/////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SHADER_PARAMETER_STRUCT(FRenderCurveCommonParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)

	SHADER_PARAMETER(FIntPoint, Resolution)
	
	SHADER_PARAMETER(uint32, BinTileSize)
	SHADER_PARAMETER(uint32, RasterTileSize)
	SHADER_PARAMETER(uint32, NumBinners)
	SHADER_PARAMETER(uint32, NumRasterizers)
	SHADER_PARAMETER(FIntPoint, BinTileRes)
	SHADER_PARAMETER(FIntPoint, RasterTileRes)

	SHADER_PARAMETER(uint32, MaxTileDataCount)
	SHADER_PARAMETER(uint32, MaxSegmentDataCount)

	SHADER_PARAMETER(uint32, MaxZBinDataCount)
	SHADER_PARAMETER(uint32, MaxZBinSegmentDataCount)

	SHADER_PARAMETER(uint32, MaxRasterWorkCount)
	SHADER_PARAMETER(float, MinCoverageThreshold)

END_SHADER_PARAMETER_STRUCT()

/////////////////////////////////////////////////////////////////////////////////////////

class FRenderCurveSegmentLUTCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveSegmentLUTCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveSegmentLUTCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER(FIntPoint, DebugOutputResolution)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint2>, RWSegmentLUT)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWDebugOutput)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetRasterResolution() { return 16u; }
	static uint32 GetGroupSizeX() { return GetRasterResolution(); }
	static uint32 GetGroupSizeY() { return GetRasterResolution(); }
	static uint32 GetGroupSize()  { return GetGroupSizeX() * GetGroupSizeY(); }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GetGroupSizeX());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GetGroupSizeY());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveSegmentLUTCS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "SegmentLUTCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

class FRenderCurveInstanceCullingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveInstanceCullingCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveInstanceCullingCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWVisibleInstanceArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWVisibleInstances)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWMinMaxZ)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 64u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveInstanceCullingCS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "InstanceCullingCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

class FRenderCurveClusterCullingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveClusterCullingCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveClusterCullingCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, VisibleInstanceArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, VisibleInstances)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWVisibleClusterArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint2>, RWVisibleClusters)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWMinMaxZ)
		RDG_BUFFER_ACCESS(VisibleInstanceIndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 64u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveClusterCullingCS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "ClusterCullingCS", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Compute depth tile data based on scene data

class FRenderCurveSceneTileDepthCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveSceneTileDepthCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveSceneTileDepthCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint2>, OutSceneTileDepthTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 32u * 32u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveSceneTileDepthCS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "SceneTileDepthCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////
// Indirect args for raster

class FRenderCurveIndirectArgsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveIndirectArgsCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveIndirectArgsCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, 				InVisibleClusterArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, 			OutHWRasterIndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>,	OutVisibleClusterCount)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 32u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters &Parameters, FShaderCompilerEnvironment &OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveIndirectArgsCS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "IndirectArgsCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////
// Bin segments

class FRenderCurveBinningCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveBinningCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveBinningCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ViewMinMaxZ)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint2>, SceneTileDepthTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, VisibleClusters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>,  VisibleClustersCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>,VisibleClustersQueue)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, RWTileSegmentCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWTileData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPackedSegmentType>, RWSegmentData)	
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWTileDataAllocatedCount)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSizeX() { return 64u; }
	static uint32 GetGroupSizeY() { return 16u; }
	static uint32 GetGroupSize()  { return GetGroupSizeX() * GetGroupSizeY(); }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GetGroupSizeX());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GetGroupSizeY());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveBinningCS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "BinningCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////
// Compact binned segments into contiguous list 

class FRenderCurveCompactionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveCompactionCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveCompactionCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, 	ViewMinMaxZ)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint2>, 				SceneTileDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>,			TileSegmentCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>,		TileData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedSegmentType>, SegmentData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, 	TileDataAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, 	RWZBinDataAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint2>,	RWZBinData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, 	RWZBinSegmentAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPackedSegmentType>, RWZBinSegmentData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, 	RWRasterWorkAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint2>, 	RWRasterWork)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 1024u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters &Parameters, FShaderCompilerEnvironment &OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveCompactionCS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "CompactionCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////
// Compact binned segments into contiguous list 

class FRenderCurveSWRasterizerCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveSWRasterizerCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveSWRasterizerCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, 	ViewMinMaxZ)	
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint2>, 				SceneTileDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, 				SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, 				SegmentLUT)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, 	ZBinDataAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>,	ZBinData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, 	ZBinSegmentAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedSegmentType>,	ZBinSegmentData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, 	RasterWorkAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, 	RasterWork)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>,	RasterWorkQueue)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, 		OutputTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 1024u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters &Parameters, FShaderCompilerEnvironment &OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveSWRasterizerCS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "SWRasterizerCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////
class FRenderCurveHWRasterizerMS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveHWRasterizerMS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveHWRasterizerMS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ViewMinMaxZ)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint2>, SceneTileDepthTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, VisibleClusters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>,  VisibleClustersCount)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSize() { return 128u; }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters &Parameters, FShaderCompilerEnvironment &OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveHWRasterizerMS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "HWRasterizerMS", SF_Mesh);

/////////////////////////////////////////////////////////////////////////////////////////
class FRenderCurveHWRasterizerPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveHWRasterizerPS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveHWRasterizerPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveCommonParameters, CommonParameters)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters &Parameters, FShaderCompilerEnvironment &OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveHWRasterizerPS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "HWRasterizerPS", SF_Pixel);

/////////////////////////////////////////////////////////////////////////////////////////

BEGIN_SHADER_PARAMETER_STRUCT(FRenderCurveHWRasterizerParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveHWRasterizerMS::FParameters, MS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveHWRasterizerPS::FParameters, PS)
	RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
END_SHADER_PARAMETER_STRUCT()

/////////////////////////////////////////////////////////////////////////////////////////

class FRenderCurveDebugDrawingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderCurveDebugDrawingCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderCurveDebugDrawingCS, FGlobalShader);

	class FSWRaster : SHADER_PERMUTATION_BOOL("PERMUTATION_SWRASTER");
	using FPermutationDomain = TShaderPermutationDomain<FSWRaster>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, TotalBufferMemoryInMBytes)
		SHADER_PARAMETER(uint32, TotalTextureMemoryInMBytes)
		SHADER_PARAMETER_STRUCT_INCLUDE(FRenderCurveCommonParameters, CommonParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ViewMinMaxZ)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, VisibleInstanceArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, VisibleInstances)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, VisibleClusterArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, VisibleClusters)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint2>, SceneTileDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, TileSegmentCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileDataAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, ZBinData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, RasterWork)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, RasterWorkAllocatedCount)	
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ZBinSegmentAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ZBinDataAllocatedCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPackedSegmentType>, ZBinSegmentData)
	END_SHADER_PARAMETER_STRUCT()

public:
	static uint32 GetGroupSizeX() { return 8u; }
	static uint32 GetGroupSizeY() { return 8u; }
	static uint32 GetGroupSize()  { return GetGroupSizeX() * GetGroupSizeY(); }
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsRenderCurveSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("PERMUTATION_DEBUG"), 1);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GetGroupSizeX());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GetGroupSizeY());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GetGroupSize());
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderCurveDebugDrawingCS, "/Engine/Private/HairStrands/RenderCurveRaster.usf", "DebugDrawingCS", SF_Compute);

/////////////////////////////////////////////////////////////////////////////////////////

struct FRenderCurveTransientData
{
	FRDGBufferSRVRef VisibleInstancesSRV = nullptr;
	FRDGBufferSRVRef VisibleInstanceArgsSRV = nullptr;
	FRDGBufferRef    VisibleInstanceArgs = nullptr;

	FRDGBufferSRVRef VisibleClustersSRV = nullptr;
	FRDGBufferSRVRef VisibleClusterArgsSRV = nullptr;
	FRDGBufferRef    VisibleClusterArgs = nullptr;

	FRDGBufferRef 	 VisibleClusterCount = nullptr;
	FRDGBufferSRVRef VisibleClusterCountSRV = nullptr;
	FRDGBufferRef 	 HWRasterizerArgs = nullptr;

	FRDGTextureRef   SceneTileDepth = nullptr;

	FRDGTextureRef   TileSegmentCount = nullptr;	
	FRDGBufferSRVRef TileDataAllocatedCount = nullptr;
	
	FRDGBufferSRVRef TileData = nullptr;
	FRDGBufferSRVRef SegmentData = nullptr;

	FRDGBufferUAVRef RWMinMaxZ = nullptr;
	FRDGBufferSRVRef MinMaxZ = nullptr;
	FRDGBufferSRVRef ZBinData = nullptr;
	FRDGBufferSRVRef ZBinDataAllocatedCount = nullptr;
	FRDGBufferSRVRef ZBinSegmentData = nullptr;
	FRDGBufferSRVRef ZBinSegmentAllocatedCount = nullptr;

	FRDGBufferSRVRef RasterWork = nullptr;
	FRDGBufferSRVRef RasterWorkAllocatedCount = nullptr;
};

/////////////////////////////////////////////////////////////////////////////////////////
// Memory
struct FRenderCurveMemoryTracker
{
	FRenderCurveMemoryTracker()
	{
		Infos.Reserve(12u);
	}

	void Add(FRDGBufferRef In)
	{
		if (In)
		{
			FInfo& Info = Infos.AddDefaulted_GetRef();
			Info.Size = In->Desc.GetSize();
			Info.Name = In->Name;
			TotalBufferMemoryInBytes += Info.Size;
		}
	}

	void Add(FRDGTextureRef In)
	{
		FInfo& Info = Infos.AddDefaulted_GetRef();
		Info.Size = In->Desc.CalcMemorySizeEstimate();
		Info.Name = In->Name;
		TotalTextureMemoryInBytes += Info.Size;
	}

	struct FInfo
	{
		const TCHAR* Name = nullptr;
		uint64 Size = 0;
	};

	TArray<FInfo> Infos;
	uint64 TotalBufferMemoryInBytes = 0;
	uint64 TotalTextureMemoryInBytes = 0;
};

/////////////////////////////////////////////////////////////////////////////////////////

void AddRenderCurveRasterPipeline(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const TArray<FViewInfo>& Views,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture)
{
	if (Views.Num() == 0 || !IsRenderCurveSupported(Views[0].GetShaderPlatform()))
	{
		return;
	}

	// Need wave size to be at least 32
	if (GRHISupportsWaveOperations && RHISupportsWaveOperations(Views[0].GetShaderPlatform()) && GRHIMinimumWaveSize < 32)
	{
		return;
	}

	uint32 InstanceCount = 0;
	uint32 UniqueClusterCount = 0;
	if (const RenderCurve::FRenderCurveSceneExtension* Extension = Scene->GetExtensionPtr<RenderCurve::FRenderCurveSceneExtension>())
	{
		InstanceCount = Extension->GetInstanceCount();
		UniqueClusterCount = Extension->GetClusterCount();
	}
	
	if (InstanceCount == 0) { return; }

	const bool bSWRaster = CVarRenderCurveSWRaster.GetValueOnRenderThread() > 0;
	const bool bHWRaster = CVarRenderCurveHWRaster.GetValueOnRenderThread() > 0;
	if (!bSWRaster && !bHWRaster)
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_CLM_CurveRasterPipeline);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, CurveRasterPipeline, "CurveRasterPipeline");

	const uint32 MaxVisibleClusterCount = 8u * 1024u * InstanceCount; // 1k cluster per instance in average?
	const uint32 IndirectArgsSizeInBytes = FMath::Max(uint32(sizeof(FRHIDispatchIndirectParameters)), 16u); // Needs at least 4 elements
	const bool bDebug = CVarRenderCurveDebug.GetValueOnRenderThread() > 0;
	const FViewInfo& View = Views[0];
	auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);//TODO

	FRenderCurveMemoryTracker MemoryTracker;

	// Segment LUT generation
	// This LUT translate a segment (defined by its two points) into a 8x8 bitmask (stored as 2uints)
	// Disable as we don't use it yet.
	#if 0
	FRDGTextureRef SegmentLUT = nullptr;
	if (Scene->HairStrandsSceneData.SegmentLUT == nullptr)
	{
		const uint32 Res = FRenderCurveSegmentLUTCS::GetRasterResolution();

		SegmentLUT = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(Res*Res, Res*Res), PF_R32G32_UINT, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), TEXT("RenderCurve.SegmentLUT"));
		FRDGTextureUAVRef RWSegmentLUT = GraphBuilder.CreateUAV(SegmentLUT);
		AddClearUAVPass(GraphBuilder, RWSegmentLUT, 0u);
		
		FRenderCurveSegmentLUTCS::FPermutationDomain PermutationVector;
		TShaderMapRef<FRenderCurveSegmentLUTCS> ComputeShader(ShaderMap, PermutationVector);

		FRenderCurveSegmentLUTCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveSegmentLUTCS::FParameters>();
		Parameters->RWSegmentLUT = RWSegmentLUT;
		Parameters->RWDebugOutput = GraphBuilder.CreateUAV(SceneColorTexture);
		Parameters->DebugOutputResolution = SceneColorTexture->Desc.Extent;
		{
			ShaderPrint::SetEnabled(true);
			ShaderPrint::RequestSpaceForLines(1024 * 1024);
			ShaderPrint::RequestSpaceForCharacters(4096 * 128);
			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintUniformBuffer);
		}
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderCurve::DebugDrawing"), ComputeShader, Parameters, FIntVector(Res, Res, 1));

		GraphBuilder.QueueTextureExtraction(SegmentLUT, &Scene->HairStrandsSceneData.SegmentLUT, ERHIAccess::SRVMask);
	}
	else
	{
		SegmentLUT = GraphBuilder.RegisterExternalTexture(Scene->HairStrandsSceneData.SegmentLUT);
	}
	MemoryTracker.Add(SegmentLUT);
	#endif

	// Common parameters
	FRenderCurveCommonParameters CommonParameters;
	{
		TRDGUniformBufferRef<FSceneUniformParameters> SceneUniformParameters = GetSceneUniformBufferRef(GraphBuilder, View);
		CommonParameters.View = View.ViewUniformBuffer;
		CommonParameters.Scene = SceneUniformParameters;
		if (const FViewInfo* ViewInfo = View.bIsViewInfo ? reinterpret_cast<const FViewInfo*>(&View) : nullptr)
		{
			CommonParameters.HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
		}
		else
		{
			CommonParameters.HZBParameters = GetDummyHZBParameters(GraphBuilder);
		}
		CommonParameters.Resolution = SceneDepthTexture->Desc.Extent; // Rect view instead? ViewSizeAndInvSize
		CommonParameters.BinTileSize = 32;
		CommonParameters.RasterTileSize = 8;
		CommonParameters.NumBinners = FMath::Min(FMath::Max(CVarRenderCurveNumBinners.GetValueOnRenderThread(), 1), 256);
		CommonParameters.NumRasterizers = FMath::Min(FMath::Max(CVarRenderCurveNumRasterizers.GetValueOnRenderThread(), 1), 1024);
		CommonParameters.BinTileRes = FIntPoint(FMath::DivideAndRoundUp(uint32(CommonParameters.Resolution.X), CommonParameters.BinTileSize), FMath::DivideAndRoundUp(uint32(CommonParameters.Resolution.Y), CommonParameters.BinTileSize));
		CommonParameters.RasterTileRes = FIntPoint(FMath::DivideAndRoundUp(uint32(CommonParameters.Resolution.X), CommonParameters.RasterTileSize), FMath::DivideAndRoundUp(uint32(CommonParameters.Resolution.Y), CommonParameters.RasterTileSize));

		const uint32 MaxVisibleSegmentCountPerTile = FMath::Max(CommonParameters.NumBinners, 8u) * 128u;
		const uint32 MaxVisibleTileDataPerTile     = FMath::Max(CommonParameters.NumBinners, 8u) * 2;
		CommonParameters.MaxTileDataCount = CommonParameters.BinTileRes.X * CommonParameters.BinTileRes.Y * MaxVisibleTileDataPerTile;
		CommonParameters.MaxSegmentDataCount = CommonParameters.BinTileRes.X * CommonParameters.BinTileRes.Y * MaxVisibleSegmentCountPerTile;

		CommonParameters.MaxZBinDataCount = CommonParameters.MaxTileDataCount;
		CommonParameters.MaxRasterWorkCount = CommonParameters.BinTileRes.X * CommonParameters.BinTileRes.Y;
		CommonParameters.MaxZBinSegmentDataCount = CommonParameters.MaxSegmentDataCount;

		CommonParameters.MinCoverageThreshold = FMath::Clamp(CVarRenderCurveMinCoverage.GetValueOnRenderThread(), 0.f, 1.f);

		if (bDebug)
		{
			ShaderPrint::SetEnabled(true);
			ShaderPrint::RequestSpaceForLines((12u * 1024u + 16u * 2048u) * InstanceCount);
			ShaderPrint::RequestSpaceForCharacters(4096 * 128);
			ShaderPrint::RequestSpaceForTriangles(CommonParameters.BinTileRes.X * CommonParameters.BinTileRes.Y * 2u);
			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, CommonParameters.ShaderPrintUniformBuffer);
		}
	}

	FRenderCurveTransientData TransientData;

	// Instance culling
	{
		FRDGBufferRef MinMaxZ = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 2), TEXT("RenderCurve.Culling.ZMinMax"));
		TransientData.MinMaxZ = GraphBuilder.CreateSRV(MinMaxZ);
		TransientData.RWMinMaxZ = GraphBuilder.CreateUAV(MinMaxZ);
		MemoryTracker.Add(MinMaxZ);

		FRDGBufferRef VisibleInstanceArgs = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(IndirectArgsSizeInBytes, 1u), TEXT("RenderCurve.Culling.VisibleInstanceArgs"));
		FRDGBufferRef VisibleInstances = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), InstanceCount), TEXT("RenderCurve.Culling.VisibleInstances"));
		MemoryTracker.Add(VisibleInstanceArgs);
		MemoryTracker.Add(VisibleInstances);

		FRDGBufferUAVRef RWVisibleInstances = GraphBuilder.CreateUAV(VisibleInstances, PF_R32_UINT);
		FRDGBufferUAVRef RWVisibleInstanceArgs = GraphBuilder.CreateUAV(VisibleInstanceArgs, PF_R32_UINT);
		TransientData.VisibleInstanceArgs = VisibleInstanceArgs;
		TransientData.VisibleInstanceArgsSRV = GraphBuilder.CreateSRV(VisibleInstanceArgs, PF_R32_UINT);
		TransientData.VisibleInstancesSRV = GraphBuilder.CreateSRV(VisibleInstances, PF_R32_UINT);

		AddClearUAVPass(GraphBuilder, RWVisibleInstanceArgs, 0u);

		FRenderCurveInstanceCullingCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRenderCurveInstanceCullingCS::FDebug>(bDebug);
		TShaderMapRef<FRenderCurveInstanceCullingCS> ComputeShaderRaster(ShaderMap, PermutationVector);

		FRenderCurveInstanceCullingCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveInstanceCullingCS::FParameters>();
		Parameters->CommonParameters = CommonParameters;
		Parameters->RWVisibleInstanceArgs = RWVisibleInstanceArgs;
		Parameters->RWVisibleInstances = RWVisibleInstances;
		Parameters->RWMinMaxZ = TransientData.RWMinMaxZ;

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderCurve::InstanceCulling"), ComputeShaderRaster, Parameters, FIntVector(InstanceCount, 1, 1));
	}

	// Cluster culling
	{
		FRDGBufferRef VisibleClusterArgs = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(IndirectArgsSizeInBytes, 1u), TEXT("RenderCurve.Culling.VisibleClusterArgs"));
		TransientData.VisibleClusterArgs = VisibleClusterArgs;
		MemoryTracker.Add(VisibleClusterArgs);

		FRDGBufferRef VisibleClusters = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32) * 2u, MaxVisibleClusterCount), TEXT("RenderCurve.Culling.VisibleClusters"));
		FRDGBufferUAVRef RWVisibleClusters = GraphBuilder.CreateUAV(VisibleClusters, PF_R32G32_UINT);
		TransientData.VisibleClustersSRV = GraphBuilder.CreateSRV(VisibleClusters, PF_R32G32_UINT);
		MemoryTracker.Add(VisibleClusters);

		FRDGBufferUAVRef RWVisibleClusterArgs = GraphBuilder.CreateUAV(VisibleClusterArgs, PF_R32_UINT);
		TransientData.VisibleClusterArgsSRV = GraphBuilder.CreateSRV(VisibleClusterArgs, PF_R32_UINT);

		AddClearUAVPass(GraphBuilder, RWVisibleClusterArgs, 0u);


		FRenderCurveClusterCullingCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRenderCurveClusterCullingCS::FDebug>(bDebug);
		TShaderMapRef<FRenderCurveClusterCullingCS> ComputeShaderRaster(ShaderMap, PermutationVector);

		FRenderCurveClusterCullingCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveClusterCullingCS::FParameters>();
		Parameters->CommonParameters = CommonParameters;
		Parameters->VisibleInstanceArgs = TransientData.VisibleInstanceArgsSRV;
		Parameters->VisibleInstances = TransientData.VisibleInstancesSRV;
		Parameters->RWVisibleClusterArgs = RWVisibleClusterArgs;
		Parameters->RWVisibleClusters = RWVisibleClusters;
		Parameters->RWMinMaxZ = TransientData.RWMinMaxZ;
		Parameters->VisibleInstanceIndirectArgs = TransientData.VisibleInstanceArgs;

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderCurve::ClusterCulling"), ComputeShaderRaster, Parameters, TransientData.VisibleInstanceArgs, 0u);
	}
	
	// Fill in tile depth
	{
		TransientData.SceneTileDepth = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(CommonParameters.BinTileRes, PF_R32G32_UINT, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource), TEXT("RenderCurve.Raster.SceneTileDepth"));

		TShaderMapRef<FRenderCurveSceneTileDepthCS> ComputeShaderRaster(ShaderMap);
		FRenderCurveSceneTileDepthCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveSceneTileDepthCS::FParameters>();
		Parameters->CommonParameters = CommonParameters;
		Parameters->SceneDepthTexture = SceneDepthTexture;
		Parameters->OutSceneTileDepthTexture = GraphBuilder.CreateUAV(TransientData.SceneTileDepth);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderCurve::SceneTileDepth"), ComputeShaderRaster, Parameters, FIntVector(CommonParameters.BinTileRes.X, CommonParameters.BinTileRes.Y, 1));
	}

	// Indirect args for HW rasterizer and visible cluster count
	{
		TransientData.VisibleClusterCount = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("RenderCurve.Binning.VisibleClusterCount"));
		TransientData.VisibleClusterCountSRV = GraphBuilder.CreateSRV(TransientData.VisibleClusterCount);
		TransientData.HWRasterizerArgs = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(IndirectArgsSizeInBytes, 1u), TEXT("RenderCurve.HWRasterizerArgs"));
		
		TShaderMapRef<FRenderCurveIndirectArgsCS> ComputeShader(ShaderMap);
		FRenderCurveIndirectArgsCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveIndirectArgsCS::FParameters>();
		Parameters->InVisibleClusterArgs = TransientData.VisibleClusterArgsSRV;
		Parameters->OutHWRasterIndirectArgs = GraphBuilder.CreateUAV(TransientData.HWRasterizerArgs, PF_R32_UINT);
		Parameters->OutVisibleClusterCount = GraphBuilder.CreateUAV(TransientData.VisibleClusterCount, PF_R32_UINT);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderCurve::IndirectArgs"), ComputeShader, Parameters, FIntVector(1, 1, 1));
	}

	// Binning
	if (bSWRaster)
	{
		FRDGBufferRef VisibleClusterQueue = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("RenderCurve.Binning.VisibleClusterQueue"));
		FRDGBufferUAVRef RWVisibleClusterQueue = GraphBuilder.CreateUAV(VisibleClusterQueue, PF_R32_UINT);
		AddClearUAVPass(GraphBuilder, RWVisibleClusterQueue, 0u);
		MemoryTracker.Add(VisibleClusterQueue);

		const uint32 TileSegmentCountNumLayers = CommonParameters.NumBinners * 3; // 3 layers per binner: segment count | last seg. count | tile info
		FRDGTextureRef TileSegmentCount = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2DArray(CommonParameters.BinTileRes, PF_R32_UINT, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV, TileSegmentCountNumLayers), TEXT("RenderCurve.Binning.TileSegmentCount"));
		FRDGTextureUAVRef RWTileSegmentCount = GraphBuilder.CreateUAV(TileSegmentCount);
		AddClearUAVPass(GraphBuilder, RWTileSegmentCount, 0u);
		TransientData.TileSegmentCount = TileSegmentCount;
		MemoryTracker.Add(TileSegmentCount);

		FRDGBufferRef TileDataAllocatedCount = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("RenderCurve.Binning.TileDataAllocatedCount"));
		//FRDGBufferRef TileDataArgs = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(IndirectArgsSizeInBytes, 1u), TEXT("RenderCurve.Binning.TileDataAllocatedCount"));
		FRDGBufferUAVRef RWTileDataAllocatedCount = GraphBuilder.CreateUAV(TileDataAllocatedCount);
		AddClearUAVPass(GraphBuilder, RWTileDataAllocatedCount, 0u);
		TransientData.TileDataAllocatedCount = GraphBuilder.CreateSRV(TileDataAllocatedCount);
		MemoryTracker.Add(TileDataAllocatedCount);

		const uint32 EntryPerTileData = 4u;
		FRDGBufferRef TileData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), EntryPerTileData * CommonParameters.MaxTileDataCount), TEXT("RenderCurve.Binning.TileData"));
		FRDGBufferRef SegmentData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(FPackedSegmentSizeInBytes, CommonParameters.MaxSegmentDataCount), TEXT("RenderCurve.Binning.SegmentData"));
		TransientData.TileData = GraphBuilder.CreateSRV(TileData);
		TransientData.SegmentData = GraphBuilder.CreateSRV(SegmentData);
		MemoryTracker.Add(TileData);
		MemoryTracker.Add(SegmentData);

		FRenderCurveBinningCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveBinningCS::FParameters>();
		Parameters->CommonParameters = CommonParameters;
		Parameters->ViewMinMaxZ = TransientData.MinMaxZ;
		Parameters->SceneTileDepthTexture = TransientData.SceneTileDepth;
		Parameters->VisibleClusters = TransientData.VisibleClustersSRV;
		Parameters->VisibleClustersCount = TransientData.VisibleClusterCountSRV;
		Parameters->VisibleClustersQueue = RWVisibleClusterQueue;
		Parameters->RWTileSegmentCount = RWTileSegmentCount;
		Parameters->RWTileData = GraphBuilder.CreateUAV(TileData, PF_R32_UINT);
		Parameters->RWSegmentData = GraphBuilder.CreateUAV(SegmentData);
		Parameters->RWTileDataAllocatedCount = RWTileDataAllocatedCount;
		
		FRenderCurveBinningCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRenderCurveBinningCS::FDebug>(bDebug);
		TShaderMapRef<FRenderCurveBinningCS> ComputeShaderBinning(ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("RenderCurve::Binning"),
			ComputeShaderBinning,
			Parameters,
			FIntVector(CommonParameters.NumBinners, 1, 1));
	}

	// Compaction
	// Each tile segments are compacted into a list of ZBins which each contains a list of segments. 
	// A ZBin is defined by a (fixed) depth range. All segments within this Z ranges belongs to this ZBin
	//  _ _ _ _ _ _ _ _ _ _ 
	// |   |   |   |   |   |--> ZBinOffset|ZBinCount
	// |_ _|_ _|_ _|_ _|_ _|
	// |   |   |   |   |   |
	// |_ _|_ _|_ _|_ _|_ _|
	// |   |   |   |   |   |
	// |_ _|_ _|_ _|_ _|_ _|
	// |   |   |   |   |   |
	// |_ _|_ _|_ _|_ _|_ _|
	// 
	// Definitions:
	// * ZBinData - Compacted/sparse list of ZBin, containing the offset/count of segment belonging to this ZBin
	//   | Depth0 [ZBinSegmentOffset|ZBinSegmentCount] | Depth7 [ZBinSegmentOffset|ZBinSegmentCount] | Depth24 [ZBinSegmentOffset|ZBinSegmentCount]
	//
	// * ZBinSegments
	//   v-----Count-----v
	//   [ / / / / / / / ][ / / / ][ / / / / / ][ / / ] ... 
	//   ^                ^        ^
	//   Offset           Offset   Offset
	//
	// * RasterWork - List of work passed to the rasterize. Contains ZBinOffset|ZBinCount|TileCoord.
	if (bSWRaster)
	{
		FRDGBufferRef ZBinData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32)*2, CommonParameters.MaxZBinDataCount), TEXT("RenderCurve.Compaction.ZBinData"));
		TransientData.ZBinData = GraphBuilder.CreateSRV(ZBinData);
		MemoryTracker.Add(ZBinData);

		FRDGBufferRef ZBinDataAllocatedCount = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("RenderCurve.Compaction.ZBinDataAllocatedCount"));
		FRDGBufferUAVRef RWZBinDataAllocatedCount = GraphBuilder.CreateUAV(ZBinDataAllocatedCount);
		AddClearUAVPass(GraphBuilder, RWZBinDataAllocatedCount, 0u);
		TransientData.ZBinDataAllocatedCount = GraphBuilder.CreateSRV(ZBinDataAllocatedCount);
		MemoryTracker.Add(ZBinDataAllocatedCount);

		FRDGBufferRef RasterWork = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32)*2, CommonParameters.MaxRasterWorkCount), TEXT("RenderCurve.Compaction.RasterWork"));
		TransientData.RasterWork = GraphBuilder.CreateSRV(RasterWork);
		MemoryTracker.Add(RasterWork);

		FRDGBufferRef RasterWorkAllocatedCount = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("RenderCurve.Compaction.RasterWorkAllocatedCount"));
		FRDGBufferUAVRef RWRasterWorkAllocatedCount = GraphBuilder.CreateUAV(RasterWorkAllocatedCount);
		AddClearUAVPass(GraphBuilder, RWRasterWorkAllocatedCount, 0u);
		TransientData.RasterWorkAllocatedCount = GraphBuilder.CreateSRV(RasterWorkAllocatedCount);
		MemoryTracker.Add(RasterWorkAllocatedCount);

		FRDGBufferRef ZBinSegmentData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(FPackedSegmentSizeInBytes, CommonParameters.MaxZBinSegmentDataCount), TEXT("RenderCurve.Compaction.ZBinSegmentData"));
		TransientData.ZBinSegmentData = GraphBuilder.CreateSRV(ZBinSegmentData);
		MemoryTracker.Add(ZBinSegmentData);

		FRDGBufferRef ZBinSegmentAllocatedCount = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("RenderCurve.Compaction.ZBinSegmentAllocatedCount"));
		FRDGBufferUAVRef RWZBinSegmentAllocatedCount = GraphBuilder.CreateUAV(ZBinSegmentAllocatedCount);
		AddClearUAVPass(GraphBuilder, RWZBinSegmentAllocatedCount, 0u);
		TransientData.ZBinSegmentAllocatedCount = GraphBuilder.CreateSRV(ZBinSegmentAllocatedCount);
		MemoryTracker.Add(ZBinSegmentAllocatedCount);

		FRenderCurveCompactionCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveCompactionCS::FParameters>();
		Parameters->CommonParameters = CommonParameters;
		Parameters->ViewMinMaxZ					= TransientData.MinMaxZ;
		Parameters->SceneTileDepthTexture		= TransientData.SceneTileDepth;
		Parameters->TileSegmentCount			= TransientData.TileSegmentCount;
		Parameters->TileData					= TransientData.TileData;
		Parameters->SegmentData					= TransientData.SegmentData;
		Parameters->TileDataAllocatedCount		= TransientData.TileDataAllocatedCount;
		Parameters->RWZBinDataAllocatedCount	= RWZBinDataAllocatedCount;
		Parameters->RWZBinData					= GraphBuilder.CreateUAV(ZBinData);
		Parameters->RWZBinSegmentAllocatedCount	= RWZBinSegmentAllocatedCount;
		Parameters->RWZBinSegmentData			= GraphBuilder.CreateUAV(ZBinSegmentData);
		Parameters->RWRasterWorkAllocatedCount	= RWRasterWorkAllocatedCount;
		Parameters->RWRasterWork				= GraphBuilder.CreateUAV(RasterWork);
		FRenderCurveCompactionCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRenderCurveCompactionCS::FDebug>(bDebug);
		TShaderMapRef<FRenderCurveCompactionCS> ComputeShader(ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderCurve::Compaction"), ComputeShader, Parameters, FIntVector(CommonParameters.BinTileRes.X, CommonParameters.BinTileRes.Y, 1));
	}

	// Rasterizer
	if (bSWRaster)
	{		
		FRDGBufferRef RasterWorkQueue = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("RenderCurve.Rasterizer.WorkQueue"));
		FRDGBufferUAVRef RWRasterWorkQueue = GraphBuilder.CreateUAV(RasterWorkQueue, PF_R32_UINT);
		AddClearUAVPass(GraphBuilder, RWRasterWorkQueue, 0u);
		MemoryTracker.Add(RasterWorkQueue);

		FRenderCurveSWRasterizerCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveSWRasterizerCS::FParameters>();
		Parameters->CommonParameters 			= CommonParameters;
		Parameters->SceneTileDepthTexture		= TransientData.SceneTileDepth;
		Parameters->SceneDepthTexture 			= SceneDepthTexture;
		Parameters->SegmentLUT					= GSystemTextures.GetBlackDummy(GraphBuilder);
		Parameters->ViewMinMaxZ					= TransientData.MinMaxZ;
		Parameters->ZBinDataAllocatedCount		= TransientData.ZBinDataAllocatedCount;
		Parameters->ZBinData					= TransientData.ZBinData;
		Parameters->ZBinSegmentAllocatedCount	= TransientData.ZBinSegmentAllocatedCount;
		Parameters->ZBinSegmentData				= TransientData.ZBinSegmentData;
		Parameters->RasterWorkAllocatedCount	= TransientData.RasterWorkAllocatedCount;
		Parameters->RasterWork					= TransientData.RasterWork;
		Parameters->RasterWorkQueue				= RWRasterWorkQueue;
		Parameters->OutputTexture				= GraphBuilder.CreateUAV(SceneColorTexture);

		FRenderCurveSWRasterizerCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRenderCurveSWRasterizerCS::FDebug>(bDebug);
		TShaderMapRef<FRenderCurveSWRasterizerCS> ComputeShader(ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderCurve::SWRasterizer"), ComputeShader, Parameters, FIntVector(CommonParameters.NumRasterizers, 1, 1));
	}

	// Rasterizer
	if (bHWRaster)
	{	
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(CommonParameters.Resolution, PF_FloatRGBA, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable);
		FRDGTextureRef Output = GraphBuilder.CreateTexture(Desc, TEXT("RenderCurve.HWRasterizerOutput"));
		AddClearRenderTargetPass(GraphBuilder, Output, FLinearColor::Black);

		FRenderCurveHWRasterizerParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveHWRasterizerParameters>();
		Parameters->MS.CommonParameters = CommonParameters;
		Parameters->MS.ViewMinMaxZ = TransientData.MinMaxZ;
		Parameters->MS.SceneTileDepthTexture = TransientData.SceneTileDepth;
		Parameters->MS.VisibleClusters = TransientData.VisibleClustersSRV;
		Parameters->MS.VisibleClustersCount = TransientData.VisibleClusterCountSRV;
		Parameters->PS.RenderTargets[0] = FRenderTargetBinding(Output, ERenderTargetLoadAction::ENoAction);
		Parameters->IndirectArgs = TransientData.HWRasterizerArgs;

		TShaderMapRef<FRenderCurveHWRasterizerMS> MeshShader(ShaderMap);
		TShaderMapRef<FRenderCurveHWRasterizerPS> PixelShader(ShaderMap);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RenderCurve::HWRasterizer"),
			Parameters,
			ERDGPassFlags::Raster,
			[Parameters, MeshShader, PixelShader](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
			GraphicsPSOInit.BoundShaderState.SetMeshShader(MeshShader.GetMeshShader());
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, MeshShader, MeshShader.GetMeshShader(), Parameters->MS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), Parameters->PS);

			RHICmdList.SetViewport(0, 0, 0.0f, Parameters->MS.CommonParameters.Resolution.X, Parameters->MS.CommonParameters.Resolution.Y, 1.0f); // MinRect?
			//RHICmdList.DispatchMeshShader(64, 1, 1);			
			RHICmdList.DispatchIndirectMeshShader(Parameters->IndirectArgs->GetIndirectRHICallBuffer(), 0);
			
			//FORCEINLINE_DEBUGGABLE void DispatchIndirectMeshShader(FRHIBuffer* ArgumentBuffer, uint32 ArgumentOffset)
		});
	}

	// Debug drawing
	if (bDebug)
	{
		FRenderCurveDebugDrawingCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRenderCurveDebugDrawingCS::FSWRaster>(bSWRaster);
		TShaderMapRef<FRenderCurveDebugDrawingCS> ComputeShader(ShaderMap, PermutationVector);

		FRenderCurveDebugDrawingCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRenderCurveDebugDrawingCS::FParameters>();
		Parameters->CommonParameters = CommonParameters;
		Parameters->ViewMinMaxZ = TransientData.MinMaxZ;
		Parameters->VisibleInstanceArgs = TransientData.VisibleInstanceArgsSRV;
		Parameters->VisibleInstances = TransientData.VisibleInstancesSRV;
		Parameters->VisibleClusterArgs = TransientData.VisibleClusterArgsSRV;
		Parameters->VisibleClusters = TransientData.VisibleClustersSRV;
		Parameters->SceneTileDepthTexture = TransientData.SceneTileDepth;
		Parameters->TileSegmentCount = TransientData.TileSegmentCount;
		Parameters->TileDataAllocatedCount = TransientData.TileDataAllocatedCount;
		Parameters->ZBinData = TransientData.ZBinData;
		Parameters->RasterWork = TransientData.RasterWork;
		Parameters->RasterWorkAllocatedCount = TransientData.RasterWorkAllocatedCount;
		Parameters->ZBinSegmentData = TransientData.ZBinSegmentData;
		Parameters->TotalBufferMemoryInMBytes = uint32(MemoryTracker.TotalBufferMemoryInBytes>>20);
		Parameters->TotalTextureMemoryInMBytes = uint32(MemoryTracker.TotalTextureMemoryInBytes>>20);
		Parameters->ZBinSegmentAllocatedCount = TransientData.ZBinSegmentAllocatedCount;
		Parameters->ZBinDataAllocatedCount = TransientData.ZBinDataAllocatedCount;

		//FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderCurve::DebugDrawing"), ComputeShader, Parameters, TransientData.VisibleClusterArgs, 0u);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderCurve::DebugDrawing"), ComputeShader, Parameters, 
			FIntVector(
				FMath::DivideAndRoundUp(uint32(CommonParameters.Resolution.X), FRenderCurveDebugDrawingCS::GetGroupSizeX()), 
				FMath::DivideAndRoundUp(uint32(CommonParameters.Resolution.Y), FRenderCurveDebugDrawingCS::GetGroupSizeY()), 1));
	}

	// Add streaming request
	{
		// Smaller allocation: Move root pages down then resize
		//ClusterPageDataBuffer = GraphBuilder.RegisterExternalBuffer(ClusterPageData.DataBuffer);
		//AddPass_Memmove(GraphBuilder, GraphBuilder.CreateUAV(ClusterPageDataBuffer), MaxStreamingPages * NANITE_STREAMING_PAGE_GPU_SIZE, OldMaxStreamingPages * NANITE_STREAMING_PAGE_GPU_SIZE, RootPagesDataSize);				
		//ClusterPageDataBuffer = ResizeByteAddressBufferIfNeeded(GraphBuilder, ClusterPageData.DataBuffer, AllocatedPagesSize, TEXT("Nanite.StreamingManager.ClusterPageData"));
	}
}