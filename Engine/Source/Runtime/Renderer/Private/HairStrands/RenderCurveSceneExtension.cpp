// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderCurveSceneExtension.h"

#include "GlobalRenderResources.h"
#include "ScenePrivate.h"
#include "SceneUniformBuffer.h"
#include "ShaderParameterMacros.h"
#include "HairStrandsInterface.h"
#include "RenderCurveDefinitions.h"

namespace RenderCurve
{

IMPLEMENT_SCENE_EXTENSION(FRenderCurveSceneExtension);

BEGIN_SHADER_PARAMETER_STRUCT(FRenderCurveSceneParameters, RENDERER_API)
	SHADER_PARAMETER(uint32, InstanceCount)
	SHADER_PARAMETER(uint32, ClusterCount)
	SHADER_PARAMETER(uint32, MaxClusterStrideInBytes)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, RenderCurveInstanceData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, ClusterData)
END_SHADER_PARAMETER_STRUCT()

DECLARE_SCENE_UB_STRUCT(FRenderCurveSceneParameters, RenderCurve, RENDERER_API)

///////////////////////////////////////////////////////////////////////////////////////////////////

static bool InternalIsEnabled(FScene& InScene)
{
	return 
		IsRenderCurveEnabled() && 
		IsHairStrandsEnabled(EHairStrandsShaderType::All, InScene.GetShaderPlatform());
}

FRenderCurveSceneExtension::FRenderCurveSceneExtension(FScene& InScene) : ISceneExtension(InScene)
{

}

FRenderCurveSceneExtension::~FRenderCurveSceneExtension()
{

}

bool FRenderCurveSceneExtension::ShouldCreateExtension(FScene& InScene)
{
	return InternalIsEnabled(InScene);
}

void FRenderCurveSceneExtension::InitExtension(FScene& InScene)
{
	SetEnabled(InternalIsEnabled(InScene));
}

uint32 FRenderCurveSceneExtension::GetInstanceCount() const 
{ 
	return Datas.Num();
}

uint32 FRenderCurveSceneExtension::GetClusterCount() const
{
	return Header.TotalClusterCount;
}

bool FRenderCurveSceneExtension::IsEnabled() const
{
	return Buffers.IsValid();
}

void FRenderCurveSceneExtension::SetEnabled(bool bEnabled)
{
	if (bEnabled != IsEnabled())
	{
		if (bEnabled)
		{
			Buffers = MakeUnique<FBuffers>();
		}
		else
		{
			Buffers = nullptr;
			Datas.Reset();
			bDirtyData = false;
		}
	}
}

static FRDGBufferRef UploadUniqueCurveResource(FRDGBuilder& GraphBuilder, const TSparseArray<FRenderCurveSceneExtension::FData>& InDatas, TRefCountPtr<FRDGPooledBuffer>& InClusterDataBuffer, uint32& OutTotalClusterCount, uint32& OutClusterStideInBytes)
{
	TArray<FRenderCurveResourceData*> UniqueResources;
	UniqueResources.SetNum(0);
	uint64 TotalSizeResourceToUploadInBytes = 0;
	for (auto& Data : InDatas)
	{
		if (FRenderCurveResourceData* Resource = Data.CurveResourceData)
		{
			if (UniqueResources.FindLastByPredicate([Resource](FRenderCurveResourceData* A){ return Resource->Header.Id == A->Header.Id; }) == INDEX_NONE)
			{
				UniqueResources.Add(Resource);
				TotalSizeResourceToUploadInBytes += Resource->Data.BulkData.GetBulkDataSize();
			}
		}
	}
	FRDGBufferRef OutClusterDataBuffer = nullptr;
	OutTotalClusterCount = 0;
	OutClusterStideInBytes = 0;
	if (!UniqueResources.IsEmpty())
	{
		check(TotalSizeResourceToUploadInBytes <= InClusterDataBuffer->Desc.GetSize());

		// Resize (reserved) buffer to have a least the required size
		OutClusterDataBuffer = ResizeByteAddressBufferIfNeeded(GraphBuilder, InClusterDataBuffer, TotalSizeResourceToUploadInBytes, TEXT("RenderCurve.ClusterData"));

		// TODO: add support for book keeping (remove/defrag/transcode/..)
		uint64 DstOffset = 0;
		for (FRenderCurveResourceData* Resource : UniqueResources)
		{
			OutClusterStideInBytes = Resource->Header.MaxClusterStrideInBytes;
			OutTotalClusterCount += Resource->Header.ClusterCount;
			const uint32 SrcDataSizeInBytes = Resource->Data.BulkData.GetBulkDataSize();
			if (SrcDataSizeInBytes > 0)
			{
				const uint8* Data = (const uint8*)Resource->Data.BulkData.Lock(LOCK_READ_ONLY);
				FRDGBufferRef SrcBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateByteAddressDesc(SrcDataSizeInBytes), TEXT("RenderCurve.UploadBuffer"));
				if (Data)
				{
					GraphBuilder.QueueBufferUpload(SrcBuffer, Data, SrcDataSizeInBytes, ERDGInitialDataFlags::None);
					Resource->Data.BulkData.Unlock();
				}
				AddCopyBufferPass(GraphBuilder, OutClusterDataBuffer, DstOffset, SrcBuffer, 0/*SrcOffset*/, SrcDataSizeInBytes);

				// Place all the cluster data at the same stride for easing cluster data fetching.
				// TODO: Update this to place them at the same boundary or store them into block of element of same size
				//DstOffset += SrcDataSizeInBytes;
				DstOffset += Resource->Header.MaxClusterStrideInBytes;
			}
		}
	}
	else
	{
		OutClusterDataBuffer = GraphBuilder.RegisterExternalBuffer(InClusterDataBuffer);
	}

	return OutClusterDataBuffer;
}

void FRenderCurveSceneExtension::FinishBufferUpload(FRDGBuilder& GraphBuilder,	FRenderCurveSceneParameters* OutParameters)
{
	if (!IsEnabled())
	{
		return;
	}

	bool bUploadResource = false;
	if (bDirtyData)
	{
		check(!Uploader.IsValid());
		Uploader = MakeUnique<FUploader>();
		uint32 CurrentIndex = 0;
		for (auto& Data : Datas)
		{
			const uint32 ClusterOffset = 0u; // TODO upload the offset at which the cluster are uploaded
			const int32 PersistentIndex = Data.PrimitiveSceneInfo->GetPersistentIndex().Index;
			Uploader->InstanceDataUploader.Add(Data.Pack(ClusterOffset), CurrentIndex/*PersistentIndex*/);
			++CurrentIndex;
		}
		bUploadResource = true;
		bDirtyData = false; // check there counldn't be any race here
	}

	FRDGBufferRef RenderCurveInstanceDataBuffer = nullptr;

	const uint32 MinDataSize = Datas.GetMaxIndex() + 1;

	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	if (Uploader.IsValid())
	{
		RenderCurveInstanceDataBuffer = Uploader->InstanceDataUploader.ResizeAndUploadTo(
			GraphBuilder,
			Buffers->RenderCurveInstanceDataBuffer,
			MinDataSize
		);
		Uploader = nullptr;
	}
	else
	{
		RenderCurveInstanceDataBuffer = Buffers->RenderCurveInstanceDataBuffer.ResizeBufferIfNeeded(GraphBuilder, MinDataSize);
	}

	// Create cluster data buffer
	if (!Buffers->ClusterDataBuffer)
	{
		const bool bReservedResource = GRHIGlobals.ReservedResources.Supported;
		check(bReservedResource);

		const uint64 MaxClusterePoolSizeInMB = 512u;
		const uint64 MaxSizeInBytes = MaxClusterePoolSizeInMB << 20;

		FRDGBufferDesc ClusterDataBufferDesc = {};
		ClusterDataBufferDesc = FRDGBufferDesc::CreateByteAddressDesc(MaxSizeInBytes);
		ClusterDataBufferDesc.Usage |= EBufferUsageFlags::ReservedResource;
		Buffers->ClusterDataBuffer = AllocatePooledBuffer(ClusterDataBufferDesc, TEXT("RenderCurve.ClusterData"));

		bUploadResource = true;
	}

	// Upload cluster data 
	FRDGBufferRef ClusterDataBuffer = nullptr;
	if (bUploadResource)
	{
		ClusterDataBuffer = UploadUniqueCurveResource(GraphBuilder, Datas, Buffers->ClusterDataBuffer, Header.TotalClusterCount, Header.ClusterStideInBytes);
	}
	else
	{
		ClusterDataBuffer = GraphBuilder.RegisterExternalBuffer(Buffers->ClusterDataBuffer);
	}

	if (OutParameters != nullptr)
	{
		static_assert(RENDERCURVE_PRIMITIVE_DATA_STRIDE_IN_BYTES == sizeof(FPackedRenderCurveInstanceData));

		OutParameters->InstanceCount = Datas.Num();
		OutParameters->RenderCurveInstanceData = GraphBuilder.CreateSRV(RenderCurveInstanceDataBuffer);
		OutParameters->ClusterData = GraphBuilder.CreateSRV(ClusterDataBuffer);
		OutParameters->ClusterCount = Header.TotalClusterCount;
		OutParameters->MaxClusterStrideInBytes = Header.ClusterStideInBytes;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Updater

ISceneExtensionUpdater* FRenderCurveSceneExtension::CreateUpdater()
{
	return new FUpdater(*this);
}

FRenderCurveSceneExtension::FUpdater::FUpdater(FRenderCurveSceneExtension& InSceneData)
: SceneData(&InSceneData)
{

}
void FRenderCurveSceneExtension::FUpdater::End()
{
	// Ensure these tasks finish before we fall out of scope.
	// NOTE: This should be unnecessary if the updater shares the graph builder's lifetime but we don't enforce that
	//SceneData->SyncAllTasks();
}

void FRenderCurveSceneExtension::FUpdater::PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
{
	// If there was a pending upload from a prior update (due to the buffer never being used), finish the upload now.
	// This keeps the upload entries from growing unbounded and prevents any undefined behavior caused by any
	// updates that overlap primitives.
	SceneData->FinishBufferUpload(GraphBuilder, nullptr);

	// Update whether or not we are enabled based on in Nanite is enabled
	SceneData->SetEnabled(InternalIsEnabled(SceneData->Scene));

	if (!SceneData->IsEnabled())
	{
		return;
	}

	// Remove and free transform data for removed primitives
	// NOTE: Using the ID list instead of the primitive list since we're in an async task
	for (const auto& PersistentIndex : ChangeSet.RemovedPrimitiveIds)
	{
		if (SceneData->Datas.IsValidIndex(PersistentIndex.Index))
		{
			FRenderCurveSceneExtension::FData& Data = SceneData->Datas[PersistentIndex.Index];	
			SceneData->Datas.RemoveAt(PersistentIndex.Index);
			SceneData->bDirtyData = true;
		}
	}
}

void FRenderCurveSceneExtension::FUpdater::PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
{
	for (auto PrimitiveSceneInfo : ChangeSet.AddedPrimitiveSceneInfos)
	{
		if (!PrimitiveSceneInfo->Proxy)
		{
			continue;
		}

		if (FRenderCurveResourceData* CurveResourceData = PrimitiveSceneInfo->Proxy->GetRenderCurveResourceData())
		{
			const int32 PersistentIndex = PrimitiveSceneInfo->GetPersistentIndex().Index;
			FData NewHeader;
			NewHeader.PrimitiveSceneInfo = PrimitiveSceneInfo;
			NewHeader.CurveResourceData = CurveResourceData;
			SceneData->Datas.EmplaceAt(PersistentIndex, NewHeader);
			SceneData->bDirtyData = true;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Renderer

ISceneExtensionRenderer* FRenderCurveSceneExtension::CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags)
{
	return new FRenderer(InSceneRenderer, *this);
}

static void GetRenderCurveSceneParameters(FRDGBuilder& GraphBuilder, FRenderCurveSceneExtension* InSceneData, FRenderCurveSceneParameters& OutParameters)
{
	auto DefaultBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, 4u));
	OutParameters.RenderCurveInstanceData = DefaultBuffer;
	OutParameters.ClusterData = DefaultBuffer;
	OutParameters.InstanceCount = InSceneData ? InSceneData->GetInstanceCount() : 0u;
}

static void GetDefaultRenderCurveSceneParameters(FRenderCurveSceneParameters& OutParameters, FRDGBuilder& GraphBuilder)
{
	GetRenderCurveSceneParameters(GraphBuilder, nullptr, OutParameters);
}

IMPLEMENT_SCENE_UB_STRUCT(FRenderCurveSceneParameters, RenderCurve, GetDefaultRenderCurveSceneParameters);

void FRenderCurveSceneExtension::FRenderer::UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer)
{
	FRenderCurveSceneParameters Parameters;
	SceneData->FinishBufferUpload(GraphBuilder, &Parameters);
	SceneUniformBuffer.Set(SceneUB::RenderCurve, Parameters);
}


FRenderCurveSceneExtension::FBuffers::FBuffers() 
: RenderCurveInstanceDataBuffer(sizeof(FPackedRenderCurveInstanceData) * 32/* = default number of instance count */, TEXT("RenderCurve.Scene.RenderCurveInstanceDataBuffer"))
{
}

FRenderCurveSceneExtension::FPackedRenderCurveInstanceData FRenderCurveSceneExtension::FData::Pack(uint32 InClusterOffset) const
{
	const FBoxSphereBounds Bound = PrimitiveSceneInfo->Proxy->GetBounds();

	FRenderCurveSceneExtension::FPackedRenderCurveInstanceData Out;
	Out.PersistentIndex 		= PrimitiveSceneInfo->GetPersistentIndex().Index;
	Out.InstanceSceneDataOffset = PrimitiveSceneInfo->GetInstanceSceneDataOffset();
	Out.ClusterOffset 			= InClusterOffset;
	Out.ClusterCount 			= CurveResourceData->Header.ClusterCount;
	return Out;
}

///////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace RenderCurve