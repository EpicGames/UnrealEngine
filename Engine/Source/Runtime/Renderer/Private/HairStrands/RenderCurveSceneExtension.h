// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneExtensions.h"
#include "SpanAllocator.h"
#include "RendererPrivateUtils.h"

struct FRenderCurveResourceData;

namespace RenderCurve
{

class FRenderCurveSceneParameters;

class FRenderCurveSceneExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FRenderCurveSceneExtension);

public:
	static bool ShouldCreateExtension(FScene& Scene);
	uint32 GetInstanceCount() const;
	uint32 GetClusterCount() const;
	bool IsEnabled() const;
	void SetEnabled(bool In);
	void FinishBufferUpload(FRDGBuilder& GraphBuilder, FRenderCurveSceneParameters* OutParams);

	//~ Begin ISceneExtension Interface.
	virtual void InitExtension(FScene& InScene) override;
	virtual ISceneExtensionUpdater* CreateUpdater() override;
	virtual ISceneExtensionRenderer* CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags) override;
	//~ End ISceneExtension Interface.

	explicit FRenderCurveSceneExtension(FScene& InScene);
	virtual ~FRenderCurveSceneExtension();

public:
	class FUpdater : public ISceneExtensionUpdater
	{
		DECLARE_SCENE_EXTENSION_UPDATER(FUpdater, ISceneExtensionUpdater);

	public:
		FUpdater(FRenderCurveSceneExtension& InSceneData);
		virtual void End();
		virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override;
		virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override;
	private:
		FRenderCurveSceneExtension* SceneData = nullptr;
	};

	class FRenderer : public ISceneExtensionRenderer
	{
		DECLARE_SCENE_EXTENSION_RENDERER(FRenderer, FRenderCurveSceneExtension);

	public:
		FRenderer(FSceneRendererBase& InSceneRenderer, FRenderCurveSceneExtension& InSceneData) : ISceneExtensionRenderer(InSceneRenderer), SceneData(&InSceneData) {}
		//~ Begin ISceneExtensionRenderer Interface.
		virtual void UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer) override;
		//~ End ISceneExtensionRenderer Interface.
	private:
		FRenderCurveSceneExtension* SceneData = nullptr;
	};

	struct FPackedRenderCurveInstanceData
	{
		uint32 PersistentIndex = 0;
		uint32 InstanceSceneDataOffset = 0;
		uint32 ClusterOffset = 0;
		uint32 ClusterCount = 0;
	};

	struct FData
	{
		FPrimitiveSceneInfo* PrimitiveSceneInfo	= nullptr;
		FRenderCurveResourceData* CurveResourceData = nullptr;
		FPackedRenderCurveInstanceData Pack(uint32 InClusterOffset) const;
	};

	struct FHeader
	{
		uint32 TotalClusterCount = 0;
		uint32 ClusterStideInBytes = 0;
	};

	class FBuffers
	{
	public:
		FBuffers();
		TPersistentByteAddressBuffer<FPackedRenderCurveInstanceData> RenderCurveInstanceDataBuffer;
		TRefCountPtr<FRDGPooledBuffer> ClusterDataBuffer;
	};
	
	class FUploader
	{
	public:
		TByteAddressBufferScatterUploader<FPackedRenderCurveInstanceData> InstanceDataUploader;
	};

private:
	bool bIsEnabled = true;
	bool bDirtyData = false;
	FHeader Header;
	TSparseArray<FData> Datas;
	TUniquePtr<FBuffers> Buffers;
	TUniquePtr<FUploader> Uploader;
};

} // namespace RenderCurve