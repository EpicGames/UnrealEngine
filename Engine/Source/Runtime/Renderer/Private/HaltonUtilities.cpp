// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HaltonUtilities.cpp: Class implementation for optimized evaluation of
	Halton sequences.
=============================================================================*/

// Copyright (c) 2012 Leonhard Gruenschloss (leonhard@gruenschloss.org)
//
// Permission is hereby granted, free of charge, to any person obtaining a copy
// of this software and associated documentation files (the "Software"), to deal
// in the Software without restriction, including without limitation the rights to
// use, copy, modify, merge, publish, distribute, sublicense, and/or sell copies
// of the Software, and to permit persons to whom the Software is furnished to do
// so, subject to the following conditions:
//
// The above copyright notice and this permission notice shall be included in
// all copies or substantial portions of the Software.
//
// THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
// IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
// FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
// AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
// LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
// OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE
// SOFTWARE.

// This file has been adapted to UE

#include "HaltonUtilities.h"

#define PERMUTATION_BUFFER TArray

static float asfloat(unsigned int integer)
{
	union {
		unsigned int u;
		float f;
	} value;
	value.u = integer;
	return value.f;
}

#include "HaltonUtilities.ush"

/**
 * Binds dimensions to consecutive Halton sequences. Note that sequences based on primes
 * 17 and 19 show strong correlation problems among the first 16 pairs. Historical 
 * recommendations to avoid such artifacts is to avoid using the first entries.
 */
float FHaltonSequence::Sample(int Dimension, unsigned int Index) const
{
	switch (Dimension)
	{
	case 0: return Halton_Sample3(Index, FaurePermutations, FaurePermutationOffsets);
	case 1: return Halton_Sample5(Index, FaurePermutations, FaurePermutationOffsets);
	case 2: return Halton_Sample7(Index, FaurePermutations, FaurePermutationOffsets);
	case 3: return Halton_Sample11(Index, FaurePermutations, FaurePermutationOffsets);
	case 4: return Halton_Sample13(Index, FaurePermutations, FaurePermutationOffsets);
	case 5: return Halton_Sample17(Index, FaurePermutations, FaurePermutationOffsets);
	case 6: return Halton_Sample19(Index, FaurePermutations, FaurePermutationOffsets);
	case 7: return Halton_Sample23(Index, FaurePermutations, FaurePermutationOffsets);
	case 8: return Halton_Sample29(Index, FaurePermutations, FaurePermutationOffsets);
	case 9: return Halton_Sample31(Index, FaurePermutations, FaurePermutationOffsets);
	case 10: return Halton_Sample37(Index, FaurePermutations, FaurePermutationOffsets);
	case 11: return Halton_Sample41(Index, FaurePermutations, FaurePermutationOffsets);
	case 12: return Halton_Sample43(Index, FaurePermutations, FaurePermutationOffsets);
	case 13: return Halton_Sample47(Index, FaurePermutations, FaurePermutationOffsets);
	case 14: return Halton_Sample53(Index, FaurePermutations, FaurePermutationOffsets);
	case 15: return Halton_Sample59(Index, FaurePermutations, FaurePermutationOffsets);
	case 16: return Halton_Sample61(Index, FaurePermutations, FaurePermutationOffsets);
	case 17: return Halton_Sample67(Index, FaurePermutations, FaurePermutationOffsets);
	case 18: return Halton_Sample71(Index, FaurePermutations, FaurePermutationOffsets);
	case 19: return Halton_Sample73(Index, FaurePermutations, FaurePermutationOffsets);
	case 20: return Halton_Sample79(Index, FaurePermutations, FaurePermutationOffsets);
	case 21: return Halton_Sample83(Index, FaurePermutations, FaurePermutationOffsets);
	case 22: return Halton_Sample89(Index, FaurePermutations, FaurePermutationOffsets);
	case 23: return Halton_Sample97(Index, FaurePermutations, FaurePermutationOffsets);
	case 24: return Halton_Sample101(Index, FaurePermutations, FaurePermutationOffsets);
	case 25: return Halton_Sample103(Index, FaurePermutations, FaurePermutationOffsets);
	case 26: return Halton_Sample107(Index, FaurePermutations, FaurePermutationOffsets);
	case 27: return Halton_Sample109(Index, FaurePermutations, FaurePermutationOffsets);
	case 28: return Halton_Sample113(Index, FaurePermutations, FaurePermutationOffsets);
	case 29: return Halton_Sample127(Index, FaurePermutations, FaurePermutationOffsets);
	case 30: return Halton_Sample131(Index, FaurePermutations, FaurePermutationOffsets);
	case 31: return Halton_Sample137(Index, FaurePermutations, FaurePermutationOffsets);
	case 32: return Halton_Sample139(Index, FaurePermutations, FaurePermutationOffsets);
	case 33: return Halton_Sample149(Index, FaurePermutations, FaurePermutationOffsets);
	case 34: return Halton_Sample151(Index, FaurePermutations, FaurePermutationOffsets);
	case 35: return Halton_Sample157(Index, FaurePermutations, FaurePermutationOffsets);
	case 36: return Halton_Sample163(Index, FaurePermutations, FaurePermutationOffsets);
	case 37: return Halton_Sample167(Index, FaurePermutations, FaurePermutationOffsets);
	case 38: return Halton_Sample173(Index, FaurePermutations, FaurePermutationOffsets);
	case 39: return Halton_Sample179(Index, FaurePermutations, FaurePermutationOffsets);
	case 40: return Halton_Sample181(Index, FaurePermutations, FaurePermutationOffsets);
	case 41: return Halton_Sample191(Index, FaurePermutations, FaurePermutationOffsets);
	case 42: return Halton_Sample193(Index, FaurePermutations, FaurePermutationOffsets);
	case 43: return Halton_Sample197(Index, FaurePermutations, FaurePermutationOffsets);
	case 44: return Halton_Sample199(Index, FaurePermutations, FaurePermutationOffsets);
	case 45: return Halton_Sample211(Index, FaurePermutations, FaurePermutationOffsets);
	case 46: return Halton_Sample223(Index, FaurePermutations, FaurePermutationOffsets);
	case 47: return Halton_Sample227(Index, FaurePermutations, FaurePermutationOffsets);
	case 48: return Halton_Sample229(Index, FaurePermutations, FaurePermutationOffsets);
	case 49: return Halton_Sample233(Index, FaurePermutations, FaurePermutationOffsets);
	case 50: return Halton_Sample239(Index, FaurePermutations, FaurePermutationOffsets);
	case 51: return Halton_Sample241(Index, FaurePermutations, FaurePermutationOffsets);
	case 52: return Halton_Sample251(Index, FaurePermutations, FaurePermutationOffsets);
	case 53: return Halton_Sample257(Index, FaurePermutations, FaurePermutationOffsets);
	case 54: return Halton_Sample263(Index, FaurePermutations, FaurePermutationOffsets);
	case 55: return Halton_Sample269(Index, FaurePermutations, FaurePermutationOffsets);
	case 56: return Halton_Sample271(Index, FaurePermutations, FaurePermutationOffsets);
	case 57: return Halton_Sample277(Index, FaurePermutations, FaurePermutationOffsets);
	case 58: return Halton_Sample281(Index, FaurePermutations, FaurePermutationOffsets);
	case 59: return Halton_Sample283(Index, FaurePermutations, FaurePermutationOffsets);
	case 60: return Halton_Sample293(Index, FaurePermutations, FaurePermutationOffsets);
	case 61: return Halton_Sample307(Index, FaurePermutations, FaurePermutationOffsets);
	case 62: return Halton_Sample311(Index, FaurePermutations, FaurePermutationOffsets);
	default:
	case 63: return Halton_Sample313(Index, FaurePermutations, FaurePermutationOffsets);
	}
}

FHaltonSequence::FHaltonSequence()
{
	// Initialize Faure permutations
	const int MaxBase = 313;// 137
	TArray<TArray<int> > Permutations;
	Permutations.SetNum(MaxBase + 1);
	// Identity permutations for base 1, 2, 3
	for (int k = 1; k <= 3; ++k)
	{
		Permutations[k].SetNum(k);
		for (int i = 0; i < k; ++i)
		{
			Permutations[k][i] = i;
		}
	}

	for (int Base = 4; Base <= MaxBase; ++Base)
	{
		Permutations[Base].SetNum(Base);
		int b = Base / 2;
		if (Base & 1) // odd
		{
			for (int i = 0; i < Base - 1; ++i)
			{
				Permutations[Base][i + (i >= b)] = Permutations[Base - 1][i] + (Permutations[Base - 1][i] >= b);
			}
			Permutations[Base][b] = b;
		}
		else // even
		{
			for (int i = 0; i < b; ++i)
			{
				Permutations[Base][i] = 2 * Permutations[b][i];
				Permutations[Base][b + i] = 2 * Permutations[b][i] + 1;
			}
		}
	}

	InitTables(Permutations);
}

int FHaltonSequence::Invert(int Base, int Digits, int Index, const TArray<int>& Permutation)
{
	int Result = 0;
	for (int i = 0; i < Digits; ++i)
	{
		Result = Result * Base + Permutation[Index % Base];
		Index /= Base;
	}
	return Result;
}

void FHaltonSequence::FlattenPermutation(const TArray<int>& Permutation)
{
	FaurePermutationOffsets.Add(FaurePermutations.Num());
	FaurePermutations.Append(Permutation);
}

void FHaltonSequence::BuildPermutation(int Size, int Base, int Digits, const TArray<TArray<int>>& Permutations)
{
	TArray<int> Permutation;
	Permutation.SetNum(Size);
	for (int Index = 0; Index < Size; ++Index)
	{
		Permutation[Index] = Invert(Base, Digits, Index, Permutations[Base]);
	}
	
	FlattenPermutation(Permutation);
}

inline void FHaltonSequence::InitTables(const TArray<TArray<int>>& Permutations)
{
	// Dimension 1
	BuildPermutation(243, 3, 5, Permutations);
	BuildPermutation(125, 5, 3, Permutations);
	BuildPermutation(343, 7, 3, Permutations);
	BuildPermutation(121, 11, 2, Permutations);

	// Dimension 5
	BuildPermutation(169, 13, 2, Permutations);
	BuildPermutation(289, 17, 2, Permutations);
	BuildPermutation(361, 19, 2, Permutations);
	BuildPermutation(23, 23, 1, Permutations);
	BuildPermutation(29, 29, 1, Permutations);

	// Dimension 10
	BuildPermutation(31, 31, 1, Permutations);
	BuildPermutation(37, 37, 1, Permutations);
	BuildPermutation(41, 41, 1, Permutations);
	BuildPermutation(43, 43, 1, Permutations);
	BuildPermutation(47, 47, 1, Permutations);

	// Dimension 15
	BuildPermutation(53, 53, 1, Permutations);
	BuildPermutation(59, 59, 1, Permutations);
	BuildPermutation(61, 61, 1, Permutations);
	BuildPermutation(67, 67, 1, Permutations);
	BuildPermutation(71, 71, 1, Permutations);

	// Dimension 20
	BuildPermutation(73, 73, 1, Permutations);
	BuildPermutation(79, 79, 1, Permutations);
	BuildPermutation(83, 83, 1, Permutations);
	BuildPermutation(89, 89, 1, Permutations);
	BuildPermutation(97, 97, 1, Permutations);

	// Dimension 25
	BuildPermutation(101, 101, 1, Permutations);
	BuildPermutation(103, 103, 1, Permutations);
	BuildPermutation(107, 107, 1, Permutations);
	BuildPermutation(109, 109, 1, Permutations);
	BuildPermutation(113, 113, 1, Permutations);

	// Dimension 30
	BuildPermutation(127, 127, 1, Permutations);
	BuildPermutation(131, 131, 1, Permutations);
	BuildPermutation(137, 137, 1, Permutations);
	BuildPermutation(139, 139, 1, Permutations);
	BuildPermutation(149, 149, 1, Permutations);

	// Dimension 35
	BuildPermutation(151, 151, 1, Permutations);
	BuildPermutation(157, 157, 1, Permutations);
	BuildPermutation(163, 163, 1, Permutations);
	BuildPermutation(167, 167, 1, Permutations);
	BuildPermutation(173, 173, 1, Permutations);

	// Dimension 40
	BuildPermutation(179, 179, 1, Permutations);
	BuildPermutation(181, 181, 1, Permutations);
	BuildPermutation(191, 191, 1, Permutations);
	BuildPermutation(193, 193, 1, Permutations);
	BuildPermutation(197, 197, 1, Permutations);

	// Dimension 45
	BuildPermutation(199, 199, 1, Permutations);
	BuildPermutation(211, 211, 1, Permutations);
	BuildPermutation(223, 223, 1, Permutations);
	BuildPermutation(227, 227, 1, Permutations);
	BuildPermutation(229, 229, 1, Permutations);

	// Dimension 50
	BuildPermutation(233, 233, 1, Permutations);
	BuildPermutation(239, 239, 1, Permutations);
	BuildPermutation(241, 241, 1, Permutations);
	BuildPermutation(251, 251, 1, Permutations);
	BuildPermutation(257, 257, 1, Permutations);

	// Dimension 55
	BuildPermutation(263, 263, 1, Permutations);
	BuildPermutation(269, 269, 1, Permutations);
	BuildPermutation(271, 271, 1, Permutations);
	BuildPermutation(277, 277, 1, Permutations);
	BuildPermutation(281, 281, 1, Permutations);

	// Dimension 60
	BuildPermutation(283, 283, 1, Permutations);
	BuildPermutation(293, 293, 1, Permutations);
	BuildPermutation(307, 307, 1, Permutations);
	BuildPermutation(311, 311, 1, Permutations);
	BuildPermutation(313, 313, 1, Permutations);

	// Consider adding support for 128 dimensions
}
