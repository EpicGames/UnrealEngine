// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HaltonUtilities.h: Class definition for optimized evaluation of Halton
	sequences.
=============================================================================*/

#pragma once

// Copyright (c) 2012 Leonhard Gruenschloss (leonhard@gruenschloss.org)
//
// Permission is hereby granted, free of charge, to any person obtaining a copy
// of this software and associated documentation files (the "Software"), to deal
// in the Software without restriction, including without limitation the rights to
// use, copy, modify, merge, publish, distribute, sublicense, and/or sell copies
// of the Software, and to permit persons to whom the Software is furnished to do
// so, subject to the following conditions:
//
// The above copyright notice and this permission notice shall be included in
// all copies or substantial portions of the Software.
//
// THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND, EXPRESS OR
// IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF MERCHANTABILITY,
// FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT. IN NO EVENT SHALL THE
// AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR ANY CLAIM, DAMAGES OR OTHER
// LIABILITY, WHETHER IN AN ACTION OF CONTRACT, TORT OR OTHERWISE, ARISING FROM,
// OUT OF OR IN CONNECTION WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE
// SOFTWARE.

// This file has been adapted to UE

#include "Containers/Array.h"

/**
 * Builds Faure permutation sets to support up to 64 dimensions. The permutation sets
 * are linearized for quick upload to the GPU is desired.
 *
 * Consider adding extended dimensionality support if necessary.
 */
class FHaltonSequence
{
public:
	FHaltonSequence();

	static uint32 GetNumberOfDimensions() { return 64; }

	float Sample(int Dimension, unsigned int Index) const;

	const TArray<int>& GetFaurePermutations() const { return FaurePermutations; }
	const TArray<int>& GetFaurePermutationOffsets() const { return FaurePermutationOffsets; }

private:
	int Invert(int Base, int Digits, int Index, const TArray<int>& Permutation);
	void InitTables(const TArray<TArray<int>>& Permutations);
	void BuildPermutation(int Size, int Base, int Digits, const TArray<TArray<int>>& Permutations);
	void FlattenPermutation(const TArray<int>& PermutationLocal);

	TArray<int> FaurePermutations;
	TArray<int> FaurePermutationOffsets;
};
