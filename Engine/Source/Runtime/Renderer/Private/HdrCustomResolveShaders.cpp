// Copyright Epic Games, Inc. All Rights Reserved.

#include "HdrCustomResolveShaders.h"

IMPLEMENT_SHADER_TYPE(,FHdrCustomResolveVS,TEXT("/Engine/Private/HdrCustomResolveShaders.usf"),TEXT("HdrCustomResolveVS"),SF_Vertex);
IMPLEMENT_SHADER_TYPE(,FHdrCustomResolve2xPS,TEXT("/Engine/Private/HdrCustomResolveShaders.usf"),TEXT("HdrCustomResolve2xPS"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FHdrCustomResolve4xPS,TEXT("/Engine/Private/HdrCustomResolveShaders.usf"),TEXT("HdrCustomResolve4xPS"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FHdrCustomResolve8xPS,TEXT("/Engine/Private/HdrCustomResolveShaders.usf"),TEXT("HdrCustomResolve8xPS"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FHdrCustomResolveFMask2xPS,TEXT("/Engine/Private/HdrCustomResolveShaders.usf"),TEXT("HdrCustomResolveFMaskPS"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FHdrCustomResolveFMask4xPS,TEXT("/Engine/Private/HdrCustomResolveShaders.usf"),TEXT("HdrCustomResolveFMaskPS"),SF_Pixel);
IMPLEMENT_SHADER_TYPE(,FHdrCustomResolveFMask8xPS,TEXT("/Engine/Private/HdrCustomResolveShaders.usf"),TEXT("HdrCustomResolveFMaskPS"),SF_Pixel);

IMPLEMENT_SHADER_TYPE(, FHdrCustomResolveArrayVS, TEXT("/Engine/Private/HdrCustomResolveShaders.usf"), TEXT("HdrCustomResolveVS"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(, FHdrCustomResolveArray2xPS, TEXT("/Engine/Private/HdrCustomResolveShaders.usf"), TEXT("HdrCustomResolve2xPS"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FHdrCustomResolveArray4xPS, TEXT("/Engine/Private/HdrCustomResolveShaders.usf"), TEXT("HdrCustomResolve4xPS"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(, FHdrCustomResolveArray8xPS, TEXT("/Engine/Private/HdrCustomResolveShaders.usf"), TEXT("HdrCustomResolve8xPS"), SF_Pixel);
