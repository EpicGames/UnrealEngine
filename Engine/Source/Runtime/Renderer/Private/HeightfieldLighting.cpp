// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeightfieldLighting.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "Materials/Material.h"
#include "GlobalShader.h"
#include "MaterialShaderType.h"
#include "MaterialShader.h"
#include "DistanceFieldLightingShared.h"
#include "ScreenRendering.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "LightRendering.h"
#include "PipelineStateCache.h"

// In float4's, must match usf
static const int32 HEIGHTFIELD_DATA_STRIDE = 12;

void FillHeightfieldDescriptionData(const TArray<FHeightfieldComponentDescription>& HeightfieldDescriptions, TArray<FVector4f, SceneRenderingAllocator>& HeightfieldDescriptionData)
{
	HeightfieldDescriptionData.Empty(HeightfieldDescriptions.Num() * HEIGHTFIELD_DATA_STRIDE);

	for (int32 DescriptionIndex = 0; DescriptionIndex < HeightfieldDescriptions.Num(); DescriptionIndex++)
	{
		const FHeightfieldComponentDescription& Description = HeightfieldDescriptions[DescriptionIndex];

		FVector4f HeightfieldScaleBias = Description.HeightfieldScaleBias;
		check(HeightfieldScaleBias.X > 0);

		// CalculateHeightfieldOcclusionCS needs to be fixed up if other values are ever supported
		check(Description.NumSubsections == 1 || Description.NumSubsections == 2);

		// Store the presence of subsections in the sign bit
		HeightfieldScaleBias.X *= Description.NumSubsections > 1 ? -1 : 1;

		HeightfieldDescriptionData.Add(HeightfieldScaleBias);
		HeightfieldDescriptionData.Add(Description.MinMaxUV);

		HeightfieldDescriptionData.Add(FVector4f(Description.HeightfieldRect.Size().X, Description.HeightfieldRect.Size().Y, 1.f / Description.HeightfieldRect.Size().X, 1.f / Description.HeightfieldRect.Size().Y));

		const FDFVector3 WorldPosition(Description.LocalToWorld.GetOrigin());

		// Inverse on FMatrix44f can generate NaNs if the source matrix contains large scaling, so do it in double precision.
		const FMatrix LocalToRelativeWorld = FDFMatrix::MakeToRelativeWorldMatrixDouble(FVector(WorldPosition.High), Description.LocalToWorld);

		HeightfieldDescriptionData.Add(WorldPosition.High);

		const FMatrix44f WorldToLocalT = FMatrix44f(LocalToRelativeWorld.Inverse().GetTransposed());

		HeightfieldDescriptionData.Add(*(FVector4f*)&WorldToLocalT.M[0]);
		HeightfieldDescriptionData.Add(*(FVector4f*)&WorldToLocalT.M[1]);
		HeightfieldDescriptionData.Add(*(FVector4f*)&WorldToLocalT.M[2]);

		const FMatrix44f LocalToWorldT = FMatrix44f(LocalToRelativeWorld.GetTransposed());

		HeightfieldDescriptionData.Add(*(FVector4f*)&LocalToWorldT.M[0]);
		HeightfieldDescriptionData.Add(*(FVector4f*)&LocalToWorldT.M[1]);
		HeightfieldDescriptionData.Add(*(FVector4f*)&LocalToWorldT.M[2]);

		FVector4f ChannelMask(0.f, 0.f, 0.f, 0.f);
		if (Description.VisibilityChannel >= 0 && Description.VisibilityChannel < 4)
		{
			ChannelMask.Component(Description.VisibilityChannel) = 1.f;
		}
		HeightfieldDescriptionData.Add(ChannelMask);

		FVector4f& V0 = HeightfieldDescriptionData.AddDefaulted_GetRef();
		V0.X = *(float*)&Description.GPUSceneInstanceIndex;
		V0.Y = 1.0f / LocalToRelativeWorld.GetMaximumAxisScale();
		V0.Z = 0.0f;
		V0.W = 0.0f;
	}

	check(HeightfieldDescriptionData.Num() % HEIGHTFIELD_DATA_STRIDE == 0);
}

FRDGBufferRef UploadHeightfieldDescriptions(FRDGBuilder& GraphBuilder, const TArray<FHeightfieldComponentDescription>& HeightfieldDescriptions)
{
	TArray<FVector4f, SceneRenderingAllocator> HeightfieldDescriptionData;

	FillHeightfieldDescriptionData(HeightfieldDescriptions, /*out*/ HeightfieldDescriptionData);

	FRDGBufferRef HeightfieldDescriptionsBuffer = CreateUploadBuffer(
		GraphBuilder,
		TEXT("HeightfieldDescriptionsBuffer"),
		sizeof(FVector4f),
		FMath::RoundUpToPowerOfTwo(FMath::Max(HeightfieldDescriptionData.Num(), 1)),
		HeightfieldDescriptionData.GetData(),
		HeightfieldDescriptionData.Num() * HeightfieldDescriptionData.GetTypeSize()
	);

	return HeightfieldDescriptionsBuffer;
}
