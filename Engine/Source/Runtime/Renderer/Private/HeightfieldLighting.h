// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	HeightfieldLighting.h
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "ShaderParameters.h"
#include "Engine/Texture2D.h"
#include "RHIStaticStates.h"
#include "RendererInterface.h"
#include "PrimitiveSceneProxy.h"
#include "RenderGraphResources.h"

class FHeightfieldComponentTextures
{
public:

	FHeightfieldComponentTextures(UTexture2D* InHeightAndNormal, UTexture2D* InVisibility) :
		HeightAndNormal(InHeightAndNormal),
		Visibility(InVisibility)
	{}

	FORCEINLINE bool operator==(FHeightfieldComponentTextures Other) const
	{
		return HeightAndNormal == Other.HeightAndNormal && Visibility == Other.Visibility;
	}

	FORCEINLINE friend uint32 GetTypeHash(FHeightfieldComponentTextures ComponentTextures)
	{
		return GetTypeHash(ComponentTextures.HeightAndNormal);
	}

	UTexture2D* HeightAndNormal;
	UTexture2D* Visibility;
};

class FHeightfieldDescription
{
public:
	TMap<FHeightfieldComponentTextures, TArray<FHeightfieldComponentDescription>> ComponentDescriptions;
};