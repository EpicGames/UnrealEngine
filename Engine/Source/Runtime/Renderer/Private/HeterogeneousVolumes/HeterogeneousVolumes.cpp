// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumes.h"
#include "HeterogeneousVolumeInterface.h"

#include "PixelShaderUtils.h"
#include "PostProcess/PostProcessing.h"
#include "RayTracingDefinitions.h"
#include "RayTracingInstance.h"
#include "RayTracingInstanceBufferUtil.h"
#include "ScenePrivate.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneViewState.h"
#include "Materials/MaterialRenderProxy.h"
#include "DeferredShadingRenderer.h"
#include "ShaderCompilerCore.h"

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumes(
	TEXT("r.HeterogeneousVolumes"),
	1,
	TEXT("Whether to use Heterogeneous volumes by default (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesAllowed(
	TEXT("r.HeterogeneousVolumes.Allowed"),
	1,
	TEXT("Whether Heterogeneous Volumes is allowed by scalability and device profiles (Default = 1)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadows(
	TEXT("r.HeterogeneousVolumes.Shadows"),
	0,
	TEXT("Enables heterogeneous volume-casting shadows (default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarTranslucencyHeterogeneousVolumes(
	TEXT("r.Translucency.HeterogeneousVolumes"),
	0,
	TEXT("Enables composting with heterogeneous volumes when rendering translucency (default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarTranslucencyHeterogeneousVolumesType(
	TEXT("r.Translucency.HeterogeneousVolumes.Type"),
	1,
	TEXT("Determines the type of shadow map used for compositing with translucency\n")
	TEXT("0: Adaptive Volumetric Shadow Map\n")
	TEXT("1: Beer Shadow Map (Default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesDownsampleFactor(
	TEXT("r.HeterogeneousVolumes.DownsampleFactor"),
	1.0,
	TEXT("Downsamples the rendered viewport (Default = 1.0)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesComposition(
	TEXT("r.HeterogeneousVolumes.Composition"),
	0,
	TEXT("Change the order of Heterogeneous Volumes composition (Default = 0)\n")
	TEXT("0: Before Translucency\n")
	TEXT("1: After Translucency\n")
	TEXT("Requires enabling Heterogeneous Volumes Project Setting: 'Composite with Translucency'"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesUpsample(
	TEXT("r.HeterogeneousVolumes.Upsample"),
	2,
	TEXT("Upsampling iterative smoothing (Default = 2)\n")
	TEXT("0: Off\n")
	TEXT("1: Nearest Neighbor\n")
	TEXT("2: Bilinear"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesFilter(
	TEXT("r.HeterogeneousVolumes.Filter"),
	2,
	TEXT("Controls iterative smoothing filter applied during upsampling (Default = 2)\n")
	TEXT("0: Off\n")
	TEXT("1: Bilateral\n")
	TEXT("2: Gaussian 3x3\n")
	TEXT("3: Gaussian 5x5"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesFilterWidth(
	TEXT("r.HeterogeneousVolumes.Filter.Width"),
	3,
	TEXT("Adjusts filter width of bilateral kernel (Default = 3)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowType(
	TEXT("r.HeterogeneousVolumes.Shadows.Type"),
	1,
	TEXT("0: Adaptive Volumetric Shadow Map\n")
	TEXT("1: Beer Shadow Map (Default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowPipeline(
	TEXT("r.HeterogeneousVolumes.Shadows.Pipeline"),
	0,
	TEXT("0: Live-Shading (Default)\n")
	TEXT("1: Preshaded Voxel Grid"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesDebug(
	TEXT("r.HeterogeneousVolumes.Debug"),
	0,
	TEXT("Creates auxillary output buffers for debugging (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesHardwareRayTracing(
	TEXT("r.HeterogeneousVolumes.HardwareRayTracing"),
	0,
	TEXT("Enables hardware ray tracing acceleration (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesIndirectLighting(
	TEXT("r.HeterogeneousVolumes.IndirectLighting"),
	0.0f,
	TEXT("Enables indirect lighting (Default = 0.0)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesIndirectLightingMode(
	TEXT("r.HeterogeneousVolumes.IndirectLighting.Mode"),
	0,
	TEXT("Changes where indirect is accumulated in the pipeline (Default = 0)\n")
	TEXT("0: Off\n")
	TEXT("1: Lighting cache\n")
	TEXT("2: Single-scattering\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesStochasticFiltering(
	TEXT("r.HeterogeneousVolumes.StochasticFiltering"),
	3,
	TEXT("Configures the stochastic filtering kernel (Default = 3)\n")
	TEXT("0: Disabled\n")
	TEXT("1: Constant\n")
	TEXT("2: Linear\n")
	TEXT("3: Cubic"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesJitter(
	TEXT("r.HeterogeneousVolumes.Jitter"),
	1,
	TEXT("Enables jitter when ray marching (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesMaxStepCount(
	TEXT("r.HeterogeneousVolumes.MaxStepCount"),
	512,
	TEXT("The maximum ray-marching step count (Default = 512)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesMaxTraceDistance(
	TEXT("r.HeterogeneousVolumes.MaxTraceDistance"),
	30000.0,
	TEXT("The maximum trace view-distance for direct volume rendering (Default = 30000)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesMaxShadowTraceDistance(
	TEXT("r.HeterogeneousVolumes.MaxShadowTraceDistance"),
	30000.0,
	TEXT("The maximum shadow-trace distance (Default = 30000)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesPreshading(
	TEXT("r.HeterogeneousVolumes.Preshading"),
	0,
	TEXT("Evaluates the material into a canonical preshaded volume before rendering the result (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesPreshadingMipLevel(
	TEXT("r.HeterogeneousVolumes.Preshading.MipLevel"),
	0,
	TEXT("Statically determines the MIP-level when evaluating preshaded volume data (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesPreshadingVolumeResolutionX(
	TEXT("r.HeterogeneousVolumes.VolumeResolution.X"),
	0,
	TEXT("Overrides the preshading and lighting volume resolution in X (Default = 0)")
	TEXT("0: Disabled, uses per-volume attribute\n")
	TEXT(">0: Overrides resolution in X\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesPreshadingVolumeResolutionY(
	TEXT("r.HeterogeneousVolumes.VolumeResolution.Y"),
	0,
	TEXT("Overrides the preshading and lighting volume resolution in X (Default = 0)")
	TEXT("0: Disabled, uses per-volume attribute\n")
	TEXT(">0: Overrides resolution in Y\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesPreshadingVolumeResolutionZ(
	TEXT("r.HeterogeneousVolumes.VolumeResolution.Z"),
	0,
	TEXT("Overrides the preshading and lighting volume resolution in X (Default = 0)")
	TEXT("0: Disabled, uses per-volume attribute\n")
	TEXT(">0: Overrides resolution in Z\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesShadowStepSize(
	TEXT("r.HeterogeneousVolumes.ShadowStepSize"),
	-1.0,
	TEXT("The ray-marching step-size override for shadow rays (Default = -1.0, disabled)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesSparseVoxel(
	TEXT("r.HeterogeneousVolumes.SparseVoxel"),
	0,
	TEXT("Uses sparse-voxel rendering algorithms (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesSparseVoxelGenerationMipBias(
	TEXT("r.HeterogeneousVolumes.SparseVoxel.GenerationMipBias"),
	0,
	TEXT("Determines MIP bias for sparse voxel generation (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesSparseVoxelPerTileCulling(
	TEXT("r.HeterogeneousVolumes.SparseVoxel.PerTileCulling"),
	0,
	TEXT("Enables sparse-voxel culling when using tiled rendering (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesSparseVoxelRefinement(
	TEXT("r.HeterogeneousVolumes.SparseVoxel.Refinement"),
	0,
	TEXT("Uses hierarchical refinement to coalesce neighboring sparse-voxels (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesStepSize(
	TEXT("r.HeterogeneousVolumes.StepSize"),
	-1.0,
	TEXT("The ray-marching step-size override (Default = -1.0, disabled)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesLightingCache(
	TEXT("r.HeterogeneousVolumes.LightingCache"),
	2,
	TEXT("Enables an optimized pre-pass, caching certain volumetric rendering lighting quantities (Default = 2)\n")
	TEXT("0: Disabled\n")
	TEXT("1: Cache transmittance (deprecated)\n")
	TEXT("2: Cache in-scattering\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesLightingCacheUseAVSM(
	TEXT("r.HeterogeneousVolumes.LightingCache.UseAVSM"),
	0,
	TEXT("Enables use of AVSMs when evaluating self-shadowing (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesLightingCacheDownsampleFactor(
	TEXT("r.HeterogeneousVolumes.LightingCache.DownsampleFactor"),
	0,
	TEXT("Overrides the lighting-cache downsample factor, relative to the preshading volume resolution (Default = 0)\n")
	TEXT("0: Disabled, uses per-volume attribute\n")
	TEXT(">0: Overrides the lighting-cache downsample factor"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesDepthSort(
	TEXT("r.HeterogeneousVolumes.DepthSort"),
	1,
	TEXT("Iterates over volumes in depth-sorted order, based on its centroid (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesApplyHeightFog(
	TEXT("r.HeterogeneousVolumes.HeightFog"),
	1,
	TEXT("Applies height fog to Heterogeneous Volumes (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesApplyVolumetricFog(
	TEXT("r.HeterogeneousVolumes.VolumetricFog"),
	1,
	TEXT("Applies volumetric fog to Heterogeneous Volumes (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesApplyFogInscatteringMode(
	TEXT("r.HeterogeneousVolumes.ApplyFogInscattering"),
	2,
	TEXT("Determines the method for applying fog in-scattering (default = 2)\n")
	TEXT("0: Off\n")
	TEXT("1: Reference\n")
	TEXT("2: Linear approximation\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesUseAnalyticDerivatives(
	TEXT("r.HeterogeneousVolumes.UseAnalyticDerivatives"),
	0,
	TEXT("Enables support for analytic derivatives (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesVelocity(
	TEXT("r.HeterogeneousVolumes.Velocity"),
	0,
	TEXT("Writes Heterogeneous Volumes velocity to the feature buffer (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesCLOD(
	TEXT("r.HeterogeneousVolumes.CLOD"),
	1,
	TEXT("Uses Continuous Level-of-Detail to accelerate rendering (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesCLODBias(
	TEXT("r.HeterogeneousVolumes.CLOD.Bias"),
	0.0,
	TEXT("Biases evaluation result when computing Continuous Level-of-Detail (Default = 0.0)\n")
	TEXT("> 0: Coarser\n")
	TEXT("< 0: Sharper\n"),
	ECVF_RenderThreadSafe
);

DECLARE_GPU_STAT_NAMED(HeterogeneousVolumeShadowsStat, TEXT("HeterogeneousVolumeShadows"));
DECLARE_GPU_STAT_NAMED(HeterogeneousVolumesStat, TEXT("HeterogeneousVolumes"));

static bool IsHeterogeneousVolumesEnabled()
{
	return CVarHeterogeneousVolumes.GetValueOnRenderThread() != 0 &&
		CVarHeterogeneousVolumesAllowed.GetValueOnRenderThread() != 0;
}

bool ShouldHeterogeneousVolumesCastShadows()
{
	return CVarHeterogeneousVolumesShadows.GetValueOnAnyThread() != 0;
}

bool ShouldCompositeHeterogeneousVolumesWithTranslucency()
{
	return CVarTranslucencyHeterogeneousVolumes.GetValueOnAnyThread() != 0;
}

namespace HeterogeneousVolumes
{
	EShadowType GetShadowType()
	{
		int32 ShadowValue = FMath::Clamp(CVarHeterogeneousVolumesShadowType.GetValueOnAnyThread(), 0, 1);
		return static_cast<EShadowType>(ShadowValue);
	}

	EShadowType GetTranslucencyCompositingMode()
	{
		int32 ShadowValue = FMath::Clamp(CVarTranslucencyHeterogeneousVolumesType.GetValueOnAnyThread(), 0, 1);
		return static_cast<EShadowType>(ShadowValue);
	}
}

EHeterogeneousVolumesCompositionType GetHeterogeneousVolumesCompositionType()
{
	int32 CompositionOrder = CVarHeterogeneousVolumesComposition.GetValueOnRenderThread();
	switch (CompositionOrder)
	{
		case 0:
		default:
			return EHeterogeneousVolumesCompositionType::BeforeTranslucent;
		case 1:
			return EHeterogeneousVolumesCompositionType::AfterTranslucent;
	}
}

EHeterogeneousVolumesCompositionType GetHeterogeneousVolumesComposition()
{
	// Composition order can only be modified if the Project Setting is enabled
	if (!ShouldCompositeHeterogeneousVolumesWithTranslucency())
	{
		return EHeterogeneousVolumesCompositionType::AfterTranslucent;
	}

	return GetHeterogeneousVolumesCompositionType();
}

bool ShouldRenderHeterogeneousVolumes(
	const FScene* Scene
)
{
	return IsHeterogeneousVolumesEnabled()
		&& Scene != nullptr
		&& DoesPlatformSupportHeterogeneousVolumes(Scene->GetShaderPlatform());
}

bool ShouldRenderHeterogeneousVolumesForAnyView(
	const TArrayView<FViewInfo>& Views
)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		if (ShouldRenderHeterogeneousVolumesForView(Views, ViewIndex))
		{
			return true;
		}
	}

	return false;
}

bool ShouldRenderHeterogeneousVolumesForView(
	const FViewInfo& View
)
{
	return IsHeterogeneousVolumesEnabled()
		&& IStereoRendering::IsAPrimaryView(View)
		&& !View.HeterogeneousVolumesMeshBatches.IsEmpty()
		&& View.Family
		&& !View.bIsReflectionCapture;
}

int32 GetPrimaryViewIndex(const TArrayView<FViewInfo>& Views)
{
	int32 PrimaryViewIndex = Views.Num() - 1;
	while (PrimaryViewIndex >= 0)
	{
		if (IStereoRendering::IsAPrimaryView(Views[PrimaryViewIndex]))
		{
			break;
		}
		PrimaryViewIndex--;
	}

	return PrimaryViewIndex;
}

FViewInfo* GetPrimaryViewForView(
	const TArrayView<FViewInfo>& Views,
	int32 ViewIndex
)
{
	check(ViewIndex < Views.Num())
	FViewInfo* PrimaryView = &Views[ViewIndex];

	// For stereo views, mesh batches will only be defined on the primary
	if (PrimaryView->bIsInstancedStereoEnabled && IStereoRendering::IsStereoEyeView(*PrimaryView) && !IStereoRendering::IsAPrimaryView(*PrimaryView))
	{
		int32 PrimaryViewIndex = GetPrimaryViewIndex(Views);
		check(PrimaryViewIndex >= 0);
		PrimaryView = &Views[PrimaryViewIndex];
	}

	return IStereoRendering::IsAPrimaryView(*PrimaryView) ? PrimaryView : nullptr;
}

bool ShouldRenderHeterogeneousVolumesForView(
	const TArrayView<FViewInfo>& Views,
	int32 ViewIndex
)
{
	FViewInfo* PrimaryView = GetPrimaryViewForView(Views, ViewIndex);
	return PrimaryView ? ShouldRenderHeterogeneousVolumesForView(*PrimaryView) : false;
}

TArray<FVolumetricMeshBatch, SceneRenderingAllocator> GetHeterogeneousVolumesMeshBatches(
	const TArrayView<FViewInfo>& Views,
	int32 ViewIndex
)
{
	FViewInfo* PrimaryView = GetPrimaryViewForView(Views, ViewIndex);
	return PrimaryView ? PrimaryView->HeterogeneousVolumesMeshBatches : TArray<FVolumetricMeshBatch, SceneRenderingAllocator>();
}

bool ShouldRenderHeterogeneousVolumesAsHoldoutForView(
	const FViewInfo& View
)
{
	check(IStereoRendering::IsAPrimaryView(View));

	// This query returns true if any volume is marked as a holdout; otherwise, the query returns false
	if (ShouldRenderHeterogeneousVolumesForView(View))
	{
		const TArray<FVolumetricMeshBatch, SceneRenderingAllocator>& MeshBatches = View.HeterogeneousVolumesMeshBatches;
		for (int32 MeshBatchIndex = 0; MeshBatchIndex < MeshBatches.Num(); ++MeshBatchIndex)
		{
			const FMeshBatch* Mesh = MeshBatches[MeshBatchIndex].Mesh;
			const FPrimitiveSceneProxy* PrimitiveSceneProxy = MeshBatches[MeshBatchIndex].Proxy;
			if (ShouldRenderMeshBatchWithHeterogeneousVolumes(Mesh, PrimitiveSceneProxy, View.GetFeatureLevel()))
			{
				for (int32 VolumeIndex = 0; VolumeIndex < Mesh->Elements.Num(); ++VolumeIndex)
				{
					const IHeterogeneousVolumeInterface* HeterogeneousVolume = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeIndex].UserData;
					if (HeterogeneousVolumes::IsHoldout(HeterogeneousVolume))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

bool DoesMaterialShaderSupportHeterogeneousVolumes(const FMaterialShaderParameters& MaterialShaderParameters)
{
	return (MaterialShaderParameters.MaterialDomain == MD_Volume)
		&& MaterialShaderParameters.bIsUsedWithHeterogeneousVolumes;
}

bool DoesMaterialShaderSupportHeterogeneousVolumes(const FMaterial& Material)
{
	return (Material.GetMaterialDomain() == MD_Volume)
		&& Material.IsUsedWithHeterogeneousVolumes();
}

bool ShouldRenderMeshBatchWithHeterogeneousVolumes(
	const FMeshBatch* Mesh,
	const FPrimitiveSceneProxy* Proxy,
	ERHIFeatureLevel::Type FeatureLevel
)
{
	check(Mesh);
	check(Proxy);
	check(Mesh->MaterialRenderProxy);

	const FMaterialRenderProxy* MaterialRenderProxy = Mesh->MaterialRenderProxy;
	const FMaterial& Material = MaterialRenderProxy->GetMaterialWithFallback(FeatureLevel, MaterialRenderProxy);
	return IsHeterogeneousVolumesEnabled()
		&& Proxy->IsHeterogeneousVolume()
		&& DoesMaterialShaderSupportHeterogeneousVolumes(Material);
}

namespace HeterogeneousVolumes
{
	// CVars
	int32 GetDownsampleFactor()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesDownsampleFactor.GetValueOnRenderThread(), 1, 8);
	}

	int32 GetUpsampleMode()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesUpsample.GetValueOnRenderThread(), 0, 2);
	}

	int32 GetFilterMode()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesFilter.GetValueOnRenderThread(), 0, 3);
	}

	int32 GetFilterWidth()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesFilterWidth.GetValueOnRenderThread(), 0, 11);
	}

	FIntPoint GetDownsampledResolution(FIntPoint Resolution, int32 DownsampleFactor)
	{
		return FIntPoint::DivideAndRoundUp(Resolution, DownsampleFactor);
	}

	FIntPoint GetScaledViewRect(FIntRect ViewRect)
	{
		return GetDownsampledResolution(ViewRect.Size(), GetDownsampleFactor());
	}

	FIntVector GetVolumeResolution(const IHeterogeneousVolumeInterface* Interface)
	{
		FIntVector VolumeResolution = Interface->GetVoxelResolution();

		FIntVector OverrideVolumeResolution = FIntVector(
			CVarHeterogeneousVolumesPreshadingVolumeResolutionX.GetValueOnRenderThread(),
			CVarHeterogeneousVolumesPreshadingVolumeResolutionY.GetValueOnRenderThread(),
			CVarHeterogeneousVolumesPreshadingVolumeResolutionZ.GetValueOnRenderThread());

		VolumeResolution.X = OverrideVolumeResolution.X > 0 ? OverrideVolumeResolution.X : VolumeResolution.X;
		VolumeResolution.Y = OverrideVolumeResolution.Y > 0 ? OverrideVolumeResolution.Y : VolumeResolution.Y;
		VolumeResolution.Z = OverrideVolumeResolution.Z > 0 ? OverrideVolumeResolution.Z : VolumeResolution.Z;

		// Clamp each dimension to [1, 1024]
		VolumeResolution.X = FMath::Clamp(VolumeResolution.X, 1, 1024);
		VolumeResolution.Y = FMath::Clamp(VolumeResolution.Y, 1, 1024);
		VolumeResolution.Z = FMath::Clamp(VolumeResolution.Z, 1, 1024);
		return VolumeResolution;
	}

	float GetShadowStepSize()
	{
		return CVarHeterogeneousVolumesShadowStepSize.GetValueOnRenderThread();
	}

	float GetMaxTraceDistance()
	{
		return CVarHeterogeneousVolumesMaxTraceDistance.GetValueOnRenderThread();
	}

	float GetMaxShadowTraceDistance()
	{
		return CVarHeterogeneousVolumesMaxShadowTraceDistance.GetValueOnRenderThread();
	}

	float GetStepSize()
	{
		return CVarHeterogeneousVolumesStepSize.GetValueOnRenderThread();
	}

	float GetMaxStepCount()
	{
		return CVarHeterogeneousVolumesMaxStepCount.GetValueOnRenderThread();
	}

	int32 GetMipLevel()
	{
		return CVarHeterogeneousVolumesPreshadingMipLevel.GetValueOnRenderThread();
	}

	uint32 GetSparseVoxelMipBias()
	{
		// TODO: Clamp based on texture dimension..
		return FMath::Clamp(CVarHeterogeneousVolumesSparseVoxelGenerationMipBias.GetValueOnRenderThread(), 0, 10);
	}

	int32 GetDebugMode()
	{
		return CVarHeterogeneousVolumesDebug.GetValueOnRenderThread();
	}

	EShadowPipeline GetShadowPipeline()
	{
		int32 ShadowValue = FMath::Clamp(CVarHeterogeneousVolumesShadowPipeline.GetValueOnRenderThread(), 0, 1);
		return static_cast<EShadowPipeline>(ShadowValue);
	}

	EStochasticFilteringMode GetStochasticFilteringMode()
	{
		return static_cast<EStochasticFilteringMode>(CVarHeterogeneousVolumesStochasticFiltering.GetValueOnRenderThread());
	}

	bool UseSparseVoxelPipeline()
	{
		return CVarHeterogeneousVolumesSparseVoxel.GetValueOnAnyThread() != 0;
	}

	bool ShouldRefineSparseVoxels()
	{
		return CVarHeterogeneousVolumesSparseVoxelRefinement.GetValueOnRenderThread() != 0;
	}

	bool UseSparseVoxelPerTileCulling()
	{
		return CVarHeterogeneousVolumesSparseVoxelPerTileCulling.GetValueOnAnyThread() != 0;
	}

	int32 GetLightingCacheMode()
	{
		// Force in-scattering lighting cache for all but cinematic scalability
		if (GetScalabilityMode() != EScalabilityMode::Cinematic)
		{
			return 2;
		}

		int32 Value = FMath::Clamp(CVarHeterogeneousVolumesLightingCache.GetValueOnAnyThread(), 0, 2);
		return Value;
	}

	bool UseAdaptiveVolumetricShadowMapForSelfShadowing(const FPrimitiveSceneProxy* PrimitiveSceneProxy)
	{
		bool bUseAVSM = CVarHeterogeneousVolumesLightingCacheUseAVSM.GetValueOnRenderThread() != 0;
		bool bPrimitiveCastsDynamicShadows = PrimitiveSceneProxy->CastsDynamicShadow();
		return ShouldHeterogeneousVolumesCastShadows() && bUseAVSM && bPrimitiveCastsDynamicShadows;
	}

	bool UseLightingCacheForInscattering()
	{
		return GetLightingCacheMode() == 2;
	}

	bool UseLightingCacheForTransmittance()
	{
		return GetLightingCacheMode() == 1;
	}

	bool ShouldJitter()
	{
		return CVarHeterogeneousVolumesJitter.GetValueOnRenderThread() != 0;
	}

	bool UseHardwareRayTracing()
	{
		return IsRayTracingEnabled()
			&& (CVarHeterogeneousVolumesHardwareRayTracing.GetValueOnRenderThread() != 0);
	}

	bool UseIndirectLighting()
	{
		return CVarHeterogeneousVolumesIndirectLighting.GetValueOnRenderThread() != 0;
	}

	float GetIndirectLightingFactor()
	{
		return FMath::Max(CVarHeterogeneousVolumesIndirectLighting.GetValueOnRenderThread(), 0.0f);
	}

	EIndirectLightingMode GetIndirectLightingMode()
	{
		return static_cast<EIndirectLightingMode>(FMath::Clamp(CVarHeterogeneousVolumesIndirectLightingMode.GetValueOnRenderThread(), 0, 2));
	}

	bool ShouldDepthSort()
	{
		return CVarHeterogeneousVolumesDepthSort.GetValueOnAnyThread() != 0;
	}

	bool ShouldApplyHeightFog()
	{
		return CVarHeterogeneousVolumesApplyHeightFog.GetValueOnRenderThread() != 0;
	}

	bool ShouldApplyVolumetricFog()
	{
		return CVarHeterogeneousVolumesApplyVolumetricFog.GetValueOnRenderThread() != 0;
	}

	EFogMode GetFogInscatteringMode()
	{
		return static_cast<EFogMode>(FMath::Clamp(CVarHeterogeneousVolumesApplyFogInscatteringMode.GetValueOnRenderThread(), 0, 2));
	}

	bool UseAnalyticDerivatives()
	{
		return CVarHeterogeneousVolumesUseAnalyticDerivatives.GetValueOnRenderThread() != 0;
	}

	bool ShouldWriteVelocity()
	{
		return CVarHeterogeneousVolumesVelocity.GetValueOnRenderThread() != 0;
	}

	bool UseContinuousLOD()
	{
		return CVarHeterogeneousVolumesCLOD.GetValueOnRenderThread() != 0;
	}

	float GetCLODBias()
	{
		return CVarHeterogeneousVolumesCLODBias.GetValueOnRenderThread();
	}

	// Convenience Utils
	int GetVoxelCount(FIntVector VolumeResolution)
	{
		return VolumeResolution.X * VolumeResolution.Y * VolumeResolution.Z;
	}

	int GetVoxelCount(const FRDGTextureDesc& TextureDesc)
	{
		return TextureDesc.Extent.X * TextureDesc.Extent.Y * TextureDesc.Depth;
	}

	FIntVector GetMipVolumeResolution(FIntVector VolumeResolution, uint32 MipLevel)
	{
		return FIntVector(
			FMath::Max(VolumeResolution.X >> MipLevel, 1),
			FMath::Max(VolumeResolution.Y >> MipLevel, 1),
			FMath::Max(VolumeResolution.Z >> MipLevel, 1)
		);
	}
	
	float CalcLODBias(const IHeterogeneousVolumeInterface* HeterogeneousVolume)
	{
		return HeterogeneousVolume->GetMipBias() + HeterogeneousVolumes::GetCLODBias();
	}

	FLODValue CalcLOD(
		const HeterogeneousVolumes::FLODInfo& LODInfo,
		const IHeterogeneousVolumeInterface* HeterogeneousVolume
	)
	{
		if (!HeterogeneousVolumes::UseContinuousLOD())
		{
			return FLODValue();
		}

		FBoxSphereBounds WorldBounds = HeterogeneousVolume->GetBounds();
		FIntVector VoxelResolution = HeterogeneousVolume->GetVoxelResolution();
		float VoxelResolutionMin = VoxelResolution.GetMin();

		const float MaxLOD = FMath::Floor(FMath::Log2(VoxelResolutionMin));
		FLODValue LODValue;
		LODValue.LOD = MaxLOD;
		if (!LODInfo.bIsPerspective)
		{
			float VolumeRatio = FVector(LODInfo.WorldSceneBounds.BoxExtent / WorldBounds.BoxExtent).Length();
			float ViewLODValue = FMath::Log2(VolumeRatio) + HeterogeneousVolume->GetMipBias() + HeterogeneousVolumes::GetCLODBias();
			ViewLODValue = FMath::Max(ViewLODValue, 0);

			LODValue.LOD = FMath::Min(ViewLODValue, LODValue.LOD);
		}
		else if (LODInfo.WorldShadowFrustum.IntersectBox(WorldBounds.Origin, WorldBounds.BoxExtent))
		{
			// Determine the pixel-width at the near-plane
			float TanHalfFOV = FMath::Tan(LODInfo.FOV * 0.5);
			float HalfViewWidth = LODInfo.ViewRect.Width() * 0.5 / LODInfo.DownsampleFactor;
			float PixelWidth = TanHalfFOV / HalfViewWidth;

			// Project to nearest distance of volume bounds
			float Distance = FMath::Max(FVector::Dist(WorldBounds.Origin, LODInfo.WorldOrigin) - WorldBounds.SphereRadius, LODInfo.NearClippingDistance);
			float ProjectedPixelWidth = Distance * PixelWidth;

			// MIP is defined as the log of the ratio of native voxel resolution to pixel-coverage of volume bounds
			float PixelWidthCoverage = (2.0 * WorldBounds.BoxExtent.GetMax()) / ProjectedPixelWidth;

			// Clamp LOD to heighten the effect on foreground elements before applying bias controls
			float ViewLODValue = FMath::Max(FMath::Log2(VoxelResolutionMin / PixelWidthCoverage), 0.0f);

			LODValue.LOD = FMath::Min(ViewLODValue, LODValue.LOD);
		}

		float TotalBias = HeterogeneousVolume->GetMipBias() + HeterogeneousVolumes::GetCLODBias();
		LODValue.Bias = FMath::Min(TotalBias, MaxLOD - LODValue.LOD);

		return LODValue;
	}
	
	FLODValue CalcLOD(const FSceneView& View, const IHeterogeneousVolumeInterface* HeterogeneousVolume)
	{
		FLODInfo LODInfo;
		// TODO: Not supporting orthographic projection for now
		LODInfo.bIsPerspective = true;
		LODInfo.WorldSceneBounds = FBoxSphereBounds(EForceInit::ForceInitToZero);

		LODInfo.WorldOrigin = View.ViewMatrices.GetViewOrigin();
		LODInfo.ViewRect = View.UnconstrainedViewRect;
		LODInfo.WorldShadowFrustum = View.ViewFrustum;
		LODInfo.FOV = FMath::DegreesToRadians(View.FOV);
		LODInfo.NearClippingDistance = View.NearClippingDistance;
		LODInfo.DownsampleFactor = HeterogeneousVolumes::GetDownsampleFactor();

		return CalcLOD(LODInfo, HeterogeneousVolume);
	}

	float CalcLODFactor(float LODValue, float LODBias)
	{
		return FMath::Pow(2, LODValue + LODBias);
	}

	float CalcLODFactor(
		const HeterogeneousVolumes::FLODInfo& LODInfo,
		const IHeterogeneousVolumeInterface* HeterogeneousVolume
	)
	{
		FLODValue LODValue = CalcLOD(LODInfo, HeterogeneousVolume);
		return CalcLODFactor(LODValue.LOD, LODValue.Bias);
	}

	float CalcLODFactor(const FSceneView& View, const IHeterogeneousVolumeInterface* HeterogeneousVolume)
	{
		FLODValue LODValue = CalcLOD(View, HeterogeneousVolume);
		return CalcLODFactor(LODValue.LOD, LODValue.Bias);
	}

	FIntVector GetLightingCacheResolution(const IHeterogeneousVolumeInterface* RenderInterface, FLODValue LODValue)
	{
		float LODFactor = CalcLODFactor(LODValue.LOD, 0.0f);
		float OverrideDownsampleFactor = CVarHeterogeneousVolumesLightingCacheDownsampleFactor.GetValueOnRenderThread();
		float DownsampleFactor = OverrideDownsampleFactor > 0.0 ? OverrideDownsampleFactor : RenderInterface->GetLightingDownsampleFactor() * LODFactor;
		DownsampleFactor = FMath::Max(DownsampleFactor, 0.125);

		FVector VolumeResolution = FVector(GetVolumeResolution(RenderInterface));
		FIntVector LightingCacheResolution = FIntVector(VolumeResolution / DownsampleFactor);
		LightingCacheResolution.X = FMath::Clamp(LightingCacheResolution.X, 1, 1024);
		LightingCacheResolution.Y = FMath::Clamp(LightingCacheResolution.Y, 1, 1024);
		LightingCacheResolution.Z = FMath::Clamp(LightingCacheResolution.Z, 1, 512);
		return LightingCacheResolution;
	}

	bool IsHoldout(const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface)
	{
		return IsPostProcessingWithAlphaChannelSupported() && HeterogeneousVolumeInterface->IsHoldout();
	}
}

bool ShouldBuildVoxelGrids(const FScene* Scene)
{
	// TODO: Build the light list once
	if (ShouldHeterogeneousVolumesCastShadows())
	{
		for(const FLightSceneInfoCompact& LightCompactInfo : Scene->Lights)		
		{
			if (LightCompactInfo.LightSceneInfo->Proxy->CastsVolumetricShadow())
			{
				return true;
			}
		}
	}

	if (ShouldCompositeHeterogeneousVolumesWithTranslucency())
	{
		return true;
	}

	if (HeterogeneousVolumes::GetShadowPipeline() == HeterogeneousVolumes::EShadowPipeline::VoxelGrid)
	{
		return true;
	}

	return false;
}

bool ShouldCacheVoxelGrids(const FScene* Scene, FSceneViewState* ViewState)
{
	// If the caching structure exists
	if (ViewState == nullptr)
	{
		return false;
	}

	if (HeterogeneousVolumes::GetShadowPipeline() == HeterogeneousVolumes::EShadowPipeline::VoxelGrid)
	{
		return true;
	}
	
	// TODO: If any light supports ray tracing

	return false;
}

DECLARE_CYCLE_STAT(TEXT("Heterogeneous Volumes Render"), STATGROUP_HeterogeneousVolumesRender, STATGROUP_HeterogeneousVolumesRT);

void RenderHeterogeneousVolumeShadows(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const FSceneTextures& SceneTextures,
	FViewInfo& View,
	const FVisibleLightInfoArray& VisibleLightInfos
)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HeterogeneousVolumeShadowsStat, "HeterogeneousVolumeShadows");
	SCOPED_NAMED_EVENT(HeterogeneousVolumes, FColor::Emerald);

	if (HeterogeneousVolumes::GetShadowPipeline() == HeterogeneousVolumes::EShadowPipeline::LiveShading)
	{
		RenderAdaptiveVolumetricShadowMapWithLiveShading(
			GraphBuilder,
			SceneTextures,
			Scene,
			View,
			VisibleLightInfos
		);
	}
}

void FDeferredShadingSceneRenderer::RenderHeterogeneousVolumeShadows(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures
)
{
	if (!ShouldBuildVoxelGrids(Scene))
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, HeterogeneousVolumeShadowsStat, "HeterogeneousVolumeShadows");
	SCOPED_NAMED_EVENT(HeterogeneousVolumes, FColor::Emerald);

	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters> OrthoGridUniformBuffer = nullptr;
	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters> FrustumGridUniformBuffer = nullptr;
	if (HeterogeneousVolumes::GetShadowPipeline() == HeterogeneousVolumes::EShadowPipeline::VoxelGrid)
	{
		FVoxelGridBuildOptions BuildOptions;
		BuildOptions.VoxelGridBuildMode = EVoxelGridBuildMode::Shadows;
		BuildOptions.ShadingRateInFrustum = HeterogeneousVolumes::GetShadingRateForShadows();
		BuildOptions.ShadingRateOutOfFrustum = HeterogeneousVolumes::GetOutOfFrustumShadingRateForShadows();
		BuildOptions.bBuildOrthoGrid = true;
		BuildOptions.bBuildFrustumGrid = false;
		BuildOptions.bUseProjectedPixelSizeForOrthoGrid = true;
		BuildOptions.bJitter = HeterogeneousVolumes::EnableJitterForShadows();

		BuildOrthoVoxelGrid(GraphBuilder, Scene, Views, VisibleLightInfos, BuildOptions, OrthoGridUniformBuffer);
		BuildFrustumVoxelGrid(GraphBuilder, Scene, Views[0], BuildOptions, FrustumGridUniformBuffer);
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];

		if (ShouldCompositeHeterogeneousVolumesWithTranslucency() &&
			HeterogeneousVolumes::GetTranslucencyCompositingMode() == HeterogeneousVolumes::EShadowType::AdaptiveVolumetricShadowMap)
		{
			if (HeterogeneousVolumes::GetShadowPipeline() == HeterogeneousVolumes::EShadowPipeline::LiveShading)
			{
				RenderAdaptiveVolumetricCameraMapWithLiveShading(
					GraphBuilder,
					SceneTextures,
					Scene,
					View
				);
			}
			else
			{
				RenderAdaptiveVolumetricCameraMapWithVoxelGrid(
					GraphBuilder,
					// Scene data
					SceneTextures,
					Scene,
					View,
					// Volume data
					OrthoGridUniformBuffer,
					FrustumGridUniformBuffer
				);
			}
		}

		if (ShouldHeterogeneousVolumesCastShadows())
		{
			if (HeterogeneousVolumes::GetShadowPipeline() == HeterogeneousVolumes::EShadowPipeline::LiveShading)
			{
				// This path is taken care of now in ShadowDepthRendering
#if 0
				RenderAdaptiveVolumetricShadowMapWithLiveShading(
					GraphBuilder,
					// Scene data
					SceneTextures,
					Scene,
					View,
					// Light data
					VisibleLightInfos
				);
#endif
			}
			else
			{
				RenderAdaptiveVolumetricShadowMapWithVoxelGrid(
					GraphBuilder,
					// Scene data
					SceneTextures,
					Scene,
					View,
					// Shadow Data
					VisibleLightInfos,
					VirtualShadowMapArray,
					// Volume data
					OrthoGridUniformBuffer,
					FrustumGridUniformBuffer
				);
			}
		}
	}

	FSceneViewState* ViewState = Views[0].ViewState;
	if (ShouldCacheVoxelGrids(Scene, ViewState))
	{
		ViewState->OrthoVoxelGridUniformBuffer = OrthoGridUniformBuffer;
		ViewState->FrustumVoxelGridUniformBuffer = FrustumGridUniformBuffer;
	}
}

void FDeferredShadingSceneRenderer::RenderHeterogeneousVolumes(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures
)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HeterogeneousVolumesStat, "HeterogeneousVolumes");
	SCOPED_NAMED_EVENT(HeterogeneousVolumes, FColor::Emerald);

	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters> OrthoGridUniformBuffer = HeterogeneousVolumes::GetOrthoVoxelGridUniformBuffer(GraphBuilder, Views[0].ViewState);
	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters> FrustumGridUniformBuffer = HeterogeneousVolumes::GetFrustumVoxelGridUniformBuffer(GraphBuilder, Views[0].ViewState);

	FRDGTextureRef HeterogeneousVolumeRadiance = GSystemTextures.GetBlackDummy(GraphBuilder);
	FRDGTextureRef HeterogeneousVolumeHoldout = GSystemTextures.GetBlackDummy(GraphBuilder);
	FRDGTextureRef HeterogeneousVolumeBeerShadowMap = GSystemTextures.GetBlackDummy(GraphBuilder);
	FRDGTextureRef HeterogeneousVolumeVelocity = SceneTextures.Velocity;
	if (ShouldRenderHeterogeneousVolumesForAnyView(Views))
	{
		FRDGTextureDesc Desc = SceneTextures.Color.Target->Desc;
		Desc.Extent = HeterogeneousVolumes::GetDownsampledResolution(Desc.Extent, HeterogeneousVolumes::GetDownsampleFactor());
		Desc.Format = PF_FloatRGBA;
		Desc.Flags &= ~(TexCreate_FastVRAM);
		HeterogeneousVolumeRadiance = GraphBuilder.CreateTexture(Desc, TEXT("HeterogeneousVolumes"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HeterogeneousVolumeRadiance), FLinearColor::Black);

		if (IsDeferredPrimitiveAlphaHoldoutSupported())
		{
			Desc.Format = PF_R8;
			HeterogeneousVolumeHoldout = GraphBuilder.CreateTexture(Desc, TEXT("HeterogeneousVolume.Holdout"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HeterogeneousVolumeHoldout), FLinearColor::Black);
		}

		if (ShouldCompositeHeterogeneousVolumesWithTranslucency() &&
			HeterogeneousVolumes::GetTranslucencyCompositingMode() == HeterogeneousVolumes::EShadowType::BeerShadowMap)
		{
			Desc.Format = PF_FloatRGBA;
			HeterogeneousVolumeBeerShadowMap = GraphBuilder.CreateTexture(Desc, TEXT("HeterogeneousVolume.BeerShadowMap"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HeterogeneousVolumeBeerShadowMap), FVector4d(HeterogeneousVolumes::GetMaxTraceDistance(), 0.0, 1.0, 1.0));
		}
		else
		{
			Desc.Extent = FIntPoint(1);
			HeterogeneousVolumeBeerShadowMap = GraphBuilder.CreateTexture(Desc, TEXT("HeterogeneousVolume.BeerShadowMap"));
		}
#if 0
		// Create velocity texture if it doesn't already exist
		if (!HasBeenProduced(HeterogeneousVolumeVelocity))
		{
			bool bRequireMultiView = false; // ViewFamily required
			FRDGTextureDesc VelocityDesc = FVelocityRendering::GetRenderTargetDesc(Scene->GetShaderPlatform(), SceneTextures.Color.Target->Desc.Extent, bRequireMultiView);
			HeterogeneousVolumeVelocity = GraphBuilder.CreateTexture(VelocityDesc, TEXT("HeterogeneousVolume.Velocity"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HeterogeneousVolumeVelocity), FLinearColor::Black);
		}
#endif
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];
		if (ShouldRenderHeterogeneousVolumesForView(Views, ViewIndex))
		{
			if (HeterogeneousVolumes::GetDebugMode() != 0)
			{
				// TODO: Replace with single-scattering voxel grid implementation.
				RenderTransmittanceWithVoxelGrid(
					GraphBuilder,
					SceneTextures,
					Scene,
					View,
					OrthoGridUniformBuffer,
					FrustumGridUniformBuffer,
					HeterogeneousVolumeRadiance
				);
			}
			else
			{
				// Collect volume interfaces
				struct VolumeMesh
				{
					const IHeterogeneousVolumeInterface* Volume;
					const FMaterialRenderProxy* MaterialRenderProxy;

					VolumeMesh(const IHeterogeneousVolumeInterface* V, const FMaterialRenderProxy* M) : 
						Volume(V),
						MaterialRenderProxy(M)
					{}
				};

				TArray<VolumeMesh> VolumeMeshes;

				const TArray<FVolumetricMeshBatch, SceneRenderingAllocator>& MeshBatches = GetHeterogeneousVolumesMeshBatches(Views, ViewIndex);
				for (int32 MeshBatchIndex = 0; MeshBatchIndex < MeshBatches.Num(); ++MeshBatchIndex)
				{
					const FMeshBatch* Mesh = MeshBatches[MeshBatchIndex].Mesh;
					const FPrimitiveSceneProxy* PrimitiveSceneProxy = MeshBatches[MeshBatchIndex].Proxy;
					if (!ShouldRenderMeshBatchWithHeterogeneousVolumes(Mesh, PrimitiveSceneProxy, View.GetFeatureLevel()))
					{
						continue;
					}

					const FMaterialRenderProxy* MaterialRenderProxy = Mesh->MaterialRenderProxy;
					for (int32 VolumeIndex = 0; VolumeIndex < Mesh->Elements.Num(); ++VolumeIndex)
					{
						const IHeterogeneousVolumeInterface* HeterogeneousVolume = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeIndex].UserData;
						//check(HeterogeneousVolume != nullptr);
						if (HeterogeneousVolume == nullptr)
						{
							continue;
						}

						VolumeMeshes.Add(VolumeMesh(HeterogeneousVolume, MaterialRenderProxy));
					}
				}

				// Provide coarse depth-sorting, based on camera-distance to world centroid
				if (HeterogeneousVolumes::ShouldDepthSort())
				{
					struct FDepthCompareHeterogeneousVolumes
					{
						FVector WorldCameraOrigin;
						FDepthCompareHeterogeneousVolumes(FViewInfo& View) : WorldCameraOrigin(View.ViewMatrices.GetViewOrigin()) {}

						FORCEINLINE bool operator()(const VolumeMesh& A, const VolumeMesh& B) const
						{
							FVector CameraToA = A.Volume->GetBounds().Origin - WorldCameraOrigin;
							float SquaredDistanceToA = FVector::DotProduct(CameraToA, CameraToA);

							FVector CameraToB = B.Volume->GetBounds().Origin - WorldCameraOrigin;
							float SquaredDistanceToB = FVector::DotProduct(CameraToB, CameraToB);

							return SquaredDistanceToA < SquaredDistanceToB;
						}
					};

					VolumeMeshes.Sort(FDepthCompareHeterogeneousVolumes(View));
				}

				for (int32 VolumeIndex = 0; VolumeIndex < VolumeMeshes.Num(); ++VolumeIndex)
				{
					const IHeterogeneousVolumeInterface* HeterogeneousVolume = VolumeMeshes[VolumeIndex].Volume;
					const FMaterialRenderProxy* MaterialRenderProxy = VolumeMeshes[VolumeIndex].MaterialRenderProxy;
					const FPrimitiveSceneProxy* PrimitiveSceneProxy = HeterogeneousVolume->GetPrimitiveSceneProxy();
					const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
					const FPersistentPrimitiveIndex PrimitiveId = PrimitiveSceneInfo->GetPersistentIndex();
					const FBoxSphereBounds LocalBoxSphereBounds = HeterogeneousVolume->GetLocalBounds();

					RDG_EVENT_SCOPE(GraphBuilder, "%s [%d]", *HeterogeneousVolume->GetReadableName(), VolumeIndex);

					// Allocate transmittance volume
					FRDGTextureRef LightingCacheTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
					if (HeterogeneousVolumes::GetLightingCacheMode() != 0)
					{
						// TODO: Allow option for scalar transmittance to conserve bandwidth
						HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolume);
						FIntVector LightingCacheResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolume, LODValue);
						uint32 NumMips = FMath::Log2(float(FMath::Min(FMath::Min(LightingCacheResolution.X, LightingCacheResolution.Y), LightingCacheResolution.Z))) + 1;
						FRDGTextureDesc LightingCacheDesc = FRDGTextureDesc::Create3D(
							LightingCacheResolution,
							!IsMetalPlatform(GShaderPlatformForFeatureLevel[View.FeatureLevel]) ? PF_FloatR11G11B10 : PF_FloatRGBA,
							FClearValueBinding::Black,
							TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling,
							NumMips
						);
						LightingCacheTexture = GraphBuilder.CreateTexture(LightingCacheDesc, TEXT("HeterogeneousVolumes.LightingCacheTexture"));
						AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(LightingCacheTexture), FLinearColor::Black);
					}

					// Material baking executes a pre-shading pipeline
					if (CVarHeterogeneousVolumesPreshading.GetValueOnRenderThread())
					{
						RenderWithPreshading(
							GraphBuilder,
							SceneTextures,
							Scene,
							View, ViewIndex,
							// Shadow Data
							VisibleLightInfos,
							VirtualShadowMapArray,
							// Object data
							HeterogeneousVolume,
							MaterialRenderProxy,
							PrimitiveId,
							LocalBoxSphereBounds,
							// Transmittance accleration
							LightingCacheTexture,
							// Output
							HeterogeneousVolumeRadiance
						);
					}
					// Otherwise execute a live-shading pipeline
					else
					{
						RenderWithLiveShading(
							GraphBuilder,
							SceneTextures,
							Scene,
							View, ViewIndex,
							// Shadow Data
							VisibleLightInfos,
							VirtualShadowMapArray,
							// Object Data
							HeterogeneousVolume,
							MaterialRenderProxy,
							PrimitiveId,
							LocalBoxSphereBounds,
							// Transmittance accleration
							LightingCacheTexture,
							// Output
							HeterogeneousVolumeRadiance,
							HeterogeneousVolumeVelocity,
							HeterogeneousVolumeHoldout,
							HeterogeneousVolumeBeerShadowMap
						);

						// Validate that a view state exists to store the AVSM
						if (View.ViewState)
						{
							// Stash the temporary beer-law shadow map as a camera shadow if we aren't already creating one
							if (ShouldCompositeHeterogeneousVolumesWithTranslucency() &&
								HeterogeneousVolumes::GetTranslucencyCompositingMode() == HeterogeneousVolumes::EShadowType::BeerShadowMap)
							{
								// Resolution
								FIntPoint ShadowMapResolution = HeterogeneousVolumes::GetDownsampledResolution(View.ViewRect.Size(), HeterogeneousVolumes::GetDownsampleFactor());
								FRDGBufferRef VolumetricShadowIndirectionBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMIndirectionPackedData));
								FRDGBufferRef VolumetricShadowSampleBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMSamplePackedData));

								ConvertBeerLawShadowMapToVolumetricShadowMap(
									GraphBuilder,
									View,
									//GroupCount,
									ShadowMapResolution,
									HeterogeneousVolumeBeerShadowMap,
									VolumetricShadowIndirectionBuffer,
									VolumetricShadowSampleBuffer
								);

								// Transform
								const FMatrix ProjectionMatrix = View.ViewMatrices.GetViewToClip();
								float FOV = FMath::Atan(1.0f / ProjectionMatrix.M[0][0]);
								FMatrix ViewToClip = FPerspectiveMatrix(
									FOV,
									ShadowMapResolution.X,
									ShadowMapResolution.Y,
									1.0,
									HeterogeneousVolumes::GetMaxTraceDistance()
								);
								FMatrix ClipToView = ViewToClip.Inverse();
								FMatrix ScreenMatrix = FScaleMatrix(FVector(0.5, -0.5, -0.5)) * FTranslationMatrix(FVector(0.5, 0.5, 0.5));

								int32 NumShadowMatrices = 1;
								FMatrix44f TranslatedWorldToShadow[] = {
									FMatrix44f(View.ViewMatrices.GetTranslatedViewMatrix() * ViewToClip * ScreenMatrix)
								};
								FMatrix44f ShadowToTranslatedWorld[] = {
									TranslatedWorldToShadow[0].Inverse()
								};
								FVector3f TranslatedWorldOrigin[] = {
									ShadowToTranslatedWorld[0].GetOrigin()
								};
								FVector4f TranslatedWorldPlane[] = {
									FVector4f::Zero()
								};
								FVector4f SplitDepths[] = {
									FVector4f::Zero()
								};

								// Generic data
								int32 MaxSampleCount = 2;
								float DownsampleFactor = HeterogeneousVolumes::GetDownsampleFactor();
								bool bIsDirectionalLight = false;
								FRDGBufferRef VolumetricShadowLinkedListBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMLinkedListPackedData));

								// Uniform buffer
								CreateAdaptiveVolumetricShadowMapUniformBufferParameters(
									GraphBuilder,
									TranslatedWorldOrigin,
									TranslatedWorldPlane,
									TranslatedWorldToShadow,
									SplitDepths,
									ShadowMapResolution,
									DownsampleFactor,
									NumShadowMatrices,
									MaxSampleCount,
									bIsDirectionalLight,
									VolumetricShadowLinkedListBuffer,
									VolumetricShadowIndirectionBuffer,
									VolumetricShadowSampleBuffer,
									View.ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters
								);
							}

							// Append beauty image and build uniform buffer
							if (ShouldCompositeHeterogeneousVolumesWithTranslucency() && View.ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters)
							{
								// Append beauty image
								View.ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters->RadianceTexture = GraphBuilder.CreateSRV(HeterogeneousVolumeRadiance);
								View.ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters->TextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

								View.ViewState->AdaptiveVolumetricCameraMapUniformBuffer =
									GraphBuilder.CreateUniformBuffer(View.ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters);
							}
						}
					}
				}
			}

			View.HeterogeneousVolumeRadiance = HeterogeneousVolumeRadiance;
			View.HeterogeneousVolumeHoldout = HeterogeneousVolumeHoldout;
			View.HeterogeneousVolumeBeerShadowMap = HeterogeneousVolumeBeerShadowMap;
		}
	}
}

class FHeterogeneousVolumesUpsampleCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHeterogeneousVolumesUpsampleCS);
	SHADER_USE_PARAMETER_STRUCT(FHeterogeneousVolumesUpsampleCS, FGlobalShader);

	class FUpsampleMode : SHADER_PERMUTATION_INT("UPSAMPLE_MODE", 3);
	using FPermutationDomain = TShaderPermutationDomain<FUpsampleMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		// Texture data
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, HeterogeneousVolumeRadiance)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		SHADER_PARAMETER(FIntPoint, UpsampledResolution)
		SHADER_PARAMETER(FIntPoint, UpsampledViewRect)
		SHADER_PARAMETER(FIntPoint, UpsampledViewRectMin)
		SHADER_PARAMETER(int, DownsampleFactor)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWUpsampledTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.template Get<FUpsampleMode>() == 0)
		{
			return false;
		}

		// Apply conditional project settings for Heterogeneous volumes?
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		//OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		//OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		//OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
};

IMPLEMENT_GLOBAL_SHADER(FHeterogeneousVolumesUpsampleCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesComposite.usf", "HeterogeneousVolumesUpsampleCS", SF_Compute);

class FHeterogeneousVolumesFilterCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHeterogeneousVolumesFilterCS);
	SHADER_USE_PARAMETER_STRUCT(FHeterogeneousVolumesFilterCS, FGlobalShader);

	class FFilterMode : SHADER_PERMUTATION_INT("FILTER_MODE", 4);
	using FPermutationDomain = TShaderPermutationDomain<FFilterMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		// Volume data
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, HeterogeneousVolumeRadiance)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		SHADER_PARAMETER(FIntPoint, UpsampledResolution)
		SHADER_PARAMETER(FIntPoint, UpsampledViewRect)
		SHADER_PARAMETER(FIntPoint, UpsampledViewRectMin)
		SHADER_PARAMETER(int, DownsampleFactor)
		SHADER_PARAMETER(int, FilterWidth)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWFilteredTexture)
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.template Get<FFilterMode>() == 0)
		{
			return false;
		}

		// Apply conditional project settings for Heterogeneous volumes?
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		//OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
};

IMPLEMENT_GLOBAL_SHADER(FHeterogeneousVolumesFilterCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesComposite.usf", "HeterogeneousVolumesFilterCS", SF_Compute);


class FHeterogeneousVolumesCompositeCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHeterogeneousVolumesCompositeCS);
	SHADER_USE_PARAMETER_STRUCT(FHeterogeneousVolumesCompositeCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		// Volume data
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, HeterogeneousVolumeRadiance)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float>, HeterogeneousVolumeHoldout)

		// Transmittance structure
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)
		SHADER_PARAMETER(int32, bUseAVSM)

		// Dispatch data
		SHADER_PARAMETER(FIntVector, GroupCount)
		SHADER_PARAMETER(int, DownsampleFactor)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWColorTexture)
		END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		// Apply conditional project settings for Heterogeneous volumes?
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		//OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
};

IMPLEMENT_GLOBAL_SHADER(FHeterogeneousVolumesCompositeCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesComposite.usf", "HeterogeneousVolumesCompositeCS", SF_Compute);


void FDeferredShadingSceneRenderer::CompositeHeterogeneousVolumes(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures
)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HeterogeneousVolumesStat, "HeterogeneousVolumes");
	SCOPED_NAMED_EVENT(HeterogeneousVolumes, FColor::Emerald);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];

		if (ShouldRenderHeterogeneousVolumesForView(Views, ViewIndex))
		{
			FRDGTextureDesc UpsampleDesc = View.HeterogeneousVolumeRadiance->Desc;
			int32 DownsampleFactor = HeterogeneousVolumes::GetDownsampleFactor();

			bool bIterativeUpsampling = HeterogeneousVolumes::GetUpsampleMode() != 0;
			while (bIterativeUpsampling && (DownsampleFactor > 1))
			{
				int32 DownsampleIterationFactor = (DownsampleFactor % 3 == 0) ? 3 : 2;

				FIntPoint DownsampledViewRect = FIntPoint::DivideAndRoundUp(View.ViewRect.Size(), DownsampleFactor);
				FIntPoint DownsampledViewRectMin = FIntPoint::DivideAndRoundUp(View.ViewRect.Min, DownsampleFactor);
				FIntPoint UpsampledViewRect = DownsampledViewRect * DownsampleIterationFactor;
				FIntPoint UpsampledViewRectMin = DownsampledViewRectMin * DownsampleIterationFactor;
				FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(UpsampledViewRect, FHeterogeneousVolumesUpsampleCS::GetThreadGroupSize2D());
				UpsampleDesc.Extent *= DownsampleIterationFactor;

				// Upsample
				{
					FRDGTextureRef HeterogeneousVolumeUpsample = GraphBuilder.CreateTexture(UpsampleDesc, TEXT("HeterogeneousVolumeUpsample"));

					FHeterogeneousVolumesUpsampleCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHeterogeneousVolumesUpsampleCS::FParameters>();
					{
						// Scene data
						PassParameters->View = View.ViewUniformBuffer;
						PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);

						// Texture data
						PassParameters->HeterogeneousVolumeRadiance = View.HeterogeneousVolumeRadiance;
						PassParameters->TextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
						PassParameters->UpsampledResolution = UpsampleDesc.Extent;
						PassParameters->UpsampledViewRect = UpsampledViewRect;
						PassParameters->UpsampledViewRectMin = UpsampledViewRectMin;
						PassParameters->DownsampleFactor = DownsampleIterationFactor;

						// Output
						PassParameters->RWUpsampledTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeUpsample);
					}

					FHeterogeneousVolumesUpsampleCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FHeterogeneousVolumesUpsampleCS::FUpsampleMode>(HeterogeneousVolumes::GetUpsampleMode());
					TShaderRef<FHeterogeneousVolumesUpsampleCS> ComputeShader = View.ShaderMap->GetShader<FHeterogeneousVolumesUpsampleCS>(PermutationVector);
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("FHeterogeneousVolumesUpsampleCS"),
						ComputeShader,
						PassParameters,
						GroupCount);

					View.HeterogeneousVolumeRadiance = HeterogeneousVolumeUpsample;
				}

				// Filter neighborhood
				if (HeterogeneousVolumes::GetFilterMode() != 0)
				{
					FRDGTextureRef HeterogeneousVolumeFilter = GraphBuilder.CreateTexture(UpsampleDesc, TEXT("HeterogeneousVolumeFilter"));

					FHeterogeneousVolumesFilterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHeterogeneousVolumesFilterCS::FParameters>();
					{
						// Scene data
						PassParameters->View = View.ViewUniformBuffer;
						PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
						// Volume data
						PassParameters->HeterogeneousVolumeRadiance = View.HeterogeneousVolumeRadiance;
						PassParameters->TextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
						PassParameters->UpsampledResolution = UpsampleDesc.Extent;
						PassParameters->UpsampledViewRect = UpsampledViewRect;
						PassParameters->UpsampledViewRectMin = UpsampledViewRectMin;
						PassParameters->DownsampleFactor = 1;
						PassParameters->FilterWidth = HeterogeneousVolumes::GetFilterWidth();

						// Output
						PassParameters->RWFilteredTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeFilter);
					}

					FHeterogeneousVolumesFilterCS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FHeterogeneousVolumesFilterCS::FFilterMode>(HeterogeneousVolumes::GetFilterMode());
					TShaderRef<FHeterogeneousVolumesFilterCS> ComputeShader = View.ShaderMap->GetShader<FHeterogeneousVolumesFilterCS>(PermutationVector);
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("FHeterogeneousVolumesFilterCS"),
						ComputeShader,
						PassParameters,
						GroupCount);

					View.HeterogeneousVolumeRadiance = HeterogeneousVolumeFilter;
				}

				DownsampleFactor /= DownsampleIterationFactor;
			}

			// Composite
			{
				FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), FHeterogeneousVolumesUpsampleCS::GetThreadGroupSize2D());
				FHeterogeneousVolumesCompositeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHeterogeneousVolumesCompositeCS::FParameters>();
				{
					// Scene data
					PassParameters->View = View.ViewUniformBuffer;
					PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
					// Volume data
					PassParameters->HeterogeneousVolumeRadiance = View.HeterogeneousVolumeRadiance;
					PassParameters->HeterogeneousVolumeHoldout = View.HeterogeneousVolumeHoldout;
					// Transmittance structure
					PassParameters->bUseAVSM = ShouldCompositeHeterogeneousVolumesWithTranslucency();
					PassParameters->AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricCameraMapUniformBuffer(GraphBuilder, View.ViewState);
					// Dispatch data
					PassParameters->GroupCount = GroupCount;
					//PassParameters->DownsampleFactor = HeterogeneousVolumes::GetDownsampleFactor();
					PassParameters->DownsampleFactor = DownsampleFactor;

					// Output
					PassParameters->RWColorTexture = GraphBuilder.CreateUAV(SceneTextures.Color.Target);
				}

				TShaderRef<FHeterogeneousVolumesCompositeCS> ComputeShader = View.ShaderMap->GetShader<FHeterogeneousVolumesCompositeCS>();
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("FHeterogeneousVolumesCompositeCS"),
					ComputeShader,
					PassParameters,
					GroupCount);
			}
		}
	}
}

namespace HeterogeneousVolumes
{
	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> GatherLights(const FScene* Scene, const FViewInfo& View, const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface)
	{
		TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact;
		for (const FLightSceneInfoCompact& LightCompactInfo : Scene->Lights)
		{
			if (SupportsLightType(LightCompactInfo.LightType) &&
				(LightCompactInfo.LightSceneInfo->Proxy->GetViewLightingChannelMask() & View.ViewLightingChannelMask) &&
				LightCompactInfo.AffectsPrimitive(HeterogeneousVolumeInterface->GetBounds(), HeterogeneousVolumeInterface->GetPrimitiveSceneProxy()))
			{
				LightSceneInfoCompact.Add(LightCompactInfo);
			}
		}
		return LightSceneInfoCompact;
	}

	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> GatherShadowLights(const FScene* Scene, uint32 ViewLightingChannelMask)
	{
		TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact;
		for (const FLightSceneInfoCompact& LightCompactInfo : Scene->Lights)
		{
			// TODO: Use global bounds information..
			//if (LightCompactInfo.AffectsPrimitive(...))
			// TODO: For view/light masking, should we check if the light is visible in any view,
			// instead of just the first view? For now, we assume all views share the same channel mask.
			if (SupportsShadowForLightType(LightCompactInfo.LightType) &&
				(LightCompactInfo.LightSceneInfo->Proxy->GetViewLightingChannelMask() & ViewLightingChannelMask))
			{
				LightSceneInfoCompact.Add(LightCompactInfo);
			}
		}
		return LightSceneInfoCompact;
	}

	void PostRender(FScene& Scene, TArray<FViewInfo>& Views)
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FSceneViewState* ViewState = Views[ViewIndex].ViewState;
			if (ViewState)
			{
				ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters = nullptr;
				DestroyAdaptiveVolumetricShadowMapUniformBuffer(ViewState->AdaptiveVolumetricCameraMapUniformBuffer);

				for (TPair<FLightSceneId, TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>>& Pair : ViewState->AdaptiveVolumetricShadowMapUniformBufferMap)
				{
					DestroyAdaptiveVolumetricShadowMapUniformBuffer(Pair.Value);
				}
				ViewState->AdaptiveVolumetricShadowMapUniformBufferMap.Empty();
				ViewState->BeerShadowMapUniformBufferMap.Empty();
			}
		}
	}
}
