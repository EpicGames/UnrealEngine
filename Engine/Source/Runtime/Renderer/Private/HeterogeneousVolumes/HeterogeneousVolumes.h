// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "ConvexVolume.h"
#include "LightSceneInfo.h"
#include "RendererInterface.h"
#include "ShaderParameterMacros.h"
#include "SceneTexturesConfig.h"
#include "HeterogeneousVolumeInterface.h"
#include "ScenePrivateBase.h"

class FLightSceneInfo;
class FPrimitiveSceneProxy;
class FProjectedShadowInfo;
class FRayTracingScene;
class FRDGBuilder;
class FScene;
class FSceneView;
class FSceneViewState;
class FViewInfo;
class FVirtualShadowMapArray;
class FVisibleLightInfo;
class IHeterogeneousVolumeInterface;

struct FMaterialShaderParameters;
struct FRDGTextureDesc;
struct FSceneTextures;
struct FPersistentPrimitiveIndex;
struct FVolumetricMeshBatch;

//
// External API
//

bool ShouldRenderHeterogeneousVolumes(const FScene* Scene);
bool ShouldRenderHeterogeneousVolumesForAnyView(const TArrayView<FViewInfo>& Views);
bool ShouldRenderHeterogeneousVolumesForView(const FViewInfo& View);
bool ShouldRenderHeterogeneousVolumesAsHoldoutForView(const FViewInfo& View);
bool ShouldRenderHeterogeneousVolumesForView(const TArrayView<FViewInfo>& Views, int32 ViewIndex);
bool DoesMaterialShaderSupportHeterogeneousVolumes(const FMaterialShaderParameters& Parameters);
bool DoesMaterialShaderSupportHeterogeneousVolumes(const FMaterial& Material);
bool ShouldRenderMeshBatchWithHeterogeneousVolumes(
	const FMeshBatch* Mesh,
	const FPrimitiveSceneProxy* Proxy,
	ERHIFeatureLevel::Type FeatureLevel
);

bool ShouldCompositeHeterogeneousVolumesWithTranslucency();
bool ShouldHeterogeneousVolumesCastShadows();

enum class EHeterogeneousVolumesCompositionType : uint8
{
	BeforeTranslucent,
	AfterTranslucent
};
EHeterogeneousVolumesCompositionType GetHeterogeneousVolumesComposition();

//
// Internal API
//

namespace HeterogeneousVolumes
{
	// CVars
	FIntVector GetVolumeResolution(const IHeterogeneousVolumeInterface*);

	int32 GetDownsampleFactor();
	FIntPoint GetDownsampledResolution(FIntPoint Resolution, int32 DownsampleFactor);
	FIntPoint GetScaledViewRect(FIntRect ViewRect);
	float GetShadowStepSize();
	float GetMaxTraceDistance();
	float GetMaxShadowTraceDistance();
	float GetStepSize();
	float GetMaxStepCount();
	float GetMinimumVoxelSizeInFrustum();
	float GetMinimumVoxelSizeOutsideFrustum();
	float GetShadingRateForFrustumGrid();
	float GetShadingRateForOrthoGrid();

	enum class EScalabilityMode
	{
		Low,
		High,
		Epic,
		Cinematic
	};

	EScalabilityMode GetScalabilityMode();

	// Shadow generation
	enum class EShadowType : uint8
	{
		AdaptiveVolumetricShadowMap,
		BeerShadowMap
	};
	EShadowType GetShadowType();

	enum class EShadowPipeline
	{
		LiveShading,
		VoxelGrid
	};
	EShadowPipeline GetShadowPipeline();
	FIntPoint GetShadowMapResolution();
	float GetShadingRateForShadows();
	float GetOutOfFrustumShadingRateForShadows();
	bool EnableJitterForShadows();
	float GetStepSizeForShadows();
	uint32 GetShadowMaxSampleCount();
	float GetShadowAbsoluteErrorThreshold();
	float GetShadowRelativeErrorThreshold();
	bool UseAVSMCompression();
	float GetCameraDownsampleFactor();
	float GetIndirectLightingFactor();
	enum class EIndirectLightingMode
	{
		Disabled,
		LightingCachePass,
		SingleScatteringPass
	};
	EIndirectLightingMode GetIndirectLightingMode();

	// Translucency compositing
	EShadowType GetTranslucencyCompositingMode();

	int32 GetMipLevel();
	int32 GetDebugMode();
	int32 GetLightingCacheMode();
	uint32 GetSparseVoxelMipBias();
	int32 GetBottomLevelGridResolution();
	int32 GetIndirectionGridResolution();
	enum class EStochasticFilteringMode
	{
		Disabled,
		Constant,
		Linear,
		Cubic
	};
	EStochasticFilteringMode GetStochasticFilteringMode();
	
	bool ShouldJitter();
	bool ShouldRefineSparseVoxels();
	bool UseHardwareRayTracing();
	bool UseIndirectLighting();
	bool UseSparseVoxelPipeline();
	bool UseSparseVoxelPerTileCulling();
	bool UseLightingCacheForInscattering();
	bool UseLightingCacheForTransmittance();
	bool UseAdaptiveVolumetricShadowMapForSelfShadowing(const FPrimitiveSceneProxy* PrimitiveSceneProxy);
	bool ShouldDepthSort();
	bool ShouldApplyHeightFog();
	bool ShouldApplyVolumetricFog();
	bool SupportsOverlappingVolumes();
	bool EnableAmbientOcclusion();
	bool UseExistenceMask();
	bool UseAnalyticDerivatives();
	bool SupportsLightType(uint32 LightType);
	bool SupportsShadowForLightType(uint32 LightType);
	bool SupportsCascadeShadowsForDirectionalLight();

	// Helper functions to cull lights, used internally
	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> GatherLights(const FScene* Scene, const FViewInfo& View, const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface);
	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> GatherShadowLights(const FScene* Scene, uint32 ViewLightingChannelMask);
	
	enum class ECascadeShadowMode
	{
		Disabled,
		Frustums,
		Clipmaps,
		Autofit
	};
	ECascadeShadowMode GetCascadeShadowMode();
	int32 GetCascadeShadowsQuantizationUnit();
	
	enum class EFogMode
	{
		Disabled,
		Reference,
		LinearApprox
	};
	EFogMode GetFogInscatteringMode();
	bool ShouldWriteVelocity();

	bool EnableIndirectionGrid();
	bool EnableLinearInterpolation();

	// Convenience Utils
	int GetVoxelCount(FIntVector VolumeResolution);
	int GetVoxelCount(const FRDGTextureDesc& TextureDesc);
	FIntVector GetMipVolumeResolution(FIntVector VolumeResolution, uint32 MipLevel);

	struct FLODInfo
	{
		// Orthographic projection
		FBoxSphereBounds WorldSceneBounds = FBoxSphereBounds(EForceInit::ForceInit);

		// Perspective projection
		FVector WorldOrigin = FVector::ZeroVector;
		FIntRect ViewRect;

		FConvexVolume WorldShadowFrustum;
		float FOV = PI / 4.0f;
		float NearClippingDistance = 1.0f;
		float DownsampleFactor = 1.0f;

		// Projection type
		bool bIsPerspective = false;
	};

	struct FLODValue
	{
		float LOD = 0.0f;
		float Bias = 0.0f;
	};

	FLODValue CalcLOD(const HeterogeneousVolumes::FLODInfo& LODInfo, const IHeterogeneousVolumeInterface* HeterogeneousVolume);
	FLODValue CalcLOD(const FSceneView& View, const IHeterogeneousVolumeInterface* HeterogeneousVolume);
	FIntVector GetLightingCacheResolution(const IHeterogeneousVolumeInterface*, FLODValue LODValue);
	FIntVector GetAmbientOcclusionResolution(const IHeterogeneousVolumeInterface*, FLODValue LODValue);

	float CalcLODBias(const IHeterogeneousVolumeInterface* HeterogeneousVolume);
	float CalcLODFactor(float LODValue, float LODBias);
	float CalcLODFactor(const HeterogeneousVolumes::FLODInfo& LODInfo, const IHeterogeneousVolumeInterface* HeterogeneousVolume);
	float CalcLODFactor(const FSceneView& View, const IHeterogeneousVolumeInterface* HeterogeneousVolume);

	bool IsHoldout(const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface);

	const FProjectedShadowInfo* GetProjectedShadowInfo(const FVisibleLightInfo* VisibleLightInfo, int32 ShadowIndex);
	bool IsDynamicShadow(const FVisibleLightInfo* VisibleLightInfo);
}

uint32 GetTypeHash(const FVolumetricMeshBatch& MeshBatch);

struct FVoxelDataPacked
{
	uint32 LinearIndex;
	uint32 MipLevel;
};

BEGIN_UNIFORM_BUFFER_STRUCT(FSparseVoxelUniformBufferParameters, )
	// Object data
	SHADER_PARAMETER(FMatrix44f, LocalToWorld)
	SHADER_PARAMETER(FMatrix44f, WorldToLocal)
	SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
	SHADER_PARAMETER(FVector3f, LocalBoundsExtent)

	// Volume data
	SHADER_PARAMETER(FIntVector, VolumeResolution)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, ExtinctionTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, EmissionTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, AlbedoTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)

	// Resolution
	SHADER_PARAMETER(FIntVector, LightingCacheResolution)

	// Sparse voxel data
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, NumVoxelsBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVoxelDataPacked>, VoxelBuffer)
	SHADER_PARAMETER(int, MipLevel)

	// Traversal hints
	SHADER_PARAMETER(float, MaxTraceDistance)
	SHADER_PARAMETER(float, MaxShadowTraceDistance)
	SHADER_PARAMETER(float, StepSize)
	SHADER_PARAMETER(float, StepFactor)
	SHADER_PARAMETER(float, ShadowStepSize)
	SHADER_PARAMETER(float, ShadowStepFactor)
	SHADER_PARAMETER(float, IndirectInscatteringFactor)
	SHADER_PARAMETER(int, bApplyHeightFog)
	SHADER_PARAMETER(int, bApplyVolumetricFog)
END_UNIFORM_BUFFER_STRUCT()

BEGIN_UNIFORM_BUFFER_STRUCT(FPreshadingUniformBufferParameters, )
	// Object data
	SHADER_PARAMETER(FMatrix44f, LocalToWorld)
	SHADER_PARAMETER(FMatrix44f, WorldToLocal)
	SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
	SHADER_PARAMETER(FVector3f, LocalBoundsExtent)

	// Volume data
	SHADER_PARAMETER(FIntVector, VolumeResolution)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, ExtinctionTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, EmissionTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, AlbedoTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)

	SHADER_PARAMETER(FIntVector, MajorantResolution)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, MajorantTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, PointTextureSampler)
END_UNIFORM_BUFFER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FLightingCacheParameters, )
	SHADER_PARAMETER(FIntVector, LightingCacheResolution)
	SHADER_PARAMETER(float, LightingCacheVoxelBias)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, LightingCacheTexture)
END_SHADER_PARAMETER_STRUCT()

// Path tracing structures

struct FHeterogeneousVolumesPrimitiveInstanceKey
{
	uint32 PersistentPrimitiveIndex;
	uint32 InstanceIndex;

	friend FORCEINLINE bool operator == (
		const FHeterogeneousVolumesPrimitiveInstanceKey& A,
		const FHeterogeneousVolumesPrimitiveInstanceKey& B
	)
	{
		return A.PersistentPrimitiveIndex == B.PersistentPrimitiveIndex && A.InstanceIndex == B.InstanceIndex;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FHeterogeneousVolumesPrimitiveInstanceKey& Key)
	{
		return HashCombine(GetTypeHash(Key.PersistentPrimitiveIndex), GetTypeHash(Key.InstanceIndex));
	}
};

struct FPathTracingHeterogeneousVolumesCallableInfo
{
	// Preshading data
	TRefCountPtr<IPooledRenderTarget> MajorantRT;
	TRefCountPtr<IPooledRenderTarget> ExtinctionRT;
	TRefCountPtr<IPooledRenderTarget> EmissionRT;
	TRefCountPtr<IPooledRenderTarget> AlbedoRT;

	// Callable interface
	int32 IntersectShaderId;
	int32 GetDensityBoundsShaderId;
	int32 GetDensityShaderId;
	int32 SampleDistanceShaderId;
	int32 GetTransmittanceShaderId;
};

// Adaptive Voxel Grid structures

struct FTopLevelGridBitmaskData
{
	uint32 PackedData[2];
};

struct FTopLevelGridData
{
	uint32 PackedData[1];
};

struct FScalarGridData
{
	uint32 PackedData[2];
};

struct FVectorGridData
{
	uint32 PackedData[2];
};


BEGIN_UNIFORM_BUFFER_STRUCT(FOrthoVoxelGridUniformBufferParameters, )
	SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMin)
	SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMax)
	SHADER_PARAMETER(FIntVector, TopLevelGridResolution)

	SHADER_PARAMETER(int32, bUseOrthoGrid)
	SHADER_PARAMETER(int32, bUseMajorantGrid)
	SHADER_PARAMETER(int32, bEnableIndirectionGrid)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTopLevelGridBitmaskData>, TopLevelGridBitmaskBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTopLevelGridData>, TopLevelGridBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTopLevelGridData>, IndirectionGridBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FScalarGridData>, ExtinctionGridBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVectorGridData>, EmissionGridBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVectorGridData>, ScatteringGridBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FScalarGridData>, MajorantGridBuffer)
END_UNIFORM_BUFFER_STRUCT()

BEGIN_UNIFORM_BUFFER_STRUCT(FFrustumVoxelGridUniformBufferParameters, )
	SHADER_PARAMETER(FMatrix44f, WorldToClip)
	SHADER_PARAMETER(FMatrix44f, ClipToWorld)

	SHADER_PARAMETER(FMatrix44f, WorldToView)
	SHADER_PARAMETER(FMatrix44f, ViewToWorld)

	SHADER_PARAMETER(FMatrix44f, ViewToClip)
	SHADER_PARAMETER(FMatrix44f, ClipToView)

	SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMin)
	SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMax)
	SHADER_PARAMETER(FIntVector, TopLevelFroxelGridResolution)
	SHADER_PARAMETER(FIntVector, VoxelDimensions)

	SHADER_PARAMETER(int32, bUseFrustumGrid)

	SHADER_PARAMETER(float, NearPlaneDepth)
	SHADER_PARAMETER(float, FarPlaneDepth)
	SHADER_PARAMETER(float, TanHalfFOV)

	SHADER_PARAMETER_ARRAY(FVector4f, ViewFrustumPlanes, [6])

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTopLevelGridData>, TopLevelFroxelGridBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FScalarGridData>, ExtinctionFroxelGridBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVectorGridData>, EmissionFroxelGridBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVectorGridData>, ScatteringFroxelGridBuffer)
END_UNIFORM_BUFFER_STRUCT()

// Render specializations

enum class EHeterogeneousVolumesShadowMode
{
	LiveShading,
	VoxelGrid
};

enum class EVoxelGridBuildMode
{
	PathTracing,
	Shadows,
};

struct FVoxelGridBuildOptions
{
	EVoxelGridBuildMode VoxelGridBuildMode = EVoxelGridBuildMode::PathTracing;
	float ShadingRateInFrustum = HeterogeneousVolumes::GetShadingRateForFrustumGrid();
	float ShadingRateOutOfFrustum = HeterogeneousVolumes::GetShadingRateForOrthoGrid();

	bool bBuildOrthoGrid = true;
	bool bBuildFrustumGrid = true;
	bool bUseProjectedPixelSizeForOrthoGrid = false;
	bool bJitter = HeterogeneousVolumes::ShouldJitter();
};

void BuildOrthoVoxelGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	/*const*/ TArray<FViewInfo>& Views,
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVoxelGridBuildOptions& BuildOptions,
	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoVoxelGridUniformBuffer
);

void BuildFrustumVoxelGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FVoxelGridBuildOptions& BuildOptions,
	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumVoxelGridUniformBuffer
);

void CollectRenderableHeterogeneousVolumes(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	TArray<const IHeterogeneousVolumeInterface*>& HeterogeneousVolumeInterfaces,
	TArray<const FMaterialRenderProxy*>& MaterialRenderProxies
);

void BuildPreshadingGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	TRefCountPtr<IPooledRenderTarget>& PersistentMajorantRT,
	TRDGUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBuffer
);

void BuildPreshadingGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	TRefCountPtr<IPooledRenderTarget>& PersistentMajorantRT,
	FPathTracingHeterogeneousVolumesCallableInfo& CallableInfo,
	FPreshadingUniformBufferParameters& PreshadingUniformBufferParameters
);

void BuildEmptyPreshadingUniformBuffer(
	FRDGBuilder& GraphBuilder,
	TRDGUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBuffer
);

struct FAdaptiveFrustumGridParameterCache
{
	FMatrix44f WorldToClip;
	FMatrix44f ClipToWorld;

	FMatrix44f WorldToView;
	FMatrix44f ViewToWorld;

	FMatrix44f ViewToClip;
	FMatrix44f ClipToView;

	FVector3f TopLevelGridWorldBoundsMin;
	FVector3f TopLevelGridWorldBoundsMax;
	FIntVector TopLevelGridResolution;
	FIntVector VoxelDimensions;

	int32 bUseFrustumGrid = false;

	float NearPlaneDepth;
	float FarPlaneDepth;
	float TanHalfFOV;

	FVector4f ViewFrustumPlanes[6];

	TRefCountPtr<FRDGPooledBuffer> TopLevelGridBuffer;
	TRefCountPtr<FRDGPooledBuffer> ExtinctionGridBuffer;
	TRefCountPtr<FRDGPooledBuffer> EmissionGridBuffer;
	TRefCountPtr<FRDGPooledBuffer> ScatteringGridBuffer;
};

struct FAdaptiveOrthoGridParameterCache
{
	FVector3f TopLevelGridWorldBoundsMin;
	FVector3f TopLevelGridWorldBoundsMax;
	FIntVector TopLevelGridResolution;
	int32 bUseOrthoGrid = false;
	int32 bUseMajorantGrid;
	int32 bEnableIndirectionGrid;

	TRefCountPtr<FRDGPooledBuffer> TopLevelGridBitmaskBuffer;
	TRefCountPtr<FRDGPooledBuffer> TopLevelGridBuffer;
	TRefCountPtr<FRDGPooledBuffer> IndirectionGridBuffer;

	TRefCountPtr<FRDGPooledBuffer> ExtinctionGridBuffer;
	TRefCountPtr<FRDGPooledBuffer> EmissionGridBuffer;
	TRefCountPtr<FRDGPooledBuffer> ScatteringGridBuffer;

	TRefCountPtr<FRDGPooledBuffer> MajorantGridBuffer;
};

void ExtractFrustumVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer,
	FAdaptiveFrustumGridParameterCache& AdaptiveFrustumGridParameterCache
);

void RegisterExternalFrustumVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FAdaptiveFrustumGridParameterCache& AdaptiveFrustumGridParameterCache,
	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer
);

void ExtractOrthoVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	FAdaptiveOrthoGridParameterCache& AdaptiveOrthoGridParameterCache
);

void RegisterExternalOrthoVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FAdaptiveOrthoGridParameterCache& AdaptiveOrthoGridParameterCache,
	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer
);

struct FAVSMLinkedListPackedData
{
	uint32 Data[2];
};

struct FAVSMIndirectionPackedData
{
	uint32 Data[4];
};

struct FAVSMSamplePackedData
{
	uint32 Data;
};

BEGIN_UNIFORM_BUFFER_STRUCT(FAdaptiveVolumetricShadowMapUniformBufferParameters, )
	SHADER_PARAMETER_ARRAY(FMatrix44f, TranslatedWorldToShadow, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, TranslatedWorldOrigin, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, TranslatedWorldPlane, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, SplitDepths, [6])

	SHADER_PARAMETER(FIntPoint, Resolution)
	SHADER_PARAMETER(int32, NumShadowMatrices)
	SHADER_PARAMETER(int32, MaxSampleCount)
	SHADER_PARAMETER(int32, bIsEmpty)
	SHADER_PARAMETER(int32, bIsDirectionalLight)
	SHADER_PARAMETER(float, DownsampleFactor)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, LinkedListBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, IndirectionBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, SampleBuffer)

	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, RadianceTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
END_UNIFORM_BUFFER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FAdaptiveVolumetricShadowMapParameters, RENDERER_API)
	SHADER_PARAMETER_ARRAY(FMatrix44f, TranslatedWorldToShadow, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, TranslatedWorldOrigin, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, TranslatedWorldPlane, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, SplitDepths, [6])

	SHADER_PARAMETER(FIntPoint, Resolution)
	SHADER_PARAMETER(int32, NumShadowMatrices)
	SHADER_PARAMETER(int32, MaxSampleCount)
	SHADER_PARAMETER(int32, bIsEmpty)
	SHADER_PARAMETER(int32, bIsDirectionalLight)
	SHADER_PARAMETER(float, DownsampleFactor)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, LinkedListBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint4>, IndirectionBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, SampleBuffer)
	
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, RadianceTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
END_SHADER_PARAMETER_STRUCT()

BEGIN_UNIFORM_BUFFER_STRUCT(FAdaptiveVolumetricShadowMaps, )
	SHADER_PARAMETER_STRUCT(FAdaptiveVolumetricShadowMapParameters, AVSM)
	SHADER_PARAMETER_STRUCT(FAdaptiveVolumetricShadowMapParameters, CameraAVSM)
END_UNIFORM_BUFFER_STRUCT()

BEGIN_UNIFORM_BUFFER_STRUCT(FBeerShadowMapUniformBufferParameters, )
	SHADER_PARAMETER_ARRAY(FMatrix44f, TranslatedWorldToShadow, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, TranslatedWorldOrigin, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, TranslatedWorldPlane, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, SplitDepths, [6])

	SHADER_PARAMETER(FIntPoint, Resolution)
	SHADER_PARAMETER(int32, NumShadowMatrices)
	SHADER_PARAMETER(int32, MaxSampleCount)
	SHADER_PARAMETER(int32, bIsEmpty)
	SHADER_PARAMETER(int32, bIsDirectionalLight)
	SHADER_PARAMETER(float, DownsampleFactor)

	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, BeerShadowMapTexture)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, RadianceTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
END_UNIFORM_BUFFER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FBeerShadowMapParameters, RENDERER_API)
	SHADER_PARAMETER_ARRAY(FMatrix44f, TranslatedWorldToShadow, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, TranslatedWorldOrigin, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, TranslatedWorldPlane, [6])
	SHADER_PARAMETER_ARRAY(FVector4f, SplitDepths, [6])

	SHADER_PARAMETER(FIntPoint, Resolution)
	SHADER_PARAMETER(int32, NumShadowMatrices)
	SHADER_PARAMETER(int32, MaxSampleCount)
	SHADER_PARAMETER(int32, bIsEmpty)
	SHADER_PARAMETER(int32, bIsDirectionalLight)
	SHADER_PARAMETER(float, DownsampleFactor)

	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, BeerShadowMapTexture)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, RadianceTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
END_SHADER_PARAMETER_STRUCT()

namespace HeterogeneousVolumes {
	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> GetAdaptiveVolumetricShadowMapUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState,
		const FLightSceneInfo* LightSceneInfo
	);

	void DestroyAdaptiveVolumetricShadowMapUniformBuffer(
		TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>& AdaptiveVolumetricShadowMapUniformBuffer
	);

	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> GetAdaptiveVolumetricCameraMapUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState
	);

	FAdaptiveVolumetricShadowMapUniformBufferParameters GetAdaptiveVolumetricCameraMapParameters(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState
	);

	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> CreateEmptyAdaptiveVolumetricShadowMapUniformBuffer(
		FRDGBuilder& GraphBuilder
	);

	TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters> GetBeerShadowMapUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState,
		const FLightSceneInfo* LightSceneInfo
	);

	void DestroyBeerShadowMapUniformBuffer(
		TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters>& BeerShadowMapUniformBuffer
	);

	TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters> CreateEmptyBeerShadowMapUniformBuffer(
		FRDGBuilder& GraphBuilder
	);

	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters> GetFrustumVoxelGridUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState
	);

	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters> GetOrthoVoxelGridUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState
	);

	void PostRender(FScene& Scene, TArray<FViewInfo>& Views);

} // namespace HeterogeneousVolumes

TRDGUniformBufferRef<FAdaptiveVolumetricShadowMaps> CreateAdaptiveVolumetricShadowMapUniformBuffers(
	FRDGBuilder& GraphBuilder,
	FSceneViewState* ViewState,
	const FLightSceneInfo* LightSceneInfo
);

void RenderWithLiveShading(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FPersistentPrimitiveIndex &PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance,
	FRDGTextureRef& HeterogeneousVolumeVelocity,
	FRDGTextureRef& HeterogeneousVolumeHoldout,
	FRDGTextureRef& HeterogeneousVolumeBeerShadowMap
);

void RenderWithPreshading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FPersistentPrimitiveIndex &PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance
);

void RenderTransmittanceWithVoxelGrid(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance
);

void RenderAdaptiveVolumetricShadowMapWithVoxelGrid(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Volume data
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer
);

void RenderAdaptiveVolumetricShadowMapWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Light data
	const FVisibleLightInfoArray& VisibleLightInfos
);

bool ShouldRenderVolumetricShadowMaskForLight(
	FRDGBuilder& GraphBuilder,
	const TConstArrayView<FViewInfo>& Views,
	const FLightSceneInfo* LightSceneInfo
);

void RenderVolumetricShadowMaskForLight(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTexturesConfig& Config,
	const TConstArrayView<FViewInfo>& Views,
	// Light data
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Resultant shadow mask
	FRDGTextureRef& ScreenShadowMaskTexture
);

void RenderAdaptiveVolumetricCameraMapWithVoxelGrid(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Volume data
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer
);

void RenderAdaptiveVolumetricCameraMapWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View
);

void CompressVolumetricShadowMap(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FIntVector GroupCount,
	// Input
	FIntPoint ShadowMapResolution,
	uint32 MaxSampleCount,
	FRDGBufferRef VolumetricShadowLinkedListBuffer,
	// Output
	FRDGBufferRef& VolumetricShadowIndirectionBuffer,
	FRDGBufferRef& VolumetricShadowTransmittanceBuffer
);

void CombineVolumetricShadowMap(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FIntVector GroupCount,
	// Input
	uint32 LightType,
	FIntPoint ShadowMapResolution,
	uint32 MaxSampleCount,
	FRDGBufferRef VolumetricShadowLinkedListBuffer0,
	FRDGBufferRef VolumetricShadowLinkedListBuffer1,
	// Output
	FRDGBufferRef& VolumetricShadowLinkedListBuffer
);

void ConvertBeerLawShadowMapToVolumetricShadowMap(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	// Input
	FIntPoint ShadowMapResolution,
	FRDGTextureRef BeerShadowMapTexture,
	// Output
	FRDGBufferRef& VolumetricShadowIndirectionBuffer,
	FRDGBufferRef& VolumetricShadowTransmittanceBuffer
);

void ConvertVolumetricShadowMapToBeerLawShadowMap(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	FViewInfo& View,
	// Input
	const FString& ShadowMapName,
	FIntPoint ShadowMapResolution,
	FIntVector GroupCount,
	FRDGBufferRef VolumetricShadowLinkedListBuffer,
	// Output
	FRDGTextureRef& BeerShadowMapTexture
);

void CreateAdaptiveVolumetricShadowMapUniformBufferParameters(
	FRDGBuilder& GraphBuilder,
	const FVector3f* TranslatedWorldOrigin,
	const FVector4f* TranslatedWorldPlane,
	const FMatrix44f* TranslatedWorldToShadow,
	const FVector4f* SplitDepths,
	FIntPoint VolumetricShadowMapResolution,
	float VolumetricShadowMapDownsampleFactor,
	int32 NumShadowMatrices,
	uint32 VolumetricShadowMapMaxSampleCount,
	bool bIsDirectionalLight,
	FRDGBufferRef VolumetricShadowMapLinkedListBuffer,
	FRDGBufferRef VolumetricShadowMapIndirectionBuffer,
	FRDGBufferRef VolumetricShadowMapSampleBuffer,
	FAdaptiveVolumetricShadowMapUniformBufferParameters*& AdaptiveVolumetricShadowMapUniformBufferParameters
);

void CreateAdaptiveVolumetricShadowMapUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FVector3f* TranslatedWorldOrigin,
	const FVector4f* TranslatedWorldPlane,
	const FMatrix44f* TranslatedWorldToShadow,
	const FVector4f* SplitDepths,
	FIntPoint VolumetricShadowMapResolution,
	float VolumetricShadowMapDownsampleFactor,
	int32 NumShadowMatrices,
	uint32 VolumetricShadowMapMaxSampleCount,
	bool bIsDirectionalLight,
	FRDGBufferRef VolumetricShadowMapLinkedListBuffer,
	FRDGBufferRef VolumetricShadowMapIndirectionBuffer,
	FRDGBufferRef VolumetricShadowMapSampleBuffer,
	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>& AdaptiveVolumetricShadowMapUniformBuffer
);

void CreateBeerShadowMapUniformBufferParameters(
	FRDGBuilder& GraphBuilder,
	const FVector3f* TranslatedWorldOrigin,
	const FVector4f* TranslatedWorldPlane,
	const FMatrix44f* TranslatedWorldToShadow,
	const FVector4f* SplitDepths,
	FIntPoint VolumetricShadowMapResolution,
	float VolumetricShadowMapDownsampleFactor,
	int32 NumShadowMatrices,
	uint32 VolumetricShadowMapMaxSampleCount,
	bool bIsDirectionalLight,
	FRDGBufferRef VolumetricShadowMapLinkedListBuffer,
	FRDGBufferRef VolumetricShadowMapIndirectionBuffer,
	FRDGBufferRef VolumetricShadowMapSampleBuffer,
	FBeerShadowMapUniformBufferParameters*& BeerShadowMapUniformBufferParameters
);

void CreateBeerShadowMapUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FVector3f* TranslatedWorldOrigin,
	const FVector4f* TranslatedWorldPlane,
	const FMatrix44f* TranslatedWorldToShadow,
	const FVector4f* SplitDepths,
	FIntPoint VolumetricShadowMapResolution,
	float VolumetricShadowMapDownsampleFactor,
	int32 NumShadowMatrices,
	uint32 VolumetricShadowMapMaxSampleCount,
	bool bIsDirectionalLight,
	FRDGTextureRef BeerShadowMapTexture,
	TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters>& BeerShadowMapUniformBuffer
);

void RenderSingleScatteringWithVoxelGrid(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Volume data
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance
);

// Preshading Pipeline
void ComputeHeterogeneousVolumeBakeMaterial(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FPersistentPrimitiveIndex &PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Sample data
	bool bJitter,
	bool bLooseBounds,
	bool bEvaluateAlbedo,
	bool bEvaluateEmission,
	// Volume data
	FIntVector VolumeResolution,
	// Output
	FRDGTextureRef HeterogeneousVolumeExtinctionTexture,
	FRDGTextureRef HeterogeneousVolumeEmissionTexture,
	FRDGTextureRef HeterogeneousVolumeAlbedoTexture
);

void ComputeHeterogeneousVolumeBakeMaterial(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FPersistentPrimitiveIndex& PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Sample data
	bool bJitter,
	bool bLooseBounds,
	bool bEvaluateAlbedo,
	bool bEvaluateEmission,
	// Volume data
	FIntVector VolumeResolution,
	// Output
	TRefCountPtr<IPooledRenderTarget>& HeterogeneousVolumeExtinctionTexture,
	TRefCountPtr<IPooledRenderTarget>& HeterogeneousVolumeEmissionTexture,
	TRefCountPtr<IPooledRenderTarget>& HeterogeneousVolumeAlbedoTexture
);

// Sparse Voxel Pipeline

void CopyTexture3D(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef Texture,
	uint32 InputMipLevel,
	FRDGTextureRef& OutputTexture
);

void GenerateSparseVoxels(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef VoxelMinTexture,
	FIntVector VolumeResolution,
	uint32 MipLevel,
	FRDGBufferRef& NumVoxelsBuffer,
	FRDGBufferRef& VoxelBuffer
);

void RenderExistenceMaskWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* DefaultMaterialRenderProxy,
	FPersistentPrimitiveIndex PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	FIntVector ExistenceMaskTextureResolution,
	// Output
	FRDGTextureRef& ExistenceMaskTexture
);

void DilateExistenceMask(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	// Existence texture data
	FRDGTextureRef ExistenceMaskTexture,
	FIntVector ExistenceMaskTextureResolution,
	// Output
	FRDGTextureRef& DilatedExistenceTexture
);

void VisualizeCascades(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	const FScene* Scene,
	const FViewInfo& View,
	// Light data
	const FLightSceneInfo* LightSceneInfo,
	// Output
	FRDGTextureRef& VisualizationTexture
);

void RenderAmbientOcclusionWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* DefaultMaterialRenderProxy,
	FPersistentPrimitiveIndex PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Output
	FRDGTextureRef& AmbientOcclusionTexture
);

#if RHI_RAYTRACING

void GenerateRayTracingGeometryInstance(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	// Sparse voxel data
	FRDGBufferRef NumVoxelsBuffer,
	TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters> SparseVoxelUniformBuffer,
	// Output
	TArray<FRayTracingGeometryRHIRef, SceneRenderingAllocator>& RayTracingGeometries,
	TArray<FMatrix>& RayTracingTransforms
);

void GenerateRayTracingScene(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	FViewInfo& View,
	// Ray tracing data
	TConstArrayView<FRayTracingGeometryRHIRef> RayTracingGeometries,
	TConstArrayView<FMatrix> RayTracingTransforms,
	// Output
	FRayTracingScene& RayTracingScene
);

void RenderLightingCacheWithPreshadingHardwareRayTracing(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	const FSceneTextures& SceneTextures,
	// Light data
	bool bApplyEmissionAndTransmittance,
	bool bApplyDirectLighting,
	bool bApplyShadowTransmittance,
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Shadow data
	const FVisibleLightInfo* VisibleLightInfo,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	// Sparse voxel data
	TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters> SparseVoxelUniformBuffer,
	// Ray tracing data
	FRayTracingScene& RayTracingScene,
	TConstArrayView<FRayTracingGeometryRHIRef> RayTracingGeometries,
	// Output
	FRDGTextureRef& LightingCacheTexture
);

void RenderSingleScatteringWithPreshadingHardwareRayTracing(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	const FSceneTextures& SceneTextures,
	// Light data
	bool bApplyEmissionAndTransmittance,
	bool bApplyDirectLighting,
	bool bApplyShadowTransmittance,
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Shadow data
	const FVisibleLightInfo* VisibleLightInfo,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	// Sparse voxel data
	TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters> SparseVoxelUniformBuffer,
	// Ray tracing data
	FRayTracingScene& RayTracingScene,
	TConstArrayView<FRayTracingGeometryRHIRef> RayTracingGeometries,
	// Transmittance volume
	FRDGTextureRef LightingCacheTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeTexture
);

#endif // RHI_RAYTRACING

DECLARE_CYCLE_STAT(TEXT("Ambient Occlusion"), STATGROUP_HeterogeneousVolumesAmbientOcclusion, STATGROUP_HeterogeneousVolumesRT);
DECLARE_CYCLE_STAT(TEXT("Light Cache"), STATGROUP_HeterogeneousVolumesLightCache, STATGROUP_HeterogeneousVolumesRT);
DECLARE_CYCLE_STAT(TEXT("Material Baking"), STATGROUP_HeterogeneousVolumesMaterialBaking, STATGROUP_HeterogeneousVolumesRT);
DECLARE_CYCLE_STAT(TEXT("Shadows"), STATGROUP_HeterogeneousVolumesShadows, STATGROUP_HeterogeneousVolumesRT);
DECLARE_CYCLE_STAT(TEXT("Single Scattering"), STATGROUP_HeterogeneousVolumesSingleScattering, STATGROUP_HeterogeneousVolumesRT);
