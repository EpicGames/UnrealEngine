// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumes.h"
#include "HeterogeneousVolumeInterface.h"

#include "LightRendering.h"
#include "LocalVertexFactory.h"
#include "MeshPassUtils.h"
#include "PixelShaderUtils.h"
#include "RayTracingDefinitions.h"
#include "RayTracingInstance.h"
#include "RayTracingInstanceBufferUtil.h"
#include "ScenePrivate.h"
#include "PrimitiveDrawingUtils.h"
#include "VolumeLighting.h"
#include "VolumetricFog.h"
#include "BlueNoise.h"
#include "SceneTextureParameters.h"
#include "Materials/MaterialRenderProxy.h"
#include "ShaderCompilerCore.h"

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesAmbientOcclusion(
	TEXT("r.HeterogeneousVolumes.AmbientOcclusion"),
	0,
	TEXT("Enables ambient occlusion computation (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesAmbientOcclusionDownsampleFactor(
	TEXT("r.HeterogeneousVolumes.AmbientOcclusion.DownsampleFactor"),
	4,
	TEXT("Performs downsampling when determining the ambient occlusion voxel resolution (Default = 4)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesAmbientOcclusionRayCountX(
	TEXT("r.HeterogeneousVolumes.AmbientOcclusion.RayCount.X"),
	4,
	TEXT("With the Y-counterpart, determines the number of AO rays when calculating ambient occlusion (Default = 4)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesAmbientOcclusionRayCountY(
	TEXT("r.HeterogeneousVolumes.AmbientOcclusion.RayCount.Y"),
	4,
	TEXT(" With the X-counterpart, determines the number of AO rays when calculating ambient occlusion (Default = 4)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesAmbientOcclusionMaxTraceDistance(
	TEXT("r.HeterogeneousVolumes.AmbientOcclusion.MaxTraceDistance"),
	1000.0,
	TEXT("Determines the number of rays when calculating ambient occlusion (Default = 1000.0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesAmbientOcclusionMaxStepCount(
	TEXT("r.HeterogeneousVolumes.AmbientOcclusion.MaxStepCount"),
	64,
	TEXT("Determines the maximum steps when ray marching ambient occlusion (Default = 64)"),
	ECVF_RenderThreadSafe
);

namespace HeterogeneousVolumes
{
	bool EnableAmbientOcclusion()
	{
		return CVarHeterogeneousVolumesAmbientOcclusion.GetValueOnRenderThread() != 0;
	}

	float GetAmbientOcclusionDownsampleFactor()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesAmbientOcclusionDownsampleFactor.GetValueOnRenderThread(), 0.125, 32);
	}

	FIntPoint GetAmbientOcclusionRayCount()
	{
		return FIntPoint(
			FMath::Clamp(CVarHeterogeneousVolumesAmbientOcclusionRayCountX.GetValueOnRenderThread(), 1, 8),
			FMath::Clamp(CVarHeterogeneousVolumesAmbientOcclusionRayCountY.GetValueOnRenderThread(), 1, 8)
		);
	}

	float GetAmbientOcclusionMaxTraceDistance()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesAmbientOcclusionMaxTraceDistance.GetValueOnRenderThread(), 0.0f, GetMaxShadowTraceDistance());
	}

	int32 GetAmbientOcclusionMaxStepCount()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesAmbientOcclusionMaxStepCount.GetValueOnRenderThread(), 1, GetMaxStepCount());
	}

	FIntVector GetAmbientOcclusionResolution(const IHeterogeneousVolumeInterface* RenderInterface, FLODValue LODValue)
	{
		float LODFactor = CalcLODFactor(LODValue.LOD, 0.0f);
		float DownsampleFactor = GetAmbientOcclusionDownsampleFactor() * LODFactor;
		DownsampleFactor = FMath::Max(DownsampleFactor, 0.125f);

		FVector VolumeResolution = FVector(GetVolumeResolution(RenderInterface));
		FIntVector AmbientOcclusionResolution = FIntVector(VolumeResolution / DownsampleFactor);
		AmbientOcclusionResolution.X = FMath::Clamp(AmbientOcclusionResolution.X, 1, 1024);
		AmbientOcclusionResolution.Y = FMath::Clamp(AmbientOcclusionResolution.Y, 1, 1024);
		AmbientOcclusionResolution.Z = FMath::Clamp(AmbientOcclusionResolution.Z, 1, 512);
		return AmbientOcclusionResolution;
	}

} // namespace HeterogeneousVolumes

//-OPT: Remove duplicate bindings
// At the moment we need to bind the mesh draw parameters as they will be applied and on some RHIs this will crash if the texture is nullptr
// We have the same parameters in the loose FParameters shader structure that are applied after the mesh draw.
class FRenderAmbientOcclusionLooseBindings
{
	DECLARE_TYPE_LAYOUT(FRenderAmbientOcclusionLooseBindings, NonVirtual);

public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		SceneDepthTextureBinding.Bind(ParameterMap, TEXT("SceneDepthTexture"));
	}

	template<typename TPassParameters>
	void SetParameters(FMeshDrawSingleShaderBindings& ShaderBindings, const TPassParameters* PassParameters)
	{
		ShaderBindings.AddTexture(
			SceneDepthTextureBinding,
			FShaderResourceParameter(),
			TStaticSamplerState<SF_Point>::GetRHI(),
			PassParameters->SceneTextures.SceneDepthTexture->GetRHI()
		);
	}

	LAYOUT_FIELD(FShaderResourceParameter, SceneDepthTextureBinding);
};

IMPLEMENT_TYPE_LAYOUT(FRenderAmbientOcclusionLooseBindings);

class FRenderExistenceMaskWithLiveShadingCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRenderExistenceMaskWithLiveShadingCS, MeshMaterial);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)

		// Shadow data
		SHADER_PARAMETER(float, ShadowStepSize)
		SHADER_PARAMETER(float, ShadowStepFactor)

		// Object data
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLocal)
		SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
		SHADER_PARAMETER(FVector3f, LocalBoundsExtent)
		SHADER_PARAMETER(int32, PrimitiveId)

		// Ray data
		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(float, MaxShadowTraceDistance)
		SHADER_PARAMETER(float, StepSize)
		SHADER_PARAMETER(float, StepFactor)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER(int, StochasticFilteringMode)

		// Volume data
		SHADER_PARAMETER(FIntVector, VoxelResolution)
		SHADER_PARAMETER(FIntVector, VoxelMin)
		SHADER_PARAMETER(FIntVector, VoxelMax)

		// Optional cinematic features
		SHADER_PARAMETER(int, bIsOfflineRender)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, RWExistenceMaskTexture)
	END_SHADER_PARAMETER_STRUCT()

	FRenderExistenceMaskWithLiveShadingCS() = default;

	FRenderExistenceMaskWithLiveShadingCS(
		const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer
	)
		: FMeshMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false
		);
		ShaderLooseBindings.Bind(Initializer.ParameterMap);
	}

	static bool ShouldCompilePermutation(
		const FMaterialShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform)
			&& DoesMaterialShaderSupportHeterogeneousVolumes(Parameters.MaterialParameters);
	}

	static void ModifyCompilationEnvironment(
		const FMaterialShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		//OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC); // @lh-todo - Disabled to workaround SPIRV-Cross bug: StructuredBuffer<uint> is translated to ByteAddressBuffer in HLSL backend
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }

	LAYOUT_FIELD(FRenderAmbientOcclusionLooseBindings, ShaderLooseBindings);
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRenderExistenceMaskWithLiveShadingCS, TEXT("/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesAmbientOcclusionPipeline.usf"), TEXT("RenderExistenceMaskWithLiveShadingCS"), SF_Compute);

void RenderExistenceMaskWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* DefaultMaterialRenderProxy,
	FPersistentPrimitiveIndex PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	FIntVector ExistenceTextureResolution,
	// Output
	FRDGTextureRef& ExistenceMaskTexture
)
{
	FRDGTextureDesc ExistenceTextureDesc = FRDGTextureDesc::Create3D(
		ExistenceTextureResolution,
		PF_R8,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling
	);
	ExistenceMaskTexture = GraphBuilder.CreateTexture(ExistenceTextureDesc, TEXT("HeterogeneousVolumes.ExistenceMaskTexture"));

	const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
	const FMaterial& Material = DefaultMaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
	MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : DefaultMaterialRenderProxy;

	check(Material.GetMaterialDomain() == MD_Volume);

	FRenderExistenceMaskWithLiveShadingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderExistenceMaskWithLiveShadingCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);

		// Object data
		// LWC_TODO: Convert to relative-local space
		//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
		//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
		FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
		FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
		PassParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
		PassParameters->WorldToLocal = PassParameters->LocalToWorld.Inverse();

		// Ray data
		PassParameters->ShadowStepSize = HeterogeneousVolumes::GetShadowStepSize();
		PassParameters->ShadowStepFactor = HeterogeneousVolumeInterface->GetShadowStepFactor();
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetAmbientOcclusionMaxTraceDistance();
		PassParameters->MaxShadowTraceDistance = HeterogeneousVolumes::GetAmbientOcclusionMaxTraceDistance();
		PassParameters->StepSize = HeterogeneousVolumes::GetStepSize();
		PassParameters->StepFactor = HeterogeneousVolumeInterface->GetStepFactor();
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetAmbientOcclusionMaxStepCount();
		PassParameters->bJitter = 0;
		PassParameters->StochasticFilteringMode = static_cast<int32>(HeterogeneousVolumes::GetStochasticFilteringMode());

		FMatrix LocalToInstance = InstanceToLocal.Inverse();
		FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(LocalToInstance);
		PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
		PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);
		PassParameters->PrimitiveId = PersistentPrimitiveIndex.Index;

		// Ambient occlusion volume
		PassParameters->VoxelResolution = ExistenceTextureResolution;
		PassParameters->VoxelMin = FIntVector::ZeroValue;
		PassParameters->VoxelMax = ExistenceTextureResolution - FIntVector(1);

		// Optional cinematic features
		PassParameters->bIsOfflineRender = View.bIsOfflineRender;

		// Output
		PassParameters->RWExistenceMaskTexture = GraphBuilder.CreateUAV(ExistenceMaskTexture);
	}

	FIntVector GroupCount;
	GroupCount.X = FMath::DivideAndRoundUp(ExistenceTextureResolution.X, FRenderExistenceMaskWithLiveShadingCS::GetThreadGroupSize3D());
	GroupCount.Y = FMath::DivideAndRoundUp(ExistenceTextureResolution.Y, FRenderExistenceMaskWithLiveShadingCS::GetThreadGroupSize3D());
	GroupCount.Z = FMath::DivideAndRoundUp(ExistenceTextureResolution.Z, FRenderExistenceMaskWithLiveShadingCS::GetThreadGroupSize3D());

	FRenderExistenceMaskWithLiveShadingCS::FPermutationDomain PermutationVector;
	TShaderRef<FRenderExistenceMaskWithLiveShadingCS> ComputeShader = Material.GetShader<FRenderExistenceMaskWithLiveShadingCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);
	if (!ComputeShader.IsNull())
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("ExistenceMask"),
			PassParameters,
			ERDGPassFlags::Compute,
			[ComputeShader, PassParameters, Scene, MaterialRenderProxy, &Material, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
			{
				FMeshMaterialShaderElementData ShaderElementData;
				ShaderElementData.InitializeMeshMaterialData();

				FMeshProcessorShaders PassShaders;
				PassShaders.ComputeShader = ComputeShader;

				FMeshDrawShaderBindings ShaderBindings;
				ShaderBindings.Initialize(PassShaders);
				{
					FMeshDrawSingleShaderBindings SingleShaderBindings = ShaderBindings.GetSingleShaderBindings(SF_Compute);
					ComputeShader->GetShaderBindings(Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, Material, ShaderElementData, SingleShaderBindings);
					ComputeShader->ShaderLooseBindings.SetParameters(SingleShaderBindings, PassParameters);
					ShaderBindings.Finalize(&PassShaders);
				}

				UE::MeshPassUtils::Dispatch(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, GroupCount);
			}
		);
	}
}

class FDilateExistenceMaskCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDilateExistenceMaskCS);
	SHADER_USE_PARAMETER_STRUCT(FDilateExistenceMaskCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, TextureResolution)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, ExistenceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, RWDilatedExistenceTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return 64; }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FDilateExistenceMaskCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesLiveShadingGlobalPipeline.usf", "DilateExistenceMaskCS", SF_Compute);

void DilateExistenceMask(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	// Existence texture data
	FRDGTextureRef ExistenceTexture,
	FIntVector ExistenceTextureResolution,
	// Output
	FRDGTextureRef& DilatedExistenceTexture
)
{
	FRDGTextureDesc ExistenceTextureDesc = FRDGTextureDesc::Create3D(
		ExistenceTextureResolution,
		PF_R8,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling
	);
	DilatedExistenceTexture = GraphBuilder.CreateTexture(ExistenceTextureDesc, TEXT("HeterogeneousVolumes.DilatedExistenceTexture"));

	FDilateExistenceMaskCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDilateExistenceMaskCS::FParameters>();
	{
		PassParameters->TextureResolution = ExistenceTextureResolution;
		PassParameters->ExistenceTexture = GraphBuilder.CreateSRV(ExistenceTexture);
		PassParameters->TextureSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		PassParameters->RWDilatedExistenceTexture = GraphBuilder.CreateUAV(DilatedExistenceTexture);
	}

	FIntVector GroupCount;
	GroupCount.X = FMath::DivideAndRoundUp(ExistenceTextureResolution.X, FDilateExistenceMaskCS::GetThreadGroupSize3D());
	GroupCount.Y = FMath::DivideAndRoundUp(ExistenceTextureResolution.Y, FDilateExistenceMaskCS::GetThreadGroupSize3D());
	GroupCount.Z = FMath::DivideAndRoundUp(ExistenceTextureResolution.Z, FDilateExistenceMaskCS::GetThreadGroupSize3D());

	TShaderRef<FDilateExistenceMaskCS> ComputeShader = View.ShaderMap->GetShader<FDilateExistenceMaskCS>();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("DilateExistenceMaskCS"),
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

class FRenderAmbientOcclusionWithLiveShadingCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRenderAmbientOcclusionWithLiveShadingCS, MeshMaterial);

	class FUseExistenceMask : SHADER_PERMUTATION_INT("USE_EXISTENCE_MASK", 2);
	class FIsOfflineRender : SHADER_PERMUTATION_INT("IS_OFFLINE_RENDER", 2);
	using FPermutationDomain = TShaderPermutationDomain<FUseExistenceMask, FIsOfflineRender>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)

		// Shadow data
		SHADER_PARAMETER(float, ShadowStepSize)
		SHADER_PARAMETER(float, ShadowStepFactor)

		// Object data
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLocal)
		SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
		SHADER_PARAMETER(FVector3f, LocalBoundsExtent)
		SHADER_PARAMETER(int32, PrimitiveId)

		// Ray data
		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(float, MaxShadowTraceDistance)
		SHADER_PARAMETER(float, StepSize)
		SHADER_PARAMETER(float, StepFactor)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER(int, StochasticFilteringMode)

		// Volume data
		SHADER_PARAMETER(FIntVector, VoxelResolution)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightingCacheParameters, AmbientOcclusion)
		SHADER_PARAMETER(FIntVector, VoxelMin)
		SHADER_PARAMETER(FIntVector, VoxelMax)

		// AO data
		SHADER_PARAMETER(FIntPoint, NumRays)

		// Processing Mask
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, ExistenceMaskTexture)

		// Optional cinematic features
		SHADER_PARAMETER(int, bIsOfflineRender)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<uint>, RWAmbientOcclusionUIntTexture)
	END_SHADER_PARAMETER_STRUCT()

	FRenderAmbientOcclusionWithLiveShadingCS() = default;

	FRenderAmbientOcclusionWithLiveShadingCS(
		const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer
	)
		: FMeshMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false
		);
		ShaderLooseBindings.Bind(Initializer.ParameterMap);
	}

	static bool ShouldCompilePermutation(
		const FMaterialShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform)
			&& DoesMaterialShaderSupportHeterogeneousVolumes(Parameters.MaterialParameters);
	}

	static void ModifyCompilationEnvironment(
		const FMaterialShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		//OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC); // @lh-todo - Disabled to workaround SPIRV-Cross bug: StructuredBuffer<uint> is translated to ByteAddressBuffer in HLSL backend
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }

	LAYOUT_FIELD(FRenderAmbientOcclusionLooseBindings, ShaderLooseBindings);
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRenderAmbientOcclusionWithLiveShadingCS, TEXT("/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesAmbientOcclusionPipeline.usf"), TEXT("RenderAmbientOcclusionWithLiveShadingCS"), SF_Compute);

void RenderAmbientOcclusionWithLiveShadingAsFixedPoint(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* DefaultMaterialRenderProxy,
	FPersistentPrimitiveIndex PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Intermediary data
	FRDGTextureRef ExistenceMaskTexture,
	FIntVector AmbientOcclusionTextureResolution,
	// Output
	FRDGTextureRef& AmbientOcclusionUIntTexture
)
{
	FRDGTextureDesc AmbientOcclusionDesc = FRDGTextureDesc::Create3D(
		AmbientOcclusionTextureResolution,
		PF_R32_UINT,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling
	);
	AmbientOcclusionUIntTexture = GraphBuilder.CreateTexture(AmbientOcclusionDesc, TEXT("HeterogeneousVolumes.AmbientOcclusionUIntTexture"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(AmbientOcclusionUIntTexture), 0u);

	// Build Ambient Occlusion score
	const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
	const FMaterial& Material = DefaultMaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
	MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : DefaultMaterialRenderProxy;
	check(Material.GetMaterialDomain() == MD_Volume);

	FIntPoint RayCount = HeterogeneousVolumes::GetAmbientOcclusionRayCount();

	FRenderAmbientOcclusionWithLiveShadingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderAmbientOcclusionWithLiveShadingCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);

		// Object data
		// LWC_TODO: Convert to relative-local space
		//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
		//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
		FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
		FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
		PassParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
		PassParameters->WorldToLocal = PassParameters->LocalToWorld.Inverse();

		// Ray data
		PassParameters->ShadowStepSize = HeterogeneousVolumes::GetShadowStepSize();
		PassParameters->ShadowStepFactor = HeterogeneousVolumeInterface->GetShadowStepFactor();
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetAmbientOcclusionMaxTraceDistance();
		PassParameters->MaxShadowTraceDistance = HeterogeneousVolumes::GetAmbientOcclusionMaxTraceDistance();
		PassParameters->StepSize = HeterogeneousVolumes::GetStepSize();
		PassParameters->StepFactor = HeterogeneousVolumeInterface->GetStepFactor();
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetAmbientOcclusionMaxStepCount();
		PassParameters->bJitter = 0;
		PassParameters->StochasticFilteringMode = static_cast<int32>(HeterogeneousVolumes::GetStochasticFilteringMode());

		FMatrix LocalToInstance = InstanceToLocal.Inverse();
		FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(LocalToInstance);
		PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
		PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);
		PassParameters->PrimitiveId = PersistentPrimitiveIndex.Index;

		// Ambient occlusion volume
		PassParameters->VoxelResolution = HeterogeneousVolumeInterface->GetVoxelResolution();
		PassParameters->AmbientOcclusion.LightingCacheResolution = AmbientOcclusionTextureResolution;
		PassParameters->AmbientOcclusion.LightingCacheVoxelBias = HeterogeneousVolumeInterface->GetShadowBiasFactor();
		PassParameters->AmbientOcclusion.LightingCacheTexture = FRDGSystemTextures::Get(GraphBuilder).VolumetricBlack;

		// Ambient occlusion data
		PassParameters->NumRays = RayCount;
		PassParameters->ExistenceMaskTexture = GraphBuilder.CreateSRV(ExistenceMaskTexture);

		// Optional cinematic features
		PassParameters->bIsOfflineRender = View.bIsOfflineRender;

		// Output
		PassParameters->RWAmbientOcclusionUIntTexture = GraphBuilder.CreateUAV(AmbientOcclusionUIntTexture);
		PassParameters->VoxelMin = FIntVector::ZeroValue;
		PassParameters->VoxelMax = AmbientOcclusionTextureResolution - FIntVector(1);
	}

	FIntVector GroupCount;
	GroupCount.X = FMath::DivideAndRoundUp(AmbientOcclusionTextureResolution.X, FRenderAmbientOcclusionWithLiveShadingCS::GetThreadGroupSize3D());
	GroupCount.Y = FMath::DivideAndRoundUp(AmbientOcclusionTextureResolution.Y, FRenderAmbientOcclusionWithLiveShadingCS::GetThreadGroupSize3D());
	GroupCount.Z = FMath::DivideAndRoundUp(AmbientOcclusionTextureResolution.Z, FRenderAmbientOcclusionWithLiveShadingCS::GetThreadGroupSize3D());
	GroupCount.Z *= RayCount.X * RayCount.Y;

	FRenderAmbientOcclusionWithLiveShadingCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRenderAmbientOcclusionWithLiveShadingCS::FUseExistenceMask>(HeterogeneousVolumes::UseExistenceMask());
	PermutationVector.Set<FRenderAmbientOcclusionWithLiveShadingCS::FIsOfflineRender>(View.bIsOfflineRender);
	TShaderRef<FRenderAmbientOcclusionWithLiveShadingCS> ComputeShader = Material.GetShader<FRenderAmbientOcclusionWithLiveShadingCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("AmbientOcclusion"),
		PassParameters,
		ERDGPassFlags::Compute,
		[ComputeShader, PassParameters, Scene, MaterialRenderProxy, &Material, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			FMeshMaterialShaderElementData ShaderElementData;
			ShaderElementData.InitializeMeshMaterialData();

			FMeshProcessorShaders PassShaders;
			PassShaders.ComputeShader = ComputeShader;

			FMeshDrawShaderBindings ShaderBindings;
			ShaderBindings.Initialize(PassShaders);
			{
				FMeshDrawSingleShaderBindings SingleShaderBindings = ShaderBindings.GetSingleShaderBindings(SF_Compute);
				ComputeShader->GetShaderBindings(Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, Material, ShaderElementData, SingleShaderBindings);
				ComputeShader->ShaderLooseBindings.SetParameters(SingleShaderBindings, PassParameters);
				ShaderBindings.Finalize(&PassShaders);
			}

			UE::MeshPassUtils::Dispatch(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, GroupCount);
		}
	);
}

class FConvertTexture3DFixedPointToFloatCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FConvertTexture3DFixedPointToFloatCS);
	SHADER_USE_PARAMETER_STRUCT(FConvertTexture3DFixedPointToFloatCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, TextureResolution)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D<uint>, UIntTexture3D)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, RWFloatTexture3D)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return 64; }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FConvertTexture3DFixedPointToFloatCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesLiveShadingGlobalPipeline.usf", "ConvertTexture3DFixedPointToFloatCS", SF_Compute);

void ConvertFixedPointToFloatingPoint(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	FIntVector AmbientOcclusionTextureResolution,
	FRDGTextureRef AmbientOcclusionUIntTexture,
	FRDGTextureRef& AmbientOcclusionTexture
)
{
	FConvertTexture3DFixedPointToFloatCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FConvertTexture3DFixedPointToFloatCS::FParameters>();
	{
		PassParameters->TextureResolution = AmbientOcclusionTextureResolution;
		PassParameters->UIntTexture3D = GraphBuilder.CreateSRV(AmbientOcclusionUIntTexture);
		PassParameters->RWFloatTexture3D = GraphBuilder.CreateUAV(AmbientOcclusionTexture);
	}

	FIntVector GroupCount;
	GroupCount.X = FMath::DivideAndRoundUp(AmbientOcclusionTextureResolution.X, FConvertTexture3DFixedPointToFloatCS::GetThreadGroupSize3D());
	GroupCount.Y = FMath::DivideAndRoundUp(AmbientOcclusionTextureResolution.Y, FConvertTexture3DFixedPointToFloatCS::GetThreadGroupSize3D());
	GroupCount.Z = FMath::DivideAndRoundUp(AmbientOcclusionTextureResolution.Z, FConvertTexture3DFixedPointToFloatCS::GetThreadGroupSize3D());

	FConvertTexture3DFixedPointToFloatCS::FPermutationDomain PermutationVector;
	TShaderRef<FConvertTexture3DFixedPointToFloatCS> ComputeShader = View.ShaderMap->GetShader<FConvertTexture3DFixedPointToFloatCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("ConvertTexture3DFixedPointToFloatCS"),
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

void RenderAmbientOcclusionWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* DefaultMaterialRenderProxy,
	FPersistentPrimitiveIndex PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Output
	FRDGTextureRef& AmbientOcclusionTexture
)
{
	SCOPE_CYCLE_COUNTER(STATGROUP_HeterogeneousVolumesAmbientOcclusion);

	bool bRenderAmbientOcclusion = HeterogeneousVolumes::EnableAmbientOcclusion() && HeterogeneousVolumes::UseIndirectLighting();
	if (!bRenderAmbientOcclusion || HeterogeneousVolumeInterface->IsHoldout())
	{
		FRDGTextureDesc AmbientOcclusionDesc = FRDGTextureDesc::Create3D(
			FIntVector(1),
			PF_R8,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling
		);

		AmbientOcclusionTexture = GraphBuilder.CreateTexture(AmbientOcclusionDesc, TEXT("HeterogeneousVolumes.AmbientOcclusionTexture"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(AmbientOcclusionTexture), 1.0f);
		return;
	}

	HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
	FIntVector AmbientOcclusionTextureResolution = HeterogeneousVolumes::GetAmbientOcclusionResolution(HeterogeneousVolumeInterface, LODValue);

	// Build existence mask
	FRDGTextureRef DilatedExistenceMaskTexture = FRDGSystemTextures::Get(GraphBuilder).VolumetricBlack;
	if (HeterogeneousVolumes::UseExistenceMask())
	{
		FRDGTextureRef ExistenceMaskTexture;
		RenderExistenceMaskWithLiveShading(
			GraphBuilder,
			// Scene
			Scene,
			View,
			SceneTextures,
			// Object
			HeterogeneousVolumeInterface,
			DefaultMaterialRenderProxy,
			PersistentPrimitiveIndex,
			LocalBoxSphereBounds,
			// Mask
			AmbientOcclusionTextureResolution,
			ExistenceMaskTexture
		);

		DilateExistenceMask(
			GraphBuilder,
			Scene,
			View,
			ExistenceMaskTexture,
			AmbientOcclusionTextureResolution,
			DilatedExistenceMaskTexture
		);
	}

	// Calculate ambient occlusion
	FRDGTextureRef AmbientOcclusionTextureAsFixedPoint;
	RenderAmbientOcclusionWithLiveShadingAsFixedPoint(
		GraphBuilder,
		// Scene data
		Scene,
		View,
		SceneTextures,
		// Object data
		HeterogeneousVolumeInterface,
		DefaultMaterialRenderProxy,
		PersistentPrimitiveIndex,
		LocalBoxSphereBounds,
		DilatedExistenceMaskTexture,
		AmbientOcclusionTextureResolution,
		// Output
		AmbientOcclusionTextureAsFixedPoint
	);

	FRDGTextureDesc AmbientOcclusionDesc = FRDGTextureDesc::Create3D(
		AmbientOcclusionTextureResolution,
		PF_R8,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling
	);
	AmbientOcclusionTexture = GraphBuilder.CreateTexture(AmbientOcclusionDesc, TEXT("HeterogeneousVolumes.AmbientOcclusionTexture"));

	ConvertFixedPointToFloatingPoint(
		GraphBuilder,
		Scene,
		View,
		AmbientOcclusionTextureResolution,
		AmbientOcclusionTextureAsFixedPoint,
		AmbientOcclusionTexture
	);
}
