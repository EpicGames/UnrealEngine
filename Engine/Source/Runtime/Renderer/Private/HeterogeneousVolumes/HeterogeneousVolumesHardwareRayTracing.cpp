// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumes.h"
#include "HeterogeneousVolumeInterface.h"

#include "LightRendering.h"
#include "PixelShaderUtils.h"
#include "RayTracingDefinitions.h"
#include "RayTracingInstance.h"
#include "RayTracingInstanceBufferUtil.h"
#include "RayTracingPayloadType.h"
#include "PrimitiveDrawingUtils.h"
#include "VolumeLighting.h"
#include "VolumetricFog.h"
#include "FogRendering.h"
#include "SceneTextureParameters.h"
#include "ShaderCompilerCore.h"

#if RHI_RAYTRACING
FIntVector GetVoxelCoord(uint32 VoxelIndex, FIntVector VolumeResolution)
{
	FIntVector VoxelCoord;

	uint32 SliceSize = VolumeResolution.X * VolumeResolution.Y;
	uint32 SliceIndex = VoxelIndex / SliceSize;
	uint32 SliceCoord = VoxelIndex - SliceIndex * SliceSize;

	VoxelCoord.X = SliceCoord % VolumeResolution.X;
	VoxelCoord.Y = SliceCoord / VolumeResolution.X;
	VoxelCoord.Z = SliceIndex;

	return VoxelCoord;
}

FBox GetVoxelBounds(uint32 VoxelIndex, FIntVector VolumeResolution, FVector LocalBoundsOrigin, FVector LocalBoundsExtent)
{
	FBox VoxelBounds;

	FIntVector VoxelCoord = GetVoxelCoord(VoxelIndex, VolumeResolution);
	FVector VoxelSize = (LocalBoundsExtent * 2.0) / FVector(VolumeResolution);

	VoxelBounds.Min = LocalBoundsOrigin - LocalBoundsExtent + FVector(VoxelCoord) * VoxelSize;
	VoxelBounds.Max = VoxelBounds.Min + VoxelSize;
	return VoxelBounds;
}

class FCreateSparseVoxelBLAS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCreateSparseVoxelBLAS);
	SHADER_USE_PARAMETER_STRUCT(FCreateSparseVoxelBLAS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSparseVoxelUniformBufferParameters, SparseVoxelUniformBuffer)

		// Output
		// Using RWStructuredBuffer<float> instead of RWStructuredBuffer<float3> to overcome Vulkan alignment error:
		// error: cannot instantiate RWStructuredBuffer with given packed alignment; 'VK_EXT_scalar_block_layout' not supported
		// SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<FVector>, RWPositionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, RWPositionBuffer)

		// Indirect args
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCreateSparseVoxelBLAS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesSparseVoxelPipeline.usf", "CreateSparseVoxelBLAS", SF_Compute);

void CreateSparseVoxelBLAS(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FViewInfo& View,
	// Sparse voxel data
	TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters> SparseVoxelUniformBuffer,
	FRDGBufferRef NumVoxelsBuffer,
	// Output
	FRDGBufferRef PositionBuffer
)
{
	FCreateSparseVoxelBLAS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCreateSparseVoxelBLAS::FParameters>();
	{
		PassParameters->SparseVoxelUniformBuffer = SparseVoxelUniformBuffer;
		PassParameters->RWPositionBuffer = GraphBuilder.CreateUAV(PositionBuffer);
		PassParameters->IndirectArgs = NumVoxelsBuffer;
	}

	TShaderRef<FCreateSparseVoxelBLAS> ComputeShader = View.ShaderMap->GetShader<FCreateSparseVoxelBLAS>();

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("CreateSparseVoxelBLAS"),
		ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		PassParameters->IndirectArgs,
		0);
}

void GenerateRayTracingGeometryInstance(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	// Volume data
	// Sparse voxel data
	FRDGBufferRef NumVoxelsBuffer,
	TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters> SparseVoxelUniformBuffer,
	// Output
	TArray<FRayTracingGeometryRHIRef, SceneRenderingAllocator>& RayTracingGeometries,
	TArray<FMatrix>& RayTracingTransforms
)
{
	FIntVector MipVolumeResolution = HeterogeneousVolumes::GetMipVolumeResolution(SparseVoxelUniformBuffer->GetParameters()->VolumeResolution, SparseVoxelUniformBuffer->GetParameters()->MipLevel);
	uint32 MipVoxelCount = HeterogeneousVolumes::GetVoxelCount(MipVolumeResolution);

	TRefCountPtr<FRDGPooledBuffer> PooledVertexBuffer;
	{
#if 0
		TResourceArray<FVector3f> PositionData;

		PositionData.SetNumUninitialized(MipVoxelCount * 2);
		for (uint32 MipVoxelIndex = 0; MipVoxelIndex < MipVoxelCount; ++MipVoxelIndex)
		{
			FBox VoxelBounds = GetVoxelBounds(MipVoxelIndex, MipVolumeResolution, FVector(SparseVoxelUniformBuffer->GetParameters()->LocalBoundsOrigin), FVector(SparseVoxelUniformBuffer->GetParameters()->LocalBoundsExtent));

			uint32 BoundsIndex = MipVoxelIndex * 2;
			PositionData[BoundsIndex] = FVector3f(VoxelBounds.Min);
			PositionData[BoundsIndex + 1] = FVector3f(VoxelBounds.Max);
		}

		FRHIResourceCreateInfo CreateInfo(TEXT("HeterogeneousVolumesVB"));
		CreateInfo.ResourceArray = &PositionData;
#endif
		FRDGBufferRef VertexBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FVector3f), MipVoxelCount * 2),
		TEXT("CreateSparseVoxelBLAS.VertexBuffer"));

		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(VertexBuffer), 0.0);
		PooledVertexBuffer = GraphBuilder.ConvertToExternalBuffer(VertexBuffer);
	}

	// Morphs the dense-voxel topology into the sparse-voxel topology.
	CreateSparseVoxelBLAS(GraphBuilder, View, SparseVoxelUniformBuffer, NumVoxelsBuffer, GraphBuilder.RegisterExternalBuffer(PooledVertexBuffer));

	FRayTracingGeometryInitializer GeometryInitializer;
	GeometryInitializer.DebugName = TEXT(" (HeterogeneousVolume)"); // TODO: Include resource name ie: *PrimitiveSceneProxy->GetResourceName().ToString();
	//GeometryInitializer.IndexBuffer = EmptyIndexBuffer;
	GeometryInitializer.GeometryType = RTGT_Procedural;
	GeometryInitializer.bFastBuild = false;

	FRayTracingGeometrySegment Segment;
	Segment.NumPrimitives = MipVoxelCount;
	Segment.MaxVertices = MipVoxelCount * 2;
	Segment.VertexBufferStride = 2u * sizeof(FVector3f);
	Segment.VertexBuffer = PooledVertexBuffer->GetRHI();

	GeometryInitializer.Segments.Add(Segment);
	GeometryInitializer.TotalPrimitiveCount = Segment.NumPrimitives;
	RayTracingGeometries.Add(GraphBuilder.RHICmdList.CreateRayTracingGeometry(GeometryInitializer));
	RayTracingTransforms.Add(HeterogeneousVolumeInterface->GetLocalToWorld());
}

BEGIN_SHADER_PARAMETER_STRUCT(FBuildBLASPassParams, )
	RDG_BUFFER_ACCESS(ScratchBuffer, ERHIAccess::UAVCompute)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FBuildTLASPassParams, )
	RDG_BUFFER_ACCESS(RayTracingSceneScratchBuffer, ERHIAccess::UAVCompute)
	RDG_BUFFER_ACCESS(RayTracingSceneInstanceBuffer, ERHIAccess::SRVCompute)
	RDG_BUFFER_ACCESS(RayTracingSceneBuffer, ERHIAccess::BVHWrite)
END_SHADER_PARAMETER_STRUCT()

void GenerateRayTracingScene(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	FViewInfo& View,
	// Ray tracing data
	TConstArrayView<FRayTracingGeometryRHIRef> RayTracingGeometries,
	TConstArrayView<FMatrix> RayTracingTransforms,
	// Output
	FRayTracingScene& RayTracingScene
)
{
	RayTracingScene.Reset();
	View.SetRayTracingSceneViewHandle(RayTracingScene.AddView(0)); // TODO: Could use View.GetViewKey() if view is persistent

	TArray<FRayTracingGeometryBuildParams> BuildParams;
	uint32 BLASScratchSize = 0;

	// Collect instances
	for (int32 GeometryIndex = 0; GeometryIndex < RayTracingGeometries.Num(); ++GeometryIndex)
	{
		FRHIRayTracingGeometry* RayTracingGeometry = RayTracingGeometries[GeometryIndex];
		checkf(RayTracingGeometry, TEXT("RayTracingGeometryInstance not created."));

		FRayTracingGeometryBuildParams Params;
		Params.Geometry = RayTracingGeometry;
		Params.BuildMode = EAccelerationStructureBuildMode::Build;

		BuildParams.Add(Params);

		const FRayTracingGeometryInitializer& Initializer = RayTracingGeometry->GetInitializer();

		FRayTracingAccelerationStructureSize SizeInfo = RHICalcRayTracingGeometrySize(Initializer);
		BLASScratchSize = Align(BLASScratchSize + SizeInfo.BuildScratchSize, GRHIRayTracingScratchBufferAlignment);

		FRayTracingGeometryInstance RayTracingGeometryInstance = {};
		RayTracingGeometryInstance.GeometryRHI = RayTracingGeometry;
		RayTracingGeometryInstance.NumTransforms = 1;
		RayTracingGeometryInstance.Transforms = MakeArrayView(&RayTracingTransforms[GeometryIndex], 1);

		RayTracingScene.AddTransientInstance(RayTracingGeometryInstance, ERayTracingSceneLayer::Base, View.GetRayTracingSceneViewHandle());
	}

	FRDGBufferDesc ScratchBufferDesc;
	ScratchBufferDesc.Usage = EBufferUsageFlags::RayTracingScratch | EBufferUsageFlags::StructuredBuffer;
	ScratchBufferDesc.BytesPerElement = GRHIRayTracingScratchBufferAlignment;
	ScratchBufferDesc.NumElements = FMath::DivideAndRoundUp(BLASScratchSize, GRHIRayTracingScratchBufferAlignment);

	FRDGBufferRef ScratchBuffer = GraphBuilder.CreateBuffer(ScratchBufferDesc, TEXT("HeterogeneousVolumes.BLASSharedScratchBuffer"));

	// Build instance BLAS
	FBuildBLASPassParams* PassParamsBLAS = GraphBuilder.AllocParameters<FBuildBLASPassParams>();
	PassParamsBLAS->ScratchBuffer = ScratchBuffer;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("BuildRayTracingGeometries"),
		PassParamsBLAS,
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull | ERDGPassFlags::NeverParallel,
		[
			PassParamsBLAS,
			BuildParams = MoveTemp(BuildParams)
		](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FRHIBufferRange ScratchBufferRange;
			ScratchBufferRange.Buffer = PassParamsBLAS->ScratchBuffer->GetRHI();
			ScratchBufferRange.Offset = 0;

			RHICmdList.BuildAccelerationStructures(BuildParams, ScratchBufferRange);
		}
	);

	// Create RayTracingScene
	const FGPUScene* EmptyGPUScene = nullptr;
	RayTracingScene.Update(GraphBuilder, View.GetSceneUniforms(), EmptyGPUScene, ERDGPassFlags::Compute);
	RayTracingScene.Build(GraphBuilder, ERDGPassFlags::Compute | ERDGPassFlags::NeverCull | ERDGPassFlags::NeverParallel, nullptr);

	GraphBuilder.AddDispatchHint();
}

IMPLEMENT_RT_PAYLOAD_TYPE(ERayTracingPayloadType::SparseVoxel, 28);

class FHeterogeneousVolumesSparseVoxelsHitGroup : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHeterogeneousVolumesSparseVoxelsHitGroup)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FHeterogeneousVolumesSparseVoxelsHitGroup, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSparseVoxelUniformBufferParameters, SparseVoxelUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform) && DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform) &&
			FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::SparseVoxel;
	}
};

IMPLEMENT_GLOBAL_SHADER(FHeterogeneousVolumesSparseVoxelsHitGroup, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesHardwareRayTracing.usf", "closesthit=SparseVoxelsClosestHitShader anyhit=SparseVoxelsAnyHitShader intersection=SparseVoxelsIntersectionShader", SF_RayHitGroup);

class FHeterogeneousVolumesSparseVoxelMS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHeterogeneousVolumesSparseVoxelMS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FHeterogeneousVolumesSparseVoxelMS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform) && DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform) &&
			FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::SparseVoxel;
	}

	using FParameters = FEmptyShaderParameters;
};

IMPLEMENT_GLOBAL_SHADER(FHeterogeneousVolumesSparseVoxelMS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesHardwareRayTracing.usf", "SparseVoxelsMissShader", SF_RayMiss);

class FRenderLightingCacheWithPreshadingRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderLightingCacheWithPreshadingRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRenderLightingCacheWithPreshadingRGS, FGlobalShader)

	class FLightingCacheMode : SHADER_PERMUTATION_INT("DIM_LIGHTING_CACHE_MODE", 2);
	class FIndirectLightingMode : SHADER_PERMUTATION_INT("INDIRECT_LIGHTING_MODE", 3);
	using FPermutationDomain = TShaderPermutationDomain<FLightingCacheMode, FIndirectLightingMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene 
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		// Lighting data
		SHADER_PARAMETER(int, bApplyEmissionAndTransmittance)
		SHADER_PARAMETER(int, bApplyDirectLighting)
		SHADER_PARAMETER(int, bApplyShadowTransmittance)
		SHADER_PARAMETER(int, LightType)
		SHADER_PARAMETER_STRUCT_REF(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER(float, VolumetricScatteringIntensity)

		// Shadow data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardDirectionalLightShadowMapParameters, ForwardDirLightShadowStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingShaderParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER(int32, VirtualShadowMapId)

		// Global illumination data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLumenTranslucencyLightingUniforms, LumenGIVolumeStruct)
		// TODO: Ambient occlusion pipeline
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, AmbientOcclusionTexture)
		SHADER_PARAMETER(FIntVector, AmbientOcclusionResolution)
		SHADER_PARAMETER(float, IndirectInscatteringFactor)

		// Sparse Volume
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSparseVoxelUniformBufferParameters, SparseVoxelUniformBuffer)

		// Volume
		SHADER_PARAMETER(int, MipLevel)

		// Ray
		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, bJitter)

		// Transmittance volume data
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightingCacheParameters, LightingCache)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWLightingCacheTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.template Get<FIndirectLightingMode>() != static_cast<int32>(HeterogeneousVolumes::EIndirectLightingMode::LightingCachePass))
		{
			return false;
		}

		return ShouldCompileRayTracingShadersForProject(Parameters.Platform) && DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform) &&
			FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		if (DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::SparseVoxel;
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderLightingCacheWithPreshadingRGS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesHardwareRayTracing.usf", "RenderLightingCacheWithPreshadingRGS", SF_RayGen);

class FRenderSingleScatteringWithPreshadingRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderSingleScatteringWithPreshadingRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRenderSingleScatteringWithPreshadingRGS, FGlobalShader)

	class FApplyShadowTransmittanceDim : SHADER_PERMUTATION_BOOL("DIM_APPLY_SHADOW_TRANSMITTANCE");
	//class FUseTransmittanceVolume : SHADER_PERMUTATION_BOOL("DIM_USE_TRANSMITTANCE_VOLUME");
	class FUseInscatteringVolume : SHADER_PERMUTATION_BOOL("DIM_USE_INSCATTERING_VOLUME");
	class FUseLumenGI : SHADER_PERMUTATION_BOOL("DIM_USE_LUMEN_GI");
	using FPermutationDomain = TShaderPermutationDomain<FApplyShadowTransmittanceDim, FUseInscatteringVolume, FUseLumenGI>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene 
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		// Lighting data
		SHADER_PARAMETER(int, bApplyEmissionAndTransmittance)
		SHADER_PARAMETER(int, bApplyDirectLighting)
		SHADER_PARAMETER(int, LightType)
		SHADER_PARAMETER_STRUCT_REF(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER(float, VolumetricScatteringIntensity)

		// Shadow data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardDirectionalLightShadowMapParameters, ForwardDirLightShadowStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingShaderParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER(int32, VirtualShadowMapId)

		// Atmosphere
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogStruct)

		// Indirect Lighting
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLumenTranslucencyLightingUniforms, LumenGIVolumeStruct)

		// Sparse Volume
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSparseVoxelUniformBufferParameters, SparseVoxelUniformBuffer)

		// Volume
		SHADER_PARAMETER(int, MipLevel)

		// Transmittance volume
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightingCacheParameters, LightingCache)

		// Ray
		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, bJitter)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWLightingTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform) && DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform) &&
			FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		if (DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::SparseVoxel;
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderSingleScatteringWithPreshadingRGS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesHardwareRayTracing.usf", "RenderSingleScatteringWithPreshadingRGS", SF_RayGen);

FRayTracingLocalShaderBindings* BuildRayTracingMaterialBindings(
	FRHICommandList& RHICmdList,
	const FViewInfo& View,
	FRHIUniformBuffer* UniformBuffer,
	TConstArrayView<FRayTracingGeometryRHIRef> RayTracingGeometries
)
{
	auto Alloc = [&](uint32 Size, uint32 Align)
	{
		return RHICmdList.Bypass()
			? FMemStack::Get().Alloc(Size, Align)
			: RHICmdList.Alloc(Size, Align);
	};

	// Allocate bindings
	const uint32 NumBindings = 1;
	FRayTracingLocalShaderBindings* Bindings = (FRayTracingLocalShaderBindings*)Alloc(sizeof(FRayTracingLocalShaderBindings) * NumBindings, alignof(FRayTracingLocalShaderBindings));

	// Allocate and assign uniform buffers
	const uint32 NumUniformBuffers = 1;
	FRHIUniformBuffer** UniformBufferArray = (FRHIUniformBuffer**)Alloc(sizeof(FRHIUniformBuffer*) * NumUniformBuffers, alignof(FRHIUniformBuffer*));
	UniformBufferArray[0] = UniformBuffer;

	// Fill bindings
	for (uint32 BindingIndex = 0; BindingIndex < NumBindings; ++BindingIndex)
	{
		// TODO: Declare useful user-data??
		uint32 UserData = 0;

		FRayTracingLocalShaderBindings Binding = {};
		Binding.RecordIndex = 0;
		Binding.Geometry = RayTracingGeometries[BindingIndex];
		Binding.SegmentIndex = 0;
		Binding.UserData = UserData;
		Binding.UniformBuffers = UniformBufferArray;
		Binding.NumUniformBuffers = NumUniformBuffers;

		Bindings[BindingIndex] = Binding;
	}

	return Bindings;
}

FRayTracingPipelineState* BuildRayTracingPipelineState(
	FRHICommandList& RHICmdList,
	const FViewInfo& View,
	FRHIRayTracingShader* RayGenerationShader,
	uint32& OutMaxLocalBindingDataSize
)
{
	FRayTracingPipelineStateInitializer Initializer;
	Initializer.MaxPayloadSizeInBytes = GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType::SparseVoxel);

	// Get the ray tracing materials
	auto HitGroupShaders = View.ShaderMap->GetShader<FHeterogeneousVolumesSparseVoxelsHitGroup>();
	FRHIRayTracingShader* HitShaderTable[] = {
		HitGroupShaders.GetRayTracingShader()
	};
	Initializer.SetHitGroupTable(HitShaderTable);

	auto MissShader = View.ShaderMap->GetShader<FHeterogeneousVolumesSparseVoxelMS>();
	FRHIRayTracingShader* MissShaderTable[] = {
		MissShader.GetRayTracingShader()
	};
	Initializer.SetMissShaderTable(MissShaderTable);

	FRHIRayTracingShader* RayGenShaderTable[] = {
		RayGenerationShader
	};
	Initializer.SetRayGenShaderTable(RayGenShaderTable);

	OutMaxLocalBindingDataSize = Initializer.GetMaxLocalBindingDataSize();

	FRayTracingPipelineState* RayTracingPipelineState = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(RHICmdList, Initializer);

	return RayTracingPipelineState;
}

void RenderLightingCacheWithPreshadingHardwareRayTracing(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	const FSceneTextures& SceneTextures,
	// Light data
	bool bApplyEmissionAndTransmittance,
	bool bApplyDirectLighting,
	bool bApplyShadowTransmittance,
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Shadow data
	const FVisibleLightInfo* VisibleLightInfo,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	// Sparse voxel data
	TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters> SparseVoxelUniformBuffer,
	// Ray tracing data
	FRayTracingScene& RayTracingScene,
	TConstArrayView<FRayTracingGeometryRHIRef> RayTracingGeometries,
	// Output
	FRDGTextureRef& LightingCacheTexture
)
{
	// Note must be done in the same scope as we add the pass otherwise the UB lifetime will not be guaranteed
	FDeferredLightUniformStruct DeferredLightUniform = GetDeferredLightParameters(View, *LightSceneInfo);
	TUniformBufferRef<FDeferredLightUniformStruct> DeferredLightUB = CreateUniformBufferImmediate(DeferredLightUniform, UniformBuffer_SingleDraw);

	FRenderLightingCacheWithPreshadingRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderLightingCacheWithPreshadingRGS::FParameters>();
	{
		// Scene
		PassParameters->TLAS = RayTracingScene.GetLayerView(ERayTracingSceneLayer::Base, View.GetRayTracingSceneViewHandle());
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);

		// Light data
		check(LightSceneInfo != nullptr);
		PassParameters->bApplyEmissionAndTransmittance = bApplyEmissionAndTransmittance;
		PassParameters->bApplyDirectLighting = bApplyDirectLighting;
		PassParameters->bApplyShadowTransmittance = bApplyShadowTransmittance;
		PassParameters->DeferredLight = DeferredLightUB;
		PassParameters->LightType = LightType;
		PassParameters->VolumetricScatteringIntensity = LightSceneInfo->Proxy->GetVolumetricScatteringIntensity();

		// Sparse Voxel data
		PassParameters->SparseVoxelUniformBuffer = SparseVoxelUniformBuffer;

		// Transmittance volume
		HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
		PassParameters->LightingCache.LightingCacheResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
		PassParameters->LightingCache.LightingCacheVoxelBias = HeterogeneousVolumeInterface->GetShadowBiasFactor();
		//PassParameters->LightingCache.LightingCacheTexture = GraphBuilder.CreateSRV(LightingCacheTexture);

		// Ray data
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetMaxStepCount();
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();
		PassParameters->MipLevel = HeterogeneousVolumes::GetMipLevel();

		// Shadow data
		PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		PassParameters->ForwardDirLightShadowStruct = View.ForwardLightingResources.ForwardDirLightShadowUniformBuffer;
		if (VisibleLightInfo != nullptr)
		{
			const FProjectedShadowInfo* ProjectedShadowInfo = GetShadowForInjectionIntoVolumetricFog(*VisibleLightInfo);
			bool bDynamicallyShadowed = ProjectedShadowInfo != NULL;
			if (bDynamicallyShadowed)
			{
				GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, ProjectedShadowInfo, PassParameters->VolumeShadowingShaderParameters);
			}
			else
			{
				SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			}
			PassParameters->VirtualShadowMapId = VisibleLightInfo->GetVirtualShadowMapId(&View);
		}
		else
		{
			SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			PassParameters->VirtualShadowMapId = -1;
		}
		PassParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);

		// Global illumination data
		auto* LumenUniforms = GraphBuilder.AllocParameters<FLumenTranslucencyLightingUniforms>();
		LumenUniforms->Parameters = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());
		PassParameters->LumenGIVolumeStruct = GraphBuilder.CreateUniformBuffer(LumenUniforms);
		// TODO: Ambient occlusion pipeline
		//PassParameters->AmbientOcclusionTexture = AmbientOcclusionTexture;
		//PassParameters->AmbientOcclusionResolution = HeterogeneousVolumes::GetAmbientOcclusionResolution(HeterogeneousVolumeInterface, LODValue);
		PassParameters->IndirectInscatteringFactor = HeterogeneousVolumes::GetIndirectLightingFactor();

		// Output
		PassParameters->RWLightingCacheTexture = GraphBuilder.CreateUAV(LightingCacheTexture);
	}

	FString PassName;
#if WANTS_DRAW_MESH_EVENTS
	if (GetEmitDrawEvents())
	{
		FString LightName = "none";
		if (LightSceneInfo != nullptr)
		{
			FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
		}
		FString ModeName = HeterogeneousVolumes::UseLightingCacheForInscattering() ? TEXT("In-Scattering") : TEXT("Transmittance");
		PassName = FString::Printf(TEXT("RenderLightingCacheWithPreshadingRGS [%s] (Light = %s)"), *ModeName, *LightName);
	}
#endif // WANTS_DRAW_MESH_EVENTS

	// Indirect lighting accumulation is coupled with directional light, because it doesn't voxel cull. It is assumed to exist and shadow.
	int32 IndirectLightingMode = LightType == LightType_Directional ? static_cast<int32>(HeterogeneousVolumes::GetIndirectLightingMode()) : 0;

	FRenderLightingCacheWithPreshadingRGS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRenderLightingCacheWithPreshadingRGS::FLightingCacheMode>(HeterogeneousVolumes::GetLightingCacheMode() - 1);
	PermutationVector.Set<FRenderLightingCacheWithPreshadingRGS::FIndirectLightingMode>(IndirectLightingMode);
	TShaderRef<FRenderLightingCacheWithPreshadingRGS> RayGenerationShader = View.ShaderMap->GetShader<FRenderLightingCacheWithPreshadingRGS>(PermutationVector);
	HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
	FIntVector VolumeResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
	FIntPoint DispatchResolution = FIntPoint(VolumeResolution.X, VolumeResolution.Y * VolumeResolution.Z);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("%s %ux%u", *PassName, DispatchResolution.X, DispatchResolution.Y),
		PassParameters,
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		[
			PassParameters,
			&View,
			&RayTracingScene,
			RayTracingGeometries,
			RayGenerationShader,
			DispatchResolution
		](FRHICommandList& RHICmdList)
		{
			// Set ray-gen bindings
			FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
			SetShaderParameters(GlobalResources, RayGenerationShader, *PassParameters);

			// Create pipeline
			uint32 MaxLocalBindingDataSize = 0;
			FRayTracingPipelineState* RayTracingPipelineState = BuildRayTracingPipelineState(RHICmdList, View, RayGenerationShader.GetRayTracingShader(), MaxLocalBindingDataSize);
			
			FRayTracingShaderBindingTableInitializer SBTInitializer;
			// WARNING: Currently hit-group indexing is required to bind uniform buffers to hit-group shaders.
			SBTInitializer.HitGroupIndexingMode = ERayTracingHitGroupIndexingMode::Allow;
			SBTInitializer.ShaderBindingMode = ERayTracingShaderBindingMode::RTPSO;
			SBTInitializer.NumGeometrySegments = 1;
			SBTInitializer.NumShaderSlotsPerGeometrySegment = RAY_TRACING_NUM_SHADER_SLOTS;
			SBTInitializer.NumMissShaderSlots = 1;
			SBTInitializer.NumCallableShaderSlots = 0;
			SBTInitializer.LocalBindingDataSize = MaxLocalBindingDataSize;

			FShaderBindingTableRHIRef SBT = RHICmdList.CreateRayTracingShaderBindingTable(SBTInitializer);

			// Set hit-group bindings
			const uint32 NumBindings = 1;
			FRayTracingLocalShaderBindings* Bindings = BuildRayTracingMaterialBindings(RHICmdList, View, PassParameters->SparseVoxelUniformBuffer->GetRHI(), RayTracingGeometries);
			RHICmdList.SetRayTracingHitGroups(SBT, RayTracingPipelineState, NumBindings, Bindings);
			RHICmdList.SetRayTracingMissShaders(SBT, RayTracingPipelineState, NumBindings, Bindings);
			RHICmdList.CommitShaderBindingTable(SBT);

			// Dispatch
			RHICmdList.RayTraceDispatch(
				RayTracingPipelineState,
				RayGenerationShader.GetRayTracingShader(),
				SBT,
				GlobalResources,
				DispatchResolution.X, DispatchResolution.Y);
		}
	);
}

void RenderSingleScatteringWithPreshadingHardwareRayTracing(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	const FSceneTextures& SceneTextures,
	// Light data
	bool bApplyEmissionAndTransmittance,
	bool bApplyDirectLighting,
	bool bApplyShadowTransmittance,
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Shadow data
	const FVisibleLightInfo* VisibleLightInfo,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	// Sparse voxel data
	TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters> SparseVoxelUniformBuffer,
	// Ray tracing data
	FRayTracingScene& RayTracingScene,
	TConstArrayView<FRayTracingGeometryRHIRef> RayTracingGeometries,
	// Transmittance volume
	FRDGTextureRef LightingCacheTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeTexture
)
{
	// Note must be done in the same scope as we add the pass otherwise the UB lifetime will not be guaranteed
	FDeferredLightUniformStruct DeferredLightUniform;
	if (bApplyDirectLighting && (LightSceneInfo != nullptr))
	{
		DeferredLightUniform = GetDeferredLightParameters(View, *LightSceneInfo);
	}
	TUniformBufferRef<FDeferredLightUniformStruct> DeferredLightUB = CreateUniformBufferImmediate(DeferredLightUniform, UniformBuffer_SingleDraw);

	FRenderSingleScatteringWithPreshadingRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderSingleScatteringWithPreshadingRGS::FParameters>();
	{
		// Scene
		PassParameters->TLAS = RayTracingScene.GetLayerView(ERayTracingSceneLayer::Base, View.GetRayTracingSceneViewHandle());
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);

		// Light data
		PassParameters->bApplyEmissionAndTransmittance = bApplyEmissionAndTransmittance;
		PassParameters->bApplyDirectLighting = bApplyDirectLighting;
		if (PassParameters->bApplyDirectLighting && (LightSceneInfo != nullptr))
		{
			PassParameters->VolumetricScatteringIntensity = LightSceneInfo->Proxy->GetVolumetricScatteringIntensity();
		}
		PassParameters->DeferredLight = DeferredLightUB;
		PassParameters->LightType = LightType;

		// Shadow data
		PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		PassParameters->ForwardDirLightShadowStruct = View.ForwardLightingResources.ForwardDirLightShadowUniformBuffer;
		if (VisibleLightInfo != nullptr)
		{
			const FProjectedShadowInfo* ProjectedShadowInfo = GetShadowForInjectionIntoVolumetricFog(*VisibleLightInfo);
			bool bDynamicallyShadowed = ProjectedShadowInfo != NULL;
			if (bDynamicallyShadowed)
			{
				GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, ProjectedShadowInfo, PassParameters->VolumeShadowingShaderParameters);
			}
			else
			{
				SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			}
			PassParameters->VirtualShadowMapId = VisibleLightInfo->GetVirtualShadowMapId(&View);
		}
		else
		{
			SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			PassParameters->VirtualShadowMapId = -1;
		}
		PassParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);

		TRDGUniformBufferRef<FFogUniformParameters> FogBuffer = CreateFogUniformBuffer(GraphBuilder, View);
		PassParameters->FogStruct = FogBuffer;

		// Indirect lighting data
		auto* LumenUniforms = GraphBuilder.AllocParameters<FLumenTranslucencyLightingUniforms>();
		LumenUniforms->Parameters = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());
		PassParameters->LumenGIVolumeStruct = GraphBuilder.CreateUniformBuffer(LumenUniforms);

		// Sparse Voxel data
		PassParameters->SparseVoxelUniformBuffer = SparseVoxelUniformBuffer;

		// Volume data
		PassParameters->MipLevel = HeterogeneousVolumes::GetMipLevel();

		// Transmittance volume
		if ((HeterogeneousVolumes::UseLightingCacheForTransmittance() && bApplyShadowTransmittance) || HeterogeneousVolumes::UseLightingCacheForInscattering())
		{
			HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
			PassParameters->LightingCache.LightingCacheResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
			PassParameters->LightingCache.LightingCacheVoxelBias = HeterogeneousVolumeInterface->GetShadowBiasFactor();
			PassParameters->LightingCache.LightingCacheTexture = LightingCacheTexture;
		}
		else
		{
			PassParameters->LightingCache.LightingCacheResolution = FIntVector::ZeroValue;
			PassParameters->LightingCache.LightingCacheVoxelBias = 0.0f;
			PassParameters->LightingCache.LightingCacheTexture = FRDGSystemTextures::Get(GraphBuilder).VolumetricBlack;
		}

		// Ray data
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetMaxStepCount();
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();

		// Output
		PassParameters->RWLightingTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeTexture);
	}

	FRenderSingleScatteringWithPreshadingRGS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingRGS::FApplyShadowTransmittanceDim>(bApplyShadowTransmittance);
	//PermutationVector.Set<FRenderSingleScatteringWithPreshadingRGS::FUseTransmittanceVolume>(HeterogeneousVolumes::UseLightingCacheForTransmittance());
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingRGS::FUseInscatteringVolume>(HeterogeneousVolumes::UseLightingCacheForInscattering());
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingRGS::FUseLumenGI>(HeterogeneousVolumes::UseIndirectLighting() && View.GetLumenTranslucencyGIVolume().Texture0 != nullptr);
	TShaderRef<FRenderSingleScatteringWithPreshadingRGS> RayGenerationShader = View.ShaderMap->GetShader<FRenderSingleScatteringWithPreshadingRGS>(PermutationVector);
	FIntPoint DispatchResolution = View.ViewRect.Size();

	FString LightName = "none";
	if (LightSceneInfo != nullptr)
	{
		FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("RenderSingleScatteringWithPreshadingRGS (Light = %s) %ux%u", *LightName, DispatchResolution.X, DispatchResolution.Y),
		PassParameters,
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		[
			PassParameters,
			&View,
			&RayTracingScene,
			RayTracingGeometries,
			RayGenerationShader,
			DispatchResolution
		](FRHICommandList& RHICmdList)
		{
			// Set ray-gen bindings
			FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
			SetShaderParameters(GlobalResources, RayGenerationShader, *PassParameters);

			// Create pipeline
			uint32 MaxLocalBindingDataSize = 0;
			FRayTracingPipelineState* RayTracingPipelineState = BuildRayTracingPipelineState(RHICmdList, View, RayGenerationShader.GetRayTracingShader(), MaxLocalBindingDataSize);
			
			FRayTracingShaderBindingTableInitializer SBTInitializer;
			// WARNING: Currently hit-group indexing is required to bind uniform buffers to hit-group shaders.
			SBTInitializer.HitGroupIndexingMode = ERayTracingHitGroupIndexingMode::Allow;
			SBTInitializer.ShaderBindingMode = ERayTracingShaderBindingMode::RTPSO;
			SBTInitializer.NumGeometrySegments = 1;
			SBTInitializer.NumShaderSlotsPerGeometrySegment = RAY_TRACING_NUM_SHADER_SLOTS;
			SBTInitializer.NumMissShaderSlots = 0;
			SBTInitializer.NumCallableShaderSlots = 0;
			SBTInitializer.LocalBindingDataSize = MaxLocalBindingDataSize;

			FShaderBindingTableRHIRef SBT = RHICmdList.CreateRayTracingShaderBindingTable(SBTInitializer);

			// Set hit-group bindings
			const uint32 NumBindings = 1;
			FRayTracingLocalShaderBindings* Bindings = BuildRayTracingMaterialBindings(RHICmdList, View, PassParameters->SparseVoxelUniformBuffer->GetRHI(), RayTracingGeometries);
			RHICmdList.SetRayTracingHitGroups(SBT, RayTracingPipelineState, NumBindings, Bindings);
			RHICmdList.SetRayTracingMissShaders(SBT, RayTracingPipelineState, NumBindings, Bindings);
			RHICmdList.CommitShaderBindingTable(SBT);

			// Dispatch
			RHICmdList.RayTraceDispatch(
				RayTracingPipelineState,
				RayGenerationShader.GetRayTracingShader(),
				SBT,
				GlobalResources,
				DispatchResolution.X, DispatchResolution.Y);
		}
	);
}

#endif // RHI_RAYTRACING
