// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "BlueNoise.h"
#include "Containers/ArrayView.h"
#include "Containers/SetUtilities.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "ConvexVolume.h"
#include "DistanceFieldLightingShared.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "ExtensibleUniformBuffer.h"
#include "FogRendering.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "HeterogeneousVolumeInterface.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "LightRendering.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "LocalVertexFactory.h"
#include "Logging/LogMacros.h"
#include "Lumen/LumenTranslucencyVolumeLighting.h"
#include "MaterialDomain.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "Math/Plane.h"
#include "Math/Sphere.h"
#include "MeshBatch.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshPassProcessor.h"
#include "MeshPassUtils.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/LargeWorldCoordinates.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RayTracing/RaytracingOptions.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphParameter.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneTypes.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShadowRendering.h"
#include "Stats/Stats.h"
#include "SystemTextures.h"
#include "UniformBuffer.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VolumeLighting.h"
#include "VolumetricFog.h"
#include "SceneViewState.h"
#include "ShaderCompilerCore.h"

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesScalability(
	TEXT("r.HeterogeneousVolumes.Scalability"),
	3,
	TEXT("Determines the scalability setting for Heterogeneous Volumes (Default = 3)\n")
	TEXT("0: Low\n")
	TEXT("1: High\n")
	TEXT("2: Epic\n")
	TEXT("3: Cinematic"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarHeterogeneousLightTypeDirectional(
	TEXT("r.HeterogeneousVolumes.LightType.Directional"),
	1,
	TEXT("Enables illumination from the directional light (Default = 1)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarHeterogeneousLightTypePoint(
	TEXT("r.HeterogeneousVolumes.LightType.Point"),
	1,
	TEXT("Enables illumination from point lights (Default = 1)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarHeterogeneousLightTypeSpot(
	TEXT("r.HeterogeneousVolumes.LightType.Spot"),
	1,
	TEXT("Enables illumination from spot lights (Default = 1)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarHeterogeneousLightTypeRect(
	TEXT("r.HeterogeneousVolumes.LightType.Rect"),
	1,
	TEXT("Enables illumination from rect lights (Default = 1)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousLightingCacheBoundsCulling(
	TEXT("r.HeterogeneousVolumes.LightingCache.BoundsCulling"),
	1,
	TEXT("Enables bounds culling when populating the lighting cache (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousLightingLiveShadingScreenTileClassification(
	TEXT("r.HeterogeneousVolumes.LiveShading.ScreenTileClassification"),
	0,
	TEXT("Enables screen tile classification for increased occupancy (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesSupportOverlappingVolumes(
	TEXT("r.HeterogeneousVolumes.SupportOverlappingVolumes"),
	0,
	TEXT("Enables support for overlapping volumes (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesUseExistenceMask(
	TEXT("r.HeterogeneousVolumes.UseExistenceMask"),
	1,
	TEXT("Creates an evaluation mask which culls operations to the areas with non-zero extinction (Default = 1)"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowsUseCameraSceneDepth(
	TEXT("r.HeterogeneousVolumes.Shadows.UseCameraSceneDepth"),
	0,
	TEXT("Culls Camera AVSM by SceneDepth (Default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesShadowsNearClippingDistance(
	TEXT("r.HeterogeneousVolumes.Shadows.NearClippingDistance"),
	1.0,
	TEXT("Near clipping plane distance for shadow projection (Default = 1.0)"),
	ECVF_RenderThreadSafe
);

#if 0
static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesBilinearInterpolation(
	TEXT("r.HeterogeneousVolumes.BilinearInterpolation"),
	1,
	TEXT("Enables bilinear interpolation when querying AVSM (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesAdaptiveMarching(
	TEXT("r.HeterogeneousVolumes.AdaptiveMarching"),
	0,
	TEXT("Enables adaptive marching (Default = 0)"),
	ECVF_RenderThreadSafe
);
#endif

static TAutoConsoleVariable<bool> CVarHeterogeneousVolumesReferenceFastPath(
	TEXT("r.HeterogeneousVolumes.ReferenceFastPath"),
	0,
	TEXT("Enables minimal VGPR mode (Default = 0)"),
	ECVF_RenderThreadSafe
);

namespace HeterogeneousVolumes
{
	EScalabilityMode GetScalabilityMode()
	{
		int32 ScalabilityValue = FMath::Clamp(CVarHeterogeneousVolumesScalability.GetValueOnAnyThread(), 0, 3);
		return static_cast<EScalabilityMode>(ScalabilityValue);
	}

	bool SupportsLightType(uint32 LightType)
	{
		switch (LightType)
		{
		case LightType_Directional:
			return CVarHeterogeneousLightTypeDirectional.GetValueOnRenderThread();
		case LightType_Point:
			return CVarHeterogeneousLightTypePoint.GetValueOnRenderThread();
		case LightType_Spot:
			return CVarHeterogeneousLightTypeSpot.GetValueOnRenderThread();
		case LightType_Rect:
			return CVarHeterogeneousLightTypeRect.GetValueOnRenderThread();
		}

		return false;
	}

	bool ShouldBoundsCull()
	{
		return CVarHeterogeneousLightingCacheBoundsCulling.GetValueOnRenderThread() != 0;
	}

	bool ShouldUseScreenTileClassification()
	{
		return CVarHeterogeneousLightingLiveShadingScreenTileClassification.GetValueOnRenderThread() != 0;
	}

	bool SupportsOverlappingVolumes()
	{
		return ShouldCompositeHeterogeneousVolumesWithTranslucency() &&
			(GetTranslucencyCompositingMode() == EShadowType::AdaptiveVolumetricShadowMap) &&
			(CVarHeterogeneousVolumesSupportOverlappingVolumes.GetValueOnRenderThread() != 0);
	}

	bool UseExistenceMask()
	{
		return CVarHeterogeneousVolumesUseExistenceMask.GetValueOnRenderThread() != 0;
	}
#if 0
	bool UseBilinearInterpolation()
	{
		return CVarHeterogeneousVolumesBilinearInterpolation.GetValueOnRenderThread() != 0;
	}
	bool ShouldAdaptiveMarch()
	{
		return CVarHeterogeneousVolumesAdaptiveMarching.GetValueOnRenderThread() != 0;
	}
#endif
	bool UseReferenceFastPath()
	{
		return CVarHeterogeneousVolumesReferenceFastPath.GetValueOnRenderThread() != 0;
	}

	bool ShadowsUseCameraSceneDepth()
	{
		return CVarHeterogeneousVolumesShadowsUseCameraSceneDepth.GetValueOnRenderThread() != 0;
	}

	float GetShadowNearClippingDistance()
	{
		return FMath::Max(CVarHeterogeneousVolumesShadowsNearClippingDistance.GetValueOnRenderThread(), 0.1);
	}

	enum class EAVSMSampleMode
	{
		Disabled,
		Performance,
		Quality
	};

	EAVSMSampleMode GetAVSMSampleMode(bool bEnabled)
	{
		EAVSMSampleMode SampleMode = EAVSMSampleMode::Disabled;
		if (bEnabled)
		{
			SampleMode = HeterogeneousVolumes::GetShadowMaxSampleCount() > 16 ? EAVSMSampleMode::Quality : EAVSMSampleMode::Performance;
		}

		return SampleMode;
	}
}

//-OPT: Remove duplicate bindings
// At the moment we need to bind the mesh draw parameters as they will be applied and on some RHIs this will crash if the texture is nullptr
// We have the same parameters in the loose FParameters shader structure that are applied after the mesh draw.
class FRenderLightingCacheLooseBindings
{
	DECLARE_TYPE_LAYOUT(FRenderLightingCacheLooseBindings, NonVirtual);

public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		SceneDepthTextureBinding.Bind(ParameterMap, TEXT("SceneDepthTexture"));
		ShadowDepthTextureBinding.Bind(ParameterMap, TEXT("ShadowDepthTexture"));
		ShadowDepthTextureSamplerBinding.Bind(ParameterMap, TEXT("ShadowDepthTextureSampler"));
		StaticShadowDepthTextureBinding.Bind(ParameterMap, TEXT("StaticShadowDepthTexture"));
		StaticShadowDepthTextureSamplerBinding.Bind(ParameterMap, TEXT("StaticShadowDepthTextureSampler"));
		ShadowDepthCubeTextureBinding.Bind(ParameterMap, TEXT("ShadowDepthCubeTexture"));
		ShadowDepthCubeTexture2Binding.Bind(ParameterMap, TEXT("ShadowDepthCubeTexture2"));
		ShadowDepthCubeTextureSamplerBinding.Bind(ParameterMap, TEXT("ShadowDepthCubeTextureSampler"));
		LightingCacheTextureBinding.Bind(ParameterMap, TEXT("LightingCacheTexture"));
	}

	template<typename TPassParameters>	
	void SetParameters(FMeshDrawSingleShaderBindings& ShaderBindings, const TPassParameters* PassParameters)
	{
		ShaderBindings.AddTexture(
			SceneDepthTextureBinding,
			FShaderResourceParameter(),
			TStaticSamplerState<SF_Point>::GetRHI(),
			PassParameters->SceneTextures.SceneDepthTexture->GetRHI()
		);
		ShaderBindings.AddTexture(
			ShadowDepthTextureBinding,
			ShadowDepthTextureSamplerBinding,
			PassParameters->VolumeShadowingShaderParameters.ShadowDepthTextureSampler,
			PassParameters->VolumeShadowingShaderParameters.ShadowDepthTexture->GetRHI()
		);
		ShaderBindings.AddTexture(
			StaticShadowDepthTextureBinding,
			StaticShadowDepthTextureSamplerBinding,
			PassParameters->VolumeShadowingShaderParameters.StaticShadowDepthTextureSampler,
			PassParameters->VolumeShadowingShaderParameters.StaticShadowDepthTexture
		);
		ShaderBindings.AddTexture(
			ShadowDepthCubeTextureBinding,
			ShadowDepthCubeTextureSamplerBinding,
			PassParameters->VolumeShadowingShaderParameters.OnePassPointShadowProjection.ShadowDepthCubeTextureSampler,
			PassParameters->VolumeShadowingShaderParameters.OnePassPointShadowProjection.ShadowDepthCubeTexture->GetRHI()
		);
		ShaderBindings.AddTexture(
			ShadowDepthCubeTexture2Binding,
			ShadowDepthCubeTextureSamplerBinding,
			PassParameters->VolumeShadowingShaderParameters.OnePassPointShadowProjection.ShadowDepthCubeTextureSampler,
			PassParameters->VolumeShadowingShaderParameters.OnePassPointShadowProjection.ShadowDepthCubeTexture->GetRHI()
		);
		ShaderBindings.AddTexture(
			LightingCacheTextureBinding,
			FShaderResourceParameter(),
			TStaticSamplerState<SF_Point>::GetRHI(),
			PassParameters->LightingCache.LightingCacheTexture->GetRHI()
		);
	}

	LAYOUT_FIELD(FShaderResourceParameter, SceneDepthTextureBinding);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthTextureBinding);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthTextureSamplerBinding);
	LAYOUT_FIELD(FShaderResourceParameter, StaticShadowDepthTextureBinding);
	LAYOUT_FIELD(FShaderResourceParameter, StaticShadowDepthTextureSamplerBinding);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthCubeTextureBinding);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthCubeTexture2Binding);
	LAYOUT_FIELD(FShaderResourceParameter, ShadowDepthCubeTextureSamplerBinding);
	LAYOUT_FIELD(FShaderResourceParameter, LightingCacheTextureBinding);
};
IMPLEMENT_TYPE_LAYOUT(FRenderLightingCacheLooseBindings);

class FRenderLightingCacheWithLiveShadingCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRenderLightingCacheWithLiveShadingCS, MeshMaterial);

	class FScalabilityMode : SHADER_PERMUTATION_INT("HV_SCALABILITY_MODE", 4);
	class FLightingCacheMode : SHADER_PERMUTATION_INT("DIM_LIGHTING_CACHE_MODE", 2);
	class FAVSMSampleMode : SHADER_PERMUTATION_INT("AVSM_SAMPLE_MODE", 3);
	using FPermutationDomain = TShaderPermutationDomain<FScalabilityMode, FLightingCacheMode, FAVSMSampleMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)

		// Light data
		SHADER_PARAMETER(int, bApplyEmissionAndTransmittance)
		SHADER_PARAMETER(int, bApplyDirectLighting)
		SHADER_PARAMETER(int, bApplyShadowTransmittance)
		SHADER_PARAMETER(int, LightType)
		SHADER_PARAMETER_STRUCT_REF(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER(float, VolumetricScatteringIntensity)

		// Shadow data
		SHADER_PARAMETER(float, ShadowStepSize)
		SHADER_PARAMETER(float, ShadowStepFactor)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardDirectionalLightShadowMapParameters, ForwardDirLightShadowStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingShaderParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER(int32, VirtualShadowMapId)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FBeerShadowMapUniformBufferParameters, BeerShadowMap)

		// Global illumination data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLumenTranslucencyLightingUniforms, LumenGIVolumeStruct)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, AmbientOcclusionTexture)
		SHADER_PARAMETER(FIntVector, AmbientOcclusionResolution)
		SHADER_PARAMETER(float, IndirectInscatteringFactor)

		// Object data
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLocal)
		SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
		SHADER_PARAMETER(FVector3f, LocalBoundsExtent)
		SHADER_PARAMETER(int32, PrimitiveId)

		// Ray data
		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(float, MaxShadowTraceDistance)
		SHADER_PARAMETER(float, StepSize)
		SHADER_PARAMETER(float, StepFactor)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER(int, StochasticFilteringMode)

		// Volume data
		SHADER_PARAMETER(FIntVector, VoxelResolution)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightingCacheParameters, LightingCache)
		SHADER_PARAMETER(FIntVector, VoxelMin)
		SHADER_PARAMETER(FIntVector, VoxelMax)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D<float>, ExistenceMaskTexture)

		// Optional cinematic features
		SHADER_PARAMETER(int, AVSMSampleMode)
		SHADER_PARAMETER(int, bUseLightingCacheForInscattering)
		SHADER_PARAMETER(int, bUseExistenceMask)
		SHADER_PARAMETER(int, bIsOfflineRender)
		SHADER_PARAMETER(int, IndirectLightingMode)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, RWLightingCacheTexture)
	END_SHADER_PARAMETER_STRUCT()

	FRenderLightingCacheWithLiveShadingCS() = default;

	FRenderLightingCacheWithLiveShadingCS(
		const FMeshMaterialShaderType::CompiledShaderInitializerType & Initializer
	)
		: FMeshMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false
		);
		ShaderLooseBindings.Bind(Initializer.ParameterMap);
	}

	static bool ShouldCompilePermutation(
		const FMaterialShaderPermutationParameters & Parameters
	)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}

		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform)
			&& DoesMaterialShaderSupportHeterogeneousVolumes(Parameters.MaterialParameters);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		// All scalability modes store in-scattering, except for Cinematic which can store transmittance or in-scattering
		if (PermutationVector.template Get<FScalabilityMode>() != static_cast<int32>(HeterogeneousVolumes::EScalabilityMode::Cinematic))
		{
			PermutationVector.template Set<FLightingCacheMode>(1);
		}

		// Remap all other scalability settings to Epic
		if (PermutationVector.template Get<FScalabilityMode>() != static_cast<int32>(HeterogeneousVolumes::EScalabilityMode::Cinematic))
		{
			PermutationVector.template Set<FScalabilityMode>(static_cast<int32>(HeterogeneousVolumes::EScalabilityMode::Epic));
		}

		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(
		const FMaterialShaderPermutationParameters & Parameters,
		FShaderCompilerEnvironment & OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		if (DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		//OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC); // @lh-todo - Disabled to workaround SPIRV-Cross bug: StructuredBuffer<uint> is translated to ByteAddressBuffer in HLSL backend
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }

	LAYOUT_FIELD(FRenderLightingCacheLooseBindings, ShaderLooseBindings);
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRenderLightingCacheWithLiveShadingCS, TEXT("/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesLiveShadingPipeline.usf"), TEXT("RenderLightingCacheWithLiveShadingCS"), SF_Compute);

namespace HeterogeneousVolumes
{
	struct FScreenTile
	{
		int32 Id;
	};
} // namespace

namespace HeterogeneousVolumes {
	enum EDispatchMode
	{
		DirectDispatch,
		IndirectDispatch
	};
} // namespace HeterogeneousVolumes

template <HeterogeneousVolumes::EDispatchMode DispatchMode>
class FRenderSingleScatteringWithLiveShadingCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRenderSingleScatteringWithLiveShadingCS, MeshMaterial);

	class FScalabilityMode : SHADER_PERMUTATION_INT("HV_SCALABILITY_MODE", 4);
	class FUseInscatteringVolume : SHADER_PERMUTATION_BOOL("DIM_USE_INSCATTERING_VOLUME");
	class FAVSMSampleMode : SHADER_PERMUTATION_INT("AVSM_SAMPLE_MODE", 3);
	class FSupportOverlappingVolumes : SHADER_PERMUTATION_BOOL("SUPPORT_OVERLAPPING_VOLUMES");
	class FWriteVelocity : SHADER_PERMUTATION_BOOL("DIM_WRITE_VELOCITY");
	using FPermutationDomain = TShaderPermutationDomain<
		FScalabilityMode, 
		FUseInscatteringVolume, 
		FAVSMSampleMode, 
		FSupportOverlappingVolumes,
		FWriteVelocity
	>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)

		// Light data
		SHADER_PARAMETER(int, bHoldout)
		SHADER_PARAMETER(int, bApplyEmissionAndTransmittance)
		SHADER_PARAMETER(int, bApplyDirectLighting)
		SHADER_PARAMETER(int, bApplyShadowTransmittance)
		SHADER_PARAMETER(int, LightType)
		SHADER_PARAMETER_STRUCT_REF(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER(float, VolumetricScatteringIntensity)

		// Shadow data
		SHADER_PARAMETER(float, ShadowStepSize)
		SHADER_PARAMETER(float, ShadowStepFactor)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardDirectionalLightShadowMapParameters, ForwardDirLightShadowStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingShaderParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER(int32, VirtualShadowMapId)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FAdaptiveVolumetricShadowMaps, AVSMs)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FBeerShadowMapUniformBufferParameters, BeerShadowMap)

		// Atmosphere
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogStruct)
		SHADER_PARAMETER(int, bApplyHeightFog)
		SHADER_PARAMETER(int, bApplyVolumetricFog)
		SHADER_PARAMETER(int, bCreateBeerShadowMap)

		// Indirect Lighting
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLumenTranslucencyLightingUniforms, LumenGIVolumeStruct)
		SHADER_PARAMETER(float, IndirectInscatteringFactor)

		// Object data
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLocal)
		SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
		SHADER_PARAMETER(FVector3f, LocalBoundsExtent)
		SHADER_PARAMETER(int32, PrimitiveId)

		// Volume data
		SHADER_PARAMETER(FIntVector, VoxelResolution)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightingCacheParameters, LightingCache)

		SHADER_PARAMETER(FIntVector, AmbientOcclusionResolution)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, AmbientOcclusionTexture)

		// Ray data
		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(float, StepSize)
		SHADER_PARAMETER(float, StepFactor)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER(int, StochasticFilteringMode)

		// Dispatch data
		SHADER_PARAMETER(FIntVector, GroupCount)
		SHADER_PARAMETER(int32, DownsampleFactor)

		// Optional indirect dispatch data
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<HeterogeneousVolumes::FScreenTile>, ScreenTileBuffer)

		// Optional cinematic features
		SHADER_PARAMETER(int, bUseLightingCacheForInscattering)
		SHADER_PARAMETER(int, IndirectLightingMode)
		SHADER_PARAMETER(int, bWriteVelocity)
		SHADER_PARAMETER(int, AVSMSampleMode)
		SHADER_PARAMETER(int, bSupportsOverlappingVolumes)
		SHADER_PARAMETER(int, bIsOfflineRender)
		SHADER_PARAMETER(int, FogInscatteringMode)
		SHADER_PARAMETER(int, bUseAnalyticDerivatives)
		SHADER_PARAMETER(int, bUseReferenceFastPath)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWLightingTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWHoldoutTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWVelocityTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWBeerShadowMapTexture)
		//SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<Volumes::FDebugOutput>, RWDebugOutputBuffer)
	END_SHADER_PARAMETER_STRUCT()

	FRenderSingleScatteringWithLiveShadingCS() = default;

	FRenderSingleScatteringWithLiveShadingCS(
		const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer
	)
		: FMeshMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false
		);
		ShaderLooseBindings.Bind(Initializer.ParameterMap);
	}

	static bool ShouldCompilePermutation(
		const FMaterialShaderPermutationParameters& Parameters
	)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}

		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform)
			&& DoesMaterialShaderSupportHeterogeneousVolumes(Parameters.MaterialParameters);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		// All scalability modes store in-scattering, except for Cinematic which can store transmittance or in-scattering
		if (PermutationVector.template Get<FScalabilityMode>() != static_cast<int32>(HeterogeneousVolumes::EScalabilityMode::Cinematic))
		{
			PermutationVector.template Set<FUseInscatteringVolume>(1);
		}

		// Remap all other scalability settings to Epic
		if (PermutationVector.template Get<FScalabilityMode>() != static_cast<int32>(HeterogeneousVolumes::EScalabilityMode::Cinematic))
		{
			PermutationVector.template Set<FScalabilityMode>(static_cast<int32>(HeterogeneousVolumes::EScalabilityMode::Epic));
		}

		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(
		const FMaterialShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("FOG_MATERIALBLENDING_OVERRIDE"), 1);

		if (DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }

	LAYOUT_FIELD(FRenderLightingCacheLooseBindings, ShaderLooseBindings);
};

typedef FRenderSingleScatteringWithLiveShadingCS<HeterogeneousVolumes::DirectDispatch> FRenderSingleScatteringWithLiveShadingDirectCS;
//typedef FRenderSingleScatteringWithLiveShadingCS<HeterogeneousVolumes::IndirectDispatch> FRenderSingleScatteringWithLiveShadingIndirectCS;
typedef FRenderSingleScatteringWithLiveShadingCS<HeterogeneousVolumes::DirectDispatch> FRenderSingleScatteringWithLiveShadingIndirectCS;


IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FRenderSingleScatteringWithLiveShadingDirectCS, TEXT("/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesLiveShadingPipeline.usf"), TEXT("RenderSingleScatteringWithLiveShadingCS"), SF_Compute);
//IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FRenderSingleScatteringWithLiveShadingIndirectCS, TEXT("/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesLiveShadingPipeline.usf"), TEXT("RenderSingleScatteringWithLiveShadingIndirectCS"), SF_Compute);

template<bool bWithLumen, HeterogeneousVolumes::EDispatchMode DispatchMode, typename ComputeShaderType>
void AddComputePass(
	FRDGBuilder& GraphBuilder,
	TShaderRef<ComputeShaderType>& ComputeShader,
	typename ComputeShaderType::FParameters* PassParameters,
	const FScene* Scene,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FMaterial& Material,
	const FString& PassName,
	FIntVector GroupCount,
	FRDGBufferRef IndirectArgsBuffer,
	uint32 IndirectArgOffset
)
{
	//ClearUnusedGraphResources(ComputeShader, PassParameters);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("%s", *PassName),
		PassParameters,
		ERDGPassFlags::Compute,
		[ComputeShader, PassParameters, Scene, MaterialRenderProxy, &Material, GroupCount, IndirectArgsBuffer, IndirectArgOffset](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			FMeshMaterialShaderElementData ShaderElementData;
			ShaderElementData.InitializeMeshMaterialData();

			FMeshProcessorShaders PassShaders;
			PassShaders.ComputeShader = ComputeShader;

			FMeshDrawShaderBindings ShaderBindings;
			ShaderBindings.Initialize(PassShaders);
			{
				FMeshDrawSingleShaderBindings SingleShaderBindings = ShaderBindings.GetSingleShaderBindings(SF_Compute);
				ComputeShader->GetShaderBindings(Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, Material, ShaderElementData, SingleShaderBindings);
				SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FDeferredLightUniformStruct>(), PassParameters->DeferredLight.GetUniformBuffer());
				SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FForwardLightUniformParameters>(), PassParameters->ForwardLightStruct.GetUniformBuffer()->GetRHIRef());
				SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FForwardDirectionalLightShadowMapParameters>(), PassParameters->ForwardDirLightShadowStruct.GetUniformBuffer()->GetRHIRef());
				SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FVirtualShadowMapUniformParameters>(), PassParameters->VirtualShadowMapSamplingParameters.VirtualShadowMap.GetUniformBuffer()->GetRHIRef());
				SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FAdaptiveVolumetricShadowMaps>(), PassParameters->AVSM.GetUniformBuffer()->GetRHIRef());
				SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FAdaptiveVolumetricShadowMapUniformBufferParameters>(), PassParameters->AVSM.GetUniformBuffer()->GetRHIRef());
				if constexpr (bWithLumen)
				{
					SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FFogUniformParameters>(), PassParameters->FogStruct.GetUniformBuffer()->GetRHIRef());
					SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FLumenTranslucencyLightingUniforms>(), PassParameters->LumenGIVolumeStruct.GetUniformBuffer()->GetRHIRef());
					SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FBlueNoise>(), PassParameters->BlueNoise.GetUniformBuffer());
				}
				ComputeShader->ShaderLooseBindings.SetParameters(SingleShaderBindings, PassParameters);
				ShaderBindings.Finalize(&PassShaders);
			}

			if constexpr (DispatchMode == HeterogeneousVolumes::EDispatchMode::IndirectDispatch)
			{
				UE::MeshPassUtils::DispatchIndirect(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, IndirectArgsBuffer->GetIndirectRHICallBuffer(), IndirectArgOffset);
			}
			else
			{
				UE::MeshPassUtils::Dispatch(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, GroupCount);
			}
		}
	);
}

static void RenderLightingCacheWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	const FSceneTextures& SceneTextures,
	// Light data
	bool bApplyEmissionAndTransmittance,
	bool bApplyDirectLighting,
	bool bApplyShadowTransmittance,
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Shadow data
	const FVisibleLightInfo* VisibleLightInfo,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Global illumination data
	FRDGTextureRef AmbientOcclusionTexture,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* DefaultMaterialRenderProxy,
	FPersistentPrimitiveIndex PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Output
	FRDGTextureRef LightingCacheTexture
)
{
	const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
	const FMaterial& Material = DefaultMaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
	MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : DefaultMaterialRenderProxy;

	check(Material.GetMaterialDomain() == MD_Volume);

	// Note must be done in the same scope as we add the pass otherwise the UB lifetime will not be guaranteed
	FDeferredLightUniformStruct DeferredLightUniform = GetDeferredLightParameters(View, *LightSceneInfo);
	TUniformBufferRef<FDeferredLightUniformStruct> DeferredLightUB = CreateUniformBufferImmediate(DeferredLightUniform, UniformBuffer_SingleDraw);

	HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
	FIntVector LightingCacheResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);

	FRDGTextureRef DilatedExistenceTexture = FRDGSystemTextures::Get(GraphBuilder).VolumetricBlack;
	if (HeterogeneousVolumes::UseExistenceMask())
	{
		FRDGTextureRef ExistenceMaskTexture;
		RenderExistenceMaskWithLiveShading(
			GraphBuilder,
			// Scene data
			Scene,
			View,
			SceneTextures,
			// Object data
			HeterogeneousVolumeInterface,
			DefaultMaterialRenderProxy,
			PersistentPrimitiveIndex,
			LocalBoxSphereBounds,
			LightingCacheResolution,
			// Output
			ExistenceMaskTexture
		);

		DilateExistenceMask(
			GraphBuilder,
			// Scene data
			Scene,
			View,
			// Existence texture data
			ExistenceMaskTexture,
			LightingCacheResolution,
			// Output
			DilatedExistenceTexture
		);
	}

	FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
	FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
	FMatrix InstanceToWorld = InstanceToLocal * LocalToWorld;
	FMatrix WorldToInstance = InstanceToWorld.Inverse();
	FMatrix LocalToInstance = InstanceToLocal.Inverse();
	FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(LocalToInstance);

	FRenderLightingCacheWithLiveShadingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderLightingCacheWithLiveShadingCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);

		// Light data
		check(LightSceneInfo != nullptr);
		PassParameters->bApplyEmissionAndTransmittance = bApplyEmissionAndTransmittance;
		PassParameters->bApplyDirectLighting = bApplyDirectLighting;
		PassParameters->bApplyShadowTransmittance = bApplyShadowTransmittance;
		PassParameters->DeferredLight = DeferredLightUB;
		PassParameters->LightType = LightType;
		PassParameters->VolumetricScatteringIntensity = LightSceneInfo->Proxy->GetVolumetricScatteringIntensity();

		// Object data
		// TODO: Convert to relative-local space
		//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
		//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
		PassParameters->LocalToWorld = FMatrix44f(InstanceToWorld);
		PassParameters->WorldToLocal = FMatrix44f(WorldToInstance);

		PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
		PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);
		PassParameters->PrimitiveId = PersistentPrimitiveIndex.Index;

		// Transmittance volume
		PassParameters->VoxelResolution = HeterogeneousVolumeInterface->GetVoxelResolution();
		PassParameters->LightingCache.LightingCacheResolution = LightingCacheResolution;
		PassParameters->LightingCache.LightingCacheVoxelBias = HeterogeneousVolumeInterface->GetShadowBiasFactor();
		//PassParameters->LightingCache.LightingCacheTexture = GraphBuilder.CreateSRV(LightingCacheTexture);
		PassParameters->LightingCache.LightingCacheTexture = FRDGSystemTextures::Get(GraphBuilder).VolumetricBlack;
		PassParameters->ExistenceMaskTexture = GraphBuilder.CreateSRV(DilatedExistenceTexture);

		// Ray data
		float LODFactor = HeterogeneousVolumes::CalcLODFactor(LODValue.LOD, LODValue.Bias);
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();
		PassParameters->MaxShadowTraceDistance = HeterogeneousVolumes::GetMaxShadowTraceDistance();
		PassParameters->StepSize = HeterogeneousVolumes::GetStepSize();
		PassParameters->StepFactor = HeterogeneousVolumeInterface->GetStepFactor() * LODFactor;
		PassParameters->ShadowStepSize = HeterogeneousVolumes::GetShadowStepSize();
		PassParameters->ShadowStepFactor = HeterogeneousVolumeInterface->GetShadowStepFactor() * LODFactor;
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetMaxStepCount();
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();
		PassParameters->StochasticFilteringMode = static_cast<int32>(HeterogeneousVolumes::GetStochasticFilteringMode());

		// Shadow data
		PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		PassParameters->ForwardDirLightShadowStruct = View.ForwardLightingResources.ForwardDirLightShadowUniformBuffer;
		if (VisibleLightInfo != nullptr)
		{
			const FProjectedShadowInfo* ProjectedShadowInfo = GetShadowForInjectionIntoVolumetricFog(*VisibleLightInfo);
			bool bDynamicallyShadowed = ProjectedShadowInfo != NULL;
			if (bDynamicallyShadowed)
			{
				GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, ProjectedShadowInfo, PassParameters->VolumeShadowingShaderParameters);
			}
			else
			{
				SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			}
			PassParameters->VirtualShadowMapId = VisibleLightInfo->GetVirtualShadowMapId(&View);
		}
		else
		{
			SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			PassParameters->VirtualShadowMapId = -1;
		}
		PassParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
		PassParameters->AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);
		PassParameters->BeerShadowMap = HeterogeneousVolumes::GetBeerShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);

		// Global illumination data
		auto* LumenUniforms = GraphBuilder.AllocParameters<FLumenTranslucencyLightingUniforms>();
		LumenUniforms->Parameters = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());
		PassParameters->LumenGIVolumeStruct = GraphBuilder.CreateUniformBuffer(LumenUniforms);
		PassParameters->AmbientOcclusionTexture = AmbientOcclusionTexture;
		PassParameters->AmbientOcclusionResolution = HeterogeneousVolumes::GetAmbientOcclusionResolution(HeterogeneousVolumeInterface, LODValue);
		PassParameters->IndirectInscatteringFactor = HeterogeneousVolumes::GetIndirectLightingFactor();

		// Optional cinematic features
		bool bUseAVSM = HeterogeneousVolumes::UseAdaptiveVolumetricShadowMapForSelfShadowing(HeterogeneousVolumeInterface->GetPrimitiveSceneProxy());
		int32 IndirectLightingMode = 0;
		if ((View.GetLumenTranslucencyGIVolume().Texture0 != nullptr) &&
			(LightType == LightType_Directional))
		{
			IndirectLightingMode = static_cast<int32>(HeterogeneousVolumes::GetIndirectLightingMode());
		}

		PassParameters->AVSMSampleMode = static_cast<int32>(HeterogeneousVolumes::GetAVSMSampleMode(bUseAVSM));
		PassParameters->bUseLightingCacheForInscattering = HeterogeneousVolumes::UseLightingCacheForInscattering();
		PassParameters->bUseExistenceMask = HeterogeneousVolumes::UseExistenceMask();
		PassParameters->bIsOfflineRender = View.bIsOfflineRender;
		PassParameters->IndirectLightingMode = IndirectLightingMode;

		// Output
		PassParameters->RWLightingCacheTexture = GraphBuilder.CreateUAV(LightingCacheTexture);
	}

	FString PassName;
#if WANTS_DRAW_MESH_EVENTS
	if (GetEmitDrawEvents())
	{
		FString LightName = "none";
		if (LightSceneInfo != nullptr)
		{
			FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
		}
		FString ModeName = HeterogeneousVolumes::UseLightingCacheForInscattering() ? TEXT("In-Scattering") : TEXT("Transmittance");
		PassName = FString::Printf(TEXT("RenderLightingCacheWithLiveShadingCS [%s] (Light = %s)"), *ModeName, *LightName);
	}
#endif // WANTS_DRAW_MESH_EVENTS

	PassParameters->VoxelMin = FIntVector::ZeroValue;
	PassParameters->VoxelMax = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue) - FIntVector(1);
	
	bool bShouldBoundsCull = HeterogeneousVolumes::ShouldBoundsCull();
	if (LightType != LightType_Directional && bShouldBoundsCull)
	{
		auto FloorVector = [](const FVector& V) {
			return FIntVector(
				FMath::FloorToFloat(V.X),
				FMath::FloorToFloat(V.Y),
				FMath::FloorToFloat(V.Z)
			);
		};

		auto CeilVector = [](const FVector& V) {
			return FIntVector(
				FMath::CeilToFloat(V.X),
				FMath::CeilToFloat(V.Y),
				FMath::CeilToFloat(V.Z)
			);
		};

		auto ClampVector = [](const FIntVector& V, const FIntVector& Min, const FIntVector& Max) {
			FIntVector IntV;
			IntV.X = FMath::Clamp(V.X, Min.X, Max.X);
			IntV.Y = FMath::Clamp(V.Y, Min.Y, Max.Y);
			IntV.Z = FMath::Clamp(V.Z, Min.Z, Max.Z);
			return IntV;
		};

		FSphere WorldLightBoundingSphere = LightSceneInfo->Proxy->GetBoundingSphere();
		FVector LocalLightCenter = WorldToInstance.TransformPosition(WorldLightBoundingSphere.Center);
		FVector LocalLightExtent = WorldToInstance.GetScaleVector() * WorldLightBoundingSphere.W;

		FVector LocalLightMin = LocalLightCenter - LocalLightExtent;
		FVector LocalLightMax = LocalLightCenter + LocalLightExtent;

		FVector LightingCacheMin = InstanceBoxSphereBounds.Origin - InstanceBoxSphereBounds.BoxExtent;
		FVector LightingCacheMax = InstanceBoxSphereBounds.Origin + InstanceBoxSphereBounds.BoxExtent;

		FVector LocalLightMinUV = (LocalLightMin - LightingCacheMin) / (LightingCacheMax - LightingCacheMin);
		FVector LocalLightMaxUV = (LocalLightMax - LightingCacheMin) / (LightingCacheMax - LightingCacheMin);
		FVector LightingCacheResolutionVector = FVector(PassParameters->LightingCache.LightingCacheResolution);
		PassParameters->VoxelMin = ClampVector(FloorVector(LocalLightMinUV * LightingCacheResolutionVector), FIntVector::ZeroValue, PassParameters->VoxelMax);
		PassParameters->VoxelMax = ClampVector(CeilVector(LocalLightMaxUV * LightingCacheResolutionVector), FIntVector::ZeroValue, PassParameters->VoxelMax);
	}

	FIntVector VoxelDimensions = PassParameters->VoxelMax - PassParameters->VoxelMin;
	if (VoxelDimensions.GetMin() > 0)
	{
		FIntVector GroupCount = PassParameters->VoxelMax - PassParameters->VoxelMin + FIntVector(1);
		check(GroupCount.X > 0 && GroupCount.Y > 0 && GroupCount.Z > 0);
		GroupCount.X = FMath::DivideAndRoundUp(GroupCount.X, FRenderLightingCacheWithLiveShadingCS::GetThreadGroupSize3D());
		GroupCount.Y = FMath::DivideAndRoundUp(GroupCount.Y, FRenderLightingCacheWithLiveShadingCS::GetThreadGroupSize3D());
		GroupCount.Z = FMath::DivideAndRoundUp(GroupCount.Z, FRenderLightingCacheWithLiveShadingCS::GetThreadGroupSize3D());

		bool bUseAVSM = HeterogeneousVolumes::UseAdaptiveVolumetricShadowMapForSelfShadowing(HeterogeneousVolumeInterface->GetPrimitiveSceneProxy());

		int32 IndirectLightingMode = 0;
		if ((View.GetLumenTranslucencyGIVolume().Texture0 != nullptr) &&
			(LightType == LightType_Directional))
		{
			IndirectLightingMode = static_cast<int32>(HeterogeneousVolumes::GetIndirectLightingMode());
		}

		FRenderLightingCacheWithLiveShadingCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRenderLightingCacheWithLiveShadingCS::FLightingCacheMode>(HeterogeneousVolumes::GetLightingCacheMode() - 1);
		PermutationVector.Set<FRenderLightingCacheWithLiveShadingCS::FScalabilityMode>(static_cast<int32>(HeterogeneousVolumes::GetScalabilityMode()));
		PermutationVector.Set<FRenderLightingCacheWithLiveShadingCS::FAVSMSampleMode>(static_cast<int32>(HeterogeneousVolumes::GetAVSMSampleMode(bUseAVSM)));
		PermutationVector = FRenderLightingCacheWithLiveShadingCS::RemapPermutation(PermutationVector);
		TShaderRef<FRenderLightingCacheWithLiveShadingCS> ComputeShader = Material.GetShader<FRenderLightingCacheWithLiveShadingCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);
		if (!ComputeShader.IsNull())
		{
			FRDGBufferRef IndirectArgsBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, 4);
			AddComputePass<false, HeterogeneousVolumes::EDispatchMode::DirectDispatch>(
				GraphBuilder,
				ComputeShader,
				PassParameters,
				Scene,
				MaterialRenderProxy,
				Material,
				PassName,
				GroupCount,
				IndirectArgsBuffer,
				0
			);
		}
	}
}

IMPLEMENT_UNIFORM_BUFFER_STRUCT(FAdaptiveVolumetricShadowMaps, "AVSMs");

FAdaptiveVolumetricShadowMapParameters GetAdaptiveVolumetricShadowMapParametersFromUniformBuffer(
	const TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>& UniformBuffer
)
{
	FAdaptiveVolumetricShadowMapParameters Parameters;
	{
		Parameters.NumShadowMatrices = UniformBuffer->GetParameters()->NumShadowMatrices;
		for (int32 i = 0; i < Parameters.NumShadowMatrices; ++i)
		{
			Parameters.TranslatedWorldToShadow[i] = UniformBuffer->GetParameters()->TranslatedWorldToShadow[i];
			Parameters.TranslatedWorldOrigin[i] = UniformBuffer->GetParameters()->TranslatedWorldOrigin[i];
			Parameters.TranslatedWorldPlane[i] = UniformBuffer->GetParameters()->TranslatedWorldPlane[i];
			Parameters.SplitDepths[i] = UniformBuffer->GetParameters()->SplitDepths[i];
		}

		Parameters.Resolution = UniformBuffer->GetParameters()->Resolution;
		Parameters.MaxSampleCount = UniformBuffer->GetParameters()->MaxSampleCount;
		Parameters.bIsEmpty = UniformBuffer->GetParameters()->bIsEmpty;
		Parameters.bIsDirectionalLight = UniformBuffer->GetParameters()->bIsDirectionalLight;
		Parameters.LinkedListBuffer = UniformBuffer->GetParameters()->LinkedListBuffer;
		Parameters.IndirectionBuffer = UniformBuffer->GetParameters()->IndirectionBuffer;
		Parameters.SampleBuffer = UniformBuffer->GetParameters()->SampleBuffer;
		Parameters.RadianceTexture = UniformBuffer->GetParameters()->RadianceTexture;
		Parameters.TextureSampler = UniformBuffer->GetParameters()->TextureSampler;
	}

	return Parameters;
}

TRDGUniformBufferRef<FAdaptiveVolumetricShadowMaps> CreateAdaptiveVolumetricShadowMapUniformBuffers(
	FRDGBuilder& GraphBuilder,
	FSceneViewState* ViewState,
	const FLightSceneInfo* LightSceneInfo
)
{
	FAdaptiveVolumetricShadowMaps* UniformBufferParameters = GraphBuilder.AllocParameters<FAdaptiveVolumetricShadowMaps>();
	{
		UniformBufferParameters->AVSM = GetAdaptiveVolumetricShadowMapParametersFromUniformBuffer(HeterogeneousVolumes::GetAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder, ViewState, LightSceneInfo));
		UniformBufferParameters->CameraAVSM = GetAdaptiveVolumetricShadowMapParametersFromUniformBuffer(HeterogeneousVolumes::GetAdaptiveVolumetricCameraMapUniformBuffer(GraphBuilder, ViewState));
	}

	return GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
}

class FScreenTileClassificationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FScreenTileClassificationCS);
	SHADER_USE_PARAMETER_STRUCT(FScreenTileClassificationCS, FGlobalShader);


	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		// Object data
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLocal)
		SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
		SHADER_PARAMETER(FVector3f, LocalBoundsExtent)

		// Ray data
		SHADER_PARAMETER(float, MaxTraceDistance)

		// Dispatch data
		SHADER_PARAMETER(FIntVector, GroupCount)
		SHADER_PARAMETER(int32, DownsampleFactor)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWNumScreenTilesBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<HeterogeneousVolumes::FScreenTile>, RWScreenTileBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
};

IMPLEMENT_GLOBAL_SHADER(FScreenTileClassificationCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesLiveShadingGlobalPipeline.usf", "ScreenTileClassificationCS", SF_Compute);

static void ScreenTileClassification(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	// Output
	FRDGBufferRef& ScreenTileIndirectArgsBuffer,
	FRDGBufferRef& ScreenTileBuffer
)
{
	FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(HeterogeneousVolumes::GetScaledViewRect(View.ViewRect), FRenderSingleScatteringWithLiveShadingIndirectCS::GetThreadGroupSize2D());
	int32 NumTiles = GroupCount.X * GroupCount.Y;

	FRDGBufferRef NumScreenTilesBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1),
		TEXT("HeterogeneousVolume.NumScreenTilesBuffer")
	);
	// TODO: Initialize elsewhere??
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(NumScreenTilesBuffer, PF_R32_UINT), 0);

	ScreenTileBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(HeterogeneousVolumes::FScreenTile), NumTiles),
		TEXT("HeterogeneousVolume.ScreenTileBuffer")
	);

	FScreenTileClassificationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FScreenTileClassificationCS::FParameters>();
	{
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);

		// Object data
		// TODO: Convert to relative-local space
		//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
		//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
		FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
		FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
		PassParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
		PassParameters->WorldToLocal = PassParameters->LocalToWorld.Inverse();

		FMatrix LocalToInstance = InstanceToLocal.Inverse();
		FBoxSphereBounds InstanceBoxSphereBounds = HeterogeneousVolumeInterface->GetLocalBounds().TransformBy(LocalToInstance);
		PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
		PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);

		// Ray data
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();

		// Dispatch data
		PassParameters->GroupCount = GroupCount;
		PassParameters->DownsampleFactor = HeterogeneousVolumes::GetDownsampleFactor();

		PassParameters->RWNumScreenTilesBuffer = GraphBuilder.CreateUAV(NumScreenTilesBuffer, PF_R32_UINT);
		PassParameters->RWScreenTileBuffer = GraphBuilder.CreateUAV(ScreenTileBuffer);
	}

	FScreenTileClassificationCS::FPermutationDomain PermutationVector;
	TShaderRef<FScreenTileClassificationCS> ComputeShader = View.ShaderMap->GetShader<FScreenTileClassificationCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("ScreenTileClassificationCS"),
		ComputeShader,
		PassParameters,
		GroupCount
	);

	ScreenTileIndirectArgsBuffer = NumScreenTilesBuffer;
}

template <HeterogeneousVolumes::EDispatchMode DispatchMode>
void RenderSingleScatteringWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	const FSceneTextures& SceneTextures,
	// Light data
	bool bApplyEmissionAndTransmittance,
	bool bApplyDirectLighting,
	bool bApplyShadowTransmittance,
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Shadow data
	const FVisibleLightInfo* VisibleLightInfo,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* DefaultMaterialRenderProxy,
	FPersistentPrimitiveIndex PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	FRDGTextureRef AmbientOcclusionTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadianceTexture,
	FRDGTextureRef& HeterogeneousVolumeVelocityTexture,
	FRDGTextureRef& HeterogeneousVolumeHoldoutTexture,
	FRDGTextureRef& HeterogeneousVolumeBeerShadowMapTexture
)
{
	typedef FRenderSingleScatteringWithLiveShadingCS<DispatchMode> FRenderSingleScatteringWithLiveShadingDispatchTypeCS;

	const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
	const FMaterial& Material = DefaultMaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
	MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : DefaultMaterialRenderProxy;
	check(Material.GetMaterialDomain() == MD_Volume);

	FRDGBufferRef NumScreenTilesBuffer;
	FRDGBufferRef ScreenTileBuffer;
	if (DispatchMode == HeterogeneousVolumes::IndirectDispatch)
	{
		ScreenTileClassification(
			GraphBuilder,
			Scene,
			View,
			SceneTextures,
			HeterogeneousVolumeInterface,
			NumScreenTilesBuffer,
			ScreenTileBuffer
		);
	}

	FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(HeterogeneousVolumes::GetScaledViewRect(View.ViewRect), FRenderSingleScatteringWithLiveShadingDispatchTypeCS::GetThreadGroupSize2D());

	// Note must be done in the same scope as we add the pass otherwise the UB lifetime will not be guaranteed
	FDeferredLightUniformStruct DeferredLightUniform;
	if (bApplyDirectLighting && (LightSceneInfo != nullptr))
	{
		DeferredLightUniform = GetDeferredLightParameters(View, *LightSceneInfo);
	}
	TUniformBufferRef<FDeferredLightUniformStruct> DeferredLightUB = CreateUniformBufferImmediate(DeferredLightUniform, UniformBuffer_SingleDraw);

	bool bUseAVSM = HeterogeneousVolumes::UseAdaptiveVolumetricShadowMapForSelfShadowing(HeterogeneousVolumeInterface->GetPrimitiveSceneProxy());
	bool bWriteVelocity = HeterogeneousVolumes::ShouldWriteVelocity() && HasBeenProduced(SceneTextures.Velocity);

	typename FRenderSingleScatteringWithLiveShadingDispatchTypeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<typename FRenderSingleScatteringWithLiveShadingDispatchTypeCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
		PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));
		FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
		PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

		// Light data
		HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
		float LODFactor = HeterogeneousVolumes::CalcLODFactor(LODValue.LOD, LODValue.Bias);
		PassParameters->bHoldout = HeterogeneousVolumes::IsHoldout(HeterogeneousVolumeInterface);
		PassParameters->bApplyEmissionAndTransmittance = bApplyEmissionAndTransmittance;
		PassParameters->bApplyDirectLighting = bApplyDirectLighting;
		PassParameters->bApplyShadowTransmittance = bApplyShadowTransmittance;
		if (bApplyDirectLighting && (LightSceneInfo != nullptr))
		{
			PassParameters->VolumetricScatteringIntensity = LightSceneInfo->Proxy->GetVolumetricScatteringIntensity();
		}
		else
		{
			PassParameters->VolumetricScatteringIntensity = 1.0f;
		}
		PassParameters->DeferredLight = DeferredLightUB;
		PassParameters->LightType = LightType;
		PassParameters->ShadowStepSize = HeterogeneousVolumes::GetShadowStepSize();
		PassParameters->ShadowStepFactor = HeterogeneousVolumeInterface->GetShadowStepFactor() * LODFactor;

		// Object data
		// TODO: Convert to relative-local space
		//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
		//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
		FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
		FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
		PassParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
		PassParameters->WorldToLocal = PassParameters->LocalToWorld.Inverse();

		FMatrix LocalToInstance = InstanceToLocal.Inverse();
		FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(LocalToInstance);
		PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
		PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);
		PassParameters->PrimitiveId = PersistentPrimitiveIndex.Index;

		// Volume data
		PassParameters->VoxelResolution = HeterogeneousVolumeInterface->GetVoxelResolution();

		// Ray data
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();
		PassParameters->StepSize = HeterogeneousVolumes::GetStepSize();
		PassParameters->StepFactor = HeterogeneousVolumeInterface->GetStepFactor() * LODFactor;
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetMaxStepCount();
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();
		PassParameters->StochasticFilteringMode = static_cast<int32>(HeterogeneousVolumes::GetStochasticFilteringMode());

		// Shadow data
		PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		PassParameters->ForwardDirLightShadowStruct = View.ForwardLightingResources.ForwardDirLightShadowUniformBuffer;
		if (VisibleLightInfo != nullptr)
		{
			const FProjectedShadowInfo* ProjectedShadowInfo = GetShadowForInjectionIntoVolumetricFog(*VisibleLightInfo);
			bool bDynamicallyShadowed = ProjectedShadowInfo != NULL;
			if (bDynamicallyShadowed)
			{
				GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, ProjectedShadowInfo, PassParameters->VolumeShadowingShaderParameters);
			}
			else
			{
				SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			}
			PassParameters->VirtualShadowMapId = VisibleLightInfo->GetVirtualShadowMapId(&View);
		}
		else
		{
			SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
		}
		PassParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
		if (HeterogeneousVolumes::SupportsOverlappingVolumes())
		{
			PassParameters->AVSMs = CreateAdaptiveVolumetricShadowMapUniformBuffers(GraphBuilder, View.ViewState, LightSceneInfo);
		}
		else
		{
			PassParameters->AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);
		}
		PassParameters->BeerShadowMap = HeterogeneousVolumes::GetBeerShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);

		TRDGUniformBufferRef<FFogUniformParameters> FogBuffer = CreateFogUniformBuffer(GraphBuilder, View);
		PassParameters->FogStruct = FogBuffer;
		PassParameters->bApplyHeightFog = HeterogeneousVolumes::ShouldApplyHeightFog();
		PassParameters->bApplyVolumetricFog = HeterogeneousVolumes::ShouldApplyVolumetricFog();
		bool bCreateBeerShadowMap = ShouldCompositeHeterogeneousVolumesWithTranslucency() && HeterogeneousVolumes::GetTranslucencyCompositingMode() == HeterogeneousVolumes::EShadowType::BeerShadowMap;
		PassParameters->bCreateBeerShadowMap = bCreateBeerShadowMap;

		// Indirect lighting data
		PassParameters->IndirectInscatteringFactor = HeterogeneousVolumes::GetIndirectLightingFactor();
		auto* LumenUniforms = GraphBuilder.AllocParameters<FLumenTranslucencyLightingUniforms>();
		LumenUniforms->Parameters = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());
		PassParameters->LumenGIVolumeStruct = GraphBuilder.CreateUniformBuffer(LumenUniforms);

		// Volume data
		if ((HeterogeneousVolumes::UseLightingCacheForTransmittance() && bApplyShadowTransmittance) || HeterogeneousVolumes::UseLightingCacheForInscattering())
		{
			PassParameters->LightingCache.LightingCacheResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
			PassParameters->LightingCache.LightingCacheVoxelBias = HeterogeneousVolumeInterface->GetShadowBiasFactor();
			PassParameters->LightingCache.LightingCacheTexture = LightingCacheTexture;
		}
		else
		{
			if (bUseAVSM)
			{
				PassParameters->LightingCache.LightingCacheResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
				PassParameters->LightingCache.LightingCacheVoxelBias = HeterogeneousVolumeInterface->GetShadowBiasFactor();
			}
			else
			{
				PassParameters->LightingCache.LightingCacheResolution = FIntVector::ZeroValue;
				PassParameters->LightingCache.LightingCacheVoxelBias = 0.0f;
			}
			PassParameters->LightingCache.LightingCacheTexture = FRDGSystemTextures::Get(GraphBuilder).VolumetricBlack;
		}

		PassParameters->AmbientOcclusionResolution = HeterogeneousVolumes::GetAmbientOcclusionResolution(HeterogeneousVolumeInterface, LODValue);
		PassParameters->AmbientOcclusionTexture = AmbientOcclusionTexture;

		// Dispatch data
		PassParameters->GroupCount = GroupCount;
		PassParameters->DownsampleFactor = HeterogeneousVolumes::GetDownsampleFactor();
		if (DispatchMode == HeterogeneousVolumes::IndirectDispatch)
		{
			PassParameters->IndirectArgs = NumScreenTilesBuffer;
			PassParameters->ScreenTileBuffer = GraphBuilder.CreateSRV(ScreenTileBuffer);
		}

		// Optional cinematic features
		// Indirect lighting accumulation is coupled with directional light, because it doesn't cull voxels. It is assumed to exist and shadow.
		int32 IndirectLightingMode = View.GetLumenTranslucencyGIVolume().Texture0 != nullptr ? static_cast<int32>(HeterogeneousVolumes::GetIndirectLightingMode()) : 0;

		PassParameters->bUseLightingCacheForInscattering = HeterogeneousVolumes::UseLightingCacheForInscattering();
		PassParameters->IndirectLightingMode = IndirectLightingMode;
		PassParameters->bWriteVelocity = bWriteVelocity;
		PassParameters->AVSMSampleMode = static_cast<int32>(HeterogeneousVolumes::GetAVSMSampleMode(bUseAVSM));
		PassParameters->bSupportsOverlappingVolumes = HeterogeneousVolumes::SupportsOverlappingVolumes();
		PassParameters->bIsOfflineRender = View.bIsOfflineRender;
		PassParameters->FogInscatteringMode = static_cast<int32>(HeterogeneousVolumes::GetFogInscatteringMode());
		PassParameters->bUseAnalyticDerivatives = HeterogeneousVolumes::UseAnalyticDerivatives();
		PassParameters->bUseReferenceFastPath = HeterogeneousVolumes::UseReferenceFastPath();

		// Output
		PassParameters->RWLightingTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeRadianceTexture);

		if (IsDeferredPrimitiveAlphaHoldoutSupported())
		{
			PassParameters->RWHoldoutTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeHoldoutTexture);
		}
		if (bWriteVelocity)
		{
			PassParameters->RWVelocityTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeVelocityTexture);
		}
		//if (bCreateBeerShadowMap)
		{
			PassParameters->RWBeerShadowMapTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeBeerShadowMapTexture);
		}
		//PassParameters->RWDebugOutputBuffer = GraphBuilder.CreateUAV(DebugOutputBuffer);
	}

	FString PassName;
#if WANTS_DRAW_MESH_EVENTS
	if (GetEmitDrawEvents())
	{
		FString LightName = "none";
		if (LightSceneInfo != nullptr)
		{
			FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
		}
		PassName = FString::Printf(TEXT("RenderSingleScatteringWithLiveShadingCS (Light = %s)"), *LightName);
	}
#endif // WANTS_DRAW_MESH_EVENTS
	

	typename FRenderSingleScatteringWithLiveShadingDispatchTypeCS::FPermutationDomain PermutationVector;
	PermutationVector.template Set<typename FRenderSingleScatteringWithLiveShadingDispatchTypeCS::FScalabilityMode>(static_cast<int32>(HeterogeneousVolumes::GetScalabilityMode()));
	PermutationVector.template Set<typename FRenderSingleScatteringWithLiveShadingDispatchTypeCS::FUseInscatteringVolume>(HeterogeneousVolumes::UseLightingCacheForInscattering());
	PermutationVector.template Set<typename FRenderSingleScatteringWithLiveShadingDispatchTypeCS::FAVSMSampleMode>(static_cast<int32>(HeterogeneousVolumes::GetAVSMSampleMode(bUseAVSM)));
	PermutationVector.template Set<typename FRenderSingleScatteringWithLiveShadingDispatchTypeCS::FSupportOverlappingVolumes>(HeterogeneousVolumes::SupportsOverlappingVolumes());
	PermutationVector.template Set<typename FRenderSingleScatteringWithLiveShadingDispatchTypeCS::FWriteVelocity>(bWriteVelocity);
	PermutationVector = FRenderSingleScatteringWithLiveShadingDispatchTypeCS::RemapPermutation(PermutationVector);
	TShaderRef<FRenderSingleScatteringWithLiveShadingDispatchTypeCS> ComputeShader = Material.GetShader<FRenderSingleScatteringWithLiveShadingDispatchTypeCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);
	if (!ComputeShader.IsNull())
	{
		AddComputePass<true, DispatchMode>(GraphBuilder, ComputeShader, PassParameters, Scene, MaterialRenderProxy, Material, PassName, GroupCount, PassParameters->IndirectArgs, 0);
	}
}

static void RenderWithTransmittanceVolumePipeline(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	FPersistentPrimitiveIndex PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	FRDGTextureRef AmbientOcclusionTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance,
	FRDGTextureRef& HeterogeneousVolumeVelocity,
	FRDGTextureRef& HeterogeneousVolumeHoldout,
	FRDGTextureRef& HeterogeneousVolumeBeerShadowMap
)
{
	// Light culling
	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact = HeterogeneousVolumes::GatherLights(Scene, View, HeterogeneousVolumeInterface);

	// Light loop:
	int32 NumPasses = FMath::Max(LightSceneInfoCompact.Num(), 1);
	for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
	{
		bool bIsLastPass = (PassIndex == (NumPasses - 1));
		bool bApplyEmissionAndTransmittance = bIsLastPass;
		bool bApplyDirectLighting = !LightSceneInfoCompact.IsEmpty();
		bool bApplyShadowTransmittance = false;

		uint32 LightType = 0;
		FLightSceneInfo* LightSceneInfo = nullptr;
		const FVisibleLightInfo* VisibleLightInfo = nullptr;
		if (bApplyDirectLighting)
		{
			LightType = LightSceneInfoCompact[PassIndex].LightType;
			LightSceneInfo = LightSceneInfoCompact[PassIndex].LightSceneInfo;
			check(LightSceneInfo != nullptr);

			bApplyDirectLighting = (LightSceneInfo != nullptr);
			if (LightSceneInfo)
			{
				VisibleLightInfo = &VisibleLightInfos[LightSceneInfo->Id];
				bApplyShadowTransmittance = LightSceneInfo->Proxy->CastsVolumetricShadow();
			}
		}

		if (HeterogeneousVolumes::UseLightingCacheForTransmittance() && bApplyShadowTransmittance)
		{
			RenderLightingCacheWithLiveShading(
				GraphBuilder,
				// Scene data
				Scene,
				View, ViewIndex,
				SceneTextures,
				// Light data
				bApplyEmissionAndTransmittance,
				bApplyDirectLighting,
				bApplyShadowTransmittance,
				LightType,
				LightSceneInfo,
				// Shadow data
				VisibleLightInfo,
				VirtualShadowMapArray,
				// Global illumination data
				AmbientOcclusionTexture,
				// Object data
				HeterogeneousVolumeInterface,
				MaterialRenderProxy,
				PersistentPrimitiveIndex,
				LocalBoxSphereBounds,
				// Output
				LightingCacheTexture
			);
		}
#if 0
		if (HeterogeneousVolumes::ShouldUseScreenTileClassification())
		{
			RenderSingleScatteringWithLiveShading<HeterogeneousVolumes::IndirectDispatch>(
				GraphBuilder,
				// Scene data
				Scene,
				View, ViewIndex,
				SceneTextures,
				// Light data
				bApplyEmissionAndTransmittance,
				bApplyDirectLighting,
				bApplyShadowTransmittance,
				LightType,
				LightSceneInfo,
				// Shadow data
				VisibleLightInfo,
				VirtualShadowMapArray,
				// Object data
				HeterogeneousVolumeInterface,
				MaterialRenderProxy,
				PersistentPrimitiveIndex,
				LocalBoxSphereBounds,
				// Transmittance acceleration
				LightingCacheTexture,
				AmbientOcclusionTexture,
				// Output
				HeterogeneousVolumeRadiance,
				HeterogeneousVolumeVelocity,
				HeterogeneousVolumeHoldout,
				HeterogeneousVolumeBeerShadowMap
			);
		}
		else
#endif
		{
			RenderSingleScatteringWithLiveShading<HeterogeneousVolumes::DirectDispatch>(
				GraphBuilder,
				// Scene data
				Scene,
				View, ViewIndex,
				SceneTextures,
				// Light data
				bApplyEmissionAndTransmittance,
				bApplyDirectLighting,
				bApplyShadowTransmittance,
				LightType,
				LightSceneInfo,
				// Shadow data
				VisibleLightInfo,
				VirtualShadowMapArray,
				// Object data
				HeterogeneousVolumeInterface,
				MaterialRenderProxy,
				PersistentPrimitiveIndex,
				LocalBoxSphereBounds,
				// Transmittance acceleration
				LightingCacheTexture,
				AmbientOcclusionTexture,
				// Output
				HeterogeneousVolumeRadiance,
				HeterogeneousVolumeVelocity,
				HeterogeneousVolumeHoldout,
				HeterogeneousVolumeBeerShadowMap
			);
		}
	}
}

static void RenderWithInscatteringVolumePipeline(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	FPersistentPrimitiveIndex PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	FRDGTextureRef AmbientOcclusionTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance,
	FRDGTextureRef& HeterogeneousVolumeVelocity ,
	FRDGTextureRef& HeterogeneousVolumeHoldout,
	FRDGTextureRef& HeterogeneousVolumeBeerShadowMap
)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);

	bool bRenderLightingCache = !HeterogeneousVolumes::IsHoldout(HeterogeneousVolumeInterface);
	if (bRenderLightingCache)
	{
		SCOPE_CYCLE_COUNTER(STATGROUP_HeterogeneousVolumesLightCache);

		// Light culling
		TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact = HeterogeneousVolumes::GatherLights(Scene, View, HeterogeneousVolumeInterface);

		// Light loop:
		int32 NumPasses = LightSceneInfoCompact.Num();
		for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
		{
			bool bApplyEmissionAndTransmittance = (PassIndex == (NumPasses - 1));
			bool bApplyDirectLighting = !LightSceneInfoCompact.IsEmpty();
			bool bApplyShadowTransmittance = false;

			uint32 LightType = 0;
			FLightSceneInfo* LightSceneInfo = nullptr;
			const FVisibleLightInfo* VisibleLightInfo = nullptr;
			if (bApplyDirectLighting)
			{
				LightType = LightSceneInfoCompact[PassIndex].LightType;
				LightSceneInfo = LightSceneInfoCompact[PassIndex].LightSceneInfo;
				check(LightSceneInfo != nullptr);

				bApplyDirectLighting = (LightSceneInfo != nullptr);
				if (LightSceneInfo)
				{
					VisibleLightInfo = &VisibleLightInfos[LightSceneInfo->Id];
					bApplyShadowTransmittance = LightSceneInfo->Proxy->CastsVolumetricShadow();
				}
			}

			RenderLightingCacheWithLiveShading(
				GraphBuilder,
				// Scene data
				Scene,
				View, ViewIndex,
				SceneTextures,
				// Light data
				bApplyEmissionAndTransmittance,
				bApplyDirectLighting,
				bApplyShadowTransmittance,
				LightType,
				LightSceneInfo,
				// Shadow data
				VisibleLightInfo,
				VirtualShadowMapArray,
				// Global illumination data
				AmbientOcclusionTexture,
				// Object data
				HeterogeneousVolumeInterface,
				MaterialRenderProxy,
				PersistentPrimitiveIndex,
				LocalBoxSphereBounds,
				// Output
				LightingCacheTexture
			);
		}
	}

	// Direct volume integrator
	{
		SCOPE_CYCLE_COUNTER(STATGROUP_HeterogeneousVolumesSingleScattering);

		bool bApplyEmissionAndTransmittance = true;
		bool bApplyDirectLighting = true;
		bool bApplyShadowTransmittance = true;

		uint32 LightType = 0;
		FLightSceneInfo* LightSceneInfo = nullptr;
		const FVisibleLightInfo* VisibleLightInfo = nullptr;
#if 0
		if (HeterogeneousVolumes::ShouldUseScreenTileClassification())
		{
			RenderSingleScatteringWithLiveShading<HeterogeneousVolumes::IndirectDispatch>(
				GraphBuilder,
				// Scene data
				Scene,
				View, ViewIndex,
				SceneTextures,
				// Light data
				bApplyEmissionAndTransmittance,
				bApplyDirectLighting,
				bApplyShadowTransmittance,
				LightType,
				LightSceneInfo,
				// Shadow data
				VisibleLightInfo,
				VirtualShadowMapArray,
				// Object data
				HeterogeneousVolumeInterface,
				MaterialRenderProxy,
				PersistentPrimitiveIndex,
				LocalBoxSphereBounds,
				// Transmittance acceleration
				LightingCacheTexture,
				AmbientOcclusionTexture,
				// Output
				HeterogeneousVolumeRadiance,
				HeterogeneousVolumeVelocity,
				HeterogeneousVolumeHoldout,
				HeterogeneousVolumeBeerShadowMap
			);
		}
		else
#endif
		{
			RenderSingleScatteringWithLiveShading<HeterogeneousVolumes::DirectDispatch>(
				GraphBuilder,
				// Scene data
				Scene,
				View, ViewIndex,
				SceneTextures,
				// Light data
				bApplyEmissionAndTransmittance,
				bApplyDirectLighting,
				bApplyShadowTransmittance,
				LightType,
				LightSceneInfo,
				// Shadow data
				VisibleLightInfo,
				VirtualShadowMapArray,
				// Object data
				HeterogeneousVolumeInterface,
				MaterialRenderProxy,
				PersistentPrimitiveIndex,
				LocalBoxSphereBounds,
				// Transmittance acceleration
				LightingCacheTexture,
				AmbientOcclusionTexture,
				// Output
				HeterogeneousVolumeRadiance,
				HeterogeneousVolumeVelocity,
				HeterogeneousVolumeHoldout,
				HeterogeneousVolumeBeerShadowMap
			);
		}
	}
}

void RenderWithLiveShading(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FPersistentPrimitiveIndex &PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance,
	FRDGTextureRef& HeterogeneousVolumeVelocity,
	FRDGTextureRef& HeterogeneousVolumeHoldout,
	FRDGTextureRef& HeterogeneousVolumeBeerShadowMap
)
{
	FRDGTextureRef AmbientOcclusionTexture;
	RenderAmbientOcclusionWithLiveShading(
		GraphBuilder,
		// Scene data
		Scene,
		View,
		SceneTextures,
		// Object data
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		PersistentPrimitiveIndex,
		LocalBoxSphereBounds,
		// Output
		AmbientOcclusionTexture
	);

	if (HeterogeneousVolumes::UseLightingCacheForInscattering())
	{
		RenderWithInscatteringVolumePipeline(
			GraphBuilder,
			SceneTextures,
			Scene,
			View, ViewIndex,
			// Shadow data
			VisibleLightInfos,
			VirtualShadowMapArray,
			// Object data
			HeterogeneousVolumeInterface,
			MaterialRenderProxy,
			PersistentPrimitiveIndex,
			LocalBoxSphereBounds,
			// Transmittance acceleration
			LightingCacheTexture,
			AmbientOcclusionTexture,
			// Output
			HeterogeneousVolumeRadiance,
			HeterogeneousVolumeVelocity,
			HeterogeneousVolumeHoldout,
			HeterogeneousVolumeBeerShadowMap
		);
	}
	else
	{
		RenderWithTransmittanceVolumePipeline(
			GraphBuilder,
			SceneTextures,
			Scene,
			View, ViewIndex,
			// Shadow data
			VisibleLightInfos,
			VirtualShadowMapArray,
			// Object data
			HeterogeneousVolumeInterface,
			MaterialRenderProxy,
			PersistentPrimitiveIndex,
			LocalBoxSphereBounds,
			// Transmittance acceleration
			LightingCacheTexture,
			AmbientOcclusionTexture,
			// Output
			HeterogeneousVolumeRadiance,
			HeterogeneousVolumeVelocity,
			HeterogeneousVolumeHoldout,
			HeterogeneousVolumeBeerShadowMap
		);
	}
}

class FRenderShadowMapLooseBindings
{
	DECLARE_TYPE_LAYOUT(FRenderShadowMapLooseBindings, NonVirtual);

public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		SceneDepthTextureBinding.Bind(ParameterMap, TEXT("SceneDepthTexture"));
	}

	template<typename TPassParameters>
	void SetParameters(FMeshDrawSingleShaderBindings& ShaderBindings, const TPassParameters* PassParameters)
	{
		ShaderBindings.AddTexture(
			SceneDepthTextureBinding,
			FShaderResourceParameter(),
			TStaticSamplerState<SF_Point>::GetRHI(),
			PassParameters->SceneTextures.SceneDepthTexture->GetRHI()
		);
	}

	LAYOUT_FIELD(FShaderResourceParameter, SceneDepthTextureBinding);
};
IMPLEMENT_TYPE_LAYOUT(FRenderShadowMapLooseBindings);

class FClearVolumetricShadowMapCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearVolumetricShadowMapCS);
	SHADER_USE_PARAMETER_STRUCT(FClearVolumetricShadowMapCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, ShadowResolution)
		SHADER_PARAMETER(int32, MaxSampleCount)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int2>, RWVolumetricShadowLinkedListBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// Disable in-scattering features
		//OutEnvironment.SetDefine(TEXT("DIM_USE_TRANSMITTANCE_VOLUME"), 0);
		//OutEnvironment.SetDefine(TEXT("DIM_USE_INSCATTERING_VOLUME"), 0);
		//OutEnvironment.SetDefine(TEXT("DIM_USE_LUMEN_GI"), 0);

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		//OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC); // @lh-todo - Disabled to workaround SPIRV-Cross bug: StructuredBuffer<uint> is translated to ByteAddressBuffer in HLSL backend
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		//OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FClearVolumetricShadowMapCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridShadows.usf", "ClearVolumetricShadowMapCS", SF_Compute);

void ClearVolumetricShadowMap(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	// Shadow data
	FIntPoint ShadowMapResolution,
	FIntVector& GroupCount,
	// Output
	FRDGBufferRef& VolumetricShadowLinkedListBuffer
)
{
	FClearVolumetricShadowMapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FClearVolumetricShadowMapCS::FParameters>();
	{
		PassParameters->ShadowResolution = HeterogeneousVolumes::GetShadowMapResolution();
		PassParameters->MaxSampleCount = HeterogeneousVolumes::GetShadowMaxSampleCount();
		// Output
		PassParameters->RWVolumetricShadowLinkedListBuffer = GraphBuilder.CreateUAV(VolumetricShadowLinkedListBuffer);
	}

	FClearVolumetricShadowMapCS::FPermutationDomain PermutationVector;
	TShaderRef<FClearVolumetricShadowMapCS> ComputeShader = View.ShaderMap->GetShader<FClearVolumetricShadowMapCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		// Add Light name..
		RDG_EVENT_NAME("ClearVolumetricShadowMapsCS"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

class FRenderVolumetricShadowMapForLightWithLiveShadingCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRenderVolumetricShadowMapForLightWithLiveShadingCS, MeshMaterial);

	class FUseAVSMCompression : SHADER_PERMUTATION_BOOL("USE_AVSM_COMPRESSION");
	class FUseCameraSceneDepth : SHADER_PERMUTATION_BOOL("USE_CAMERA_SCENE_DEPTH");
	class FUseAnalyticDerivatives : SHADER_PERMUTATION_BOOL("USE_ANALYTIC_DERIVATIVES");
	class FShadowMapType : SHADER_PERMUTATION_INT("SHADOW_MAP_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FUseAVSMCompression, FUseCameraSceneDepth, FUseAnalyticDerivatives, FShadowMapType>;


	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)

		// Volumetric Shadow Map data
		SHADER_PARAMETER_ARRAY(FVector4f, TranslatedWorldOrigin, [6])
		SHADER_PARAMETER(FIntPoint, ShadowResolution)
		SHADER_PARAMETER(int, MaxSampleCount)
		SHADER_PARAMETER(float, AbsoluteErrorThreshold)
		SHADER_PARAMETER(float, RelativeErrorThreshold)

		SHADER_PARAMETER(int, NumShadowMatrices)
		SHADER_PARAMETER_ARRAY(FMatrix44f, TranslatedWorldToShadow, [6])
		SHADER_PARAMETER_ARRAY(FMatrix44f, ShadowToTranslatedWorld, [6])

		// Object data
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLocal)
		SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
		SHADER_PARAMETER(FVector3f, LocalBoundsExtent)
		SHADER_PARAMETER(int32, PrimitiveId)

		SHADER_PARAMETER(FIntVector, VoxelResolution)

		// Ray data
		SHADER_PARAMETER(float, ShadowStepSize)
		SHADER_PARAMETER(float, ShadowStepFactor)
		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, bJitter)

		// Dispatch data
		SHADER_PARAMETER(FIntVector, GroupCount)
		SHADER_PARAMETER(int, ShadowDebugTweak)
		SHADER_PARAMETER(int, CameraDownsampleFactor)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>, RWVolumetricShadowLinkedListAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<int2>, RWVolumetricShadowLinkedListBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWBeerShadowMapTexture)

		// Debug
		//SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVolumetricShadowMapDebugData>, RWDebugBuffer)
	END_SHADER_PARAMETER_STRUCT()

	FRenderVolumetricShadowMapForLightWithLiveShadingCS() = default;

	FRenderVolumetricShadowMapForLightWithLiveShadingCS(
		const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer
	)
		: FMeshMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false
		);
		ShaderLooseBindings.Bind(Initializer.ParameterMap);
	}

	static bool ShouldCompilePermutation(
		const FMaterialShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform)
			&& DoesMaterialShaderSupportHeterogeneousVolumes(Parameters.MaterialParameters);
	}

	static void ModifyCompilationEnvironment(
		const FMaterialShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// Disable in-scattering features
		OutEnvironment.SetDefine(TEXT("DIM_USE_TRANSMITTANCE_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_INSCATTERING_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_LUMEN_GI"), 0);

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC); // @lh-todo - Disabled to workaround SPIRV-Cross bug: StructuredBuffer<uint> is translated to ByteAddressBuffer in HLSL backend
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }

	LAYOUT_FIELD(FRenderShadowMapLooseBindings, ShaderLooseBindings);
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRenderVolumetricShadowMapForLightWithLiveShadingCS, TEXT("/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesLiveShadingShadows.usf"), TEXT("RenderVolumetricShadowMapForLightWithLiveShadingCS"), SF_Compute);

void CollectHeterogeneousVolumeMeshBatchesForView(
	const FViewInfo& View,
	bool bCollectForShadowCasting,
	TSet<FVolumetricMeshBatch>& HeterogeneousVolumesMeshBatches,
	FBoxSphereBounds::Builder& WorldBoundsBuilder
)
{
	for (int32 MeshBatchIndex = 0; MeshBatchIndex < View.HeterogeneousVolumesMeshBatches.Num(); ++MeshBatchIndex)
	{
		const FVolumetricMeshBatch& MeshBatch = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex];

		// TODO: Is material determiniation too expensive?
		const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
		const FMaterialRenderProxy* DefaultMaterialRenderProxy = MeshBatch.Mesh->MaterialRenderProxy;
		const FMaterial& Material = DefaultMaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
		MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : DefaultMaterialRenderProxy;
		bool bIsVolumeMaterial = Material.GetMaterialDomain() == MD_Volume;

		bool bCollectMeshBatch = bIsVolumeMaterial;
		if (bCollectForShadowCasting)
		{
			bool bIsShadowCast = MeshBatch.Proxy->IsShadowCast(&View);
			bCollectMeshBatch = bCollectMeshBatch && bIsShadowCast;
		}

		if (bCollectMeshBatch)
		{
			HeterogeneousVolumesMeshBatches.FindOrAdd(FVolumetricMeshBatch(MeshBatch.Mesh, MeshBatch.Proxy));
			WorldBoundsBuilder += MeshBatch.Proxy->GetBounds();
		}
	}
}

void CollectHeterogeneousVolumeMeshBatchesForLightWithShadowIndex(
	const FLightSceneInfo* LightSceneInfo,
	const FVisibleLightInfo* VisibleLightInfo,
	int32 ShadowIndexDiscard,
	const FViewInfo& View,
	TSet<FVolumetricMeshBatch>& HeterogeneousVolumesMeshBatches,
	FBoxSphereBounds::Builder& WorldBoundsBuilder
)
{
	check(LightSceneInfo);
	check(VisibleLightInfo);

	Experimental::TRobinHoodHashMap<const FPrimitiveSceneProxy*, FVolumetricMeshBatch> MeshBatchHashMap;

	if (LightSceneInfo->Proxy->CastsVolumetricShadow())
	{
		int32 ShadowIndexMin = ShadowIndexDiscard;
		int32 ShadowIndexMax = ShadowIndexDiscard + 1;
		if (ShadowIndexDiscard < 0)
		{
			ShadowIndexMin = 0;
			ShadowIndexMax = VisibleLightInfo->ShadowsToProject.Num();
		}

		for (int32 ShadowIndex = ShadowIndexMin; ShadowIndex < ShadowIndexMax; ++ShadowIndex)
		{
			const FProjectedShadowInfo* ProjectedShadowInfo = HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, ShadowIndex);
			if (ProjectedShadowInfo != nullptr)
			{
				const TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>& MeshBatches = ProjectedShadowInfo->GetDynamicSubjectHeterogeneousVolumeMeshElements();
				for (int32 MeshBatchIndex = 0; MeshBatchIndex < MeshBatches.Num(); ++MeshBatchIndex)
				{
					const FMeshBatchAndRelevance& MeshBatch = MeshBatches[MeshBatchIndex];
					check(MeshBatch.PrimitiveSceneProxy);
					bool bIsShadowCast = MeshBatch.PrimitiveSceneProxy->IsShadowCast(ProjectedShadowInfo->ShadowDepthView);

					// TODO: Is material determiniation too expensive?
					const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
					const FMaterialRenderProxy* DefaultMaterialRenderProxy = MeshBatch.Mesh->MaterialRenderProxy;
					const FMaterial& Material = DefaultMaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
					MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : DefaultMaterialRenderProxy;
					bool bIsVolumeMaterial = Material.GetMaterialDomain() == MD_Volume;

					if (bIsShadowCast && bIsVolumeMaterial)
					{
						FBoxSphereBounds VolumeBounds = MeshBatch.PrimitiveSceneProxy->GetBounds();
						WorldBoundsBuilder += VolumeBounds;

						MeshBatchHashMap.FindOrAdd(MeshBatch.PrimitiveSceneProxy, FVolumetricMeshBatch(MeshBatch.Mesh, MeshBatch.PrimitiveSceneProxy));
					}
				}
			}
		}

		// TODO: Determine if coarse sorting is beneficial for shadow combining
		for (auto Itr = MeshBatchHashMap.begin(); Itr != MeshBatchHashMap.end(); ++Itr)
		{
			HeterogeneousVolumesMeshBatches.Add((*Itr).Value);
		}
	}
}

bool RenderVolumetricShadowMapForLightForHeterogeneousVolumeWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Light data
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	const FVisibleLightInfo* VisibleLightInfo,
	// Shadow data
	HeterogeneousVolumes::EShadowType ShadowType,
	HeterogeneousVolumes::FLODInfo LODInfo,
	const FVector3f* TranslatedWorldOrigin,
	int32 NumShadowMatrices,
	FMatrix44f* TranslatedWorldToShadow,
	FMatrix44f* ShadowToTranslatedWorld,
	FIntPoint ShadowMapResolution,
	uint32 MaxSampleCount,
	float MaxShadowTraceDistance,
	bool bUseCameraSceneDepth,
	// Volume
	const FVolumetricMeshBatch& VolumetricMeshBatch,
	// Dispatch
	FIntVector& GroupCount,
	// Output
	FRDGTextureRef& BeerShadowMapTexture,
	FRDGBufferRef& VolumetricShadowLinkedListBuffer
)
{
	// TODO: Understand how the default world material can be triggered here during a recompilation, but not elsewhere..
	const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
	const FMaterialRenderProxy* DefaultMaterialRenderProxy = VolumetricMeshBatch.Mesh->MaterialRenderProxy;
	const FMaterial& Material = DefaultMaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
	MaterialRenderProxy = MaterialRenderProxy ? MaterialRenderProxy : DefaultMaterialRenderProxy;
	if (Material.GetMaterialDomain() != MD_Volume)
	{
		return false;
	}

	FRDGBufferRef VolumetricShadowLinkedListAllocatorBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1),
		TEXT("HeterogeneousVolume.VolumetricShadowLinkedListAllocatorBuffer")
	);
	// Initialize allocator to contain 1-spp
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(VolumetricShadowLinkedListAllocatorBuffer, PF_R32_UINT), ShadowMapResolution.X * ShadowMapResolution.Y);

	FRenderVolumetricShadowMapForLightWithLiveShadingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderVolumetricShadowMapForLightWithLiveShadingCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
		FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
		PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

		// Shadow map data
		PassParameters->ShadowResolution = ShadowMapResolution;
		PassParameters->MaxSampleCount = MaxSampleCount;
		PassParameters->AbsoluteErrorThreshold = HeterogeneousVolumes::GetShadowAbsoluteErrorThreshold();
		PassParameters->RelativeErrorThreshold = HeterogeneousVolumes::GetShadowRelativeErrorThreshold();

		PassParameters->NumShadowMatrices = NumShadowMatrices;
		for (int32 i = 0; i < PassParameters->NumShadowMatrices; ++i)
		{
			PassParameters->TranslatedWorldOrigin[i] = TranslatedWorldOrigin[i];
			PassParameters->TranslatedWorldToShadow[i] = TranslatedWorldToShadow[i];
			PassParameters->ShadowToTranslatedWorld[i] = ShadowToTranslatedWorld[i];
		}

		// TODO: Instancing support
		int32 VolumeIndex = 0;

		// Object data
		const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface = (IHeterogeneousVolumeInterface*)VolumetricMeshBatch.Mesh->Elements[VolumeIndex].UserData;
		FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
		FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
		PassParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
		PassParameters->WorldToLocal = PassParameters->LocalToWorld.Inverse();

		FMatrix LocalToInstance = InstanceToLocal.Inverse();
		FBoxSphereBounds InstanceBoxSphereBounds = HeterogeneousVolumeInterface->GetLocalBounds().TransformBy(LocalToInstance);
		PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
		PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);
		const FPrimitiveSceneInfo* PrimitiveSceneInfo = VolumetricMeshBatch.Proxy->GetPrimitiveSceneInfo();
		check(PrimitiveSceneInfo);
		PassParameters->PrimitiveId = PrimitiveSceneInfo->GetPersistentIndex().Index;

		PassParameters->VoxelResolution = HeterogeneousVolumeInterface->GetVoxelResolution();

		// Ray Data
		PassParameters->ShadowStepSize = HeterogeneousVolumes::GetShadowStepSize();
		PassParameters->ShadowStepFactor = HeterogeneousVolumeInterface->GetShadowStepFactor();

		// TODO: Assign LOD factor per sub-region shadow for CSM
		bool bCascadeShadowForDirectionalLight = LightType == LightType_Directional && HeterogeneousVolumes::SupportsCascadeShadowsForDirectionalLight();
		if (!bCascadeShadowForDirectionalLight)
		{
			float LODFactor = HeterogeneousVolumes::CalcLODFactor(LODInfo, HeterogeneousVolumeInterface);
			PassParameters->ShadowStepFactor *= LODFactor;
		}

		//PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();
		//PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxShadowTraceDistance();
		PassParameters->MaxTraceDistance = MaxShadowTraceDistance;
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetMaxStepCount();
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();

		// Dispatch data
		PassParameters->GroupCount = GroupCount;
		//PassParameters->ShadowDebugTweak = CVarHeterogeneousVolumesShadowDebugTweak.GetValueOnRenderThread();
		PassParameters->ShadowDebugTweak = 0;
		PassParameters->CameraDownsampleFactor = HeterogeneousVolumes::GetCameraDownsampleFactor();

		// Output
		PassParameters->RWVolumetricShadowLinkedListAllocatorBuffer = GraphBuilder.CreateUAV(VolumetricShadowLinkedListAllocatorBuffer, PF_R32_UINT);
		PassParameters->RWVolumetricShadowLinkedListBuffer = GraphBuilder.CreateUAV(VolumetricShadowLinkedListBuffer);
		PassParameters->RWBeerShadowMapTexture = GraphBuilder.CreateUAV(BeerShadowMapTexture);
		//PassParameters->RWDebugBuffer = GraphBuilder.CreateUAV(DebugBuffer);
	}

	FString PassName;
#if WANTS_DRAW_MESH_EVENTS
	if (GetEmitDrawEvents())
	{
		FString LightName = "none";
		if (LightSceneInfo != nullptr)
		{
			FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
		}
		PassName = FString::Printf(TEXT("RenderVolumetricShadowMapForLightWithLiveShadingCS (Light = %s)"), *LightName);
	}
#endif // WANTS_DRAW_MESH_EVENTS

	FRenderVolumetricShadowMapForLightWithLiveShadingCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRenderVolumetricShadowMapForLightWithLiveShadingCS::FUseAVSMCompression>(HeterogeneousVolumes::UseAVSMCompression());
	PermutationVector.Set<FRenderVolumetricShadowMapForLightWithLiveShadingCS::FUseCameraSceneDepth>(bUseCameraSceneDepth && HeterogeneousVolumes::ShadowsUseCameraSceneDepth());
	PermutationVector.Set<FRenderVolumetricShadowMapForLightWithLiveShadingCS::FUseAnalyticDerivatives>(HeterogeneousVolumes::UseAnalyticDerivatives());
	PermutationVector.Set<FRenderVolumetricShadowMapForLightWithLiveShadingCS::FShadowMapType>(static_cast<int32>(ShadowType));
	TShaderRef<FRenderVolumetricShadowMapForLightWithLiveShadingCS> ComputeShader = Material.GetShader<FRenderVolumetricShadowMapForLightWithLiveShadingCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);
	if (!ComputeShader.IsNull())
	{
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("%s", *PassName),
			PassParameters,
			ERDGPassFlags::Compute,
			[ComputeShader, PassParameters, Scene, MaterialRenderProxy, &Material, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
			{
				FMeshMaterialShaderElementData ShaderElementData;
				ShaderElementData.InitializeMeshMaterialData();

				FMeshProcessorShaders PassShaders;
				PassShaders.ComputeShader = ComputeShader;

				FMeshDrawShaderBindings ShaderBindings;
				ShaderBindings.Initialize(PassShaders);
				{
					FMeshDrawSingleShaderBindings SingleShaderBindings = ShaderBindings.GetSingleShaderBindings(SF_Compute);
					ComputeShader->GetShaderBindings(Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, Material, ShaderElementData, SingleShaderBindings);
					SingleShaderBindings.Add(ComputeShader->template GetUniformBufferParameter<FBlueNoise>(), PassParameters->BlueNoise.GetUniformBuffer());
					ComputeShader->ShaderLooseBindings.SetParameters(SingleShaderBindings, PassParameters);
					ShaderBindings.Finalize(&PassShaders);
				}

				UE::MeshPassUtils::Dispatch(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, GroupCount);
			}
		);
	}

	return true;
}

FVector FloorToDouble(const FVector Vec)
{
	return FVector(
		FMath::FloorToDouble(Vec.X),
		FMath::FloorToDouble(Vec.Y),
		FMath::FloorToDouble(Vec.Z)
	);
}

TArray<FVector> GetFrustumVertices(const FMatrix& ClipToWorld)
{
	TArray<FVector> Vertices;

	Vertices.SetNum(8);
	for (int32 i = 0; i < Vertices.Num(); ++i)
	{
		float x = i & 1 ? 1.0 : -1.0;
		float y = (i >> 1) & 1 ? 1.0 : -1.0;
		float z = (i >> 2) & 1 ? 1.0 : -1.0;

		//Vertices[i] = ClipToWorld.TransformPosition(FVector(x, y, z));
		FVector4 Result = ClipToWorld.TransformFVector4(FVector4(x, y, z, 1.0));
		Vertices[i] = Result / Result.W;
	}

	return Vertices;
}

TArray<FVector> TransformVertices(const TArray<FVector>& Vertices, const FMatrix& ToMatrix)
{
	TArray<FVector> ToVertices;

	ToVertices.SetNum(Vertices.Num());
	for (int i = 0; i < ToVertices.Num(); ++i)
	{
		ToVertices[i] = ToMatrix.TransformPosition(Vertices[i]);
	}

	return ToVertices;
}

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowsPrecision(
	TEXT("r.HeterogeneousVolumes.Shadows.Precision"),
	0,
	TEXT("Determines the bit-depth precision for shadows")
	TEXT("0: 16-bit (default)")
	TEXT("1: 32-bit"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

bool RenderVolumetricShadowMapForLightWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Light data
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	const FVisibleLightInfo* VisibleLightInfo,
	// Output
	bool& bIsDirectionalLight,
	FVector3f* TranslatedWorldOrigin,
	FVector4f* TranslatedWorldPlane,
	FMatrix44f* TranslatedWorldToShadow,
	FVector4f* SplitDepths,
	FIntVector& GroupCount,
	int32& NumShadowMatrices,
	FIntPoint& ShadowMapResolution,
	uint32& MaxSampleCount,
	FRDGTextureRef& BeerShadowMapTexture,
	FRDGBufferRef& VolumetricShadowLinkedListBuffer
)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);
	SCOPE_CYCLE_COUNTER(STATGROUP_HeterogeneousVolumesShadows);
	check(LightSceneInfo);
	check(VisibleLightInfo);

	check(HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, 0) != NULL)

	// Collect all shadow-casting volumes
	int32 GlobalCascadeShadowIndex = -1;
	TSet<FVolumetricMeshBatch> GlobalHeterogeneousVolumesMeshBatches;
	FBoxSphereBounds::Builder GlobalWorldVolumeBoundsBuilder;
	CollectHeterogeneousVolumeMeshBatchesForLightWithShadowIndex(LightSceneInfo, VisibleLightInfo, GlobalCascadeShadowIndex, View, GlobalHeterogeneousVolumesMeshBatches, GlobalWorldVolumeBoundsBuilder);
	if (GlobalHeterogeneousVolumesMeshBatches.IsEmpty() || !GlobalWorldVolumeBoundsBuilder.IsValid())
	{
		return false;
	}
	FBoxSphereBounds GlobalWorldVolumeBounds(GlobalWorldVolumeBoundsBuilder);

	ShadowMapResolution = HeterogeneousVolumes::GetShadowMapResolution();

	bool bIsMultiProjection = (LightType == LightType_Point) || (LightType == LightType_Rect);
	GroupCount = FIntVector(1);
	GroupCount.X = FMath::DivideAndRoundUp(ShadowMapResolution.X, FRenderVolumetricShadowMapForLightWithLiveShadingCS::GetThreadGroupSize2D());
	GroupCount.Y = FMath::DivideAndRoundUp(ShadowMapResolution.Y, FRenderVolumetricShadowMapForLightWithLiveShadingCS::GetThreadGroupSize2D());
	GroupCount.Z = bIsMultiProjection ? 6 : 1;

	float MaxShadowTraceDistance = HeterogeneousVolumes::GetMaxShadowTraceDistance();

	FMatrix44f ShadowToTranslatedWorld[6];
	if (LightType == LightType_Directional)
	{
		bIsDirectionalLight = true;

		if (HeterogeneousVolumes::SupportsCascadeShadowsForDirectionalLight())
		{
			NumShadowMatrices = VisibleLightInfo->ShadowsToProject.Num();
			bIsMultiProjection = true;
			GroupCount.Z = NumShadowMatrices;

			// Auto-fit may cause quality popping as new volumes are spawned.
			float MaxTraceDistance = HeterogeneousVolumes::GetMaxShadowTraceDistance();
			if (MaxTraceDistance < 0)
			{
				MaxTraceDistance = (GlobalWorldVolumeBounds.Origin - View.ViewMatrices.GetViewOrigin()).Length() + GlobalWorldVolumeBounds.SphereRadius;
			}

			for (int32 ShadowIndex = 0; ShadowIndex < NumShadowMatrices; ++ShadowIndex)
			{
				// Build orthographic projection centered around volume..
				FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
				FMatrix TranslatedWorldToWorldMatrix = FTranslationMatrix(-PreViewTranslation);
				FVector LightDirection = LightSceneInfo->Proxy->GetDirection().GetSafeNormal();

				FMatrix TranslationMatrix;
				FMatrix RotationMatrix;
				FMatrix ScaleMatrix;

				HeterogeneousVolumes::ECascadeShadowMode CascadeShadowMode = HeterogeneousVolumes::GetCascadeShadowMode();
				if (NumShadowMatrices == 1)
				{
					CascadeShadowMode = HeterogeneousVolumes::ECascadeShadowMode::Disabled;
				}

				if (CascadeShadowMode == HeterogeneousVolumes::ECascadeShadowMode::Frustums)
				{
					// Define viewing frustum from SplitNear to Split Far
					const FMatrix ViewMatrix = View.ViewMatrices.GetWorldToView();
					const FMatrix ProjectionMatrix = View.ViewMatrices.GetViewToClip();
					const FVector4 ViewOrigin = View.ViewMatrices.GetViewOrigin();

					float AspectRatio = ProjectionMatrix.M[1][1] / ProjectionMatrix.M[0][0];
					float ActualFOV = (ViewOrigin.W > 0.0f) ? FMath::Atan(1.0f / ProjectionMatrix.M[0][0]) : PI / 4.0f;
					const FProjectedShadowInfo* ProjectedShadowInfo = HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, ShadowIndex);
					float Near = ProjectedShadowInfo->CascadeSettings.SplitNear;
					float Far = ProjectedShadowInfo->CascadeSettings.SplitFar;

					// Get frustum vertices in world-space
					FMatrix CascadeViewProjectionMatrix = ViewMatrix * FPerspectiveMatrix(ActualFOV, AspectRatio, 1.0f, Near, Far);
					FMatrix CascadeToWorld = CascadeViewProjectionMatrix.Inverse();
					TArray<FVector> WorldFrustumVertices = GetFrustumVertices(CascadeToWorld);

					// Build AABB about the frustum in world-space
					FBoxSphereBounds::Builder WorldSpaceBoundsBuilder;
					for (int32 i = 0; i < WorldFrustumVertices.Num(); ++i)
					{
						WorldSpaceBoundsBuilder += WorldFrustumVertices[i];
					}
					FBoxSphereBounds WorldSpaceBounds(WorldSpaceBoundsBuilder);

					// Define the light projection
					float MaxRadius = FMath::Max(WorldSpaceBounds.SphereRadius, MaxShadowTraceDistance / 2.0);
					FVector ToLight = WorldSpaceBounds.Origin - LightDirection * MaxRadius;
					TranslatedWorldOrigin[ShadowIndex] = FVector3f(PreViewTranslation + ToLight);
					MaxShadowTraceDistance = MaxRadius * 2.0;

					TranslationMatrix = FTranslationMatrix(-ToLight);
					RotationMatrix = FInverseRotationMatrix(LightDirection.Rotation());
					FMatrix WorldToLight = TranslationMatrix * RotationMatrix;
					FMatrix LightToWorld = WorldToLight.Inverse();

					// Build a bounding volume for light rays in light-space
					TArray<FVector> LightFrustumVertices = TransformVertices(WorldFrustumVertices, WorldToLight);

					FBoxSphereBounds::Builder LightSpaceBoundsBuilder;
					for (int32 i = 0; i < LightFrustumVertices.Num(); ++i)
					{
						LightSpaceBoundsBuilder += LightFrustumVertices[i];
					}
					// The virtual light position bounds the parallelpiped
					LightSpaceBoundsBuilder += FVector::Zero();
					FBoxSphereBounds LightSpaceBounds(LightSpaceBoundsBuilder);

					// Transform the centroid to world-space and use this as the new translation matrix
					WorldSpaceBounds.Origin = LightToWorld.TransformPosition(LightSpaceBounds.Origin);

					int32 QuantizeDistance = HeterogeneousVolumes::GetCascadeShadowsQuantizationUnit();
					if (QuantizeDistance > 0)
					{
						float PixelDistance = FMath::Max(LightSpaceBounds.BoxExtent.Y, LightSpaceBounds.BoxExtent.Z) / ShadowMapResolution.X;
						float QuantizePixelDistance = QuantizeDistance * PixelDistance;
						FVector OriginSnapUnits = FVector(
							FMath::FloorToDouble(WorldSpaceBounds.Origin.X / QuantizePixelDistance),
							FMath::FloorToDouble(WorldSpaceBounds.Origin.Y / QuantizePixelDistance),
							FMath::FloorToDouble(WorldSpaceBounds.Origin.Z / QuantizePixelDistance)
						);
						FVector QuantizedWorldSpaceBoundsOrigin = OriginSnapUnits * QuantizePixelDistance;
						FVector OriginDelta = QuantizedWorldSpaceBoundsOrigin - WorldSpaceBounds.Origin;
						WorldSpaceBounds.Origin = QuantizedWorldSpaceBoundsOrigin;

						FVector QuantizedLocalSpaceBoundsBoxExtent = LightSpaceBounds.BoxExtent;
						FVector BoxExtentSnapUnits = FVector(
							FMath::FloorToDouble(QuantizedLocalSpaceBoundsBoxExtent.X / QuantizePixelDistance + 1.0),
							FMath::FloorToDouble(QuantizedLocalSpaceBoundsBoxExtent.Y / QuantizePixelDistance + 1.0),
							FMath::FloorToDouble(QuantizedLocalSpaceBoundsBoxExtent.Z / QuantizePixelDistance + 1.0)
						);
						BoxExtentSnapUnits += FVector(1);
						QuantizedLocalSpaceBoundsBoxExtent = BoxExtentSnapUnits * QuantizePixelDistance;
						LightSpaceBounds.BoxExtent.Y = QuantizedLocalSpaceBoundsBoxExtent.Y;
						LightSpaceBounds.BoxExtent.Z = QuantizedLocalSpaceBoundsBoxExtent.Z;

						// Recompute light virtual origin
						ToLight += OriginDelta;
						TranslatedWorldOrigin[ShadowIndex] = FVector3f(PreViewTranslation + ToLight);
					}

					TranslationMatrix = FTranslationMatrix(-WorldSpaceBounds.Origin);
					ScaleMatrix = FScaleMatrix(FVector(1.0) / LightSpaceBounds.BoxExtent);
				}
				else if (CascadeShadowMode == HeterogeneousVolumes::ECascadeShadowMode::Clipmaps)
				{
					const FProjectedShadowInfo* ProjectedShadowInfo = HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, ShadowIndex);
					float Far = ProjectedShadowInfo->CascadeSettings.SplitFar;

					GlobalWorldVolumeBounds.Origin = View.ViewMatrices.GetViewOrigin();
					GlobalWorldVolumeBounds.BoxExtent = FVector(Far, Far, Far);
					GlobalWorldVolumeBounds.SphereRadius = Far; // GlobalWorldVolumeBounds.BoxExtent.Length();

					// Build AABB about the camera in world-space
					FBoxSphereBounds::Builder WorldSpaceBoundsBuilder;
					TArray<FVector> GlobalWorldVolumeBoundsVertices;
					GlobalWorldVolumeBoundsVertices.SetNum(8);
					for (int32 i = 0; i < GlobalWorldVolumeBoundsVertices.Num(); ++i)
					{
						FVector Sign((i >> 2) & 1 ? 1 : -1, (i >> 1) & 1 ? 1 : -1, i & 1 ? 1 : -1);
						GlobalWorldVolumeBoundsVertices[i] = GlobalWorldVolumeBounds.Origin + Sign * GlobalWorldVolumeBounds.BoxExtent;
						WorldSpaceBoundsBuilder += GlobalWorldVolumeBoundsVertices[i];
					}
					FBoxSphereBounds WorldSpaceBounds(WorldSpaceBoundsBuilder);

					// Define the light projection
					float MaxRadius = FMath::Max(WorldSpaceBounds.SphereRadius, MaxShadowTraceDistance / 2.0);
					FVector ToLight = WorldSpaceBounds.Origin - LightDirection * MaxRadius;
					TranslatedWorldOrigin[ShadowIndex] = FVector3f(PreViewTranslation + ToLight);
					MaxShadowTraceDistance = MaxRadius * 2.0;

					TranslationMatrix = FTranslationMatrix(-ToLight);
					RotationMatrix = FInverseRotationMatrix(LightDirection.Rotation());
					FMatrix WorldToLight = TranslationMatrix * RotationMatrix;
					FMatrix LightToWorld = WorldToLight.Inverse();

					// Build a bounding volume for light rays in light-space
					TArray<FVector> LightVolumeBoundsVertices = TransformVertices(GlobalWorldVolumeBoundsVertices, WorldToLight);

					FBoxSphereBounds::Builder LightSpaceBoundsBuilder;
					for (int32 i = 0; i < LightVolumeBoundsVertices.Num(); ++i)
					{
						LightSpaceBoundsBuilder += LightVolumeBoundsVertices[i];
					}
					// The virtual light position bounds the parallelpiped
					LightSpaceBoundsBuilder += FVector::Zero();
					FBoxSphereBounds LightSpaceBounds(LightSpaceBoundsBuilder);

					// Transform the centroid to world-space and use this as the new translation matrix
					WorldSpaceBounds.Origin = LightToWorld.TransformPosition(LightSpaceBounds.Origin);

					int32 QuantizeDistance = HeterogeneousVolumes::GetCascadeShadowsQuantizationUnit();
					if (QuantizeDistance > 0)
					{
						float PixelDistance = FMath::Max(LightSpaceBounds.BoxExtent.Y, LightSpaceBounds.BoxExtent.Z) / ShadowMapResolution.X;
						float QuantizePixelDistance = QuantizeDistance * PixelDistance;
						FVector OriginSnapUnits = FVector(
							FMath::FloorToDouble(WorldSpaceBounds.Origin.X / QuantizePixelDistance),
							FMath::FloorToDouble(WorldSpaceBounds.Origin.Y / QuantizePixelDistance),
							FMath::FloorToDouble(WorldSpaceBounds.Origin.Z / QuantizePixelDistance)
						);
						FVector QuantizedWorldSpaceBoundsOrigin = OriginSnapUnits * QuantizePixelDistance;
						FVector OriginDelta = QuantizedWorldSpaceBoundsOrigin - WorldSpaceBounds.Origin;
						WorldSpaceBounds.Origin = QuantizedWorldSpaceBoundsOrigin;

						FVector QuantizedLocalSpaceBoundsBoxExtent = LightSpaceBounds.BoxExtent;
						FVector BoxExtentSnapUnits = FVector(
							FMath::FloorToDouble(QuantizedLocalSpaceBoundsBoxExtent.X / QuantizePixelDistance + 1.0),
							FMath::FloorToDouble(QuantizedLocalSpaceBoundsBoxExtent.Y / QuantizePixelDistance + 1.0),
							FMath::FloorToDouble(QuantizedLocalSpaceBoundsBoxExtent.Z / QuantizePixelDistance + 1.0)
						);
						BoxExtentSnapUnits += FVector(1);
						QuantizedLocalSpaceBoundsBoxExtent = BoxExtentSnapUnits * QuantizePixelDistance;
						LightSpaceBounds.BoxExtent.Y = QuantizedLocalSpaceBoundsBoxExtent.Y;
						LightSpaceBounds.BoxExtent.Z = QuantizedLocalSpaceBoundsBoxExtent.Z;

						// Recompute light virtual origin
						ToLight += OriginDelta;
						TranslatedWorldOrigin[ShadowIndex] = FVector3f(PreViewTranslation + ToLight);
					}

					TranslationMatrix = FTranslationMatrix(-WorldSpaceBounds.Origin);
					ScaleMatrix = FScaleMatrix(FVector(1.0) / LightSpaceBounds.BoxExtent);
				}
				else if (CascadeShadowMode == HeterogeneousVolumes::ECascadeShadowMode::Autofit)
				{
					TSet<FVolumetricMeshBatch> HeterogeneousVolumesMeshBatches;
					FBoxSphereBounds::Builder WorldVolumeBoundsBuilder;
					CollectHeterogeneousVolumeMeshBatchesForLightWithShadowIndex(LightSceneInfo, VisibleLightInfo, ShadowIndex, View, HeterogeneousVolumesMeshBatches, WorldVolumeBoundsBuilder);
					FBoxSphereBounds WorldVolumeBounds(WorldVolumeBoundsBuilder);

					// Define transforms
					TranslationMatrix = FTranslationMatrix(-WorldVolumeBounds.Origin);
					RotationMatrix = FInverseRotationMatrix(LightDirection.Rotation());
					ScaleMatrix = FScaleMatrix(FVector(1.0 / WorldVolumeBounds.SphereRadius));

					TranslatedWorldOrigin[ShadowIndex] = FVector3f(PreViewTranslation + WorldVolumeBounds.Origin - LightDirection * WorldVolumeBounds.SphereRadius);
				}
				else // CascadeShadowMode == HeterogeneousVolumes::ECascadeShadowMode::Disabled
				{
					const FProjectedShadowInfo* ProjectedShadowInfo = HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, ShadowIndex);
					FShadowCascadeSettings ShadowCascadeSettings;
					bool IsPrecomputedLightingValid = false;
					LightSceneInfo->Proxy->GetShadowSplitBounds(View, ProjectedShadowInfo->CascadeSettings.ShadowSplitIndex, IsPrecomputedLightingValid, &ShadowCascadeSettings);

					float DepthSpan = ShadowCascadeSettings.SplitFar - ShadowCascadeSettings.SplitNear;
					float QuantizeDepthSpan = FMath::FloorToDouble(DepthSpan);
					float PixelDistance = QuantizeDepthSpan / ShadowMapResolution.GetMax();

					// Define transforms
					TranslationMatrix = FTranslationMatrix(-GlobalWorldVolumeBounds.Origin);
					RotationMatrix = FInverseRotationMatrix(LightDirection.Rotation());
					ScaleMatrix = FScaleMatrix(FVector(1.0 / GlobalWorldVolumeBounds.SphereRadius));

					TranslatedWorldOrigin[ShadowIndex] = FVector3f(PreViewTranslation + GlobalWorldVolumeBounds.Origin - LightDirection * GlobalWorldVolumeBounds.SphereRadius);
				}

				FMatrix FaceMatrix(
					FPlane(0, 0, 1, 0),
					FPlane(0, 1, 0, 0),
					FPlane(-1, 0, 0, 0),
					FPlane(0, 0, 0, 1));

				// Invert Z to match reverse-Z for the rest of the shadow types!
				FMatrix ScreenMatrix = FScaleMatrix(FVector(0.5, -0.5, -0.5)) * FTranslationMatrix(FVector(0.5, 0.5, 0.5));
				FMatrix WorldToShadowMatrix = TranslationMatrix * RotationMatrix * ScaleMatrix * FaceMatrix * ScreenMatrix;
				FMatrix TranslatedWorldToShadowMatrix = TranslatedWorldToWorldMatrix * WorldToShadowMatrix;

				TranslatedWorldToShadow[ShadowIndex] = FMatrix44f(TranslatedWorldToShadowMatrix);
				ShadowToTranslatedWorld[ShadowIndex] = TranslatedWorldToShadow[ShadowIndex].Inverse();

				float W = -FVector3f::DotProduct(TranslatedWorldOrigin[ShadowIndex], FVector3f(LightDirection));
				TranslatedWorldPlane[ShadowIndex] = FVector4f(LightDirection.X, LightDirection.Y, LightDirection.Z, W);

				const FProjectedShadowInfo* ProjectedShadowInfo = HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, ShadowIndex);
				SplitDepths[ShadowIndex] = FVector4f(
					ProjectedShadowInfo->CascadeSettings.SplitNear,
					ProjectedShadowInfo->CascadeSettings.SplitFar,
					ProjectedShadowInfo->CascadeSettings.SplitNearFadeRegion,
					ProjectedShadowInfo->CascadeSettings.SplitFarFadeRegion
				);
			}
		}
		else
		{
			// Build orthographic projection centered around volume..
			FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
			FMatrix TranslatedWorldToWorldMatrix = FTranslationMatrix(-PreViewTranslation);

			FMatrix TranslationMatrix = FTranslationMatrix(-GlobalWorldVolumeBounds.Origin);

			FVector LightDirection = LightSceneInfo->Proxy->GetDirection().GetSafeNormal();
			FMatrix RotationMatrix = FInverseRotationMatrix(LightDirection.Rotation());
			FMatrix ScaleMatrix = FScaleMatrix(FVector(1.0 / GlobalWorldVolumeBounds.SphereRadius));

			const FMatrix FaceMatrix(
				FPlane(0, 0, 1, 0),
				FPlane(0, 1, 0, 0),
				FPlane(-1, 0, 0, 0),
				FPlane(0, 0, 0, 1));

			// Invert Z to match reverse-Z for the rest of the shadow types!
			FMatrix ScreenMatrix = FScaleMatrix(FVector(0.5, -0.5, -0.5)) * FTranslationMatrix(FVector(0.5, 0.5, 0.5));
			FMatrix WorldToShadowMatrix = TranslationMatrix * RotationMatrix * ScaleMatrix * FaceMatrix * ScreenMatrix;
			FMatrix TranslatedWorldToShadowMatrix = TranslatedWorldToWorldMatrix * WorldToShadowMatrix;

			NumShadowMatrices = 1;
			TranslatedWorldToShadow[0] = FMatrix44f(TranslatedWorldToShadowMatrix);
			ShadowToTranslatedWorld[0] = TranslatedWorldToShadow[0].Inverse();
			TranslatedWorldOrigin[0] = FVector3f(PreViewTranslation + GlobalWorldVolumeBounds.Origin - LightDirection * GlobalWorldVolumeBounds.SphereRadius);

			//FVector LightDirection = LightSceneInfo->Proxy->GetDirection().GetSafeNormal();
			float W = -FVector3f::DotProduct(TranslatedWorldOrigin[0], FVector3f(LightDirection));
			TranslatedWorldPlane[0] = FVector4f(LightDirection.X, LightDirection.Y, LightDirection.Z, W);
		}
	}
	else
	{

		// Collect shadow-casting volumes
		//const FProjectedShadowInfo* ProjectedShadowInfo = HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, ShadowIndex);
		const FProjectedShadowInfo* ProjectedShadowInfo = HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, 0);
		NumShadowMatrices = ProjectedShadowInfo->OnePassShadowViewProjectionMatrices.Num();
		if (NumShadowMatrices > 0)
		{
			FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
			FMatrix TranslatedWorldToWorldMatrix = FTranslationMatrix(-PreViewTranslation);
			FVector LightPosition = LightSceneInfo->Proxy->GetPosition();
			FMatrix WorldToLightMatrix = LightSceneInfo->Proxy->GetWorldToLight();

			// Remove light rotation when building the RectLight projections..
			FMatrix RotationalAdjustmentMatrix = FMatrix::Identity;
			if (LightType == LIGHT_TYPE_RECT)
			{
				FVector LightDirection = LightSceneInfo->Proxy->GetDirection().GetSafeNormal();
				RotationalAdjustmentMatrix = FRotationMatrix(LightDirection.Rotation());
			}

			FMatrix ViewMatrix[] = {
				FLookFromMatrix(FVector::Zero(), FVector(-1, 0, 0), FVector(0, 0, 1)),
				FLookFromMatrix(FVector::Zero(), FVector(1, 0, 0), FVector(0, 0, 1)),
				FLookFromMatrix(FVector::Zero(), FVector(0, -1, 0), FVector(0, 0, 1)),
				FLookFromMatrix(FVector::Zero(), FVector(0, 1, 0), FVector(0, 0, 1)),
				FLookFromMatrix(FVector::Zero(), FVector(0, 0, -1), FVector(1, 0, 0)),
				FLookFromMatrix(FVector::Zero(), FVector(0, 0, 1), FVector(1, 0, 0))
			};

			FMatrix PerspectiveMatrix = FPerspectiveMatrix(
				PI / 4.0f,
				ShadowMapResolution.X,
				ShadowMapResolution.Y,
				HeterogeneousVolumes::GetShadowNearClippingDistance(),
				LightSceneInfo->Proxy->GetRadius()
			);

			FMatrix ScreenMatrix = FScaleMatrix(FVector(0.5, -0.5, -0.5)) * FTranslationMatrix(FVector(0.5, 0.5, 0.5));

			for (int32 i = 0; i < NumShadowMatrices; ++i)
			{
				FMatrix WorldToShadowMatrix = WorldToLightMatrix * RotationalAdjustmentMatrix * ViewMatrix[i] * PerspectiveMatrix * ScreenMatrix;
				TranslatedWorldToShadow[i] = FMatrix44f(TranslatedWorldToWorldMatrix * WorldToShadowMatrix);
				ShadowToTranslatedWorld[i] = TranslatedWorldToShadow[i].Inverse();
				TranslatedWorldOrigin[i] = FVector3f(PreViewTranslation + LightPosition);
				FVector LightDirection = LightSceneInfo->Proxy->GetDirection().GetSafeNormal();
				float W = -FVector3f::DotProduct(TranslatedWorldOrigin[i], FVector3f(LightDirection));
				TranslatedWorldPlane[i] = FVector4f(LightDirection.X, LightDirection.Y, LightDirection.Z, W);
			}
		}
		else
		{
			FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
			FMatrix TranslatedWorldToWorldMatrix = FTranslationMatrix(-PreViewTranslation);
			FVector4f ShadowmapMinMax = FVector4f::Zero();
			FMatrix WorldToShadowMatrix = ProjectedShadowInfo->GetWorldToShadowMatrix(ShadowmapMinMax);
			FMatrix TranslatedWorldToShadowMatrix = TranslatedWorldToWorldMatrix * WorldToShadowMatrix;

			NumShadowMatrices = 1;
			TranslatedWorldToShadow[0] = FMatrix44f(TranslatedWorldToShadowMatrix);
			ShadowToTranslatedWorld[0] = TranslatedWorldToShadow[0].Inverse();
			TranslatedWorldOrigin[0] = FVector3f(View.ViewMatrices.GetPreViewTranslation() - ProjectedShadowInfo->PreShadowTranslation);

			FVector LightDirection = LightSceneInfo->Proxy->GetDirection().GetSafeNormal();
			float W = -FVector3f::DotProduct(TranslatedWorldOrigin[0], FVector3f(LightDirection));
			TranslatedWorldPlane[0] = FVector4f(LightDirection.X, LightDirection.Y, LightDirection.Z, W);
		}
	}

	// TODO: LODInfo needs to be calculated independently for each cascade
	FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
	HeterogeneousVolumes::FLODInfo LODInfo;
	LODInfo.WorldSceneBounds = GlobalWorldVolumeBounds;
	//LODInfo.WorldOrigin = FVector(TranslatedWorldOrigin) - PreViewTranslation;
	LODInfo.WorldOrigin = FVector(TranslatedWorldOrigin[0]) - PreViewTranslation;
	LODInfo.ViewRect = FIntRect(0, 0, ShadowMapResolution.X, ShadowMapResolution.Y);

	FMatrix WorldToTranslatedWorldMatrix = FTranslationMatrix(PreViewTranslation);
	FMatrix WorldToShadowMatrix = WorldToTranslatedWorldMatrix * FMatrix(TranslatedWorldToShadow[0]);
	GetViewFrustumBounds(LODInfo.WorldShadowFrustum, WorldToShadowMatrix, true);
	LODInfo.FOV = PI / 4.0f;
	LODInfo.NearClippingDistance = HeterogeneousVolumes::GetShadowNearClippingDistance();
	LODInfo.DownsampleFactor = 1.0;
	// TODO: Support alternate test for rect lights
	LODInfo.bIsPerspective = (LightType == LightType_Spot);

	// Iterate over shadow-casting volumes
	bool bHasShadowCastingVolume = false;
	if (!GlobalHeterogeneousVolumesMeshBatches.IsEmpty())
	{
		auto VolumeMeshBatchItr = GlobalHeterogeneousVolumesMeshBatches.begin();

		MaxSampleCount = HeterogeneousVolumes::GetShadowMaxSampleCount();
		int32 VolumetricShadowLinkedListElementCount = ShadowMapResolution.X * ShadowMapResolution.Y * MaxSampleCount;
		if (bIsMultiProjection)
		{
			VolumetricShadowLinkedListElementCount *= NumShadowMatrices;
		}
		VolumetricShadowLinkedListBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMLinkedListPackedData), VolumetricShadowLinkedListElementCount),
			TEXT("HeterogeneousVolume.VolumetricShadowLinkedListBuffer")
		);
		//ClearVolumetricShadowMap(GraphBuilder, Scene, View, ShadowMapResolution, GroupCount, VolumetricShadowLinkedListBuffer);

		FRDGTextureDesc Desc = SceneTextures.Color.Target->Desc;
		Desc.Format = PF_FloatRGBA;
		if (CVarHeterogeneousVolumesShadowsPrecision.GetValueOnRenderThread() == 1)
		{
			Desc.Format = PF_A32B32G32R32F;
		}
		Desc.Flags &= ~(TexCreate_FastVRAM);
		Desc.Extent = FIntPoint(ShadowMapResolution.X * GroupCount.Z, ShadowMapResolution.Y);
		BeerShadowMapTexture = GraphBuilder.CreateTexture(Desc, TEXT("HeterogeneousVolumes.Light.BeerShadowMapTexture"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BeerShadowMapTexture), FLinearColor::Transparent);

		bool bUseCameraSceneDepth = false;
		RenderVolumetricShadowMapForLightForHeterogeneousVolumeWithLiveShading(
			GraphBuilder,
			SceneTextures,
			Scene,
			View,
			// Light Info
			LightType,
			LightSceneInfo,
			VisibleLightInfo,
			// Shadow Info
			HeterogeneousVolumes::GetShadowType(),
			LODInfo,
			TranslatedWorldOrigin,
			NumShadowMatrices,
			TranslatedWorldToShadow,
			ShadowToTranslatedWorld,
			ShadowMapResolution,
			MaxSampleCount,
			MaxShadowTraceDistance,
			bUseCameraSceneDepth,
			// Volume
			*VolumeMeshBatchItr,
			// Dispatch
			GroupCount,
			// Output
			BeerShadowMapTexture,
			VolumetricShadowLinkedListBuffer
		);

		++VolumeMeshBatchItr;
		for (; VolumeMeshBatchItr != GlobalHeterogeneousVolumesMeshBatches.end(); ++VolumeMeshBatchItr)
		{
			FRDGBufferRef VolumetricShadowLinkedListBuffer1 = GraphBuilder.CreateBuffer(
				FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMLinkedListPackedData), VolumetricShadowLinkedListElementCount),
				TEXT("HeterogeneousVolume.VolumetricShadowLinkedListBuffer1")
			);

			RenderVolumetricShadowMapForLightForHeterogeneousVolumeWithLiveShading(
				GraphBuilder,
				SceneTextures,
				Scene,
				View,
				// Light Info
				LightType,
				LightSceneInfo,
				VisibleLightInfo,
				// Shadow Info
				HeterogeneousVolumes::GetShadowType(),
				LODInfo,
				TranslatedWorldOrigin,
				NumShadowMatrices,
				TranslatedWorldToShadow,
				ShadowToTranslatedWorld,
				ShadowMapResolution,
				MaxSampleCount,
				MaxShadowTraceDistance,
				bUseCameraSceneDepth,
				// Volume
				*VolumeMeshBatchItr,
				// Dispatch
				GroupCount,
				// Output
				BeerShadowMapTexture,
				VolumetricShadowLinkedListBuffer1
			);

			if (HeterogeneousVolumes::GetShadowType() == HeterogeneousVolumes::EShadowType::AdaptiveVolumetricShadowMap)
			{
				FRDGBufferRef VolumetricShadowLinkedListBuffer2 = GraphBuilder.CreateBuffer(
					FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMLinkedListPackedData), VolumetricShadowLinkedListElementCount),
					TEXT("HeterogeneousVolume.VolumetricShadowLinkedListBuffer2")
				);

				CombineVolumetricShadowMap(
					GraphBuilder,
					View,
					GroupCount,
					LightType,
					ShadowMapResolution,
					MaxSampleCount,
					VolumetricShadowLinkedListBuffer,
					VolumetricShadowLinkedListBuffer1,
					VolumetricShadowLinkedListBuffer2
				);

				VolumetricShadowLinkedListBuffer = VolumetricShadowLinkedListBuffer2;
			}
		}

		if (HeterogeneousVolumes::GetDebugMode() &&
			HeterogeneousVolumes::GetShadowType() == HeterogeneousVolumes::EShadowType::AdaptiveVolumetricShadowMap)
		{
			FString LightName;
			FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
			RDG_EVENT_SCOPE(GraphBuilder, "%s", *LightName);

			// Create debug beer shadow map texture
			ConvertVolumetricShadowMapToBeerLawShadowMap(
				GraphBuilder,
				SceneTextures,
				View,
				LightName,
				ShadowMapResolution,
				GroupCount,
				VolumetricShadowLinkedListBuffer,
				BeerShadowMapTexture
			);
		}
	}

	return true;
}

void RenderAdaptiveVolumetricShadowMapWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Light data
	const FVisibleLightInfoArray& VisibleLightInfos
)
{
	RDG_EVENT_SCOPE(GraphBuilder, "Adaptive Volumetric Shadow Maps");
	bool bShouldRenderShadowMaps = !View.ViewRect.IsEmpty();

	// Light culling
	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact = HeterogeneousVolumes::GatherShadowLights(Scene, View.ViewLightingChannelMask);

	// Light loop:
	int32 NumPasses = LightSceneInfoCompact.Num();
	for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
	{
		bool bApplyDirectLighting = !LightSceneInfoCompact.IsEmpty();
		bool bApplyEmissionAndTransmittance = false;
		bool bCastsVolumetricShadow = false;

		uint32 LightType = 0;
		FLightSceneInfo* LightSceneInfo = nullptr;
		const FVisibleLightInfo* VisibleLightInfo = nullptr;
		if (!LightSceneInfoCompact.IsEmpty())
		{
			LightType = LightSceneInfoCompact[PassIndex].LightType;
			LightSceneInfo = LightSceneInfoCompact[PassIndex].LightSceneInfo;
			check(LightSceneInfo != nullptr);

			bool bDynamicallyShadowed = false;
			if (LightSceneInfo)
			{
				VisibleLightInfo = &VisibleLightInfos[LightSceneInfo->Id];
				bCastsVolumetricShadow = LightSceneInfo->Proxy && LightSceneInfo->Proxy->CastsVolumetricShadow();
				bDynamicallyShadowed = HeterogeneousVolumes::IsDynamicShadow(VisibleLightInfo);
			}

			TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> AdaptiveVolumetricShadowMapUniformBuffer;
			TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters> BeerShadowMapUniformBuffer;
			bool bCreateShadowMap = bShouldRenderShadowMaps && bCastsVolumetricShadow && bDynamicallyShadowed && !ShouldRenderRayTracingShadowsForLight(*View.Family, LightSceneInfoCompact[PassIndex]);
			if (bCreateShadowMap)
			{
#if 1
				FRDGTextureRef BeerShadowMapTexture;
#else
				FString LightName;
				FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
				RDG_EVENT_SCOPE(GraphBuilder, "%s", *LightName);

				FRDGTextureDesc Desc = SceneTextures.Color.Target->Desc;
				Desc.Format = PF_FloatRGBA;
				Desc.Flags &= ~(TexCreate_FastVRAM);
				FString ShadowMapName = FString::Printf(TEXT("HeterogeneousVolumes.%s.BeerShadowMapTexture"), *LightName);
				//FRDGTextureRef BeerShadowMapTexture = GraphBuilder.CreateTexture(Desc, TEXT("HeterogeneousVolumes.BeerShadowMapTexture"));
				FRDGTextureRef BeerShadowMapTexture = GraphBuilder.CreateTexture(Desc, *ShadowMapName);
				AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BeerShadowMapTexture), FLinearColor::Transparent);
#endif
				bool bIsDirectionalLight = false;
				FVector3f TranslatedWorldOrigin[6];
				FVector4f TranslatedWorldPlane[6];
				FVector4f SplitDepths[6];
				FMatrix44f TranslatedWorldToShadow[6];
				for (int i = 0; i < 6; ++i)
				{
					TranslatedWorldOrigin[i] = FVector3f::Zero();
					TranslatedWorldPlane[i] = FVector4f::Zero();
					SplitDepths[i] = FVector4f::Zero();
					TranslatedWorldToShadow[i] = FMatrix44f::Identity;
				}

				FIntVector GroupCount = FIntVector::ZeroValue;
				int32 NumShadowMatrices = 0;
				FIntPoint VolumetricShadowMapResolution = FIntPoint::NoneValue;
				uint32 VolumetricShadowMapMaxSampleCount = 0;
				FRDGBufferRef VolumetricShadowMapLinkedListBuffer;
				bool bIsCreated = RenderVolumetricShadowMapForLightWithLiveShading(
					GraphBuilder,
					// Scene data
					SceneTextures,
					Scene,
					View,
					// Light data
					LightType,
					LightSceneInfo,
					VisibleLightInfo,
					// Output
					bIsDirectionalLight,
					TranslatedWorldOrigin,
					TranslatedWorldPlane,
					TranslatedWorldToShadow,
					SplitDepths,
					GroupCount,
					NumShadowMatrices,
					VolumetricShadowMapResolution,
					VolumetricShadowMapMaxSampleCount,
					BeerShadowMapTexture,
					VolumetricShadowMapLinkedListBuffer
				);

				if (bIsCreated)
				{
					float DownsampleFactor = 1.0f;
					if (HeterogeneousVolumes::GetShadowType() == HeterogeneousVolumes::EShadowType::AdaptiveVolumetricShadowMap)
					{
						FRDGBufferRef VolumetricShadowMapIndirectionBuffer;
						FRDGBufferRef VolumetricShadowMapSampleBuffer;
						CompressVolumetricShadowMap(
							GraphBuilder,
							View,
							GroupCount,
							VolumetricShadowMapResolution,
							VolumetricShadowMapMaxSampleCount,
							VolumetricShadowMapLinkedListBuffer,
							VolumetricShadowMapIndirectionBuffer,
							VolumetricShadowMapSampleBuffer
						);

						CreateAdaptiveVolumetricShadowMapUniformBuffer(
							GraphBuilder,
							TranslatedWorldOrigin,
							TranslatedWorldPlane,
							TranslatedWorldToShadow,
							SplitDepths,
							HeterogeneousVolumes::GetShadowType() == HeterogeneousVolumes::EShadowType::AdaptiveVolumetricShadowMap ? VolumetricShadowMapResolution : FIntPoint::ZeroValue,
							DownsampleFactor,
							NumShadowMatrices,
							VolumetricShadowMapMaxSampleCount,
							bIsDirectionalLight,
							VolumetricShadowMapLinkedListBuffer,
							VolumetricShadowMapIndirectionBuffer,
							VolumetricShadowMapSampleBuffer,
							AdaptiveVolumetricShadowMapUniformBuffer
						);
						BeerShadowMapUniformBuffer = HeterogeneousVolumes::CreateEmptyBeerShadowMapUniformBuffer(GraphBuilder);
					}
					else
					{
						CreateBeerShadowMapUniformBuffer(
							GraphBuilder,
							TranslatedWorldOrigin,
							TranslatedWorldPlane,
							TranslatedWorldToShadow,
							SplitDepths,
							HeterogeneousVolumes::GetShadowType() == HeterogeneousVolumes::EShadowType::BeerShadowMap ? VolumetricShadowMapResolution : FIntPoint::ZeroValue,
							DownsampleFactor,
							NumShadowMatrices,
							VolumetricShadowMapMaxSampleCount,
							bIsDirectionalLight,
							BeerShadowMapTexture,
							BeerShadowMapUniformBuffer
						);
						AdaptiveVolumetricShadowMapUniformBuffer = HeterogeneousVolumes::CreateEmptyAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder);
					}
				}
				else
				{
					AdaptiveVolumetricShadowMapUniformBuffer = HeterogeneousVolumes::CreateEmptyAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder);
					BeerShadowMapUniformBuffer = HeterogeneousVolumes::CreateEmptyBeerShadowMapUniformBuffer(GraphBuilder);
				}
			}
			else
			{
				AdaptiveVolumetricShadowMapUniformBuffer = HeterogeneousVolumes::CreateEmptyAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder);
				BeerShadowMapUniformBuffer = HeterogeneousVolumes::CreateEmptyBeerShadowMapUniformBuffer(GraphBuilder);
			}

			if (View.ViewState)
			{
				TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>& AdaptiveVolumetricShadowMap = View.ViewState->AdaptiveVolumetricShadowMapUniformBufferMap.FindOrAdd(LightSceneInfo->Id);
				AdaptiveVolumetricShadowMap = AdaptiveVolumetricShadowMapUniformBuffer;

				TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters>& BeerShadowMap = View.ViewState->BeerShadowMapUniformBufferMap.FindOrAdd(LightSceneInfo->Id);
				BeerShadowMap = BeerShadowMapUniformBuffer;
			}

			if (HeterogeneousVolumes::GetDebugMode() != 0)
			{
				FRDGTextureRef VisualizationTexture;
				VisualizeCascades(
					GraphBuilder,
					// Scene data
					SceneTextures,
					Scene,
					View,
					// Light data
					LightSceneInfo,
					// Output
					VisualizationTexture
				);
			}
		}
	}
}

void RenderAdaptiveVolumetricCameraMapWithLiveShading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View
)
{
	if (View.ViewState == nullptr)
	{
		return;
	}
	RDG_EVENT_SCOPE(GraphBuilder, "Adaptive Volumetric Camera Map");

	// Collect all volumes for view
	bool bCollectForShadowCasting = false;
	TSet<FVolumetricMeshBatch> HeterogeneousVolumesMeshBatches;
	FBoxSphereBounds::Builder WorldBoundsBuilder;
	CollectHeterogeneousVolumeMeshBatchesForView(View, bCollectForShadowCasting, HeterogeneousVolumesMeshBatches, WorldBoundsBuilder);
	if (!WorldBoundsBuilder.IsValid())
	{
		return;
	}

	FBoxSphereBounds WorldVolumeBounds(WorldBoundsBuilder);

	bool bShouldRenderCameraMap = !View.ViewRect.IsEmpty() && !HeterogeneousVolumesMeshBatches.IsEmpty();
	if (bShouldRenderCameraMap)
	{
		// Resolution
		FIntPoint ShadowMapResolution = HeterogeneousVolumes::GetDownsampledResolution(View.ViewRect.Size(), HeterogeneousVolumes::GetCameraDownsampleFactor());

		// Transform
		const FMatrix ProjectionMatrix = View.ViewMatrices.GetViewToClip();
		float FOV = FMath::Atan(1.0f / ProjectionMatrix.M[0][0]);
		FMatrix ViewToClip = FPerspectiveMatrix(
			FOV,
			ShadowMapResolution.X,
			ShadowMapResolution.Y,
			1.0,
			HeterogeneousVolumes::GetMaxTraceDistance()
		);
		FMatrix ClipToView = ViewToClip.Inverse();
		FMatrix ScreenMatrix = FScaleMatrix(FVector(0.5, -0.5, -0.5)) * FTranslationMatrix(FVector(0.5, 0.5, 0.5));

		int32 NumShadowMatrices = 1;
		FMatrix44f TranslatedWorldToShadow[] = {
			FMatrix44f(View.ViewMatrices.GetTranslatedViewMatrix() * ViewToClip * ScreenMatrix)
		};
		FMatrix44f ShadowToTranslatedWorld[] = {
			TranslatedWorldToShadow[0].Inverse()
		};
		FVector3f TranslatedWorldOrigin[] = {
			ShadowToTranslatedWorld[0].GetOrigin()
		};

		// Dispatch
		FIntVector GroupCount = FIntVector(1);
		GroupCount.X = FMath::DivideAndRoundUp(ShadowMapResolution.X, FRenderVolumetricShadowMapForLightWithLiveShadingCS::GetThreadGroupSize2D());
		GroupCount.Y = FMath::DivideAndRoundUp(ShadowMapResolution.Y, FRenderVolumetricShadowMapForLightWithLiveShadingCS::GetThreadGroupSize2D());

		// Visualization Texture
		FRDGTextureDesc Desc = SceneTextures.Color.Target->Desc;
		Desc.Format = PF_FloatRGBA;
		Desc.Flags &= ~(TexCreate_FastVRAM);
		FRDGTextureRef BeerShadowMapTexture = GraphBuilder.CreateTexture(Desc, TEXT("BeerShadowMapTexture"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BeerShadowMapTexture), FLinearColor::Transparent);

		auto VolumeMeshBatchItr = HeterogeneousVolumesMeshBatches.begin();
		int32 MaxSampleCount = HeterogeneousVolumes::GetShadowMaxSampleCount();
		int32 VolumetricShadowLinkedListElementCount = ShadowMapResolution.X * ShadowMapResolution.Y * MaxSampleCount;

		FRDGBufferRef VolumetricShadowLinkedListBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMLinkedListPackedData), VolumetricShadowLinkedListElementCount),
			TEXT("HeterogeneousVolume.VolumetricShadowLinkedListBuffer")
		);

		FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
		FMatrix WorldToTranslatedWorldMatrix = FTranslationMatrix(PreViewTranslation);

		HeterogeneousVolumes::FLODInfo LODInfo;
		LODInfo.WorldSceneBounds = WorldVolumeBounds;
		LODInfo.WorldOrigin = FVector(TranslatedWorldOrigin[0]) - PreViewTranslation;
		LODInfo.ViewRect = FIntRect(0, 0, ShadowMapResolution.X, ShadowMapResolution.Y);
		GetViewFrustumBounds(LODInfo.WorldShadowFrustum, WorldToTranslatedWorldMatrix * FMatrix(TranslatedWorldToShadow[0]), true);
		LODInfo.FOV = FOV;
		LODInfo.NearClippingDistance = HeterogeneousVolumes::GetShadowNearClippingDistance();
		LODInfo.DownsampleFactor = HeterogeneousVolumes::GetCameraDownsampleFactor();
		LODInfo.bIsPerspective = true;

		HeterogeneousVolumes::EShadowType ShadowType = HeterogeneousVolumes::GetTranslucencyCompositingMode();
		// Build a camera shadow for one volume
		int32 LightType = 0;
		FLightSceneInfo* LightSceneInfo = nullptr;
		FVisibleLightInfo* VisibleLightInfo = nullptr;
		bool bUseCameraSceneDepth = true;
		RenderVolumetricShadowMapForLightForHeterogeneousVolumeWithLiveShading(
			GraphBuilder,
			SceneTextures,
			Scene,
			View,
			// Light Info
			LightType,
			LightSceneInfo,
			VisibleLightInfo,
			// Shadow Info
			ShadowType,
			LODInfo,
			TranslatedWorldOrigin,
			NumShadowMatrices,
			TranslatedWorldToShadow,
			ShadowToTranslatedWorld,
			ShadowMapResolution,
			MaxSampleCount,
			HeterogeneousVolumes::GetMaxTraceDistance(),
			bUseCameraSceneDepth,
			// Volume
			*VolumeMeshBatchItr,
			// Dispatch
			GroupCount,
			// Output
			BeerShadowMapTexture,
			VolumetricShadowLinkedListBuffer
		);

		// Iterate over volumes, combining each into the existing shadow map
		++VolumeMeshBatchItr;
		for (; VolumeMeshBatchItr != HeterogeneousVolumesMeshBatches.end(); ++VolumeMeshBatchItr)
		{
			FRDGBufferRef VolumetricShadowLinkedListBuffer1 = GraphBuilder.CreateBuffer(
				FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMLinkedListPackedData), VolumetricShadowLinkedListElementCount),
				TEXT("HeterogeneousVolume.VolumetricShadowLinkedListBuffer1")
			);

			RenderVolumetricShadowMapForLightForHeterogeneousVolumeWithLiveShading(
				GraphBuilder,
				SceneTextures,
				Scene,
				View,
				// Light Info
				LightType,
				LightSceneInfo,
				VisibleLightInfo,
				// Shadow Info
				ShadowType,
				LODInfo,
				TranslatedWorldOrigin,
				NumShadowMatrices,
				TranslatedWorldToShadow,
				ShadowToTranslatedWorld,
				ShadowMapResolution,
				MaxSampleCount,
				HeterogeneousVolumes::GetMaxTraceDistance(),
				bUseCameraSceneDepth,
				// Volume
				*VolumeMeshBatchItr,
				// Dispatch
				GroupCount,
				// Output
				BeerShadowMapTexture,
				VolumetricShadowLinkedListBuffer1
			);

			FRDGBufferRef VolumetricShadowLinkedListBuffer2 = GraphBuilder.CreateBuffer(
				FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMLinkedListPackedData), VolumetricShadowLinkedListElementCount),
				TEXT("HeterogeneousVolume.VolumetricShadowLinkedListBuffer2")
			);

			CombineVolumetricShadowMap(
				GraphBuilder,
				View,
				GroupCount,
				LightType,
				ShadowMapResolution,
				MaxSampleCount,
				VolumetricShadowLinkedListBuffer,
				VolumetricShadowLinkedListBuffer1,
				VolumetricShadowLinkedListBuffer2
			);

			VolumetricShadowLinkedListBuffer = VolumetricShadowLinkedListBuffer2;
		}

		FRDGBufferRef VolumetricShadowIndirectionBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMIndirectionPackedData));
		FRDGBufferRef VolumetricShadowSampleBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMSamplePackedData));
		CompressVolumetricShadowMap(
			GraphBuilder,
			View,
			GroupCount,
			ShadowMapResolution,
			MaxSampleCount,
			VolumetricShadowLinkedListBuffer,
			VolumetricShadowIndirectionBuffer,
			VolumetricShadowSampleBuffer
		);

		FVector4f TranslatedWorldPlane[] = {
			FVector4f::Zero()
		};
		FVector4f SplitDepths[] = {
			FVector4f::Zero()
		};
		float DownsampleFactor = HeterogeneousVolumes::GetCameraDownsampleFactor();
		bool bIsDirectionalLight = false;
		CreateAdaptiveVolumetricShadowMapUniformBufferParameters(
			GraphBuilder,
			TranslatedWorldOrigin,
			TranslatedWorldPlane,
			TranslatedWorldToShadow,
			SplitDepths,
			ShadowMapResolution,
			DownsampleFactor,
			NumShadowMatrices,
			MaxSampleCount,
			bIsDirectionalLight,
			VolumetricShadowLinkedListBuffer,
			VolumetricShadowIndirectionBuffer,
			VolumetricShadowSampleBuffer,
			View.ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters
		);
	}
}

class FRenderVolumetricShadowMaskCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderVolumetricShadowMaskCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderVolumetricShadowMaskCS, FGlobalShader);

	class FShadowType : SHADER_PERMUTATION_INT("SHADOW_TYPE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FShadowType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)

		// Light data
		SHADER_PARAMETER(int, LightType)
		SHADER_PARAMETER_STRUCT_REF(FDeferredLightUniformStruct, DeferredLight)

		// Shadow data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FBeerShadowMapUniformBufferParameters, BeerShadowMap)

		// Dispatch data
		SHADER_PARAMETER(FIntVector, GroupCount)
		SHADER_PARAMETER(int32, DownsampleFactor)

		// Shadow mask
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, ShadowMaskTexture)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWVolumetricShadowMaskTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}

		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::Precached;
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());

		if (DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
};

IMPLEMENT_GLOBAL_SHADER(FRenderVolumetricShadowMaskCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesShadowMask.usf", "RenderVolumetricShadowMaskCS", SF_Compute);

bool RenderVolumetricShadowMaskForLight(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef ShadowMaskTexture,
	FRDGTextureRef& VolumetricShadowMaskTexture
)
{
	FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), FRenderVolumetricShadowMaskCS::GetThreadGroupSize2D());

	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);
	TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters> BeerShadowMap = HeterogeneousVolumes::GetBeerShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);
	if (AVSM->GetParameters()->bIsEmpty && BeerShadowMap->GetParameters()->bIsEmpty)
	{
		return false;
	}

	// Note must be done in the same scope as we add the pass otherwise the UB lifetime will not be guaranteed
	FDeferredLightUniformStruct DeferredLightUniform = GetDeferredLightParameters(View, *LightSceneInfo);
	TUniformBufferRef<FDeferredLightUniformStruct> DeferredLightUB = CreateUniformBufferImmediate(DeferredLightUniform, UniformBuffer_SingleDraw);

	FRenderVolumetricShadowMaskCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderVolumetricShadowMaskCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, View.GetSceneTextures());
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);

		// Light data
		PassParameters->LightType = LightSceneInfo->Type;
		PassParameters->DeferredLight = DeferredLightUB;

		// Shadow data
		PassParameters->AVSM = AVSM;
		PassParameters->BeerShadowMap = BeerShadowMap;

		// Dispatch data
		PassParameters->GroupCount = GroupCount;
		PassParameters->DownsampleFactor = 1; // Disallow downsampling

		// Shadow Mask
		PassParameters->ShadowMaskTexture = GraphBuilder.CreateSRV(ShadowMaskTexture);

		// Output
		PassParameters->RWVolumetricShadowMaskTexture = GraphBuilder.CreateUAV(VolumetricShadowMaskTexture);
	}

	FRenderVolumetricShadowMaskCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRenderVolumetricShadowMaskCS::FShadowType>(static_cast<int32>(HeterogeneousVolumes::GetShadowType()));
	PermutationVector = FRenderVolumetricShadowMaskCS::RemapPermutation(PermutationVector);
	TShaderRef<FRenderVolumetricShadowMaskCS> ComputeShader = View.ShaderMap->GetShader<FRenderVolumetricShadowMaskCS>(PermutationVector);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		//RDG_EVENT_NAME("%s", *PassName),
		RDG_EVENT_NAME("RenderVolumetricShadowMask"),
		ComputeShader,
		PassParameters,
		GroupCount);

	return true;
}

DECLARE_GPU_STAT_NAMED(VolumetricShadowMask, TEXT("Volumetric Shadow Mask"));

bool ShouldRenderVolumetricShadowMaskForLight(
	FRDGBuilder& GraphBuilder,
	const TConstArrayView<FViewInfo>& Views,
	const FLightSceneInfo* LightSceneInfo
)
{
	for (const FViewInfo& View : Views)
	{
		TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);
		TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters> BeerShadowMap = HeterogeneousVolumes::GetBeerShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);
		bool bShadowMapExists = (!AVSM->GetParameters()->bIsEmpty) || (!BeerShadowMap->GetParameters()->bIsEmpty);
		if (bShadowMapExists)
		{
			return true;
		}
	}

	return false;
}

void RenderVolumetricShadowMaskForLight(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTexturesConfig& Config,
	const TConstArrayView<FViewInfo>& Views,
	// Light data
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Current shadow mask
	FRDGTextureRef& ScreenShadowMaskTexture)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, VolumetricShadowMask, "VolumetricShadowMask");

	// Render Volumetric Shadow Mask
	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
		Config.Extent,
		PF_B8G8R8A8,
		FClearValueBinding::White,
		TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
	FRDGTextureRef VolumetricShadowMaskTexture = GraphBuilder.CreateTexture(Desc, TEXT("HeterogeneousVolumes.VolumetricShadowMaskTexture"));

	bool bVolumetricShadowMaskTexture = false;
	for (const FViewInfo& View : Views)
	{
		FRDGTextureRef InputShadowMaskTexture = ScreenShadowMaskTexture ? ScreenShadowMaskTexture : GSystemTextures.GetWhiteDummy(GraphBuilder);
		bVolumetricShadowMaskTexture |= RenderVolumetricShadowMaskForLight(GraphBuilder, View, LightSceneInfo, InputShadowMaskTexture, VolumetricShadowMaskTexture);
	}

	if (bVolumetricShadowMaskTexture)
	{
		ScreenShadowMaskTexture = VolumetricShadowMaskTexture;
	}
}