// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumes.h"
#include "HeterogeneousVolumeInterface.h"

#include "LocalVertexFactory.h"
#include "MeshPassUtils.h"
#include "PixelShaderUtils.h"
#include "RayTracingDefinitions.h"
#include "RayTracingInstance.h"
#include "RayTracingInstanceBufferUtil.h"
#include "ScenePrivate.h"
#include "PrimitiveDrawingUtils.h"
#include "Materials/MaterialRenderProxy.h"
#include "SceneTextureParameters.h"
#include "ShaderCompilerCore.h"

class FHeterogeneousVolumesBakeMaterialCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FHeterogeneousVolumesBakeMaterialCS, MeshMaterial);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)

		// Object data
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLocal)
		SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
		SHADER_PARAMETER(FVector3f, LocalBoundsExtent)
		SHADER_PARAMETER(int32, PrimitiveId)

		// Volume data
		SHADER_PARAMETER(FIntVector, VolumeResolution)

		// Sample data
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER(int, bEvaluateAlbedo)
		SHADER_PARAMETER(int, bEvaluateEmission)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)

		// Dispatch data
		SHADER_PARAMETER(FIntVector, GroupCount)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, RWExtinctionTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, RWEmissionTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, RWAlbedoTexture)
		//SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<Volumes::FDebugOutput>, RWDebugOutputBuffer)
	END_SHADER_PARAMETER_STRUCT()

	FHeterogeneousVolumesBakeMaterialCS() = default;

	FHeterogeneousVolumesBakeMaterialCS(
		const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer
	)
		: FMeshMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false
		);
	}

	static bool ShouldCompilePermutation(
		const FMaterialShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform)
			&& DoesMaterialShaderSupportHeterogeneousVolumes(Parameters.MaterialParameters);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(
		const FMaterialShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FHeterogeneousVolumesBakeMaterialCS, TEXT("/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesMaterialBakingPipeline.usf"), TEXT("HeterogeneousVolumesBakeMaterialCS"), SF_Compute);

void ComputeHeterogeneousVolumeBakeMaterial(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FPersistentPrimitiveIndex& PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Sample data
	bool bJitter,
	bool bLooseBounds,
	bool bEvaluateAlbedo,
	bool bEvaluateEmission,
	// Volume data
	FIntVector VolumeResolution,
	// Output
	FRDGTextureRef HeterogeneousVolumeExtinctionTexture,
	FRDGTextureRef HeterogeneousVolumeEmissionTexture,
	FRDGTextureRef HeterogeneousVolumeAlbedoTexture
)
{
	const FMaterial& Material = MaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
	check(Material.GetMaterialDomain() == MD_Volume);

	// FMath::DivideAndRoundUp() is not compatible with FIntVector?
	FIntVector GroupSize = FIntVector(FHeterogeneousVolumesBakeMaterialCS::GetThreadGroupSize3D());
	FIntVector GroupCount = (VolumeResolution + GroupSize - FIntVector(1)) / FHeterogeneousVolumesBakeMaterialCS::GetThreadGroupSize3D();

#if 0
	FRDGBufferRef DebugOutputBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(Volumes::FDebugOutput), GetVoxelCount(VolumeResolution)),
		TEXT("HeterogeneousVolumes.DebugOutputBuffer"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(DebugOutputBuffer), 0);
#endif

	FHeterogeneousVolumesBakeMaterialCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHeterogeneousVolumesBakeMaterialCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
		PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));

		// Object data
		// TODO: Convert to relative-local space
		//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
		//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
		FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
		FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
		PassParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
		PassParameters->WorldToLocal = PassParameters->LocalToWorld.Inverse();

		FMatrix LocalToInstance = InstanceToLocal.Inverse();
		FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(LocalToInstance);
		PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
		PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);
		PassParameters->PrimitiveId = PersistentPrimitiveIndex.Index;

		// Volume data
		PassParameters->VolumeResolution = VolumeResolution;

		// Sample data
		PassParameters->bJitter = bJitter;
		FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
		PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);
		PassParameters->bEvaluateAlbedo = bEvaluateAlbedo;
		PassParameters->bEvaluateEmission = bEvaluateEmission;

		// Dispatch data
		PassParameters->GroupCount = GroupCount;

		PassParameters->RWExtinctionTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeExtinctionTexture);
		PassParameters->RWEmissionTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeEmissionTexture);
		PassParameters->RWAlbedoTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeAlbedoTexture);
		//PassParameters->RWDebugOutputBuffer = GraphBuilder.CreateUAV(DebugOutputBuffer);
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HeterogeneousVolumesBakedMaterialCS"),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, Scene, &View, MaterialRenderProxy, &Material, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			FHeterogeneousVolumesBakeMaterialCS::FPermutationDomain PermutationVector;
			TShaderRef<FHeterogeneousVolumesBakeMaterialCS> ComputeShader = Material.GetShader<FHeterogeneousVolumesBakeMaterialCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);

			if (!ComputeShader.IsNull())
			{
				FMeshDrawShaderBindings ShaderBindings;
				UE::MeshPassUtils::SetupComputeBindings(ComputeShader, Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, Material, ShaderBindings);

				UE::MeshPassUtils::Dispatch(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, GroupCount);
			}
		}
	);
}

void ComputeHeterogeneousVolumeBakeMaterial(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FPersistentPrimitiveIndex& PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Sample data
	bool bJitter,
	bool bLooseBounds,
	bool bEvaluateAlbedo,
	bool bEvaluateEmission,
	// Volume data
	FIntVector VolumeResolution,
	// Output
	TRefCountPtr<IPooledRenderTarget>& HeterogeneousVolumeExtinctionTexture,
	TRefCountPtr<IPooledRenderTarget>& HeterogeneousVolumeEmissionTexture,
	TRefCountPtr<IPooledRenderTarget>& HeterogeneousVolumeAlbedoTexture
)
{
	const FMaterial& Material = MaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);
	check(Material.GetMaterialDomain() == MD_Volume);

	// FMath::DivideAndRoundUp() is not compatible with FIntVector?
	FIntVector GroupSize = FIntVector(FHeterogeneousVolumesBakeMaterialCS::GetThreadGroupSize3D());
	FIntVector GroupCount = (VolumeResolution + GroupSize - FIntVector(1)) / FHeterogeneousVolumesBakeMaterialCS::GetThreadGroupSize3D();

#if 0
	FRDGBufferRef DebugOutputBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(Volumes::FDebugOutput), GetVoxelCount(VolumeResolution)),
		TEXT("HeterogeneousVolumes.DebugOutputBuffer"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(DebugOutputBuffer), 0);
#endif

	FHeterogeneousVolumesBakeMaterialCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHeterogeneousVolumesBakeMaterialCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
		PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));

		// Object data
		// TODO: Convert to relative-local space
		//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
		//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
		FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
		FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
		PassParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
		PassParameters->WorldToLocal = PassParameters->LocalToWorld.Inverse();

		FMatrix LocalToInstance = InstanceToLocal.Inverse();
		FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(LocalToInstance);
		PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
		PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);
		PassParameters->PrimitiveId = PersistentPrimitiveIndex.Index;

		// Volume data
		PassParameters->VolumeResolution = VolumeResolution;

		// Sample data
		PassParameters->bJitter = bJitter;
		FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
		PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);
		PassParameters->bEvaluateAlbedo = bEvaluateAlbedo;
		PassParameters->bEvaluateEmission = bEvaluateEmission;

		// Dispatch data
		PassParameters->GroupCount = GroupCount;

		PassParameters->RWExtinctionTexture = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(HeterogeneousVolumeExtinctionTexture));
		PassParameters->RWEmissionTexture = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(HeterogeneousVolumeEmissionTexture));
		PassParameters->RWAlbedoTexture = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(HeterogeneousVolumeAlbedoTexture));
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HeterogeneousVolumesBakedMaterialCS"),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, Scene, &View, MaterialRenderProxy, &Material, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			FHeterogeneousVolumesBakeMaterialCS::FPermutationDomain PermutationVector;
			TShaderRef<FHeterogeneousVolumesBakeMaterialCS> ComputeShader = Material.GetShader<FHeterogeneousVolumesBakeMaterialCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);

			if (!ComputeShader.IsNull())
			{
				FMeshDrawShaderBindings ShaderBindings;
				UE::MeshPassUtils::SetupComputeBindings(ComputeShader, Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, Material, ShaderBindings);

				UE::MeshPassUtils::Dispatch(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, GroupCount);
			}
		}
	);
}
