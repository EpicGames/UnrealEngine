// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumes.h"
#include "HeterogeneousVolumeInterface.h"

#include "LightRendering.h"
#include "PixelShaderUtils.h"
#include "RayTracingDefinitions.h"
#include "RayTracingInstance.h"
#include "RayTracingInstanceBufferUtil.h"
#include "ScenePrivate.h"
#include "PrimitiveDrawingUtils.h"
#include "VolumeLighting.h"
#include "VolumetricFog.h"
#include "SceneTextureParameters.h"
#include "FogRendering.h"
#include "ShaderCompilerCore.h"

IMPLEMENT_UNIFORM_BUFFER_STRUCT(FPreshadingUniformBufferParameters, "PreshadingUniformBuffer");

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesPreshadingPixelFormat(
	TEXT("r.HeterogeneousVolumes.Preshading.PixelFormat"),
	2,
	TEXT("Determines the pixel format for preshaded results (Default = 2)\n")
	TEXT("0: PF_R16F\n")
	TEXT("1: PF_FloatR11G11B10\n")
	TEXT("2: PF_FloatRGB\n")
	TEXT("3: PF_FloatRGBA\n"),
	ECVF_RenderThreadSafe
);

namespace HeterogeneousVolumes {
	EPixelFormat GetPreshadingPixelFormat()
	{
		int Value = CVarHeterogeneousVolumesPreshadingPixelFormat.GetValueOnAnyThread();
		switch (Value)
		{
		case 0:
			return PF_R16F;
		case 1:
			return PF_FloatR11G11B10;
		case 2:
		default:
			return PF_FloatRGB;
		case 3:
			return PF_FloatRGBA;
		}
	}

} // namespace HeterogeneousVolumes

class FRenderLightingCacheWithPreshadingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderLightingCacheWithPreshadingCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderLightingCacheWithPreshadingCS, FGlobalShader);

	class FLightingCacheMode : SHADER_PERMUTATION_INT("DIM_LIGHTING_CACHE_MODE", 2);
	class FUseAdaptiveVolumetricShadowMap : SHADER_PERMUTATION_BOOL("DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP");
	class FIndirectLightingMode : SHADER_PERMUTATION_INT("INDIRECT_LIGHTING_MODE", 3);
	using FPermutationDomain = TShaderPermutationDomain<FLightingCacheMode, FUseAdaptiveVolumetricShadowMap, FIndirectLightingMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		// Light data
		SHADER_PARAMETER(int, bApplyEmissionAndTransmittance)
		SHADER_PARAMETER(int, bApplyDirectLighting)
		SHADER_PARAMETER(int, bApplyShadowTransmittance)
		SHADER_PARAMETER(int, LightType)
		SHADER_PARAMETER_STRUCT_REF(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER(float, VolumetricScatteringIntensity)

		// Shadow data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardDirectionalLightShadowMapParameters, ForwardDirLightShadowStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingShaderParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER(int32, VirtualShadowMapId)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FBeerShadowMapUniformBufferParameters, BeerShadowMap)

		// Global illumination data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLumenTranslucencyLightingUniforms, LumenGIVolumeStruct)
		// TODO: Ambient occlusion pipeline
		//SHADER_PARAMETER_RDG_TEXTURE(Texture3D, AmbientOcclusionTexture)
		//SHADER_PARAMETER(FIntVector, AmbientOcclusionResolution)
		SHADER_PARAMETER(float, IndirectInscatteringFactor)

		// Volume structures
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FPreshadingUniformBufferParameters, PreshadingUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightingCacheParameters, LightingCache)

		// Ray data
		SHADER_PARAMETER(float, StepSize)
		SHADER_PARAMETER(float, StepFactor)
		SHADER_PARAMETER(float, ShadowStepSize)
		SHADER_PARAMETER(float, ShadowStepFactor)

		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(float, MaxShadowTraceDistance)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, MipLevel)
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER(int, StochasticFilteringMode)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, RWLightingCacheTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.template Get<FIndirectLightingMode>() == static_cast<int32>(HeterogeneousVolumes::EIndirectLightingMode::SingleScatteringPass))
		{
			return false;
		}

		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FIndirectLightingMode>() != static_cast<int32>(HeterogeneousVolumes::EIndirectLightingMode::LightingCachePass))
		{
			PermutationVector.Set<FIndirectLightingMode>(static_cast<int32>(HeterogeneousVolumes::EIndirectLightingMode::Disabled));
		}

		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		if (DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		//OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FRenderLightingCacheWithPreshadingCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesPreshadingPipeline.usf", "RenderLightingCacheWithPreshadingCS", SF_Compute);

class FRenderSingleScatteringWithPreshadingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderSingleScatteringWithPreshadingCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderSingleScatteringWithPreshadingCS, FGlobalShader);

	class FApplyShadowTransmittanceDim : SHADER_PERMUTATION_BOOL("DIM_APPLY_SHADOW_TRANSMITTANCE");
	class FVoxelCullingDim : SHADER_PERMUTATION_BOOL("DIM_VOXEL_CULLING");
	class FSparseVoxelTracingDim : SHADER_PERMUTATION_BOOL("DIM_SPARSE_VOXEL_TRACING");
	class FFogInscatteringMode : SHADER_PERMUTATION_INT("FOG_INSCATTERING_MODE", 3);
	class FUseInscatteringVolume : SHADER_PERMUTATION_BOOL("DIM_USE_INSCATTERING_VOLUME");
	class FIndirectLightingMode : SHADER_PERMUTATION_INT("INDIRECT_LIGHTING_MODE", 3);
	class FWriteVelocity : SHADER_PERMUTATION_BOOL("DIM_WRITE_VELOCITY");
	class FUseAdaptiveVolumetricShadowMap : SHADER_PERMUTATION_BOOL("DIM_USE_ADAPTIVE_VOLUMETRIC_SHADOW_MAP");
	class FDebugDim : SHADER_PERMUTATION_BOOL("DIM_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FApplyShadowTransmittanceDim, FVoxelCullingDim, FSparseVoxelTracingDim, FFogInscatteringMode, FUseInscatteringVolume, FIndirectLightingMode, FWriteVelocity, FUseAdaptiveVolumetricShadowMap, FDebugDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		// Light data
		SHADER_PARAMETER(int, bApplyEmissionAndTransmittance)
		SHADER_PARAMETER(int, bApplyDirectLighting)
		SHADER_PARAMETER(int, LightType)
		SHADER_PARAMETER_STRUCT_REF(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER(float, VolumetricScatteringIntensity)

		// Shadow data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardDirectionalLightShadowMapParameters, ForwardDirLightShadowStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumeShadowingShaderParameters, VolumeShadowingShaderParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVirtualShadowMapSamplingParameters, VirtualShadowMapSamplingParameters)
		SHADER_PARAMETER(int32, VirtualShadowMapId)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FBeerShadowMapUniformBufferParameters, BeerShadowMap)

		// Atmosphere
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogStruct)

		// Indirect Lighting
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLumenTranslucencyLightingUniforms, LumenGIVolumeStruct)

		// Volume structures
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FPreshadingUniformBufferParameters, PreshadingUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightingCacheParameters, LightingCache)

		// Ray data
		SHADER_PARAMETER(float, StepSize)
		SHADER_PARAMETER(float, StepFactor)
		SHADER_PARAMETER(float, ShadowStepSize)
		SHADER_PARAMETER(float, ShadowStepFactor)

		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(float, MaxShadowTraceDistance)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, MipLevel)
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER(int, StochasticFilteringMode)

		// Dispatch data
		SHADER_PARAMETER(FIntVector, GroupCount)
		SHADER_PARAMETER(int32, DownsampleFactor)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWLightingTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWVelocityTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVoxelDataPacked>, RWVoxelOutputBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FIndirectLightingMode>() == static_cast<int32>(HeterogeneousVolumes::EIndirectLightingMode::LightingCachePass))
		{
			return false;
		}

		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FIndirectLightingMode>() != static_cast<int32>(HeterogeneousVolumes::EIndirectLightingMode::SingleScatteringPass))
		{
			PermutationVector.Set<FIndirectLightingMode>(static_cast<int32>(HeterogeneousVolumes::EIndirectLightingMode::Disabled));
		}

		return PermutationVector;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDebugDim>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}

		if (PermutationVector.Get<FVoxelCullingDim>() != HeterogeneousVolumes::UseSparseVoxelPerTileCulling())
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		if (PermutationVector.Get<FSparseVoxelTracingDim>() != HeterogeneousVolumes::UseSparseVoxelPipeline())
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}
		if (PermutationVector.Get<FUseInscatteringVolume>() != HeterogeneousVolumes::UseLightingCacheForInscattering())
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		return EShaderPermutationPrecacheRequest::Precached;
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());

		if (DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("VIRTUAL_SHADOW_MAP"), 1);
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
		}

		// This shader does not compile with FXC, due to the following error,
		// so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		//   /Engine/Private/Common.ush(307,9-83): error X3538: Sampler parameter must come from a literal expression.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize2D() * GetThreadGroupSize2D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
};

IMPLEMENT_GLOBAL_SHADER(FRenderSingleScatteringWithPreshadingCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesPreshadingPipeline.usf", "RenderSingleScatteringWithPreshadingCS", SF_Compute);

class FMaxTexture3D : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMaxTexture3D);
	SHADER_USE_PARAMETER_STRUCT(FMaxTexture3D, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER(FIntVector, TextureResolution)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, InputTextureLHS)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, InputTextureRHS)
		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, RWOutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FMaxTexture3D, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesPreshadingPipeline.usf", "MaxTexture3D", SF_Compute);

void MaxTexture3D(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef InputTextureLHS,
	FRDGTextureRef InputTextureRHS,
	FRDGTextureRef OutputTexture
)
{
	FRDGTextureDesc OutputTextureDesc = OutputTexture->Desc;
	const FIntVector TextureResolution(OutputTextureDesc.Extent.X, OutputTextureDesc.Extent.Y, OutputTextureDesc.Depth);
	FMaxTexture3D::FParameters* PassParameters = GraphBuilder.AllocParameters<FMaxTexture3D::FParameters>();
	{
		PassParameters->TextureResolution = TextureResolution;
		PassParameters->InputTextureLHS = GraphBuilder.CreateSRV(InputTextureLHS);
		PassParameters->InputTextureRHS = GraphBuilder.CreateSRV(InputTextureRHS);
		PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(OutputTexture);
	}

	uint32 GroupCountX = FMath::DivideAndRoundUp(TextureResolution.X, FMaxTexture3D::GetThreadGroupSize3D());
	uint32 GroupCountY = FMath::DivideAndRoundUp(TextureResolution.Y, FMaxTexture3D::GetThreadGroupSize3D());
	uint32 GroupCountZ = FMath::DivideAndRoundUp(TextureResolution.Z, FMaxTexture3D::GetThreadGroupSize3D());
	FIntVector GroupCount(GroupCountX, GroupCountY, GroupCountZ);

	TShaderRef<FMaxTexture3D> ComputeShader = View.ShaderMap->GetShader<FMaxTexture3D>();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("MaxTexture3D"),
		ComputeShader,
		PassParameters,
		GroupCount);
}

void RenderLightingCacheWithPreshadingCompute(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	const FSceneTextures& SceneTextures,
	// Light data
	bool bApplyEmissionAndTransmittance,
	bool bApplyDirectLighting,
	bool bApplyShadowTransmittance,
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Shadow data
	const FVisibleLightInfo* VisibleLightInfo,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	// Preshading data
	const TRDGUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBuffer,
	// Output
	FRDGTextureRef& LightingCacheTexture
)
{
	// Note must be done in the same scope as we add the pass otherwise the UB lifetime will not be guaranteed
	FDeferredLightUniformStruct DeferredLightUniform = GetDeferredLightParameters(View, *LightSceneInfo);
	TUniformBufferRef<FDeferredLightUniformStruct> DeferredLightUB = CreateUniformBufferImmediate(DeferredLightUniform, UniformBuffer_SingleDraw);

	HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
	float LODFactor = HeterogeneousVolumes::CalcLODFactor(LODValue.LOD, LODValue.Bias);

	FRenderLightingCacheWithPreshadingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderLightingCacheWithPreshadingCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);

		// Light data
		check(LightSceneInfo != nullptr);
		PassParameters->bApplyEmissionAndTransmittance = bApplyEmissionAndTransmittance;
		PassParameters->bApplyDirectLighting = bApplyDirectLighting;
		PassParameters->bApplyShadowTransmittance = bApplyShadowTransmittance;
		PassParameters->DeferredLight = DeferredLightUB;
		PassParameters->LightType = LightType;
		PassParameters->VolumetricScatteringIntensity = LightSceneInfo->Proxy->GetVolumetricScatteringIntensity();

		// Sparse voxel data
		PassParameters->PreshadingUniformBuffer = PreshadingUniformBuffer;

		// Transmittance volume
		PassParameters->LightingCache.LightingCacheResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
		PassParameters->LightingCache.LightingCacheVoxelBias = HeterogeneousVolumeInterface->GetShadowBiasFactor();
		PassParameters->LightingCache.LightingCacheTexture = LightingCacheTexture;

		// Ray data
		PassParameters->StepSize = HeterogeneousVolumes::GetStepSize();
		PassParameters->StepFactor = HeterogeneousVolumeInterface->GetStepFactor() * LODFactor;
		PassParameters->ShadowStepSize = HeterogeneousVolumes::GetShadowStepSize();
		PassParameters->ShadowStepFactor = HeterogeneousVolumeInterface->GetShadowStepFactor() * LODFactor;

		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();
		PassParameters->MaxShadowTraceDistance = HeterogeneousVolumes::GetMaxShadowTraceDistance();
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetMaxStepCount();
		PassParameters->MipLevel = HeterogeneousVolumes::GetMipLevel();
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();
		PassParameters->StochasticFilteringMode = static_cast<int32>(HeterogeneousVolumes::GetStochasticFilteringMode());

		// Shadow data
		PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		PassParameters->ForwardDirLightShadowStruct = View.ForwardLightingResources.ForwardDirLightShadowUniformBuffer;
		if (VisibleLightInfo != nullptr)
		{
			const FProjectedShadowInfo* ProjectedShadowInfo = GetShadowForInjectionIntoVolumetricFog(*VisibleLightInfo);
			bool bDynamicallyShadowed = ProjectedShadowInfo != NULL;
			if (bDynamicallyShadowed)
			{
				GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, ProjectedShadowInfo, PassParameters->VolumeShadowingShaderParameters);
			}
			else
			{
				SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			}
			PassParameters->VirtualShadowMapId = VisibleLightInfo->GetVirtualShadowMapId(&View);
		}
		else
		{
			SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			PassParameters->VirtualShadowMapId = -1;
		}
		PassParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
		PassParameters->AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);
		PassParameters->BeerShadowMap = HeterogeneousVolumes::GetBeerShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);

		// Global illumination data
		auto* LumenUniforms = GraphBuilder.AllocParameters<FLumenTranslucencyLightingUniforms>();
		LumenUniforms->Parameters = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());
		PassParameters->LumenGIVolumeStruct = GraphBuilder.CreateUniformBuffer(LumenUniforms);
		// TODO: Ambient occlusion pipeline
		//PassParameters->AmbientOcclusionTexture = AmbientOcclusionTexture;
		//PassParameters->AmbientOcclusionResolution = HeterogeneousVolumes::GetAmbientOcclusionResolution(HeterogeneousVolumeInterface, LODValue);
		PassParameters->IndirectInscatteringFactor = HeterogeneousVolumes::GetIndirectLightingFactor();

		// Output
		PassParameters->RWLightingCacheTexture = GraphBuilder.CreateUAV(LightingCacheTexture);
	}

	FString PassName;
#if WANTS_DRAW_MESH_EVENTS
	if (GetEmitDrawEvents())
	{
		FString LightName = "none";
		if (LightSceneInfo != nullptr)
		{
			FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
		}
		FString ModeName = HeterogeneousVolumes::UseLightingCacheForInscattering() ? TEXT("In-Scattering") : TEXT("Transmittance");
		PassName = FString::Printf(TEXT("RenderLightingCacheWithPreshadingCS [%s] (Light = %s)"), *ModeName, *LightName);
	}
#endif // WANTS_DRAW_MESH_EVENTS
	bool bUseAVSM = HeterogeneousVolumes::UseAdaptiveVolumetricShadowMapForSelfShadowing(HeterogeneousVolumeInterface->GetPrimitiveSceneProxy());
	// Indirect lighting accumulation is coupled with directional light, because it doesn't voxel cull. It is assumed to exist and shadow.
	int32 IndirectLightingMode = LightType == LightType_Directional ? static_cast<int32>(HeterogeneousVolumes::GetIndirectLightingMode()) : 0;

	FRenderLightingCacheWithPreshadingCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRenderLightingCacheWithPreshadingCS::FLightingCacheMode>(HeterogeneousVolumes::GetLightingCacheMode() - 1);
	PermutationVector.Set<FRenderLightingCacheWithPreshadingCS::FUseAdaptiveVolumetricShadowMap>(bUseAVSM);
	PermutationVector.Set<FRenderLightingCacheWithPreshadingCS::FIndirectLightingMode>(IndirectLightingMode);
	PermutationVector = FRenderLightingCacheWithPreshadingCS::RemapPermutation(PermutationVector);
	TShaderRef<FRenderLightingCacheWithPreshadingCS> ComputeShader = View.ShaderMap->GetShader<FRenderLightingCacheWithPreshadingCS>(PermutationVector);

	FIntVector GroupCount = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
	GroupCount.X = FMath::DivideAndRoundUp(GroupCount.X, FRenderLightingCacheWithPreshadingCS::GetThreadGroupSize3D());
	GroupCount.Y = FMath::DivideAndRoundUp(GroupCount.Y, FRenderLightingCacheWithPreshadingCS::GetThreadGroupSize3D());
	GroupCount.Z = FMath::DivideAndRoundUp(GroupCount.Z, FRenderLightingCacheWithPreshadingCS::GetThreadGroupSize3D());

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("%s", *PassName),
		ComputeShader,
		PassParameters,
		GroupCount);
}

void RenderSingleScatteringWithPreshadingCompute(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FScene* Scene,
	const FViewInfo& View, int32 ViewIndex,
	const FSceneTextures& SceneTextures,
	// Light data
	bool bApplyEmissionAndTransmittance,
	bool bApplyDirectLighting,
	bool bApplyShadowTransmittance,
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Shadow data
	const FVisibleLightInfo* VisibleLightInfo,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	// Preshading data
	const TRDGUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBuffer,
	FRDGTextureRef LightingCacheTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeTexture
)
{
	FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(HeterogeneousVolumes::GetScaledViewRect(View.ViewRect), FRenderSingleScatteringWithPreshadingCS::GetThreadGroupSize2D());

	// Note must be done in the same scope as we add the pass otherwise the UB lifetime will not be guaranteed
	FDeferredLightUniformStruct DeferredLightUniform;
	if (bApplyDirectLighting && (LightSceneInfo != nullptr))
	{
		DeferredLightUniform = GetDeferredLightParameters(View, *LightSceneInfo);
	}
	TUniformBufferRef<FDeferredLightUniformStruct> DeferredLightUB = CreateUniformBufferImmediate(DeferredLightUniform, UniformBuffer_SingleDraw);

	HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
	//FIntVector LightingCacheResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);

	bool bWriteVelocity = HeterogeneousVolumes::ShouldWriteVelocity() && HasBeenProduced(SceneTextures.Velocity);
	FRenderSingleScatteringWithPreshadingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderSingleScatteringWithPreshadingCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);

		// Light data
		PassParameters->bApplyEmissionAndTransmittance = bApplyEmissionAndTransmittance;
		PassParameters->bApplyDirectLighting = bApplyDirectLighting;
		if (PassParameters->bApplyDirectLighting && (LightSceneInfo != nullptr))
		{
			PassParameters->VolumetricScatteringIntensity = LightSceneInfo->Proxy->GetVolumetricScatteringIntensity();
		}
		PassParameters->DeferredLight = DeferredLightUB;
		PassParameters->LightType = LightType;

		// Shadow data
		PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
		PassParameters->ForwardDirLightShadowStruct = View.ForwardLightingResources.ForwardDirLightShadowUniformBuffer;
		if (VisibleLightInfo != nullptr)
		{
			const FProjectedShadowInfo* ProjectedShadowInfo = GetShadowForInjectionIntoVolumetricFog(*VisibleLightInfo);
			bool bDynamicallyShadowed = ProjectedShadowInfo != NULL;
			if (bDynamicallyShadowed)
			{
				GetVolumeShadowingShaderParameters(GraphBuilder, View, LightSceneInfo, ProjectedShadowInfo, PassParameters->VolumeShadowingShaderParameters);
			}
			else
			{
				SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			}
			PassParameters->VirtualShadowMapId = VisibleLightInfo->GetVirtualShadowMapId(&View);
		}
		else
		{
			SetVolumeShadowingDefaultShaderParametersGlobal(GraphBuilder, PassParameters->VolumeShadowingShaderParameters);
			PassParameters->VirtualShadowMapId = -1;
		}
		PassParameters->VirtualShadowMapSamplingParameters = VirtualShadowMapArray.GetSamplingParameters(GraphBuilder, ViewIndex);
		PassParameters->AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);
		PassParameters->BeerShadowMap = HeterogeneousVolumes::GetBeerShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);

		TRDGUniformBufferRef<FFogUniformParameters> FogBuffer = CreateFogUniformBuffer(GraphBuilder, View);
		PassParameters->FogStruct = FogBuffer;

		// Indirect lighting data
		auto* LumenUniforms = GraphBuilder.AllocParameters<FLumenTranslucencyLightingUniforms>();
		LumenUniforms->Parameters = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());
		PassParameters->LumenGIVolumeStruct = GraphBuilder.CreateUniformBuffer(LumenUniforms);

		// Volume data
		PassParameters->MipLevel = HeterogeneousVolumes::GetMipLevel();

		// Sparse voxel data
		PassParameters->PreshadingUniformBuffer = PreshadingUniformBuffer;

		// Transmittance volume
		if ((HeterogeneousVolumes::UseLightingCacheForTransmittance() && bApplyShadowTransmittance) || HeterogeneousVolumes::UseLightingCacheForInscattering())
		{
			PassParameters->LightingCache.LightingCacheResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
			PassParameters->LightingCache.LightingCacheVoxelBias = HeterogeneousVolumeInterface->GetShadowBiasFactor();
			PassParameters->LightingCache.LightingCacheTexture = LightingCacheTexture;
		}
		else
		{
			PassParameters->LightingCache.LightingCacheResolution = FIntVector::ZeroValue;
			PassParameters->LightingCache.LightingCacheVoxelBias = 0.0f;
			PassParameters->LightingCache.LightingCacheTexture = FRDGSystemTextures::Get(GraphBuilder).VolumetricBlack;
		}

		// Ray data
		float LODFactor = HeterogeneousVolumes::CalcLODFactor(LODValue.LOD, LODValue.Bias);
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();
		PassParameters->MaxShadowTraceDistance = HeterogeneousVolumes::GetMaxShadowTraceDistance();
		PassParameters->StepSize = HeterogeneousVolumes::GetStepSize();
		PassParameters->StepFactor = HeterogeneousVolumeInterface->GetStepFactor() * LODFactor;
		PassParameters->ShadowStepSize = HeterogeneousVolumes::GetShadowStepSize();
		PassParameters->ShadowStepFactor = HeterogeneousVolumeInterface->GetShadowStepFactor() * LODFactor;
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetMaxStepCount();
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();
		PassParameters->StochasticFilteringMode = static_cast<int32>(HeterogeneousVolumes::GetStochasticFilteringMode());

		// Dispatch data
		PassParameters->GroupCount = GroupCount;
		PassParameters->DownsampleFactor = HeterogeneousVolumes::GetDownsampleFactor();

		// Output
		PassParameters->RWLightingTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeTexture);
		if (bWriteVelocity)
		{
			PassParameters->RWVelocityTexture = GraphBuilder.CreateUAV(SceneTextures.Velocity);
		}
	}

	FString LightName = "none";
	if (LightSceneInfo != nullptr)
	{
		FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
	}
	bool bUseAVSM = HeterogeneousVolumes::UseAdaptiveVolumetricShadowMapForSelfShadowing(HeterogeneousVolumeInterface->GetPrimitiveSceneProxy());
	// Indirect lighting accumulation is coupled with directional light, because it doesn't cull voxels. It is assumed to exist and shadow.
	int32 IndirectLightingMode = View.GetLumenTranslucencyGIVolume().Texture0 != nullptr ? static_cast<int32>(HeterogeneousVolumes::GetIndirectLightingMode()) : 0;

	FRenderSingleScatteringWithPreshadingCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FApplyShadowTransmittanceDim>(bApplyShadowTransmittance);
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FVoxelCullingDim>(HeterogeneousVolumes::UseSparseVoxelPerTileCulling());
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FSparseVoxelTracingDim>(HeterogeneousVolumes::UseSparseVoxelPipeline());
	//PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FUseTransmittanceVolume>(HeterogeneousVolumes::UseLightingCacheForTransmittance());
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FFogInscatteringMode>(static_cast<int32>(HeterogeneousVolumes::GetFogInscatteringMode()));
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FUseInscatteringVolume>(HeterogeneousVolumes::UseLightingCacheForInscattering());
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FIndirectLightingMode>(IndirectLightingMode);
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FWriteVelocity>(bWriteVelocity);
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FUseAdaptiveVolumetricShadowMap>(bUseAVSM);
	PermutationVector.Set<FRenderSingleScatteringWithPreshadingCS::FDebugDim>(HeterogeneousVolumes::GetDebugMode() != 0);
	PermutationVector = FRenderSingleScatteringWithPreshadingCS::RemapPermutation(PermutationVector);

	TShaderRef<FRenderSingleScatteringWithPreshadingCS> ComputeShader = View.ShaderMap->GetShader<FRenderSingleScatteringWithPreshadingCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("RenderSingleScatteringWithPreshadingCS (Light = %s)", *LightName),
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

void RenderWithInscatteringVolumePipelineWithPreshadingCompute(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	const FScene* Scene,
	FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	// Prreshading data
	TRDGUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBuffer,
	// Output
	FRDGTextureRef& LightingCacheTexture,
	FRDGTextureRef& HeterogeneousVolumeRadiance
)
{
	RDG_EVENT_SCOPE(GraphBuilder, "Direct Volume Rendering");

	bool bRenderLightingCache = !HeterogeneousVolumes::IsHoldout(HeterogeneousVolumeInterface);
	if (bRenderLightingCache)
	{
		SCOPE_CYCLE_COUNTER(STATGROUP_HeterogeneousVolumesLightCache);

		// Light culling
		TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact = HeterogeneousVolumes::GatherLights(Scene, View, HeterogeneousVolumeInterface);

		// Light loop:
		int32 NumPasses = LightSceneInfoCompact.Num();
		for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
		{
			bool bApplyEmissionAndTransmittance = PassIndex == 0;
			bool bApplyDirectLighting = !LightSceneInfoCompact.IsEmpty();
			bool bApplyShadowTransmittance = false;

			uint32 LightType = 0;
			FLightSceneInfo* LightSceneInfo = nullptr;
			const FVisibleLightInfo* VisibleLightInfo = nullptr;
			if (bApplyDirectLighting)
			{
				LightType = LightSceneInfoCompact[PassIndex].LightType;
				LightSceneInfo = LightSceneInfoCompact[PassIndex].LightSceneInfo;
				check(LightSceneInfo != nullptr);

				bApplyDirectLighting = (LightSceneInfo != nullptr);
				if (LightSceneInfo)
				{
					VisibleLightInfo = &VisibleLightInfos[LightSceneInfo->Id];
					bApplyShadowTransmittance = LightSceneInfo->Proxy->CastsVolumetricShadow();
				}
			}

			RenderLightingCacheWithPreshadingCompute(
				GraphBuilder,
				// Scene data
				Scene,
				View, ViewIndex,
				SceneTextures,
				// Light data
				bApplyEmissionAndTransmittance,
				bApplyDirectLighting,
				bApplyShadowTransmittance,
				LightType,
				LightSceneInfo,
				// Shadow data
				VisibleLightInfo,
				VirtualShadowMapArray,
				// Object data
				HeterogeneousVolumeInterface,
				// Sparse voxel data
				PreshadingUniformBuffer,
				// Output
				LightingCacheTexture
			);
		}
	}

	// Direct volume integrator
	{
		SCOPE_CYCLE_COUNTER(STATGROUP_HeterogeneousVolumesSingleScattering);

		bool bApplyEmissionAndTransmittance = true;
		bool bApplyDirectLighting = true;
		bool bApplyShadowTransmittance = true;

		uint32 LightType = 0;
		FLightSceneInfo* LightSceneInfo = nullptr;
		const FVisibleLightInfo* VisibleLightInfo = nullptr;

		RenderSingleScatteringWithPreshadingCompute(
			GraphBuilder,
			// Scene
			Scene,
			View, ViewIndex,
			SceneTextures,
			// Light
			bApplyEmissionAndTransmittance,
			bApplyDirectLighting,
			bApplyShadowTransmittance,
			LightType,
			LightSceneInfo,
			// Shadow
			VisibleLightInfo,
			VirtualShadowMapArray,
			// Object
			HeterogeneousVolumeInterface,
			// Sparse voxel data
			PreshadingUniformBuffer,
			LightingCacheTexture,
			// Output
			HeterogeneousVolumeRadiance
		);
	}
}

void RenderWithPreshadingCompute(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	const FScene* Scene,
	FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	// Preshading data
	TRDGUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBuffer,
	// Output
	FRDGTextureRef& LightingCacheTexture,
	FRDGTextureRef& HeterogeneousVolumeRadiance
)
{
	RDG_EVENT_SCOPE(GraphBuilder, "Software Ray Tracing");

	RenderWithInscatteringVolumePipelineWithPreshadingCompute(
		GraphBuilder,
		// Scene data
		SceneTextures,
		Scene,
		View, ViewIndex,
		// Shadow data
		VisibleLightInfos,
		VirtualShadowMapArray,
		// Object data
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		// Preshading data
		PreshadingUniformBuffer,
		// Output
		LightingCacheTexture,
		HeterogeneousVolumeRadiance
	);
}

void RenderWithInscatteringVolumePipelineWithPreshadingHardwareRayTracing(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	// Sparse voxel data
	FRDGBufferRef NumVoxelsBuffer,
	const TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters>& SparseVoxelUniformBuffer,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	// Ray tracing data
	TConstArrayView<FRayTracingGeometryRHIRef> RayTracingGeometries,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance
)
{
#if RHI_RAYTRACING
	RDG_EVENT_SCOPE(GraphBuilder, "Direct Volume Rendering");

	// Light culling
	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact = HeterogeneousVolumes::GatherLights(Scene, View, HeterogeneousVolumeInterface);

	// Single-scattering
	int32 NumPasses = FMath::Max(LightSceneInfoCompact.Num(), 1);
	for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
	{
		bool bApplyEmissionAndTransmittance = PassIndex == 0;
		bool bApplyDirectLighting = !LightSceneInfoCompact.IsEmpty();
		bool bApplyShadowTransmittance = false;

		uint32 LightType = 0;
		FLightSceneInfo* LightSceneInfo = nullptr;
		const FVisibleLightInfo* VisibleLightInfo = nullptr;
		if (bApplyDirectLighting)
		{
			LightType = LightSceneInfoCompact[PassIndex].LightType;
			LightSceneInfo = LightSceneInfoCompact[PassIndex].LightSceneInfo;
			check(LightSceneInfo != nullptr);

			bApplyDirectLighting = (LightSceneInfo != nullptr);
			if (LightSceneInfo)
			{
				VisibleLightInfo = &VisibleLightInfos[LightSceneInfo->Id];
				bApplyShadowTransmittance = LightSceneInfo->Proxy->CastsVolumetricShadow();
			}
		}

		RenderLightingCacheWithPreshadingHardwareRayTracing(
			GraphBuilder,
			// Scene data
			Scene,
			View, ViewIndex,
			SceneTextures,
			// Light data
			bApplyEmissionAndTransmittance,
			bApplyDirectLighting,
			bApplyShadowTransmittance,
			LightType,
			LightSceneInfo,
			// Shadow
			VisibleLightInfo,
			VirtualShadowMapArray,
			// Object data
			HeterogeneousVolumeInterface,
			// Sparse voxel
			SparseVoxelUniformBuffer,
			// Ray tracing data
			Scene->HeterogeneousVolumesRayTracingScene,
			RayTracingGeometries,
			// Transmittance volume
			LightingCacheTexture
		);
	}

	// Direct volume integrator
	{
		bool bApplyEmissionAndTransmittance = true;
		bool bApplyDirectLighting = true;
		bool bApplyShadowTransmittance = true;

		uint32 LightType = 0;
		FLightSceneInfo* LightSceneInfo = nullptr;
		const FVisibleLightInfo* VisibleLightInfo = nullptr;

		RenderSingleScatteringWithPreshadingHardwareRayTracing(
			GraphBuilder,
			// Scene data
			Scene,
			View, ViewIndex,
			SceneTextures,
			// Light data
			bApplyEmissionAndTransmittance,
			bApplyDirectLighting,
			bApplyShadowTransmittance,
			LightType,
			LightSceneInfo,
			// Shadow
			VisibleLightInfo,
			VirtualShadowMapArray,
			// Object data
			HeterogeneousVolumeInterface,
			// Sparse voxel
			SparseVoxelUniformBuffer,
			// Ray tracing data
			Scene->HeterogeneousVolumesRayTracingScene,
			RayTracingGeometries,
			// Transmittance volume
			LightingCacheTexture,
			// Output
			HeterogeneousVolumeRadiance
		);
	}
#endif // RHI_RAYTRACING
}

void RenderWithTransmittanceVolumePipelineWithPreshadingHardwareRayTracing(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	// Sparse voxel data
	FRDGBufferRef NumVoxelsBuffer,
	const TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters>& SparseVoxelUniformBuffer,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	// Ray tracing data
	TConstArrayView<FRayTracingGeometryRHIRef> RayTracingGeometries,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance
)
{
#if RHI_RAYTRACING
	RDG_EVENT_SCOPE(GraphBuilder, "Direct Volume Rendering");

	// Light culling
	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact = HeterogeneousVolumes::GatherLights(Scene, View, HeterogeneousVolumeInterface);

	// Single-scattering
	int32 NumPasses = FMath::Max(LightSceneInfoCompact.Num(), 1);
	for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
	{
		bool bApplyEmissionAndTransmittance = PassIndex == 0;
		bool bApplyDirectLighting = !LightSceneInfoCompact.IsEmpty();
		bool bApplyShadowTransmittance = false;

		uint32 LightType = 0;
		FLightSceneInfo* LightSceneInfo = nullptr;
		const FVisibleLightInfo* VisibleLightInfo = nullptr;
		if (bApplyDirectLighting)
		{
			LightType = LightSceneInfoCompact[PassIndex].LightType;
			LightSceneInfo = LightSceneInfoCompact[PassIndex].LightSceneInfo;
			check(LightSceneInfo != nullptr);

			bApplyDirectLighting = (LightSceneInfo != nullptr);
			if (LightSceneInfo)
			{
				VisibleLightInfo = &VisibleLightInfos[LightSceneInfo->Id];
				bApplyShadowTransmittance = LightSceneInfo->Proxy->CastsVolumetricShadow();
			}
		}

		if (HeterogeneousVolumes::UseLightingCacheForTransmittance() && bApplyShadowTransmittance)
		{
			RenderLightingCacheWithPreshadingHardwareRayTracing(
				GraphBuilder,
				// Scene data
				Scene,
				View, ViewIndex,
				SceneTextures,
				// Light data
				bApplyEmissionAndTransmittance,
				bApplyDirectLighting,
				bApplyShadowTransmittance,
				LightType,
				LightSceneInfo,
				// Shadow
				VisibleLightInfo,
				VirtualShadowMapArray,
				// Object data
				HeterogeneousVolumeInterface,
				// Sparse voxel
				SparseVoxelUniformBuffer,
				// Ray tracing data
				Scene->HeterogeneousVolumesRayTracingScene,
				RayTracingGeometries,
				// Transmittance volume
				LightingCacheTexture
			);
		}

		RenderSingleScatteringWithPreshadingHardwareRayTracing(
			GraphBuilder,
			// Scene data
			Scene,
			View, ViewIndex,
			SceneTextures,
			// Light data
			bApplyEmissionAndTransmittance,
			bApplyDirectLighting,
			bApplyShadowTransmittance,
			LightType,
			LightSceneInfo,
			// Shadow
			VisibleLightInfo,
			VirtualShadowMapArray,
			// Object data
			HeterogeneousVolumeInterface,
			// Sparse voxel
			SparseVoxelUniformBuffer,
			// Ray tracing data
			Scene->HeterogeneousVolumesRayTracingScene,
			RayTracingGeometries,
			// Transmittance volume
			LightingCacheTexture,
			// Output
			HeterogeneousVolumeRadiance
		);
	}
#endif // RHI_RAYTRACING
}

void RenderWithPreshadingHardwareRayTracing(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	// Sparse voxel data
	FRDGBufferRef NumVoxelsBuffer,
	const TRDGUniformBufferRef<FSparseVoxelUniformBufferParameters>& SparseVoxelUniformBuffer,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance
)
{
#if RHI_RAYTRACING
	RDG_EVENT_SCOPE(GraphBuilder, "Hardware Ray Tracing");

	// WARNING: Currently works, but I'm skeptical if all RHI resources have the correct lifetime management
	TArray<FRayTracingGeometryRHIRef, SceneRenderingAllocator> RayTracingGeometries = GraphBuilder.AllocArray<FRayTracingGeometryRHIRef>();
	TArray<FMatrix> RayTracingTransforms;
	{
		RDG_EVENT_SCOPE(GraphBuilder, "Acceleration Structure Build");

		GenerateRayTracingGeometryInstance(
			GraphBuilder,
			// Scene
			Scene,
			View,
			// Object
			HeterogeneousVolumeInterface,
			// Sparse voxel
			NumVoxelsBuffer,
			SparseVoxelUniformBuffer,
			// Output
			RayTracingGeometries,
			RayTracingTransforms
		);

		GenerateRayTracingScene(
			GraphBuilder,
			// Scene
			Scene,
			View,
			// Ray tracing data
			RayTracingGeometries,
			RayTracingTransforms,
			// Output
			Scene->HeterogeneousVolumesRayTracingScene
		);
	}

	if (HeterogeneousVolumes::UseLightingCacheForInscattering())
	{
		RenderWithInscatteringVolumePipelineWithPreshadingHardwareRayTracing(
			GraphBuilder,
			SceneTextures,
			Scene,
			View, ViewIndex,
			// Shadow data
			VisibleLightInfos,
			VirtualShadowMapArray,
			// Object data
			HeterogeneousVolumeInterface,
			MaterialRenderProxy,
			NumVoxelsBuffer,
			SparseVoxelUniformBuffer,
			// Transmittance acceleration
			LightingCacheTexture,
			// Ray tracing data
			RayTracingGeometries,
			// Output
			HeterogeneousVolumeRadiance
		);
	}
	else
	{
		RenderWithTransmittanceVolumePipelineWithPreshadingHardwareRayTracing(
			GraphBuilder,
			SceneTextures,
			Scene,
			View, ViewIndex,
			// Shadow data
			VisibleLightInfos,
			VirtualShadowMapArray,
			// Object data
			HeterogeneousVolumeInterface,
			MaterialRenderProxy,
			NumVoxelsBuffer,
			SparseVoxelUniformBuffer,
			// Transmittance acceleration
			LightingCacheTexture,
			// Ray tracing data
			RayTracingGeometries,
			// Output
			HeterogeneousVolumeRadiance
		);
	}
#endif // RHI_RAYTRACING
}

class FGenerateMips3D : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGenerateMips3D);
	SHADER_USE_PARAMETER_STRUCT(FGenerateMips3D, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		//SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, InputTexture)
		SHADER_PARAMETER(FIntVector, TextureResolution)
		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, RWOutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FGenerateMips3D, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesPreshadingPipeline.usf", "GenerateMips3D", SF_Compute);

class FGenerateMin3D : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGenerateMin3D);
	SHADER_USE_PARAMETER_STRUCT(FGenerateMin3D, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		//SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, InputTexture)
		SHADER_PARAMETER(FIntVector, TextureResolution)
		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, RWOutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FGenerateMin3D, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesPreshadingPipeline.usf", "GenerateMin3D", SF_Compute);

template<typename ShaderType>
void GenerateMips3D(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef Texture,
	uint32 MipLevel
)
{
	FRDGTextureDesc TextureDesc = Texture->Desc;
	const FIntVector TextureResolution(
		FMath::Max(TextureDesc.Extent.X >> MipLevel, 1),
		FMath::Max(TextureDesc.Extent.Y >> MipLevel, 1),
		FMath::Max(TextureDesc.Depth >> MipLevel, 1));

	typename ShaderType::FParameters* PassParameters = GraphBuilder.AllocParameters<typename ShaderType::FParameters>();
	{
		PassParameters->TextureResolution = TextureResolution;
		PassParameters->InputTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Texture, MipLevel - 1));
		PassParameters->TextureSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		//PassParameters->InputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Texture, MipLevel - 1));
		PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Texture, MipLevel));
	}

	uint32 GroupCountX = FMath::DivideAndRoundUp(TextureResolution.X, ShaderType::GetThreadGroupSize3D());
	uint32 GroupCountY = FMath::DivideAndRoundUp(TextureResolution.Y, ShaderType::GetThreadGroupSize3D());
	uint32 GroupCountZ = FMath::DivideAndRoundUp(TextureResolution.Z, ShaderType::GetThreadGroupSize3D());
	FIntVector GroupCount(GroupCountX, GroupCountY, GroupCountZ);

	TShaderRef<ShaderType> ComputeShader = View.ShaderMap->GetShader<ShaderType>();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FGenerateMips3D"),
		ComputeShader,
		PassParameters,
		GroupCount);
}

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesPreshadingCLOD(
	TEXT("r.HeterogeneousVolumes.Preshading.CLOD"),
	1,
	TEXT("Enables continuous level-of-detail for preshading resolution (Default = 1)"),
	ECVF_RenderThreadSafe
);

void CollectRenderableHeterogeneousVolumes(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	TArray<const IHeterogeneousVolumeInterface*>& HeterogeneousVolumeInterfaces,
	TArray<const FMaterialRenderProxy*>& MaterialRenderProxies
)
{
	for (int32 MeshBatchIndex = 0; MeshBatchIndex < View.HeterogeneousVolumesMeshBatches.Num(); ++MeshBatchIndex)
	{
		const FMeshBatch* Mesh = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex].Mesh;
		const FPrimitiveSceneProxy* PrimitiveSceneProxy = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex].Proxy;
		const FMaterialRenderProxy* MaterialRenderProxy = Mesh->MaterialRenderProxy;
		if (!ShouldRenderMeshBatchWithHeterogeneousVolumes(Mesh, PrimitiveSceneProxy, View.GetFeatureLevel()))
		{
			continue;
		}

		for (int32 VolumeIndex = 0; VolumeIndex < Mesh->Elements.Num(); ++VolumeIndex)
		{
			const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeIndex].UserData;
			//check(HeterogeneousVolumeInterface != nullptr);
			if (HeterogeneousVolumeInterface == nullptr)
			{
				continue;
			}

			// Append
			HeterogeneousVolumeInterfaces.Add(HeterogeneousVolumeInterface);
			MaterialRenderProxies.Add(MaterialRenderProxy);
		}
	}
}

void BuildPreshadingUniformBufferParameters(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	// Volume data
	const FIntVector& BakingResolution,
	TRefCountPtr<IPooledRenderTarget> ExtinctionTexture,
	TRefCountPtr<IPooledRenderTarget> EmissionTexture,
	TRefCountPtr<IPooledRenderTarget> AlbedoTexture,
	const FIntVector& MajorantResolution,
	TRefCountPtr<IPooledRenderTarget> MajorantTexture,
	FPreshadingUniformBufferParameters& PreshadingUniformBufferParameters
)
{
	// Object data
	// TODO: Convert to relative-local space
	//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
	//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
	FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
	FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
	PreshadingUniformBufferParameters.LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
	PreshadingUniformBufferParameters.WorldToLocal = PreshadingUniformBufferParameters.LocalToWorld.Inverse();

	FMatrix LocalToInstance = InstanceToLocal.Inverse();
	const FBoxSphereBounds LocalBoxSphereBounds = HeterogeneousVolumeInterface->GetLocalBounds();
	FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(LocalToInstance);
	PreshadingUniformBufferParameters.LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
	PreshadingUniformBufferParameters.LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);

	// Volume data
	PreshadingUniformBufferParameters.VolumeResolution = BakingResolution;
	PreshadingUniformBufferParameters.ExtinctionTexture = GraphBuilder.RegisterExternalTexture(ExtinctionTexture);
	PreshadingUniformBufferParameters.EmissionTexture = GraphBuilder.RegisterExternalTexture(EmissionTexture);
	PreshadingUniformBufferParameters.AlbedoTexture = GraphBuilder.RegisterExternalTexture(AlbedoTexture);
	PreshadingUniformBufferParameters.TextureSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PreshadingUniformBufferParameters.MajorantResolution = MajorantResolution;
	PreshadingUniformBufferParameters.MajorantTexture = GraphBuilder.RegisterExternalTexture(MajorantTexture);
	PreshadingUniformBufferParameters.PointTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
}

void BuildPreshadingUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	// Volume data
	const FIntVector& BakingResolution,
	FRDGTextureRef ExtinctionTexture,
	FRDGTextureRef EmissionTexture,
	FRDGTextureRef AlbedoTexture,
	const FIntVector& MajorantResolution,
	FRDGTextureRef MajorantTexture,
	TRDGUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBuffer
)
{
	// Create Preshading UniformBuffer
	FPreshadingUniformBufferParameters* PreshadingUniformBufferParameters = GraphBuilder.AllocParameters<FPreshadingUniformBufferParameters>();
	{
		// Object data
		// TODO: Convert to relative-local space
		//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
		//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
		FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
		FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
		PreshadingUniformBufferParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
		PreshadingUniformBufferParameters->WorldToLocal = PreshadingUniformBufferParameters->LocalToWorld.Inverse();

		FMatrix LocalToInstance = InstanceToLocal.Inverse();
		const FBoxSphereBounds LocalBoxSphereBounds = HeterogeneousVolumeInterface->GetLocalBounds();
		FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(LocalToInstance);
		PreshadingUniformBufferParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
		PreshadingUniformBufferParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);

		// Volume data
		PreshadingUniformBufferParameters->VolumeResolution = BakingResolution;
		PreshadingUniformBufferParameters->ExtinctionTexture = ExtinctionTexture;
		PreshadingUniformBufferParameters->EmissionTexture = EmissionTexture;
		PreshadingUniformBufferParameters->AlbedoTexture = AlbedoTexture;
		PreshadingUniformBufferParameters->TextureSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		PreshadingUniformBufferParameters->MajorantResolution = MajorantResolution;
		PreshadingUniformBufferParameters->MajorantTexture = MajorantTexture;
		PreshadingUniformBufferParameters->PointTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
	PreshadingUniformBuffer = GraphBuilder.CreateUniformBuffer(PreshadingUniformBufferParameters);
}

void BuildEmptyPreshadingUniformBuffer(
	FRDGBuilder& GraphBuilder,
	TRDGUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBuffer
)
{
	FPreshadingUniformBufferParameters* PreshadingUniformBufferParameters = GraphBuilder.AllocParameters<FPreshadingUniformBufferParameters>();
	{
		PreshadingUniformBufferParameters->LocalToWorld = FMatrix44f::Identity;
		PreshadingUniformBufferParameters->WorldToLocal = FMatrix44f::Identity;
		PreshadingUniformBufferParameters->LocalBoundsOrigin = FVector3f::ZeroVector;
		PreshadingUniformBufferParameters->LocalBoundsExtent = FVector3f::ZeroVector;

		PreshadingUniformBufferParameters->VolumeResolution = FIntVector::ZeroValue;
		PreshadingUniformBufferParameters->ExtinctionTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		PreshadingUniformBufferParameters->EmissionTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		PreshadingUniformBufferParameters->AlbedoTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		PreshadingUniformBufferParameters->TextureSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		PreshadingUniformBufferParameters->MajorantResolution = FIntVector::ZeroValue;
		PreshadingUniformBufferParameters->MajorantTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		PreshadingUniformBufferParameters->PointTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
	PreshadingUniformBuffer = GraphBuilder.CreateUniformBuffer(PreshadingUniformBufferParameters);
}

void BuildMajorantTexture(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	FIntVector& MajorantResolution,
	FRDGTextureRef MajorantHistoryTexture,
	FRDGTextureRef& MajorantTexture
)
{
	RDG_EVENT_SCOPE(GraphBuilder, "Majorant Update");

	// Collect material and object information
	const FPrimitiveSceneProxy* PrimitiveSceneProxy = HeterogeneousVolumeInterface->GetPrimitiveSceneProxy();
	const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
	const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitiveSceneInfo->GetPersistentIndex();
	const FBoxSphereBounds LocalBoxSphereBounds = HeterogeneousVolumeInterface->GetLocalBounds();

	// Create baked material grids
	const uint32 NumMips = 1;
	const EPixelFormat PixelFormat = HeterogeneousVolumes::GetPreshadingPixelFormat();
	MajorantResolution = FIntVector(64, 64, 64);

	FRDGTextureDesc MajorantDesc = FRDGTextureDesc::Create3D(
		MajorantResolution,
		PixelFormat,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling,
		NumMips
	);
	FRDGTextureRef ExtinctionTexture = GraphBuilder.CreateTexture(MajorantDesc, TEXT("HeterogeneousVolumes.ExtinctionTexture"));

	FRDGTextureDesc EmptyDesc = FRDGTextureDesc::Create3D(
		FIntVector(1, 1, 1),
		PixelFormat,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling,
		NumMips
	);
	FRDGTextureRef DummyTexture = GraphBuilder.CreateTexture(EmptyDesc, TEXT("HeterogeneousVolumes.DummyTexture"));

	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(ExtinctionTexture), FLinearColor::Black);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(DummyTexture), FLinearColor::White);

	// Sample data
	bool bJitter = true;
	bool bLooseBounds = true;
	bool bEvaluateAlbedo = false;
	bool bEvaluateEmission = false;
	ComputeHeterogeneousVolumeBakeMaterial(
		GraphBuilder,
		// Scene data
		Scene,
		View,
		// Object data
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		PersistentPrimitiveIndex,
		LocalBoxSphereBounds,
		// Sample data
		bJitter,
		bLooseBounds,
		bEvaluateAlbedo,
		bEvaluateEmission,
		// Volume data
		MajorantResolution,
		// Output
		ExtinctionTexture,
		DummyTexture,
		DummyTexture
	);

	// Update majorant
	if (MajorantHistoryTexture)
	{
		MajorantTexture = GraphBuilder.CreateTexture(MajorantDesc, TEXT("HeterogeneousVolumes.MajorantTexture"));
		MaxTexture3D(
			GraphBuilder,
			View,
			MajorantHistoryTexture,
			ExtinctionTexture,
			MajorantTexture
		);
	}
	else
	{
		MajorantTexture = ExtinctionTexture;
	}
}

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesMajorantResolution(
	TEXT("r.HeterogeneousVolumes.Majorant.Resolution"),
	64,
	TEXT("Determines the underlying resolution for the majorant acceleration texture (Default = 64)\n")
	TEXT("When a value of 0 is used, the majorant corresponds to the native resolution of the volume"),
	ECVF_RenderThreadSafe
);

FIntVector GetMajorantResolution(const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface)
{
	int32 Value = CVarHeterogeneousVolumesMajorantResolution.GetValueOnRenderThread();
	return (Value == 0) ? HeterogeneousVolumeInterface->GetVoxelResolution() : FIntVector(Value);
}

void BuildMajorantTexture(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	FIntVector& MajorantResolution,
	TRefCountPtr<IPooledRenderTarget> MajorantHistoryTexture,
	TRefCountPtr<IPooledRenderTarget>& MajorantTexture
)
{
	RDG_EVENT_SCOPE(GraphBuilder, "Majorant Update");

	// Collect material and object information
	const FPrimitiveSceneProxy* PrimitiveSceneProxy = HeterogeneousVolumeInterface->GetPrimitiveSceneProxy();
	const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
	const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitiveSceneInfo->GetPersistentIndex();
	const FBoxSphereBounds LocalBoxSphereBounds = HeterogeneousVolumeInterface->GetLocalBounds();

	// Create baked material grids
	const uint32 NumMips = 1;
	const EPixelFormat PixelFormat = HeterogeneousVolumes::GetPreshadingPixelFormat();
	MajorantResolution = GetMajorantResolution(HeterogeneousVolumeInterface);

	FRHITextureCreateDesc MajorantDesc = FRHITextureCreateDesc::Create3D(TEXT("HeterogeneousVolumes.MajorantTexture"), MajorantResolution, PixelFormat)
		.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);
	TRefCountPtr<IPooledRenderTarget> ExtinctionTexture = CreateRenderTarget(GraphBuilder.RHICmdList.CreateTexture(MajorantDesc), MajorantDesc.DebugName);

	FRDGTextureDesc EmptyDesc = FRDGTextureDesc::Create3D(
		FIntVector(1, 1, 1),
		PixelFormat,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling,
		NumMips
	);
	FRDGTextureRef DummyTexture = GraphBuilder.CreateTexture(EmptyDesc, TEXT("HeterogeneousVolumes.DummyTexture"));

	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(ExtinctionTexture)), FLinearColor::Black);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(DummyTexture), FLinearColor::White);

	// Sample data
	bool bJitter = true;
	bool bLooseBounds = true;
	bool bEvaluateAlbedo = false;
	bool bEvaluateEmission = false;
	ComputeHeterogeneousVolumeBakeMaterial(
		GraphBuilder,
		// Scene data
		Scene,
		View,
		// Object data
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		PersistentPrimitiveIndex,
		LocalBoxSphereBounds,
		// Sample data
		bJitter,
		bLooseBounds,
		bEvaluateAlbedo,
		bEvaluateEmission,
		// Volume data
		MajorantResolution,
		// Output
		GraphBuilder.RegisterExternalTexture(ExtinctionTexture),
		DummyTexture,
		DummyTexture
	);

	// Update majorant
	if (MajorantHistoryTexture && MajorantTexture)
	{
		MaxTexture3D(
			GraphBuilder,
			View,
			GraphBuilder.RegisterExternalTexture(MajorantHistoryTexture),
			GraphBuilder.RegisterExternalTexture(ExtinctionTexture),
			GraphBuilder.RegisterExternalTexture(MajorantTexture)
		);
	}
	else
	{
		MajorantTexture = ExtinctionTexture;
	}
}

void BuildPreshadingGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	TRefCountPtr<IPooledRenderTarget>& MajorantHistoryRT,
	TRDGUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBuffer
)
{
	check(View.ViewState);

	// Determine baking resolution
	FIntVector VolumeResolution = HeterogeneousVolumes::GetVolumeResolution(HeterogeneousVolumeInterface);
	FIntVector BakingResolution = VolumeResolution;
	if (CVarHeterogeneousVolumesPreshadingCLOD.GetValueOnAnyThread() == 1)
	{
		// Determine baking voxel resolution
		float LODFactor = HeterogeneousVolumes::CalcLODFactor(View, HeterogeneousVolumeInterface);
		BakingResolution = FIntVector(BakingResolution / LODFactor);
		BakingResolution.X = FMath::Clamp(BakingResolution.X, 1, 1024);
		BakingResolution.Y = FMath::Clamp(BakingResolution.Y, 1, 1024);
		BakingResolution.Z = FMath::Clamp(BakingResolution.Z, 1, 1024);

	}
	else if (CVarHeterogeneousVolumesPreshadingCLOD.GetValueOnAnyThread() == 2)
	{
		HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
		BakingResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
	}

	// Collect material and object information
	const FPrimitiveSceneProxy* PrimitiveSceneProxy = HeterogeneousVolumeInterface->GetPrimitiveSceneProxy();
	const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
	const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitiveSceneInfo->GetPersistentIndex();
	const FBoxSphereBounds LocalBoxSphereBounds = HeterogeneousVolumeInterface->GetLocalBounds();

	// Create baked material grids
	const uint32 NumMips = 1;
	const EPixelFormat PixelFormat = HeterogeneousVolumes::GetPreshadingPixelFormat();

	FRDGTextureDesc BakedMaterialDesc = FRDGTextureDesc::Create3D(
		BakingResolution,
		PixelFormat,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV | TexCreate_3DTiling,
		NumMips
	);
	FRDGTextureRef ExtinctionTexture = GraphBuilder.CreateTexture(BakedMaterialDesc, TEXT("HeterogeneousVolumes.ExtinctionTexture"));
	FRDGTextureRef EmissionTexture = GraphBuilder.CreateTexture(BakedMaterialDesc, TEXT("HeterogeneousVolumes.EmissionTexture"));
	FRDGTextureRef AlbedoTexture = GraphBuilder.CreateTexture(BakedMaterialDesc, TEXT("HeterogeneousVolumes.AlbedoTexture"));

	bool bJitter = HeterogeneousVolumes::ShouldJitter();
	bool bLooseBounds = false;
	bool bEvaluateAlbedo = true;
	bool bEvaluateEmission = true;
	ComputeHeterogeneousVolumeBakeMaterial(
		GraphBuilder,
		// Scene data
		Scene,
		View,
		// Object data
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		PersistentPrimitiveIndex,
		LocalBoxSphereBounds,
		// Sample data
		bJitter,
		bLooseBounds,
		bEvaluateAlbedo,
		bEvaluateEmission,
		// Volume data
		BakingResolution,
		// Output
		ExtinctionTexture,
		EmissionTexture,
		AlbedoTexture
	);

	FRDGTextureRef MajorantHistory = MajorantHistoryRT ? GraphBuilder.RegisterExternalTexture(MajorantHistoryRT) : nullptr;

	FIntVector MajorantResolution;
	FRDGTextureRef MajorantTexture;
	BuildMajorantTexture(
		GraphBuilder,
		Scene,
		View,
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		MajorantResolution,
		MajorantHistory,
		MajorantTexture
	);

	// Cache result
	GraphBuilder.QueueTextureExtraction(MajorantTexture, &MajorantHistoryRT);

	BuildPreshadingUniformBuffer(
		GraphBuilder,
		Scene,
		View,
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		BakingResolution,
		ExtinctionTexture,
		EmissionTexture,
		AlbedoTexture,
		MajorantResolution,
		MajorantTexture,
		PreshadingUniformBuffer
	);
}

void BuildPreshadingGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	TRefCountPtr<IPooledRenderTarget>& PersistentMajorantRT,
	FPathTracingHeterogeneousVolumesCallableInfo& CallableInfo,
	FPreshadingUniformBufferParameters& PreshadingUniformBufferParameters
)
{
	// Determine baking resolution
	FIntVector VolumeResolution = HeterogeneousVolumes::GetVolumeResolution(HeterogeneousVolumeInterface);
	FIntVector BakingResolution = VolumeResolution;
	if (CVarHeterogeneousVolumesPreshadingCLOD.GetValueOnAnyThread() == 1)
	{
		// Determine baking voxel resolution
		float LODFactor = HeterogeneousVolumes::CalcLODFactor(View, HeterogeneousVolumeInterface);
		BakingResolution = FIntVector(BakingResolution / LODFactor);
		BakingResolution.X = FMath::Clamp(BakingResolution.X, 1, 1024);
		BakingResolution.Y = FMath::Clamp(BakingResolution.Y, 1, 1024);
		BakingResolution.Z = FMath::Clamp(BakingResolution.Z, 1, 1024);

	}
	else if (CVarHeterogeneousVolumesPreshadingCLOD.GetValueOnAnyThread() == 2)
	{
		HeterogeneousVolumes::FLODValue LODValue = HeterogeneousVolumes::CalcLOD(View, HeterogeneousVolumeInterface);
		BakingResolution = HeterogeneousVolumes::GetLightingCacheResolution(HeterogeneousVolumeInterface, LODValue);
	}

	// Collect material and object information
	const FPrimitiveSceneProxy* PrimitiveSceneProxy = HeterogeneousVolumeInterface->GetPrimitiveSceneProxy();
	const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
	const FPersistentPrimitiveIndex PersistentPrimitiveIndex = PrimitiveSceneInfo->GetPersistentIndex();
	const FBoxSphereBounds LocalBoxSphereBounds = HeterogeneousVolumeInterface->GetLocalBounds();

	// Create baked material grids
	const uint32 NumMips = 1;
	const EPixelFormat PixelFormat = HeterogeneousVolumes::GetPreshadingPixelFormat();

	FRHITextureCreateDesc ExtinctionMaterialDesc = FRHITextureCreateDesc::Create3D(TEXT("HeterogeneousVolumes.ExtinctionTexture"), BakingResolution, PixelFormat)
		.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);
	CallableInfo.ExtinctionRT = CreateRenderTarget(GraphBuilder.RHICmdList.CreateTexture(ExtinctionMaterialDesc), ExtinctionMaterialDesc.DebugName);

	FRHITextureCreateDesc EmissionMaterialDesc = FRHITextureCreateDesc::Create3D(TEXT("HeterogeneousVolumes.EmissionTexture"), BakingResolution, PixelFormat)
		.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);
	CallableInfo.EmissionRT = CreateRenderTarget(GraphBuilder.RHICmdList.CreateTexture(EmissionMaterialDesc), EmissionMaterialDesc.DebugName);

	FRHITextureCreateDesc AlbedoMaterialDesc = FRHITextureCreateDesc::Create3D(TEXT("HeterogeneousVolumes.AlbedoTexture"), BakingResolution, PixelFormat)
		.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);
	CallableInfo.AlbedoRT = CreateRenderTarget(GraphBuilder.RHICmdList.CreateTexture(AlbedoMaterialDesc), AlbedoMaterialDesc.DebugName);

	bool bJitter = HeterogeneousVolumes::ShouldJitter();
	bool bLooseBounds = false;
	bool bEvaluateAlbedo = true;
	bool bEvaluateEmission = true;
	ComputeHeterogeneousVolumeBakeMaterial(
		GraphBuilder,
		// Scene data
		Scene,
		View,
		// Object data
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		PersistentPrimitiveIndex,
		LocalBoxSphereBounds,
		// Sample data
		bJitter,
		bLooseBounds,
		bEvaluateAlbedo,
		bEvaluateEmission,
		// Volume data
		BakingResolution,
		// Output
		CallableInfo.ExtinctionRT,
		CallableInfo.EmissionRT,
		CallableInfo.AlbedoRT
	);

	//TRefCountPtr<IPooledRenderTarget> MajorantHistoryRT = View.ViewState ? CallableInfo.MajorantRT : nullptr;
	TRefCountPtr<IPooledRenderTarget> MajorantHistoryRT = View.ViewState ? PersistentMajorantRT : nullptr;
	FIntVector MajorantResolution;
	BuildMajorantTexture(
		GraphBuilder,
		Scene,
		View,
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		MajorantResolution,
		MajorantHistoryRT,
		PersistentMajorantRT
	);

	BuildPreshadingUniformBufferParameters(
		GraphBuilder,
		Scene,
		View,
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		BakingResolution,
		CallableInfo.ExtinctionRT,
		CallableInfo.EmissionRT,
		CallableInfo.AlbedoRT,
		MajorantResolution,
		PersistentMajorantRT,
		PreshadingUniformBufferParameters
	);
}

void RenderWithPreshading(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View, int32 ViewIndex,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Object data
	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FPersistentPrimitiveIndex& PersistentPrimitiveIndex,
	const FBoxSphereBounds LocalBoxSphereBounds,
	// Transmittance acceleration
	FRDGTextureRef LightingCacheTexture,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance
)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Effects);

	TRefCountPtr<IPooledRenderTarget> MajorantHistoryRT;
	TRDGUniformBufferRef<FPreshadingUniformBufferParameters> PreshadingUniformBuffer;
	BuildPreshadingGrid(
		GraphBuilder,
		Scene,
		View,
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		MajorantHistoryRT,
		PreshadingUniformBuffer
	);

	RenderWithPreshadingCompute(
		GraphBuilder,
		// Scene data
		SceneTextures,
		Scene,
		View, ViewIndex,
		// Shadow data
		VisibleLightInfos,
		VirtualShadowMapArray,
		// Object data
		HeterogeneousVolumeInterface,
		MaterialRenderProxy,
		// Preshading data
		PreshadingUniformBuffer,
		// Transmittance acceleration
		LightingCacheTexture,
		// Output
		HeterogeneousVolumeRadiance
	);
}
