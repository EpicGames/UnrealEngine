// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeterogeneousVolumes.h"

#include "PixelShaderUtils.h"
#include "RayTracingDefinitions.h"
#include "RayTracingInstance.h"
#include "RayTracingInstanceBufferUtil.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneRendering.h"

IMPLEMENT_UNIFORM_BUFFER_STRUCT(FSparseVoxelUniformBufferParameters, "SparseVoxelUniformBuffer");

class FCopyTexture3D : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyTexture3D);
	SHADER_USE_PARAMETER_STRUCT(FCopyTexture3D, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		SHADER_PARAMETER(FIntVector, TextureResolution)
		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float3>, RWOutputTexture)
		END_SHADER_PARAMETER_STRUCT()

		static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FCopyTexture3D, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesPreshadingPipeline.usf", "CopyTexture3D", SF_Compute);

void CopyTexture3D(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef Texture,
	uint32 InputMipLevel,
	FRDGTextureRef& OutputTexture
)
{
	FRDGTextureDesc OutputTextureDesc = OutputTexture->Desc;
	const FIntVector TextureResolution(OutputTextureDesc.Extent.X, OutputTextureDesc.Extent.Y, OutputTextureDesc.Depth);
	FCopyTexture3D::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyTexture3D::FParameters>();
	{
		PassParameters->TextureResolution = TextureResolution;
		PassParameters->InputTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Texture, InputMipLevel));
		PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutputTexture));
	}

	uint32 GroupCountX = FMath::DivideAndRoundUp(TextureResolution.X, FCopyTexture3D::GetThreadGroupSize3D());
	uint32 GroupCountY = FMath::DivideAndRoundUp(TextureResolution.Y, FCopyTexture3D::GetThreadGroupSize3D());
	uint32 GroupCountZ = FMath::DivideAndRoundUp(TextureResolution.Z, FCopyTexture3D::GetThreadGroupSize3D());
	FIntVector GroupCount(GroupCountX, GroupCountY, GroupCountZ);

	TShaderRef<FCopyTexture3D> ComputeShader = View.ShaderMap->GetShader<FCopyTexture3D>();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("CopyTexture3D"),
		ComputeShader,
		PassParameters,
		GroupCount);
}

class FClearAllocator : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearAllocator);
	SHADER_USE_PARAMETER_STRUCT(FClearAllocator, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint32>, RWAllocatorBuffer)
		END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FClearAllocator, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesSparseVoxelPipeline.usf", "ClearAllocator", SF_Compute);

class FAllocateSparseVoxels : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAllocateSparseVoxels);
	SHADER_USE_PARAMETER_STRUCT(FAllocateSparseVoxels, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, VoxelMinTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		SHADER_PARAMETER(FIntVector, VoxelMinResolution)
		SHADER_PARAMETER(uint32, MipLevel)
		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWNumVoxelsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVoxelDataPacked>, RWVoxelBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FAllocateSparseVoxels, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesSparseVoxelPipeline.usf", "AllocateSparseVoxels", SF_Compute);

class FAllocateSparseVoxelsToRefine : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAllocateSparseVoxelsToRefine);
	SHADER_USE_PARAMETER_STRUCT(FAllocateSparseVoxelsToRefine, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, VoxelMinTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		SHADER_PARAMETER(FIntVector, VoxelMinResolution)
		SHADER_PARAMETER(uint32, MipLevel)
		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWNumVoxelsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVoxelDataPacked>, RWVoxelBuffer)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWNumVoxelsToRefineBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVoxelDataPacked>, RWVoxelsToRefineBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FAllocateSparseVoxelsToRefine, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesSparseVoxelPipeline.usf", "AllocateSparseVoxelsToRefine", SF_Compute);

class FRefineSparseVoxels : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRefineSparseVoxels);
	SHADER_USE_PARAMETER_STRUCT(FRefineSparseVoxels, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture3D, VoxelMinTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
		SHADER_PARAMETER(FIntVector, VolumeResolution)
		SHADER_PARAMETER(FIntVector, MipVolumeResolution)
		SHADER_PARAMETER(uint32, MipLevel)

		// Refinement data
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, NumVoxelsToRefineBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FVoxelDataPacked>, VoxelsToRefineBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWNumVoxelsToRefineBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVoxelDataPacked>, RWVoxelsToRefineBuffer)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWNumVoxelsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVoxelDataPacked>, RWVoxelBuffer)

		// Indirect args
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)
		END_SHADER_PARAMETER_STRUCT()

		static int32 GetThreadGroupSize1D() { return 64; }
};

IMPLEMENT_GLOBAL_SHADER(FRefineSparseVoxels, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesSparseVoxelPipeline.usf", "RefineSparseVoxels", SF_Compute);

void GenerateSparseVoxels(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef VoxelMinTexture,
	FIntVector VolumeResolution,
	uint32 MipLevel,
	FRDGBufferRef& NumVoxelsBuffer,
	FRDGBufferRef& VoxelBuffer
)
{
	FIntVector VoxelMinResolution = FIntVector(
		VoxelMinTexture->Desc.Extent.X,
		VoxelMinTexture->Desc.Extent.Y,
		VoxelMinTexture->Desc.Depth
	);

	NumVoxelsBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1),
		TEXT("HeterogeneousVolumes.NumVoxelsBuffer")
	);

	// Clear allocator
	{
		FClearAllocator::FParameters* PassParameters = GraphBuilder.AllocParameters<FClearAllocator::FParameters>();
		{
			PassParameters->RWAllocatorBuffer = GraphBuilder.CreateUAV(NumVoxelsBuffer, PF_R32_UINT);
		}

		TShaderRef<FClearAllocator> ComputeShader = View.ShaderMap->GetShader<FClearAllocator>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ClearAllocator"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			ComputeShader,
			PassParameters,
			FIntVector(1)
		);
	}

	// Generate sets of OOBBs at a candidate MIP level for a given VRE coefficient
	VoxelBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FVoxelDataPacked), HeterogeneousVolumes::GetVoxelCount(VoxelMinResolution)),
		TEXT("HeterogeneousVolumes.VoxelBuffer")
	);

	if (HeterogeneousVolumes::ShouldRefineSparseVoxels())
	{
		// Refinement buffers
		FRDGBufferRef NumVoxelsToRefineBuffer[2] = {
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("HeterogeneousVolume.NumVoxelsToRefineBuffer1")),
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("HeterogeneousVolume.NumVoxelsToRefineBuffer2"))
		};
		FRDGBufferRef VoxelsToRefineBuffer[2] = {
			GraphBuilder.CreateBuffer(VoxelBuffer->Desc, TEXT("HeterogeneousVolume.VoxelsToRefineBuffer1")),
			GraphBuilder.CreateBuffer(VoxelBuffer->Desc, TEXT("HeterogeneousVolume.VoxelsToRefineBuffer2"))
		};
		uint32 InputIndex = 0;
		uint32 OutputIndex = 1;

		// Initial dense dispatch
		{
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(NumVoxelsToRefineBuffer[OutputIndex], PF_R32_UINT), 0);

			FAllocateSparseVoxelsToRefine::FParameters* PassParameters = GraphBuilder.AllocParameters<FAllocateSparseVoxelsToRefine::FParameters>();
			{
				PassParameters->VoxelMinTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(VoxelMinTexture, 0));
				PassParameters->VoxelMinResolution = VoxelMinResolution;
				PassParameters->MipLevel = MipLevel;

				PassParameters->RWNumVoxelsBuffer = GraphBuilder.CreateUAV(NumVoxelsBuffer, PF_R32_UINT);
				PassParameters->RWVoxelBuffer = GraphBuilder.CreateUAV(VoxelBuffer);

				PassParameters->RWNumVoxelsToRefineBuffer = GraphBuilder.CreateUAV(NumVoxelsToRefineBuffer[OutputIndex], PF_R32_UINT);
				PassParameters->RWVoxelsToRefineBuffer = GraphBuilder.CreateUAV(VoxelsToRefineBuffer[OutputIndex]);
			}

			uint32 GroupCountX = FMath::DivideAndRoundUp(VoxelMinResolution.X, FAllocateSparseVoxelsToRefine::GetThreadGroupSize3D());
			uint32 GroupCountY = FMath::DivideAndRoundUp(VoxelMinResolution.Y, FAllocateSparseVoxelsToRefine::GetThreadGroupSize3D());
			uint32 GroupCountZ = FMath::DivideAndRoundUp(VoxelMinResolution.Z, FAllocateSparseVoxelsToRefine::GetThreadGroupSize3D());
			FIntVector GroupCount(GroupCountX, GroupCountY, GroupCountZ);
			TShaderRef<FAllocateSparseVoxelsToRefine> ComputeShader = View.ShaderMap->GetShader<FAllocateSparseVoxelsToRefine>();
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("AllocateSparseVoxelsToRefine"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				PassParameters,
				GroupCount
			);
		}

		// Flip input/output buffers for next iteration
		Swap(InputIndex, OutputIndex);

		for (uint32 MipLevelOffset = 1; MipLevelOffset < VoxelMinTexture->Desc.NumMips; ++MipLevelOffset)
		{
			// Clear refinement output buffers
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(NumVoxelsToRefineBuffer[OutputIndex], PF_R32_UINT), 0);

			FRefineSparseVoxels::FParameters* PassParameters = GraphBuilder.AllocParameters<FRefineSparseVoxels::FParameters>();
			{
				PassParameters->VoxelMinTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(VoxelMinTexture, MipLevelOffset));
				FIntVector MipVolumeResolution = FIntVector(
					VoxelMinResolution.X >> MipLevelOffset,
					VoxelMinResolution.Y >> MipLevelOffset,
					VoxelMinResolution.Z >> MipLevelOffset
				);
				PassParameters->VolumeResolution = VolumeResolution;
				PassParameters->MipVolumeResolution = MipVolumeResolution;
				PassParameters->MipLevel = MipLevel + MipLevelOffset;

				// Intermediate refinement buffers
				PassParameters->NumVoxelsToRefineBuffer = GraphBuilder.CreateSRV(NumVoxelsToRefineBuffer[InputIndex], PF_R32_UINT);
				PassParameters->VoxelsToRefineBuffer = GraphBuilder.CreateSRV(VoxelsToRefineBuffer[InputIndex]);
				PassParameters->RWNumVoxelsToRefineBuffer = GraphBuilder.CreateUAV(NumVoxelsToRefineBuffer[OutputIndex], PF_R32_UINT);
				PassParameters->RWVoxelsToRefineBuffer = GraphBuilder.CreateUAV(VoxelsToRefineBuffer[OutputIndex]);

				// Sparse voxel output
				PassParameters->RWNumVoxelsBuffer = GraphBuilder.CreateUAV(NumVoxelsBuffer, PF_R32_UINT);
				PassParameters->RWVoxelBuffer = GraphBuilder.CreateUAV(VoxelBuffer);

				PassParameters->IndirectArgs = NumVoxelsToRefineBuffer[InputIndex];
			}

			TShaderRef<FRefineSparseVoxels> ComputeShader = View.ShaderMap->GetShader<FRefineSparseVoxels>();
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("RefineSparseVoxels"),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				PassParameters->IndirectArgs,
				0
			);

			// Flip input/output buffers for next iteration
			Swap(InputIndex, OutputIndex);
		}
	}
	else
	{
		FAllocateSparseVoxels::FParameters* PassParameters = GraphBuilder.AllocParameters<FAllocateSparseVoxels::FParameters>();
		{
			//PassParameters->VoxelMinTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(VoxelMinTexture, MipLevel));
			PassParameters->VoxelMinTexture = GraphBuilder.CreateSRV(VoxelMinTexture);
			// TODO: Make sure point-sampling isn't causing clipping of the isosurface
			PassParameters->TextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->VoxelMinResolution = VoxelMinResolution;
			PassParameters->MipLevel = MipLevel;

			PassParameters->RWNumVoxelsBuffer = GraphBuilder.CreateUAV(NumVoxelsBuffer, PF_R32_UINT);
			PassParameters->RWVoxelBuffer = GraphBuilder.CreateUAV(VoxelBuffer);
		}

		uint32 GroupCountX = FMath::DivideAndRoundUp(VoxelMinResolution.X, FAllocateSparseVoxels::GetThreadGroupSize3D());
		uint32 GroupCountY = FMath::DivideAndRoundUp(VoxelMinResolution.Y, FAllocateSparseVoxels::GetThreadGroupSize3D());
		uint32 GroupCountZ = FMath::DivideAndRoundUp(VoxelMinResolution.Z, FAllocateSparseVoxels::GetThreadGroupSize3D());
		FIntVector GroupCount(GroupCountX, GroupCountY, GroupCountZ);
		TShaderRef<FAllocateSparseVoxels> ComputeShader = View.ShaderMap->GetShader<FAllocateSparseVoxels>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("AllocateSparseVoxels"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			ComputeShader,
			PassParameters,
			GroupCount
		);
	}
}