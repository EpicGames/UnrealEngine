// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "BlueNoise.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "DecalRenderingShared.h"
#include "DistanceFieldLightingShared.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "ExtensibleUniformBuffer.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "HeterogeneousVolumeInterface.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "LightRendering.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "LocalVertexFactory.h"
#include "Logging/LogMacros.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "Math/Plane.h"
#include "Math/Sphere.h"
#include "Math/TransformVectorized.h"
#include "MeshBatch.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshPassProcessor.h"
#include "MeshPassUtils.h"
#include "Misc/LargeWorldCoordinates.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "RayTracing/RaytracingOptions.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphParameter.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "SceneTextures.h"
#include "SceneTypes.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShadowRendering.h"
#include "Stats/Stats.h"
#include "SystemTextures.h"
#include "UniformBuffer.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "SceneViewState.h"

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesEnableFrustumVoxelGrid(
	TEXT("r.HeterogeneousVolumes.FrustumGrid"),
	1,
	TEXT("Enables a frustum voxel grid (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesFrustumGridShadingRate(
	TEXT("r.HeterogeneousVolumes.FrustumGrid.ShadingRate"),
	4.0,
	TEXT("The voxel tessellation rate, in pixel-space (Default = 4.0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesFrustumGridNearPlaneDistance(
	TEXT("r.HeterogeneousVolumes.FrustumGrid.NearPlaneDistance"),
	1.0,
	TEXT("Sets near-plane distance for the frustum grid (Default = 1.0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesFrustumGridFarPlaneDistance(
	TEXT("r.HeterogeneousVolumes.FrustumGrid.FarPlaneDistance"),
	-1.0,
	TEXT("Sets far-plane distance for the frustum grid (Default = -1.0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesFrustumGridDepthSliceCount(
	TEXT("r.HeterogeneousVolumes.FrustumGrid.DepthSliceCount"),
	512,
	TEXT("The number of depth slices (Default = 512)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesFrustumGridMaxBottomLevelMemoryInMegabytes(
	TEXT("r.HeterogeneousVolumes.FrustumGrid.MaxBottomLevelMemoryInMegabytes"),
	128,
	TEXT("The minimum voxel size (Default = 128)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesEnableOrthoVoxelGrid(
	TEXT("r.HeterogeneousVolumes.OrthoGrid"),
	1,
	TEXT("Enables an ortho voxel grid (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesOrthoGridShadingRate(
	TEXT("r.HeterogeneousVolumes.OrthoGrid.ShadingRate"),
	4.0,
	TEXT("The voxelization rate (Default = 4.0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesOrthoGridVoxelizationMode(
	TEXT("r.HeterogeneousVolumes.OrthoGrid.VoxelizationMode"),
	1,
	TEXT("Voxelization mode (Default = 1)\n")
	TEXT("0: Screen-space voxel size (legacy behavior)\n")
	TEXT("1: World-space voxel size\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesOrthoGridMaxBottomLevelMemoryInMegabytes(
	TEXT("r.HeterogeneousVolumes.OrthoGrid.MaxBottomLevelMemoryInMegabytes"),
	128,
	TEXT("The minimum voxel size (Default = 128)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesMarchingMode(
	TEXT("r.HeterogeneousVolumes.Debug.MarchingMode"),
	1,
	TEXT("The marching mode (Default = 0)\n")
	TEXT("0: Ray Marching (dt=StepSize)\n")
	TEXT("1: Naive DDA\n")
	TEXT("2: Optimized DDA\n")
	TEXT("3: Optimized DDA w/ bitmask\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesTessellationJitter(
	TEXT("r.HeterogeneousVolumes.Tessellation.Jitter"),
	1,
	TEXT("Enables jittering when tessellating the acceleration grids (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesBottomLevelGridResolution(
	TEXT("r.HeterogeneousVolumes.Tessellation.BottomLevelGrid.Resolution"),
	4,
	TEXT("Determines intra-tile bottom-level grid resolution (Default = 4)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesBottomLevelGridVoxelHashing(
	TEXT("r.HeterogeneousVolumes.Tessellation.BottomLevelGrid.VoxelHashing"),
	0,
	TEXT("Enables bottom-level voxel hashing for deduplication (Default = 0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesMinimumVoxelSizeInFrustum(
	TEXT("r.HeterogeneousVolumes.Tessellation.MinimumVoxelSizeInFrustum"),
	0.1f,
	TEXT("The minimum voxel size (Default = 0.1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesMinimumVoxelSizeOutsideFrustum(
	TEXT("r.HeterogeneousVolumes.Tessellation.MinimumVoxelSizeOutsideFrustum"),
	100.0,
	TEXT("The minimum voxel size (Default = 100.0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesBottomLevelGridVoxelHashingMemoryInMegabytes(
	TEXT("r.HeterogeneousVolumes.Tessellation.BottomLevelGrid.VoxelHashingMemoryInMegabytes"),
	64,
	TEXT("Enables bottom-level voxel hashing for deduplication (Default = 64)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesBottomLevelGridHomogeneousAggregation(
	TEXT("r.HeterogeneousVolumes.Tessellation.BottomLevelGrid.HomogeneousAggregation"),
	1,
	TEXT("Enables bottom-level voxel homogeneous aggregation (Default = 1)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesBottomLevelGridHomogeneousAggregationThreshold(
	TEXT("r.HeterogeneousVolumes.Tessellation.BottomLevelGrid.HomogeneousAggregationThreshold"),
	1.0e-3f,
	TEXT("Threshold for bottom-level voxel homogeneous aggregation (Default = 1.0e-3)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesIndirectionGrid(
	TEXT("r.HeterogeneousVolumes.Tessellation.IndirectionGrid"),
	1,
	TEXT("Enables lazy allocation of bottom-level memory (Default = 1)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesIndirectionGridResolution(
	TEXT("r.HeterogeneousVolumes.Tessellation.IndirectionGrid.Resolution"),
	4,
	TEXT("Determines intra-tile indirection grid resolution (Default = 4)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesFarPlaneAutoTransition(
	TEXT("r.HeterogeneousVolumes.Tessellation.FarPlaneAutoTransition"),
	1,
	TEXT("Enables auto transitioning of far-plane distance, based on projected minimum voxel size (Default = 1)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesEnableTopLevelBitmask(
	TEXT("r.HeterogeneousVolumes.Tessellation.TopLevelBitmask"),
	0,
	TEXT("Enables top-level bitmask to accelerate grid traversal (Default = 0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesEnableMajorantGrid(
	TEXT("r.HeterogeneousVolumes.Tessellation.MajorantGrid"),
	1,
	TEXT("Enables building majorant grids to accelerate volume tracking (Default = 0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesEnableMajorantGridMax(
	TEXT("r.HeterogeneousVolumes.Tessellation.MajorantGrid.Max"),
	0,
	TEXT("Enables building majorant grids to accelerate volume tracking (Default = 0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowCameraDownsampleFactor(
	TEXT("r.HeterogeneousVolumes.Shadows.CameraDownsampleFactor"),
	2,
	TEXT("Controls downsample factor for camera volumetric shadow map (default = 2)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowResolution(
	TEXT("r.HeterogeneousVolumes.Shadows.Resolution"),
	512,
	TEXT("Resolution when building volumetric shadow map (Default = 512)\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesStepSizeForShadows(
	TEXT("r.HeterogeneousVolumes.Shadows.StepSize"),
	2.0,
	TEXT("Ray marching step size when building volumetric shadow map (Default = 2.0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowMaxSampleCount(
	TEXT("r.HeterogeneousVolumes.Shadows.MaxSampleCount"),
	8,
	TEXT("Maximum sample count when building volumetric shadow map (Default = 8)\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesShadowAbsoluteErrorThreshold(
	TEXT("r.HeterogeneousVolumes.Shadows.AbsoluteErrorThreshold"),
	0.0,
	TEXT("Absolute error threshold for volume shadow compression (Default = 0.0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesShadowRelativeErrorThreshold(
	TEXT("r.HeterogeneousVolumes.Shadows.RelativeErrorThreshold"),
	0.05f,
	TEXT("Relative error threshold for volume shadow compression (Default = 0.05)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowUseAVSMCompression(
	TEXT("r.HeterogeneousVolumes.Shadows.UseAVSMCompression"),
	1,
	TEXT("Enables AVSM compression (Default = 1)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowDebugTweak(
	TEXT("r.HeterogeneousVolumes.Shadows.DebugTweak"),
	0,
	TEXT("Debug tweak value (Default = 0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarHeterogeneousVolumesShadowShadingRate(
	TEXT("r.HeterogeneousVolumes.Shadows.ShadingRate"),
	1.0f,
	TEXT("Shading rate (in-frustum) for computing shadows (Default = 1.0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowOutOfFrustumShadingRate(
	TEXT("r.HeterogeneousVolumes.Shadows.OutOfFrustumShadingRate"),
	2,
	TEXT("Shading rate (out-of-frustum) for computing shadows (Default = 2.0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesShadowJitter(
	TEXT("r.HeterogeneousVolumes.Shadows.Jitter"),
	0,
	TEXT("Enables jittering when constructing shadows (Default = 0)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarHeterogeneousShadowsForLightTypeDirectional(
	TEXT("r.HeterogeneousVolumes.Shadows.LightType.Directional"),
	1,
	TEXT("Enables shadows from the directional light (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousCascadeShadowsForDirectionalLight(
	TEXT("r.HeterogeneousVolumes.Shadows.Cascades"),
	1,
	TEXT("Enables cascade shadows for the directional light (Default = 1)\n")
	TEXT("0: Disabled\n")
	TEXT("1: View-based frustums (Default)\n")
	TEXT("2: View-based clipmaps centered about the camera\n")
	TEXT("3: Auto-fit shadow maps based on bounding dimensions of contained volumes"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesCascadeShadowsQuantization(
	TEXT("r.HeterogeneousVolumes.Shadows.Cascades.PixelSnapping"),
	16,
	TEXT("Cascade quantization unit in pixel-space (Default = 16)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarHeterogeneousShadowsForLightTypePoint(
	TEXT("r.HeterogeneousVolumes.Shadows.LightType.Point"),
	1,
	TEXT("Enables shadows from point lights (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarHeterogeneousShadowsForLightTypeSpot(
	TEXT("r.HeterogeneousVolumes.Shadows.LightType.Spot"),
	1,
	TEXT("Enables shadows from spot lights (Default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarHeterogeneousShadowsForLightTypeRect(
	TEXT("r.HeterogeneousVolumes.Shadows.LightType.Rect"),
	1,
	TEXT("Enables shadows from rect lights (Default = 1)"),
	ECVF_RenderThreadSafe
);

IMPLEMENT_UNIFORM_BUFFER_STRUCT(FOrthoVoxelGridUniformBufferParameters, "OrthoGridUniformBuffer");
IMPLEMENT_UNIFORM_BUFFER_STRUCT(FFrustumVoxelGridUniformBufferParameters, "FrustumGridUniformBuffer");
IMPLEMENT_UNIFORM_BUFFER_STRUCT(FAdaptiveVolumetricShadowMapUniformBufferParameters, "AVSM");
IMPLEMENT_UNIFORM_BUFFER_STRUCT(FBeerShadowMapUniformBufferParameters, "BeerShadowMap");

struct FRasterTileData
{
	uint32 TopLevelGridLinearIndex;
	uint32 BottomLevelGridLinearOffset;
};

namespace HeterogeneousVolumes
{
	bool EnableFrustumVoxelGrid()
	{
		return CVarHeterogeneousVolumesEnableFrustumVoxelGrid.GetValueOnRenderThread() != 0;
	}

	float GetShadingRateForFrustumGrid()
	{
		return FMath::Max(CVarHeterogeneousVolumesFrustumGridShadingRate.GetValueOnRenderThread(), 0.1);
	}

	float GetNearPlaneDistanceForFrustumGrid()
	{
		return CVarHeterogeneousVolumesFrustumGridNearPlaneDistance.GetValueOnRenderThread();
	}

	float GetFarPlaneDistanceForFrustumGrid()
	{
		return CVarHeterogeneousVolumesFrustumGridFarPlaneDistance.GetValueOnRenderThread();
	}

	int32 GetDepthSliceCountForFrustumGrid()
	{
		return FMath::Max(CVarHeterogeneousVolumesFrustumGridDepthSliceCount.GetValueOnRenderThread(), 1);
	}

	int32 GetMaxBottomLevelMemoryInMegabytesForFrustumGrid()
	{
		return FMath::Max(CVarHeterogeneousVolumesFrustumGridMaxBottomLevelMemoryInMegabytes.GetValueOnRenderThread(), 1);
	}

	bool EnableOrthoVoxelGrid()
	{
		return CVarHeterogeneousVolumesEnableOrthoVoxelGrid.GetValueOnRenderThread() != 0;
	}

	float GetShadingRateForOrthoGrid()
	{
		return FMath::Max(CVarHeterogeneousVolumesOrthoGridShadingRate.GetValueOnRenderThread(), 0.1);
	}

	int32 GetMaxBottomLevelMemoryInMegabytesForOrthoGrid()
	{
		return FMath::Max(CVarHeterogeneousVolumesOrthoGridMaxBottomLevelMemoryInMegabytes.GetValueOnRenderThread(), 1);
	}

	bool EnableFarPlaneAutoTransition()
	{
		return CVarHeterogeneousVolumesFarPlaneAutoTransition.GetValueOnRenderThread() != 0;
	}

	float GetMinimumVoxelSizeInFrustum()
	{
		return FMath::Max(CVarHeterogeneousVolumesMinimumVoxelSizeInFrustum.GetValueOnRenderThread(), 0.01);
	}

	float GetMinimumVoxelSizeOutsideFrustum()
	{
		return FMath::Max(CVarHeterogeneousVolumesMinimumVoxelSizeOutsideFrustum.GetValueOnRenderThread(), 0.01);
	}

	int32 GetMarchingMode()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesMarchingMode.GetValueOnRenderThread(), 0, 3);
	}

	bool EnableVoxelHashing()
	{
		//return CVarHeterogeneousVolumesBottomLevelGridVoxelHashing.GetValueOnRenderThread() != 0;
		return false;
	}

	uint32 GetVoxelHashingMemoryInMegabytes()
	{
		return CVarHeterogeneousVolumesBottomLevelGridVoxelHashingMemoryInMegabytes.GetValueOnRenderThread();
	}

	bool EnableHomogeneousAggregation()
	{
		return CVarHeterogeneousVolumesBottomLevelGridHomogeneousAggregation.GetValueOnRenderThread() != 0;
	}

	float GetHomogeneousAggregationThreshold()
	{
		return CVarHeterogeneousVolumesBottomLevelGridHomogeneousAggregationThreshold.GetValueOnRenderThread();
	}

	bool EnableIndirectionGrid()
	{
		return CVarHeterogeneousVolumesIndirectionGrid.GetValueOnRenderThread() != 0;
	}

	bool EnableJitter()
	{
		return CVarHeterogeneousVolumesTessellationJitter.GetValueOnRenderThread() != 0;
	}

	int32 GetBottomLevelGridResolution()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesBottomLevelGridResolution.GetValueOnRenderThread(), 1, 4);
	}

	int32 GetIndirectionGridResolution()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesIndirectionGridResolution.GetValueOnRenderThread(), 1, 4);
	}

	bool EnableLinearInterpolation()
	{
		return false;
	}

	bool EnableTopLevelBitmask()
	{
		return CVarHeterogeneousVolumesEnableTopLevelBitmask.GetValueOnRenderThread() != 0;
	}

	bool EnableMajorantGrid()
	{
		return CVarHeterogeneousVolumesEnableMajorantGrid.GetValueOnRenderThread() != 0;
	}

	float CalcTanHalfFOV(float FOVInDegrees)
	{
		return FMath::Tan(FMath::DegreesToRadians(FOVInDegrees * 0.5));
	}

	FIntPoint GetShadowMapResolution()
	{
		return FIntPoint(FMath::Clamp(CVarHeterogeneousVolumesShadowResolution.GetValueOnRenderThread(), 1, 2048));
	}

	float GetStepSizeForShadows()
	{
		return FMath::Max(CVarHeterogeneousVolumesStepSizeForShadows.GetValueOnRenderThread(), 0.01f);
	}

	uint32 GetShadowMaxSampleCount()
	{
		return FMath::Clamp(CVarHeterogeneousVolumesShadowMaxSampleCount.GetValueOnRenderThread(), 2, 64);
	}

	float GetShadowAbsoluteErrorThreshold()
	{
		return FMath::Max(CVarHeterogeneousVolumesShadowAbsoluteErrorThreshold.GetValueOnRenderThread(), 0.0);
	}

	float GetShadowRelativeErrorThreshold()
	{
		return FMath::Max(CVarHeterogeneousVolumesShadowRelativeErrorThreshold.GetValueOnRenderThread(), 0.0);
	}

	bool UseAVSMCompression()
	{
		return CVarHeterogeneousVolumesShadowUseAVSMCompression.GetValueOnRenderThread() != 0;
	}

	float GetCameraDownsampleFactor()
	{
		return FMath::Max(CVarHeterogeneousVolumesShadowCameraDownsampleFactor.GetValueOnRenderThread(), 1);
	}

	float GetShadingRateForShadows()
	{
		return FMath::Max(CVarHeterogeneousVolumesShadowShadingRate.GetValueOnRenderThread(), 0.1);
	}

	float GetOutOfFrustumShadingRateForShadows()
	{
		return FMath::Max(CVarHeterogeneousVolumesShadowOutOfFrustumShadingRate.GetValueOnRenderThread(), 0.1);
	}

	bool EnableJitterForShadows()
	{
		return CVarHeterogeneousVolumesShadowJitter.GetValueOnRenderThread() != 0;
	}

	bool SupportsShadowForLightType(uint32 LightType)
	{
		if (SupportsLightType(LightType))
		{
			switch (LightType)
			{
			case LightType_Directional:
				return CVarHeterogeneousShadowsForLightTypeDirectional.GetValueOnRenderThread();
			case LightType_Point:
				return CVarHeterogeneousShadowsForLightTypePoint.GetValueOnRenderThread();
			case LightType_Spot:
				return CVarHeterogeneousShadowsForLightTypeSpot.GetValueOnRenderThread();
			case LightType_Rect:
				return CVarHeterogeneousShadowsForLightTypeRect.GetValueOnRenderThread();
			}
		}

		return false;
	}

	bool SupportsCascadeShadowsForDirectionalLight()
	{
		if (SupportsLightType(LightType_Directional) && SupportsShadowForLightType(LightType_Directional))
		{
			return CVarHeterogeneousCascadeShadowsForDirectionalLight.GetValueOnRenderThread() != 0;
		}

		return false;
	}

	ECascadeShadowMode GetCascadeShadowMode()
	{
		return static_cast<ECascadeShadowMode>(CVarHeterogeneousCascadeShadowsForDirectionalLight.GetValueOnRenderThread());
	}

	int32 GetCascadeShadowsQuantizationUnit()
	{
		if (SupportsCascadeShadowsForDirectionalLight())
		{
			return CVarHeterogeneousVolumesCascadeShadowsQuantization.GetValueOnRenderThread();
		}

		return 0;
	}

	bool IsDynamicShadow(const FVisibleLightInfo* VisibleLightInfo)
	{
		check(VisibleLightInfo != nullptr);
		if (VisibleLightInfo)
		{
			return !VisibleLightInfo->ShadowsToProject.IsEmpty();
		}

		return false;
	}

	const FProjectedShadowInfo* GetProjectedShadowInfo(const FVisibleLightInfo* VisibleLightInfo, int32 ShadowIndex)
	{
		check(VisibleLightInfo != nullptr);
		if (VisibleLightInfo && ShadowIndex < VisibleLightInfo->ShadowsToProject.Num())
		{
			return VisibleLightInfo->ShadowsToProject[ShadowIndex];
		}

		return nullptr;
	}
}

uint32 GetTypeHash(const FVolumetricMeshBatch& MeshBatch)
{
	return HashCombineFast(GetTypeHash(MeshBatch.Mesh), GetTypeHash(MeshBatch.Proxy));
}

class FMarkTopLevelGridVoxelsForFrustumGrid : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMarkTopLevelGridVoxelsForFrustumGrid);
	SHADER_USE_PARAMETER_STRUCT(FMarkTopLevelGridVoxelsForFrustumGrid, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)

		SHADER_PARAMETER(FIntVector, TopLevelGridResolution)

		SHADER_PARAMETER(FVector3f, PrimitiveWorldBoundsMin)
		SHADER_PARAMETER(FVector3f, PrimitiveWorldBoundsMax)

		SHADER_PARAMETER(FMatrix44f, ViewToWorld)
		SHADER_PARAMETER(float, TanHalfFOV)
		SHADER_PARAMETER(float, NearPlaneDepth)
		SHADER_PARAMETER(float, FarPlaneDepth)

		SHADER_PARAMETER(FIntVector, VoxelDimensions)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FTopLevelGridData>, RWTopLevelGridBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FMarkTopLevelGridVoxelsForFrustumGrid, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesFrustumVoxelGrid.usf", "MarkTopLevelGridVoxelsForFrustumGridCS", SF_Compute);

class FRasterizeBottomLevelFrustumGridCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRasterizeBottomLevelFrustumGridCS, MeshMaterial);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)

		// Primitive data
		SHADER_PARAMETER(FVector3f, PrimitiveWorldBoundsMin)
		SHADER_PARAMETER(FVector3f, PrimitiveWorldBoundsMax)
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLocal)
		SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
		SHADER_PARAMETER(FVector3f, LocalBoundsExtent)
		SHADER_PARAMETER(int32, PrimitiveId)

		// Volume data
		SHADER_PARAMETER(FIntVector, TopLevelGridResolution)
		SHADER_PARAMETER(FIntVector, VoxelDimensions)
		SHADER_PARAMETER(FMatrix44f, ViewToWorld)
		SHADER_PARAMETER(float, TanHalfFOV)
		SHADER_PARAMETER(float, NearPlaneDepth)
		SHADER_PARAMETER(float, FarPlaneDepth)

		SHADER_PARAMETER(int, BottomLevelGridBufferSize)

		// Sampling data
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)

		// Raster tile data
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, RasterTileAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRasterTileData>, RasterTileBuffer)

		// Indirect args
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)

		// Grid data
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FTopLevelGridData>, RWTopLevelGridBuffer)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWBottomLevelGridAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FScalarGridData>, RWExtinctionGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVectorGridData>, RWEmissionGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVectorGridData>, RWScatteringGridBuffer)
	END_SHADER_PARAMETER_STRUCT()

	FRasterizeBottomLevelFrustumGridCS() = default;

	FRasterizeBottomLevelFrustumGridCS(
		const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer
	)
		: FMeshMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false
		);
	}

	static bool ShouldCompilePermutation(
		const FMaterialShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform)
			&& DoesMaterialShaderSupportHeterogeneousVolumes(Parameters.MaterialParameters);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(
		const FMaterialShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}
#if 0
	void SetParameters(
		FRHIComputeCommandList& RHICmdList,
		FRHIComputeShader* ShaderRHI,
		const FViewInfo& View,
		const FMaterialRenderProxy* MaterialProxy,
		const FMaterial& Material
	)
	{
		FMaterialShader::SetViewParameters(RHICmdList, ShaderRHI, View, View.ViewUniformBuffer);
		FMaterialShader::SetParameters(RHICmdList, ShaderRHI, MaterialProxy, Material, View);
	}
#endif
	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRasterizeBottomLevelFrustumGridCS, TEXT("/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesMaterialBakingPipeline.usf"), TEXT("RasterizeBottomLevelFrustumGridCS"), SF_Compute);

class FTopLevelGridCalculateVoxelSize : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTopLevelGridCalculateVoxelSize);
	SHADER_USE_PARAMETER_STRUCT(FTopLevelGridCalculateVoxelSize, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)

		SHADER_PARAMETER(FIntVector, TopLevelGridResolution)
		SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMin)
		SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMax)

		SHADER_PARAMETER(FVector3f, PrimitiveWorldBoundsMin)
		SHADER_PARAMETER(FVector3f, PrimitiveWorldBoundsMax)
		
		SHADER_PARAMETER(float, ShadingRateInFrustum)
		SHADER_PARAMETER(float, ShadingRateOutOfFrustum)
		SHADER_PARAMETER(float, MinVoxelSizeInFrustum)
		SHADER_PARAMETER(float, MinVoxelSizeOutOfFrustum)
		SHADER_PARAMETER(float, DownsampleFactor)
		SHADER_PARAMETER(int, bUseProjectedPixelSize)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FTopLevelGridData>, RWTopLevelGridBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FTopLevelGridCalculateVoxelSize, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridPipeline.usf", "TopLevelGridCalculateVoxelSize", SF_Compute);

class FAllocateBottomLevelGrid : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAllocateBottomLevelGrid);
	SHADER_USE_PARAMETER_STRUCT(FAllocateBottomLevelGrid, FGlobalShader);

	class FEnableIndirectionGrid : SHADER_PERMUTATION_BOOL("DIM_ENABLE_INDIRECTION_GRID");
	using FPermutationDomain = TShaderPermutationDomain<FEnableIndirectionGrid>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)

		SHADER_PARAMETER(FIntVector, TopLevelGridResolution)
		SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMin)
		SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMax)

		SHADER_PARAMETER(int, MaxVoxelResolution)
		SHADER_PARAMETER(int, bSampleAtVertices)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FTopLevelGridData>, RWTopLevelGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWBottomLevelGridAllocatorBuffer)

		// Indirection Grid
		SHADER_PARAMETER(int, IndirectionGridBufferSize)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FTopLevelGridData>, RWIndirectionGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectionGridAllocatorBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FAllocateBottomLevelGrid, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridPipeline.usf", "AllocateBottomLevelGrid", SF_Compute);

class FGenerateRasterTiles : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGenerateRasterTiles);
	SHADER_USE_PARAMETER_STRUCT(FGenerateRasterTiles, FGlobalShader);

	class FEnableIndirectionGrid : SHADER_PERMUTATION_BOOL("DIM_ENABLE_INDIRECTION_GRID");
	using FPermutationDomain = TShaderPermutationDomain<FEnableIndirectionGrid>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		//SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)

		SHADER_PARAMETER(FIntVector, TopLevelGridResolution)
		SHADER_PARAMETER(int, RasterTileVoxelResolution)
		SHADER_PARAMETER(int, MaxNumRasterTiles)
		//SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMin)
		//SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMax)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTopLevelGridData>, TopLevelGridBuffer)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWRasterTileAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FRasterTileData>, RWRasterTileBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FGenerateRasterTiles, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridPipeline.usf", "GenerateRasterTiles", SF_Compute);

class FSetRasterizeBottomLevelGridIndirectArgs : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSetRasterizeBottomLevelGridIndirectArgs);
	SHADER_USE_PARAMETER_STRUCT(FSetRasterizeBottomLevelGridIndirectArgs, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, MaxDispatchThreadGroupsPerDimension)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, RasterTileAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWRasterizeBottomLevelGridIndirectArgsBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FSetRasterizeBottomLevelGridIndirectArgs, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridPipeline.usf", "SetRasterizeBottomLevelGridIndirectArgs", SF_Compute);

class FRasterizeBottomLevelOrthoGridCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRasterizeBottomLevelOrthoGridCS, MeshMaterial);

	class FEnableIndirectionGrid : SHADER_PERMUTATION_BOOL("DIM_ENABLE_INDIRECTION_GRID");
	//class FEnableVoxelHashing : SHADER_PERMUTATION_BOOL("DIM_ENABLE_VOXEL_HASHING");
	class FEnableHomogeneousAggregation : SHADER_PERMUTATION_BOOL("DIM_ENABLE_HOMOGENEOUS_AGGREGATION");
	using FPermutationDomain = TShaderPermutationDomain<FEnableIndirectionGrid, FEnableHomogeneousAggregation>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)

		// Primitive data
		SHADER_PARAMETER(FVector3f, PrimitiveWorldBoundsMin)
		SHADER_PARAMETER(FVector3f, PrimitiveWorldBoundsMax)
		SHADER_PARAMETER(FMatrix44f, LocalToWorld)
		SHADER_PARAMETER(FMatrix44f, WorldToLocal)
		SHADER_PARAMETER(FVector3f, LocalBoundsOrigin)
		SHADER_PARAMETER(FVector3f, LocalBoundsExtent)
		SHADER_PARAMETER(int32, PrimitiveId)

		// Volume data
		SHADER_PARAMETER(FIntVector, TopLevelGridResolution)
		SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMin)
		SHADER_PARAMETER(FVector3f, TopLevelGridWorldBoundsMax)

		SHADER_PARAMETER(int, BottomLevelGridBufferSize)

		// Sampling data
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)

		// Volume sample mode
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER(int, bSampleAtVertices)

		// Raster tile data
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, RasterTileAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRasterTileData>, RasterTileBuffer)

		// Indirect args
		RDG_BUFFER_ACCESS(IndirectArgs, ERHIAccess::IndirectArgs)

		// Grid data
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTopLevelGridData>, TopLevelGridBuffer)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWBottomLevelGridAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FTopLevelGridData>, RWTopLevelGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FScalarGridData>, RWExtinctionGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVectorGridData>, RWEmissionGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVectorGridData>, RWScatteringGridBuffer)

		// Indirection Grid
		SHADER_PARAMETER(int, IndirectionGridBufferSize)
		SHADER_PARAMETER(int, FixedBottomLevelResolution)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FTopLevelGridData>, RWIndirectionGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectionGridAllocatorBuffer)

		// Hash Table
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWHashTable)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWHashToVoxelBuffer)
		SHADER_PARAMETER(int, HashTableSize)

		// Homogeneous aggregation
		SHADER_PARAMETER(float, HomogeneousThreshold)
	END_SHADER_PARAMETER_STRUCT()
	
	FRasterizeBottomLevelOrthoGridCS() = default;

	FRasterizeBottomLevelOrthoGridCS(
		const FMeshMaterialShaderType::CompiledShaderInitializerType & Initializer
	)
		: FMeshMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false
		);
	}

	static bool ShouldCompilePermutation(
		const FMaterialShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform)
			&& DoesMaterialShaderSupportHeterogeneousVolumes(Parameters.MaterialParameters);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(
		const FMaterialShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("GET_PRIMITIVE_DATA_OVERRIDE"), 1);
		OutEnvironment.SetDefine(TEXT("USE_INSTANCE_CULLING"), 0);
	}
#if 0
	void SetParameters(
		FRHIComputeCommandList& RHICmdList,
		FRHIComputeShader* ShaderRHI,
		const FViewInfo& View,
		const FMaterialRenderProxy* MaterialProxy,
		const FMaterial& Material
	)
	{
		FMaterialShader::SetViewParameters(RHICmdList, ShaderRHI, View, View.ViewUniformBuffer);
		FMaterialShader::SetParameters(RHICmdList, ShaderRHI, MaterialProxy, Material, View);
	}
#endif
	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRasterizeBottomLevelOrthoGridCS, TEXT("/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesMaterialBakingPipeline.usf"), TEXT("RasterizeBottomLevelOrthoGridCS"), SF_Compute);

class FTopLevelCreateBitmaskCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTopLevelCreateBitmaskCS);
	SHADER_USE_PARAMETER_STRUCT(FTopLevelCreateBitmaskCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, TopLevelGridResolution)

		// Grid data
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTopLevelGridData>, TopLevelGridBuffer)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FTopLevelGridBitmaskData>, RWTopLevelGridBitmaskBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FTopLevelCreateBitmaskCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridPipeline.usf", "TopLevelCreateBitmaskCS", SF_Compute);

void CreateTopLevelBitmask(
	FRDGBuilder& GraphBuilder,
	TArray<FViewInfo>& Views,
	FIntVector TopLevelGridResolution,
	FRDGBufferRef TopLevelGridBuffer,
	FRDGBufferRef& TopLevelGridBitmaskBuffer
)
{
	if (!HeterogeneousVolumes::EnableTopLevelBitmask())
	{
		TopLevelGridBitmaskBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FTopLevelGridBitmaskData));
	}
	else
	{
		FIntVector BitmaskGridResolution = FIntVector(
			FMath::DivideAndRoundUp(TopLevelGridResolution.X, 4),
			FMath::DivideAndRoundUp(TopLevelGridResolution.Y, 4),
			FMath::DivideAndRoundUp(TopLevelGridResolution.Z, 4)
		);
		uint32 TopLevelGridBitmaskBufferSize = BitmaskGridResolution.X * BitmaskGridResolution.Y * BitmaskGridResolution.Z;

		TopLevelGridBitmaskBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(FTopLevelGridBitmaskData), TopLevelGridBitmaskBufferSize),
			TEXT("HeterogeneousVolumes.BottomLevelGridBuffer")
		);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(TopLevelGridBitmaskBuffer), 0x0);

		FTopLevelCreateBitmaskCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FTopLevelCreateBitmaskCS::FParameters>();
		{
			PassParameters->TopLevelGridResolution = TopLevelGridResolution;
			PassParameters->TopLevelGridBuffer = GraphBuilder.CreateSRV(TopLevelGridBuffer);
			PassParameters->RWTopLevelGridBitmaskBuffer = GraphBuilder.CreateUAV(TopLevelGridBitmaskBuffer);
		}

		//FTopLevelCreateBitmaskCS::FPermutationDomain PermutationVector;
		const FGlobalShaderMap* GlobalShaderMap = Views[0].ShaderMap;
		TShaderRef<FTopLevelCreateBitmaskCS> ComputeShader = GlobalShaderMap->GetShader<FTopLevelCreateBitmaskCS>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("RenderTransmittanceTopLevelGridCS"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			ComputeShader,
			PassParameters,
			BitmaskGridResolution
		);
	}
}

class FBuildMajorantVoxelGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildMajorantVoxelGridCS);
	SHADER_USE_PARAMETER_STRUCT(FBuildMajorantVoxelGridCS, FGlobalShader);

	class FEnableIndirectionGrid : SHADER_PERMUTATION_BOOL("DIM_ENABLE_INDIRECTION_GRID");
	using FPermutationDomain = TShaderPermutationDomain<FEnableIndirectionGrid>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, TopLevelGridResolution)

		// Grid data
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTopLevelGridData>, TopLevelGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FTopLevelGridData>, IndirectionGridBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FScalarGridData>, ExtinctionGridBuffer)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FScalarGridData>, RWMajorantVoxelGridBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FBuildMajorantVoxelGridCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridPipeline.usf", "BuildMajorantVoxelGridCS", SF_Compute);

class FDownsampleMajorantVoxelGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDownsampleMajorantVoxelGridCS);
	SHADER_USE_PARAMETER_STRUCT(FDownsampleMajorantVoxelGridCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector, InputDimensions)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FScalarGridData>, InputBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FScalarGridData>, RWOutputBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FDownsampleMajorantVoxelGridCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridPipeline.usf", "DownsampleMajorantVoxelGridCS", SF_Compute);

class FCopyMaxIntoMajorantVoxelGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyMaxIntoMajorantVoxelGridCS);
	SHADER_USE_PARAMETER_STRUCT(FCopyMaxIntoMajorantVoxelGridCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FScalarGridData>, InputBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FScalarGridData>, RWMajorantVoxelGridBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FCopyMaxIntoMajorantVoxelGridCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridPipeline.usf", "CopyMaximumIntoMajorantVoxelGridCS", SF_Compute);


void BuildMajorantVoxelGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FIntVector& TopLevelGridResolution,
	FRDGBufferRef TopLevelGridBuffer,
	FRDGBufferRef IndirectionGridBuffer,
	FRDGBufferRef ExtinctionGridBuffer,
	FRDGBufferRef& MajorantVoxelGridBuffer
)
{
	if (!HeterogeneousVolumes::EnableMajorantGrid())
	{
		MajorantVoxelGridBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FScalarGridData));
	}
	else
	{
		uint32 MajorantVoxelGridBufferSize = TopLevelGridResolution.X * TopLevelGridResolution.Y * TopLevelGridResolution.Z;
		MajorantVoxelGridBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(FScalarGridData), MajorantVoxelGridBufferSize),
			TEXT("HeterogeneousVolumes.MajorantVoxelGridBuffer")
		);

		const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Scene->GetShaderPlatform());

		// Build Majorant Grid
		{
			FBuildMajorantVoxelGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildMajorantVoxelGridCS::FParameters>();
			{
				PassParameters->TopLevelGridResolution = TopLevelGridResolution;
				PassParameters->TopLevelGridBuffer = GraphBuilder.CreateSRV(TopLevelGridBuffer);
				PassParameters->IndirectionGridBuffer = GraphBuilder.CreateSRV(IndirectionGridBuffer);
				PassParameters->ExtinctionGridBuffer = GraphBuilder.CreateSRV(ExtinctionGridBuffer);
				PassParameters->RWMajorantVoxelGridBuffer = GraphBuilder.CreateUAV(MajorantVoxelGridBuffer);
			}

			FBuildMajorantVoxelGridCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FBuildMajorantVoxelGridCS::FEnableIndirectionGrid>(HeterogeneousVolumes::EnableIndirectionGrid());
			TShaderRef<FBuildMajorantVoxelGridCS> ComputeShader = GlobalShaderMap->GetShader<FBuildMajorantVoxelGridCS>(PermutationVector);

			FIntVector GroupCount = FIntVector(
				FMath::DivideAndRoundUp(TopLevelGridResolution.X, FBuildMajorantVoxelGridCS::GetThreadGroupSize3D()),
				FMath::DivideAndRoundUp(TopLevelGridResolution.Y, FBuildMajorantVoxelGridCS::GetThreadGroupSize3D()),
				FMath::DivideAndRoundUp(TopLevelGridResolution.Z, FBuildMajorantVoxelGridCS::GetThreadGroupSize3D())
			);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("BuildMajorantVoxelGridCS"),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				GroupCount
			);
		}

		bool bUseMax = CVarHeterogeneousVolumesEnableMajorantGridMax.GetValueOnRenderThread() != 0;
		if (bUseMax)
		{

			// Downsample majorant grid
			FIntVector InputDimensions = TopLevelGridResolution;
			FRDGBufferRef InputBuffer = MajorantVoxelGridBuffer;
			FRDGBufferRef OutputBuffer = nullptr;
			while (InputDimensions.X > 1 && InputDimensions.Y && InputDimensions.Z)
			{
				FIntVector OutputDimensions = FIntVector(
					FMath::DivideAndRoundUp(InputDimensions.X, FDownsampleMajorantVoxelGridCS::GetThreadGroupSize3D()),
					FMath::DivideAndRoundUp(InputDimensions.Y, FDownsampleMajorantVoxelGridCS::GetThreadGroupSize3D()),
					FMath::DivideAndRoundUp(InputDimensions.Z, FDownsampleMajorantVoxelGridCS::GetThreadGroupSize3D())
				);
				uint32 OutputBufferSize = OutputDimensions.X * OutputDimensions.Y * OutputDimensions.Z;
				OutputBuffer = GraphBuilder.CreateBuffer(
					FRDGBufferDesc::CreateStructuredDesc(sizeof(FScalarGridData), OutputBufferSize),
					TEXT("HeterogeneousVolumes.DownsampledBuffer")
				);

				FDownsampleMajorantVoxelGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDownsampleMajorantVoxelGridCS::FParameters>();
				{
					PassParameters->InputDimensions = InputDimensions;
					PassParameters->InputBuffer = GraphBuilder.CreateSRV(InputBuffer);
					PassParameters->RWOutputBuffer = GraphBuilder.CreateUAV(OutputBuffer);
				}

				TShaderRef<FDownsampleMajorantVoxelGridCS> ComputeShader = GlobalShaderMap->GetShader<FDownsampleMajorantVoxelGridCS>();
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("FDownsampleMajorantVoxelGridCS"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					PassParameters,
					TopLevelGridResolution
				);

				InputBuffer = OutputBuffer;
				InputDimensions = OutputDimensions;
			}

			// Copy maximum value into the first index of the majorant grid
			if (OutputBuffer)
			{
				FCopyMaxIntoMajorantVoxelGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCopyMaxIntoMajorantVoxelGridCS::FParameters>();
				{
					PassParameters->InputBuffer = GraphBuilder.CreateSRV(OutputBuffer);
					PassParameters->RWMajorantVoxelGridBuffer = GraphBuilder.CreateUAV(MajorantVoxelGridBuffer);
				}

				TShaderRef<FCopyMaxIntoMajorantVoxelGridCS> ComputeShader = GlobalShaderMap->GetShader<FCopyMaxIntoMajorantVoxelGridCS>();
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("FCopyMaxIntoMajorantVoxelGridCS"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					PassParameters,
					FIntVector(1)
				);
			}
		}
	}
}

struct FRenderDebugData
{
	FVector4f RayOrigin;
	FVector4f RayDirection;
	float TMin;
	float TMax;
	float Distance;
	FVector4f EstimateAndPdf;

	FMatrix44f ScreenToTranslatedWorld;
	FMatrix44f ClipToTranslatedWorld;
};

class FRenderTransmittanceWithVoxelGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderTransmittanceWithVoxelGridCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderTransmittanceWithVoxelGridCS, FGlobalShader);

	class FDebugMode : SHADER_PERMUTATION_INT("DEBUG_MODE", 9);
	using FPermutationDomain = TShaderPermutationDomain<FDebugMode>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		// Ray data
		SHADER_PARAMETER(int, bJitter)
		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(float, StepSize)
		SHADER_PARAMETER(int, MaxStepCount)

		// Voxel grids
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FOrthoVoxelGridUniformBufferParameters, OrthoGridUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFrustumVoxelGridUniformBufferParameters, FrustumGridUniformBuffer)

		// Dispatch data
		SHADER_PARAMETER(FIntVector, GroupCount)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWLightingTexture)
		// Debug
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FRenderDebugData>, RWDebugBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FRenderTransmittanceWithVoxelGridCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridRendering.usf", "RenderTransmittanceWithVoxelGridCS", SF_Compute);

void CalcViewBoundsAndMinimumVoxelSize(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FVoxelGridBuildOptions& BuildOptions,
	FBoxSphereBounds::Builder& TopLevelGridBoundsBuilder,
	float& MinimumVoxelSize
)
{
	MinimumVoxelSize = HeterogeneousVolumes::GetMinimumVoxelSizeOutsideFrustum();

	// Build view bounds
	FVector WorldCameraOrigin = View.ViewMatrices.GetViewOrigin();
	FBoxSphereBounds WorldCameraBounds(FSphere(WorldCameraOrigin, HeterogeneousVolumes::GetMaxTraceDistance()));

	float TanHalfFOV = HeterogeneousVolumes::CalcTanHalfFOV(View.FOV);
	float HalfWidth = View.ViewRect.Width() * 0.5 / HeterogeneousVolumes::GetDownsampleFactor();
	float PixelWidth = TanHalfFOV / HalfWidth;

	for (int32 MeshBatchIndex = 0; MeshBatchIndex < View.HeterogeneousVolumesMeshBatches.Num(); ++MeshBatchIndex)
	{
		// Only Niagara mesh particles bound to volume materials
		const FMeshBatch* Mesh = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex].Mesh;
		const FPrimitiveSceneProxy* PrimitiveSceneProxy = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex].Proxy;
		if (!ShouldRenderMeshBatchWithHeterogeneousVolumes(Mesh, PrimitiveSceneProxy, View.GetFeatureLevel()))
		{
			continue;
		}

		for (int32 VolumeIndex = 0; VolumeIndex < Mesh->Elements.Num(); ++VolumeIndex)
		{
			IHeterogeneousVolumeInterface* HeterogeneousVolume = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeIndex].UserData;
			//check(HeterogeneousVolume != nullptr);
			if (HeterogeneousVolume == nullptr)
			{
				continue;
			}

			// Only incorporate the primitive if it intersects with the camera bounding sphere where radius=MaxTraceDistance
			const FBoxSphereBounds& PrimitiveBounds = HeterogeneousVolume->GetBounds();
			if (View.ViewFrustum.IntersectBox(PrimitiveBounds.Origin, PrimitiveBounds.BoxExtent))
			{
				TopLevelGridBoundsBuilder += PrimitiveBounds;

				// Bandlimit minimum voxel size request with projected voxel size, based on shading rate
				FVector VoxelCenter = PrimitiveBounds.Origin;
				float Distance = FMath::Max(FVector(PrimitiveBounds.Origin - WorldCameraOrigin).Length() - PrimitiveBounds.BoxExtent.Length(), 0.0);
				float VoxelWidth = Distance * PixelWidth * BuildOptions.ShadingRateInFrustum;

				float PerVolumeMinimumVoxelSize = FMath::Max(VoxelWidth, HeterogeneousVolume->GetMinimumVoxelSize());
				MinimumVoxelSize = FMath::Min(PerVolumeMinimumVoxelSize, MinimumVoxelSize);
			}
		}
	}
}

void CalcGlobalBoundsAndMinimumVoxelSize(
	FRDGBuilder& GraphBuilder,
	const TArray<FViewInfo>& Views,
	const TSet<FVolumetricMeshBatch>& HeterogeneousVolumesMeshBatches,
	const FVoxelGridBuildOptions& BuildOptions,
	FBoxSphereBounds::Builder& TopLevelGridBoundsBuilder,
	float& GlobalMinimumVoxelSize
)
{
	GlobalMinimumVoxelSize = HeterogeneousVolumes::GetMinimumVoxelSizeOutsideFrustum();

	// Cycle through all Volume PrimitiveSceneProxies to collect bounds information and minimum voxel-size
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		float ViewMinimumVoxelSize = HeterogeneousVolumes::GetMinimumVoxelSizeOutsideFrustum();
		FBoxSphereBounds::Builder AggregatePrimitiveBoundsBuilder;

		// Build view bounds
		const FViewInfo& View = Views[ViewIndex];
		FVector WorldCameraOrigin = View.ViewMatrices.GetViewOrigin();
		FBoxSphereBounds WorldCameraBounds(FSphere(WorldCameraOrigin, HeterogeneousVolumes::GetMaxTraceDistance()));

		float TanHalfFOV = HeterogeneousVolumes::CalcTanHalfFOV(View.FOV);
		float HalfWidth = View.ViewRect.Width() * 0.5 / HeterogeneousVolumes::GetDownsampleFactor();
		float PixelWidth = TanHalfFOV / HalfWidth;

		for (auto MeshBatchIt = HeterogeneousVolumesMeshBatches.begin(); MeshBatchIt != HeterogeneousVolumesMeshBatches.end(); ++MeshBatchIt)
		{
			const FVolumetricMeshBatch& MeshBatch = *MeshBatchIt;
			const FMeshBatch* Mesh = MeshBatch.Mesh;
			const FPrimitiveSceneProxy* PrimitiveSceneProxy = MeshBatch.Proxy;

			if (!ShouldRenderMeshBatchWithHeterogeneousVolumes(Mesh, PrimitiveSceneProxy, View.GetFeatureLevel()))
			{
				continue;
			}

			for (int32 VolumeIndex = 0; VolumeIndex < Mesh->Elements.Num(); ++VolumeIndex)
			{
				IHeterogeneousVolumeInterface* HeterogeneousVolume = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeIndex].UserData;
				//check(HeterogeneousVolume != nullptr);
				if (HeterogeneousVolume == nullptr)
				{
					continue;
				}

				const FBoxSphereBounds& PrimitiveBounds = HeterogeneousVolume->GetBounds();
				AggregatePrimitiveBoundsBuilder += PrimitiveBounds;

				bool bIntersectsViewFrustum = View.ViewFrustum.IntersectBox(PrimitiveBounds.Origin, PrimitiveBounds.BoxExtent);
				float VoxelWidth = bIntersectsViewFrustum ? BuildOptions.ShadingRateInFrustum : BuildOptions.ShadingRateOutOfFrustum;
				if (BuildOptions.bUseProjectedPixelSizeForOrthoGrid)
				{
					FVector VoxelCenter = PrimitiveBounds.Origin;
					float Distance = FMath::Max(FVector(PrimitiveBounds.Origin - WorldCameraOrigin).Length() - PrimitiveBounds.SphereRadius, 0.0);
					VoxelWidth *= Distance * PixelWidth;
				}

				VoxelWidth = FMath::Max(VoxelWidth, HeterogeneousVolume->GetMinimumVoxelSize());
				ViewMinimumVoxelSize = FMath::Min(VoxelWidth, ViewMinimumVoxelSize);
			}
		}

		// When converting to pixel-space units, clamp per-view minimum voxel-size to in-frustum minimum
		if (AggregatePrimitiveBoundsBuilder.IsValid())
		{
			FBoxSphereBounds AggregatePrimitiveBounds(AggregatePrimitiveBoundsBuilder);
			if (BuildOptions.bUseProjectedPixelSizeForOrthoGrid && View.ViewFrustum.IntersectBox(AggregatePrimitiveBounds.Origin, AggregatePrimitiveBounds.BoxExtent))
			{
				ViewMinimumVoxelSize = FMath::Max(ViewMinimumVoxelSize, HeterogeneousVolumes::GetMinimumVoxelSizeInFrustum());
			}

			// Update global bounds and minimum voxel-size
			TopLevelGridBoundsBuilder += AggregatePrimitiveBounds;
			GlobalMinimumVoxelSize = FMath::Min(GlobalMinimumVoxelSize, ViewMinimumVoxelSize);
		}
	}
}

void ExtractFrustumVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer,
	FAdaptiveFrustumGridParameterCache& AdaptiveFrustumGridParameterCache
)
{
	const TRDGParameterStruct<FFrustumVoxelGridUniformBufferParameters>& Parameters = FrustumGridUniformBuffer->GetParameters();

	AdaptiveFrustumGridParameterCache.TopLevelGridWorldBoundsMin = Parameters->TopLevelGridWorldBoundsMin;
	AdaptiveFrustumGridParameterCache.TopLevelGridWorldBoundsMax = Parameters->TopLevelGridWorldBoundsMax;
	AdaptiveFrustumGridParameterCache.TopLevelGridResolution = Parameters->TopLevelFroxelGridResolution;
	AdaptiveFrustumGridParameterCache.VoxelDimensions = Parameters->VoxelDimensions;
	AdaptiveFrustumGridParameterCache.bUseFrustumGrid = Parameters->bUseFrustumGrid;
	AdaptiveFrustumGridParameterCache.NearPlaneDepth = Parameters->NearPlaneDepth;
	AdaptiveFrustumGridParameterCache.FarPlaneDepth = Parameters->FarPlaneDepth;
	AdaptiveFrustumGridParameterCache.TanHalfFOV = Parameters->TanHalfFOV;


	AdaptiveFrustumGridParameterCache.WorldToClip = Parameters->WorldToClip;
	AdaptiveFrustumGridParameterCache.ClipToWorld = Parameters->ClipToWorld;
	AdaptiveFrustumGridParameterCache.WorldToView = Parameters->WorldToView;
	AdaptiveFrustumGridParameterCache.ViewToWorld = Parameters->ViewToWorld;
	AdaptiveFrustumGridParameterCache.ViewToClip = Parameters->ViewToClip;
	AdaptiveFrustumGridParameterCache.ClipToView = Parameters->ClipToView;

	for (int i = 0; i < 6; ++i)
	{
		AdaptiveFrustumGridParameterCache.ViewFrustumPlanes[i] = Parameters->ViewFrustumPlanes[i];
	}

	GraphBuilder.QueueBufferExtraction(Parameters->TopLevelFroxelGridBuffer->GetParent(), &AdaptiveFrustumGridParameterCache.TopLevelGridBuffer);
	GraphBuilder.QueueBufferExtraction(Parameters->ExtinctionFroxelGridBuffer->GetParent(), &AdaptiveFrustumGridParameterCache.ExtinctionGridBuffer);
	GraphBuilder.QueueBufferExtraction(Parameters->EmissionFroxelGridBuffer->GetParent(), &AdaptiveFrustumGridParameterCache.EmissionGridBuffer);
	GraphBuilder.QueueBufferExtraction(Parameters->ScatteringFroxelGridBuffer->GetParent(), &AdaptiveFrustumGridParameterCache.ScatteringGridBuffer);
}

void RegisterExternalFrustumVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FAdaptiveFrustumGridParameterCache& AdaptiveFrustumGridParameterCache,
	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer
)
{
	FFrustumVoxelGridUniformBufferParameters* UniformBufferParameters = GraphBuilder.AllocParameters<FFrustumVoxelGridUniformBufferParameters>();
	{
		UniformBufferParameters->WorldToClip = AdaptiveFrustumGridParameterCache.WorldToClip;
		UniformBufferParameters->ClipToWorld = AdaptiveFrustumGridParameterCache.ClipToWorld;

		UniformBufferParameters->WorldToView = AdaptiveFrustumGridParameterCache.WorldToView;
		UniformBufferParameters->ViewToWorld = AdaptiveFrustumGridParameterCache.ViewToWorld;

		UniformBufferParameters->ViewToClip = AdaptiveFrustumGridParameterCache.ViewToClip;
		UniformBufferParameters->ClipToView = AdaptiveFrustumGridParameterCache.ClipToView;

		UniformBufferParameters->TopLevelGridWorldBoundsMin = AdaptiveFrustumGridParameterCache.TopLevelGridWorldBoundsMin;
		UniformBufferParameters->TopLevelGridWorldBoundsMax = AdaptiveFrustumGridParameterCache.TopLevelGridWorldBoundsMax;
		UniformBufferParameters->TopLevelFroxelGridResolution = AdaptiveFrustumGridParameterCache.TopLevelGridResolution;
		UniformBufferParameters->VoxelDimensions = AdaptiveFrustumGridParameterCache.TopLevelGridResolution;
		UniformBufferParameters->bUseFrustumGrid = AdaptiveFrustumGridParameterCache.bUseFrustumGrid;
		UniformBufferParameters->NearPlaneDepth = AdaptiveFrustumGridParameterCache.NearPlaneDepth;
		UniformBufferParameters->FarPlaneDepth = AdaptiveFrustumGridParameterCache.FarPlaneDepth;
		UniformBufferParameters->TanHalfFOV = AdaptiveFrustumGridParameterCache.TanHalfFOV;

		// Frustum assignment
		for (int i = 0; i < 6; ++i)
		{
			UniformBufferParameters->ViewFrustumPlanes[i] = AdaptiveFrustumGridParameterCache.ViewFrustumPlanes[i];
		}

		UniformBufferParameters->TopLevelFroxelGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveFrustumGridParameterCache.TopLevelGridBuffer));
		UniformBufferParameters->ExtinctionFroxelGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveFrustumGridParameterCache.ExtinctionGridBuffer));
		UniformBufferParameters->EmissionFroxelGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveFrustumGridParameterCache.EmissionGridBuffer));
		UniformBufferParameters->ScatteringFroxelGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveFrustumGridParameterCache.ScatteringGridBuffer));
	}
	FrustumGridUniformBuffer = GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
}

TRDGUniformBufferRef <FFrustumVoxelGridUniformBufferParameters> CreateEmptyFrustumVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder
)
{
	FFrustumVoxelGridUniformBufferParameters* UniformBufferParameters = GraphBuilder.AllocParameters<FFrustumVoxelGridUniformBufferParameters>();
	{
		UniformBufferParameters->WorldToClip = FMatrix44f::Identity;
		UniformBufferParameters->ClipToWorld = FMatrix44f::Identity;
		UniformBufferParameters->WorldToView = FMatrix44f::Identity;
		UniformBufferParameters->ViewToWorld = FMatrix44f::Identity;
		UniformBufferParameters->ViewToClip = FMatrix44f::Identity;
		UniformBufferParameters->ClipToView = FMatrix44f::Identity;

		UniformBufferParameters->TopLevelFroxelGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FTopLevelGridData)));
		UniformBufferParameters->ExtinctionFroxelGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FScalarGridData)));
		UniformBufferParameters->EmissionFroxelGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVectorGridData)));
		UniformBufferParameters->ScatteringFroxelGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVectorGridData)));

		UniformBufferParameters->TopLevelGridWorldBoundsMin = FVector3f(0);
		UniformBufferParameters->TopLevelGridWorldBoundsMax = FVector3f(0);
		UniformBufferParameters->TopLevelFroxelGridResolution = FIntVector(0);
		UniformBufferParameters->VoxelDimensions = FIntVector(0);
		UniformBufferParameters->bUseFrustumGrid = false;
		UniformBufferParameters->NearPlaneDepth = 0.0;
		UniformBufferParameters->FarPlaneDepth = 0.0;
		UniformBufferParameters->TanHalfFOV = 1.0;
	}
	return GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
}

void ClipNearFarDistances(const FViewInfo& View, const FBoxSphereBounds& TopLevelGridBounds, float& NearPlaneDistance, float& FarPlaneDistance)
{
	// Determine near and far planes, in world-space
	float d = -FVector::DotProduct(View.GetViewDirection(), View.ViewLocation);

	// Analyze the input volumes and determine the near/far extents
	FVector Center = TopLevelGridBounds.GetSphere().Center;

	// Project center onto the camera view-plane
	float SignedDistance = FVector::DotProduct(View.GetViewDirection(), Center) + d;

	float Radius = TopLevelGridBounds.GetSphere().W;
	float NearDistance = SignedDistance - Radius;
	float FarDistance = SignedDistance + Radius;

	if (NearPlaneDistance < 0.0)
	{
		NearPlaneDistance = NearDistance;
	}

	if (FarPlaneDistance < 0.0)
	{
		FarPlaneDistance = FarDistance;
	}

	NearPlaneDistance = FMath::Max(NearPlaneDistance, 0.01);
	FarPlaneDistance = FMath::Max(FarPlaneDistance, NearDistance + 1.0);
}

void CalculateTopLevelGridResolution(
	FBoxSphereBounds TopLevelGridBounds,
	float GlobalMinimumVoxelSize,
	FIntVector& TopLevelGridResolution
)
{
	// Bound Top-level grid resolution to cover fully allocated child grids at minimum voxel size
	FIntVector CombinedChildGridResolution = FIntVector(HeterogeneousVolumes::GetBottomLevelGridResolution());
	if (HeterogeneousVolumes::EnableIndirectionGrid())
	{
		CombinedChildGridResolution *= HeterogeneousVolumes::GetIndirectionGridResolution();
	}

	FVector TopLevelGridVoxelSize = FVector(CombinedChildGridResolution) * GlobalMinimumVoxelSize;
	FVector TopLevelGridResolutionAsFloat = (TopLevelGridBounds.BoxExtent * 2.0) / TopLevelGridVoxelSize;

	TopLevelGridResolution.X = FMath::CeilToInt(TopLevelGridResolutionAsFloat.X);
	TopLevelGridResolution.Y = FMath::CeilToInt(TopLevelGridResolutionAsFloat.Y);
	TopLevelGridResolution.Z = FMath::CeilToInt(TopLevelGridResolutionAsFloat.Z);

	// Clamp to a moderate limit to also handle indirection grid allocation
	TopLevelGridResolution.X = FMath::Clamp(TopLevelGridResolution.X, 1, 128);
	TopLevelGridResolution.Y = FMath::Clamp(TopLevelGridResolution.Y, 1, 128);
	TopLevelGridResolution.Z = FMath::Clamp(TopLevelGridResolution.Z, 1, 256);
}

void CalculateVoxelSize(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	/*const*/ TArray<FViewInfo>& Views,
	const TSet<FVolumetricMeshBatch>& HeterogeneousVolumesMeshBatches,
	const FVoxelGridBuildOptions& BuildOptions,
	FBoxSphereBounds TopLevelGridBounds,
	FIntVector TopLevelGridResolution,
	FRDGBufferRef& TopLevelGridBuffer
)
{
	TopLevelGridBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FTopLevelGridData), TopLevelGridResolution.X * TopLevelGridResolution.Y * TopLevelGridResolution.Z),
		TEXT("HeterogeneousVolumes.TopLevelGridBuffer")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(TopLevelGridBuffer), 0xFFFFFFF8);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		// Build view
		const FViewInfo& View = Views[ViewIndex];
		FVector WorldCameraOrigin = View.ViewMatrices.GetViewOrigin();
		FBoxSphereBounds WorldCameraBounds(FSphere(WorldCameraOrigin, HeterogeneousVolumes::GetMaxTraceDistance()));

		for (auto MeshBatchIt = HeterogeneousVolumesMeshBatches.begin(); MeshBatchIt != HeterogeneousVolumesMeshBatches.end(); ++MeshBatchIt)
		{
			const FVolumetricMeshBatch& MeshBatch = *MeshBatchIt;
			const FMeshBatch* Mesh = MeshBatch.Mesh;
			const FPrimitiveSceneProxy* PrimitiveSceneProxy = MeshBatch.Proxy;

			for (int32 VolumeIndex = 0; VolumeIndex < Mesh->Elements.Num(); ++VolumeIndex)
			{
				IHeterogeneousVolumeInterface* HeterogeneousVolume = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeIndex].UserData;
				//check(HeterogeneousVolume != nullptr);
				if (HeterogeneousVolume == nullptr)
				{
					continue;
				}

				const FBoxSphereBounds& PrimitiveBounds = HeterogeneousVolume->GetBounds();
				FTopLevelGridCalculateVoxelSize::FParameters* PassParameters = GraphBuilder.AllocParameters<FTopLevelGridCalculateVoxelSize::FParameters>();
				{
					PassParameters->View = View.ViewUniformBuffer;

					PassParameters->TopLevelGridResolution = TopLevelGridResolution;
					PassParameters->TopLevelGridWorldBoundsMin = FVector3f(TopLevelGridBounds.Origin - TopLevelGridBounds.BoxExtent);
					PassParameters->TopLevelGridWorldBoundsMax = FVector3f(TopLevelGridBounds.Origin + TopLevelGridBounds.BoxExtent);

					PassParameters->PrimitiveWorldBoundsMin = FVector3f(PrimitiveBounds.Origin - PrimitiveBounds.BoxExtent);
					PassParameters->PrimitiveWorldBoundsMax = FVector3f(PrimitiveBounds.Origin + PrimitiveBounds.BoxExtent);

					PassParameters->ShadingRateInFrustum = BuildOptions.ShadingRateInFrustum;
					PassParameters->ShadingRateOutOfFrustum = BuildOptions.ShadingRateOutOfFrustum;
					PassParameters->MinVoxelSizeInFrustum = FMath::Max(HeterogeneousVolume->GetMinimumVoxelSize(), HeterogeneousVolumes::GetMinimumVoxelSizeInFrustum());
					PassParameters->MinVoxelSizeOutOfFrustum = HeterogeneousVolumes::GetMinimumVoxelSizeOutsideFrustum();
					PassParameters->DownsampleFactor = HeterogeneousVolumes::GetDownsampleFactor();
					PassParameters->bUseProjectedPixelSize = BuildOptions.bUseProjectedPixelSizeForOrthoGrid;

					PassParameters->RWTopLevelGridBuffer = GraphBuilder.CreateUAV(TopLevelGridBuffer);
				}

				FIntVector GroupCount;
				GroupCount.X = FMath::DivideAndRoundUp(TopLevelGridResolution.X, FTopLevelGridCalculateVoxelSize::GetThreadGroupSize3D());
				GroupCount.Y = FMath::DivideAndRoundUp(TopLevelGridResolution.Y, FTopLevelGridCalculateVoxelSize::GetThreadGroupSize3D());
				GroupCount.Z = FMath::DivideAndRoundUp(TopLevelGridResolution.Z, FTopLevelGridCalculateVoxelSize::GetThreadGroupSize3D());

				FTopLevelGridCalculateVoxelSize::FPermutationDomain PermutationVector;
				TShaderRef<FTopLevelGridCalculateVoxelSize> ComputeShader = View.ShaderMap->GetShader<FTopLevelGridCalculateVoxelSize>(PermutationVector);
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("TopLevelGridCalculateVoxelSize"),
					ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
					ComputeShader,
					PassParameters,
					GroupCount
				);
			}
		}
	}
}

void MarkTopLevelGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	FBoxSphereBounds TopLevelGridBounds,
	FIntVector TopLevelGridResolution,
	FRDGBufferRef& TopLevelGridBuffer,
	FRDGBufferRef& IndirectionGridBuffer
)
{
	int32 IndirectionGridBufferSize = TopLevelGridResolution.X * TopLevelGridResolution.Y * TopLevelGridResolution.Z * 16 * sizeof(FTopLevelGridData);
	IndirectionGridBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FTopLevelGridData), IndirectionGridBufferSize),
		TEXT("HeterogeneousVolumes.OrthoGrid.IndirectionGridBuffer")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(IndirectionGridBuffer), 0xFFFFFFF8);

	FRDGBufferRef IndirectionGridAllocatorBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 2),
		TEXT("HeterogeneousVolume.OrthoGrid.IndirectionGridAllocatorBuffer")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(IndirectionGridAllocatorBuffer, PF_R32_UINT), 0);

	FAllocateBottomLevelGrid::FParameters* PassParameters = GraphBuilder.AllocParameters<FAllocateBottomLevelGrid::FParameters>();
	{
		//PassParameters->View = View.ViewUniformBuffer;
		PassParameters->TopLevelGridResolution = TopLevelGridResolution;
		PassParameters->TopLevelGridWorldBoundsMin = FVector3f(TopLevelGridBounds.Origin - TopLevelGridBounds.BoxExtent);
		PassParameters->TopLevelGridWorldBoundsMax = FVector3f(TopLevelGridBounds.Origin + TopLevelGridBounds.BoxExtent);

		PassParameters->MaxVoxelResolution = HeterogeneousVolumes::GetBottomLevelGridResolution();
		PassParameters->bSampleAtVertices = HeterogeneousVolumes::EnableLinearInterpolation();

		PassParameters->RWTopLevelGridBuffer = GraphBuilder.CreateUAV(TopLevelGridBuffer);

		// Indirection Grid
		PassParameters->IndirectionGridBufferSize = IndirectionGridBufferSize;
		PassParameters->RWIndirectionGridBuffer = GraphBuilder.CreateUAV(IndirectionGridBuffer);
		PassParameters->RWIndirectionGridAllocatorBuffer = GraphBuilder.CreateUAV(IndirectionGridAllocatorBuffer, PF_R32_UINT);
	}

	FIntVector GroupCount;
	GroupCount.X = FMath::DivideAndRoundUp(TopLevelGridResolution.X, FAllocateBottomLevelGrid::GetThreadGroupSize3D());
	GroupCount.Y = FMath::DivideAndRoundUp(TopLevelGridResolution.Y, FAllocateBottomLevelGrid::GetThreadGroupSize3D());
	GroupCount.Z = FMath::DivideAndRoundUp(TopLevelGridResolution.Z, FAllocateBottomLevelGrid::GetThreadGroupSize3D());

	const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Scene->GetShaderPlatform());

	FAllocateBottomLevelGrid::FPermutationDomain PermutationVector;
	PermutationVector.Set<FAllocateBottomLevelGrid::FEnableIndirectionGrid>(HeterogeneousVolumes::EnableIndirectionGrid());
	TShaderRef<FAllocateBottomLevelGrid> ComputeShader = GlobalShaderMap->GetShader<FAllocateBottomLevelGrid>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("AllocateBottomLevelGrid"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

void GenerateRasterTiles(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	bool bEnableIndirectionGrid,
	FIntVector TopLevelGridResolution,
	FRDGBufferRef TopLevelGridBuffer,
	FRDGBufferRef& RasterTileBuffer,
	FRDGBufferRef& RasterTileAllocatorBuffer
)
{
	const uint32 RasterTileVoxelResolution = HeterogeneousVolumes::GetBottomLevelGridResolution();
	uint32 TileFactor = HeterogeneousVolumes::EnableIndirectionGrid() ? 64 : 1;
	uint32 MaxNumRasterTiles = TileFactor * TopLevelGridResolution.X * TopLevelGridResolution.Y * TopLevelGridResolution.Z;
	RasterTileBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FRasterTileData), MaxNumRasterTiles),
		TEXT("HeterogeneousVolumes.OrthoGrid.RasterTileBuffer")
	);

	RasterTileAllocatorBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1),
		TEXT("HeterogeneousVolume.OrthoGrid.RasterTileAllocatorBuffer")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(RasterTileAllocatorBuffer, PF_R32_UINT), 0);

	{
		FGenerateRasterTiles::FParameters* PassParameters = GraphBuilder.AllocParameters<FGenerateRasterTiles::FParameters>();
		{
			PassParameters->TopLevelGridResolution = TopLevelGridResolution;
			PassParameters->TopLevelGridBuffer = GraphBuilder.CreateSRV(TopLevelGridBuffer);

			PassParameters->RasterTileVoxelResolution = RasterTileVoxelResolution;
			PassParameters->MaxNumRasterTiles = MaxNumRasterTiles;

			PassParameters->RWRasterTileAllocatorBuffer = GraphBuilder.CreateUAV(RasterTileAllocatorBuffer, PF_R32_UINT);
			PassParameters->RWRasterTileBuffer = GraphBuilder.CreateUAV(RasterTileBuffer);
		}

		FIntVector GroupCount;
		GroupCount.X = FMath::DivideAndRoundUp(TopLevelGridResolution.X, FTopLevelGridCalculateVoxelSize::GetThreadGroupSize3D());
		GroupCount.Y = FMath::DivideAndRoundUp(TopLevelGridResolution.Y, FTopLevelGridCalculateVoxelSize::GetThreadGroupSize3D());
		GroupCount.Z = FMath::DivideAndRoundUp(TopLevelGridResolution.Z, FTopLevelGridCalculateVoxelSize::GetThreadGroupSize3D());

		const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Scene->GetShaderPlatform());

		FGenerateRasterTiles::FPermutationDomain PermutationVector;
		PermutationVector.Set<FGenerateRasterTiles::FEnableIndirectionGrid>(bEnableIndirectionGrid);
		TShaderRef<FGenerateRasterTiles> ComputeShader = GlobalShaderMap->GetShader<FGenerateRasterTiles>(PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("GenerateRasterTiles"),
			ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
			ComputeShader,
			PassParameters,
			GroupCount
		);
	}
}

void CalculateTopLevelGridResolutionForFrustumGrid(
	const FViewInfo& View,
	float MinimumVoxelSize,
	float NearPlaneDistance,
	float& FarPlaneDistance,
	FIntVector& TopLevelGridResolution
)
{
	// Determine top-level grid resolution
	// Build a grid where the resolution is proportional to a bottom-level grid that is using all available memory
	int32 BottomLevelGridResolution = HeterogeneousVolumes::GetBottomLevelGridResolution();

	float ShadingRate = HeterogeneousVolumes::GetShadingRateForFrustumGrid();
	int32 Width = FMath::CeilToInt32(View.ViewRect.Width() / ShadingRate);
	int32 Height = FMath::CeilToInt32(View.ViewRect.Height() / ShadingRate);

	// Use view frustum field-of-view and preferred minimum voxel size to find optimal far-plane distance
	if (HeterogeneousVolumes::EnableOrthoVoxelGrid() &&
		HeterogeneousVolumes::EnableFarPlaneAutoTransition() &&
		ShadingRate >= HeterogeneousVolumes::GetShadingRateForOrthoGrid())
	{
		float GlobalMinimumVoxelSize = FMath::Max(MinimumVoxelSize, HeterogeneousVolumes::GetMinimumVoxelSizeInFrustum());

		float Theta = FMath::DegreesToRadians(View.FOV * 0.5);
		float PixelTheta = (2.0 * Theta) / Width;
		float TanOfPixel = FMath::Tan(PixelTheta);
		float OptimalFarPlaneDepth = GlobalMinimumVoxelSize / TanOfPixel;
		FarPlaneDistance = FMath::Min(FarPlaneDistance, OptimalFarPlaneDepth);
	}

	// Depth slices should not be smaller than the declared minimum voxel size
	int32 MaxDepth = FMath::CeilToInt32((FarPlaneDistance - NearPlaneDistance) / MinimumVoxelSize);
	int32 Depth = FMath::Min(FMath::CeilToInt32(HeterogeneousVolumes::GetDepthSliceCountForFrustumGrid() / ShadingRate), MaxDepth);

	TopLevelGridResolution = FIntVector(Width, Height, Depth);
	TopLevelGridResolution.X = FMath::Max(FMath::DivideAndRoundUp(TopLevelGridResolution.X, BottomLevelGridResolution), 1);
	TopLevelGridResolution.Y = FMath::Max(FMath::DivideAndRoundUp(TopLevelGridResolution.Y, BottomLevelGridResolution), 1);
	TopLevelGridResolution.Z = FMath::Max(FMath::DivideAndRoundUp(TopLevelGridResolution.Z, BottomLevelGridResolution), 1);
}

void MarkTopLevelGridVoxelsForFrustumGrid(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FIntVector TopLevelGridResolution,
	float NearPlaneDistance,
	float FarPlaneDistance,
	const FMatrix& ViewToWorld,
	FRDGBufferRef& TopLevelGridBuffer
)
{
	int32 TopLevelVoxelCount = TopLevelGridResolution.X * TopLevelGridResolution.Y * TopLevelGridResolution.Z;

	TopLevelGridBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FTopLevelGridData), TopLevelVoxelCount),
		TEXT("HeterogeneousVolumes.FrustumGrid.TopLevelGridBuffer")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(TopLevelGridBuffer), 0xFFFFFFF8);

	for (int32 MeshBatchIndex = 0; MeshBatchIndex < View.HeterogeneousVolumesMeshBatches.Num(); ++MeshBatchIndex)
	{
		const FMeshBatch* Mesh = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex].Mesh;
		const FPrimitiveSceneProxy* PrimitiveSceneProxy = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex].Proxy;

		for (int32 VolumeIndex = 0; VolumeIndex < Mesh->Elements.Num(); ++VolumeIndex)
		{
			IHeterogeneousVolumeInterface* HeterogeneousVolume = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeIndex].UserData;
			//check(HeterogeneousVolume != nullptr);
			if (HeterogeneousVolume == nullptr)
			{
				continue;
			}

			const FBoxSphereBounds& PrimitiveBounds = HeterogeneousVolume->GetBounds();

			FMarkTopLevelGridVoxelsForFrustumGrid::FParameters* PassParameters = GraphBuilder.AllocParameters<FMarkTopLevelGridVoxelsForFrustumGrid::FParameters>();
			{
				PassParameters->View = View.ViewUniformBuffer;

				PassParameters->PrimitiveWorldBoundsMin = FVector3f(PrimitiveBounds.Origin - PrimitiveBounds.BoxExtent);
				PassParameters->PrimitiveWorldBoundsMax = FVector3f(PrimitiveBounds.Origin + PrimitiveBounds.BoxExtent);

				PassParameters->ViewToWorld = FMatrix44f(ViewToWorld);
				PassParameters->TanHalfFOV = HeterogeneousVolumes::CalcTanHalfFOV(View.FOV);
				PassParameters->NearPlaneDepth = NearPlaneDistance;
				PassParameters->FarPlaneDepth = FarPlaneDistance;

				PassParameters->VoxelDimensions = TopLevelGridResolution;
				PassParameters->TopLevelGridResolution = TopLevelGridResolution;

				PassParameters->RWTopLevelGridBuffer = GraphBuilder.CreateUAV(TopLevelGridBuffer);
			}

			FIntVector GroupCount;
			GroupCount.X = FMath::DivideAndRoundUp(TopLevelGridResolution.X, FMarkTopLevelGridVoxelsForFrustumGrid::GetThreadGroupSize3D());
			GroupCount.Y = FMath::DivideAndRoundUp(TopLevelGridResolution.Y, FMarkTopLevelGridVoxelsForFrustumGrid::GetThreadGroupSize3D());
			GroupCount.Z = FMath::DivideAndRoundUp(TopLevelGridResolution.Z, FMarkTopLevelGridVoxelsForFrustumGrid::GetThreadGroupSize3D());

			FMarkTopLevelGridVoxelsForFrustumGrid::FPermutationDomain PermutationVector;
			TShaderRef<FMarkTopLevelGridVoxelsForFrustumGrid> ComputeShader = View.ShaderMap->GetShader<FMarkTopLevelGridVoxelsForFrustumGrid>(PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("MarkTopLevelGridVoxelsForFrustumGrid"),
				ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
				ComputeShader,
				PassParameters,
				GroupCount
			);
		}
	}
}

void RasterizeVolumesIntoFrustumVoxelGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FVoxelGridBuildOptions& BuildOptions,
	// Transform data
	FMatrix& ViewToWorld,
	float NearPlaneDistance,
	float FarPlaneDistance,
	// Raster tile
	FRDGBufferRef RasterTileBuffer,
	FRDGBufferRef RasterTileAllocatorBuffer,
	// Top-level grid
	FBoxSphereBounds TopLevelGridBounds,
	FIntVector TopLevelGridResolution,
	FRDGBufferRef& TopLevelGridBuffer,
	// Bottom-level grid
	FRDGBufferRef& ExtinctionGridBuffer,
	FRDGBufferRef& EmissionGridBuffer,
	FRDGBufferRef& ScatteringGridBuffer
)
{
	//Setup indirect dispatch
	FRDGBufferRef RasterizeBottomLevelGridIndirectArgsBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1),
		TEXT("HeterogeneousVolume.FrustumGrid.RasterizeBottomLevelGridIndirectArgs")
	);

	FSetRasterizeBottomLevelGridIndirectArgs::FParameters* IndirectArgsPassParameters = GraphBuilder.AllocParameters<FSetRasterizeBottomLevelGridIndirectArgs::FParameters>();
	{
		IndirectArgsPassParameters->MaxDispatchThreadGroupsPerDimension = GRHIMaxDispatchThreadGroupsPerDimension;
		IndirectArgsPassParameters->RasterTileAllocatorBuffer = GraphBuilder.CreateSRV(RasterTileAllocatorBuffer, PF_R32_UINT);
		IndirectArgsPassParameters->RWRasterizeBottomLevelGridIndirectArgsBuffer = GraphBuilder.CreateUAV(RasterizeBottomLevelGridIndirectArgsBuffer, PF_R32_UINT);
	}

	FIntVector GroupCount(1, 1, 1);
	TShaderRef<FSetRasterizeBottomLevelGridIndirectArgs> ComputeShader = View.ShaderMap->GetShader<FSetRasterizeBottomLevelGridIndirectArgs>();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("SetRasterizeBottomLevelGridIndirectArgs"),
		ERDGPassFlags::Compute,
		ComputeShader,
		IndirectArgsPassParameters,
		GroupCount
	);

	// Pre-allocate bottom-level voxel grid pool based on user-defined budget
	int32 MaxBottomLevelVoxelCount = (HeterogeneousVolumes::GetMaxBottomLevelMemoryInMegabytesForFrustumGrid() * 1e6) / sizeof(FVectorGridData);
	int32 BottomLevelGridBufferSize = MaxBottomLevelVoxelCount;

	ExtinctionGridBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FScalarGridData), BottomLevelGridBufferSize),
		TEXT("HeterogeneousVolumes.FrustumGrid.ExtinctionGridBuffer")
	);
	EmissionGridBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FVectorGridData), BottomLevelGridBufferSize),
		TEXT("HeterogeneousVolumes.FrustumGrid.EmissionGridBuffer")
	);
	ScatteringGridBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FVectorGridData), BottomLevelGridBufferSize),
		TEXT("HeterogeneousVolumes.FrustumGrid.ScatteringGridBuffer")
	);

	FRDGBufferRef BottomLevelGridAllocatorBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 2),
		TEXT("HeterogeneousVolume.FrustumGrid.BottomLevelGridAllocatorBuffer")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BottomLevelGridAllocatorBuffer, PF_R32_UINT), 0);

	// Rasterize volumes into bottom-level grid
	for (int32 MeshBatchIndex = 0; MeshBatchIndex < View.HeterogeneousVolumesMeshBatches.Num(); ++MeshBatchIndex)
	{
		const FMeshBatch* Mesh = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex].Mesh;
		const FPrimitiveSceneProxy* PrimitiveSceneProxy = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex].Proxy;
		const FMaterialRenderProxy* MaterialRenderProxy = Mesh->MaterialRenderProxy;
		if (!ShouldRenderMeshBatchWithHeterogeneousVolumes(Mesh, PrimitiveSceneProxy, View.GetFeatureLevel()))
		{
			continue;
		}

		for (int32 VolumeIndex = 0; VolumeIndex < Mesh->Elements.Num(); ++VolumeIndex)
		{
			const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeIndex].UserData;
			//check(HeterogeneousVolumeInterface != nullptr);
			if (HeterogeneousVolumeInterface == nullptr)
			{
				continue;
			}

			const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
			const int32 PrimitiveId = PrimitiveSceneInfo->GetIndex();
			const FBoxSphereBounds LocalBoxSphereBounds = PrimitiveSceneProxy->GetLocalBounds();
			const FBoxSphereBounds PrimitiveBounds = PrimitiveSceneProxy->GetBounds();
			const FMaterial& Material = MaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);

			FRasterizeBottomLevelFrustumGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRasterizeBottomLevelFrustumGridCS::FParameters>();
			{
				// Scene data
				PassParameters->View = View.ViewUniformBuffer;
				PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
				PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));

				// Primitive data
				PassParameters->PrimitiveWorldBoundsMin = FVector3f(PrimitiveBounds.Origin - PrimitiveBounds.BoxExtent);
				PassParameters->PrimitiveWorldBoundsMax = FVector3f(PrimitiveBounds.Origin + PrimitiveBounds.BoxExtent);
				
				// TODO: Convert to relative-local space
				//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
				//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
				FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
				FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
				PassParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
				PassParameters->WorldToLocal = FMatrix44f(PassParameters->LocalToWorld.Inverse());

				FMatrix LocalToInstance = InstanceToLocal.Inverse();
				FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(FTransform(LocalToInstance));
				PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
				PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);
				PassParameters->PrimitiveId = PrimitiveId;

				// Volume data
				PassParameters->TopLevelGridResolution = TopLevelGridResolution;
				PassParameters->VoxelDimensions = TopLevelGridResolution;
				PassParameters->ViewToWorld = FMatrix44f(ViewToWorld);
				PassParameters->TanHalfFOV = HeterogeneousVolumes::CalcTanHalfFOV(View.FOV);
				PassParameters->NearPlaneDepth = NearPlaneDistance;
				PassParameters->FarPlaneDepth = FarPlaneDistance;

				// Sampling data
				PassParameters->bJitter = BuildOptions.bJitter;
				FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
				PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

				// Raster tile data
				PassParameters->RasterTileAllocatorBuffer = GraphBuilder.CreateSRV(RasterTileAllocatorBuffer, PF_R32_UINT);
				PassParameters->RasterTileBuffer = GraphBuilder.CreateSRV(RasterTileBuffer);

				// Indirect args
				PassParameters->IndirectArgs = RasterizeBottomLevelGridIndirectArgsBuffer;

				// Grid data
				PassParameters->RWBottomLevelGridAllocatorBuffer = GraphBuilder.CreateUAV(BottomLevelGridAllocatorBuffer, PF_R32_UINT);
				PassParameters->RWTopLevelGridBuffer = GraphBuilder.CreateUAV(TopLevelGridBuffer);

				PassParameters->RWExtinctionGridBuffer = GraphBuilder.CreateUAV(ExtinctionGridBuffer);
				PassParameters->RWEmissionGridBuffer = GraphBuilder.CreateUAV(EmissionGridBuffer);
				PassParameters->RWScatteringGridBuffer = GraphBuilder.CreateUAV(ScatteringGridBuffer);
				PassParameters->BottomLevelGridBufferSize = BottomLevelGridBufferSize;
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("FrustumGrid.RasterizeBottomLevelGrid"),
				PassParameters,
				ERDGPassFlags::Compute,
				[PassParameters, Scene, &View, MaterialRenderProxy, &Material](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
				{
					FRasterizeBottomLevelFrustumGridCS::FPermutationDomain PermutationVector;
					TShaderRef<FRasterizeBottomLevelFrustumGridCS> ComputeShader = Material.GetShader<FRasterizeBottomLevelFrustumGridCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);

					if (!ComputeShader.IsNull())
					{
						ClearUnusedGraphResources(ComputeShader, PassParameters);

						FMeshDrawShaderBindings ShaderBindings;
						UE::MeshPassUtils::SetupComputeBindings(ComputeShader, Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, Material, ShaderBindings);

						UE::MeshPassUtils::DispatchIndirect(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, PassParameters->IndirectArgs->GetIndirectRHICallBuffer(), 0);
					}
				}
			);
		}
	}
}

void BuildFrustumVoxelGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FVoxelGridBuildOptions& BuildOptions,
	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer
)
{
	if (!ShouldRenderHeterogeneousVolumesForView(View) || !HeterogeneousVolumes::EnableFrustumVoxelGrid() || !BuildOptions.bBuildFrustumGrid)
	{
		FrustumGridUniformBuffer = CreateEmptyFrustumVoxelGridUniformBuffer(GraphBuilder);
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "Frustum Grid Build");

	// Determine the minimum voxel size for the scene, based on screen projection or user-defined minima
	FBoxSphereBounds::Builder TopLevelGridBoundsBuilder;
	float MinimumVoxelSize;
	CalcViewBoundsAndMinimumVoxelSize(GraphBuilder, View, BuildOptions, TopLevelGridBoundsBuilder, MinimumVoxelSize);
	if (!TopLevelGridBoundsBuilder.IsValid())
	{
		FrustumGridUniformBuffer = CreateEmptyFrustumVoxelGridUniformBuffer(GraphBuilder);
		return;
	}

	FBoxSphereBounds TopLevelGridBounds(TopLevelGridBoundsBuilder);
	float NearPlaneDistance = HeterogeneousVolumes::GetNearPlaneDistanceForFrustumGrid();
	float FarPlaneDistance = HeterogeneousVolumes::GetFarPlaneDistanceForFrustumGrid();
	ClipNearFarDistances(View, TopLevelGridBounds, NearPlaneDistance, FarPlaneDistance);

	FIntVector TopLevelGridResolution;
	CalculateTopLevelGridResolutionForFrustumGrid(
		View,
		MinimumVoxelSize,
		NearPlaneDistance,
		FarPlaneDistance,
		TopLevelGridResolution
	);

	// Construct top-level grid over global bounding domain with some pre-determined resolution
	int32 TopLevelVoxelCount = TopLevelGridResolution.X * TopLevelGridResolution.Y * TopLevelGridResolution.Z;
	if (TopLevelVoxelCount == 0)
	{
		FrustumGridUniformBuffer = CreateEmptyFrustumVoxelGridUniformBuffer(GraphBuilder);
		return;
	}

	FMatrix WorldToView = View.ViewMatrices.GetWorldToView();
	FMatrix ViewToWorld = View.ViewMatrices.GetViewToWorld();

	// Mark top-level voxels for rasterization
	FRDGBufferRef TopLevelGridBuffer;
	MarkTopLevelGridVoxelsForFrustumGrid(
		GraphBuilder,
		View,
		TopLevelGridResolution,
		NearPlaneDistance,
		FarPlaneDistance,
		ViewToWorld,
		TopLevelGridBuffer
	);

	// Generate raster tiles of approximately equal work
	bool bEnableIndirectionGrid = false;
	FRDGBufferRef RasterTileBuffer;
	FRDGBufferRef RasterTileAllocatorBuffer;
	GenerateRasterTiles(
		GraphBuilder,
		Scene,
		bEnableIndirectionGrid,
		// Grid data
		TopLevelGridResolution,
		TopLevelGridBuffer,
		// Tile data
		RasterTileBuffer,
		RasterTileAllocatorBuffer
	);

	FRDGBufferRef ExtinctionGridBuffer;
	FRDGBufferRef EmissionGridBuffer;
	FRDGBufferRef ScatteringGridBuffer;
	RasterizeVolumesIntoFrustumVoxelGrid(
		GraphBuilder,
		Scene,
		View,
		BuildOptions,
		ViewToWorld,
		NearPlaneDistance,
		FarPlaneDistance,
		// Raster tile
		RasterTileBuffer,
		RasterTileAllocatorBuffer,
		// Top-level grid
		TopLevelGridBounds,
		TopLevelGridResolution,
		TopLevelGridBuffer,
		// Bottom-level grid
		ExtinctionGridBuffer,
		EmissionGridBuffer,
		ScatteringGridBuffer
	);

	// Create Adpative Voxel Grid uniform buffer
	FFrustumVoxelGridUniformBufferParameters* UniformBufferParameters = GraphBuilder.AllocParameters<FFrustumVoxelGridUniformBufferParameters>();
	{
		FMatrix ViewToClip = FPerspectiveMatrix(
			FMath::DegreesToRadians(View.FOV * 0.5),
			TopLevelGridResolution.X,
			TopLevelGridResolution.Y,
			NearPlaneDistance,
			FarPlaneDistance
		);
		FMatrix ClipToView = ViewToClip.Inverse();
		UniformBufferParameters->ViewToClip = FMatrix44f(ViewToClip);
		UniformBufferParameters->ClipToView = FMatrix44f(ClipToView);

		FMatrix WorldToClip = WorldToView * ViewToClip;
		FMatrix ClipToWorld = ClipToView * ViewToWorld;
		UniformBufferParameters->WorldToClip = FMatrix44f(WorldToClip);
		UniformBufferParameters->ClipToWorld = FMatrix44f(ClipToWorld);

		UniformBufferParameters->WorldToView = FMatrix44f(WorldToView);
		UniformBufferParameters->ViewToWorld = FMatrix44f(ViewToWorld);

		UniformBufferParameters->TopLevelGridWorldBoundsMin = FVector3f(TopLevelGridBounds.Origin - TopLevelGridBounds.BoxExtent);
		UniformBufferParameters->TopLevelGridWorldBoundsMax = FVector3f(TopLevelGridBounds.Origin + TopLevelGridBounds.BoxExtent);
		UniformBufferParameters->TopLevelFroxelGridResolution = TopLevelGridResolution;
		UniformBufferParameters->VoxelDimensions = TopLevelGridResolution;

		UniformBufferParameters->bUseFrustumGrid = HeterogeneousVolumes::EnableFrustumVoxelGrid();
		UniformBufferParameters->NearPlaneDepth = NearPlaneDistance;
		UniformBufferParameters->FarPlaneDepth = FarPlaneDistance;
		UniformBufferParameters->TanHalfFOV = HeterogeneousVolumes::CalcTanHalfFOV(View.FOV);

		// Frustum assignment
		{
			// Near/Far plane definition is reversed when explicitly specifying clipping planes
			FPlane NearPlane;
			ViewToClip.GetFrustumFarPlane(NearPlane);
			UniformBufferParameters->ViewFrustumPlanes[0] = FVector4f(NearPlane.X, NearPlane.Y, NearPlane.Z, NearPlane.W);

			FPlane FarPlane;
			ViewToClip.GetFrustumNearPlane(FarPlane);
			UniformBufferParameters->ViewFrustumPlanes[1] = FVector4f(FarPlane.X, FarPlane.Y, FarPlane.Z, FarPlane.W);

			FPlane LeftPlane;
			ViewToClip.GetFrustumLeftPlane(LeftPlane);
			UniformBufferParameters->ViewFrustumPlanes[2] = FVector4f(LeftPlane.X, LeftPlane.Y, LeftPlane.Z, LeftPlane.W);

			FPlane RightPlane;
			ViewToClip.GetFrustumRightPlane(RightPlane);
			UniformBufferParameters->ViewFrustumPlanes[3] = FVector4f(RightPlane.X, RightPlane.Y, RightPlane.Z, RightPlane.W);

			FPlane TopPlane;
			ViewToClip.GetFrustumTopPlane(TopPlane);
			UniformBufferParameters->ViewFrustumPlanes[4] = FVector4f(TopPlane.X, TopPlane.Y, TopPlane.Z, TopPlane.W);

			FPlane BottomPlane;
			ViewToClip.GetFrustumBottomPlane(BottomPlane);
			UniformBufferParameters->ViewFrustumPlanes[5] = FVector4f(BottomPlane.X, BottomPlane.Y, BottomPlane.Z, BottomPlane.W);
		}

		UniformBufferParameters->TopLevelFroxelGridBuffer = GraphBuilder.CreateSRV(TopLevelGridBuffer);
		UniformBufferParameters->ExtinctionFroxelGridBuffer = GraphBuilder.CreateSRV(ExtinctionGridBuffer);
		UniformBufferParameters->EmissionFroxelGridBuffer = GraphBuilder.CreateSRV(EmissionGridBuffer);
		UniformBufferParameters->ScatteringFroxelGridBuffer = GraphBuilder.CreateSRV(ScatteringGridBuffer);
	}

	FrustumGridUniformBuffer = GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
}

void ExtractOrthoVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	FAdaptiveOrthoGridParameterCache& AdaptiveOrthoGridParameterCache
)
{
	const TRDGParameterStruct<FOrthoVoxelGridUniformBufferParameters>& Parameters = OrthoGridUniformBuffer->GetParameters();

	AdaptiveOrthoGridParameterCache.TopLevelGridWorldBoundsMin = Parameters->TopLevelGridWorldBoundsMin;
	AdaptiveOrthoGridParameterCache.TopLevelGridWorldBoundsMax = Parameters->TopLevelGridWorldBoundsMax;
	AdaptiveOrthoGridParameterCache.TopLevelGridResolution = Parameters->TopLevelGridResolution;
	AdaptiveOrthoGridParameterCache.bUseOrthoGrid = Parameters->bUseOrthoGrid;
	AdaptiveOrthoGridParameterCache.bUseMajorantGrid = Parameters->bUseMajorantGrid;
	AdaptiveOrthoGridParameterCache.bEnableIndirectionGrid = Parameters->bEnableIndirectionGrid;

	GraphBuilder.QueueBufferExtraction(Parameters->TopLevelGridBitmaskBuffer->GetParent(), &AdaptiveOrthoGridParameterCache.TopLevelGridBitmaskBuffer);
	GraphBuilder.QueueBufferExtraction(Parameters->TopLevelGridBuffer->GetParent(), &AdaptiveOrthoGridParameterCache.TopLevelGridBuffer);
	GraphBuilder.QueueBufferExtraction(Parameters->IndirectionGridBuffer->GetParent(), &AdaptiveOrthoGridParameterCache.IndirectionGridBuffer);

	GraphBuilder.QueueBufferExtraction(Parameters->ExtinctionGridBuffer->GetParent(), &AdaptiveOrthoGridParameterCache.ExtinctionGridBuffer);
	GraphBuilder.QueueBufferExtraction(Parameters->EmissionGridBuffer->GetParent(), &AdaptiveOrthoGridParameterCache.EmissionGridBuffer);
	GraphBuilder.QueueBufferExtraction(Parameters->ScatteringGridBuffer->GetParent(), &AdaptiveOrthoGridParameterCache.ScatteringGridBuffer);
	GraphBuilder.QueueBufferExtraction(Parameters->MajorantGridBuffer->GetParent(), &AdaptiveOrthoGridParameterCache.MajorantGridBuffer);

}

void RegisterExternalOrthoVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FAdaptiveOrthoGridParameterCache& AdaptiveOrthoGridParameterCache,
	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer
)
{
	FOrthoVoxelGridUniformBufferParameters* UniformBufferParameters = GraphBuilder.AllocParameters<FOrthoVoxelGridUniformBufferParameters>();
	{
		UniformBufferParameters->TopLevelGridWorldBoundsMin = AdaptiveOrthoGridParameterCache.TopLevelGridWorldBoundsMin;
		UniformBufferParameters->TopLevelGridWorldBoundsMax = AdaptiveOrthoGridParameterCache.TopLevelGridWorldBoundsMax;
		UniformBufferParameters->TopLevelGridResolution = AdaptiveOrthoGridParameterCache.TopLevelGridResolution;
		UniformBufferParameters->bUseOrthoGrid = AdaptiveOrthoGridParameterCache.bUseOrthoGrid;
		UniformBufferParameters->bUseMajorantGrid = AdaptiveOrthoGridParameterCache.bUseMajorantGrid;
		UniformBufferParameters->bEnableIndirectionGrid = AdaptiveOrthoGridParameterCache.bEnableIndirectionGrid;

		UniformBufferParameters->TopLevelGridBitmaskBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveOrthoGridParameterCache.TopLevelGridBitmaskBuffer));
		UniformBufferParameters->TopLevelGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveOrthoGridParameterCache.TopLevelGridBuffer));
		UniformBufferParameters->IndirectionGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveOrthoGridParameterCache.IndirectionGridBuffer));

		UniformBufferParameters->ExtinctionGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveOrthoGridParameterCache.ExtinctionGridBuffer));
		UniformBufferParameters->EmissionGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveOrthoGridParameterCache.EmissionGridBuffer));
		UniformBufferParameters->ScatteringGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveOrthoGridParameterCache.ScatteringGridBuffer));
		UniformBufferParameters->MajorantGridBuffer = GraphBuilder.CreateSRV(GraphBuilder.RegisterExternalBuffer(AdaptiveOrthoGridParameterCache.MajorantGridBuffer));
	}
	OrthoGridUniformBuffer = GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
}

TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters> CreateEmptyOrthoVoxelGridUniformBuffer(
	FRDGBuilder& GraphBuilder
)
{
	FOrthoVoxelGridUniformBufferParameters* OrthoGridUniformBufferParameters = GraphBuilder.AllocParameters<FOrthoVoxelGridUniformBufferParameters>();
	{
		OrthoGridUniformBufferParameters->TopLevelGridWorldBoundsMin = FVector3f(0);
		OrthoGridUniformBufferParameters->TopLevelGridWorldBoundsMax = FVector3f(0);
		OrthoGridUniformBufferParameters->TopLevelGridResolution = FIntVector(0);

		OrthoGridUniformBufferParameters->TopLevelGridBitmaskBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FTopLevelGridBitmaskData)));
		OrthoGridUniformBufferParameters->TopLevelGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FTopLevelGridData)));
		OrthoGridUniformBufferParameters->IndirectionGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FTopLevelGridData)));
		OrthoGridUniformBufferParameters->ExtinctionGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FScalarGridData)));
		OrthoGridUniformBufferParameters->EmissionGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVectorGridData)));
		OrthoGridUniformBufferParameters->ScatteringGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVectorGridData)));

		OrthoGridUniformBufferParameters->bUseOrthoGrid = false;
		OrthoGridUniformBufferParameters->bUseMajorantGrid = false;
		OrthoGridUniformBufferParameters->bEnableIndirectionGrid = false;
		OrthoGridUniformBufferParameters->MajorantGridBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FScalarGridData)));
	}
	return GraphBuilder.CreateUniformBuffer(OrthoGridUniformBufferParameters);
}

namespace HeterogeneousVolumes
{
	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters> GetOrthoVoxelGridUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState
	)
	{
		if (ViewState && ViewState->OrthoVoxelGridUniformBuffer)
		{
			return ViewState->OrthoVoxelGridUniformBuffer;
		}

		return CreateEmptyOrthoVoxelGridUniformBuffer(GraphBuilder);
	}

	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters> GetFrustumVoxelGridUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState
	)
	{
		if (ViewState && ViewState->FrustumVoxelGridUniformBuffer)
		{
			return ViewState->FrustumVoxelGridUniformBuffer;
		}

		return CreateEmptyFrustumVoxelGridUniformBuffer(GraphBuilder);
	}
}

void RasterizeVolumesIntoOrthoVoxelGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const TArray<FViewInfo>& Views,
	const TSet<FVolumetricMeshBatch>& HeterogeneousVolumesMeshBatches,
	const FVoxelGridBuildOptions BuildOptions,
	// Raster tile
	FRDGBufferRef RasterTileBuffer,
	FRDGBufferRef RasterTileAllocatorBuffer,
	// Top-level grid
	FBoxSphereBounds TopLevelGridBounds,
	FIntVector TopLevelGridResolution,
	FRDGBufferRef& TopLevelGridBuffer,
	// Indirection grid
	FRDGBufferRef& IndirectionGridBuffer,
	// Bottom-level grid
	FRDGBufferRef& ExtinctionGridBuffer,
	FRDGBufferRef& EmissionGridBuffer,
	FRDGBufferRef& ScatteringGridBuffer
)
{
	// Setup indirect dispatch
	FRDGBufferRef RasterizeBottomLevelGridIndirectArgsBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1),
		TEXT("HeterogeneousVolume.RasterizeBottomLevelGridIndirectArgs")
	);

	FSetRasterizeBottomLevelGridIndirectArgs::FParameters* IndirectArgsPassParameters = GraphBuilder.AllocParameters<FSetRasterizeBottomLevelGridIndirectArgs::FParameters>();
	{
		IndirectArgsPassParameters->MaxDispatchThreadGroupsPerDimension = GRHIMaxDispatchThreadGroupsPerDimension;
		IndirectArgsPassParameters->RasterTileAllocatorBuffer = GraphBuilder.CreateSRV(RasterTileAllocatorBuffer, PF_R32_UINT);
		IndirectArgsPassParameters->RWRasterizeBottomLevelGridIndirectArgsBuffer = GraphBuilder.CreateUAV(RasterizeBottomLevelGridIndirectArgsBuffer, PF_R32_UINT);
	}

	const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(Scene->GetShaderPlatform());

	FIntVector GroupCount(1, 1, 1);
	TShaderRef<FSetRasterizeBottomLevelGridIndirectArgs> ComputeShader = GlobalShaderMap->GetShader<FSetRasterizeBottomLevelGridIndirectArgs>();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("SetRasterizeBottomLevelGridIndirectArgs"),
		ERDGPassFlags::Compute,
		ComputeShader,
		IndirectArgsPassParameters,
		GroupCount
	);

	int32 MaxBottomLevelVoxelCount = (HeterogeneousVolumes::GetMaxBottomLevelMemoryInMegabytesForOrthoGrid() * 1e6) / sizeof(FVectorGridData);
	int32 BottomLevelGridBufferSize = MaxBottomLevelVoxelCount;

	ExtinctionGridBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FScalarGridData), BottomLevelGridBufferSize),
		TEXT("HeterogeneousVolumes.OrthoGrid.ExtinctionGridBuffer")
	);
	EmissionGridBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FVectorGridData), BottomLevelGridBufferSize),
		TEXT("HeterogeneousVolumes.OrthoGrid.EmissionGridBuffer")
	);
	ScatteringGridBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FVectorGridData), BottomLevelGridBufferSize),
		TEXT("HeterogeneousVolumes.OrthoGrid.ScatteringGridBuffer")
	);

	FRDGBufferRef BottomLevelGridAllocatorBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 2),
		TEXT("HeterogeneousVolume.OrthoGrid.BottomLevelGridAllocatorBuffer")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BottomLevelGridAllocatorBuffer, PF_R32_UINT), 0);

#if 0
	// Enable bottom-level voxel hashing
	uint32 HashTableBufferSize = 1;
	if (HeterogeneousVolumes::EnableVoxelHashing())
	{
		HashTableBufferSize = HeterogeneousVolumes::GetVoxelHashingMemoryInMegabytes() * 1.0e6;
	}

	FRDGBufferRef HashTableBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), HashTableBufferSize),
		TEXT("HeterogeneousVolume.OrthoGrid.HashTableBufferSize")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HashTableBuffer, PF_R32_UINT), 0);

	FRDGBufferRef HashToVoxelBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), HashTableBufferSize),
		TEXT("HeterogeneousVolume.OrthoGrid.HashToVoxelBuffer")
	);
#endif

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		// Build view
		const FViewInfo& View = Views[ViewIndex];
		FVector WorldCameraOrigin = View.ViewMatrices.GetViewOrigin();
		FBoxSphereBounds WorldCameraBounds(FSphere(WorldCameraOrigin, HeterogeneousVolumes::GetMaxTraceDistance()));

		for (auto MeshBatchIt = HeterogeneousVolumesMeshBatches.begin(); MeshBatchIt != HeterogeneousVolumesMeshBatches.end(); ++MeshBatchIt)
		{
			FVolumetricMeshBatch VolumetricMeshBatch = *MeshBatchIt;
			const FMeshBatch* Mesh = VolumetricMeshBatch.Mesh;
			const FPrimitiveSceneProxy* PrimitiveSceneProxy = VolumetricMeshBatch.Proxy;
			const FMaterialRenderProxy* MaterialRenderProxy = Mesh->MaterialRenderProxy;
			if (!ShouldRenderMeshBatchWithHeterogeneousVolumes(Mesh, PrimitiveSceneProxy, View.GetFeatureLevel()))
			{
				continue;
			}

			for (int32 VolumeIndex = 0; VolumeIndex < Mesh->Elements.Num(); ++VolumeIndex)
			{
				const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeIndex].UserData;
				//check(HeterogeneousVolumeInterface != nullptr);
				if (HeterogeneousVolumeInterface == nullptr)
				{
					continue;
				}

				const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
				const int32 PrimitiveId = PrimitiveSceneInfo->GetIndex();
				const FBoxSphereBounds LocalBoxSphereBounds = PrimitiveSceneProxy->GetLocalBounds();
				const FBoxSphereBounds PrimitiveBounds = PrimitiveSceneProxy->GetBounds();
				const FMaterial& Material = MaterialRenderProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialRenderProxy);

				FRasterizeBottomLevelOrthoGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRasterizeBottomLevelOrthoGridCS::FParameters>();
				{
					// Scene data
					PassParameters->View = View.ViewUniformBuffer;
					PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
					PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));

					// Primitive data
					PassParameters->PrimitiveWorldBoundsMin = FVector3f(PrimitiveBounds.Origin - PrimitiveBounds.BoxExtent);
					PassParameters->PrimitiveWorldBoundsMax = FVector3f(PrimitiveBounds.Origin + PrimitiveBounds.BoxExtent);

					// TODO: Convert to relative-local space
					//FVector3f ViewOriginHigh = FDFVector3(View.ViewMatrices.GetViewOrigin()).High;
					//FMatrix44f RelativeLocalToWorld = FDFMatrix::MakeToRelativeWorldMatrix(ViewOriginHigh, HeterogeneousVolumeInterface->GetLocalToWorld()).M;
					FMatrix InstanceToLocal = HeterogeneousVolumeInterface->GetInstanceToLocal();
					FMatrix LocalToWorld = HeterogeneousVolumeInterface->GetLocalToWorld();
					PassParameters->LocalToWorld = FMatrix44f(InstanceToLocal * LocalToWorld);
					PassParameters->WorldToLocal = FMatrix44f(PassParameters->LocalToWorld.Inverse());

					FMatrix LocalToInstance = InstanceToLocal.Inverse();
					FBoxSphereBounds InstanceBoxSphereBounds = LocalBoxSphereBounds.TransformBy(LocalToInstance);
					PassParameters->LocalBoundsOrigin = FVector3f(InstanceBoxSphereBounds.Origin);
					PassParameters->LocalBoundsExtent = FVector3f(InstanceBoxSphereBounds.BoxExtent);
					PassParameters->PrimitiveId = PrimitiveId;

					// Volume data
					PassParameters->TopLevelGridResolution = TopLevelGridResolution;
					PassParameters->TopLevelGridWorldBoundsMin = FVector3f(TopLevelGridBounds.Origin - TopLevelGridBounds.BoxExtent);
					PassParameters->TopLevelGridWorldBoundsMax = FVector3f(TopLevelGridBounds.Origin + TopLevelGridBounds.BoxExtent);

					// Sampling data
					FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
					PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

					// Unify with "object" definition??
					PassParameters->PrimitiveWorldBoundsMin = FVector3f(PrimitiveBounds.Origin - PrimitiveBounds.BoxExtent);
					PassParameters->PrimitiveWorldBoundsMax = FVector3f(PrimitiveBounds.Origin + PrimitiveBounds.BoxExtent);

					PassParameters->BottomLevelGridBufferSize = BottomLevelGridBufferSize;

					// Raster tile data
					PassParameters->RasterTileAllocatorBuffer = GraphBuilder.CreateSRV(RasterTileAllocatorBuffer, PF_R32_UINT);
					PassParameters->RasterTileBuffer = GraphBuilder.CreateSRV(RasterTileBuffer);

					// Indirect args
					PassParameters->IndirectArgs = RasterizeBottomLevelGridIndirectArgsBuffer;

					// Sampling mode
					PassParameters->bJitter = BuildOptions.bJitter;
					PassParameters->bSampleAtVertices = HeterogeneousVolumes::EnableLinearInterpolation();

					// Grid data
					PassParameters->TopLevelGridBuffer = GraphBuilder.CreateSRV(TopLevelGridBuffer);
					PassParameters->RWBottomLevelGridAllocatorBuffer = GraphBuilder.CreateUAV(BottomLevelGridAllocatorBuffer, PF_R32_UINT);
					PassParameters->RWTopLevelGridBuffer = GraphBuilder.CreateUAV(TopLevelGridBuffer);
					PassParameters->RWExtinctionGridBuffer = GraphBuilder.CreateUAV(ExtinctionGridBuffer);
					PassParameters->RWEmissionGridBuffer = GraphBuilder.CreateUAV(EmissionGridBuffer);
					PassParameters->RWScatteringGridBuffer = GraphBuilder.CreateUAV(ScatteringGridBuffer);

					// Indirection Grid
					PassParameters->FixedBottomLevelResolution = HeterogeneousVolumes::GetBottomLevelGridResolution();
					PassParameters->RWIndirectionGridBuffer = GraphBuilder.CreateUAV(IndirectionGridBuffer);

	#if 0
					// Hash table
					if (HeterogeneousVolumes::EnableVoxelHashing())
					{
						PassParameters->RWHashTable = GraphBuilder.CreateUAV(HashTableBuffer);
						PassParameters->RWHashToVoxelBuffer = GraphBuilder.CreateUAV(HashToVoxelBuffer);
						PassParameters->HashTableSize = HashTableBufferSize;
					}
	#endif
					PassParameters->HomogeneousThreshold = HeterogeneousVolumes::GetHomogeneousAggregationThreshold();
				}

				GraphBuilder.AddPass(
					RDG_EVENT_NAME("RasterizeBottomLevelGrid"),
					PassParameters,
					ERDGPassFlags::Compute,
					// Why is scene explicitly copied?
					[PassParameters, Scene, &View, MaterialRenderProxy, &Material](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
					{
						FRasterizeBottomLevelOrthoGridCS::FPermutationDomain PermutationVector;
						PermutationVector.Set<FRasterizeBottomLevelOrthoGridCS::FEnableIndirectionGrid>(HeterogeneousVolumes::EnableIndirectionGrid());
						PermutationVector.Set<FRasterizeBottomLevelOrthoGridCS::FEnableHomogeneousAggregation>(HeterogeneousVolumes::EnableHomogeneousAggregation());
						TShaderRef<FRasterizeBottomLevelOrthoGridCS> ComputeShader = Material.GetShader<FRasterizeBottomLevelOrthoGridCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);

						if (!ComputeShader.IsNull())
						{
							ClearUnusedGraphResources(ComputeShader, PassParameters);

							FMeshDrawShaderBindings ShaderBindings;
							UE::MeshPassUtils::SetupComputeBindings(ComputeShader, Scene, Scene->GetFeatureLevel(), nullptr, *MaterialRenderProxy, Material, ShaderBindings);

							UE::MeshPassUtils::DispatchIndirect(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, PassParameters->IndirectArgs->GetIndirectRHICallBuffer(), 0);
						}
					}
				);
			}
		}
	}
}

bool ShouldAddToVoxelGrid(const FPrimitiveSceneProxy* Proxy, const FViewInfo& View, const FVoxelGridBuildOptions& BuildOptions)
{
	// TODO: Should only add element if it is within the MaxTraceDistance
	switch (BuildOptions.VoxelGridBuildMode)
	{
		default:
		case EVoxelGridBuildMode::PathTracing:
			return Proxy->IsShown(&View);
		case EVoxelGridBuildMode::Shadows:
			return Proxy->IsShadowCast(&View);
	}
}

void CollectHeterogeneousVolumeMeshBatches(
	FRDGBuilder& GraphBuilder,
	const FVoxelGridBuildOptions& BuildOptions,
	const FScene* Scene,
	const TArray<FViewInfo>& Views,
	const FVisibleLightInfoArray& VisibleLightInfos,
	TSet<FVolumetricMeshBatch>& HeterogeneousVolumesMeshBatches
)
{
	HeterogeneousVolumesMeshBatches.Reset();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		for (int32 MeshBatchIndex = 0; MeshBatchIndex < View.HeterogeneousVolumesMeshBatches.Num(); ++MeshBatchIndex)
		{
			const FVolumetricMeshBatch& MeshBatch = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex];
			if (ShouldAddToVoxelGrid(MeshBatch.Proxy, View, BuildOptions))
			{
				HeterogeneousVolumesMeshBatches.FindOrAdd(MeshBatch);
			}
		}
	}

	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact = HeterogeneousVolumes::GatherShadowLights(Scene, Views[0].ViewLightingChannelMask);

	// TODO: Temporarily disable per-light primitive gathering because it causes a crash with VoxelGrid builder
	if (HeterogeneousVolumes::GetShadowPipeline() == HeterogeneousVolumes::EShadowPipeline::VoxelGrid)
	{
		return;
	}

	int32 NumPasses = LightSceneInfoCompact.Num();
	for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
	{
		const FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact[PassIndex].LightSceneInfo;
		if (LightSceneInfo->Proxy->CastsVolumetricShadow())
		{
			const FVisibleLightInfo* VisibleLightInfo = &VisibleLightInfos[LightSceneInfo->Id];
			check(VisibleLightInfo);
			for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightInfo->ShadowsToProject.Num(); ++ShadowIndex)
			{
				const FProjectedShadowInfo* ProjectedShadowInfo = HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, ShadowIndex);
				if (ProjectedShadowInfo != nullptr)
				{
					const TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>& MeshBatches = ProjectedShadowInfo->GetDynamicSubjectHeterogeneousVolumeMeshElements();
					for (int32 MeshBatchIndex = 0; MeshBatchIndex < MeshBatches.Num(); ++MeshBatchIndex)
					{
						const FMeshBatchAndRelevance& MeshBatch = MeshBatches[MeshBatchIndex];
						if (ShouldAddToVoxelGrid(MeshBatch.PrimitiveSceneProxy, *ProjectedShadowInfo->ShadowDepthView, BuildOptions))
						{
							HeterogeneousVolumesMeshBatches.FindOrAdd(FVolumetricMeshBatch(MeshBatch.Mesh, MeshBatch.PrimitiveSceneProxy));
						}
					}
				}
			}
		}
	}
}

void BuildOrthoVoxelGrid(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	/*const*/ TArray<FViewInfo>& Views,
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVoxelGridBuildOptions& BuildOptions,
	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer
)
{
	if (!ShouldRenderHeterogeneousVolumes(Scene) || !HeterogeneousVolumes::EnableOrthoVoxelGrid() || !BuildOptions.bBuildOrthoGrid)
	{
		OrthoGridUniformBuffer = CreateEmptyOrthoVoxelGridUniformBuffer(GraphBuilder);
		return;
	}
	check(!Views.IsEmpty());

	RDG_EVENT_SCOPE(GraphBuilder, "Ortho Grid Build");

	TSet<FVolumetricMeshBatch> HeterogeneousVolumesMeshBatches;
	CollectHeterogeneousVolumeMeshBatches(
		GraphBuilder,
		BuildOptions,
		Scene,
		Views,
		VisibleLightInfos,
		HeterogeneousVolumesMeshBatches
	);

	if (HeterogeneousVolumesMeshBatches.IsEmpty())
	{
		OrthoGridUniformBuffer = CreateEmptyOrthoVoxelGridUniformBuffer(GraphBuilder);
		return;
	}

	// Collect global bounds
	FBoxSphereBounds::Builder TopLevelGridBoundsBuilder;
	float GlobalMinimumVoxelSize;
	CalcGlobalBoundsAndMinimumVoxelSize(
		GraphBuilder,
		Views,
		HeterogeneousVolumesMeshBatches,
		BuildOptions,
		TopLevelGridBoundsBuilder,
		GlobalMinimumVoxelSize
	);

	if (!TopLevelGridBoundsBuilder.IsValid())
	{
		OrthoGridUniformBuffer = CreateEmptyOrthoVoxelGridUniformBuffer(GraphBuilder);
		return;
	}
	FBoxSphereBounds TopLevelGridBounds(TopLevelGridBoundsBuilder);

	// Determine top-level grid resolution
	FIntVector TopLevelGridResolution;
	CalculateTopLevelGridResolution(
		TopLevelGridBounds,
		GlobalMinimumVoxelSize,
		TopLevelGridResolution
	);

	// Calculate the preferred voxel size for each bottom-level grid in a top-level cell
	FRDGBufferRef TopLevelGridBuffer;
	CalculateVoxelSize(
		GraphBuilder,
		Scene,
		Views,
		HeterogeneousVolumesMeshBatches,
		BuildOptions,
		TopLevelGridBounds,
		TopLevelGridResolution,
		TopLevelGridBuffer
	);

	// Allocate bottom-level grid
	FRDGBufferRef IndirectionGridBuffer;
	MarkTopLevelGrid(
		GraphBuilder,
		Scene,
		// Grid data
		TopLevelGridBounds,
		TopLevelGridResolution,
		TopLevelGridBuffer,
		IndirectionGridBuffer
	);

	// Generate raster tiles
	FRDGBufferRef RasterTileBuffer;
	FRDGBufferRef RasterTileAllocatorBuffer;
	GenerateRasterTiles(
		GraphBuilder,
		Scene,
		HeterogeneousVolumes::EnableIndirectionGrid(),
		// Grid data
		TopLevelGridResolution,
		TopLevelGridBuffer,
		// Tile data
		RasterTileBuffer,
		RasterTileAllocatorBuffer
	);

	// Volume rasterization
	FRDGBufferRef ExtinctionGridBuffer;
	FRDGBufferRef EmissionGridBuffer;
	FRDGBufferRef ScatteringGridBuffer;
	RasterizeVolumesIntoOrthoVoxelGrid(
		GraphBuilder,
		Scene,
		Views,
		HeterogeneousVolumesMeshBatches,
		BuildOptions,
		// Tile data
		RasterTileBuffer,
		RasterTileAllocatorBuffer,
		// Grid data
		TopLevelGridBounds,
		TopLevelGridResolution,
		TopLevelGridBuffer,
		IndirectionGridBuffer,
		ExtinctionGridBuffer,
		EmissionGridBuffer,
		ScatteringGridBuffer
	);

	FRDGBufferRef TopLevelGridBitmaskBuffer;
	CreateTopLevelBitmask(GraphBuilder, Views, TopLevelGridResolution, TopLevelGridBuffer, TopLevelGridBitmaskBuffer);

	FRDGBufferRef MajorantGridBuffer;
	BuildMajorantVoxelGrid(GraphBuilder, Scene, TopLevelGridResolution, TopLevelGridBuffer, IndirectionGridBuffer, ExtinctionGridBuffer, MajorantGridBuffer);

	// Create Adpative Voxel Grid uniform buffer
	FOrthoVoxelGridUniformBufferParameters* OrthoGridUniformBufferParameters = GraphBuilder.AllocParameters<FOrthoVoxelGridUniformBufferParameters>();
	{
		OrthoGridUniformBufferParameters->TopLevelGridWorldBoundsMin = FVector3f(TopLevelGridBounds.Origin - TopLevelGridBounds.BoxExtent);
		OrthoGridUniformBufferParameters->TopLevelGridWorldBoundsMax = FVector3f(TopLevelGridBounds.Origin + TopLevelGridBounds.BoxExtent);
		OrthoGridUniformBufferParameters->TopLevelGridResolution = TopLevelGridResolution;

		OrthoGridUniformBufferParameters->TopLevelGridBitmaskBuffer = GraphBuilder.CreateSRV(TopLevelGridBitmaskBuffer);
		OrthoGridUniformBufferParameters->TopLevelGridBuffer = GraphBuilder.CreateSRV(TopLevelGridBuffer);
		OrthoGridUniformBufferParameters->IndirectionGridBuffer = GraphBuilder.CreateSRV(IndirectionGridBuffer);
		OrthoGridUniformBufferParameters->ExtinctionGridBuffer = GraphBuilder.CreateSRV(ExtinctionGridBuffer);
		OrthoGridUniformBufferParameters->EmissionGridBuffer = GraphBuilder.CreateSRV(EmissionGridBuffer);
		OrthoGridUniformBufferParameters->ScatteringGridBuffer = GraphBuilder.CreateSRV(ScatteringGridBuffer);

		OrthoGridUniformBufferParameters->bUseOrthoGrid = HeterogeneousVolumes::EnableOrthoVoxelGrid();
		OrthoGridUniformBufferParameters->bUseMajorantGrid = HeterogeneousVolumes::EnableMajorantGrid();
		OrthoGridUniformBufferParameters->bEnableIndirectionGrid = HeterogeneousVolumes::EnableIndirectionGrid();
		OrthoGridUniformBufferParameters->MajorantGridBuffer = GraphBuilder.CreateSRV(MajorantGridBuffer);
	}

	OrthoGridUniformBuffer = GraphBuilder.CreateUniformBuffer(OrthoGridUniformBufferParameters);
}

void RenderTransmittanceWithVoxelGrid(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer,
	// Output
	FRDGTextureRef& HeterogeneousVolumeRadiance
)
{
	FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(HeterogeneousVolumes::GetScaledViewRect(View.ViewRect), FRenderTransmittanceWithVoxelGridCS::GetThreadGroupSize2D());

	int32 BufferSize = View.ViewRect.Width() * View.ViewRect.Height();
	FRDGBufferRef DebugBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FRenderDebugData), BufferSize),
		TEXT("HeterogeneousVolumes.RenderDebugData")
	);

	FRenderTransmittanceWithVoxelGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderTransmittanceWithVoxelGridCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);

		// Ray data
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();

		// Volume data
		PassParameters->OrthoGridUniformBuffer = OrthoGridUniformBuffer;
		PassParameters->FrustumGridUniformBuffer = FrustumGridUniformBuffer;

		// Dispatch data
		PassParameters->GroupCount = GroupCount;

		// Output
		PassParameters->RWLightingTexture = GraphBuilder.CreateUAV(HeterogeneousVolumeRadiance);
		PassParameters->RWDebugBuffer = GraphBuilder.CreateUAV(DebugBuffer);
	}

	FRenderTransmittanceWithVoxelGridCS::FPermutationDomain PermutationVector;
	int32 DebugMode = FMath::Clamp(HeterogeneousVolumes::GetDebugMode() - 1, 0, FRenderTransmittanceWithVoxelGridCS::FDebugMode::MaxValue);
	PermutationVector.Set<FRenderTransmittanceWithVoxelGridCS::FDebugMode>(DebugMode);
	TShaderRef<FRenderTransmittanceWithVoxelGridCS> ComputeShader = View.ShaderMap->GetShader<FRenderTransmittanceWithVoxelGridCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("RenderTransmittanceWithVoxelGridCS"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

struct FVolumetricShadowMapDebugData
{
	FVector3f LightRayStart;
	FVector3f LightRayEnd;
	FVector3f RayOrigin;
	FVector3f RayEnd;
	float HitSpan[2];
};

class FRenderVolumetricShadowMapForLightWithVoxelGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderVolumetricShadowMapForLightWithVoxelGridCS);
	SHADER_USE_PARAMETER_STRUCT(FRenderVolumetricShadowMapForLightWithVoxelGridCS, FGlobalShader);

	class FUseAVSMCompression : SHADER_PERMUTATION_BOOL("USE_AVSM_COMPRESSION");
	class FIsOfflineRender : SHADER_PERMUTATION_INT("IS_OFFLINE_RENDER", 2);
	using FPermutationDomain = TShaderPermutationDomain<FUseAVSMCompression, FIsOfflineRender>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)

		// Shadow data
		SHADER_PARAMETER(float, ShadowStepSize)
		SHADER_PARAMETER(float, ShadowStepFactor)

		// Volumetric Shadow Map data
		SHADER_PARAMETER(FVector3f, TranslatedWorldOrigin)
		SHADER_PARAMETER(FIntPoint, ShadowResolution)
		SHADER_PARAMETER(int, MaxSampleCount)
		SHADER_PARAMETER(int, StochasticFilteringMode)
		SHADER_PARAMETER(float, AbsoluteErrorThreshold)
		SHADER_PARAMETER(float, RelativeErrorThreshold)

		SHADER_PARAMETER(int, NumShadowMatrices)
		SHADER_PARAMETER_ARRAY(FMatrix44f, TranslatedWorldToShadow, [6])
		SHADER_PARAMETER_ARRAY(FMatrix44f, ShadowToTranslatedWorld, [6])

		// Volume data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FOrthoVoxelGridUniformBufferParameters, OrthoGridUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFrustumVoxelGridUniformBufferParameters, FrustumGridUniformBuffer)

		// Ray data
		SHADER_PARAMETER(float, MaxTraceDistance)
		SHADER_PARAMETER(int, MaxStepCount)
		SHADER_PARAMETER(int, bJitter)

		// Dispatch data
		SHADER_PARAMETER(FIntVector, GroupCount)
		SHADER_PARAMETER(int, ShadowDebugTweak)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>, RWVolumetricShadowLinkedListAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FAVSMLinkedListPackedData>, RWVolumetricShadowLinkedListBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWBeerShadowMapTexture)

		// Debug
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FVolumetricShadowMapDebugData>, RWDebugBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// Temporary disabling..
		OutEnvironment.SetDefine(TEXT("DIM_USE_TRANSMITTANCE_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_INSCATTERING_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_LUMEN_GI"), 0);

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FRenderVolumetricShadowMapForLightWithVoxelGridCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridShadows.usf", "RenderVolumetricShadowMapForLightWithVoxelGridCS", SF_Compute);


class FCompressVolumetricShadowMapCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompressVolumetricShadowMapCS);
	SHADER_USE_PARAMETER_STRUCT(FCompressVolumetricShadowMapCS, FGlobalShader);

	class FUseAVSMCompression : SHADER_PERMUTATION_BOOL("USE_AVSM_COMPRESSION");
	using FPermutationDomain = TShaderPermutationDomain<FUseAVSMCompression>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER(FIntPoint, ShadowResolution)
		SHADER_PARAMETER(int, MaxSampleCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FAVSMLinkedListPackedData>, VolumetricShadowLinkedListBuffer)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>, RWVolumetricShadowIndirectionAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FAVSMIndirectionPackedData>, RWVolumetricShadowIndirectionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FAVSMSamplePackedData>, RWVolumetricShadowTransmittanceBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// Temporary disabling..
		OutEnvironment.SetDefine(TEXT("DIM_USE_TRANSMITTANCE_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_INSCATTERING_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_LUMEN_GI"), 0);

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FCompressVolumetricShadowMapCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridShadows.usf", "CompressVolumetricShadowMapCS", SF_Compute);

class FCombineVolumetricShadowMapsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCombineVolumetricShadowMapsCS);
	SHADER_USE_PARAMETER_STRUCT(FCombineVolumetricShadowMapsCS, FGlobalShader);

	class FUseAVSMCompression : SHADER_PERMUTATION_BOOL("USE_AVSM_COMPRESSION");
	using FPermutationDomain = TShaderPermutationDomain<FUseAVSMCompression>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER(FIntPoint, ShadowResolution)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FAVSMLinkedListPackedData>, VolumetricShadowLinkedListBuffer0)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FAVSMLinkedListPackedData>, VolumetricShadowLinkedListBuffer1)

		// Volumetric Shadow Map data
		SHADER_PARAMETER(int, MaxSampleCount)
		SHADER_PARAMETER(float, AbsoluteErrorThreshold)
		SHADER_PARAMETER(float, RelativeErrorThreshold)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FAVSMLinkedListPackedData>, RWVolumetricShadowLinkedListBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// Temporary disabling..
		OutEnvironment.SetDefine(TEXT("DIM_USE_TRANSMITTANCE_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_INSCATTERING_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_LUMEN_GI"), 0);

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FCombineVolumetricShadowMapsCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridShadows.usf", "CombineVolumetricShadowMapsCS", SF_Compute);

class FConvertBeerLawShadowMapToVolumetricShadowMapCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FConvertBeerLawShadowMapToVolumetricShadowMapCS);
	SHADER_USE_PARAMETER_STRUCT(FConvertBeerLawShadowMapToVolumetricShadowMapCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER(FIntPoint, ShadowResolution)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, BeerShadowMapTexture)

		// Output
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>, RWVolumetricShadowIndirectionAllocatorBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FAVSMIndirectionPackedData>, RWVolumetricShadowIndirectionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FAVSMSamplePackedData>, RWVolumetricShadowTransmittanceBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		OutEnvironment.SetDefine(TEXT("DIM_USE_TRANSMITTANCE_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_INSCATTERING_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_LUMEN_GI"), 0);

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FConvertBeerLawShadowMapToVolumetricShadowMapCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridShadows.usf", "ConvertBeerLawShadowMapToVolumetricShadowMapCS", SF_Compute);

void ConvertBeerLawShadowMapToVolumetricShadowMap(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	//FIntVector GroupCount,
	// Input
	FIntPoint ShadowMapResolution,
	FRDGTextureRef BeerShadowMapTexture,
	// Output
	FRDGBufferRef& VolumetricShadowIndirectionBuffer,
	FRDGBufferRef& VolumetricShadowTransmittanceBuffer
)
{
	FIntVector GroupCount = FIntVector(1);
	GroupCount.X = FMath::DivideAndRoundUp(ShadowMapResolution.X, FConvertBeerLawShadowMapToVolumetricShadowMapCS::GetThreadGroupSize2D());
	GroupCount.Y = FMath::DivideAndRoundUp(ShadowMapResolution.X, FConvertBeerLawShadowMapToVolumetricShadowMapCS::GetThreadGroupSize2D());

	int32 ThreadGroupSize2D = FConvertBeerLawShadowMapToVolumetricShadowMapCS::GetThreadGroupSize2D();
	int32 VolumetricShadowPixelCount = GroupCount.X * ThreadGroupSize2D * GroupCount.Y * ThreadGroupSize2D;
	FRDGBufferRef VolumetricShadowIndirectionAllocatorBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1),
		TEXT("HeterogeneousVolume.VolumetricShadowIndirectionAllocatorBuffer")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(VolumetricShadowIndirectionAllocatorBuffer, PF_R32_UINT), 0);

	VolumetricShadowIndirectionBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMIndirectionPackedData), VolumetricShadowPixelCount),
		TEXT("HeterogeneousVolume.VolumetricShadowIndirectionBuffer")
	);

	// Beer-Lambert Shadow Maps have two samples
	const int32 MaxSampleCount = 2;
	int32 TopLevelSampleCount = FMath::CeilToInt(MaxSampleCount / 4.0);
	int32 VolumetricShadowTransmittanceMaxCount = VolumetricShadowPixelCount * (Align(TopLevelSampleCount, 4) + Align(MaxSampleCount, 4));
	VolumetricShadowTransmittanceBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMSamplePackedData), VolumetricShadowTransmittanceMaxCount),
		TEXT("HeterogeneousVolume.VolumetricShadowTransmittanceBuffer")
	);

	FConvertBeerLawShadowMapToVolumetricShadowMapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FConvertBeerLawShadowMapToVolumetricShadowMapCS::FParameters>();
	{
		// Input
		PassParameters->ShadowResolution = ShadowMapResolution;
		PassParameters->BeerShadowMapTexture = GraphBuilder.CreateSRV(BeerShadowMapTexture);

		// Output
		PassParameters->RWVolumetricShadowIndirectionAllocatorBuffer = GraphBuilder.CreateUAV(VolumetricShadowIndirectionAllocatorBuffer, PF_R32_UINT);
		PassParameters->RWVolumetricShadowIndirectionBuffer = GraphBuilder.CreateUAV(VolumetricShadowIndirectionBuffer);
		PassParameters->RWVolumetricShadowTransmittanceBuffer = GraphBuilder.CreateUAV(VolumetricShadowTransmittanceBuffer);
	}

	FConvertBeerLawShadowMapToVolumetricShadowMapCS::FPermutationDomain PermutationVector;
	TShaderRef<FConvertBeerLawShadowMapToVolumetricShadowMapCS> ComputeShader = View.ShaderMap->GetShader<FConvertBeerLawShadowMapToVolumetricShadowMapCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		// Add Light name..
		RDG_EVENT_NAME("FConvertBeerLawShadowMapToVolumetricShadowMapCS"),
		ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

class FConvertVolumetricShadowMapToBeerLawShadowMapCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FConvertVolumetricShadowMapToBeerLawShadowMapCS);
	SHADER_USE_PARAMETER_STRUCT(FConvertVolumetricShadowMapToBeerLawShadowMapCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Input
		SHADER_PARAMETER(FIntPoint, ShadowResolution)
		SHADER_PARAMETER(int32, MaxSampleCount)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FAVSMLinkedListPackedData>, VolumetricShadowLinkedListBuffer0)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWBeerShadowMapTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(
		const FGlobalShaderPermutationParameters& Parameters
	)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		OutEnvironment.SetDefine(TEXT("DIM_USE_TRANSMITTANCE_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_INSCATTERING_VOLUME"), 0);
		OutEnvironment.SetDefine(TEXT("DIM_USE_LUMEN_GI"), 0);

		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return GetThreadGroupSize3D() * GetThreadGroupSize3D() * GetThreadGroupSize3D(); }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FConvertVolumetricShadowMapToBeerLawShadowMapCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVoxelGridShadows.usf", "ConvertVolumetricShadowMapToBeerLawShadowMapCS", SF_Compute);

void ConvertVolumetricShadowMapToBeerLawShadowMap(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	FViewInfo& View,
	// Input
	const FString& ShadowMapName,
	FIntPoint ShadowMapResolution,
	FIntVector GroupCount,
	FRDGBufferRef VolumetricShadowLinkedListBuffer,
	// Output
	FRDGTextureRef& BeerShadowMapTexture
)
{
	FRDGTextureDesc Desc = SceneTextures.Color.Target->Desc;
	Desc.Format = PF_FloatRGBA;
	Desc.Flags &= ~(TexCreate_FastVRAM);
	Desc.Extent = FIntPoint(ShadowMapResolution.X * GroupCount.Z, ShadowMapResolution.Y);
	//FString ShadowMapString = FString::Printf(TEXT("HeterogeneousVolumes.%s.BeerShadowMapTexture"), *ShadowMapName);
	//BeerShadowMapTexture = GraphBuilder.CreateTexture(Desc, *ShadowMapString);
	BeerShadowMapTexture = GraphBuilder.CreateTexture(Desc, TEXT("HeterogeneousVolumes.Light.BeerShadowMapTexture"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BeerShadowMapTexture), FLinearColor::Transparent);

	FConvertVolumetricShadowMapToBeerLawShadowMapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FConvertVolumetricShadowMapToBeerLawShadowMapCS::FParameters>();
	{
		// Input
		PassParameters->ShadowResolution = ShadowMapResolution;
		PassParameters->MaxSampleCount = HeterogeneousVolumes::GetShadowMaxSampleCount();
		PassParameters->VolumetricShadowLinkedListBuffer0 = GraphBuilder.CreateSRV(VolumetricShadowLinkedListBuffer);

		// Output
		PassParameters->RWBeerShadowMapTexture = GraphBuilder.CreateUAV(BeerShadowMapTexture);
	}

	FConvertVolumetricShadowMapToBeerLawShadowMapCS::FPermutationDomain PermutationVector;
	TShaderRef<FConvertVolumetricShadowMapToBeerLawShadowMapCS> ComputeShader = View.ShaderMap->GetShader<FConvertVolumetricShadowMapToBeerLawShadowMapCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		// Add Light name..
		RDG_EVENT_NAME("FConvertBeerLawShadowMapToVolumetricShadowMapCS"),
		ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

void CreateAdaptiveVolumetricShadowMapUniformBufferParameters(
	FRDGBuilder& GraphBuilder,
	const FVector3f* TranslatedWorldOrigin,
	const FVector4f* TranslatedWorldPlane,
	const FMatrix44f* TranslatedWorldToShadow,
	const FVector4f* SplitDepths,
	FIntPoint VolumetricShadowMapResolution,
	float VolumetricShadowMapDownsampleFactor,
	int32 NumShadowMatrices,
	uint32 VolumetricShadowMapMaxSampleCount,
	bool bIsDirectionalLight,
	FRDGBufferRef VolumetricShadowMapLinkedListBuffer,
	FRDGBufferRef VolumetricShadowMapIndirectionBuffer,
	FRDGBufferRef VolumetricShadowMapSampleBuffer,
	FAdaptiveVolumetricShadowMapUniformBufferParameters*& Parameters
)
{
	Parameters = GraphBuilder.AllocParameters<FAdaptiveVolumetricShadowMapUniformBufferParameters>();
	{
		Parameters->NumShadowMatrices = NumShadowMatrices;
		for (int32 i = 0; i < NumShadowMatrices; ++i)
		{
			Parameters->TranslatedWorldToShadow[i] = TranslatedWorldToShadow[i];
			Parameters->TranslatedWorldOrigin[i] = TranslatedWorldOrigin[i];
			Parameters->TranslatedWorldPlane[i] = TranslatedWorldPlane[i];
			Parameters->SplitDepths[i] = SplitDepths[i];
		}

		Parameters->Resolution = VolumetricShadowMapResolution;
		Parameters->DownsampleFactor = VolumetricShadowMapDownsampleFactor;
		Parameters->MaxSampleCount = VolumetricShadowMapMaxSampleCount;
		Parameters->bIsEmpty = (VolumetricShadowMapResolution == FIntPoint::ZeroValue);
		Parameters->bIsDirectionalLight = bIsDirectionalLight;
		Parameters->LinkedListBuffer = GraphBuilder.CreateSRV(VolumetricShadowMapLinkedListBuffer);
		Parameters->IndirectionBuffer = GraphBuilder.CreateSRV(VolumetricShadowMapIndirectionBuffer);
		Parameters->SampleBuffer = GraphBuilder.CreateSRV(VolumetricShadowMapSampleBuffer);
		Parameters->RadianceTexture = GraphBuilder.CreateSRV(GSystemTextures.GetBlackDummy(GraphBuilder));
		Parameters->TextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
}

void CreateAdaptiveVolumetricShadowMapUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FVector3f* TranslatedWorldOrigin,
	const FVector4f* TranslatedWorldPlane,
	const FMatrix44f* TranslatedWorldToShadow,
	const FVector4f* SplitDepths,
	FIntPoint VolumetricShadowMapResolution,
	float VolumetricShadowMapDownsampleFactor,
	int32 NumShadowMatrices,
	uint32 VolumetricShadowMapMaxSampleCount,
	bool bIsDirectionalLight,
	FRDGBufferRef VolumetricShadowMapLinkedListBuffer,
	FRDGBufferRef VolumetricShadowMapIndirectionBuffer,
	FRDGBufferRef VolumetricShadowMapSampleBuffer,
	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>& AdaptiveVolumetricShadowMapUniformBuffer
)
{
	FAdaptiveVolumetricShadowMapUniformBufferParameters* UniformBufferParameters;
	CreateAdaptiveVolumetricShadowMapUniformBufferParameters(
		GraphBuilder,
		TranslatedWorldOrigin,
		TranslatedWorldPlane,
		TranslatedWorldToShadow,
		SplitDepths,
		VolumetricShadowMapResolution,
		VolumetricShadowMapDownsampleFactor,
		NumShadowMatrices,
		VolumetricShadowMapMaxSampleCount,
		bIsDirectionalLight,
		VolumetricShadowMapLinkedListBuffer,
		VolumetricShadowMapIndirectionBuffer,
		VolumetricShadowMapSampleBuffer,
		UniformBufferParameters
	);
	AdaptiveVolumetricShadowMapUniformBuffer = GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
}

void CreateBeerShadowMapUniformBufferParameters(
	FRDGBuilder& GraphBuilder,
	const FVector3f* TranslatedWorldOrigin,
	const FVector4f* TranslatedWorldPlane,
	const FMatrix44f* TranslatedWorldToShadow,
	const FVector4f* SplitDepths,
	FIntPoint VolumetricShadowMapResolution,
	float VolumetricShadowMapDownsampleFactor,
	int32 NumShadowMatrices,
	uint32 VolumetricShadowMapMaxSampleCount,
	bool bIsDirectionalLight,
	FRDGTextureRef BeerShadowMapTexture,
	FBeerShadowMapUniformBufferParameters*& Parameters
)
{
	Parameters = GraphBuilder.AllocParameters<FBeerShadowMapUniformBufferParameters>();
	{
		Parameters->NumShadowMatrices = NumShadowMatrices;
		for (int32 i = 0; i < NumShadowMatrices; ++i)
		{
			Parameters->TranslatedWorldToShadow[i] = TranslatedWorldToShadow[i];
			Parameters->TranslatedWorldOrigin[i] = TranslatedWorldOrigin[i];
			Parameters->TranslatedWorldPlane[i] = TranslatedWorldPlane[i];
			Parameters->SplitDepths[i] = SplitDepths[i];
		}

		Parameters->Resolution = VolumetricShadowMapResolution;
		Parameters->DownsampleFactor = VolumetricShadowMapDownsampleFactor;
		Parameters->MaxSampleCount = VolumetricShadowMapMaxSampleCount;
		Parameters->bIsEmpty = (VolumetricShadowMapResolution == FIntPoint::ZeroValue);
		Parameters->bIsDirectionalLight = bIsDirectionalLight;
		Parameters->BeerShadowMapTexture = GraphBuilder.CreateSRV(BeerShadowMapTexture);
		Parameters->RadianceTexture = GraphBuilder.CreateSRV(GSystemTextures.GetBlackDummy(GraphBuilder));
		Parameters->TextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
}

void CreateBeerShadowMapUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FVector3f* TranslatedWorldOrigin,
	const FVector4f* TranslatedWorldPlane,
	const FMatrix44f* TranslatedWorldToShadow,
	const FVector4f* SplitDepths,
	FIntPoint VolumetricShadowMapResolution,
	float VolumetricShadowMapDownsampleFactor,
	int32 NumShadowMatrices,
	uint32 VolumetricShadowMapMaxSampleCount,
	bool bIsDirectionalLight,
	FRDGTextureRef BeerShadowMapTexture,
	TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters>& BeerShadowMapUniformBuffer
)
{
	FBeerShadowMapUniformBufferParameters* UniformBufferParameters;
	CreateBeerShadowMapUniformBufferParameters(
		GraphBuilder,
		TranslatedWorldOrigin,
		TranslatedWorldPlane,
		TranslatedWorldToShadow,
		SplitDepths,
		VolumetricShadowMapResolution,
		VolumetricShadowMapDownsampleFactor,
		NumShadowMatrices,
		VolumetricShadowMapMaxSampleCount,
		bIsDirectionalLight,
		BeerShadowMapTexture,
		UniformBufferParameters
	);
	BeerShadowMapUniformBuffer = GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
}

namespace HeterogeneousVolumes
{
	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> CreateEmptyAdaptiveVolumetricShadowMapUniformBuffer(
		FRDGBuilder& GraphBuilder
	)
	{
		FAdaptiveVolumetricShadowMapUniformBufferParameters* UniformBufferParameters = GraphBuilder.AllocParameters<FAdaptiveVolumetricShadowMapUniformBufferParameters>();
		{
			UniformBufferParameters->NumShadowMatrices = 1;
			for (int32 i = 0; i < 6; ++i)
			{
				UniformBufferParameters->TranslatedWorldToShadow[i] = FMatrix44f::Identity;
				UniformBufferParameters->TranslatedWorldOrigin[i] = FVector3f::ZeroVector;
				UniformBufferParameters->TranslatedWorldPlane[i] = FVector4f::Zero();
				UniformBufferParameters->SplitDepths[i] = FVector4f::Zero();
			}

			UniformBufferParameters->Resolution = FIntPoint::ZeroValue;
			UniformBufferParameters->DownsampleFactor = 1.0f;
			UniformBufferParameters->MaxSampleCount = 0;
			UniformBufferParameters->bIsEmpty = true;
			UniformBufferParameters->bIsDirectionalLight = false;
			UniformBufferParameters->LinkedListBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMLinkedListPackedData)));
			UniformBufferParameters->IndirectionBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMIndirectionPackedData)));
			UniformBufferParameters->SampleBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMSamplePackedData)));
			UniformBufferParameters->RadianceTexture = GraphBuilder.CreateSRV(GSystemTextures.GetBlackDummy(GraphBuilder));
			UniformBufferParameters->TextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		}

		return GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
	}

	void DestroyAdaptiveVolumetricShadowMapUniformBuffer(
		TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>& AdaptiveVolumetricShadowMapUniformBuffer
	)
	{
		AdaptiveVolumetricShadowMapUniformBuffer = nullptr;
	}

	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> GetAdaptiveVolumetricShadowMapUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState,
		const FLightSceneInfo* LightSceneInfo
	)
	{
		if (ViewState && LightSceneInfo)
		{
			TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>* AdaptiveVolumetricShadowMapUniformBuffer = ViewState->AdaptiveVolumetricShadowMapUniformBufferMap.Find(LightSceneInfo->Id);
			if (AdaptiveVolumetricShadowMapUniformBuffer != nullptr)
			{
				return *AdaptiveVolumetricShadowMapUniformBuffer;
			}
		}
		
		return CreateEmptyAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder);
	}

	TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> GetAdaptiveVolumetricCameraMapUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState
	)
	{
		if (ViewState && ShouldCompositeHeterogeneousVolumesWithTranslucency())
		{
			if (ViewState->AdaptiveVolumetricCameraMapUniformBuffer != nullptr)
			{
				return ViewState->AdaptiveVolumetricCameraMapUniformBuffer;
			}
			else if (ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters != nullptr)
			{
				return GraphBuilder.CreateUniformBuffer(ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters);
			}
		}
		
		return CreateEmptyAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder);
	}

	FAdaptiveVolumetricShadowMapUniformBufferParameters GetAdaptiveVolumetricCameraMapParameters(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState)
	{
		FAdaptiveVolumetricShadowMapUniformBufferParameters Parameters;

		TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> UniformBuffer = GetAdaptiveVolumetricCameraMapUniformBuffer(GraphBuilder, ViewState);
		if (ViewState && UniformBuffer)
		{

			Parameters.NumShadowMatrices = UniformBuffer->GetParameters()->NumShadowMatrices;
			for (int32 i = 0; i < Parameters.NumShadowMatrices; ++i)
			{
				Parameters.TranslatedWorldToShadow[i] = UniformBuffer->GetParameters()->TranslatedWorldToShadow[i];
				Parameters.TranslatedWorldOrigin[i] = UniformBuffer->GetParameters()->TranslatedWorldOrigin[i];
				Parameters.TranslatedWorldPlane[i] = UniformBuffer->GetParameters()->TranslatedWorldPlane[i];
				Parameters.SplitDepths[i] = UniformBuffer->GetParameters()->SplitDepths[i];
			}

			Parameters.Resolution = UniformBuffer->GetParameters()->Resolution;
			Parameters.DownsampleFactor = UniformBuffer->GetParameters()->DownsampleFactor;
			Parameters.MaxSampleCount = UniformBuffer->GetParameters()->MaxSampleCount;
			Parameters.bIsEmpty = UniformBuffer->GetParameters()->bIsEmpty;
			Parameters.bIsDirectionalLight = UniformBuffer->GetParameters()->bIsDirectionalLight;
			Parameters.LinkedListBuffer = UniformBuffer->GetParameters()->LinkedListBuffer;
			Parameters.IndirectionBuffer = UniformBuffer->GetParameters()->IndirectionBuffer;
			Parameters.SampleBuffer = UniformBuffer->GetParameters()->SampleBuffer;
			Parameters.RadianceTexture = UniformBuffer->GetParameters()->RadianceTexture;
			Parameters.TextureSampler = UniformBuffer->GetParameters()->TextureSampler;
		}
		else
		{
			Parameters.NumShadowMatrices = 1;
			for (int32 i = 0; i < Parameters.NumShadowMatrices; ++i)
			{
				Parameters.TranslatedWorldToShadow[i] = FMatrix44f::Identity;
				Parameters.TranslatedWorldOrigin[i] = FVector3f::ZeroVector;
				Parameters.TranslatedWorldPlane[i] = FVector4f::Zero();
			}

			Parameters.Resolution = FIntPoint::ZeroValue;
			Parameters.DownsampleFactor = 1.0f;
			Parameters.MaxSampleCount = 0;
			Parameters.bIsEmpty = true;
			Parameters.bIsDirectionalLight = false;
			Parameters.LinkedListBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMLinkedListPackedData)));
			Parameters.IndirectionBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMIndirectionPackedData)));
			Parameters.SampleBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMSamplePackedData)));
			Parameters.RadianceTexture = GraphBuilder.CreateSRV(GSystemTextures.GetBlackDummy(GraphBuilder));
			Parameters.TextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		}

		return Parameters;
	}

	// Beer Shadow Map Management Functions
	TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters> CreateEmptyBeerShadowMapUniformBuffer(
		FRDGBuilder& GraphBuilder
	)
	{
		FBeerShadowMapUniformBufferParameters* UniformBufferParameters = GraphBuilder.AllocParameters<FBeerShadowMapUniformBufferParameters>();
		{
			UniformBufferParameters->NumShadowMatrices = 1;
			for (int32 i = 0; i < 6; ++i)
			{
				UniformBufferParameters->TranslatedWorldToShadow[i] = FMatrix44f::Identity;
				UniformBufferParameters->TranslatedWorldOrigin[i] = FVector3f::ZeroVector;
				UniformBufferParameters->TranslatedWorldPlane[i] = FVector4f::Zero();
				UniformBufferParameters->SplitDepths[i] = FVector4f::Zero();
			}

			UniformBufferParameters->Resolution = FIntPoint::ZeroValue;
			UniformBufferParameters->DownsampleFactor = 1.0f;
			UniformBufferParameters->MaxSampleCount = 0;
			UniformBufferParameters->bIsEmpty = true;
			UniformBufferParameters->bIsDirectionalLight = false;
			UniformBufferParameters->BeerShadowMapTexture = GraphBuilder.CreateSRV(GSystemTextures.GetBlackDummy(GraphBuilder));
			UniformBufferParameters->RadianceTexture = GraphBuilder.CreateSRV(GSystemTextures.GetBlackDummy(GraphBuilder));
			UniformBufferParameters->TextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		}

		return GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
	}

	void DestroyBeerShadowMapUniformBuffer(
		TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters>& BeerShadowMapUniformBuffer
	)
	{
		BeerShadowMapUniformBuffer = nullptr;
	}

	TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters> GetBeerShadowMapUniformBuffer(
		FRDGBuilder& GraphBuilder,
		FSceneViewState* ViewState,
		const FLightSceneInfo* LightSceneInfo
	)
	{
		bool bIsEnabled = HeterogeneousVolumes::GetShadowType() == HeterogeneousVolumes::EShadowType::BeerShadowMap;
		if (ViewState && LightSceneInfo && bIsEnabled)
		{
			TRDGUniformBufferRef<FBeerShadowMapUniformBufferParameters>* BeerShadowMapUniformBuffer = ViewState->BeerShadowMapUniformBufferMap.Find(LightSceneInfo->Id);
			if (BeerShadowMapUniformBuffer != nullptr)
			{
				return *BeerShadowMapUniformBuffer;
			}
		}

		return CreateEmptyBeerShadowMapUniformBuffer(GraphBuilder);
	}

} // namespace HeterogeneousVolumes

void CompressVolumetricShadowMap(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FIntVector GroupCount,
	// Input
	FIntPoint ShadowMapResolution,
	uint32 MaxSampleCount,
	FRDGBufferRef VolumetricShadowLinkedListBuffer,
	// Output
	FRDGBufferRef& VolumetricShadowIndirectionBuffer,
	FRDGBufferRef& VolumetricShadowTransmittanceBuffer
)
{
	int32 VolumetricShadowPixelCount = ShadowMapResolution.X * ShadowMapResolution.Y * GroupCount.Z;
	FRDGBufferRef VolumetricShadowIndirectionAllocatorBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1),
		TEXT("HeterogeneousVolume.VolumetricShadowIndirectionAllocatorBuffer")
	);
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(VolumetricShadowIndirectionAllocatorBuffer, PF_R32_UINT), 0);

	VolumetricShadowIndirectionBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMIndirectionPackedData), VolumetricShadowPixelCount),
		TEXT("HeterogeneousVolume.VolumetricShadowIndirectionBuffer")
	);

	int32 TopLevelSampleCount = FMath::CeilToInt(MaxSampleCount / 4.0);
	int32 VolumetricShadowTransmittanceMaxCount = VolumetricShadowPixelCount * (Align(TopLevelSampleCount, 4) + Align(MaxSampleCount, 4));
	VolumetricShadowTransmittanceBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMSamplePackedData), VolumetricShadowTransmittanceMaxCount),
		TEXT("HeterogeneousVolume.VolumetricShadowTransmittanceBuffer")
	);

	FCompressVolumetricShadowMapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompressVolumetricShadowMapCS::FParameters>();
	{
		// Input
		PassParameters->ShadowResolution = ShadowMapResolution;
		PassParameters->MaxSampleCount = MaxSampleCount;
		PassParameters->VolumetricShadowLinkedListBuffer = GraphBuilder.CreateSRV(VolumetricShadowLinkedListBuffer);

		// Output
		PassParameters->RWVolumetricShadowIndirectionAllocatorBuffer = GraphBuilder.CreateUAV(VolumetricShadowIndirectionAllocatorBuffer, PF_R32_UINT);
		PassParameters->RWVolumetricShadowIndirectionBuffer = GraphBuilder.CreateUAV(VolumetricShadowIndirectionBuffer);
		PassParameters->RWVolumetricShadowTransmittanceBuffer = GraphBuilder.CreateUAV(VolumetricShadowTransmittanceBuffer);
	}

	FCompressVolumetricShadowMapCS::FPermutationDomain PermutationVector;
	TShaderRef<FCompressVolumetricShadowMapCS> ComputeShader = View.ShaderMap->GetShader<FCompressVolumetricShadowMapCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		// Add Light name..
		RDG_EVENT_NAME("CompressVolumetricShadowMapCS"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

void CombineVolumetricShadowMap(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FIntVector GroupCount,
	// Input
	uint32 LightType,
	FIntPoint ShadowMapResolution,
	uint32 MaxSampleCount,
	FRDGBufferRef VolumetricShadowLinkedListBuffer0,
	FRDGBufferRef VolumetricShadowLinkedListBuffer1,
	// Output
	FRDGBufferRef& VolumetricShadowLinkedListBuffer
)
{
	bool bIsMultiProjection = (LightType == LightType_Point) || (LightType == LightType_Rect);

	MaxSampleCount = HeterogeneousVolumes::GetShadowMaxSampleCount();
	int32 VolumetricShadowLinkedListElementCount = ShadowMapResolution.X * ShadowMapResolution.Y * MaxSampleCount;
	VolumetricShadowLinkedListElementCount *= GroupCount.Z;

	VolumetricShadowLinkedListBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMLinkedListPackedData), VolumetricShadowLinkedListElementCount),
		TEXT("HeterogeneousVolume.VolumetricShadowLinkedListBuffer")
	);

	FCombineVolumetricShadowMapsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCombineVolumetricShadowMapsCS::FParameters>();
	{
		// Input
		PassParameters->ShadowResolution = ShadowMapResolution;
		PassParameters->VolumetricShadowLinkedListBuffer0 = GraphBuilder.CreateSRV(VolumetricShadowLinkedListBuffer0);
		PassParameters->VolumetricShadowLinkedListBuffer1 = GraphBuilder.CreateSRV(VolumetricShadowLinkedListBuffer1);

		PassParameters->MaxSampleCount = MaxSampleCount;
		PassParameters->AbsoluteErrorThreshold = HeterogeneousVolumes::GetShadowAbsoluteErrorThreshold();
		PassParameters->RelativeErrorThreshold = HeterogeneousVolumes::GetShadowRelativeErrorThreshold();

		// Output
		PassParameters->RWVolumetricShadowLinkedListBuffer = GraphBuilder.CreateUAV(VolumetricShadowLinkedListBuffer);
	}

	FCombineVolumetricShadowMapsCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FCombineVolumetricShadowMapsCS::FUseAVSMCompression>(HeterogeneousVolumes::UseAVSMCompression());
	TShaderRef<FCombineVolumetricShadowMapsCS> ComputeShader = View.ShaderMap->GetShader<FCombineVolumetricShadowMapsCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		// Add Light name..
		RDG_EVENT_NAME("CombineVolumetricShadowMapsCS"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

void RenderVolumetricShadowMapForLightWithVoxelGrid(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Light data
	bool bApplyEmissionAndTransmittance,
	bool bApplyDirectLighting,
	bool bApplyShadowTransmittance,
	uint32 LightType,
	const FLightSceneInfo* LightSceneInfo,
	// Shadow data
	const FVisibleLightInfo* VisibleLightInfo,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Volume data
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer,
	// Output
	bool& bIsDirectionalLight,
	FVector3f* TranslatedWorldOrigin,
	FVector4f* TranslatedWorldPlane,
	FMatrix44f* TranslatedWorldToShadow,
	FIntVector& GroupCount,
	int32& NumShadowMatrices,
	FIntPoint& ShadowMapResolution,
	uint32& MaxSampleCount,
	FRDGTextureRef& BeerShadowMapTexture,
	FRDGBufferRef& VolumetricShadowLinkedListBuffer
)
{
	check(LightSceneInfo);
	check(VisibleLightInfo);

	const FProjectedShadowInfo* ProjectedShadowInfo = HeterogeneousVolumes::GetProjectedShadowInfo(VisibleLightInfo, 0);
	check(ProjectedShadowInfo != NULL)

	ShadowMapResolution = HeterogeneousVolumes::GetShadowMapResolution();

	bool bIsMultiProjection = (LightType == LightType_Point) || (LightType == LightType_Rect);
	GroupCount = FIntVector(1);
	GroupCount.X = FMath::DivideAndRoundUp(ShadowMapResolution.X, FRenderVolumetricShadowMapForLightWithVoxelGridCS::GetThreadGroupSize2D());
	GroupCount.Y = FMath::DivideAndRoundUp(ShadowMapResolution.Y, FRenderVolumetricShadowMapForLightWithVoxelGridCS::GetThreadGroupSize2D());
	GroupCount.Z = bIsMultiProjection ? 6 : 1;

	// TODO: Allocate debug data
	int32 BufferSize = ShadowMapResolution.X * ShadowMapResolution.Y;
	if (bIsMultiProjection)
	{
		BufferSize *= 6;
	}
	FRDGBufferRef DebugBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FVolumetricShadowMapDebugData), BufferSize),
		TEXT("HeterogeneousVolume.RenderDebugData")
	);

	FRDGBufferRef VolumetricShadowLinkedListAllocatorBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1),
		TEXT("HeterogeneousVolume.VolumetricShadowLinkedListAllocatorBuffer")
	);

	MaxSampleCount = HeterogeneousVolumes::GetShadowMaxSampleCount();
	int32 VolumetricShadowLinkedListElementCount = ShadowMapResolution.X * ShadowMapResolution.Y * MaxSampleCount;
	if (bIsMultiProjection)
	{
		VolumetricShadowLinkedListElementCount *= 6;
	}
	VolumetricShadowLinkedListBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMLinkedListPackedData), VolumetricShadowLinkedListElementCount),
		TEXT("HeterogeneousVolume.VolumetricShadowLinkedListBuffer")
	);

	const float RelativeErrorThreshold = HeterogeneousVolumes::GetShadowRelativeErrorThreshold();

	// Initialize allocator to contain 1-spp
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(VolumetricShadowLinkedListAllocatorBuffer, PF_R32_UINT), ShadowMapResolution.X * ShadowMapResolution.Y);

	FDeferredLightUniformStruct DeferredLightUniform;
	if (bApplyDirectLighting && (LightSceneInfo != nullptr))
	{
		DeferredLightUniform = GetDeferredLightParameters(View, *LightSceneInfo);
	}
	TUniformBufferRef<FDeferredLightUniformStruct> DeferredLightUB = CreateUniformBufferImmediate(DeferredLightUniform, UniformBuffer_SingleDraw);

	FRenderVolumetricShadowMapForLightWithVoxelGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderVolumetricShadowMapForLightWithVoxelGridCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
		FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
		PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

		// Ray Data
		PassParameters->ShadowStepSize = HeterogeneousVolumes::GetStepSizeForShadows();
		PassParameters->ShadowStepFactor = 1.0;
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetMaxStepCount();
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();
		PassParameters->StochasticFilteringMode = static_cast<int32>(HeterogeneousVolumes::GetStochasticFilteringMode());

		PassParameters->NumShadowMatrices = ProjectedShadowInfo->OnePassShadowViewProjectionMatrices.Num();
		if (PassParameters->NumShadowMatrices > 0)
		{
			FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
			FMatrix TranslatedWorldToWorldMatrix = FTranslationMatrix(-PreViewTranslation);
			FVector LightPosition = LightSceneInfo->Proxy->GetPosition();
			FMatrix WorldToLightMatrix = LightSceneInfo->Proxy->GetWorldToLight();

			// Remove light rotation when building the RectLight projections..
			FMatrix RotationalAdjustmentMatrix = FMatrix::Identity;
			if (LightType == LIGHT_TYPE_RECT)
			{
				FVector LightDirection = LightSceneInfo->Proxy->GetDirection().GetSafeNormal();
				RotationalAdjustmentMatrix = FRotationMatrix(LightDirection.Rotation());
			}

			FMatrix ViewMatrix[] = {
				FLookFromMatrix(FVector::Zero(), FVector(-1, 0, 0), FVector(0, 0, 1)),
				FLookFromMatrix(FVector::Zero(), FVector(1, 0, 0), FVector(0, 0, 1)),
				FLookFromMatrix(FVector::Zero(), FVector(0, -1, 0), FVector(0, 0, 1)),
				FLookFromMatrix(FVector::Zero(), FVector(0, 1, 0), FVector(0, 0, 1)),
				FLookFromMatrix(FVector::Zero(), FVector(0, 0, -1), FVector(1, 0, 0)),
				FLookFromMatrix(FVector::Zero(), FVector(0, 0, 1), FVector(1, 0, 0))
			};

			FMatrix PerspectiveMatrix = FPerspectiveMatrix(
				PI / 4.0f,
				ShadowMapResolution.X,
				ShadowMapResolution.Y,
				1.0,
				LightSceneInfo->Proxy->GetRadius()
			);

			FMatrix ScreenMatrix = FScaleMatrix(FVector(0.5, -0.5, -0.5)) * FTranslationMatrix(FVector(0.5, 0.5, 0.5));

			for (int32 i = 0; i < PassParameters->NumShadowMatrices; ++i)
			{
				FMatrix WorldToShadowMatrix = WorldToLightMatrix * RotationalAdjustmentMatrix * ViewMatrix[i] * PerspectiveMatrix * ScreenMatrix;
				PassParameters->TranslatedWorldToShadow[i] = FMatrix44f(TranslatedWorldToWorldMatrix * WorldToShadowMatrix);
				PassParameters->ShadowToTranslatedWorld[i] = PassParameters->TranslatedWorldToShadow[i].Inverse();
			}
			PassParameters->TranslatedWorldOrigin = FVector3f(PreViewTranslation + LightPosition);
		}
		else if (LightType == LightType_Directional)
		{
			bIsDirectionalLight = true;
			// Build orthographic projection centered around volume..
			FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
			FMatrix TranslatedWorldToWorldMatrix = FTranslationMatrix(-PreViewTranslation);

			FVector WorldBoundsMin = FVector(OrthoGridUniformBuffer->GetParameters()->TopLevelGridWorldBoundsMin);
			FVector WorldBoundsMax = FVector(OrthoGridUniformBuffer->GetParameters()->TopLevelGridWorldBoundsMax);
			FBoxSphereBounds VolumeBounds(FBox(WorldBoundsMin, WorldBoundsMax));
			FMatrix TranslationMatrix = FTranslationMatrix(-VolumeBounds.Origin);

			FVector LightDirection = LightSceneInfo->Proxy->GetDirection().GetSafeNormal();
			FMatrix RotationMatrix = FInverseRotationMatrix(LightDirection.Rotation());
			FMatrix ScaleMatrix = FScaleMatrix(FVector(1.0 / VolumeBounds.SphereRadius));

			const FMatrix FaceMatrix(
				FPlane(0, 0, 1, 0),
				FPlane(0, 1, 0, 0),
				FPlane(-1, 0, 0, 0),
				FPlane(0, 0, 0, 1));

			// Invert Z to match reverse-Z for the rest of the shadow types!
			FMatrix ScreenMatrix = FScaleMatrix(FVector(0.5, -0.5, -0.5)) * FTranslationMatrix(FVector(0.5, 0.5, 0.5));
			FMatrix WorldToShadowMatrix = TranslationMatrix * RotationMatrix * ScaleMatrix * FaceMatrix * ScreenMatrix;
			FMatrix44f TranslatedWorldToShadowMatrix = FMatrix44f(TranslatedWorldToWorldMatrix * WorldToShadowMatrix);

			PassParameters->NumShadowMatrices = 1;
			PassParameters->TranslatedWorldToShadow[0] = TranslatedWorldToShadowMatrix;
			PassParameters->ShadowToTranslatedWorld[0] = TranslatedWorldToShadowMatrix.Inverse();
			PassParameters->TranslatedWorldOrigin = FVector3f(PreViewTranslation + VolumeBounds.Origin - LightDirection * VolumeBounds.SphereRadius);
		}
		else
		{
			FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
			FMatrix TranslatedWorldToWorldMatrix = FTranslationMatrix(-PreViewTranslation);
			FVector4f ShadowmapMinMax = FVector4f::Zero();
			FMatrix WorldToShadowMatrix = ProjectedShadowInfo->GetWorldToShadowMatrix(ShadowmapMinMax);
			FMatrix44f TranslatedWorldToShadowMatrix = FMatrix44f(TranslatedWorldToWorldMatrix * WorldToShadowMatrix);

			PassParameters->NumShadowMatrices = 1;
			PassParameters->TranslatedWorldToShadow[0] = TranslatedWorldToShadowMatrix;
			PassParameters->ShadowToTranslatedWorld[0] = TranslatedWorldToShadowMatrix.Inverse();
			PassParameters->TranslatedWorldOrigin = FVector3f(View.ViewMatrices.GetPreViewTranslation() - ProjectedShadowInfo->PreShadowTranslation);
		}

		TranslatedWorldOrigin[0] = PassParameters->TranslatedWorldOrigin;
		NumShadowMatrices = PassParameters->NumShadowMatrices;
		for (int32 i = 0; i < PassParameters->NumShadowMatrices; ++i)
		{
			TranslatedWorldToShadow[i] = PassParameters->TranslatedWorldToShadow[i];
		}

		FVector LightDirection = LightSceneInfo->Proxy->GetDirection().GetSafeNormal();
		float W = -FVector3f::DotProduct(TranslatedWorldOrigin[0], FVector3f(LightDirection));
		TranslatedWorldPlane[0] = FVector4f(LightDirection.X, LightDirection.Y, LightDirection.Z, W);

		PassParameters->ShadowResolution = ShadowMapResolution;
		PassParameters->MaxSampleCount = MaxSampleCount;
		PassParameters->AbsoluteErrorThreshold = HeterogeneousVolumes::GetShadowAbsoluteErrorThreshold();
		PassParameters->RelativeErrorThreshold = RelativeErrorThreshold;

		// Volume data
		PassParameters->OrthoGridUniformBuffer = OrthoGridUniformBuffer;
		PassParameters->FrustumGridUniformBuffer = FrustumGridUniformBuffer;

		// Dispatch data
		PassParameters->GroupCount = GroupCount;
		PassParameters->ShadowDebugTweak = CVarHeterogeneousVolumesShadowDebugTweak.GetValueOnRenderThread();

		// Output
		PassParameters->RWVolumetricShadowLinkedListAllocatorBuffer = GraphBuilder.CreateUAV(VolumetricShadowLinkedListAllocatorBuffer, PF_R32_UINT);
		PassParameters->RWVolumetricShadowLinkedListBuffer = GraphBuilder.CreateUAV(VolumetricShadowLinkedListBuffer);
		PassParameters->RWBeerShadowMapTexture = GraphBuilder.CreateUAV(BeerShadowMapTexture);
		PassParameters->RWDebugBuffer = GraphBuilder.CreateUAV(DebugBuffer);
	}

	FRenderVolumetricShadowMapForLightWithVoxelGridCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRenderVolumetricShadowMapForLightWithVoxelGridCS::FUseAVSMCompression>(HeterogeneousVolumes::UseAVSMCompression());
	PermutationVector.Set<FRenderVolumetricShadowMapForLightWithVoxelGridCS::FIsOfflineRender>(View.bIsOfflineRender);
	TShaderRef<FRenderVolumetricShadowMapForLightWithVoxelGridCS> ComputeShader = View.ShaderMap->GetShader<FRenderVolumetricShadowMapForLightWithVoxelGridCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		// Add Light name..
		RDG_EVENT_NAME("RenderVolumetricShadowMapForLightWithVoxelGridCS"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

void RenderVolumetricShadowMapForCameraWithVoxelGrid(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Volume data
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer,
	// Output
	FVector3f& TranslatedWorldOrigin,
	FMatrix44f& TranslatedWorldToShadow,
	FIntVector& GroupCount,
	FIntPoint& ShadowMapResolution,
	uint32& MaxSampleCount,
	FRDGTextureRef& BeerShadowMapTexture,
	FRDGBufferRef& VolumetricShadowLinkedListBuffer
)
{
	ShadowMapResolution = HeterogeneousVolumes::GetDownsampledResolution(View.ViewRect.Size(), HeterogeneousVolumes::GetCameraDownsampleFactor());

	GroupCount = FIntVector(1);
	GroupCount.X = FMath::DivideAndRoundUp(ShadowMapResolution.X, FRenderVolumetricShadowMapForLightWithVoxelGridCS::GetThreadGroupSize2D());
	GroupCount.Y = FMath::DivideAndRoundUp(ShadowMapResolution.Y, FRenderVolumetricShadowMapForLightWithVoxelGridCS::GetThreadGroupSize2D());

	// TODO: Allocate debug data
	int32 BufferSize = ShadowMapResolution.X * ShadowMapResolution.Y;
	FRDGBufferRef DebugBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FVolumetricShadowMapDebugData), BufferSize),
		TEXT("HeterogeneousVolume.RenderDebugData")
	);

	FRDGBufferRef VolumetricShadowLinkedListAllocatorBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1),
		TEXT("HeterogeneousVolume.CameraAVSM.LinkedListAllocatorBuffer")
	);

	MaxSampleCount = HeterogeneousVolumes::GetShadowMaxSampleCount();
	int32 VolumetricShadowLinkedListElementCount = ShadowMapResolution.X * ShadowMapResolution.Y * MaxSampleCount;
	VolumetricShadowLinkedListBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateStructuredDesc(sizeof(FAVSMLinkedListPackedData), VolumetricShadowLinkedListElementCount),
		TEXT("HeterogeneousVolume.CameraAVSM.LinkedListBuffer")
	);

	const float RelativeErrorThreshold = HeterogeneousVolumes::GetShadowRelativeErrorThreshold();

	// Initialize allocator to contain 1-spp
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(VolumetricShadowLinkedListAllocatorBuffer, PF_R32_UINT), ShadowMapResolution.X * ShadowMapResolution.Y);

	// TODO: Use the frustum grid bounds instead of re-based trace-distance bounds
	// Intersect TopLevelGridBounds with light ray to get appropriate culling distances.

	FMatrix ViewToClip = FPerspectiveMatrix(
		FMath::DegreesToRadians(View.FOV * 0.5),
		ShadowMapResolution.X,
		ShadowMapResolution.Y,
		1.0,
		HeterogeneousVolumes::GetMaxTraceDistance()
	);
	FMatrix ClipToView = ViewToClip.Inverse();
	FMatrix ScreenMatrix = FScaleMatrix(FVector(0.5, -0.5, -0.5)) * FTranslationMatrix(FVector(0.5, 0.5, 0.5));
	TranslatedWorldToShadow = FMatrix44f(View.ViewMatrices.GetTranslatedViewMatrix() * ViewToClip * ScreenMatrix);
	FMatrix44f ShadowToTranslatedWorld = TranslatedWorldToShadow.Inverse();
	TranslatedWorldOrigin = ShadowToTranslatedWorld.GetOrigin();

	FRenderVolumetricShadowMapForLightWithVoxelGridCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRenderVolumetricShadowMapForLightWithVoxelGridCS::FParameters>();
	{
		// Scene data
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
		FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
		PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

		// Ray Data
		PassParameters->ShadowStepSize = HeterogeneousVolumes::GetStepSizeForShadows();
		PassParameters->ShadowStepFactor = 1.0;
		PassParameters->MaxTraceDistance = HeterogeneousVolumes::GetMaxTraceDistance();
		PassParameters->MaxStepCount = HeterogeneousVolumes::GetMaxStepCount();
		PassParameters->bJitter = HeterogeneousVolumes::ShouldJitter();

		PassParameters->ShadowResolution = ShadowMapResolution;
		PassParameters->NumShadowMatrices = 1;
		PassParameters->TranslatedWorldToShadow[0] = TranslatedWorldToShadow;
		PassParameters->ShadowToTranslatedWorld[0] = ShadowToTranslatedWorld;
		
		PassParameters->MaxSampleCount = MaxSampleCount;
		PassParameters->AbsoluteErrorThreshold = HeterogeneousVolumes::GetShadowAbsoluteErrorThreshold();
		PassParameters->RelativeErrorThreshold = RelativeErrorThreshold;

		// Volume data
		PassParameters->OrthoGridUniformBuffer = OrthoGridUniformBuffer;
		PassParameters->FrustumGridUniformBuffer = FrustumGridUniformBuffer;

		// Dispatch data
		PassParameters->GroupCount = GroupCount;

		// Output
		PassParameters->RWVolumetricShadowLinkedListAllocatorBuffer = GraphBuilder.CreateUAV(VolumetricShadowLinkedListAllocatorBuffer, PF_R32_UINT);
		PassParameters->RWVolumetricShadowLinkedListBuffer = GraphBuilder.CreateUAV(VolumetricShadowLinkedListBuffer);
		PassParameters->RWBeerShadowMapTexture = GraphBuilder.CreateUAV(BeerShadowMapTexture);
		PassParameters->RWDebugBuffer = GraphBuilder.CreateUAV(DebugBuffer);
	}

	bool bUseCustomProjection = false;
	FRenderVolumetricShadowMapForLightWithVoxelGridCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRenderVolumetricShadowMapForLightWithVoxelGridCS::FUseAVSMCompression>(HeterogeneousVolumes::UseAVSMCompression());
	TShaderRef<FRenderVolumetricShadowMapForLightWithVoxelGridCS> ComputeShader = View.ShaderMap->GetShader<FRenderVolumetricShadowMapForLightWithVoxelGridCS>(PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		// Add Light name..
		RDG_EVENT_NAME("RenderVolumetricShadowMapForCameraWithVoxelGridCS"),
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		ComputeShader,
		PassParameters,
		GroupCount
	);
}

void RenderAdaptiveVolumetricShadowMapWithVoxelGrid(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Shadow data
	const FVisibleLightInfoArray& VisibleLightInfos,
	const FVirtualShadowMapArray& VirtualShadowMapArray,
	// Volume data
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer
)
{
	RDG_EVENT_SCOPE(GraphBuilder, "Adaptive Volumetric Shadow Maps");
	bool bShouldRenderShadowMaps = !View.ViewRect.IsEmpty() &&
		(OrthoGridUniformBuffer->GetParameters()->bUseOrthoGrid || FrustumGridUniformBuffer->GetParameters()->bUseFrustumGrid);

	// Light culling
	TArray<FLightSceneInfoCompact, TInlineAllocator<64>> LightSceneInfoCompact = HeterogeneousVolumes::GatherShadowLights(Scene, View.ViewLightingChannelMask);

	// Light loop:
	int32 NumPasses = LightSceneInfoCompact.Num();
	for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
	{
		bool bApplyEmissionAndTransmittance = (PassIndex == (NumPasses - 1));
		bool bApplyDirectLighting = !LightSceneInfoCompact.IsEmpty();
		bool bApplyShadowTransmittance = false;

		uint32 LightType = 0;
		FLightSceneInfo* LightSceneInfo = nullptr;
		const FVisibleLightInfo* VisibleLightInfo = nullptr;
		if (bApplyDirectLighting)
		{
			LightType = LightSceneInfoCompact[PassIndex].LightType;
			LightSceneInfo = LightSceneInfoCompact[PassIndex].LightSceneInfo;
			check(LightSceneInfo != nullptr);

			bApplyDirectLighting = (LightSceneInfo != nullptr);
			bool bDynamicallyShadowed = false;
			if (LightSceneInfo)
			{
				VisibleLightInfo = &VisibleLightInfos[LightSceneInfo->Id];
				bApplyShadowTransmittance = LightSceneInfo->Proxy && LightSceneInfo->Proxy->CastsVolumetricShadow();
				bDynamicallyShadowed = HeterogeneousVolumes::IsDynamicShadow(VisibleLightInfo);
			}

			TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters> AdaptiveVolumetricShadowMapUniformBuffer;
			bool bCreateShadowMap = bShouldRenderShadowMaps && bApplyShadowTransmittance && bDynamicallyShadowed && !ShouldRenderRayTracingShadowsForLight(*View.Family, LightSceneInfoCompact[PassIndex]);
			if (bCreateShadowMap)
			{
				FString LightName;
				FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightName);
				RDG_EVENT_SCOPE(GraphBuilder, "%s", *LightName);

				FRDGTextureDesc Desc = SceneTextures.Color.Target->Desc;
				Desc.Format = PF_FloatRGBA;
				Desc.Flags &= ~(TexCreate_FastVRAM);
				FRDGTextureRef BeerShadowMapTexture = GraphBuilder.CreateTexture(Desc, TEXT("BeerShadowMapTexture"));
				AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(BeerShadowMapTexture), FLinearColor::Transparent);

				bool bIsDirectionalLight = false;
				FVector3f TranslatedWorldOrigin[6];
				FVector4f TranslatedWorldPlane[6];
				FMatrix44f TranslatedWorldToShadow[6];
				FVector4f SplitDepths[6];
				for (int i = 0; i < 6; ++i)
				{
					TranslatedWorldOrigin[i] = FVector3f::Zero();
					TranslatedWorldPlane[i] = FVector4f::Zero();
					TranslatedWorldToShadow[i] = FMatrix44f::Identity;
					SplitDepths[i] = FVector4f::Zero();
				}

				FIntVector GroupCount = FIntVector::ZeroValue;
				int32 NumShadowMatrices = 0;
				FIntPoint VolumetricShadowMapResolution = FIntPoint::NoneValue;
				uint32 VolumetricShadowMapMaxSampleCount = 0;
				FRDGBufferRef VolumetricShadowMapLinkedListBuffer;
				RenderVolumetricShadowMapForLightWithVoxelGrid(
					GraphBuilder,
					// Scene data
					SceneTextures,
					Scene,
					View,
					// Light data
					bApplyEmissionAndTransmittance,
					bApplyDirectLighting,
					bApplyShadowTransmittance,
					LightType,
					LightSceneInfo,
					// Shadow data
					VisibleLightInfo,
					VirtualShadowMapArray,
					// Volume data
					OrthoGridUniformBuffer,
					FrustumGridUniformBuffer,
					// Output
					bIsDirectionalLight,
					TranslatedWorldOrigin,
					TranslatedWorldPlane,
					TranslatedWorldToShadow,
					GroupCount,
					NumShadowMatrices,
					VolumetricShadowMapResolution,
					VolumetricShadowMapMaxSampleCount,
					BeerShadowMapTexture,
					VolumetricShadowMapLinkedListBuffer
				);

				FRDGBufferRef VolumetricShadowMapIndirectionBuffer;
				FRDGBufferRef VolumetricShadowMapSampleBuffer;
				CompressVolumetricShadowMap(
					GraphBuilder,
					View,
					GroupCount,
					VolumetricShadowMapResolution,
					VolumetricShadowMapMaxSampleCount,
					VolumetricShadowMapLinkedListBuffer,
					VolumetricShadowMapIndirectionBuffer,
					VolumetricShadowMapSampleBuffer
				);

				float DownsampleFactor = 1.0f;
				CreateAdaptiveVolumetricShadowMapUniformBuffer(
					GraphBuilder,
					TranslatedWorldOrigin,
					TranslatedWorldPlane,
					TranslatedWorldToShadow,
					SplitDepths,
					VolumetricShadowMapResolution,
					DownsampleFactor,
					NumShadowMatrices,
					VolumetricShadowMapMaxSampleCount,
					bIsDirectionalLight,
					VolumetricShadowMapLinkedListBuffer,
					VolumetricShadowMapIndirectionBuffer,
					VolumetricShadowMapSampleBuffer,
					AdaptiveVolumetricShadowMapUniformBuffer
				);
			}
			else
			{
				AdaptiveVolumetricShadowMapUniformBuffer = HeterogeneousVolumes::CreateEmptyAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder);
			}

			if (View.ViewState)
			{
				TRDGUniformBufferRef<FAdaptiveVolumetricShadowMapUniformBufferParameters>& AdaptiveVolumetricShadowMap = View.ViewState->AdaptiveVolumetricShadowMapUniformBufferMap.FindOrAdd(LightSceneInfo->Id);
				AdaptiveVolumetricShadowMap = AdaptiveVolumetricShadowMapUniformBuffer;
			}
		}
	}
}

void RenderAdaptiveVolumetricCameraMapWithVoxelGrid(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	FScene* Scene,
	FViewInfo& View,
	// Volume data
	const TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters>& OrthoGridUniformBuffer,
	const TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters>& FrustumGridUniformBuffer
)
{
	if (View.ViewState == nullptr)
	{
		return;
	}
	RDG_EVENT_SCOPE(GraphBuilder, "Adaptive Volumetric Camera Map");

	int32 NumShadowMatrices = 1;
	FMatrix44f TranslatedWorldToShadow[] = {
		FMatrix44f::Identity
	};
	FVector3f TranslatedWorldOrigin[] = {
		FVector3f::ZeroVector
	};
	FVector4f TranslatedWorldPlane[] = {
		FVector4f::Zero()
	};
	FVector4f SplitDepths[] = {
		FVector4f::Zero()
	};
	FIntPoint VolumetricShadowMapResolution = FIntPoint::NoneValue;
	uint32 VolumetricShadowMapMaxSampleCount = 0;
	FRDGBufferRef VolumetricShadowMapLinkedListBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMLinkedListPackedData));;
	FRDGBufferRef VolumetricShadowMapIndirectionBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMIndirectionPackedData));
	FRDGBufferRef VolumetricShadowMapSampleBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FAVSMSamplePackedData));

	bool bShouldRenderCameraMap = !View.ViewRect.IsEmpty()
		&& (OrthoGridUniformBuffer->GetParameters()->bUseOrthoGrid || FrustumGridUniformBuffer->GetParameters()->bUseFrustumGrid);

	if (bShouldRenderCameraMap)
	{
		FRDGTextureDesc Desc = SceneTextures.Color.Target->Desc;
		Desc.Format = PF_FloatRGBA;
		Desc.Flags &= ~(TexCreate_FastVRAM);
		FRDGTextureRef CameraShadowMapTexture = GraphBuilder.CreateTexture(Desc, TEXT("CameraShadowMapTexture"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(CameraShadowMapTexture), FLinearColor::Transparent);

		FIntVector GroupCount = FIntVector::ZeroValue;
		RenderVolumetricShadowMapForCameraWithVoxelGrid(
			GraphBuilder,
			// Scene data
			SceneTextures,
			Scene,
			View,
			// Volume data
			OrthoGridUniformBuffer,
			FrustumGridUniformBuffer,
			// Output
			TranslatedWorldOrigin[0],
			TranslatedWorldToShadow[0],
			GroupCount,
			VolumetricShadowMapResolution,
			VolumetricShadowMapMaxSampleCount,
			CameraShadowMapTexture,
			VolumetricShadowMapLinkedListBuffer
		);

		CompressVolumetricShadowMap(
			GraphBuilder,
			View,
			GroupCount,
			VolumetricShadowMapResolution,
			VolumetricShadowMapMaxSampleCount,
			VolumetricShadowMapLinkedListBuffer,
			VolumetricShadowMapIndirectionBuffer,
			VolumetricShadowMapSampleBuffer
		);

		bool bIsDirectionalLight = false;
		float DownsampleFactor = HeterogeneousVolumes::GetCameraDownsampleFactor();

		CreateAdaptiveVolumetricShadowMapUniformBufferParameters(
			GraphBuilder,
			TranslatedWorldOrigin,
			TranslatedWorldPlane,
			TranslatedWorldToShadow,
			SplitDepths,
			VolumetricShadowMapResolution,
			DownsampleFactor,
			NumShadowMatrices,
			VolumetricShadowMapMaxSampleCount,
			bIsDirectionalLight,
			VolumetricShadowMapLinkedListBuffer,
			VolumetricShadowMapIndirectionBuffer,
			VolumetricShadowMapSampleBuffer,
			View.ViewState->AdaptiveVolumetricCameraMapUniformBufferParameters
		);
	}
}

class FVisualizeCascadesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeCascadesCS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeCascadesCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Scene data
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)

		// Shadow data
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FAdaptiveVolumetricShadowMapUniformBufferParameters, AVSM)

		// Dispatch data
		SHADER_PARAMETER(int32, DownsampleFactor)

		// Output
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float>, RWVisualizationTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportHeterogeneousVolumes(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(
		const FGlobalShaderPermutationParameters& Parameters,
		FShaderCompilerEnvironment& OutEnvironment
	)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_1D"), GetThreadGroupSize1D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_2D"), GetThreadGroupSize2D());
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_3D"), GetThreadGroupSize3D());

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	static int32 GetThreadGroupSize1D() { return 64; }
	static int32 GetThreadGroupSize2D() { return 8; }
	static int32 GetThreadGroupSize3D() { return 4; }
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeCascadesCS, "/Engine/Private/HeterogeneousVolumes/HeterogeneousVolumesVisualizeCascades.usf", "VisualizeCascadesCS", SF_Compute);

void VisualizeCascades(
	FRDGBuilder& GraphBuilder,
	// Scene data
	const FSceneTextures& SceneTextures,
	const FScene* Scene,
	const FViewInfo& View,
	// Light data
	const FLightSceneInfo* LightSceneInfo,
	// Output
	FRDGTextureRef& VisualizationTexture
)
{
	FRDGTextureDesc VisualizationTextureDesc = SceneTextures.Color.Target->Desc;
	VisualizationTextureDesc.Extent = HeterogeneousVolumes::GetScaledViewRect(View.ViewRect);
	VisualizationTexture = GraphBuilder.CreateTexture(VisualizationTextureDesc, TEXT("HeterogeneousVolumes.VisualizationTexture"));

	FVisualizeCascadesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeCascadesCS::FParameters>();
	{
		// Scene
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);

		// Shadow
		PassParameters->AVSM = HeterogeneousVolumes::GetAdaptiveVolumetricShadowMapUniformBuffer(GraphBuilder, View.ViewState, LightSceneInfo);

		// Dispatch
		PassParameters->DownsampleFactor = HeterogeneousVolumes::GetDownsampleFactor();

		// Output
		PassParameters->RWVisualizationTexture = GraphBuilder.CreateUAV(VisualizationTexture);
	}

	FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(VisualizationTextureDesc.Extent, FVisualizeCascadesCS::GetThreadGroupSize2D());

	TShaderRef<FVisualizeCascadesCS> ComputeShader = View.ShaderMap->GetShader<FVisualizeCascadesCS>();
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("FVisualizeCascadesCS"),
		ComputeShader,
		PassParameters,
		GroupCount
	);
}