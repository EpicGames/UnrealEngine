// Copyright Epic Games, Inc. All Rights Reserved.

#include "IESTextureManager.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "RenderGraphUtils.h"
#include "ShaderPrintParameters.h"
#include "ShaderPrint.h"
#include "RenderingThread.h"
#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "SceneView.h"
#include "RHIDefinitions.h"
#include "SceneRendering.h"
#include "SystemTextures.h"
#include "TextureLayout.h"
#include "CommonRenderResources.h"
#include "ScreenPass.h"
#include "Engine/TextureLightProfile.h"
#include "PixelShaderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Config

static TAutoConsoleVariable<int32> CVarIESTextureResolution(
	TEXT("r.IESAtlas.Resolution"),
	256,
	TEXT("Resolution for storing IES textures.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarIESTextureMaxProfileCount(
	TEXT("r.IESAtlas.MaxProfileCount"),
	32,
	TEXT("The maximum number of IES profiles which can be stored.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarIESTextureDebug(
	TEXT("r.IESAtlas.Debug"),
	0,
	TEXT("Enable IES atlas debug information."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarIESForceUpdate(
	TEXT("r.IESAtlas.ForceUpdate"),
	0,
	TEXT("Force IES atlas to update very frame."),
	ECVF_RenderThreadSafe);

namespace IESAtlas
{

///////////////////////////////////////////////////////////////////////////////////////////////////
// Structs & constants

static const uint32 InvalidSlotIndex = ~0u;
static const uint32 InvalidSliceIndex = ~0u;

struct FAtlasSlot
{
	uint32 Id = InvalidSlotIndex;
	uint32 SliceIndex = InvalidSliceIndex;
	uint32 RefCount = 0;
	FIntPoint CachedResolution = FIntPoint::ZeroValue;
	FTextureReferenceRHIRef SourceTexture = nullptr;

	bool bForceRefresh = false;
	bool IsValid() const { return SourceTexture != nullptr; }
	FRHITexture* GetTextureRHI() const { return SourceTexture->GetReferencedTexture(); }
};

struct FIESTextureManager : public FRenderResource
{
	TRefCountPtr<IPooledRenderTarget> AtlasTexture = nullptr;
	TArray<FAtlasSlot> Slots;
	TQueue<uint32> FreeSlots;
	TQueue<uint32> FreeSlices;

	uint32 RequestedSliceCountWhenTextureAllocated = 0;
	bool bHasPendingAdds = false;
	bool bHasPendingRefreshes = false;

	FIESTextureManager()
		// Initialize feature level to make sure this resource is reset on a feature level changes
		: FRenderResource(GMaxRHIFeatureLevel) 
	{
	}

	virtual void ReleaseRHI()
	{
		AtlasTexture.SafeRelease();
	}
};

// lives on the render thread
TGlobalResource<FIESTextureManager> GIESTextureManager;


///////////////////////////////////////////////////////////////////////////////////////////////////
// Debug

class FIESAtlasDebugInfoCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FIESAtlasDebugInfoCS);
	SHADER_USE_PARAMETER_STRUCT(FIESAtlasDebugInfoCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, AtlasResolution)
		SHADER_PARAMETER(uint32, AtlasSliceCount)
		SHADER_PARAMETER(uint32, TotalSlotCount)
		SHADER_PARAMETER(uint32, ValidSlotCount)
		SHADER_PARAMETER(uint32, UsedSliceCount)
		SHADER_PARAMETER(uint32, bForceUpdate)
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER_SAMPLER(SamplerState, AtlasSampler)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, ValidSliceBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, AtlasTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return ShaderPrint::IsSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SHADER_DEBUG"), 1);
	}
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FIESAtlasDebugInfoCS, "/Engine/Private/IESAtlas.usf", "MainCS", SF_Compute);

static FRDGTextureRef AddIESDebugPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureDesc& OutputDesc)
{
	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);

	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputDesc.Extent, OutputDesc.Format, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource), TEXT("IESAtlas.DebugTexture"));
	FRDGTextureRef AtlasTexture = GIESTextureManager.AtlasTexture ? GraphBuilder.RegisterExternalTexture(GIESTextureManager.AtlasTexture) : GSystemTextures.GetBlackDummy(GraphBuilder);

	const FIntPoint OutputResolution(OutputTexture->Desc.Extent);
	FIESAtlasDebugInfoCS::FParameters* Parameters = GraphBuilder.AllocParameters<FIESAtlasDebugInfoCS::FParameters>();
	Parameters->AtlasResolution = AtlasTexture->Desc.Extent;
	Parameters->AtlasSliceCount = AtlasTexture->Desc.ArraySize;
	Parameters->ValidSlotCount = 0;
	Parameters->UsedSliceCount = 0;
	Parameters->TotalSlotCount = 0;
	Parameters->bForceUpdate = CVarIESForceUpdate.GetValueOnRenderThread() > 0 ? 1u : 0u;
	Parameters->OutputResolution = OutputResolution;
	Parameters->AtlasSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters->AtlasTexture = AtlasTexture;
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);
	Parameters->OutputTexture = GraphBuilder.CreateUAV(OutputTexture);

	TArray<uint32> ValidSliceBuffer;
	const uint32 SliceInfoInBytes = sizeof(uint32);
	ValidSliceBuffer.Init(0, Parameters->AtlasSliceCount);
	for (const FAtlasSlot& Slot : GIESTextureManager.Slots)
	{
		Parameters->TotalSlotCount++;
		if (Slot.IsValid())
		{
			Parameters->ValidSlotCount++;
			if (Slot.SliceIndex != InvalidSliceIndex)
			{
				ValidSliceBuffer[Slot.SliceIndex] = 1;
				Parameters->UsedSliceCount++;
			}
		}
	}
	
	// Upload buffer with valid slice index info
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(SliceInfoInBytes, ValidSliceBuffer.Num()), TEXT("IESAtlas.DebugValidSliceBuffer"));
	GraphBuilder.QueueBufferUpload(Buffer, ValidSliceBuffer.GetData(), ValidSliceBuffer.Num() * SliceInfoInBytes);
	Parameters->ValidSliceBuffer = GraphBuilder.CreateSRV(Buffer, PF_R32_UINT);
	
	TShaderMapRef<FIESAtlasDebugInfoCS> ComputeShader(View.ShaderMap);
	const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(OutputResolution.X, OutputResolution.Y, 1), FIntVector(8, 8, 1));
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("IESAtlas::DebugInfo"), ComputeShader, Parameters, DispatchCount);

	return OutputTexture;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Add pass

static bool UseComputeForAtlasUpdate(const FStaticShaderPlatform ShaderPlatform)
{
	// OpenGL ES does not support writing to R16F images from compute
	return !IsOpenGLPlatform(ShaderPlatform);
}

class FIESAtlasAddTextureCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FIESAtlasAddTextureCS);
	SHADER_USE_PARAMETER_STRUCT(FIESAtlasAddTextureCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE_ARRAY(Texture2D, InTexture, [8])
		SHADER_PARAMETER_ARRAY(FUintVector4, InSliceIndex, [8])
		SHADER_PARAMETER_SAMPLER_ARRAY(SamplerState, InSampler, [8])
		SHADER_PARAMETER(FIntPoint, AtlasResolution)
		SHADER_PARAMETER(uint32, AtlasSliceCount)
		SHADER_PARAMETER(uint32, ValidCount)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray, OutAtlasTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return UseComputeForAtlasUpdate(Parameters.Platform); 
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ADD_TEXTURE_CS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FIESAtlasAddTextureCS, "/Engine/Private/IESAtlas.usf", "MainCS", SF_Compute);

// Insert a texture into the atlas. Compute version
static void AddSlotsPassCS(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const TArray<FAtlasSlot>& Slots,
	FRDGTextureRef& OutAtlas)
{
	FRDGTextureUAVRef AtlasTextureUAV = GraphBuilder.CreateUAV(OutAtlas);
	TShaderMapRef<FIESAtlasAddTextureCS> ComputeShader(ShaderMap);

	// Batch new slots into several passes
	const uint32 SlotCountPerPass = 8u;
	const uint32 PassCount = FMath::DivideAndRoundUp(uint32(Slots.Num()), SlotCountPerPass);
	for (uint32 PassIt = 0; PassIt < PassCount; ++PassIt)
	{
		const uint32 SlotOffset = PassIt * SlotCountPerPass;
		const uint32 SlotCount = SlotCountPerPass * (PassIt+1) <= uint32(Slots.Num()) ? SlotCountPerPass : uint32(Slots.Num()) - (SlotCountPerPass * PassIt);
		
		FIESAtlasAddTextureCS::FParameters* Parameters = GraphBuilder.AllocParameters<FIESAtlasAddTextureCS::FParameters>();
		Parameters->OutAtlasTexture = AtlasTextureUAV;
		Parameters->AtlasResolution = OutAtlas->Desc.Extent;
		Parameters->AtlasSliceCount = OutAtlas->Desc.ArraySize;
		Parameters->ValidCount = SlotCount;
		for (uint32 SlotIt = 0; SlotIt < SlotCountPerPass; ++SlotIt)
		{
			Parameters->InTexture[SlotIt] = GSystemTextures.BlackDummy->GetRHI();
			Parameters->InSliceIndex[SlotIt].X = InvalidSlotIndex;
			Parameters->InSampler[SlotIt] = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}
		for (uint32 SlotIt = 0; SlotIt<SlotCount;++SlotIt)
		{
			const FAtlasSlot& Slot = Slots[SlotOffset + SlotIt];
			check(Slot.SourceTexture);
			Parameters->InTexture[SlotIt] = Slot.GetTextureRHI();
			Parameters->InSliceIndex[SlotIt].X = Slot.SliceIndex;
		}

		const FIntVector DispatchCount = FComputeShaderUtils::GetGroupCount(FIntVector(Parameters->AtlasResolution.X, Parameters->AtlasResolution.Y, SlotCount), FIntVector(8, 8, 1));
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("IESAtlas::AddTexture"), ComputeShader, Parameters, DispatchCount);
	}
}

class FIESAtlasAddTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FIESAtlasAddTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FIESAtlasAddTexturePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSampler)
		SHADER_PARAMETER(FIntPoint, AtlasResolution)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return !UseComputeForAtlasUpdate(Parameters.Platform); 
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ADD_TEXTURE_PS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FIESAtlasAddTexturePS, "/Engine/Private/IESAtlas.usf", "MainPS", SF_Pixel);

// Insert a texture into the atlas. Raster version
static void AddSlotsPassPS(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const TArray<FAtlasSlot>& Slots,
	FRDGTextureRef& OutAtlas)
{
	TShaderMapRef<FIESAtlasAddTexturePS> PixelShader(ShaderMap);

	const uint32 PassCount = Slots.Num();
	for (uint32 PassIt = 0; PassIt < PassCount; ++PassIt)
	{
		const FAtlasSlot& Slot = Slots[PassIt];
		const FIntRect ViewRect(0, 0, OutAtlas->Desc.Extent.X, OutAtlas->Desc.Extent.Y);

		auto* Parameters = GraphBuilder.AllocParameters<FIESAtlasAddTexturePS::FParameters>();
		Parameters->InTexture = Slot.GetTextureRHI();
		Parameters->InSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Parameters->AtlasResolution = OutAtlas->Desc.Extent;
		Parameters->RenderTargets[0] = FRenderTargetBinding(OutAtlas, ERenderTargetLoadAction::ENoAction, 0, Slot.SliceIndex);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("IESAtlas::AddTexturePS (Slot: %d)", PassIt),
			PixelShader,
			Parameters,
			ViewRect);
	}
}

static void AddSlotsPass(
	FRDGBuilder& GraphBuilder,
	const FStaticShaderPlatform ShaderPlatform,
	FGlobalShaderMap* ShaderMap,
	const TArray<FAtlasSlot>& Slots,
	FRDGTextureRef& OutAtlas)
{
	if (UseComputeForAtlasUpdate(ShaderPlatform))
	{
		AddSlotsPassCS(GraphBuilder, ShaderMap, Slots, OutAtlas);
	}
	else
	{
		AddSlotsPassPS(GraphBuilder, ShaderMap, Slots, OutAtlas);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// API

uint32 AddTexture(UTextureLightProfile* In)
{
	check(IsInRenderingThread());

	uint32 SlotIndex = InvalidSlotIndex;
	if (In)
	{
		// 1. Find if the texture already exist (simple linear search assuming a low number of textures)
		bool bFound = false;
		for (FAtlasSlot& Slot : GIESTextureManager.Slots)
		{
			if (Slot.SourceTexture == In->TextureReference.TextureReferenceRHI)
			{
				Slot.RefCount++;
				SlotIndex = Slot.Id;
				bFound = true;
				break;
			}
		}

		// 2. If not found, then add an atlas slot for this new texture
		if (!bFound)
		{
			FAtlasSlot* Slot = nullptr;
			if (GIESTextureManager.FreeSlots.Dequeue(SlotIndex))
			{
				Slot = &GIESTextureManager.Slots[SlotIndex];
			}
			else
			{
				SlotIndex = GIESTextureManager.Slots.Num();
				Slot = &GIESTextureManager.Slots.AddDefaulted_GetRef();
			}

			*Slot = FAtlasSlot();
			Slot->SourceTexture = In->TextureReference.TextureReferenceRHI;
			Slot->Id = SlotIndex;
			Slot->SliceIndex = InvalidSliceIndex;
			Slot->RefCount = 1;
			Slot->CachedResolution = Slot->SourceTexture ? Slot->SourceTexture->GetSizeXY() : FIntPoint::ZeroValue;

			GIESTextureManager.bHasPendingAdds = true;
		}
	}
	return SlotIndex;
}

void RemoveTexture(uint32 InSlotIndex)
{
	check(IsInRenderingThread());

	if (InSlotIndex != InvalidSlotIndex && InSlotIndex < uint32(GIESTextureManager.Slots.Num()))
	{
		// If it is the last light referencing this texture, we retires the atlas slot
		--GIESTextureManager.Slots[InSlotIndex].RefCount;
		if (GIESTextureManager.Slots[InSlotIndex].RefCount == 0)
		{
			// Enqueue free slice index
			if (GIESTextureManager.Slots[InSlotIndex].SliceIndex != InvalidSlotIndex)
			{
				GIESTextureManager.FreeSlices.Enqueue(GIESTextureManager.Slots[InSlotIndex].SliceIndex);
			}

			// Enqueue free slot or shrink slot array
			const int32 SlotCount = GIESTextureManager.Slots.Num();
			if (InSlotIndex == SlotCount-1)
			{
				GIESTextureManager.Slots.SetNum(SlotCount-1);
			}
			else
			{
				GIESTextureManager.FreeSlots.Enqueue(InSlotIndex);
				GIESTextureManager.Slots[InSlotIndex] = FAtlasSlot();
			}
		}
	}
}

static FRDGTextureRef CreateAtlasTexture(FRDGBuilder& GraphBuilder, const FStaticShaderPlatform ShaderPlatform, const FIntPoint& Resolution, uint32 SliceCount)
{
	ETextureCreateFlags CreateAddFlags = UseComputeForAtlasUpdate(ShaderPlatform) ? ETextureCreateFlags::UAV : ETextureCreateFlags::TargetArraySlicesIndependently;
	if (GIsEditor)
	{
		// Make sure UAV flag is always present in Editor environment as target can be used by both compute and raster
		CreateAddFlags |= ETextureCreateFlags::UAV;
	}
	
	return GraphBuilder.CreateTexture(FRDGTextureDesc::Create2DArray(
		Resolution,
		PF_R16F,
		FClearValueBinding::Transparent,
		ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable | CreateAddFlags,
		FMath::Max(1u, SliceCount),
		1),
		TEXT("IES.AtlasTexture"),
		ERDGTextureFlags::MultiFrame);
}

float GetAtlasSlot(uint32 InSlotId)
{
	if (InSlotId < uint32(GIESTextureManager.Slots.Num()) && GIESTextureManager.AtlasTexture)
	{
		uint32 SlotSliceIndex = GIESTextureManager.Slots[InSlotId].SliceIndex;
		return (SlotSliceIndex != InvalidSliceIndex) ? float(SlotSliceIndex) : float(INDEX_NONE);
	}
	else
	{
		return float(INDEX_NONE);
	}
}

static void InvalidateSlots(FIESTextureManager& In, uint32 SliceCount)
{
	// Mark all slots to have an invalid slice index
	for (FAtlasSlot& Slot : GIESTextureManager.Slots)
	{
		Slot.SliceIndex = InvalidSliceIndex;
	}
	In.bHasPendingAdds = true;

	// Refill free slice indices
	In.FreeSlices.Empty();
	for (uint32 It = 0; It < SliceCount; ++It)
	{
		In.FreeSlices.Enqueue(It);
	}
}

static uint32 GetRequestedSlotCount(const FIESTextureManager& In)
{
	uint32 OutCount = 0;
	for (const FAtlasSlot& Slot : In.Slots)
	{
		if (Slot.IsValid())
		{
			++OutCount;
		}
	}
	return OutCount;
}

void UpdateAtlasTexture(FRDGBuilder& GraphBuilder, const FStaticShaderPlatform ShaderPlatform)
{
	// Force update by resetting the atlas layout
	static uint32 CachedAtlasResolution = CVarIESTextureResolution.GetValueOnRenderThread();
	static uint32 CachedAtlasMaxSlice = CVarIESTextureMaxProfileCount.GetValueOnRenderThread();
	const bool bHasSettingChanged = CachedAtlasResolution != CVarIESTextureResolution.GetValueOnRenderThread() || CachedAtlasMaxSlice != CVarIESTextureMaxProfileCount.GetValueOnRenderThread();
	const bool bForceUpdate = CVarIESForceUpdate.GetValueOnRenderThread() > 0;

	// 1. Determine if a new atlas texture allocation is needed: If there are new entries, ensure it can fit into the atlas
	bool bForceRecreate = false;
	uint32 RequestedSliceCount = 0;
	if (GIESTextureManager.bHasPendingAdds || bForceUpdate)
	{
		RequestedSliceCount = FMath::Min(CachedAtlasMaxSlice, GetRequestedSlotCount(GIESTextureManager));
		bForceRecreate = (GIESTextureManager.AtlasTexture == nullptr) || (GIESTextureManager.RequestedSliceCountWhenTextureAllocated < RequestedSliceCount);
	}
	bForceRecreate = bForceRecreate || bHasSettingChanged;

	// 2. Invalid slots 
	// * If a global update is requested, invalid all slots
	// * Otherwise invalid slots which have new texture data (e.g., streamed in)
	if (bForceUpdate || bForceRecreate)
	{
		CachedAtlasResolution = CVarIESTextureResolution.GetValueOnRenderThread();
		CachedAtlasMaxSlice = CVarIESTextureMaxProfileCount.GetValueOnRenderThread();
		InvalidateSlots(GIESTextureManager, RequestedSliceCount);
	}
	else 
	{
		// Force update if among the existing valid slot a streamed a higher resolution than the existing one
		for (FAtlasSlot& Slot : GIESTextureManager.Slots)
		{
			if (Slot.IsValid() && Slot.GetTextureRHI())
			{
				FRHITexture* TextureRHI = Slot.GetTextureRHI();
				check(TextureRHI != nullptr);

				const FIntPoint SourceResolution = TextureRHI->GetSizeXY();
				if (SourceResolution != Slot.CachedResolution)
				{
					// Invalid the slot
					Slot.bForceRefresh = true;
					Slot.CachedResolution = SourceResolution;
					GIESTextureManager.bHasPendingRefreshes = true;
				}
			}
		}
	}

	// 3. Process new atlas entries (texture allocate, slice allocation, ...) or refresh slots
	if (GIESTextureManager.bHasPendingAdds)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(ShaderPlatform);

		// 3.1. Allocate new slice
		TArray<FAtlasSlot> NewSlots;
		for (FAtlasSlot& Slot : GIESTextureManager.Slots)
		{
			if (Slot.IsValid() && Slot.SliceIndex == InvalidSliceIndex)
			{
				// Try to allocate a slot with a slice index, but allocation might failed if 
				// we have reached CVarIESTextureMaxSlice
				if (GIESTextureManager.FreeSlices.Dequeue(Slot.SliceIndex))
				{
					NewSlots.Add(Slot);
				}
			}
		}

		// 3.2. Apply layout modification
		{
			// 3.2.1 Register or create the texture atlas
			FRDGTextureRef AtlasTexture = nullptr;
			bool bNeedExtraction = false;
			if (bForceRecreate)
			{
				AtlasTexture = CreateAtlasTexture(GraphBuilder, ShaderPlatform, CachedAtlasResolution, RequestedSliceCount);
				bNeedExtraction = true;
				GIESTextureManager.RequestedSliceCountWhenTextureAllocated = RequestedSliceCount;
			}
			else
			{
				AtlasTexture = GraphBuilder.RegisterExternalTexture(GIESTextureManager.AtlasTexture);
			}

			// 3.2.2 Insert new slots into the atlas texture
			if (NewSlots.Num() > 0)
			{
				AddSlotsPass(GraphBuilder, ShaderPlatform, ShaderMap, NewSlots, AtlasTexture);
			}

			if (bNeedExtraction)
			{
				GIESTextureManager.AtlasTexture = GraphBuilder.ConvertToExternalTexture(AtlasTexture);
			}
		}

		GIESTextureManager.bHasPendingAdds = false;
	}
	// Process forced refresh slots
	else if (GIESTextureManager.bHasPendingRefreshes)
	{
		TArray<FAtlasSlot> RefreshSlots;
		RefreshSlots.Reserve(FMath::Max(1, GIESTextureManager.Slots.Num()));
		for (FAtlasSlot& Slot : GIESTextureManager.Slots)
		{
			if (Slot.bForceRefresh)
			{
				Slot.bForceRefresh = false;
				RefreshSlots.Add(Slot);
			}
		}

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(ShaderPlatform);
		FRDGTextureRef AtlasTexture = GraphBuilder.RegisterExternalTexture(GIESTextureManager.AtlasTexture);
		AddSlotsPass(GraphBuilder, ShaderPlatform, ShaderMap, RefreshSlots, AtlasTexture);
		GIESTextureManager.bHasPendingRefreshes = false;
	}

	// If the atlas was registered into RDG make sure it gets transitioned to external access on all pipes.
	if (FRDGTexture* AtlasTexture = GraphBuilder.FindExternalTexture(GIESTextureManager.AtlasTexture))
	{
		GraphBuilder.UseExternalAccessMode(AtlasTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}
}

void AddDebugPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef OutputTexture)
{
	if (CVarIESTextureDebug.GetValueOnRenderThread() > 0 && ShaderPrint::IsSupported(View.Family->GetShaderPlatform()))
	{
		if (FRDGTextureRef DebugOutput = AddIESDebugPass(GraphBuilder, View, OutputTexture->Desc))
		{
			// Debug output is blend on top of the SceneColor/OutputTexture, as debug pass is a CS pass, and SceneColor/OutputTexture might not have a UAV flag
			FCopyRectPS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
			Parameters->InputTexture = DebugOutput;
			Parameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ELoad);

			const FScreenPassTextureViewport InputViewport(DebugOutput->Desc.Extent);
			const FScreenPassTextureViewport OutputViewport(OutputTexture);
			TShaderMapRef<FCopyRectPS> PixelShader(View.ShaderMap);

			TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
			FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
			FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("IESAtlas::BlitDebug"), View, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, DepthStencilState, Parameters, EScreenPassDrawFlags::None);
		}
	}
}

FRHITexture* GetAtlasTexture()
{
	return GIESTextureManager.AtlasTexture ? GIESTextureManager.AtlasTexture->GetRHI() : nullptr;
}

} // namespace IESAtlas