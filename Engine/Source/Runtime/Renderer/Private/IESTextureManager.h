// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"
#include "RenderGraphDefinitions.h"

class FRHITexture;
class UTextureLightProfile;
class FViewInfo;

namespace IESAtlas
{
	
// Add an IES light texture to the atlas
RENDERER_API uint32 AddTexture(UTextureLightProfile* IESTexture);

// Remove an IES texture from the atlas
RENDERER_API void RemoveTexture(uint32 InSlotId);

// Return the atlas texture coordinate for a particular slot
RENDERER_API float GetAtlasSlot(uint32 InSlotId);

// Return the atlas texture
RENDERER_API FRHITexture* GetAtlasTexture();

// Update the rect light atlas texture
RENDERER_API void UpdateAtlasTexture(FRDGBuilder& GraphBuilder, const FStaticShaderPlatform ShaderPlatform);

// Return the rect light atlas debug pass
RENDERER_API void AddDebugPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef OutputTexture);
}