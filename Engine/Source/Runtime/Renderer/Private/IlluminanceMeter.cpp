// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Functionality for a real time illuminance meter reference for the sky light and sun.
=============================================================================*/

#include "IlluminanceMeter.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ReflectionEnvironmentCapture.h"
#include "RHIResources.h"
#include "RHIShaderPlatform.h"
#include "PixelShaderUtils.h"
#include "SceneRendering.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "ShaderCompiler.h"
#include "PostProcess/PostProcessing.h"

DECLARE_GPU_STAT(IlluminanceMeter);

class FCopyCubemapToIlluminanceMeterCubemapPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyCubemapToIlluminanceMeterCubemapPS);
	SHADER_USE_PARAMETER_STRUCT(FCopyCubemapToIlluminanceMeterCubemapPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_TEXTURE(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER(FVector2f, SvPositionToUVScale)
		SHADER_PARAMETER(int32, CubeFace)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCopyCubemapToIlluminanceMeterCubemapPS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "CopyCubemapToIlluminanceMeterCubemapPS", SF_Pixel);

class FDownsampleIntegrateCubemapIlluminancePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDownsampleIntegrateCubemapIlluminancePS);
	SHADER_USE_PARAMETER_STRUCT(FDownsampleIntegrateCubemapIlluminancePS, FGlobalShader);
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER(FVector2f, SvPositionToUVScale)
		SHADER_PARAMETER(int32, CubeFace)
		SHADER_PARAMETER(int32, MipIndex)
		SHADER_PARAMETER(int32, NumMips)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FDownsampleIntegrateCubemapIlluminancePS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "DownsampleIntegrateCubemapIlluminancePS", SF_Pixel);

static bool CanIlluminanceMeterDisplayOnPlatform(EShaderPlatform Platform)
{
	// On some consoles, this ALU heavy shader (and with optimisation disables for the sake of low compilation time) would spill registers. So only keep it for the editor.
	return GetMaxSupportedFeatureLevel(Platform) >= ERHIFeatureLevel::ES3_1;
}

class FPrintIlluminanceMeterPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPrintIlluminanceMeterPS);
	SHADER_USE_PARAMETER_STRUCT(FPrintIlluminanceMeterPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER_TEXTURE(TextureCube, SkyLightCubeTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SkyLightCubeSampler)
		SHADER_PARAMETER(int32, CubeMipIndexToSampleIlluminance)
		SHADER_PARAMETER(int32, SkyLightCubemapMaxResolution)
		SHADER_PARAMETER(int32, HardwareCubemapMaxResolution)
		SHADER_PARAMETER(int32, CurrentCubemapResolution)
		SHADER_PARAMETER(FVector3f, SkyLightCaptureWorlPos)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return CanIlluminanceMeterDisplayOnPlatform(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Stay debug and skip optimizations to reduce compilation time on this long shader.
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.SetDefine(TEXT("VISUALIZE_ILLUMINANCE_METER"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FPrintIlluminanceMeterPS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "VisualizeIlluminanceMeterPS", SF_Pixel);

void FScene::ProcessAndRenderIlluminanceMeter(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views, FRDGTextureRef SceneColorTexture)
{
	// Deprecated
}

FScreenPassTexture ProcessAndRenderIlluminanceMeter(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor)
{
	const FScene* Scene = (const FScene*)View.Family->Scene;
	FSkyLightSceneProxy* SkyLight = Scene ? Scene->SkyLight : nullptr;
	if (!SkyLight || !View.Family->EngineShowFlags.VisualizeSkyLightIlluminance || View.bIsSceneCapture || View.bIsReflectionCapture || View.bIsPlanarReflection ||
		!CanIlluminanceMeterDisplayOnPlatform(View.GetShaderPlatform()))
	{
		return MoveTemp(ScreenPassSceneColor);
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, IlluminanceMeter, "IlluminanceMeter");

	// 0- Get source cubemap and allocate cube map transient resources

	FTextureRHIRef SkyLightTextureRHI = GBlackTextureCube->TextureRHI;
	if (Scene && Scene->CanSampleSkyLightRealTimeCaptureData())
	{
		// Cannot blend with this capture mode as of today.
		SkyLightTextureRHI = Scene->ConvolvedSkyRenderTarget[Scene->ConvolvedSkyRenderTargetReadyIndex]->GetRHI();

		//
	}
	else if (SkyLight->ProcessedTexture)
	{
		SkyLightTextureRHI = SkyLight->ProcessedTexture->TextureRHI;
	}

	const int32 CubemapSize = SkyLightTextureRHI->GetDesc().GetSize().X;
	const int32 NumReflectionCaptureMips = GetNumMips(CubemapSize);
	const FRDGTextureDesc TextureDesc = FRDGTextureDesc::CreateCube(CubemapSize, PF_A32B32G32R32F, FClearValueBinding::Black, 
		TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_TargetArraySlicesIndependently | TexCreate_DisableDCC, NumReflectionCaptureMips);
	FRDGTextureRef IlluminanceMeterCubeTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("IlluminanceMeterTexture"));

	// 1- Compute illuminance contribution for each texel luminance as a function of their solid angle in the source cube map
	{
		RDG_EVENT_SCOPE(GraphBuilder, "CopyCubemapToIlluminanceMeterCubemap");

		TShaderMapRef<FCopyCubemapToIlluminanceMeterCubemapPS> PixelShader(View.ShaderMap);

		for (uint32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
		{
			auto* PassParameters = GraphBuilder.AllocParameters<FCopyCubemapToIlluminanceMeterCubemapPS::FParameters>();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->SourceCubemapTexture = SkyLightTextureRHI;
			PassParameters->CubeFace = CubeFace;
			PassParameters->SvPositionToUVScale = FVector2f(1.0f / CubemapSize, 1.0f / CubemapSize);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(IlluminanceMeterCubeTexture, ERenderTargetLoadAction::ENoAction, 0, CubeFace);

			const FIntRect ViewRect(0, 0, CubemapSize, CubemapSize);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				View.ShaderMap,
				RDG_EVENT_NAME("FCopyCubemapToIlluminanceMeterCubemap"),
				PixelShader,
				PassParameters,
				ViewRect);
		}
	}

	// 2- Generate all the mips of the cubemap, each time storing the sum the contributions to illuminance.
	{
		RDG_EVENT_SCOPE(GraphBuilder, "IntegrateCubemapIlluminance");

		TShaderMapRef<FDownsampleIntegrateCubemapIlluminancePS> PixelShader(View.ShaderMap);
		const int32 NumMips = IlluminanceMeterCubeTexture->Desc.NumMips;

		for (int32 MipIndex = 1; MipIndex < NumMips; MipIndex++)
		{
			const int32 MipSize = 1 << (NumMips - MipIndex - 1);
			const FIntRect ViewRect(0, 0, MipSize, MipSize);

			for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
			{
				auto* PassParameters = GraphBuilder.AllocParameters<FDownsampleIntegrateCubemapIlluminancePS::FParameters>();
				PassParameters->SourceCubemapTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(IlluminanceMeterCubeTexture, MipIndex - 1));
				PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->CubeFace = CubeFace;
				PassParameters->MipIndex = MipIndex;
				PassParameters->NumMips = NumMips;
				PassParameters->SvPositionToUVScale = FVector2f(1.0f / MipSize, 1.0f / MipSize);
				PassParameters->RenderTargets[0] = FRenderTargetBinding(IlluminanceMeterCubeTexture, ERenderTargetLoadAction::ENoAction, MipIndex, CubeFace);

				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					View.ShaderMap,
					RDG_EVENT_NAME("CreateCubeMips (Mip: %d, Face: %d)", MipIndex, CubeFace),
					PixelShader,
					PassParameters,
					ViewRect);
			}
		}
	}

	// 3- Sum up each faces and display illuminance on screen
	{
		ShaderPrint::SetEnabled(true);
		ShaderPrint::RequestSpaceForLines(1024);
		ShaderPrint::RequestSpaceForCharacters(1024);

		FRHIBlendState* PreMultipliedColorTransmittanceBlend = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();

		FPrintIlluminanceMeterPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPrintIlluminanceMeterPS::FParameters>();
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->SourceCubemapTexture = IlluminanceMeterCubeTexture;
		PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->SkyLightCubeTexture = SkyLightTextureRHI;
		PassParameters->SkyLightCubeSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->CubeMipIndexToSampleIlluminance = IlluminanceMeterCubeTexture->Desc.NumMips - 1;
		PassParameters->SkyLightCaptureWorlPos = FVector3f(SkyLight->CapturePosition);

		PassParameters->CurrentCubemapResolution = SkyLightTextureRHI ? SkyLightTextureRHI->GetDesc().Extent.X : 0;
		PassParameters->HardwareCubemapMaxResolution = GetMaxCubeTextureDimension();
		static auto* SkyLightCubemapMaxResolutionCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SkyLight.CubemapMaxResolution"));
		PassParameters->SkyLightCubemapMaxResolution = SkyLightCubemapMaxResolutionCVar ? SkyLightCubemapMaxResolutionCVar->GetInt() : 0;

		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintParameters);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenPassSceneColor.Texture, ERenderTargetLoadAction::ELoad);

		FPrintIlluminanceMeterPS::FPermutationDomain PermutationVector;
		TShaderMapRef<FPrintIlluminanceMeterPS> PixelShader(View.ShaderMap, PermutationVector);

		FPixelShaderUtils::AddFullscreenPass<FPrintIlluminanceMeterPS>(GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("Substrate::VisualizeMaterial(Draw)"), PixelShader, PassParameters, View.ViewRect, PreMultipliedColorTransmittanceBlend);
	}

	return MoveTemp(ScreenPassSceneColor);
}
