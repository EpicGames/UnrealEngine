// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphFwd.h"

class FViewInfo;
class FSkyLightSceneProxy;
class FRDGBuilder;
struct FScreenPassTexture;

FScreenPassTexture ProcessAndRenderIlluminanceMeter(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor);
