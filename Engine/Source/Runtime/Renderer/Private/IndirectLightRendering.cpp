// Copyright Epic Games, Inc. All Rights Reserved.

#include "IndirectLightRendering.h"
#include "CoreMinimal.h"
#include "AmbientCubemapParameters.h"
#include "BasePassRendering.h"
#include "BlueNoise.h"
#include "CommonRenderResources.h"
#include "CompositionLighting/PostProcessAmbientOcclusion.h"
#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DeferredShadingRenderer.h"
#include "DeferredShadingRendererTypes.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "DistanceFieldLightingShared.h"
#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "FinalPostProcessSettings.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "HairStrands/HairStrandsData.h"
#include "HairStrands/HairStrandsEnvironment.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "Logging/LogMacros.h"
#include "Lumen/Lumen.h"
#include "Lumen/LumenRadianceCacheInterpolation.h"
#include "Lumen/LumenReflections.h"
#include "Lumen/LumenSceneData.h"
#include "Lumen/LumenShortRangeAO.h"
#include "Lumen/LumenTracingUtils.h"
#include "Lumen/LumenViewState.h"
#include "Math/Color.h"
#include "MegaLights/MegaLights.h"
#include "MegaLights/MegaLightsViewState.h"
#include "Misc/LargeWorldCoordinates.h"
#include "MultiGPU.h"
#include "PipelineStateCache.h"
#include "PixelShaderUtils.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/TemporalAA.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/TagTrace.h"
#include "RayTracing/RaytracingOptions.h"
#include "RendererInterface.h"
#include "LogRenderer.h"
#include "RendererUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/SubstrateMaterialShared.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "ScreenSpaceDenoise.h"
#include "ScreenSpaceRayTracing.h"
#include "ScreenSpaceReflectionTiles.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "StochasticLighting/StochasticLightingViewState.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "UniformBuffer.h"
#include "VariableRateShadingImageManager.h"
#include "VisualizeTexture.h"
#include "VolumetricCloudRendering.h"
#include "SceneViewState.h"
#include "PostProcess/PostProcessSubsurface.h"

// This is the project default dynamic global illumination, NOT the scalability setting (see r.Lumen.DiffuseIndirect.Allow for scalability)
// Must match EDynamicGlobalIlluminationMethod
// Note: Default for new projects set by GameProjectUtils
static TAutoConsoleVariable<int32> CVarDynamicGlobalIlluminationMethod(
	TEXT("r.DynamicGlobalIlluminationMethod"), 0,
	TEXT("0 - None.  Global Illumination can be baked into Lightmaps but no technique will be used for Dynamic Global Illumination.\n")
	TEXT("1 - Lumen.  Use Lumen Global Illumination for all lights, emissive materials casting light and SkyLight Occlusion.  Requires 'Generate Mesh Distance Fields' enabled for Software Ray Tracing and 'Support Hardware Ray Tracing' enabled for Hardware Ray Tracing.\n")
	TEXT("2 - SSGI.  Standalone Screen Space Global Illumination.  Low cost, but limited by screen space information.\n")
	TEXT("3 - Plugin.  Use a plugin for Global Illumination."),
	ECVF_RenderThreadSafe);

// This is the project default reflection method, NOT the scalability setting (see r.Lumen.Reflections.Allow for scalability)
// Must match EReflectionMethod
// Note: Default for new projects set by GameProjectUtils
static TAutoConsoleVariable<int32> CVarReflectionMethod(
	TEXT("r.ReflectionMethod"), 2,
	TEXT("0 - None.  Reflections can come from placed Reflection Captures, Planar Reflections and Skylight but no global reflection method will be used.\n")
	TEXT("1 - Lumen.  Use Lumen Reflections, which supports Screen / Software / Hardware Ray Tracing together and integrates with Lumen Global Illumination for rough reflections and Global Illumination seen in reflections.\n")
	TEXT("2 - SSR.  Standalone Screen Space Reflections.  Low cost, but limited by screen space information.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDiffuseIndirectHalfRes(
	TEXT("r.DiffuseIndirect.HalfRes"), 1,
	TEXT("TODO(Guillaume)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDiffuseIndirectRayPerPixel(
	TEXT("r.DiffuseIndirect.RayPerPixel"), 6, // TODO(Guillaume): Matches old Lumen hard code sampling pattern.
	TEXT("TODO(Guillaume)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDiffuseIndirectDenoiser(
	TEXT("r.DiffuseIndirect.Denoiser"), 1,
	TEXT("Denoising options (default = 1)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDenoiseSSR(
	TEXT("r.SSR.ExperimentalDenoiser"), 0,
	TEXT("Replace SSR's TAA pass with denoiser."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSkySpecularOcclusionStrength(
	TEXT("r.SkySpecularOcclusionStrength"),
	1,
	TEXT("Strength of skylight specular occlusion from DFAO (default is 1.0)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarReflectionCaptureEnableDeferredReflectionsAndSkyLighting(
	TEXT("r.ReflectionCapture.EnableDeferredReflectionsAndSkyLighting"),
	1,
	TEXT("Whether to evaluate deferred reflections and sky contribution when rendering reflection captures."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarProbeSamplePerPixel(
	TEXT("r.Lumen.ProbeHierarchy.SamplePerPixel"), 8,
	TEXT("Number of sample to do per full res pixel."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarDiffuseIndirectOffUseDepthBoundsAO(
	TEXT("r.DiffuseIndirectOffUseDepthBoundsAO"), true,
	TEXT("Use depth bounds when we apply the AO when DiffuseIndirect is disabled."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarDiffuseIndirectForceCopyPass(
	TEXT("r.DiffuseIndirectForceCopyPass"), false,
	TEXT("Forces use of copy pass instead of dual source blend. (for debugging)"),
	ECVF_RenderThreadSafe);

DEFINE_GPU_STAT(ReflectionEnvironment);
DECLARE_GPU_STAT_NAMED(RayTracingReflections, TEXT("Ray Tracing Reflections"));
DECLARE_GPU_STAT(SkyLightDiffuse);

float GetLumenReflectionSpecularScale();
float GetLumenReflectionContrast();
int GetReflectionEnvironmentCVar();
bool IsAmbientCubemapPassRequired(const FSceneView& View);

bool IsDeferredReflectionsAndSkyLightingDisabledForReflectionCapture()
{
	return CVarReflectionCaptureEnableDeferredReflectionsAndSkyLighting.GetValueOnRenderThread() == 0;
}

class FDiffuseIndirectCompositePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDiffuseIndirectCompositePS)
	SHADER_USE_PARAMETER_STRUCT(FDiffuseIndirectCompositePS, FGlobalShader)

	class FApplyDiffuseIndirectDim : SHADER_PERMUTATION_INT("DIM_APPLY_DIFFUSE_INDIRECT", 3);
	class FUpscaleSSGIDim : SHADER_PERMUTATION_BOOL("DIM_UPSCALE_SSGI");
	class FScreenBentNormalMode : SHADER_PERMUTATION_RANGE_INT("DIM_SCREEN_BENT_NORMAL_MODE", 0, 3);
	class FUpscaleLumenGIDim : SHADER_PERMUTATION_BOOL("DIM_UPSCALE_LUMEN_GI");
	class FShortRangeGI : SHADER_PERMUTATION_BOOL("DIM_SHORT_RANGE_GI");
	class FEnableDualSrcBlending : SHADER_PERMUTATION_BOOL("ENABLE_DUAL_SRC_BLENDING");
	class FHairComplexTransmittance : SHADER_PERMUTATION_BOOL("USE_HAIR_COMPLEX_TRANSMITTANCE");
	class FScreenSpaceReflectionTiledComposition : SHADER_PERMUTATION_BOOL("SSR_TILED_COMPOSITION");

	using FPermutationDomain = TShaderPermutationDomain<FApplyDiffuseIndirectDim, FUpscaleSSGIDim, FScreenBentNormalMode, FUpscaleLumenGIDim, FShortRangeGI, Substrate::FSubstrateTileType, FEnableDualSrcBlending, FScreenSpaceReflectionTiledComposition, FHairComplexTransmittance>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (IsMobilePlatform(Parameters.Platform))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// Only upscale SSGI
		if (PermutationVector.Get<FApplyDiffuseIndirectDim>() != 1 && PermutationVector.Get<FUpscaleSSGIDim>())
		{
			return false;
		}

		// Lumen-only permutations
		if (PermutationVector.Get<FApplyDiffuseIndirectDim>() != 2 
			&& (PermutationVector.Get<FScreenBentNormalMode>() != 0 || PermutationVector.Get<FShortRangeGI>() || PermutationVector.Get<FUpscaleLumenGIDim>()))
		{
			return false;
		}

		// Build Substrate tile permutation only for Lumen
		if (PermutationVector.Get<Substrate::FSubstrateTileType>() != ESubstrateTileType::EComplex)
		{
			return Substrate::IsSubstrateEnabled() && PermutationVector.Get<FApplyDiffuseIndirectDim>() == 2;
		}

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// Other final gathers should upscale in their temporal filter
		if (PermutationVector.Get<FUpscaleLumenGIDim>() && !Lumen::UseIrradianceFieldGather())
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		extern int32 GLumenShortRangeGlobalIllumination;
		if (PermutationVector.Get<FShortRangeGI>() != (GLumenShortRangeGlobalIllumination != 0))
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		return EShaderPermutationPrecacheRequest::Precached;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, AmbientOcclusionStaticFraction)
		SHADER_PARAMETER(int32, bVisualizeDiffuseIndirect)
		SHADER_PARAMETER_STRUCT_INCLUDE(LumenReflections::FCompositeParameters, ReflectionsCompositeParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLumenScreenSpaceBentNormalParameters, ScreenBentNormalParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLumenUpsampleParameters, LumenUpsampleParameters)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)
		SHADER_PARAMETER(uint32, bLumenSupportBackfaceDiffuse)
		SHADER_PARAMETER(uint32, bLumenReflectionInputIsSSR)
		SHADER_PARAMETER(uint32, LumenGIDownsampleFactor)
		SHADER_PARAMETER(float, LumenFoliageOcclusionStrength)
		SHADER_PARAMETER(float, LumenMaxAOMultibounceAlbedo)
		SHADER_PARAMETER(float, LumenReflectionSpecularScale)
		SHADER_PARAMETER(float, LumenAmbientOcclusionIntensity)
		SHADER_PARAMETER(float, LumenReflectionContrast)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, DiffuseIndirect_Lumen_0)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, DiffuseIndirect_Lumen_1)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, DiffuseIndirect_Lumen_2)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, DiffuseIndirect_Lumen_3)

		SHADER_PARAMETER_STRUCT(FSSDSignalTextures, DiffuseIndirect)
		SHADER_PARAMETER_SAMPLER(SamplerState, DiffuseIndirectSampler)
		
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AmbientOcclusionTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  AmbientOcclusionSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)

		SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
		SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(Denoiser::FCommonShaderParameters, DenoiserCommonParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceDiffuseWriteParameters, SubsurfaceDiffuseParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, PassDebugOutput)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, OutOpaqueRoughRefractionSceneColor)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, OutSubSurfaceSceneColor)

		SHADER_PARAMETER(FIntPoint, SSRTiledViewRes)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, SSRTileMaskBuffer)

		SHADER_PARAMETER(FVector2f, BufferUVToOutputPixelPosition)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptation)
		SHADER_PARAMETER_RDG_TEXTURE_ARRAY(Texture2D<uint>, CompressedMetadata, [2])

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		const bool bSeparatedSubsurfaceDiffuse = PlatformSupportsSeparatedSubsurfaceDiffuse(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE"), bSeparatedSubsurfaceDiffuse  ? 1 : 0);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceOptimization);
		if ((Substrate::IsSubstrateEnabled() && Substrate::IsOpaqueRoughRefractionEnabled(Parameters.Platform))
			|| bSeparatedSubsurfaceDiffuse)
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		}
	}

};

BEGIN_SHADER_PARAMETER_STRUCT(FDiffuseIndirectCompositeParameters, )
	SHADER_PARAMETER_STRUCT(FDiffuseIndirectCompositePS::FParameters, PS)
	SHADER_PARAMETER_STRUCT(Substrate::FSubstrateTilePassVS::FParameters, VS)
END_SHADER_PARAMETER_STRUCT()

class FAmbientCubemapCompositePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAmbientCubemapCompositePS)
	SHADER_USE_PARAMETER_STRUCT(FAmbientCubemapCompositePS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE"), PlatformSupportsSeparatedSubsurfaceDiffuse(Parameters.Platform) ? 1 : 0);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
		SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AmbientOcclusionTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  AmbientOcclusionSampler)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(FAmbientCubemapParameters, AmbientCubemap)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

BEGIN_SHADER_PARAMETER_STRUCT(FAmbientCubemapCompositeParameters, )
	SHADER_PARAMETER_STRUCT(FAmbientCubemapCompositePS::FParameters, PS)
	SHADER_PARAMETER_STRUCT(Substrate::FSubstrateTilePassVS::FParameters, VS)
END_SHADER_PARAMETER_STRUCT()

// Setups all shader parameters related to skylight.
FSkyDiffuseLightingParameters GetSkyDiffuseLightingParameters(const FSkyLightSceneProxy* SkyLight, float DynamicBentNormalAO)
{
	float SkyLightContrast = 0.01f;
	float SkyLightOcclusionExponent = 1.0f;
	FVector4f SkyLightOcclusionTintAndMinOcclusion(0.0f, 0.0f, 0.0f, 0.0f);
	EOcclusionCombineMode SkyLightOcclusionCombineMode = EOcclusionCombineMode::OCM_MAX;
	if (SkyLight)
	{
		FDistanceFieldAOParameters Parameters(SkyLight->OcclusionMaxDistance, SkyLight->Contrast);
		SkyLightContrast = Parameters.Contrast;
		SkyLightOcclusionExponent = SkyLight->OcclusionExponent;
		SkyLightOcclusionTintAndMinOcclusion = FVector4f(SkyLight->OcclusionTint);
		SkyLightOcclusionTintAndMinOcclusion.W = SkyLight->MinOcclusion;
		SkyLightOcclusionCombineMode = SkyLight->OcclusionCombineMode;
	}

	// Scale and bias to remap the contrast curve to [0,1]
	const float Min = 1 / (1 + FMath::Exp(-SkyLightContrast * (0 * 10 - 5)));
	const float Max = 1 / (1 + FMath::Exp(-SkyLightContrast * (1 * 10 - 5)));
	const float Mul = 1.0f / (Max - Min);
	const float Add = -Min / (Max - Min);

	FSkyDiffuseLightingParameters Out;
	Out.OcclusionTintAndMinOcclusion = SkyLightOcclusionTintAndMinOcclusion;
	Out.ContrastAndNormalizeMulAdd = FVector3f(SkyLightContrast, Mul, Add);
	Out.OcclusionExponent = SkyLightOcclusionExponent;
	Out.OcclusionCombineMode = SkyLightOcclusionCombineMode == OCM_Minimum ? 0.0f : 1.0f;
	Out.ApplyBentNormalAO = DynamicBentNormalAO;
	Out.InvSkySpecularOcclusionStrength = 1.0f / FMath::Max(CVarSkySpecularOcclusionStrength.GetValueOnRenderThread(), 0.1f);
	return Out;
}

/** Pixel shader that does tiled deferred culling of reflection captures, then sorts and composites them. */
class FReflectionEnvironmentSkyLightingPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FReflectionEnvironmentSkyLightingPS);
	SHADER_USE_PARAMETER_STRUCT(FReflectionEnvironmentSkyLightingPS, FGlobalShader)

	enum class EReflectionSourceType : uint32
	{
		ScreenSpaceReflection,
		TiledScreenSpaceReflection,
		LumenStandalone,
		MAX
	};

	class FHasBoxCaptures : SHADER_PERMUTATION_BOOL("REFLECTION_COMPOSITE_HAS_BOX_CAPTURES");
	class FHasSphereCaptures : SHADER_PERMUTATION_BOOL("REFLECTION_COMPOSITE_HAS_SPHERE_CAPTURES");
	class FDFAOIndirectOcclusion : SHADER_PERMUTATION_BOOL("SUPPORT_DFAO_INDIRECT_OCCLUSION");
	class FSkyLight : SHADER_PERMUTATION_BOOL("ENABLE_SKY_LIGHT");
	class FDynamicSkyLight : SHADER_PERMUTATION_BOOL("ENABLE_DYNAMIC_SKY_LIGHT");
	class FSkyShadowing : SHADER_PERMUTATION_BOOL("APPLY_SKY_SHADOWING");
	class FReflectionSourceType : SHADER_PERMUTATION_ENUM_CLASS("REFLECTION_SOURCE_TYPE", EReflectionSourceType);
	class FHairComplexTransmittance : SHADER_PERMUTATION_BOOL("USE_HAIR_COMPLEX_TRANSMITTANCE");

	using FPermutationDomain = TShaderPermutationDomain<
		FHasBoxCaptures,
		FHasSphereCaptures,
		FDFAOIndirectOcclusion,
		FSkyLight,
		FDynamicSkyLight,
		FSkyShadowing,
		FReflectionSourceType,
		Substrate::FSubstrateTileType,
		FHairComplexTransmittance>;

	static EReflectionSourceType GetReflectionSourceType(bool bIsLumenStandalone, bool bIsTiledSSR)
	{
		if (bIsLumenStandalone)
		{
			return EReflectionSourceType::LumenStandalone;
		}
		else if (bIsTiledSSR)
		{
			return EReflectionSourceType::TiledScreenSpaceReflection;
		}

		return EReflectionSourceType::ScreenSpaceReflection;
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		// FSkyLightingDynamicSkyLight requires FSkyLightingSkyLight.
		if (!PermutationVector.Get<FSkyLight>())
		{
			PermutationVector.Set<FDynamicSkyLight>(false);
		}

		// FSkyLightingSkyShadowing requires FSkyLightingDynamicSkyLight.
		if (!PermutationVector.Get<FDynamicSkyLight>())
		{
			PermutationVector.Set<FSkyShadowing>(false);
		}

		if (PermutationVector.Get<Substrate::FSubstrateTileType>() && !Substrate::IsSubstrateEnabled())
		{
			PermutationVector.Set<Substrate::FSubstrateTileType>(0);
		}

		// Hair shading is classified only for complex & complex special tiles
		if (Substrate::IsSubstrateEnabled() && PermutationVector.Get<FHairComplexTransmittance>() && PermutationVector.Get<Substrate::FSubstrateTileType>() < ESubstrateTileType::EComplex) 
		{
			PermutationVector.Set<Substrate::FSubstrateTileType>(0);
		}

		return PermutationVector;
	}

	static FPermutationDomain BuildPermutationVector(const FViewInfo& View, bool bBoxCapturesOnly, bool bSphereCapturesOnly, bool bSupportDFAOIndirectOcclusion, bool bEnableSkyLight, bool bEnableDynamicSkyLight, bool bApplySkyShadowing, bool bLumenStandaloneReflections, bool bIsTiledSSR, bool bHasHairCardsOrMeshes, ESubstrateTileType TileType)
	{
		FPermutationDomain PermutationVector;

		PermutationVector.Set<FHasBoxCaptures>(bBoxCapturesOnly);
		PermutationVector.Set<FHasSphereCaptures>(bSphereCapturesOnly);
		PermutationVector.Set<FDFAOIndirectOcclusion>(bSupportDFAOIndirectOcclusion);
		PermutationVector.Set<FSkyLight>(bEnableSkyLight);
		PermutationVector.Set<FDynamicSkyLight>(bEnableDynamicSkyLight);
		PermutationVector.Set<FSkyShadowing>(bApplySkyShadowing);
		PermutationVector.Set<FReflectionSourceType>(GetReflectionSourceType(bLumenStandaloneReflections,bIsTiledSSR));
		PermutationVector.Set<Substrate::FSubstrateTileType>(0);
		PermutationVector.Set<FHairComplexTransmittance>(bHasHairCardsOrMeshes);
		if (Substrate::IsSubstrateEnabled())
		{
			check(TileType <= ESubstrateTileType::EComplexSpecial);
			PermutationVector.Set<Substrate::FSubstrateTileType>(TileType);
		}
		return RemapPermutation(PermutationVector);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (!Substrate::ShouldCompileSubstrateTileTypePermutations(PermutationVector.Get<Substrate::FSubstrateTileType>(), Parameters.Platform))
		{
			return false;
		}

		return PermutationVector == RemapPermutation(PermutationVector);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MAX_CAPTURES"), GetMaxNumReflectionCaptures(Parameters.Platform));
		OutEnvironment.SetDefine(TEXT("SUPPORTS_ANISOTROPIC_MATERIALS"), FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Parameters.Platform));
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_GLINTS_IS"), 0);
		OutEnvironment.SetDefine(TEXT("SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE"), PlatformSupportsSeparatedSubsurfaceDiffuse(Parameters.Platform) ? 1 : 0);
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);

		// FXC is too slow at compiling Substrate lighting shaders
		if (Substrate::IsSubstrateEnabled() && FDataDrivenShaderPlatformInfo::GetSupportsDxc(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
		}
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		// Distance field AO parameters.
		// TODO. FDFAOUpsampleParameters
		SHADER_PARAMETER(FVector2f, AOBufferBilinearUVMax)
		SHADER_PARAMETER(float, DistanceFadeScale)
		SHADER_PARAMETER(float, AOMaxViewDistance)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BentNormalAOTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BentNormalAOSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AmbientOcclusionTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, AmbientOcclusionSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ReflectionTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ReflectionTextureSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, LumenReflectionTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LumenReflectionTextureSampler)

		SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
		SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float2>, CloudSkyAOTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, CloudSkyAOSampler)
		SHADER_PARAMETER(FMatrix44f, CloudSkyAOWorldToLightClipMatrix)
		SHADER_PARAMETER(float, CloudSkyAOFarDepthKm)
		SHADER_PARAMETER(int32, CloudSkyAOEnabled)

		// Tiling info to combine SSR
		SHADER_PARAMETER(FIntPoint, SSRTiledViewRes)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, SSRTileMaskBuffer)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSkyDiffuseLightingParameters, SkyDiffuseLighting)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FReflectionUniformParameters, ReflectionsParameters)
		SHADER_PARAMETER_STRUCT_REF(FReflectionCaptureShaderData, ReflectionCaptureData)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(LumenReflections::FCompositeParameters, ReflectionsCompositeParameters)
		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)
	END_SHADER_PARAMETER_STRUCT()
}; // FReflectionEnvironmentSkyLightingPS


BEGIN_SHADER_PARAMETER_STRUCT(FReflectionEnvironmentSkyLightingParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FReflectionEnvironmentSkyLightingPS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(Substrate::FSubstrateTilePassVS::FParameters, VS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER(FDiffuseIndirectCompositePS, "/Engine/Private/DiffuseIndirectComposite.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FAmbientCubemapCompositePS, "/Engine/Private/AmbientCubemapComposite.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FReflectionEnvironmentSkyLightingPS, "/Engine/Private/ReflectionEnvironmentPixelShader.usf", "ReflectionEnvironmentSkyLighting", SF_Pixel);


IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FReflectionUniformParameters, "ReflectionStruct");

static bool ShouldRenderPluginGlobalIllumination(const FViewInfo& View)
{
	return View.FinalPostProcessSettings.DynamicGlobalIlluminationMethod == EDynamicGlobalIlluminationMethod::Plugin;
}

void FDeferredShadingSceneRenderer::CommitIndirectLightingState()
{
	for (int32 ViewIndex = 0; ViewIndex < AllViews.Num(); ViewIndex++)
	{
		const FViewInfo& View = *AllViews[ViewIndex];
		TPipelineState<FPerViewPipelineState>& ViewPipelineState = GetViewPipelineStateWritable(View);

		EDiffuseIndirectMethod DiffuseIndirectMethod = EDiffuseIndirectMethod::Disabled;
		EAmbientOcclusionMethod AmbientOcclusionMethod = EAmbientOcclusionMethod::Disabled;
		EReflectionsMethod ReflectionsMethod = EReflectionsMethod::Disabled;
		EReflectionsMethod ReflectionsMethodWater = EReflectionsMethod::Disabled;
		IScreenSpaceDenoiser::EMode DiffuseIndirectDenoiser = IScreenSpaceDenoiser::EMode::Disabled;
		bool bHasPlanarReflections = false;

		if (ViewFamily.EngineShowFlags.Lighting)
		{
			if (ShouldRenderLumenDiffuseGI(Scene, View))
			{
				DiffuseIndirectMethod = EDiffuseIndirectMethod::Lumen;
			}
			else if (ScreenSpaceRayTracing::IsScreenSpaceDiffuseIndirectSupported(View))
			{
				DiffuseIndirectMethod = EDiffuseIndirectMethod::SSGI;
				DiffuseIndirectDenoiser = IScreenSpaceDenoiser::GetDenoiserMode(CVarDiffuseIndirectDenoiser);

				static bool bHasWarnedSSGIDeprecated = false;
				if (!bHasWarnedSSGIDeprecated)
				{
					bHasWarnedSSGIDeprecated = true;
					UE_LOGF(LogRenderer, Warning, "Screen Space Global Illumination (SSGI) is deprecated (from 5.8) and will be removed in a future release. Please migrate to Lumen Global Illumination.");
				}
			}
			else if (ShouldRenderPluginGlobalIllumination(View))
			{
				DiffuseIndirectMethod = EDiffuseIndirectMethod::Plugin;
				DiffuseIndirectDenoiser = IScreenSpaceDenoiser::GetDenoiserMode(CVarDiffuseIndirectDenoiser);
			}
		
			const bool bLumenWantsSSAO = DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen && ShouldRenderAOWithLumenGI();

			if (DiffuseIndirectMethod == EDiffuseIndirectMethod::SSGI)
			{
				AmbientOcclusionMethod = EAmbientOcclusionMethod::SSGI;
				DiffuseIndirectDenoiser = IScreenSpaceDenoiser::GetDenoiserMode(CVarDiffuseIndirectDenoiser);
			}
			else if (DiffuseIndirectMethod != EDiffuseIndirectMethod::Lumen || bLumenWantsSSAO)
			{
				extern bool ShouldRenderScreenSpaceAmbientOcclusion(const FViewInfo& View, bool bLumenWantsSSAO);

				if (ShouldRenderRayTracingAmbientOcclusion(View) && (Views.Num() == 1) && !bLumenWantsSSAO)
				{
					AmbientOcclusionMethod = EAmbientOcclusionMethod::RTAO;
				}
				else if (ShouldRenderScreenSpaceAmbientOcclusion(View, bLumenWantsSSAO))
				{
					AmbientOcclusionMethod = EAmbientOcclusionMethod::SSAO;
				}
			}
		
			if (ShouldRenderLumenReflections(View))
			{
				ReflectionsMethod = EReflectionsMethod::Lumen;
			}
			else if (ScreenSpaceRayTracing::ShouldRenderScreenSpaceReflections(View))
			{
				ReflectionsMethod = EReflectionsMethod::SSR;
			}

			if (ShouldRenderLumenReflectionsWater(View))
			{
				ReflectionsMethodWater = EReflectionsMethod::Lumen;
			}
			else if (ScreenSpaceRayTracing::ShouldRenderScreenSpaceReflectionsWater(View))
			{
				ReflectionsMethodWater = EReflectionsMethod::SSR;
			}

			bHasPlanarReflections = HasDeferredPlanarReflections(View);
		}

		ViewPipelineState.Set(&FPerViewPipelineState::DiffuseIndirectMethod, DiffuseIndirectMethod);
		ViewPipelineState.Set(&FPerViewPipelineState::DiffuseIndirectDenoiser, DiffuseIndirectDenoiser);
		ViewPipelineState.Set(&FPerViewPipelineState::AmbientOcclusionMethod, AmbientOcclusionMethod);
		ViewPipelineState.Set(&FPerViewPipelineState::ReflectionsMethod, ReflectionsMethod);
		ViewPipelineState.Set(&FPerViewPipelineState::ReflectionsMethodWater, ReflectionsMethodWater);

		ViewPipelineState.Set(&FPerViewPipelineState::bComposePlanarReflections, bHasPlanarReflections);
	}
}

void SetupReflectionUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FReflectionUniformParameters& OutParameters)
{
	FTextureRHIRef SkyLightTextureResource = nullptr;
	FSamplerStateRHIRef SkyLightCubemapSampler = TStaticSamplerState<SF_Trilinear>::GetRHI();
	FTexture* SkyLightBlendDestinationTextureResource = GBlackTextureCube;
	float ApplySkyLightMask = 0;
	float BlendFraction = 0;
	bool bSkyLightIsDynamic = false;
	float SkyAverageBrightness = 1.0f;

	const bool bApplySkyLight = View.Family->EngineShowFlags.SkyLighting;
	const FScene* Scene = (const FScene*)View.Family->Scene;
	ERDGTextureFlags SkyLightTextureFlags = ERDGTextureFlags::None;

	if (Scene
		&& Scene->SkyLight
		&& (Scene->SkyLight->ProcessedTexture || Scene->CanSampleSkyLightRealTimeCaptureData())
		&& bApplySkyLight)
	{
		const FSkyLightSceneProxy& SkyLight = *Scene->SkyLight;

		if (Scene->CanSampleSkyLightRealTimeCaptureData())
		{
			// Cannot blend with this capture mode as of today.
			SkyLightTextureResource = Scene->ConvolvedSkyRenderTarget[Scene->ConvolvedSkyRenderTargetReadyIndex]->GetRHI();
		}
		else if (SkyLight.ProcessedTexture)
		{
			SkyLightTextureResource = SkyLight.ProcessedTexture->TextureRHI;
			SkyLightCubemapSampler = SkyLight.ProcessedTexture->SamplerStateRHI;
			BlendFraction = SkyLight.BlendFraction;

			if (SkyLight.BlendFraction > 0.0f && SkyLight.BlendDestinationProcessedTexture)
			{
				if (SkyLight.BlendFraction < 1.0f)
				{
					SkyLightBlendDestinationTextureResource = SkyLight.BlendDestinationProcessedTexture;
				}
				else
				{
					SkyLightTextureResource = SkyLight.BlendDestinationProcessedTexture->TextureRHI;
					SkyLightCubemapSampler = SkyLight.ProcessedTexture->SamplerStateRHI;
					BlendFraction = 0;
				}
			}
			
			SkyLightTextureFlags = ERDGTextureFlags::SkipTracking;
		}

		ApplySkyLightMask = 1;
		bSkyLightIsDynamic = !SkyLight.bHasStaticLighting && !SkyLight.bWantsStaticShadowing;
		SkyAverageBrightness = SkyLight.AverageBrightness;
	}

	FRDGTextureRef SkyLightTexture = nullptr;
	if (SkyLightTextureResource)
	{
		SkyLightTexture = RegisterExternalTexture(GraphBuilder, SkyLightTextureResource, TEXT("SkyLightTexture"), SkyLightTextureFlags);
	}
	else
	{
		SkyLightTexture = GSystemTextures.GetCubeBlackDummy(GraphBuilder);
	}

	const int32 CubemapWidth = SkyLightTexture->Desc.Extent.X;
	const float SkyMipCount = FMath::Log2(static_cast<float>(CubemapWidth)) + 1.0f;
		
	OutParameters.SkyLightCubemap = SkyLightTexture;
	OutParameters.SkyLightCubemapSampler = SkyLightCubemapSampler;
	OutParameters.SkyLightBlendDestinationCubemap = SkyLightBlendDestinationTextureResource->TextureRHI;
	OutParameters.SkyLightBlendDestinationCubemapSampler = SkyLightBlendDestinationTextureResource->SamplerStateRHI;
	OutParameters.SkyLightParameters = FVector4f(SkyMipCount - 1.0f, ApplySkyLightMask, bSkyLightIsDynamic ? 1.0f : 0.0f, BlendFraction);

	// Note: GBlackCubeArrayTexture has an alpha of 0, which is needed to represent invalid data so the sky cubemap can still be applied
	FRHITexture* CubeArrayTexture = (SupportsTextureCubeArray(View.FeatureLevel))? GBlackCubeArrayTexture->TextureRHI : GBlackTextureCube->TextureRHI;

	if (View.Family->EngineShowFlags.ReflectionEnvironment
		&& SupportsTextureCubeArray(View.FeatureLevel)
		&& Scene
		&& Scene->ReflectionSceneData.CubemapArray.IsValid()
		&& Scene->ReflectionSceneData.RegisteredReflectionCaptures.Num())
	{
		FVisualizeTexture::AddVisualizeTexturePass(GraphBuilder, GetGlobalShaderMap(GMaxRHIFeatureLevel), GraphBuilder.RegisterExternalTexture(Scene->ReflectionSceneData.CubemapArray.GetRenderTarget()), /*bUseConfig=*/ true);

		CubeArrayTexture = Scene->ReflectionSceneData.CubemapArray.GetRenderTarget()->GetRHI();
	}

	OutParameters.ReflectionCubemap = CubeArrayTexture;
	OutParameters.ReflectionCubemapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	OutParameters.PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
	OutParameters.PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
}

TRDGUniformBufferRef<FReflectionUniformParameters> CreateReflectionUniformBuffer(class FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	FReflectionUniformParameters* Parameters = GraphBuilder.AllocParameters<FReflectionUniformParameters>();
	SetupReflectionUniformParameters(GraphBuilder, View, *Parameters);
	return GraphBuilder.CreateUniformBuffer(Parameters);
}

#if RHI_RAYTRACING
bool ShouldRenderPluginRayTracingGlobalIllumination(const FViewInfo& View)
{
	if (!ShouldRenderPluginGlobalIllumination(View))
	{
		return false;
	}

	bool bAnyRayTracingPassEnabled = false;
	FGlobalIlluminationPluginDelegates::FAnyRayTracingPassEnabled& Delegate = FGlobalIlluminationPluginDelegates::AnyRayTracingPassEnabled();
	Delegate.Broadcast(bAnyRayTracingPassEnabled);

	return ShouldRenderRayTracingEffect(bAnyRayTracingPassEnabled, ERayTracingPipelineCompatibilityFlags::FullPipeline, View);
}

void FDeferredShadingSceneRenderer::PrepareRayTracingGlobalIlluminationPlugin(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders)
{
	// Call the GI plugin delegate function to prepare ray tracing
	FGlobalIlluminationPluginDelegates::FPrepareRayTracing& Delegate = FGlobalIlluminationPluginDelegates::PrepareRayTracing();
	Delegate.Broadcast(View, OutRayGenShaders);
}
#endif // RHI_RAYTRACING

static const FVector SampleArray4x4x6[96] = {
	FVector(0.72084325551986694, -0.44043412804603577, -0.53516626358032227),
	FVector(-0.51286971569061279, 0.57541996240615845, 0.63706874847412109),
	FVector(0.40988105535507202, -0.54854905605316162, 0.7287602424621582),
	FVector(0.10012730211019516, 0.96548169851303101, 0.24045705795288086),
	FVector(0.60404115915298462, -0.24702678620815277, 0.75770187377929688),
	FVector(-0.3765418529510498, -0.88114023208618164, -0.28602123260498047),
	FVector(0.32646462321281433, -0.87295228242874146, 0.362457275390625),
	FVector(0.42743760347366333, 0.90328741073608398, 0.036999702453613281),
	FVector(0.22851260006427765, 0.8621140718460083, 0.45226240158081055),
	FVector(-0.45865404605865479, 0.13879022002220154, 0.87770938873291016),
	FVector(0.87793588638305664, -0.059370972216129303, -0.4750828742980957),
	FVector(-0.13470140099525452, -0.62868881225585938, 0.76590204238891602),
	FVector(-0.92216378450393677, 0.28097033500671387, 0.2658381462097168),
	FVector(0.60047566890716553, 0.69588732719421387, 0.39391613006591797),
	FVector(-0.39624685049057007, 0.41653379797935486, -0.8182225227355957),
	FVector(-0.062934115529060364, -0.8080487847328186, 0.58574438095092773),
	FVector(0.91241759061813354, 0.25627326965332031, 0.31908941268920898),
	FVector(-0.052628953009843826, -0.62639027833938599, -0.77773094177246094),
	FVector(-0.5764470100402832, 0.81458288431167603, 0.064527034759521484),
	FVector(0.99443376064300537, 0.074419610202312469, -0.074586391448974609),
	FVector(-0.73749303817749023, 0.27192473411560059, 0.61819171905517578),
	FVector(0.0065485797822475433, 0.031124366447329521, -0.99949407577514648),
	FVector(-0.80738329887390137, -0.185280442237854, 0.56018161773681641),
	FVector(-0.07494085282087326, -0.28872856497764587, -0.95447349548339844),
	FVector(-0.71886318922042847, 0.51697421073913574, -0.46472930908203125),
	FVector(0.36451923847198486, -0.048588402569293976, 0.92992734909057617),
	FVector(-0.14970993995666504, 0.9416164755821228, -0.30157136917114258),
	FVector(-0.88286900520324707, -0.22010664641857147, -0.41484403610229492),
	FVector(-0.082083694636821747, 0.71625971794128418, -0.69298934936523438),
	FVector(0.69106018543243408, -0.52244770526885986, 0.49948406219482422),
	FVector(-0.53267019987106323, -0.47341263294219971, 0.70152902603149414),
	FVector(0.29150104522705078, 0.25167185068130493, 0.92286968231201172),
	FVector(-0.069487690925598145, -0.038241758942604065, 0.99684953689575195),
	FVector(0.8140520453453064, -0.5661388635635376, -0.129638671875),
	FVector(-0.53156429529190063, -0.12362374365329742, 0.83794784545898438),
	FVector(-0.99061417579650879, 0.10804177820682526, -0.083728790283203125),
	FVector(-0.74865245819091797, -0.30845105648040771, -0.58683681488037109),
	FVector(0.91350913047790527, -0.40578946471214294, 0.028915882110595703),
	FVector(0.50082063674926758, 0.54374086856842041, 0.67344236373901367),
	FVector(0.81965327262878418, 0.26622962951660156, -0.50723791122436523),
	FVector(0.92761707305908203, 0.36275100708007813, -0.089097023010253906),
	FVector(-0.42358329892158508, 0.61944448947906494, -0.66095829010009766),
	FVector(-0.7335321307182312, 0.6022765040397644, 0.31494998931884766),
	FVector(-0.42763453722000122, -0.68648850917816162, -0.58810043334960938),
	FVector(0.33124133944511414, -0.55470693111419678, -0.76326894760131836),
	FVector(-0.45972469449043274, 0.80634123086929321, -0.37211132049560547),
	FVector(0.66711258888244629, 0.23602110147476196, 0.70657968521118164),
	FVector(0.6689566969871521, -0.6665724515914917, -0.32890462875366211),
	FVector(-0.80882930755615234, 0.54724687337875366, -0.21521186828613281),
	FVector(-0.9384690523147583, 0.1244773343205452, -0.32215070724487305),
	FVector(0.76181924343109131, 0.63499248027801514, -0.12812519073486328),
	FVector(-0.32306095957756042, -0.19621354341506958, -0.92581415176391602),
	FVector(0.66310489177703857, 0.73788946866989136, 0.12574243545532227),
	FVector(-0.20186452567577362, 0.83092141151428223, 0.5184788703918457),
	FVector(0.53397935628890991, 0.83287245035171509, -0.14556646347045898),
	FVector(0.23261035978794098, -0.73981714248657227, 0.63131856918334961),
	FVector(0.058953113853931427, -0.8071245551109314, -0.58743047714233398),
	FVector(0.389873206615448, -0.89669209718704224, -0.20962429046630859),
	FVector(0.27890536189079285, -0.95770633220672607, 0.070785999298095703),
	FVector(0.49739769101142883, 0.65539705753326416, -0.5683751106262207),
	FVector(0.24464209377765656, 0.69406133890151978, 0.67707395553588867),
	FVector(0.50111770629882813, -0.28282597661018372, -0.81785726547241211),
	FVector(-0.17602752149105072, -0.47110596299171448, -0.8643341064453125),
	FVector(-0.97248852252960205, -0.16396185755729675, -0.16547727584838867),
	FVector(-0.073738411068916321, 0.50019288063049316, -0.86276865005493164),
	FVector(0.32744523882865906, 0.87091207504272461, -0.36645841598510742),
	FVector(-0.31269559264183044, 0.076923489570617676, -0.94673347473144531),
	FVector(0.01456754095852375, -0.99774020910263062, -0.065592288970947266),
	FVector(-0.16201893985271454, -0.91921764612197876, 0.3588714599609375),
	FVector(-0.78776562213897705, -0.57289564609527588, 0.22630929946899414),
	FVector(0.17262700200080872, -0.24015434086322784, -0.95526218414306641),
	FVector(-0.18667444586753845, 0.54918664693832397, 0.81458377838134766),
	FVector(-0.79800719022750854, -0.48015907406806946, -0.36418628692626953),
	FVector(-0.56875032186508179, -0.47388201951980591, -0.67227888107299805),
	FVector(-0.65060615539550781, -0.72076064348220825, -0.23919820785522461),
	FVector(-0.50273716449737549, 0.78802609443664551, 0.35534524917602539),
	FVector(-0.50821197032928467, -0.85936188697814941, 0.056725025177001953),
	FVector(-0.80488336086273193, -0.57371330261230469, -0.15170955657958984),
	FVector(0.62941837310791016, -0.77012932300567627, 0.10360288619995117),
	FVector(0.30598652362823486, 0.93730741739273071, -0.16681432723999023),
	FVector(-0.44517397880554199, -0.81244134902954102, 0.37650918960571289),
	FVector(0.19359703361988068, -0.22458808124065399, 0.95502901077270508),
	FVector(0.25138014554977417, -0.85482656955718994, -0.45395994186401367),
	FVector(-0.01443319208920002, -0.4333033561706543, 0.90113258361816406),
	FVector(0.53525072336196899, 0.14575909078121185, -0.83202219009399414),
	FVector(0.7941555380821228, 0.48903325200080872, 0.36078166961669922),
	FVector(-0.73473215103149414, -0.00092182925436645746, -0.67835664749145508),
	FVector(-0.96874326467514038, -0.22764001786708832, 0.098572254180908203),
	FVector(-0.31607705354690552, -0.25417521595954895, 0.91405153274536133),
	FVector(0.62423157691955566, 0.718100905418396, -0.3076786994934082),
	FVector(0.022177176550030708, 0.34121012687683105, 0.93972539901733398),
	FVector(0.96729189157485962, -0.022050032392144203, 0.25270605087280273),
	FVector(0.8255578875541687, -0.18236646056175232, 0.53403806686401367),
	FVector(-0.49254557490348816, 0.38371419906616211, 0.78112888336181641),
	FVector(-0.30691400170326233, 0.94623136520385742, 0.10222578048706055),
	FVector(0.061273753643035889, 0.37138348817825317, -0.92645549774169922)
}; // SampleArray4x4x6

static uint32 ReverseBits(uint32 Value, uint32 NumBits)
{
	Value = ((Value & 0x55555555u) << 1u) | ((Value & 0xAAAAAAAAu) >> 1u);
	Value = ((Value & 0x33333333u) << 2u) | ((Value & 0xCCCCCCCCu) >> 2u);
	Value = ((Value & 0x0F0F0F0Fu) << 4u) | ((Value & 0xF0F0F0F0u) >> 4u);
	Value = ((Value & 0x00FF00FFu) << 8u) | ((Value & 0xFF00FF00u) >> 8u);
	Value = (Value << 16u) | (Value >> 16u);
	return Value >> (32u - NumBits);
}

float Hammersley(uint32 Index, uint32 NumSamples)
{
	float E2 = float(ReverseBits(Index, 32)) * 2.3283064365386963e-10;
	return E2;
}


const static uint32 MaxConeDirections = 512;

extern int32 GLumenDiffuseNumTargetCones;

void FDeferredShadingSceneRenderer::SetupCommonDiffuseIndirectParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfo& View,
	HybridIndirectLighting::FCommonParameters& OutCommonDiffuseParameters)
{
	using namespace HybridIndirectLighting;

	const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);

	int32 DownscaleFactor = CVarDiffuseIndirectHalfRes.GetValueOnRenderThread() ? 2 : 1;

	int32 RayCountPerPixel = FMath::Clamp(
		int32(CVarDiffuseIndirectRayPerPixel.GetValueOnRenderThread()),
		1, HybridIndirectLighting::kMaxRayPerPixel);

	if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::SSGI)
	{
		// Standalone SSGI have the number of ray baked in the shader permutation.
		RayCountPerPixel = ScreenSpaceRayTracing::GetSSGIRayCountPerTracingPixel();
	}

	FIntPoint RayStoragePerPixelVector;
	{
		TStaticArray<FIntPoint, 3> RayStoragePerPixelVectorPolicies;
		// X axis needs to be a power of two because of FCommonParameters::PixelRayIndexAbscissMask to avoid a integer division on the GPU
		RayStoragePerPixelVectorPolicies[0].X = FMath::RoundUpToPowerOfTwo(FMath::CeilToInt(FMath::Sqrt(static_cast<float>(RayCountPerPixel))));
		RayStoragePerPixelVectorPolicies[1].X = FMath::RoundUpToPowerOfTwo(FMath::FloorToInt(FMath::Sqrt(static_cast<float>(RayCountPerPixel))));
		RayStoragePerPixelVectorPolicies[2].X = FMath::RoundUpToPowerOfTwo(FMath::CeilToInt(FMath::Sqrt(static_cast<float>(RayCountPerPixel)))) / 2;

		// Compute the Y coordinate.
		for (int32 PolicyId = 0; PolicyId < RayStoragePerPixelVectorPolicies.Num(); PolicyId++)
		{
			if (RayStoragePerPixelVectorPolicies[PolicyId].X == 0)
				RayStoragePerPixelVectorPolicies[PolicyId].X = 1;
			RayStoragePerPixelVectorPolicies[PolicyId].Y = FMath::DivideAndRoundUp(RayCountPerPixel, RayStoragePerPixelVectorPolicies[PolicyId].X);
		}

		// Select the best policy to minimize amount of wasted memory.
		{
			int32 BestPolicyId = -1;
			int32 BestWastage = RayCountPerPixel;

			for (int32 PolicyId = 0; PolicyId < RayStoragePerPixelVectorPolicies.Num(); PolicyId++)
			{
				int32 PolicyWastage = RayStoragePerPixelVectorPolicies[PolicyId].X * RayStoragePerPixelVectorPolicies[PolicyId].Y - RayCountPerPixel;

				if (PolicyWastage < BestWastage)
				{
					BestPolicyId = PolicyId;
					BestWastage = PolicyWastage;
				}

				if (PolicyWastage == 0)
				{
					break;
				}
			}

			check(BestPolicyId != -1);
			RayStoragePerPixelVector = RayStoragePerPixelVectorPolicies[BestPolicyId];
		}
	}

	OutCommonDiffuseParameters.TracingViewportSize = FIntPoint::DivideAndRoundUp(View.ViewRect.Size(), DownscaleFactor);
	ensure(OutCommonDiffuseParameters.TracingViewportSize.X <= kMaxTracingResolution);
	ensure(OutCommonDiffuseParameters.TracingViewportSize.Y <= kMaxTracingResolution);

	OutCommonDiffuseParameters.TracingViewportBufferSize = FIntPoint::DivideAndRoundUp(
		SceneTextures.SceneDepthTexture->Desc.Extent, DownscaleFactor);
	OutCommonDiffuseParameters.DownscaleFactor = DownscaleFactor;
	OutCommonDiffuseParameters.RayCountPerPixel = RayCountPerPixel;
	OutCommonDiffuseParameters.RayStoragePerPixelVector = RayStoragePerPixelVector;
	OutCommonDiffuseParameters.PixelRayIndexAbscissMask = RayStoragePerPixelVector.X - 1;
	OutCommonDiffuseParameters.PixelRayIndexOrdinateShift = FMath::Log2(static_cast<float>(RayStoragePerPixelVector.X));

	OutCommonDiffuseParameters.SceneTextures = SceneTextures;
	OutCommonDiffuseParameters.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
}

void FDeferredShadingSceneRenderer::DispatchAsyncLumenIndirectLightingWork(
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	FInstanceCullingManager& InstanceCullingManager,
	FLumenSceneFrameTemporaries& LumenFrameTemporaries,
	FDynamicShadowsTaskData* DynamicShadowsTaskData,
	FRDGTextureRef LightingChannelsTexture,
	FAsyncLumenIndirectLightingOutputs& Outputs)
{
	const bool bAsyncComputeDiffuseIndirect = LumenDiffuseIndirect::UseAsyncCompute(ViewFamily, GetViewPipelineState(Views[0]).DiffuseIndirectMethod);
	const bool bAsyncComputeReflections = LumenReflections::UseAsyncCompute(ViewFamily, GetViewPipelineState(Views[0]).DiffuseIndirectMethod, GetViewPipelineState(Views[0]).ReflectionsMethod);
	extern int32 GLumenVisualizeIndirectDiffuse;

	if (!Lumen::UseAsyncCompute(ViewFamily)
		|| !bAsyncComputeDiffuseIndirect
		|| GLumenVisualizeIndirectDiffuse
		|| ViewFamily.EngineShowFlags.VisualizeLightCulling)
	{
		return;
	}

	if (DynamicShadowsTaskData)
	{
		for (const FViewInfo& View : Views)
		{
			const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);

			// Write shadow GPU instance data prior to launching async compute tasks to avoid the early join back to graphics.
			if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen)
			{
				FinishDynamicShadowMeshPassSetup(GraphBuilder, DynamicShadowsTaskData);
				break;
			}
		}
	}

	LLM_SCOPE_BYTAG(Lumen);
	RDG_EVENT_SCOPE(GraphBuilder, "DiffuseIndirectAndAO");
	SCOPED_NAMED_EVENT(DispatchAsyncLumenIndirectLightingWork, FColor::Emerald);
	
	Outputs.DoneAsync(bAsyncComputeReflections);
	Outputs.Resize(Views.Num());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);
		FAsyncLumenIndirectLightingOutputs::FViewOutputs& ViewOutputs = Outputs.ViewOutputs[ViewIndex];

		if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen)
		{
			ViewOutputs.IndirectLightingTextures = RenderLumenFinalGather(
				GraphBuilder,
				SceneTextures,
				LumenFrameTemporaries,
				LightingChannelsTexture,
				View,
				ViewIndex,
				&View.PrevViewInfo,
				ViewOutputs.MeshSDFGridParameters,
				ViewOutputs.RadianceCacheParameters,
				ViewOutputs.ScreenBentNormalParameters,
				ViewOutputs.LumenUpsampleParameters,
				ERDGPassFlags::AsyncCompute);

			if (ViewPipelineState.ReflectionsMethod == EReflectionsMethod::Lumen && bAsyncComputeReflections)
			{
				FLumenReflectionsConfig LumenReflectionsConfig;

				ViewOutputs.IndirectLightingTextures.Textures[3] = RenderLumenReflections(
					GraphBuilder,
					View,
					SceneTextures,
					LumenFrameTemporaries,
					ViewOutputs.MeshSDFGridParameters,
					ViewOutputs.RadianceCacheParameters,
					ELumenReflectionPass::Opaque,
					LumenReflectionsConfig,
					ERDGPassFlags::AsyncCompute);
			}

		}
	}
}

void FDeferredShadingSceneRenderer::RenderDiffuseIndirectAndAmbientOcclusion(
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	FLumenSceneFrameTemporaries& LumenFrameTemporaries,
	FRDGTextureRef LightingChannelsTexture,
	bool bCompositeRegularLumenOnly,
	bool bIsVisualizePass,
	FAsyncLumenIndirectLightingOutputs& AsyncLumenIndirectLightingOutputs)
{
	using namespace HybridIndirectLighting;

	extern int32 GLumenVisualizeIndirectDiffuse;
	if ((GLumenVisualizeIndirectDiffuse != 0) != bIsVisualizePass
		|| ViewFamily.EngineShowFlags.VisualizeLightCulling)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FDeferredShadingSceneRenderer::RenderDiffuseIndirectAndAmbientOcclusion);
	RDG_EVENT_SCOPE(GraphBuilder, "DiffuseIndirectAndAO");

	FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures.UniformBuffer);
	FRDGTextureRef SceneColorTexture = SceneTextures.Color.Target;
	FRDGTextureRef SceneDepthTexture = SceneTextures.Depth.Target;

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	auto ShouldSkipView = [bCompositeRegularLumenOnly, &AsyncLumenIndirectLightingOutputs](const FPerViewPipelineState& ViewPipelineState, bool bApplyAmbientCubmap)
	{
		const bool bOnlyCompositeStepLeft = AsyncLumenIndirectLightingOutputs.StepsLeft == ELumenIndirectLightingSteps::Composite;
		const bool bRegularLumenView = ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen;

		// If Static lighting is disabled and this pass only composes (SS)AO on top of the scene color, we can skip this step all together.
		const bool bSkipSSAOComposition = Substrate::IsSubstrateEnabled() && ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Disabled && ViewPipelineState.AmbientOcclusionMethod == EAmbientOcclusionMethod::SSAO && !IsStaticLightingAllowed() && !bApplyAmbientCubmap;
		return bCompositeRegularLumenOnly != (bOnlyCompositeStepLeft && bRegularLumenView) || bSkipSSAOComposition;
	};

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);
		const bool bApplyAmbientCubmap = IsAmbientCubemapPassRequired(View) && !bIsVisualizePass;

		if (ShouldSkipView(ViewPipelineState, bApplyAmbientCubmap))
		{
			continue;
		}

		int32 DenoiseMode = CVarDiffuseIndirectDenoiser.GetValueOnRenderThread();

		// Setup the common diffuse parameter for this view.
		FCommonParameters CommonDiffuseParameters;
		SetupCommonDiffuseIndirectParameters(GraphBuilder, SceneTextureParameters, View, /* out */ CommonDiffuseParameters);

		// Update old ray tracing config for the denoiser.
		IScreenSpaceDenoiser::FAmbientOcclusionRayTracingConfig RayTracingConfig;
		{
			RayTracingConfig.RayCountPerPixel = CommonDiffuseParameters.RayCountPerPixel;
			RayTracingConfig.ResolutionFraction = 1.0f / float(CommonDiffuseParameters.DownscaleFactor);
		}

		ScreenSpaceRayTracing::FPrevSceneColorMip PrevSceneColorMip;
		if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::SSGI
			&& View.PrevViewInfo.ScreenSpaceRayTracingInput.IsValid())
		{
			PrevSceneColorMip = ScreenSpaceRayTracing::ReducePrevSceneColorMip(GraphBuilder, SceneTextureParameters, View);
		}

		FSSDSignalTextures DenoiserOutputs;
		IScreenSpaceDenoiser::FDiffuseIndirectInputs DenoiserInputs;
		IScreenSpaceDenoiser::FDiffuseIndirectHarmonic DenoiserSphericalHarmonicInputs;
		FLumenScreenSpaceBentNormalParameters ScreenBentNormalParameters;
		FLumenUpsampleParameters LumenUpsampleParameters;
		LumenUpsampleParameters.DownsampleFactor = 1;

		//Tiled screen space reflection context
		FScreenSpaceReflectionTileClassification ScreenSpaceReflectionTileClassification;
		bool bRunSSRTiledComposite = false;

		if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::SSGI)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "SSGI %dx%d", CommonDiffuseParameters.TracingViewportSize.X, CommonDiffuseParameters.TracingViewportSize.Y);
			DenoiserInputs = ScreenSpaceRayTracing::CastStandaloneDiffuseIndirectRays(
				GraphBuilder, CommonDiffuseParameters, PrevSceneColorMip, View);
		}
		else if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen)
		{
			check(ViewPipelineState.DiffuseIndirectDenoiser == IScreenSpaceDenoiser::EMode::Disabled);

			FLumenMeshSDFGridParameters MeshSDFGridParameters;
			LumenRadianceCache::FRadianceCacheInterpolationParameters RadianceCacheParameters;
			const ELumenIndirectLightingSteps StepsLeft = AsyncLumenIndirectLightingOutputs.StepsLeft;
			const bool bDoComposite = StepsLeft == ELumenIndirectLightingSteps::All || StepsLeft == ELumenIndirectLightingSteps::Composite;

			if (EnumHasAnyFlags(StepsLeft, ELumenIndirectLightingSteps::ScreenProbeGather))
			{
				DenoiserOutputs = RenderLumenFinalGather(
					GraphBuilder,
					SceneTextures,
					LumenFrameTemporaries,
					LightingChannelsTexture,
					View,
					ViewIndex,
					&View.PrevViewInfo,
					MeshSDFGridParameters,
					RadianceCacheParameters,
					ScreenBentNormalParameters,
					LumenUpsampleParameters,
					ERDGPassFlags::Compute);
			}

			auto& AllViewOutputs = AsyncLumenIndirectLightingOutputs.ViewOutputs;
			auto& OutTextures = bDoComposite ? DenoiserOutputs : AllViewOutputs[ViewIndex].IndirectLightingTextures;

			if (ViewPipelineState.ReflectionsMethod == EReflectionsMethod::Lumen)
			{
				if (EnumHasAnyFlags(StepsLeft, ELumenIndirectLightingSteps::Reflections))
				{
					const auto& MeshSDFGridParams = bDoComposite ? MeshSDFGridParameters : AllViewOutputs[ViewIndex].MeshSDFGridParameters;
					const auto& RadianceCacheParams = bDoComposite ? RadianceCacheParameters : AllViewOutputs[ViewIndex].RadianceCacheParameters;

					FLumenReflectionsConfig LumenReflectionsConfig;

					OutTextures.Textures[3] = RenderLumenReflections(
						GraphBuilder,
						View,
						SceneTextures,
						LumenFrameTemporaries,
						MeshSDFGridParams,
						RadianceCacheParams,
						ELumenReflectionPass::Opaque,
						LumenReflectionsConfig,
						ERDGPassFlags::Compute);
				}
			}
			else if (ViewPipelineState.ReflectionsMethod == EReflectionsMethod::SSR)
			{
				ESSRQuality SSRQuality;
				IScreenSpaceDenoiser::FReflectionsRayTracingConfig DenoiserConfig;
				ScreenSpaceRayTracing::GetSSRQualityForView(View, &SSRQuality, &DenoiserConfig);

				RDG_EVENT_SCOPE(GraphBuilder, "ScreenSpaceReflections(Quality=%d)", int32(SSRQuality));

				ScreenSpaceReflectionTileClassification = ClassifySSRTiles(GraphBuilder, View, SceneTextures, nullptr);
				bRunSSRTiledComposite = IsDefaultSSRTileEnabled(View);

				IScreenSpaceDenoiser::FReflectionsInputs SSRDenoiserInputs;
				ScreenSpaceRayTracing::RenderScreenSpaceReflections(
					GraphBuilder, SceneTextureParameters, SceneColorTexture, View, SSRQuality, /*bDenoise*/ false, &SSRDenoiserInputs,/*bSingleLayerWater*/ false, 
					bRunSSRTiledComposite ? &ScreenSpaceReflectionTileClassification.TiledReflection : nullptr);
				OutTextures.Textures[3] = SSRDenoiserInputs.Color;
			}
			else if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen)
			{
				// No Lumen Reflections and no SSR - use Lumen Rough Reflections only.
				OutTextures.Textures[3] = OutTextures.Textures[2];
			}

			if (!bDoComposite)
			{
				continue;
			}
			else if (bCompositeRegularLumenOnly)
			{
				FAsyncLumenIndirectLightingOutputs::FViewOutputs& ViewOutputs = AsyncLumenIndirectLightingOutputs.ViewOutputs[ViewIndex];
				DenoiserOutputs = ViewOutputs.IndirectLightingTextures;
				MeshSDFGridParameters = ViewOutputs.MeshSDFGridParameters;
				RadianceCacheParameters = ViewOutputs.RadianceCacheParameters;
				ScreenBentNormalParameters = ViewOutputs.ScreenBentNormalParameters;
				LumenUpsampleParameters = ViewOutputs.LumenUpsampleParameters;
			}
		}
		else if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Plugin)
		{
			// Get the resources and call the GI plugin's rendering function delegate
			FGlobalIlluminationPluginResources GIPluginResources;
			GIPluginResources.GBufferA = SceneTextures.GBufferA;
			GIPluginResources.GBufferB = SceneTextures.GBufferB;
			GIPluginResources.GBufferC = SceneTextures.GBufferC;
			GIPluginResources.LightingChannelsTexture = LightingChannelsTexture;
			GIPluginResources.SceneDepthZ = SceneTextures.Depth.Target;
			GIPluginResources.SceneColor = SceneTextures.Color.Target;

			FGlobalIlluminationPluginDelegates::FRenderDiffuseIndirectLight& Delegate = FGlobalIlluminationPluginDelegates::RenderDiffuseIndirectLight();
			Delegate.Broadcast(*Scene, View, GraphBuilder, GIPluginResources);
		}

		// Free view state resources when no longer enabled, ie scalability change
		if (View.ViewState)
		{
			const bool bLumen = View.ViewState
				&& (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen || ViewPipelineState.ReflectionsMethod != EReflectionsMethod::Lumen);
			const bool bMegaLights = View.ViewState && MegaLights::IsEnabled(ViewFamily);

			if (!bLumen)
			{
				View.ViewState->Lumen.SafeRelease();
			}

			if (!bMegaLights)
			{
				View.ViewState->MegaLights.SafeRelease();
			}
			else
			{
				// Release sub-feature resources when their specific feature is disabled while MegaLights itself remains enabled

				if (!MegaLights::UseFrontLayerTranslucencyDirectLighting(View))
				{
					View.ViewState->MegaLights.FrontLayerTranslucency.SafeRelease();
				}

				if (!MegaLights::IsHairStrandsEnabled(View))
				{
					View.ViewState->MegaLights.HairStrands.SafeRelease();
				}
			}

			if (!bLumen && !bMegaLights)
			{
				View.ViewState->StochasticLighting.SafeRelease();
			}
		}

		FRDGTextureRef AmbientOcclusionMask = DenoiserInputs.AmbientOcclusionMask;

		if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen)
		{
			// NOP
		}
		else if (ViewPipelineState.DiffuseIndirectDenoiser == IScreenSpaceDenoiser::EMode::Disabled)
		{
			DenoiserOutputs.Textures[0] = DenoiserInputs.Color;
			DenoiserOutputs.Textures[1] = SystemTextures.White;
		}
		else
		{
			const IScreenSpaceDenoiser* DefaultDenoiser = IScreenSpaceDenoiser::GetDefaultDenoiser();
			const IScreenSpaceDenoiser* DenoiserToUse = 
				ViewPipelineState.DiffuseIndirectDenoiser == IScreenSpaceDenoiser::EMode::DefaultDenoiser
				? DefaultDenoiser : GScreenSpaceDenoiser;

			RDG_EVENT_SCOPE(GraphBuilder, "%s%s(DiffuseIndirect) %dx%d",
				DenoiserToUse != DefaultDenoiser ? TEXT("ThirdParty ") : TEXT(""),
				DenoiserToUse->GetDebugName(),
				View.ViewRect.Width(), View.ViewRect.Height());

			if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::SSGI)
			{
				DenoiserOutputs = DenoiserToUse->DenoiseScreenSpaceDiffuseIndirect(
					GraphBuilder,
					View,
					&View.PrevViewInfo,
					SceneTextureParameters,
					DenoiserInputs,
					RayTracingConfig);

				AmbientOcclusionMask = DenoiserOutputs.Textures[1];
			}
		}

		bool bWritableAmbientOcclusionMask = true;
		if (ViewPipelineState.AmbientOcclusionMethod == EAmbientOcclusionMethod::Disabled)
		{
			AmbientOcclusionMask = nullptr;
			bWritableAmbientOcclusionMask = false;
		}
		else if (ViewPipelineState.AmbientOcclusionMethod == EAmbientOcclusionMethod::RTAO)
		{
			RenderRayTracingAmbientOcclusion(
				GraphBuilder,
				View,
				SceneTextureParameters,
				&AmbientOcclusionMask);
		}
		else if (ViewPipelineState.AmbientOcclusionMethod == EAmbientOcclusionMethod::SSGI)
		{
			check(AmbientOcclusionMask);
		}
		else if (ViewPipelineState.AmbientOcclusionMethod == EAmbientOcclusionMethod::SSAO)
		{
			// Fetch result of SSAO that was done earlier.
			if (HasBeenProduced(SceneTextures.ScreenSpaceAO))
			{
				AmbientOcclusionMask = SceneTextures.ScreenSpaceAO;
			}
			else
			{
				AmbientOcclusionMask = GetScreenSpaceAOFallback(SystemTextures);
				bWritableAmbientOcclusionMask = false;
			}
		}
		else
		{
			unimplemented();
			bWritableAmbientOcclusionMask = false;
		}

		// Extract the dynamic AO for application of AO beyond RenderDiffuseIndirectAndAmbientOcclusion()
		if (AmbientOcclusionMask && ViewPipelineState.AmbientOcclusionMethod != EAmbientOcclusionMethod::SSAO)
		{
			//ensureMsgf(!bApplySSAO, TEXT("Looks like SSAO has been computed for this view but is being overridden."));
			ensureMsgf(Views.Num() == 1, TEXT("Need to add support for one AO texture per view in FSceneTextures")); // TODO.
			SceneTextures.ScreenSpaceAO = AmbientOcclusionMask;
		}

		if (HairStrands::HasViewHairStrandsData(View) && (ViewPipelineState.AmbientOcclusionMethod == EAmbientOcclusionMethod::SSGI || ViewPipelineState.AmbientOcclusionMethod == EAmbientOcclusionMethod::SSAO) && bWritableAmbientOcclusionMask)
		{
			RenderHairStrandsAmbientOcclusion(
				GraphBuilder,
				View,
				AmbientOcclusionMask);
		}

		// Applies diffuse indirect and ambient occlusion to the scene color.
        // TODO: Why are we disabling Metal SM5 here?
		const bool bApplyDiffuseIndirect = (DenoiserOutputs.Textures[0] || AmbientOcclusionMask) && (!bIsVisualizePass || ViewPipelineState.DiffuseIndirectDenoiser != IScreenSpaceDenoiser::EMode::Disabled || ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen)
			&& !(IsMetalPlatform(ShaderPlatform) && !IsMetalSM5Platform(ShaderPlatform) && !IsMetalSM6Platform(ShaderPlatform));

		// When Lumen visualization is enabled, clear the scene color to ensure sky pixels are cleared correctly.
		if (Substrate::IsSubstrateEnabled() && bIsVisualizePass)
		{
			AddClearRenderTargetPass(GraphBuilder, SceneColorTexture, FLinearColor::Black);
		}

		bool bEnableCopyPass = (!RHISupportsDualSourceBlending(ShaderPlatform) || CVarDiffuseIndirectForceCopyPass.GetValueOnRenderThread()) 
								&& DenoiserOutputs.Textures[0];

		FRDGTextureRef SceneColorCopyTexture;

		if (bEnableCopyPass)
		{
			FRDGTextureDesc SceneColorCopyTextureDesc = FRDGTextureDesc::Create2D(
				SceneColorTexture->Desc.Extent,
				SceneColorTexture->Desc.Format,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_RenderTargetable);

			
			SceneColorCopyTexture = GraphBuilder.CreateTexture(SceneColorCopyTextureDesc, TEXT("SceneCopyTextureCopy0"));
		}

		auto SetDefaultBlackIfNotTexture2DArray = [&SystemTextures](FRDGTextureRef In)
		{
			return In ? (In->Desc.Dimension == ETextureDimension::Texture2DArray ? In : SystemTextures.BlackArray) : nullptr;
		};

		auto SetDefaultBlackIfNotTexture2D = [&SystemTextures](FRDGTextureRef In)
		{
			return In ? (In->Desc.Dimension == ETextureDimension::Texture2D ? In : SystemTextures.Black) : nullptr;
		};

		auto ApplyDiffuseIndirect = [&](ESubstrateTileType TileType)
		{
			if (bEnableCopyPass)
			{
				FRHICopyTextureInfo CopyInfo;
				CopyInfo.Size = SceneColorTexture->Desc.GetSize();
				AddCopyTexturePass(GraphBuilder, SceneColorTexture, SceneColorCopyTexture, CopyInfo);
			}
			FDiffuseIndirectCompositeParameters* PassParameters = GraphBuilder.AllocParameters<FDiffuseIndirectCompositeParameters>();
			FDiffuseIndirectCompositePS::FParameters* PSParameters = &PassParameters->PS;
			PSParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
			PSParameters->SceneTexturesStruct = SceneTextures.UniformBuffer;
			PSParameters->AmbientOcclusionStaticFraction = FMath::Clamp(View.FinalPostProcessSettings.AmbientOcclusionStaticFraction, 0.0f, 1.0f);

			const FIntPoint BufferExtent = SceneTextureParameters.SceneDepthTexture->Desc.Extent;

			{
				// Placeholder texture for textures pulled in from SSDCommon.ush
				FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
					FIntPoint(1),
					PF_R32_UINT,
					FClearValueBinding::Black,
					TexCreate_ShaderResource);
				FRDGTextureRef CompressedMetadataPlaceholder = GraphBuilder.CreateTexture(Desc, TEXT("CompressedMetadataPlaceholder"));

				PSParameters->CompressedMetadata[0] = CompressedMetadataPlaceholder;
				PSParameters->CompressedMetadata[1] = CompressedMetadataPlaceholder;
			}

			PSParameters->BufferUVToOutputPixelPosition = BufferExtent;
			PSParameters->EyeAdaptation = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));
			LumenReflections::SetupCompositeParameters(View, ViewPipelineState.ReflectionsMethod, PSParameters->ReflectionsCompositeParameters);
			PSParameters->ScreenBentNormalParameters = ScreenBentNormalParameters;
			PSParameters->LumenUpsampleParameters = LumenUpsampleParameters;

			if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen)
			{
				FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
				TUniformBufferRef<FBlueNoise> BlueNoiseUniformBuffer = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);
				PSParameters->BlueNoise = BlueNoiseUniformBuffer;
			}

			PSParameters->bLumenSupportBackfaceDiffuse = ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen && DenoiserOutputs.Textures[1] != SystemTextures.Black;
			PSParameters->bLumenReflectionInputIsSSR = ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen && ViewPipelineState.ReflectionsMethod == EReflectionsMethod::SSR;
			PSParameters->LumenGIDownsampleFactor = LumenUpsampleParameters.DownsampleFactor;
			PSParameters->LumenFoliageOcclusionStrength = LumenShortRangeAO::GetFoliageOcclusionStrength();
			extern float GLumenMaxShortRangeAOMultibounceAlbedo;
			PSParameters->LumenMaxAOMultibounceAlbedo = GLumenMaxShortRangeAOMultibounceAlbedo;
			PSParameters->LumenReflectionSpecularScale = GetLumenReflectionSpecularScale();
			PSParameters->LumenAmbientOcclusionIntensity = FMath::Clamp<float>(View.FinalPostProcessSettings.LumenAmbientOcclusionIntensity, 0.0f, 1.0f);
			PSParameters->LumenReflectionContrast = GetLumenReflectionContrast();

			PSParameters->bVisualizeDiffuseIndirect = bIsVisualizePass;

			PSParameters->DiffuseIndirect_Lumen_0 = SetDefaultBlackIfNotTexture2DArray(DenoiserOutputs.Textures[0]);
			PSParameters->DiffuseIndirect_Lumen_1 = SetDefaultBlackIfNotTexture2DArray(DenoiserOutputs.Textures[1]);
			PSParameters->DiffuseIndirect_Lumen_2 = SetDefaultBlackIfNotTexture2DArray(DenoiserOutputs.Textures[2]);
			PSParameters->DiffuseIndirect_Lumen_3 = SetDefaultBlackIfNotTexture2DArray(DenoiserOutputs.Textures[3]);

			PSParameters->DiffuseIndirect = DenoiserOutputs;
			for (uint32 EntryIt = 0; EntryIt < kMaxDenoiserBufferProcessingCount; ++EntryIt)
			{
				PSParameters->DiffuseIndirect.Textures[EntryIt] = SetDefaultBlackIfNotTexture2D(PSParameters->DiffuseIndirect.Textures[EntryIt]);
			}

			PSParameters->DiffuseIndirectSampler = TStaticSamplerState<SF_Point>::GetRHI();

			PSParameters->PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
			PSParameters->PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			PSParameters->AmbientOcclusionTexture = AmbientOcclusionMask;
			PSParameters->AmbientOcclusionSampler = TStaticSamplerState<SF_Point>::GetRHI();

			if (bRunSSRTiledComposite)
			{
				PSParameters->SSRTiledViewRes = ScreenSpaceReflectionTileClassification.TiledViewRes;
				PSParameters->SSRTileMaskBuffer = GraphBuilder.CreateSRV(ScreenSpaceReflectionTileClassification.TileMaskBuffer);
			}

			if (bEnableCopyPass)
			{
				PSParameters->SceneColorTexture = SceneColorCopyTexture;
				PSParameters->SceneColorSampler = TStaticSamplerState<SF_Point>::GetRHI();
			}
			
			if (!PSParameters->AmbientOcclusionTexture || bIsVisualizePass)
			{
				PSParameters->AmbientOcclusionTexture = SystemTextures.White;
			}

			Denoiser::SetupCommonShaderParameters(
				View, SceneTextureParameters,
				View.ViewRect,
				1.0f / CommonDiffuseParameters.DownscaleFactor,
				/* out */ &PSParameters->DenoiserCommonParameters);
			PSParameters->SceneTextures = SceneTextureParameters;
			PSParameters->ViewUniformBuffer = View.ViewUniformBuffer;

			PSParameters->RenderTargets[0] = FRenderTargetBinding(
				SceneColorTexture, ERenderTargetLoadAction::ELoad);

			if (UseSeparatedSubsurfaceDiffuse(ShaderPlatform))
			{
				if (bEnableCopyPass)
				{
					PSParameters->RenderTargets[1] = FRenderTargetBinding(
						View.GetSceneTextures().SeparatedSubsurfaceDiffuse, ERenderTargetLoadAction::ELoad);
				}
				else
				{
					SetupSubsurfaceDiffuseWriteParameters(GraphBuilder, View, PSParameters->SubsurfaceDiffuseParameters);
				}
			}

			const bool bWriteStencil = !bCompositeRegularLumenOnly && Substrate::IsSubstrateEnabled() && Substrate::GetStencilPassStage() == Substrate::EStencilPassStage::DuringIndirectLighting;
			if (bWriteStencil)
			{
				PSParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);
			}

			{
				FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
					SceneColorTexture->Desc.Extent,
					PF_FloatRGBA,
					FClearValueBinding::None,
					TexCreate_ShaderResource | TexCreate_UAV);

				PSParameters->PassDebugOutput = GraphBuilder.CreateUAV(
					GraphBuilder.CreateTexture(Desc, TEXT("DebugDiffuseIndirectComposite")));
			}

			const TCHAR* DiffuseIndirectSampling = TEXT("Disabled");
			FDiffuseIndirectCompositePS::FPermutationDomain PermutationVector;
			PermutationVector.Set<Substrate::FSubstrateTileType>(ESubstrateTileType::EComplex);

			bool bUpscale = false;
			bool bApplyAOToSceneColor = true;

			if (DenoiserOutputs.Textures[0])
			{
				if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen)
				{
					PermutationVector.Set<FDiffuseIndirectCompositePS::FApplyDiffuseIndirectDim>(2);
					PermutationVector.Set<FDiffuseIndirectCompositePS::FScreenBentNormalMode>(ScreenBentNormalParameters.ShortRangeAOMode);
					PermutationVector.Set<FDiffuseIndirectCompositePS::FUpscaleLumenGIDim>(LumenUpsampleParameters.DownsampleFactor != 1);
					PermutationVector.Set<FDiffuseIndirectCompositePS::FShortRangeGI>(ScreenBentNormalParameters.ShortRangeGITexture != nullptr);

					if (Substrate::IsSubstrateEnabled() && TileType != ESubstrateTileType::ECount)
					{
						PermutationVector.Set<Substrate::FSubstrateTileType>(TileType);
					}
					DiffuseIndirectSampling = TEXT("ScreenProbeGather");
					bApplyAOToSceneColor = false;
				}
				else
				{
					PermutationVector.Set<FDiffuseIndirectCompositePS::FApplyDiffuseIndirectDim>(1);
					DiffuseIndirectSampling = TEXT("SSGI");
					bUpscale = DenoiserOutputs.Textures[0]->Desc.Extent != SceneColorTexture->Desc.Extent;
				}

				PermutationVector.Set<FDiffuseIndirectCompositePS::FUpscaleSSGIDim>(bUpscale);
			}

			PermutationVector.Set<FDiffuseIndirectCompositePS::FEnableDualSrcBlending>(!bEnableCopyPass);
			PermutationVector.Set<FDiffuseIndirectCompositePS::FScreenSpaceReflectionTiledComposition>(bRunSSRTiledComposite);
			
			const bool bHasHairCardsOrMeshes = View.HairCardsMeshElements.Num() > 0;
			PermutationVector.Set<FDiffuseIndirectCompositePS::FHairComplexTransmittance>(bHasHairCardsOrMeshes);

			TShaderMapRef<FDiffuseIndirectCompositePS> PixelShader(View.ShaderMap, PermutationVector);

			FRHIBlendState* BlendState;
			if (!bEnableCopyPass)
			{
				if (PermutationVector.Get<FDiffuseIndirectCompositePS::FApplyDiffuseIndirectDim>() > 0)
				{
					if (bApplyAOToSceneColor)
					{
						// FinalColor = SceneColor * AO + Indirect;
						BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Source1Color, BO_Add, BF_One, BF_Source1Alpha>::GetRHI();
					}
					else
					{
						// FinalColor = SceneColor + Indirect;
						BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
					}
				}		
				else
				{
					BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_SourceColor, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
				}
			}
			else
			{
				if (UseSeparatedSubsurfaceDiffuse(ShaderPlatform))
				{
					BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
						                           CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
				}
				else
				{
					BlendState = TStaticBlendState<>::GetRHI();
				}
			}

			if (bIsVisualizePass)
			{
				BlendState = TStaticBlendState<>::GetRHI();
			}

			if (TileType == ESubstrateTileType::ECount)
			{
				ClearUnusedGraphResources(PixelShader, PSParameters);

				// only use depth bound optimization when diffuse indirect is disable
				bool bUseDepthBounds = CVarDiffuseIndirectOffUseDepthBoundsAO.GetValueOnRenderThread() &&
										!PermutationVector.Get<FDiffuseIndirectCompositePS::FUpscaleSSGIDim>() &&
										AmbientOcclusionMask &&
										ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Disabled;

				if (bUseDepthBounds)
				{
					PSParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target,
						ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
				}

				GraphBuilder.AddPass(
					RDG_EVENT_NAME(
						"DiffuseIndirectComposite(DiffuseIndirect=%s%s%s) %dx%d",
						DiffuseIndirectSampling,
						PermutationVector.Get<FDiffuseIndirectCompositePS::FUpscaleSSGIDim>() ? TEXT(" UpscaleDiffuseIndirect") : TEXT(""),
						AmbientOcclusionMask ? TEXT(" ApplyAOToSceneColor") : TEXT(""),
						View.ViewRect.Width(), View.ViewRect.Height()), 
					PSParameters,
					ERDGPassFlags::Raster,
					[&View, PSParameters,PixelShader, BlendState, bUseDepthBounds](FRDGAsyncTask, FRHICommandList& RHICmdList)
					{
						check(PixelShader.IsValid());
						RHICmdList.SetViewport((float)View.ViewRect.Min.X, (float)View.ViewRect.Min.Y, 0.0f, (float)View.ViewRect.Max.X, (float)View.ViewRect.Max.Y, 1.0f);

						FGraphicsPipelineStateInitializer GraphicsPSOInit;
						FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, View.ShaderMap, PixelShader, /* out */GraphicsPSOInit);
						GraphicsPSOInit.BlendState = BlendState;
						GraphicsPSOInit.bDepthBounds = false;

						if (bUseDepthBounds)
						{
							GraphicsPSOInit.bDepthBounds = true;
							const FFinalPostProcessSettings& Settings = View.FinalPostProcessSettings;
							const FMatrix& ProjectionMatrix = View.ViewMatrices.GetViewToClip();
							const FVector4f Far = (FVector4f)ProjectionMatrix.TransformFVector4(FVector4(0, 0, Settings.AmbientOcclusionFadeDistance));
							float DepthFar = FMath::Clamp(Far.Z / Far.W, 0.0f, 1.0f);
							RHICmdList.SetDepthBounds(DepthFar, 1.0f);
						}

						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
						SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSParameters);

						FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList);

						if (bUseDepthBounds)
						{
							RHICmdList.SetDepthBounds(0.f, 1.0f);
						}
					});
			}
			else
			{
				check(Substrate::IsSubstrateEnabled());

				// Rough refraction targets
				if (Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()))
				{
					const FSubstrateSceneData& SubstrateSceneData = *SceneTextures.SubstrateSceneData;
					PSParameters->OutOpaqueRoughRefractionSceneColor = GraphBuilder.CreateUAV(SubstrateSceneData.SeparatedOpaqueRoughRefractionSceneColor);
					PSParameters->OutSubSurfaceSceneColor = GraphBuilder.CreateUAV(SubstrateSceneData.SeparatedSubSurfaceSceneColor);
				}

				Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableDebug >(false);
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector >(false);
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FClassificationTileSize16 >(Substrate::UseTileSize16());
				TShaderMapRef<Substrate::FSubstrateTilePassVS> TileVertexShader(View.ShaderMap, VSPermutationVector);

				ClearUnusedGraphResources(PixelShader, PSParameters);

				EPrimitiveType PrimitiveType = PT_TriangleList;
				PassParameters->VS = Substrate::SetTileParameters(GraphBuilder, View, TileType, PrimitiveType);
				ClearUnusedGraphResources(TileVertexShader, &PassParameters->VS);

				GraphBuilder.AddPass(
					RDG_EVENT_NAME(
						"DiffuseIndirectComposite(%s%s%s)(%s) %dx%d",
						DiffuseIndirectSampling,
						PermutationVector.Get<FDiffuseIndirectCompositePS::FUpscaleSSGIDim>() ? TEXT(" UpscaleDiffuseIndirect") : TEXT(""),
						AmbientOcclusionMask ? TEXT(" ApplyAOToSceneColor") : TEXT(""),
						ToString(TileType),
						View.ViewRect.Width(), View.ViewRect.Height()),
					PassParameters,
					ERDGPassFlags::Raster,
					[&View, TileVertexShader, PixelShader, PassParameters, TileType, BlendState, PrimitiveType, bWriteStencil](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					// Set the device viewport for the view.
					RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

					GraphicsPSOInit.BlendState = BlendState;
					GraphicsPSOInit.PrimitiveType = PrimitiveType;
					GraphicsPSOInit.bDepthBounds = false;
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = TileVertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();		
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					uint32 StencilRef = 0;
					if (bWriteStencil)
					{
						Substrate::GetSubstrateTiledDepthStencilStateWrite(TileType, GraphicsPSOInit, StencilRef);
					}
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);
					SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), PassParameters->VS);
					RHICmdList.DrawPrimitiveIndirect(PassParameters->VS.TileIndirectBuffer->GetIndirectRHICallBuffer(), Substrate::TileTypeDrawIndirectArgOffset(TileType));
				});
			}

		}; // ApplyDiffuseIndirect

		if (bApplyDiffuseIndirect)
		{
			if (Substrate::IsSubstrateEnabled())
			{
				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplexSpecial)) 	{ ApplyDiffuseIndirect(ESubstrateTileType::EComplexSpecial);}
				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplex)) 			{ ApplyDiffuseIndirect(ESubstrateTileType::EComplex);       }
				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESingle)) 			{ ApplyDiffuseIndirect(ESubstrateTileType::ESingle);        }
				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESimple)) 			{ ApplyDiffuseIndirect(ESubstrateTileType::ESimple);        }
			}
			else
			{
				ApplyDiffuseIndirect(ESubstrateTileType::ECount);
			}
		}

		// Apply the ambient cubemaps
		if (bApplyAmbientCubmap)
		{
			auto ApplyAmbientCubemapComposite = [&](ESubstrateTileType TileType)
			{			
				FAmbientCubemapCompositeParameters* PassParameters = GraphBuilder.AllocParameters<FAmbientCubemapCompositeParameters>();
				FAmbientCubemapCompositePS::FParameters* PSParameters = &PassParameters->PS;
				PSParameters->PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
				PSParameters->PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			
				PSParameters->AmbientOcclusionTexture = AmbientOcclusionMask;
				PSParameters->AmbientOcclusionSampler = TStaticSamplerState<SF_Point>::GetRHI();
			
				if (!PSParameters->AmbientOcclusionTexture)
				{
					PSParameters->AmbientOcclusionTexture = SystemTextures.White;
				}

				PSParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
				PSParameters->SceneTextures = SceneTextureParameters;
				PSParameters->SceneTexturesStruct = SceneTextures.UniformBuffer;
				PSParameters->ViewUniformBuffer = View.ViewUniformBuffer;

				PSParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);
				if (Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()))
				{
					const FSubstrateSceneData& SubstrateSceneData = *SceneTextures.SubstrateSceneData;
					PSParameters->RenderTargets[1] = FRenderTargetBinding(SubstrateSceneData.SeparatedOpaqueRoughRefractionSceneColor, ERenderTargetLoadAction::ELoad);
					PSParameters->RenderTargets[2] = FRenderTargetBinding(SubstrateSceneData.SeparatedSubSurfaceSceneColor, ERenderTargetLoadAction::ELoad);
				}

				const bool bUseSeparatedSubsurfaceDiffuse = UseSeparatedSubsurfaceDiffuse(View.GetShaderPlatform());
				if (bUseSeparatedSubsurfaceDiffuse)
				{
					PSParameters->RenderTargets[1] = FRenderTargetBinding(View.GetSceneTextures().SeparatedSubsurfaceDiffuse, ERenderTargetLoadAction::ELoad);
				}
				
				const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();

				TShaderMapRef<FPostProcessVS> FullScreenVertexShader(View.ShaderMap);

				Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableDebug >(false);
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector >(false);
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FClassificationTileSize16 >(bSubstrateEnabled && Substrate::UseTileSize16());
				TShaderMapRef<Substrate::FSubstrateTilePassVS> TileVertexShader(View.ShaderMap, VSPermutationVector);

				EPrimitiveType PrimitiveType = PT_TriangleList;
				if (Substrate::IsSubstrateEnabled())
				{
					PassParameters->VS = Substrate::SetTileParameters(GraphBuilder, View, TileType, PrimitiveType);
				}

				FRHIBlendState* BlendState;
				if (bUseSeparatedSubsurfaceDiffuse)
				{
					BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
						                           CW_RED, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
				}
				else
				{
					BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
				}

				TShaderMapRef<FAmbientCubemapCompositePS> PixelShader(View.ShaderMap);
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("AmbientCubemapComposite(%dx%d%s%s)", View.ViewRect.Width(), View.ViewRect.Height(), bSubstrateEnabled ? TEXT(",") : TEXT(""), bSubstrateEnabled ? ToString(TileType) : TEXT("")),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, &View, FullScreenVertexShader, TileVertexShader, PixelShader, BlendState, PrimitiveType, TileType, bSubstrateEnabled](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{				
					RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 0.0);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

					// set the state
					GraphicsPSOInit.BlendState = BlendState;
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = bSubstrateEnabled ? TileVertexShader.GetVertexShader() : FullScreenVertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PrimitiveType;

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					uint32 Count = View.FinalPostProcessSettings.ContributingCubemaps.Num();
					for (const FFinalPostProcessSettings::FCubemapEntry& CubemapEntry : View.FinalPostProcessSettings.ContributingCubemaps)
					{
						FAmbientCubemapCompositePS::FParameters ShaderParameters = PassParameters->PS;
						SetupAmbientCubemapParameters(CubemapEntry, &ShaderParameters.AmbientCubemap);
						SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), ShaderParameters);
					
						if (bSubstrateEnabled)
						{
							SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), PassParameters->VS);
							RHICmdList.DrawPrimitiveIndirect(PassParameters->VS.TileIndirectBuffer->GetIndirectRHICallBuffer(), Substrate::TileTypeDrawIndirectArgOffset(TileType));
						}
						else
						{
							DrawPostProcessPass(
								RHICmdList,
								0, 0,
								View.ViewRect.Width(), View.ViewRect.Height(),
								View.ViewRect.Min.X,   View.ViewRect.Min.Y,
								View.ViewRect.Width(), View.ViewRect.Height(),
								View.ViewRect.Size(),
								View.GetSceneTexturesConfig().Extent,
								FullScreenVertexShader,
								View.StereoViewIndex,
								false, // TODO.
								EDRF_UseTriangleOptimization);
						}
					}
				});
			};

			if (Substrate::IsSubstrateEnabled())
			{
				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplexSpecial)) 	{ ApplyAmbientCubemapComposite(ESubstrateTileType::EComplexSpecial); }
				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplex))			{ ApplyAmbientCubemapComposite(ESubstrateTileType::EComplex); }
				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESingle)) 			{ ApplyAmbientCubemapComposite(ESubstrateTileType::ESingle); }
				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESimple)) 			{ ApplyAmbientCubemapComposite(ESubstrateTileType::ESimple); }
			}
			else
			{
				ApplyAmbientCubemapComposite(ESubstrateTileType::ECount);
			}
		} // if (IsAmbientCubemapPassRequired(View))
	} // for (FViewInfo& View : Views)

	if (bCompositeRegularLumenOnly)
	{
		AsyncLumenIndirectLightingOutputs.DoneComposite();
	}
	else
	{
		AsyncLumenIndirectLightingOutputs.DonePreLights();
	}
}

void GetSubstrateTiledDepthStencilStateNoDepthTest(ESubstrateTileType InTileType, FGraphicsPipelineStateInitializer& GraphicsPSOInit, uint32& OutStencilRef);

static void AddSkyReflectionPass(
	FRDGBuilder& GraphBuilder, 
	FViewInfo& View, 
	FScene* Scene, 
	const FSceneTextures& SceneTextures,
	FRDGTextureRef DynamicBentNormalAOTexture,
	FRDGTextureRef ReflectionsColor,
	FSceneTextureParameters& SceneTextureParameters,
	const FScreenSpaceReflectionTileClassification& ScreenSpaceReflectionTileClassification,
	EReflectionsMethod ReflectionsMethod,
	bool bSkyLight, 
	bool bDynamicSkyLight, 
	bool bApplySkyShadowing,
	bool bLumenStandaloneReflections,
	ESubstrateTileType SubstrateTileMaterialType)
{
	// Render the reflection environment with tiled deferred culling
	bool bHasBoxCaptures = (View.NumBoxReflectionCaptures > 0);
	bool bHasSphereCaptures = (View.NumSphereReflectionCaptures > 0);
	bool bIsTiledSSR = ScreenSpaceReflectionTileClassification.TileMaskBuffer != nullptr;

	float DynamicBentNormalAO = DynamicBentNormalAOTexture ? 1.0f : 0.0f;
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
	FRDGTextureRef AmbientOcclusionTexture = HasBeenProduced(SceneTextures.ScreenSpaceAO) ? SceneTextures.ScreenSpaceAO : GetScreenSpaceAOFallback(SystemTextures);

	const auto& SceneColorTexture = SceneTextures.Color;

	FReflectionEnvironmentSkyLightingParameters* PassParameters = GraphBuilder.AllocParameters<FReflectionEnvironmentSkyLightingParameters>();

	// Setup the parameters of the shader.
	{
		// Setups all shader parameters related to skylight.
		PassParameters->PS.SkyDiffuseLighting = GetSkyDiffuseLightingParameters(Scene->SkyLight, DynamicBentNormalAO);

		// Setups all shader parameters related to distance field AO
		{
			// The view must sample into the texture atlas correctly with the appropriate clamping
			const FIntPoint AOBufferSize = GetBufferSizeForAO(View);
			const FIntPoint AOViewSize = View.ViewRect.Size() / GAODownsampleFactor;

			PassParameters->PS.AOBufferBilinearUVMax = FVector2f(
				(AOViewSize.X - 0.51f) / AOBufferSize.X,
				(AOViewSize.Y - 0.51f) / AOBufferSize.Y);

			extern float GAOViewFadeDistanceScale;
			PassParameters->PS.AOMaxViewDistance = GetMaxAOViewDistance();
			PassParameters->PS.DistanceFadeScale = 1.0f / ((1.0f - GAOViewFadeDistanceScale) * GetMaxAOViewDistance());

			PassParameters->PS.BentNormalAOTexture = DynamicBentNormalAOTexture;
			PassParameters->PS.BentNormalAOSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		}

		PassParameters->PS.AmbientOcclusionTexture = AmbientOcclusionTexture;
		PassParameters->PS.AmbientOcclusionSampler = TStaticSamplerState<SF_Point>::GetRHI();

		if (bLumenStandaloneReflections)
		{
			PassParameters->PS.LumenReflectionTexture = ReflectionsColor ? ReflectionsColor : SystemTextures.BlackArray;
			PassParameters->PS.LumenReflectionTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		}
		else
		{
			PassParameters->PS.ReflectionTexture = ReflectionsColor ? ReflectionsColor : SystemTextures.Black;
			PassParameters->PS.ReflectionTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		}

		if (bIsTiledSSR)
		{
			PassParameters->PS.SSRTiledViewRes = ScreenSpaceReflectionTileClassification.TiledViewRes;
			PassParameters->PS.SSRTileMaskBuffer = GraphBuilder.CreateSRV(ScreenSpaceReflectionTileClassification.TileMaskBuffer);
		}

		if (Scene->HasVolumetricCloud())
		{
			FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();

			PassParameters->PS.CloudSkyAOTexture = View.VolumetricCloudSkyAO != nullptr ? View.VolumetricCloudSkyAO : SystemTextures.Black;
			PassParameters->PS.CloudSkyAOWorldToLightClipMatrix = CloudInfo->GetVolumetricCloudCommonShaderParameters().CloudSkyAOTranslatedWorldToLightClipMatrix;
			PassParameters->PS.CloudSkyAOFarDepthKm = CloudInfo->GetVolumetricCloudCommonShaderParameters().CloudSkyAOFarDepthKm;
			PassParameters->PS.CloudSkyAOEnabled = 1;
		}
		else
		{
			PassParameters->PS.CloudSkyAOTexture = SystemTextures.Black;
			PassParameters->PS.CloudSkyAOEnabled = 0;
		}
		PassParameters->PS.CloudSkyAOSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

		PassParameters->PS.PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
		PassParameters->PS.PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		PassParameters->PS.SceneTextures = SceneTextureParameters;

		PassParameters->PS.ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->PS.ReflectionCaptureData = View.ReflectionCaptureUniformBuffer;
		PassParameters->PS.ReflectionsParameters = CreateReflectionUniformBuffer(GraphBuilder, View);
		PassParameters->PS.ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;

		PassParameters->PS.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		if (Substrate::IsGlintEnabled(View.GetShaderPlatform()))
		{
			FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
			PassParameters->PS.BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);
		}

		LumenReflections::SetupCompositeParameters(View, ReflectionsMethod, PassParameters->PS.ReflectionsCompositeParameters);
	}

	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture.Target, ERenderTargetLoadAction::ELoad);
	if (Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()))
	{
		const FSubstrateSceneData& SubstrateSceneData = *SceneTextures.SubstrateSceneData;
		PassParameters->RenderTargets[1] = FRenderTargetBinding(SubstrateSceneData.SeparatedOpaqueRoughRefractionSceneColor, ERenderTargetLoadAction::ELoad);
		PassParameters->RenderTargets[2] = FRenderTargetBinding(SubstrateSceneData.SeparatedSubSurfaceSceneColor, ERenderTargetLoadAction::ELoad);
	}

	if (UseSeparatedSubsurfaceDiffuse(View.GetShaderPlatform()))
	{
		PassParameters->RenderTargets[1] = FRenderTargetBinding(View.GetSceneTextures().SeparatedSubsurfaceDiffuse, ERenderTargetLoadAction::ELoad);
	}

	const bool bUseDepthBounds = GSupportsDepthBoundsTest;
	if (bUseDepthBounds)
	{
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target,
			ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
	}
	
	PassParameters->RenderTargets.ShadingRateTexture = GVRSImageManager.GetVariableRateShadingImage(GraphBuilder, View, FVariableRateShadingImageManager::EVRSPassType::ReflectionEnvironmentAndSky);

	// Bind hair data
	const bool bCheckerboardSubsurfaceRendering = IsSubsurfaceCheckerboardFormat(SceneColorTexture.Target->Desc.Format, View);
	const bool bHasHairCardsOrMeshes = View.HairCardsMeshElements.Num() > 0;

	auto PermutationVector = FReflectionEnvironmentSkyLightingPS::BuildPermutationVector(
		View, bHasBoxCaptures, bHasSphereCaptures, DynamicBentNormalAO != 0.0f,
		bSkyLight, bDynamicSkyLight, bApplySkyShadowing,
		bLumenStandaloneReflections, bIsTiledSSR, bHasHairCardsOrMeshes,
		SubstrateTileMaterialType);

	const bool bSubstrateEnabled = Substrate::IsSubstrateEnabled();
	const bool bSubstrateLightingPassesTiled = Substrate::AreLightingPassesTiled();

	TShaderMapRef<FReflectionEnvironmentSkyLightingPS> PixelShader(View.ShaderMap, PermutationVector);
	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

	EPrimitiveType SubstrateTilePrimitiveType = PT_TriangleList;
	Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
	VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableDebug >(false);
	VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector >(false);
	VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FClassificationTileSize16 >(bSubstrateEnabled && bSubstrateLightingPassesTiled && Substrate::UseTileSize16());
	TShaderMapRef<Substrate::FSubstrateTilePassVS> SubstrateTilePassVertexShader(View.ShaderMap, VSPermutationVector);
	PassParameters->VS.TileIndirectBuffer = nullptr;
	PassParameters->VS.TileListBuffer = nullptr;
	if (bSubstrateEnabled && bSubstrateLightingPassesTiled)
	{
		check(SubstrateTileMaterialType <= ESubstrateTileType::EComplexSpecial);
		PassParameters->VS = Substrate::SetTileParameters(GraphBuilder, View, SubstrateTileMaterialType, SubstrateTilePrimitiveType);
		ClearUnusedGraphResources(SubstrateTilePassVertexShader, &PassParameters->VS);
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ReflectionEnvironmentAndSky(%dx%d,SubstrateMat=%s)", 
			View.ViewRect.Width(), View.ViewRect.Height(), 
			bSubstrateEnabled ? ToString(SubstrateTileMaterialType) : TEXT("Off")),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, &View, PixelShader, bCheckerboardSubsurfaceRendering, SubstrateTileMaterialType, bUseDepthBounds,
		SubstrateTilePassVertexShader, bSubstrateEnabled, SubstrateTilePrimitiveType, bSubstrateLightingPassesTiled](FRDGAsyncTask, FRHICommandList& InRHICmdList)
	{
		InRHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		FPixelShaderUtils::InitFullscreenPipelineState(InRHICmdList, View.ShaderMap, PixelShader, GraphicsPSOInit);

		extern int32 GAOOverwriteSceneColor;
		if (GetReflectionEnvironmentCVar() == 2 || GAOOverwriteSceneColor)
		{
			// override scene color for debugging
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		}
		else
		{
			if (bCheckerboardSubsurfaceRendering)
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
			}
			else
			{
				if (Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()))
				{
					GraphicsPSOInit.BlendState = TStaticBlendState<
						CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
						CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
						CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
				}
				else if (UseSeparatedSubsurfaceDiffuse(View.GetShaderPlatform()))
				{
					GraphicsPSOInit.BlendState = TStaticBlendState<
						CW_RGBA,  BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
						CW_RGBA,  BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
				}
				else
				{
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
				}
			}
		}

		if (bUseDepthBounds)
		{
			// We do not want to process sky pixels so we take advantage of depth bound test when available to skip launching pointless GPU wavefront/work.
			GraphicsPSOInit.bDepthBounds = true;

			FDepthBounds::FDepthBoundsValues Values = FDepthBounds::CalculateNearFarDepthExcludingSky();
			InRHICmdList.SetDepthBounds(Values.MinDepth, Values.MaxDepth);
		}


		uint32 StencilRef = 0u;
		if (bSubstrateEnabled && !bSubstrateLightingPassesTiled)
		{
			GetSubstrateTiledDepthStencilStateNoDepthTest(SubstrateTileMaterialType, GraphicsPSOInit, StencilRef);
		}

		if (bSubstrateEnabled && bSubstrateLightingPassesTiled)
		{
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = SubstrateTilePassVertexShader.GetVertexShader();
			GraphicsPSOInit.PrimitiveType = SubstrateTilePrimitiveType;
		}

		SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, StencilRef);
		SetShaderParameters(InRHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

		if (bSubstrateEnabled && bSubstrateLightingPassesTiled)
		{
			SetShaderParameters(InRHICmdList, SubstrateTilePassVertexShader, SubstrateTilePassVertexShader.GetVertexShader(), PassParameters->VS);
			InRHICmdList.DrawPrimitiveIndirect(PassParameters->VS.TileIndirectBuffer->GetIndirectRHICallBuffer(), Substrate::TileTypeDrawIndirectArgOffset(SubstrateTileMaterialType));
		}
		else
		{
			FPixelShaderUtils::DrawFullscreenTriangle(InRHICmdList);
		}
	});
}

void FDeferredShadingSceneRenderer::RenderDeferredReflectionsAndSkyLighting(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	FLumenSceneFrameTemporaries& LumenFrameTemporaries,
	TArray<FRDGTextureRef>& DynamicBentNormalAOTextures)
{
	extern int32 GLumenVisualizeIndirectDiffuse;
	if (ViewFamily.EngineShowFlags.VisualizeLightCulling 
		|| HasRayTracedOverlay(ViewFamily)
		|| ViewFamily.EngineShowFlags.PathTracing
		|| !ViewFamily.EngineShowFlags.Lighting
		|| GLumenVisualizeIndirectDiffuse != 0)
	{
		return;
	}

	bool bReflectionCapture = false;
	for (int32 ViewIndex = 0, Num = Views.Num(); ViewIndex < Num; ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		bReflectionCapture = bReflectionCapture || View.bIsReflectionCapture;
	}

	if (bReflectionCapture 
		&& IsDeferredReflectionsAndSkyLightingDisabledForReflectionCapture())
	{
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "DiffuseIndirectAndAO");

	// The specular sky light contribution is also needed by RT Reflections as a fallback.
	const bool bSkyLight = Scene->SkyLight
		&& (Scene->SkyLight->ProcessedTexture || Scene->SkyLight->bRealTimeCaptureEnabled)
		&& !Scene->SkyLight->bHasStaticLighting;

	bool bDynamicSkyLight = ShouldRenderDeferredDynamicSkyLight(Scene, ViewFamily) && AnyViewHasGIMethodSupportingDFAO();
	bool bApplySkyShadowing = false;
	if (bDynamicSkyLight)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, SkyLightDiffuse, "SkyLightDiffuse");

		extern int32 GDistanceFieldAOApplyToStaticIndirect;
		if (Scene->SkyLight->bCastShadows
			&& !GDistanceFieldAOApplyToStaticIndirect
			&& ShouldRenderDistanceFieldAO(Views, ViewFamily.EngineShowFlags, AnyViewHasGIMethodSupportingDFAO())
			&& ShouldRenderDistanceFieldLighting(Scene->DistanceFieldSceneData, Views)
			&& ViewFamily.EngineShowFlags.AmbientOcclusion
			&& !bReflectionCapture)
		{
			bApplySkyShadowing = true;
			FDistanceFieldAOParameters Parameters(Scene->SkyLight->OcclusionMaxDistance, Scene->SkyLight->Contrast);
			RenderDistanceFieldLighting(GraphBuilder, SceneTextures, Parameters, DynamicBentNormalAOTextures, false, false);
		}
	}

	RDG_EVENT_SCOPE(GraphBuilder, "ReflectionIndirect");

	const bool bReflectionEnv = ShouldDoReflectionEnvironment(Scene, ViewFamily);

	FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures);
	const auto& SceneColorTexture = SceneTextures.Color;

	uint32 ViewIndex = 0;
	for (FViewInfo& View : Views)
	{
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		const uint32 CurrentViewIndex = ViewIndex++;
		const FPerViewPipelineState& ViewPipelineState = GetViewPipelineState(View);

		const bool bScreenSpaceReflections = ViewPipelineState.ReflectionsMethod == EReflectionsMethod::SSR;
		const bool bComposePlanarReflections = HasDeferredPlanarReflections(View);

		FScreenSpaceReflectionTileClassification ScreenSpaceReflectionTileClassification;
		bool bRunSSRTiledComposite = false;

		FRDGTextureRef ReflectionsColor = nullptr;
		if (ViewPipelineState.DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen)
		{
			// Specular was already composited in FDiffuseIndirectCompositePS
			continue;
		}
		// Standalone Lumen Reflections using HWRT and hit lighting
		else if (ViewPipelineState.ReflectionsMethod == EReflectionsMethod::Lumen)
		{
			FLumenMeshSDFGridParameters MeshSDFGridParameters;
			LumenRadianceCache::FRadianceCacheInterpolationParameters RadianceCacheParameters;
			FLumenReflectionsConfig LumenReflectionsConfig;

			ReflectionsColor = RenderLumenReflections(
				GraphBuilder,
				View,
				SceneTextures,
				LumenFrameTemporaries,
				MeshSDFGridParameters,
				RadianceCacheParameters,
				ELumenReflectionPass::Opaque,
				LumenReflectionsConfig,
				ERDGPassFlags::Compute);
		}
		else if (ViewPipelineState.ReflectionsMethod == EReflectionsMethod::SSR)
		{
			int32 DenoiserMode = GetReflectionsDenoiserMode();

			// Traces the reflections, either using screen space reflection, or ray tracing.
			IScreenSpaceDenoiser::FReflectionsInputs DenoiserInputs;
			IScreenSpaceDenoiser::FReflectionsRayTracingConfig DenoiserConfig;
			bool bDenoise = DenoiserMode != 0 && CVarDenoiseSSR.GetValueOnRenderThread();
			bool bTemporalFilter = !bDenoise && View.ViewState && ScreenSpaceRayTracing::IsSSRTemporalPassRequired(View);

			{
				ESSRQuality SSRQuality;
				ScreenSpaceRayTracing::GetSSRQualityForView(View, &SSRQuality, &DenoiserConfig);

				RDG_EVENT_SCOPE(GraphBuilder, "ScreenSpaceReflections(Quality=%d)", int32(SSRQuality));

				ScreenSpaceReflectionTileClassification = ClassifySSRTiles(GraphBuilder, View, SceneTextures, nullptr);
				bRunSSRTiledComposite = IsDefaultSSRTileEnabled(View);

				ScreenSpaceRayTracing::RenderScreenSpaceReflections(
					GraphBuilder, SceneTextureParameters, SceneColorTexture.Resolve, View, SSRQuality, bDenoise, &DenoiserInputs,/*bSingleLayerWater*/false, 
					bRunSSRTiledComposite ? &ScreenSpaceReflectionTileClassification.TiledReflection : nullptr);
			}
			if (bDenoise)
			{
				const IScreenSpaceDenoiser* DefaultDenoiser = IScreenSpaceDenoiser::GetDefaultDenoiser();
				const IScreenSpaceDenoiser* DenoiserToUse = DenoiserMode == 1 ? DefaultDenoiser : GScreenSpaceDenoiser;

				// Standard event scope for denoiser to have all profiling information not matter what, and with explicit detection of third party.
				RDG_EVENT_SCOPE(GraphBuilder, "%s%s(Reflections) %dx%d",
					DenoiserToUse != DefaultDenoiser ? TEXT("ThirdParty ") : TEXT(""),
					DenoiserToUse->GetDebugName(),
					View.ViewRect.Width(), View.ViewRect.Height());

				IScreenSpaceDenoiser::FReflectionsOutputs DenoiserOutputs = DenoiserToUse->DenoiseReflections(
					GraphBuilder,
					View,
					&View.PrevViewInfo,
					SceneTextureParameters,
					DenoiserInputs,
					DenoiserConfig);

				ReflectionsColor = DenoiserOutputs.Color;
			}
			else if (bTemporalFilter)
			{
				check(View.ViewState);
				FTAAPassParameters TAASettings(View);
				TAASettings.Pass = ETAAPassConfig::ScreenSpaceReflections;
				TAASettings.SceneDepthTexture = SceneTextureParameters.SceneDepthTexture;
				TAASettings.SceneVelocityTexture = SceneTextureParameters.GBufferVelocityTexture;
				TAASettings.SceneColorInput = DenoiserInputs.Color;
				TAASettings.bOutputRenderTargetable = (
					ViewPipelineState.bComposePlanarReflections ||
					ViewPipelineState.ReflectionsMethod == EReflectionsMethod::Lumen);

				FTAAOutputs TAAOutputs = AddTemporalAAPass(
					GraphBuilder,
					View,
					TAASettings,
					View.PrevViewInfo.SSRHistory,
					&View.ViewState->PrevFrameViewInfo.SSRHistory);

				ReflectionsColor = TAAOutputs.SceneColor;
			}
			else
			{
				ReflectionsColor = DenoiserInputs.Color;
			}
		} // if (RayTracingReflectionOptions.bEnabled || bScreenSpaceReflections)

		if (ViewPipelineState.bComposePlanarReflections)
		{
			RenderDeferredPlanarReflections(GraphBuilder, SceneTextureParameters, View, /* inout */ ReflectionsColor);
		}

		const bool bLumenStandaloneReflections = ViewPipelineState.ReflectionsMethod == EReflectionsMethod::Lumen;
		const bool bRequiresApply = ReflectionsColor != nullptr || bSkyLight || bDynamicSkyLight || bReflectionEnv;

		if (bRequiresApply)
		{
			// Update the stencil buffer, marking single/simple/complex Substrate material only if not already done.
			if (Substrate::IsSubstrateEnabled() && Substrate::GetStencilPassStage() == Substrate::EStencilPassStage::StandaloneAdaptive)
			{
				Substrate::AddSubstrateStencilPass(GraphBuilder, Scene, Views, SceneTextures, *SceneTextures.SubstrateSceneData);
			}

			RDG_EVENT_SCOPE_STAT(GraphBuilder, ReflectionEnvironment, "ReflectionEnvironment");

			FRDGTextureRef DynamicBentNormalAOTexture = DynamicBentNormalAOTextures.IsEmpty() ? nullptr : DynamicBentNormalAOTextures[CurrentViewIndex];

			if (Substrate::IsSubstrateEnabled())
			{
				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplexSpecial))
				{
					AddSkyReflectionPass(
						GraphBuilder,
						View,
						Scene,
						SceneTextures,
						DynamicBentNormalAOTexture,
						ReflectionsColor,
						SceneTextureParameters,
						ScreenSpaceReflectionTileClassification,
						ViewPipelineState.ReflectionsMethod,
						bSkyLight,
						bDynamicSkyLight,
						bApplySkyShadowing,
						bLumenStandaloneReflections,
						ESubstrateTileType::EComplexSpecial);
				}

				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplex))
				{
					AddSkyReflectionPass(
						GraphBuilder,
						View,
						Scene,
						SceneTextures,
						DynamicBentNormalAOTexture,
						ReflectionsColor,
						SceneTextureParameters,
						ScreenSpaceReflectionTileClassification,
						ViewPipelineState.ReflectionsMethod,
						bSkyLight,
						bDynamicSkyLight,
						bApplySkyShadowing,
						bLumenStandaloneReflections,
						ESubstrateTileType::EComplex);
				}

				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESingle))
				{
					AddSkyReflectionPass(
						GraphBuilder,
						View,
						Scene,
						SceneTextures,
						DynamicBentNormalAOTexture,
						ReflectionsColor,
						SceneTextureParameters,
						ScreenSpaceReflectionTileClassification,
						ViewPipelineState.ReflectionsMethod,
						bSkyLight,
						bDynamicSkyLight,
						bApplySkyShadowing,
						bLumenStandaloneReflections,
						ESubstrateTileType::ESingle);
				}

				if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESimple))
				{
					AddSkyReflectionPass(
						GraphBuilder,
						View,
						Scene,
						SceneTextures,
						DynamicBentNormalAOTexture,
						ReflectionsColor,
						SceneTextureParameters,
						ScreenSpaceReflectionTileClassification,
						ViewPipelineState.ReflectionsMethod,
						bSkyLight,
						bDynamicSkyLight,
						bApplySkyShadowing,
						bLumenStandaloneReflections,
						ESubstrateTileType::ESimple);
				}
			}
			else
			{
				// Typical path uses when Substrate is not enabled
				AddSkyReflectionPass(
					GraphBuilder,
					View,
					Scene,
					SceneTextures,
					DynamicBentNormalAOTexture,
					ReflectionsColor,
					SceneTextureParameters,
					ScreenSpaceReflectionTileClassification,
					ViewPipelineState.ReflectionsMethod,
					bSkyLight,
					bDynamicSkyLight,
					bApplySkyShadowing,
					bLumenStandaloneReflections,
					ESubstrateTileType::ECount);
			}
		}

		if (HairStrands::HasViewHairStrandsData(View))
		{
			RenderHairStrandsEnvironmentLighting(GraphBuilder, Scene, View);
		}
	}

	AddResolveSceneColorPass(GraphBuilder, Views, SceneColorTexture);
}

void FDeferredShadingSceneRenderer::RenderDeferredReflectionsAndSkyLightingHair(FRDGBuilder& GraphBuilder)
{
	if (ViewFamily.EngineShowFlags.VisualizeLightCulling || !ViewFamily.EngineShowFlags.Lighting)
	{
		return;
	}

	for (FViewInfo& View : Views)
	{
		// if we are rendering a reflection capture then we can skip this pass entirely (no reflection and no sky contribution evaluated in this pass)
		if (View.bIsReflectionCapture)
		{
			continue;
		}

		if (HairStrands::HasViewHairStrandsData(View))
		{
			RenderHairStrandsEnvironmentLighting(GraphBuilder, Scene, View);
		}
	}

}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void FDeferredShadingSceneRenderer::RenderGlobalIlluminationPluginVisualizations(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef LightingChannelsTexture)
{
	// Early out if GI plugins aren't enabled
	bool bGIPluginEnabled = false;
	for (const FViewInfo& View : Views)
	{
		if (View.FinalPostProcessSettings.DynamicGlobalIlluminationMethod == EDynamicGlobalIlluminationMethod::Plugin)
		{
			bGIPluginEnabled = true;
			break;
		}
	}
	if (!bGIPluginEnabled)
	{
		return;
	}

	const FSceneTextures& SceneTextures = GetActiveSceneTextures();

	// Get the resources passed to GI plugins
	FGlobalIlluminationPluginResources GIPluginResources;
	GIPluginResources.GBufferA = SceneTextures.GBufferA;
	GIPluginResources.GBufferB = SceneTextures.GBufferB;
	GIPluginResources.GBufferC = SceneTextures.GBufferC;
	GIPluginResources.LightingChannelsTexture = LightingChannelsTexture;
	GIPluginResources.SceneDepthZ = SceneTextures.Depth.Target;
	GIPluginResources.SceneColor = SceneTextures.Color.Target;

	// Render visualizations to all views by calling the GI plugin's delegate
	FGlobalIlluminationPluginDelegates::FRenderDiffuseIndirectVisualizations& PRVDelegate = FGlobalIlluminationPluginDelegates::RenderDiffuseIndirectVisualizations();
	for (int32 ViewIndexZ = 0; ViewIndexZ < Views.Num(); ViewIndexZ++)
	{
		PRVDelegate.Broadcast(*Scene, Views[ViewIndexZ], GraphBuilder, GIPluginResources);
	}
}
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
