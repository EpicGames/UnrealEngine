// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "SceneTextureParameters.h"
#include "ShaderParameterMacros.h"
#include "Substrate/Substrate.h"

class FSkyLightSceneProxy;

namespace HybridIndirectLighting
{

/** Size of the interleaving tile, 4x4. */
constexpr int32 kInterleavingTileSize = 4;

/** Total number of bucked used to interleave. */
constexpr int32 kInterleavingBucketCount = kInterleavingTileSize * kInterleavingTileSize;

/** Maximum number of rays that can be shot per ray tracing pixel. */
constexpr int32 kMaxRayPerPixel = 8;

/** Maximum resolution of rays ray tracing pixel 8192x8192. */
constexpr int32 kMaxTracingResolution = 8192;


/** Shader parameter structure shared across all indirect diffuse technics. */
BEGIN_SHADER_PARAMETER_STRUCT(FCommonParameters, )
	// Size of the viewport to do the ray tracing with.
	SHADER_PARAMETER(FIntPoint, TracingViewportSize)

	// Standard buffer size to store one viewport.
	SHADER_PARAMETER(FIntPoint, TracingViewportBufferSize)

	// 1.0f / TracingViewportBufferSize
	SHADER_PARAMETER(FVector2f, TracingViewportTexelSize)

	// How much downscale the ray tracing is done 
	SHADER_PARAMETER(int32, DownscaleFactor)

	// Number of ray per pixel.
	SHADER_PARAMETER(int32, RayCountPerPixel)

	// Size of the ray storage coordinates.
	// RayCountPerPixel <= (RayStoragePerPixelVector.x * RayStoragePerPixelVector.y) 
	SHADER_PARAMETER(FIntPoint, RayStoragePerPixelVector)

	// Bits operator to transfor a tracing PixelRayIndex into ray storage coordinates.
	SHADER_PARAMETER(int32, PixelRayIndexAbscissMask)
	SHADER_PARAMETER(int32, PixelRayIndexOrdinateShift)

	// Scene textures and its sampler.
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
END_SHADER_PARAMETER_STRUCT()

} // HybridIndirectLighting

// Sky light parameters.
BEGIN_SHADER_PARAMETER_STRUCT(FSkyDiffuseLightingParameters, )
	SHADER_PARAMETER(FVector4f, OcclusionTintAndMinOcclusion)
	SHADER_PARAMETER(FVector3f, ContrastAndNormalizeMulAdd)
	SHADER_PARAMETER(float, ApplyBentNormalAO)
	SHADER_PARAMETER(float, InvSkySpecularOcclusionStrength)
	SHADER_PARAMETER(float, OcclusionExponent)
	SHADER_PARAMETER(float, OcclusionCombineMode)
END_SHADER_PARAMETER_STRUCT()

FSkyDiffuseLightingParameters GetSkyDiffuseLightingParameters(const FSkyLightSceneProxy* SkyLight, float DynamicBentNormalAO);
