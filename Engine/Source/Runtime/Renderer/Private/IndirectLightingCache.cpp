// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Implements a volume texture atlas for caching indirect lighting on a per-object basis
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "Async/TaskGraphInterfaces.h"
#include "RHI.h"
#include "RenderResource.h"
#include "SceneTypes.h"
#include "RendererInterface.h"
#include "Math/GenericOctree.h"
#include "PrimitiveSceneInfo.h"
#include "DynamicPrimitiveDrawing.h"
#include "ScenePrivate.h"
#include "PrecomputedLightVolume.h"
#include "RenderCore.h"
#include "UnrealEngine.h"
#include "RenderTargetPool.h"

/** 
 * Primitive bounds size will be rounded up to the next value of Pow(BoundSizeRoundUpBase, N) and N is an integer. 
 * This provides some stability as bounds get larger and smaller, although by adding some waste.
 */
const float BoundSizeRoundUpBase = FMath::Sqrt(2.f);
const float LogEBoundSizeRoundUpBase = FMath::Loge(BoundSizeRoundUpBase);

//////////////////////////////////////////////////////////////////////////////////////////////////////////
// Console variables that can be changed at runtime to configure or debug the indirect lighting cache
//////////////////////////////////////////////////////////////////////////////////////////////////////////

int32 GCacheDrawLightingSamples = 0;
static FAutoConsoleVariableRef CVarCacheDrawLightingSamples(
	TEXT("r.Cache.DrawLightingSamples"),
	GCacheDrawLightingSamples,
	TEXT("Whether to draw indirect lighting sample points as generated by Lightmass.\n")
	TEXT("0 is off (default), 1 is on"),
	ECVF_RenderThreadSafe
	);

int32 GCacheDrawDirectionalShadowing = 0;
static FAutoConsoleVariableRef CVarCacheDrawDirectionalShadowing(
	TEXT("r.Cache.DrawDirectionalShadowing"),
	GCacheDrawDirectionalShadowing,
	TEXT("Whether to draw direct shadowing sample points as generated by Lightmass.\n")
	TEXT("0 is off (default), 1 is on"),
	ECVF_RenderThreadSafe
	);

int32 GCacheDrawInterpolationPoints = 0;
static FAutoConsoleVariableRef CVarCacheDrawInterpolationPoints(
	TEXT("r.Cache.DrawInterpolationPoints"),
	GCacheDrawInterpolationPoints,
	TEXT("Whether to draw positions that indirect lighting is interpolated at when they are updated, which are stored in the cache.\n")
	TEXT("Probably need 'r.CacheUpdateEveryFrame 1' as well to be useful, otherwise points will flicker as they update.\n")
	TEXT("0 is off (default), 1 is on"),
	ECVF_RenderThreadSafe
	);

int32 GCacheUpdateEveryFrame = 0;
static FAutoConsoleVariableRef CVarCacheUpdateEveryFrame(
	TEXT("r.Cache.UpdateEveryFrame"),
	GCacheUpdateEveryFrame,
	TEXT("Whether to update indirect lighting cache allocations every frame, even if they would have been cached.  0 is off (default), 1 is on"),
	ECVF_RenderThreadSafe
	);

float GSingleSampleTransitionSpeed = 800;
static FAutoConsoleVariableRef CVarSingleSampleTransitionSpeed(
	TEXT("r.Cache.SampleTransitionSpeed"),
	GSingleSampleTransitionSpeed,
	TEXT("When using single sample lighting, controls the speed of the transition between two point samples (fade over time)."),
	ECVF_RenderThreadSafe
	);

int32 GCacheReduceSHRinging = 1;
static FAutoConsoleVariableRef CVarCacheReduceSHRinging(
	TEXT("r.Cache.ReduceSHRinging"),
	GCacheReduceSHRinging,
	TEXT("Whether to modify indirect lighting cache SH samples to reduce ringing.  0 is off, 1 is on (default)"),
	ECVF_RenderThreadSafe
	);

int32 GIndirectLightingCache = 1;
static FAutoConsoleVariableRef CVarIndirectLightingCache(
	TEXT("r.IndirectLightingCache"),
	GIndirectLightingCache,
	TEXT("Whether to use the indirect lighting cache on dynamic objects.  0 is off, 1 is on (default)"),
	ECVF_RenderThreadSafe
	);

int32 GCacheQueryNodeLevel = 3;
static FAutoConsoleVariableRef CVarCacheQueryNodeLevel(
	TEXT("r.Cache.QueryNodeLevel"),
	GCacheQueryNodeLevel,
	TEXT("Level of the lighting sample octree whose node's extents should be the target size for queries into the octree.\n")
	TEXT("Primitive blocks will be broken up into multiple octree queries if they are larger than this.")
	TEXT("0 is the root, 12 is the leaf level"),
	ECVF_RenderThreadSafe
	);

int32 GCacheLimitQuerySize = 1;
static FAutoConsoleVariableRef CVarCacheLimitQuerySize(
	TEXT("r.Cache.LimitQuerySize"),
	GCacheLimitQuerySize,
	TEXT("0 is off, 1 is on (default)"),
	ECVF_RenderThreadSafe
	);

//////////////////////////////////////////////////////////////////////////////////////////////////////////
// Console variables that cannot be changed at runtime
// These are console variables so their values can be read from an ini
//////////////////////////////////////////////////////////////////////////////////////////////////////////

int32 GLightingCacheMovableObjectAllocationSize = 5;

static FAutoConsoleVariableRef CVarLightingCacheMovableObjectAllocationSize(
	TEXT("r.Cache.LightingCacheMovableObjectAllocationSize"),
	GLightingCacheMovableObjectAllocationSize,
	TEXT("Resolution of the interpolation sample volume used to light a dynamic object.  \n")
	TEXT("Values of 1 or 2 will result in a single interpolation sample per object which does not provide continuous lighting under movement, so interpolation is done over time.  \n")
	TEXT("Values of 3 or more support the necessary padding to provide continuous results under movement."),
	ECVF_ReadOnly
	);

// PS4 wants multiple of 16 for fast RHIUpdateTexture3D
int32 GLightingCacheDimension = 64;
static FAutoConsoleVariableRef CVarLightingCacheDimension(
	TEXT("r.Cache.LightingCacheDimension"),
	GLightingCacheDimension,
	TEXT("Dimensions of the lighting cache.  This should be a multiple of r.LightingCacheMovableObjectAllocationSize for least waste."),
	ECVF_ReadOnly
	);

bool IsIndirectLightingCacheAllowed(ERHIFeatureLevel::Type InFeatureLevel) 
{
	return GIndirectLightingCache != 0 && IsStaticLightingAllowed();
}

bool CanIndirectLightingCacheUseVolumeTexture(ERHIFeatureLevel::Type InFeatureLevel)
{
	// @todo Mac OS X/OpenGL: For OpenGL devices which don't support volume-texture rendering we need to use the simpler point indirect lighting shaders.
	return InFeatureLevel >= ERHIFeatureLevel::SM5 && RHISupportsVolumeTextureRendering(GetFeatureLevelShaderPlatform(InFeatureLevel));
}

FIndirectLightingCache::FIndirectLightingCache(ERHIFeatureLevel::Type InFeatureLevel)
	: FRenderResource(InFeatureLevel)
	, bUpdateAllCacheEntries(true)
	, BlockAllocator(0, 0, 0, GLightingCacheDimension, GLightingCacheDimension, GLightingCacheDimension, false, false)
{
	NextPointId = GLightingCacheDimension + 1;
	CacheSize = GLightingCacheDimension;
}

void FIndirectLightingCache::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (CanIndirectLightingCacheUseVolumeTexture(GetFeatureLevel()))
	{
		ETextureCreateFlags Flags = TexCreate_ShaderResource | TexCreate_NoTiling;

		FPooledRenderTargetDesc Desc(FPooledRenderTargetDesc::CreateVolumeDesc(
			CacheSize,
			CacheSize,
			CacheSize,
			PF_FloatRGBA,
			FClearValueBinding::None,
			Flags,
			TexCreate_None,
			false, 
			1));

		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, Texture0, TEXT("IndirectLightingCache_0"));
		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, Texture1, TEXT("IndirectLightingCache_1"));
		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, Texture2, TEXT("IndirectLightingCache_2"));
	}
}

void FIndirectLightingCache::ReleaseRHI()
{
	GRenderTargetPool.FreeUnusedResource(Texture0);
	GRenderTargetPool.FreeUnusedResource(Texture1);
	GRenderTargetPool.FreeUnusedResource(Texture2);
}

static bool IsTexelMinValid(FIntVector TexelMin)
{
	return TexelMin.X >= 0 && TexelMin.Y >= 0 && TexelMin.Z >= 0;
}

FIndirectLightingCacheBlock& FIndirectLightingCache::FindBlock(FIntVector TexelMin)
{	
	checkSlow(IsTexelMinValid(TexelMin));
	return VolumeBlocks.FindChecked(TexelMin);
}

const FIndirectLightingCacheBlock& FIndirectLightingCache::FindBlock(FIntVector TexelMin) const
{
	checkSlow(IsTexelMinValid(TexelMin));
	return VolumeBlocks.FindChecked(TexelMin);
}

void FIndirectLightingCache::DeallocateBlock(FIntVector Min, int32 Size)
{
	if (Size > 1)
	{
		verify(BlockAllocator.RemoveElement(Min.X, Min.Y, Min.Z, Size, Size, Size));
	}
	
	VolumeBlocks.Remove(Min);
}

bool FIndirectLightingCache::AllocateBlock(int32 Size, FIntVector& OutMin)
{
	if (Size == 1)
	{
		// Assign a min that won't overlap with any of the samples allocated from the volume texture, so we can be added to VolumeBlocks without collisions
		// This min is not used for anything else for point samples		
		OutMin = FIntVector(NextPointId, 0, 0);
		NextPointId++;
		// Point samples don't go through the volume texture, allocation always succeeds
		return true;
	}
	else
	{
		return BlockAllocator.AddElement((uint32&)OutMin.X, (uint32&)OutMin.Y, (uint32&)OutMin.Z, Size, Size, Size);
	}
}

void FIndirectLightingCache::CalculateBlockPositionAndSize(const FBoxSphereBounds& Bounds, int32 TexelSize, FVector& OutMin, FVector& OutSize) const
{	
	FVector RoundedBoundsSize;

	// Find the exponent needed to represent the bounds size if BoundSizeRoundUpBase is the base	
	RoundedBoundsSize.X = FMath::Max(1.f, FMath::Loge(Bounds.BoxExtent.X * 2) / LogEBoundSizeRoundUpBase);
	RoundedBoundsSize.Y = FMath::Max(1.f, FMath::Loge(Bounds.BoxExtent.Y * 2) / LogEBoundSizeRoundUpBase);
	RoundedBoundsSize.Z = FMath::Max(1.f, FMath::Loge(Bounds.BoxExtent.Z * 2) / LogEBoundSizeRoundUpBase);

	// Round up to the next integer exponent to provide stability even when Bounds.BoxExtent is changing	
	RoundedBoundsSize.X = FMath::Pow(BoundSizeRoundUpBase, FMath::TruncToInt(RoundedBoundsSize.X) + 1);
	RoundedBoundsSize.Y = FMath::Pow(BoundSizeRoundUpBase, FMath::TruncToInt(RoundedBoundsSize.Y) + 1);
	RoundedBoundsSize.Z = FMath::Pow(BoundSizeRoundUpBase, FMath::TruncToInt(RoundedBoundsSize.Z) + 1);

	// For single sample allocations, use an effective texel size of 5 for snapping
	const int32 EffectiveTexelSize = TexelSize > 2 ? TexelSize : 5;

	// Setup a cell size that positions will be snapped to, in world space
	// The block allocation has to be padded by one texel in world space, twice
	// First to handle having snapped the allocation min to the next lowest cell size
	// Second to provide padding to handle trilinear volume texture filtering
	// Hence the 'EffectiveTexelSize - 2'
	const FVector CellSize = RoundedBoundsSize / (EffectiveTexelSize - 2);
	const FVector BoundsMin = Bounds.Origin - Bounds.BoxExtent;

	if (TexelSize > 2)
	{
		FVector SnappedMin;
		SnappedMin.X = CellSize.X * FMath::FloorToFloat(BoundsMin.X / CellSize.X);
		SnappedMin.Y = CellSize.Y * FMath::FloorToFloat(BoundsMin.Y / CellSize.Y);
		SnappedMin.Z = CellSize.Z * FMath::FloorToFloat(BoundsMin.Z / CellSize.Z);

		// Shift the min down so that the center of the voxel is at the min
		// This is necessary so that all pixels inside the bounds only interpolate from valid voxels
		SnappedMin -= CellSize * .5f;

		const FVector SnappedSize = TexelSize * CellSize;

		OutMin = SnappedMin;
		OutSize = SnappedSize;
	}
	else
	{
		FVector SnappedCenter;
		SnappedCenter.X = CellSize.X * FMath::FloorToFloat((Bounds.Origin.X + .5f * CellSize.X) / CellSize.X);
		SnappedCenter.Y = CellSize.Y * FMath::FloorToFloat((Bounds.Origin.Y + .5f * CellSize.Y) / CellSize.Y);
		SnappedCenter.Z = CellSize.Z * FMath::FloorToFloat((Bounds.Origin.Z + .5f * CellSize.Z) / CellSize.Z);

		// Place the min at the snapped center of the object, which will be most representative for single sample lighting
		OutMin = SnappedCenter;
		OutSize = FVector(0);
	}
}

void FIndirectLightingCache::CalculateBlockScaleAndAdd(FIntVector InTexelMin, int32 AllocationTexelSize, FVector InMin, FVector InSize, FVector& OutScale, FVector& OutAdd, FVector& OutMinUV, FVector& OutMaxUV) const
{
	const FVector MinUV(InTexelMin.X / (float)CacheSize, InTexelMin.Y / (float)CacheSize, InTexelMin.Z / (float)CacheSize);

	// Half texel offset to make sure we don't read from texels in other allocations through filtering
	OutMinUV = MinUV + FVector(.5f / CacheSize);

	if (AllocationTexelSize > 2)
	{
		const float UVSize = AllocationTexelSize / (float)CacheSize;

		// need to remove 0
		if (InSize.X == 0.f)
		{
			InSize.X = 0.01f;
		}
		if(InSize.Y == 0.f)
		{
			InSize.Y = 0.01f;
		}
		if(InSize.Z == 0.f)
		{
			InSize.Z = 0.01f;
		}

		// Setup a scale and add to convert from world space position to volume texture UV
		OutScale = FVector(UVSize) / InSize;
		OutAdd = -InMin * UVSize / InSize + MinUV;
		// Half texel offset to make sure we don't read from texels in other allocations through filtering
		OutMaxUV = MinUV + UVSize - FVector(.5f / CacheSize);
	}
	else
	{
		// All pixels sample from center of texel so that neighbors don't contribute, since there's no padding
		OutScale = FVector(0);
		OutAdd = MinUV + FVector(.5f / CacheSize);
		OutMaxUV = OutMinUV;
	}
}

FIndirectLightingCacheAllocation* FIndirectLightingCache::AllocatePrimitive(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bUnbuiltPreview)
{
	const bool bPointSample = PrimitiveSceneInfo->Proxy->GetIndirectLightingCacheQuality() == ILCQ_Point || bUnbuiltPreview;
	const int32 BlockSize = bPointSample ? 1 : GLightingCacheMovableObjectAllocationSize;
	return PrimitiveAllocations.Add(PrimitiveSceneInfo->PrimitiveComponentId, CreateAllocation(BlockSize, PrimitiveSceneInfo->Proxy->GetBounds(), bPointSample, bUnbuiltPreview));
}

FIndirectLightingCacheAllocation* FIndirectLightingCache::CreateAllocation(int32 BlockSize, const FBoxSphereBounds& Bounds, bool bPointSample, bool bUnbuiltPreview)
{	
	check(BlockSize > 1 || bPointSample);

	FIndirectLightingCacheAllocation* NewAllocation = new FIndirectLightingCacheAllocation();
	FIndirectLightingCacheBlock NewBlock;

	//@todo - don't allocate point samples from the layout, they don't go through the volume texture
	if (AllocateBlock(BlockSize, NewBlock.MinTexel))
	{
		NewBlock.TexelSize = BlockSize;
		CalculateBlockPositionAndSize(Bounds, BlockSize, NewBlock.Min, NewBlock.Size);

		FVector Scale;
		FVector Add;
		FVector MinUV;
		FVector MaxUV;
		CalculateBlockScaleAndAdd(NewBlock.MinTexel, NewBlock.TexelSize, NewBlock.Min, NewBlock.Size, Scale, Add, MinUV, MaxUV);

		VolumeBlocks.Add(NewBlock.MinTexel, NewBlock);
		NewAllocation->SetParameters(NewBlock.MinTexel, NewBlock.TexelSize, Scale, Add, MinUV, MaxUV, bPointSample, bUnbuiltPreview);
		checkf(NewAllocation->AllocationTexelSize > 1 || NewAllocation->bPointSample, TEXT("%i, %i"), NewAllocation->AllocationTexelSize, NewAllocation->bPointSample ? 1 : 0);
	}

	return NewAllocation;
}

void FIndirectLightingCache::ReleasePrimitive(FPrimitiveComponentId PrimitiveId)
{
	FIndirectLightingCacheAllocation* PrimitiveAllocation;

	if (PrimitiveAllocations.RemoveAndCopyValue(PrimitiveId, PrimitiveAllocation))
	{
		check(PrimitiveAllocation);

		if (PrimitiveAllocation->IsValid())
		{
			DeallocateBlock(PrimitiveAllocation->MinTexel, PrimitiveAllocation->AllocationTexelSize);
		}

		delete PrimitiveAllocation;
	}
}

FIndirectLightingCacheAllocation* FIndirectLightingCache::FindPrimitiveAllocation(FPrimitiveComponentId PrimitiveId) const
{
	return PrimitiveAllocations.FindRef(PrimitiveId);
}

FAutoConsoleTaskPriority CPrio_FUpdateCachePrimitivesTask(
	TEXT("TaskGraph.TaskPriorities.UpdateCachePrimitivesTask"),
	TEXT("Task and thread priority for FUpdateCachePrimitivesTask."),
	ENamedThreads::HighThreadPriority, // if we have high priority task threads, then use them...
	ENamedThreads::NormalTaskPriority, // .. at normal task priority
	ENamedThreads::HighTaskPriority // if we don't have hi pri threads, then use normal priority threads at high task priority instead
	);

class FUpdateCachePrimitivesTask
{
	FIndirectLightingCache* ILC;
	FScene* Scene;
	FSceneRenderer& Renderer;
	TMap<FIntVector, FBlockUpdateInfo>& BlocksToUpdate;
	TArray<FIndirectLightingCacheAllocation*>& TransitionsOverTimeToUpdate;
	TArray<FPrimitiveSceneInfo*>& PrimitivesToUpdateStaticMeshes;
	bool bAllowUnbuiltPreview;

public:

	FUpdateCachePrimitivesTask(
		FIndirectLightingCache* InILC, 
		FScene* InScene, FSceneRenderer& InRenderer, 
		bool bInAllowUnbuiltPreview, 
		TMap<FIntVector, FBlockUpdateInfo>& OutBlocksToUpdate, 
		TArray<FIndirectLightingCacheAllocation*>& OutTransitionsOverTimeToUpdate, 
		TArray<FPrimitiveSceneInfo*>& OutPrimitivesToUpdateStaticMeshes)
		: ILC(InILC)
		, Scene(InScene)
		, Renderer(InRenderer)
		, BlocksToUpdate(OutBlocksToUpdate)
		, TransitionsOverTimeToUpdate(OutTransitionsOverTimeToUpdate)
		, PrimitivesToUpdateStaticMeshes(OutPrimitivesToUpdateStaticMeshes)
		, bAllowUnbuiltPreview(bInAllowUnbuiltPreview)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FUpdateCachePrimitivesTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_FUpdateCachePrimitivesTask.Get();
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		ILC->UpdateCachePrimitivesInternal(Scene, Renderer, bAllowUnbuiltPreview, BlocksToUpdate, TransitionsOverTimeToUpdate, PrimitivesToUpdateStaticMeshes);
	}
};

void FIndirectLightingCache::StartUpdateCachePrimitivesTask(FScene* Scene, FSceneRenderer& Renderer, bool bAllowUnbuiltPreview, FILCUpdatePrimTaskData& OutTaskData)
{
	OutTaskData.TaskRef = TGraphTask<FUpdateCachePrimitivesTask>::CreateTask().ConstructAndDispatchWhenReady(
		this, 
		Scene, 
		Renderer, 
		bAllowUnbuiltPreview, 
		OutTaskData.OutBlocksToUpdate, 
		OutTaskData.OutTransitionsOverTimeToUpdate,
		OutTaskData.OutPrimitivesToUpdateStaticMeshes
	);
}

void FIndirectLightingCache::FinalizeCacheUpdates(FRHICommandListBase& RHICmdList, FScene* Scene, FSceneRenderer& Renderer, FILCUpdatePrimTaskData& TaskData)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateIndirectLightingCacheFinalize);	
	FTaskGraphInterface::Get().WaitUntilTaskCompletes(TaskData.TaskRef, ENamedThreads::GetRenderThread_Local());
	FinalizeUpdateInternal_RenderThread(RHICmdList, Scene, Renderer, TaskData.OutBlocksToUpdate, TaskData.OutTransitionsOverTimeToUpdate, TaskData.OutPrimitivesToUpdateStaticMeshes);
}

void FIndirectLightingCache::UpdateCache(FRHICommandListBase& RHICmdList, FScene* Scene, FSceneRenderer& Renderer, bool bAllowUnbuiltPreview)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateIndirectLightingCache);

	TMap<FIntVector, FBlockUpdateInfo>BlocksToUpdate;
	TArray<FIndirectLightingCacheAllocation*> TransitionsOverTimeToUpdate;
	TArray<FPrimitiveSceneInfo*> PrimitivesToUpdateStaticMeshes;

	UpdateCachePrimitivesInternal(Scene, Renderer, bAllowUnbuiltPreview, BlocksToUpdate, TransitionsOverTimeToUpdate, PrimitivesToUpdateStaticMeshes);
	FinalizeUpdateInternal_RenderThread(RHICmdList, Scene, Renderer, BlocksToUpdate, TransitionsOverTimeToUpdate, PrimitivesToUpdateStaticMeshes);
}

bool FIndirectLightingCache::IndirectLightingAllowed(FScene* Scene, FSceneRenderer& Renderer) const
{
	bool bAnyViewAllowsIndirectLightingCache = false;
	if (IsIndirectLightingCacheAllowed(GetFeatureLevel()) && Scene->PrecomputedLightVolumes.Num() > 0)
	{
		for (int32 ViewIndex = 0; ViewIndex < Renderer.Views.Num(); ViewIndex++)
		{
			bAnyViewAllowsIndirectLightingCache |= (bool)Renderer.Views[ViewIndex].Family->EngineShowFlags.IndirectLightingCache;
		}
	}
	return bAnyViewAllowsIndirectLightingCache;
}

void FIndirectLightingCache::ProcessPrimitiveUpdate(FScene* Scene, FViewInfo& View, int32 PrimitiveIndex, bool bAllowUnbuiltPreview, bool bAllowVolumeSample, TMap<FIntVector, FBlockUpdateInfo>& OutBlocksToUpdate, TArray<FIndirectLightingCacheAllocation*>& OutTransitionsOverTimeToUpdate, TArray<FPrimitiveSceneInfo*>& OutPrimitivesToUpdateStaticMeshes)
{
	FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene->Primitives[PrimitiveIndex];
	const bool bIndirectLightingCacheBufferWasDirty = PrimitiveSceneInfo->NeedsIndirectLightingCacheBufferUpdate();
	
	const TMap<FPrimitiveComponentId, FAttachmentGroupSceneInfo>& AttachmentGroups = Scene->AttachmentGroups;
	UpdateCachePrimitive(AttachmentGroups, PrimitiveSceneInfo, bAllowUnbuiltPreview, bAllowVolumeSample, OutBlocksToUpdate, OutTransitionsOverTimeToUpdate, OutPrimitivesToUpdateStaticMeshes);

	// If it was already dirty, then the primitive is already in one of the view dirty primitive list at this point.
	// This also ensures that a primitive does not get added twice to the list, which could create an array reallocation.
	if (!bIndirectLightingCacheBufferWasDirty && PrimitiveSceneInfo->NeedsIndirectLightingCacheBufferUpdate())
	{
		View.DirtyIndirectLightingCacheBufferPrimitives.Push(PrimitiveSceneInfo);
	}
}

void FIndirectLightingCache::UpdateCachePrimitivesInternal(FScene* Scene, FSceneRenderer& Renderer, bool bAllowUnbuiltPreview, TMap<FIntVector, FBlockUpdateInfo>& OutBlocksToUpdate, TArray<FIndirectLightingCacheAllocation*>& OutTransitionsOverTimeToUpdate, TArray<FPrimitiveSceneInfo*>& OutPrimitivesToUpdateStaticMeshes)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateIndirectLightingCachePrims);

	const TMap<FPrimitiveComponentId, FAttachmentGroupSceneInfo>& AttachmentGroups = Scene->AttachmentGroups;

	if (IndirectLightingAllowed(Scene, Renderer))
	{
		for (FViewInfo& View : Renderer.Views)
		{
			View.DirtyIndirectLightingCacheBufferPrimitivesMutex.Lock();
		}

		if (bUpdateAllCacheEntries)
		{
			const uint32 PrimitiveCount = Scene->Primitives.Num();

			for (uint32 PrimitiveIndex = 0; PrimitiveIndex < PrimitiveCount; ++PrimitiveIndex)
			{
				FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene->Primitives[PrimitiveIndex];
				const bool bIndirectLightingCacheBufferWasDirty = PrimitiveSceneInfo->NeedsIndirectLightingCacheBufferUpdate();
				
				UpdateCachePrimitive(AttachmentGroups, PrimitiveSceneInfo, false, true, OutBlocksToUpdate, OutTransitionsOverTimeToUpdate, OutPrimitivesToUpdateStaticMeshes);

				// If it was already dirty, then the primitive is already in one of the view dirty primitive list at this point.
				if (!bIndirectLightingCacheBufferWasDirty)
				{
					PrimitiveSceneInfo->MarkIndirectLightingCacheBufferDirty();

					// Check if it is visible otherwise, it will be updated next time it is visible.
					for (int32 ViewIndex = 0; ViewIndex < Renderer.Views.Num(); ViewIndex++)
					{
						FViewInfo& View = Renderer.Views[ViewIndex];

						if (View.PrimitiveVisibilityMap[PrimitiveIndex])
						{
							View.DirtyIndirectLightingCacheBufferPrimitives.Push(PrimitiveSceneInfo);
							break; // We only need to add it in one of the view list.
						}
					}
				}
			}
		}
		else
		{
			for (int32 ViewIndex = 0; ViewIndex < Renderer.Views.Num(); ViewIndex++)
			{
				FViewInfo& View = Renderer.Views[ViewIndex];

				for (FSceneSetBitIterator BitIt(View.PrimitiveVisibilityMap); BitIt; ++BitIt)
				{
					uint32 PrimitiveIndex = BitIt.GetIndex();
					// FDrawTranslucentMeshAction::AllowIndirectLightingCacheVolumeTexture doesn't allow volume samples on translucency, so we only need to support one if the primitive has at least one opaque material
					const bool bAllowVolumeSample = View.PrimitiveViewRelevanceMap[PrimitiveIndex].bOpaque;
					ProcessPrimitiveUpdate(Scene, View, PrimitiveIndex, bAllowUnbuiltPreview, bAllowVolumeSample, OutBlocksToUpdate, OutTransitionsOverTimeToUpdate, OutPrimitivesToUpdateStaticMeshes);
				}

				// Any visible primitives with an indirect shadow need their ILC updated, since that determines the indirect shadow direction
				for (int32 IndirectPrimitiveIndex = 0; IndirectPrimitiveIndex < View.IndirectShadowPrimitives.Num(); IndirectPrimitiveIndex++)
				{
					int32 PrimitiveIndex = View.IndirectShadowPrimitives[IndirectPrimitiveIndex]->GetIndex();

					if (!View.PrimitiveVisibilityMap[PrimitiveIndex])
					{
						ProcessPrimitiveUpdate(Scene, View, PrimitiveIndex, bAllowUnbuiltPreview, true, OutBlocksToUpdate, OutTransitionsOverTimeToUpdate, OutPrimitivesToUpdateStaticMeshes);
					}
				}
			}
		}

		for (FViewInfo& View : Renderer.Views)
		{
			View.DirtyIndirectLightingCacheBufferPrimitivesMutex.Unlock();
		}

		bUpdateAllCacheEntries = false;
	}
}

void FIndirectLightingCache::FinalizeUpdateInternal_RenderThread(FRHICommandListBase& RHICmdList, FScene* Scene, FSceneRenderer& Renderer, TMap<FIntVector, FBlockUpdateInfo>& BlocksToUpdate, const TArray<FIndirectLightingCacheAllocation*>& TransitionsOverTimeToUpdate, TArray<FPrimitiveSceneInfo*>& PrimitivesToUpdateStaticMeshes)
{
	check(IsInRenderingThread());

	if (IndirectLightingAllowed(Scene, Renderer))
	{
		{
			SCOPE_CYCLE_COUNTER(STAT_UpdateIndirectLightingCacheBlocks);
			UpdateBlocks(RHICmdList, Scene, Renderer.Views.GetData(), BlocksToUpdate);
		}

		{
			SCOPE_CYCLE_COUNTER(STAT_UpdateIndirectLightingCacheTransitions);
			UpdateTransitionsOverTime(TransitionsOverTimeToUpdate, Renderer.ViewFamily.Time.GetDeltaWorldTimeSeconds());
		}

		for (int32 PrimitiveIndex = 0; PrimitiveIndex < PrimitivesToUpdateStaticMeshes.Num(); ++PrimitiveIndex)
		{
			PrimitivesToUpdateStaticMeshes[PrimitiveIndex]->RequestStaticMeshUpdate();
		}
	}	

	if (GCacheDrawLightingSamples || Renderer.ViewFamily.EngineShowFlags.VolumeLightingSamples || GCacheDrawDirectionalShadowing)
	{
		FViewElementPDI DebugPDI(&Renderer.Views[0], nullptr, &Renderer.Views[0].DynamicPrimitiveCollector);

		for (int32 VolumeIndex = 0; VolumeIndex < Scene->PrecomputedLightVolumes.Num(); VolumeIndex++)
		{
			const FPrecomputedLightVolume* PrecomputedLightVolume = Scene->PrecomputedLightVolumes[VolumeIndex];

			PrecomputedLightVolume->DebugDrawSamples(&DebugPDI, GCacheDrawDirectionalShadowing != 0);
		}
	}
}

bool FIndirectLightingCache::UpdateCacheAllocation(
	const FBoxSphereBounds& Bounds,
	int32 BlockSize,
	bool bPointSample,
	bool bUnbuiltPreview,
	FIndirectLightingCacheAllocation*& Allocation,
	TMap<FIntVector, FBlockUpdateInfo>& BlocksToUpdate,
	TArray<FIndirectLightingCacheAllocation*>& TransitionsOverTimeToUpdate)
{
	bool bUpdated = false;

	if (Allocation 
		&& Allocation->IsValid()
		&& Allocation->AllocationTexelSize == BlockSize
		&& Allocation->bPointSample == bPointSample)
	{
		FIndirectLightingCacheBlock& Block = FindBlock(Allocation->MinTexel);

		// Calculate a potentially new min and size based on the current bounds
		FVector NewMin;
		FVector NewSize;		
		CalculateBlockPositionAndSize(Bounds, Block.TexelSize, NewMin, NewSize);

		// If the primitive has moved enough to change its block min and size, we need to interpolate it again
		if (Allocation->bIsDirty || GCacheUpdateEveryFrame || !Block.Min.Equals(NewMin) || !Block.Size.Equals(NewSize))
		{
			// Update the block and primitive allocation with the new bounds
			Block.Min = NewMin;
			Block.Size = NewSize;

			FVector NewScale;
			FVector NewAdd;
			FVector MinUV;
			FVector MaxUV;
			CalculateBlockScaleAndAdd(Allocation->MinTexel, Allocation->AllocationTexelSize, NewMin, NewSize, NewScale, NewAdd, MinUV, MaxUV);

			Allocation->SetParameters(Allocation->MinTexel, Allocation->AllocationTexelSize, NewScale, NewAdd, MinUV, MaxUV, bPointSample, bUnbuiltPreview);
			BlocksToUpdate.Add(Block.MinTexel, FBlockUpdateInfo(Block, Allocation));

			// Updating the block will also update the target position, meaning the ILC must be updated too.
			TransitionsOverTimeToUpdate.AddUnique(Allocation);
			bUpdated = true;
		}
		else if ((Allocation->SingleSamplePosition - Allocation->TargetPosition).SizeSquared() > DELTA)
		{
			TransitionsOverTimeToUpdate.AddUnique(Allocation);
			bUpdated = true;
		}
	}
	else
	{
		delete Allocation;
		Allocation = CreateAllocation(BlockSize, Bounds, bPointSample, bUnbuiltPreview);

		if (Allocation->IsValid())
		{
			// Must interpolate lighting for this new block
			BlocksToUpdate.Add(Allocation->MinTexel, FBlockUpdateInfo(VolumeBlocks.FindChecked(Allocation->MinTexel), Allocation));
		}
		bUpdated = true;
	}

	return bUpdated;
}

void FIndirectLightingCache::UpdateCachePrimitive(
	const TMap<FPrimitiveComponentId, FAttachmentGroupSceneInfo>& AttachmentGroups,
	FPrimitiveSceneInfo* PrimitiveSceneInfo,
	bool bAllowUnbuiltPreview,
	bool bAllowVolumeSample,
	TMap<FIntVector, FBlockUpdateInfo>& BlocksToUpdate,
	TArray<FIndirectLightingCacheAllocation*>& TransitionsOverTimeToUpdate,
	TArray<FPrimitiveSceneInfo*>& PrimitivesToUpdateStaticMeshes)
{
	FPrimitiveSceneProxy* PrimitiveSceneProxy = PrimitiveSceneInfo->Proxy;	
	FIndirectLightingCacheAllocation** PrimitiveAllocationPtr = PrimitiveAllocations.Find(PrimitiveSceneInfo->PrimitiveComponentId);
	FIndirectLightingCacheAllocation* PrimitiveAllocation = PrimitiveAllocationPtr != NULL ? *PrimitiveAllocationPtr : NULL;

	const bool bIsMovable = PrimitiveSceneProxy->IsMovable();

	if (PrimitiveSceneProxy->WillEverBeLit()
		&& ((bAllowUnbuiltPreview && PrimitiveSceneProxy->HasStaticLighting() && PrimitiveAllocation && PrimitiveAllocation->bIsDirty)
		|| (PrimitiveSceneProxy->IsMovable() && PrimitiveSceneProxy->GetIndirectLightingCacheQuality() != ILCQ_Off)))
	{
		const FIndirectLightingCacheAllocation* AttachmentParentAllocation = NULL;

		if (PrimitiveSceneInfo->LightingAttachmentRoot.IsValid())
		{			
			const FAttachmentGroupSceneInfo& AttachmentGroup = AttachmentGroups.FindChecked(PrimitiveSceneInfo->LightingAttachmentRoot);

			if (AttachmentGroup.ParentSceneInfo && AttachmentGroup.ParentSceneInfo->Proxy->LightAttachmentsAsGroup())
			{
				AttachmentParentAllocation = FindPrimitiveAllocation(AttachmentGroup.ParentSceneInfo->PrimitiveComponentId);
			}
		}

		const FIndirectLightingCacheAllocation* OriginalCachedIndirectLightingCacheAllocation = PrimitiveSceneInfo->IndirectLightingCacheAllocation;

		if (AttachmentParentAllocation)
		{
			// Reuse the attachment parent's lighting allocation if part of an attachment group
			PrimitiveSceneInfo->IndirectLightingCacheAllocation = AttachmentParentAllocation;

			// Don't know here if this parent ILC is or will be dirty or not. Always update.
			PrimitiveSceneInfo->MarkIndirectLightingCacheBufferDirty();
		}
		else
		{
			FIndirectLightingCacheAllocation* OriginalAllocation = PrimitiveAllocation;
			const bool bUnbuiltPreview = bAllowUnbuiltPreview && !bIsMovable;
			const bool bPointSample = PrimitiveSceneProxy->GetIndirectLightingCacheQuality() == ILCQ_Point || bUnbuiltPreview || !bAllowVolumeSample;
			const int32 BlockSize = bPointSample ? 1 : GLightingCacheMovableObjectAllocationSize;

			// Light with the cumulative bounds of the entire attachment group
			const bool bUpdated = UpdateCacheAllocation(PrimitiveSceneInfo->GetAttachmentGroupBounds(), BlockSize, bPointSample, bUnbuiltPreview, PrimitiveAllocation, BlocksToUpdate, TransitionsOverTimeToUpdate);

			// Cache the primitive allocation pointer on the FPrimitiveSceneInfo for base pass rendering
			PrimitiveSceneInfo->IndirectLightingCacheAllocation = PrimitiveAllocation;

			if (OriginalAllocation != PrimitiveAllocation)
			{
				if (OriginalAllocation)
				{
					PrimitiveAllocations.Remove(PrimitiveSceneInfo->PrimitiveComponentId);
				}

				// Allocate space in the atlas for this primitive and add it to a map, whose key is the component, so the allocation will persist through a re-register
				PrimitiveAllocations.Add(PrimitiveSceneInfo->PrimitiveComponentId, PrimitiveAllocation);
			}

			if (bUpdated)
			{
				PrimitiveSceneInfo->MarkIndirectLightingCacheBufferDirty();
			}
		}

		// Cached mesh draw command shader selection depends on IndirectLightingCacheAllocation existence and bPointSample value.
		// If any of those changes, we need to re-cache mesh draw commands.
		if (OriginalCachedIndirectLightingCacheAllocation == nullptr
			|| (OriginalCachedIndirectLightingCacheAllocation 
				&& PrimitiveSceneInfo->IndirectLightingCacheAllocation 
				&& OriginalCachedIndirectLightingCacheAllocation->bPointSample != PrimitiveSceneInfo->IndirectLightingCacheAllocation->bPointSample))
		{
			PrimitivesToUpdateStaticMeshes.Add(PrimitiveSceneInfo);
		}
	}
}

void FIndirectLightingCache::UpdateBlocks(FRHICommandListBase& RHICmdList, FScene* Scene, FViewInfo* DebugDrawingView, TMap<FIntVector, FBlockUpdateInfo>& BlocksToUpdate)
{
	if (BlocksToUpdate.Num() > 0 && !IsInitialized())
	{
		InitResource(RHICmdList);
	}

	INC_DWORD_STAT_BY(STAT_IndirectLightingCacheUpdates, BlocksToUpdate.Num());

	for (TMap<FIntVector, FBlockUpdateInfo>::TIterator It(BlocksToUpdate); It; ++It)
	{
		UpdateBlock(RHICmdList, Scene, DebugDrawingView, It.Value());
	}
}

void FIndirectLightingCache::UpdateTransitionsOverTime(const TArray<FIndirectLightingCacheAllocation*>& TransitionsOverTimeToUpdate, float DeltaWorldTime) const
{
	for (int32 AllocationIndex = 0; AllocationIndex < TransitionsOverTimeToUpdate.Num(); AllocationIndex++)
	{
		FIndirectLightingCacheAllocation* Allocation = TransitionsOverTimeToUpdate[AllocationIndex];
		const float TransitionDistance = (Allocation->SingleSamplePosition - Allocation->TargetPosition).Size();

		if (TransitionDistance > DELTA)
		{
			// Transition faster for unbuilt meshes which is important for meshing visualization
			const float EffectiveTransitionSpeed = GSingleSampleTransitionSpeed * (Allocation->bUnbuiltPreview ? 4 : 1);
			const float LerpFactor = FMath::Clamp(GSingleSampleTransitionSpeed * DeltaWorldTime / TransitionDistance, 0.0f, 1.0f);
			Allocation->SingleSamplePosition = FMath::Lerp(Allocation->SingleSamplePosition, Allocation->TargetPosition, LerpFactor);

			for (int32 VectorIndex = 0; VectorIndex < 3; VectorIndex++) // RGB
			{
				Allocation->SingleSamplePacked0[VectorIndex] = FMath::Lerp(Allocation->SingleSamplePacked0[VectorIndex], Allocation->TargetSamplePacked0[VectorIndex], LerpFactor);
				Allocation->SingleSamplePacked1[VectorIndex] = FMath::Lerp(Allocation->SingleSamplePacked1[VectorIndex], Allocation->TargetSamplePacked1[VectorIndex], LerpFactor);
			}
			Allocation->SingleSamplePacked2 = FMath::Lerp(Allocation->SingleSamplePacked2, Allocation->TargetSamplePacked2, LerpFactor);
			Allocation->CurrentDirectionalShadowing = FMath::Lerp(Allocation->CurrentDirectionalShadowing, Allocation->TargetDirectionalShadowing, LerpFactor);

			const FVector3f CurrentSkyBentNormal = FMath::Lerp(
				FVector3f(Allocation->CurrentSkyBentNormal) * Allocation->CurrentSkyBentNormal.W, 
				FVector3f(Allocation->TargetSkyBentNormal) * Allocation->TargetSkyBentNormal.W, 
				LerpFactor);

			const float BentNormalLength = CurrentSkyBentNormal.Size();

			Allocation->CurrentSkyBentNormal = FVector4f(CurrentSkyBentNormal / FMath::Max(BentNormalLength, .0001f), BentNormalLength);
		}
	}
}

void FIndirectLightingCache::SetLightingCacheDirty(FScene* Scene, const FPrecomputedLightVolume* Volume)
{
	if (Volume)
	{
		for (int32 PrimitiveIndex = 0; PrimitiveIndex < Scene->Primitives.Num(); ++PrimitiveIndex)
		{
			FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene->Primitives[PrimitiveIndex];
			FIndirectLightingCacheAllocation** PrimitiveAllocationPtr = PrimitiveAllocations.Find(PrimitiveSceneInfo->PrimitiveComponentId);
			FIndirectLightingCacheAllocation* PrimitiveAllocation = PrimitiveAllocationPtr != NULL ? *PrimitiveAllocationPtr : NULL;

			if (PrimitiveAllocation)
			{
				if (Volume->IntersectBounds(PrimitiveSceneInfo->Proxy->GetBounds()))
				{
					PrimitiveAllocation->bIsDirty = true;
					PrimitiveAllocation->bHasEverUpdatedSingleSample = false;
				}
			}
		}
	}
	else
	{
		for (TMap<FPrimitiveComponentId, FIndirectLightingCacheAllocation*>::TIterator It(PrimitiveAllocations); It; ++It)
		{
			It.Value()->bIsDirty = true;
			It.Value()->bHasEverUpdatedSingleSample = false;
		}
	
		// next rendering we update all entries no matter if they are visible to avoid further hitches
		bUpdateAllCacheEntries = true;
	}
}

void FIndirectLightingCache::UpdateBlock(FRHICommandListBase& RHICmdList, FScene* Scene, FViewInfo* DebugDrawingView, FBlockUpdateInfo& BlockInfo)
{
	const int32 NumSamplesPerBlock = BlockInfo.Block.TexelSize * BlockInfo.Block.TexelSize * BlockInfo.Block.TexelSize;
	FSHVectorRGB3 SingleSample;

	float DirectionalShadowing = 1;
	FVector SkyBentNormal(0, 0, 1);

	//always do point interpolation to get valid 3band single sample and directional data.
	InterpolatePoint(Scene, BlockInfo.Block, DirectionalShadowing, SingleSample, SkyBentNormal);

	if (CanIndirectLightingCacheUseVolumeTexture(GetFeatureLevel()) && !BlockInfo.Allocation->bPointSample)
	{
		static TArray<float> AccumulatedWeight;
		AccumulatedWeight.Reset(NumSamplesPerBlock);
		AccumulatedWeight.AddZeroed(NumSamplesPerBlock);

		//volume textures are encoded as two band, so no reason to waste perf interpolating 3 bands.
		static TArray<FSHVectorRGB2> AccumulatedIncidentRadiance;

		AccumulatedIncidentRadiance.Reset(NumSamplesPerBlock);
		AccumulatedIncidentRadiance.AddZeroed(NumSamplesPerBlock);

		// Interpolate SH samples from precomputed lighting samples and accumulate lighting data for an entire block
		InterpolateBlock(Scene, BlockInfo.Block, AccumulatedWeight, AccumulatedIncidentRadiance);

		static TArray<FFloat16Color> Texture0Data;
		static TArray<FFloat16Color> Texture1Data;
		static TArray<FFloat16Color> Texture2Data;
		Texture0Data.Reset(NumSamplesPerBlock);
		Texture1Data.Reset(NumSamplesPerBlock);
		Texture2Data.Reset(NumSamplesPerBlock);
		Texture0Data.AddUninitialized(NumSamplesPerBlock);
		Texture1Data.AddUninitialized(NumSamplesPerBlock);
		Texture2Data.AddUninitialized(NumSamplesPerBlock);

		const int32 FormatSize = GPixelFormats[PF_FloatRGBA].BlockBytes;
		check(FormatSize == sizeof(FFloat16Color));

		// Encode the SH samples into a texture format
		// Note the single sample is updated even if this is a volume allocation, because translucent materials only use the single sample
		EncodeBlock(DebugDrawingView, BlockInfo.Block, AccumulatedWeight, AccumulatedIncidentRadiance, Texture0Data, Texture1Data, Texture2Data);

		// Setup an update region
		const FUpdateTextureRegion3D UpdateRegion(
			BlockInfo.Block.MinTexel.X,
			BlockInfo.Block.MinTexel.Y,
			BlockInfo.Block.MinTexel.Z,
			0,
			0,
			0,
			BlockInfo.Block.TexelSize,
			BlockInfo.Block.TexelSize,
			BlockInfo.Block.TexelSize);

		// Update the volume texture atlas
		RHICmdList.UpdateTexture3D(GetTexture0(), 0, UpdateRegion, BlockInfo.Block.TexelSize * FormatSize, BlockInfo.Block.TexelSize * BlockInfo.Block.TexelSize * FormatSize, (const uint8*)Texture0Data.GetData());
		RHICmdList.UpdateTexture3D(GetTexture1(), 0, UpdateRegion, BlockInfo.Block.TexelSize * FormatSize, BlockInfo.Block.TexelSize * BlockInfo.Block.TexelSize * FormatSize, (const uint8*)Texture1Data.GetData());
		RHICmdList.UpdateTexture3D(GetTexture2(), 0, UpdateRegion, BlockInfo.Block.TexelSize * FormatSize, BlockInfo.Block.TexelSize * BlockInfo.Block.TexelSize * FormatSize, (const uint8*)Texture2Data.GetData());
	}
	else
	{
		if (GCacheDrawInterpolationPoints != 0 && DebugDrawingView)
		{
			FViewElementPDI DebugPDI(DebugDrawingView, nullptr, nullptr);
			const FVector WorldPosition = BlockInfo.Block.Min;
			DebugPDI.DrawPoint(WorldPosition, FLinearColor(0, 0, 1), 10, SDPG_World);
		}
	}

	// Record the position that the sample was taken at
	BlockInfo.Allocation->TargetPosition = BlockInfo.Block.Min + BlockInfo.Block.Size / 2;

	BlockInfo.Allocation->TargetSamplePacked0[0] = FVector4f(SingleSample.R.V[0], SingleSample.R.V[1], SingleSample.R.V[2], SingleSample.R.V[3]) / PI;
	BlockInfo.Allocation->TargetSamplePacked0[1] = FVector4f(SingleSample.G.V[0], SingleSample.G.V[1], SingleSample.G.V[2], SingleSample.G.V[3]) / PI;
	BlockInfo.Allocation->TargetSamplePacked0[2] = FVector4f(SingleSample.B.V[0], SingleSample.B.V[1], SingleSample.B.V[2], SingleSample.B.V[3]) / PI;
	BlockInfo.Allocation->TargetSamplePacked1[0] = FVector4f(SingleSample.R.V[4], SingleSample.R.V[5], SingleSample.R.V[6], SingleSample.R.V[7]) / PI;
	BlockInfo.Allocation->TargetSamplePacked1[1] = FVector4f(SingleSample.G.V[4], SingleSample.G.V[5], SingleSample.G.V[6], SingleSample.G.V[7]) / PI;
	BlockInfo.Allocation->TargetSamplePacked1[2] = FVector4f(SingleSample.B.V[4], SingleSample.B.V[5], SingleSample.B.V[6], SingleSample.B.V[7]) / PI;
	BlockInfo.Allocation->TargetSamplePacked2 = FVector4f(SingleSample.R.V[8], SingleSample.G.V[8], SingleSample.B.V[8], 0) / PI;

	BlockInfo.Allocation->TargetDirectionalShadowing = DirectionalShadowing;

	const float BentNormalLength = SkyBentNormal.Size();
	BlockInfo.Allocation->TargetSkyBentNormal = FVector4f(FVector3f(SkyBentNormal / FMath::Max(BentNormalLength, .0001f)), BentNormalLength);

	if (!BlockInfo.Allocation->bHasEverUpdatedSingleSample)
	{
		// If this is the first update, also set the interpolated state to match the new target
		//@todo - detect and handle teleports in the same way
		BlockInfo.Allocation->SingleSamplePosition = BlockInfo.Allocation->TargetPosition;

		for (int32 VectorIndex = 0; VectorIndex < 3; VectorIndex++) // RGB
		{
			BlockInfo.Allocation->SingleSamplePacked0[VectorIndex] = BlockInfo.Allocation->TargetSamplePacked0[VectorIndex];
			BlockInfo.Allocation->SingleSamplePacked1[VectorIndex] = BlockInfo.Allocation->TargetSamplePacked1[VectorIndex];
		}
		BlockInfo.Allocation->SingleSamplePacked2 = BlockInfo.Allocation->TargetSamplePacked2;
		BlockInfo.Allocation->CurrentDirectionalShadowing = BlockInfo.Allocation->TargetDirectionalShadowing;
		BlockInfo.Allocation->CurrentSkyBentNormal = BlockInfo.Allocation->TargetSkyBentNormal;

		BlockInfo.Allocation->bHasEverUpdatedSingleSample = true;
	}

	BlockInfo.Block.bHasEverBeenUpdated = true;
}

template class TSHVector<2>;
template class TSHVector<3>;

template <int32 SHOrder>
static void ReduceSHRinging(TSHVectorRGB<SHOrder>& IncidentRadiance)
{
	const FVector BrightestDirection = IncidentRadiance.GetLuminance().GetMaximumDirection();
	TSHVector<SHOrder> BrigthestDiffuseTransferSH = TSHVector<SHOrder>::CalcDiffuseTransfer(BrightestDirection);
	FLinearColor BrightestLighting = Dot(IncidentRadiance, BrigthestDiffuseTransferSH);

	TSHVector<SHOrder> OppositeDiffuseTransferSH = TSHVector<SHOrder>::CalcDiffuseTransfer(-BrightestDirection);
	FLinearColor OppositeLighting = Dot(IncidentRadiance, OppositeDiffuseTransferSH);

	// Try to maintain 5% of the brightest side on the opposite side
	// This is necessary to reduce ringing artifacts when the SH contains mostly strong, directional lighting from one direction
	FVector MinOppositeLighting = FVector(BrightestLighting) * .05f;
	FVector NegativeAmount = (MinOppositeLighting - FVector(OppositeLighting)).ComponentMax(FVector(0));

	//@todo - do this in a way that preserves energy and doesn't change hue
	IncidentRadiance.AddAmbient(FLinearColor(NegativeAmount) * TSHVector<SHOrder>::ConstantBasisIntegral);
}

void FIndirectLightingCache::InterpolatePoint(
	FScene* Scene, 
	const FIndirectLightingCacheBlock& Block,
	float& OutDirectionalShadowing, 
	FSHVectorRGB3& OutIncidentRadiance,
	FVector& OutSkyBentNormal)
{
	FSHVectorRGB3 AccumulatedIncidentRadiance;
	FVector AccumulatedSkyBentNormal(0, 0, 0);
	float AccumulatedDirectionalShadowing = 0;
	float AccumulatedWeight = 0;

	for (int32 VolumeIndex = 0; VolumeIndex < Scene->PrecomputedLightVolumes.Num(); VolumeIndex++)
	{
		const FPrecomputedLightVolume* PrecomputedLightVolume = Scene->PrecomputedLightVolumes[VolumeIndex];
		if (PrecomputedLightVolume)
		{
			PrecomputedLightVolume->InterpolateIncidentRadiancePoint(
				Block.Min + Block.Size / 2,
				AccumulatedWeight,
				AccumulatedDirectionalShadowing,
				AccumulatedIncidentRadiance,
				AccumulatedSkyBentNormal);
		}
	}

	if (AccumulatedWeight > 0)
	{
		OutDirectionalShadowing = AccumulatedDirectionalShadowing / AccumulatedWeight;
		OutIncidentRadiance = AccumulatedIncidentRadiance / AccumulatedWeight;
		OutSkyBentNormal = AccumulatedSkyBentNormal / AccumulatedWeight;

		if (GCacheReduceSHRinging != 0)
		{
			ReduceSHRinging(OutIncidentRadiance);
		}
	}
	else
	{
		OutIncidentRadiance = AccumulatedIncidentRadiance;
		OutDirectionalShadowing = 1;
		// Use an unoccluded vector if no valid samples were found for interpolation
		OutSkyBentNormal = FVector(0, 0, 1);
	}
}

void FIndirectLightingCache::InterpolateBlock(
	FScene* Scene, 
	const FIndirectLightingCacheBlock& Block, 
	TArray<float>& AccumulatedWeight, 
	TArray<FSHVectorRGB2>& AccumulatedIncidentRadiance)
{
	const FBoxCenterAndExtent BlockBoundingBox(Block.Min + Block.Size / 2, Block.Size / 2);
	const FVector HalfTexelWorldOffset = BlockBoundingBox.Extent / FVector(Block.TexelSize);	

	if (GCacheLimitQuerySize && Block.TexelSize > 2)
	{
		for (int32 VolumeIndex = 0; VolumeIndex < Scene->PrecomputedLightVolumes.Num(); VolumeIndex++)
		{
			const FPrecomputedLightVolume* PrecomputedLightVolume = Scene->PrecomputedLightVolumes[VolumeIndex];

			// Compute the target query size
			// We will try to split up the allocation into groups that are smaller than this before querying the octree
			// This prevents very large objects from finding all the samples in the level in their octree search
			const float WorldTargetSize = PrecomputedLightVolume->GetNodeLevelExtent(GCacheQueryNodeLevel) * 2;

			const FVector WorldCellSize = Block.Size / FVector(Block.TexelSize);

			// Number of cells to increment by for query blocks
			FIntVector NumStepCells;
			NumStepCells.X = FMath::Max(1, FMath::FloorToInt(WorldTargetSize / WorldCellSize.X));
			NumStepCells.Y = FMath::Max(1, FMath::FloorToInt(WorldTargetSize / WorldCellSize.Y));
			NumStepCells.Z = FMath::Max(1, FMath::FloorToInt(WorldTargetSize / WorldCellSize.Z));
			FIntVector NumQueryStepCells(0, 0, 0);

			// World space size to increment by for query blocks
			const FVector WorldStepSize = FVector(NumStepCells) * WorldCellSize;
			FVector QueryWorldStepSize(0, 0, 0);

			check(NumStepCells.X > 0 && NumStepCells.Y > 0 && NumStepCells.Z > 0);

			// This will track the position in cells of the query block being built
			FIntVector CellIndex(0, 0, 0);

			// This will track the min world position of the query block being built
			FVector MinPosition = Block.Min;

			for (MinPosition.Z = Block.Min.Z, CellIndex.Z = 0; 
				CellIndex.Z < Block.TexelSize;
				MinPosition.Z += WorldStepSize.Z, CellIndex.Z += NumStepCells.Z)
			{
				QueryWorldStepSize.Z = WorldStepSize.Z;
				NumQueryStepCells.Z = NumStepCells.Z;

				// If this is the last query block in this dimension, adjust both the world space and cell sizes to match
				if (CellIndex.Z + NumStepCells.Z > Block.TexelSize)
				{
					QueryWorldStepSize.Z = Block.Min.Z + Block.Size.Z - MinPosition.Z;
					NumQueryStepCells.Z = Block.TexelSize - CellIndex.Z;
				}

				for (MinPosition.Y = Block.Min.Y, CellIndex.Y = 0; 
					CellIndex.Y < Block.TexelSize;
					MinPosition.Y += WorldStepSize.Y, CellIndex.Y += NumStepCells.Y)
				{
					QueryWorldStepSize.Y = WorldStepSize.Y;
					NumQueryStepCells.Y = NumStepCells.Y;

					if (CellIndex.Y + NumStepCells.Y > Block.TexelSize)
					{
						QueryWorldStepSize.Y = Block.Min.Y + Block.Size.Y - MinPosition.Y;
						NumQueryStepCells.Y = Block.TexelSize - CellIndex.Y;
					}

					for (MinPosition.X = Block.Min.X, CellIndex.X = 0; 
						CellIndex.X < Block.TexelSize;
						MinPosition.X += WorldStepSize.X, CellIndex.X += NumStepCells.X)
					{
						QueryWorldStepSize.X = WorldStepSize.X;
						NumQueryStepCells.X = NumStepCells.X;

						if (CellIndex.X + NumStepCells.X > Block.TexelSize)
						{
							QueryWorldStepSize.X = Block.Min.X + Block.Size.X - MinPosition.X;
							NumQueryStepCells.X = Block.TexelSize - CellIndex.X;
						}

						FVector BoxExtent = QueryWorldStepSize / 2;
						// Use a 0 query extent in dimensions that only have one cell, these become point queries
						BoxExtent.X = NumQueryStepCells.X == 1 ? 0 : BoxExtent.X;
						BoxExtent.Y = NumQueryStepCells.Y == 1 ? 0 : BoxExtent.Y;
						BoxExtent.Z = NumQueryStepCells.Z == 1 ? 0 : BoxExtent.Z;

						// Build a bounding box for the query block
						const FBoxCenterAndExtent BoundingBox(MinPosition + BoxExtent + HalfTexelWorldOffset, BoxExtent);

						checkSlow(CellIndex.X < Block.TexelSize && CellIndex.Y < Block.TexelSize && CellIndex.Z < Block.TexelSize);
						checkSlow(CellIndex.X + NumQueryStepCells.X <= Block.TexelSize
							&& CellIndex.Y + NumQueryStepCells.Y <= Block.TexelSize
							&& CellIndex.Z + NumQueryStepCells.Z <= Block.TexelSize);

						// Interpolate from the SH volume lighting samples that Lightmass computed
						PrecomputedLightVolume->InterpolateIncidentRadianceBlock(
							BoundingBox,
							NumQueryStepCells,
							FIntVector(Block.TexelSize),
							CellIndex,
							AccumulatedWeight,
							AccumulatedIncidentRadiance);
					}
				}
			}
		}
	}
	else
	{
		for (int32 VolumeIndex = 0; VolumeIndex < Scene->PrecomputedLightVolumes.Num(); VolumeIndex++)
		{
			const FPrecomputedLightVolume* PrecomputedLightVolume = Scene->PrecomputedLightVolumes[VolumeIndex];
			check(PrecomputedLightVolume);
			check(PrecomputedLightVolume->IsUsingHighQualityLightMap() == AllowHighQualityLightmaps(Scene->GetFeatureLevel()));
			// Interpolate from the SH volume lighting samples that Lightmass computed
			// Query using the bounds of all the samples in this block
			// There will be a performance cliff for large objects which end up intersecting with the entire octree
			PrecomputedLightVolume->InterpolateIncidentRadianceBlock(
				FBoxCenterAndExtent(BlockBoundingBox.Center + HalfTexelWorldOffset, BlockBoundingBox.Extent), 
				FIntVector(Block.TexelSize), 
				FIntVector(Block.TexelSize), 
				FIntVector(0), 
				AccumulatedWeight, 
				AccumulatedIncidentRadiance);
		}
	}	
}

void FIndirectLightingCache::EncodeBlock(
	FViewInfo* DebugDrawingView,
	const FIndirectLightingCacheBlock& Block, 
	const TArray<float>& AccumulatedWeight, 
	const TArray<FSHVectorRGB2>& AccumulatedIncidentRadiance,
	TArray<FFloat16Color>& Texture0Data,
	TArray<FFloat16Color>& Texture1Data,
	TArray<FFloat16Color>& Texture2Data	
	)
{
	FViewElementPDI DebugPDI(DebugDrawingView, nullptr, nullptr);

	for (int32 Z = 0; Z < Block.TexelSize; Z++)
	{
		for (int32 Y = 0; Y < Block.TexelSize; Y++)
		{
			for (int32 X = 0; X < Block.TexelSize; X++)
			{
				const int32 LinearIndex = Z * Block.TexelSize * Block.TexelSize + Y * Block.TexelSize + X;

				FSHVectorRGB2 IncidentRadiance = AccumulatedIncidentRadiance[LinearIndex];
				float Weight = AccumulatedWeight[LinearIndex];

				if (Weight > 0)
				{
					IncidentRadiance = IncidentRadiance / Weight;

					if (GCacheReduceSHRinging != 0)
					{
						ReduceSHRinging(IncidentRadiance);
					}
				}				

				if (GCacheDrawInterpolationPoints != 0 && DebugDrawingView)
				{
					const FVector WorldPosition = Block.Min + (FVector(X, Y, Z) + .5f) / Block.TexelSize * Block.Size;
					DebugPDI.DrawPoint(WorldPosition, FLinearColor(0, 0, 1), 10, SDPG_World);
				}
				
				Texture0Data[LinearIndex] = FLinearColor(IncidentRadiance.R.V[0], IncidentRadiance.G.V[0], IncidentRadiance.B.V[0], IncidentRadiance.R.V[3]);
				Texture1Data[LinearIndex] = FLinearColor(IncidentRadiance.R.V[1], IncidentRadiance.G.V[1], IncidentRadiance.B.V[1], IncidentRadiance.G.V[3]);
				Texture2Data[LinearIndex] = FLinearColor(IncidentRadiance.R.V[2], IncidentRadiance.G.V[2], IncidentRadiance.B.V[2], IncidentRadiance.B.V[3]);
			}
		}
	}
}
