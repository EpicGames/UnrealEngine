// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceCulling/InstanceCullingContext.h"
#include "CoreMinimal.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHI.h"
#include "ShaderParameterMacros.h"
#include "RenderGraphResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneRendering.h"
#include "ScenePrivate.h"
#include "SystemTextures.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "InstanceCullingDefinitions.h"
#include "InstanceCullingLoadBalancer.h"
#include "InstanceCullingMergedContext.h"
#include "InstanceCullingOcclusionQuery.h"
#include "RenderCore.h"
#include "MeshDrawCommandStats.h"
#include "UnrealEngine.h"
#include "RHIBreadcrumbs.h"
#include "Materials/MaterialRenderProxy.h"
#include "ViewData.h"
#include "GPUSkinCache.h"
#include "SceneCulling/SceneCullingRenderer.h"

static TAutoConsoleVariable<int32> CVarCullInstances(
	TEXT("r.CullInstances"),
	1,
	TEXT("CullInstances."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOcclusionCullInstances(
	TEXT("r.InstanceCulling.OcclusionCull"),
	0,
	TEXT("Whether to do per instance occlusion culling for GPU instance culling."),
	ECVF_RenderThreadSafe | ECVF_Preview);

static int32 GOcclusionForceInstanceCulling = 0;
static FAutoConsoleVariableRef CVarOcclusionForceInstanceCulling(
	TEXT("r.InstanceCulling.ForceInstanceCulling"),
	GOcclusionForceInstanceCulling,
	TEXT("Whether to force per instance occlusion culling."),
	ECVF_RenderThreadSafe);

static int32 GInstanceCullingAllowOrderPreservation = 1;
static FAutoConsoleVariableRef CVarInstanceCullingAllowOrderPreservation(
	TEXT("r.InstanceCulling.AllowInstanceOrderPreservation"),
	GInstanceCullingAllowOrderPreservation,
	TEXT("Whether or not to allow instances to preserve instance draw order using GPU compaction."),
	ECVF_RenderThreadSafe);

IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(InstanceCullingUbSlot);
IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FInstanceCullingGlobalUniforms, "InstanceCulling", InstanceCullingUbSlot);

IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(BatchedPrimitive);
IMPLEMENT_STATIC_AND_SHADER_UNIFORM_BUFFER_STRUCT_EX(FBatchedPrimitiveParameters, "BatchedPrimitive", BatchedPrimitive, FShaderParametersMetadata::EUsageFlags::UniformView);

static const TCHAR* BatchProcessingModeStr[] =
{
	TEXT("Generic"),
	TEXT("UnCulled"),
};

static_assert(UE_ARRAY_COUNT(BatchProcessingModeStr) == uint32(EBatchProcessingMode::Num), "BatchProcessingModeStr length does not match EBatchProcessingMode::Num, these must be kept in sync.");

DECLARE_GPU_STAT(BuildRenderingCommandsDeferred);
DECLARE_GPU_STAT(BuildRenderingCommands);

static bool IsInstanceOrderPreservationAllowed(EShaderPlatform ShaderPlatform)
{
	// Instance order preservation is currently not supported on mobile platforms
	return GInstanceCullingAllowOrderPreservation && !IsMobilePlatform(ShaderPlatform);
}

static FUintVector2 PackDrawCommandDesc(bool bMaterialUsesWorldPositionOffset, bool bMaterialAlwaysEvaluatesWorldPositionOffset, FMeshDrawCommandCullingPayload CullingPayload, EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags)
{
	// See UnpackDrawCommandDesc() in shader code.
	uint32 PackedData = bMaterialUsesWorldPositionOffset ? 1U : 0U;
	PackedData |= bMaterialAlwaysEvaluatesWorldPositionOffset ? 2U : 0U;
	PackedData |= CullingPayload.LodIndex << 2;
	if (EnumHasAnyFlags(CullingPayloadFlags, EMeshDrawCommandCullingPayloadFlags::MinScreenSizeCull))
	{
		PackedData |= CullingPayload.MinScreenSize << 6;
	}
	if (EnumHasAnyFlags(CullingPayloadFlags, EMeshDrawCommandCullingPayloadFlags::MaxScreenSizeCull))
	{
		PackedData |= CullingPayload.MaxScreenSize << 19;
	}
	return FUintVector2(PackedData, uint32(CullingPayload.DynamicMeshBoundsIndex));
}

FMeshDrawCommandOverrideArgs GetMeshDrawCommandOverrideArgs(const FInstanceCullingDrawParams& InstanceCullingDrawParams)
{
	FMeshDrawCommandOverrideArgs Result;
	Result.IndirectArgsBuffer = InstanceCullingDrawParams.DrawIndirectArgsBuffer.GetBuffer() != nullptr ? InstanceCullingDrawParams.DrawIndirectArgsBuffer.GetBuffer()->GetRHI() : nullptr;
	Result.InstanceDataByteOffset = InstanceCullingDrawParams.InstanceDataByteOffset;
	Result.IndirectArgsByteOffset = InstanceCullingDrawParams.IndirectArgsByteOffset;
	
	if (InstanceCullingDrawParams.BatchedPrimitive.GetUniformBuffer() != nullptr)
	{
		Result.InstanceBuffer = nullptr;
		Result.InstanceCullingStaticUB = InstanceCullingDrawParams.BatchedPrimitive.GetUniformBuffer()->GetRHI();
	}
	else
	{
		Result.InstanceBuffer = InstanceCullingDrawParams.InstanceIdOffsetBuffer.GetBuffer() != nullptr ? InstanceCullingDrawParams.InstanceIdOffsetBuffer.GetBuffer()->GetRHI() : nullptr;
		Result.InstanceCullingStaticUB = InstanceCullingDrawParams.InstanceCulling.GetUniformBuffer() != nullptr ? InstanceCullingDrawParams.InstanceCulling.GetUniformBuffer()->GetRHI() : nullptr;
	}

	return Result;
}

FUniformBufferStaticSlot FInstanceCullingContext::GetStaticUniformBufferSlot(EShaderPlatform ShaderPlatform)
{
	const static FName InstanceCullingSlotName = "InstanceCullingUbSlot";
	const static FName BatchedPrimitiveSlotName = "BatchedPrimitive";
	return FUniformBufferStaticSlotRegistry::Get().FindSlotByName(PlatformGPUSceneUsesUniformBufferView(ShaderPlatform) ? BatchedPrimitiveSlotName : InstanceCullingSlotName);
}

static uint32 GetInstanceDataStrideElements(EShaderPlatform ShaderPlatform, EBatchProcessingMode Mode)
{
	if (PlatformGPUSceneUsesUniformBufferView(ShaderPlatform))
	{
		// float4 elements, stride depends on whether we writing instances or primitives
		return FInstanceCullingContext::UniformViewInstanceStride[static_cast<uint32>(Mode)] / 16u;
	}
	else
	{ 
		// one uint element per-instance
		return 1u;
	}
}

uint32 FInstanceCullingContext::GetInstanceIdBufferStride(EShaderPlatform ShaderPlatform)
{
	if (PlatformGPUSceneUsesUniformBufferView(ShaderPlatform))
	{
		return UniformViewInstanceStride[1]; // UnCulled
	}
	else
	{
		return sizeof(uint32);
	}
}

uint32 FInstanceCullingContext::StepInstanceDataOffsetBytes(uint32 NumStepDraws) const
{
	// UniformBufferView path uses one instance step rate, on default step is once per draw
	if (bUsesUniformBufferView)
	{
		uint32 GenericStride = LoadBalancers[0]->GetTotalNumInstances() * UniformViewInstanceStride[0];
		uint32 UnculledStride = LoadBalancers[1]->GetTotalNumInstances() * UniformViewInstanceStride[1];
		return (GenericStride + UnculledStride) * ViewIds.Num();
	}
	else
	{
		return NumStepDraws * sizeof(uint32);
	}
}

uint32 FInstanceCullingContext::GetInstanceIdNumElements() const
{
	if (bUsesUniformBufferView)
	{
		// This data is used in CS to compute offset for writing instance data
		uint32 GenericStride = LoadBalancers[0]->GetTotalNumInstances() * UniformViewInstanceStride[0] / 16u;
		uint32 UnculledStride = LoadBalancers[1]->GetTotalNumInstances() * UniformViewInstanceStride[1] / 16u;
		return (GenericStride + UnculledStride) * ViewIds.Num();
	}
	else
	{
		return TotalInstances * ViewIds.Num();
	}
}

FInstanceCullingContext::FInstanceCullingContext(
	const TCHAR* PassName,
	EShaderPlatform InShaderPlatform,
	FInstanceCullingManager* InInstanceCullingManager,
	TArrayView<const int32> InViewIds,
	const TRefCountPtr<IPooledRenderTarget>& InPrevHZB,
	EInstanceCullingMode InInstanceCullingMode,
	EInstanceCullingFlags InFlags,
	EBatchProcessingMode InSingleInstanceProcessingMode,
	FSceneCullingRendererHierarchicalCullingQuery* InCullingQuery) :
	InstanceCullingManager(InInstanceCullingManager),
	ShaderPlatform(InShaderPlatform),
	ViewIds(InViewIds),
	PrevHZB(InPrevHZB),
	bIsEnabled(InInstanceCullingManager == nullptr || InInstanceCullingManager->IsEnabled()),
	InstanceCullingMode(InInstanceCullingMode),
	Flags(InFlags),
	SingleInstanceProcessingMode(InSingleInstanceProcessingMode),
	InstanceCullingStaticSlot(IsMobilePlatform(InShaderPlatform) ? GetStaticUniformBufferSlot(InShaderPlatform) : MAX_UNIFORM_BUFFER_STATIC_SLOTS),
	bUsesUniformBufferView(PlatformGPUSceneUsesUniformBufferView(InShaderPlatform)),
	CullingQuery(InCullingQuery)
{
#if MESH_DRAW_COMMAND_STATS
	if (FMeshDrawCommandStatsManager* Instance = FMeshDrawCommandStatsManager::Get())
	{
		if (FCString::Strcmp(PassName, TEXT("HitProxy")) != 0)
		{
			MeshDrawCommandPassStats = Instance->CreatePassStats(PassName);
		}
	}
#endif
}

bool FInstanceCullingContext::IsGPUCullingEnabled()
{
	return CVarCullInstances.GetValueOnAnyThread() != 0;
}

bool FInstanceCullingContext::IsOcclusionCullingEnabled()
{
	return IsGPUCullingEnabled() && CVarOcclusionCullInstances.GetValueOnAnyThread() != 0;
}

FInstanceCullingContext::~FInstanceCullingContext()
{
	for (auto& LoadBalancer : LoadBalancers)
	{
		if (LoadBalancer != nullptr)
		{
			delete LoadBalancer;
		}
	}
}

void FInstanceCullingContext::ResetCommands(int32 MaxNumCommands)
{
	IndirectArgs.Empty(MaxNumCommands);
	MeshDrawCommandInfos.Empty(MaxNumCommands);
	DrawCommandDescs.Empty(MaxNumCommands);
	InstanceIdOffsets.Empty(MaxNumCommands);
	PayloadData.Empty(MaxNumCommands);
	TotalInstances = 0U;
	bHasAnyIndirectDraw = false;

	DrawCommandCompactionData.Empty(MaxNumCommands);
	CompactionBlockDataIndices.Reset();
	NumCompactionInstances = 0U;
}

bool FInstanceCullingContext::IsInstanceOrderPreservationEnabled() const
{
	// NOTE: Instance compaction is currently not enabled on mobile platforms
	return IsInstanceOrderPreservationAllowed(ShaderPlatform) && !EnumHasAnyFlags(Flags, EInstanceCullingFlags::NoInstanceOrderPreservation);
}

uint32 FInstanceCullingContext::AllocateIndirectArgs(const FMeshDrawCommand *MeshDrawCommand)
{
	const uint32 NumPrimitives = MeshDrawCommand->NumPrimitives;
	if (ensure(MeshDrawCommand->PrimitiveType < PT_Num))
	{
		// default to PT_TriangleList & PT_RectList
		uint32 NumVerticesOrIndices = NumPrimitives * 3U;
		switch (MeshDrawCommand->PrimitiveType)
		{
		case PT_QuadList:
			NumVerticesOrIndices = NumPrimitives * 4U;
			break;
		case PT_TriangleStrip:
			NumVerticesOrIndices = NumPrimitives + 2U;
			break;
		case PT_LineList:
			NumVerticesOrIndices = NumPrimitives * 2U;
			break;
		case PT_PointList:
			NumVerticesOrIndices = NumPrimitives;
			break;
		default:
			break;
		}

		return IndirectArgs.Emplace(FRHIDrawIndexedIndirectParameters{ NumVerticesOrIndices, 0U, MeshDrawCommand->FirstIndex, int32(MeshDrawCommand->VertexParams.BaseVertexIndex), 0U });
	}
	return 0U;
}

void FInstanceCullingContext::BeginAsyncSetup(SyncPrerequisitesFuncType&& InSyncPrerequisitesFunc) 
{ 
	SyncPrerequisitesFunc = MoveTemp(InSyncPrerequisitesFunc); 
}

void FInstanceCullingContext::WaitForSetupTask()
{
	if (SyncPrerequisitesFunc)
	{
		SyncPrerequisitesFunc(*this);
	}
	SyncPrerequisitesFunc = SyncPrerequisitesFuncType();
}

void FInstanceCullingContext::SetDynamicPrimitiveInstanceOffsets(int32 InDynamicInstanceIdOffset, int32 InDynamicInstanceIdNum)
{
	DynamicInstanceIdOffset = InDynamicInstanceIdOffset;
	DynamicInstanceIdNum = InDynamicInstanceIdNum;
}

// Key things to achieve
// 1. low-data handling of since ID/Primitive path
// 2. no redundant alloc upload of indirect cmd if none needed.
// 2.1 Only allocate indirect draw cmd if needed, 
// 3. 

void FInstanceCullingContext::AddInstancesToDrawCommand(uint32 IndirectArgsOffset, int32 InstanceDataOffset, uint32 RunOffset, uint32 NumInstances, EInstanceFlags InstanceFlags)
{
	checkSlow(InstanceDataOffset >= 0);

	const bool bDynamicInstanceDataOffset = EnumHasAnyFlags(InstanceFlags, EInstanceFlags::DynamicInstanceDataOffset);
	const bool bPreserveInstanceOrder = EnumHasAnyFlags(InstanceFlags, EInstanceFlags::PreserveInstanceOrder);
	const bool bForceInstanceCulling = EnumHasAnyFlags(InstanceFlags, EInstanceFlags::ForceInstanceCulling);	

	uint32 Payload = (bDynamicInstanceDataOffset ? INSTANCE_CULLING_DYNAMIC_INSTANCE_DATA_OFFSET_BIT_MASK : 0U);
	if (bPreserveInstanceOrder)
	{
		checkSlow(!EnumHasAnyFlags(Flags, EInstanceCullingFlags::NoInstanceOrderPreservation)); // this should have already been handled

		// We need to provide full payload data for these instances
		// NOTE: The extended payload data flag is in the lowest bit instead of the highest because the payload is not a full dword, see FInstanceCullingLoadBalancerBase::PackItem
		Payload |= (INSTANCE_CULLING_PRESERVE_INSTANCE_ORDER_BIT_MASK | (uint32(PayloadData.Num()) << INSTANCE_CULLING_PAYLOAD_NUM_COMMON_BITS));
		PayloadData.Emplace(bDynamicInstanceDataOffset, IndirectArgsOffset, InstanceDataOffset, RunOffset, DrawCommandCompactionData.Num());
	}
	else
	{
		// Conserve space by packing the relevant payload information into the dword
		Payload |= (IndirectArgsOffset << INSTANCE_CULLING_PAYLOAD_NUM_COMMON_BITS);
	}

	// We special-case the single-instance (i.e., regular primitives) as they don't need culling (again), except where explicitly specified.
	// In actual fact this is not 100% true because dynamic path primitives may not have been culled.
	EBatchProcessingMode Mode = (NumInstances == 1 && !bForceInstanceCulling) ? SingleInstanceProcessingMode : EBatchProcessingMode::Generic;
	LoadBalancers[uint32(Mode)]->Add(uint32(InstanceDataOffset), NumInstances, Payload);
	TotalInstances += NumInstances;
}

void FInstanceCullingContext::AddInstancesToDrawCommand(uint32 IndirectArgsOffset, int32 InstanceDataOffset, uint32 RunOffset, uint32 NumInstances, EInstanceFlags InstanceFlags, uint32 MaxBatchSize)
{
	// Batching is disabled or first run of instances fit into batch size
	if (MaxBatchSize == MAX_uint32 || (NumInstances <= MaxBatchSize && RunOffset == 0))
	{
		AddInstancesToDrawCommand(IndirectArgsOffset, InstanceDataOffset, RunOffset, NumInstances, InstanceFlags);
		return;
	}
			
	// In case we are adding more than one instance run 
	// we will need to append instances to a last batch until its full
	if (RunOffset > 0 && NumInstances > 0)
	{
		uint32 NumInstancesInBatch = RunOffset % MaxBatchSize;
		if (NumInstancesInBatch > 0)
		{
			NumInstancesInBatch = FMath::Min(MaxBatchSize - NumInstancesInBatch, NumInstances);
			// appending to a last batch
			IndirectArgsOffset = (IndirectArgs.Num() - 1);
			AddInstancesToDrawCommand(IndirectArgsOffset, InstanceDataOffset, RunOffset, NumInstancesInBatch, InstanceFlags);
			InstanceDataOffset += NumInstancesInBatch;
			NumInstances -= NumInstancesInBatch;
		}
	}

	// Split rest of the instances into batches
	if (NumInstances > 0)
	{
		uint32 NumBatches = FMath::DivideAndRoundUp(NumInstances, MaxBatchSize);
		FMeshDrawCommandInfo& RESTRICT DrawCmd = MeshDrawCommandInfos.Last();
		FRHIDrawIndexedIndirectParameters LastIndirectArgs = IndirectArgs.Last();
		FUintVector2 LastCommandDesc = DrawCommandDescs.Last();
		uint32 NumViews = ViewIds.Num();

		for (uint32 BatchIdx = 0; BatchIdx < NumBatches; BatchIdx++)
		{
			uint32 NumInstancesInBatch = FMath::Min(MaxBatchSize, NumInstances);

			if (RunOffset > 0 || BatchIdx != 0)
			{
				DrawCommandDescs.Add(LastCommandDesc);
				IndirectArgsOffset = IndirectArgs.Add(LastIndirectArgs);
				InstanceIdOffsets.Add(GetInstanceIdNumElements());
				DrawCmd.NumBatches++;
			}

			AddInstancesToDrawCommand(IndirectArgsOffset, InstanceDataOffset, RunOffset, NumInstancesInBatch, InstanceFlags);
			InstanceDataOffset += NumInstancesInBatch;
			NumInstances -= NumInstancesInBatch;
		}
	}
}

void FInstanceCullingContext::AddInstanceRunsToDrawCommand(uint32 IndirectArgsOffset, int32 InstanceDataOffset, const uint32* Runs, uint32 NumRuns, EInstanceFlags InstanceFlags, uint32 MaxBatchSize)
{
	// Add items to current generic batch as they are instanced for sure.
	uint32 NumInstancesInRuns = 0;
	for (uint32 Index = 0; Index < NumRuns; ++Index)
	{
		uint32 RunStart = Runs[Index * 2];
		uint32 RunEndIncl = Runs[Index * 2 + 1];
		uint32 NumInstances = (RunEndIncl + 1U) - RunStart;
		AddInstancesToDrawCommand(IndirectArgsOffset, InstanceDataOffset + RunStart, NumInstancesInRuns, NumInstances, InstanceFlags | EInstanceFlags::ForceInstanceCulling, MaxBatchSize);
		NumInstancesInRuns += NumInstances;
	}
}


// Base class that provides common functionality between all compaction phases
class FCompactVisibleInstancesBaseCs : public FGlobalShader
{
public:
	/** A compaction block is a group of instance IDs sized (N * NumViews). This is N. */
	static constexpr int32 CompactionBlockNumInstances = 64;

	FCompactVisibleInstancesBaseCs() = default;
	FCompactVisibleInstancesBaseCs(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Currently compaction isn't supported on mobile
		return UseGPUScene(Parameters.Platform) && GetMaxSupportedFeatureLevel(Parameters.Platform) > ERHIFeatureLevel::ES3_1;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("COMPACTION_BLOCK_NUM_INSTANCES"), CompactionBlockNumInstances);
		OutEnvironment.SetDefine(TEXT("INDIRECT_ARGS_NUM_WORDS"), FInstanceCullingContext::IndirectArgsNumWords);
	}
};

// Compaction shader for phase one - calculate instance offsets for each instance compaction "block"
class FCalculateCompactBlockInstanceOffsetsCs final : public FCompactVisibleInstancesBaseCs
{
	DECLARE_GLOBAL_SHADER(FCalculateCompactBlockInstanceOffsetsCs);
	SHADER_USE_PARAMETER_STRUCT(FCalculateCompactBlockInstanceOffsetsCs, FCompactVisibleInstancesBaseCs)

public:
	static constexpr int32 NumThreadsPerGroup = 512;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FCompactVisibleInstancesBaseCs::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("CALCULATE_COMPACT_BLOCK_INSTANCE_OFFSETS"), 1);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FInstanceCullingContext::FCompactionData>, DrawCommandCompactionData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, BlockInstanceCounts)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint32>, BlockDestInstanceOffsetsOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint32>, DrawIndirectArgsBufferOut)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCalculateCompactBlockInstanceOffsetsCs, "/Engine/Private/InstanceCulling/CompactVisibleInstances.usf", "CalculateCompactBlockInstanceOffsetsCS", SF_Compute);

// Compaction shader for phase two - output visible instances, compacted and in original draw order
class FCompactVisibleInstancesCs final : public FCompactVisibleInstancesBaseCs
{
	DECLARE_GLOBAL_SHADER(FCompactVisibleInstancesCs);
	SHADER_USE_PARAMETER_STRUCT(FCompactVisibleInstancesCs, FCompactVisibleInstancesBaseCs)

public:
	static constexpr int32 NumThreadsPerGroup = FCompactVisibleInstancesBaseCs::CompactionBlockNumInstances;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FCompactVisibleInstancesBaseCs::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("COMPACT_VISIBLE_INSTANCES"), 1);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FInstanceCullingContext::FCompactionData>, DrawCommandCompactionData)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, BlockDrawCommandIndices)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, InstanceIdsBufferIn)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, BlockDestInstanceOffsets)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint32>, InstanceIdsBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, InstanceIdsBufferOutMobile)		
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FCompactVisibleInstancesCs, "/Engine/Private/InstanceCulling/CompactVisibleInstances.usf", "CompactVisibleInstances", SF_Compute);

class FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs);
	SHADER_USE_PARAMETER_STRUCT(FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs, FGlobalShader)

public:
	static constexpr int32 NumThreadsPerGroup = FInstanceProcessingGPULoadBalancer::ThreadGroupSize;

	// GPUCULL_TODO: remove once buffer is somehow unified
	class FSingleInstanceModeDim : SHADER_PERMUTATION_BOOL("SINGLE_INSTANCE_MODE");
	class FCullInstancesDim : SHADER_PERMUTATION_BOOL("CULL_INSTANCES");
	class FAllowWPODisableDim : SHADER_PERMUTATION_BOOL("ALLOW_WPO_DISABLE");
	class FOcclusionCullInstancesDim : SHADER_PERMUTATION_BOOL("OCCLUSION_CULL_INSTANCES");
	class FStereoModeDim : SHADER_PERMUTATION_BOOL("STEREO_CULLING_MODE");
	class FBatchedDim : SHADER_PERMUTATION_BOOL("ENABLE_BATCH_MODE");
	class FInstanceCompactionDim : SHADER_PERMUTATION_BOOL("ENABLE_INSTANCE_COMPACTION");

	using FPermutationDomain = TShaderPermutationDomain<FSingleInstanceModeDim, FCullInstancesDim, FAllowWPODisableDim, FOcclusionCullInstancesDim, FStereoModeDim, FBatchedDim, FInstanceCompactionDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!UseGPUScene(Parameters.Platform))
		{
			return false;
		}
		
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		
		// Currently, instance compaction is not supported on mobile platforms
		if (PermutationVector.Get<FInstanceCompactionDim>() && IsMobilePlatform(Parameters.Platform))
		{
			return false;
		}

		// Current behavior is that instance culling coerces the WPO disable distance check, so don't compile permutations
		// that include the former and exclude the latter
		if (PermutationVector.Get<FCullInstancesDim>() && !PermutationVector.Get<FAllowWPODisableDim>())
		{
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FInstanceProcessingGPULoadBalancer::SetShaderDefines(OutEnvironment);

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);

		OutEnvironment.SetDefine(TEXT("INDIRECT_ARGS_NUM_WORDS"), FInstanceCullingContext::IndirectArgsNumWords);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_LIGHTMAP_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("NANITE_MULTI_VIEW"), 1);
		OutEnvironment.SetDefine(TEXT("PRIM_ID_DYNAMIC_FLAG"), GPrimIDDynamicFlag);
		OutEnvironment.SetDefine(TEXT("COMPACTION_BLOCK_NUM_INSTANCES"), FCompactVisibleInstancesBaseCs::CompactionBlockNumInstances);

		OutEnvironment.SetDefine(TEXT("BATCH_PROCESSING_MODE_GENERIC"), uint32(EBatchProcessingMode::Generic));
		OutEnvironment.SetDefine(TEXT("BATCH_PROCESSING_MODE_UNCULLED"), uint32(EBatchProcessingMode::UnCulled));
		OutEnvironment.SetDefine(TEXT("BATCH_PROCESSING_MODE_NUM"), uint32(EBatchProcessingMode::Num));
		
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		EBatchProcessingMode ProcessingMode = (PermutationVector.Get<FSingleInstanceModeDim>() ? EBatchProcessingMode::UnCulled : EBatchProcessingMode::Generic);
		OutEnvironment.SetDefine(TEXT("INSTANCE_DATA_STRIDE_ELEMENTS"), GetInstanceDataStrideElements(Parameters.Platform, ProcessingMode));

		OutEnvironment.SetDefine(TEXT("PRIMITIVE_HAS_TILEOFFSET_DATA"), GPrimitiveHasTileOffsetData);

		DynamicMeshBoundsModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUSceneParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceProcessingGPULoadBalancer::FShaderParameters, LoadBalancerParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(RendererViewData::FCullingShaderParameters, ViewDataCullingParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDynamicMeshBoundsShaderParameters, DynamicBoundsParameters )

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FUintVector2 >, DrawCommandDescs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FInstanceCullingContext::FPayloadData >, InstanceCullingPayloads)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint32 >, ViewIds)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FContextBatchInfo >, BatchInfos)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< uint32 >, BatchInds)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, InstanceIdOffsetBuffer)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, InstanceOcclusionQueryBuffer)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, InstanceIdsBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, InstanceIdsBufferOutMobile)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, DrawIndirectArgsBufferOut)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FInstanceCullingContext::FCompactionData>, DrawCommandCompactionData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, CompactInstanceIdsBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, CompactionBlockCounts)


		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HZBTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, HZBSampler)
		SHADER_PARAMETER(FVector2f, HZBSize)

		SHADER_PARAMETER(uint32, NumViewIds)
		SHADER_PARAMETER(uint32, CurrentBatchProcessingMode)

		SHADER_PARAMETER(int32, DynamicInstanceIdOffset)
		SHADER_PARAMETER(int32, DynamicInstanceIdMax)
		SHADER_PARAMETER(uint32, bDrawOnlySelected)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs, "/Engine/Private/InstanceCulling/BuildInstanceDrawCommands.usf", "InstanceCullBuildInstanceIdBufferCS", SF_Compute);

const TRDGUniformBufferRef<FInstanceCullingGlobalUniforms> FInstanceCullingContext::CreateDummyInstanceCullingUniformBuffer(FRDGBuilder& GraphBuilder)
{
	FInstanceCullingGlobalUniforms* InstanceCullingGlobalUniforms = GraphBuilder.AllocParameters<FInstanceCullingGlobalUniforms>();
	FRDGBufferRef DummyBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, 4);
	InstanceCullingGlobalUniforms->InstanceIdsBuffer = GraphBuilder.CreateSRV(DummyBuffer);
	InstanceCullingGlobalUniforms->PageInfoBuffer = GraphBuilder.CreateSRV(DummyBuffer);
	InstanceCullingGlobalUniforms->BufferCapacity = 0;
	return GraphBuilder.CreateUniformBuffer(InstanceCullingGlobalUniforms);
}


class FInstanceCullingDeferredContext : public FInstanceCullingMergedContext
{
public:
	FInstanceCullingDeferredContext(EShaderPlatform InShaderPlatform, FInstanceCullingManager* InInstanceCullingManager = nullptr, int32 InNumBins=2)
		: FInstanceCullingMergedContext(InShaderPlatform, false, InNumBins)
		, InstanceCullingManager(InInstanceCullingManager)
	{}

	FInstanceCullingManager* InstanceCullingManager;

	FRDGBufferRef DrawIndirectArgsBuffer = nullptr;
	FRDGBufferRef InstanceDataBuffer = nullptr;
	TRDGUniformBufferRef<FInstanceCullingGlobalUniforms> UniformBuffer = nullptr;
	TRDGUniformBufferRef<FBatchedPrimitiveParameters> BatchedPrimitive = nullptr;

	bool bProcessed = false;

	void ProcessBatched(TArray<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FParameters*> PassParameters);

#if MESH_DRAW_COMMAND_STATS
	FRHIGPUBufferReadback* MeshDrawCommandStatsIndirectArgsReadbackBuffer = nullptr;
#endif
};

static uint32 GetInstanceIdBufferSize(EShaderPlatform ShaderPlatform, uint32 NumInstanceElements)
{
	if (PlatformGPUSceneUsesUniformBufferView(ShaderPlatform))
	{
		// Add an additional max range slack to a buffer size, so when binding last element we still have a full UBO range
		NumInstanceElements += (PLATFORM_MAX_UNIFORM_BUFFER_RANGE / 16u);
		return NumInstanceElements;
	}
	else
	{
		// Desktop uses StructuredBuffer<uint> NumElements==NumInstances
		return NumInstanceElements;
	}
}

static FRDGBufferDesc CreateInstanceIdBufferDesc(EShaderPlatform ShaderPlatform, uint32 NumInstanceElements)
{
	if (PlatformGPUSceneUsesUniformBufferView(ShaderPlatform))
	{
		// float4
		FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredDesc(16u, NumInstanceElements);
		Desc.Usage |= EBufferUsageFlags::UniformBuffer;
		return Desc;
	}
	else
	{
		return FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumInstanceElements);
	}
}

void FInstanceCullingContext::BuildRenderingCommands(
	FRDGBuilder& GraphBuilder,
	const FGPUScene& GPUScene,
	int32 InDynamicInstanceIdOffset,
	int32 InDynamicInstanceIdNum,
	FInstanceCullingResult& Results)
{
	check(!SyncPrerequisitesFunc);
	Results = FInstanceCullingResult();
	SetDynamicPrimitiveInstanceOffsets(InDynamicInstanceIdOffset, InDynamicInstanceIdNum);
	BuildRenderingCommandsInternal(GraphBuilder, GPUScene, EAsyncProcessingMode::Synchronous, &Results.Parameters);
}


void FInstanceCullingContext::BuildRenderingCommands(FRDGBuilder& GraphBuilder, const FGPUScene& GPUScene, FInstanceCullingDrawParams* InstanceCullingDrawParams)
{
	BuildRenderingCommandsInternal(GraphBuilder, GPUScene, EAsyncProcessingMode::DeferredOrAsync, InstanceCullingDrawParams);
}

bool FInstanceCullingContext::HasCullingCommands() const
{
	check(!SyncPrerequisitesFunc);  
	return bHasAnyIndirectDraw || TotalInstances > 0u;
}

void FInstanceCullingContext::BuildRenderingCommandsInternal(
	FRDGBuilder& GraphBuilder,
	const FGPUScene& GPUScene,
	EAsyncProcessingMode AsyncProcessingMode,
	FInstanceCullingDrawParams* InstanceCullingDrawParams)
{
#if MESH_DRAW_COMMAND_STATS
	if (MeshDrawCommandPassStats)
	{
		check(!MeshDrawCommandPassStats->bBuildRenderingCommandsCalled);
		MeshDrawCommandPassStats->bBuildRenderingCommandsCalled = true;
	}
#endif

	check(InstanceCullingDrawParams);
	FMemory::Memzero(*InstanceCullingDrawParams);

	if (InstanceCullingManager)
	{
		InstanceCullingDrawParams->Scene = InstanceCullingManager->SceneUniforms.GetBuffer(GraphBuilder);
	}

	if (AsyncProcessingMode != EAsyncProcessingMode::Synchronous && InstanceCullingManager && InstanceCullingManager->IsDeferredCullingActive() && (InstanceCullingMode == EInstanceCullingMode::Normal))
	{
		FInstanceCullingDeferredContext *DeferredContext = InstanceCullingManager->DeferredContext;

		// If this is true, then RDG Execute or Drain has been called, and no further contexts can be deferred. 
		if (!DeferredContext->bProcessed)
		{
			InstanceCullingDrawParams->DrawIndirectArgsBuffer = DeferredContext->DrawIndirectArgsBuffer;
			InstanceCullingDrawParams->InstanceIdOffsetBuffer = DeferredContext->InstanceDataBuffer;
			InstanceCullingDrawParams->InstanceCulling = DeferredContext->UniformBuffer;
			InstanceCullingDrawParams->BatchedPrimitive = DeferredContext->BatchedPrimitive;
			DeferredContext->AddBatch(GraphBuilder, this, InstanceCullingDrawParams);
		}
		return;
	}
	WaitForSetupTask();

	if (!HasCullingCommands())
	{
		if (InstanceCullingManager)
		{
			InstanceCullingDrawParams->InstanceCulling = InstanceCullingManager->GetDummyInstanceCullingUniformBuffer();
		}
		return;
	}

	check(DynamicInstanceIdOffset >= 0);
	check(DynamicInstanceIdNum >= 0);

	ensure(InstanceCullingMode == EInstanceCullingMode::Normal || ViewIds.Num() == 2);

	// If there is no manager, then there is no data on culling, so set flag to skip that and ignore buffers.
	const bool bCullInstances = InstanceCullingManager != nullptr && CVarCullInstances.GetValueOnRenderThread() != 0;
	const bool bAllowWPODisable = InstanceCullingManager != nullptr;

	RDG_EVENT_SCOPE_STAT(GraphBuilder, BuildRenderingCommands, "BuildRenderingCommands(Culling=%s)", bCullInstances ? TEXT("On") : TEXT("Off"));

	const bool bOrderPreservationEnabled = IsInstanceOrderPreservationEnabled();
	const uint32 NumCompactionBlocks = uint32(CompactionBlockDataIndices.Num());
	FRDGBufferRef CompactInstanceIdsBuffer = nullptr;
	FRDGBufferUAVRef CompactInstanceIdsUAV = nullptr;
	FRDGBufferRef CompactionBlockCountsBuffer = nullptr;
	FRDGBufferUAVRef CompactionBlockCountsUAV = nullptr;
	FRDGBufferSRVRef DrawCommandCompactionDataSRV = nullptr;	
	
	if (bOrderPreservationEnabled)
	{
		// Create buffers for compacting instances for draw commands that need it
		CompactInstanceIdsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::Max(NumCompactionInstances, 1u)), TEXT("InstanceCulling.Compaction.TempInstanceIdsBuffer"));
		CompactInstanceIdsUAV = GraphBuilder.CreateUAV(CompactInstanceIdsBuffer);
		CompactionBlockCountsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), FMath::Max(NumCompactionBlocks, 1u)), TEXT("InstanceCulling.Compaction.BlockInstanceCounts"));
		CompactionBlockCountsUAV = GraphBuilder.CreateUAV(CompactionBlockCountsBuffer);

		FRDGBufferRef DrawCommandCompactionDataBuffer = nullptr;
		if (DrawCommandCompactionData.Num() > 0)
		{
			DrawCommandCompactionDataBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCulling.DrawCommandCompactionData"), DrawCommandCompactionData);
		}
		else
		{
			DrawCommandCompactionDataBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FCompactionData));
		}
		DrawCommandCompactionDataSRV = GraphBuilder.CreateSRV(DrawCommandCompactionDataBuffer);

		if (NumCompactionBlocks > 0)
		{
			ensure(NumCompactionInstances > 0);

			// We must clear the block counts buffer, as it will be written to using atomic increments
			AddClearUAVPass(GraphBuilder, CompactionBlockCountsUAV, 0);
		}
	}

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(ShaderPlatform);

	FRDGBufferRef ViewIdsBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCulling.ViewIds"), ViewIds);

	const uint32 InstanceIdBufferSize = GetInstanceIdBufferSize(ShaderPlatform, GetInstanceIdNumElements());
	FRDGBufferRef InstanceIdsBuffer = GraphBuilder.CreateBuffer(CreateInstanceIdBufferDesc(ShaderPlatform, InstanceIdBufferSize), TEXT("InstanceCulling.InstanceIdsBuffer"));
	FRDGBufferUAVRef InstanceIdsBufferUAV = GraphBuilder.CreateUAV(InstanceIdsBuffer, ERDGUnorderedAccessViewFlags::SkipBarrier);

	FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FParameters PassParametersTmp;

	PassParametersTmp.DrawCommandDescs = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCulling.DrawCommandDescs"), DrawCommandDescs));

	PassParametersTmp.InstanceCullingPayloads = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCulling.PayloadData"), PayloadData));

	// NOTE: because it is possible to not have access to the SceneUB (for convoluted reasons), we bind the scene data directly, 
	// but to get access to the other facilities when available, we also bind the scene UB). This works because culling is disabled when InstanceCullingManager is not available. 
	PassParametersTmp.Scene = InstanceCullingManager ? InstanceCullingManager->SceneUniforms.GetBuffer(GraphBuilder) : nullptr;
	PassParametersTmp.GPUSceneParameters = GPUScene.GetShaderParameters(GraphBuilder);
	PassParametersTmp.DynamicInstanceIdOffset = DynamicInstanceIdOffset;
	PassParametersTmp.DynamicInstanceIdMax = DynamicInstanceIdOffset + DynamicInstanceIdNum;
	PassParametersTmp.bDrawOnlySelected = EnumHasAnyFlags(Flags, EInstanceCullingFlags::DrawOnlySelected);

	// Compaction parameters
	PassParametersTmp.DrawCommandCompactionData = DrawCommandCompactionDataSRV;
	PassParametersTmp.CompactInstanceIdsBufferOut = CompactInstanceIdsUAV;
	PassParametersTmp.CompactionBlockCounts = CompactionBlockCountsUAV;

	// Create buffer for indirect args and upload draw arg data, also clears the instance to zero
	FRDGBufferDesc IndirectArgsDesc = FRDGBufferDesc::CreateIndirectDesc(IndirectArgsNumWords * IndirectArgs.Num());
	IndirectArgsDesc.Usage = EBufferUsageFlags(IndirectArgsDesc.Usage | BUF_MultiGPUGraphIgnore);

	FRDGBufferRef DrawIndirectArgsRDG = GraphBuilder.CreateBuffer(IndirectArgsDesc, TEXT("InstanceCulling.DrawIndirectArgsBuffer"));
	GraphBuilder.QueueBufferUpload(DrawIndirectArgsRDG, IndirectArgs.GetData(), IndirectArgs.GetTypeSize() * IndirectArgs.Num());

	// Note: we redundantly clear the instance counts here as there is some issue with replays on certain consoles.
	AddClearIndirectArgInstanceCountPass(GraphBuilder, ShaderMap, DrawIndirectArgsRDG);

	// not using structured buffer as we have to get at it as a vertex buffer 
	FRDGBufferRef InstanceIdOffsetBufferRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), InstanceIdOffsets.Num()), TEXT("InstanceCulling.InstanceIdOffsetBuffer"));
	GraphBuilder.QueueBufferUpload(InstanceIdOffsetBufferRDG, InstanceIdOffsets.GetData(), InstanceIdOffsets.GetTypeSize() * InstanceIdOffsets.Num());

	PassParametersTmp.ViewIds = GraphBuilder.CreateSRV(ViewIdsBuffer);
	PassParametersTmp.ViewDataCullingParameters.NumCullingViews = 0;
	if ((bCullInstances || bAllowWPODisable) && InstanceCullingManager)
	{
		PassParametersTmp.ViewDataCullingParameters = InstanceCullingManager->ViewDataManager.GetCullingParameters(GraphBuilder);

#if DO_CHECK
		for (int32 ViewId : ViewIds)
		{
			checkf(ViewId < int32(PassParametersTmp.ViewDataCullingParameters.NumCullingViews), TEXT("Attempting to process a culling context that references a view that has not been uploaded yet."));
		}
#endif 
	}
	PassParametersTmp.DynamicBoundsParameters = GetDynamicMeshBoundsShaderParameters(GraphBuilder);
	PassParametersTmp.NumViewIds = ViewIds.Num();
	// only one of these will be used in the shader
	PassParametersTmp.InstanceIdsBufferOut = InstanceIdsBufferUAV;
	PassParametersTmp.InstanceIdsBufferOutMobile = InstanceIdsBufferUAV;

	PassParametersTmp.DrawIndirectArgsBufferOut = GraphBuilder.CreateUAV(DrawIndirectArgsRDG, PF_R32_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
	PassParametersTmp.InstanceIdOffsetBuffer = GraphBuilder.CreateSRV(InstanceIdOffsetBufferRDG, PF_R32_UINT);

	const bool bOcclusionCullInstances = PrevHZB.IsValid() && IsOcclusionCullingEnabled();
	if (bOcclusionCullInstances)
	{
		PassParametersTmp.HZBTexture = GraphBuilder.RegisterExternalTexture(PrevHZB);
		PassParametersTmp.HZBSize = PassParametersTmp.HZBTexture->Desc.Extent;
		PassParametersTmp.HZBSampler = TStaticSamplerState< SF_Point, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI();
	}

	if (InstanceCullingManager && InstanceCullingManager->InstanceOcclusionQueryBuffer)
	{
		PassParametersTmp.InstanceOcclusionQueryBuffer = GraphBuilder.CreateSRV(
			InstanceCullingManager->InstanceOcclusionQueryBuffer, 
			InstanceCullingManager->InstanceOcclusionQueryBufferFormat);
	}
	else
	{
		FRDGBufferRef DummyBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, 4, 0u);
		PassParametersTmp.InstanceOcclusionQueryBuffer = GraphBuilder.CreateSRV(DummyBuffer, PF_R32_UINT);
	}

	for (uint32 Mode = 0U; Mode < uint32(EBatchProcessingMode::Num); ++Mode)
	{
		FInstanceProcessingGPULoadBalancer* LoadBalancer = LoadBalancers[Mode];
		if (!LoadBalancer->IsEmpty())
		{
			FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FParameters>();
			*PassParameters = PassParametersTmp;
			// Upload data etc
			auto GPUData = LoadBalancer->Upload(GraphBuilder);
			GPUData.GetShaderParameters(GraphBuilder, PassParameters->LoadBalancerParameters);
			PassParameters->CurrentBatchProcessingMode = Mode;

			// UnCulled bucket is used for a single instance mode
			check(EBatchProcessingMode(Mode) != EBatchProcessingMode::UnCulled || LoadBalancer->HasSingleInstanceItemsOnly());

			FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FPermutationDomain PermutationVector;
			PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FSingleInstanceModeDim>(EBatchProcessingMode(Mode) == EBatchProcessingMode::UnCulled);
			PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FCullInstancesDim>(bCullInstances && EBatchProcessingMode(Mode) != EBatchProcessingMode::UnCulled);
			PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FAllowWPODisableDim>(bAllowWPODisable);
			PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FOcclusionCullInstancesDim>(bOcclusionCullInstances);
			PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FStereoModeDim>(InstanceCullingMode == EInstanceCullingMode::Stereo);
			PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FBatchedDim>(false);
			PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FInstanceCompactionDim>(bOrderPreservationEnabled);

			auto ComputeShader = ShaderMap->GetShader<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs>(PermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("CullInstances(%s)", BatchProcessingModeStr[Mode]),
				ComputeShader,
				PassParameters,
				LoadBalancer->GetWrappedCsGroupCount()
			);
		}
	}

	if (bOrderPreservationEnabled && NumCompactionBlocks > 0)
	{
		FRDGBufferRef BlockDestInstanceOffsets = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumCompactionBlocks), TEXT("InstanceCulling.Compaction.BlockDestInstanceOffsets"));

		// Compaction phase one - prefix sum of the compaction "blocks"
		{
			auto PassParameters = GraphBuilder.AllocParameters<FCalculateCompactBlockInstanceOffsetsCs::FParameters>();
			PassParameters->DrawCommandCompactionData = DrawCommandCompactionDataSRV;
			PassParameters->BlockInstanceCounts = GraphBuilder.CreateSRV(CompactionBlockCountsBuffer);
			PassParameters->BlockDestInstanceOffsetsOut = GraphBuilder.CreateUAV(BlockDestInstanceOffsets);
			PassParameters->DrawIndirectArgsBufferOut = PassParametersTmp.DrawIndirectArgsBufferOut;

			auto ComputeShader = ShaderMap->GetShader<FCalculateCompactBlockInstanceOffsetsCs>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("Instance Compaction Phase 1"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCountWrapped(DrawCommandCompactionData.Num())
			);
		}

		// Compaction phase two - write instances to compact final location
		{
			FRDGBufferRef BlockDrawCommandIndices = CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCulling.Compaction.BlockDrawCommandIndices"), CompactionBlockDataIndices);

			auto PassParameters = GraphBuilder.AllocParameters<FCompactVisibleInstancesCs::FParameters>();
			PassParameters->DrawCommandCompactionData = DrawCommandCompactionDataSRV;
			PassParameters->BlockDrawCommandIndices = GraphBuilder.CreateSRV(BlockDrawCommandIndices);
			PassParameters->InstanceIdsBufferIn = GraphBuilder.CreateSRV(CompactInstanceIdsBuffer);
			PassParameters->BlockDestInstanceOffsets = GraphBuilder.CreateSRV(BlockDestInstanceOffsets);			
			PassParameters->InstanceIdsBufferOut = InstanceIdsBufferUAV;
			PassParameters->InstanceIdsBufferOutMobile = InstanceIdsBufferUAV;			

			auto ComputeShader = ShaderMap->GetShader<FCompactVisibleInstancesCs>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("Instance Compaction Phase 2"),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCountWrapped(NumCompactionBlocks)
			);
		}
	}

	// If there were no instances & thus the compute passes were skipped we need to clear the buffer to signal to RDG that it is ok to bind it later.
	if (TotalInstances == 0u)
	{
		AddClearUAVPass(GraphBuilder, InstanceIdsBufferUAV, 0u);
	}

	InstanceCullingDrawParams->DrawIndirectArgsBuffer = DrawIndirectArgsRDG;
	InstanceCullingDrawParams->InstanceIdOffsetBuffer = InstanceIdOffsetBufferRDG;
	
	if (PlatformGPUSceneUsesUniformBufferView(ShaderPlatform))
	{
		FBatchedPrimitiveParameters* BatchedPrimitiveParameters = GraphBuilder.AllocParameters<FBatchedPrimitiveParameters>();
		BatchedPrimitiveParameters->Data = GraphBuilder.CreateSRV(InstanceIdsBuffer);
		InstanceCullingDrawParams->BatchedPrimitive = GraphBuilder.CreateUniformBuffer(BatchedPrimitiveParameters);
	}
	else
	{
		FInstanceCullingGlobalUniforms* UniformParameters = GraphBuilder.AllocParameters<FInstanceCullingGlobalUniforms>();
		UniformParameters->InstanceIdsBuffer = GraphBuilder.CreateSRV(InstanceIdsBuffer);
		UniformParameters->PageInfoBuffer = GraphBuilder.CreateSRV(InstanceIdsBuffer);
		UniformParameters->BufferCapacity = InstanceIdBufferSize;
		InstanceCullingDrawParams->InstanceCulling = GraphBuilder.CreateUniformBuffer(UniformParameters);
	}

#if MESH_DRAW_COMMAND_STATS
	if (MeshDrawCommandPassStats)
	{
		FRHIGPUBufferReadback* GPUBufferReadback = FMeshDrawCommandStatsManager::Get()->QueueDrawRDGIndirectArgsReadback(GraphBuilder, DrawIndirectArgsRDG);
		MeshDrawCommandPassStats->SetInstanceCullingGPUBufferReadback(GPUBufferReadback, 0);
	}
#endif
}

void FInstanceCullingDeferredContext::ProcessBatched(TArray<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FParameters*> PassParameters)
{
	if (bProcessed)
	{
		return;
	}

	MergeBatches();

#if MESH_DRAW_COMMAND_STATS
	// Setup the indirect buffer and correct offset for each pass in the merged buffer
	if (MeshDrawCommandStatsIndirectArgsReadbackBuffer)
	{
		for (int32 BatchIndex = 0; BatchIndex < Batches.Num(); ++BatchIndex)
		{
			const FBatchItem& BatchItem = Batches[BatchIndex];
			if (BatchItem.Context->MeshDrawCommandPassStats)
			{
				BatchItem.Context->MeshDrawCommandPassStats->SetInstanceCullingGPUBufferReadback(MeshDrawCommandStatsIndirectArgsReadbackBuffer, BatchInfos[BatchIndex].IndirectArgsOffset);
			}
		}
	}
#endif // MESH_DRAW_COMMAND_STATS

	bProcessed = true;

	// Finalize culling pass parameters
	for (uint32 BinIndex = 0U; BinIndex < uint32(LoadBalancers.Num()); ++BinIndex)
	{
		PassParameters[BinIndex]->NumViewIds = ViewIds.Num();
		PassParameters[BinIndex]->LoadBalancerParameters.NumBatches = LoadBalancers[BinIndex].GetBatches().Num();
		PassParameters[BinIndex]->LoadBalancerParameters.NumItems = LoadBalancers[BinIndex].GetItems().Num();
		PassParameters[BinIndex]->LoadBalancerParameters.NumGroupsPerBatch = 1;
	}
}

template <typename DataType>
FORCEINLINE int32 GetArrayDataSize(const TArrayView<const DataType>& Array)
{
	return Array.GetTypeSize() * Array.Num();
}

template <typename DataType, typename AllocatorType>
FORCEINLINE int32 GetArrayDataSize(const TArray<DataType, AllocatorType>& Array)
{
	return Array.GetTypeSize() * Array.Num();
}

FInstanceCullingDeferredContext *FInstanceCullingContext::CreateDeferredContext(
	FRDGBuilder& GraphBuilder,
	const FGPUScene& GPUScene,
	FInstanceCullingManager& InstanceCullingManager)
{
	// Bin 0 is reserved for UnCulled batches, every other bin is for each HZB. So at the very least we must have 2 bins.
	// Generic batches with a null HZB will go in bin 1, together with the ones associated to the first HZB.

	uint32 NumBins = FMath::Max(2, InstanceCullingManager.ViewPrevHZBs.Num() + 1);

#define INST_CULL_CALLBACK_BIN_INDEX(CustomCode) \
	[PassParameters, DeferredContext, BinIndex]() \
	{ \
		DeferredContext->ProcessBatched(PassParameters); \
		return CustomCode; \
	}

#define INST_CULL_CALLBACK(CustomCode) \
	[PassParameters, DeferredContext]() \
	{ \
		DeferredContext->ProcessBatched(PassParameters); \
		return CustomCode; \
	}

#define INST_CULL_CREATE_STRUCT_BUFF_ARGS(ArrayName) \
	GraphBuilder, \
	TEXT("InstanceCulling.") TEXT(#ArrayName), \
	DeferredContext->ArrayName.GetTypeSize(), \
	INST_CULL_CALLBACK(DeferredContext->ArrayName.Num()), \
	INST_CULL_CALLBACK(DeferredContext->ArrayName.GetData()), \
	INST_CULL_CALLBACK(DeferredContext->ArrayName.Num() * DeferredContext->ArrayName.GetTypeSize())

#define INST_CULL_CREATE_STRUCT_BUFF_ARGS_BIN_INDEX(ArrayName) \
	GraphBuilder, \
	TEXT("InstanceCulling.") TEXT(#ArrayName), \
	sizeof(DeferredContext->ArrayName[BinIndex][0]), \
	INST_CULL_CALLBACK_BIN_INDEX(DeferredContext->ArrayName[BinIndex].Num()), \
	INST_CULL_CALLBACK_BIN_INDEX(DeferredContext->ArrayName[BinIndex].GetData()), \
	INST_CULL_CALLBACK_BIN_INDEX(DeferredContext->ArrayName[BinIndex].Num() * sizeof(DeferredContext->ArrayName[BinIndex][0]))

	const EShaderPlatform ShaderPlatform = GPUScene.GetShaderPlatform();

	FInstanceCullingDeferredContext* DeferredContext = GraphBuilder.AllocObject<FInstanceCullingDeferredContext>(ShaderPlatform, &InstanceCullingManager, NumBins);

	const bool bCullInstances = CVarCullInstances.GetValueOnRenderThread() != 0;
	const bool bAllowWPODisable = true;

	RDG_EVENT_SCOPE_STAT(GraphBuilder, BuildRenderingCommandsDeferred, "BuildRenderingCommandsDeferred(Culling=%s)", bCullInstances ? TEXT("On") : TEXT("Off"));

	TArray<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FParameters*> PassParameters;
	PassParameters.SetNum(NumBins);
	for (uint32 BinIndex = 0U; BinIndex < NumBins; ++BinIndex)
	{
		PassParameters[BinIndex] = GraphBuilder.AllocParameters<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FParameters>();
	}

	// Create buffers for compacting instances for draw commands that need it
	const bool bEnableInstanceCompaction = IsInstanceOrderPreservationAllowed(ShaderPlatform);
	FRDGBufferSRVRef DrawCommandCompactionDataSRV = nullptr;
	FRDGBufferRef CompactInstanceIdsBuffer = nullptr;
	FRDGBufferUAVRef CompactInstanceIdsUAV = nullptr;
	FRDGBufferRef CompactionBlockCountsBuffer = nullptr;
	FRDGBufferUAVRef CompactionBlockCountsUAV = nullptr;

	if (bEnableInstanceCompaction)
	{
		DrawCommandCompactionDataSRV = GraphBuilder.CreateSRV(CreateStructuredBuffer(INST_CULL_CREATE_STRUCT_BUFF_ARGS(DrawCommandCompactionData)));
		CompactInstanceIdsBuffer = CreateStructuredBuffer(
			GraphBuilder,
			TEXT("InstanceCulling.Compaction.TempInstanceIdsBuffer"),
			sizeof(uint32),
			INST_CULL_CALLBACK(FMath::Max(DeferredContext->TotalCompactionInstances, 1)),
			INST_CULL_CALLBACK(nullptr),
			INST_CULL_CALLBACK(0));
		CompactInstanceIdsUAV = GraphBuilder.CreateUAV(CompactInstanceIdsBuffer);
		CompactionBlockCountsBuffer = CreateStructuredBuffer(
			GraphBuilder,
			TEXT("InstanceCulling.Compaction.BlockInstanceCounts"),
			sizeof(uint32),
			INST_CULL_CALLBACK(FMath::Max(DeferredContext->TotalCompactionBlocks, 1)),
			INST_CULL_CALLBACK(nullptr),
			INST_CULL_CALLBACK(0));
		CompactionBlockCountsUAV = GraphBuilder.CreateUAV(CompactionBlockCountsBuffer);

		// We must clear the block counts buffer, as they will be written to using atomic increments
		// TODO: Come up with a clever way to cull this pass when no compaction is needed (currently can't know until the batch is complete on the RDG execution timeline).
		AddClearUAVPass(GraphBuilder, CompactionBlockCountsUAV, 0);
	}

	FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FParameters PassParametersTmp = {};

	FRDGBufferRef DrawCommandDescsRDG = CreateStructuredBuffer(INST_CULL_CREATE_STRUCT_BUFF_ARGS(DrawCommandDescs));
	FRDGBufferRef InstanceCullingPayloadsRDG = CreateStructuredBuffer(INST_CULL_CREATE_STRUCT_BUFF_ARGS(PayloadData));
	FRDGBufferRef ViewIdsRDG = CreateStructuredBuffer(INST_CULL_CREATE_STRUCT_BUFF_ARGS(ViewIds));
	FRDGBufferRef BatchInfosRDG = CreateStructuredBuffer(INST_CULL_CREATE_STRUCT_BUFF_ARGS(BatchInfos));

	DeferredContext->DrawIndirectArgsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(), TEXT("InstanceCulling.DrawIndirectArgsBuffer"), INST_CULL_CALLBACK(IndirectArgsNumWords * DeferredContext->IndirectArgs.Num()));
	GraphBuilder.QueueBufferUpload(DeferredContext->DrawIndirectArgsBuffer, INST_CULL_CALLBACK(DeferredContext->IndirectArgs.GetData()), INST_CULL_CALLBACK(GetArrayDataSize(DeferredContext->IndirectArgs)));

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(ShaderPlatform);

	// Note: we redundantly clear the instance counts here as there is some issue with replays on certain consoles.
	AddClearIndirectArgInstanceCountPass(GraphBuilder, ShaderMap, DeferredContext->DrawIndirectArgsBuffer, INST_CULL_CALLBACK(DeferredContext->IndirectArgs.Num()));

	// not using structured buffer as we want/have to get at it as a vertex buffer 
	FRDGBufferRef InstanceIdOffsetBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1), TEXT("InstanceCulling.InstanceIdOffsetBuffer"), INST_CULL_CALLBACK(DeferredContext->InstanceIdOffsets.Num()));
	GraphBuilder.QueueBufferUpload(InstanceIdOffsetBuffer, INST_CULL_CALLBACK(DeferredContext->InstanceIdOffsets.GetData()), INST_CULL_CALLBACK(DeferredContext->InstanceIdOffsets.GetTypeSize() * DeferredContext->InstanceIdOffsets.Num()));

	FRDGBufferRef InstanceIdsBuffer = GraphBuilder.CreateBuffer(
			CreateInstanceIdBufferDesc(ShaderPlatform, 1), 
			TEXT("InstanceCulling.InstanceIdsBuffer"), 
			INST_CULL_CALLBACK(GetInstanceIdBufferSize(DeferredContext->ShaderPlatform, DeferredContext->InstanceIdBufferElements))
	);
	FRDGBufferUAVRef InstanceIdsBufferUAV = GraphBuilder.CreateUAV(InstanceIdsBuffer, ERDGUnorderedAccessViewFlags::SkipBarrier);
	DeferredContext->InstanceDataBuffer = InstanceIdOffsetBuffer;

	// Because the view uniforms are not set up by the time this runs
	// PassParameters->View = View.ViewUniformBuffer;
	// Set up global GPU-scene data instead...
	// NOTE: because it is possible in the non-deferred path to not have access to the SceneUB (for convoluted reasons), we bind the scene data directly, 
	// but to get access to the other facilities when available, we also bind the scene UB). This works because culling is disabled when InstanceCullingManager is not available. 
	PassParametersTmp.Scene = InstanceCullingManager.SceneUniforms.GetBuffer(GraphBuilder);
	PassParametersTmp.GPUSceneParameters = GPUScene.GetShaderParameters(GraphBuilder);

	PassParametersTmp.DrawCommandDescs = GraphBuilder.CreateSRV(DrawCommandDescsRDG);
	PassParametersTmp.InstanceCullingPayloads = GraphBuilder.CreateSRV(InstanceCullingPayloadsRDG);
	PassParametersTmp.BatchInfos = GraphBuilder.CreateSRV(BatchInfosRDG);
	PassParametersTmp.ViewIds = GraphBuilder.CreateSRV(ViewIdsRDG);
	// only one of these will be used in the shader
	PassParametersTmp.InstanceIdsBufferOut = InstanceIdsBufferUAV;
	PassParametersTmp.InstanceIdsBufferOutMobile = InstanceIdsBufferUAV;

	PassParametersTmp.DrawIndirectArgsBufferOut = GraphBuilder.CreateUAV(DeferredContext->DrawIndirectArgsBuffer, PF_R32_UINT, ERDGUnorderedAccessViewFlags::SkipBarrier);
	PassParametersTmp.InstanceIdOffsetBuffer = GraphBuilder.CreateSRV(InstanceIdOffsetBuffer, PF_R32_UINT);	
	if (bCullInstances || bAllowWPODisable)
	{
		PassParametersTmp.ViewDataCullingParameters = InstanceCullingManager.ViewDataManager.GetCullingParameters(GraphBuilder);
	}
	PassParametersTmp.DynamicBoundsParameters = GetDynamicMeshBoundsShaderParameters(GraphBuilder);

	// Compaction parameters
	PassParametersTmp.DrawCommandCompactionData = DrawCommandCompactionDataSRV;
	PassParametersTmp.CompactInstanceIdsBufferOut = CompactInstanceIdsUAV;
	PassParametersTmp.CompactionBlockCounts = CompactionBlockCountsUAV;

	if (InstanceCullingManager.InstanceOcclusionQueryBuffer)
	{
		PassParametersTmp.InstanceOcclusionQueryBuffer = GraphBuilder.CreateSRV(
			InstanceCullingManager.InstanceOcclusionQueryBuffer,
			InstanceCullingManager.InstanceOcclusionQueryBufferFormat);
	}
	else
	{
		FRDGBufferRef DummyBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, 4, 0u);
		PassParametersTmp.InstanceOcclusionQueryBuffer = GraphBuilder.CreateSRV(DummyBuffer, PF_R32_UINT);
	}

	// Record the number of culling views to be able to check that no views referencing out-of bounds views are queued up
	DeferredContext->NumCullingViews = InstanceCullingManager.ViewDataManager.GetNumCullingViews();

	for (uint32 BinIndex = 0U; BinIndex < NumBins; ++BinIndex)
	{
		*PassParameters[BinIndex] = PassParametersTmp;

		FRDGBufferRef BatchIndsRDG = CreateStructuredBuffer(INST_CULL_CREATE_STRUCT_BUFF_ARGS_BIN_INDEX(BatchInds));
		PassParameters[BinIndex]->BatchInds = GraphBuilder.CreateSRV(BatchIndsRDG);

		// The first bin (0) is used for the EBatchProcessingMode::UnCulled batches
		EBatchProcessingMode CurrentBatchProcessingMode = (BinIndex == 0) ? EBatchProcessingMode::UnCulled : EBatchProcessingMode::Generic;

		FInstanceProcessingGPULoadBalancer::FGPUData Result;
		FRDGBufferRef BatchBuffer = CreateStructuredBuffer(
			GraphBuilder,
			TEXT("InstanceCullingLoadBalancer.Batches"),
			sizeof(FInstanceProcessingGPULoadBalancer::FPackedBatch),
			INST_CULL_CALLBACK_BIN_INDEX(DeferredContext->LoadBalancers[BinIndex].GetBatches().Num()),
			INST_CULL_CALLBACK_BIN_INDEX(DeferredContext->LoadBalancers[BinIndex].GetBatches().GetData()),
			INST_CULL_CALLBACK_BIN_INDEX(GetArrayDataSize(DeferredContext->LoadBalancers[BinIndex].GetBatches())));

		FRDGBufferRef ItemBuffer = CreateStructuredBuffer(
			GraphBuilder,
			TEXT("InstanceCullingLoadBalancer.Items"),
			sizeof(FInstanceProcessingGPULoadBalancer::FPackedItem),
			INST_CULL_CALLBACK_BIN_INDEX(DeferredContext->LoadBalancers[BinIndex].GetItems().Num()),
			INST_CULL_CALLBACK_BIN_INDEX(DeferredContext->LoadBalancers[BinIndex].GetItems().GetData()),
			INST_CULL_CALLBACK_BIN_INDEX(GetArrayDataSize(DeferredContext->LoadBalancers[BinIndex].GetItems())));

		PassParameters[BinIndex]->LoadBalancerParameters.BatchBuffer = GraphBuilder.CreateSRV(BatchBuffer);
		PassParameters[BinIndex]->LoadBalancerParameters.ItemBuffer = GraphBuilder.CreateSRV(ItemBuffer);
		PassParameters[BinIndex]->LoadBalancerParameters.NumGroupsPerBatch = 1;
		PassParameters[BinIndex]->CurrentBatchProcessingMode = static_cast<uint32>(CurrentBatchProcessingMode);

		FRDGTextureRef ViewPrevHZB = nullptr;

		if (BinIndex > 0 && InstanceCullingManager.ViewPrevHZBs.Num() >= (int32)BinIndex)
		{
			ViewPrevHZB = GraphBuilder.RegisterExternalTexture(InstanceCullingManager.ViewPrevHZBs[BinIndex - 1]);
		}
		
		bool bOcclusionCullInstances = ViewPrevHZB != nullptr && FInstanceCullingContext::IsOcclusionCullingEnabled();
		
		if (bOcclusionCullInstances)
		{
			PassParameters[BinIndex]->HZBTexture = ViewPrevHZB;
			PassParameters[BinIndex]->HZBSize = ViewPrevHZB->Desc.Extent;
			PassParameters[BinIndex]->HZBSampler = TStaticSamplerState< SF_Point, AM_Clamp, AM_Clamp, AM_Clamp >::GetRHI();
		}

		FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FPermutationDomain PermutationVector;
		PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FBatchedDim>(true);
		PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FSingleInstanceModeDim>(CurrentBatchProcessingMode == EBatchProcessingMode::UnCulled);
		PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FCullInstancesDim>(bCullInstances && CurrentBatchProcessingMode != EBatchProcessingMode::UnCulled);
		PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FAllowWPODisableDim>(bAllowWPODisable);
		PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FOcclusionCullInstancesDim>(bOcclusionCullInstances);
		PermutationVector.Set<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs::FInstanceCompactionDim>(bEnableInstanceCompaction);

		auto ComputeShader = ShaderMap->GetShader<FBuildInstanceIdBufferAndCommandsFromPrimitiveIdsCs>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("CullInstances(%s). Bin %d", BatchProcessingModeStr[uint32(CurrentBatchProcessingMode)], BinIndex),
			ComputeShader,
			PassParameters[BinIndex],
			INST_CULL_CALLBACK_BIN_INDEX(DeferredContext->LoadBalancers[BinIndex].GetWrappedCsGroupCount()));
	}

	// TODO: Come up with a way to cull these passes when no compaction is needed. The group count resulting in (0, 0, 0) causes the pass lambdas to not execute,
	// but currently cannot cull resource transitions
	if (bEnableInstanceCompaction)
	{
		FRDGBufferRef BlockDestInstanceOffsets = CreateStructuredBuffer(
			GraphBuilder,
			TEXT("InstanceCulling.Compaction.BlockDestInstanceOffsets"),
			sizeof(uint32),
			INST_CULL_CALLBACK(FMath::Max<uint32>(DeferredContext->TotalCompactionBlocks, 1U)),
			INST_CULL_CALLBACK(nullptr),
			INST_CULL_CALLBACK(0));

		// Compaction phase one - prefix sum of the compaction "blocks"
		{
			auto PassParameters2 = GraphBuilder.AllocParameters<FCalculateCompactBlockInstanceOffsetsCs::FParameters>();
			PassParameters2->DrawCommandCompactionData = DrawCommandCompactionDataSRV;
			PassParameters2->BlockInstanceCounts = GraphBuilder.CreateSRV(CompactionBlockCountsBuffer);
			PassParameters2->BlockDestInstanceOffsetsOut = GraphBuilder.CreateUAV(BlockDestInstanceOffsets);
			PassParameters2->DrawIndirectArgsBufferOut = PassParametersTmp.DrawIndirectArgsBufferOut;

			auto ComputeShader = ShaderMap->GetShader<FCalculateCompactBlockInstanceOffsetsCs>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("Instance Compaction Phase 1"),
				ComputeShader,
				PassParameters2,
				[DeferredContext]()
			{
				return FComputeShaderUtils::GetGroupCountWrapped(DeferredContext->TotalCompactionDrawCommands);
			});
		}

		// Compaction phase two - write instances to compact final location
		{
			FRDGBufferRef BlockDrawCommandIndices = CreateStructuredBuffer(INST_CULL_CREATE_STRUCT_BUFF_ARGS(CompactionBlockDataIndices));

			auto PassParameters2 = GraphBuilder.AllocParameters<FCompactVisibleInstancesCs::FParameters>();
			PassParameters2->DrawCommandCompactionData = DrawCommandCompactionDataSRV;
			PassParameters2->BlockDrawCommandIndices = GraphBuilder.CreateSRV(BlockDrawCommandIndices);
			PassParameters2->InstanceIdsBufferIn = GraphBuilder.CreateSRV(CompactInstanceIdsBuffer);
			PassParameters2->BlockDestInstanceOffsets = GraphBuilder.CreateSRV(BlockDestInstanceOffsets);
			PassParameters2->InstanceIdsBufferOut = InstanceIdsBufferUAV;
			PassParameters2->InstanceIdsBufferOutMobile = InstanceIdsBufferUAV;			

			auto ComputeShader = ShaderMap->GetShader<FCompactVisibleInstancesCs>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("Instance Compaction Phase 2"),
				ComputeShader,
				PassParameters2,
				[PassParameters2, DeferredContext]()
			{				
				return FComputeShaderUtils::GetGroupCountWrapped(DeferredContext->TotalCompactionBlocks);
			});
		}
	}

	if (PlatformGPUSceneUsesUniformBufferView(ShaderPlatform))
	{
		FBatchedPrimitiveParameters* BatchedPrimitiveParameters = GraphBuilder.AllocParameters<FBatchedPrimitiveParameters>();
		BatchedPrimitiveParameters->Data = GraphBuilder.CreateSRV(InstanceIdsBuffer);
		DeferredContext->BatchedPrimitive = GraphBuilder.CreateUniformBuffer(BatchedPrimitiveParameters);
	}
	else
	{
		FInstanceCullingGlobalUniforms* UniformParameters = GraphBuilder.AllocParameters<FInstanceCullingGlobalUniforms>();
		UniformParameters->InstanceIdsBuffer = GraphBuilder.CreateSRV(InstanceIdsBuffer);
		UniformParameters->PageInfoBuffer = GraphBuilder.CreateSRV(InstanceIdsBuffer);
		UniformParameters->BufferCapacity = 0U; // TODO: this is not used at the moment, but is intended for range checks so would have been good.
		DeferredContext->UniformBuffer = GraphBuilder.CreateUniformBuffer(UniformParameters);
	}

#undef INST_CULL_CREATE_STRUCT_BUFF_ARGS
#undef INST_CULL_CALLBACK
#undef INST_CULL_CALLBACK_MODE
#undef INST_CULL_CREATE_STRUCT_BUFF_ARGS_MODE

#if MESH_DRAW_COMMAND_STATS
	if (FMeshDrawCommandStatsManager* Instance = FMeshDrawCommandStatsManager::Get())
	{
		if (Instance->CollectStats())
		{
			DeferredContext->MeshDrawCommandStatsIndirectArgsReadbackBuffer = Instance->QueueDrawRDGIndirectArgsReadback(GraphBuilder, DeferredContext->DrawIndirectArgsBuffer);;
		}
	}
#endif // MESH_DRAW_COMMAND_STATS

	return DeferredContext;
}

void FInstanceCullingContext::SetDeferredContextCullingParams(const FInstanceCullingDeferredContext& DeferredContext, FInstanceCullingDrawParams& Parameters)
{
	Parameters.DrawIndirectArgsBuffer = DeferredContext.DrawIndirectArgsBuffer;
	Parameters.InstanceIdOffsetBuffer = DeferredContext.InstanceDataBuffer;
	Parameters.InstanceCulling = DeferredContext.UniformBuffer;
	Parameters.BatchedPrimitive = DeferredContext.BatchedPrimitive;
}

class FClearIndirectArgInstanceCountCs : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearIndirectArgInstanceCountCs);
	SHADER_USE_PARAMETER_STRUCT(FClearIndirectArgInstanceCountCs, FGlobalShader)

public:
	static constexpr int32 NumThreadsPerGroup = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return UseGPUScene(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FInstanceProcessingGPULoadBalancer::SetShaderDefines(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("INDIRECT_ARGS_NUM_WORDS"), FInstanceCullingContext::IndirectArgsNumWords);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);
		OutEnvironment.SetDefine(TEXT("COMPACTION_BLOCK_NUM_INSTANCES"), FCompactVisibleInstancesBaseCs::CompactionBlockNumInstances);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, DrawIndirectArgsBufferOut)
		SHADER_PARAMETER(uint32, NumIndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FClearIndirectArgInstanceCountCs, "/Engine/Private/InstanceCulling/BuildInstanceDrawCommands.usf", "ClearIndirectArgInstanceCountCS", SF_Compute);


void FInstanceCullingContext::AddClearIndirectArgInstanceCountPass(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGBufferRef DrawIndirectArgsBuffer, TFunction<int32()> NumIndirectArgsCallback)
{
	FClearIndirectArgInstanceCountCs::FParameters* PassParameters = GraphBuilder.AllocParameters<FClearIndirectArgInstanceCountCs::FParameters>();
	// Upload data etc
	PassParameters->DrawIndirectArgsBufferOut = GraphBuilder.CreateUAV(DrawIndirectArgsBuffer, PF_R32_UINT);
	PassParameters->NumIndirectArgs = DrawIndirectArgsBuffer->Desc.NumElements / FInstanceCullingContext::IndirectArgsNumWords;

	auto ComputeShader = ShaderMap->GetShader<FClearIndirectArgInstanceCountCs>();

	if (NumIndirectArgsCallback)
	{
		const FShaderParametersMetadata* ParametersMetadata = FClearIndirectArgInstanceCountCs::FParameters::FTypeInfo::GetStructMetadata();
		ClearUnusedGraphResources(ComputeShader, ParametersMetadata, PassParameters);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("ClearIndirectArgInstanceCount"),
			ParametersMetadata,
			PassParameters,
			ERDGPassFlags::Compute,
			[ParametersMetadata, PassParameters, ComputeShader, NumIndirectArgsCallback = MoveTemp(NumIndirectArgsCallback)](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			int32 NumIndirectArgs = NumIndirectArgsCallback();
			PassParameters->NumIndirectArgs = NumIndirectArgs;
			const FIntVector GroupCount = FComputeShaderUtils::GetGroupCountWrapped(NumIndirectArgs, FClearIndirectArgInstanceCountCs::NumThreadsPerGroup);
			if (GroupCount.X > 0 && GroupCount.Y > 0 && GroupCount.Z > 0)
			{
				FComputeShaderUtils::ValidateGroupCount(GroupCount);
				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, ParametersMetadata, *PassParameters, GroupCount);
			}
		});
	}
	else
	{
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("ClearIndirectArgInstanceCount"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCountWrapped(PassParameters->NumIndirectArgs, FClearIndirectArgInstanceCountCs::NumThreadsPerGroup)
		);
	}
}

static float UnpackScreenSizeHelper(uint32 PackedScreenSize, EMeshDrawCommandCullingPayloadFlags CurrentFlags, EMeshDrawCommandCullingPayloadFlags TestFlag, float DefaultValue)
{
	if (EnumHasAnyFlags(CurrentFlags, TestFlag))
	{
		return FMeshDrawCommandCullingPayload::UnpackScreenSize(PackedScreenSize);
	}

	return DefaultValue;
}

/**
 * Allocate indirect arg slots for all meshes to use instancing,
 * add commands that populate the indirect calls and index & id buffers, and
 * Collapse all commands that share the same state bucket ID
 * NOTE: VisibleMeshDrawCommandsInOut can only become shorter.
 */
void FInstanceCullingContext::SetupDrawCommands(
	FMeshCommandOneFrameArray& VisibleMeshDrawCommandsInOut,
	bool bInCompactIdenticalCommands,
	const FScene* Scene,
	// Stats
	int32& MaxInstances,
	int32& VisibleMeshDrawCommandsNum,
	int32& NewPassVisibleMeshDrawCommandsNum)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BuildMeshDrawCommandPrimitiveIdBuffer);

	FVisibleMeshDrawCommand* RESTRICT PassVisibleMeshDrawCommands = VisibleMeshDrawCommandsInOut.GetData();

	// TODO: make VSM set this for now to force the processing down a single batch (to simplify), maybe.
	const bool bForceGenericProcessing = false;
	const bool bMultiView = ViewIds.Num() > 1 && !(ViewIds.Num() == 2 && InstanceCullingMode == EInstanceCullingMode::Stereo);
	if (bMultiView || bForceGenericProcessing)
	{
		// multi-view defaults to culled path to make cube-maps more efficient
		SingleInstanceProcessingMode = EBatchProcessingMode::Generic;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_DynamicInstancingOfVisibleMeshDrawCommands);

	ResetCommands(VisibleMeshDrawCommandsInOut.Num());
	for (auto& LoadBalancer : LoadBalancers)
	{
		if (LoadBalancer == nullptr)
		{
			LoadBalancer = new FInstanceProcessingGPULoadBalancer;
		}
#if DO_CHECK
		if (InstanceCullingMode == EInstanceCullingMode::Stereo)
		{
			check(ViewIds.Num() == 2);
		}
		else
		{
			check(LoadBalancer->IsEmpty());
		}
#endif
	}

	int32 CurrentStateBucketId = -1;
	EMeshDrawCommandCullingPayloadFlags CurrentCullingPayloadFlags = EMeshDrawCommandCullingPayloadFlags::Default;
	FMeshDrawCommandCullingPayload CurrentCullingPayload;
	MaxInstances = 1;
	// Only used to supply stats
	uint32 CurrentAutoInstanceCount = 1;
	// Scan through and compact away all with consecutive statebucked ID, and record primitive IDs in GPU-scene culling command
	const int32 NumDrawCommandsIn = VisibleMeshDrawCommandsInOut.Num();
	int32 NumDrawCommandsOut = 0;
	uint32 CurrentIndirectArgsOffset = 0U;
	const int32 NumViews = ViewIds.Num();
	const bool bAlwaysUseIndirectDraws = (SingleInstanceProcessingMode != EBatchProcessingMode::UnCulled);
	const bool bOrderPreservationEnabled = IsInstanceOrderPreservationEnabled();
	const uint32 MaxGenericBatchSize = bUsesUniformBufferView ? PLATFORM_MAX_UNIFORM_BUFFER_RANGE / UniformViewInstanceStride[0] : MAX_uint32;
	const uint32 MaxPrimitiveBatchSize = bUsesUniformBufferView ? PLATFORM_MAX_UNIFORM_BUFFER_RANGE / UniformViewInstanceStride[1] : MAX_uint32;
	
	// Used to detect when we change from direct to indirect and break auto instancing there. This is a workaround for the fact that we need an indirect draw to support GPU culling and when MDCs are identical they could batch together otherwise.
	// In the future we should rework the below to instead decide the draw command at the end, but this is complicated by the mobile GPU scene batching.
	// Note: initial value is arbitrary because we will never use the result during the first iteration as CurrentStateBucketId is initialized to -1
	bool bPrevUseIndirectDraw = false;

	// Allocate conservatively for all commands, may not use all.
	for (int32 DrawCommandIndex = 0; DrawCommandIndex < NumDrawCommandsIn; ++DrawCommandIndex)
	{
		const FVisibleMeshDrawCommand& RESTRICT VisibleMeshDrawCommand = PassVisibleMeshDrawCommands[DrawCommandIndex];
		const FMeshDrawCommand* RESTRICT MeshDrawCommand = VisibleMeshDrawCommand.MeshDrawCommand;

		const bool bFetchInstanceCountFromScene = EnumHasAnyFlags(VisibleMeshDrawCommand.Flags, EFVisibleMeshDrawCommandFlags::FetchInstanceCountFromScene);
		check(!bFetchInstanceCountFromScene || Scene != nullptr);

		const bool bSupportsGPUSceneInstancing = EnumHasAnyFlags(VisibleMeshDrawCommand.Flags, EFVisibleMeshDrawCommandFlags::HasPrimitiveIdStreamIndex);
		const bool bMaterialUsesWorldPositionOffset = EnumHasAnyFlags(VisibleMeshDrawCommand.Flags, EFVisibleMeshDrawCommandFlags::MaterialUsesWorldPositionOffset);
		const bool bMaterialAlwaysEvaluatesWorldPositionOffset = EnumHasAnyFlags(VisibleMeshDrawCommand.Flags, EFVisibleMeshDrawCommandFlags::MaterialAlwaysEvaluatesWorldPositionOffset);
		const bool bForceInstanceCulling = EnumHasAnyFlags(VisibleMeshDrawCommand.Flags, EFVisibleMeshDrawCommandFlags::ForceInstanceCulling) || (GOcclusionForceInstanceCulling != 0);
		const bool bPreserveInstanceOrder = bOrderPreservationEnabled && EnumHasAnyFlags(VisibleMeshDrawCommand.Flags, EFVisibleMeshDrawCommandFlags::PreserveInstanceOrder);
		const bool bUseIndirectDraw = bFetchInstanceCountFromScene || bAlwaysUseIndirectDraws || bForceInstanceCulling || (VisibleMeshDrawCommand.NumRuns > 0 || MeshDrawCommand->NumInstances > 1);
		// UniformBufferView path and instance order preservation do not support merging ISM draws atm
		const bool bCompactIdenticalCommands = bInCompactIdenticalCommands && !(bUseIndirectDraw && bPreserveInstanceOrder) 
			&& (bUsesUniformBufferView ? (CurrentAutoInstanceCount < MaxPrimitiveBatchSize && !bUseIndirectDraw) : true);

		const bool bChangedFromDirectToIndirect = !bPrevUseIndirectDraw && bUseIndirectDraw;
		bPrevUseIndirectDraw = bUseIndirectDraw;

		bool bMeshDrawCommandMatch = VisibleMeshDrawCommand.StateBucketId == CurrentStateBucketId
									&& VisibleMeshDrawCommand.CullingPayloadFlags == CurrentCullingPayloadFlags
									&& VisibleMeshDrawCommand.CullingPayload.PackedData == CurrentCullingPayload.PackedData;

		if (bCompactIdenticalCommands && CurrentStateBucketId != -1 && bMeshDrawCommandMatch && !bChangedFromDirectToIndirect)
		{
			// Drop since previous covers for this

			CurrentAutoInstanceCount++;
			MaxInstances = FMath::Max<int32>(CurrentAutoInstanceCount, MaxInstances);

			FMeshDrawCommandInfo& RESTRICT DrawCmd = MeshDrawCommandInfos.Last();
			if (DrawCmd.bUseIndirect == 0)
			{
				DrawCmd.IndirectArgsOffsetOrNumInstances += 1;
			}

			// Nothing needs to be done when indirect rendering is used on the draw command because the current cached value CurrentIndirectArgsOffset won't change
			// and these instances will be added to the same previous draw command in AddInstancesToDrawCommand below
		}
		else
		{
			// Reset auto-instance count (only needed for logging)
			CurrentAutoInstanceCount = 1;

			// kept 1:1 with the retained (not compacted) mesh draw commands, implicitly clears num instances
			FMeshDrawCommandInfo& RESTRICT DrawCmd = MeshDrawCommandInfos.AddZeroed_GetRef();
			DrawCmd.NumBatches = 1;
			DrawCmd.BatchDataStride = PLATFORM_MAX_UNIFORM_BUFFER_RANGE;

			// TODO: redundantly create an indirect arg slot for every draw command (even thoug those that don't support GPU-scene don't need one)
			//       the unsupported ones are skipped in FMeshDrawCommand::SubmitDrawBegin/End.
			//       in the future pipe through draw command info to submit, such that they may be skipped.
			//if (bSupportsGPUSceneInstancing)
			{
				DrawCmd.bUseIndirect = bUseIndirectDraw;
				
				CurrentIndirectArgsOffset = AllocateIndirectArgs(MeshDrawCommand);
				
				DrawCommandDescs.Add(
					PackDrawCommandDesc(
						bMaterialUsesWorldPositionOffset,
						bMaterialAlwaysEvaluatesWorldPositionOffset,
						VisibleMeshDrawCommand.CullingPayload,
						VisibleMeshDrawCommand.CullingPayloadFlags
					)
				);

				if (bUseIndirectDraw)
				{
					bHasAnyIndirectDraw = true;
 					DrawCmd.IndirectArgsOffsetOrNumInstances = CurrentIndirectArgsOffset * FInstanceCullingContext::IndirectArgsNumWords * sizeof(uint32);
				}
				else
				{
					DrawCmd.IndirectArgsOffsetOrNumInstances = 1;
				}

				const uint32 CurrentNumDraws = InstanceIdOffsets.Num();
				// drawcall specific offset into per-instance buffer
				DrawCmd.InstanceDataByteOffset = StepInstanceDataOffsetBytes(CurrentNumDraws);
				InstanceIdOffsets.Emplace(GetInstanceIdNumElements());
			}
			
			// Record the last bucket ID (may be -1)
			CurrentStateBucketId = VisibleMeshDrawCommand.StateBucketId;
			CurrentCullingPayloadFlags = VisibleMeshDrawCommand.CullingPayloadFlags;
			CurrentCullingPayload = VisibleMeshDrawCommand.CullingPayload;

			// If we have dropped any we need to move up to maintain 1:1
			if (DrawCommandIndex > NumDrawCommandsOut)
			{
				PassVisibleMeshDrawCommands[NumDrawCommandsOut] = PassVisibleMeshDrawCommands[DrawCommandIndex];
			}
			NumDrawCommandsOut++;
		}

		if (bSupportsGPUSceneInstancing)
		{
			EInstanceFlags InstanceFlags = EInstanceFlags::None;
			if (VisibleMeshDrawCommand.PrimitiveIdInfo.bIsDynamicPrimitive)
			{
				EnumAddFlags(InstanceFlags, EInstanceFlags::DynamicInstanceDataOffset);
			}
			if (bForceInstanceCulling)
			{
				EnumAddFlags(InstanceFlags, EInstanceFlags::ForceInstanceCulling);
			}
			if (bPreserveInstanceOrder)
			{
				EnumAddFlags(InstanceFlags, EInstanceFlags::PreserveInstanceOrder);
			}

			const uint32 InstanceOffset = TotalInstances;

			// append 'culling command' targeting the current slot
			// This will cause all instances belonging to the Primitive to be added to the command, if they are visible etc (GPU-Scene knows all - sees all)
			if (VisibleMeshDrawCommand.RunArray)
			{
				AddInstanceRunsToDrawCommand(CurrentIndirectArgsOffset, VisibleMeshDrawCommand.PrimitiveIdInfo.InstanceSceneDataOffset, VisibleMeshDrawCommand.RunArray, VisibleMeshDrawCommand.NumRuns, InstanceFlags, MaxGenericBatchSize);
			}
			else if (bFetchInstanceCountFromScene)
			{
				check(Scene != nullptr);
				check(!VisibleMeshDrawCommand.PrimitiveIdInfo.bIsDynamicPrimitive);
				
				const FPrimitiveSceneInfo* RESTRICT PrimitiveSceneInfo = Scene->Primitives[VisibleMeshDrawCommand.PrimitiveIdInfo.ScenePrimitiveId];
				uint32 NumInstances = uint32(PrimitiveSceneInfo->GetNumInstanceSceneDataEntries());
				if (NumInstances > 0u)
				{
					if (const FSceneCulling::FPrimitiveCullingInfo* PrimitiveCullingData = CullingQuery ? CullingQuery->GetCullingInfo(PrimitiveSceneInfo->GetPersistentIndex()) : nullptr)
					{
						FVector2f MinMaxLODScreenSize(
							UnpackScreenSizeHelper(VisibleMeshDrawCommand.CullingPayload.MinScreenSize, CurrentCullingPayloadFlags, EMeshDrawCommandCullingPayloadFlags::MinScreenSizeCull, std::numeric_limits<float>::lowest()),
							UnpackScreenSizeHelper(VisibleMeshDrawCommand.CullingPayload.MaxScreenSize, CurrentCullingPayloadFlags, EMeshDrawCommandCullingPayloadFlags::MaxScreenSizeCull, std::numeric_limits<float>::max())
						);
						
						int32 NumInstancesDrawn = 0;
						int32 FirstInstance = 0;
						for (int32 ItemIndex = 0; ItemIndex < PrimitiveCullingData->Items.Num(); ++ItemIndex)
						{
							const auto& Item = PrimitiveCullingData->Items[ItemIndex];
							bool bIsVisible = CullingQuery->IsVisible(*PrimitiveCullingData, ItemIndex);
							
							// TODO: track start/end and only add once if e.g., all are visible?
							if (bIsVisible && CullingQuery->IsLODVisible(*PrimitiveCullingData, ItemIndex, MinMaxLODScreenSize))
							{
								AddInstancesToDrawCommand(CurrentIndirectArgsOffset, VisibleMeshDrawCommand.PrimitiveIdInfo.InstanceSceneDataOffset + FirstInstance, NumInstancesDrawn, Item.NumInstances, InstanceFlags, MaxGenericBatchSize);
								NumInstancesDrawn += Item.NumInstances;
							}
							FirstInstance += Item.NumInstances;
						}
					}
					else
					{
						AddInstancesToDrawCommand(CurrentIndirectArgsOffset, VisibleMeshDrawCommand.PrimitiveIdInfo.InstanceSceneDataOffset, 0, NumInstances, InstanceFlags, MaxGenericBatchSize);
					}
				}
			}
			else
			{
				if (Scene != nullptr)
				{
					// Make sure the cached MDC matches what is stored in the scene
					checkSlow(VisibleMeshDrawCommand.PrimitiveIdInfo.bIsDynamicPrimitive 
						|| !bSupportsGPUSceneInstancing
						|| VisibleMeshDrawCommand.MeshDrawCommand->NumInstances == uint32(Scene->Primitives[VisibleMeshDrawCommand.PrimitiveIdInfo.ScenePrimitiveId]->GetNumInstanceSceneDataEntries()));
					// This condition is used to skip re-caching MDCs and thus should not be set on anything that doesn't take the above path
					checkSlow(!Scene->Primitives[VisibleMeshDrawCommand.PrimitiveIdInfo.ScenePrimitiveId]->Proxy->DoesMeshBatchesUseSceneInstanceCount());
				}
				AddInstancesToDrawCommand(CurrentIndirectArgsOffset, VisibleMeshDrawCommand.PrimitiveIdInfo.InstanceSceneDataOffset, 0, VisibleMeshDrawCommand.MeshDrawCommand->NumInstances, InstanceFlags, MaxGenericBatchSize);
			}

			const uint32 NumInstancesAdded = TotalInstances - InstanceOffset;
			if (bPreserveInstanceOrder && NumInstancesAdded > 0)
			{
				const uint32 CompactionDataIndex = uint32(DrawCommandCompactionData.Num());
				DrawCommandCompactionData.Emplace(
					NumInstancesAdded,
					NumViews,
					uint32(CompactionBlockDataIndices.Num()),
					CurrentIndirectArgsOffset,
					NumCompactionInstances,
					InstanceIdOffsets.Last());

				const int32 FirstBlock = CompactionBlockDataIndices.Num();
				const uint32 NumCompactionBlocksThisCommand = FMath::DivideAndRoundUp(NumInstancesAdded, CompactionBlockNumInstances);
				CompactionBlockDataIndices.AddUninitialized(NumCompactionBlocksThisCommand);
				for (int32 Block = FirstBlock; Block < CompactionBlockDataIndices.Num(); ++Block)
				{
					CompactionBlockDataIndices[Block] = CompactionDataIndex;
				}

				NumCompactionInstances += NumInstancesAdded * NumViews;
			}
		}
	}
	check(bInCompactIdenticalCommands || NumDrawCommandsIn == NumDrawCommandsOut);
	checkf(NumDrawCommandsOut == MeshDrawCommandInfos.Num(), TEXT("There must be a 1:1 mapping between MeshDrawCommandInfos and mesh draw commands, as this assumption is made in SubmitDrawCommands."));

	// Setup instancing stats for logging.
	VisibleMeshDrawCommandsNum = VisibleMeshDrawCommandsInOut.Num();
	NewPassVisibleMeshDrawCommandsNum = NumDrawCommandsOut;

	// Resize array post-compaction of dynamic instances
	VisibleMeshDrawCommandsInOut.SetNum(NumDrawCommandsOut, EAllowShrinking::No);
}

void FInstanceCullingContext::SubmitDrawCommands(
	const FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	const FMeshDrawCommandOverrideArgs& OverrideArgs,
	int32 StartIndex,
	int32 NumMeshDrawCommands,
	uint32 InInstanceFactor,
	FRHICommandList& RHICmdList) const
{
	if (VisibleMeshDrawCommands.Num() == 0)
	{
		// FIXME: looks like parallel rendering can spawn empty FDrawVisibleMeshCommandsAnyThreadTask
		return;
	}
	
	if (IsEnabled())
	{
		check(MeshDrawCommandInfos.Num() >= (StartIndex + NumMeshDrawCommands));
				
		FMeshDrawCommandSceneArgs SceneArgs;
		SceneArgs.PrimitiveIdsBuffer = OverrideArgs.InstanceBuffer;
		SceneArgs.RootConstants = OverrideArgs.RootConstants;
		
		if (IsUniformBufferStaticSlotValid(InstanceCullingStaticSlot))
		{
			if (bUsesUniformBufferView)
			{
				SceneArgs.BatchedPrimitiveSlot = InstanceCullingStaticSlot;
			}
			// Only does anything when batching is disabled and each mesh pass has an unique InstanceCulling/BatchedPrimitive UB
			// otherwise static UBs set once when we start render-pass
			RHICmdList.SetStaticUniformBuffer(InstanceCullingStaticSlot, OverrideArgs.InstanceCullingStaticUB);
		}
							
		FMeshDrawCommandStateCache StateCache;
		INC_DWORD_STAT_BY(STAT_MeshDrawCalls, NumMeshDrawCommands);

		for (int32 DrawCommandIndex = StartIndex; DrawCommandIndex < StartIndex + NumMeshDrawCommands; DrawCommandIndex++)
		{
			//SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, MeshEvent, GEmitMeshDrawEvent != 0, TEXT("Mesh Draw"));
			const FVisibleMeshDrawCommand& VisibleMeshDrawCommand = VisibleMeshDrawCommands[DrawCommandIndex];
			const FMeshDrawCommandInfo& DrawCommandInfo = MeshDrawCommandInfos[DrawCommandIndex];
			
			uint32 InstanceFactor = InInstanceFactor;
			SceneArgs.IndirectArgsByteOffset = 0u;
			SceneArgs.IndirectArgsBuffer = nullptr;
			if (DrawCommandInfo.bUseIndirect)
			{
				SceneArgs.IndirectArgsByteOffset = OverrideArgs.IndirectArgsByteOffset + DrawCommandInfo.IndirectArgsOffsetOrNumInstances;
				SceneArgs.IndirectArgsBuffer = OverrideArgs.IndirectArgsBuffer;
			}
			else
			{
				// TODO: need a better way to override number of instances
				InstanceFactor = InInstanceFactor * DrawCommandInfo.IndirectArgsOffsetOrNumInstances;
			}
			
			SceneArgs.PrimitiveIdOffset = OverrideArgs.InstanceDataByteOffset + DrawCommandInfo.InstanceDataByteOffset;

#if WANTS_DRAW_MESH_EVENTS
			RHI_BREADCRUMB_EVENT_CONDITIONAL_F(RHICmdList, GShowMaterialDrawEvents != 0, "MaterialDrawEvent", "%s %s (%u instances)"
				, VisibleMeshDrawCommand.MeshDrawCommand->GetDebugData().MaterialRenderProxyName
				, VisibleMeshDrawCommand.MeshDrawCommand->GetDebugData().ResourceName
				, VisibleMeshDrawCommand.MeshDrawCommand->NumInstances * InstanceFactor
			);
#endif
			const bool bAllowSkipDrawCommand = true;
			if (FMeshDrawCommand::SubmitDrawBegin(*VisibleMeshDrawCommand.MeshDrawCommand, GraphicsMinimalPipelineStateSet, SceneArgs, InstanceFactor, RHICmdList, StateCache, bAllowSkipDrawCommand))
			{
				FMeshDrawCommand::SubmitDrawEnd(*VisibleMeshDrawCommand.MeshDrawCommand, SceneArgs, InstanceFactor, RHICmdList);

				// If MDC was split to a more than one batch, submit them without changing state
				for (uint32 BatchIdx = 1; BatchIdx < DrawCommandInfo.NumBatches; ++BatchIdx)
				{
					SceneArgs.PrimitiveIdOffset += DrawCommandInfo.BatchDataStride;
					SceneArgs.IndirectArgsByteOffset += sizeof(FRHIDrawIndexedIndirectParameters);
					FMeshDrawCommand::SubmitDrawEnd(*VisibleMeshDrawCommand.MeshDrawCommand, SceneArgs, InstanceFactor, RHICmdList);
				}
			}
		}
	}
	else
	{
		FMeshDrawCommandSceneArgs SceneArgs;
		SubmitMeshDrawCommandsRange(VisibleMeshDrawCommands, GraphicsMinimalPipelineStateSet, SceneArgs, 0, false, StartIndex, NumMeshDrawCommands, InInstanceFactor, RHICmdList);
	}
}
