// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceCullingLoadBalancer.h"
#include "RenderGraphUtils.h"
#include "ShaderCore.h"

void FInstanceCullingLoadBalancerBase::SetShaderDefines(FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), ThreadGroupSize);
	OutEnvironment.SetDefine(TEXT("NUM_INSTANCES_ITEM_BITS"), NumInstancesItemBits);
	OutEnvironment.SetDefine(TEXT("NUM_INSTANCES_ITEM_MASK"), NumInstancesItemMask);
	OutEnvironment.SetDefine(TEXT("PREFIX_BITS"), PrefixBits);
	OutEnvironment.SetDefine(TEXT("PREFIX_BIT_MASK"), PrefixBitMask);
}

void FInstanceCullingLoadBalancerBase::FGPUData::GetShaderParameters(FRDGBuilder& GraphBuilder, FShaderParameters& ShaderParameters) const
{
	ShaderParameters.BatchBuffer = GraphBuilder.CreateSRV(BatchBuffer);
	ShaderParameters.ItemBuffer = GraphBuilder.CreateSRV(ItemBuffer);
	ShaderParameters.NumBatches = NumBatches;
	ShaderParameters.NumItems = NumItems;
	ShaderParameters.NumGroupsPerBatch = NumGroupsPerBatch;
}

FInstanceCullingLoadBalancerBase::FGPUData FInstanceCullingLoadBalancerBase::Upload(FRDGBuilder& GraphBuilder, TConstArrayView<FPackedBatch> Batches, TConstArrayView<FPackedItem> Items, ERDGInitialDataFlags RDGInitialDataFlags, int32 NumGroupsPerBatch) const
{
	FGPUData Result;
	Result.BatchBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCullingLoadBalancer.Batches"), Batches, RDGInitialDataFlags);
	Result.ItemBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCullingLoadBalancer.Items"), Items, RDGInitialDataFlags);
	Result.NumBatches = Batches.Num();
	Result.NumItems = Items.Num();
	Result.NumGroupsPerBatch = NumGroupsPerBatch;

	return Result;
}

FIntVector FInstanceCullingLoadBalancerBase::GetWrappedCsGroupCount(TConstArrayView<FPackedBatch> Batches, int32 NumGroupsPerBatch) const
{
	return FComputeShaderUtils::GetGroupCountWrapped(Batches.Num() * NumGroupsPerBatch);
}
