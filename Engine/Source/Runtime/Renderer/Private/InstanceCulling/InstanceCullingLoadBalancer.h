// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "RenderGraphFwd.h"
#include "RenderGraphEvent.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterMacros.h"
#include "Templates/SharedPointer.h"


struct FShaderCompilerEnvironment;

/*
 * Helper to build the needed data to run per-instance operation on the GPU in a balanced way
 */
class FInstanceCullingLoadBalancerBase
{
public:
	static constexpr uint32 ThreadGroupSize = 64U;

	// Number of bits needed for prefix sum storage
	static constexpr uint32 PrefixBits = 6U;//ILog2Const(uint32(ThreadGroupSize));
	static_assert((1U << PrefixBits) == ThreadGroupSize, "ThreadGroupSize and PrefixBits must be kept in sync");
	static constexpr uint32 PrefixBitMask = (1U << PrefixBits) - 1U;

	static constexpr uint32 NumInstancesItemBits = PrefixBits + 1U;
	static constexpr uint32 NumInstancesItemMask = (1U << NumInstancesItemBits) - 1U;


	struct FPackedBatch
	{
		uint32 FirstItem_NumItems;
	};

	FPackedBatch PackBatch(uint32 FirstItem, uint32 NumItems)
	{
		checkSlow(NumItems < (1U << NumInstancesItemBits));
		checkSlow(FirstItem < (1U << (32U - NumInstancesItemBits)));

		return FPackedBatch{ (FirstItem << NumInstancesItemBits) | (NumItems & NumInstancesItemMask) };
	}

	struct FPackedItem
	{
		// packed 32-NumInstancesItemBits:NumInstancesItemBits - need one more bit for the case where one item has ThreadGroupSize work to do
		uint32 InstanceDataOffset_NumInstances;
		// packed 32-PrefixBits:PrefixBits
		uint32 Payload_BatchPrefixOffset;
	};
	FPackedItem PackItem(uint32 InstanceDataOffset, uint32 NumInstances, uint32 Payload, uint32 BatchPrefixSum)
	{
		checkSlow(NumInstances < (1U << NumInstancesItemBits));
		checkSlow(InstanceDataOffset < (1U << (32U - NumInstancesItemBits)));
		checkSlow(BatchPrefixSum < (1U << PrefixBits));
		checkSlow(Payload < (1U << (32U - PrefixBits)));

		return FPackedItem
		{
			(InstanceDataOffset << NumInstancesItemBits) | (NumInstances & NumInstancesItemMask),
			(Payload << PrefixBits) | (BatchPrefixSum & PrefixBitMask)
		};
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedBatch >, BatchBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer< FPackedItem >, ItemBuffer)
		SHADER_PARAMETER(uint32, NumBatches)
		SHADER_PARAMETER(uint32, NumItems)
		SHADER_PARAMETER(uint32, NumGroupsPerBatch)
	END_SHADER_PARAMETER_STRUCT()

	/*
	* Publish constants to a shader implementing a kernel using the load balancer.
	* Call from ModifyCompilationEnvironment
	*/
	static void SetShaderDefines(FShaderCompilerEnvironment& OutEnvironment);

	struct FGPUData
	{
		int32 NumBatches = 0;
		int32 NumItems = 0;
		// Optional to allow launching multiple groups that all get the same batch on the shader side
		int32 NumGroupsPerBatch = 1;
		FRDGBufferRef BatchBuffer = nullptr;
		FRDGBufferRef ItemBuffer = nullptr;

		void GetShaderParameters(FRDGBuilder& GraphBuilder, FShaderParameters& ShaderParameters) const;

		/**
		 * Helper function to add a pass to run a CS using the loadbalancer, automatically sets the parameters and dispatch dimensions.
		 * Expects "LoadBalancerParameters" in the Parameters struct.
		 */
		template <typename TShaderClass>
		FRDGPassRef AddPass(
			FRDGBuilder& GraphBuilder,
			FRDGEventName&& PassName,
			const TShaderRef<TShaderClass>& ComputeShader,
			typename TShaderClass::FParameters* Parameters) const
		{
			GetShaderParameters(GraphBuilder, Parameters->LoadBalancerParameters);
			return FComputeShaderUtils::AddPass(GraphBuilder, Forward<FRDGEventName>(PassName), ComputeShader, Parameters, GetWrappedCsGroupCount());
		}

		FIntVector GetWrappedCsGroupCount() const { return FComputeShaderUtils::GetGroupCountWrapped(NumBatches); }

	};

	FGPUData Upload(FRDGBuilder& GraphBuilder, TConstArrayView<FPackedBatch> Batches, TConstArrayView<FPackedItem> Items, ERDGInitialDataFlags RDGInitialDataFlags, int32 NumGroupsPerBatch) const;

	FIntVector GetWrappedCsGroupCount(TConstArrayView<FPackedBatch> Batches, int32 NumGroupsPerBatch) const;
};

/*
 * Helper to build the needed data to run per-instance operation on the GPU in a balanced way
 */
template <typename InAllocatorType = FDefaultAllocator>
class TInstanceCullingLoadBalancer : public FInstanceCullingLoadBalancerBase
{
public:
	using AllocatorType = InAllocatorType;

	void ReserveStorage(int32 NumBatches, int32 NumItems)
	{
		Data->Batches.Empty(NumBatches);
		Data->Items.Empty(NumItems);
	}

	/**
	 * Add a span of instances to be processed.
	 */
	void Add(uint32 InstanceDataOffset, uint32 NumInstanceDataEntries, uint32 Payload)
	{
		uint32 InstancesAdded = 0;
		while (InstancesAdded < NumInstanceDataEntries)
		{
			uint32 MaxInstancesThisBatch = ThreadGroupSize - CurrentBatchPrefixSum;

			if (MaxInstancesThisBatch > 0)
			{
				const uint32 NumInstancesThisItem = FMath::Min(MaxInstancesThisBatch, NumInstanceDataEntries - InstancesAdded);

				Data->Items.Add(PackItem(InstanceDataOffset + InstancesAdded, NumInstancesThisItem, Payload, CurrentBatchPrefixSum));
				if (CurrentBatchNumItems * PrefixBits < sizeof(CurrentBatchPackedPrefixSum) * 8U)
				{
					CurrentBatchPackedPrefixSum |= CurrentBatchPackedPrefixSum << (PrefixBits * CurrentBatchNumItems);
				}
				CurrentBatchNumItems += 1U;
				InstancesAdded += NumInstancesThisItem;
				CurrentBatchPrefixSum += NumInstancesThisItem;

			}

			// Flush batch if it is not possible to add any more items (for one of the reasons)
			if (MaxInstancesThisBatch <= 0U || CurrentBatchPrefixSum >= ThreadGroupSize)
			{
				Data->Batches.Add(PackBatch(CurrentBatchFirstItem, CurrentBatchNumItems));
				CurrentBatchFirstItem = uint32(Data->Items.Num());
				CurrentBatchPrefixSum = 0u;
				CurrentBatchNumItems = 0U;
				CurrentBatchPackedPrefixSum = 0U;
			}
		}
		TotalInstances += InstancesAdded;
	}

	bool IsEmpty() const
	{
		return Data->Items.IsEmpty();
	}

	FGPUData Upload(FRDGBuilder& GraphBuilder, int32 NumGroupsPerBatch = 1)
	{
		FinalizeBatches();

		const auto& LocalBatches = Data->Batches;
		const auto& LocalItems   = Data->Items;

		GraphBuilder.AllocObject<TRefCountPtr<FData>>(Data);

		return FInstanceCullingLoadBalancerBase::Upload(GraphBuilder, LocalBatches, LocalItems, ERDGInitialDataFlags::NoCopy, NumGroupsPerBatch);
	}

	/* Const variant that assumes the batches have already been finalized */
	FGPUData UploadFinalized(FRDGBuilder& GraphBuilder, int32 NumGroupsPerBatch = 1) const
	{
		check(CurrentBatchNumItems == 0);

		const auto& LocalBatches = Data->Batches;
		const auto& LocalItems   = Data->Items;

		GraphBuilder.AllocObject<TRefCountPtr<FData>>(Data);

		return FInstanceCullingLoadBalancerBase::Upload(GraphBuilder, LocalBatches, LocalItems, ERDGInitialDataFlags::NoCopy, NumGroupsPerBatch);
	}

	/**
	 * Call when finished adding work items to the balancer to flush any in-progress batches.
	 */
	void FinalizeBatches()
	{
		if (CurrentBatchNumItems != 0)
		{
			Data->Batches.Add(PackBatch(CurrentBatchFirstItem, CurrentBatchNumItems));
			CurrentBatchNumItems = 0;
		}
	}

	/**
	 * Returns a 3D group count that is large enough to generate one group per batch using FComputeShaderUtils::GetGroupCountWrapped.
	 * Use GetUnWrappedDispatchGroupId in the shader to retrieve the linear index.
	 * NOTE: NumGroupsPerBatch must be consistent with the value passed to Upload
	 */
	FIntVector GetWrappedCsGroupCount(int32 NumGroupsPerBatch = 1) const
	{
		return FInstanceCullingLoadBalancerBase::GetWrappedCsGroupCount(Data->Batches, NumGroupsPerBatch);
	}

	const TArray<FPackedBatch, AllocatorType> &GetBatches() const
	{
		check(CurrentBatchNumItems == 0);
		return Data->Batches;
	};

	const TArray<FPackedItem, AllocatorType> &GetItems() const
	{
		check(CurrentBatchNumItems == 0);
		return Data->Items;
	}

	uint32 GetTotalNumInstances() const { return TotalInstances; }

	template <typename AllocatorType>
	void AppendData(const TInstanceCullingLoadBalancer<AllocatorType> &Other)
	{
		Data->Batches.Append(Other.GetBatches().GetData(), Other.GetBatches().Num());
		Data->Items.Append(Other.GetItems().GetData(), Other.GetItems().Num());
		TotalInstances += Other.GetTotalNumInstances();
	}

	bool HasSingleInstanceItemsOnly() const
	{
		return TotalInstances == Data->Items.Num();
	}

protected:
	struct FData : public TRefCountingMixin<FData>
	{
		TArray<FPackedBatch, AllocatorType>  Batches;
		TArray<FPackedItem, AllocatorType>  Items;
	};

	TRefCountPtr<FData> Data{ new FData };

	uint32 CurrentBatchPrefixSum = 0u;
	uint32 CurrentBatchNumItems = 0U;
	uint32 CurrentBatchPackedPrefixSum = 0U;
	uint32 CurrentBatchFirstItem = 0U;
	uint32 TotalInstances = 0U;
};
