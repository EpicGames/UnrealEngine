// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceCullingManager.h"
#include "CoreMinimal.h"
#include "RHI.h"
#include "ShaderParameterMacros.h"
#include "RenderGraphResources.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "SceneRendering.h"
#include "ScenePrivate.h"
#include "ViewData.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#include "InstanceCulling/InstanceCullingContext.h"

static int32 GAllowBatchedBuildRenderingCommands = 1;
static FAutoConsoleVariableRef CVarAllowBatchedBuildRenderingCommands(
	TEXT("r.InstanceCulling.AllowBatchedBuildRenderingCommands"),
	GAllowBatchedBuildRenderingCommands,
	TEXT("Whether to allow batching BuildRenderingCommands for GPU instance culling"),
	ECVF_RenderThreadSafe);

FInstanceCullingManager::FInstanceCullingManager(FRDGBuilder& GraphBuilder, const FScene& InScene, FSceneUniformBuffer& InSceneUniforms, FRendererViewDataManager& InViewDataManager)
	: Scene(InScene), GPUScene(InScene.GPUScene), SceneUniforms(InSceneUniforms), ViewDataManager(InViewDataManager), bIsEnabled(GPUScene.IsEnabled())
{
	DummyUniformBuffer = FInstanceCullingContext::CreateDummyInstanceCullingUniformBuffer(GraphBuilder);
	
	for (FViewInfo *ViewInfo : ViewDataManager.GetRegisteredPrimaryViews())
	{
		if (ViewInfo->PrevViewInfo.HZB != nullptr)
		{
			check(IsInRenderingThread());

			ViewPrevHZBs.AddUnique(ViewInfo->PrevViewInfo.HZB);
		}
	}
}

FInstanceCullingManager::~FInstanceCullingManager()
{
}

void FInstanceCullingManager::AllocateViews(int32 NumViews)
{
	return ViewDataManager.AllocateViews(NumViews);
}

int32 FInstanceCullingManager::RegisterView(const Nanite::FPackedViewParams& Params)
{
	return ViewDataManager.RegisterView(Params);
}


int32 FInstanceCullingManager::GetBinIndex(EBatchProcessingMode Mode, const TRefCountPtr<IPooledRenderTarget>& HZB)
{
	if (Mode == EBatchProcessingMode::UnCulled)
	{
		return 0;
	}

	int32 BinIndex;
	// all contexts without a valid HZB go in the first bin, together with the first view's HZB
	if (!HZB.IsValid())
	{
		return 1;
	}

	if (!ViewPrevHZBs.Find(HZB, BinIndex))
	{
		// error: the HZB is not registered correctly
		return -1;
	}

	// bin 0 is used for EBatchProcessingMode::UnCulled batches
	BinIndex += 1;

	return BinIndex;
}

void FInstanceCullingManager::SetDummyCullingParams(FRDGBuilder& GraphBuilder, FInstanceCullingDrawParams& Parameters)
{
	Parameters.Scene = SceneUniforms.GetBuffer(GraphBuilder);
	Parameters.InstanceCulling = GetDummyInstanceCullingUniformBuffer();

	// Mobile renderer renders several mesh passes in a single RDG pass, however RDG pass can accept only a single FInstanceCullingDrawParams
	// This makes sure that merged culling parameters are always end up in a RDG pass struct for a mobile renderer (see BuildMeshRenderingCommands)
	if (DeferredContext && IsMobilePlatform(Scene.GetShaderPlatform()))
	{
		FInstanceCullingContext::SetDeferredContextCullingParams(*DeferredContext, Parameters);
	}
}

bool FInstanceCullingManager::AllowBatchedBuildRenderingCommands(const FGPUScene& GPUScene)
{
	return GPUScene.IsEnabled() && GAllowBatchedBuildRenderingCommands != 0 && !FRDGBuilder::IsImmediateMode();
}

void FInstanceCullingManager::BeginDeferredCulling(FRDGBuilder& GraphBuilder)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FInstanceCullingManager::BeginDeferredCulling);

	// We need the dynamic mesh bounds buffer for culling.
	Scene.AddGPUSkinCacheAsyncComputeWait(GraphBuilder);

	// TODO: is this needed(?)
	ViewDataManager.FlushRegisteredViews(GraphBuilder);

	// Cannot defer pass execution in immediate mode.
	if (!AllowBatchedBuildRenderingCommands(GPUScene))
	{
		return;
	}

	// If there are no instances, there can be no work to perform later.
	if (GPUScene.GetNumInstances() == 0 || ViewDataManager.GetNumCullingViews() == 0)
	{
		return;
	}

	DeferredContext = FInstanceCullingContext::CreateDeferredContext(GraphBuilder, GPUScene, *this);
}
