// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCullingLoadBalancer.h"
#include "RenderGraphResources.h"
#include "Async/Mutex.h"

class FRendererViewDataManager;
class FSceneUniformBuffer;

class FGPUScene;

namespace Nanite
{
	struct FPackedView;
	struct FPackedViewParams;
}

class FInstanceProcessingGPULoadBalancer : public TInstanceCullingLoadBalancer<>
{
public:
};

/**
 * Only needed for compatibility, used to explicitly opt out of async processing (when there is no capturable pointer to an FInstanceCullingDrawParams).
 */
struct FInstanceCullingResult
{
	FInstanceCullingDrawParams Parameters;

	inline void GetDrawParameters(FInstanceCullingDrawParams &OutParams) const
	{
		OutParams = Parameters;
	}
};

class FInstanceCullingDeferredContext;

/**
 * Manages allocation of indirect arguments and culling jobs for all instanced draws (use the GPU Scene culling).
 */
class FInstanceCullingManager
{
public:
	/**
	 * Construct the instance culling manager for a scene renderer with the set of primary views that are used.
	 */
	FInstanceCullingManager(FRDGBuilder& GraphBuilder, const FScene& InScene, FSceneUniformBuffer& InSceneUniforms, FRendererViewDataManager& InViewDataManager);

	~FInstanceCullingManager();

	bool IsEnabled() const { return bIsEnabled; }

	// Register a non-primary view for culling, returns integer ID of the view.
	int32 RegisterView(const Nanite::FPackedViewParams& Params);

	// Allocate space for views ahead of time prior to calling RegisterView.
	void AllocateViews(int32 NumViews);

	const TRDGUniformBufferRef<FInstanceCullingGlobalUniforms> GetDummyInstanceCullingUniformBuffer() const { return DummyUniformBuffer; }
	
	static bool AllowBatchedBuildRenderingCommands(const FGPUScene& GPUScene);

	/**
	 * Add a deferred, batched, gpu culling pass. Each batch represents a BuildRenderingCommands call from a mesh pass.
	 * Batches are collected as we walk through the main render setup and call BuildRenderingCommands, and are processed when RDG Execute or Drain is called.
	 * This implicitly ends the deferred context, so if Drain is used, it should be paired with a new call to BeginDeferredCulling.
	 * Can be called multiple times, and will collect subsequent BuildRenderingCommands. Care must be taken that the views referenced in the build rendering commands
	 * have been registered before BeginDeferredCulling.
	 * Calls FlushRegisteredViews that uploads the registered views to the GPU.
	 */
	void BeginDeferredCulling(FRDGBuilder& GraphBuilder);


	/** Whether we are actively batching GPU instance culling work. */
	bool IsDeferredCullingActive() const { return DeferredContext != nullptr; }

	// Reference to a buffer owned by FInstanceCullingOcclusionQueryRenderer
	FRDGBufferRef InstanceOcclusionQueryBuffer = {};
	EPixelFormat InstanceOcclusionQueryBufferFormat = PF_Unknown;

	TArray<TRefCountPtr<IPooledRenderTarget>> ViewPrevHZBs;

	// to support merging of multiple contexts with different HZBs, we use separate bins (LoadBalancers) in the InstanceCullingDeferredContext
	// bin 0 is reserved for UnCulled batches. Bins >= 1 are for the different HZBs.
	int32 GetBinIndex(EBatchProcessingMode Mode, const TRefCountPtr<IPooledRenderTarget>& HZB);

	void SetDummyCullingParams(FRDGBuilder& GraphBuilder, FInstanceCullingDrawParams& Parameters);

private:
	friend class FInstanceCullingContext;
	
	// Polulated by FInstanceCullingContext::BuildRenderingCommandsDeferred, used to hold instance culling related data that needs to be passed around
	FInstanceCullingDeferredContext* DeferredContext = nullptr;

	FInstanceCullingManager() = delete;
	FInstanceCullingManager(FInstanceCullingManager &) = delete;

	const FScene& Scene;
	const FGPUScene& GPUScene;
	FSceneUniformBuffer& SceneUniforms;
	FRendererViewDataManager& ViewDataManager;

	bool bIsEnabled;

	TRDGUniformBufferRef<FInstanceCullingGlobalUniforms> DummyUniformBuffer;
};

