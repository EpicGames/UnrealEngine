// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceCulling/InstanceCullingMergedContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "RenderGraphBuilder.h"
#include "InstanceCullingDefinitions.h"

FInstanceCullingMergedContext::FInstanceCullingMergedContext(EShaderPlatform InShaderPlatform, bool bInMustAddAllContexts, int32 InNumBins)
	: ShaderPlatform(InShaderPlatform)
	, bMustAddAllContexts(bInMustAddAllContexts)
{
	// make sure we always have at least 2 bins (for UnCulled + Generic batching modes)
	check(InNumBins >= 2);

	LoadBalancers.SetNum(InNumBins);
	BatchInds.SetNum(InNumBins);
	TotalBatches.SetNumZeroed(InNumBins);
	TotalItems.SetNumZeroed(InNumBins);
}

int32 FInstanceCullingMergedContext::GetLoadBalancerIndex(EBatchProcessingMode Mode, const FBatchItem& BatchItem)
{
	int32 BinIndex = (Mode == EBatchProcessingMode::UnCulled) ? 0 : BatchItem.GenericBinIndex;

	check(BinIndex >= 0 && BinIndex < LoadBalancers.Num());
	
	return BinIndex;
}

void FInstanceCullingMergedContext::MergeBatches()
{
	for (FBatchItem& AsyncBatchItem : AsyncBatches)
	{
		AsyncBatchItem.Context->WaitForSetupTask();
		check(AsyncBatchItem.Context->DynamicInstanceIdOffset >= 0);
		check(AsyncBatchItem.Context->DynamicInstanceIdNum >= 0);

		AddBatchItem(AsyncBatchItem);
	}
	AsyncBatches.Reset();

	for (uint32 BinIndex = 0U; BinIndex < uint32(LoadBalancers.Num()); ++BinIndex)
	{
		LoadBalancers[BinIndex].ReserveStorage(TotalBatches[BinIndex], TotalItems[BinIndex]);
	}
	// Pre-size all arrays
	IndirectArgs.Empty(TotalIndirectArgs);
	DrawCommandDescs.Empty(TotalIndirectArgs);
	InstanceIdOffsets.Empty(TotalIndirectArgs);
	PayloadData.Empty(TotalPayloads);
	ViewIds.Empty(TotalViewIds);
	DrawCommandCompactionData.Empty(TotalCompactionDrawCommands);
	CompactionBlockDataIndices.Empty(TotalCompactionBlocks);

	BatchInfos.Reserve(Batches.Num());
	uint32 InstanceIdBufferOffset = 0U; // in buffer elements
	uint32 InstanceDataByteOffset = 0U;
	uint32 TempCompactionInstanceOffset = 0U;
	
	// Index that maps from each command to the corresponding batch - maybe not the utmost efficiency
	for (int32 BatchIndex = 0; BatchIndex < Batches.Num(); ++BatchIndex)
	{
		const FBatchItem& BatchItem = Batches[BatchIndex];
		const FInstanceCullingContext& InstanceCullingContext = *BatchItem.Context;

		// Empty contexts should never be added to this list!
		check(InstanceCullingContext.HasCullingCommands());

		int32 BatchInfoIndex = BatchInfos.Num();
		FContextBatchInfoPacked& BatchInfo = BatchInfos.AddDefaulted_GetRef();

		BatchInfo.IndirectArgsOffset = IndirectArgs.Num();
		//BatchInfo.NumIndirectArgs = InstanceCullingContext.IndirectArgs.Num();
		check(InstanceCullingContext.DrawCommandDescs.Num() == InstanceCullingContext.IndirectArgs.Num());
		IndirectArgs.Append(InstanceCullingContext.IndirectArgs);
		DrawCommandDescs.Append(InstanceCullingContext.DrawCommandDescs);

		BatchInfo.PayloadDataOffset = PayloadData.Num();
		PayloadData.Append(InstanceCullingContext.PayloadData);

		check(InstanceCullingContext.InstanceIdOffsets.Num() == InstanceCullingContext.IndirectArgs.Num());
		InstanceIdOffsets.AddDefaulted(InstanceCullingContext.InstanceIdOffsets.Num());
		// TODO: perform offset on GPU
		// InstanceIdOffsets.Append(InstanceCullingContext.InstanceIdOffsets);
		for (int32 Index = 0; Index < InstanceCullingContext.InstanceIdOffsets.Num(); ++Index)
		{
			InstanceIdOffsets[BatchInfo.IndirectArgsOffset + Index] = InstanceCullingContext.InstanceIdOffsets[Index] + InstanceIdBufferOffset;
		}

		BatchInfo.ViewIdsOffset = ViewIds.Num();
		BatchInfo.NumViewIds_Flags = uint32(InstanceCullingContext.ViewIds.Num()) << INSTANCE_CULLING_FLAGS_BITS;
		if (InstanceCullingContext.PrevHZB.IsValid())
		{
			BatchInfo.NumViewIds_Flags |= INSTANCE_CULLING_FLAGS_ALLOW_OCCLUSION_CULLING;
		}
		if (EnumHasAnyFlags(InstanceCullingContext.Flags, EInstanceCullingFlags::DrawOnlySelected))
		{
			BatchInfo.NumViewIds_Flags |= INSTANCE_CULLING_FLAGS_DRAW_ONLY_SELECTED;
		}
		ViewIds.Append(InstanceCullingContext.ViewIds);

		check(InstanceCullingContext.DynamicInstanceIdOffset >= 0);
		check(InstanceCullingContext.DynamicInstanceIdNum >= 0);

		BatchInfo.DynamicInstanceIdOffset = InstanceCullingContext.DynamicInstanceIdOffset;
		BatchInfo.DynamicInstanceIdMax = InstanceCullingContext.DynamicInstanceIdOffset + InstanceCullingContext.DynamicInstanceIdNum;

		for (uint32 Mode = 0U; Mode < uint32(EBatchProcessingMode::Num); ++Mode)
		{
			int32 BinIndex = GetLoadBalancerIndex(static_cast<EBatchProcessingMode>(Mode), BatchItem);

			int32 StartIndex = BatchInds[BinIndex].Num();
			TInstanceCullingLoadBalancer<SceneRenderingAllocator>* MergedLoadBalancer = &LoadBalancers[BinIndex];

			BatchInfo.ItemDataOffset[Mode] = MergedLoadBalancer->GetItems().Num();
			FInstanceProcessingGPULoadBalancer* LoadBalancer = InstanceCullingContext.LoadBalancers[Mode];
			LoadBalancer->FinalizeBatches();

			// UnCulled bucket is used for a single instance mode
			check(EBatchProcessingMode(Mode) != EBatchProcessingMode::UnCulled || LoadBalancer->HasSingleInstanceItemsOnly());

			BatchInds[BinIndex].AddDefaulted(LoadBalancer->GetBatches().Num());

			MergedLoadBalancer->AppendData(*LoadBalancer);
			for (int32 Index = StartIndex; Index < BatchInds[BinIndex].Num(); ++Index)
			{
				BatchInds[BinIndex][Index] = BatchInfoIndex;
			}
		}
		const uint32 BatchTotalInstances = InstanceCullingContext.TotalInstances * InstanceCullingContext.ViewIds.Num();
		const uint32 BatchTotalDraws = InstanceCullingContext.InstanceIdOffsets.Num();

		FInstanceCullingDrawParams& Result = *BatchItem.Result;
		Result.InstanceDataByteOffset = InstanceDataByteOffset;
		Result.IndirectArgsByteOffset = BatchInfo.IndirectArgsOffset * FInstanceCullingContext::IndirectArgsNumWords * sizeof(uint32);

		BatchInfo.InstanceDataWriteOffset = InstanceIdBufferOffset;

		// Append the compaction data, but fix up the offsets for the batch
		BatchInfo.CompactionDataOffset = DrawCommandCompactionData.Num();
		const uint32 CompactionBlockOffset = CompactionBlockDataIndices.Num();
		for (auto CompactionData : InstanceCullingContext.DrawCommandCompactionData)
		{			
			CompactionData.BlockOffset += CompactionBlockOffset;
			CompactionData.IndirectArgsIndex += BatchInfo.IndirectArgsOffset;
			CompactionData.SrcInstanceIdOffset += TempCompactionInstanceOffset;
			CompactionData.DestInstanceIdOffset += InstanceIdBufferOffset;
			DrawCommandCompactionData.Add(CompactionData);
		}
		for (uint32 CompactionDataIndex : InstanceCullingContext.CompactionBlockDataIndices)
		{
			CompactionBlockDataIndices.Add(CompactionDataIndex + BatchInfo.CompactionDataOffset);
		}
		TempCompactionInstanceOffset += InstanceCullingContext.NumCompactionInstances;

		// Advance offset into instance ID and per-instance buffer
		InstanceIdBufferOffset += InstanceCullingContext.GetInstanceIdNumElements();
		InstanceDataByteOffset += InstanceCullingContext.StepInstanceDataOffsetBytes(BatchTotalDraws);
	}
}

void FInstanceCullingMergedContext::AddBatch(FRDGBuilder& GraphBuilder, FInstanceCullingContext* Context, FInstanceCullingDrawParams* InstanceCullingDrawParams)
{
	checkfSlow(Batches.FindByPredicate([InstanceCullingDrawParams](const FBatchItem& Item) { return Item.Result == InstanceCullingDrawParams; }) == nullptr, TEXT("Output draw paramters registered twice."));

	const bool bOcclusionCullInstances = Context->PrevHZB.IsValid() && FInstanceCullingContext::IsOcclusionCullingEnabled();

	// Resolve the bin index based on the PrevHZB. Bin 0 is reserved for UnCulled batches, every other bin is for each HZB.
	// Generic batches with a null HZB will go in bin 1, together with the ones associated to the first HZB.
	int32 BinIndex = 1;
	if (bOcclusionCullInstances)
	{
		BinIndex = Context->InstanceCullingManager->GetBinIndex(EBatchProcessingMode::Generic, Context->PrevHZB);

		// make sure that this Context's PrevHZB is registered correctly
		check(BinIndex > 0);
	}

	if (Context->SyncPrerequisitesFunc)
	{
		AsyncBatches.Add(FBatchItem{ Context, InstanceCullingDrawParams, BinIndex });

	}
	else
	{
		AddBatchItem(FBatchItem{ Context, InstanceCullingDrawParams, BinIndex });
	}

}

void FInstanceCullingMergedContext::AddBatchItem(const FBatchItem& BatchItem)
{
	const FInstanceCullingContext* Context = BatchItem.Context;
	if (Context->HasCullingCommands())
	{
		Batches.Add(BatchItem);

		// Accumulate the totals so the deferred processing can pre-size the arrays
		for (uint32 Mode = 0U; Mode < uint32(EBatchProcessingMode::Num); ++Mode)
		{
			Context->LoadBalancers[Mode]->FinalizeBatches();

			int32 BinIndex = GetLoadBalancerIndex(static_cast<EBatchProcessingMode>(Mode), BatchItem);

			TotalBatches[BinIndex] += Context->LoadBalancers[Mode]->GetBatches().Max();
			TotalItems[BinIndex] += Context->LoadBalancers[Mode]->GetItems().Max();
		}
#if DO_CHECK
		for (int32 ViewId : Context->ViewIds)
		{
			checkf(NumCullingViews < 0 || ViewId < NumCullingViews, TEXT("Attempting to defer a culling context that references a view that has not been uploaded yet."));
		}
#endif 

		TotalIndirectArgs += Context->IndirectArgs.Num();
		TotalPayloads += Context->PayloadData.Num();
		TotalViewIds += Context->ViewIds.Num();
		InstanceIdBufferElements += Context->GetInstanceIdNumElements();
		TotalInstances += Context->TotalInstances;
		TotalCompactionDrawCommands += Context->DrawCommandCompactionData.Num();
		TotalCompactionBlocks += Context->CompactionBlockDataIndices.Num();
		TotalCompactionInstances += Context->NumCompactionInstances;
	}
#if DO_CHECK
	else
	{
		check(!bMustAddAllContexts);
	}
#endif 
}