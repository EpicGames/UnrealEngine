// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingLoadBalancer.h"

class FInstanceCullingMergedContext
{
public:
	FInstanceCullingMergedContext(EShaderPlatform InShaderPlatform, bool bInMustAddAllContexts = false, int32 InNumBins = 2);
	
	struct FBatchItem
	{
		FInstanceCullingContext* Context = nullptr;
		FInstanceCullingDrawParams* Result = nullptr;
		// EBatchProcessingMode::Generic batches are put in bins, based on their view's PrevHZB
		int32 GenericBinIndex = -1;
	};

	// Info about a batch of culling work produced by a context, when part of a batched job
	// Store once per context, provides start offsets to commands/etc for the context.
	struct FContextBatchInfoPacked
	{
		uint32 IndirectArgsOffset;
		uint32 InstanceDataWriteOffset;
		uint32 PayloadDataOffset;
		uint32 CompactionDataOffset;
		uint32 ViewIdsOffset;
		uint32 NumViewIds_Flags;
		uint32 DynamicInstanceIdOffset;
		uint32 DynamicInstanceIdMax;
		uint32 ItemDataOffset[uint32(EBatchProcessingMode::Num)];
	};

	/** Bin 0 is used for UnCulled batches, Culled ones go in bins >= 1. */
	static constexpr uint32 FirstGenericBinIndex = 1;

	/** Batches of GPU instance culling input data. */
	TArray<FBatchItem, SceneRenderingAllocator> Batches;

	/** Async (and thus added as to the above as late as possible) Batches of GPU instance culling input data. */
	TArray<FBatchItem, SceneRenderingAllocator> AsyncBatches;

	/** 
	 * Merged data, derived in MergeBatches(), follows.
	 */
	TArray<int32, SceneRenderingAllocator> ViewIds;
	//TArray<FMeshDrawCommandInfo, SceneRenderingAllocator> MeshDrawCommandInfos;
	TArray<FRHIDrawIndexedIndirectParameters, SceneRenderingAllocator> IndirectArgs;
	TArray<FUintVector2, SceneRenderingAllocator> DrawCommandDescs;
	TArray<FInstanceCullingContext::FPayloadData, SceneRenderingAllocator> PayloadData;
	TArray<uint32, SceneRenderingAllocator> InstanceIdOffsets;
	TArray<FInstanceCullingContext::FCompactionData, SceneRenderingAllocator> DrawCommandCompactionData;
	TArray<uint32, SceneRenderingAllocator> CompactionBlockDataIndices;	

	// these are the actual BINS, so we would need one per HZB
	TArray<TInstanceCullingLoadBalancer<SceneRenderingAllocator>, SceneRenderingAllocator> LoadBalancers;
	TArray<TArray<uint32, SceneRenderingAllocator>, SceneRenderingAllocator> BatchInds; // TODO: rename to ContextInds
	TArray<FContextBatchInfoPacked, SceneRenderingAllocator> BatchInfos;

	EShaderPlatform ShaderPlatform = SP_NumPlatforms;
	// if true, the contexts that are supplied through calling AddBatch must all have an 1:1 entry in the resulting merged Batches array
	// this adds a check to prevent empty contexts from being added (!HasCullingCommands()).
	bool bMustAddAllContexts = false;
	// Counters to sum up all sizes to facilitate pre-sizing
	uint32 InstanceIdBufferElements = 0U;
	// preallocate 5 to cover all scenarios up to UnCulled bin + 4 HZBs in case of 4 primary views
	TArray<int32, TInlineAllocator<5>> TotalBatches;
	TArray<int32, TInlineAllocator<5>> TotalItems;
	int32 TotalIndirectArgs = 0;
	int32 TotalPayloads = 0;
	int32 TotalViewIds = 0;
	int32 TotalInstances = 0;
	int32 TotalCompactionDrawCommands = 0;
	int32 TotalCompactionBlocks = 0;
	int32 TotalCompactionInstances = 0;

	int32 NumCullingViews = 0;

	// Merge the queued batches and populate the derived data.
	void MergeBatches();


	void AddBatch(FRDGBuilder& GraphBuilder, FInstanceCullingContext* Context, FInstanceCullingDrawParams* InstanceCullingDrawParams);

private:
	void AddBatchItem(const FBatchItem& BatchItem);

	int32 GetLoadBalancerIndex(EBatchProcessingMode Mode, const FBatchItem& BatchItem);

};