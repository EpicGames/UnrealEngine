// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstanceCullingOcclusionQuery.h"

#include "Containers/ArrayView.h"
#include "Containers/ResourceArray.h"
#include "GPUScene.h"
#include "GlobalShader.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "RHIAccess.h"
#include "RHIFeatureLevel.h"
#include "RHIGlobals.h"
#include "RHIResourceUtils.h"
#include "RHIShaderPlatform.h"
#include "RHIStaticStates.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "SceneRendering.h"
#include "ScenePrivate.h"
#include "UnifiedBuffer.h"
#include "HAL/IConsoleManager.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

static TAutoConsoleVariable<int32> CVarInstanceCullingOcclusionQueries(
	TEXT("r.InstanceCulling.OcclusionQueries"),
	0,
	TEXT("EXPERIMENTAL: Use per-instance software occlusion queries to perform less conservative visibility test than what's possible with HZB alone"),
	ECVF_RenderThreadSafe | ECVF_Preview);

static int32 GInstanceCullingUseLoadBalancer = 1;
static FAutoConsoleVariableRef CVarInstanceCullingUseLoadBalancer(
	TEXT("r.InstanceCulling.UseLoadBalancer"),
	GInstanceCullingUseLoadBalancer,
	TEXT("Prefer to use UseLoadBalancer"),
	ECVF_RenderThreadSafe);

struct FInstanceCullingOcclusionQueryDeferredContext;

namespace
{




static EPixelFormat GetPreferredVisibilityMaskFormat()
{
	EPixelFormat PossibleFormats[] =
	{
		PF_R8_UINT,  // may be available if typed UAV load/store is supported on current hardware
		PF_R32_UINT, // guaranteed to be supported
	};

	for (EPixelFormat Format : PossibleFormats)
	{
		EPixelFormatCapabilities Capabilities = GPixelFormats[Format].Capabilities;
		if (EnumHasAllFlags(Capabilities, EPixelFormatCapabilities::TypedUAVLoad | EPixelFormatCapabilities::TypedUAVStore))
		{
			return Format;
		}
	}

	return PF_Unknown;
}

}

/*
* Prepares indirect draw parameters for per-instance per-pixel occlusion query rendering pass.
*/
class FInstanceCullingOcclusionQueryCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInstanceCullingOcclusionQueryCS);
	SHADER_USE_PARAMETER_STRUCT(FInstanceCullingOcclusionQueryCS, FGlobalShader);

public:

	class FMultiView : SHADER_PERMUTATION_BOOL("DIM_MULTI_VIEW");
	class FUseLoadBalancerDim : SHADER_PERMUTATION_BOOL("USE_LOAD_BALANCER");
	using FPermutationDomain = TShaderPermutationDomain<FMultiView, FUseLoadBalancerDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderSRVs(Parameters.Platform);
	}

	static constexpr int32 NumThreadsPerGroup = 64;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		
		// Currently, instance compaction is not supported on mobile platforms
		if (PermutationVector.Get<FUseLoadBalancerDim>())
		{
			FInstanceProcessingGPULoadBalancer::SetShaderDefines(OutEnvironment);
		}

		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP_DEFAULT"), NumThreadsPerGroup);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUSceneParameters)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint32>, OutIndirectArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint32>, OutInstanceIdBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWVisibilityMask) // One uint8/32 per instance (0 if instance is culled, non-0 otherwise)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint32>, InstanceIdBuffer)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceProcessingGPULoadBalancer::FShaderParameters, LoadBalancerParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER(float, OcclusionSlop)
		SHADER_PARAMETER(int32, NumInstances)
		SHADER_PARAMETER(uint32, ViewMask)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FInstanceCullingOcclusionQueryCS, "/Engine/Private/InstanceCulling/InstanceCullingOcclusionQuery.usf", "MainCS", SF_Compute);

class FInstanceCullingOcclusionQueryVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInstanceCullingOcclusionQueryVS);
	SHADER_USE_PARAMETER_STRUCT(FInstanceCullingOcclusionQueryVS, FGlobalShader);

public:

	class FMultiView : SHADER_PERMUTATION_BOOL("DIM_MULTI_VIEW");
	using FPermutationDomain = TShaderPermutationDomain<FMultiView>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderSRVs(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUSceneParameters)
		RDG_BUFFER_ACCESS(IndirectDrawArgsBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint32>, InstanceIdBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWVisibilityMask) // One uint8/32 per instance (0 if instance is culled, non-0 otherwise)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER(float, OcclusionSlop)
		SHADER_PARAMETER(uint32, ViewMask)
	END_SHADER_PARAMETER_STRUCT()
};

class FInstanceCullingOcclusionQueryPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInstanceCullingOcclusionQueryPS);
	SHADER_USE_PARAMETER_STRUCT(FInstanceCullingOcclusionQueryPS, FGlobalShader);

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderSRVs(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWVisibilityMask) // One uint8/32 per instance (0 if instance is culled, non-0 otherwise)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FInstanceCullingOcclusionQueryVS, "/Engine/Private/InstanceCulling/InstanceCullingOcclusionQuery.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FInstanceCullingOcclusionQueryPS, "/Engine/Private/InstanceCulling/InstanceCullingOcclusionQuery.usf", "MainPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FOcclusionInstanceCullingParameters,)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingOcclusionQueryVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingOcclusionQueryPS::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FInstanceCullingOcclusionQueryBox : public FRenderResource
{
public:
	FBufferRHIRef IndexBuffer;
	FBufferRHIRef VertexBuffer;
	FVertexDeclarationRHIRef VertexDeclaration;

	// Destructor
	virtual ~FInstanceCullingOcclusionQueryBox() {}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		static const uint16 BoxIndexBufferData[] =
		{
			// Tri list
			0, 1, 2, 0, 2, 3,
			4, 5, 6, 4, 6, 7,
			1, 4, 7, 1, 7, 2,
			5, 0, 3, 5, 3, 6,
			5, 4, 1, 5, 1, 0,
			3, 2, 7, 3, 7, 6,
			// Line list
			0, 1, 0, 3, 0, 5,
			7, 2, 7, 6, 7, 4,
			3, 2, 1, 2, 3, 6,
			5, 6, 5, 4, 1, 4
		};

		static const FVector3f BoxVertexBufferData[] =
		{
			FVector3f(-1.0f, +1.0f, +1.0f),
			FVector3f(+1.0f, +1.0f, +1.0f),
			FVector3f(+1.0f, -1.0f, +1.0f),
			FVector3f(-1.0f, -1.0f, +1.0f),
			FVector3f(+1.0f, +1.0f, -1.0f),
			FVector3f(-1.0f, +1.0f, -1.0f),
			FVector3f(-1.0f, -1.0f, -1.0f),
			FVector3f(+1.0f, -1.0f, -1.0f),
		};

		IndexBuffer = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FInstanceCullingOcclusionQueryBox_IndexBuffer"), MakeConstArrayView(BoxIndexBufferData));
		VertexBuffer = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FInstanceCullingOcclusionQueryBox_VertexBuffer"), MakeConstArrayView(BoxVertexBufferData));

		FVertexDeclarationElementList VertexDeclarationElements;
		VertexDeclarationElements.Add(FVertexElement(0, 0, VET_Float3, 0, 12));
		VertexDeclaration = PipelineStateCache::GetOrCreateVertexDeclaration(VertexDeclarationElements);
	}

	virtual void ReleaseRHI() override
	{
		IndexBuffer.SafeRelease();
		VertexBuffer.SafeRelease();
		VertexDeclaration.SafeRelease();
	}
};

TGlobalResource<FInstanceCullingOcclusionQueryBox> GInstanceCullingOcclusionQueryBox;

static void RenderInstanceOcclusionCulling(
	FRHICommandList& RHICmdList,
	FViewInfo& View,
	FOcclusionInstanceCullingParameters* PassParameters,
	bool bMultiView)
{
	FInstanceCullingOcclusionQueryVS::FPermutationDomain VSPermutationVector;
	VSPermutationVector.Set<FInstanceCullingOcclusionQueryVS::FMultiView>(bMultiView);
	TShaderMapRef<FInstanceCullingOcclusionQueryVS> VertexShader(View.ShaderMap, VSPermutationVector);

	TShaderMapRef<FInstanceCullingOcclusionQueryPS> PixelShader(View.ShaderMap);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	FIntVector4 ViewRect = FIntVector4(View.ViewRect.Min.X, View.ViewRect.Min.Y, View.ViewRect.Max.X, View.ViewRect.Max.Y);
	RHICmdList.SetViewport(ViewRect.X, ViewRect.Y, 0.0f, ViewRect.Z, ViewRect.W, 1.0f);

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GInstanceCullingOcclusionQueryBox.VertexDeclaration;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI(); // Depth test, no write
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI(); // Blend state does not matter, as we are not writing to render targets
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

	SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

	RHICmdList.SetStreamSource(0, GInstanceCullingOcclusionQueryBox.VertexBuffer, 0);

	FRDGBufferRef IndirectArgsBuffer = PassParameters->VS.IndirectDrawArgsBuffer;
	IndirectArgsBuffer->MarkResourceAsUsed();

	RHICmdList.DrawIndexedPrimitiveIndirect(GInstanceCullingOcclusionQueryBox.IndexBuffer, IndirectArgsBuffer->GetRHI(), 0);
}


/*
* Structure to compute data that's not available on the rendering thread during RDG setup.
* In particular, we want to wait for visible mesh draw commands as late as possible.
*/
struct FInstanceCullingOcclusionQueryDeferredContext
{
	FInstanceCullingOcclusionQueryDeferredContext(const FViewInfo* InView, int32 InNumGPUSceneInstances, EMeshPass::Type InMeshPass, FInstanceCullingContext* InInstanceCullingContext)
		: View(InView)
		, NumGPUSceneInstances(InNumGPUSceneInstances)
		, MeshPass(InMeshPass)
		, InstanceCullingContext(InInstanceCullingContext)
	{
	}

	static FORCEINLINE bool IsRelevantCommand(const FVisibleMeshDrawCommand& VisibleCommand)
	{
		// There may be multiple visible mesh draw commands that refer to the same instance when GPU-based LOD selection is used.
		// This filter is designed to remove the duplicates, keeping only the "authoritative" instance.
		// TODO: a less implicit mechanism would be welcome here, such as a dedicated flag.
		const EMeshDrawCommandCullingPayloadFlags Flags = VisibleCommand.CullingPayloadFlags;
		const bool bCompatibleFlags = Flags == EMeshDrawCommandCullingPayloadFlags::Default
			|| Flags == EMeshDrawCommandCullingPayloadFlags::MinScreenSizeCull;

		// Only commands with HasPrimitiveIdStreamIndex are compatible with GPU Instance Culling
		const bool bSupportsGPUSceneInstancing = EnumHasAnyFlags(VisibleCommand.Flags, EFVisibleMeshDrawCommandFlags::HasPrimitiveIdStreamIndex);

		// NumPrimitives is 0 if mesh draw command uses IndirectArgs
		// This path is currently not implemented/supported by oclcusion query culling.
		// Commands that use instance runs are currently not supported.
		return bCompatibleFlags
			&& bSupportsGPUSceneInstancing
			&& VisibleCommand.PrimitiveIdInfo.InstanceSceneDataOffset != INDEX_NONE
			&& VisibleCommand.NumRuns == 0;
	};

	static FORCEINLINE uint32 GetCommandNumInstances(const FVisibleMeshDrawCommand& VisibleMeshDrawCommand, const FScene *Scene)
	{
		const bool bFetchInstanceCountFromScene = EnumHasAnyFlags(VisibleMeshDrawCommand.Flags, EFVisibleMeshDrawCommandFlags::FetchInstanceCountFromScene);
		if (bFetchInstanceCountFromScene)
		{
			check(Scene != nullptr);
			check(!VisibleMeshDrawCommand.PrimitiveIdInfo.bIsDynamicPrimitive);
			return uint32(Scene->Primitives[VisibleMeshDrawCommand.PrimitiveIdInfo.ScenePrimitiveId]->GetNumInstanceSceneDataEntries());
		}
		return VisibleMeshDrawCommand.MeshDrawCommand->NumInstances;
	}

	void Execute()
	{
		if (bFunctionExecuted)
		{
			return;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FInstanceCullingOcclusionQueryDeferredContext::Execute);

		bFunctionExecuted = true;

		if (!View->ParallelMeshDrawCommandPasses[MeshPass])
		{
			return;
		}

		FParallelMeshDrawCommandPass& MeshDrawCommandPass = *View->ParallelMeshDrawCommandPasses[MeshPass];

		// Execute() is expected to run late enough to not stall here.
		// If it does happen, then we may have to move the render pass to later point in the frame.
		MeshDrawCommandPass.WaitForSetupTask(); 

		if (InstanceCullingContext != nullptr)
		{
			InstanceProcessingGPULoadBalancer = InstanceCullingContext->LoadBalancers[int(EBatchProcessingMode::Generic)];
			bValid = (InstanceProcessingGPULoadBalancer != nullptr);
			static FInstanceProcessingGPULoadBalancer Dummy;
			// Always provide a load balancer so that CreateLoadBalancerGPUDataDeferred doesn't crash. bValid=false will skip the dispatch
			if (!bValid)
			{
				InstanceProcessingGPULoadBalancer = &Dummy;
			}

			// in case something goes wrong: we will skip the compute since bValid won't be true and we will fill up the data from VisibleInstanceIds
			AlignedNumInstances = FInstanceCullingOcclusionQueryCS::NumThreadsPerGroup;
			VisibleInstanceIds.SetNumZeroed(AlignedNumInstances);
			InstanceProcessingGPULoadBalancer->FinalizeBatches();
			FIntVector LoadBalancerNumThreadGroups = InstanceProcessingGPULoadBalancer->GetWrappedCsGroupCount();
			// Needed to allocate the buffer holding the instance ids after the culling pass, see DeferredAlignedNumInstancesOutputCulling
			AlignedNumInstances = LoadBalancerNumThreadGroups.X * LoadBalancerNumThreadGroups.Y * LoadBalancerNumThreadGroups.Z * FInstanceCullingOcclusionQueryCS::NumThreadsPerGroup;
			
			return;
		}

		const FMeshCommandOneFrameArray& VisibleMeshDrawCommands = MeshDrawCommandPass.GetMeshDrawCommands();

		const FScene *Scene = View->Family->Scene->GetRenderScene();

		NumInstances = CountVisibleInstances(VisibleMeshDrawCommands, Scene);

		NumThreadGroups = FComputeShaderUtils::GetGroupCount(NumInstances, FInstanceCullingOcclusionQueryCS::NumThreadsPerGroup);

		const int32 MaxSupportedInstances = GRHIGlobals.MaxDispatchThreadGroupsPerDimension.X * FInstanceCullingOcclusionQueryCS::NumThreadsPerGroup;
		if (!ensureMsgf(NumThreadGroups.X * FInstanceCullingOcclusionQueryCS::NumThreadsPerGroup <= MaxSupportedInstances,
			TEXT("Number of instances (%d) is greater than currently supported by FInstanceCullingOcclusionQueryRenderer (%d). ")
			TEXT("Per-instance occlusion queries will be disabled. ")
			TEXT("Increase FInstanceCullingOcclusionQueryCS::NumThreadsPerGroup or implement wrapped group count support."),
			NumInstances, MaxSupportedInstances))
		{
			return;
		}

		// Align buffer sizes to ensure each thread in the thread group has a valid slot to write without introducing bounds checks
		AlignedNumInstances = NumThreadGroups.X * FInstanceCullingOcclusionQueryCS::NumThreadsPerGroup;

		if (AlignedNumInstances == 0)
		{
			return;
		}

		const uint32 DynamicPrimitiveInstanceOffset = View->DynamicPrimitiveCollector.GetInstanceSceneDataOffset();

		FillVisibleInstanceIds(VisibleMeshDrawCommands, DynamicPrimitiveInstanceOffset, Scene);

		bValid = true;
	}

	uint32 CountVisibleInstances(const FMeshCommandOneFrameArray& VisibleMeshDrawCommands, const FScene *Scene) const 
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInstanceCullingOcclusionQueryDeferredContext::CountVisibleInstances);

		uint32 Result = 0;

		for (const FVisibleMeshDrawCommand& VisibleCommand : VisibleMeshDrawCommands)
		{
			if (!IsRelevantCommand(VisibleCommand))
			{
				continue;
			}
			Result += GetCommandNumInstances(VisibleCommand, Scene);
		}

		return Result;
	}

	void FillVisibleInstanceIds(const FMeshCommandOneFrameArray& VisibleMeshDrawCommands, const uint32 DynamicPrimitiveInstanceOffset, const FScene *Scene)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FInstanceCullingOcclusionQueryDeferredContext::FillVisibleInstanceIds);

		check(AlignedNumInstances != 0);

		// Write output data directly, bypassing TArray::Add overhead (resize branch, etc.)
		VisibleInstanceIds.SetNumUninitialized(AlignedNumInstances);
		uint32* ResultData = VisibleInstanceIds.GetData();
		uint32* ResultCursor = ResultData;
		
		for (const FVisibleMeshDrawCommand& VisibleCommand : VisibleMeshDrawCommands)
		{
			if (!IsRelevantCommand(VisibleCommand))
			{
				continue;
			}
			uint32 CommandNumInstances = GetCommandNumInstances(VisibleCommand, Scene);
			if (CommandNumInstances == 0u)
			{
				continue;
			}

			uint32 InstanceBaseIndex = VisibleCommand.PrimitiveIdInfo.InstanceSceneDataOffset;
			if (VisibleCommand.PrimitiveIdInfo.bIsDynamicPrimitive)
			{
				InstanceBaseIndex += DynamicPrimitiveInstanceOffset;
			}

			check(InstanceBaseIndex + CommandNumInstances <= uint32(NumGPUSceneInstances));

			for (uint32 i = 0; i < CommandNumInstances; ++i)
			{
				*ResultCursor = InstanceBaseIndex + i;
				++ResultCursor;
			}
		}

		for (int32 i = NumInstances; i < AlignedNumInstances; ++i)
		{
			*ResultCursor = 0;
			++ResultCursor;
		}

		check(ResultCursor == ResultData + AlignedNumInstances);
	}

	FRDGBufferNumElementsCallback DeferredAlignedNumInstancesOutputCulling()
	{
		return [Context = this]() -> uint32
		{
			Context->Execute();
			return Context->AlignedNumInstances;
		};
	}

	FRDGBufferNumElementsCallback DeferredNumInstanceIdData()
	{
		return [Context = this]() -> uint32
		{
			Context->Execute();
			return Context->VisibleInstanceIds.Num();
		};
	}

	FRDGBufferInitialDataCallback DeferredInstanceIdData()
	{
		return [Context = this]() -> const void*
			{
				Context->Execute();
				return Context->VisibleInstanceIds.GetData();
			};
	}

	FRDGBufferInitialDataSizeCallback DeferredInstanceIdDataSize()
	{
		return [Context = this]() -> uint64
			{
				Context->Execute();
				return Context->VisibleInstanceIds.Num() * Context->VisibleInstanceIds.GetTypeSize();
			};
	}

	// Execute function may be called multiple times, but we only want to run computations once
	bool bFunctionExecuted = false;

	// If this is false, then some late validation have failed and rendering should be skipped
	bool bValid = false;

	const FViewInfo* View = nullptr;
	int32 NumGPUSceneInstances = 0;
	EMeshPass::Type MeshPass = EMeshPass::Num;
	FInstanceCullingContext* InstanceCullingContext = nullptr;
	FInstanceProcessingGPULoadBalancer* InstanceProcessingGPULoadBalancer = nullptr;
	int32 NumInstances = 0;
	int32 AlignedNumInstances = 0;
	FIntVector NumThreadGroups = FIntVector::ZeroValue;

	TArray<uint32> VisibleInstanceIds;
};

static void CreateLoadBalancerGPUDataDeferred(FRDGBuilder& GraphBuilder, FInstanceCullingOcclusionQueryCS::FParameters* PassParameters, FInstanceCullingOcclusionQueryDeferredContext* DeferredContext)
{
	PassParameters->LoadBalancerParameters.BatchBuffer = GraphBuilder.CreateSRV(
		CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCullingLoadBalancer.Batches"), [DeferredContext]() 
			-> const TArray<FInstanceCullingLoadBalancerBase::FPackedBatch>& 
			{ 
				DeferredContext->Execute(); 
				return DeferredContext->InstanceProcessingGPULoadBalancer->GetBatches(); 
			}));

	PassParameters->LoadBalancerParameters.ItemBuffer = GraphBuilder.CreateSRV(
		CreateStructuredBuffer(GraphBuilder, TEXT("InstanceCullingLoadBalancer.Items"), [DeferredContext]() 
			-> const TArray<FInstanceCullingLoadBalancerBase::FPackedItem>& 
			{ 
				DeferredContext->Execute(); 
				return DeferredContext->InstanceProcessingGPULoadBalancer->GetItems(); 
			}));
}

uint32 FInstanceCullingOcclusionQueryRenderer::Render(
	FRDGBuilder& GraphBuilder,
	FGPUScene& GPUScene,
	FViewInfo& View)
{
	if (!IsCompatibleWithView(View))
	{
		return 0;
	}

	const uint32 ViewMask = FindOrAddViewSlot(View);

	if (ViewMask == 0)
	{
		// Silently fall back to no culling when we hit the limit of maximum supported views
		return 0;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FInstanceCullingOcclusionQueryRenderer::Render);

	// Whether to use shader permutation that preserves visibility bits corresponding to other views (slight extra cost)
	const bool bMultiView = CurrentRenderedViewIDs.Num() > 1;

	const int32 NumGPUSceneInstances = GPUScene.GetNumInstances();

	FInstanceCullingContext* InstanceCullingContext = nullptr;
	if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::BasePass]; Pass && GInstanceCullingUseLoadBalancer > 0)
	{
		// At this point in time, we don't have the guarantee that MeshDrawCommandPass is done. Only access stable members, not batches/items/mdcs
		InstanceCullingContext = Pass->GetInstanceCullingContext();
	}

	FInstanceCullingOcclusionQueryDeferredContext* DeferredContext = GraphBuilder.AllocObject<FInstanceCullingOcclusionQueryDeferredContext>(&View, NumGPUSceneInstances, 
		EMeshPass::BasePass, InstanceCullingContext);

	FRDGTextureRef DepthTexture = View.GetSceneTextures().Depth.Target;

	checkf(DepthTexture && IsHZBValid(View, EHZBType::FurthestHZB),
		TEXT("Occlusion query instance culling pass requires scene depth texture and HZB. See FInstanceCullingOcclusionQueryRenderer::IsCompatibleWithView()"));

	const FGPUSceneResourceParameters GPUSceneParameters = GPUScene.GetShaderParameters(GraphBuilder);

	const FIntPoint ViewRectSize = View.ViewRect.Size();

	EPixelFormat VisibilityMaskFormat = GetPreferredVisibilityMaskFormat();
	int32 VisibilityMaskStride = GPixelFormats[VisibilityMaskFormat].BlockBytes;

	// Create the result buffer on demand
	if (!CurrentInstanceOcclusionQueryBuffer)
	{
		const int32 AlignedNumGPUSceneInstances =
			FMath::DivideAndRoundUp(NumGPUSceneInstances, FInstanceCullingOcclusionQueryCS::NumThreadsPerGroup)
			* FInstanceCullingOcclusionQueryCS::NumThreadsPerGroup;

		CurrentInstanceOcclusionQueryBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateBufferDesc(VisibilityMaskStride, AlignedNumGPUSceneInstances),
			TEXT("FInstanceCullingOcclusionQueryRenderer_VisibleInstanceMask"));

		InstanceOcclusionQueryBufferFormat = VisibilityMaskFormat;

		AllocatedNumInstances = NumGPUSceneInstances;

		// Create a wide-format alias for the underlying resource for a more efficient clear
		FRDGBufferUAVRef UAV = GraphBuilder.CreateUAV(CurrentInstanceOcclusionQueryBuffer, PF_R32G32B32A32_UINT);
		AddClearUAVPass(GraphBuilder, UAV, 0xFFFFFFFF);
	}

	checkf(uint32(NumGPUSceneInstances) == AllocatedNumInstances, TEXT("Number of instances in GPUScene is not expected change to during the frame"));

	FRDGBufferRef VisibleInstanceMaskBuffer = CurrentInstanceOcclusionQueryBuffer;
	FRDGBufferUAVRef VisibilityMaskUAV = GraphBuilder.CreateUAV(VisibleInstanceMaskBuffer, VisibilityMaskFormat);

	FRDGBufferRef IndirectArgsBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndexedIndirectParameters>(1),
		TEXT("FInstanceCullingOcclusionQueryRenderer_IndirectArgsBuffer"));
	FRDGBufferUAVRef IndirectArgsUAV = GraphBuilder.CreateUAV(IndirectArgsBuffer, PF_R32_UINT);

	// Buffer of GPUScene instance indices to run occlusion queries for (input for setup CS)
	FRDGBufferRef SetupInstanceIdBuffer;

	// When using the GPU load balancer, the upload of the data holding instance ids happens below with InstanceProcessingGPULoadBalancer->Upload
	if (InstanceCullingContext == nullptr)
	{
		SetupInstanceIdBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1 /*real size is provided via callback later*/),
			TEXT("FInstanceCullingOcclusionQueryRenderer_SetupInstanceIdBuffer"), 
			DeferredContext->DeferredNumInstanceIdData());
		GraphBuilder.QueueBufferUpload(SetupInstanceIdBuffer,
									   DeferredContext->DeferredInstanceIdData(),
									   DeferredContext->DeferredInstanceIdDataSize());
	}
	else
	{
		SetupInstanceIdBuffer = GSystemTextures.GetDefaultBuffer(GraphBuilder, 4);
	}

	FRDGBufferSRVRef SetupInstanceIdBufferSRV = GraphBuilder.CreateSRV(SetupInstanceIdBuffer, PF_R32_UINT);

	// Buffer of GPUScene instance indices that passed the filtering in the setup CS pass and should be rendered in the subsequent graphics pass
	FRDGBufferRef InstanceIdBuffer = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1 /*real size is provided via callback later*/),
		TEXT("FInstanceCullingOcclusionQueryRenderer_InstanceIdBuffer"),
		DeferredContext->DeferredAlignedNumInstancesOutputCulling());

	FRDGBufferUAVRef InstanceIdUAV = GraphBuilder.CreateUAV(InstanceIdBuffer, PF_R32_UINT);
	FRDGBufferSRVRef InstanceIdSRV = GraphBuilder.CreateSRV(InstanceIdBuffer, PF_R32_UINT);

	AddClearUAVPass(GraphBuilder, IndirectArgsUAV, 0);

	// Compute pass to perform initial per-instance filtering and prepare instance list for per-pixel occlusion tests
	{
		FInstanceCullingOcclusionQueryCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FInstanceCullingOcclusionQueryCS::FParameters>();

		// FInstanceGPULoadBalancer uses the SceneRenderingAllocator which should keep data alive until the graphbuilder execution
		bool bUseGPULoadBalancer = false;
		if (InstanceCullingContext != nullptr)
		{
			CreateLoadBalancerGPUDataDeferred(GraphBuilder, PassParameters, DeferredContext);
			bUseGPULoadBalancer = true;
		}

		PassParameters->OutIndirectArgsBuffer = IndirectArgsUAV;
		PassParameters->OutInstanceIdBuffer = InstanceIdUAV;
		PassParameters->RWVisibilityMask = VisibilityMaskUAV;
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
		PassParameters->OcclusionSlop = OCCLUSION_SLOP;
		PassParameters->GPUSceneParameters = GPUSceneParameters;
		PassParameters->NumInstances = 0; // filled from DeferredContext later
		PassParameters->InstanceIdBuffer = SetupInstanceIdBufferSRV;
		PassParameters->ViewMask = ViewMask;

		FInstanceCullingOcclusionQueryCS::FPermutationDomain CSPermutationVector;
		CSPermutationVector.Set<FInstanceCullingOcclusionQueryCS::FMultiView>(bMultiView);
		CSPermutationVector.Set<FInstanceCullingOcclusionQueryCS::FUseLoadBalancerDim>(bUseGPULoadBalancer);
		TShaderMapRef<FInstanceCullingOcclusionQueryCS> ComputeShader(View.ShaderMap, CSPermutationVector);

		ClearUnusedGraphResources(ComputeShader, PassParameters);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("InstanceCullingOcclusionQueryRenderer_Setup"),
			PassParameters,
			ERDGPassFlags::Compute,
			[PassParameters, DeferredContext, ComputeShader](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			if (!DeferredContext->bValid)
			{
				return;
			}

			PassParameters->NumInstances = DeferredContext->NumInstances;

			FIntVector CullingNumThreadGroups(DeferredContext->NumThreadGroups);
			FInstanceProcessingGPULoadBalancer* DeferredContextInstanceProcessingGPULoadBalancer = DeferredContext->InstanceProcessingGPULoadBalancer;
			if (DeferredContextInstanceProcessingGPULoadBalancer)
			{
				PassParameters->LoadBalancerParameters.NumBatches = DeferredContextInstanceProcessingGPULoadBalancer->GetBatches().Num();
				PassParameters->LoadBalancerParameters.NumItems = DeferredContextInstanceProcessingGPULoadBalancer->GetItems().Num();
				CullingNumThreadGroups = DeferredContextInstanceProcessingGPULoadBalancer->GetWrappedCsGroupCount();
			}

			FComputeShaderUtils::Dispatch(
				RHICmdList,
				ComputeShader,
				*PassParameters,
				CullingNumThreadGroups);
		});
	}

	// Perform per-instance per-pixel occlusion tests by drawing bounding boxes that write into VisibleInstanceMaskBuffer slots for visible instances
	{
		FOcclusionInstanceCullingParameters* PassParameters = GraphBuilder.AllocParameters<FOcclusionInstanceCullingParameters>();

		PassParameters->VS.IndirectDrawArgsBuffer = IndirectArgsBuffer;
		PassParameters->VS.View = View.ViewUniformBuffer;
		PassParameters->VS.HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
		PassParameters->VS.OcclusionSlop = OCCLUSION_SLOP;
		PassParameters->VS.GPUSceneParameters = GPUSceneParameters;
		PassParameters->VS.InstanceIdBuffer = InstanceIdSRV;
		PassParameters->VS.ViewMask = ViewMask;
		PassParameters->VS.RWVisibilityMask = VisibilityMaskUAV;
		PassParameters->PS.RWVisibilityMask = VisibilityMaskUAV;
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthTexture,
			ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction,
			FExclusiveDepthStencil::DepthRead_StencilNop);


		GraphBuilder.AddPass(
			RDG_EVENT_NAME("InstanceCullingOcclusionQueryRenderer_Draw"),
			PassParameters, ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
			[PassParameters, DeferredContext, bMultiView, &View](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				if (!DeferredContext->bValid)
				{
					return;
				}

				RenderInstanceOcclusionCulling(RHICmdList, View, PassParameters, bMultiView);
			});
	}

	return ViewMask;
}

void FInstanceCullingOcclusionQueryRenderer::MarkInstancesVisible(FRDGBuilder& GraphBuilder, TConstArrayView<FGPUSceneInstanceRange> Ranges)
{
	if (!InstanceOcclusionQueryBuffer)
	{
		// Previous frame buffer does not exist, nothing to clear
		return;
	}

	EPixelFormat VisibilityMaskFormat = GetPreferredVisibilityMaskFormat();

	FRDGBufferRef Buffer = GraphBuilder.RegisterExternalBuffer(InstanceOcclusionQueryBuffer);

	// Consecutive uses of the UAV will run in parallel.
	// Allocating a unique RDG UAV here will still ensure that a barrier is inserted before the first dispatch.
	FRDGBufferUAVRef UAV = GraphBuilder.CreateUAV(Buffer, VisibilityMaskFormat, ERDGUnorderedAccessViewFlags::SkipBarrier);

	// NOTE: It is possible to make this more efficient using a specialized GPU scatter shader, if we see many small batches here in practice
	for (FGPUSceneInstanceRange Range : Ranges)
	{
		FMemsetResourceParams MemsetParams;
		MemsetParams.Value = 0xFFFFFFFF; // Mark instance visible in all views
		MemsetParams.Count = Range.NumInstanceSceneDataEntries;
		MemsetParams.DstOffset = Range.InstanceSceneDataOffset;
		MemsetResource(GraphBuilder, UAV, MemsetParams);
	}
}

void FInstanceCullingOcclusionQueryRenderer::EndFrame(FRDGBuilder& GraphBuilder)
{
	if (CurrentInstanceOcclusionQueryBuffer)
	{
		GraphBuilder.QueueBufferExtraction(CurrentInstanceOcclusionQueryBuffer, &InstanceOcclusionQueryBuffer, ERHIAccess::SRVMask);
		CurrentInstanceOcclusionQueryBuffer = {};
		AllocatedNumInstances = 0;
	}
	CurrentRenderedViewIDs.Empty();
}

uint32 FInstanceCullingOcclusionQueryRenderer::FindOrAddViewSlot(const FViewInfo& View)
{
	const uint32 ViewKey = View.GetViewKey();

	if (CurrentRenderedViewIDs.Num() < MaxViews && ViewKey != 0)
	{
		int32 Index = CurrentRenderedViewIDs.AddUnique(ViewKey);
		check(Index >= 0 && Index < MaxViews);
		return 1u << Index;
	}
	else
	{
		return 0;
	}
}

bool FInstanceCullingOcclusionQueryRenderer::IsCompatibleWithView(const FViewInfo& View)
{
	EPixelFormat VisibilityMaskFormat = GetPreferredVisibilityMaskFormat();
	return FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderSRVs(View.GetShaderPlatform())
		&& View.GetViewKey() != 0
		&& View.GetSceneTextures().Depth.Target
		&& IsHZBValid(View, EHZBType::FurthestHZB)
		&& VisibilityMaskFormat != PF_Unknown
		&& CVarInstanceCullingOcclusionQueries.GetValueOnRenderThread() != 0;
}

// Debugging utilities

class FInstanceCullingOcclusionQueryDebugVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInstanceCullingOcclusionQueryDebugVS);
	SHADER_USE_PARAMETER_STRUCT(FInstanceCullingOcclusionQueryDebugVS, FGlobalShader);

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderSRVs(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUSceneParameters)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, InstanceOcclusionQueryBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWVisibilityMask) // One uint8/32 per instance (0 if instance is culled, non-0 otherwise)
		SHADER_PARAMETER(float, OcclusionSlop)
		SHADER_PARAMETER(uint32, ViewMask)
	END_SHADER_PARAMETER_STRUCT()
};

class FInstanceCullingOcclusionQueryDebugPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FInstanceCullingOcclusionQueryDebugPS);
	SHADER_USE_PARAMETER_STRUCT(FInstanceCullingOcclusionQueryDebugPS, FGlobalShader);

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsVertexShaderSRVs(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FInstanceCullingOcclusionQueryDebugVS, "/Engine/Private/InstanceCulling/InstanceCullingOcclusionQuery.usf", "DebugMainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FInstanceCullingOcclusionQueryDebugPS, "/Engine/Private/InstanceCulling/InstanceCullingOcclusionQuery.usf", "DebugMainPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FOcclusionInstanceCullingDebugParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingOcclusionQueryDebugVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingOcclusionQueryDebugPS::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static void RenderInstanceOcclusionCullingDebug(
	FRHICommandList& RHICmdList,
	const FViewInfo& View,
	const FIntRect& ViewRect,
	FOcclusionInstanceCullingDebugParameters* PassParameters,
	int32 NumInstances)
{
	TShaderMapRef<FInstanceCullingOcclusionQueryDebugVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FInstanceCullingOcclusionQueryDebugPS> PixelShader(View.ShaderMap);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GInstanceCullingOcclusionQueryBox.VertexDeclaration;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();
	GraphicsPSOInit.DepthStencilAccess = FExclusiveDepthStencil::DepthWrite_StencilNop;
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_SourceAlpha, BF_DestAlpha>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.PrimitiveType = PT_LineList;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

	SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

	RHICmdList.SetStreamSource(0, GInstanceCullingOcclusionQueryBox.VertexBuffer, 0);

	RHICmdList.DrawIndexedPrimitive(GInstanceCullingOcclusionQueryBox.IndexBuffer, 0, 0, 24, 36, 12, NumInstances);
}

void FInstanceCullingOcclusionQueryRenderer::RenderDebug(FRDGBuilder& GraphBuilder, FGPUScene& GPUScene, const FViewInfo& View, const FIntRect& ViewRect, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth)
{
	if (!IsCompatibleWithView(View) || !InstanceOcclusionQueryBuffer)
	{
		return;
	}

	check(SceneColor);

	const uint32 ViewMask = FindOrAddViewSlot(View);

	FRDGBufferRef InstanceOcclusionQueryBufferRDG = GraphBuilder.RegisterExternalBuffer(InstanceOcclusionQueryBuffer);

	FRDGTextureRef DepthTexture = View.GetSceneTextures().Depth.Target;

	checkf(DepthTexture && IsHZBValid(View, EHZBType::FurthestHZB),
		TEXT("Occlusion query instance culling requires scene depth texture and HZB. See FInstanceCullingOcclusionQueryRenderer::IsCompatibleWithView()"));

	const int32 NumInstances = GPUScene.GetNumInstances();
	const FGPUSceneResourceParameters GPUSceneParameters = GPUScene.GetShaderParameters(GraphBuilder);

	FOcclusionInstanceCullingDebugParameters* PassParameters = GraphBuilder.AllocParameters<FOcclusionInstanceCullingDebugParameters>();

	PassParameters->VS.OcclusionSlop = OCCLUSION_SLOP;
	PassParameters->VS.View = View.ViewUniformBuffer;
	PassParameters->VS.GPUSceneParameters = GPUSceneParameters;
	PassParameters->VS.InstanceOcclusionQueryBuffer = GraphBuilder.CreateSRV(InstanceOcclusionQueryBufferRDG, InstanceOcclusionQueryBufferFormat);
	PassParameters->VS.HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
	PassParameters->VS.ViewMask = ViewMask;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColor, ERenderTargetLoadAction::ELoad);
	if (SceneDepth)
	{
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepth,
			ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction,
			FExclusiveDepthStencil::DepthWrite_StencilNop);
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("InstanceCullingOcclusionQueryRenderer_Draw"),
		PassParameters, ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
		[PassParameters, NumInstances, &View, ViewRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RenderInstanceOcclusionCullingDebug(RHICmdList, View, ViewRect, PassParameters, NumInstances);
		});
}

