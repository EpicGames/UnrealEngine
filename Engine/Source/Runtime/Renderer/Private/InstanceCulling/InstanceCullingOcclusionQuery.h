// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphFwd.h"
#include "RenderGraphResources.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"

class FRDGBuilder;
class FGPUScene;
class FViewInfo;
struct IPooledRenderTarget;
struct FGPUSceneInstanceRange;

/** 
* Implements accurate per-instance occlusion culling similar to FOcclusionFeedback, except using GPUScene.
*/
class FInstanceCullingOcclusionQueryRenderer
{
public:

	/**
	* Perform per-instance occlusion culling using HZB and software occlusion queries.
	* Returns view-specific bit mask that should be used when interpreting query result buffer, since it contains 1 bit per view.
	*/	
	uint32 Render(FRDGBuilder& GraphBuilder, FGPUScene& GPUScene, FViewInfo& View);

	/**
	* Render object bounding boxes, color-coded based on visibility
	*/
	void RenderDebug(FRDGBuilder& GraphBuilder, FGPUScene& GPUScene, const FViewInfo& View, const FIntRect& ViewRect, FRDGTextureRef SceneColor, FRDGTextureRef SceneDepth);

	/**
	* Mark blocks of occlusion query buffer entries as visible in all views.
	* Required, for example, when GPUScene instances are invalidated betwen frames,
	* i.e. when when GPUScene instance buffer slot is re-allocated.
	*/
	void MarkInstancesVisible(FRDGBuilder& GraphBuilder, TConstArrayView<FGPUSceneInstanceRange> Ranges);

	/**
	* Perform various book-keeping tasks that must run at the end of each frame
	* Saves view ID mapping data, extracts result buffer to be available next frame, etc.
	*/
	void EndFrame(FRDGBuilder& GraphBuilder);

	/**
	* One uint8 per instance in GPUScene, with 1 bit per view
	* Available when when instance culling is enabled and r.InstanceCulling.OcclusionQueries=1.
	* Contains data for *previous* frame. Assumes that GPUScene instance indices are consistent.
	*/
	TRefCountPtr<FRDGPooledBuffer> InstanceOcclusionQueryBuffer;

	/*
	* Format that should be used to create views for InstanceOcclusionQueryBuffer.
	* May be R8_UINT or R32_UINT, basd on current hardware capability.
	*/
	EPixelFormat InstanceOcclusionQueryBufferFormat = PF_Unknown;

	/*
	* Returns true if per-instance occlusion queries can be rendered for the view.
	*/
	bool IsCompatibleWithView(const FViewInfo& View);

private:

	/**
	* Allocates a slot in the output buffer for this view. Views are automatically registered during Render().
	* Returns bit mask with a single set bit that should be used to access the occlusion results.
	* Returns 0 if maximum number of supported views is reached, falling back to no-occlusion-query code path.
	*/
	uint32 FindOrAddViewSlot(const FViewInfo& View);

	FRDGBufferRef CurrentInstanceOcclusionQueryBuffer = {};

	static constexpr uint32 MaxViews = 8; // Visibility mask is stored as uint8
	TArray<uint32> CurrentRenderedViewIDs;

	uint32 AllocatedNumInstances = 0;
};
