// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightFunctionAtlas.h"

#include "HAL/IConsoleManager.h"
#include "LightSceneProxy.h"
#include "LightSceneInfo.h"
#include "LightRendering.h"
#include "Materials/MaterialRenderProxy.h"
#include "Materials/MaterialInterface.h"
#include "RenderGraphBuilder.h"
#include "SystemTextures.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "MegaLights/MegaLights.h"
#include "ShadowRendering.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Containers/HashTable.h"
#include "RenderUtils.h"
#include "VolumetricFog.h"
#include "RectLightSceneProxy.h"
#include "ScenePrivate.h"
#include "UnrealEngine.h"

DECLARE_GPU_STAT(LightFunctionAtlasGeneration);

/*

This LightFunctionAtlas stores light functions as 2D sub region of a texture2D atlas for all the views of a scene.
Each material is stored only onces, de-duplicated based on its unique ID. Material instances and MID are correctly handled separately.
Each slot stores a texture for a material that then can be applied on any light type. This only works if
 - only Tex Coord are used to generate the light function
 - only light/view direction are used to generate light function for point light
Later, we could have atlas slot for a light/material pair if needed. That could automatically detected for instance when a material is reading instance data. Or if a material is reading view/light dir or world position.

To avoid allocating SRV when using the atlas we use a constant buffer to store all atlas slot and light mapping data:
	- A single SRV is used, being the atlas Texture2D.
	- Otherwise a single constant buffer entry is used for each view, storing: AtlasSlotIndex=>{SubUVs} read for a Light LightIndex=>{FadeParams, AtlasSlotIndex, TranslatedWorlViewProjectionMatrix}

How to use the Atlas:
 - Add FLightFunctionAtlasGlobalParameters to your shader
 - #include "LightFunctionAtlas/LightFunctionAtlasCommon.usf" 
 - Call GetLocalLightFunctionCommon(DerivedParams.TranslatedWorldPosition, LightData.LightFunctionAtlasLightIndex); where LightData is a FDeferredLightData recovered from uniform or the light grid.

 What is next:
 - Super sample CVAR
 - Convert systems:
    - Path tracer
	- Ray tracing?

*/

static TAutoConsoleVariable<int32> CVarLightFunctionAtlas(
	TEXT("r.LightFunctionAtlas"),
	1,
	TEXT("Enable the light function atlas generation at runtime. The atlas will only be generated if other systems are using it at runtime."),
	ECVF_RenderThreadSafe);

// We do not dynamically scale allocated slot resolution for now.
static TAutoConsoleVariable<int32> CVarLightFunctionAtlasSlotResolution(
	TEXT("r.LightFunctionAtlas.SlotResolution"),
	128,
	TEXT("Experimental: The resolution of each atlas slot. If the resolution is too large for the target platforms, it will be reduced to fit."),
	ECVF_RenderThreadSafe);

// We do not dynamically scale allocated slot resolution for now.
static TAutoConsoleVariable<int32> CVarLightFunctionAtlasSize(
	TEXT("r.LightFunctionAtlas.Size"),
	4,
	TEXT("Experimental: The default size (atlas slot count) for each dimension of the the 2D texture atlas. Maximum value is 16 since we only allow up to 16x16=256 light functions."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLightFunctionAtlasMaxLightCount(
	TEXT("r.LightFunctionAtlas.MaxLightCount"),
	-1,
	TEXT("Experimental: Clamp the number of lights that can sample light function atlas. -1 means unlimited light count."),
	ECVF_RenderThreadSafe);



//////////////////////////////////////////////////////////////////////////

// The CVars here represent systems that can request the creation/sampling of the light function atlas.
// They do not require shader recompilation since they are handled via permutations

// Volumetric fog always generate a light function for the directional light.
// So this alias really only controls the use of the LightFunctionAtlas on the local lights.
int GVolumetricFogUsesLightFunctionAtlas = 1;
FAutoConsoleVariableRef CVarVolumetricFogLightFunction(
	TEXT("r.VolumetricFog.LightFunction"),
	GVolumetricFogUsesLightFunctionAtlas,
	TEXT("This is an alias, please use r.VolumetricFog.UsesLightFunctionAtlas."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);
FAutoConsoleVariableRef CVarVolumetricFogUsesLightFunctionAtlas(
	TEXT("r.VolumetricFog.UsesLightFunctionAtlas"),
	GVolumetricFogUsesLightFunctionAtlas,
	TEXT("Whether the light function atlas is sampled when rendering local lights."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

// This deferred CVar includes deferred lights splatting (batched or not) as well as clustered lighting.
int GDeferredUsesLightFunctionAtlas = 1;
FAutoConsoleVariableRef CVarDeferredLightsUsesLightFunctionAtlas(
	TEXT("r.Deferred.UsesLightFunctionAtlas"),
	GDeferredUsesLightFunctionAtlas,
	TEXT("Whether the light function atlas is sampled when rendering local lights."),
	ECVF_RenderThreadSafe
);

int GLumenUsesLightFunctionAtlas = 1;
FAutoConsoleVariableRef CVarLumenUsesLightFunctionAtlas(
	TEXT("r.Lumen.UsesLightFunctionAtlas"),
	GLumenUsesLightFunctionAtlas,
	TEXT("Whether the light function atlas is sampled for lumen scene lighting."),
	ECVF_RenderThreadSafe
);

//////////////////////////////////////////////////////////////////////////

namespace LightFunctionAtlas
{

static FLightFunctionAtlasSetup GetLightFunctionAtlasSetup()
{

	// 16x16 is the maximum slot count of LIGHT_FUNCTION_ATLAS_MAX_LIGHT_FUNCTION_COUNT=256 we currently allow
	const uint32 MaxEdgeSize = 16;
	check((MaxEdgeSize * MaxEdgeSize) <= LIGHT_FUNCTION_ATLAS_MAX_LIGHT_FUNCTION_COUNT);

	uint32 RequestedEdgeSize = FMath::Clamp(CVarLightFunctionAtlasSize.GetValueOnRenderThread(), 2, MaxEdgeSize);
	uint32 RequestedSlotResolution = FMath::Max(CVarLightFunctionAtlasSlotResolution.GetValueOnRenderThread(), 32);

	// The atlas texture resolution should not be larger than the maximum resolution supported by the current platforms.
	const uint32 MaxAtlasResolution = 32 * 1024;
	const uint32 Max2DTextureDimension = GetMax2DTextureDimension();
	if ((RequestedEdgeSize * RequestedSlotResolution) > Max2DTextureDimension)
	{
		RequestedSlotResolution = Max2DTextureDimension / RequestedEdgeSize;
	}

	// Due to the way we pack AtlasSlot.MinU and MinV, we should allow resolution up to 32K only. (UVs are stored as uint16 so 64K but we need half texel precision).
	check((RequestedSlotResolution * RequestedEdgeSize) <= MaxAtlasResolution);

	FLightFunctionAtlasSetup AtlasSetup;
	AtlasSetup.EdgeSize = RequestedEdgeSize;
	AtlasSetup.SlotResolution = RequestedSlotResolution;
	return AtlasSetup;
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLightFunctionAtlasGlobalParameters, "LightFunctionAtlas");

//////////////////////////////////////////////////////////////////////////

class FLightFunctionAtlasSlotPS : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FLightFunctionAtlasSlotPS, Material);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, SvPositionToUVScaleBias)
		SHADER_PARAMETER(FVector2f, LightFunctionTexelSize)
		SHADER_PARAMETER_STRUCT_REF(FPrimitiveUniformShaderParameters, PrimitiveUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

	FLightFunctionAtlasSlotPS() {}

	FLightFunctionAtlasSlotPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMaterialShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(
			this,
			Initializer.PermutationId,
			Initializer.ParameterMap,
			*FParameters::FTypeInfo::GetStructMetadata(),
			// Don't require full bindings, we use FMaterialShader::SetParameters
			false);
	}

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.MaterialParameters.MaterialDomain == MD_LightFunction;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("LIGHT_ATLAS"), 1);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FMaterialRenderProxy* MaterialProxy)
	{
		const FMaterial& Material = MaterialProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialProxy);
		FMaterialShader::SetParameters(BatchedParameters, MaterialProxy, Material, View);
	}

	FParameters GetParameters(const FVector2f& LightFunctionTexelSize, const FVector4f& SvPositionToUVScaleBias)
	{
		FParameters PS;
		PS.SvPositionToUVScaleBias = SvPositionToUVScaleBias;
		PS.LightFunctionTexelSize  = LightFunctionTexelSize;
		PS.PrimitiveUniformBuffer = GIdentityPrimitiveUniformBuffer.GetUniformBufferRef();
		return PS;
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FLightFunctionAtlasSlotPS, TEXT("/Engine/Private/LightFunctionAtlas/LightFunctionAtlasRender.usf"), TEXT("Main"), SF_Pixel);



//////////////////////////////////////////////////////////////////////////

FLightFunctionSlotKey::FLightFunctionSlotKey(FLightSceneInfo* InLightSceneInfo)
{
	const FMaterialRenderProxy* LightFunctionMaterial = InLightSceneInfo->Proxy->GetLightFunctionMaterial();
	if (LightFunctionMaterial != nullptr)
	{
		const UMaterialInterface* LightFunctionMaterialInterface = LightFunctionMaterial->GetMaterialInterface();
		LFMaterialUniqueID = LightFunctionMaterialInterface->GetUniqueID();
	}
	// else Default Key
}



//////////////////////////////////////////////////////////////////////////

FLightFunctionAtlas::FLightFunctionAtlas()
{
	RegisteredLights.Reserve(64); // Reserve a minimal amount to avoid multiple allocations
}

FLightFunctionAtlas::~FLightFunctionAtlas()
{
}

void FLightFunctionAtlas::UpdateRegisterLightSceneInfo(FLightSceneInfo* LightSceneInfo)
{
	if (LightSceneInfo->Proxy->GetLightFunctionMaterial() != nullptr && IsLightFunctionAtlasEnabled())
	{
	#if !UE_BUILD_SHIPPING
		check(!RegisteredLights.Contains(LightSceneInfo));
	#endif
		RegisteredLights.Push(LightSceneInfo);
	}
	else
	{
		LightSceneInfo->Proxy->SetLightFunctionAtlasIndices(0);
	}
}

void FLightFunctionAtlas::ClearEmptySceneFrame(FViewInfo* View, uint32 ViewIndex, FLightFunctionAtlasSceneData* LightFunctionAtlasSceneData)
{
	RegisteredLights.Empty(64);
	DefaultLightFunctionAtlasGlobalParameters = nullptr;
	DefaultLightFunctionAtlasGlobalParametersUB = nullptr;
	ViewLightFunctionAtlasGlobalParametersArray.Empty(4);
	ViewLightFunctionAtlasGlobalParametersUBArray.Empty(4);

	bLightFunctionAtlasEnabled = false;
	if (LightFunctionAtlasSceneData)
	{
		LightFunctionAtlasSceneData->SetData(this, false);
		LightFunctionAtlasSceneData->ClearSystems();
	}

	if (View && LightFunctionAtlasSceneData)
	{
		View->LightFunctionAtlasViewData = FLightFunctionAtlasViewData(LightFunctionAtlasSceneData, ViewIndex);
	}
}

void FLightFunctionAtlas::BeginSceneFrame(const FViewFamilyInfo& ViewFamily, TArray<FViewInfo>& Views, FLightFunctionAtlasSceneData& LightFunctionAtlasSceneData, bool bShouldRenderVolumetricFog)
{
	// Sync any pending RDG async tasks prior to modifying the atlas, since we issue async RDG tasks.
	FRDGBuilder::WaitForAsyncExecuteTask();

	ClearEmptySceneFrame(nullptr, 0, &LightFunctionAtlasSceneData);

	AtlasSetup = GetLightFunctionAtlasSetup();

	// Now lets check if we need to generate the atlas for this frame
	bLightFunctionAtlasEnabled = CVarLightFunctionAtlas.GetValueOnRenderThread() > 0 && ViewFamily.EngineShowFlags.LightFunctions;
#if !UE_BUILD_SHIPPING
	LightCountSkippedDueToMissingAtlasSlot = 0;
	SkippedLightFunctionsSet.Empty();
#endif
#if WITH_EDITOR
	LightCountWithLFMaterialsNotSamplingAtlas = 0;
	NonCompatibleLightFunctionMaterials.Reset();
#endif

	// But only really enable the atlas generation if a system asks for it
	bool bVolumetricFogRequestsLF = false;
	bool bDeferredlightingRequestsLF = false;
	bool bMegaLightsRequestsLF = false;
	bool bLumenRequestsLF = false;
	if (bLightFunctionAtlasEnabled)
	{
		bVolumetricFogRequestsLF 	= bShouldRenderVolumetricFog && GVolumetricFogUsesLightFunctionAtlas > 0;
		bDeferredlightingRequestsLF	= GDeferredUsesLightFunctionAtlas > 0;
		bMegaLightsRequestsLF		= MegaLights::IsUsingLightFunctions(ViewFamily);
		bLumenRequestsLF 			= GLumenUsesLightFunctionAtlas > 0;// && IsLumenTranslucencyGIEnabled();// GLumenScene enabled ...;

		bLightFunctionAtlasEnabled = bLightFunctionAtlasEnabled && 
			(bVolumetricFogRequestsLF || 
			bDeferredlightingRequestsLF || 
			bMegaLightsRequestsLF ||
			bLumenRequestsLF ||
			GetSingleLayerWaterUsesLightFunctionAtlas() || 
			GetTranslucentUsesLightFunctionAtlas()); 
	}

	// We propagate bLightFunctionAtlasEnabled to all the views to ease later shader parameter decision and binding for lighting, shadow or volumetric fog for instance (avoid sending lots of parameters all over the place)
	LightFunctionAtlasSceneData.SetData(this, bLightFunctionAtlasEnabled);
	if (bLightFunctionAtlasEnabled)
	{
		if (bVolumetricFogRequestsLF) 		{ LightFunctionAtlasSceneData.AddSystem(ELightFunctionAtlasSystem::VolumetricFog); }
		if (bDeferredlightingRequestsLF)	{ LightFunctionAtlasSceneData.AddSystem(ELightFunctionAtlasSystem::DeferredLighting); }
		if (bMegaLightsRequestsLF)			{ LightFunctionAtlasSceneData.AddSystem(ELightFunctionAtlasSystem::MegaLights); }
		if (bLumenRequestsLF) 				{ LightFunctionAtlasSceneData.AddSystem(ELightFunctionAtlasSystem::Lumen); }
	}

	for (uint32 ViewIndex=0,ViewCount=Views.Num();ViewIndex<ViewCount;++ViewIndex)
	{
		Views[ViewIndex].LightFunctionAtlasViewData = FLightFunctionAtlasViewData(&LightFunctionAtlasSceneData, ViewIndex);
	}
}

void FLightFunctionAtlas::UpdateLightFunctionAtlas(const TArray<FViewInfo>& Views)
{
	if (!IsLightFunctionAtlasEnabled())
	{
		return;
	}

	AllocateAtlasSlots(Views);
}

void FLightFunctionAtlas::AllocateAtlasSlots(const TArray<FViewInfo>& Views)
{
	if (Views.Num() == 0)
	{
		return;
	}
	const FViewInfo& View = Views[0];
	ERHIFeatureLevel::Type FeatureLevel = View.GetFeatureLevel();

	//
	// Sort the list of lights registered as having light function in order to keep directional lights first, then each lights closer to each views
	//
	struct FSortedRegisteredLights
	{
		float MinDistanceToViews;
		uint32 RegisteredLightIndex;
		FORCEINLINE bool operator!=(FSortedRegisteredLights B) const
		{
			return MinDistanceToViews != B.MinDistanceToViews;
		}

		FORCEINLINE bool operator<(FSortedRegisteredLights B) const
		{
			return MinDistanceToViews < B.MinDistanceToViews;
		}
	};
	TArray<FSortedRegisteredLights> SortedRegisteredLights;
	const uint32 ViewCount = Views.Num();
	{
		SortedRegisteredLights.Reserve(RegisteredLights.Num());

		const FVector View0Pos = Views[0].ViewMatrices.GetViewOrigin();
		uint32 RegisteredLightCount = RegisteredLights.Num();
		for (uint32 RegisteredLightIndex = 0; RegisteredLightIndex < RegisteredLightCount; ++RegisteredLightIndex)
		{
			const FLightSceneInfo* LightSceneInfo = RegisteredLights[RegisteredLightIndex];
			FLightSceneProxy* Proxy = LightSceneInfo->Proxy;

			if (Proxy->GetLightType() == LightType_Directional)
			{
				// Directional light are considered at a 0 distance from each view
				SortedRegisteredLights.Push({ 0.0f, RegisteredLightIndex });
			}
			else
			{
				const FVector ProxyPos = FVector(Proxy->GetPosition());
				float MinDistanceToViews = (View0Pos - ProxyPos).SquaredLength();
				for (uint32 ViewId = 1; ViewId < ViewCount; ++ViewId)
				{
					FVector ViewXPos = Views[ViewId].ViewMatrices.GetViewOrigin();
					MinDistanceToViews = FMath::Min(MinDistanceToViews, (ViewXPos - Proxy->GetPosition()).SquaredLength());
				}
				SortedRegisteredLights.Push({ MinDistanceToViews, RegisteredLightIndex });
			}
		}

		// Now sort according to priority
		SortedRegisteredLights.Sort();
	}

	//
	// Allocate slots until we cannot anymore and set light function slot index on FLightSceneInfo to be send to the GPU later
	//
	const uint32 AtlasSlotResolution = AtlasSetup.SlotResolution;
	const float AtlasEdgeSize = AtlasSetup.EdgeSize;
	const float AtlasResolution = AtlasSlotResolution * AtlasEdgeSize;
	check(AtlasEdgeSize * AtlasEdgeSize <= LIGHT_FUNCTION_ATLAS_MAX_LIGHT_FUNCTION_COUNT);
	const uint32 AtlasMaxLightFunctionCount = AtlasEdgeSize * AtlasEdgeSize;

	EffectiveLightFunctionSlotArray.Reset();
	EffectiveLightFunctionSlotArray.Reserve(AtlasMaxLightFunctionCount);

	EffectiveLocalLightSlotArray.Reset();
	EffectiveLocalLightSlotArray.Reserve(SortedRegisteredLights.Num());

	LightFunctionsSet.Reset();
	LightFunctionsSet.Reserve(AtlasMaxLightFunctionCount);

	uint32 NextAtlasSlotX = 0;
	uint32 NextAtlasSlotY = 0;
	auto AddAtlasSlot = [&](FLightFunctionSlotKey Key, const FMaterialRenderProxy* LightFunctionMaterial)
	{
		check(uint32(LightFunctionsSet.Num()) < AtlasMaxLightFunctionCount);
		const uint32 NewSlotIndex = EffectiveLightFunctionSlotArray.Num();

		Key.EffectiveLightFunctionSlotIndex = NewSlotIndex;

		LightFunctionsSet.Add(Key);

		EffectiveLightFunctionSlot& NewAtlasSlot = EffectiveLightFunctionSlotArray.Emplace_GetRef();

		NewAtlasSlot.Min = FIntPoint(NextAtlasSlotX * AtlasSlotResolution, NextAtlasSlotY * AtlasSlotResolution);
		NewAtlasSlot.Max = NewAtlasSlot.Min + FIntPoint(AtlasSlotResolution, AtlasSlotResolution);
		NewAtlasSlot.MinU = (float(NewAtlasSlot.Min.X) + 0.5f) / AtlasResolution;
		NewAtlasSlot.MinV = (float(NewAtlasSlot.Min.Y) + 0.5f) / AtlasResolution;

		NewAtlasSlot.LightFunctionMaterial = LightFunctionMaterial;

		NextAtlasSlotX++;
		if (NextAtlasSlotX == AtlasEdgeSize)
		{
			NextAtlasSlotX = 0;
			NextAtlasSlotY++;
		}
		return NewSlotIndex;
	};

	uint32 LocalLightWithLightFunctionCount = 0;
	auto AddLightSlot = [&](FLightSceneInfo* LightSceneInfo, uint8 LightFunctionAtlasSlotIndex)
	{
		EffectiveLocalLightSlot& NewLightSlot = EffectiveLocalLightSlotArray.Emplace_GetRef();
		NewLightSlot.LightSceneInfo = LightSceneInfo;
		NewLightSlot.LightFunctionAtlasSlotIndex = LightFunctionAtlasSlotIndex;
		const uint32 LightSlotIndex = LocalLightWithLightFunctionCount;
		LocalLightWithLightFunctionCount++;
		return LightSlotIndex;
	};

	// Add the default invalid light slot at the beginning. This is done so that we always allocate buffer with at least one element.
	AddLightSlot(nullptr, 0);

	int32 MaxLightCount = CVarLightFunctionAtlasMaxLightCount.GetValueOnRenderThread();
	for (FSortedRegisteredLights& SortedRegisteredLight : SortedRegisteredLights)
	{
		FLightSceneInfo* LightSceneInfo = RegisteredLights[SortedRegisteredLight.RegisteredLightIndex];
		FLightSceneProxy* Proxy = LightSceneInfo->Proxy;
		const FMaterialRenderProxy* LightFunctionMaterial = LightSceneInfo->Proxy->GetLightFunctionMaterial();

		if ((MaxLightCount >= 0 && int32(LocalLightWithLightFunctionCount) >= MaxLightCount) || LightFunctionMaterial == nullptr)
		{
			// We cannot register anymore light, so set them to no light function
			Proxy->SetLightFunctionAtlasIndices(0);
			continue;
		}

		if (LightFunctionMaterial != nullptr)
		{
			const FMaterial& Material = LightFunctionMaterial->GetIncompleteMaterialWithFallback(FeatureLevel);
			if (!Material.MaterialIsLightFunctionAtlasCompatible_RenderThread())
			{
			#if WITH_EDITOR
				const UMaterialInterface* LightFunctionMaterialInterface = LightFunctionMaterial->GetMaterialInterface();
				if (LightFunctionMaterialInterface)
				{
					NonCompatibleLightFunctionMaterials.Add(LightFunctionMaterialInterface->GetUniqueID(), LightFunctionMaterialInterface);
				}
				LightCountWithLFMaterialsNotSamplingAtlas++;
			#endif
				Proxy->SetLightFunctionAtlasIndices(0);
				continue;
			}
		}

		FLightFunctionSlotKey NewKey = FLightFunctionSlotKey(LightSceneInfo);

		FLightFunctionSlotKey* ExistingKey = LightFunctionsSet.Find(NewKey);

	#if !UE_BUILD_SHIPPING
		if (ExistingKey == nullptr && (uint32(LightFunctionsSet.Num()) >= AtlasMaxLightFunctionCount))
		{
			// We cannot account for this light's required light function
			LightCountSkippedDueToMissingAtlasSlot++;
			SkippedLightFunctionsSet.Add(NewKey);
		}
	#endif

		if (ExistingKey == nullptr && (uint32(LightFunctionsSet.Num()) < AtlasMaxLightFunctionCount))
		{
			// Allocate slots for the light and views
			Proxy->SetLightFunctionAtlasIndices(AddLightSlot(LightSceneInfo, AddAtlasSlot(NewKey, LightFunctionMaterial)));

		}
		else if (ExistingKey != nullptr)
		{
			// The key already exist, make the light point to the existing light function slot
			Proxy->SetLightFunctionAtlasIndices(AddLightSlot(LightSceneInfo, ExistingKey->EffectiveLightFunctionSlotIndex));
		}
		else
		{
			// The key does not exist, or there is no space to allocate a new slot. Disable light function on that light
			Proxy->SetLightFunctionAtlasIndices(0);
		}
	}

	// TODO we could do all the constant buffer setup inline above (done in RenderAtlasSlots right now) if we would send a GraphBuilder here.
}

FLightFunctionAtlasGlobalParameters* FLightFunctionAtlas::GetLightFunctionAtlasGlobalParametersStruct(FRDGBuilder& GraphBuilder, uint32 ViewIndex)
{
	if (IsLightFunctionAtlasEnabled())
	{
		const bool bViewIndexIsValid = ViewIndex < uint32(ViewLightFunctionAtlasGlobalParametersUBArray.Num());
		check(bViewIndexIsValid);
		if (bViewIndexIsValid)
		{
			return ViewLightFunctionAtlasGlobalParametersArray[ViewIndex];
		}
	}

	return GetDefaultLightFunctionAtlasGlobalParametersStruct(GraphBuilder);
}

TRDGUniformBufferRef<FLightFunctionAtlasGlobalParameters> FLightFunctionAtlas::GetLightFunctionAtlasGlobalParameters(FRDGBuilder& GraphBuilder, uint32 ViewIndex)
{
	if (IsLightFunctionAtlasEnabled())
	{
		const bool bViewIndexIsValid = ViewIndex < uint32(ViewLightFunctionAtlasGlobalParametersUBArray.Num());
		check(bViewIndexIsValid);
		if (bViewIndexIsValid)
		{
			return ViewLightFunctionAtlasGlobalParametersUBArray[ViewIndex];
		}
	}

	return GetDefaultLightFunctionAtlasGlobalParameters(GraphBuilder);
}

FLightFunctionAtlasGlobalParameters* FLightFunctionAtlas::GetDefaultLightFunctionAtlasGlobalParametersStruct(FRDGBuilder& GraphBuilder)
{
	FLightFunctionAtlasGlobalParameters* DefaultLightFunctionAtlasGlobalParameters = GraphBuilder.AllocParameters<FLightFunctionAtlasGlobalParameters>();
	DefaultLightFunctionAtlasGlobalParameters->LightFunctionAtlasTexture = GSystemTextures.GetWhiteDummy(GraphBuilder);
	DefaultLightFunctionAtlasGlobalParameters->LightFunctionAtlasSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	DefaultLightFunctionAtlasGlobalParameters->LightInfoDataBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f) * 1, 0.0f), PF_A32B32G32R32F);
	DefaultLightFunctionAtlasGlobalParameters->Slot_UVSize = 1.0f;
	return DefaultLightFunctionAtlasGlobalParameters;
}

TRDGUniformBufferRef<FLightFunctionAtlasGlobalParameters> FLightFunctionAtlas::GetDefaultLightFunctionAtlasGlobalParameters(FRDGBuilder& GraphBuilder)
{
	if (DefaultLightFunctionAtlasGlobalParametersUB == nullptr) // Only create the default buffer once per frame
	{
		FLightFunctionAtlasGlobalParameters* DefaultLightFunctionAtlasGlobalParametersStruct = GetDefaultLightFunctionAtlasGlobalParametersStruct(GraphBuilder);
		DefaultLightFunctionAtlasGlobalParametersUB = GraphBuilder.CreateUniformBuffer(DefaultLightFunctionAtlasGlobalParametersStruct);
	}
	return DefaultLightFunctionAtlasGlobalParametersUB;
}

void FLightFunctionAtlas::RenderLightFunctionAtlas(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views)
{
	if (!IsLightFunctionAtlasEnabled())
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(RenderLightFunctionAtlas);

	//
	// Render the atlas
	//
	RenderAtlasSlots(GraphBuilder, Views);

	//
	// Allocate and fill up the global light function atlas UB
	//
	for (uint32 ViewIndex = 0; ViewIndex < uint32(Views.Num()); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];

		FLightFunctionAtlasGlobalParameters* LightFunctionAtlasGlobalParameters = GraphBuilder.AllocParameters<FLightFunctionAtlasGlobalParameters>();

		check(RDGAtlasTexture2D != nullptr);
		LightFunctionAtlasGlobalParameters->LightFunctionAtlasTexture = RDGAtlasTexture2D;
		LightFunctionAtlasGlobalParameters->LightFunctionAtlasSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

		const uint32 AtlasSlotResolution = AtlasSetup.SlotResolution;
		const float AtlasEdgeSize = AtlasSetup.EdgeSize;
		float AtlasResolution = AtlasSlotResolution * AtlasEdgeSize;
		LightFunctionAtlasGlobalParameters->Slot_UVSize = (AtlasSlotResolution - 1.0f) / AtlasResolution; // -1.0 because we remove a bit more than half a texel at the border.

		const FIntPoint LightFunctionResolution = FIntPoint(AtlasSlotResolution, AtlasSlotResolution);

		// Write the light data needed to rotate and fade the light function in the world.
		// UVMinMax for pointed to atlas slot is also packed into the light structure.
		const uint32 InitialLightInfoDataLightCount = FMath::DivideAndRoundUp(uint32(EffectiveLocalLightSlotArray.Num()), 32u) * 32u;	// Allocated with 32 lights step to better reuse shared buffers pool
		const uint32 InitialLightInfoDataSize = InitialLightInfoDataLightCount * sizeof(FAtlasLightInfoData);
		FAtlasLightInfoData* LightInfoDataBufferPtr = (FAtlasLightInfoData*)GraphBuilder.Alloc(InitialLightInfoDataSize, 16);
		uint32 OutputBufferLightIndex = 0;
		for (EffectiveLocalLightSlot& LightSlot : EffectiveLocalLightSlotArray)
		{
			FLightSceneInfo* LightSceneInfo = LightSlot.LightSceneInfo;
			const bool bIsDefaultSlot = LightSceneInfo == nullptr;
			if (bIsDefaultSlot)
			{
				LightInfoDataBufferPtr[OutputBufferLightIndex].Parameters = FVector4f(1.0f, 1.0f, 1.0f, 0.0f);
				OutputBufferLightIndex++;
				continue;
			}

			const uint8 LightFunctionAtlasSlotIndex = LightSlot.LightFunctionAtlasSlotIndex;
			check(LightSceneInfo);
			const FLightSceneProxy* Proxy = LightSceneInfo->Proxy;

			float ShadowFadeFraction = 1.0f;

			FMatrix44f TranslatedWorldToLight;

			{
				const FVector Scale = Proxy->GetLightFunctionScale();
				// Switch x and z so that z of the user specified scale affects the distance along the light direction
				const FVector InverseScale = FVector(1.f / Scale.Z, 1.f / Scale.Y, 1.f / Scale.X);
				const FMatrix WorldToLight = Proxy->GetWorldToLight() * FScaleMatrix(FVector(InverseScale));
				TranslatedWorldToLight = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToLight);
			}

			LightInfoDataBufferPtr[OutputBufferLightIndex].Transform = TranslatedWorldToLight;

			uint8 LightType = Proxy->GetLightType();
			
			FFloat16 PackedDisabledBrightness(Proxy->GetLightFunctionDisabledBrightness());
			uint32 PackedLightInfoDataParams = uint32(LightType) | (uint32(PackedDisabledBrightness.Encoded) << 8);

			const EffectiveLightFunctionSlot& AtlasSlot = EffectiveLightFunctionSlotArray[LightFunctionAtlasSlotIndex];

			const uint32 PackedAtlasSlotMinU = uint32(round(AtlasSlot.MinU * 65536.0f));
			const uint32 PackedAtlasSlotMinV = uint32(round(AtlasSlot.MinV * 65536.0f));
			const uint32 PackedAtlasSlotMinUV = (PackedAtlasSlotMinU | (PackedAtlasSlotMinV << 16));

			const float IntegerUVTolerance = 0.5f / 65536.0f;
			ensure(FMath::IsNearlyEqual((PackedAtlasSlotMinUV & 0xFFFF) / 65536.0f, AtlasSlot.MinU, IntegerUVTolerance));
			ensure(FMath::IsNearlyEqual(((PackedAtlasSlotMinUV >> 16) & 0xFFFF) / 65536.0f, AtlasSlot.MinV, IntegerUVTolerance));

			float TanOuterAngle = LightType == LightType_Spot ? FMath::Tan(LightSceneInfo->Proxy->GetOuterConeAngle()) : -1.0f;
			if (LightType == LightType_Rect)
			{
				FRectLightSceneProxy* RectLightProxy = (FRectLightSceneProxy*)LightSceneInfo->Proxy;
				if (RectLightProxy->LightFunctionConeAngleTangent > 0.0f)
				{
					TanOuterAngle = RectLightProxy->LightFunctionConeAngleTangent;
				}
			}

			// ShadowFadeFraction is unused.
			LightInfoDataBufferPtr[OutputBufferLightIndex].Parameters = FVector4f(Proxy->GetLightFunctionFadeDistance(), FMath::AsFloat(PackedLightInfoDataParams), FMath::AsFloat(PackedAtlasSlotMinUV), TanOuterAngle);

			OutputBufferLightIndex++;
		}

		// Create the light instance data buffer SRV
		const uint32 Float4Count = sizeof(FAtlasLightInfoData) / sizeof(FVector4f);
		RDGLightInfoDataBuffer = CreateStructuredBuffer(
			GraphBuilder, TEXT("LightFunctionAtlasLightInfoData"), 
			sizeof(FVector4f), Float4Count * InitialLightInfoDataLightCount,
			reinterpret_cast<void*>(LightInfoDataBufferPtr), InitialLightInfoDataSize, ERDGInitialDataFlags::NoCopy);
		LightFunctionAtlasGlobalParameters->LightInfoDataBuffer = GraphBuilder.CreateSRV(RDGLightInfoDataBuffer, PF_A32B32G32R32F);

		ViewLightFunctionAtlasGlobalParametersArray.Add(LightFunctionAtlasGlobalParameters);
		ViewLightFunctionAtlasGlobalParametersUBArray.Add(GraphBuilder.CreateUniformBuffer(LightFunctionAtlasGlobalParameters));
	}
}

void FLightFunctionAtlas::AllocateTexture2DAtlas(FRDGBuilder& GraphBuilder)
{
	const uint32 AtlasSlotResolution = AtlasSetup.SlotResolution;
	const float AtlasEdgeSize = AtlasSetup.EdgeSize;
	uint32 AtlasResolution = AtlasSlotResolution * AtlasEdgeSize;
	const uint32 MipCount = 1;

	int32 LightFunctionAtlasFormat = GetLightFunctionAtlasFormat();

	RDGAtlasTexture2D = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(
		FIntPoint(AtlasResolution, AtlasResolution),
		LightFunctionAtlasFormat == 0 ? PF_R8 : PF_R8G8B8A8,
		FClearValueBinding::Black,
		ETextureCreateFlags::UAV | ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable,
		MipCount),
		TEXT("LightFunction.Atlas"),
		ERDGTextureFlags::MultiFrame);
}

BEGIN_SHADER_PARAMETER_STRUCT(FLightFunctionAtlasRenderParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FLightFunctionAtlas::RenderAtlasSlots(FRDGBuilder& GraphBuilder, const TArray<FViewInfo>& Views)
{
	AllocateTexture2DAtlas(GraphBuilder);

	SCOPED_NAMED_EVENT(LightFunctionAtlasGeneration, FColor::Emerald);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, LightFunctionAtlasGeneration, "LightFunctionAtlasGeneration");
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, LightFunctionAtlasGeneration);
	
	FLightFunctionAtlasRenderParameters* PassParameters = GraphBuilder.AllocParameters<FLightFunctionAtlasRenderParameters>();
	PassParameters->View = Views[0].GetShaderParameters();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(RDGAtlasTexture2D, ERenderTargetLoadAction::ENoAction);
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("LightFunctionAtlas Generation"),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, &Views, this](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			const uint32 AtlasSlotResolution = AtlasSetup.SlotResolution;
			const float AtlasEdgeSize = AtlasSetup.EdgeSize;
			uint32 AtlasResolution = AtlasSlotResolution * AtlasEdgeSize;

			// This always work because in this case we do not need anything from any view.
			const FViewInfo& View = Views[0];

			// Render all light functions and update light info
			for (EffectiveLightFunctionSlot& Slot : EffectiveLightFunctionSlotArray)
			{
				const FMaterialRenderProxy* MaterialProxyForRendering = Slot.LightFunctionMaterial;
				if (MaterialProxyForRendering == nullptr)
				{
					continue;	// This is to skip the unused slot at index 0 because index 0 disable the sampling from the light function atlas.
				}
				const FMaterial& Material = MaterialProxyForRendering->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialProxyForRendering);

				RHICmdList.SetViewport(0.f, 0.f, 0.f, AtlasResolution, AtlasResolution, 1.f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;

				const FMaterialShaderMap* MaterialShaderMap = Material.GetRenderingThreadShaderMap();
				TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap); 

				FLightFunctionAtlasSlotPS::FPermutationDomain PermutationVector;
				TShaderRef<FLightFunctionAtlasSlotPS> PixelShader = MaterialShaderMap->GetShader<FLightFunctionAtlasSlotPS>(PermutationVector);

				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				FVector2f LightFunctionTexelSize  = FVector2f(1.0f / AtlasSlotResolution, 1.0f / AtlasSlotResolution);
				FVector4f SvPositionToUVScaleBias = FVector4f(
					1.0f / float(AtlasSlotResolution - 1), 1.0f / float(AtlasSlotResolution - 1),
					float(Slot.Min.X) + 0.5f, float(Slot.Min.Y) + 0.5f);

				FLightFunctionAtlasSlotPS::FParameters PS = PixelShader->GetParameters(LightFunctionTexelSize, SvPositionToUVScaleBias);

				ClearUnusedGraphResources(PixelShader, &PS);
				SetShaderParametersMixedPS(RHICmdList, PixelShader, PS, View, MaterialProxyForRendering);

				FIntPoint RectSize = FIntPoint(Slot.Max.X - Slot.Min.X, Slot.Max.Y - Slot.Min.Y);
				DrawRectangle(
					RHICmdList,
					Slot.Min.X, Slot.Min.Y,
					RectSize.X, RectSize.Y,
					Slot.Min.X, Slot.Min.Y,
					RectSize.X, RectSize.Y,
					AtlasResolution,
					AtlasSlotResolution,
					VertexShader);
			}
		}
	);
}

bool FLightFunctionAtlas::IsOutOfSlots() 
{
#if !UE_BUILD_SHIPPING
	return LightCountSkippedDueToMissingAtlasSlot > 0;
#else
	return false;
#endif
}

FString FLightFunctionAtlas::GetOutOfSlotWarningMessage()
{
#if !UE_BUILD_SHIPPING
	if (!IsLightFunctionAtlasEnabled() || !GAreScreenMessagesEnabled)
	{
		return FString();
	}

	// In case we became out of budget, let's notify the game developers.
	if (LightCountSkippedDueToMissingAtlasSlot > 0)
	{
		FString Text = FString::Printf(TEXT("Light Functions Atlas:\n"));
		Text += FString::Printf(TEXT("  - %d light(s) will skip light functions due to out of atlas slot (see r.LightFunctionAtlas.Size).\n"), LightCountSkippedDueToMissingAtlasSlot);
		Text += FString::Printf(TEXT("  - %d light function material(s) have been skipped.\n"), SkippedLightFunctionsSet.Num());

		return Text;
	}
#endif // !UE_BUILD_SHIPPING
	return FString();
}

FScreenPassTexture FLightFunctionAtlas::AddDebugVisualizationPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor, FScreenPassRenderTarget& OverrideOutput) const
{
	FScreenPassRenderTarget OutputRT = OverrideOutput;

#if WITH_EDITOR

	if (!IsLightFunctionAtlasEnabled())
	{
		return MoveTemp(OutputRT);
	}

	if (!OutputRT.IsValid())
	{
		OutputRT = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, ScreenPassSceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeLightFunctionAtlas"));
	}

	if (ScreenPassSceneColor.Texture != OutputRT.GetRenderTargetBinding().GetTexture())
	{
		// Copy the scene color because below we do not write and copy all the view pixel. We only stamp the atlas and its tiles on top.
		AddDrawTexturePass(
			GraphBuilder,
			View,
			ScreenPassSceneColor.Texture,
			OutputRT.GetRenderTargetBinding().GetTexture(),
			FIntPoint::ZeroValue,
			OutputRT.GetRenderTargetBinding().GetTexture()->Desc.Extent,
			FIntPoint::ZeroValue,
			OutputRT.GetRenderTargetBinding().GetTexture()->Desc.Extent);
	}

	const uint32 AtlasSlotResolution	= AtlasSetup.SlotResolution;
	const float AtlasEdgeSize			= AtlasSetup.EdgeSize;
	const uint32 AtlasResolution		= AtlasSlotResolution * AtlasEdgeSize;

	const FIntPoint SrcPoint			= FIntPoint::ZeroValue;
	const FIntPoint SrcSize				= FIntPoint(1, 1);			// Size of MidGreyDummy, used for initial clear of the atlas to grey
	const FIntPoint DstPoint			= FIntPoint(100, 100);
	const FIntPoint DstSize				= FIntPoint(512, 512);

	const float DisplayResolutionRatio	= float(DstSize.X) / float(AtlasResolution);

	// Draw the atlas, first make it all grey,
	AddDrawTexturePass(
		GraphBuilder,
		View,
		GSystemTextures.GetMidGreyDummy(GraphBuilder),
		OutputRT.GetRenderTargetBinding().GetTexture(),
		SrcPoint,
		SrcSize,
		DstPoint,
		DstSize);
	// then compose all tiles independentely.  (to not show flickering/uninitialised memory on unused tiles)
	for (auto& AtlasSlot : EffectiveLightFunctionSlotArray)
	{
		const FIntPoint SlotDstPoint	= FIntPoint(DstPoint.X + float(AtlasSlot.Min.X) * DisplayResolutionRatio, DstPoint.Y + float(AtlasSlot.Min.Y) * DisplayResolutionRatio);
		const FIntPoint SlotDstSize		= AtlasSlotResolution * DisplayResolutionRatio;
		AddDrawTexturePass(
			GraphBuilder,
			View,
			RDGAtlasTexture2D,
			OutputRT.GetRenderTargetBinding().GetTexture(),
			AtlasSlot.Min,
			AtlasSlot.Max-AtlasSlot.Min,
			SlotDstPoint,
			SlotDstSize);
	}

	// Now debug print
	AddDrawCanvasPass(GraphBuilder, {}, View, FScreenPassRenderTarget(OutputRT, ERenderTargetLoadAction::ELoad),
		[&View, this, DstPoint, DstSize, AtlasResolution, DisplayResolutionRatio](FCanvas& Canvas)
	{
		FString Text;

		const float ViewPortWidth = float(View.ViewRect.Width());
		const float ViewPortHeight = float(View.ViewRect.Height());
		float DrawPosX = float(DstPoint.X + DstSize.X) + 30.0f;
		float DrawPosY = float(DstPoint.Y) + 10.0f;

		const float DPIScale = Canvas.GetDPIScale();
		Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale) * Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

		Canvas.DrawShadowedString(DstPoint.X + 180.0f, DstPoint.Y - 40.0f, TEXT("LIGHT FUNCTION ATLAS"), GEngine->GetLargeFont(), FLinearColor::White);

		Text = FString::Printf(TEXT("Slot Resolution = %dx%d  -  Size = %dx%d"), AtlasSetup.SlotResolution, AtlasSetup.SlotResolution, AtlasSetup.EdgeSize, AtlasSetup.EdgeSize);
		Canvas.DrawShadowedString(DstPoint.X + 130.0f, DstPoint.Y - 20.0f, *Text, GEngine->GetLargeFont(), FLinearColor::White);

		Text = FString::Printf(TEXT("Light Functions in atlas:         %d"), EffectiveLightFunctionSlotArray.Num());
		Canvas.DrawShadowedString(DrawPosX, DrawPosY, *Text, GEngine->GetLargeFont(), FLinearColor::White);
		DrawPosY += 20.0f;

		Text = FString::Printf(TEXT("Local Lights sampling atlas: %d"), EffectiveLocalLightSlotArray.Num() - 1);	// minus one because we add a dummy light at slot 0 using AddLightSlot(nullptr, 0);
		Canvas.DrawShadowedString(DrawPosX, DrawPosY, *Text, GEngine->GetLargeFont(), FLinearColor::White);
		DrawPosY += 40.0f;

		uint32 LightFunctionAtlasSlotIndex = 0;
		for (auto& AtlasSlot : EffectiveLightFunctionSlotArray)
		{
			const FMaterialRenderProxy* LightFunctionMaterial = AtlasSlot.LightFunctionMaterial;
			check(LightFunctionMaterial);

			const FString& MaterialName = LightFunctionMaterial->GetMaterialName();
			const uint32 LFMaterialUniqueID = LightFunctionMaterial->GetMaterialInterface() ? MurmurFinalize32(MurmurFinalize32(LightFunctionMaterial->GetMaterialInterface()->GetUniqueID())) : 0xFFFFFFFF;

			const FLinearColor MaterialColor(FColor(LFMaterialUniqueID & 0xFF, (LFMaterialUniqueID >> 8) & 0xFF, (LFMaterialUniqueID >> 16) & 0xFF));
			const FLinearColor WarningColor(1.0f, 100.0f / 255.0f, 20.0f / 255.0f);

			uint32 LightCountUsingThisMaterial = 0;
			for (auto& LocalLight : EffectiveLocalLightSlotArray)
			{
				LightCountUsingThisMaterial += ((LocalLight.LightSceneInfo != nullptr) && (LocalLight.LightFunctionAtlasSlotIndex == LightFunctionAtlasSlotIndex)) ? 1 : 0;
			}

			// Draw the light function material
			Canvas.DrawTile(DrawPosX - 20.0f, DrawPosY, 15.0f, 15.0f, 0.0f, 0.0f, 1.0f, 1.0f, MaterialColor, nullptr, false);

			Text = FString::Printf(TEXT("%2d"), LightCountUsingThisMaterial);
			Canvas.DrawShadowedString(DrawPosX, DrawPosY, *Text, GEngine->GetLargeFont(), MaterialColor);

			FString MaterialDetails;
			const bool bDefaultMaterialSpecified = LightFunctionMaterial->GetFallback(View.GetFeatureLevel()) == nullptr;
			if (bDefaultMaterialSpecified)
			{
				MaterialDetails = TEXT("[Default Material]   ");
			}
			const bool bMaterialCompilationFailed= !bDefaultMaterialSpecified && LightFunctionMaterial->GetIncompleteMaterialWithFallback(View.GetFeatureLevel()).IsDefaultMaterial();
			if (bMaterialCompilationFailed)
			{
				MaterialDetails = TEXT("[Compilation Failed] ");
			}

			Text = FString::Printf(TEXT("lights - %s"), *MaterialName);
			if (!MaterialDetails.IsEmpty())
			{
				Canvas.DrawShadowedString(DrawPosX +  20, DrawPosY, *MaterialDetails, GEngine->GetLargeFont(), WarningColor);
				Canvas.DrawShadowedString(DrawPosX + 140, DrawPosY, *Text, GEngine->GetLargeFont(), MaterialColor);
			}
			else
			{
				Canvas.DrawShadowedString(DrawPosX + 20, DrawPosY, *Text, GEngine->GetLargeFont(), MaterialColor);
			}

			// Draw a line around the corresponding atlas tile
			auto OutLineAtlasSlot = [&](float X0, float Y0, float X1, float Y1)
			{
				FCanvasLineItem LineItem(
					FIntPoint(DstPoint.X + X0 * DisplayResolutionRatio, DstPoint.Y + Y0 * DisplayResolutionRatio),
					FIntPoint(DstPoint.X + X1 * DisplayResolutionRatio, DstPoint.Y + Y1 * DisplayResolutionRatio));
				LineItem.LineThickness = 4.0f;
				LineItem.SetColor(MaterialColor);
				Canvas.DrawItem(LineItem);
			};
			OutLineAtlasSlot(AtlasSlot.Min.X + 2, AtlasSlot.Min.Y + 2, AtlasSlot.Max.X - 2, AtlasSlot.Min.Y + 2);
			OutLineAtlasSlot(AtlasSlot.Max.X - 2, AtlasSlot.Min.Y + 2, AtlasSlot.Max.X - 2, AtlasSlot.Max.Y - 2);
			OutLineAtlasSlot(AtlasSlot.Max.X - 2, AtlasSlot.Max.Y - 2, AtlasSlot.Min.X + 2, AtlasSlot.Max.Y - 2);
			OutLineAtlasSlot(AtlasSlot.Min.X + 2, AtlasSlot.Max.Y - 2, AtlasSlot.Min.X + 2, AtlasSlot.Min.Y + 2);

			DrawPosY += 20.0f;
			LightFunctionAtlasSlotIndex++;
		}

		DrawPosY += 50.0f;

		// Now display incompatible materials
		Text = FString::Printf(TEXT("Light functions not compatible with the atlas:   %d"), NonCompatibleLightFunctionMaterials.Num());
		Canvas.DrawShadowedString(DrawPosX, DrawPosY, *Text, GEngine->GetLargeFont(), FLinearColor::White);
		DrawPosY += 20.0f;

		Text = FString::Printf(TEXT("Local Lights with light function not in the atlas: %d"), LightCountWithLFMaterialsNotSamplingAtlas);
		Canvas.DrawShadowedString(DrawPosX, DrawPosY, *Text, GEngine->GetLargeFont(), FLinearColor::White);
		DrawPosY += 40.0f;

		for (auto& NotCompatibleMaterial : NonCompatibleLightFunctionMaterials)
		{
			const uint32 LFMaterialUniqueID = MurmurFinalize32(MurmurFinalize32(NotCompatibleMaterial.Value->GetUniqueID()));
			const FLinearColor MaterialColor(FColor(LFMaterialUniqueID & 0xFF, (LFMaterialUniqueID >> 8) & 0xFF, (LFMaterialUniqueID >> 16) & 0xFF));

			Canvas.DrawTile(DrawPosX - 20.0f, DrawPosY, 15.0f, 15.0f, 0.0f, 0.0f, 1.0f, 1.0f, MaterialColor, nullptr, false);

			Text = FString::Printf(TEXT("lights - %s"), *NotCompatibleMaterial.Value->GetRenderProxy()->GetMaterialName());
			Canvas.DrawShadowedString(DrawPosX + 20, DrawPosY, *Text, GEngine->GetLargeFont(), MaterialColor);
			DrawPosY += 20.0f;
		}
	});

#endif // WITH_EDITOR

	return MoveTemp(OutputRT);
}

bool IsEnabled(const FViewInfo& InView, ELightFunctionAtlasSystem In)
{
	return InView.LightFunctionAtlasViewData.UsesLightFunctionAtlas(In);
}

bool IsEnabled(const FScene& InScene, ELightFunctionAtlasSystem In)
{
	return InScene.LightFunctionAtlasSceneData.UsesLightFunctionAtlas(In);
}

void OnRenderBegin(FLightFunctionAtlas& In, FScene& InScene, TArray<FViewInfo>& InViews, const FViewFamilyInfo& InViewFamily)
{
	In.BeginSceneFrame(InViewFamily, InViews, InScene.LightFunctionAtlasSceneData, ShouldRenderVolumetricFog(&InScene, InViewFamily));
}

TRDGUniformBufferRef<FLightFunctionAtlasGlobalParameters> BindGlobalParameters(FRDGBuilder& GraphBuilder, const FViewInfo& InView)
{
	return InView.LightFunctionAtlasViewData.GetLightFunctionAtlas()->GetLightFunctionAtlasGlobalParameters(GraphBuilder, InView.LightFunctionAtlasViewData.GetViewIndex());
}

FLightFunctionAtlasGlobalParameters* GetGlobalParametersStruct(FRDGBuilder& GraphBuilder, const FViewInfo& InView)
{
	if (FLightFunctionAtlas* LightFunctionAtlas = InView.LightFunctionAtlasViewData.GetLightFunctionAtlas())
	{
		return LightFunctionAtlas->GetLightFunctionAtlasGlobalParametersStruct(GraphBuilder, InView.LightFunctionAtlasViewData.GetViewIndex());
	}
	else
	{
		return FLightFunctionAtlas::GetDefaultLightFunctionAtlasGlobalParametersStruct(GraphBuilder);
	}
}

} // namespace LightFunctionAtlas
