// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LightSceneProxy.h"
#include "RenderGraphFwd.h"
#include "ShaderParameterMacros.h"

class FViewFamilyInfo;
class FViewInfo;
class FScene;
class FMaterialRenderProxy;
class FRDGBuilder;
class FLightSceneInfo;
class FRDGTexture;

struct FScreenPassTexture;
struct FScreenPassRenderTarget;
struct FSortedLightSetSceneInfo;
struct IPooledRenderTarget;



namespace LightFunctionAtlas
{

// This allows to not have to store more data per GPU light representation on GPU. The light only needs an index into the array.
// Using a constant buffer also workaround the fact that we would otherwise need another SRV in forward shaders. 
// The light atlas texture itself already use 1 extra SRV. We could an extra SRV and have LightInfoDataXXX be in a buffer that scale with amount of light in the scene.
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLightFunctionAtlasGlobalParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, LightFunctionAtlasTexture)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, LightInfoDataBuffer)
	SHADER_PARAMETER_SAMPLER(SamplerState, LightFunctionAtlasSampler)
	SHADER_PARAMETER(float, Slot_UVSize)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

enum class ELightFunctionAtlasSystem
{
	VolumetricFog,
	DeferredLighting,
	MegaLights,
	Lumen,
};

struct FLightFunctionAtlas;

struct FLightFunctionAtlasSceneData
{
	void SetData(FLightFunctionAtlas* InLightFunctionAtlas, bool bInLightFunctionAtlasEnabled)
	{
		LightFunctionAtlas = InLightFunctionAtlas;
		bLightFunctionAtlasEnabled = bInLightFunctionAtlasEnabled;
	}

	void AddSystem(ELightFunctionAtlasSystem In)
	{
		SystemFlags |= 1u << uint32(In);
	}

	void ClearSystems()
	{
		SystemFlags = 0;
	}

	FLightFunctionAtlas* GetLightFunctionAtlas()				const { return LightFunctionAtlas; }
	bool UsesLightFunctionAtlas(ELightFunctionAtlasSystem In)	const { return (SystemFlags & (1u<<uint32(In))) != 0; }
	bool GetLightFunctionAtlasEnabled()							const { return bLightFunctionAtlasEnabled; }

private:
	FLightFunctionAtlas* LightFunctionAtlas = nullptr;
	bool bLightFunctionAtlasEnabled = false;
	uint32 SystemFlags = 0;
};

struct FLightFunctionAtlasViewData
{
	FLightFunctionAtlasViewData() {}
	FLightFunctionAtlasViewData(FLightFunctionAtlasSceneData* InSceneData, uint32 InViewIndex) : SceneData(InSceneData), ViewIndex(InViewIndex) {}

	FLightFunctionAtlas* GetLightFunctionAtlas()				const { return SceneData ? SceneData->GetLightFunctionAtlas() : nullptr; }
	bool GetLightFunctionAtlasEnabled()							const { return SceneData ? SceneData->GetLightFunctionAtlasEnabled() : false; }
	bool UsesLightFunctionAtlas(ELightFunctionAtlasSystem In)	const { return SceneData ? SceneData->UsesLightFunctionAtlas(In) : false; }
	uint32 GetViewIndex() 										const { return ViewIndex; }

private:
	FLightFunctionAtlasSceneData* SceneData = nullptr;
	uint32 ViewIndex = 0;
};

struct FLightFunctionSlotKey
{
	uint32 LFMaterialUniqueID = 0;

	uint32 EffectiveLightFunctionSlotIndex = 0;	// Not used to de-duplicate light function. It is the index of the effective lighting function for this frame in EffectiveLightFunctionSlotArray.

	FLightFunctionSlotKey() {};

	FLightFunctionSlotKey(FLightSceneInfo* InLightSceneInfo);

	inline bool operator==(const FLightFunctionSlotKey& Other) const 
	{ 
		return Other.LFMaterialUniqueID == LFMaterialUniqueID;
	}
};

inline uint32 GetTypeHash(FLightFunctionSlotKey Key)
{
	return ::GetTypeHash(Key.LFMaterialUniqueID);
}

struct FLightFunctionSlot
{
	const FMaterialRenderProxy* LightFunctionMaterial;
	FIntPoint Min;
	FIntPoint Max;
};

#define LIGHT_FUNCTION_ATLAS_MAX_LIGHT_FUNCTION_COUNT 256

struct FAtlasLightInfoData
{
	FVector4f  Parameters;
	FMatrix44f Transform;
};

struct FLightFunctionAtlasSetup
{
	uint32 EdgeSize = 2;
	uint32 SlotResolution = 32;
};



// This class holds all data and resources related light function for a single scene, including multiple views.
struct FLightFunctionAtlas
{
	FLightFunctionAtlas();
	virtual ~FLightFunctionAtlas();

	bool IsLightFunctionAtlasEnabled() const { return bLightFunctionAtlasEnabled; }

	void ClearEmptySceneFrame(FViewInfo* View = nullptr, uint32 ViewIndex = 0, FLightFunctionAtlasSceneData* LightFunctionAtlasSceneData = nullptr);

	void BeginSceneFrame(const FViewFamilyInfo& ViewFamily, TArray<FViewInfo>& Views, FLightFunctionAtlasSceneData& LightFunctionAtlasSceneData, bool bShouldRenderVolumetricFog);

	void UpdateRegisterLightSceneInfo(FLightSceneInfo* LightSceneInfo);

	void UpdateLightFunctionAtlas(const TArray<FViewInfo>& Views);

	void RenderLightFunctionAtlas(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views);


	FScreenPassTexture AddDebugVisualizationPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor, FScreenPassRenderTarget& OverrideOutput)  const;

	FLightFunctionAtlasGlobalParameters*						GetLightFunctionAtlasGlobalParametersStruct(FRDGBuilder& GraphBuilder, uint32 ViewIndex);
	TRDGUniformBufferRef<FLightFunctionAtlasGlobalParameters>	GetLightFunctionAtlasGlobalParameters(FRDGBuilder& GraphBuilder, uint32 ViewIndex);
	
	static FLightFunctionAtlasGlobalParameters*					GetDefaultLightFunctionAtlasGlobalParametersStruct(FRDGBuilder& GraphBuilder);
	TRDGUniformBufferRef<FLightFunctionAtlasGlobalParameters>	GetDefaultLightFunctionAtlasGlobalParameters(FRDGBuilder& GraphBuilder);

	bool IsOutOfSlots();
	FString GetOutOfSlotWarningMessage();

private:

	void AllocateAtlasSlots(const TArray<FViewInfo>& Views);

	void AllocateTexture2DAtlas(FRDGBuilder& GraphBuilder);

	void RenderAtlasSlots(FRDGBuilder& GraphBuilder, const TArray<FViewInfo>& Views);

	bool bLightFunctionAtlasEnabled = false;

	FRDGTextureRef RDGAtlasTexture2D = nullptr;
	FRDGBufferRef RDGLightInfoDataBuffer = nullptr;

	// All the lights that wants to sample light functions
	TArray<FLightSceneInfo*> RegisteredLights;

	// This set is used to de-duplicate light functions in the atlas. An alternative to set would be to use simple array with simple loops.
	TSet<FLightFunctionSlotKey> LightFunctionsSet;

	// The structure used to render light function and generate the associated constant buffer, containing UVs and transformation matrices.
	struct EffectiveLightFunctionSlot
	{
		const FMaterialRenderProxy* LightFunctionMaterial = nullptr;
		FIntPoint Min;
		FIntPoint Max;
		float MinU;
		float MinV;
	};
	TArray<EffectiveLightFunctionSlot> EffectiveLightFunctionSlotArray;

	struct EffectiveLocalLightSlot
	{
		FLightSceneInfo*	LightSceneInfo = nullptr;
		uint8				LightFunctionAtlasSlotIndex = 0;
	};
	TArray<EffectiveLocalLightSlot>										EffectiveLocalLightSlotArray;

	FLightFunctionAtlasGlobalParameters*								DefaultLightFunctionAtlasGlobalParameters = nullptr;
	TRDGUniformBufferRef<FLightFunctionAtlasGlobalParameters>			DefaultLightFunctionAtlasGlobalParametersUB;

	TArray<FLightFunctionAtlasGlobalParameters*>						ViewLightFunctionAtlasGlobalParametersArray;
	TArray<TRDGUniformBufferRef<FLightFunctionAtlasGlobalParameters>>	ViewLightFunctionAtlasGlobalParametersUBArray;

#if WITH_EDITOR
	uint32																LightCountWithLFMaterialsNotSamplingAtlas = 0;
	TMap<uint32, const UMaterialInterface*>								NonCompatibleLightFunctionMaterials;
#endif

#if !UE_BUILD_SHIPPING
	uint32																LightCountSkippedDueToMissingAtlasSlot = 0;
	TSet<FLightFunctionSlotKey>											SkippedLightFunctionsSet;
#endif

	FLightFunctionAtlasSetup AtlasSetup;
};


bool IsEnabled(const FViewInfo& InView, ELightFunctionAtlasSystem In);
bool IsEnabled(const FScene& InScene, ELightFunctionAtlasSystem In);
void OnRenderBegin(FLightFunctionAtlas& In, FScene& InScene, TArray<FViewInfo>& Views, const FViewFamilyInfo& ViewFamily);


TRDGUniformBufferRef<FLightFunctionAtlasGlobalParameters> BindGlobalParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View);
FLightFunctionAtlasGlobalParameters* GetGlobalParametersStruct(FRDGBuilder& GraphBuilder, const FViewInfo& View);


} // namespace LightFunctionAtlas
