// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightFunctionRendering.cpp: Implementation for rendering light functions.
=============================================================================*/

#include "LightFunctionRendering.h"
#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "EngineGlobals.h"
#include "Materials/Material.h"
#include "RHIDefinitions.h"
#include "RHI.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "PostProcess/SceneRenderTargets.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShaderType.h"
#include "SceneRenderTargetParameters.h"
#include "MaterialShader.h"
#include "ShadowRendering.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "LightRendering.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "HairStrands/HairStrandsData.h"
#include "VariableRateShadingImageManager.h"
#include "HairStrands/HairStrandsUtils.h"

using namespace LightFunctionAtlas;

/**
 * A vertex shader for projecting a light function onto the scene.
 */
class FLightFunctionVS : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FLightFunctionVS,Material);
public:

	/**
	  * Makes sure only shaders for materials that are explicitly flagged
	  * as 'UsedAsLightFunction' in the Material Editor gets compiled into
	  * the shader cache.
	  */
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.MaterialParameters.MaterialDomain == MD_LightFunction && IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	FLightFunctionVS( )	{ }
	FLightFunctionVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMaterialShader(Initializer)
	{
		StencilingGeometryParameters.Bind(Initializer.ParameterMap);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FLightSceneInfo* LightSceneInfo )
	{
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
		
		// Light functions are projected using a bounding sphere.
		// Calculate transform for bounding stencil sphere.
		FSphere LightBounds = LightSceneInfo->Proxy->GetBoundingSphere();
		if (LightSceneInfo->Proxy->GetLightType() == LightType_Directional)
		{
			LightBounds.Center = View.ViewMatrices.GetViewOrigin();
		}

		FVector4f StencilingSpherePosAndScale;
		StencilingGeometry::GStencilSphereVertexBuffer.CalcTransform(StencilingSpherePosAndScale, LightBounds, View.ViewMatrices.GetPreViewTranslation());
		StencilingGeometryParameters.Set(BatchedParameters, (FVector4f)StencilingSpherePosAndScale); // LWC_TODO: precision loss
	}

private:
	LAYOUT_FIELD(FStencilingGeometryShaderParameters, StencilingGeometryParameters);
};

IMPLEMENT_MATERIAL_SHADER_TYPE(,FLightFunctionVS,TEXT("/Engine/Private/LightFunctionVertexShader.usf"),TEXT("Main"),SF_Vertex);

void LightFunctionSvPositionToLightTransform(FMatrix44f& OutMatrix, const FViewInfo& View, const FLightSceneInfo& LightSceneInfo)
{
	const FVector Scale = LightSceneInfo.Proxy->GetLightFunctionScale();
	// Switch x and z so that z of the user specified scale affects the distance along the light direction
	const FVector InverseScale = FVector(1.0 / Scale.Z, 1.0 / Scale.Y, 1.0 / Scale.X);
	const FMatrix WorldToLight = LightSceneInfo.Proxy->GetWorldToLight() * FScaleMatrix(InverseScale);
	const FVector2D InvViewSize = FVector2D(1.0 / View.ViewRect.Width(), 1.0 / View.ViewRect.Height());

	// setup a matrix to transform float4(SvPosition.xyz,1) directly to Light (quality, performance as we don't need to convert or use interpolator)
	//	new_xy = (xy - ViewRectMin.xy) * ViewSizeAndInvSize.zw * float2(2,-2) + float2(-1, 1);
	//  transformed into one MAD:  new_xy = xy * ViewSizeAndInvSize.zw * float2(2,-2)      +       (-ViewRectMin.xy) * ViewSizeAndInvSize.zw * float2(2,-2) + float2(-1, 1);

	const double Mx = 2.0 * InvViewSize.X;
	const double My = -2.0 * InvViewSize.Y;
	const double Ax = -1.0 - 2.0 * View.ViewRect.Min.X * InvViewSize.X;
	const double Ay = 1.0 + 2.0 * View.ViewRect.Min.Y * InvViewSize.Y;

	// todo: we could use InvTranslatedViewProjectionMatrix and TranslatedWorldToLight for better quality
	const FMatrix SvPositionToLightValue =
		FMatrix(
			FPlane(Mx,  0,   0,  0),
			FPlane(0, My,   0,  0),
			FPlane(0,  0,   1,  0),
			FPlane(Ax, Ay,   0,  1)
		) * View.ViewMatrices.GetClipToWorld() * WorldToLight;

	OutMatrix = FMatrix44f(SvPositionToLightValue);
}

FVector3f GetCamRelativeLightPosition(const FViewMatrices& ViewMatrices, const FLightSceneInfo& LightSceneInfo)
{
	if (LightSceneInfo.Type == LightType_Directional)
	{
		return FVector3f::Zero();
	}

	FVector CameraPosition = ViewMatrices.GetViewOrigin();
	FVector LightOrigin = LightSceneInfo.Proxy->GetOrigin();
	return FVector3f(LightOrigin - CameraPosition);
}

/**
 * A pixel shader for projecting a light function onto the scene.
 */
class FLightFunctionPS : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FLightFunctionPS,Material);
public:
	/**
	  * Makes sure only shaders for materials that are explicitly flagged
	  * as 'UsedAsLightFunction' in the Material Editor gets compiled into
	  * the shader cache.
	  */
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.MaterialParameters.MaterialDomain == MD_LightFunction && IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
		OutEnvironment.SetDefine(TEXT("HAIR_STRANDS_SUPPORTED"), IsHairStrandsSupported(EHairStrandsShaderType::Strands, Parameters.Platform) ? 1 : 0);
	}

	FLightFunctionPS() {}
	FLightFunctionPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMaterialShader(Initializer)
	{
		SvPositionToLight.Bind(Initializer.ParameterMap,TEXT("SvPositionToLight"));
		LightFunctionParameters.Bind(Initializer.ParameterMap);
		LightFunctionParameters2.Bind(Initializer.ParameterMap,TEXT("LightFunctionParameters2"));
		HairOnlyDepthTexture.Bind(Initializer.ParameterMap, TEXT("HairOnlyDepthTexture"));
		CameraRelativeLightPosition.Bind(Initializer.ParameterMap, TEXT("CameraRelativeLightPosition"));
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FViewInfo& View, const FLightSceneInfo* LightSceneInfo, const FMaterialRenderProxy* MaterialProxy, const FMaterial& Material, bool bRenderingPreviewShadowIndicator, float ShadowFadeFraction, bool bUseHairStrands, FRHITexture* InHairOnlyDepthTexture)
	{
		FMaterialShader::SetViewParameters(BatchedParameters, View, View.ViewUniformBuffer);
		FMaterialShader::SetParameters(BatchedParameters, MaterialProxy, Material, View);

		// Set the transform from screen space to light space.
		if ( SvPositionToLight.IsBound() )
		{
			FMatrix44f SvPositionToLightValue;
			LightFunctionSvPositionToLightTransform(SvPositionToLightValue, View, *LightSceneInfo);
			SetShaderValue(BatchedParameters, SvPositionToLight, SvPositionToLightValue);
		}

		LightFunctionParameters.Set(BatchedParameters, LightSceneInfo, ShadowFadeFraction);

		SetShaderValue(BatchedParameters, LightFunctionParameters2, FVector4f(
			LightSceneInfo->Proxy->GetLightFunctionFadeDistance(), 
			LightSceneInfo->Proxy->GetLightFunctionDisabledBrightness(),
			bRenderingPreviewShadowIndicator ? 1.0f : 0.0f,
			bUseHairStrands ? 1.0f : 0.0f));

		if (HairOnlyDepthTexture.IsBound() && InHairOnlyDepthTexture)
		{
			SetTextureParameter(BatchedParameters, HairOnlyDepthTexture, InHairOnlyDepthTexture);
		}

		if (CameraRelativeLightPosition.IsBound())
		{
			SetShaderValue(BatchedParameters, CameraRelativeLightPosition, GetCamRelativeLightPosition(View.ViewMatrices, *LightSceneInfo));
		}

		auto DeferredLightParameter = GetUniformBufferParameter<FDeferredLightUniformStruct>();
		if (DeferredLightParameter.IsBound())
		{
			SetDeferredLightParameters(BatchedParameters, DeferredLightParameter, LightSceneInfo, View, LightFunctionAtlas::IsEnabled(View, ELightFunctionAtlasSystem::DeferredLighting));
		}
	}

private:
	LAYOUT_FIELD(FShaderParameter, SvPositionToLight);
	LAYOUT_FIELD(FLightFunctionSharedParameters, LightFunctionParameters);
	LAYOUT_FIELD(FShaderParameter, LightFunctionParameters2);
	LAYOUT_FIELD(FShaderResourceParameter, HairOnlyDepthTexture);
	LAYOUT_FIELD(FShaderParameter, CameraRelativeLightPosition);
};

IMPLEMENT_MATERIAL_SHADER_TYPE(,FLightFunctionPS,TEXT("/Engine/Private/LightFunctionPixelShader.usf"),TEXT("Main"),SF_Pixel);

float GetLightFunctionFadeFraction(const FViewInfo& View, FSphere LightBounds)
{
	extern float CalculateShadowFadeAlpha(const float MaxUnclampedResolution, const uint32 ShadowFadeResolution, const uint32 MinShadowResolution);

	// Override the global settings with the light's settings if the light has them specified
	static auto CVarMinShadowResolution = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Shadow.MinResolution"));
	static auto CVarShadowFadeResolution = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Shadow.FadeResolution"));

	const uint32 MinShadowResolution  = FMath::Max<int32>(0, CVarMinShadowResolution->GetValueOnRenderThread());
	const uint32 ShadowFadeResolution = FMath::Max<int32>(0, CVarShadowFadeResolution->GetValueOnRenderThread());

	// Project the bounds onto the view
	const FVector4 ScreenPosition = View.WorldToScreen(LightBounds.Center);

	int32 SizeX = View.ViewRect.Width();
	int32 SizeY = View.ViewRect.Height();
	const float ScreenRadius = FMath::Max(
		SizeX / 2.0f * View.ViewMatrices.GetViewToClip().M[0][0],
		SizeY / 2.0f * View.ViewMatrices.GetViewToClip().M[1][1]) *
		LightBounds.W /
		FMath::Max(ScreenPosition.W, 1.0f);

	static auto CVarShadowTexelsPerPixel = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.Shadow.TexelsPerPixel"));
	const float UnclampedResolution = ScreenRadius * CVarShadowTexelsPerPixel->GetValueOnRenderThread();
	const float ResolutionFadeAlpha = CalculateShadowFadeAlpha(UnclampedResolution, ShadowFadeResolution, MinShadowResolution);
	return ResolutionFadeAlpha;
}

/**
* Used by RenderLights to figure out if light functions need to be rendered to the attenuation buffer.
*
* @param LightSceneInfo Represents the current light
* @return true if anything got rendered
*/
bool FSceneRenderer::CheckForLightFunction( const FLightSceneInfo* LightSceneInfo ) const
{
	// NOTE: The extra check is necessary because there could be something wrong with the material.
	if( LightSceneInfo->Proxy->GetLightFunctionMaterial() && 
		LightSceneInfo->Proxy->GetLightFunctionMaterial()->GetIncompleteMaterialWithFallback(Scene->GetFeatureLevel()).IsLightFunction())
	{
		FSphere LightBounds = LightSceneInfo->Proxy->GetBoundingSphere();
		for (int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];

			if (LightSceneInfo->Proxy->GetLightType() == LightType_Directional)
			{
				LightBounds.Center = View.ViewMatrices.GetViewOrigin();
			}

			if(View.VisibleLightInfos[LightSceneInfo->Id].bInViewFrustum
				// Only draw the light function if it hasn't completely faded out
				&& GetLightFunctionFadeFraction(View, LightBounds) > 1.0f / 256.0f)
			{
				return true;
			}
		}
	}
	return false;
}

/**
 * Used by RenderLights to render a light function to the attenuation buffer.
 *
 * @param LightSceneInfo Represents the current light
 */
bool FDeferredShadingSceneRenderer::RenderLightFunction(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef ScreenShadowMaskTexture,
	bool bLightAttenuationCleared,
	bool bProjectingForForwardShading,
	bool bUseHairStrands)
{
	if (ViewFamily.EngineShowFlags.LightFunctions)
	{
		return RenderLightFunctionForMaterial(GraphBuilder, SceneTextures, LightSceneInfo, ScreenShadowMaskTexture, LightSceneInfo->Proxy->GetLightFunctionMaterial(), bLightAttenuationCleared, bProjectingForForwardShading, false, bUseHairStrands);
	}
	
	return false;
}

bool FDeferredShadingSceneRenderer::RenderPreviewShadowsIndicator(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef ScreenShadowMaskTexture,
	bool bLightAttenuationCleared,
	bool bUseHairStrands)
{
	if (GEngine->PreviewShadowsIndicatorMaterial)
	{
		return RenderLightFunctionForMaterial(GraphBuilder, SceneTextures, LightSceneInfo, ScreenShadowMaskTexture, GEngine->PreviewShadowsIndicatorMaterial->GetRenderProxy(), bLightAttenuationCleared, false, true, bUseHairStrands);
	}

	return false;
}

BEGIN_SHADER_PARAMETER_STRUCT(FRenderLightFunctionForMaterialParameters, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairOnlyDepthTexture)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static bool TryGetLightFunctionShaders(ERHIFeatureLevel::Type InFeatureLevel, FMaterialRenderProxy const*& OutMaterialProxy, FMaterial const*& OutMaterial, FMaterialShaders& OutShaders)
{
	while (OutMaterialProxy)
	{
		OutMaterial = OutMaterialProxy->GetMaterialNoFallback(InFeatureLevel);
		if (OutMaterial && OutMaterial->IsLightFunction())
		{
			FMaterialShaderTypes ShaderTypes;
			ShaderTypes.AddShaderType<FLightFunctionVS>();
			ShaderTypes.AddShaderType<FLightFunctionPS>();
			if (OutMaterial->TryGetShaders(ShaderTypes, nullptr, OutShaders))
			{
				return true;
			}
		}
		OutMaterialProxy = OutMaterialProxy->GetFallback(InFeatureLevel);
	}
	return false;
}

bool FDeferredShadingSceneRenderer::RenderLightFunctionForMaterial(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef ScreenShadowMaskTexture,
	const FMaterialRenderProxy* MaterialProxy,
	bool bLightAttenuationCleared,
	bool bProjectingForForwardShading,
	bool bRenderingPreviewShadowsIndicator,
	bool bUseHairStrands)
{
	check(ScreenShadowMaskTexture);
	check(LightSceneInfo);

	FMaterialShaders MaterialShaders;
	const FMaterialRenderProxy* MaterialProxyForRendering = MaterialProxy;
	const FMaterial* MaterialForRendering = nullptr;
	if (!TryGetLightFunctionShaders(Scene->GetFeatureLevel(), MaterialProxyForRendering, MaterialForRendering, MaterialShaders))
	{
		return false;
	}

	const FLightSceneProxy* LightSceneProxy = LightSceneInfo->Proxy;

	// Render to the light attenuation buffer for all views.
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

		if (View.VisibleLightInfos[LightSceneInfo->Id].bInViewFrustum)
		{
			if (bUseHairStrands && !View.HairStrandsViewData.bIsValid)
			{
				// bUseHairStrands is true if one of the view has valid hair strands data.
				// Skip views that do not have valid hair strands data to prevent HairOnlyDepthTexture nullptr access
				continue;
			}

			FRenderLightFunctionForMaterialParameters* PassParameters = GraphBuilder.AllocParameters<FRenderLightFunctionForMaterialParameters>();
			PassParameters->SceneTextures = SceneTextures.UniformBuffer;
			PassParameters->HairOnlyDepthTexture = (View.HairStrandsViewData.bIsValid && View.HairStrandsViewData.VisibilityData.HairOnlyDepthTexture) ? View.HairStrandsViewData.VisibilityData.HairOnlyDepthTexture : GSystemTextures.GetDepthDummy(GraphBuilder);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenShadowMaskTexture, bLightAttenuationCleared ? ERenderTargetLoadAction::ELoad : ERenderTargetLoadAction::ENoAction);
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);
			PassParameters->RenderTargets.ShadingRateTexture = GVRSImageManager.GetVariableRateShadingImage(GraphBuilder, View, FVariableRateShadingImageManager::EVRSPassType::LightFunctions);

			// If render shadow mask for hair strands, then swap depth to hair only depth
			if (bUseHairStrands)
			{
				PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(View.HairStrandsViewData.VisibilityData.HairOnlyDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("LightFunction Material=%s", *MaterialForRendering->GetFriendlyName()),
				PassParameters,
				ERDGPassFlags::Raster,
				[&View, PassParameters, LightSceneInfo, LightSceneProxy, MaterialProxyForRendering, MaterialForRendering, MaterialShaders, bLightAttenuationCleared, bProjectingForForwardShading, bRenderingPreviewShadowsIndicator, bUseHairStrands](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FSphere LightBounds = LightSceneProxy->GetBoundingSphere();

				if (LightSceneProxy->GetLightType() == LightType_Directional)
				{
					LightBounds.Center = View.ViewMatrices.GetViewOrigin();
				}
				const float FadeAlpha = GetLightFunctionFadeFraction(View, LightBounds);
				// Don't draw the light function if it has completely faded out
				if (FadeAlpha < 1.0f / 256.0f)
				{
					if (!bLightAttenuationCleared)
					{
						LightSceneProxy->SetScissorRect(RHICmdList, View, View.ViewRect);
						DrawClearQuad(RHICmdList, FLinearColor::White);
					}
				}
				else
				{
					// Set the device viewport for the view.
					RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

					// Set the states to modulate the light function with the render target.
					TShaderRef<FLightFunctionVS> VertexShader;
					TShaderRef<FLightFunctionPS> PixelShader;
					MaterialShaders.TryGetVertexShader(VertexShader);
					MaterialShaders.TryGetPixelShader(PixelShader);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;

					if (bLightAttenuationCleared)
					{
						if (bRenderingPreviewShadowsIndicator)
						{
							GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Max, BF_One, BF_One, BO_Max, BF_One, BF_One>::GetRHI();
						}
						else
						{
							GraphicsPSOInit.BlendState = FProjectedShadowInfo::GetBlendStateForProjection(
								LightSceneInfo->GetDynamicShadowMapChannel(),
								false,
								false,
								bProjectingForForwardShading,
								false);
						}
					}
					else
					{
						GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA>::GetRHI();
					}

					if (((FVector)View.ViewMatrices.GetViewOrigin() - LightBounds.Center).SizeSquared() < FMath::Square(LightBounds.W * 1.05f + View.NearClippingDistance * 2.0f))
					{
						// Render backfaces with depth tests disabled since the camera is inside (or close to inside) the light function geometry
						GraphicsPSOInit.RasterizerState = View.bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
					}
					else
					{
						// Render frontfaces with depth tests on to get the speedup from HiZ since the camera is outside the light function geometry
						GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
						GraphicsPSOInit.RasterizerState = View.bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();
					}

					// Set the light's scissor rectangle.
					FIntRect ClippedViewRect = View.ViewRect;
					LightSceneProxy->SetScissorRect(RHICmdList, View, View.ViewRect, &ClippedViewRect);
					if (bUseHairStrands)
					{
						FIntRect TotalRect = ComputeVisibleHairStrandsMacroGroupsRect(View, View.ViewRect, View.HairStrandsViewData.MacroGroupDatas);
						TotalRect.Clip(ClippedViewRect);
						RHICmdList.SetScissorRect(true, TotalRect.Min.X, TotalRect.Min.Y, TotalRect.Max.X, TotalRect.Max.Y);
					}

					// Render a bounding light sphere.
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParametersLegacyVS(RHICmdList, VertexShader, View, LightSceneInfo);
					SetShaderParametersLegacyPS(RHICmdList, PixelShader, View, LightSceneInfo, MaterialProxyForRendering, *MaterialForRendering, bRenderingPreviewShadowsIndicator, FadeAlpha, bUseHairStrands, PassParameters->HairOnlyDepthTexture->GetRHI());

					// Project the light function using a sphere around the light
					//@todo - could use a cone for spotlights
					StencilingGeometry::DrawSphere(RHICmdList);
				}

				RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
			});
		}
	}

	return true;
}
