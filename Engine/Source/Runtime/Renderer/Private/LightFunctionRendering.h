// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

class FViewInfo;
struct FViewMatrices;
class FLightSceneInfo;

// Computes a matrix to transform float4(SvPosition.xyz,1) directly to coordinate system of the given light
void LightFunctionSvPositionToLightTransform(FMatrix44f& OutMatrix, const FViewInfo& View, const FLightSceneInfo& LightSceneInfo);

FVector3f GetCamRelativeLightPosition(const FViewMatrices& ViewMatrices, const FLightSceneInfo& LightSceneInfo);

// Returns a fade fraction for a light function and a given view based on the appropriate fade settings.
float GetLightFunctionFadeFraction(const FViewInfo& View, FSphere LightBounds);
