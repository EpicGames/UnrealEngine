// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneExtensions.h"
#include "PrimitiveSceneProxy.h"

class FLightGridSceneExtension : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FLightGridSceneExtension);

	friend class FLightGridSceneExtensionUpdater;

public:
	using ISceneExtension::ISceneExtension;

	//~ Begin ISceneExtension Interface.
	virtual ISceneExtensionUpdater* CreateUpdater() override;
	//~ End ISceneExtension Interface.

	struct FViewData
	{
		// Map from Light ID in GPU Scene to index in the View's ForwardLightData array
		// This is cached between frames so we can access previous frame mapping
		TArray<int32> LightSceneIdToForwardLightIndex;

		// Similar to LightSceneIdToForwardLightIndex but for simple lights
		TMap<FSimpleLightId, int32> SimpleLightIdToForwardLightIndex;
	};

	FViewData* GetViewData(const FViewInfo& View);

private:

	// Indexed by persistent view ID
	TSparseArray<FViewData> ViewDatas;
};
