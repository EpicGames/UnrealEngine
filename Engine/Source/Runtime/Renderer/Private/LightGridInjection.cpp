// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LightGridInjection.cpp
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "UniformBuffer.h"
#include "ShaderParameters.h"
#include "RendererInterface.h"
#include "EngineDefines.h"
#include "PrimitiveSceneProxy.h"
#include "Shader.h"
#include "SceneUtils.h"
#include "PostProcess/SceneRenderTargets.h"
#include "LightSceneInfo.h"
#include "RectLightSceneProxy.h"
#include "GlobalShader.h"
#include "SceneRendering.h"
#include "DeferredShadingRenderer.h"
#include "BasePassRendering.h"
#include "LogRenderer.h"
#include "ScenePrivate.h"
#include "ClearQuad.h"
#include "VolumetricFog.h"
#include "VolumetricCloudRendering.h"
#include "Components/LightComponent.h"
#include "Components/RectLightComponent.h"
#include "Engine/MapBuildDataRegistry.h"
#include "PixelShaderUtils.h"
#include "ShaderPrint.h"
#include "ShaderPrintParameters.h"
#include "RenderUtils.h"
#include "MegaLights/MegaLights.h"
#include "StochasticLighting/StochasticLighting.h"
#include "LightGrid.h"
#include "LightGridDefinitions.h"
#include "VolumetricFog.h"
#include "LightViewData.h"
#include "HairStrands/HairStrandsTransmittance.h"
#include "GPUMessaging.h"
#include "Templates/Greater.h"

static TAutoConsoleVariable<bool> CVarLightGridAsyncCompute(
	TEXT("r.Forward.LightGridAsyncCompute"),
	false,
	TEXT("Run the light culling passes in async compute."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

enum class ELightBufferMode
{
	VisibleLocalLights			= 0,
	VisibleLights				= 1,
	VisibleLightsStableIndices	= 2
};

static int32 GLightBufferMode = (int32)ELightBufferMode::VisibleLights;
static FAutoConsoleVariableRef CVarLightBufferMode(
	TEXT("r.Forward.LightBuffer.Mode"),
	GLightBufferMode,
	TEXT("0 - Visible local lights.\n")
	TEXT("1 - Visible local + directional lights.\n")
	TEXT("2 - Visible local + directional lights (with stable indices)."),
	ECVF_RenderThreadSafe
);

int32 GLightGridPixelSize = 64;
FAutoConsoleVariableRef CVarLightGridPixelSize(
	TEXT("r.Forward.LightGridPixelSize"),
	GLightGridPixelSize,
	TEXT("Size of a cell in the light grid, in pixels."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GLightGridSizeZ = 32;
FAutoConsoleVariableRef CVarLightGridSizeZ(
	TEXT("r.Forward.LightGridSizeZ"),
	GLightGridSizeZ,
	TEXT("Number of Z slices in the light grid."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GForwardLightGridDebug = 0;
FAutoConsoleVariableRef CVarLightGridDebug(
	TEXT("r.Forward.LightGridDebug"),
	GForwardLightGridDebug,
	TEXT("Whether to display on screen culledlight per tile.\n")
	TEXT(" 0: off (default)\n")
	TEXT(" 1: on - showing light count onto the depth buffer\n")
	TEXT(" 2: on - showing max light count per tile accoung for each slice but the last one (culling there is too conservative)\n")
	TEXT(" 3: on - showing max light count per tile accoung for each slice and the last one \n"),
	ECVF_RenderThreadSafe
);

int32 GForwardLightGridDebugMaxThreshold = 8;
FAutoConsoleVariableRef CVarLightGridDebugMaxThreshold(
	TEXT("r.Forward.LightGridDebug.MaxThreshold"),
	GForwardLightGridDebugMaxThreshold,
	TEXT("Maximum light threshold for heat map visualization. (default = 8)\n"),
	ECVF_RenderThreadSafe
);

int32 GLightGridHZBCull = 1;
FAutoConsoleVariableRef CVarLightGridHZBCull(
	TEXT("r.Forward.LightGridHZBCull"),
	GLightGridHZBCull,
	TEXT("Whether to use HZB culling to skip occluded grid cells."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GLightGridRefineRectLightBounds = 1;
FAutoConsoleVariableRef CVarLightGridRefineRectLightBounds(
	TEXT("r.Forward.LightGridDebug.RectLightBounds"),
	GLightGridRefineRectLightBounds,
	TEXT("Whether to refine rect light bounds (should only be disabled for debugging purposes)."),
	ECVF_RenderThreadSafe
);

int32 GMaxCulledLightsPerCell = 32;
FAutoConsoleVariableRef CVarMaxCulledLightsPerCell(
	TEXT("r.Forward.MaxCulledLightsPerCell"),
	GMaxCulledLightsPerCell,
	TEXT("Controls how much memory is allocated for each cell for light culling.  When r.Forward.LightLinkedListCulling is enabled, this is used to compute a global max instead of a per-cell limit on culled lights."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GLightLinkedListCulling = 1;
FAutoConsoleVariableRef CVarLightLinkedListCulling(
	TEXT("r.Forward.LightLinkedListCulling"),
	GLightLinkedListCulling,
	TEXT("Uses a reverse linked list to store culled lights, removing the fixed limit on how many lights can affect a cell - it becomes a global limit instead."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

int32 GLightCullingQuality = 1;
FAutoConsoleVariableRef CVarLightCullingQuality(
	TEXT("r.LightCulling.Quality"),
	GLightCullingQuality,
	TEXT("Whether to run compute light culling pass.\n")
	TEXT(" 0: off \n")
	TEXT(" 1: on (default)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarLightCullingWorkloadDistributionMode(
	TEXT("r.LightCulling.WorkloadDistributionMode"),
	0,
	TEXT("0 - single thread per cell.\n")
	TEXT("1 - thread group per cell (64 threads).\n")
	TEXT("2 - thread group per cell (32 threads if supported, otherwise single thread).\n")
	TEXT("(This cvar only applies to fine light grid. When using two levels, coarse grid always uses thread group per cell."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarLightCullingTwoLevel(
	TEXT("r.LightCulling.TwoLevel"),
	false,
	TEXT("Whether to build light grid in two passes."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarLightCullingTwoLevelThreshold(
	TEXT("r.LightCulling.TwoLevel.Threshold"),
	128,
	TEXT("Threshold used to determine whether to use two level culling based on the number of lights in view."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarLightCullingTwoLevelExponent(
	TEXT("r.LightCulling.TwoLevel.Exponent"),
	2,
	TEXT("Exponent used to derive the coarse grid size (base 2)."),
	ECVF_RenderThreadSafe
);

float GLightCullingMaxDistanceOverrideKilometers = -1.0f;
FAutoConsoleVariableRef CVarLightCullingMaxDistanceOverride(
	TEXT("r.LightCulling.MaxDistanceOverrideKilometers"),
	GLightCullingMaxDistanceOverrideKilometers,
	TEXT("Used to override the maximum far distance at which we can store data in the light grid.\n If this is increase, you might want to update r.Forward.LightGridSizeZ to a reasonable value according to your use case light count and distribution.")
	TEXT(" <=0: off \n")
	TEXT(" >0: the far distance in kilometers.\n"),
	ECVF_RenderThreadSafe
);

// Experimental version of FLightGridInjectionCS that uses specialization constants
static TAutoConsoleVariable<int32> CVarLightGridInjectionUseSpecializationConstants(
	TEXT("r.LightGridInjection.UseSpecializationConstants"),
	1,
	TEXT("Experimental: Selects whether LightGridInjection should use specialization constant. (default: 1)"),
	ECVF_RenderThreadSafe);

static inline bool UseSpecializationConstants(EShaderPlatform ShaderPlatform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsSpecializationConstants(ShaderPlatform) &&
		CVarLightGridInjectionUseSpecializationConstants.GetValueOnAnyThread() == 1;
}


DECLARE_GPU_STAT(LightGrid);

#if !UE_BUILD_SHIPPING
void LightGridFeedbackStatus(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FRDGBufferRef CulledLightDataAllocatorBuffer,
	uint32 NumCulledLightDataEntries,
	FRDGBufferRef CulledLightLinkAllocatorBuffer,
	uint32 NumCulledLightLinks,
	bool bUseAsyncCompute);
#endif // !UE_BUILD_SHIPPING

bool ShouldVisualizeLightGrid(EShaderPlatform InShaderPlatform)
{
	const bool bEnabled = GForwardLightGridDebug > 0;

	if (bEnabled && !GIsEditor)
	{
		GEngine->AddOnScreenDebugMessage(uint64(0xA2985F1D), 1.f, FColor::Red, TEXT("Light grid visualization is only support in editor builds."));
	}

	return bEnabled && ShaderPrint::IsSupported(InShaderPlatform) && GIsEditor;
}

// If this is changed, the LIGHT_GRID_USES_16BIT_BUFFERS define from LightGridCommon.ush should also be updated.
bool LightGridUses16BitBuffers(EShaderPlatform Platform)
{
	// CulledLightDataGrid, is typically 16bit elements to save on memory and bandwidth. So to not introduce any regressions it will stay as texel buffer on all platforms, except mobile and Metal (which does not support type conversions).
	return RHISupportsBufferLoadTypeConversion(Platform) && !IsMobilePlatform(Platform);
}

void SetupDummyForwardLightUniformParameters(FRDGBuilder& GraphBuilder, FForwardLightUniformParameters& ForwardLightUniformParameters, EShaderPlatform ShaderPlatform)
{
	FRDGBufferRef ForwardLightBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f));
	ForwardLightUniformParameters.ForwardLightBuffer = GraphBuilder.CreateSRV(ForwardLightBuffer);

	FRDGBufferRef NumCulledLightsGrid = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32));
	ForwardLightUniformParameters.NumCulledLightsGrid = GraphBuilder.CreateSRV(NumCulledLightsGrid);

	const bool bLightGridUses16BitBuffers = LightGridUses16BitBuffers(ShaderPlatform);
	FRDGBufferSRVRef CulledLightDataGridSRV = nullptr;
	if (bLightGridUses16BitBuffers)
	{
		FRDGBufferRef CulledLightDataGrid = GSystemTextures.GetDefaultBuffer(GraphBuilder, sizeof(uint16));
		CulledLightDataGridSRV = GraphBuilder.CreateSRV(CulledLightDataGrid, PF_R16_UINT);
	}
	else
	{
		FRDGBufferRef CulledLightDataGrid = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32));
		CulledLightDataGridSRV = GraphBuilder.CreateSRV(CulledLightDataGrid);
	}
	ForwardLightUniformParameters.CulledLightDataGrid32Bit = CulledLightDataGridSRV;
	ForwardLightUniformParameters.CulledLightDataGrid16Bit = CulledLightDataGridSRV;

	ForwardLightUniformParameters.DirectionalLightLightFunctionAtlasLightIndex = 0;

	ForwardLightUniformParameters.DirectionalLightAffectsTranslucentLighting = 0;

	FRDGBufferRef DirectionalLightIndicesBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32));
	ForwardLightUniformParameters.DirectionalLightIndices = GraphBuilder.CreateSRV(DirectionalLightIndicesBuffer);

	FRDGBufferRef LightViewDataBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FLightViewData));
	ForwardLightUniformParameters.LightViewData = GraphBuilder.CreateSRV(LightViewDataBuffer);

	FRDGBufferRef LightNamesBuffer = GSystemTextures.GetDefaultByteAddressBuffer(GraphBuilder, sizeof(uint32));
	ForwardLightUniformParameters.LightNamesBuffer = GraphBuilder.CreateSRV(LightNamesBuffer);
	ForwardLightUniformParameters.LightNameDwords = 0;
}

void SetupDummyForwardDirLightShadowParameters(FRDGBuilder& GraphBuilder, FForwardDirectionalLightShadowMapParameters& ShadowParameters)
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
	ShadowParameters.DirectionalLightShadowmapAtlas = SystemTextures.Black;
	ShadowParameters.DirectionalLightStaticShadowmap = GBlackTexture->TextureRHI;
}

TRDGUniformBufferRef<FForwardLightUniformParameters> CreateDummyForwardLightUniformBuffer(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform)
{
	FForwardLightUniformParameters* ForwardLightUniformParameters = GraphBuilder.AllocParameters<FForwardLightUniformParameters>();
	SetupDummyForwardLightUniformParameters(GraphBuilder, *ForwardLightUniformParameters, ShaderPlatform);
	return GraphBuilder.CreateUniformBuffer(ForwardLightUniformParameters);
}

TRDGUniformBufferRef<FForwardDirectionalLightShadowMapParameters> CreateDummyForwardDirLightShadowUniformBuffer(FRDGBuilder& GraphBuilder)
{
	FForwardDirectionalLightShadowMapParameters* ShadowParameters = GraphBuilder.AllocParameters<FForwardDirectionalLightShadowMapParameters>();
	SetupDummyForwardDirLightShadowParameters(GraphBuilder, *ShadowParameters);
	return GraphBuilder.CreateUniformBuffer(ShadowParameters);
}

void SetDummyForwardLightUniformBufferOnViews(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform, TArray<FViewInfo>& Views)
{
	TRDGUniformBufferRef<FForwardLightUniformParameters> ForwardLightUniformBuffer = CreateDummyForwardLightUniformBuffer(GraphBuilder, ShaderPlatform);
	TRDGUniformBufferRef<FForwardDirectionalLightShadowMapParameters> ForwardDirLightShadowUniformBuffer = CreateDummyForwardDirLightShadowUniformBuffer(GraphBuilder);
	for (auto& View : Views)
	{
		View.ForwardLightingResources.SetUniformBuffer(ForwardLightUniformBuffer);
		View.ForwardLightingResources.SetShadowUniformBuffer(ForwardDirLightShadowUniformBuffer);
	}
}

void SetForwardLightUniformParametersFromView(FRDGBuilder& GraphBuilder, const FViewInfo& View, FForwardLightUniformParameters& ForwardLightUniformParameters)
{
	if (View.ForwardLightingResources.ForwardLightUniformParameters)
	{
		ForwardLightUniformParameters = *View.ForwardLightingResources.ForwardLightUniformParameters;
	}
	else
	{
		SetupDummyForwardLightUniformParameters(GraphBuilder, ForwardLightUniformParameters, View.GetShaderPlatform());
	}
}

void SetForwardDirLightShadowParametersFromView(FRDGBuilder& GraphBuilder, const FViewInfo& View, FForwardDirectionalLightShadowMapParameters& Parameters)
{
	if (View.ForwardLightingResources.ForwardDirLightShadowParameters)
	{
		Parameters = *View.ForwardLightingResources.ForwardDirLightShadowParameters;
	}
	else
	{
		SetupDummyForwardDirLightShadowParameters(GraphBuilder, Parameters);
	}
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FForwardLightUniformParameters, "ForwardLightStruct");

FForwardLightUniformParameters::FForwardLightUniformParameters()
{
	FMemory::Memzero(*this);
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FForwardDirectionalLightShadowMapParameters, "ForwardDirLightShadowStruct");

FForwardDirectionalLightShadowMapParameters::FForwardDirectionalLightShadowMapParameters()
{
	FMemory::Memzero(*this);
	ShadowmapSampler = TStaticSamplerState<SF_Point,AM_Clamp,AM_Clamp,AM_Clamp>::GetRHI();
	DirectionalLightStaticShadowmap = GBlackTexture->TextureRHI;
	StaticShadowmapSampler = TStaticSamplerState<SF_Bilinear,AM_Clamp,AM_Clamp,AM_Clamp>::GetRHI();
}

IMPLEMENT_SCENE_EXTENSION(FLightGridSceneExtension);

class FLightGridSceneExtensionUpdater : public ISceneExtensionUpdater
{
	DECLARE_SCENE_EXTENSION_UPDATER(FLightGridSceneExtensionUpdater, FLightGridSceneExtension);

public:
	FLightGridSceneExtensionUpdater(FLightGridSceneExtension& InSceneExtension) : SceneExtension(InSceneExtension) {}

	virtual void PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet) override;
	virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override;
	virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override;

private:
	FLightGridSceneExtension& SceneExtension;

	const FScenePostUpdateChangeSet* PostUpdateChangeSet = nullptr;
};

void FLightGridSceneExtensionUpdater::PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet)
{
	if (LightSceneChangeSet.RemovedLightIds.Num() > 0)
	{
		for (FLightGridSceneExtension::FViewData& ViewData : SceneExtension.ViewDatas)
		{
			for (FLightSceneId Id : LightSceneChangeSet.RemovedLightIds)
			{
				if (ViewData.LightSceneIdToForwardLightIndex.IsValidIndex(Id.GetIndex()))
				{
					ViewData.LightSceneIdToForwardLightIndex[Id.GetIndex()] = INDEX_NONE;
				}
			}
		}
	}
}

void FLightGridSceneExtensionUpdater::PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
{
	if (ChangeSet.ViewUpdateChangeSet)
	{
		for (FPersistentViewId ViewId : ChangeSet.ViewUpdateChangeSet->RemovedViewIds)
		{
			check(SceneExtension.ViewDatas.IsAllocated(ViewId.Index));
			SceneExtension.ViewDatas.RemoveAt(ViewId.Index);
		}
	}
}

void FLightGridSceneExtensionUpdater::PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
{
	if (ChangeSet.ViewUpdateChangeSet)
	{
		for (FPersistentViewId ViewId : ChangeSet.ViewUpdateChangeSet->AddedViewIds)
		{
			if (!SceneExtension.ViewDatas.IsValidIndex(ViewId.Index))
			{
				SceneExtension.ViewDatas.EmplaceAt(ViewId.Index);
			}
		}
	}
}

ISceneExtensionUpdater* FLightGridSceneExtension::CreateUpdater()
{
	return new FLightGridSceneExtensionUpdater(*this);
}

FLightGridSceneExtension::FViewData* FLightGridSceneExtension::GetViewData(const FViewInfo& View)
{
	if (View.PersistentViewId.IsValid())
	{
		check(ViewDatas.IsAllocated(View.PersistentViewId.Index));
		return &ViewDatas[View.PersistentViewId.Index];
	}
	
	return nullptr;
}

int32 NumCulledLightsGridStride = 2;
int32 NumCulledGridPrimitiveTypes = 2;
int32 LightLinkStride = 2;

// 65k indexable light limit
typedef uint16 FLightIndexType;
// UINT_MAX indexable light limit
typedef uint32 FLightIndexType32;


class FLightGridInjectionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLightGridInjectionCS);
	SHADER_USE_PARAMETER_STRUCT(FLightGridInjectionCS, FGlobalShader)
public:
	class FUseLinkedList : SHADER_PERMUTATION_BOOL("USE_LINKED_CULL_LIST");
	class FRefineRectLightBounds : SHADER_PERMUTATION_BOOL("REFINE_RECTLIGHT_BOUNDS");
	class FUseHZBCull : SHADER_PERMUTATION_BOOL("USE_HZB_CULL");
	class FUseParentLightGrid : SHADER_PERMUTATION_BOOL("USE_PARENT_LIGHT_GRID");
	class FUseThreadGroupPerCell : SHADER_PERMUTATION_BOOL("USE_THREAD_GROUP_PER_CELL");
	class FUseThreadGroupSize32 : SHADER_PERMUTATION_BOOL("USE_THREAD_GROUP_SIZE_32");
	class FApplyIndirection : SHADER_PERMUTATION_BOOL("APPLY_INDIRECTION");
	using FPermutationDomain = TShaderPermutationDomain<FUseLinkedList, FRefineRectLightBounds, FUseHZBCull, FUseParentLightGrid, FUseThreadGroupPerCell, FUseThreadGroupSize32, FApplyIndirection>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FReflectionCaptureShaderData, ReflectionCapture)
		SHADER_PARAMETER_STRUCT_REF(FMobileReflectionCaptureShaderData, MobileReflectionCaptureData)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWNumCulledLightsGrid)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCulledLightDataGrid32Bit)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWCulledLightDataGrid16Bit)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCulledLightLinkAllocator)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCulledLightDataAllocator)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWCulledLightLinks)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, LightViewSpacePositionAndRadius)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, LightViewSpaceDirAndPreprocAngle)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, LightViewSpaceRectPlanes)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, LightViewSpaceWeightEstimate)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<int32>, IndirectionIndices)

		SHADER_PARAMETER(FIntVector, CulledGridSize)
		SHADER_PARAMETER(uint32, NumReflectionCaptures)
		SHADER_PARAMETER(FVector3f, LightGridZParams)
		SHADER_PARAMETER(uint32, NumLocalLights)
		SHADER_PARAMETER(uint32, NumGridCells)
		SHADER_PARAMETER(uint32, MaxCulledLightsPerCell)
		SHADER_PARAMETER(uint32, NumAvailableLinks)
		SHADER_PARAMETER(uint32, LightGridPixelSizeShift)
		SHADER_PARAMETER(uint32, MegaLightsSupportedStartIndex)
		SHADER_PARAMETER(float, MinSampleWeightEstimate)

		SHADER_PARAMETER(uint32, LightGridZSliceScale)
		SHADER_PARAMETER(uint32, LightGridCullMarginXY)
		SHADER_PARAMETER(uint32, LightGridCullMarginZ)
		SHADER_PARAMETER(FVector3f, LightGridCullMarginZParams)
		SHADER_PARAMETER(uint32, LightGridCullMaxZ)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ParentNumCulledLightsGrid)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ParentCulledLightDataGrid32Bit)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, ParentCulledLightDataGrid16Bit)
		SHADER_PARAMETER(FIntVector, ParentGridSize)
		SHADER_PARAMETER(uint32, NumParentGridCells)
		SHADER_PARAMETER(uint32, ParentGridSizeFactor)

		SHADER_PARAMETER(uint32, ViewCulledDataOffset)
		SHADER_PARAMETER(uint32, ViewGridCellOffset)

		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Do not precache if not in use
		if (UseSpecializationConstants(Parameters.Platform))
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}
		else
		{
			return FGlobalShader::ShouldPrecachePermutation(Parameters);
		}
	}

	static FIntVector GetGroupSize(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FUseThreadGroupSize32>())
		{
			return FIntVector(4, 4, 2);
		}
		else
		{
			return FIntVector(4, 4, 4);
		}
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("LIGHT_LINK_STRIDE"), LightLinkStride);

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		FIntVector GroupSize = GetGroupSize(PermutationVector);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize.X * GroupSize.Y * GroupSize.Z);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), GroupSize.Z);
	}
};

IMPLEMENT_GLOBAL_SHADER(FLightGridInjectionCS, "/Engine/Private/LightGridInjection.usf", "LightGridInjectionCS", SF_Compute);



// Experimental version of FLightGridInjectionCS that uses specialization constants
class FLightGridInjectionSpecConstCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLightGridInjectionSpecConstCS);
	SHADER_USE_PARAMETER_STRUCT(FLightGridInjectionSpecConstCS, FGlobalShader)
public:
	class FUseLinkedList : SHADER_SPECIALIZATION_BOOL("USE_LINKED_CULL_LIST");
	class FRefineRectLightBounds : SHADER_SPECIALIZATION_BOOL("REFINE_RECTLIGHT_BOUNDS");
	class FUseHZBCull : SHADER_SPECIALIZATION_BOOL("USE_HZB_CULL");
	class FUseParentLightGrid : SHADER_SPECIALIZATION_BOOL("USE_PARENT_LIGHT_GRID");
	class FUseThreadGroupPerCell : SHADER_SPECIALIZATION_BOOL("USE_THREAD_GROUP_PER_CELL");
	class FUseThreadGroupSize32 : SHADER_PERMUTATION_BOOL("USE_THREAD_GROUP_SIZE_32");
	class FApplyIndirection : SHADER_SPECIALIZATION_BOOL("APPLY_INDIRECTION");
	using FPermutationDomain = TShaderPermutationDomain<FUseLinkedList, FRefineRectLightBounds, FUseHZBCull, FUseParentLightGrid, FUseThreadGroupPerCell, FUseThreadGroupSize32, FApplyIndirection>;

	using FParameters = FLightGridInjectionCS::FParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsSpecializationConstants(Parameters.Platform);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Do not precache if not in use
		if (UseSpecializationConstants(Parameters.Platform))
		{
			return FGlobalShader::ShouldPrecachePermutation(Parameters);
		}
		else
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}
	}

	static FIntVector GetGroupSize(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FUseThreadGroupSize32>())
		{
			return FIntVector(4, 4, 2);
		}
		else
		{
			return FIntVector(4, 4, 4);
		}
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("LIGHT_LINK_STRIDE"), LightLinkStride);

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		const FIntVector GroupSize = GetGroupSize(PermutationVector);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize.X * GroupSize.Y * GroupSize.Z);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Z"), GroupSize.Z);
	}
};

IMPLEMENT_GLOBAL_SHADER(FLightGridInjectionSpecConstCS, "/Engine/Private/LightGridInjection.usf", "LightGridInjectionCS", SF_Compute);



/**
 */
FORCEINLINE float GetTanRadAngleOrZero(float coneAngle)
{
	if (coneAngle < PI / 2.001f)
	{
		return FMath::Tan(coneAngle);
	}

	return 0.0f;
}


FVector GetLightGridZParams(float NearPlane, float FarPlane)
{
	// Space out the slices so they aren't all clustered at the near plane
	float DepthDistributionScale = 4.05f;

	// reserve last slice to cover a larger range (see LightGridInjection.usf)
	return CalculateGridZParams(NearPlane, FarPlane, DepthDistributionScale, GLightGridSizeZ - 1);
}

uint32 PackRG16(float In0, float In1)
{
	return uint32(FFloat16(In0).Encoded) | (uint32(FFloat16(In1).Encoded) << 16);
}

static uint32 PackRGB10(float In0, float In1, float In2)
{
	return 
		(uint32(FMath::Clamp(In0 * 1023u, 0u, 1023u))      )|
		(uint32(FMath::Clamp(In1 * 1023u, 0u, 1023u)) << 10)|
		(uint32(FMath::Clamp(In2 * 1023u, 0u, 1023u)) << 20);
}

static FVector2f PackLightColor(const FVector3f& LightColor)
{
	FVector3f LightColorDir;
	float LightColorLength;
	LightColor.ToDirectionAndLength(LightColorDir, LightColorLength);

	FVector2f LightColorPacked;
	uint32 LightColorDirPacked = 
		((static_cast<uint32>(LightColorDir.X * 0x3FF) & 0x3FF) <<  0) |
		((static_cast<uint32>(LightColorDir.Y * 0x3FF) & 0x3FF) << 10) |
		((static_cast<uint32>(LightColorDir.Z * 0x3FF) & 0x3FF) << 20);

	LightColorPacked.X = LightColorLength / 0x3FF;
	*(uint32*)(&LightColorPacked.Y) = LightColorDirPacked;

	return LightColorPacked;
}

static uint32 PackVirtualShadowMapIdAndPrevLocalLightIndex(int32 VirtualShadowMapId, int32 PrevLocalLightIndex)
{
	// NOTE: Both of these could possibly be INDEX_NONE, which needs to be represented
	// We map all negative numbers to 0, and add one to any positive ones
	uint32 VSMPacked = VirtualShadowMapId < 0 ? 0 : uint32(VirtualShadowMapId + 1);
	uint32 PrevPacked = PrevLocalLightIndex < 0 ? 0 : uint32(PrevLocalLightIndex + 1);

	// Pack to 16 bits each
	check(VSMPacked <= MAX_uint16);
	check(PrevPacked <= MAX_uint16);
	return (VSMPacked << 16) | (PrevPacked & 0xFFFF);
}

static void PackLightData(
	FForwardLightData& Out,
	const FViewInfo& View,
	const FSimpleLightEntry& SimpleLight,
	const FSimpleLightPerViewEntry& SimpleLightPerViewData,
	FLinearColor SimpleLightColor,
	int32 PrevLocalLightIndex,
	bool bHandledByMegaLights,
	const float RayEndBias)
{
	// Put simple lights in all lighting channels
	FLightingChannels SimpleLightLightingChannels;
	SimpleLightLightingChannels.bChannel0 = SimpleLightLightingChannels.bChannel1 = SimpleLightLightingChannels.bChannel2 = true;

	const uint32 SimpleLightLightingChannelMask = GetLightingChannelMaskForStruct(SimpleLightLightingChannels);
	const FVector3f LightTranslatedWorldPosition(View.ViewMatrices.GetPreViewTranslation() + SimpleLightPerViewData.Position);

	// No shadowmap channels for simple lights
	uint32 LightSceneInfoExtraDataPacked = 0;
	LightSceneInfoExtraDataPacked |= SimpleLightLightingChannelMask << LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHTING_CHANNEL_MASK;
	LightSceneInfoExtraDataPacked |= uint32(LightType_Point) << LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHT_TYPE;
	LightSceneInfoExtraDataPacked |= (bHandledByMegaLights && SimpleLight.bMegaLightsCastShadows ? 1u : 0u) << LIGHT_EXTRA_DATA_BIT_OFFSET_CAST_SHADOW;
	LightSceneInfoExtraDataPacked |= (SimpleLight.bAffectTranslucency ? 1u : 0u) << LIGHT_EXTRA_DATA_BIT_OFFSET_AFFECT_TRANSLUCENT_LIGHTING;
	LightSceneInfoExtraDataPacked |= (bHandledByMegaLights ? 1u : 0u) << LIGHT_EXTRA_DATA_BIT_OFFSET_MEGA_LIGHT;
	LightSceneInfoExtraDataPacked |= (bHandledByMegaLights ? 0u : 1u) << LIGHT_EXTRA_DATA_BIT_OFFSET_CLUSTERED_DEFERRED_SUPPORTED;

	const float SourceRadius = 0;
	const float SourceSoftRadius = 0;
	const float SimpleLightSourceLength = 0;

	const uint32 Packed0 = PackRG16(SimpleLight.Exponent, RayEndBias);
	const uint32 Packed1 = PackRG16(SourceRadius, SourceSoftRadius);
	const uint32 Packed2 = PackRG16(SimpleLightSourceLength, SimpleLight.VolumetricScatteringIntensity);

	// Pack both rect light data (barn door length is initialized to -2 
	const uint32 RectPackedX = 0;
	const uint32 RectPackedY = 0;
	const uint32 RectPackedZ = FFloat16(-2.f).Encoded;

	// Pack specular scale and IES profile index.
	// Simple light specular and diffuse scales can be greater than one. If that happens, we encode reciprocals instead
	const float SpecularScale = SimpleLight.SpecularScale > 1.0f ? FMath::Max(1.0f / SimpleLight.SpecularScale, 1.f / 1023.f) : SimpleLight.SpecularScale;
	const float DiffuseScale  = SimpleLight.DiffuseScale > 1.0f ? FMath::Max(1.0f / SimpleLight.DiffuseScale, 1.f / 1023.f) : SimpleLight.DiffuseScale;
	const float IESAtlasIndex = INDEX_NONE;

	// Offset IESAtlasIndex here in order to preserve INDEX_NONE = -1 after encoding
	const uint32 SpecularScale_DiffuseScale_IESData = PackRGB10(SpecularScale, DiffuseScale, (IESAtlasIndex + 1) * (1.f / 1023.f))
		| (SimpleLight.SpecularScale > 1.0f ? 1u << 30u : 0u)
		| (SimpleLight.DiffuseScale > 1.0f ? 1u << 31u : 0u);

	const FVector2f LightColorPacked = PackLightColor(FVector3f(SimpleLightColor));

	const uint32 VirtualShadowMapIdAndPrevLocalLightIndex = PackVirtualShadowMapIdAndPrevLocalLightIndex(INDEX_NONE, PrevLocalLightIndex);

	Out.LightPositionAndInvRadius							= FVector4f(LightTranslatedWorldPosition, 1.0f / FMath::Max(SimpleLight.Radius, KINDA_SMALL_NUMBER));
	Out.LightColorAndIdAndFalloffExponentAndRayEndBias		= FVector4f(LightColorPacked.X, LightColorPacked.Y, INDEX_NONE, FMath::AsFloat(Packed0));
	Out.LightDirectionAndSceneInfoExtraDataPacked			= FVector4f(FVector3f(1, 0, 0), FMath::AsFloat(LightSceneInfoExtraDataPacked));
	Out.SpotAnglesAndSourceRadiusPacked						= FVector4f(-2, 1, FMath::AsFloat(Packed1), FMath::AsFloat(Packed2));
	Out.LightTangentAndIESDataAndSpecularScale				= FVector4f(1.0f, 0.0f, 0.0f, FMath::AsFloat(SpecularScale_DiffuseScale_IESData));
	Out.RectDataAndVirtualShadowMapIdOrPrevLocalLightIndex	= FVector4f(FMath::AsFloat(RectPackedX), FMath::AsFloat(RectPackedY), FMath::AsFloat(RectPackedZ), FMath::AsFloat(VirtualShadowMapIdAndPrevLocalLightIndex));
}

static void PackLightData(
	FForwardLightData& Out,
	const FViewInfo& View,
	const FLightRenderParameters& LightParameters,
	const uint32 LightSceneInfoExtraDataPacked,
	const int32 LightSceneId,
	const int32 VirtualShadowMapId,
	const int32 PrevLocalLightIndex,
	const float VolumetricScatteringIntensity,
	const float RayEndBias)
{
	const FVector3f LightTranslatedWorldPosition(View.ViewMatrices.GetPreViewTranslation() + LightParameters.WorldPosition);

	const uint32 Packed0 = PackRG16(LightParameters.FalloffExponent, RayEndBias);
	const uint32 Packed1 = PackRG16(LightParameters.SourceRadius, LightParameters.SoftSourceRadius);
	const uint32 Packed2 = PackRG16(LightParameters.SourceLength, VolumetricScatteringIntensity);
	
	// Pack rect light data
	uint32 RectPackedX = PackRG16(LightParameters.RectLightAtlasUVOffset.X, LightParameters.RectLightAtlasUVOffset.Y);
	uint32 RectPackedY = PackRG16(LightParameters.RectLightAtlasUVScale.X, LightParameters.RectLightAtlasUVScale.Y);
	uint32 RectPackedZ = 0;
	RectPackedZ |= FFloat16(LightParameters.RectLightBarnLength).Encoded;									// 16 bits
	RectPackedZ |= uint32(FMath::Clamp(LightParameters.RectLightBarnCosAngle,  0.f, 1.0f) * 0x3FF) << 16;	// 10 bits
	RectPackedZ |= uint32(FMath::Clamp(LightParameters.RectLightAtlasMaxLevel, 0.f, 63.f)) << 26;			//  6 bits

	// Pack specular scale and IES profile index
	// Offset IESAtlasIndex here in order to preserve INDEX_NONE = -1 after encoding
	// IESAtlasIndex requires scaling because PackRGB10 expects inputs to be [0:1]
	const uint32 SpecularScale_DiffuseScale_IESData = PackRGB10(LightParameters.SpecularScale, LightParameters.DiffuseScale, (LightParameters.IESAtlasIndex + 1) * (1.f / 1023.f)); // pack atlas id here? 16bit specular 8bit IES and 8 bit LightFunction

	const FVector2f LightColorPacked = PackLightColor(FVector3f(LightParameters.Color));

	const uint32 VirtualShadowMapIdAndPrevLocalLightIndex = 
		PackVirtualShadowMapIdAndPrevLocalLightIndex(VirtualShadowMapId, PrevLocalLightIndex);

	// NOTE: SpotAngles needs full-precision for VSM one pass projection
	Out.LightPositionAndInvRadius							= FVector4f(LightTranslatedWorldPosition, LightParameters.InvRadius);
	Out.LightColorAndIdAndFalloffExponentAndRayEndBias		= FVector4f(LightColorPacked.X, LightColorPacked.Y, LightSceneId, FMath::AsFloat(Packed0));
	Out.LightDirectionAndSceneInfoExtraDataPacked			= FVector4f(LightParameters.Direction, FMath::AsFloat(LightSceneInfoExtraDataPacked));
	Out.SpotAnglesAndSourceRadiusPacked						= FVector4f(LightParameters.SpotAngles.X, LightParameters.SpotAngles.Y, FMath::AsFloat(Packed1), FMath::AsFloat(Packed2));
	Out.LightTangentAndIESDataAndSpecularScale				= FVector4f(LightParameters.Tangent, FMath::AsFloat(SpecularScale_DiffuseScale_IESData));
	Out.RectDataAndVirtualShadowMapIdOrPrevLocalLightIndex	= FVector4f(FMath::AsFloat(RectPackedX), FMath::AsFloat(RectPackedY), FMath::AsFloat(RectPackedZ), FMath::AsFloat(VirtualShadowMapIdAndPrevLocalLightIndex));
}

static const uint32 NUM_PLANES_PER_RECT_LIGHT = 4;

static void CalculateRectLightCullingPlanes(const FRectLightSceneProxy* RectProxy, TArray<FPlane, TInlineAllocator<NUM_PLANES_PER_RECT_LIGHT>>& OutPlanes)
{
	const float BarnMaxAngle = GetRectLightBarnDoorMaxAngle();
	const float AngleRad = FMath::DegreesToRadians(FMath::Clamp(RectProxy->BarnDoorAngle, 0.f, BarnMaxAngle));

	// horizontal barn doors
	{
		float HorizontalBarnExtent;
		float HorizontalBarnDepth;
		CalculateRectLightCullingBarnExtentAndDepth(RectProxy->SourceWidth, RectProxy->BarnDoorLength, AngleRad, RectProxy->Radius, HorizontalBarnExtent, HorizontalBarnDepth);

		TStaticArray<FVector, 8> Corners;
		CalculateRectLightBarnCorners(RectProxy->SourceWidth, RectProxy->SourceHeight, HorizontalBarnExtent, HorizontalBarnDepth, Corners);

		OutPlanes.Add(FPlane(Corners[1], Corners[0], Corners[3])); // right
		OutPlanes.Add(FPlane(Corners[5], Corners[7], Corners[4])); // left
	}
	
	// vertical barn doors
	{
		float VerticalBarnExtent;
		float VerticalBarnDepth;
		CalculateRectLightCullingBarnExtentAndDepth(RectProxy->SourceHeight, RectProxy->BarnDoorLength, AngleRad, RectProxy->Radius, VerticalBarnExtent, VerticalBarnDepth);

		TStaticArray<FVector, 8> Corners;
		CalculateRectLightBarnCorners(RectProxy->SourceWidth, RectProxy->SourceHeight, VerticalBarnExtent, VerticalBarnDepth, Corners);

		OutPlanes.Add(FPlane(Corners[4], Corners[6], Corners[0])); // top
		OutPlanes.Add(FPlane(Corners[1], Corners[3], Corners[5])); // bottom
	}

	check(OutPlanes.Num() == NUM_PLANES_PER_RECT_LIGHT);
}

struct FLightGrid
{
	FRDGBufferSRVRef CulledLightDataGridSRV = nullptr;
	FRDGBufferSRVRef NumCulledLightsGridSRV = nullptr;
};

FLightGrid LightGridInjection(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FIntVector GridSize,
	uint32 LightGridPixelSizeShift,
	uint32 ZSliceScale,
	uint32 MaxNumCells,
	FVector3f ZParams,
	uint32 LightGridCullMarginXY,
	uint32 LightGridCullMarginZ,
	FVector3f LightGridCullMarginZParams,
	uint32 LightGridCullMaxZ,
	uint32 NumLocalLights,
	uint32 NumReflectionCaptures,
	uint32 MegaLightsSupportedStartIndex,
	bool bUse16BitBuffers,
	bool bRefineRectLightBounds,
	FRDGBufferSRVRef LightViewSpacePositionAndRadiusSRV,
	FRDGBufferSRVRef LightViewSpaceDirAndPreprocAngleSRV,
	FRDGBufferSRVRef LightViewSpaceRectPlanesSRV,
	FRDGBufferSRVRef LightViewSpaceWeightEstimateSRV,
	FRDGBufferSRVRef IndirectionIndicesSRV,
	bool bThreadGroupPerCell,
	bool bThreadGroupSize32,
	// parent params
	FRDGBufferSRVRef ParentNumCulledLightsGridSRV,
	FRDGBufferSRVRef ParentCulledLightDataGridSRV,
	uint32 ParentGridSizeFactor)
{
	const uint32 NumCulledLightEntries = MaxNumCells * GMaxCulledLightsPerCell;

	uint32 NumCulledLightLinks = MaxNumCells * GMaxCulledLightsPerCell;
	
	if (bThreadGroupPerCell)
	{
		ensureMsgf(NumLocalLights <= LIGHT_GRID_CELL_WRITER_MAX_NUM_PRIMITIVES, TEXT("NumLocalLights limited to 16M by FCellWriter."));
		ensureMsgf(NumReflectionCaptures <= LIGHT_GRID_CELL_WRITER_MAX_NUM_PRIMITIVES, TEXT("NumLocalLights limited to 16M by FCellWriter."));

		NumCulledLightLinks = FMath::Min(NumCulledLightLinks, (uint32)LIGHT_GRID_CELL_WRITER_MAX_NUM_LINKS); // limited to 16M by FCellWriter (will cause warning if exceeded, see FLightGridFeedbackStatus)
	}

	const FIntVector ParentGridSize = FIntVector::DivideAndRoundUp(GridSize, ParentGridSizeFactor);

	FRDGBufferRef CulledLightLinksBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumCulledLightLinks * LightLinkStride), TEXT("CulledLightLinks"));
	FRDGBufferUAVRef CulledLightLinksUAV = GraphBuilder.CreateUAV(CulledLightLinksBuffer);

	FRDGBufferRef CulledLightLinkAllocatorBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("CulledLightLinkAllocator"));
	FRDGBufferUAVRef CulledLightLinkAllocatorUAV = GraphBuilder.CreateUAV(CulledLightLinkAllocatorBuffer);

	FRDGBufferRef CulledLightDataAllocatorBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("CulledLightDataAllocator"));
	FRDGBufferUAVRef CulledLightDataAllocatorUAV = GraphBuilder.CreateUAV(CulledLightDataAllocatorBuffer);

	const FViewInfo* AssociatedSecondaryView = nullptr;
	const FViewInfo* AssociatedPrimaryView = nullptr;
	if (View.bIsSinglePassStereo)
	{
		if (View.StereoPass == EStereoscopicPass::eSSP_PRIMARY)
		{
			AssociatedSecondaryView = View.GetInstancedView();
		}
		else if (View.StereoPass == EStereoscopicPass::eSSP_SECONDARY)
		{
			AssociatedPrimaryView = View.GetPrimaryView();
		}
	}

	FRDGBufferSRVRef NumCulledLightsGridSRV;
	FRDGBufferUAVRef NumCulledLightsGridUAV;
	FRDGBufferSRVRef CulledLightDataGridSRV;
	FRDGBufferUAVRef CulledLightDataGridUAV;
	if (AssociatedPrimaryView)
	{
		// 32 bit and 16 bit have the same buffer, so it doesn't matter which one we copy.
		CulledLightDataGridSRV = AssociatedPrimaryView->ForwardLightingResources.CulledLightDataGridSRV;
		CulledLightDataGridUAV = AssociatedPrimaryView->ForwardLightingResources.CulledLightDataGridUAV;
		NumCulledLightsGridSRV = AssociatedPrimaryView->ForwardLightingResources.NumCulledLightsGridSRV;
		NumCulledLightsGridUAV = AssociatedPrimaryView->ForwardLightingResources.NumCulledLightsGridUAV;
	}
	else
	{
		// Allocate cells for both primary and secondary views in one buffer
		int32 StereoMultiplier = AssociatedSecondaryView ? 2 : 1;

		FRDGBufferRef NumCulledLightsGrid = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), MaxNumCells * StereoMultiplier * NumCulledLightsGridStride), TEXT("NumCulledLightsGrid"));
		NumCulledLightsGridUAV = GraphBuilder.CreateUAV(NumCulledLightsGrid);
		NumCulledLightsGridSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(NumCulledLightsGrid));

		if (bUse16BitBuffers)
		{
			const SIZE_T LightIndexTypeSize = sizeof(FLightIndexType);
			const EPixelFormat CulledLightDataGridFormat = PF_R16_UINT;
			FRDGBufferRef CulledLightDataGrid = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(LightIndexTypeSize, NumCulledLightEntries * StereoMultiplier), TEXT("CulledLightDataGrid"));
			CulledLightDataGridSRV = GraphBuilder.CreateSRV(CulledLightDataGrid, CulledLightDataGridFormat);
			CulledLightDataGridUAV = GraphBuilder.CreateUAV(CulledLightDataGrid, CulledLightDataGridFormat);
		}
		else
		{
			const SIZE_T LightIndexTypeSize = sizeof(FLightIndexType32);
			const EPixelFormat CulledLightDataGridFormat = PF_R32_UINT;
			FRDGBufferRef CulledLightDataGrid = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(LightIndexTypeSize, NumCulledLightEntries * StereoMultiplier), TEXT("CulledLightDataGrid"));
			CulledLightDataGridSRV = GraphBuilder.CreateSRV(CulledLightDataGrid);
			CulledLightDataGridUAV = GraphBuilder.CreateUAV(CulledLightDataGrid);
		}
	}
	View.ForwardLightingResources.CulledLightDataGridSRV = CulledLightDataGridSRV;
	View.ForwardLightingResources.CulledLightDataGridUAV = CulledLightDataGridUAV;
	View.ForwardLightingResources.NumCulledLightsGridSRV = NumCulledLightsGridSRV;
	View.ForwardLightingResources.NumCulledLightsGridUAV = NumCulledLightsGridUAV;

	const bool bUseAsyncCompute = CVarLightGridAsyncCompute.GetValueOnRenderThread();
	const ERDGPassFlags RDGPassFlags = bUseAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute;
	AddClearUAVPass(GraphBuilder, CulledLightLinkAllocatorUAV, 0, RDGPassFlags);
	AddClearUAVPass(GraphBuilder, CulledLightDataAllocatorUAV, 0, RDGPassFlags);
	if (!AssociatedPrimaryView)
	{
		AddClearUAVPass(GraphBuilder, NumCulledLightsGridUAV, 0, RDGPassFlags);
	}

	FLightGridInjectionCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FLightGridInjectionCS::FParameters>();

	PassParameters->View = View.ViewUniformBuffer;

	if (IsMobilePlatform(View.GetShaderPlatform()))
	{
		PassParameters->MobileReflectionCaptureData = View.MobileReflectionCaptureUniformBuffer;
	}
	else
	{
		PassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;
	}

	PassParameters->RWNumCulledLightsGrid = NumCulledLightsGridUAV;
	PassParameters->RWCulledLightDataGrid32Bit = CulledLightDataGridUAV;
	PassParameters->RWCulledLightDataGrid16Bit = CulledLightDataGridUAV;
	PassParameters->RWCulledLightLinkAllocator = CulledLightLinkAllocatorUAV;
	PassParameters->RWCulledLightDataAllocator = CulledLightDataAllocatorUAV;
	PassParameters->RWCulledLightLinks = CulledLightLinksUAV;
	PassParameters->CulledGridSize = GridSize;
	PassParameters->LightGridZParams = ZParams;
	PassParameters->NumReflectionCaptures = NumReflectionCaptures;
	PassParameters->NumLocalLights = NumLocalLights;
	PassParameters->MaxCulledLightsPerCell = GMaxCulledLightsPerCell;
	PassParameters->NumAvailableLinks = NumCulledLightLinks;
	PassParameters->NumGridCells = GridSize.X * GridSize.Y * GridSize.Z;
	PassParameters->LightGridPixelSizeShift = LightGridPixelSizeShift;
	PassParameters->LightGridZSliceScale = ZSliceScale;
	PassParameters->LightGridCullMarginXY = LightGridCullMarginXY;
	PassParameters->LightGridCullMarginZ = LightGridCullMarginZ;
	PassParameters->LightGridCullMarginZParams = LightGridCullMarginZParams;
	PassParameters->LightGridCullMaxZ = LightGridCullMaxZ;
	PassParameters->MegaLightsSupportedStartIndex = MegaLightsSupportedStartIndex;
	PassParameters->MinSampleWeightEstimate = MegaLights::GetMinSampleWeightEstimate();
	PassParameters->ViewCulledDataOffset = AssociatedPrimaryView ? MaxNumCells * GMaxCulledLightsPerCell : 0;
	PassParameters->ViewGridCellOffset = AssociatedPrimaryView ? MaxNumCells : 0;

	PassParameters->ParentNumCulledLightsGrid = ParentNumCulledLightsGridSRV;
	PassParameters->ParentCulledLightDataGrid32Bit = ParentCulledLightDataGridSRV;
	PassParameters->ParentCulledLightDataGrid16Bit = ParentCulledLightDataGridSRV;
	PassParameters->ParentGridSize = ParentGridSize;
	PassParameters->NumParentGridCells = ParentGridSize.X * ParentGridSize.Y * ParentGridSize.Z;
	PassParameters->ParentGridSizeFactor = ParentGridSizeFactor;

	PassParameters->LightViewSpacePositionAndRadius = LightViewSpacePositionAndRadiusSRV;
	PassParameters->LightViewSpaceDirAndPreprocAngle = LightViewSpaceDirAndPreprocAngleSRV;
	PassParameters->LightViewSpaceRectPlanes = LightViewSpaceRectPlanesSRV;
	PassParameters->LightViewSpaceWeightEstimate = LightViewSpaceWeightEstimateSRV;
	PassParameters->IndirectionIndices = IndirectionIndicesSRV;

	const bool bIsHZBValid = IsHZBValid(View, EHZBType::FurthestHZB);
	if (bIsHZBValid)
	{
		PassParameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);
	}
	else
	{
		PassParameters->HZBParameters = GetDummyHZBParameters(GraphBuilder);
	}

	if (UseSpecializationConstants(View.GetShaderPlatform()))
	{
		FLightGridInjectionSpecConstCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FLightGridInjectionSpecConstCS::FUseLinkedList>(GLightLinkedListCulling != 0);
		PermutationVector.Set<FLightGridInjectionSpecConstCS::FRefineRectLightBounds>(bRefineRectLightBounds);
		PermutationVector.Set<FLightGridInjectionSpecConstCS::FUseHZBCull>(GLightGridHZBCull != 0 && bIsHZBValid);
		PermutationVector.Set<FLightGridInjectionSpecConstCS::FUseParentLightGrid>(ParentNumCulledLightsGridSRV != nullptr && ParentCulledLightDataGridSRV != nullptr);
		PermutationVector.Set<FLightGridInjectionSpecConstCS::FUseThreadGroupPerCell>(bThreadGroupPerCell);
		PermutationVector.Set<FLightGridInjectionSpecConstCS::FUseThreadGroupSize32>(bThreadGroupSize32);
		PermutationVector.Set<FLightGridInjectionSpecConstCS::FApplyIndirection>(IndirectionIndicesSRV != nullptr);
		auto ComputeShader = View.ShaderMap->GetShader<FLightGridInjectionSpecConstCS>(PermutationVector);

		const FIntVector NumGroups = bThreadGroupPerCell ? GridSize : FComputeShaderUtils::GetGroupCount(GridSize, FLightGridInjectionSpecConstCS::GetGroupSize(PermutationVector));

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("LightGridInject SpecConsts %s %s %s %s %s",
				PermutationVector.Get<FLightGridInjectionSpecConstCS::FUseLinkedList>() ? TEXT("LinkedList") : TEXT("NoLinkedList"),
				PermutationVector.Get<FLightGridInjectionSpecConstCS::FRefineRectLightBounds>() ? TEXT("RefineRectLightBounds") : TEXT(""),
				PermutationVector.Get<FLightGridInjectionSpecConstCS::FUseHZBCull>() ? TEXT("UseHZBCull") : TEXT(""),
				PermutationVector.Get<FLightGridInjectionSpecConstCS::FUseParentLightGrid>() ? TEXT("UseParentLightGrid") : TEXT(""),
				PermutationVector.Get<FLightGridInjectionSpecConstCS::FUseThreadGroupPerCell>() ? TEXT("ThreadGroup") : TEXT("SingleThread")),
			RDGPassFlags,
			ComputeShader,
			PassParameters,
			NumGroups);
	}
	else
	{
		FLightGridInjectionCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FLightGridInjectionCS::FUseLinkedList>(GLightLinkedListCulling != 0);
		PermutationVector.Set<FLightGridInjectionCS::FRefineRectLightBounds>(bRefineRectLightBounds);
		PermutationVector.Set<FLightGridInjectionCS::FUseHZBCull>(GLightGridHZBCull != 0 && bIsHZBValid);
		PermutationVector.Set<FLightGridInjectionCS::FUseParentLightGrid>(ParentNumCulledLightsGridSRV != nullptr && ParentCulledLightDataGridSRV != nullptr);
		PermutationVector.Set<FLightGridInjectionCS::FUseThreadGroupPerCell>(bThreadGroupPerCell);
		PermutationVector.Set<FLightGridInjectionCS::FUseThreadGroupSize32>(bThreadGroupSize32);
		PermutationVector.Set<FLightGridInjectionCS::FApplyIndirection>(IndirectionIndicesSRV != nullptr);
		auto ComputeShader = View.ShaderMap->GetShader<FLightGridInjectionCS>(PermutationVector);

		const FIntVector NumGroups = bThreadGroupPerCell ? GridSize : FComputeShaderUtils::GetGroupCount(GridSize, FLightGridInjectionCS::GetGroupSize(PermutationVector));

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("LightGridInject %s %s %s %s %s",
				PermutationVector.Get<FLightGridInjectionCS::FUseLinkedList>() ? TEXT("LinkedList") : TEXT("NoLinkedList"),
				PermutationVector.Get<FLightGridInjectionCS::FRefineRectLightBounds>() ? TEXT("RefineRectLightBounds") : TEXT(""),
				PermutationVector.Get<FLightGridInjectionCS::FUseHZBCull>() ? TEXT("UseHZBCull") : TEXT(""),
				PermutationVector.Get<FLightGridInjectionCS::FUseParentLightGrid>() ? TEXT("UseParentLightGrid") : TEXT(""),
				PermutationVector.Get<FLightGridInjectionCS::FUseThreadGroupPerCell>() ? TEXT("ThreadGroup") : TEXT("SingleThread")),
			RDGPassFlags,
			ComputeShader,
			PassParameters,
			NumGroups);
	}

	FLightGrid Output;
	Output.CulledLightDataGridSRV = CulledLightDataGridSRV;
	Output.NumCulledLightsGridSRV = NumCulledLightsGridSRV;

#if !UE_BUILD_SHIPPING
	LightGridFeedbackStatus(GraphBuilder, View, CulledLightDataAllocatorBuffer, NumCulledLightEntries, CulledLightLinkAllocatorBuffer, NumCulledLightLinks, bUseAsyncCompute);
#endif

	return Output;
}

FLinearColor GetSimpleLightColor(const FViewInfo& View, const FSimpleLightEntry& SimpleLight)
{
	return SimpleLight.Color * FLightRenderParameters::GetLightExposureScale(View.GetLastEyeAdaptationExposure(), SimpleLight.InverseExposureBlend);
}

FComputeLightGridOutput FSceneRenderer::ComputeLightGrid(FRDGBuilder& GraphBuilder, bool bCullLightsToGrid, const FSortedLightSetSceneInfo& SortedLightSet, FComputeLightGridArray<FForwardLightUniformParameters*>& PerViewForwardLightUniformParameters, FComputeLightGridArray<FForwardDirectionalLightShadowMapParameters*>& PerViewForwardDirLightShadowParameters)
{
	FComputeLightGridOutput Result = {};

	QUICK_SCOPE_CYCLE_COUNTER(STAT_ComputeLightGrid);
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, ComputeLightGrid);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, LightGrid, "ComputeLightGrid");

	const bool bAllowStaticLighting = IsStaticLightingAllowed();
	const bool bLightGridUses16BitBuffers = LightGridUses16BitBuffers(ShaderPlatform);
	const bool bRenderRectLightsAsSpotLights = RenderRectLightsAsSpotLights(FeatureLevel);

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

#if WITH_EDITOR
	bool bMultipleDirLightsConflictForForwardShading = false;
#endif

	static const auto RayEndBiasCVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.MegaLights.HardwareRayTracing.EndBias"));
	const float GlobalRayEndBias = RayEndBiasCVar ? RayEndBiasCVar->GetValueOnRenderThread() : 0.0f;

	FLightGridSceneExtension& LightGridExtension = Scene->GetExtension<FLightGridSceneExtension>();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];

		FLightGridSceneExtension::FViewData* LightGridViewData = LightGridExtension.GetViewData(View);

		// Get the associated secondary view if this is an instanced primary view, or vice/versa
		const FViewInfo* AssociatedSecondaryView = nullptr;
		const FViewInfo* AssociatedPrimaryView = nullptr;
		if (View.bIsSinglePassStereo)
		{
			if (View.StereoPass == EStereoscopicPass::eSSP_PRIMARY)
			{
				AssociatedSecondaryView = View.GetInstancedView();
			}
			else if (View.StereoPass == EStereoscopicPass::eSSP_SECONDARY)
			{
				AssociatedPrimaryView = View.GetPrimaryView();
			}
		}

		View.ForwardLightingResources.SelectedForwardDirectionalLightProxy = nullptr;

		FForwardLightUniformParameters* ForwardLightUniformParameters = PerViewForwardLightUniformParameters[ViewIndex];
		FForwardDirectionalLightShadowMapParameters* DirLightShadowParameters = PerViewForwardDirLightShadowParameters[ViewIndex];
		DirLightShadowParameters->DirectionalLightShadowmapAtlas = SystemTextures.Black;
		DirLightShadowParameters->DirectionalLightStaticShadowmap = GBlackTexture->TextureRHI;

		TArray<FForwardLightData, SceneRenderingAllocator> ForwardLightData;
		TArray<int32, SceneRenderingAllocator>  DirectionalLightIndices;

		TArray<FVector4f, SceneRenderingAllocator> ViewSpacePosAndRadiusData;
		TArray<FVector4f, SceneRenderingAllocator> ViewSpaceDirAndPreprocAngleData;
		TArray<FVector4f, SceneRenderingAllocator> ViewSpaceRectPlanesData;
		TArray<FVector4f, SceneRenderingAllocator> ViewSpaceWeightEstimateData;

		TArray<int32, SceneRenderingAllocator> IndirectionIndices;

		TArray<uint8, SceneRenderingAllocator> LightNames;
		const bool bNeedsLightNames = MegaLights::IsUsingLightNames(View);
		const uint32 LightNameDwords = 16;
		const uint32 LightNameBytes = LightNameDwords * sizeof(uint32);
		ForwardLightUniformParameters->LightNameDwords = LightNameDwords;

		TArray<VisualizeLightShape::FSortKey> VisualizeLightShapeSortKeys;
		const bool bVisualizeLightShape = VisualizeLightShape::UpdateLastToggleTime(View);

		float FurthestLight = 1000;

		int32 ConflictingLightCountForForwardShading = 0;

		// Track the end markers for different types
		int32 SimpleLightsEnd = 0;
		int32 MegaLightsSupportedStart = 0;
		int32 DirectionalMegaLightsSupportedStart = 0;

		bool bHasRectLights = false;
		bool bHasTexturedLights = false;
		uint8 LightGridLightingChannelAndMask = 0xFF;

		const float Exposure = View.GetLastEyeAdaptationExposure();

		if (bCullLightsToGrid)
		{
			if (GLightBufferMode == (int32)ELightBufferMode::VisibleLightsStableIndices)
			{
				// when using stable light indices, indexing in ForwardLightBuffer is done using LightSceneInfo->Id
				// so need to allocate max light ID entries
				const int32 MaxLightId = Scene->GPUScene.GetMaxLightId();
				ForwardLightData.AddUninitialized(FMath::Max(1, MaxLightId));

				if (bNeedsLightNames)
				{
					LightNames.AddUninitialized(FMath::Max(1, MaxLightId) * LightNameBytes);
				}
			}

			// Simple lights are copied without view dependent checks, so same in and out
			SimpleLightsEnd = SortedLightSet.SimpleLightsEnd;

			// 1. reserve entries for simple lights
			if (SimpleLightsEnd > 0)
			{
				IndirectionIndices.AddUninitialized(SimpleLightsEnd);

				ViewSpacePosAndRadiusData.AddUninitialized(SimpleLightsEnd);
				ViewSpaceDirAndPreprocAngleData.AddZeroed(SimpleLightsEnd);
				ViewSpaceRectPlanesData.AddZeroed(SimpleLightsEnd * NUM_PLANES_PER_RECT_LIGHT);
				ViewSpaceWeightEstimateData.AddZeroed(SimpleLightsEnd);

				if (bVisualizeLightShape)
				{
					VisualizeLightShapeSortKeys.AddUninitialized(SimpleLightsEnd);
				}
			}

			const uint32 LightShaderParameterFlags = bRenderRectLightsAsSpotLights ? ELightShaderParameterFlags::RectAsSpotLight : 0u;
			float SelectedForwardDirectionalLightIntensitySq = 0.0f;
			int32 SelectedForwardDirectionalLightPriority = -1;
			const TArray<FSortedLightSceneInfo, SceneRenderingAllocator>& SortedLights = SortedLightSet.SortedLights;
			MegaLightsSupportedStart = MAX_int32;
			DirectionalMegaLightsSupportedStart = MAX_int32;

			TMap<FSimpleLightId, int32> NewSimpleLightIdToForwardLightIndex;
			if (LightGridViewData)
			{
				NewSimpleLightIdToForwardLightIndex.Reserve(LightGridViewData->SimpleLightIdToForwardLightIndex.Num());
			}

			// Next add all the other lights, track the end index for clustered supporting lights
			for (int32 SortedIndex = SimpleLightsEnd; SortedIndex < SortedLights.Num(); ++SortedIndex)
			{
				const FSortedLightSceneInfo& SortedLightInfo = SortedLights[SortedIndex];

				// This is a simple light handled by MegaLights
				if (!SortedLightInfo.SortKey.Fields.bIsNotSimpleLight)
				{
					check(SortedLightInfo.SortKey.Fields.bHandledByMegaLights && !SortedLightInfo.LightSceneInfo && SortedLightInfo.SimpleLightIndex >= 0);

					if (SortedIndex == SortedLightSet.MegaLightsLightStart)
					{
						const int32 NumMegaSimpleLights = SortedLightSet.MegaLightsSimpleLightsEnd - SortedLightSet.MegaLightsLightStart;
						ForwardLightData.Reserve(ForwardLightData.Num() + NumMegaSimpleLights);
						ViewSpacePosAndRadiusData.Reserve(ViewSpacePosAndRadiusData.Num() + NumMegaSimpleLights);
						ViewSpaceDirAndPreprocAngleData.Reserve(ViewSpaceDirAndPreprocAngleData.Num() + NumMegaSimpleLights);
						ViewSpaceRectPlanesData.Reserve(ViewSpaceRectPlanesData.Num() + NumMegaSimpleLights * NUM_PLANES_PER_RECT_LIGHT);
						ViewSpaceWeightEstimateData.Reserve(ViewSpaceWeightEstimateData.Num() + NumMegaSimpleLights);
						IndirectionIndices.Reserve(IndirectionIndices.Num() + NumMegaSimpleLights);

						if (bVisualizeLightShape)
						{
							VisualizeLightShapeSortKeys.Reserve(VisualizeLightShapeSortKeys.Num() + NumMegaSimpleLights);
						}

						if (bNeedsLightNames)
						{
							LightNames.Reserve(LightNames.Num() + NumMegaSimpleLights * LightNameBytes);
						}
					}

					if (MegaLightsSupportedStart == MAX_int32)
					{
						MegaLightsSupportedStart = ViewSpacePosAndRadiusData.Num();
					}

					const int32 SimpleLightIndex = SortedLightInfo.SimpleLightIndex;

					// New entries are appended to the end so won't break ELightBufferMode::VisibleLightsStableIndices
					const int32 IndexInBuffer = ForwardLightData.AddUninitialized(1);
					FForwardLightData& LightData = ForwardLightData[IndexInBuffer];

					const FSimpleLightEntry& SimpleLight = SortedLightSet.SimpleLights.InstanceData[SimpleLightIndex];
					const FSimpleLightPerViewEntry& SimpleLightPerViewData = SortedLightSet.SimpleLights.GetViewDependentData(SimpleLightIndex, ViewIndex, Views.Num());

					int32 PrevForwardLightIndex = INDEX_NONE;
					if (LightGridViewData && SimpleLight.LightId.IsValid())
					{
						PrevForwardLightIndex = LightGridViewData->SimpleLightIdToForwardLightIndex.FindRef(SimpleLight.LightId, INDEX_NONE);
						NewSimpleLightIdToForwardLightIndex.Add(SimpleLight.LightId, IndexInBuffer);
					}

					const FLinearColor SimpleLightColor = GetSimpleLightColor(View, SimpleLight);
					PackLightData(LightData, View, SimpleLight, SimpleLightPerViewData, SimpleLightColor, PrevForwardLightIndex, true /*bHandledByMegaLights*/, GlobalRayEndBias);

					const FVector4f ViewSpacePosAndRadius(FVector4f(View.ViewMatrices.GetWorldToView().TransformPosition(SimpleLightPerViewData.Position)), SimpleLight.Radius);
					ViewSpacePosAndRadiusData.Add(ViewSpacePosAndRadius);
					ViewSpaceDirAndPreprocAngleData.AddZeroed();
					ViewSpaceRectPlanesData.AddZeroed(NUM_PLANES_PER_RECT_LIGHT);

					FVector4f WeightEstimateData;
					WeightEstimateData.X = SimpleLightColor.GetLuminance() * FMath::Max(SimpleLight.DiffuseScale, SimpleLight.SpecularScale);
					WeightEstimateData.Y = SimpleLight.Exponent;
					WeightEstimateData.Z = 0.0f;
					WeightEstimateData.W = 0.0f;
					ViewSpaceWeightEstimateData.Add(WeightEstimateData);

					IndirectionIndices.Add(IndexInBuffer);

					if (bVisualizeLightShape)
					{
						VisualizeLightShape::FSortKey& SortKey = VisualizeLightShapeSortKeys[VisualizeLightShapeSortKeys.AddUninitialized()];
						SortKey.LightData.ForwardLightIndex = IndexInBuffer;
						SortKey.LightData.LightType = LightType_Point;
						SortKey.Distance = FVector3f(ViewSpacePosAndRadius).SizeSquared();
					}

					if (bNeedsLightNames)
					{
						const int32 FirstCharIndex = LightNames.AddUninitialized(LightNameBytes);
						uint8* Dest = &LightNames[FirstCharIndex];
						const FAnsiString LightName = FAnsiString::Printf("Simple_%d_%d", SimpleLight.LightId.SourcePrimitiveId, SimpleLight.LightId.LocalId);
						const int32 CharsToCopy = FMath::Min<int32>(LightName.Len(), LightNameBytes - 1);

						FMemory::Memcpy(Dest, *LightName, CharsToCopy);
						Dest[CharsToCopy] = 0;
					}

					continue;
				}

				const FLightSceneInfo* const LightSceneInfo = SortedLightInfo.LightSceneInfo;
				const FLightSceneProxy* LightProxy = LightSceneInfo->Proxy;

				if (LightSceneInfo->ShouldRenderLight(View) ||
					(AssociatedSecondaryView && LightSceneInfo->ShouldRenderLight(*AssociatedSecondaryView)) ||
					(AssociatedPrimaryView && LightSceneInfo->ShouldRenderLight(*AssociatedPrimaryView)))
				{
					FLightRenderParameters LightParameters;
					LightProxy->GetLightShaderParameters(LightParameters, LightShaderParameterFlags);

					if (LightProxy->IsInverseSquared())
					{
						LightParameters.FalloffExponent = 0;
					}

					// When rendering reflection captures, the direct lighting of the light is actually the indirect specular from the main view
					if (View.bIsReflectionCapture)
					{
						LightParameters.Color *= LightProxy->GetIndirectLightingScale();
					}

					uint32 LightSceneInfoExtraDataPacked = LightSceneInfo->PackExtraData(
						bAllowStaticLighting,
						SortedLightInfo.SortKey.Fields.bLightFunction,
						SortedLightInfo.SortKey.Fields.bHandledByMegaLights,
						!SortedLightInfo.SortKey.Fields.bClusteredDeferredNotSupported);

					const bool bDynamicShadows = ViewFamily.EngineShowFlags.DynamicShadows && VisibleLightInfos.IsValidIndex(LightSceneInfo->Id);
					const int32 VirtualShadowMapId = bDynamicShadows ? VisibleLightInfos[LightSceneInfo->Id].GetVirtualShadowMapId( &View ) : INDEX_NONE;
					int32 IndexInBuffer = INDEX_NONE;

					// Add light to ForwardLightData
					if ((SortedLightInfo.SortKey.Fields.LightType == LightType_Point && ViewFamily.EngineShowFlags.PointLights)
						|| (SortedLightInfo.SortKey.Fields.LightType == LightType_Spot && ViewFamily.EngineShowFlags.SpotLights)
						|| (SortedLightInfo.SortKey.Fields.LightType == LightType_Rect && ViewFamily.EngineShowFlags.RectLights)
						|| (GLightBufferMode != (int32)ELightBufferMode::VisibleLocalLights && SortedLightInfo.SortKey.Fields.LightType == LightType_Directional && ViewFamily.EngineShowFlags.DirectionalLights))
					{
						IndexInBuffer = GLightBufferMode == (int32)ELightBufferMode::VisibleLightsStableIndices ? LightSceneInfo->Id.GetIndex() : ForwardLightData.AddUninitialized(1);
						FForwardLightData& LightData = ForwardLightData[IndexInBuffer];

						int32 PrevForwardLightIndex = INDEX_NONE;
						if (LightGridViewData)
						{
							if (LightGridViewData->LightSceneIdToForwardLightIndex.IsValidIndex(LightSceneInfo->Id.GetIndex()))
							{
								PrevForwardLightIndex = LightGridViewData->LightSceneIdToForwardLightIndex[LightSceneInfo->Id.GetIndex()];
							}
							else
							{
								PrevForwardLightIndex = INDEX_NONE;

								const int32 OldSize = LightGridViewData->LightSceneIdToForwardLightIndex.Num();
								LightGridViewData->LightSceneIdToForwardLightIndex.SetNumUninitialized(LightSceneInfo->Id.GetIndex() + 1);

								for (int32 Index = OldSize; Index < LightGridViewData->LightSceneIdToForwardLightIndex.Num(); ++Index)
								{
									LightGridViewData->LightSceneIdToForwardLightIndex[Index] = INDEX_NONE;
								}
							}
							LightGridViewData->LightSceneIdToForwardLightIndex[LightSceneInfo->Id.GetIndex()] = IndexInBuffer;
						}

						if (SortedLightInfo.SortKey.Fields.LightType != LightType_Directional) // only local lights go into the grid
						{
							IndirectionIndices.Add(IndexInBuffer);
						}
						else
						{
							uint32 DirectionalLightIndex = DirectionalLightIndices.Add(IndexInBuffer);
							
							if (SortedLightInfo.SortKey.Fields.bHandledByMegaLights && DirectionalMegaLightsSupportedStart == MAX_int32)
							{
								DirectionalMegaLightsSupportedStart = DirectionalLightIndex;
							}
						}

						const float LightFade = GetLightFadeFactor(View, LightProxy);
						LightParameters.Color *= LightFade;
						LightParameters.Color *= LightParameters.GetLightExposureScale(Exposure);

						float VolumetricScatteringIntensity = LightProxy->GetVolumetricScatteringIntensity();
						if (LightNeedsSeparateInjectionIntoVolumetricFogForOpaqueShadow(View, LightSceneInfo, VisibleLightInfos[LightSceneInfo->Id], *Scene))
						{
							// Disable this lights forward shading volumetric scattering contribution
							VolumetricScatteringIntensity = 0;
						}

						float RayEndBias = LightProxy->GetRayEndBias();
						if (RayEndBias < 0.0f)
						{
							RayEndBias = GlobalRayEndBias;
						}

						PackLightData(
							LightData,
							View,
							LightParameters,
							LightSceneInfoExtraDataPacked,
							LightSceneInfo->Id.GetIndex(),
							VirtualShadowMapId,
							PrevForwardLightIndex,
							VolumetricScatteringIntensity,
							RayEndBias);

						if (bNeedsLightNames)
						{
							const int32 FirstCharIndex = GLightBufferMode == (int32)ELightBufferMode::VisibleLightsStableIndices ?
								LightSceneInfo->Id.GetIndex() * LightNameBytes :
								LightNames.AddUninitialized(LightNameBytes);
							uint8* Dest = &LightNames[FirstCharIndex];
							const FAnsiString LightName = FAnsiString::Printf("%s/%s",
								StringCast<ANSICHAR>(*LightProxy->GetOwnerNameOrLabel()).Get(),
								StringCast<ANSICHAR>(*LightProxy->GetComponentFName().ToString()).Get());
							const int32 CharsToCopy = FMath::Min<int32>(LightName.Len(), LightNameBytes - 1);

							FMemory::Memcpy(Dest, *LightName, CharsToCopy);
							Dest[CharsToCopy] = 0;
						}
					}

					if ((SortedLightInfo.SortKey.Fields.LightType == LightType_Point && ViewFamily.EngineShowFlags.PointLights)
						|| (SortedLightInfo.SortKey.Fields.LightType == LightType_Spot && ViewFamily.EngineShowFlags.SpotLights)
						|| (SortedLightInfo.SortKey.Fields.LightType == LightType_Rect && ViewFamily.EngineShowFlags.RectLights))
					{
						const int32 LocalLightIndex = ViewSpacePosAndRadiusData.Num();

						if (SortedLightInfo.SortKey.Fields.bHandledByMegaLights && MegaLightsSupportedStart == MAX_int32)
						{
							MegaLightsSupportedStart = LocalLightIndex;
						}

						const FSphere BoundingSphere = LightProxy->GetBoundingSphere();
						const FVector ViewSpaceBoundingSphereCenter(View.ViewMatrices.GetWorldToView().TransformPosition(BoundingSphere.Center));
						const float Distance = ViewSpaceBoundingSphereCenter.Z + BoundingSphere.W;
						FurthestLight = FMath::Max(FurthestLight, Distance);

						const FVector3f LightViewPosition = FVector4f(View.ViewMatrices.GetWorldToView().TransformPosition(LightParameters.WorldPosition)); // LWC_TODO: precision loss
						const FVector3f LightViewDirection = FVector4f(View.ViewMatrices.GetWorldToView().TransformVector((FVector)LightParameters.Direction)); // LWC_TODO: precision loss

						// Note: inverting radius twice seems stupid (but done in shader anyway otherwise)
						FVector4f ViewSpacePosAndRadius(LightViewPosition, 1.0f / LightParameters.InvRadius);
						ViewSpacePosAndRadiusData.Add(ViewSpacePosAndRadius);

						const bool bIsRectLight = !bRenderRectLightsAsSpotLights && LightProxy->IsRectLight();
						const bool bUseTightRectLightCulling = bIsRectLight && LightParameters.RectLightBarnLength > 0.5f && LightParameters.RectLightBarnCosAngle > FMath::Cos(FMath::DegreesToRadians(GetRectLightBarnDoorMaxAngle()));

						// Pack flags in the LSB of PreProcAngle
						const float PreProcAngle = SortedLightInfo.SortKey.Fields.LightType == LightType_Spot ? GetTanRadAngleOrZero(LightProxy->GetOuterConeAngle()) : 0.0f;
						const uint32 PackedPreProcAngleAndFlags = (FMath::AsUInt(PreProcAngle) & 0xFFFFFFF8) | (LightProxy->HasSourceTexture() ? 0x4 : 0) | (bUseTightRectLightCulling ? 0x2 : 0) | (bIsRectLight ? 0x1 : 0);
						FVector4f ViewSpaceDirAndPreprocAngleAndFlags(LightViewDirection, FMath::AsFloat(PackedPreProcAngleAndFlags)); // LWC_TODO: precision loss
						ViewSpaceDirAndPreprocAngleData.Add(ViewSpaceDirAndPreprocAngleAndFlags);

						if (bUseTightRectLightCulling)
						{
							const FRectLightSceneProxy* RectProxy = (const FRectLightSceneProxy*)LightProxy;

							TArray<FPlane, TInlineAllocator<NUM_PLANES_PER_RECT_LIGHT>> Planes;

							CalculateRectLightCullingPlanes(RectProxy, Planes);

							for (FPlane& Plane : Planes)
							{
								const FPlane4f ViewPlane(Plane.TransformBy(LightProxy->GetLightToWorld() * View.ViewMatrices.GetWorldToView()));
								ViewSpaceRectPlanesData.Add(FVector4f(FVector3f(ViewPlane), -ViewPlane.W));
							}
						}
						else
						{
							ViewSpaceRectPlanesData.AddZeroed(NUM_PLANES_PER_RECT_LIGHT);
						}

						FVector4f WeightEstimateData;
						WeightEstimateData.X = LightParameters.Color.GetLuminance() * FMath::Max(LightParameters.DiffuseScale, LightParameters.SpecularScale);
						if (bIsRectLight)
						{
							WeightEstimateData.X *= 2.0f * LightParameters.SourceRadius * 2.0f * LightParameters.SourceLength;
						}
						WeightEstimateData.Y = LightProxy->IsInverseSquared() ? 0.0f : LightParameters.FalloffExponent;
						WeightEstimateData.Z = 0.0f;
						WeightEstimateData.W = 0.0f;
						ViewSpaceWeightEstimateData.Add(WeightEstimateData);

						if (bVisualizeLightShape)
						{
							VisualizeLightShape::FSortKey& SortKey = VisualizeLightShapeSortKeys[VisualizeLightShapeSortKeys.AddUninitialized()];
							SortKey.LightData.ForwardLightIndex = IndexInBuffer;
							SortKey.LightData.LightType = SortedLightInfo.SortKey.Fields.LightType;
							SortKey.Distance = (float)ViewSpaceBoundingSphereCenter.SizeSquared();
						}

						bHasRectLights |= bIsRectLight;
						bHasTexturedLights |= LightProxy->HasSourceTexture();
						LightGridLightingChannelAndMask &= LightProxy->GetLightingChannelMask();
					}
					else if (SortedLightInfo.SortKey.Fields.LightType == LightType_Directional && ViewFamily.EngineShowFlags.DirectionalLights)
					{
						// The selected forward directional light is also used for volumetric lighting using ForwardLightUniformParameters UB.
						// Also some people noticed that depending on the order a two directional lights are made visible in a level, the selected light for volumetric fog lighting will be different.
						// So to be clear and avoid such issue, we select the most intense directional light for forward shading and volumetric lighting.
						const float LightIntensitySq = FVector3f(LightParameters.Color).SizeSquared();
						const int32 LightForwardShadingPriority = LightProxy->GetDirectionalLightForwardShadingPriority();
#if WITH_EDITOR
						if (LightForwardShadingPriority > SelectedForwardDirectionalLightPriority)
						{
							// Reset the count if the new light has a higher priority than the previous one.
							ConflictingLightCountForForwardShading = 1;
						}
						else if (LightForwardShadingPriority == SelectedForwardDirectionalLightPriority)
						{
							// Accumulate new light if also has the highest priority value.
							ConflictingLightCountForForwardShading++;
						}
#endif
						if (LightForwardShadingPriority > SelectedForwardDirectionalLightPriority
							|| (LightForwardShadingPriority == SelectedForwardDirectionalLightPriority && LightIntensitySq > SelectedForwardDirectionalLightIntensitySq))
						{

							SelectedForwardDirectionalLightPriority = LightForwardShadingPriority;
							SelectedForwardDirectionalLightIntensitySq = LightIntensitySq;
							View.ForwardLightingResources.SelectedForwardDirectionalLightProxy = LightProxy;
							View.ForwardLightingResources.SelectedForwardDirectionalLightIndex = IndexInBuffer;

							ForwardLightUniformParameters->HasDirectionalLight = 1;
							ForwardLightUniformParameters->DirectionalLightColor = FVector3f(LightParameters.Color);
							if (LightProxy->GetUsePerPixelAtmosphereTransmittance())
							{
								// When using PerPixelTransmittance, transmittance is evaluated per pixel by sampling the transmittance texture. It gives better gradient on large scale objects such as mountains.
								// However, to skip doing that texture sampling in translucent using ForwardShading/VolumetricFog, we use the simple planet top ground transmittance as a simplification.
								// That will work for most of the cases for most of the map/terrain at the top of the virtual planet.
								ForwardLightUniformParameters->DirectionalLightColor *= FVector3f(LightProxy->GetAtmosphereTransmittanceTowardSun());
							}
							ForwardLightUniformParameters->DirectionalLightVolumetricScatteringIntensity = LightProxy->GetVolumetricScatteringIntensity();
							ForwardLightUniformParameters->DirectionalLightSpecularScale = FMath::Clamp(LightProxy->GetSpecularScale(), 0.f, 1.f);
							ForwardLightUniformParameters->DirectionalLightDiffuseScale = FMath::Clamp(LightProxy->GetDiffuseScale(), 0.f, 1.f);
							ForwardLightUniformParameters->DirectionalLightDirection = LightParameters.Direction;
							ForwardLightUniformParameters->DirectionalLightSourceRadius = LightParameters.SourceRadius;
							ForwardLightUniformParameters->DirectionalLightSoftSourceRadius = LightParameters.SoftSourceRadius;
							ForwardLightUniformParameters->DirectionalLightSceneInfoExtraDataPacked = LightSceneInfoExtraDataPacked;
							ForwardLightUniformParameters->DirectionalLightVSM = INDEX_NONE;
							ForwardLightUniformParameters->DirectionalLightLightFunctionAtlasLightIndex = LightParameters.LightFunctionAtlasLightIndex;
							ForwardLightUniformParameters->DirectionalLightAffectsTranslucentLighting = LightParameters.bAffectsTranslucentLighting ? 1 : 0;
							ForwardLightUniformParameters->DirectionalLightHandledByMegaLights = SortedLightInfo.SortKey.Fields.bHandledByMegaLights ? 1 : 0;

							const FVector2D FadeParams = LightProxy->GetDirectionalLightDistanceFadeParameters(View.GetFeatureLevel(), LightSceneInfo->IsPrecomputedLightingValid(), View.MaxShadowCascades);

							ForwardLightUniformParameters->DirectionalLightDistanceFadeMAD = FVector2f(FadeParams.Y, -FadeParams.X * FadeParams.Y);	// LWC_TODO: Precision loss

							const FMatrix TranslatedWorldToWorld = FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation());

							if (bDynamicShadows)
							{
								const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& DirectionalLightShadowInfos = VisibleLightInfos[LightSceneInfo->Id].AllProjectedShadows;

								ForwardLightUniformParameters->DirectionalLightVSM = VirtualShadowMapId;

								DirLightShadowParameters->NumDirectionalLightCascades = 0;
								// Unused cascades should compare > all scene depths
								DirLightShadowParameters->CascadeEndDepths = FVector4f(MAX_FLT, MAX_FLT, MAX_FLT, MAX_FLT);

								for (const FProjectedShadowInfo* ShadowInfo : DirectionalLightShadowInfos)
								{
									if (ShadowInfo->DependentView)
									{
										// when rendering stereo views, allow using the shadows rendered for the primary view as 'close enough'
										if (ShadowInfo->DependentView != &View && ShadowInfo->DependentView != View.GetPrimaryView())
										{
											continue;
										}
									}

									const int32 CascadeIndex = ShadowInfo->CascadeSettings.ShadowSplitIndex;

									if (ShadowInfo->IsWholeSceneDirectionalShadow() && !ShadowInfo->HasVirtualShadowMap() && ShadowInfo->bAllocated && CascadeIndex < GMaxForwardShadowCascades)
									{
										const FMatrix WorldToShadow = ShadowInfo->GetWorldToShadowMatrix(DirLightShadowParameters->DirectionalLightShadowmapMinMax[CascadeIndex]);
										const FMatrix44f TranslatedWorldToShadow = FMatrix44f(TranslatedWorldToWorld * WorldToShadow);

										DirLightShadowParameters->NumDirectionalLightCascades++;
										DirLightShadowParameters->DirectionalLightTranslatedWorldToShadowMatrix[CascadeIndex] = TranslatedWorldToShadow;
										DirLightShadowParameters->CascadeEndDepths[CascadeIndex] = ShadowInfo->CascadeSettings.SplitFar;

										if (CascadeIndex == 0)
										{
											DirLightShadowParameters->DirectionalLightShadowmapAtlas = GraphBuilder.RegisterExternalTexture(ShadowInfo->RenderTargets.DepthTarget);
											DirLightShadowParameters->DirectionalLightDepthBias = ShadowInfo->GetShaderDepthBias();
											FVector2D AtlasSize = DirLightShadowParameters->DirectionalLightShadowmapAtlas->Desc.Extent;
											DirLightShadowParameters->DirectionalLightShadowmapAtlasBufferSize = FVector4f(AtlasSize.X, AtlasSize.Y, 1.0f / AtlasSize.X, 1.0f / AtlasSize.Y);
										}
									}
								}
							}

							const FStaticShadowDepthMap* StaticShadowDepthMap = LightSceneInfo->Proxy->GetStaticShadowDepthMap();
							const uint32 bStaticallyShadowedValue = LightSceneInfo->IsPrecomputedLightingValid() 
																&& StaticShadowDepthMap
																&& StaticShadowDepthMap->Data 
																&& !StaticShadowDepthMap->Data->WorldToLight.ContainsNaN()
																&& StaticShadowDepthMap->TextureRHI ? 1 : 0;
							DirLightShadowParameters->DirectionalLightUseStaticShadowing = bStaticallyShadowedValue;
							if (bStaticallyShadowedValue)
							{
								const FMatrix44f TranslatedWorldToShadow = FMatrix44f(TranslatedWorldToWorld * StaticShadowDepthMap->Data->WorldToLight);
								DirLightShadowParameters->DirectionalLightStaticShadowBufferSize = FVector4f(StaticShadowDepthMap->Data->ShadowMapSizeX, StaticShadowDepthMap->Data->ShadowMapSizeY, 1.0f / StaticShadowDepthMap->Data->ShadowMapSizeX, 1.0f / StaticShadowDepthMap->Data->ShadowMapSizeY);
								DirLightShadowParameters->DirectionalLightTranslatedWorldToStaticShadow = TranslatedWorldToShadow;
								DirLightShadowParameters->DirectionalLightStaticShadowmap = StaticShadowDepthMap->TextureRHI;
							}
							else
							{
								DirLightShadowParameters->DirectionalLightStaticShadowBufferSize = FVector4f(0, 0, 0, 0);
								DirLightShadowParameters->DirectionalLightTranslatedWorldToStaticShadow = FMatrix44f::Identity;
								DirLightShadowParameters->DirectionalLightStaticShadowmap = GWhiteTexture->TextureRHI;
							}
						}
					}
				}
			}

			if (LightGridViewData)
			{
				LightGridViewData->SimpleLightIdToForwardLightIndex = MoveTemp(NewSimpleLightIdToForwardLightIndex);
			}

			// 3. add simple lights into ForwardLightData and fill uninitialized ViewSpacePosAndRadiusData/IndirectionIndices
			if (SimpleLightsEnd > 0)
			{
				ForwardLightData.Reserve(ForwardLightData.Num() + SimpleLightsEnd);

				const FSimpleLightArray& SimpleLights = SortedLightSet.SimpleLights;

				for (int32 SortedIndex = 0; SortedIndex < SimpleLightsEnd; ++SortedIndex)
				{
					check(SortedLightSet.SortedLights[SortedIndex].LightSceneInfo == nullptr);
					check(!SortedLightSet.SortedLights[SortedIndex].SortKey.Fields.bIsNotSimpleLight);

					int32 SimpleLightIndex = SortedLightSet.SortedLights[SortedIndex].SimpleLightIndex;

					const int32 IndexInBuffer = ForwardLightData.AddUninitialized(1);
					FForwardLightData& LightData = ForwardLightData[IndexInBuffer];

					const FSimpleLightEntry& SimpleLight = SimpleLights.InstanceData[SimpleLightIndex];
					const FSimpleLightPerViewEntry& SimpleLightPerViewData = SimpleLights.GetViewDependentData(SimpleLightIndex, ViewIndex, Views.Num());
					const FLinearColor SimpleLightColor = GetSimpleLightColor(View, SimpleLight);
					PackLightData(LightData, View, SimpleLight, SimpleLightPerViewData, SimpleLightColor, INDEX_NONE /*PrevLocalLightIndex*/, false /*bHandledByMegaLights*/, GlobalRayEndBias);
					
					FVector4f ViewSpacePosAndRadius(FVector4f(View.ViewMatrices.GetWorldToView().TransformPosition(SimpleLightPerViewData.Position)), SimpleLight.Radius);
					ViewSpacePosAndRadiusData[SortedIndex] = ViewSpacePosAndRadius;

					IndirectionIndices[SortedIndex] = IndexInBuffer;

					if (bVisualizeLightShape)
					{
						VisualizeLightShape::FSortKey& SortKey = VisualizeLightShapeSortKeys[SortedIndex];
						SortKey.LightData.ForwardLightIndex = IndexInBuffer;
						SortKey.LightData.LightType = LightType_Point;
						SortKey.Distance = FVector3f(ViewSpacePosAndRadius).SizeSquared();
					}
				}
			}
		}

#if WITH_EDITOR
		// For any views, if there are more than two light that compete for the forward shaded light, we report it.
		bMultipleDirLightsConflictForForwardShading |= ConflictingLightCountForForwardShading >= 2;
#endif

		const int32 NumLightsFinal = ForwardLightData.Num();
		const int32 NumVisibleLocalLights = ViewSpacePosAndRadiusData.Num();

		MegaLightsSupportedStart = FMath::Min<int32>(MegaLightsSupportedStart, NumVisibleLocalLights);

		// Some platforms index the StructuredBuffer in the shader based on the stride specified at buffer creation time, not from the stride specified in the shader.
		// ForwardLightBuffer is a StructuredBuffer<float4> in the shader, so create the buffer with a stride of sizeof(float4)
		static_assert(sizeof(FForwardLightData) % sizeof(FVector4f) == 0, "ForwardLightBuffer is used as a StructuredBuffer<float4> in the shader");
		const uint32 ForwardLightDataSizeNumFloat4 = (NumLightsFinal * sizeof(FForwardLightData)) / sizeof(FVector4f);

		FRDGBufferRef DirectionalLightIndicesBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("DirectionalLightIndicesBuffer"), DirectionalLightIndices);

		View.bLightGridHasRectLights = bHasRectLights;
		View.bLightGridHasTexturedLights = bHasTexturedLights;
		View.LightGridLightingChannelAndMask = LightGridLightingChannelAndMask;

		const FIntPoint LightGridSizeXY = FIntPoint::DivideAndRoundUp(View.ViewRect.Size(), GLightGridPixelSize);
		if (AssociatedPrimaryView)
		{
			// The visibility lists should be exactly the same
			//checkSlow(PrimaryView->ForwardLightingResources.LocalLightVisibleLightInfosIndex == LocalLightVisibleLightInfosIndex);
			check(PerViewForwardLightUniformParameters[View.PrimaryViewIndex]->NumLocalLights == NumVisibleLocalLights);
			ForwardLightUniformParameters->ForwardLightBuffer = PerViewForwardLightUniformParameters[View.PrimaryViewIndex]->ForwardLightBuffer;
			ForwardLightUniformParameters->LightNamesBuffer = PerViewForwardLightUniformParameters[View.PrimaryViewIndex]->LightNamesBuffer;
		}
		else
		{
			FRDGBufferRef ForwardLightBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("ForwardLightBuffer"),
				TConstArrayView<FVector4f>(reinterpret_cast<const FVector4f*>(ForwardLightData.GetData()), ForwardLightDataSizeNumFloat4));
			ForwardLightUniformParameters->ForwardLightBuffer = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(ForwardLightBuffer));

			FRDGBufferRef LightNamesBuffer = CreateByteAddressBuffer(GraphBuilder, bNeedsLightNames ? TEXT("LightNamesBuffer") : TEXT("DummyLightNamesBuffer"), MoveTemp(LightNames));
			ForwardLightUniformParameters->LightNamesBuffer = GraphBuilder.CreateSRV(LightNamesBuffer);
		}

		if (AssociatedSecondaryView)
		{
			// Light positions are stored relative to the primary view, applying this offset allows finding their positions relative to the secondary view
			ForwardLightUniformParameters->PreViewTranslationOffsetISR = FVector4f(FVector3f(AssociatedSecondaryView->ViewMatrices.GetPreViewTranslation() - View.ViewMatrices.GetPreViewTranslation()), 0.0f);
		}
		else if (AssociatedPrimaryView)
		{
			// Secondary views must store this as well so that it can be used by VSMs, which access secondary instanced view buffers
			ForwardLightUniformParameters->PreViewTranslationOffsetISR = FVector4f(FVector3f(View.ViewMatrices.GetPreViewTranslation() - AssociatedPrimaryView->ViewMatrices.GetPreViewTranslation()), 0.0f);
		}
		else
		{
			ForwardLightUniformParameters->PreViewTranslationOffsetISR = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		}

		ForwardLightUniformParameters->DirectionalLightIndices = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(DirectionalLightIndicesBuffer));
		ForwardLightUniformParameters->NumLocalLights = NumVisibleLocalLights;
		ForwardLightUniformParameters->NumDirectionalLights = DirectionalLightIndices.Num();
		ForwardLightUniformParameters->NumReflectionCaptures = View.NumBoxReflectionCaptures + View.NumSphereReflectionCaptures;
		ForwardLightUniformParameters->NumGridCells = LightGridSizeXY.X * LightGridSizeXY.Y * GLightGridSizeZ;
		ForwardLightUniformParameters->CulledGridSize = FIntVector(LightGridSizeXY.X, LightGridSizeXY.Y, GLightGridSizeZ);
		ForwardLightUniformParameters->MaxCulledLightsPerCell = GLightLinkedListCulling ? NumVisibleLocalLights : GMaxCulledLightsPerCell;
		ForwardLightUniformParameters->LightGridPixelSizeShift = FMath::FloorLog2(GLightGridPixelSize);
		ForwardLightUniformParameters->DirectionalMegaLightsSupportedStartIndex = DirectionalMegaLightsSupportedStart;
		ForwardLightUniformParameters->DirectLightingShowFlag = (ViewFamily.EngineShowFlags.Lighting && ViewFamily.EngineShowFlags.DirectLighting) ? 1 : 0;

		// Clamp far plane to something reasonable
		const float KilometersToCentimeters = 100000.0f;
		const float LightCullingMaxDistance = GLightCullingMaxDistanceOverrideKilometers <= 0.0f ? (float)UE_OLD_HALF_WORLD_MAX / 5.0f : GLightCullingMaxDistanceOverrideKilometers * KilometersToCentimeters;
		float FarPlane = FMath::Min(FMath::Max(FurthestLight, View.FurthestReflectionCaptureDistance), LightCullingMaxDistance);
		FVector ZParams = GetLightGridZParams(View.NearClippingDistance, FarPlane + 10.f);
		ForwardLightUniformParameters->LightGridZParams = (FVector3f)ZParams;

		const uint64 NumIndexableLights = !bLightGridUses16BitBuffers ? (1llu << (sizeof(FLightIndexType32) * 8llu)) : (1llu << (sizeof(FLightIndexType) * 8llu));

		if ((uint64)ForwardLightData.Num() > NumIndexableLights)
		{
			static bool bWarned = false;

			if (!bWarned)
			{
				UE_LOGF(LogRenderer, Warning, "Exceeded indexable light count, glitches will be visible (%u / %llu)", ForwardLightData.Num(), NumIndexableLights);
				bWarned = true;
			}
		}

		check(ViewSpacePosAndRadiusData.Num() == NumVisibleLocalLights);
		check(ViewSpaceDirAndPreprocAngleData.Num() == NumVisibleLocalLights);
		check(ViewSpaceRectPlanesData.Num() == NumVisibleLocalLights * NUM_PLANES_PER_RECT_LIGHT);
		check(ViewSpaceWeightEstimateData.Num() == NumVisibleLocalLights);
		check(IndirectionIndices.Num() == NumVisibleLocalLights);

		if (bVisualizeLightShape && VisualizeLightShapeSortKeys.Num() > 0)
		{
			FRDGUploadData<VisualizeLightShape::FLightInfo> VisualizeLightShapeLightInfo(GraphBuilder, VisualizeLightShapeSortKeys.Num());

			GraphBuilder.AddSetupTask([SortKeys = MoveTemp(VisualizeLightShapeSortKeys), VisualizeLightShapeLightInfo]()
			{
				SCOPED_NAMED_EVENT(VisualizeLightShapeSortLights, FColor::Emerald);

				Algo::Sort(SortKeys, TGreater<>());

				for (int32 Index = 0; Index < SortKeys.Num(); ++Index)
				{
					VisualizeLightShapeLightInfo[Index] = SortKeys[Index].LightData;
				}
			});

			View.ForwardLightingResources.VisualizeLightShapeLightInfos = TConstArrayView<VisualizeLightShape::FLightInfo>(VisualizeLightShapeLightInfo.GetData(), VisualizeLightShapeLightInfo.Num());
			View.ForwardLightingResources.VisualizeLightShapeLightInfoBuffer = CreateUploadBuffer(GraphBuilder, TEXT("VisualizeLightShape.LightInfos"), VisualizeLightShapeLightInfo);
		}

		FRDGBufferRef LightViewSpacePositionAndRadius = CreateStructuredBuffer(GraphBuilder, TEXT("ViewSpacePosAndRadiusData"), TConstArrayView<FVector4f>(ViewSpacePosAndRadiusData));
		FRDGBufferRef LightViewSpaceDirAndPreprocAngle = CreateStructuredBuffer(GraphBuilder, TEXT("ViewSpaceDirAndPreprocAngleData"), TConstArrayView<FVector4f>(ViewSpaceDirAndPreprocAngleData));
		FRDGBufferRef LightViewSpaceRectPlanes = CreateStructuredBuffer(GraphBuilder, TEXT("ViewSpaceRectPlanesData"), TConstArrayView<FVector4f>(ViewSpaceRectPlanesData));
		FRDGBufferRef LightViewSpaceWeightEstimate = CreateStructuredBuffer(GraphBuilder, TEXT("ViewSpaceWeightEstimateData"), TConstArrayView<FVector4f>(ViewSpaceWeightEstimateData));

		FRDGBufferSRVRef LightViewSpacePositionAndRadiusSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(LightViewSpacePositionAndRadius));
		FRDGBufferSRVRef LightViewSpaceDirAndPreprocAngleSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(LightViewSpaceDirAndPreprocAngle));
		FRDGBufferSRVRef LightViewSpaceRectPlanesSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(LightViewSpaceRectPlanes));
		FRDGBufferSRVRef LightViewSpaceWeightEstimateSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(LightViewSpaceWeightEstimate));

		FRDGBufferRef IndirectionIndicesBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("IndirectionIndices"), TConstArrayView<int32>(IndirectionIndices));
		FRDGBufferSRVRef IndirectionIndicesSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(IndirectionIndicesBuffer));

		// Allocate buffers using the scene render targets size so we won't reallocate every frame with dynamic resolution
		const FIntPoint MaxLightGridSizeXY = FIntPoint::DivideAndRoundUp(View.GetSceneTexturesConfig().Extent, GLightGridPixelSize);

		const int32 MaxNumCells = MaxLightGridSizeXY.X * MaxLightGridSizeXY.Y * GLightGridSizeZ * NumCulledGridPrimitiveTypes;

		uint32 LightGridCullMarginXY = MegaLights::IsEnabled(ViewFamily) ? MegaLights::GetSampleMargin() : 0;
		uint32 LightGridCullMarginZ = 0;
		FVector3f LightGridCullMarginZParams = FVector3f::ZeroVector;
		uint32 LightGridCullMaxZ = 0;
		if (ShouldRenderVolumetricFog())
		{
			uint32 MarginInVolumetricFogGridCells = 1 + (MegaLights::IsEnabled(ViewFamily) && MegaLights::UseVolume() ? MegaLights::GetSampleMargin() : 0);
			LightGridCullMarginXY = MarginInVolumetricFogGridCells * GetVolumetricFogGridPixelSize();
			LightGridCullMarginZ = MarginInVolumetricFogGridCells;

			FVolumetricFogGlobalData VolumetricFogParamaters;
			SetupVolumetricFogGlobalData(View, VolumetricFogParamaters);
			LightGridCullMarginZParams = VolumetricFogParamaters.GridZParams;
			LightGridCullMaxZ = VolumetricFogParamaters.ViewGridSize.Z;
		}

		ForwardLightUniformParameters->CulledBufferOffsetISR = MaxNumCells;

		RDG_EVENT_SCOPE(GraphBuilder, "CullLights %ux%ux%u NumLights %u NumCaptures %u",
			ForwardLightUniformParameters->CulledGridSize.X,
			ForwardLightUniformParameters->CulledGridSize.Y,
			ForwardLightUniformParameters->CulledGridSize.Z,
			ForwardLightUniformParameters->NumLocalLights,
			ForwardLightUniformParameters->NumReflectionCaptures);

		FLightGrid ParentLightGrid;
		uint32 ParentLightGridFactor = 1;

		if(CVarLightCullingTwoLevel.GetValueOnRenderThread() && (int32)ForwardLightUniformParameters->NumLocalLights > CVarLightCullingTwoLevelThreshold.GetValueOnRenderThread())
		{
			ParentLightGridFactor = (uint32)FMath::Pow(2.0f, FMath::Clamp(CVarLightCullingTwoLevelExponent.GetValueOnRenderThread(), 1, 4));

			FIntVector ParentLightGridSize = FIntVector::DivideAndRoundUp(ForwardLightUniformParameters->CulledGridSize, ParentLightGridFactor);

			ParentLightGrid = LightGridInjection(
				GraphBuilder,
				View,
				ParentLightGridSize,
				FMath::FloorLog2(GLightGridPixelSize * ParentLightGridFactor),
				ParentLightGridFactor,
				MaxNumCells, // TODO: could potentially be reduced on coarse grid
				ForwardLightUniformParameters->LightGridZParams,
				LightGridCullMarginXY,
				LightGridCullMarginZ,
				LightGridCullMarginZParams,
				LightGridCullMaxZ,
				ForwardLightUniformParameters->NumLocalLights,
				ForwardLightUniformParameters->NumReflectionCaptures,
				MegaLightsSupportedStart,
				bLightGridUses16BitBuffers,
				bHasRectLights && (GLightGridRefineRectLightBounds != 0),
				LightViewSpacePositionAndRadiusSRV,
				LightViewSpaceDirAndPreprocAngleSRV,
				LightViewSpaceRectPlanesSRV,
				LightViewSpaceWeightEstimateSRV,
				nullptr,
				/*bThreadGroupPerCell*/ true,
				/*bThreadGroupSize32*/ false,
				nullptr,
				nullptr,
				1);
		}

		const int32 WorkloadDistributionMode = CVarLightCullingWorkloadDistributionMode.GetValueOnRenderThread();

		uint32 NumThreadsPerCell = 1;
		
		if (WorkloadDistributionMode == 1) // thread group per cell (64 threads)
		{
			NumThreadsPerCell = 64;
		}
		else if (WorkloadDistributionMode == 2 && GRHIMinimumWaveSize <= 32) // thread group per cell (32 threads if supported, otherwise single thread).
		{
			NumThreadsPerCell = 32;
		}

		FLightGrid LightGrid = LightGridInjection(
			GraphBuilder,
			View,
			ForwardLightUniformParameters->CulledGridSize,
			ForwardLightUniformParameters->LightGridPixelSizeShift,
			1,
			MaxNumCells,
			ForwardLightUniformParameters->LightGridZParams,
			LightGridCullMarginXY,
			LightGridCullMarginZ,
			LightGridCullMarginZParams,
			LightGridCullMaxZ,
			ForwardLightUniformParameters->NumLocalLights,
			ForwardLightUniformParameters->NumReflectionCaptures,
			MegaLightsSupportedStart,
			bLightGridUses16BitBuffers,
			bHasRectLights && (GLightGridRefineRectLightBounds != 0),
			LightViewSpacePositionAndRadiusSRV,
			LightViewSpaceDirAndPreprocAngleSRV,
			LightViewSpaceRectPlanesSRV,
			LightViewSpaceWeightEstimateSRV,
			IndirectionIndicesSRV,
			NumThreadsPerCell > 1,
			NumThreadsPerCell == 32,
			ParentLightGrid.NumCulledLightsGridSRV,
			ParentLightGrid.CulledLightDataGridSRV,
			ParentLightGridFactor);

		ForwardLightUniformParameters->CulledLightDataGrid32Bit = LightGrid.CulledLightDataGridSRV;
		ForwardLightUniformParameters->CulledLightDataGrid16Bit = LightGrid.CulledLightDataGridSRV;
		ForwardLightUniformParameters->NumCulledLightsGrid = LightGrid.NumCulledLightsGridSRV;
	}

#if WITH_EDITOR
	if (bMultipleDirLightsConflictForForwardShading)
	{
		OnGetOnScreenMessages.AddLambda([](FScreenMessageWriter& ScreenMessageWriter)->void
		{
			static const FText Message = NSLOCTEXT("Renderer", "MultipleDirLightsConflictForForwardShading", "Multiple directional lights are competing to be the single one used for forward shading, translucent, water or volumetric fog. Please adjust their ForwardShadingPriority.\nAs a fallback, the main directional light will be selected based on overall brightness.");
			ScreenMessageWriter.DrawLine(Message, 10, FColor::Orange);
		});
	}
#endif

	return Result;
}

FComputeLightGridOutput FSceneRenderer::PrepareForwardLightData(FRDGBuilder& GraphBuilder, bool bCullLightsToGrid, const FSortedLightSetSceneInfo& SortedLightSet)
{
	SCOPED_NAMED_EVENT(PrepareForwardLightData, FColor::Emerald);

	FComputeLightGridArray<FForwardLightUniformParameters*> PerViewForwardLightUniformParameters;
	PerViewForwardLightUniformParameters.Reserve(Views.Num());

	FComputeLightGridArray<FForwardDirectionalLightShadowMapParameters*> PerViewForwardDirLightShadowParameters;
	PerViewForwardDirLightShadowParameters.Reserve(Views.Num());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		PerViewForwardLightUniformParameters.Add(GraphBuilder.AllocParameters<FForwardLightUniformParameters>());
		PerViewForwardDirLightShadowParameters.Add(GraphBuilder.AllocParameters<FForwardDirectionalLightShadowMapParameters>());
	}

	// TODO: Add simple lights to GPU Scene Lights

	// Build light view data buffers
	const bool bRenderRectLightsAsSpotLights = RenderRectLightsAsSpotLights(FeatureLevel);
	const uint32 LightShaderParameterFlags = bRenderRectLightsAsSpotLights ? ELightShaderParameterFlags::RectAsSpotLight : 0u;

	UE::Tasks::FTask PrerequisiteTask; // TODO: should match prerequisite of FGPUScene::UpdateGPULights(...), currently is null

	const int32 MaxLightId = Scene->GPUScene.GetMaxLightId();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		const float Exposure = View.GetLastEyeAdaptationExposure();

		FRDGUploadData<FLightViewData> LightViewData(GraphBuilder, FMath::Max(1, MaxLightId));

		GraphBuilder.AddSetupTask([this, &View, &SortedLightSet, LightViewData, LightShaderParameterFlags, Exposure]
			{
				SCOPED_NAMED_EVENT(PrepareLightViewData, FColor::Green);

				const bool bAllowStaticLighting = IsStaticLightingAllowed();

				for (int32 SortedIndex = SortedLightSet.SimpleLightsEnd; SortedIndex < SortedLightSet.SortedLights.Num(); ++SortedIndex)
				{
					const FSortedLightSceneInfo& SortedLightInfo = SortedLightSet.SortedLights[SortedIndex];

					// We can have simple lights past SimpleLightsEnd if they are handled by MegaLights
					if (!SortedLightInfo.LightSceneInfo)
					{
						check(!SortedLightInfo.SortKey.Fields.bIsNotSimpleLight && SortedLightInfo.SortKey.Fields.bHandledByMegaLights);
						continue;
					}

					const FLightSceneInfo* const LightSceneInfo = SortedLightInfo.LightSceneInfo;
					const FLightSceneProxy* LightProxy = LightSceneInfo->Proxy;

					const FLightSceneId LightSceneId = LightSceneInfo->Id;

					if (!ensureMsgf(LightViewData.IsValidIndex(LightSceneId.GetIndex()), TEXT("Visible light is missing from GPU Scene.")))
					{
						continue;
					}

					if (!ensureMsgf(Scene->Lights.IsValidIndex(LightSceneId), TEXT("Visible light is missing from GPU Scene.")))
					{
						continue;
					}

					if (!ensureMsgf(VisibleLightInfos.IsValidIndex(LightSceneId), TEXT("Visible light doesn't have valid info.")))
					{
						continue;
					}

					FLightViewData& CurrentLightViewData = LightViewData[LightSceneId.GetIndex()];

					if (LightSceneInfo->ShouldRenderLight(View))
					{
						FLightRenderParameters LightParameters;
						LightProxy->GetLightShaderParameters(LightParameters, LightShaderParameterFlags);

						const float LightFade = GetLightFadeFactor(View, LightProxy);

						float VolumetricScatteringIntensity = LightProxy->GetVolumetricScatteringIntensity();
						if (LightNeedsSeparateInjectionIntoVolumetricFogForOpaqueShadow(View, LightSceneInfo, VisibleLightInfos[LightSceneId], *Scene))
						{
							// Disable this lights forward shading volumetric scattering contribution
							VolumetricScatteringIntensity = 0;
						}

						const int32 VirtualShadowMapId = ViewFamily.EngineShowFlags.DynamicShadows ? VisibleLightInfos[LightSceneId].GetVirtualShadowMapId(&View) : INDEX_NONE;

						CurrentLightViewData.TranslatedWorldPosition = FVector3f(LightParameters.WorldPosition + View.ViewMatrices.GetPreViewTranslation());
						CurrentLightViewData.Color = FVector3f(LightParameters.Color) * LightFade * LightParameters.GetLightExposureScale(Exposure);
						CurrentLightViewData.VolumetricScatteringIntensity = VolumetricScatteringIntensity;
						CurrentLightViewData.VirtualShadowMapId = VirtualShadowMapId;
						CurrentLightViewData.LightSceneInfoExtraDataPacked = LightSceneInfo->PackExtraData(bAllowStaticLighting, false, false, false); // TODO: bLightFunction, bMegaLight, bClusteredDeferredSupported
						CurrentLightViewData.RectLightAtlasUVOffset = LightParameters.RectLightAtlasUVOffset;
						CurrentLightViewData.RectLightAtlasUVScale = LightParameters.RectLightAtlasUVScale;
						CurrentLightViewData.RectLightAtlasMaxLevel = LightParameters.RectLightAtlasMaxLevel;
						CurrentLightViewData.IESAtlasIndex = LightParameters.IESAtlasIndex;
					}
					else
					{
						CurrentLightViewData.TranslatedWorldPosition = FVector3f::ZeroVector;
						CurrentLightViewData.Color = FVector3f::ZeroVector;
						CurrentLightViewData.VolumetricScatteringIntensity = 0.0f;
						CurrentLightViewData.VirtualShadowMapId = INDEX_NONE;
						CurrentLightViewData.LightSceneInfoExtraDataPacked = 0;
						CurrentLightViewData.RectLightAtlasUVOffset = FVector2f::ZeroVector;
						CurrentLightViewData.RectLightAtlasUVScale = FVector2f::ZeroVector;
						CurrentLightViewData.RectLightAtlasMaxLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();
						CurrentLightViewData.IESAtlasIndex = INDEX_NONE;
					}
				}
			}, PrerequisiteTask);

		FRDGBufferRef LightViewDataBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("LightViewData"), LightViewData);

		FForwardLightUniformParameters* ForwardLightUniformParameters = PerViewForwardLightUniformParameters[ViewIndex];
		ForwardLightUniformParameters->LightViewData = GraphBuilder.CreateSRV(LightViewDataBuffer);
	}

	FComputeLightGridOutput Result = ComputeLightGrid(GraphBuilder, bCullLightsToGrid, SortedLightSet, PerViewForwardLightUniformParameters, PerViewForwardDirLightShadowParameters);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];
		View.ForwardLightingResources.SetUniformBuffer(GraphBuilder.CreateUniformBuffer(PerViewForwardLightUniformParameters[ViewIndex]));
		View.ForwardLightingResources.SetShadowUniformBuffer(GraphBuilder.CreateUniformBuffer(PerViewForwardDirLightShadowParameters[ViewIndex]));
	}

	return Result;
}

void FDeferredShadingSceneRenderer::RenderForwardShadowProjections(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef& OutForwardScreenSpaceShadowMask,
	FRDGTextureRef& OutForwardScreenSpaceShadowMaskSubPixel)
{
	CheckShadowDepthRenderCompleted();

	const bool bIsHairEnable = HairStrands::HasViewHairStrandsData(Views);
	bool bScreenShadowMaskNeeded = false;

	FRDGTextureRef SceneDepthTexture = SceneTextures.Depth.Target;

	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
	{
		const FLightSceneInfo* const LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;
		const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];

		bScreenShadowMaskNeeded |= VisibleLightInfo.ShadowsToProject.Num() > 0 || VisibleLightInfo.CapsuleShadowsToProject.Num() > 0 || LightSceneInfo->Proxy->GetLightFunctionMaterial() != nullptr;
	}

	if (bScreenShadowMaskNeeded)
	{
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderForwardShadingShadowProjections);

		FRDGTextureMSAA ForwardScreenSpaceShadowMask;
		FRDGTextureMSAA ForwardScreenSpaceShadowMaskSubPixel;

		{
			FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PF_B8G8R8A8, FClearValueBinding::White, TexCreate_RenderTargetable | TexCreate_ShaderResource));
			Desc.NumSamples = SceneDepthTexture->Desc.NumSamples;
			ForwardScreenSpaceShadowMask = CreateTextureMSAA(GraphBuilder, Desc, TEXT("ShadowMaskTextureMS"), TEXT("ShadowMaskTextureResolve"), GFastVRamConfig.ScreenSpaceShadowMask);
			if (bIsHairEnable)
			{
				Desc.NumSamples = 1;
				ForwardScreenSpaceShadowMaskSubPixel = CreateTextureMSAA(GraphBuilder, Desc, TEXT("ShadowMaskSubPixelTextureMS"), TEXT("ShadowMaskSubPixelTexture"), GFastVRamConfig.ScreenSpaceShadowMask);
			}
		}

		RDG_EVENT_SCOPE_STAT(GraphBuilder, ShadowProjection, "ShadowProjectionOnOpaque");

		// All shadows render with min blending
		AddClearRenderTargetPass(GraphBuilder, ForwardScreenSpaceShadowMask.Target);
		if (bIsHairEnable)
		{
			AddClearRenderTargetPass(GraphBuilder, ForwardScreenSpaceShadowMaskSubPixel.Target);
		}

		const bool bProjectingForForwardShading = true;

		for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
		{
			const FLightSceneInfo* const LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;
			FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo->Id];

			const bool bIssueLightDrawEvent = VisibleLightInfo.ShadowsToProject.Num() > 0 || VisibleLightInfo.CapsuleShadowsToProject.Num() > 0;

			FString LightNameWithLevel;
			GetLightNameForDrawEvent(LightSceneInfo->Proxy, LightNameWithLevel);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, bIssueLightDrawEvent, "%s", *LightNameWithLevel);

			if (VisibleLightInfo.ShadowsToProject.Num() > 0)
			{
				RenderShadowProjections(
					GraphBuilder,
					SceneTextures,
					ForwardScreenSpaceShadowMask.Target,
					ForwardScreenSpaceShadowMaskSubPixel.Target,
					LightSceneInfo,
					bProjectingForForwardShading);

				if (bIsHairEnable)
				{
					RenderHairStrandsShadowMask(GraphBuilder, Views, LightSceneInfo, VisibleLightInfos, bProjectingForForwardShading, ForwardScreenSpaceShadowMask.Target);
				}
			}

			RenderCapsuleDirectShadows(GraphBuilder, *LightSceneInfo, ForwardScreenSpaceShadowMask.Target, VisibleLightInfo.CapsuleShadowsToProject, bProjectingForForwardShading);

			if (LightSceneInfo->GetDynamicShadowMapChannel() >= 0 && LightSceneInfo->GetDynamicShadowMapChannel() < 4)
			{
				RenderLightFunction(
					GraphBuilder,
					SceneTextures,
					LightSceneInfo,
					ForwardScreenSpaceShadowMask.Target,
					true, true, false);
			}
		}

		auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(ForwardScreenSpaceShadowMask.Target, ForwardScreenSpaceShadowMask.Resolve, ERenderTargetLoadAction::ELoad);
		OutForwardScreenSpaceShadowMask = ForwardScreenSpaceShadowMask.Resolve;

		if (bIsHairEnable)
		{
			OutForwardScreenSpaceShadowMaskSubPixel = ForwardScreenSpaceShadowMaskSubPixel.Target;
		}

		GraphBuilder.AddPass(RDG_EVENT_NAME("ResolveScreenSpaceShadowMask"), PassParameters, ERDGPassFlags::Raster, [](FRDGAsyncTask, FRHICommandList&) {});
	}
}

class FDebugLightGridPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDebugLightGridPS);
	SHADER_USE_PARAMETER_STRUCT(FDebugLightGridPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER(FScreenTransform, ScreenToPrimaryScreenPos)
		SHADER_PARAMETER(uint32, DebugMode)
		SHADER_PARAMETER(uint32, MaxThreshold)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) && ShaderPrint::IsSupported(Parameters.Platform);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Stay debug and skip optimizations to reduce compilation time on this long shader.
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		OutEnvironment.SetDefine(TEXT("SHADER_DEBUG_LIGHT_GRID_PS"), 1);
		OutEnvironment.SetDefine(TEXT("LIGHT_LINK_STRIDE"), LightLinkStride);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
	}
};
IMPLEMENT_GLOBAL_SHADER(FDebugLightGridPS, "/Engine/Private/LightGridInjection.usf", "DebugLightGridPS", SF_Pixel);

FScreenPassTexture AddVisualizeLightGridPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture ScreenPassSceneColor, FScreenPassTexture SceneDepthTexture)
{
	checkf(ShouldVisualizeLightGrid(View.Family->GetShaderPlatform()), TEXT("Must check ShouldVisualizeLightGrid(...) before calling AddVisualizeLightGridPass(...)."));

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeLightGrid");

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);

	ShaderPrint::RequestSpaceForLines(128);
	ShaderPrint::RequestSpaceForCharacters(128);

	FDebugLightGridPS::FPermutationDomain PermutationVector;
	TShaderMapRef<FDebugLightGridPS> PixelShader(View.ShaderMap, PermutationVector);
	FDebugLightGridPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDebugLightGridPS::FParameters>();
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintParameters);
	PassParameters->DepthTexture = SceneDepthTexture.IsValid() ? SceneDepthTexture.Texture : GSystemTextures.GetMaxFP16Depth(GraphBuilder);
	PassParameters->ScreenToPrimaryScreenPos = (
		FScreenTransform::ChangeTextureBasisFromTo(ScreenPassSceneColor, FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
		FScreenTransform::ChangeTextureBasisFromTo(SceneDepthTexture, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TexelPosition));
	PassParameters->MiniFontTexture = GetMiniFontTexture();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(ScreenPassSceneColor.Texture, ERenderTargetLoadAction::ELoad);
	PassParameters->DebugMode = GForwardLightGridDebug;
	PassParameters->MaxThreshold = GForwardLightGridDebugMaxThreshold;

	FRHIBlendState* PreMultipliedColorTransmittanceBlend = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();

	FPixelShaderUtils::AddFullscreenPass<FDebugLightGridPS>(GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("DebugLightGridCS"), PixelShader, PassParameters,
		ScreenPassSceneColor.ViewRect, PreMultipliedColorTransmittanceBlend);

	return MoveTemp(ScreenPassSceneColor);
}

class FLightGridFeedbackStatusCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLightGridFeedbackStatusCS);
	SHADER_USE_PARAMETER_STRUCT(FLightGridFeedbackStatusCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CulledLightDataAllocatorBuffer)
		SHADER_PARAMETER(uint32, NumCulledLightDataEntries)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CulledLightLinkAllocatorBuffer)
		SHADER_PARAMETER(uint32, NumAvailableLinks)

		SHADER_PARAMETER_STRUCT_INCLUDE(GPUMessage::FParameters, GPUMessageParams)
		SHADER_PARAMETER(uint32, StatusMessageId)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FLightGridFeedbackStatusCS, "/Engine/Private/LightGridInjection.usf", "FeedbackStatusCS", SF_Compute);

#if !UE_BUILD_SHIPPING
class FLightGridFeedbackStatus : public FRenderResource
{
public:
	virtual ~FLightGridFeedbackStatus() {}

	virtual void InitRHI(FRHICommandListBase& RHICmdList);

	virtual void ReleaseRHI();

	uint32 GetStatusMessageId() const { return StatusFeedbackSocket.GetMessageId().GetIndex(); }

private:
	GPUMessage::FSocket StatusFeedbackSocket;
	uint32 MaxEntriesHighWaterMark = 0;
};

TGlobalResource<FLightGridFeedbackStatus> GLightGridFeedbackStatus;

void FLightGridFeedbackStatus::InitRHI(FRHICommandListBase& RHICmdList)
{
	StatusFeedbackSocket = GPUMessage::RegisterHandler(TEXT("LightGrid.StatusFeedback"),
		[this](GPUMessage::FReader Message)
		{
			const uint32 AllocatedEntries = Message.Read<uint32>(0);
			const uint32 MaxEntries = Message.Read<uint32>(0);

			const uint32 AllocatedLinks = Message.Read<uint32>(0);
			const uint32 MaxLinks = Message.Read<uint32>(0);

			if (AllocatedEntries > MaxEntries)
			{
				bool bWarn = MaxEntries > MaxEntriesHighWaterMark;

				if (bWarn)
				{
					UE_LOGF(LogRenderer, Warning, "Building light grid exceeded number of available entries (%u / %u). "
														"Increase r.Forward.MaxCulledLightsPerCell to prevent potential visual artifacts.", AllocatedEntries, MaxEntries);
				}

				MaxEntriesHighWaterMark = FMath::Max(MaxEntriesHighWaterMark, MaxEntries);
			}

			if (AllocatedLinks > MaxLinks)
			{
				static bool bWarn = true;

				if (bWarn)
				{
					UE_LOGF(LogRenderer, Warning, "Building light grid exceeded number of available links, glitches will be visible (%u / %u).", AllocatedLinks, MaxLinks);
					bWarn = false;
				}
			}
		});
}

void FLightGridFeedbackStatus::ReleaseRHI()
{
	
}

void LightGridFeedbackStatus(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FRDGBufferRef CulledLightDataAllocatorBuffer,
	uint32 NumCulledLightDataEntries,
	FRDGBufferRef CulledLightLinkAllocatorBuffer,
	uint32 NumCulledLightLinks,
	bool bUseAsyncCompute)
{
	FLightGridFeedbackStatusCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FLightGridFeedbackStatusCS::FParameters>();

	PassParameters->CulledLightDataAllocatorBuffer = GraphBuilder.CreateSRV(CulledLightDataAllocatorBuffer);
	PassParameters->NumCulledLightDataEntries = NumCulledLightDataEntries;

	PassParameters->CulledLightLinkAllocatorBuffer = GraphBuilder.CreateSRV(CulledLightLinkAllocatorBuffer);
	PassParameters->NumAvailableLinks = NumCulledLightLinks;

	PassParameters->GPUMessageParams = GPUMessage::GetShaderParameters(GraphBuilder);
	PassParameters->StatusMessageId = GLightGridFeedbackStatus.GetStatusMessageId();

	auto ComputeShader = View.ShaderMap->GetShader<FLightGridFeedbackStatusCS>();

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("LightGridFeedbackStatus"),
		bUseAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, 1)
	);
}

#endif // !UE_BUILD_SHIPPING
