// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightMapDensityRendering.cpp: Implementation for rendering lightmap density.
=============================================================================*/

#include "LightMapDensityRendering.h"
#include "DeferredShadingRenderer.h"
#include "LightMap.h"
#include "LightMapRendering.h"
#include "Materials/Material.h"
#include "ScenePrivate.h"
#include "TextureResource.h"
#include "MeshPassProcessor.inl"

#if !UE_BUILD_DOCS
// Typedef is necessary because the C preprocessor thinks the comma in the template parameter list is a comma in the macro parameter list.
#define IMPLEMENT_DENSITY_VERTEXSHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	typedef TLightMapDensityVS< LightMapPolicyType > TLightMapDensityVS##LightMapPolicyName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TLightMapDensityVS##LightMapPolicyName,TEXT("/Engine/Private/LightMapDensityShader.usf"),TEXT("MainVertexShader"),SF_Vertex);

#define IMPLEMENT_DENSITY_PIXELSHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	typedef TLightMapDensityPS< LightMapPolicyType > TLightMapDensityPS##LightMapPolicyName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>,TLightMapDensityPS##LightMapPolicyName,TEXT("/Engine/Private/LightMapDensityShader.usf"),TEXT("MainPixelShader"),SF_Pixel);

// Implement a pixel shader type for skylights and one without, and one vertex shader that will be shared between them
#define IMPLEMENT_DENSITY_LIGHTMAPPED_SHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	IMPLEMENT_DENSITY_VERTEXSHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	IMPLEMENT_DENSITY_PIXELSHADER_TYPE(LightMapPolicyType,LightMapPolicyName);

IMPLEMENT_DENSITY_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_NO_LIGHTMAP>, FNoLightMapPolicy);
IMPLEMENT_DENSITY_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_DUMMY>, FDummyLightMapPolicy);
IMPLEMENT_DENSITY_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_LQ_LIGHTMAP>, TLightMapPolicyLQ);
IMPLEMENT_DENSITY_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_HQ_LIGHTMAP>, TLightMapPolicyHQ);

#endif

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLightmapDensityPassUniformParameters, )
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER(FVector4f, LightMapDensity)
	SHADER_PARAMETER(FVector4f, DensitySelectedColor) // The color to apply to selected objects.
	SHADER_PARAMETER(FVector4f, VertexMappedColor) // The color to apply to vertex mapped objects.
	SHADER_PARAMETER_TEXTURE(Texture2D, GridTexture) // The "Grid" texture to visualize resolution.
	SHADER_PARAMETER_SAMPLER(SamplerState, GridTextureSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FLightmapDensityPassUniformParameters, "LightmapDensityPass", SceneTextures);

void SetupLightmapDensityPassUniformBuffer(FRDGBuilder& GraphBuilder, const FSceneTextures* SceneTextures, ERHIFeatureLevel::Type FeatureLevel, FLightmapDensityPassUniformParameters& LightmapDensityPassParameters)
{
	SetupSceneTextureUniformParameters(GraphBuilder, SceneTextures, FeatureLevel, ESceneTextureSetupMode::None, LightmapDensityPassParameters.SceneTextures);

	LightmapDensityPassParameters.GridTexture = GEngine->LightMapDensityTexture->GetResource()->TextureRHI;
	LightmapDensityPassParameters.GridTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

	LightmapDensityPassParameters.LightMapDensity = FVector4f(
			1.0f,
			GEngine->MinLightMapDensity * GEngine->MinLightMapDensity,
			GEngine->IdealLightMapDensity * GEngine->IdealLightMapDensity,
			GEngine->MaxLightMapDensity * GEngine->MaxLightMapDensity);

	LightmapDensityPassParameters.DensitySelectedColor = GEngine->LightMapDensitySelectedColor;

	LightmapDensityPassParameters.VertexMappedColor = GEngine->LightMapDensityVertexMappedColor;
}

TRDGUniformBufferRef<FLightmapDensityPassUniformParameters> CreateLightmapDensityPassUniformBuffer(FRDGBuilder& GraphBuilder, const FSceneTextures* SceneTextures, ERHIFeatureLevel::Type FeatureLevel)
{
	auto* UniformBufferParameters = GraphBuilder.AllocParameters<FLightmapDensityPassUniformParameters>();
	SetupLightmapDensityPassUniformBuffer(GraphBuilder, SceneTextures, FeatureLevel, *UniformBufferParameters);
	return GraphBuilder.CreateUniformBuffer(UniformBufferParameters);
}

BEGIN_SHADER_PARAMETER_STRUCT(FLightMapDensitiesPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLightmapDensityPassUniformParameters, Pass)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void RenderLightMapDensities(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FRenderTargetBindingSlots& RenderTargets)
{
	RDG_EVENT_SCOPE(GraphBuilder, "LightMapDensity");

	// Draw the scene's emissive and light-map color.
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = const_cast<FViewInfo&>(Views[ViewIndex]);

		if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::LightmapDensity])
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
			View.BeginRenderView();

			auto* PassParameters = GraphBuilder.AllocParameters<FLightMapDensitiesPassParameters>();
			PassParameters->View = View.GetShaderParameters();
			PassParameters->Pass = CreateLightmapDensityPassUniformBuffer(GraphBuilder, View.GetSceneTexturesChecked(), View.GetFeatureLevel());
			PassParameters->RenderTargets = RenderTargets;
			FScene* Scene = View.Family->Scene->GetRenderScene();
			check(Scene != nullptr);
			Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

			GraphBuilder.AddPass(
				{},
				PassParameters,
				ERDGPassFlags::Raster,
				[&View, Pass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1);
				Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
			});
		}
	}
}

template<typename LightMapPolicyType>
bool FLightmapDensityMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	const LightMapPolicyType& RESTRICT LightMapPolicy,
	const typename LightMapPolicyType::ElementDataType& RESTRICT LightMapElementData,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;
	FVertexFactoryType* VertexFactoryType = VertexFactory->GetType();

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<TLightMapDensityVS<LightMapPolicyType>>();
	ShaderTypes.AddShaderType<TLightMapDensityPS<LightMapPolicyType>>();

	FMaterialShaders Shaders;
	if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	TMeshProcessorShaders<
		TLightMapDensityVS<LightMapPolicyType>,
		TLightMapDensityPS<LightMapPolicyType>> LightmapDensityPassShaders;
	Shaders.TryGetVertexShader(LightmapDensityPassShaders.VertexShader);
	Shaders.TryGetPixelShader(LightmapDensityPassShaders.PixelShader);

	TLightMapDensityElementData<LightMapPolicyType> ShaderElementData(LightMapElementData);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, true);

	{		
		// BuiltLightingAndSelectedFlags informs the shader is lighting is built or not for this primitive
		ShaderElementData.BuiltLightingAndSelectedFlags = FVector3f(0.0f, 0.0f, 0.0f);
		// LightMapResolutionScale is the physical resolution of the lightmap texture
		ShaderElementData.LightMapResolutionScale = FVector2f::UnitVector;

		bool bHighQualityLightMaps = AllowHighQualityLightmaps(FeatureLevel);

		ShaderElementData.bTextureMapped = false;

		if (MeshBatch.LCI &&
			MeshBatch.LCI->GetLightMapInteraction(FeatureLevel).GetType() == LMIT_Texture &&
			(MeshBatch.LCI->GetLightMapInteraction(FeatureLevel).GetTexture(bHighQualityLightMaps) || MeshBatch.LCI->GetLightMapInteraction(FeatureLevel).GetVirtualTexture()))
		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VirtualTexturedLightmaps"));
			if (CVar->GetValueOnRenderThread() == 1)
			{
				IAllocatedVirtualTexture* AllocatedVT = MeshBatch.LCI->GetResourceCluster()->AllocatedVT;
				if (AllocatedVT)
				{
					ShaderElementData.LightMapResolutionScale.X = AllocatedVT->GetWidthInPixels();
					ShaderElementData.LightMapResolutionScale.Y = AllocatedVT->GetHeightInPixels() * 2.0f; // Compensates the VT specific math in GetLightMapCoordinates (used to pack more coefficients per texture)
				}
			}
			else
			{
				ShaderElementData.LightMapResolutionScale.X = MeshBatch.LCI->GetLightMapInteraction(FeatureLevel).GetTexture(bHighQualityLightMaps)->GetSizeX();
				ShaderElementData.LightMapResolutionScale.Y = MeshBatch.LCI->GetLightMapInteraction(FeatureLevel).GetTexture(bHighQualityLightMaps)->GetSizeY();
			}

			ShaderElementData.bTextureMapped = true;

			ShaderElementData.BuiltLightingAndSelectedFlags.X = 1.0f;
			ShaderElementData.BuiltLightingAndSelectedFlags.Y = 0.0f;
		}
		else if (PrimitiveSceneProxy)
		{
			int32 LightMapResolution = PrimitiveSceneProxy->GetLightMapResolution();
		#if WITH_EDITOR
			if (GLightmassDebugOptions.bPadMappings)
			{
				LightMapResolution -= 2;
			}
		#endif
			if (PrimitiveSceneProxy->IsStatic() && LightMapResolution > 0)
			{
				ShaderElementData.bTextureMapped = true;
				ShaderElementData.LightMapResolutionScale = FVector2f(LightMapResolution, LightMapResolution);
				if (bHighQualityLightMaps)
				{	// Compensates the math in GetLightMapCoordinates (used to pack more coefficients per texture)
					ShaderElementData.LightMapResolutionScale.Y *= 2.f;
				}
				ShaderElementData.BuiltLightingAndSelectedFlags.X = 1.0f;
				ShaderElementData.BuiltLightingAndSelectedFlags.Y = 0.0f;
			}
			else
			{
				ShaderElementData.LightMapResolutionScale = FVector2f::ZeroVector;
				ShaderElementData.BuiltLightingAndSelectedFlags.X = 0.0f;
				ShaderElementData.BuiltLightingAndSelectedFlags.Y = 1.0f;
			}
		}

		if (PrimitiveSceneProxy && PrimitiveSceneProxy->IsSelected())
		{
			ShaderElementData.BuiltLightingAndSelectedFlags.Z = 1.0f;
		}
		else
		{
			ShaderElementData.BuiltLightingAndSelectedFlags.Z = 0.0f;
		}

		// Adjust for the grid texture being 2x2 repeating pattern...
		ShaderElementData.LightMapResolutionScale *= 0.5f;
	}

	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(LightmapDensityPassShaders.VertexShader, LightmapDensityPassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		LightmapDensityPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

bool FLightmapDensityMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& InMaterialRenderProxy,
	const FMaterial& InMaterial)
{
	// Determine the mesh's material and blend mode.
	const FMaterialRenderProxy* MaterialRenderProxy = &InMaterialRenderProxy;
	const FMaterial* Material = &InMaterial;
	const bool bMaterialMasked = Material->IsMasked();
	const bool bTranslucentBlendMode = IsTranslucentBlendMode(*Material);
	const bool bIsLitMaterial = Material->GetShadingModels().IsLit();
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(*Material, OverrideSettings);
	const FLightMapInteraction LightMapInteraction = (MeshBatch.LCI && bIsLitMaterial) ? MeshBatch.LCI->GetLightMapInteraction(FeatureLevel) : FLightMapInteraction();

	// Force simple lightmaps based on system settings.
	bool bAllowHighQualityLightMaps = AllowHighQualityLightmaps(FeatureLevel) && LightMapInteraction.AllowsHighQualityLightmaps();

	static const auto SupportLowQualityLightmapsVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
	const bool bAllowLowQualityLightMaps = (!SupportLowQualityLightmapsVar) || (SupportLowQualityLightmapsVar->GetValueOnAnyThread() != 0);

	if ((!bTranslucentBlendMode || ViewIfDynamicMeshCommand->Family->EngineShowFlags.Wireframe)
		&& ShouldIncludeMaterialInDefaultOpaquePass(*Material))
	{
		if (!bMaterialMasked && !Material->MaterialModifiesMeshPosition_RenderThread())
		{
			// Override with the default material for opaque materials that are not two sided
			MaterialRenderProxy = GEngine->LevelColorationLitMaterial->GetRenderProxy();
			// If LevelColorationLitMaterial happens to be compiling, use the fallback material and overwrite MaterialRenderProxy
			Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (!Material)
			{
				return false;
			}
		}

		if (!MaterialRenderProxy)
		{
			MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		}

		check(Material && MaterialRenderProxy);

		if (bIsLitMaterial && PrimitiveSceneProxy && (LightMapInteraction.GetType() == LMIT_Texture || (PrimitiveSceneProxy->IsStatic() && PrimitiveSceneProxy->GetLightMapResolution() > 0)))
		{
			// Should this object be texture lightmapped? Ie, is lighting not built for it?
			bool bUseDummyLightMapPolicy = MeshBatch.LCI == nullptr || MeshBatch.LCI->GetLightMapInteraction(FeatureLevel).GetType() != LMIT_Texture;

			// Use dummy if we don't support either lightmap quality.
			bUseDummyLightMapPolicy |= (!bAllowHighQualityLightMaps && !bAllowLowQualityLightMaps);
			if (!bUseDummyLightMapPolicy)
			{
				if (bAllowHighQualityLightMaps)
				{
					return Process<TUniformLightMapPolicy<LMP_HQ_LIGHTMAP>>(
						MeshBatch,
						BatchElementMask,
						PrimitiveSceneProxy,
						StaticMeshId,
						*MaterialRenderProxy,
						*Material,
						TUniformLightMapPolicy<LMP_HQ_LIGHTMAP>(),
						MeshBatch.LCI,
						MeshFillMode,
						MeshCullMode);
				}
				else
				{
					return Process<TUniformLightMapPolicy<LMP_LQ_LIGHTMAP>>(
						MeshBatch,
						BatchElementMask,
						PrimitiveSceneProxy,
						StaticMeshId,
						*MaterialRenderProxy,
						*Material,
						TUniformLightMapPolicy<LMP_LQ_LIGHTMAP>(),
						MeshBatch.LCI,
						MeshFillMode,
						MeshCullMode);
				}
			}
			else
			{
				return Process<TUniformLightMapPolicy<LMP_DUMMY>>(
					MeshBatch,
					BatchElementMask,
					PrimitiveSceneProxy,
					StaticMeshId,
					*MaterialRenderProxy,
					*Material,
					TUniformLightMapPolicy<LMP_DUMMY>(),
					MeshBatch.LCI,
					MeshFillMode,
					MeshCullMode);
			}
		}
		else
		{
			return Process<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>>(
				MeshBatch,
				BatchElementMask,
				PrimitiveSceneProxy,
				StaticMeshId,
				*MaterialRenderProxy,
				*Material,
				TUniformLightMapPolicy<LMP_NO_LIGHTMAP>(),
				MeshBatch.LCI,
				MeshFillMode,
				MeshCullMode);
		}
	}

	return true;
}

void FLightmapDensityMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (FeatureLevel >= ERHIFeatureLevel::SM5 && ViewIfDynamicMeshCommand->Family->EngineShowFlags.LightMapDensity && AllowDebugViewmodes() && MeshBatch.bUseForMaterial)
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material)
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

FLightmapDensityMeshProcessor::FLightmapDensityMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::LightmapDensity, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
{
	// Opaque blending, depth tests and writes.
	PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
}

FMeshPassProcessor* CreateLightmapDensityPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FLightmapDensityMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext);
}

FRegisterPassProcessorCreateFunction RegisterLightmapDensityPass(&CreateLightmapDensityPassProcessor, EShadingPath::Deferred, EMeshPass::LightmapDensity, EMeshPassFlags::MainView);