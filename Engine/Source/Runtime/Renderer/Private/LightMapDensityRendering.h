// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightMapDensityRendering.h: Definitions for rendering lightmap density.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "EngineGlobals.h"
#include "RHI.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HitProxies.h"
#include "ShaderParameters.h"
#include "RHIStaticStates.h"
#include "Shader.h"
#include "UnrealEngine.h"
#include "MeshMaterialShaderType.h"
#include "MeshMaterialShader.h"
#include "ShaderBaseClasses.h"
#include "SceneRendering.h"
#include "Engine/LightMapTexture2D.h"

template<typename LightMapPolicyType>
class TLightMapDensityElementData : public FMeshMaterialShaderElementData
{
public:
	TLightMapDensityElementData(const typename LightMapPolicyType::ElementDataType& InLightMapPolicyElementData) :
		LightMapPolicyElementData(InLightMapPolicyElementData)
	{}

	typename LightMapPolicyType::ElementDataType LightMapPolicyElementData;

	FVector3f BuiltLightingAndSelectedFlags;
	FVector2f LightMapResolutionScale; 
	bool bTextureMapped;
};

/**
 * The base shader type for vertex shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 */
template<typename LightMapPolicyType>
class TLightMapDensityVS : public FMeshMaterialShader, public LightMapPolicyType::VertexParametersType
{
	DECLARE_SHADER_TYPE_EXPLICIT_BASES(TLightMapDensityVS,MeshMaterial, FMeshMaterialShader, typename LightMapPolicyType::VertexParametersType);

public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return  AllowDebugViewmodes(Parameters.Platform) 
				&& IsStaticLightingAllowed()
				&& (Parameters.MaterialParameters.bIsSpecialEngineMaterial || Parameters.MaterialParameters.bIsMasked || Parameters.MaterialParameters.bMaterialMayModifyMeshPosition)
				&& LightMapPolicyType::ShouldCompilePermutation(Parameters)
				&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		LightMapPolicyType::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	/** Initialization constructor. */
	TLightMapDensityVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer):
		FMeshMaterialShader(Initializer)
	{
		LightMapPolicyType::VertexParametersType::Bind(Initializer.ParameterMap);
	}
	TLightMapDensityVS() {}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const TLightMapDensityElementData<LightMapPolicyType>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		LightMapPolicyType::GetVertexShaderBindings(
			PrimitiveSceneProxy,
			ShaderElementData.LightMapPolicyElementData,
			this,
			ShaderBindings);
	}
};

/**
 * The base type for pixel shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 * The base type is shared between the versions with and without sky light.
 */
template<typename LightMapPolicyType>
class TLightMapDensityPS : public FMeshMaterialShader, public LightMapPolicyType::PixelParametersType
{
	DECLARE_SHADER_TYPE_EXPLICIT_BASES(TLightMapDensityPS,MeshMaterial, FMeshMaterialShader, typename LightMapPolicyType::PixelParametersType);

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return	AllowDebugViewmodes(Parameters.Platform) 
				&& IsStaticLightingAllowed()
				&& (Parameters.MaterialParameters.bIsSpecialEngineMaterial || Parameters.MaterialParameters.bIsMasked || Parameters.MaterialParameters.bMaterialMayModifyMeshPosition)
				&& LightMapPolicyType::ShouldCompilePermutation(Parameters)
				&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		LightMapPolicyType::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	/** Initialization constructor. */
	TLightMapDensityPS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer):
		FMeshMaterialShader(Initializer)
	{
		LightMapPolicyType::PixelParametersType::Bind(Initializer.ParameterMap);
		BuiltLightingAndSelectedFlags.Bind(Initializer.ParameterMap,TEXT("BuiltLightingAndSelectedFlags"));
		LightMapResolutionScale.Bind(Initializer.ParameterMap,TEXT("LightMapResolutionScale"));
		LightMapDensityDisplayOptions.Bind(Initializer.ParameterMap,TEXT("LightMapDensityDisplayOptions"));
	}
	TLightMapDensityPS() {}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const TLightMapDensityElementData<LightMapPolicyType>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		LightMapPolicyType::GetPixelShaderBindings(
			PrimitiveSceneProxy,
			ShaderElementData.LightMapPolicyElementData,
			this,
			ShaderBindings);

		ShaderBindings.Add(BuiltLightingAndSelectedFlags, ShaderElementData.BuiltLightingAndSelectedFlags);
		ShaderBindings.Add(LightMapResolutionScale, ShaderElementData.LightMapResolutionScale);

		FVector4f OptionsParameter(
			GEngine->bRenderLightMapDensityGrayscale ? GEngine->RenderLightMapDensityGrayscaleScale : 0.0f,
			GEngine->bRenderLightMapDensityGrayscale ? 0.0f : GEngine->RenderLightMapDensityColorScale,
			(ShaderElementData.bTextureMapped == true) ? 1.0f : 0.0f,
			(ShaderElementData.bTextureMapped == false) ? 1.0f : 0.0f
			);
		ShaderBindings.Add(LightMapDensityDisplayOptions, OptionsParameter);
	}

private:
	LAYOUT_FIELD(FShaderParameter, BuiltLightingAndSelectedFlags);
	LAYOUT_FIELD(FShaderParameter, LightMapResolutionScale);
	LAYOUT_FIELD(FShaderParameter, LightMapDensityDisplayOptions);
};


class FLightmapDensityMeshProcessor : public FSceneRenderingAllocatorObject<FLightmapDensityMeshProcessor>, public FMeshPassProcessor
{
public:

	FLightmapDensityMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;


private:

	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	template<typename LightMapPolicyType>
	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		const LightMapPolicyType& RESTRICT LightMapPolicy,
		const typename LightMapPolicyType::ElementDataType& RESTRICT LightMapElementData,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	FMeshPassProcessorRenderState PassDrawRenderState;
};

void RenderLightMapDensities(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FRenderTargetBindingSlots& RenderTargets);