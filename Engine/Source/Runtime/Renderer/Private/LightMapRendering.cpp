// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LightMapRendering.cpp: Light map rendering implementations.
=============================================================================*/

#include "LightMapRendering.h"
#include "LightMap.h"
#include "PrecomputedVolumetricLightmap.h"
#include "RenderCore.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "MaterialShader.h"
#include "MeshMaterialShader.h"
#include "MeshDrawShaderBindings.h"
#include "ScenePrivate.h"

IMPLEMENT_TYPE_LAYOUT(FUniformLightMapPolicyShaderParametersType);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FIndirectLightingCacheUniformParameters, "IndirectLightingCache");

bool MobileUseCSMShaderBranch()
{
	static const auto* CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.UseCSMShaderBranch"));
	return (CVar && CVar->GetValueOnAnyThread() != 0);
}

// One of these per lightmap quality

const TCHAR* GLightmapDefineName[2] =
{
	TEXT("LQ_TEXTURE_LIGHTMAP"),
	TEXT("HQ_TEXTURE_LIGHTMAP")
};

int32 GNumLightmapCoefficients[2] = 
{
	NUM_LQ_LIGHTMAP_COEF,
	NUM_HQ_LIGHTMAP_COEF
};


void LightMapPolicyImpl::ModifyCompilationEnvironment(ELightmapQuality LightmapQuality, const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(GLightmapDefineName[LightmapQuality], TEXT("1"));
	OutEnvironment.SetDefine(TEXT("NUM_LIGHTMAP_COEFFICIENTS"), GNumLightmapCoefficients[LightmapQuality]);

	const bool VirtualTextureLightmaps = UseVirtualTextureLightmap(Parameters.Platform);
	OutEnvironment.SetDefine(TEXT("LIGHTMAP_VT_ENABLED"), VirtualTextureLightmaps);
}

bool LightMapPolicyImpl::ShouldCompilePermutation(ELightmapQuality LightmapQuality, const FMeshMaterialShaderPermutationParameters& Parameters)
{
	static const auto CVarProjectCanHaveLowQualityLightmaps = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
	static const auto CVarSupportAllShadersPermutations = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportAllShaderPermutations"));
	const bool bForceAllPermutations = CVarSupportAllShadersPermutations && CVarSupportAllShadersPermutations->GetValueOnAnyThread() != 0;

	// if GEngine doesn't exist yet to have the project flag then we should be conservative and cache the LQ lightmap policy
	const bool bProjectCanHaveLowQualityLightmaps = bForceAllPermutations || (!CVarProjectCanHaveLowQualityLightmaps) || (CVarProjectCanHaveLowQualityLightmaps->GetValueOnAnyThread() != 0);

	const bool bShouldCacheQuality = (LightmapQuality != ELightmapQuality::LQ_LIGHTMAP) || bProjectCanHaveLowQualityLightmaps;

	// GetValueOnAnyThread() as it's possible that ShouldCache is called from rendering thread. That is to output some error message.
	return (Parameters.MaterialParameters.ShadingModels.IsLit())
		&& bShouldCacheQuality
		&& Parameters.VertexFactoryType->SupportsStaticLighting()
		&& IsStaticLightingAllowed()
		&& (Parameters.MaterialParameters.bIsUsedWithStaticLighting || Parameters.MaterialParameters.bIsSpecialEngineMaterial);
}

void DistanceFieldShadowsAndLightMapPolicyImpl::ModifyCompilationEnvironment(ELightmapQuality LightmapQuality, const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("STATICLIGHTING_TEXTUREMASK"), 1);
	OutEnvironment.SetDefine(TEXT("STATICLIGHTING_SIGNEDDISTANCEFIELD"), 1);
	LightMapPolicyImpl::ModifyCompilationEnvironment(LightmapQuality, Parameters, OutEnvironment);
}

bool FDummyLightMapPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.ShadingModels.IsLit() && Parameters.VertexFactoryType->SupportsStaticLighting();
}

bool FSelfShadowedTranslucencyPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.ShadingModels.IsLit() &&
		IsTranslucentBlendMode(Parameters.MaterialParameters) &&
		IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
}

void FSelfShadowedTranslucencyPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("TRANSLUCENT_SELF_SHADOWING"), TEXT("1"));
}

FSelfShadowedTranslucencyPolicy::FSelfShadowedTranslucencyPolicy()
{
}

void FSelfShadowedTranslucencyPolicy::GetVertexShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const VertexParametersType* VertexShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
}

void FSelfShadowedTranslucencyPolicy::GetPixelShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const PixelParametersType* PixelShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
	ShaderBindings.Add(PixelShaderParameters->TranslucentSelfShadowBufferParameter, ShaderElementData);
}

void FSelfShadowedTranslucencyPolicy::GetComputeShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const ComputeParametersType* ComputeShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
	ShaderBindings.Add(ComputeShaderParameters->TranslucentSelfShadowBufferParameter, ShaderElementData);
}

bool FPrecomputedVolumetricLightmapLightingPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.ShadingModels.IsLit() && IsStaticLightingAllowed();
}

void FPrecomputedVolumetricLightmapLightingPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING"), TEXT("1"));
}

bool FCachedVolumeIndirectLightingPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.ShadingModels.IsLit()
		&& !IsTranslucentBlendMode(Parameters.MaterialParameters)
		&& IsStaticLightingAllowed()
		&& IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
}

void FCachedVolumeIndirectLightingPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("CACHED_VOLUME_INDIRECT_LIGHTING"), TEXT("1"));
}

bool FCachedPointIndirectLightingPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.ShadingModels.IsLit() && IsStaticLightingAllowed();
}

void FCachedPointIndirectLightingPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("CACHED_POINT_INDIRECT_LIGHTING"),TEXT("1"));	
}

void FMobileDirectionalLightAndCSMPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("DIRECTIONAL_LIGHT_CSM"), TEXT("1"));
}

bool FMobileDirectionalLightAndCSMPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	if (IsMobileDeferredShadingEnabled(Parameters.Platform) && 
		!(Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater) || Parameters.MaterialParameters.bIsTranslucentSurfaceLighting))
	{
		return false;
	}

	if (!IsStaticLightingAllowed() && MobileUseCSMShaderBranch())
	{
		// CSM will be handled by FNoLightMapPolicy with a branch inside shader
		return false;
	}
	
	return Parameters.MaterialParameters.ShadingModels.IsLit() &&
		(!IsTranslucentBlendMode(Parameters.MaterialParameters) || !!Parameters.MaterialParameters.bIsTranslucentSurfaceLighting) && // receive CSM when opaque or translucent surface lighting
		(!IsStaticLightingAllowed() || FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers());
}

bool FMobileDistanceFieldShadowsAndLQLightMapPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	static auto* CVarMobileAllowDistanceFieldShadows = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.AllowDistanceFieldShadows"));
	const bool bMobileAllowDistanceFieldShadows = CVarMobileAllowDistanceFieldShadows->GetValueOnAnyThread() == 1;
	
	return bMobileAllowDistanceFieldShadows &&
		(!IsTranslucentBlendMode(Parameters.MaterialParameters) || !!Parameters.MaterialParameters.bIsTranslucentSurfaceLighting) && // receive CSM when opaque or translucent surface lighting
		Super::ShouldCompilePermutation(Parameters);
}

void FMobileDistanceFieldShadowsAndLQLightMapPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

bool FMobileDistanceFieldShadowsLightMapAndCSMLightingPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	if (IsMobileDeferredShadingEnabled(Parameters.Platform))
	{
		return false;
	}

	return FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers() &&	Super::ShouldCompilePermutation(Parameters);
}

void FMobileDistanceFieldShadowsLightMapAndCSMLightingPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("DIRECTIONAL_LIGHT_CSM"), TEXT("1"));

	Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

bool FMobileDirectionalLightCSMAndLightMapPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	if (IsMobileDeferredShadingEnabled(Parameters.Platform) && 
		!Parameters.MaterialParameters.bIsTranslucentSurfaceLighting)
	{
		return false;
	}

	return (!IsTranslucentBlendMode(Parameters.MaterialParameters) || !!Parameters.MaterialParameters.bIsTranslucentSurfaceLighting) && // receive CSM when opaque or translucent surface lighting
		Super::ShouldCompilePermutation(Parameters);
}

void FMobileDirectionalLightCSMAndLightMapPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("DIRECTIONAL_LIGHT_CSM"), TEXT("1"));

	Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

bool FMobileDirectionalLightAndSHIndirectPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return IsStaticLightingAllowed() && Parameters.MaterialParameters.ShadingModels.IsLit() && FCachedPointIndirectLightingPolicy::ShouldCompilePermutation(Parameters);
}

void FMobileDirectionalLightAndSHIndirectPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FCachedPointIndirectLightingPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

bool FMobileDirectionalLightCSMAndSHIndirectPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	if (IsMobileDeferredShadingEnabled(Parameters.Platform) 
		&& !Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater))
	{
		return false;
	}

	return FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers() &&
		(!IsTranslucentBlendMode(Parameters.MaterialParameters) || !!Parameters.MaterialParameters.bIsTranslucentSurfaceLighting) && // receive CSM when opaque or translucent surface lighting
		Super::ShouldCompilePermutation(Parameters);
}

void FMobileDirectionalLightCSMAndSHIndirectPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("DIRECTIONAL_LIGHT_CSM"), TEXT("1"));

	Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

FMobileSingleLayerWaterPolicy::FMobileSingleLayerWaterPolicy()
{
}

bool FMobileSingleLayerWaterPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return IsMobilePlatform(Parameters.Platform) 
		&& !IsOpenGLPlatform(Parameters.Platform) // not enough samplers
		&& IsMobileDeferredShadingEnabled(Parameters.Platform)
		&& !FReadOnlyCVARCache::MobileAllowFramebufferFetch(Parameters.Platform)
		&& Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);
}

void FMobileSingleLayerWaterPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("DIRECTIONAL_LIGHT_CSM"), TEXT("1"));
	OutEnvironment.SetDefine(TEXT("MOBILE_SINGLELAYERWATER_FALLBACK"), 0u);
	OutEnvironment.SetDefine(TEXT("MOBILE_SSR_ENABLED"), 1u); // should it be optional?
}

void FMobileSingleLayerWaterPolicy::GetVertexShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const VertexParametersType* VertexShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
}

void FMobileSingleLayerWaterPolicy::GetPixelShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const PixelParametersType* PixelShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
}

bool FSelfShadowedCachedPointIndirectLightingPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.ShadingModels.IsLit()
		&& IsTranslucentBlendMode(Parameters.MaterialParameters)
		&& IsStaticLightingAllowed()
		&& FSelfShadowedTranslucencyPolicy::ShouldCompilePermutation(Parameters);
}

void FSelfShadowedCachedPointIndirectLightingPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("CACHED_POINT_INDIRECT_LIGHTING"),TEXT("1"));	
	FSelfShadowedTranslucencyPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

void SetupLCIUniformBuffers(const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FLightCacheInterface* LCI, FRHIUniformBuffer*& PrecomputedLightingBuffer, FRHIUniformBuffer*& LightmapResourceClusterBuffer, FRHIUniformBuffer*& IndirectLightingCacheBuffer)
{
	if (LCI)
	{
		PrecomputedLightingBuffer = LCI->GetPrecomputedLightingBuffer();
	}

	if (LCI && LCI->GetResourceCluster())
	{
		check(LCI->GetResourceCluster()->UniformBuffer);
		LightmapResourceClusterBuffer = LCI->GetResourceCluster()->UniformBuffer;
	}

	if (!PrecomputedLightingBuffer)
	{
		PrecomputedLightingBuffer = GEmptyPrecomputedLightingUniformBuffer.GetUniformBufferRHI();
	}

	if (!LightmapResourceClusterBuffer)
	{
		LightmapResourceClusterBuffer = GDefaultLightmapResourceClusterUniformBuffer.GetUniformBufferRHI();
	}

	if (PrimitiveSceneProxy && PrimitiveSceneProxy->GetPrimitiveSceneInfo())
	{
		IndirectLightingCacheBuffer = PrimitiveSceneProxy->GetPrimitiveSceneInfo()->IndirectLightingCacheUniformBuffer;
	}

	if (!IndirectLightingCacheBuffer)
	{
		IndirectLightingCacheBuffer = GEmptyIndirectLightingCacheUniformBuffer.GetUniformBufferRHI();
	}
}

void FSelfShadowedCachedPointIndirectLightingPolicy::GetPixelShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const PixelParametersType* PixelShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
	FRHIUniformBuffer* PrecomputedLightingBuffer = nullptr;
	FRHIUniformBuffer* LightmapResourceClusterBuffer = nullptr;
	FRHIUniformBuffer* IndirectLightingCacheBuffer = nullptr;

	SetupLCIUniformBuffers(PrimitiveSceneProxy, ShaderElementData.LCI, PrecomputedLightingBuffer, LightmapResourceClusterBuffer, IndirectLightingCacheBuffer);

	ShaderBindings.Add(PixelShaderParameters->PrecomputedLightingBufferParameter, PrecomputedLightingBuffer);
	ShaderBindings.Add(PixelShaderParameters->IndirectLightingCacheParameter, IndirectLightingCacheBuffer);
	ShaderBindings.Add(PixelShaderParameters->LightmapResourceCluster, LightmapResourceClusterBuffer);

	FSelfShadowedTranslucencyPolicy::GetPixelShaderBindings(PrimitiveSceneProxy, ShaderElementData.SelfShadowTranslucencyUniformBuffer, PixelShaderParameters, ShaderBindings);
}

void FSelfShadowedCachedPointIndirectLightingPolicy::GetComputeShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const ComputeParametersType* ComputeShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
	FRHIUniformBuffer* PrecomputedLightingBuffer = nullptr;
	FRHIUniformBuffer* LightmapResourceClusterBuffer = nullptr;
	FRHIUniformBuffer* IndirectLightingCacheBuffer = nullptr;

	SetupLCIUniformBuffers(PrimitiveSceneProxy, ShaderElementData.LCI, PrecomputedLightingBuffer, LightmapResourceClusterBuffer, IndirectLightingCacheBuffer);

	ShaderBindings.Add(ComputeShaderParameters->PrecomputedLightingBufferParameter, PrecomputedLightingBuffer);
	ShaderBindings.Add(ComputeShaderParameters->IndirectLightingCacheParameter, IndirectLightingCacheBuffer);
	ShaderBindings.Add(ComputeShaderParameters->LightmapResourceCluster, LightmapResourceClusterBuffer);

	FSelfShadowedTranslucencyPolicy::GetComputeShaderBindings(PrimitiveSceneProxy, ShaderElementData.SelfShadowTranslucencyUniformBuffer, ComputeShaderParameters, ShaderBindings);
}

bool FSelfShadowedVolumetricLightmapPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return Parameters.MaterialParameters.ShadingModels.IsLit()
		&& IsTranslucentBlendMode(Parameters.MaterialParameters)
		&& IsStaticLightingAllowed()
		&& FSelfShadowedTranslucencyPolicy::ShouldCompilePermutation(Parameters);
}

void FSelfShadowedVolumetricLightmapPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("PRECOMPUTED_IRRADIANCE_VOLUME_LIGHTING"), TEXT("1"));
	FSelfShadowedTranslucencyPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

/** Initialization constructor. */
FSelfShadowedVolumetricLightmapPolicy::FSelfShadowedVolumetricLightmapPolicy() {}

void FSelfShadowedVolumetricLightmapPolicy::GetPixelShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const PixelParametersType* PixelShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
	FRHIUniformBuffer* PrecomputedLightingBuffer = nullptr;
	FRHIUniformBuffer* LightmapResourceClusterBuffer = nullptr;
	FRHIUniformBuffer* IndirectLightingCacheBuffer = nullptr;

	SetupLCIUniformBuffers(PrimitiveSceneProxy, ShaderElementData.LCI, PrecomputedLightingBuffer, LightmapResourceClusterBuffer, IndirectLightingCacheBuffer);

	ShaderBindings.Add(PixelShaderParameters->PrecomputedLightingBufferParameter, PrecomputedLightingBuffer);
	ShaderBindings.Add(PixelShaderParameters->IndirectLightingCacheParameter, IndirectLightingCacheBuffer);
	ShaderBindings.Add(PixelShaderParameters->LightmapResourceCluster, LightmapResourceClusterBuffer);

	FSelfShadowedTranslucencyPolicy::GetPixelShaderBindings(PrimitiveSceneProxy, ShaderElementData.SelfShadowTranslucencyUniformBuffer, PixelShaderParameters, ShaderBindings);
}

void FSelfShadowedVolumetricLightmapPolicy::GetComputeShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const ComputeParametersType* ComputeShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
	FRHIUniformBuffer* PrecomputedLightingBuffer = nullptr;
	FRHIUniformBuffer* LightmapResourceClusterBuffer = nullptr;
	FRHIUniformBuffer* IndirectLightingCacheBuffer = nullptr;

	SetupLCIUniformBuffers(PrimitiveSceneProxy, ShaderElementData.LCI, PrecomputedLightingBuffer, LightmapResourceClusterBuffer, IndirectLightingCacheBuffer);

	ShaderBindings.Add(ComputeShaderParameters->PrecomputedLightingBufferParameter, PrecomputedLightingBuffer);
	ShaderBindings.Add(ComputeShaderParameters->IndirectLightingCacheParameter, IndirectLightingCacheBuffer);
	ShaderBindings.Add(ComputeShaderParameters->LightmapResourceCluster, LightmapResourceClusterBuffer);

	FSelfShadowedTranslucencyPolicy::GetComputeShaderBindings(PrimitiveSceneProxy, ShaderElementData.SelfShadowTranslucencyUniformBuffer, ComputeShaderParameters, ShaderBindings);
}

bool FUniformLightMapPolicy::ShouldCompilePermutation(ELightMapPolicyType Policy, const FMeshMaterialShaderPermutationParameters& Parameters)
{
	CA_SUPPRESS(6326);
	switch (Policy)
	{
	case LMP_NO_LIGHTMAP:
		return FNoLightMapPolicy::ShouldCompilePermutation(Parameters);
	case LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING:
		return FPrecomputedVolumetricLightmapLightingPolicy::ShouldCompilePermutation(Parameters);
	case LMP_CACHED_VOLUME_INDIRECT_LIGHTING:
		return FCachedVolumeIndirectLightingPolicy::ShouldCompilePermutation(Parameters);
	case LMP_CACHED_POINT_INDIRECT_LIGHTING:
		return FCachedPointIndirectLightingPolicy::ShouldCompilePermutation(Parameters);
	case LMP_LQ_LIGHTMAP:
		return TLightMapPolicy<LQ_LIGHTMAP>::ShouldCompilePermutation(Parameters);
	case LMP_HQ_LIGHTMAP:
		return TLightMapPolicy<HQ_LIGHTMAP>::ShouldCompilePermutation(Parameters);
	case LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP:
		return TDistanceFieldShadowsAndLightMapPolicy<HQ_LIGHTMAP>::ShouldCompilePermutation(Parameters);

		// Mobile specific
	case LMP_MOBILE_DISTANCE_FIELD_SHADOWS_AND_LQ_LIGHTMAP:
		return FMobileDistanceFieldShadowsAndLQLightMapPolicy::ShouldCompilePermutation(Parameters);
	case LMP_MOBILE_DISTANCE_FIELD_SHADOWS_LIGHTMAP_AND_CSM:
		return FMobileDistanceFieldShadowsLightMapAndCSMLightingPolicy::ShouldCompilePermutation(Parameters);
	case LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_LIGHTMAP:
		return FMobileDirectionalLightCSMAndLightMapPolicy::ShouldCompilePermutation(Parameters);
	case LMP_MOBILE_DIRECTIONAL_LIGHT_AND_SH_INDIRECT:
		return FMobileDirectionalLightAndSHIndirectPolicy::ShouldCompilePermutation(Parameters);
	case LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_SH_INDIRECT:
		return FMobileDirectionalLightCSMAndSHIndirectPolicy::ShouldCompilePermutation(Parameters);
	case LMP_MOBILE_DIRECTIONAL_LIGHT_CSM:
		return FMobileDirectionalLightAndCSMPolicy::ShouldCompilePermutation(Parameters);

		// LightMapDensity

	case LMP_DUMMY:
		return FDummyLightMapPolicy::ShouldCompilePermutation(Parameters);

	default:
		check(false);
		return false;
	};
}

void FUniformLightMapPolicy::ModifyCompilationEnvironment(ELightMapPolicyType Policy, const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("MAX_NUM_LIGHTMAP_COEF"), MAX_NUM_LIGHTMAP_COEF);

	CA_SUPPRESS(6326);
	switch (Policy)
	{
	case LMP_NO_LIGHTMAP:
		FNoLightMapPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING:
		FPrecomputedVolumetricLightmapLightingPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_CACHED_VOLUME_INDIRECT_LIGHTING:
		FCachedVolumeIndirectLightingPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_CACHED_POINT_INDIRECT_LIGHTING:
		FCachedPointIndirectLightingPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_LQ_LIGHTMAP:
		TLightMapPolicy<LQ_LIGHTMAP>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_HQ_LIGHTMAP:
		TLightMapPolicy<HQ_LIGHTMAP>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP:
		TDistanceFieldShadowsAndLightMapPolicy<HQ_LIGHTMAP>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;

		// Mobile specific
	case LMP_MOBILE_DISTANCE_FIELD_SHADOWS_AND_LQ_LIGHTMAP:
		FMobileDistanceFieldShadowsAndLQLightMapPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_MOBILE_DISTANCE_FIELD_SHADOWS_LIGHTMAP_AND_CSM:
		FMobileDistanceFieldShadowsLightMapAndCSMLightingPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_LIGHTMAP:
		FMobileDirectionalLightCSMAndLightMapPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_MOBILE_DIRECTIONAL_LIGHT_AND_SH_INDIRECT:
		FMobileDirectionalLightAndSHIndirectPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_SH_INDIRECT:
		FMobileDirectionalLightCSMAndSHIndirectPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;
	case LMP_MOBILE_DIRECTIONAL_LIGHT_CSM:
		FMobileDirectionalLightAndCSMPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;

		// LightMapDensity

	case LMP_DUMMY:
		FDummyLightMapPolicy::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		break;

	default:
		check(false);
		break;
	}
}

void FUniformLightMapPolicy::GetVertexShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const VertexParametersType* VertexShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings) 
{
	FRHIUniformBuffer* PrecomputedLightingBuffer = nullptr;
	FRHIUniformBuffer* LightmapResourceClusterBuffer = nullptr;
	FRHIUniformBuffer* IndirectLightingCacheBuffer = nullptr;

	SetupLCIUniformBuffers(PrimitiveSceneProxy, ShaderElementData, PrecomputedLightingBuffer, LightmapResourceClusterBuffer, IndirectLightingCacheBuffer);

	ShaderBindings.Add(VertexShaderParameters->PrecomputedLightingBufferParameter, PrecomputedLightingBuffer);
	ShaderBindings.Add(VertexShaderParameters->IndirectLightingCacheParameter, IndirectLightingCacheBuffer);
	ShaderBindings.Add(VertexShaderParameters->LightmapResourceCluster, LightmapResourceClusterBuffer);
}

void FUniformLightMapPolicy::GetPixelShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const PixelParametersType* PixelShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
	FRHIUniformBuffer* PrecomputedLightingBuffer = nullptr;
	FRHIUniformBuffer* LightmapResourceClusterBuffer = nullptr;
	FRHIUniformBuffer* IndirectLightingCacheBuffer = nullptr;

	SetupLCIUniformBuffers(PrimitiveSceneProxy, ShaderElementData, PrecomputedLightingBuffer, LightmapResourceClusterBuffer, IndirectLightingCacheBuffer);

	ShaderBindings.Add(PixelShaderParameters->PrecomputedLightingBufferParameter, PrecomputedLightingBuffer);
	ShaderBindings.Add(PixelShaderParameters->IndirectLightingCacheParameter, IndirectLightingCacheBuffer);
	ShaderBindings.Add(PixelShaderParameters->LightmapResourceCluster, LightmapResourceClusterBuffer);
}

#if RHI_RAYTRACING
void FUniformLightMapPolicy::GetRayHitGroupShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FLightCacheInterface* LCI,
	const RayHitGroupParametersType* RayHitGroupShaderParameters,
	FMeshDrawSingleShaderBindings& RayHitGroupBindings
) const
{
	FRHIUniformBuffer* PrecomputedLightingBuffer = nullptr;
	FRHIUniformBuffer* LightmapResourceClusterBuffer = nullptr;
	FRHIUniformBuffer* IndirectLightingCacheBuffer = nullptr;

	SetupLCIUniformBuffers(PrimitiveSceneProxy, LCI, PrecomputedLightingBuffer, LightmapResourceClusterBuffer, IndirectLightingCacheBuffer);

	RayHitGroupBindings.Add(RayHitGroupShaderParameters->PrecomputedLightingBufferParameter, PrecomputedLightingBuffer);
	RayHitGroupBindings.Add(RayHitGroupShaderParameters->IndirectLightingCacheParameter, IndirectLightingCacheBuffer);
	RayHitGroupBindings.Add(RayHitGroupShaderParameters->LightmapResourceCluster, LightmapResourceClusterBuffer);
}
#endif

void FUniformLightMapPolicy::GetComputeShaderBindings(
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const ElementDataType& ShaderElementData,
	const ComputeParametersType* ComputeShaderParameters,
	FMeshDrawSingleShaderBindings& ShaderBindings)
{
	FRHIUniformBuffer* PrecomputedLightingBuffer = nullptr;
	FRHIUniformBuffer* LightmapResourceClusterBuffer = nullptr;
	FRHIUniformBuffer* IndirectLightingCacheBuffer = nullptr;

	SetupLCIUniformBuffers(PrimitiveSceneProxy, ShaderElementData, PrecomputedLightingBuffer, LightmapResourceClusterBuffer, IndirectLightingCacheBuffer);

	ShaderBindings.Add(ComputeShaderParameters->PrecomputedLightingBufferParameter, PrecomputedLightingBuffer);
	ShaderBindings.Add(ComputeShaderParameters->IndirectLightingCacheParameter, IndirectLightingCacheBuffer);
	ShaderBindings.Add(ComputeShaderParameters->LightmapResourceCluster, LightmapResourceClusterBuffer);
}

void InterpolateVolumetricLightmap(
	FVector LookupPosition,
	const FVolumetricLightmapSceneData& VolumetricLightmapSceneData,
	FVolumetricLightmapInterpolation& OutInterpolation)
{
	SCOPE_CYCLE_COUNTER(STAT_InterpolateVolumetricLightmapOnCPU);

	checkSlow(VolumetricLightmapSceneData.HasData());
	const FPrecomputedVolumetricLightmapData& GlobalVolumetricLightmapData = *VolumetricLightmapSceneData.GetLevelVolumetricLightmap()->Data;
	
	const FVector IndirectionDataSourceCoordinate = ComputeIndirectionCoordinate(LookupPosition, GlobalVolumetricLightmapData.GetBounds(), GlobalVolumetricLightmapData.IndirectionTextureDimensions);

	check(GlobalVolumetricLightmapData.IndirectionTexture.Data.Num() > 0);
	checkSlow(GPixelFormats[GlobalVolumetricLightmapData.IndirectionTexture.Format].BlockBytes == sizeof(uint8) * 4);
	const int32 NumIndirectionTexels = GlobalVolumetricLightmapData.IndirectionTextureDimensions.X * GlobalVolumetricLightmapData.IndirectionTextureDimensions.Y * GlobalVolumetricLightmapData.IndirectionTextureDimensions.Z;
	check(GlobalVolumetricLightmapData.IndirectionTexture.Data.Num() * GlobalVolumetricLightmapData.IndirectionTexture.Data.GetTypeSize() == NumIndirectionTexels * sizeof(uint8) * 4);
	
	FIntVector IndirectionBrickOffset;
	int32 IndirectionBrickSize;
	int32 SubLevelIndex;
	SampleIndirectionTextureWithSubLevel(IndirectionDataSourceCoordinate, GlobalVolumetricLightmapData.IndirectionTextureDimensions, GlobalVolumetricLightmapData.IndirectionTexture.Data.GetData(), GlobalVolumetricLightmapData.CPUSubLevelIndirectionTable, IndirectionBrickOffset, IndirectionBrickSize, SubLevelIndex);

	const FPrecomputedVolumetricLightmapData& VolumetricLightmapData = *GlobalVolumetricLightmapData.CPUSubLevelBrickDataList[SubLevelIndex];

	const FVector BrickTextureCoordinate = ComputeBrickTextureCoordinate(IndirectionDataSourceCoordinate, IndirectionBrickOffset, IndirectionBrickSize, VolumetricLightmapData.BrickSize);

	const FVector AmbientVector = (FVector)FilteredVolumeLookup<FFloat3Packed>(BrickTextureCoordinate, VolumetricLightmapData.BrickDataDimensions, (const FFloat3Packed*)VolumetricLightmapData.BrickData.AmbientVector.Data.GetData());
	
	auto ReadSHCoefficient = [&BrickTextureCoordinate, &VolumetricLightmapData, &AmbientVector](uint32 CoefficientIndex)
	{
		check(CoefficientIndex < UE_ARRAY_COUNT(VolumetricLightmapData.BrickData.SHCoefficients));

		// Undo normalization done in FIrradianceBrickData::SetFromVolumeLightingSample
		const FLinearColor SHDenormalizationScales0(
			0.488603f / 0.282095f,
			0.488603f / 0.282095f,
			0.488603f / 0.282095f,
			1.092548f / 0.282095f);

		const FLinearColor SHDenormalizationScales1(
			1.092548f / 0.282095f,
			4.0f * 0.315392f / 0.282095f,
			1.092548f / 0.282095f,
			2.0f * 0.546274f / 0.282095f);

		FLinearColor SHCoefficientEncoded = FilteredVolumeLookup<FColor>(BrickTextureCoordinate, VolumetricLightmapData.BrickDataDimensions, (const FColor*)VolumetricLightmapData.BrickData.SHCoefficients[CoefficientIndex].Data.GetData());
		//Swap R and B channel because it was swapped at ImportVolumetricLightmap for changing format from BGRA to RGBA
		Swap(SHCoefficientEncoded.R, SHCoefficientEncoded.B);

		const FLinearColor& DenormalizationScales = ((CoefficientIndex & 1) == 0) ? SHDenormalizationScales0 : SHDenormalizationScales1;
		return FVector4f((SHCoefficientEncoded * 2.0f - FLinearColor(1.0f, 1.0f, 1.0f, 1.0f)) * AmbientVector[CoefficientIndex / 2] * DenormalizationScales);
	};

	auto GetSHVector3 = [](float Ambient, const FVector4f& Coeffs0, const FVector4f& Coeffs1 )
	{
		FSHVector3 Result;
		Result.V[0] = Ambient;
		FMemory::Memcpy(&Result.V[1], &Coeffs0, sizeof(Coeffs0));
		FMemory::Memcpy(&Result.V[5], &Coeffs1, sizeof(Coeffs1));
		return Result;
	};

	FSHVectorRGB3 LQSH;
	LQSH.R = GetSHVector3(AmbientVector.X, ReadSHCoefficient(0), ReadSHCoefficient(1));
	LQSH.G = GetSHVector3(AmbientVector.Y, ReadSHCoefficient(2), ReadSHCoefficient(3));
	LQSH.B = GetSHVector3(AmbientVector.Z, ReadSHCoefficient(4), ReadSHCoefficient(5));

	// Pack the 3rd order SH as the shader expects
	OutInterpolation.IndirectLightingSHCoefficients0[0] = FVector4f(LQSH.R.V[0], LQSH.R.V[1], LQSH.R.V[2], LQSH.R.V[3]) * INV_PI;
	OutInterpolation.IndirectLightingSHCoefficients0[1] = FVector4f(LQSH.G.V[0], LQSH.G.V[1], LQSH.G.V[2], LQSH.G.V[3]) * INV_PI;
	OutInterpolation.IndirectLightingSHCoefficients0[2] = FVector4f(LQSH.B.V[0], LQSH.B.V[1], LQSH.B.V[2], LQSH.B.V[3]) * INV_PI;
	OutInterpolation.IndirectLightingSHCoefficients1[0] = FVector4f(LQSH.R.V[4], LQSH.R.V[5], LQSH.R.V[6], LQSH.R.V[7]) * INV_PI;
	OutInterpolation.IndirectLightingSHCoefficients1[1] = FVector4f(LQSH.G.V[4], LQSH.G.V[5], LQSH.G.V[6], LQSH.G.V[7]) * INV_PI;
	OutInterpolation.IndirectLightingSHCoefficients1[2] = FVector4f(LQSH.B.V[4], LQSH.B.V[5], LQSH.B.V[6], LQSH.B.V[7]) * INV_PI;
	OutInterpolation.IndirectLightingSHCoefficients2 = FVector4f(LQSH.R.V[8], LQSH.G.V[8], LQSH.B.V[8], 0.0f) * INV_PI;

	//The IndirectLightingSHSingleCoefficient should be DotSH1(AmbientVector, CalcDiffuseTransferSH1(1)) / PI, and CalcDiffuseTransferSH1(1) / PI equals to 1 / (2 * sqrt(PI)) and FSHVector2::ConstantBasisIntegral is 2 * sqrt(PI)
	OutInterpolation.IndirectLightingSHSingleCoefficient = FVector4f(AmbientVector.X, AmbientVector.Y, AmbientVector.Z) / FSHVector2::ConstantBasisIntegral;

	if (VolumetricLightmapData.BrickData.SkyBentNormal.Data.Num() > 0)
	{
		const FLinearColor SkyBentNormalUnpacked = FilteredVolumeLookup<FColor>(BrickTextureCoordinate, VolumetricLightmapData.BrickDataDimensions, (const FColor*)VolumetricLightmapData.BrickData.SkyBentNormal.Data.GetData());
		const FVector SkyBentNormal(SkyBentNormalUnpacked.R, SkyBentNormalUnpacked.G, SkyBentNormalUnpacked.B);
		const float BentNormalLength = SkyBentNormal.Size();
		OutInterpolation.PointSkyBentNormal = FVector4f((FVector3f)SkyBentNormal / FMath::Max(BentNormalLength, .0001f), BentNormalLength);
		//Swap X and Z channel because it was swapped at ImportVolumetricLightmap for changing format from BGRA to RGBA
		Swap(OutInterpolation.PointSkyBentNormal.X, OutInterpolation.PointSkyBentNormal.Z);
	}
	else
	{
		OutInterpolation.PointSkyBentNormal = FVector4f(0, 0, 1, 1);
	}

	const FLinearColor DirectionalLightShadowingUnpacked = FilteredVolumeLookup<uint8>(BrickTextureCoordinate, VolumetricLightmapData.BrickDataDimensions, (const uint8*)VolumetricLightmapData.BrickData.DirectionalLightShadowing.Data.GetData());
	OutInterpolation.DirectionalLightShadowing = DirectionalLightShadowingUnpacked.R;
}

void FEmptyPrecomputedLightingUniformBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	FPrecomputedLightingUniformParameters Parameters;
	GetPrecomputedLightingParameters(GMaxRHIFeatureLevel, Parameters, NULL);
	SetContentsNoUpdate(Parameters);

	Super::InitRHI(RHICmdList);
}

/** Global uniform buffer containing the default precomputed lighting data. */
TGlobalResource< FEmptyPrecomputedLightingUniformBuffer > GEmptyPrecomputedLightingUniformBuffer;

void GetIndirectLightingCacheParameters(
	ERHIFeatureLevel::Type FeatureLevel,
	FIndirectLightingCacheUniformParameters& Parameters, 
	const FIndirectLightingCache* LightingCache, 
	const FIndirectLightingCacheAllocation* LightingAllocation, 
	FVector VolumetricLightmapLookupPosition,
	uint32 SceneFrameNumber,
	FVolumetricLightmapSceneData* VolumetricLightmapSceneData
	)
{
	// FCachedVolumeIndirectLightingPolicy, FCachedPointIndirectLightingPolicy
	{
		if (VolumetricLightmapSceneData && VolumetricLightmapSceneData->HasData())
		{
			FVolumetricLightmapInterpolation* Interpolation = VolumetricLightmapSceneData->CPUInterpolationCache.Find(VolumetricLightmapLookupPosition);

			if (!Interpolation)
			{
				Interpolation = &VolumetricLightmapSceneData->CPUInterpolationCache.Add(VolumetricLightmapLookupPosition);
				InterpolateVolumetricLightmap(VolumetricLightmapLookupPosition, *VolumetricLightmapSceneData, *Interpolation);
			}

			Interpolation->LastUsedSceneFrameNumber = SceneFrameNumber;
			
			Parameters.PointSkyBentNormal = Interpolation->PointSkyBentNormal;
			Parameters.DirectionalLightShadowing = Interpolation->DirectionalLightShadowing;

			for (int32 i = 0; i < 3; i++)
			{
				Parameters.IndirectLightingSHCoefficients0[i] = Interpolation->IndirectLightingSHCoefficients0[i];
				Parameters.IndirectLightingSHCoefficients1[i] = Interpolation->IndirectLightingSHCoefficients1[i];
			}

			Parameters.IndirectLightingSHCoefficients2 = Interpolation->IndirectLightingSHCoefficients2;
			Parameters.IndirectLightingSHSingleCoefficient = Interpolation->IndirectLightingSHSingleCoefficient;

			// Unused
			Parameters.IndirectLightingCachePrimitiveAdd = FVector3f::ZeroVector;
			Parameters.IndirectLightingCachePrimitiveScale = FVector3f::OneVector;
			Parameters.IndirectLightingCacheMinUV = FVector3f::ZeroVector;
			Parameters.IndirectLightingCacheMaxUV = FVector3f::OneVector;
		}
		else if (LightingAllocation)
		{
			Parameters.IndirectLightingCachePrimitiveAdd = (FVector3f)LightingAllocation->Add;
			Parameters.IndirectLightingCachePrimitiveScale = (FVector3f)LightingAllocation->Scale;
			Parameters.IndirectLightingCacheMinUV = (FVector3f)LightingAllocation->MinUV;
			Parameters.IndirectLightingCacheMaxUV = (FVector3f)LightingAllocation->MaxUV;
			Parameters.PointSkyBentNormal = LightingAllocation->CurrentSkyBentNormal;
			Parameters.DirectionalLightShadowing = LightingAllocation->CurrentDirectionalShadowing;

			for (uint32 i = 0; i < 3; ++i) // RGB
			{
				Parameters.IndirectLightingSHCoefficients0[i] = LightingAllocation->SingleSamplePacked0[i];
				Parameters.IndirectLightingSHCoefficients1[i] = LightingAllocation->SingleSamplePacked1[i];
			}
			Parameters.IndirectLightingSHCoefficients2 = LightingAllocation->SingleSamplePacked2;
			//The IndirectLightingSHSingleCoefficient should be DotSH1(LightingAllocation->SingleSamplePacked0[0], CalcDiffuseTransferSH1(1)) / PI, and CalcDiffuseTransferSH1(1) / PI equals to 1 / (2 * sqrt(PI)) and FSHVector2::ConstantBasisIntegral is 2 * sqrt(PI)
			Parameters.IndirectLightingSHSingleCoefficient = FVector4f(LightingAllocation->SingleSamplePacked0[0].X, LightingAllocation->SingleSamplePacked0[1].X, LightingAllocation->SingleSamplePacked0[2].X) / FSHVector2::ConstantBasisIntegral;
		}
		else
		{
			Parameters.IndirectLightingCachePrimitiveAdd = FVector3f::ZeroVector;
			Parameters.IndirectLightingCachePrimitiveScale = FVector3f::OneVector;
			Parameters.IndirectLightingCacheMinUV = FVector3f::ZeroVector;
			Parameters.IndirectLightingCacheMaxUV = FVector3f::OneVector;
			Parameters.PointSkyBentNormal = FVector4f(0, 0, 1, 1);
			Parameters.DirectionalLightShadowing = 1;

			for (uint32 i = 0; i < 3; ++i) // RGB
			{
				Parameters.IndirectLightingSHCoefficients0[i] = FVector4f(0, 0, 0, 0);
				Parameters.IndirectLightingSHCoefficients1[i] = FVector4f(0, 0, 0, 0);
			}
			Parameters.IndirectLightingSHCoefficients2 = FVector4f(0, 0, 0, 0);
			Parameters.IndirectLightingSHSingleCoefficient = FVector4f(0, 0, 0, 0);
		}

		// If we are using FCachedVolumeIndirectLightingPolicy then InitViews should have updated the lighting cache which would have initialized it
		// However the conditions for updating the lighting cache are complex and fail very occasionally in non-reproducible ways
		// Silently skipping setting the cache texture under failure for now
		if (FeatureLevel >= ERHIFeatureLevel::SM5 && LightingCache && LightingCache->IsInitialized() && RHISupportsVolumeTextureRendering(GetFeatureLevelShaderPlatform(FeatureLevel)))
		{
			Parameters.IndirectLightingCacheTexture0 = const_cast<FIndirectLightingCache*>(LightingCache)->GetTexture0();
			Parameters.IndirectLightingCacheTexture1 = const_cast<FIndirectLightingCache*>(LightingCache)->GetTexture1();
			Parameters.IndirectLightingCacheTexture2 = const_cast<FIndirectLightingCache*>(LightingCache)->GetTexture2();

			Parameters.IndirectLightingCacheTextureSampler0 = TStaticSamplerState<SF_Bilinear,AM_Clamp,AM_Clamp,AM_Clamp>::GetRHI();
			Parameters.IndirectLightingCacheTextureSampler1 = TStaticSamplerState<SF_Bilinear,AM_Clamp,AM_Clamp,AM_Clamp>::GetRHI();
			Parameters.IndirectLightingCacheTextureSampler2 = TStaticSamplerState<SF_Bilinear,AM_Clamp,AM_Clamp,AM_Clamp>::GetRHI();
		}
		else
		{
			Parameters.IndirectLightingCacheTexture0 = GBlackVolumeTexture->TextureRHI;
			Parameters.IndirectLightingCacheTexture1 = GBlackVolumeTexture->TextureRHI;
			Parameters.IndirectLightingCacheTexture2 = GBlackVolumeTexture->TextureRHI;

			Parameters.IndirectLightingCacheTextureSampler0 = GBlackVolumeTexture->SamplerStateRHI;
			Parameters.IndirectLightingCacheTextureSampler1 = GBlackVolumeTexture->SamplerStateRHI;
			Parameters.IndirectLightingCacheTextureSampler2 = GBlackVolumeTexture->SamplerStateRHI;
		}
	}
}

void FEmptyIndirectLightingCacheUniformBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	FIndirectLightingCacheUniformParameters Parameters;
	GetIndirectLightingCacheParameters(GMaxRHIFeatureLevel, Parameters, nullptr, nullptr, FVector(0, 0, 0), 0, nullptr);
	SetContentsNoUpdate(Parameters);

	Super::InitRHI(RHICmdList);
}

/** */
TGlobalResource< FEmptyIndirectLightingCacheUniformBuffer > GEmptyIndirectLightingCacheUniformBuffer;

bool FNoLightMapPolicy::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return true;
}

void FNoLightMapPolicy::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	if (IsMobilePlatform(Parameters.Platform) && !IsStaticLightingAllowed() && MobileUseCSMShaderBranch())
	{
		OutEnvironment.SetDefine(TEXT("DIRECTIONAL_LIGHT_CSM"), TEXT("1"));
		OutEnvironment.SetDefine(TEXT("MOBILE_USE_CSM_BRANCH"), TEXT("1"));
	}
}
