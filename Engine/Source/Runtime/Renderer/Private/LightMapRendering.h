// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightMapRendering.h: Light map rendering definitions.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"
#include "LightmapUniformShaderParameters.h"
#include "Math/Vector.h"
#include "ShaderParameterMacros.h"
#include "UniformBuffer.h"

class FMeshDrawSingleShaderBindings;
class FPrimitiveSceneProxy;
struct FMaterialShaderPermutationParameters;
struct FMeshMaterialShaderPermutationParameters;

RENDERER_API bool MobileUsesNoLightMapPermutation(const FMeshMaterialShaderPermutationParameters& Parameters);

extern bool MobileUseCSMShaderBranch();

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FIndirectLightingCacheUniformParameters, )
	SHADER_PARAMETER(FVector3f, IndirectLightingCachePrimitiveAdd) // FCachedVolumeIndirectLightingPolicy
	SHADER_PARAMETER(FVector3f, IndirectLightingCachePrimitiveScale) // FCachedVolumeIndirectLightingPolicy
	SHADER_PARAMETER(FVector3f, IndirectLightingCacheMinUV) // FCachedVolumeIndirectLightingPolicy
	SHADER_PARAMETER(FVector3f, IndirectLightingCacheMaxUV) // FCachedVolumeIndirectLightingPolicy
	SHADER_PARAMETER(FVector4f, PointSkyBentNormal) // FCachedPointIndirectLightingPolicy
	SHADER_PARAMETER_EX(float, DirectionalLightShadowing, EShaderPrecisionModifier::Half) // FCachedPointIndirectLightingPolicy
	SHADER_PARAMETER_ARRAY(FVector4f, IndirectLightingSHCoefficients0, [3]) // FCachedPointIndirectLightingPolicy
	SHADER_PARAMETER_ARRAY(FVector4f, IndirectLightingSHCoefficients1, [3]) // FCachedPointIndirectLightingPolicy
	SHADER_PARAMETER(FVector4f,	IndirectLightingSHCoefficients2) // FCachedPointIndirectLightingPolicy
	SHADER_PARAMETER_EX(FVector4f, IndirectLightingSHSingleCoefficient, EShaderPrecisionModifier::Half) // FCachedPointIndirectLightingPolicy used in forward Translucent
	SHADER_PARAMETER_TEXTURE(Texture3D, IndirectLightingCacheTexture0) // FCachedVolumeIndirectLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, IndirectLightingCacheTexture1) // FCachedVolumeIndirectLightingPolicy
	SHADER_PARAMETER_TEXTURE(Texture3D, IndirectLightingCacheTexture2) // FCachedVolumeIndirectLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, IndirectLightingCacheTextureSampler0) // FCachedVolumeIndirectLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, IndirectLightingCacheTextureSampler1) // FCachedVolumeIndirectLightingPolicy
	SHADER_PARAMETER_SAMPLER(SamplerState, IndirectLightingCacheTextureSampler2) // FCachedVolumeIndirectLightingPolicy
END_GLOBAL_SHADER_PARAMETER_STRUCT()

/**
 * Default precomputed lighting data. Used for fully dynamic lightmap policies.
 */
class FEmptyPrecomputedLightingUniformBuffer : public TUniformBuffer< FPrecomputedLightingUniformParameters >
{
	typedef TUniformBuffer< FPrecomputedLightingUniformParameters > Super;
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

/** Global uniform buffer containing the default precomputed lighting data. */
extern RENDERER_API TGlobalResource< FEmptyPrecomputedLightingUniformBuffer > GEmptyPrecomputedLightingUniformBuffer;

void GetIndirectLightingCacheParameters(
	ERHIFeatureLevel::Type FeatureLevel,
	FIndirectLightingCacheUniformParameters& Parameters,
	const class FIndirectLightingCache* LightingCache,
	const class FIndirectLightingCacheAllocation* LightingAllocation,
	FVector VolumetricLightmapLookupPosition,
	uint32 SceneFrameNumber,
	class FVolumetricLightmapSceneData* VolumetricLightmapSceneData);

/**
 * Default precomputed lighting data. Used for fully dynamic lightmap policies.
 */
class FEmptyIndirectLightingCacheUniformBuffer : public TUniformBuffer< FIndirectLightingCacheUniformParameters >
{
	typedef TUniformBuffer< FIndirectLightingCacheUniformParameters > Super;
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

/** Global uniform buffer containing the default precomputed lighting data. */
extern TGlobalResource< FEmptyIndirectLightingCacheUniformBuffer > GEmptyIndirectLightingCacheUniformBuffer;

/**
 * A policy for shaders without a lightmap.
 */
struct FNoLightMapPolicy
{
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

enum ELightmapQuality
{
	LQ_LIGHTMAP,
	HQ_LIGHTMAP,
};

namespace LightMapPolicyImpl
{
	void ModifyCompilationEnvironment(ELightmapQuality LightmapQuality, const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	bool ShouldCompilePermutation(ELightmapQuality LightmapQuality, const FMeshMaterialShaderPermutationParameters& Parameters);
};

/**
 * Base policy for shaders with lightmaps.
 */
template<ELightmapQuality LightmapQuality>
struct TLightMapPolicy
{
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		LightMapPolicyImpl::ModifyCompilationEnvironment(LightmapQuality, Parameters, OutEnvironment);
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return LightMapPolicyImpl::ShouldCompilePermutation(LightmapQuality, Parameters);
	}
};

namespace DistanceFieldShadowsAndLightMapPolicyImpl
{
	void ModifyCompilationEnvironment(ELightmapQuality LightmapQuality, const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

// A lightmap policy for computing up to 4 signed distance field shadow factors in the base pass.
template< ELightmapQuality LightmapQuality >
struct TDistanceFieldShadowsAndLightMapPolicy : public TLightMapPolicy< LightmapQuality >
{
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		DistanceFieldShadowsAndLightMapPolicyImpl::ModifyCompilationEnvironment(LightmapQuality, Parameters, OutEnvironment);
	}
};

/**
 * Policy for 'fake' texture lightmaps, such as the lightmap density rendering mode
 */
struct FDummyLightMapPolicy : public TLightMapPolicy< HQ_LIGHTMAP >
{
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
};

/**
 * Policy for self shadowing translucency from a directional light
 */
class FSelfShadowedTranslucencyPolicy
{
public:

	typedef FRHIUniformBuffer* ElementDataType;

	class VertexParametersType
	{
		DECLARE_INLINE_TYPE_LAYOUT(VertexParametersType, NonVirtual);
	public:
		void Bind(const FShaderParameterMap& ParameterMap) {}
		void Serialize(FArchive& Ar) {}
	};

	class PixelParametersType
	{
		DECLARE_INLINE_TYPE_LAYOUT(PixelParametersType, NonVirtual);
	public:
		void Bind(const FShaderParameterMap& ParameterMap)
		{
			TranslucentSelfShadowBufferParameter.Bind(ParameterMap, TEXT("TranslucentSelfShadow"));
		}

		void Serialize(FArchive& Ar)
		{
			Ar << TranslucentSelfShadowBufferParameter;
		}

		LAYOUT_FIELD(FShaderUniformBufferParameter, TranslucentSelfShadowBufferParameter);
	};

	class ComputeParametersType
	{
		DECLARE_INLINE_TYPE_LAYOUT(ComputeParametersType, NonVirtual);
	public:
		void Bind(const FShaderParameterMap& ParameterMap)
		{
			TranslucentSelfShadowBufferParameter.Bind(ParameterMap, TEXT("TranslucentSelfShadow"));
		}

		void Serialize(FArchive& Ar)
		{
			Ar << TranslucentSelfShadowBufferParameter;
		}

		LAYOUT_FIELD(FShaderUniformBufferParameter, TranslucentSelfShadowBufferParameter);
	};

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/** Initialization constructor. */
	FSelfShadowedTranslucencyPolicy();
	
	static void GetVertexShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const VertexParametersType* VertexShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

	static void GetPixelShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const PixelParametersType* PixelShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

	static void GetComputeShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const ComputeParametersType* ComputeShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

	friend bool operator==(const FSelfShadowedTranslucencyPolicy A,const FSelfShadowedTranslucencyPolicy B)
	{
		return true;
	}
};

/**
 * Allows precomputed irradiance lookups at any point in space.
 */
struct FPrecomputedVolumetricLightmapLightingPolicy
{
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/**
 * Allows a dynamic object to access indirect lighting through a per-object allocation in a volume texture atlas
 */
struct FCachedVolumeIndirectLightingPolicy
{
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/**
 * Allows a dynamic object to access indirect lighting through a per-object lighting sample
 */
struct FCachedPointIndirectLightingPolicy
{
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/** Mobile Specific: Combines a directional light and CSM. Not used with pre-computed lighting */
struct FMobileDirectionalLightAndCSMPolicy
{
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
};

/** Mobile Specific: Combines a distance field shadow with LQ lightmaps. */
class FMobileDistanceFieldShadowsAndLQLightMapPolicy : public TDistanceFieldShadowsAndLightMapPolicy<LQ_LIGHTMAP>
{
	typedef TDistanceFieldShadowsAndLightMapPolicy<LQ_LIGHTMAP>	Super;

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/** Mobile Specific: Combines an distance field shadow with LQ lightmaps and CSM. */
class FMobileDistanceFieldShadowsLightMapAndCSMLightingPolicy : public FMobileDistanceFieldShadowsAndLQLightMapPolicy
{
	typedef FMobileDistanceFieldShadowsAndLQLightMapPolicy Super;

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/** Mobile Specific: Combines a directional light with LQ lightmaps and CSM  */
struct FMobileDirectionalLightCSMAndLightMapPolicy : public TLightMapPolicy< LQ_LIGHTMAP >
{
	typedef TLightMapPolicy<LQ_LIGHTMAP> Super;

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/** Mobile Specific: Combines an unshadowed directional light with indirect lighting from a single SH sample. */
struct FMobileDirectionalLightAndSHIndirectPolicy
{
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/** Mobile Specific: Combines a directional light with CSM with indirect lighting from a single SH sample. */
class FMobileDirectionalLightCSMAndSHIndirectPolicy : public FMobileDirectionalLightAndSHIndirectPolicy
{
	typedef FMobileDirectionalLightAndSHIndirectPolicy Super;

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};

/** Mobile Specific: SLW rendering */
class FMobileSingleLayerWaterPolicy
{
public:
	FMobileSingleLayerWaterPolicy();

	typedef FRHIUniformBuffer* ElementDataType;

	class VertexParametersType
	{
		DECLARE_INLINE_TYPE_LAYOUT(VertexParametersType, NonVirtual);
	public:
		void Bind(const FShaderParameterMap& ParameterMap) {}
		void Serialize(FArchive& Ar) {}
	};

	class PixelParametersType
	{
		DECLARE_INLINE_TYPE_LAYOUT(PixelParametersType, NonVirtual);
	public:
		void Bind(const FShaderParameterMap& ParameterMap) {}
		void Serialize(FArchive& Ar) {}
	};


	static RENDERER_API bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static RENDERER_API void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	static RENDERER_API void GetVertexShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const VertexParametersType* VertexShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

	static RENDERER_API void GetPixelShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const PixelParametersType* PixelShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

	friend bool operator==(const FMobileSingleLayerWaterPolicy A, const FMobileSingleLayerWaterPolicy B)
	{
		return true;
	}
};

enum ELightMapPolicyType
{
	LMP_NO_LIGHTMAP,
	LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING,
	LMP_CACHED_VOLUME_INDIRECT_LIGHTING,
	LMP_CACHED_POINT_INDIRECT_LIGHTING,
	LMP_LQ_LIGHTMAP,
	LMP_HQ_LIGHTMAP,
	LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP,
	// Mobile specific
	LMP_MOBILE_DISTANCE_FIELD_SHADOWS_AND_LQ_LIGHTMAP,
	LMP_MOBILE_DISTANCE_FIELD_SHADOWS_LIGHTMAP_AND_CSM,
	LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_LIGHTMAP,
	LMP_MOBILE_DIRECTIONAL_LIGHT_AND_SH_INDIRECT,
	LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_SH_INDIRECT,
	LMP_MOBILE_DIRECTIONAL_LIGHT_CSM,

	// LightMapDensity
	LMP_DUMMY
};

class FUniformLightMapPolicyShaderParametersType
{
	DECLARE_TYPE_LAYOUT(FUniformLightMapPolicyShaderParametersType, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap)
	{
		PrecomputedLightingBufferParameter.Bind(ParameterMap, TEXT("PrecomputedLightingBuffer"));
		IndirectLightingCacheParameter.Bind(ParameterMap, TEXT("IndirectLightingCache"));
		LightmapResourceCluster.Bind(ParameterMap, TEXT("LightmapResourceCluster"));
	}

	void Serialize(FArchive& Ar)
	{
		Ar << PrecomputedLightingBufferParameter;
		Ar << IndirectLightingCacheParameter;
		Ar << LightmapResourceCluster;
	}

	LAYOUT_FIELD(FShaderUniformBufferParameter, PrecomputedLightingBufferParameter);
	LAYOUT_FIELD(FShaderUniformBufferParameter, IndirectLightingCacheParameter);
	LAYOUT_FIELD(FShaderUniformBufferParameter, LightmapResourceCluster);
};

class FUniformLightMapPolicy
{
public:

	typedef const FLightCacheInterface* ElementDataType;

	typedef FUniformLightMapPolicyShaderParametersType PixelParametersType;
	typedef FUniformLightMapPolicyShaderParametersType VertexParametersType;
#if RHI_RAYTRACING
	typedef FUniformLightMapPolicyShaderParametersType RayHitGroupParametersType;
#endif
	typedef FUniformLightMapPolicyShaderParametersType ComputeParametersType;

	FUniformLightMapPolicy(ELightMapPolicyType InIndirectPolicy) : IndirectPolicy(InIndirectPolicy) {}

	static RENDERER_API bool ShouldCompilePermutation(ELightMapPolicyType Policy, const FMeshMaterialShaderPermutationParameters& Parameters);
	static RENDERER_API void ModifyCompilationEnvironment(ELightMapPolicyType Policy, const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	static RENDERER_API void GetVertexShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const VertexParametersType* VertexShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

	static RENDERER_API void GetPixelShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const PixelParametersType* PixelShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

#if RHI_RAYTRACING
	RENDERER_API void GetRayHitGroupShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FLightCacheInterface* ElementData,
		const RayHitGroupParametersType* RayHitGroupShaderParameters,
		FMeshDrawSingleShaderBindings& RayHitGroupBindings
	) const;
#endif // RHI_RAYTRACING

	static RENDERER_API void GetComputeShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const ComputeParametersType* PixelShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

	friend bool operator==(const FUniformLightMapPolicy A,const FUniformLightMapPolicy B)
	{
		return A.IndirectPolicy == B.IndirectPolicy;
	}

	ELightMapPolicyType GetIndirectPolicy() const { return IndirectPolicy; }

private:
	const ELightMapPolicyType IndirectPolicy;
};

template <ELightMapPolicyType Policy>
class TUniformLightMapPolicy : public FUniformLightMapPolicy
{
public:
	TUniformLightMapPolicy() : FUniformLightMapPolicy(Policy) {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return FUniformLightMapPolicy::ShouldCompilePermutation(Policy, Parameters);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FUniformLightMapPolicy::ModifyCompilationEnvironment(Policy, Parameters, OutEnvironment);
	}
};

struct FSelfShadowLightCacheElementData
{
	const FLightCacheInterface* LCI;
	FRHIUniformBuffer* SelfShadowTranslucencyUniformBuffer;
};

/**
 * Self shadowing translucency from a directional light + allows a dynamic object to access indirect lighting through a per-object lighting sample
 */
class FSelfShadowedCachedPointIndirectLightingPolicy : public FSelfShadowedTranslucencyPolicy
{
public:
	typedef const FSelfShadowLightCacheElementData ElementDataType;

	class PixelParametersType : public FUniformLightMapPolicyShaderParametersType, public FSelfShadowedTranslucencyPolicy::PixelParametersType
	{
		DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(PixelParametersType, NonVirtual, FUniformLightMapPolicyShaderParametersType, FSelfShadowedTranslucencyPolicy::PixelParametersType);
	public:
		void Bind(const FShaderParameterMap& ParameterMap)
		{
			FUniformLightMapPolicyShaderParametersType::Bind(ParameterMap);
			FSelfShadowedTranslucencyPolicy::PixelParametersType::Bind(ParameterMap);
		}

		void Serialize(FArchive& Ar)
		{
			FUniformLightMapPolicyShaderParametersType::Serialize(Ar);
			FSelfShadowedTranslucencyPolicy::PixelParametersType::Serialize(Ar);
		}
	};

	class ComputeParametersType : public FUniformLightMapPolicyShaderParametersType, public FSelfShadowedTranslucencyPolicy::ComputeParametersType
	{
		DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(ComputeParametersType, NonVirtual, FUniformLightMapPolicyShaderParametersType, FSelfShadowedTranslucencyPolicy::ComputeParametersType);
	public:
		void Bind(const FShaderParameterMap& ParameterMap)
		{
			FUniformLightMapPolicyShaderParametersType::Bind(ParameterMap);
			FSelfShadowedTranslucencyPolicy::ComputeParametersType::Bind(ParameterMap);
		}

		void Serialize(FArchive& Ar)
		{
			FUniformLightMapPolicyShaderParametersType::Serialize(Ar);
			FSelfShadowedTranslucencyPolicy::ComputeParametersType::Serialize(Ar);
		}
	};

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/** Initialization constructor. */
	FSelfShadowedCachedPointIndirectLightingPolicy() {}

	static void GetVertexShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const VertexParametersType* VertexShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings) {}

	static void GetPixelShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const PixelParametersType* PixelShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

	static void GetComputeShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const ComputeParametersType* ComputeShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);
};

class FSelfShadowedVolumetricLightmapPolicy : public FSelfShadowedTranslucencyPolicy
{
public:
	typedef const FSelfShadowLightCacheElementData ElementDataType;

	class PixelParametersType : public FUniformLightMapPolicyShaderParametersType, public FSelfShadowedTranslucencyPolicy::PixelParametersType
	{
		DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(PixelParametersType, NonVirtual, FUniformLightMapPolicyShaderParametersType, FSelfShadowedTranslucencyPolicy::PixelParametersType);
	public:
		void Bind(const FShaderParameterMap& ParameterMap)
		{
			FUniformLightMapPolicyShaderParametersType::Bind(ParameterMap);
			FSelfShadowedTranslucencyPolicy::PixelParametersType::Bind(ParameterMap);
		}

		void Serialize(FArchive& Ar)
		{
			FUniformLightMapPolicyShaderParametersType::Serialize(Ar);
			FSelfShadowedTranslucencyPolicy::PixelParametersType::Serialize(Ar);
		}
	};

	class ComputeParametersType : public FUniformLightMapPolicyShaderParametersType, public FSelfShadowedTranslucencyPolicy::ComputeParametersType
	{
		DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(ComputeParametersType, NonVirtual, FUniformLightMapPolicyShaderParametersType, FSelfShadowedTranslucencyPolicy::ComputeParametersType);
	public:
		void Bind(const FShaderParameterMap& ParameterMap)
		{
			FUniformLightMapPolicyShaderParametersType::Bind(ParameterMap);
			FSelfShadowedTranslucencyPolicy::ComputeParametersType::Bind(ParameterMap);
		}

		void Serialize(FArchive& Ar)
		{
			FUniformLightMapPolicyShaderParametersType::Serialize(Ar);
			FSelfShadowedTranslucencyPolicy::ComputeParametersType::Serialize(Ar);
		}
	};

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/** Initialization constructor. */
	FSelfShadowedVolumetricLightmapPolicy();

	static void GetVertexShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const VertexParametersType* VertexShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings) {}

	static void GetPixelShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const PixelParametersType* PixelShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);

	static void GetComputeShaderBindings(
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const ElementDataType& ShaderElementData,
		const ComputeParametersType* ComputeShaderParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings);
};
