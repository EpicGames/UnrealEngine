// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightRendering.cpp: Light rendering implementation.
=============================================================================*/

#include "LightRendering.h"
#include "LogRenderer.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "Engine/SubsurfaceProfile.h"
#include "ShowFlags.h"
#include "VisualizeTexture.h"
#include "RayTracing/RaytracingOptions.h"
#include "SceneTextureParameters.h"
#include "HairStrands/HairStrandsRendering.h"
#include "ScreenPass.h"
#include "SkyAtmosphereRendering.h"
#include "VolumetricCloudRendering.h"
#include "Substrate/Substrate.h"
#include "VirtualShadowMaps/VirtualShadowMapProjection.h"
#include "HairStrands/HairStrandsData.h"
#include "AnisotropyRendering.h"
#include "Engine/SubsurfaceProfile.h"
#include "Shadows/ShadowSceneRenderer.h"
#include "RenderCore.h"
#include "BasePassRendering.h"
#include "MobileBasePassRendering.h"
#include "TranslucentLighting.h"
#include "MegaLights/MegaLights.h"
#include "LightFunctionAtlas.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "Materials/MaterialRenderProxy.h"
#include "RHIResourceUtils.h"
#include "RectLightSceneProxy.h"
#include "Shadows/FirstPersonSelfShadow.h"
#include "PixelShaderUtils.h"
#include "SceneViewState.h"

#include "PSOPrecacheValidation.h"
#include "PostProcess/PostProcessSubsurface.h"

using namespace LightFunctionAtlas;

// ENABLE_DEBUG_DISCARD_PROP is used to test the lighting code by allowing to discard lights to see how performance scales
// It ought never to be enabled in a shipping build, and is probably only really useful when woring on the shading code.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define ENABLE_DEBUG_DISCARD_PROP 1
#else // (UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define ENABLE_DEBUG_DISCARD_PROP 0
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

DECLARE_GPU_DRAWCALL_STAT(Lights);

IMPLEMENT_TYPE_LAYOUT(FLightFunctionSharedParameters);
IMPLEMENT_TYPE_LAYOUT(FStencilingGeometryShaderParameters);
IMPLEMENT_TYPE_LAYOUT(FOnePassPointShadowProjectionShaderParameters);
IMPLEMENT_TYPE_LAYOUT(FShadowProjectionShaderParameters);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FDeferredLightUniformStruct, "DeferredLightUniforms");

DECLARE_DWORD_COUNTER_STAT(TEXT("VSM Light Projections (Local One Pass Fast)"), STAT_VSMLocalProjectionOnePassFast, STATGROUP_ShadowRendering);

static const TCHAR* DeferredLightGlobalPSOCollectorName = TEXT("DeferredLightGlobalPSOCollector");

static int32 GAllowDepthBoundsTest = 1;
static FAutoConsoleVariableRef CVarAllowDepthBoundsTest(
	TEXT("r.AllowDepthBoundsTest"),
	GAllowDepthBoundsTest,
	TEXT("If true, use enable depth bounds test when rendering deferred lights.")
	);

static int32 bAllowSimpleLights = 1;
static FAutoConsoleVariableRef CVarAllowSimpleLights(
	TEXT("r.AllowSimpleLights"),
	bAllowSimpleLights,
	TEXT("If true, we allow simple (ie particle) lights")
);

static TAutoConsoleVariable<int32> CVarRayTracingOcclusion(
	TEXT("r.RayTracing.Shadows"),
	0,
	TEXT("0: use traditional rasterized shadow map (default)\n")
	TEXT("1: use ray tracing shadows"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static int32 GShadowRayTracingSamplesPerPixel = -1;
static FAutoConsoleVariableRef CVarShadowRayTracingSamplesPerPixel(
	TEXT("r.RayTracing.Shadows.SamplesPerPixel"),
	GShadowRayTracingSamplesPerPixel,
	TEXT("Sets the samples-per-pixel for directional light occlusion (default = 1)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarShadowUseDenoiser(
	TEXT("r.Shadow.Denoiser"),
	2,
	TEXT("Choose the denoising algorithm.\n")
	TEXT(" 0: Disabled (default);\n")
	TEXT(" 1: Forces the default denoiser of the renderer;\n")
	TEXT(" 2: GScreenSpaceDenoiser witch may be overriden by a third party plugin.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMaxShadowDenoisingBatchSize(
	TEXT("r.Shadow.Denoiser.MaxBatchSize"), 4,
	TEXT("Maximum number of shadow to denoise at the same time."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMaxShadowRayTracingBatchSize(
	TEXT("r.RayTracing.Shadows.MaxBatchSize"), 8,
	TEXT("Maximum number of shadows to trace at the same time."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAllowClearLightSceneExtentsOnly(
	TEXT("r.AllowClearLightSceneExtentsOnly"), 1,
	TEXT(""),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsDirectionalLight(
	TEXT("r.RayTracing.Shadows.Lights.Directional"),
	1,
	TEXT("Enables ray tracing shadows for directional lights (default = 1)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsPointLight(
	TEXT("r.RayTracing.Shadows.Lights.Point"),
	1,
	TEXT("Enables ray tracing shadows for point lights (default = 1)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsSpotLight(
	TEXT("r.RayTracing.Shadows.Lights.Spot"),
	1,
	TEXT("Enables ray tracing shadows for spot lights (default = 1)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsRectLight(
	TEXT("r.RayTracing.Shadows.Lights.Rect"),
	1,
	TEXT("Enables ray tracing shadows for rect light (default = 1)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarAppliedLightFunctionOnHair(
	TEXT("r.HairStrands.LightFunction"),
	1,
	TEXT("Enables Light function on hair"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOnePassProjectionSkipScreenShadowMask(
	TEXT("r.Shadow.Virtual.OnePassProjection.SkipScreenShadowMask"),
	1,
	TEXT("Allows skipping the screen space shadow mask entirely when only a virtual shadow map would write into it.\n")
	TEXT("Should generally be left enabled outside of debugging."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarContactShadowsOverrideLength(
	TEXT("r.ContactShadows.OverrideLength"),
	-1.0f,
	TEXT("Allows overriding the contact shadow length for all directional lights.\n")
	TEXT("Disabled when < 0.\n")
	TEXT("Should generally be left disabled outside of debugging."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarContactShadowsOverrideLengthInWS(
	TEXT("r.ContactShadows.OverrideLengthInWS"),
	false,
	TEXT("Whether r.ContactShadows.OverrideLength is in world space units or in screen space units."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarContactShadowsOverrideShadowCastingIntensity(
	TEXT("r.ContactShadows.OverrideShadowCastingIntensity"),
	-1.0f,
	TEXT("Allows overriding the contact shadow casting intensity for all directional lights.\n")
	TEXT("Disabled when < 0.\n")
	TEXT("Should generally be left disabled outside of debugging."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarContactShadowsOverrideNonShadowCastingIntensity(
	TEXT("r.ContactShadows.OverrideNonShadowCastingIntensity"),
	-1.0f,
	TEXT("Allows overriding the contact shadow non casting intensity for all directional lights.\n")
	TEXT("Disabled when < 0.\n")
	TEXT("Should generally be left disabled outside of debugging."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHairStrandsAllowOneTransmittancePass(
	TEXT("r.HairStrands.Lighting.AllowOneTransmittancePass"),
	1,
	TEXT("Allows one transmittance pass for hair strands lighting to have better performance (experimental).\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarHairStrandsSimpleLightEnable(
	TEXT("r.HairStrands.Lighting.SimpleLights"),
	1,
	TEXT("Allows simple lights to lit hair strands geometry.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMegaLightsSimpleLightMode(
	TEXT("r.MegaLights.SimpleLightMode"),
	1,
	TEXT("How MegaLights should handle simple lights:\n")
	TEXT("0 - Disabled. Simple lights are not handled by MegaLights.\n")
	TEXT("1 - Simple lights marked as allowing MegaLights are handled. (default)\n")
	TEXT("2 - All simple lights are handled."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

#if ENABLE_DEBUG_DISCARD_PROP
static float GDebugLightDiscardProp = 0.0f;
static FAutoConsoleVariableRef CVarDebugLightDiscardProp(
	TEXT("r.DebugLightDiscardProp"),
	GDebugLightDiscardProp,
	TEXT("[0,1]: Proportion of lights to discard for debug/performance profiling purposes.")
);
#endif // ENABLE_DEBUG_DISCARD_PROP

// Accessor to experimental variable static local here to not have have it exposed in public header RenderUtils.h for now.
static bool IsExperimentalToonUnifiedDiffuseEnabled()
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Substrate.Experimental.ToonUnifiedDiffuse"));
	return CVar && CVar->GetInt() > 0;
}

#if RHI_RAYTRACING

static bool ShouldRenderRayTracingShadowsForLightType(ELightComponentType LightType)
{
	switch(LightType)
	{
	case LightType_Directional:
		return !!CVarRayTracingShadowsDirectionalLight.GetValueOnRenderThread();
	case LightType_Point:
		return !!CVarRayTracingShadowsPointLight.GetValueOnRenderThread();
	case LightType_Spot:
		return !!CVarRayTracingShadowsSpotLight.GetValueOnRenderThread();
	case LightType_Rect:
		return !!CVarRayTracingShadowsRectLight.GetValueOnRenderThread();
	default:
		return true;	
	}	
}

bool ShouldRenderRayTracingShadows(const FSceneViewFamily& ViewFamily)
{
	return ShouldRenderRayTracingEffect(
		(CVarRayTracingOcclusion.GetValueOnRenderThread() > 0),
		ERayTracingPipelineCompatibilityFlags::FullPipeline | ERayTracingPipelineCompatibilityFlags::Inline, 
		ViewFamily
	);
}

bool ShouldRenderRayTracingShadowsForLight(const FSceneViewFamily& ViewFamily, ELightComponentType LightType, ECastRayTracedShadow::Type CastRayTracedShadow)
{
	if (   !ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::FullPipeline | ERayTracingPipelineCompatibilityFlags::Inline, ViewFamily)
		|| !ShouldRenderRayTracingShadowsForLightType(LightType))
	{
		return false;
	}

	switch (CastRayTracedShadow)
	{
	case ECastRayTracedShadow::Enabled:
		return true;

	case ECastRayTracedShadow::UseProjectSetting:
		return ShouldRenderRayTracingShadows(ViewFamily);

	default:
		return false;
	}
}

bool ShouldRenderRayTracingShadowsForLight(const FSceneViewFamily& ViewFamily, const FLightSceneProxy& LightProxy)
{
	return ShouldRenderRayTracingShadowsForLight(ViewFamily, ELightComponentType(LightProxy.GetLightType()), LightProxy.CastsRaytracedShadow());
}

bool ShouldRenderRayTracingShadowsForLight(const FSceneViewFamily& ViewFamily, const FLightSceneInfoCompact& LightInfo)
{
	return ShouldRenderRayTracingShadowsForLight(ViewFamily, ELightComponentType(LightInfo.LightType), LightInfo.CastRaytracedShadow);
}
#endif // RHI_RAYTRACING

void GetLightContactShadowParameters(const FLightSceneProxy* Proxy, float& OutLength, bool& bOutLengthInWS, float& OutCastingIntensity, float& OutNonCastingIntensity)
{
	OutLength = Proxy->GetContactShadowLength();
	bOutLengthInWS = Proxy->IsContactShadowLengthInWS();
	OutCastingIntensity = Proxy->GetContactShadowCastingIntensity();
	OutNonCastingIntensity = Proxy->GetContactShadowNonCastingIntensity();

	if (CVarContactShadowsOverrideLength.GetValueOnAnyThread() >= 0.0f)
	{
		OutLength = CVarContactShadowsOverrideLength.GetValueOnAnyThread();
		bOutLengthInWS = CVarContactShadowsOverrideLengthInWS.GetValueOnAnyThread();
	}

	if (CVarContactShadowsOverrideShadowCastingIntensity.GetValueOnAnyThread() >= 0.0f)
	{
		OutCastingIntensity = CVarContactShadowsOverrideShadowCastingIntensity.GetValueOnAnyThread();
	}

	if (CVarContactShadowsOverrideNonShadowCastingIntensity.GetValueOnAnyThread() >= 0.0f)
	{
		OutNonCastingIntensity = CVarContactShadowsOverrideNonShadowCastingIntensity.GetValueOnAnyThread();
	}

	if (!bOutLengthInWS)
	{
		// Multiply by 2 for screen space in order to preserve old values after introducing multiply by View.ClipToView[1][1] in shader.
		OutLength *= 2.0f;
	}
}

void FLightFunctionSharedParameters::Bind(const FShaderParameterMap& ParameterMap)
{
	LightFunctionParameters.Bind(ParameterMap, TEXT("LightFunctionParameters"));
}

FVector4f FLightFunctionSharedParameters::GetLightFunctionSharedParameters(const FLightSceneInfo* LightSceneInfo, float ShadowFadeFraction)
{
	bool bIsSpotLight = LightSceneInfo->Proxy->GetLightType() == LightType_Spot;
	const bool bIsPointLight = LightSceneInfo->Proxy->GetLightType() == LightType_Point;
	float TanOuterAngle = bIsSpotLight ? FMath::Tan(LightSceneInfo->Proxy->GetOuterConeAngle()) : 1.0f;

	if (LightSceneInfo->Proxy->GetLightType() == LightType_Rect)
	{
		// Rect light can have a spot like perspective projection
		FRectLightSceneProxy* RectLightProxy = (FRectLightSceneProxy*)LightSceneInfo->Proxy;
		if (RectLightProxy->LightFunctionConeAngleTangent > 0.0f)
		{
			bIsSpotLight = true;
			TanOuterAngle = RectLightProxy->LightFunctionConeAngleTangent;
		}
		else
		{
			bIsSpotLight = false;
			TanOuterAngle = 0.0f;
		}
	}

	return FVector4f(TanOuterAngle, ShadowFadeFraction, bIsSpotLight ? 1.0f : 0.0f, bIsPointLight ? 1.0f : 0.0f);
}

class FStencilConeIndexBuffer : public FIndexBuffer
{
public:
	// A side is a line of vertices going from the cone's origin to the edge of its SphereRadius
	static const int32 NumSides = 18;
	// A slice is a circle of vertices in the cone's XY plane
	static const int32 NumSlices = 12;

	static const uint32 NumVerts = NumSides * NumSlices * 2;

	void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		TArray<uint16> Indices;

		Indices.Empty((NumSlices - 1) * NumSides * 12);
		// Generate triangles for the vertices of the cone shape
		for (int32 SliceIndex = 0; SliceIndex < NumSlices - 1; SliceIndex++)
		{
			for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
			{
				const int32 CurrentIndex = SliceIndex * NumSides + SideIndex % NumSides;
				const int32 NextSideIndex = SliceIndex * NumSides + (SideIndex + 1) % NumSides;
				const int32 NextSliceIndex = (SliceIndex + 1) * NumSides + SideIndex % NumSides;
				const int32 NextSliceAndSideIndex = (SliceIndex + 1) * NumSides + (SideIndex + 1) % NumSides;

				Indices.Add(CurrentIndex);
				Indices.Add(NextSideIndex);
				Indices.Add(NextSliceIndex);
				Indices.Add(NextSliceIndex);
				Indices.Add(NextSideIndex);
				Indices.Add(NextSliceAndSideIndex);
			}
		}

		// Generate triangles for the vertices of the spherical cap
		const int32 CapIndexStart = NumSides * NumSlices;

		for (int32 SliceIndex = 0; SliceIndex < NumSlices - 1; SliceIndex++)
		{
			for (int32 SideIndex = 0; SideIndex < NumSides; SideIndex++)
			{
				const int32 CurrentIndex = SliceIndex * NumSides + SideIndex % NumSides + CapIndexStart;
				const int32 NextSideIndex = SliceIndex * NumSides + (SideIndex + 1) % NumSides + CapIndexStart;
				const int32 NextSliceIndex = (SliceIndex + 1) * NumSides + SideIndex % NumSides + CapIndexStart;
				const int32 NextSliceAndSideIndex = (SliceIndex + 1) * NumSides + (SideIndex + 1) % NumSides + CapIndexStart;

				Indices.Add(CurrentIndex);
				Indices.Add(NextSliceIndex);
				Indices.Add(NextSideIndex);
				Indices.Add(NextSideIndex);
				Indices.Add(NextSliceIndex);
				Indices.Add(NextSliceAndSideIndex);
			}
		}

		NumIndices = Indices.Num();

		// Create index buffer. Fill buffer with initial data upon creation
		IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FStencilConeIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
	}

	int32 GetIndexCount() const { return NumIndices; }

protected:
	int32 NumIndices;
};

/** The stencil cone index buffer. */
TGlobalResource<FStencilConeIndexBuffer> GStencilConeIndexBuffer;

/**
* Vertex buffer for a cone. It holds zero'd out data since the actual math is done on the shader
*/
class FStencilConeVertexBuffer : public FVertexBuffer
{
public:
	static const int32 NumVerts = FStencilConeIndexBuffer::NumSides * FStencilConeIndexBuffer::NumSlices * 2;

	/**
	* Initialize the RHI for this rendering resource
	*/
	void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		TArray<FVector4f> Verts;
		Verts.Empty(NumVerts);
		for (int32 s = 0; s < NumVerts; s++)
		{
			Verts.Emplace(0, 0, 0, 0);
		}

		VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("FStencilConeVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Verts));
	}

	int32 GetVertexCount() const { return NumVerts; }
};

/** The (dummy) stencil cone vertex buffer. */
TGlobalResource<FStencilConeVertexBuffer> GStencilConeVertexBuffer;

void FStencilingGeometryShaderParameters::Bind(const FShaderParameterMap& ParameterMap)
{
	StencilGeometryPosAndScale.Bind(ParameterMap, TEXT("StencilingGeometryPosAndScale"));
	StencilConeParameters.Bind(ParameterMap, TEXT("StencilingConeParameters"));
	StencilConeTransform.Bind(ParameterMap, TEXT("StencilingConeTransform"));
}

void FStencilingGeometryShaderParameters::Set(FRHIBatchedShaderParameters& BatchedParameters, const FVector4f& InStencilingGeometryPosAndScale) const
{
	const FParameters P = GetParameters(InStencilingGeometryPosAndScale);
	SetShaderValue(BatchedParameters, StencilGeometryPosAndScale, P.StencilingGeometryPosAndScale);
	SetShaderValue(BatchedParameters, StencilConeParameters, P.StencilingConeParameters);
}

void FStencilingGeometryShaderParameters::Set(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FLightSceneInfo* LightSceneInfo) const
{
	const FParameters P = GetParameters(View, LightSceneInfo);
	if (LightSceneInfo->Proxy->GetLightType() == LightType_Point ||
		LightSceneInfo->Proxy->GetLightType() == LightType_Rect)
	{
		SetShaderValue(BatchedParameters, StencilGeometryPosAndScale, P.StencilingGeometryPosAndScale);
		SetShaderValue(BatchedParameters, StencilConeParameters, P.StencilingConeParameters);
	}
	else if (LightSceneInfo->Proxy->GetLightType() == LightType_Spot)
	{
		SetShaderValue(BatchedParameters, StencilConeTransform, P.StencilingConeTransform);
		SetShaderValue(BatchedParameters, StencilConeParameters, P.StencilingConeParameters);
	}
}

FStencilingGeometryShaderParameters::FParameters FStencilingGeometryShaderParameters::GetParameters(const FVector4f& InStencilingGeometryPosAndScale)
{
	FParameters Out;
	Out.StencilingGeometryPosAndScale = InStencilingGeometryPosAndScale;
	Out.StencilingConeParameters = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
	Out.StencilingConeTransform = FMatrix44f::Identity;
	return Out;
}

FStencilingGeometryShaderParameters::FParameters FStencilingGeometryShaderParameters::GetParameters(const FSceneView& View, const FLightSceneInfo* LightSceneInfo)
{
	FParameters Out;
	if (LightSceneInfo->Proxy->GetLightType() == LightType_Point ||
		LightSceneInfo->Proxy->GetLightType() == LightType_Rect)
	{
		StencilingGeometry::GStencilSphereVertexBuffer.CalcTransform(Out.StencilingGeometryPosAndScale, LightSceneInfo->Proxy->GetBoundingSphere(), View.ViewMatrices.GetPreViewTranslation());
		Out.StencilingConeParameters = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		Out.StencilingConeTransform = FMatrix44f::Identity;
	}
	else if (LightSceneInfo->Proxy->GetLightType() == LightType_Spot)
	{
		const FMatrix WorldToTranslatedWorld = FTranslationMatrix(View.ViewMatrices.GetPreViewTranslation());
		Out.StencilingGeometryPosAndScale = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
		Out.StencilingConeTransform = FMatrix44f(LightSceneInfo->Proxy->GetLightToWorld() * WorldToTranslatedWorld);
		Out.StencilingConeParameters =
			FVector4f(
				FStencilConeIndexBuffer::NumSides,
				FStencilConeIndexBuffer::NumSlices,
				LightSceneInfo->Proxy->GetOuterConeAngle(),
				LightSceneInfo->Proxy->GetRadius());
	}
	return Out;
}

bool FDeferredLightVS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	FPermutationDomain PermutationVector(Parameters.PermutationId);
	if (PermutationVector.Get<FRadialLight>())
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) || IsMobileDeferredShadingEnabled(Parameters.Platform);
	}
	// used with FPrefilterPlanarReflectionPS on mobile
	return true;
}

void FDeferredLightVS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
}

FDrawFullScreenRectangleParameters FDeferredLightVS::GetFullScreenRectParameters(
	float X, float Y,
	float SizeX, float SizeY,
	float U, float V,
	float SizeU, float SizeV,
	FIntPoint InTargetSize,
	FIntPoint InTextureSize)
{
	FDrawFullScreenRectangleParameters Out;
	Out.PosScaleBias = FVector4f(SizeX, SizeY, X, Y);
	Out.UVScaleBias = FVector4f(SizeU, SizeV, U, V);
	Out.InvTargetSizeAndTextureSize = FVector4f(
		1.0f / InTargetSize.X, 1.0f / InTargetSize.Y,
		1.0f / InTextureSize.X, 1.0f / InTextureSize.Y);
	return Out;
}

FDeferredLightVS::FParameters FDeferredLightVS::GetParameters(const FViewInfo& View,
	float X, float Y,
	float SizeX, float SizeY,
	float U, float V,
	float SizeU, float SizeV,
	FIntPoint TargetSize,
	FIntPoint TextureSize,
	bool bBindViewUniform)
{
	FParameters Out;
	if (bBindViewUniform)
	{
		Out.View = View.ViewUniformBuffer;
	}
	Out.Geometry = FStencilingGeometryShaderParameters::GetParameters(FVector4f(0, 0, 0, 0));
	Out.FullScreenRect = GetFullScreenRectParameters(X, Y, SizeX, SizeY, U, V, SizeU, SizeV, TargetSize, TextureSize);
	return Out;
}

FDeferredLightVS::FParameters FDeferredLightVS::GetParameters(const FViewInfo& View, bool bBindViewUniform)
{
	FParameters Out;
	if (bBindViewUniform)
	{
		Out.View = View.ViewUniformBuffer;
	}
	Out.Geometry = FStencilingGeometryShaderParameters::GetParameters(FVector4f(0, 0, 0, 0));
	Out.FullScreenRect = GetFullScreenRectParameters(
		0, 0,
		View.ViewRect.Width(), View.ViewRect.Height(),
		View.ViewRect.Min.X, View.ViewRect.Min.Y,
		View.ViewRect.Width(), View.ViewRect.Height(),
		View.ViewRect.Size(),
		View.GetSceneTexturesConfig().Extent);

	return Out;
}

FDeferredLightVS::FParameters FDeferredLightVS::GetParameters(const FViewInfo& View, const FSphere& LightBounds, bool bBindViewUniform)
{
	FVector4f StencilingSpherePosAndScale;
	StencilingGeometry::GStencilSphereVertexBuffer.CalcTransform(StencilingSpherePosAndScale, LightBounds, View.ViewMatrices.GetPreViewTranslation());

	FParameters Out;
	if (bBindViewUniform)
	{
		Out.View = View.ViewUniformBuffer;
	}
	Out.Geometry = FStencilingGeometryShaderParameters::GetParameters((FVector4f)StencilingSpherePosAndScale); // LWC_TODO: Precision loss
	Out.FullScreenRect = GetFullScreenRectParameters(0, 0, 0, 0, 0, 0, 0, 0, FIntPoint(1, 1), FIntPoint(1, 1));
	return Out;
}

FDeferredLightVS::FParameters FDeferredLightVS::GetParameters(const FViewInfo& View, const FLightSceneInfo* LightSceneInfo, bool bBindViewUniform)
{
	FParameters Out;
	if (bBindViewUniform)
	{
		Out.View = View.ViewUniformBuffer;
	}
	Out.Geometry = FStencilingGeometryShaderParameters::GetParameters(View, LightSceneInfo);
	Out.FullScreenRect = GetFullScreenRectParameters(0, 0, 0, 0, 0, 0, 0, 0, FIntPoint(1, 1), FIntPoint(1, 1));
	return Out;
}

static void RenderLight(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FMinimalSceneTextures& SceneTextures,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef ScreenShadowMaskTexture,
	FRDGTextureRef LightingChannelsTexture,
	bool bRenderOverlap,
	bool bCloudShadow,
	const bool bCanLightUsesAtlasForUnbatchedLight = false,
	TRDGUniformBufferRef<FVirtualShadowMapUniformParameters> VirtualShadowMapUniformBuffer = nullptr,
	FRDGTextureRef ShadowMaskBits = nullptr,
	int32 VirtualShadowMapId = INDEX_NONE,
	ESubstrateTileType SubstrateTileType = ESubstrateTileType::ECount,
	bool bFirstPass = true);

uint32 GetShadowedBits(const FSceneView& View, const FLightSceneInfo& LightSceneInfo, bool bUseLightFunctionAtlas)
{
	uint32 ShadowedBits = 0;
	const bool bDynamicShadows = View.Family->EngineShowFlags.DynamicShadows && GetShadowQuality() > 0;
	const bool bHasLightFunction = LightSceneInfo.Proxy->GetLightFunctionMaterial() != NULL && (!bUseLightFunctionAtlas || !LightSceneInfo.Proxy->HasValidLightFunctionAtlasSlot());
	ShadowedBits = LightSceneInfo.Proxy->CastsStaticShadow() || bHasLightFunction ? 1 : 0;
	ShadowedBits |= LightSceneInfo.Proxy->CastsDynamicShadow() && View.Family->EngineShowFlags.DynamicShadows ? 3 : 0;
	return ShadowedBits;
}

FDeferredLightUniformStruct GetDeferredLightParameters(const FSceneView& View, const FLightSceneInfo& LightSceneInfo, bool bUseLightFunctionAtlas, uint32 LightFlags)
{
	FDeferredLightUniformStruct Out;

	FLightRenderParameters LightParameters;
	LightSceneInfo.Proxy->GetLightShaderParameters(LightParameters, LightFlags);
	LightParameters.MakeShaderParameters(View.ViewMatrices, View.GetLastEyeAdaptationExposure(), Out.LightParameters);

	const bool bIsRayTracedLight = ShouldRenderRayTracingShadowsForLight(*View.Family, *LightSceneInfo.Proxy);

	const FVector2D FadeParams = LightSceneInfo.Proxy->GetDirectionalLightDistanceFadeParameters(View.GetFeatureLevel(), !bIsRayTracedLight && LightSceneInfo.IsPrecomputedLightingValid(), View.MaxShadowCascades);
	
	// use MAD for efficiency in the shader
	Out.DistanceFadeMAD = FVector2f(FadeParams.Y, -FadeParams.X * FadeParams.Y);
	
	int32 ShadowMapChannel = LightSceneInfo.Proxy->GetShadowMapChannel();

	const bool bAllowStaticLighting = IsStaticLightingAllowed();

	if (!bAllowStaticLighting)
	{
		ShadowMapChannel = INDEX_NONE;
		Out.ShadowMapChannelMask = FVector4f(0);
	}
	else
	{
		Out.ShadowMapChannelMask = FVector4f(
			ShadowMapChannel == 0 ? 1 : 0,
			ShadowMapChannel == 1 ? 1 : 0,
			ShadowMapChannel == 2 ? 1 : 0,
			ShadowMapChannel == 3 ? 1 : 0);
	}

	Out.ShadowedBits = GetShadowedBits(View, LightSceneInfo, bUseLightFunctionAtlas);
	Out.VolumetricScatteringIntensity = LightSceneInfo.Proxy->GetVolumetricScatteringIntensity();

	static auto* ContactShadowsCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ContactShadows"));

	Out.ContactShadowLength = 0;
	Out.ContactShadowCastingIntensity = 1.0f;
	Out.ContactShadowNonCastingIntensity = 0.0f;

	if (ContactShadowsCVar && ContactShadowsCVar->GetValueOnRenderThread() != 0 && View.Family->EngineShowFlags.ContactShadows)
	{
		float ContactShadowLength;
		bool bContactShadowLengthInWS;
		float ContactShadowCastingIntensity;
		float ContactShadowNonCastingIntensity;
		GetLightContactShadowParameters(LightSceneInfo.Proxy, ContactShadowLength, bContactShadowLengthInWS, ContactShadowCastingIntensity, ContactShadowNonCastingIntensity);

		Out.ContactShadowLength = ContactShadowLength;
		// Sign indicates if contact shadow length is in world space or screen space.
		Out.ContactShadowLength *= bContactShadowLengthInWS ? -1.0f : 1.0f;
		Out.ContactShadowCastingIntensity = ContactShadowCastingIntensity;
		Out.ContactShadowNonCastingIntensity = ContactShadowNonCastingIntensity;
	}

	// When rendering reflection captures, the direct lighting of the light is actually the indirect specular from the main view
	if (View.bIsReflectionCapture)
	{
		Out.LightParameters.Color *= LightSceneInfo.Proxy->GetIndirectLightingScale();
	}

	const ELightComponentType LightType = (ELightComponentType)LightSceneInfo.Proxy->GetLightType();
	if ((LightType == LightType_Point || LightType == LightType_Spot || LightType == LightType_Rect) && View.IsPerspectiveProjection())
	{
		Out.LightParameters.Color *= GetLightFadeFactor(View, LightSceneInfo.Proxy);
	}

	Out.LightingChannelMask = LightSceneInfo.Proxy->GetLightingChannelMask();

	// Ensure the light falloff exponent is set to 0 so that lighting shaders handle it as inverse-squared attenuated light
	if (LightSceneInfo.Proxy->IsInverseSquared())
	{
		Out.LightParameters.FalloffExponent = 0.0f;
	}
	return Out;
}

FDeferredLightUniformStruct GetSimpleDeferredLightParameters(
	const FSceneView& View,
	const FSimpleLightEntry& SimpleLight,
	const FVector& LightWorldPosition)
{
	FDeferredLightUniformStruct Out;
	Out.ShadowMapChannelMask = FVector4f(0, 0, 0, 0);
	Out.DistanceFadeMAD = FVector2f(0, 0);
	Out.ContactShadowLength = 0.0f;
	Out.ContactShadowCastingIntensity = 1.f;
	Out.ContactShadowNonCastingIntensity = 0.f;
	Out.VolumetricScatteringIntensity = SimpleLight.VolumetricScatteringIntensity;
	Out.ShadowedBits = 0;
	Out.LightingChannelMask = 0;

	Out.LightParameters.TranslatedWorldPosition = FVector3f(LightWorldPosition + View.ViewMatrices.GetPreViewTranslation());
	Out.LightParameters.InvRadius = 1.0f / FMath::Max(SimpleLight.Radius, KINDA_SMALL_NUMBER);
	Out.LightParameters.Color = (FVector3f)SimpleLight.Color * FLightRenderParameters::GetLightExposureScale(View.GetLastEyeAdaptationExposure(), SimpleLight.InverseExposureBlend);
	Out.LightParameters.FalloffExponent = SimpleLight.Exponent;
	Out.LightParameters.Direction = FVector3f(1, 0, 0);
	Out.LightParameters.Tangent = FVector3f(1, 0, 0);
	Out.LightParameters.SpotAngles = FVector2f(-2, 1);
	Out.LightParameters.SpecularScale = SimpleLight.SpecularScale;
	Out.LightParameters.DiffuseScale = SimpleLight.DiffuseScale;
	Out.LightParameters.SourceRadius = 0.0f;
	Out.LightParameters.SoftSourceRadius = 0.0f;
	Out.LightParameters.SourceLength = 0.0f;
	Out.LightParameters.RectLightBarnCosAngle = 0;
	Out.LightParameters.RectLightBarnLength = -2.0f;
	Out.LightParameters.RectLightAtlasUVOffset = FVector2f::ZeroVector;
	Out.LightParameters.RectLightAtlasUVScale = FVector2f::ZeroVector;
	Out.LightParameters.RectLightAtlasMaxLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();
	Out.LightParameters.IESAtlasIndex = INDEX_NONE;
	Out.LightParameters.LightFunctionAtlasLightIndex = 0;
	Out.LightParameters.bAffectsTranslucentLighting = 0;
	return Out;
}
FDeferredLightUniformStruct GetSimpleDeferredLightParameters(
	const FSceneView& View,
	const FSimpleLightEntry& SimpleLight,
	const FSimpleLightPerViewEntry& SimpleLightPerViewData)
{
	return GetSimpleDeferredLightParameters(View, SimpleLight, SimpleLightPerViewData.Position);
}

ELightOcclusionType GetLightOcclusionType(const FLightSceneProxy& Proxy, const FSceneViewFamily& ViewFamily)
{
	bool bUseRaytracing = false;
#if RHI_RAYTRACING
	bUseRaytracing = ShouldRenderRayTracingShadowsForLight(ViewFamily, Proxy);
#endif
	EMegaLightsMode MegaLightsMode = MegaLights::GetMegaLightsMode(ViewFamily, Proxy.GetLightType(), Proxy.AllowMegaLights(), Proxy.GetMegaLightsShadowMethod());

	if (MegaLightsMode != EMegaLightsMode::Disabled)
	{
		return MegaLightsMode == EMegaLightsMode::EnabledVSM ? ELightOcclusionType::MegaLightsVSM : ELightOcclusionType::MegaLights;
	}

	return bUseRaytracing ? ELightOcclusionType::Raytraced : ELightOcclusionType::Shadowmap;
}

ELightOcclusionType GetLightOcclusionType(const FLightSceneInfoCompact& LightInfo, const FSceneViewFamily& ViewFamily)
{
	bool bUseRaytracing = false;
#if RHI_RAYTRACING
	bUseRaytracing = ShouldRenderRayTracingShadowsForLight(ViewFamily, LightInfo);
#endif
	EMegaLightsMode MegaLightsMode = MegaLights::GetMegaLightsMode(ViewFamily, LightInfo.LightType, LightInfo.bAllowMegaLights, LightInfo.MegaLightsShadowMethod);

	if (MegaLightsMode != EMegaLightsMode::Disabled)
	{
		return MegaLightsMode == EMegaLightsMode::EnabledVSM ? ELightOcclusionType::MegaLightsVSM : ELightOcclusionType::MegaLights;
	}

	return bUseRaytracing ? ELightOcclusionType::Raytraced : ELightOcclusionType::Shadowmap;
}

static TAutoConsoleVariable<bool> CVarLightMaxDrawDistanceScaleLegacyFade(
	TEXT("r.LightMaxDrawDistanceScale.UseLegacyFadeBehavior_WillBeRemoved"),
	false,
	TEXT("Note: Will be removed in a future release!\n")
	TEXT("  Use legacy fade behavior that does not take r.LightMaxDrawDistanceScale into account."),
	FConsoleVariableDelegate::CreateLambda(
		[](IConsoleVariable* CVar)
		{
			bool bValue = CVar->GetInt() != 0;
			if (bValue)
			{
				UE_LOGF(LogRenderer, Warning, 
					   "r.LightMaxDrawDistanceScale.UseLegacyFadeBehavior_WillBeRemoved can produce undesirable light fading behavior."
					   "This variable is deprecated and will be removed in a future version.");
			}
		}
	),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

float GetLightFadeFactor(const FSceneView& View, const FLightSceneProxy* Proxy)
{
	// Distance fade
	FSphere Bounds = Proxy->GetBoundingSphere();

	const float DistanceSquared = (Bounds.Center - View.ViewMatrices.GetViewOrigin()).SizeSquared();
	extern float GMinScreenRadiusForLights;
	float SizeFade = FMath::Square(FMath::Min(0.0002f, GMinScreenRadiusForLights / Bounds.W) * View.LODDistanceFactor) * DistanceSquared;
	SizeFade = FMath::Clamp(6.0f - 6.0f * SizeFade, 0.0f, 1.0f);

	extern float GLightMaxDrawDistanceScale;
	float ProxyMaxDist = Proxy->GetMaxDrawDistance();
	float ScaledMaxDist = ProxyMaxDist * GLightMaxDrawDistanceScale;
	float Range = FMath::Max(0.001f, Proxy->GetFadeRange() * GLightMaxDrawDistanceScale);
	// Legacy behavior under a (temporary!) chicken bit
	if (CVarLightMaxDrawDistanceScaleLegacyFade.GetValueOnRenderThread())
	{
		Range = Proxy->GetFadeRange();
	}
	float DistanceFade = ProxyMaxDist > 0.0f ? (ScaledMaxDist - FMath::Sqrt(DistanceSquared)) / Range : 1.0f;
	DistanceFade = FMath::Clamp(DistanceFade, 0.0f, 1.0f);
	return SizeFade * DistanceFade;
}

void StencilingGeometry::DrawSphere(FRHICommandList& RHICmdList, uint32 InstanceCount)
{
	RHICmdList.SetStreamSource(0, StencilingGeometry::GStencilSphereVertexBuffer.VertexBufferRHI, 0);
	RHICmdList.DrawIndexedPrimitive(StencilingGeometry::GStencilSphereIndexBuffer.IndexBufferRHI, 0, 0,
		StencilingGeometry::GStencilSphereVertexBuffer.GetVertexCount(), 0,
		StencilingGeometry::GStencilSphereIndexBuffer.GetIndexCount() / 3, InstanceCount);
}

void StencilingGeometry::DrawSphere(FRHICommandList& RHICmdList)
{
	DrawSphere(RHICmdList, 1);
}

void StencilingGeometry::DrawVectorSphere(FRHICommandList& RHICmdList)
{
	RHICmdList.SetStreamSource(0, StencilingGeometry::GStencilSphereVectorBuffer.VertexBufferRHI, 0);
	RHICmdList.DrawIndexedPrimitive(StencilingGeometry::GStencilSphereIndexBuffer.IndexBufferRHI, 0, 0,
									StencilingGeometry::GStencilSphereVectorBuffer.GetVertexCount(), 0,
									StencilingGeometry::GStencilSphereIndexBuffer.GetIndexCount() / 3, 1);
}

void StencilingGeometry::DrawCone(FRHICommandList& RHICmdList)
{
	// No Stream Source needed since it will generate vertices on the fly
	RHICmdList.SetStreamSource(0, GStencilConeVertexBuffer.VertexBufferRHI, 0);

	RHICmdList.DrawIndexedPrimitive(GStencilConeIndexBuffer.IndexBufferRHI, 0, 0,
		FStencilConeIndexBuffer::NumVerts, 0, GStencilConeIndexBuffer.GetIndexCount() / 3, 1);
}

/** The stencil sphere vertex buffer. */
TGlobalResource<StencilingGeometry::TStencilSphereVertexBuffer<18, 12, FVector4f> > StencilingGeometry::GStencilSphereVertexBuffer;
TGlobalResource<StencilingGeometry::TStencilSphereVertexBuffer<18, 12, FVector3f> > StencilingGeometry::GStencilSphereVectorBuffer;

/** The stencil sphere index buffer. */
TGlobalResource<StencilingGeometry::TStencilSphereIndexBuffer<18, 12> > StencilingGeometry::GStencilSphereIndexBuffer;

TGlobalResource<StencilingGeometry::TStencilSphereVertexBuffer<4, 4, FVector4f> > StencilingGeometry::GLowPolyStencilSphereVertexBuffer;
TGlobalResource<StencilingGeometry::TStencilSphereIndexBuffer<4, 4> > StencilingGeometry::GLowPolyStencilSphereIndexBuffer;

// Implement a version for directional lights, and a version for point / spot lights
IMPLEMENT_GLOBAL_SHADER(FDeferredLightVS, "/Engine/Private/DeferredLightVertexShaders.usf", "VertexMain", SF_Vertex);

class FDeferredLightHairVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FDeferredLightHairVS, Global);
	SHADER_USE_PARAMETER_STRUCT(FDeferredLightHairVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_HAIR"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeferredLightHairVS, "/Engine/Private/DeferredLightVertexShaders.usf", "HairVertexMain", SF_Vertex);

enum class ELightSourceShape
{
	Directional,
	Capsule,
	Rect,

	MAX
};


/** A pixel shader for rendering the light in a deferred pass. */
class FDeferredLightPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FDeferredLightPS, Global)
	SHADER_USE_PARAMETER_STRUCT(FDeferredLightPS, FGlobalShader);

	class FSourceShapeDim		: SHADER_PERMUTATION_ENUM_CLASS("LIGHT_SOURCE_SHAPE", ELightSourceShape);
	class FSourceTextureDim		: SHADER_PERMUTATION_BOOL("USE_SOURCE_TEXTURE");
	class FIESProfileDim		: SHADER_PERMUTATION_BOOL("USE_IES_PROFILE");
	class FLightFunctionAtlasDim: SHADER_PERMUTATION_BOOL("USE_LIGHT_FUNCTION_ATLAS");
	class FVisualizeCullingDim	: SHADER_PERMUTATION_BOOL("VISUALIZE_LIGHT_CULLING");
	class FLightingChannelsDim	: SHADER_PERMUTATION_BOOL("USE_LIGHTING_CHANNELS");
	class FTransmissionDim		: SHADER_PERMUTATION_BOOL("USE_TRANSMISSION");
	class FHairLighting			: SHADER_PERMUTATION_INT("USE_HAIR_LIGHTING", 2);
	class FHairComplexTransmittance: SHADER_PERMUTATION_BOOL("USE_HAIR_COMPLEX_TRANSMITTANCE");
	class FAtmosphereTransmittance : SHADER_PERMUTATION_BOOL("USE_ATMOSPHERE_TRANSMITTANCE");
	class FCloudTransmittance 	: SHADER_PERMUTATION_BOOL("USE_CLOUD_TRANSMITTANCE");
	class FAnistropicMaterials 	: SHADER_PERMUTATION_BOOL("SUPPORTS_ANISOTROPIC_MATERIALS");
	class FVirtualShadowMapMask : SHADER_PERMUTATION_BOOL("USE_VIRTUAL_SHADOW_MAP_MASK");
	class FUbershader 			: SHADER_PERMUTATION_BOOL("UBERSHADER");

	using FPermutationDomain = TShaderPermutationDomain<
		FSourceShapeDim,
		FSourceTextureDim,
		FIESProfileDim,
		FLightFunctionAtlasDim,
		FVisualizeCullingDim,
		FLightingChannelsDim,
		FTransmissionDim,
		FHairLighting,
		FHairComplexTransmittance,
		FAtmosphereTransmittance,
		FCloudTransmittance,
		FAnistropicMaterials,
		Substrate::FSubstrateTileType,
		FVirtualShadowMapMask,
		FUbershader>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVolumetricCloudShadowAOParameters, CloudShadowAO)
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightCloudTransmittanceParameters, CloudShadow)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FForwardLightUniformParameters, ForwardLightStruct)
		SHADER_PARAMETER(uint32, bLightingChannelEnabled)
		SHADER_PARAMETER(uint32, bCloudShadowEnabled)
		SHADER_PARAMETER(uint32, bAtmosphereTransmittanceEnabled)
		SHADER_PARAMETER(uint32, HairTransmittanceBufferMaxCount)
		SHADER_PARAMETER(uint32, HairShadowMaskValid)
		SHADER_PARAMETER(FVector4f, ShadowChannelMask)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, LightAttenuationTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LightAttenuationTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, LightingChannelsTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LightingChannelsSampler)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, HairTransmittanceBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ScreenShadowMaskSubPixelTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLightFunctionAtlasGlobalParameters, LightFunctionAtlas)
		// For virtual shadow map mask
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualShadowMapUniformParameters, VirtualShadowMap)
		SHADER_PARAMETER(int32, VirtualShadowMapId)
		SHADER_PARAMETER(int32, LightSceneId)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ShadowMaskBits)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool SupportsHairComplexTransmittance(EShaderPlatform InPlatform)
	{
		return IsHairStrandsSupported(EHairStrandsShaderType::All, InPlatform);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// Build FVisualizeCullingDim permutation only for a restricted number of case, as they don't impact the 'estimated cost' of lighting
		if (PermutationVector.Get< FVisualizeCullingDim >() && (
			PermutationVector.Get< FSourceTextureDim >() ||
			PermutationVector.Get< FIESProfileDim >() ||
			PermutationVector.Get< FLightFunctionAtlasDim >() ||
			PermutationVector.Get< FTransmissionDim >() ||
			PermutationVector.Get< FHairLighting >() ||
			PermutationVector.Get< FHairComplexTransmittance >() ||
			PermutationVector.Get< FAtmosphereTransmittance >() ||
			PermutationVector.Get< FCloudTransmittance >() ||
			PermutationVector.Get< FAnistropicMaterials >() ||
			PermutationVector.Get< FVirtualShadowMapMask >() || 
			PermutationVector.Get< FUbershader >()))
		{
			return false;
		}

		// Uber permutation - Enabled all the features with dynamic branching - 6 Permutations
		if (PermutationVector.Get< FUbershader >())
		{
			const ESubstrateTileType UberShaderTileType = Substrate::IsSubstrateEnabled() ? (Substrate::ShouldCompileSubstrateTileTypePermutations(ESubstrateTileType::EComplexSpecial, Parameters.Platform) ? ESubstrateTileType::EComplexSpecial : ESubstrateTileType::EComplex) : ESubstrateTileType::ESimple;
			if (PermutationVector.Get< FSourceTextureDim >() &&
				PermutationVector.Get< FIESProfileDim >() &&
				PermutationVector.Get< FLightFunctionAtlasDim >() &&
				PermutationVector.Get< FLightingChannelsDim >() &&
				PermutationVector.Get< FTransmissionDim >() &&
				PermutationVector.Get< FHairComplexTransmittance >() &&
				PermutationVector.Get< FAtmosphereTransmittance >() &&
				PermutationVector.Get< FCloudTransmittance >() &&
				PermutationVector.Get< FAnistropicMaterials >() &&
				PermutationVector.Get< FVirtualShadowMapMask >() &&
				PermutationVector.Get< Substrate::FSubstrateTileType >() == UberShaderTileType)
			{
				return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
			}
			else
			{
				return false;
			}
		}

		if (PermutationVector.Get< FSourceShapeDim >() == ELightSourceShape::Directional && PermutationVector.Get< FIESProfileDim >())
		{
			return false;
		}

		if (PermutationVector.Get< FSourceShapeDim >() != ELightSourceShape::Directional && (PermutationVector.Get<FAtmosphereTransmittance>() || PermutationVector.Get<FCloudTransmittance>()))
		{
			return false;
		}

		// Directional lights don't support virtual shadow map mask one pass projection, as they are always full screen lit and not part of the light grid
		if (PermutationVector.Get< FSourceShapeDim >() == ELightSourceShape::Directional && PermutationVector.Get< FVirtualShadowMapMask >())
		{
			return false;
		}

		if( PermutationVector.Get< FSourceShapeDim >() != ELightSourceShape::Rect && PermutationVector.Get< FSourceTextureDim >())
		{
			return false;
		}

		if (PermutationVector.Get< FHairLighting >() && PermutationVector.Get< FTransmissionDim >())
		{
			return false;
		}

		// When using Substrate, complex transmittance permutation is not used
		if (Substrate::IsSubstrateEnabled())
		{
			if (!PermutationVector.Get<FDeferredLightPS::FHairComplexTransmittance>() && !PermutationVector.Get< FVisualizeCullingDim >())
			{
				return false;
			}
		}
		else
		{
			// (Hair Lighting == 1) requires FHairComplexTransmittance 
			if (PermutationVector.Get<FHairLighting>() == 1 && !PermutationVector.Get<FDeferredLightPS::FHairComplexTransmittance>())
			{
				return false;
			}

			if (PermutationVector.Get<FHairLighting>() == 0 && PermutationVector.Get<FDeferredLightPS::FHairComplexTransmittance>() && !SupportsHairComplexTransmittance(Parameters.Platform))
			{
				return false;
			}
		}

		if (Substrate::IsSubstrateEnabled() && PermutationVector.Get<FHairLighting>() == 1 && PermutationVector.Get<Substrate::FSubstrateTileType>() != ESubstrateTileType::EComplex)
		{
			return false;
		}

		if (PermutationVector.Get<FDeferredLightPS::FAnistropicMaterials>())
		{
			// Anisotropic materials do not currently support rect lights
			if (PermutationVector.Get<FSourceShapeDim>() == ELightSourceShape::Rect || PermutationVector.Get<FSourceTextureDim>())
			{
				return false;
			}

			// (Hair Lighting == 1) has its own BxDF and anisotropic BRDF is only for DefaultLit and ClearCoat materials.
			if (PermutationVector.Get<FHairLighting>() == 1)
			{
				return false;
			}

			if (!FDataDrivenShaderPlatformInfo::GetSupportsAnisotropicMaterials(Parameters.Platform))
			{
				return false;
			}
		}

		if (!DoesPlatformSupportVirtualShadowMaps(Parameters.Platform) && PermutationVector.Get<FVirtualShadowMapMask>() != 0)
		{
			return false;
		}

		if (!Substrate::ShouldCompileSubstrateTileTypePermutations(PermutationVector.Get<Substrate::FSubstrateTileType>(), Parameters.Platform))
		{
			return false;
		}

		// We must keep FAnistropicMaterials 0/1 permutations for tiles of Complex and ComplexSpecial complexity. That is because rect light or hair can disable anisotropy explicitely.
		if (Substrate::IsSubstrateEnabled() && PermutationVector.Get<Substrate::FSubstrateTileType>() < ESubstrateTileType::EComplex && PermutationVector.Get<FAnistropicMaterials>() == 1)
		{
			// Remove all permutations for tiles of Simple and Single complexity, when anisotropy is true. No anisotropy is allowed in this case with Substrate.
			return false; 
		}

		// Skip special complex tile with GBuffer Format=0, as GBuffer Format=0 does not handle glint
		if (Substrate::IsSubstrateEnabled() && Substrate::IsSubstrateBlendableGBufferEnabled(Parameters.Platform) && PermutationVector.Get<Substrate::FSubstrateTileType>() == ESubstrateTileType::EComplexSpecial)
		{
			return false;
		}

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector, EShaderPlatform Platform)
	{
		// Build FVisualizeCullingDim permutation only for a restricted number of case, as they don't impact the 'estimated cost' of lighting
		if (PermutationVector.Get< FVisualizeCullingDim >())
		{
			PermutationVector.Set< FSourceTextureDim >(false);
			PermutationVector.Set< FIESProfileDim >(false);
			PermutationVector.Set< FLightFunctionAtlasDim >(false);
			PermutationVector.Set< FTransmissionDim >(false);
			PermutationVector.Set< FHairLighting >(false);
			PermutationVector.Set< FAtmosphereTransmittance >(false);
			PermutationVector.Set< FCloudTransmittance >(false);
			PermutationVector.Set< FAnistropicMaterials >(false);
			PermutationVector.Set< FVirtualShadowMapMask >(false);
			PermutationVector.Set< FHairComplexTransmittance >(false);
			PermutationVector.Set< FUbershader >(false);
		}

		if (Substrate::IsSubstrateEnabled())
		{
			// We must keep FAnistropicMaterials 0/1 permutations for tiles of Complex and ComplexSpecial complexity. That is because rect light or hair can disable anisotropy explicitely.
			if (PermutationVector.Get<Substrate::FSubstrateTileType>() < ESubstrateTileType::EComplex)
			{
				PermutationVector.Set<FAnistropicMaterials>(false);		// No anisotropy for tiles of Simple and Single complexity
			}

			// With Substrate, complex transmittance is always enabled (dynamic switch)
			if (!PermutationVector.Get<FVisualizeCullingDim>())
			{
				PermutationVector.Set<FHairComplexTransmittance>(true);
			}
		}

		if (PermutationVector.Get< FUbershader>())
		{
			const ESubstrateTileType UberShaderTileType = Substrate::IsSubstrateEnabled() ? (Substrate::ShouldCompileSubstrateTileTypePermutations(ESubstrateTileType::EComplexSpecial, Platform) ? ESubstrateTileType::EComplexSpecial : ESubstrateTileType::EComplex) : ESubstrateTileType::ESimple;
			// Remaining permutation (i.e. not dynamic branching)
			//  - FSourceShapeDim = 3
			//  - FHairLighting  = 2
			//  - FVisualizeCullingDim = 0
			PermutationVector.Set< FSourceTextureDim >(true);
			PermutationVector.Set< FIESProfileDim >(true);
			PermutationVector.Set< FLightFunctionAtlasDim >(true);
			PermutationVector.Set< FLightingChannelsDim >(true);
			PermutationVector.Set< FTransmissionDim >(true);
			PermutationVector.Set< FHairComplexTransmittance >(true);
			PermutationVector.Set< FAtmosphereTransmittance >(true);
			PermutationVector.Set< FCloudTransmittance >(true);
			PermutationVector.Set< FAnistropicMaterials >(true);
			PermutationVector.Set< FVirtualShadowMapMask >(true);
			PermutationVector.Set< Substrate::FSubstrateTileType >(UberShaderTileType);
		}

		return PermutationVector;
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FVisualizeCullingDim>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}

		if (!PermutationVector.Get<FUbershader>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}
		return FGlobalShader::ShouldPrecachePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		if (PermutationVector.Get< FVirtualShadowMapMask >() != 0)
		{
			FVirtualShadowMapArray::SetShaderDefines(OutEnvironment);
			FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		}

		OutEnvironment.SetDefine(TEXT("SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE"), PlatformSupportsSeparatedSubsurfaceDiffuse(Parameters.Platform) ? 1 : 0);

		// FXC is too slow at compiling Substrate lighting shaders
		if (Substrate::IsSubstrateEnabled() && FDataDrivenShaderPlatformInfo::GetSupportsDxc(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
		}
	}

	static EShaderCompileJobPriority GetOverrideJobPriority()
	{
		// FDeferredLightPS *can* take up to 11s
		return EShaderCompileJobPriority::ExtraHigh;
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FDeferredLightUniformStruct& DeferredLightUniformsValue)
	{
		SetUniformBufferParameterImmediate(BatchedParameters, GetUniformBufferParameter<FDeferredLightUniformStruct>(), DeferredLightUniformsValue);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeferredLightPS, "/Engine/Private/DeferredLightPixelShaders.usf", "DeferredLightPixelMain", SF_Pixel);

/** Shader used to compose toon diffuse lighting. */
class FDeferredLightApplyToonDiffusePS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FDeferredLightApplyToonDiffusePS, Global)
	SHADER_USE_PARAMETER_STRUCT(FDeferredLightApplyToonDiffusePS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<Substrate::FSubstrateTileType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, ToonDataTexture0)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, ToonDataTexture1)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_HAIR_LIGHTING"), -1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeferredLightApplyToonDiffusePS, "/Engine/Private/DeferredLightPixelShaders.usf", "DeferredLightApplyToonDiffusePS", SF_Pixel);

/** Shader used to visualize stationary light overlap. */
class FDeferredLightOverlapPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FDeferredLightOverlapPS, Global)
	SHADER_USE_PARAMETER_STRUCT(FDeferredLightOverlapPS, FGlobalShader);

	class FRadialAttenuation : SHADER_PERMUTATION_BOOL("RADIAL_ATTENUATION");
	using FPermutationDomain = TShaderPermutationDomain<FRadialAttenuation>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FDeferredLightUniformStruct, DeferredLight)
		SHADER_PARAMETER(float, bHasValidChannel)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeferredLightOverlapPS, "/Engine/Private/StationaryLightOverlapShaders.usf", "OverlapPixelMain", SF_Pixel);

/** Gathers simple lights from visible primtives in the passed in views. */
void FSceneRenderer::GatherSimpleLights(const FSceneViewFamily& ViewFamily, const TArray<FViewInfo>& Views, FSimpleLightArray& SimpleLights)
{
	TArray<const FPrimitiveSceneInfo*, SceneRenderingAllocator> PrimitivesWithSimpleLights;

	// Gather visible primitives from all views that might have simple lights
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		for (int32 PrimitiveIndex = 0; PrimitiveIndex < View.VisibleDynamicPrimitivesWithSimpleLights.Num(); PrimitiveIndex++)
		{
			const FPrimitiveSceneInfo* PrimitiveSceneInfo = View.VisibleDynamicPrimitivesWithSimpleLights[PrimitiveIndex];

			// TArray::AddUnique is slow, but not expecting many entries in PrimitivesWithSimpleLights
			PrimitivesWithSimpleLights.AddUnique(PrimitiveSceneInfo);
		}
	}

	// Gather simple lights from the primitives
	for (int32 PrimitiveIndex = 0; PrimitiveIndex < PrimitivesWithSimpleLights.Num(); PrimitiveIndex++)
	{
		const FPrimitiveSceneInfo* Primitive = PrimitivesWithSimpleLights[PrimitiveIndex];
		Primitive->Proxy->GatherSimpleLights(ViewFamily, SimpleLights);
	}
}

/** Gets a readable light name for use with a draw event. */
void FSceneRenderer::GetLightNameForDrawEvent(const FLightSceneProxy* LightProxy, FString& LightNameWithLevel)
{
#if WANTS_DRAW_MESH_EVENTS
	if (GetEmitDrawEvents())
	{
		FString FullLevelName = LightProxy->GetLevelName().ToString();
		const int32 LastSlashIndex = FullLevelName.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromEnd);

		if (LastSlashIndex != INDEX_NONE)
		{
			// Trim the leading path before the level name to make it more readable
			// The level FName was taken directly from the Outermost UObject, otherwise we would do this operation on the game thread
			FullLevelName.MidInline(LastSlashIndex + 1, FullLevelName.Len() - (LastSlashIndex + 1), EAllowShrinking::No);
		}

		LightNameWithLevel = FullLevelName + TEXT(".") + LightProxy->GetOwnerNameOrLabel();
	}
#endif
}

uint32 GetShadowQuality();

static bool LightRequiresDenoiser(const FLightSceneInfo& LightSceneInfo)
{
	ELightComponentType LightType = ELightComponentType(LightSceneInfo.Proxy->GetLightType());
	if (LightType == LightType_Directional)
	{
		return LightSceneInfo.Proxy->GetLightSourceAngle() > 0;
	}
	else if (LightType == LightType_Point || LightType == LightType_Spot)
	{
		return LightSceneInfo.Proxy->GetSourceRadius() > 0;
	}
	else if (LightType == LightType_Rect)
	{
		return true;
	}
	else
	{
		check(0);
	}
	return false;
}

bool FSceneRenderer::AllowSimpleLights() const
{
	return bAllowSimpleLights == 1;
}

bool CanLightUsesAtlasForUnbatchedLight(ERHIFeatureLevel::Type FeatureLevel, const FLightSceneProxy* Proxy)
{
	if (!Proxy)
	{
		return false;
	}

	// Material shoud also be compatible with light function atlas, i.e. not sample depth or world position.
	// We do not check that for other systems (translucent, water, volume fog, clustered, Lumen), 
	// because light functions were never available there before the atlas. So those LF aare still added into the atlas.
	// => If a material is designed to be used with those systems, light function mateirla it must be made be compliant from the start.
	const FMaterialRenderProxy* MaterialRenderProxy = Proxy->GetLightFunctionMaterial();
	if (MaterialRenderProxy)
	{
		const FMaterial& LFMaterial = Proxy->GetLightFunctionMaterial()->GetIncompleteMaterialWithFallback(FeatureLevel);
	
		return LFMaterial.MaterialIsLightFunctionAtlasCompatible_RenderThread();
	}

	return false;
}

void FSceneRenderer::UpdateLightFunctionAtlasTaskFunction()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateLightFunctionAtlas);
	SCOPED_NAMED_EVENT_TEXT("UpdateLightFunctionAtlas", FColor::Yellow);

	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
	{

		LightFunctionAtlas.UpdateRegisterLightSceneInfo(LightSceneInfoCompact.LightSceneInfo);
	}

	// Update the light function atlas according to registered lights and views
	LightFunctionAtlas.UpdateLightFunctionAtlas(Views);
}

void FSceneRenderer::GatherAndSortLights(FSortedLightSetSceneInfo& OutSortedLights, bool bShadowedLightsInClustered)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GatherAndSortLights);

	if (AllowSimpleLights())
	{
		GatherSimpleLights(ViewFamily, Views, OutSortedLights.SimpleLights);
	}
	FSimpleLightArray& SimpleLights = OutSortedLights.SimpleLights;
	TArray<FSortedLightSceneInfo, SceneRenderingAllocator>& SortedLights = OutSortedLights.SortedLights;

	// NOTE: we allocate space also for simple lights such that they can be referenced in the same sorted range
	SortedLights.Empty(Scene->Lights.Num() + SimpleLights.InstanceData.Num());

	bool bAnyViewbUsesLightingChannels = false;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (Views[ViewIndex].bUsesLightingChannels)
		{
			bAnyViewbUsesLightingChannels = true;
		}
	}

	const bool bUseLightFunctionAtlas = LightFunctionAtlas::IsEnabled(*Scene, ELightFunctionAtlasSystem::DeferredLighting);
	bool bDynamicShadows = ViewFamily.EngineShowFlags.DynamicShadows && GetShadowQuality() > 0;

#if ENABLE_DEBUG_DISCARD_PROP
	int32 Total = Scene->Lights.Num() + SimpleLights.InstanceData.Num();
	int32 NumToKeep = int32(float(Total) * (1.0f - GDebugLightDiscardProp));
	const float DebugDiscardStride = float(NumToKeep) / float(Total);
	float DebugDiscardCounter = 0.0f;
#endif // ENABLE_DEBUG_DISCARD_PROP
	// Build a list of visible lights.
	for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
	{
		const FLightSceneInfo* const LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;

#if ENABLE_DEBUG_DISCARD_PROP
		{
			int32 PrevCounter = int32(DebugDiscardCounter);
			DebugDiscardCounter += DebugDiscardStride;
			if (PrevCounter >= int32(DebugDiscardCounter))
			{
				continue;
			}
		}
#endif // ENABLE_DEBUG_DISCARD_PROP

		if (LightSceneInfo->ShouldRenderLightViewIndependent()
			// Reflection override skips direct specular because it tends to be blindingly bright with a perfectly smooth surface
			&& !ViewFamily.EngineShowFlags.ReflectionOverride)
		{
			// Check if the light is visible in any of the views.
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (LightSceneInfo->ShouldRenderLight(Views[ViewIndex]))
				{
					FSortedLightSceneInfo* SortedLightInfo = new(SortedLights) FSortedLightSceneInfo(LightSceneInfo);

					// Check for shadows and light functions.
					SortedLightInfo->SortKey.Fields.LightType = LightSceneInfoCompact.LightType;
					SortedLightInfo->SortKey.Fields.bIESProfile = ViewFamily.EngineShowFlags.TexturedLightProfiles && (LightSceneInfo->Proxy->GetIESTextureResource() != 0);
					SortedLightInfo->SortKey.Fields.bShadowed = bDynamicShadows && CheckForProjectedShadows(LightSceneInfo);
					SortedLightInfo->SortKey.Fields.bLightFunction = ViewFamily.EngineShowFlags.LightFunctions && CheckForLightFunction(LightSceneInfo);
					SortedLightInfo->SortKey.Fields.bUsesLightingChannels = LightSceneInfo->Proxy->GetLightingChannelMask() != GetDefaultLightingChannelMask();

					// Cached once here and the material is never queried again later.
					SortedLightInfo->bIsCompatibleWithLightFunctionAtlas = CanLightUsesAtlasForUnbatchedLight(FeatureLevel, LightSceneInfo->Proxy);

					// These are not simple lights.
					SortedLightInfo->SortKey.Fields.bIsNotSimpleLight = 1;

					// Lights handled by Mega Lights
					const bool bHandledByMegaLights = MegaLights::GetMegaLightsMode(ViewFamily, LightSceneInfoCompact.LightType, LightSceneInfoCompact.bAllowMegaLights, LightSceneInfoCompact.MegaLightsShadowMethod) != EMegaLightsMode::Disabled;

					const bool bCastsFirstPersonSelfShadow = LightCastsFirstPersonSelfShadow(*LightSceneInfo);

					// NOTE: bClusteredDeferredSupported==false means "lights cannot be batched" (tiled or clustered). When false, light will go the slower unbatched render path.
					// Tiled and clustered deferred lighting only support certain lights that don't use any additional features (like shadow or light function not compatible with the atlas.)
					// And also that are not directional (mostly because it doesn't make so much sense to insert them into every grid cell in the universe).
					// In the forward case one directional light gets put into its own variables, and in the deferred case it gets a full-screen pass.
					// Usually it'll have shadows and stuff anyway.
					// Contact shadow are not supported. Same for first person self shadow.
					const bool bClusteredDeferredSupported =
						(!SortedLightInfo->SortKey.Fields.bShadowed || bShadowedLightsInClustered) &&
						(!SortedLightInfo->SortKey.Fields.bLightFunction || (bUseLightFunctionAtlas && SortedLightInfo->bIsCompatibleWithLightFunctionAtlas)) // If not compatible with the LightFunctionAtlas, light with light function materials must go the unbatched route.
						&& LightSceneInfoCompact.LightType != LightType_Directional
						&& LightSceneInfo->Proxy->GetContactShadowLength() == 0
						&& !bCastsFirstPersonSelfShadow
						&& !bHandledByMegaLights;

					// Track feature available accross all lights
					if (SortedLightInfo->SortKey.Fields.LightType == LightType_Rect)	{ OutSortedLights.bHasRectLights = true; }
					if (SortedLightInfo->SortKey.Fields.bUsesLightingChannels)			{ OutSortedLights.bHasLightChannels = true; }
					if (SortedLightInfo->SortKey.Fields.bLightFunction) 				{ OutSortedLights.bHasLightFunctions = true; }
					if (bCastsFirstPersonSelfShadow) 									{ OutSortedLights.bHasFirstPersonSelfShadowLights = true; }

					SortedLightInfo->SortKey.Fields.bClusteredDeferredNotSupported = !bClusteredDeferredSupported;

					if (bHandledByMegaLights)
					{
						SortedLightInfo->SortKey.Fields.LightSceneId = LightSceneInfo->Id.GetIndex() & LIGHT_ID_MASK;
						SortedLightInfo->SortKey.Fields.bHandledByMegaLights = true;
					}

					break;
				}
			}
		}
	}
	// Add the simple lights also
	for (int32 SimpleLightIndex = 0; SimpleLightIndex < SimpleLights.InstanceData.Num(); SimpleLightIndex++)
	{
#if ENABLE_DEBUG_DISCARD_PROP
		{
			int32 PrevCounter = int32(DebugDiscardCounter);
			DebugDiscardCounter += DebugDiscardStride;
			if (PrevCounter >= int32(DebugDiscardCounter))
			{
				continue;
			}
		}
#endif // ENABLE_DEBUG_DISCARD_PROP

		const int32 MegaLightsSimpleLightMode = CVarMegaLightsSimpleLightMode.GetValueOnRenderThread();
		const bool bLightAllowsMegaLights = MegaLightsSimpleLightMode != 0 && (MegaLightsSimpleLightMode == 2 || SimpleLights.InstanceData[SimpleLightIndex].bAllowMegaLights);
		const bool bHandledByMegaLights = MegaLights::GetMegaLightsMode(ViewFamily, LightType_Point, bLightAllowsMegaLights, EMegaLightsShadowMethod::Default) != EMegaLightsMode::Disabled;

		FSortedLightSceneInfo* SortedLightInfo = new(SortedLights) FSortedLightSceneInfo(SimpleLightIndex);
		SortedLightInfo->SortKey.Fields.LightType = LightType_Point;
		SortedLightInfo->SortKey.Fields.bShadowed = 0;
		SortedLightInfo->SortKey.Fields.bLightFunction = 0;
		SortedLightInfo->SortKey.Fields.bUsesLightingChannels = 0;

		// These are simple lights.
		SortedLightInfo->SortKey.Fields.bIsNotSimpleLight = 0;

		// Simple lights are ok to use with tiled and clustered deferred lighting unless they are handled by MegaLights
		SortedLightInfo->SortKey.Fields.bClusteredDeferredNotSupported = bHandledByMegaLights;
		SortedLightInfo->SortKey.Fields.bHandledByMegaLights = bHandledByMegaLights;
	}

	// Sort non-shadowed, non-light function lights first to avoid render target switches.
	struct FCompareFSortedLightSceneInfo
	{
		FORCEINLINE bool operator()( const FSortedLightSceneInfo& A, const FSortedLightSceneInfo& B ) const
		{
			return A.SortKey.Packed < B.SortKey.Packed;
		}
	};
	SortedLights.Sort( FCompareFSortedLightSceneInfo() );

	// Scan and find ranges.
	OutSortedLights.SimpleLightsEnd = SortedLights.Num();
	OutSortedLights.ClusteredSupportedEnd = SortedLights.Num();
	OutSortedLights.UnbatchedLightStart = SortedLights.Num();
	OutSortedLights.MegaLightsLightStart = SortedLights.Num();
	OutSortedLights.MegaLightsSimpleLightsEnd = SortedLights.Num();

	// Iterate over all lights to be rendered and build ranges for tiled deferred and unshadowed lights
	for (int32 LightIndex = 0; LightIndex < SortedLights.Num(); LightIndex++)
	{
		const FSortedLightSceneInfo& SortedLightInfo = SortedLights[LightIndex];
		const bool bDrawShadows = SortedLightInfo.SortKey.Fields.bShadowed;
		const bool bLightingChannels = bAnyViewbUsesLightingChannels && SortedLightInfo.SortKey.Fields.bUsesLightingChannels;

		// Do not schedule unbatched lights if the atlas is used and enabled.
		// Keep in mind that when clustered shading is enabled, lights with light function are clustered compatible so are pushed in to the clustered path (if a supported light type).
		// Otherwise, those light are pushed in the non-shadowed non-lightfunction render path that still supports light function atlas sampling.
		const bool bDrawLightFunction = SortedLightInfo.SortKey.Fields.bLightFunction && (!bUseLightFunctionAtlas || !SortedLightInfo.bIsCompatibleWithLightFunctionAtlas);

		if (SortedLightInfo.SortKey.Fields.bHandledByMegaLights && OutSortedLights.MegaLightsLightStart == SortedLights.Num())
		{
			// Mark the first index that needs to be rendered
			OutSortedLights.MegaLightsLightStart = LightIndex;
		}

		if (SortedLightInfo.SortKey.Fields.bHandledByMegaLights && SortedLightInfo.SortKey.Fields.bIsNotSimpleLight && OutSortedLights.MegaLightsSimpleLightsEnd == SortedLights.Num())
		{
			// Mark one past the last index of simple lights handled by MegaLights
			OutSortedLights.MegaLightsSimpleLightsEnd = LightIndex;
		}

		if ((SortedLightInfo.SortKey.Fields.bIsNotSimpleLight || SortedLightInfo.SortKey.Fields.bHandledByMegaLights) && OutSortedLights.SimpleLightsEnd == SortedLights.Num())
		{
			// Mark the first index to not be simple
			OutSortedLights.SimpleLightsEnd = LightIndex;
		}

		if (SortedLightInfo.SortKey.Fields.bClusteredDeferredNotSupported && OutSortedLights.ClusteredSupportedEnd == SortedLights.Num())
		{
			// Mark the first index to not support clustered deferred
			OutSortedLights.ClusteredSupportedEnd = LightIndex;
		}

		if ((bDrawShadows || bDrawLightFunction || bLightingChannels) 
			&& SortedLightInfo.SortKey.Fields.bClusteredDeferredNotSupported
			&& !SortedLightInfo.SortKey.Fields.bHandledByMegaLights
			&& OutSortedLights.UnbatchedLightStart == SortedLights.Num())
		{
			OutSortedLights.UnbatchedLightStart = LightIndex;
		}
	}

	// Make sure no obvious things went wrong!
	check(OutSortedLights.ClusteredSupportedEnd >= OutSortedLights.SimpleLightsEnd);
	check(OutSortedLights.UnbatchedLightStart >= OutSortedLights.ClusteredSupportedEnd);
	check(OutSortedLights.MegaLightsSimpleLightsEnd >= OutSortedLights.MegaLightsLightStart);

	if (OutSortedLights.UnbatchedLightStart > OutSortedLights.MegaLightsLightStart)
	{
		OutSortedLights.UnbatchedLightStart = OutSortedLights.MegaLightsLightStart;
	}
}

FHairStrandsTransmittanceMaskData CreateDummyHairStrandsTransmittanceMaskData(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap);

void FDeferredShadingSceneRenderer::RenderLights(
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	FRDGTextureRef LightingChannelsTexture,
	const FSortedLightSetSceneInfo& SortedLightSet)
{
	FSubstrateSceneData& SubstrateSceneData = *SceneTextures.SubstrateSceneData.Get();

	const bool bUseHairLighting = HairStrands::HasViewHairStrandsData(Views);
#if RHI_RAYTRACING
	const bool bEnableRayTracing = true;
#else
	const bool bEnableRayTracing = false;
#endif // RHI_RAYTRACING
	const bool bUseLightFunctionAtlas = LightFunctionAtlas::IsEnabled(*Scene, ELightFunctionAtlasSystem::DeferredLighting);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, Lights, "Lights");

	SCOPED_NAMED_EVENT(FDeferredShadingSceneRenderer_RenderLights, FColor::Emerald);
	SCOPE_CYCLE_COUNTER(STAT_LightingDrawTime);
	SCOPE_CYCLE_COUNTER(STAT_LightRendering);

	const TArray<FSortedLightSceneInfo, SceneRenderingAllocator>& SortedLights = SortedLightSet.SortedLights;
	const int32 SimpleLightsEnd = SortedLightSet.SimpleLightsEnd;
	const int32 UnbatchedLightStart = SortedLightSet.UnbatchedLightStart;
	const int32 MegaLightsLightStart = SortedLightSet.MegaLightsLightStart;

	FHairStrandsTransmittanceMaskData DummyTransmittanceMaskData;
	if (bUseHairLighting && Views.Num() > 0)
	{
		DummyTransmittanceMaskData = CreateDummyHairStrandsTransmittanceMaskData(GraphBuilder, Views[0].ShaderMap);
	}

	{
		RDG_EVENT_SCOPE(GraphBuilder, "DirectLighting");

		// Update the stencil buffer, marking single/simple/complex Substrate material once for all passes (lights, env., ...)
		if (ViewFamily.EngineShowFlags.DirectLighting && Substrate::IsSubstrateEnabled() && Substrate::GetStencilPassStage() == Substrate::EStencilPassStage::Standalone)
		{
			Substrate::AddSubstrateStencilPass(GraphBuilder, Scene, Views, SceneTextures, SubstrateSceneData);
		}

		// The shadow scene renderer is not optional in the deferred renderer
		FShadowSceneRenderer& ShadowSceneRenderer = GetSceneExtensionsRenderers().GetRenderer<FShadowSceneRenderer>();

		if(ViewFamily.EngineShowFlags.DirectLighting)
		{
			ShadowSceneRenderer.RenderVirtualShadowMapProjectionMaskBits(GraphBuilder, SceneTextures);

			RDG_EVENT_SCOPE(GraphBuilder, "BatchedLights");
			INC_DWORD_STAT_BY(STAT_NumBatchedLights, UnbatchedLightStart);

			// Currently they have a special path anyway in case of standard deferred so always skip the simple lights
			int32 StandardDeferredStart = SortedLightSet.SimpleLightsEnd;

			bool bRenderSimpleLightsStandardDeferred = SortedLightSet.SimpleLights.InstanceData.Num() > 0;

			UE_CLOGF(ShouldUseClusteredDeferredShading(ViewFamily.GetShaderPlatform()) && !AreLightsInLightGrid(), LogRenderer, Warning,
				"Clustered deferred shading is enabled, but lights were not injected in grid, falling back to other methods (hint 'r.LightCulling.Quality' may cause this).");

			// True if the clustered shading is enabled and the feature level is there, and that the light grid had lights injected.
			if (ShouldUseClusteredDeferredShading(ViewFamily.GetShaderPlatform()) && AreLightsInLightGrid())
			{
				// Tell the trad. deferred that the clustered deferred capable lights are taken care of.
				// This includes the simple lights
				StandardDeferredStart = SortedLightSet.ClusteredSupportedEnd;
				// Tell the trad. deferred that the simple lights are spoken for.
				bRenderSimpleLightsStandardDeferred = false;

				AddClusteredDeferredShadingPass(GraphBuilder, SceneTextures, SortedLightSet, ShadowSceneRenderer.VirtualShadowMapMaskBits, ShadowSceneRenderer.VirtualShadowMapMaskBitsHairStrands, LightingChannelsTexture);
			}

			if (bRenderSimpleLightsStandardDeferred)
			{
				RenderSimpleLightsStandardDeferred(GraphBuilder, SceneTextures, SortedLightSet, SubstrateSceneData);
			}

			const bool bHasBatchedLights = StandardDeferredStart < UnbatchedLightStart;
			if (bHasBatchedLights)
			{
			
			// Update the stencil buffer, marking single/simple/complex Substrate material only if not already done.
			if (Substrate::IsSubstrateEnabled() && Substrate::GetStencilPassStage() == Substrate::EStencilPassStage::StandaloneAdaptive)
			{
				Substrate::AddSubstrateStencilPass(GraphBuilder, Scene, Views, SceneTextures, SubstrateSceneData);
			}

			// Draw non-shadowed non-light function lights without changing render targets between them
			for (int32 ViewIndex = 0, ViewCount = Views.Num(); ViewIndex < ViewCount; ++ViewIndex)
			{
				const FViewInfo& View = Views[ViewIndex];
				RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, ViewCount > 1, "View%d", ViewIndex);
				SCOPED_GPU_MASK(GraphBuilder.RHICmdList, View.GPUMask);

				if (Substrate::IsSubstrateEnabled())
				{					
					// Make substrate tile type the outermost sort criteria to minimize shader switching
					ESubstrateTileType TileType = ESubstrateTileType::EComplexSpecial;
					bool bFirstPass = true;
					while(true)
					{
						if (Substrate::GetSubstrateUsesTileType(View, TileType))
						{
							for (int32 LightIndex = StandardDeferredStart; LightIndex < UnbatchedLightStart; LightIndex++)
							{
								// Render the light to the scene color buffer, using a 1x1 white texture as input
								const FSortedLightSceneInfo& SortedLightInfo = SortedLights[LightIndex];
								const FLightSceneInfo* LightSceneInfo = SortedLightInfo.LightSceneInfo;
								RenderLight(GraphBuilder, Scene, View, SceneTextures, LightSceneInfo, nullptr, LightingChannelsTexture,
									false /*bRenderOverlap*/, false /*bCloudShadow*/, SortedLightInfo.bIsCompatibleWithLightFunctionAtlas,
									nullptr, nullptr, INDEX_NONE, TileType, bFirstPass);
								bFirstPass = false;
							}
						}
						if (TileType == ESubstrateTileType::ESimple)
							break;
						
						TileType = ESubstrateTileType(TileType - 1u);
					}
				}
				else
				{
					for (int32 LightIndex = StandardDeferredStart; LightIndex < UnbatchedLightStart; LightIndex++)
					{
						// Render the light to the scene color buffer, using a 1x1 white texture as input
						const FSortedLightSceneInfo& SortedLightInfo = SortedLights[LightIndex];
						const FLightSceneInfo* LightSceneInfo = SortedLightInfo.LightSceneInfo;
						RenderLight(GraphBuilder, Scene, View, SceneTextures, LightSceneInfo, nullptr, LightingChannelsTexture,
							false /*bRenderOverlap*/, false /*bCloudShadow*/, SortedLightInfo.bIsCompatibleWithLightFunctionAtlas);
					}
				}
			}

			// Add a special version when hair rendering is enabled for getting lighting on hair. 
			if (bUseHairLighting)
			{
				FRDGTextureRef NullScreenShadowMaskSubPixelTexture = nullptr;
				for (int32 ViewIndex = 0, ViewCount = Views.Num(); ViewIndex < ViewCount; ++ViewIndex)
				{
					FViewInfo& View = Views[ViewIndex];
					if (HairStrands::HasViewHairStrandsData(View))
					{
						// Draw non-shadowed non-light function lights without changing render targets between them
						for (int32 LightIndex = StandardDeferredStart; LightIndex < UnbatchedLightStart; LightIndex++)
						{
							const FSortedLightSceneInfo& SortedLightInfo = SortedLights[LightIndex];
							const FLightSceneInfo* LightSceneInfo = SortedLightInfo.LightSceneInfo;
							RenderLightForHair(GraphBuilder, View, SceneTextures, LightSceneInfo,
								NullScreenShadowMaskSubPixelTexture, LightingChannelsTexture, DummyTransmittanceMaskData,
								false /*bForwardRendering*/, SortedLightInfo.bIsCompatibleWithLightFunctionAtlas);
						}
					}
				}
			}
			} // bHasBatchedLights
		}

		{
			RDG_EVENT_SCOPE(GraphBuilder, "UnbatchedLights");

			const int32 DenoiserMode = CVarShadowUseDenoiser.GetValueOnRenderThread();

			const IScreenSpaceDenoiser* DefaultDenoiser = IScreenSpaceDenoiser::GetDefaultDenoiser();
			const IScreenSpaceDenoiser* DenoiserToUse = DenoiserMode == 1 ? DefaultDenoiser : GScreenSpaceDenoiser;

			TArray<FRDGTextureRef, SceneRenderingAllocator> PreprocessedShadowMaskTextures;
			TArray<FRDGTextureRef, SceneRenderingAllocator> PreprocessedShadowMaskSubPixelTextures;

			const int32 MaxDenoisingBatchSize = FMath::Clamp(CVarMaxShadowDenoisingBatchSize.GetValueOnRenderThread(), 1, IScreenSpaceDenoiser::kMaxBatchSize);
			const int32 MaxRTShadowBatchSize = CVarMaxShadowRayTracingBatchSize.GetValueOnRenderThread();
			const bool bDoShadowDenoisingBatching = DenoiserMode != 0 && MaxDenoisingBatchSize > 1;

			//#dxr_todo: support multiview for the batching case
			const bool bDoShadowBatching = (bDoShadowDenoisingBatching || MaxRTShadowBatchSize > 1) && Views.Num() == 1;

			// Optimizations: batches all shadow ray tracing denoising. Definitely could be smarter to avoid high VGPR pressure if this entire
			// function was converted to render graph, and want least intrusive change as possible. So right now it trades render target memory pressure
			// for denoising perf.
			if (bEnableRayTracing && bDoShadowBatching)
			{
				const uint32 ViewIndex = 0;
				FViewInfo& View = Views[ViewIndex];

				const int32 NumShadowedLights = MegaLightsLightStart - UnbatchedLightStart;
				// Allocate PreprocessedShadowMaskTextures once so QueueTextureExtraction can deferred write.
				{
					if (!View.bStatePrevViewInfoIsReadOnly)
					{
						View.ViewState->PrevFrameViewInfo.ShadowHistories.Empty();
						View.ViewState->PrevFrameViewInfo.ShadowHistories.Reserve(NumShadowedLights);
					}

					PreprocessedShadowMaskTextures.SetNum(NumShadowedLights);
				}

				PreprocessedShadowMaskTextures.SetNum(NumShadowedLights);

				if (HairStrands::HasViewHairStrandsData(View))
				{ 
					PreprocessedShadowMaskSubPixelTextures.SetNum(NumShadowedLights);
				}
			}

			const bool bDirectLighting = ViewFamily.EngineShowFlags.DirectLighting;

			FFirstPersonSelfShadowInputs FirstPersonSelfShadowInputs;
			if (SortedLightSet.bHasFirstPersonSelfShadowLights && ShouldRenderFirstPersonSelfShadow(ViewFamily))
			{
				FirstPersonSelfShadowInputs = CreateFirstPersonSelfShadowInputs(GraphBuilder, Views, SceneTextures);
			}

			FRDGTextureRef SharedScreenShadowMaskTexture = nullptr;
			FRDGTextureRef SharedScreenShadowMaskSubPixelTexture = nullptr;

			// Draw shadowed and light function lights
			auto UnbatchedLightsPass = [&](bool bIsHairPass)
			{
			for (int32 LightIndex = UnbatchedLightStart; LightIndex < MegaLightsLightStart; LightIndex++)
			{
				const FSortedLightSceneInfo& SortedLightInfo = SortedLights[LightIndex];
				const FLightSceneInfo& LightSceneInfo = *SortedLightInfo.LightSceneInfo;
				const FLightSceneProxy& LightSceneProxy = *LightSceneInfo.Proxy;
				const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightSceneInfo.Id];
				const ELightOcclusionType OcclusionType = GetLightOcclusionType(LightSceneProxy, ViewFamily);

				const bool bDrawShadows = SortedLightInfo.SortKey.Fields.bShadowed;
				const bool bDrawLightFunction = SortedLightInfo.SortKey.Fields.bLightFunction && (!bUseLightFunctionAtlas || !SortedLightInfo.bIsCompatibleWithLightFunctionAtlas);
				const bool bDrawPreviewIndicator = ViewFamily.EngineShowFlags.PreviewShadowsIndicator && !LightSceneInfo.IsPrecomputedLightingValid() && LightSceneProxy.HasStaticShadowing();
				const bool bDrawHairShadow = bDrawShadows && bUseHairLighting;
				const bool bUseHairDeepShadow = bDrawShadows && bUseHairLighting && LightSceneProxy.CastsHairStrandsDeepShadow();
				const bool bRunHairLighting = bUseHairLighting && (bIsHairPass || OcclusionType == ELightOcclusionType::Raytraced);
				const bool bDrawFirstPersonSelfShadow = bDrawShadows && !bIsHairPass && ShouldRenderFirstPersonSelfShadowForLight(*this, ViewFamily, Views, LightSceneInfo);
				const bool bShouldHeterogeneousVolumesCastShadows = ShouldHeterogeneousVolumesCastShadows();

				bool bUsedShadowMaskTexture = false;

				bool bElideScreenShadowMask = false;
				bool bElideScreenShadowMaskSubPixel = false;

				// Raytraced shadow light for hair are handled/interleaved with regular light, as the raytraced shadow masks 
				// for gbuffer & hair are computed by the same single pass.
				if (bIsHairPass && OcclusionType == ELightOcclusionType::Raytraced)
				{
					continue;
				}

				FScopeCycleCounter Context(LightSceneProxy.GetStatId());

				FRDGTextureRef ScreenShadowMaskTexture = nullptr;
				FRDGTextureRef ScreenShadowMaskSubPixelTexture = nullptr;

				if (bDrawShadows || bDrawLightFunction || bDrawPreviewIndicator)
				{
					// In certain cases we can skip creating the screen shadow mask texture
					// In particular right now this is true if we are doing one pass projection with only a virtual shadow map
					// with no light functions, as in that case we can directly sample the shadow mask bits in the lighting shader.
					bElideScreenShadowMask = 
						CVarOnePassProjectionSkipScreenShadowMask.GetValueOnRenderThread() != 0 &&
						ShadowSceneRenderer.UsePackedShadowMaskBits() &&
						OcclusionType == ELightOcclusionType::Shadowmap &&
						!(bDirectLighting && bDrawLightFunction) &&
						!bDrawPreviewIndicator &&
						SortedLightInfo.SortKey.Fields.LightType != LightType_Directional &&
						VisibleLightInfo.ContainsOnlyVirtualShadowMaps() &&
						!bDrawFirstPersonSelfShadow &&
						!bShouldHeterogeneousVolumesCastShadows;
					bElideScreenShadowMaskSubPixel = bElideScreenShadowMask && CVarHairStrandsAllowOneTransmittancePass.GetValueOnRenderThread() > 0;

					if (!SharedScreenShadowMaskTexture || !SharedScreenShadowMaskSubPixelTexture)
					{
						const FRDGTextureDesc SharedScreenShadowMaskTextureDesc(FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PF_B8G8R8A8, FClearValueBinding::White, TexCreate_RenderTargetable | TexCreate_ShaderResource | GFastVRamConfig.ScreenSpaceShadowMask));
						
						if (!SharedScreenShadowMaskTexture && !bElideScreenShadowMask)
						{
							SharedScreenShadowMaskTexture = GraphBuilder.CreateTexture(SharedScreenShadowMaskTextureDesc, TEXT("ShadowMaskTexture"));
						}
						if (!SharedScreenShadowMaskSubPixelTexture && bRunHairLighting && !bElideScreenShadowMaskSubPixel)
						{
							SharedScreenShadowMaskSubPixelTexture = GraphBuilder.CreateTexture(SharedScreenShadowMaskTextureDesc, TEXT("ShadowMaskSubPixelTexture"));
						}
					}
					ScreenShadowMaskTexture = bElideScreenShadowMask ? nullptr : SharedScreenShadowMaskTexture;
					ScreenShadowMaskSubPixelTexture = bElideScreenShadowMaskSubPixel ? nullptr : SharedScreenShadowMaskSubPixelTexture;
				}

				FString LightNameWithLevel;
				GetLightNameForDrawEvent(&LightSceneProxy, LightNameWithLevel);
				RDG_EVENT_SCOPE(GraphBuilder, "%s", *LightNameWithLevel);

				if (bDrawShadows)
				{
					INC_DWORD_STAT(STAT_NumShadowedLights);

					// Inline ray traced shadow batching, launches shadow batches when needed
					// reduces memory overhead while keeping shadows batched to optimize costs
#if RHI_RAYTRACING
					{
						const uint32 ViewIndex = 0;
						FViewInfo& View = Views[ViewIndex];

						IScreenSpaceDenoiser::FShadowRayTracingConfig RayTracingConfig;
						RayTracingConfig.RayCountPerPixel = GShadowRayTracingSamplesPerPixel > -1? GShadowRayTracingSamplesPerPixel : LightSceneProxy.GetSamplesPerPixel();

						const bool bDenoiserCompatible = !LightRequiresDenoiser(LightSceneInfo) || IScreenSpaceDenoiser::EShadowRequirements::PenumbraAndClosestOccluder == DenoiserToUse->GetShadowRequirements(View, LightSceneInfo, RayTracingConfig);

						const bool bWantsBatchedShadow = OcclusionType == ELightOcclusionType::Raytraced &&
							bDoShadowBatching &&
							bDenoiserCompatible &&
							SortedLightInfo.SortKey.Fields.bShadowed;

						// determine if this light doesn't yet have a precomputed shadow and execute a batch to amortize costs if one is needed
						if (bWantsBatchedShadow &&
							(PreprocessedShadowMaskTextures.Num() == 0 || !PreprocessedShadowMaskTextures[LightIndex - UnbatchedLightStart]))
						{
							RDG_EVENT_SCOPE(GraphBuilder, "ShadowBatch");
							TStaticArray<IScreenSpaceDenoiser::FShadowVisibilityParameters, IScreenSpaceDenoiser::kMaxBatchSize> DenoisingQueue;
							TStaticArray<int32, IScreenSpaceDenoiser::kMaxBatchSize> LightIndices;

							FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures.UniformBuffer);

							int32 ProcessShadows = 0;

							const auto QuickOffDenoisingBatch = [&]
							{
								int32 InputParameterCount = 0;
								for (int32 i = 0; i < IScreenSpaceDenoiser::kMaxBatchSize; i++)
								{
									InputParameterCount += DenoisingQueue[i].LightSceneInfo != nullptr ? 1 : 0;
								}

								check(InputParameterCount >= 1);

								TStaticArray<IScreenSpaceDenoiser::FShadowVisibilityOutputs, IScreenSpaceDenoiser::kMaxBatchSize> Outputs;

								RDG_EVENT_SCOPE(GraphBuilder, "%s%s(Shadow BatchSize=%d) %dx%d",
									DenoiserToUse != DefaultDenoiser ? TEXT("ThirdParty ") : TEXT(""),
									DenoiserToUse->GetDebugName(),
									InputParameterCount,
									View.ViewRect.Width(), View.ViewRect.Height());

								DenoiserToUse->DenoiseShadowVisibilityMasks(
									GraphBuilder,
									View,
									&View.PrevViewInfo,
									SceneTextureParameters,
									DenoisingQueue,
									InputParameterCount,
									Outputs);

								for (int32 i = 0; i < InputParameterCount; i++)
								{
									const FLightSceneInfo* LocalLightSceneInfo = DenoisingQueue[i].LightSceneInfo;

									int32 LocalLightIndex = LightIndices[i];
									FRDGTextureRef& RefDestination = PreprocessedShadowMaskTextures[LocalLightIndex - UnbatchedLightStart];
									check(RefDestination == nullptr);
									RefDestination = Outputs[i].Mask;
									DenoisingQueue[i].LightSceneInfo = nullptr;
								}
							}; // QuickOffDenoisingBatch

							// Ray trace shadows of lights, and quick off denoising batch.
							for (int32 LightBatchIndex = LightIndex; LightBatchIndex < MegaLightsLightStart; LightBatchIndex++)
							{
								const FSortedLightSceneInfo& BatchSortedLightInfo = SortedLights[LightBatchIndex];
								const FLightSceneInfo& BatchLightSceneInfo = *BatchSortedLightInfo.LightSceneInfo;

								// Denoiser does not support texture rect light important sampling.
								const bool bBatchDrawShadows = BatchSortedLightInfo.SortKey.Fields.bShadowed;

								if (!bBatchDrawShadows)
								{
									continue;
								}

								const ELightOcclusionType BatchOcclusionType = GetLightOcclusionType(*BatchLightSceneInfo.Proxy, ViewFamily);
								if (BatchOcclusionType != ELightOcclusionType::Raytraced)
								{
									continue;
								}

								const bool bRequiresDenoiser = LightRequiresDenoiser(BatchLightSceneInfo) && DenoiserMode > 0;

								IScreenSpaceDenoiser::FShadowRayTracingConfig BatchRayTracingConfig;
								BatchRayTracingConfig.RayCountPerPixel = GShadowRayTracingSamplesPerPixel > -1 ? GShadowRayTracingSamplesPerPixel : BatchLightSceneInfo.Proxy->GetSamplesPerPixel();

								IScreenSpaceDenoiser::EShadowRequirements DenoiserRequirements = bRequiresDenoiser ?
									DenoiserToUse->GetShadowRequirements(View, BatchLightSceneInfo, BatchRayTracingConfig) :
									IScreenSpaceDenoiser::EShadowRequirements::Bailout;

								// Not worth batching and increase memory pressure if the denoiser do not support this ray tracing config.
								// TODO: add suport for batch with multiple SPP.
								if (bRequiresDenoiser && DenoiserRequirements != IScreenSpaceDenoiser::EShadowRequirements::PenumbraAndClosestOccluder)
								{
									continue;
								}

								// Ray trace the shadow.
								//#dxr_todo: support multiview for the batching case
								FRDGTextureRef RayTracingShadowMaskTexture;
								{
									FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
										SceneTextures.Config.Extent,
										PF_FloatRGBA,
										FClearValueBinding::Black,
										TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
									RayTracingShadowMaskTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingOcclusion"));
								}

								FRDGTextureRef RayDistanceTexture;
								{
									FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
										SceneTextures.Config.Extent,
										PF_R16F,
										FClearValueBinding::Black,
										TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
									RayDistanceTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingOcclusionDistance"));
								}

								FRDGTextureRef SubPixelRayTracingShadowMaskTexture = nullptr;
								FRDGTextureUAV* SubPixelRayTracingShadowMaskUAV = nullptr;
								if (bUseHairLighting)
								{
									FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
										SceneTextures.Config.Extent,
										PF_FloatRGBA,
										FClearValueBinding::Black,
										TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
									SubPixelRayTracingShadowMaskTexture = GraphBuilder.CreateTexture(Desc, TEXT("SubPixelRayTracingOcclusion"));
									SubPixelRayTracingShadowMaskUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SubPixelRayTracingShadowMaskTexture));
								}

								FString BatchLightNameWithLevel;
								GetLightNameForDrawEvent(BatchLightSceneInfo.Proxy, BatchLightNameWithLevel);

								FRDGTextureUAV* RayTracingShadowMaskUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(RayTracingShadowMaskTexture));
								FRDGTextureUAV* RayHitDistanceUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(RayDistanceTexture));
								{
									RDG_EVENT_SCOPE(GraphBuilder, "%s", *BatchLightNameWithLevel);

									// Ray trace the shadow cast by opaque geometries on to hair strands geometries
									// Note: No denoiser is required on this output, as the hair strands are geometrically noisy, which make it hard to denoise
									RenderRayTracingShadows(
										GraphBuilder,
										SceneTextureParameters,
										View,
										BatchLightSceneInfo,
										BatchRayTracingConfig,
										DenoiserRequirements,
										LightingChannelsTexture,
										RayTracingShadowMaskUAV,
										RayHitDistanceUAV,
										SubPixelRayTracingShadowMaskUAV);
									
									if (HairStrands::HasViewHairStrandsData(View))
									{
										FRDGTextureRef& RefDestination = PreprocessedShadowMaskSubPixelTextures[LightBatchIndex - UnbatchedLightStart];
										check(RefDestination == nullptr);
										RefDestination = SubPixelRayTracingShadowMaskTexture;
									}
								}

								bool bBatchFull = false;

								if (bRequiresDenoiser)
								{
									// Queue the ray tracing output for shadow denoising.
									for (int32 i = 0; i < IScreenSpaceDenoiser::kMaxBatchSize; i++)
									{
										if (DenoisingQueue[i].LightSceneInfo == nullptr)
										{
											DenoisingQueue[i].LightSceneInfo = &BatchLightSceneInfo;
											DenoisingQueue[i].RayTracingConfig = RayTracingConfig;
											DenoisingQueue[i].InputTextures.Mask = RayTracingShadowMaskTexture;
											DenoisingQueue[i].InputTextures.ClosestOccluder = RayDistanceTexture;
											LightIndices[i] = LightBatchIndex;

											// If queue for this light type is full, quick of the batch.
											if ((i + 1) == MaxDenoisingBatchSize)
											{
												QuickOffDenoisingBatch();
												bBatchFull = true;
											}
											break;
										}
										else
										{
											check((i - 1) < IScreenSpaceDenoiser::kMaxBatchSize);
										}
									}
								}
								else
								{
									PreprocessedShadowMaskTextures[LightBatchIndex - UnbatchedLightStart] = RayTracingShadowMaskTexture;
								}

								// terminate batch if we filled a denoiser batch or hit our max light batch
								ProcessShadows++;
								if (bBatchFull || ProcessShadows == MaxRTShadowBatchSize)
								{
									break;
								}
							}

							// Ensures all denoising queues are processed.
							if (DenoisingQueue[0].LightSceneInfo)
							{
								QuickOffDenoisingBatch();
							}
						}
					} // end inline batched raytraced shadow

					if (PreprocessedShadowMaskTextures.Num() > 0 && PreprocessedShadowMaskTextures[LightIndex - UnbatchedLightStart])
					{
						const uint32 ShadowMaskIndex = LightIndex - UnbatchedLightStart;
						ScreenShadowMaskTexture = PreprocessedShadowMaskTextures[ShadowMaskIndex];
						PreprocessedShadowMaskTextures[ShadowMaskIndex] = nullptr;

						// Sub-pixel shadow for hair strands geometries
						if (bRunHairLighting && ShadowMaskIndex < uint32(PreprocessedShadowMaskSubPixelTextures.Num()))
						{
							ScreenShadowMaskSubPixelTexture = PreprocessedShadowMaskSubPixelTextures[ShadowMaskIndex];
							PreprocessedShadowMaskSubPixelTextures[ShadowMaskIndex] = nullptr;
						}

						// Inject deep shadow mask if the light supports it
						if (bUseHairDeepShadow)
						{
							RenderHairStrandsDeepShadowMask(GraphBuilder, Views, &LightSceneInfo, VisibleLightInfos, ScreenShadowMaskTexture);
						}
					}
					else 
#endif // RHI_RAYTRACING
					if (OcclusionType == ELightOcclusionType::Raytraced)
					{
						FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures.UniformBuffer);

						FRDGTextureRef RayTracingShadowMaskTexture;
						{
							FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
								SceneTextures.Config.Extent,
								PF_FloatRGBA,
								FClearValueBinding::Black,
								TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
							RayTracingShadowMaskTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingOcclusion"));
						}

						FRDGTextureRef RayDistanceTexture;
						{
							FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
								SceneTextures.Config.Extent,
								PF_R16F,
								FClearValueBinding::Black,
								TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
							RayDistanceTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingOcclusionDistance"));
						}

						FRDGTextureUAV* RayTracingShadowMaskUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(RayTracingShadowMaskTexture));
						FRDGTextureUAV* RayHitDistanceUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(RayDistanceTexture));

						FRDGTextureRef SubPixelRayTracingShadowMaskTexture = nullptr;
						FRDGTextureUAV* SubPixelRayTracingShadowMaskUAV = nullptr;
						if (bRunHairLighting)
						{
							FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
								SceneTextures.Config.Extent,
								PF_FloatRGBA,
								FClearValueBinding::Black,
								TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
							SubPixelRayTracingShadowMaskTexture = GraphBuilder.CreateTexture(Desc, TEXT("SubPixelRayTracingOcclusion"));
							SubPixelRayTracingShadowMaskUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SubPixelRayTracingShadowMaskTexture));
						}

						FRDGTextureRef RayTracingShadowMaskTileTexture;
						{
							FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
								SceneTextures.Config.Extent,
								PF_FloatRGBA,
								FClearValueBinding::Black,
								TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
							RayTracingShadowMaskTileTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingOcclusionTile"));
						}

						bool bIsMultiview = Views.Num() > 0;

						for (FViewInfo& View : Views)
						{
							RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

							IScreenSpaceDenoiser::FShadowRayTracingConfig RayTracingConfig;
							RayTracingConfig.RayCountPerPixel = GShadowRayTracingSamplesPerPixel > -1 ? GShadowRayTracingSamplesPerPixel : LightSceneProxy.GetSamplesPerPixel();

							IScreenSpaceDenoiser::EShadowRequirements DenoiserRequirements = IScreenSpaceDenoiser::EShadowRequirements::Bailout;
							if (DenoiserMode != 0 && LightRequiresDenoiser(LightSceneInfo))
							{
								DenoiserRequirements = DenoiserToUse->GetShadowRequirements(View, LightSceneInfo, RayTracingConfig);
							}

							RenderRayTracingShadows(
								GraphBuilder,
								SceneTextureParameters,
								View,
								LightSceneInfo,
								RayTracingConfig,
								DenoiserRequirements,
								LightingChannelsTexture,
								RayTracingShadowMaskUAV,
								RayHitDistanceUAV,
								SubPixelRayTracingShadowMaskUAV);

							if (DenoiserRequirements != IScreenSpaceDenoiser::EShadowRequirements::Bailout)
							{
								TStaticArray<IScreenSpaceDenoiser::FShadowVisibilityParameters, IScreenSpaceDenoiser::kMaxBatchSize> InputParameters;
								TStaticArray<IScreenSpaceDenoiser::FShadowVisibilityOutputs, IScreenSpaceDenoiser::kMaxBatchSize> Outputs;

								InputParameters[0].InputTextures.Mask = RayTracingShadowMaskTexture;
								InputParameters[0].InputTextures.ClosestOccluder = RayDistanceTexture;
								InputParameters[0].LightSceneInfo = &LightSceneInfo;
								InputParameters[0].RayTracingConfig = RayTracingConfig;

								int32 InputParameterCount = 1;

								RDG_EVENT_SCOPE(GraphBuilder, "%s%s(Shadow BatchSize=%d) %dx%d",
									DenoiserToUse != DefaultDenoiser ? TEXT("ThirdParty ") : TEXT(""),
									DenoiserToUse->GetDebugName(),
									InputParameterCount,
									View.ViewRect.Width(), View.ViewRect.Height());

								DenoiserToUse->DenoiseShadowVisibilityMasks(
									GraphBuilder,
									View,
									&View.PrevViewInfo,
									SceneTextureParameters,
									InputParameters,
									InputParameterCount,
									Outputs);

								if (bIsMultiview)
								{
									AddDrawTexturePass(GraphBuilder, View, Outputs[0].Mask, RayTracingShadowMaskTileTexture, View.ViewRect.Min, View.ViewRect.Min, View.ViewRect.Size());
									ScreenShadowMaskTexture = RayTracingShadowMaskTileTexture;
								}
								else
								{
									ScreenShadowMaskTexture = Outputs[0].Mask;
								}
							}
							else
							{
								ScreenShadowMaskTexture = RayTracingShadowMaskTexture;
							}

							if (HairStrands::HasViewHairStrandsData(View))
							{
								ScreenShadowMaskSubPixelTexture = SubPixelRayTracingShadowMaskTexture;
							}
						}

						// Inject deep shadow mask if the light supports it
						if (bUseHairDeepShadow)
						{
							RenderHairStrandsShadowMask(GraphBuilder, Views, &LightSceneInfo, VisibleLightInfos, false /*bForwardShading*/, ScreenShadowMaskTexture);
						}
					}
					else // (OcclusionType == FOcclusionType::Shadowmap)
					{					
						const auto ClearShadowMask = [&](FRDGTextureRef InScreenShadowMaskTexture)
						{
							// Clear light attenuation for local lights with a quad covering their extents
							const bool bClearLightScreenExtentsOnly = CVarAllowClearLightSceneExtentsOnly.GetValueOnRenderThread() && SortedLightInfo.SortKey.Fields.LightType != LightType_Directional;

							if (bClearLightScreenExtentsOnly)
							{
								FRenderTargetParameters* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
								PassParameters->RenderTargets[0] = FRenderTargetBinding(InScreenShadowMaskTexture, ERenderTargetLoadAction::ENoAction);

								GraphBuilder.AddPass(
									RDG_EVENT_NAME("ClearQuad"),
									PassParameters,
									ERDGPassFlags::Raster,
									[this, &LightSceneProxy](FRDGAsyncTask, FRHICommandList& RHICmdList)
								{
									for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
									{
										const FViewInfo& View = Views[ViewIndex];
										SCOPED_GPU_MASK(RHICmdList, View.GPUMask);

										FIntRect ScissorRect;
										if (!LightSceneProxy.GetScissorRect(ScissorRect, View, View.ViewRect))
										{
											ScissorRect = View.ViewRect;
										}

										if (ScissorRect.Min.X < ScissorRect.Max.X && ScissorRect.Min.Y < ScissorRect.Max.Y)
										{
											RHICmdList.SetViewport(ScissorRect.Min.X, ScissorRect.Min.Y, 0.0f, ScissorRect.Max.X, ScissorRect.Max.Y, 1.0f);
											DrawClearQuad(RHICmdList, true, FLinearColor(1, 1, 1, 1), false, 0, false, 0);
										}
										else
										{
											LightSceneProxy.GetScissorRect(ScissorRect, View, View.ViewRect);
										}
									}
								});
							}
							else
							{
								AddClearRenderTargetPass(GraphBuilder, InScreenShadowMaskTexture);
							}
						};

						// Skip clearing if it doesn't exist (due to being elided by VSM projection)
						if (ScreenShadowMaskTexture)
						{
							ClearShadowMask(ScreenShadowMaskTexture);
						}
						if (ScreenShadowMaskSubPixelTexture)
						{
							ClearShadowMask(ScreenShadowMaskSubPixelTexture);
						}

						if (bIsHairPass)
						{
							RenderDeferredShadowProjections(GraphBuilder, SceneTextures, &LightSceneInfo, nullptr, ScreenShadowMaskSubPixelTexture);
						}
						else
						{
							RenderDeferredShadowProjections(GraphBuilder, SceneTextures, &LightSceneInfo, ScreenShadowMaskTexture, nullptr);
						}
					}

					// First person self shadow
					if (bDrawFirstPersonSelfShadow)
					{
						RenderFirstPersonSelfShadow(GraphBuilder, *this, Views, ScreenShadowMaskTexture, FirstPersonSelfShadowInputs, LightSceneInfo);
					}

					if (bShouldHeterogeneousVolumesCastShadows && ShouldRenderVolumetricShadowMaskForLight(GraphBuilder, Views, &LightSceneInfo))
					{
						RenderVolumetricShadowMaskForLight(GraphBuilder, SceneTextures.Config, Views, LightSceneInfo.Type, &LightSceneInfo, ScreenShadowMaskTexture);
					}

					bUsedShadowMaskTexture = true;
				} // if (bDrawShadows)

				// Render light function to the attenuation buffer.
				if (bDirectLighting)
				{
					if (bDrawLightFunction)
					{
						if (!bIsHairPass)
						{
							const bool bLightFunctionRendered = RenderLightFunction(GraphBuilder, SceneTextures, &LightSceneInfo, ScreenShadowMaskTexture, bDrawShadows, false, false);
							bUsedShadowMaskTexture |= bLightFunctionRendered;
						}
						
						// * When using non-raytraced shadows, the light function for hair is evaluated during the hair pass (bIsHairPass=true).
						// * When using ray traced shadows, hair lighting runs in the non-hair pass (bIsHairPass=false).
						//   The light function must also be rendered to ScreenShadowMaskSubPixelTexture for hair,
						//   since non-atlas-compatible light functions (e.g. those using AbsoluteWorldPosition)
						//   are not sampled via the light function atlas in the hair deferred light shader.
						if (bRunHairLighting && CVarAppliedLightFunctionOnHair.GetValueOnRenderThread() > 0 && ScreenShadowMaskSubPixelTexture)
						{
							const bool bLightFunctionRendered = RenderLightFunction(GraphBuilder, SceneTextures, &LightSceneInfo, ScreenShadowMaskSubPixelTexture, bDrawShadows, false, true);
							bUsedShadowMaskTexture |= bLightFunctionRendered;
						}
					}

					if (bDrawPreviewIndicator)
					{
						bUsedShadowMaskTexture |= RenderPreviewShadowsIndicator(GraphBuilder, SceneTextures, &LightSceneInfo, ScreenShadowMaskTexture, bUsedShadowMaskTexture, false);
					}

					if (!bDrawShadows)
					{
						INC_DWORD_STAT(STAT_NumLightFunctionOnlyLights);
					}
				}

				// If we never rendered into the mask, don't attempt to read from it.
				if (!bUsedShadowMaskTexture)
				{
					ScreenShadowMaskTexture = nullptr;
					ScreenShadowMaskSubPixelTexture = nullptr;
				}

				// Render the light to the scene color buffer, conditionally using the attenuation buffer or a 1x1 white texture as input 
				if (bDirectLighting && !bIsHairPass)
				{
					// Update the stencil buffer, marking single/simple/complex Substrate material only if not already done.
					if (Substrate::IsSubstrateEnabled() && Substrate::GetStencilPassStage() == Substrate::EStencilPassStage::StandaloneAdaptive)
					{
						Substrate::AddSubstrateStencilPass(GraphBuilder, Scene, Views, SceneTextures, SubstrateSceneData);
					}

					for (int32 ViewIndex = 0, ViewCount = Views.Num(); ViewIndex < ViewCount; ++ViewIndex)
					{
						const FViewInfo& View = Views[ViewIndex];

						// If the light elided the screen space shadow mask, sample directly from the packed shadow mask
						int32 VirtualShadowMapId = INDEX_NONE;
						if (bElideScreenShadowMask)
						{
							INC_DWORD_STAT(STAT_VSMLocalProjectionOnePassFast);
							VirtualShadowMapId = VisibleLightInfo.GetVirtualShadowMapId(&View);
						}

						RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, ViewCount > 1, "View%d", ViewIndex);
						SCOPED_GPU_MASK(GraphBuilder.RHICmdList, View.GPUMask);
						RenderLight(
							GraphBuilder, Scene, View, SceneTextures, &LightSceneInfo, 
							VirtualShadowMapId != INDEX_NONE ? nullptr : ScreenShadowMaskTexture, LightingChannelsTexture, false /*bRenderOverlap*/, true /*bCloudShadow*/, SortedLightInfo.bIsCompatibleWithLightFunctionAtlas,
							VirtualShadowMapArray.GetUniformBuffer(ViewIndex), ShadowSceneRenderer.VirtualShadowMapMaskBits, VirtualShadowMapId);
					}
				}

				if (bRunHairLighting)
				{
					for (int32 ViewIndex = 0, ViewCount = Views.Num(); ViewIndex < ViewCount; ++ViewIndex)
					{
						FViewInfo& View = Views[ViewIndex];

						if (HairStrands::HasViewHairStrandsData(View))
						{
							// If the light elided the screen space shadow mask, sample directly from the packed shadow mask
							// Note: this is only used when VSM one pass is enable AND hair one pass transmittance is enabled. 
							// Hence the condition using bElideScreenShadowMaskSubPixel, instead of bElideScreenShadowMask for 
							// computing VirtualShadowMapId
							int32 VirtualShadowMapId = INDEX_NONE;
							if (bElideScreenShadowMaskSubPixel)
							{
								INC_DWORD_STAT(STAT_VSMLocalProjectionOnePassFast);
								VirtualShadowMapId = VisibleLightInfo.GetVirtualShadowMapId(&View);
							}

							FHairStrandsTransmittanceMaskData TransmittanceMaskData;
							FRDGTextureRef HairShadowMask = nullptr;
							if (bDrawHairShadow && VirtualShadowMapId != INDEX_NONE)
							{
								if (ShadowSceneRenderer.HairTransmittanceMaskBits.IsValidIndex(ViewIndex))
								{
									TransmittanceMaskData.TransmittanceMask = ShadowSceneRenderer.HairTransmittanceMaskBits[ViewIndex];
								}
								HairShadowMask = nullptr;
								check(ScreenShadowMaskSubPixelTexture == nullptr);
							}
							else if (bDrawHairShadow)
							{
								TransmittanceMaskData = RenderHairStrandsTransmittanceMask(GraphBuilder, View, ViewIndex, &LightSceneInfo, false, ScreenShadowMaskSubPixelTexture);
								HairShadowMask = ScreenShadowMaskSubPixelTexture;
							}
							if (TransmittanceMaskData.TransmittanceMask == nullptr)
							{
								TransmittanceMaskData = DummyTransmittanceMaskData;
							}

							// Note: ideally the light should still be evaluated for hair when not casting shadow, but for preserving the old behavior, and not adding 
							// any perf. regression, we disable this light for hair rendering 
							RenderLightForHair(
								GraphBuilder, View, SceneTextures, &LightSceneInfo,
								VirtualShadowMapId != INDEX_NONE ? nullptr : HairShadowMask, LightingChannelsTexture, TransmittanceMaskData, 
								false /*bForwardRendering*/, SortedLightInfo.bIsCompatibleWithLightFunctionAtlas,
								VirtualShadowMapArray.GetUniformBuffer(ViewIndex), ShadowSceneRenderer.VirtualShadowMapMaskBitsHairStrands, VirtualShadowMapId);
						}
					}
				}
			}};

			// Two separate light loop:
			// * For GBuffer inputs (and hair input for light having RT shadows)
			// * For Hair inputs if any hair data are present
			UnbatchedLightsPass(false/*bIsHairPass*/);
			if (bUseHairLighting)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "UnbatchedLights(Hair)");
				UnbatchedLightsPass(true/*bIsHairPass*/);
			}
		}

		if (Substrate::IsSubstrateEnabled() && IsExperimentalToonUnifiedDiffuseEnabled())
		{
			for (int32 ViewIndex = 0, ViewCount = Views.Num(); ViewIndex < ViewCount; ++ViewIndex)
			{
				FViewInfo& View = Views[ViewIndex];

				FDeferredLightApplyToonDiffusePS::FParameters* ToonPassParameters = GraphBuilder.AllocParameters<FDeferredLightApplyToonDiffusePS::FParameters>();
				// PS - General parameters
				ToonPassParameters->View = View.ViewUniformBuffer;
				ToonPassParameters->SceneTextures = SceneTextures.UniformBuffer;
				ToonPassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

				//	const FSubstrateSceneData& SubstrateSceneData = *View.GetSceneTextures().SubstrateSceneData;
				ToonPassParameters->ToonDataTexture0 = SubstrateSceneData.SubstrateSceneToonData0;
				ToonPassParameters->ToonDataTexture1 = SubstrateSceneData.SubstrateSceneToonData1;

				ToonPassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
				if (SceneTextures.Depth.Target)
				{
					ToonPassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Resolve, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
				}

				FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();

				FDeferredLightVS::FPermutationDomain PermutationVectorVS;
				PermutationVectorVS.Set<FDeferredLightVS::FRadialLight>(false);
				TShaderMapRef<FDeferredLightVS> VertexShader(View.ShaderMap, PermutationVectorVS);

				Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableDebug >(false);
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector >(true);
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FClassificationTileSize16 >(Substrate::UseTileSize16());
				TShaderMapRef<Substrate::FSubstrateTilePassVS> TileVertexShader(View.ShaderMap, VSPermutationVector);

				for (uint32 TileType = ESubstrateTileType::EComplex; TileType <= ESubstrateTileType::EComplexSpecial; ++TileType)
				{
					const ESubstrateTileType SubstrateTileType = ESubstrateTileType(TileType);

					if (!Substrate::GetSubstrateUsesTileType(View, SubstrateTileType))
					{
						continue;
					}
					const bool bEnableSubstrateTiledPass = SubstrateTileType != ESubstrateTileType::ECount && Substrate::IsSubstrateEnabled() && Substrate::AreLightingPassesTiled();

					FDeferredLightApplyToonDiffusePS::FPermutationDomain ToonPermutationVector;
					ToonPermutationVector.Set<Substrate::FSubstrateTileType>(SubstrateTileType);
					TShaderMapRef<FDeferredLightApplyToonDiffusePS> ToonPixelShader(View.ShaderMap, ToonPermutationVector);

					GraphBuilder.AddPass(
						RDG_EVENT_NAME("Light::ApplyToonDiffuse"),
						ToonPassParameters,
						ERDGPassFlags::Raster,
						[&View, ToonPixelShader, ToonPassParameters, SubstrateTileType, BlendState, VertexShader, TileVertexShader, bEnableSubstrateTiledPass](FRDGAsyncTask, FRHICommandList& RHICmdList)
					{
						FGraphicsPipelineStateInitializer GraphicsPSOInit;
						RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

						RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
						RHICmdList.SetStreamSource(0, nullptr, 0);

						uint32 StencilRef = 0;
						Substrate::GetSubstrateTiledDepthStencilStateRead(SubstrateTileType, CF_Always, GraphicsPSOInit, StencilRef);
						GraphicsPSOInit.PrimitiveType = PT_TriangleList;
						GraphicsPSOInit.BlendState = BlendState;
						GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
						GraphicsPSOInit.bDepthBounds = false;
						GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
						GraphicsPSOInit.BoundShaderState.VertexShaderRHI = bEnableSubstrateTiledPass ? TileVertexShader.GetVertexShader() : VertexShader.GetVertexShader();
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ToonPixelShader.GetPixelShader();

						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

						SetShaderParameters(RHICmdList, ToonPixelShader, ToonPixelShader.GetPixelShader(), *ToonPassParameters);

						if (bEnableSubstrateTiledPass)
						{
							check(Substrate::IsSubstrateEnabled());
							Substrate::FSubstrateTilePassVS::FParameters VSParameters = Substrate::SetTileParameters(View, SubstrateTileType, GraphicsPSOInit.PrimitiveType);
							SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), VSParameters);
							// Render tiles for current complexity only, with stencil test.
							RHICmdList.DrawPrimitiveIndirect(VSParameters.TileIndirectBuffer->GetIndirectRHICallBuffer(), Substrate::TileTypeDrawIndirectArgOffset(SubstrateTileType));
						}
						else
						{
							FDeferredLightVS::FParameters VSParameters2 = FDeferredLightVS::GetParameters(View);
							SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParameters2);
							// Render as full screen rectangle only leveraging stencil test for better performance.
							DrawRectangle(
								RHICmdList,
								0, 0,
								View.ViewRect.Width(), View.ViewRect.Height(),
								View.ViewRect.Min.X, View.ViewRect.Min.Y,
								View.ViewRect.Width(), View.ViewRect.Height(),
								View.ViewRect.Size(),
								View.GetSceneTexturesConfig().Extent,
								VertexShader,
								EDRF_UseTriangleOptimization);
						}
					});
				} // for each TileType
			} // for each View
		}
	}

	GraphBuilder.FlushSetupQueue();
}

static void RenderLightArrayForOverlapViewmode(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const TArray<FViewInfo>& Views,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef LightingChannelsTexture,
	const FSceneLightInfoArray& LightArray)
{
	for (const FLightSceneInfoCompact& LightSceneInfoCompact : LightArray)
	{
		const FLightSceneInfo* LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;

		// Nothing to do for black lights.
		if (LightSceneInfoCompact.Color.IsAlmostBlack())
		{
			continue;
		}

		// Only render shadow casting stationary lights
		if (!LightSceneInfo->Proxy->HasStaticShadowing() ||
			 LightSceneInfo->Proxy->HasStaticLighting()  ||
			!LightSceneInfo->Proxy->CastsStaticShadow())
		{
			continue;
		}

		// Check if the light is visible in any of the views.
		for (const FViewInfo& View : Views)
		{
			SCOPED_GPU_MASK(GraphBuilder.RHICmdList, View.GPUMask);
			RenderLight(GraphBuilder, Scene, View, SceneTextures, LightSceneInfo, nullptr, LightingChannelsTexture, true /*bRenderOverlap*/, false /*bCloudShadow*/, false/*bIsCompatibleWithLightFunctionAtlas*/);
		}
	}
}

void FDeferredShadingSceneRenderer::RenderStationaryLightOverlap(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef LightingChannelsTexture)
{
	if (Scene->bIsEditorScene)
	{
		// Clear to discard base pass values in scene color since we didn't skip that, to have valid scene depths
		AddClearRenderTargetPass(GraphBuilder, SceneTextures.Color.Target, FLinearColor::Black);

		RenderLightArrayForOverlapViewmode(GraphBuilder, Scene, Views, SceneTextures, LightingChannelsTexture, Scene->Lights);

		//Note: making use of FScene::InvisibleLights, which contains lights that haven't been added to the scene in the same way as visible lights
		// So code called by RenderLightArrayForOverlapViewmode must be careful what it accesses
		RenderLightArrayForOverlapViewmode(GraphBuilder, Scene, Views, SceneTextures, LightingChannelsTexture, Scene->InvisibleLights);
	}
}

static void InternalSetBoundingGeometryRasterizerState(FGraphicsPipelineStateInitializer& GraphicsPSOInit, bool bReverseCulling, bool bCameraInsideLightGeometry)
{
	if (bCameraInsideLightGeometry)
	{
		// Render backfaces with depth tests disabled since the camera is inside (or close to inside) the light geometry
		GraphicsPSOInit.RasterizerState = bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
	}
	else
	{
		// Render frontfaces with depth tests on to get the speedup from HiZ since the camera is outside the light geometry
		GraphicsPSOInit.RasterizerState = bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();
	}
}


void GetSubstrateTiledDepthStencilStateNoDepthTest(ESubstrateTileType InTileType, FGraphicsPipelineStateInitializer& GraphicsPSOInit, uint32& OutStencilRef)
{
	return Substrate::GetSubstrateTiledDepthStencilStateRead(InTileType, CF_Always, GraphicsPSOInit, OutStencilRef);
}

template<ECompareFunction CompareFunction>
static uint32 InternalSetBoundingGeometryDepthState(FGraphicsPipelineStateInitializer& GraphicsPSOInit, ESubstrateTileType TileType)
{
	// bCameraInsideLightGeometry = true  -> CompareFunction = Always
	// bCameraInsideLightGeometry = false -> CompareFunction = CF_DepthNearOrEqual
	uint32 StencilRef = 0u;
	if (TileType != ESubstrateTileType::ECount)
	{
		Substrate::GetSubstrateTiledDepthStencilStateRead(TileType, CompareFunction, GraphicsPSOInit, StencilRef);
	}
	else
	{
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CompareFunction>::GetRHI();
	}
	return StencilRef;
}

/** Sets up rasterizer and depth state for rendering bounding geometry in a deferred pass. */
static uint32 SetBoundingGeometryRasterizerAndDepthState(FGraphicsPipelineStateInitializer& GraphicsPSOInit, bool bReverseCulling, bool bCameraInsideLightGeometry, ESubstrateTileType TileType)
{
	uint32 StencilRef = 0u;
	InternalSetBoundingGeometryRasterizerState(GraphicsPSOInit, bReverseCulling, bCameraInsideLightGeometry);
	if (bCameraInsideLightGeometry)
	{
		StencilRef = InternalSetBoundingGeometryDepthState<CF_Always>(GraphicsPSOInit, TileType);
	}
	else
	{
		StencilRef = InternalSetBoundingGeometryDepthState<CF_DepthNearOrEqual>(GraphicsPSOInit, TileType);
	}

	return StencilRef;
}

// Use DBT to allow work culling on shadow lights
static void CalculateLightNearFarDepthFromBounds(const FViewInfo& View, const FSphere& LightBounds, float& NearDepth, float& FarDepth)
{
	const FMatrix ViewProjection = View.ViewMatrices.GetWorldToClip();
	const FVector ViewDirection = View.GetViewDirection();

	// push camera relative bounds center along view vec by its radius
	const FVector FarPoint = LightBounds.Center + LightBounds.W * ViewDirection;
	const FVector4 FarPoint4 = FVector4(FarPoint, 1.f);
	const FVector4 FarPoint4Clip = ViewProjection.TransformFVector4(FarPoint4);
	FarDepth = FarPoint4Clip.Z / FarPoint4Clip.W;

	// pull camera relative bounds center along -view vec by its radius
	const FVector NearPoint = LightBounds.Center - LightBounds.W * ViewDirection;
	const FVector4 NearPoint4 = FVector4(NearPoint, 1.f);
	const FVector4 NearPoint4Clip = ViewProjection.TransformFVector4(NearPoint4);
	NearDepth = NearPoint4Clip.Z / NearPoint4Clip.W;

	// negative means behind view, but we use a NearClipPlane==1.f depth

	if (NearPoint4Clip.W < 0)
		NearDepth = 1;

	if (FarPoint4Clip.W < 0)
		FarDepth = 1;

	NearDepth = FMath::Clamp(NearDepth, 0.0f, 1.0f);
	FarDepth = FMath::Clamp(FarDepth, 0.0f, 1.0f);

}

static TRDGUniformBufferRef<FDeferredLightUniformStruct> CreateDeferredLightUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FLightSceneInfo& LightSceneInfo)
{
	auto* DeferredLightStruct = GraphBuilder.AllocParameters<FDeferredLightUniformStruct>();
	*DeferredLightStruct = GetDeferredLightParameters(View, LightSceneInfo, LightFunctionAtlas::IsEnabled(View, ELightFunctionAtlasSystem::DeferredLighting));
	return GraphBuilder.CreateUniformBuffer(DeferredLightStruct);
}

static TRDGUniformBufferRef<FDeferredLightUniformStruct> CreateDeferredLightUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FSimpleLightEntry& SimpleLight, const FVector& SimpleLightPosition)
{
	auto* DeferredLightStruct = GraphBuilder.AllocParameters<FDeferredLightUniformStruct>();
	*DeferredLightStruct = GetSimpleDeferredLightParameters(View, SimpleLight, SimpleLightPosition);
	return GraphBuilder.CreateUniformBuffer(DeferredLightStruct);
}

static bool MayRenderSubstrateToon(ESubstrateTileType SubstrateTileType)
{
	static const bool bSubstrateEnable = Substrate::IsSubstrateEnabled();						// Not runtime toggleable
	static const bool bToonUnifiedDiffuseEnable = IsExperimentalToonUnifiedDiffuseEnabled();	// Not runtime toggleable
	return bSubstrateEnable && bToonUnifiedDiffuseEnable &&
		(SubstrateTileType == ESubstrateTileType::EComplex || SubstrateTileType == ESubstrateTileType::EComplexSpecial);	// Complex and ComplexSpecial tiles maybe contain pixels with toon shading.
}

static FDeferredLightPS::FParameters GetDeferredLightPSParameters(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformBuffer,
	TRDGUniformBufferRef<FHairStrandsViewUniformParameters> HairStrandsUniformBuffer,
	FRDGTextureRef ShadowMaskTexture,
	FRDGTextureRef LightingChannelsTexture,
	bool bCloudShadow,
	TRDGUniformBufferRef<FVirtualShadowMapUniformParameters> VirtualShadowMapUniformBuffer = nullptr,
	FRDGTextureRef ShadowMaskBits = nullptr,
	int32 VirtualShadowMapId = INDEX_NONE)
{
	FDeferredLightPS::FParameters Out;

	const ELightComponentType LightType = (ELightComponentType)LightSceneInfo->Proxy->GetLightType();
	const bool bIsDirectional = LightType == LightType_Directional;

	FRDGTextureRef WhiteDummy = GSystemTextures.GetWhiteDummy(GraphBuilder);
	FRDGBufferRef BufferDummy = GSystemTextures.GetDefaultBuffer(GraphBuilder, 4, 0u);
	FRDGBufferSRVRef BufferDummySRV = GraphBuilder.CreateSRV(BufferDummy, PF_R32_UINT);

	// PS - General parameters
	const FVolumetricCloudRenderSceneInfo* CloudInfo = bCloudShadow ? Scene->GetVolumetricCloudSceneInfo() : nullptr;
	Out.SceneTextures = SceneTexturesUniformBuffer;
	Out.HairStrands = HairStrandsUniformBuffer;
	Out.ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	Out.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	Out.bLightingChannelEnabled = LightingChannelsTexture ? 1u : 0u;
	Out.LightingChannelsTexture = LightingChannelsTexture ? LightingChannelsTexture : GSystemTextures.GetZeroUIntDummy(GraphBuilder);
	Out.LightingChannelsSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Out.CloudShadowAO = GetCloudShadowAOParameters(GraphBuilder, View, CloudInfo);
	Out.bCloudShadowEnabled = SetupLightCloudTransmittanceParameters(GraphBuilder, Scene, View, LightSceneInfo, Out.CloudShadow) ? 1 : 0;
	Out.bAtmosphereTransmittanceEnabled = IsLightAtmospherePerPixelTransmittanceEnabled(Scene, View, LightSceneInfo) ? 1 : 0;
	Out.LightAttenuationTexture = ShadowMaskTexture ? ShadowMaskTexture : WhiteDummy;
	Out.LightAttenuationTextureSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	Out.View = View.ViewUniformBuffer;
	Out.DeferredLight = CreateDeferredLightUniformBuffer(GraphBuilder, View, *LightSceneInfo);
	// PS - Hair (default value)
	Out.ScreenShadowMaskSubPixelTexture = WhiteDummy;
	Out.HairTransmittanceBuffer = BufferDummySRV;
	Out.HairTransmittanceBufferMaxCount = 0;
	Out.HairShadowMaskValid = false;
	Out.ShadowChannelMask = FVector4f(1, 1, 1, 1);
	// PS - One pass projection
	Out.VirtualShadowMap = VirtualShadowMapUniformBuffer;
	Out.VirtualShadowMapId = VirtualShadowMapId;
	Out.ShadowMaskBits = ShadowMaskBits ? ShadowMaskBits : GSystemTextures.GetZeroUIntDummy(GraphBuilder);

	// If the light is not batched, it could be due to shadow, so we still specify light function atlas sampling.
	Out.LightFunctionAtlas = LightFunctionAtlas::BindGlobalParameters(GraphBuilder, View);

	// PS - Render Targets
	Out.RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad);
	if (Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()) && Substrate::UsesSubstrateMaterialBuffer(View.GetShaderPlatform()))
	{
		const FSubstrateSceneData& SubstrateSceneData = *View.GetSceneTextures().SubstrateSceneData;
		Out.RenderTargets[1] = FRenderTargetBinding(SubstrateSceneData.SeparatedOpaqueRoughRefractionSceneColor, ERenderTargetLoadAction::ELoad);
		Out.RenderTargets[2] = FRenderTargetBinding(SubstrateSceneData.SeparatedSubSurfaceSceneColor, ERenderTargetLoadAction::ELoad);
	}
	else if (UseSeparatedSubsurfaceDiffuse(View.GetShaderPlatform()))
	{
		FRDGTextureRef SeparatedSubsurfaceDiffuse = View.GetSceneTextures().SeparatedSubsurfaceDiffuse;
		Out.RenderTargets[1] = FRenderTargetBinding(SeparatedSubsurfaceDiffuse, ERenderTargetLoadAction::ELoad);
	}

	if (SceneDepthTexture)
	{
		Out.RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);
	}

	if (!VirtualShadowMapUniformBuffer)
	{
		Out.VirtualShadowMap = FVirtualShadowMapArray::CreateDummySamplingParameters(GraphBuilder).VirtualShadowMap;
		Out.VirtualShadowMapId = INDEX_NONE;
	}

	return Out;
}

uint32 SetupLightGraphicsPSOState( 
	bool bDirectional,
	bool bCameraInsideLightGeometry, 
	bool bReverseCulling, 
	ESubstrateTileType SubstrateTileMaterialType,
	FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	EShaderPlatform ShaderPlatform)
{
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	if (MayRenderSubstrateToon(SubstrateTileMaterialType))
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<
			CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
			CW_RGB,  BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
			CW_RG,   BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();

		// As of today, OpaqueRoughRefraction cannot be used together with experimental toon features. This is reflected in the implementation of IsOpaqueRoughRefractionEnabled().
		check(!Substrate::IsOpaqueRoughRefractionEnabled(ShaderPlatform));
	}
	else if (Substrate::IsOpaqueRoughRefractionEnabled(ShaderPlatform))
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<
			CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
			CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
			CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
	}
	else if (UseSeparatedSubsurfaceDiffuse(ShaderPlatform))
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<
			CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One,
			CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
	}
	else
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
	}

	uint32 StencilRef = 0;
	if (bDirectional)
	{
		// Turn DBT back off
		GraphicsPSOInit.bDepthBounds = false;
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;

		if(SubstrateTileMaterialType != ESubstrateTileType::ECount && Substrate::IsSubstrateEnabled() && !Substrate::AreLightingPassesTiled())
		{
			// Directional light only: when not tiled, we rely on the stencil test to choose which pixel to process in a pass.
			StencilRef = InternalSetBoundingGeometryDepthState<CF_Always>(GraphicsPSOInit, SubstrateTileMaterialType);
		}
		else
		{
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		}
	}
	else
	{
		// Use DBT to allow work culling on shadow lights
		// Disable depth bound when hair rendering is enabled as this rejects partially covered pixel write (with opaque background)
		GraphicsPSOInit.bDepthBounds = GSupportsDepthBoundsTest && GAllowDepthBoundsTest != 0;

		StencilRef = SetBoundingGeometryRasterizerAndDepthState(GraphicsPSOInit, bReverseCulling, bCameraInsideLightGeometry, SubstrateTileMaterialType);
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
	}

	return StencilRef;
}

TShaderMapRef<FDeferredLightPS> GetUberDeferredLightPS(FGlobalShaderMap* GlobalShaderMap, EShaderPlatform ShaderPlatform, const FDeferredLightPS::FPermutationDomain& PermutationVector)
{
	FDeferredLightPS::FPermutationDomain UberPermutationVector = PermutationVector;
	UberPermutationVector.Set < FDeferredLightPS::FUbershader > (true);
	UberPermutationVector = FDeferredLightPS::RemapPermutation(UberPermutationVector, ShaderPlatform);
	TShaderMapRef<FDeferredLightPS> UberPixelShader(GlobalShaderMap, UberPermutationVector);
	check(UberPixelShader.IsValid());
	return UberPixelShader;
}

void ValidatePSOPrecacheState(const FGraphicsPipelineStateInitializer& GraphicsPSOInit)
{
#if PSO_PRECACHING_VALIDATE
	if (PSOCollectorStats::IsFullPrecachingValidationEnabled())
	{
		static const int32 GlobalPSOCollectorIndex = FGlobalPSOCollectorManager::GetIndex(DeferredLightGlobalPSOCollectorName);
		PSOCollectorStats::CheckGlobalGraphicsPipelineStateInCache(GraphicsPSOInit, GlobalPSOCollectorIndex);
	}
#endif // PSO_PRECACHING_VALIDATE
}

// Used by RenderLights to render a light to the scene color buffer.
template<typename TShaderType, typename TParametersType>
static void InternalRenderLight(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FLightSceneInfo* LightSceneInfo,
	TShaderType& PixelShader,
	TShaderType& UberPixelShader,
	TParametersType* PassParameters,
	ESubstrateTileType SubstrateTileMaterialType,
	const TCHAR* ShaderName)
{
	const FLightSceneProxy* RESTRICT LightProxy = LightSceneInfo->Proxy;
	const bool bTransmission = LightProxy->Transmission();
	const FSphere LightBounds = LightProxy->GetBoundingSphere();
	const ELightComponentType LightType = (ELightComponentType)LightProxy->GetLightType();

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("RenderLight %s: %s", ShaderName, *LightProxy->GetOwnerNameOrLabel()),
		PassParameters,
		ERDGPassFlags::Raster,
		[Scene, &View, PixelShader, UberPixelShader, LightSceneInfo, PassParameters, LightBounds, LightType, SubstrateTileMaterialType](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		const bool bIsRadial = LightType != LightType_Directional;
		const bool bEnableSubstrateStencilTest = SubstrateTileMaterialType != ESubstrateTileType::ECount && bIsRadial;

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		const bool bCameraInsideLightGeometry = ((FVector)View.ViewMatrices.GetViewOrigin() - LightBounds.Center).SizeSquared() < FMath::Square(LightBounds.W * 1.05f + View.NearClippingDistance * 2.0f)
			//const bool bCameraInsideLightGeometry = LightProxy->AffectsBounds( FSphere( View.ViewMatrices.GetViewOrigin(), View.NearClippingDistance * 2.0f ) )
				// Always draw backfaces in ortho
				//@todo - accurate ortho camera / light intersection
			|| !View.IsPerspectiveProjection();

		const uint32 StencilRef = SetupLightGraphicsPSOState(LightType == LightType_Directional, bCameraInsideLightGeometry, View.bReverseCulling, SubstrateTileMaterialType, GraphicsPSOInit, View.GetShaderPlatform());

		// Set the device viewport for the view.
		RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

		if (LightType == LightType_Directional)
		{
			const bool bEnableSubstrateTiledPass = SubstrateTileMaterialType != ESubstrateTileType::ECount && Substrate::IsSubstrateEnabled() && Substrate::AreLightingPassesTiled();

			FDeferredLightVS::FPermutationDomain PermutationVectorVS;
			PermutationVectorVS.Set<FDeferredLightVS::FRadialLight>(false);
			TShaderMapRef<FDeferredLightVS> VertexShader(View.ShaderMap, PermutationVectorVS);

			Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
			VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableDebug >(false);
			VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector >(true);
			VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FClassificationTileSize16 >(Substrate::UseTileSize16());
			TShaderMapRef<Substrate::FSubstrateTilePassVS> TileVertexShader(View.ShaderMap, VSPermutationVector);

			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = bEnableSubstrateTiledPass ? TileVertexShader.GetVertexShader() : VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			TShaderType UsedPixelShader = PixelShader;
			if (!PrecachePSO(GraphicsPSOInit))
			{
				UsedPixelShader = UberPixelShader;
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = UsedPixelShader.GetPixelShader();
				ValidatePSOPrecacheState(GraphicsPSOInit);
			}

			Substrate::FSubstrateTilePassVS::FParameters VSParameters;
			if (Substrate::IsSubstrateEnabled())
			{
				VSParameters = Substrate::SetTileParameters(View, SubstrateTileMaterialType, GraphicsPSOInit.PrimitiveType);
			}

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);
			SetShaderParameters(RHICmdList, UsedPixelShader, UsedPixelShader.GetPixelShader(), PassParameters->PS);

			if (bEnableSubstrateTiledPass)
			{
				check(Substrate::IsSubstrateEnabled());
				SetShaderParameters(RHICmdList, TileVertexShader, TileVertexShader.GetVertexShader(), VSParameters);
				RHICmdList.DrawPrimitiveIndirect(VSParameters.TileIndirectBuffer->GetIndirectRHICallBuffer(), Substrate::TileTypeDrawIndirectArgOffset(SubstrateTileMaterialType));
			}
			else
			{
				FDeferredLightVS::FParameters VSParameters2 = FDeferredLightVS::GetParameters(View);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParameters2);

				// Apply the directional light as a full screen quad
				DrawRectangle(
					RHICmdList,
					0, 0,
					View.ViewRect.Width(), View.ViewRect.Height(),
					View.ViewRect.Min.X, View.ViewRect.Min.Y,	
					View.ViewRect.Width(), View.ViewRect.Height(),
					View.ViewRect.Size(),
					View.GetSceneTexturesConfig().Extent,
					VertexShader,
					EDRF_UseTriangleOptimization);
			}
		}
		else // Radial light (LightType_Point, LightType_Spot, LightType_Rect)
		{	
			FDeferredLightVS::FPermutationDomain PermutationVectorVS;
			PermutationVectorVS.Set<FDeferredLightVS::FRadialLight>(true);
			TShaderMapRef<FDeferredLightVS> VertexShader(View.ShaderMap, PermutationVectorVS);

			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			TShaderType UsedPixelShader = PixelShader;
			if (!PrecachePSO(GraphicsPSOInit))
			{
				UsedPixelShader = UberPixelShader;
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = UsedPixelShader.GetPixelShader();
				ValidatePSOPrecacheState(GraphicsPSOInit);
			}

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);
			SetShaderParameters(RHICmdList, UsedPixelShader, UsedPixelShader.GetPixelShader(), PassParameters->PS);

			FDeferredLightVS::FParameters VSParameters2 = FDeferredLightVS::GetParameters(View, LightSceneInfo);
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParameters2);

			// Use DBT to allow work culling on shadow lights
			if (GraphicsPSOInit.bDepthBounds)
			{
				// Can use the depth bounds test to skip work for pixels which won't be touched by the light (i.e outside the depth range)
				float NearDepth = 1.f;
				float FarDepth = 0.f;
				CalculateLightNearFarDepthFromBounds(View,LightBounds,NearDepth,FarDepth);
				if (NearDepth <= FarDepth)
				{
					NearDepth = 1.0f;
					FarDepth = 0.0f;
				}

				// UE uses reversed depth, so far < near
				RHICmdList.SetDepthBounds(FarDepth, NearDepth);
			}

			if( LightType == LightType_Point || LightType == LightType_Rect )
			{
				// Apply the point or spot light with some approximate bounding geometry,
				// So we can get speedups from depth testing and not processing pixels outside of the light's influence.
				StencilingGeometry::DrawSphere(RHICmdList);
			}
			else if (LightType == LightType_Spot)
			{
				StencilingGeometry::DrawCone(RHICmdList);
			}
		}
	});
}


/** Shader parameters for Standard Deferred Light Overlap Debug pass. */
BEGIN_SHADER_PARAMETER_STRUCT(FRenderLightParameters, )
	// PS/VS parameter structs
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightPS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightVS::FParameters, VS)
	// Substrate tiles
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateTileParameter, SubstrateTileSimple)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateTileParameter, SubstrateTileSingle)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateTileParameter, SubstrateTileComplex)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSubstrateTileParameter, SubstrateTileSpectialComplex)
END_SHADER_PARAMETER_STRUCT()

/** Shader parameters for Standard Deferred Light pass. */
BEGIN_SHADER_PARAMETER_STRUCT(FRenderLightOverlapParameters, )
	// PS/VS parameter structs
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightOverlapPS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightVS::FParameters, VS)
END_SHADER_PARAMETER_STRUCT()

static void RenderLight(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FMinimalSceneTextures& SceneTextures,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef ScreenShadowMaskTexture,
	FRDGTextureRef LightingChannelsTexture,
	bool bRenderOverlap,
	bool bCloudShadow,
	const bool bCanLightUsesAtlasForUnbatchedLight,
	TRDGUniformBufferRef<FVirtualShadowMapUniformParameters> VirtualShadowMapUniformBuffer,
	FRDGTextureRef ShadowMaskBits,
	int32 VirtualShadowMapId,
	ESubstrateTileType SubstrateTileType,
	bool bFirstPass)
{
	// Ensure the light is valid for this view
	if (!LightSceneInfo->ShouldRenderLight(View))
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_DirectLightRenderingTime);

	if (bFirstPass)
	{
		INC_DWORD_STAT(STAT_NumLightsUsingStandardDeferred);
	}
	
	const FLightSceneProxy* RESTRICT LightProxy = LightSceneInfo->Proxy;
	const bool bUseIESTexture = View.Family->EngineShowFlags.TexturedLightProfiles && (LightSceneInfo->Proxy->GetIESTextureResource() != 0);
	const bool bTransmission = LightProxy->Transmission();
	const FSphere LightBounds = LightProxy->GetBoundingSphere();
	const ELightComponentType LightType = (ELightComponentType)LightProxy->GetLightType();
	const bool bIsRadial = LightType != LightType_Directional;
	const bool bUseVirtualShadowMapMask = VirtualShadowMapId != INDEX_NONE && ShadowMaskBits;
	const bool bNeedComplexTransmittanceSupport = View.HairCardsMeshElements.Num() && FDeferredLightPS::SupportsHairComplexTransmittance(View.GetShaderPlatform());

	const bool bSupportAnisotropyPermutation = ShouldRenderAnisotropicLighting(View);

	check(!bUseVirtualShadowMapMask || bIsRadial);		// VSM mask only stores local lights

	// Debug Overlap shader
	if (bRenderOverlap)
	{
		FRenderLightOverlapParameters* PassParameters = GraphBuilder.AllocParameters<FRenderLightOverlapParameters>();
		// PS - General parameters
		PassParameters->PS.bHasValidChannel = LightSceneInfo->Proxy->GetPreviewShadowMapChannel() == INDEX_NONE ? 0.0f : 1.0f;
		PassParameters->PS.View = View.ViewUniformBuffer;
		PassParameters->PS.DeferredLight = CreateDeferredLightUniformBuffer(GraphBuilder, View, *LightSceneInfo);
		PassParameters->PS.SceneTextures = SceneTextures.UniformBuffer;
		PassParameters->PS.RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
		if (SceneTextures.Depth.Target)
		{
			PassParameters->PS.RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);
		}
		// VS - General parameters
		if (bIsRadial)
		{
			PassParameters->VS = FDeferredLightVS::GetParameters(View, LightSceneInfo, false);
		}
		else
		{
			PassParameters->VS = FDeferredLightVS::GetParameters(View, false);
		}

		FDeferredLightOverlapPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FDeferredLightOverlapPS::FRadialAttenuation>(bIsRadial);
		TShaderMapRef<FDeferredLightOverlapPS> PixelShader(View.ShaderMap, PermutationVector);		
		InternalRenderLight(GraphBuilder, Scene, View, LightSceneInfo, PixelShader, PixelShader, PassParameters, ESubstrateTileType::ECount, TEXT("Light::StandardDeferred(Overlap)"));
	}
	// Lighting shader
	else
	{
		FRenderLightParameters* PassParameters = GraphBuilder.AllocParameters<FRenderLightParameters>();
		// PS - Generatl parameters
		PassParameters->PS = GetDeferredLightPSParameters(
			GraphBuilder, Scene, View, LightSceneInfo, SceneTextures.Color.Target, SceneTextures.Depth.Target, SceneTextures.UniformBuffer, View.HairStrandsViewData.UniformBuffer, 
			ScreenShadowMaskTexture, LightingChannelsTexture, bCloudShadow, VirtualShadowMapUniformBuffer, ShadowMaskBits, VirtualShadowMapId);
		// VS - General parameters
		if (bIsRadial)
		{
			PassParameters->VS = FDeferredLightVS::GetParameters(View, LightSceneInfo, false);
		}
		else // Directional
		{
			PassParameters->VS = FDeferredLightVS::GetParameters(View, false);
		}
		// VS - Substrate tile parameters
		if (Substrate::IsSubstrateEnabled())
		{
			// Note: we register all tile types here in order to have all resources tracked properly and being able 
			//       to create a single pass parameters struct instead of created one for each tile types
			PassParameters->SubstrateTileSimple = Substrate::SetTileParameters(GraphBuilder, View, ESubstrateTileType::ESingle);
			PassParameters->SubstrateTileSingle = Substrate::SetTileParameters(GraphBuilder, View, ESubstrateTileType::ESimple);
			PassParameters->SubstrateTileComplex = Substrate::SetTileParameters(GraphBuilder, View, ESubstrateTileType::EComplex);
			PassParameters->SubstrateTileSpectialComplex = Substrate::SetTileParameters(GraphBuilder, View, ESubstrateTileType::EComplexSpecial);
		}

		FDeferredLightPS::FPermutationDomain PermutationVector;
		PermutationVector.Set< FDeferredLightPS::FTransmissionDim >(bTransmission);
		PermutationVector.Set< FDeferredLightPS::FHairLighting>(0);
		PermutationVector.Set< FDeferredLightPS::FLightingChannelsDim >(View.bUsesLightingChannels);
		PermutationVector.Set< FDeferredLightPS::FVisualizeCullingDim >(View.Family->EngineShowFlags.VisualizeLightCulling);
		PermutationVector.Set< FDeferredLightPS::FVirtualShadowMapMask >(bUseVirtualShadowMapMask);
		PermutationVector.Set< Substrate::FSubstrateTileType >(0);
		PermutationVector.Set< FDeferredLightPS::FHairComplexTransmittance >(bNeedComplexTransmittanceSupport);
		PermutationVector.Set< FDeferredLightPS::FLightFunctionAtlasDim >(
			LightFunctionAtlas::IsEnabled(View, ELightFunctionAtlasSystem::DeferredLighting) && LightSceneInfo->Proxy->HasValidLightFunctionAtlasSlot() &&
			LightSceneInfo->Proxy->GetLightFunctionMaterial() != nullptr && !View.Family->EngineShowFlags.VisualizeLightCulling && bCanLightUsesAtlasForUnbatchedLight);

		if (bIsRadial)
		{
			PermutationVector.Set< FDeferredLightPS::FSourceShapeDim >(LightProxy->IsRectLight() ? ELightSourceShape::Rect : ELightSourceShape::Capsule);
			PermutationVector.Set< FDeferredLightPS::FSourceTextureDim >(LightProxy->IsRectLight() && LightProxy->HasSourceTexture());
			PermutationVector.Set< FDeferredLightPS::FIESProfileDim >(bUseIESTexture);
			PermutationVector.Set< FDeferredLightPS::FAnistropicMaterials >(bSupportAnisotropyPermutation && !LightSceneInfo->Proxy->IsRectLight());
			PermutationVector.Set< FDeferredLightPS::FAtmosphereTransmittance >(false);
			PermutationVector.Set< FDeferredLightPS::FCloudTransmittance >(false);
		}
		else // Directional
		{
			PermutationVector.Set< FDeferredLightPS::FSourceShapeDim >(ELightSourceShape::Directional);
			PermutationVector.Set< FDeferredLightPS::FSourceTextureDim >(false);
			PermutationVector.Set< FDeferredLightPS::FIESProfileDim >(false);
			PermutationVector.Set< FDeferredLightPS::FAnistropicMaterials >(bSupportAnisotropyPermutation);
			// Only directional lights are rendered in this path, so we only need to check if it is use to light the atmosphere
			PermutationVector.Set< FDeferredLightPS::FAtmosphereTransmittance >(PassParameters->PS.bAtmosphereTransmittanceEnabled > 0);
			PermutationVector.Set< FDeferredLightPS::FCloudTransmittance >(PassParameters->PS.bCloudShadowEnabled > 0);
		}

		// Substrate tile rendering: 
		// * if the light is directional, then dispatch a set of rect tiles
		// * if the light is radial/local, then dispatch a light geometry with stencil test. The stencil buffer has been prefilled with the tile result (simple/complex) 
		//   so that the geometry get correctly stencil culled on complex/simple part of the screen
		if (Substrate::IsSubstrateEnabled())
		{
			const auto ShouldRenderTileType = [&View, SubstrateTileType](ESubstrateTileType InTileType)
				{
					return (InTileType == SubstrateTileType || SubstrateTileType == ESubstrateTileType::ECount) && Substrate::GetSubstrateUsesTileType(View, InTileType);
				};

			FRenderLightParameters* PassParametersForToon = nullptr;
			const auto SelectComplexPassParameters = [&PassParametersForToon, &PassParameters, &GraphBuilder, &View](ESubstrateTileType InTileType)
				{
					if (MayRenderSubstrateToon(InTileType))
					{
						if(PassParametersForToon == nullptr)
						{
							// Copy the parameters once for all the tile types that can render
							PassParametersForToon = GraphBuilder.AllocParameters<FRenderLightParameters>();
							*PassParametersForToon = *PassParameters;
							// And setup the needed toon render target
							const FSubstrateSceneData& SubstrateSceneData = *View.GetSceneTextures().SubstrateSceneData;
							PassParametersForToon->PS.RenderTargets[1] = FRenderTargetBinding(SubstrateSceneData.SubstrateSceneToonData0, ERenderTargetLoadAction::ELoad);
							PassParametersForToon->PS.RenderTargets[2] = FRenderTargetBinding(SubstrateSceneData.SubstrateSceneToonData1, ERenderTargetLoadAction::ELoad);
							// As of today, OpaqueRoughRefraction cannot be used together with experimental toon features. This is reflected in the implementation of IsOpaqueRoughRefractionEnabled().
							check(!Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform()));

						}
						return PassParametersForToon;
					}
					return PassParameters;
				};

			if (ShouldRenderTileType(ESubstrateTileType::EComplexSpecial))
			{
				const ESubstrateTileType TileType = ESubstrateTileType::EComplexSpecial;
				PermutationVector.Set<Substrate::FSubstrateTileType>(TileType);
				FDeferredLightPS::FPermutationDomain PermutationVectorRemap = FDeferredLightPS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
				TShaderMapRef< FDeferredLightPS > PixelShader(View.ShaderMap, PermutationVectorRemap);
				TShaderMapRef< FDeferredLightPS > UberPixelShader = GetUberDeferredLightPS(View.ShaderMap, View.GetShaderPlatform(), PermutationVectorRemap);

				FRenderLightParameters* SelectedPassParameters = SelectComplexPassParameters(TileType);
				InternalRenderLight(GraphBuilder, Scene, View, LightSceneInfo, PixelShader, UberPixelShader, SelectedPassParameters, TileType, TEXT("Light::StandardDeferred(ComplexSpecial)"));
			}
			if (ShouldRenderTileType(ESubstrateTileType::EComplex))
			{
				const ESubstrateTileType TileType = ESubstrateTileType::EComplex;
				PermutationVector.Set<Substrate::FSubstrateTileType>(TileType);
				FDeferredLightPS::FPermutationDomain PermutationVectorRemap = FDeferredLightPS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
				TShaderMapRef< FDeferredLightPS > PixelShader(View.ShaderMap, PermutationVectorRemap);
				TShaderMapRef< FDeferredLightPS > UberPixelShader = GetUberDeferredLightPS(View.ShaderMap, View.GetShaderPlatform(), PermutationVectorRemap);

				FRenderLightParameters* SelectedPassParameters = SelectComplexPassParameters(TileType);
				InternalRenderLight(GraphBuilder, Scene, View, LightSceneInfo, PixelShader, UberPixelShader, SelectedPassParameters, TileType, TEXT("Light::StandardDeferred(Complex)"));
			}
			if (ShouldRenderTileType(ESubstrateTileType::ESingle))
			{
				const ESubstrateTileType TileType = ESubstrateTileType::ESingle;
				PermutationVector.Set<Substrate::FSubstrateTileType>(TileType);
				FDeferredLightPS::FPermutationDomain PermutationVectorRemap = FDeferredLightPS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
				TShaderMapRef< FDeferredLightPS > PixelShader(View.ShaderMap, PermutationVectorRemap);
				TShaderMapRef< FDeferredLightPS > UberPixelShader = GetUberDeferredLightPS(View.ShaderMap, View.GetShaderPlatform(), PermutationVectorRemap);

				InternalRenderLight(GraphBuilder, Scene, View, LightSceneInfo, PixelShader, UberPixelShader, PassParameters, TileType, TEXT("Light::StandardDeferred(Single)"));
			}
			if (ShouldRenderTileType(ESubstrateTileType::ESimple))
			{
				const ESubstrateTileType TileType = ESubstrateTileType::ESimple;
				PermutationVector.Set<Substrate::FSubstrateTileType>(TileType);
				FDeferredLightPS::FPermutationDomain PermutationVectorRemap = FDeferredLightPS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
				TShaderMapRef< FDeferredLightPS > PixelShader(View.ShaderMap, PermutationVectorRemap);
				TShaderMapRef< FDeferredLightPS > UberPixelShader = GetUberDeferredLightPS(View.ShaderMap, View.GetShaderPlatform(), PermutationVectorRemap);

				InternalRenderLight(GraphBuilder, Scene, View, LightSceneInfo, PixelShader, UberPixelShader, PassParameters, TileType, TEXT("Light::StandardDeferred(Simple)"));
			}
		}
		else
		{
			PermutationVector.Set< Substrate::FSubstrateTileType>(0);
			PermutationVector = FDeferredLightPS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
			TShaderMapRef< FDeferredLightPS > PixelShader(View.ShaderMap, PermutationVector);
			TShaderMapRef< FDeferredLightPS > UberPixelShader = GetUberDeferredLightPS(View.ShaderMap, View.GetShaderPlatform(), PermutationVector);

			InternalRenderLight(GraphBuilder, Scene, View, LightSceneInfo, PixelShader, UberPixelShader, PassParameters, ESubstrateTileType::ECount, TEXT("Light::StandardDeferred"));
		}
	}
}

/** Shader parameters for Standard Deferred Light for HairStrands pass. */
BEGIN_SHADER_PARAMETER_STRUCT(FRenderLightForHairParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightHairVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightPS::FParameters, PS)
END_SHADER_PARAMETER_STRUCT()

void SetupLightForHairGraphicsPSOState(FGraphicsPipelineStateInitializer& GraphicsPSOInit)
{
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Max, BF_One, BF_One>::GetRHI();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
	GraphicsPSOInit.bDepthBounds = false;
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
}

void FDeferredShadingSceneRenderer::RenderLightForHair(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	const FMinimalSceneTextures& SceneTextures,
	const FLightSceneInfo* LightSceneInfo,
	FRDGTextureRef HairShadowMaskTexture,
	FRDGTextureRef LightingChannelsTexture,
	const FHairStrandsTransmittanceMaskData& InTransmittanceMaskData,
	const bool bForwardRendering,
	const bool bCanLightUsesAtlasForUnbatchedLight,
	TRDGUniformBufferRef<FVirtualShadowMapUniformParameters> VirtualShadowMapUniformBuffer,
	FRDGTextureRef ShadowMaskBits,
	int32 VirtualShadowMapId)
{
	// Ensure the light is valid for this view
	const bool bHairRenderingEnabled = HairStrands::HasViewHairStrandsData(View);
	if (!bHairRenderingEnabled || !LightSceneInfo->ShouldRenderLight(View) || View.HairStrandsViewData.VisibilityData.SampleLightingTexture == nullptr || View.Family->EngineShowFlags.VisualizeLightCulling)
	{
		return;
	}
	
	// Sanity check
	check(InTransmittanceMaskData.TransmittanceMask);

	SCOPE_CYCLE_COUNTER(STAT_DirectLightRenderingTime);
	INC_DWORD_STAT(STAT_NumLightsUsingStandardDeferred);
	RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

	const bool bIsDirectional = LightSceneInfo->Proxy->GetLightType() == LightType_Directional;
	const bool bCloudShadow   = bIsDirectional;
	const bool bUseVirtualShadowMapMask = VirtualShadowMapId != INDEX_NONE && ShadowMaskBits;

	FRenderLightForHairParameters* PassParameters = GraphBuilder.AllocParameters<FRenderLightForHairParameters>();
	// VS - General parameters
	PassParameters->VS.HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
	// PS - General parameters
	PassParameters->PS = GetDeferredLightPSParameters(
		GraphBuilder,
		Scene,
		View,
		LightSceneInfo,
		SceneTextures.Color.Target,
		SceneTextures.Depth.Target,
		SceneTextures.UniformBuffer,
		HairStrands::BindHairStrandsViewUniformParameters(View),
		HairShadowMaskTexture,
		LightingChannelsTexture,
		bCloudShadow,
		VirtualShadowMapUniformBuffer, 
		ShadowMaskBits, 
		VirtualShadowMapId);

	// PS - Hair parameters
	const FIntPoint SampleLightingViewportResolution = View.HairStrandsViewData.VisibilityData.SampleLightingViewportResolution;
	PassParameters->PS.HairTransmittanceBuffer = GraphBuilder.CreateSRV(InTransmittanceMaskData.TransmittanceMask, FHairStrandsTransmittanceMaskData::Format);
	PassParameters->PS.HairTransmittanceBufferMaxCount = InTransmittanceMaskData.TransmittanceMask ? InTransmittanceMaskData.TransmittanceMask->Desc.NumElements : 0;
	PassParameters->PS.ShadowChannelMask = FVector4f(1, 1, 1, 1);
	PassParameters->PS.LightSceneId = LightSceneInfo->Id.GetIndex();
	if (HairShadowMaskTexture)
	{
		PassParameters->PS.ScreenShadowMaskSubPixelTexture = HairShadowMaskTexture;
		PassParameters->PS.HairShadowMaskValid = true;
	}
	if (bForwardRendering)
	{
		PassParameters->PS.ShadowChannelMask = FVector4f(0, 0, 0, 0);
		PassParameters->PS.ShadowChannelMask[FMath::Clamp(LightSceneInfo->GetDynamicShadowMapChannel(), 0, 3)] = 1.f;
	}
	PassParameters->PS.RenderTargets[0] = FRenderTargetBinding(View.HairStrandsViewData.VisibilityData.SampleLightingTexture, ERenderTargetLoadAction::ELoad);
	PassParameters->PS.RenderTargets[1] = FRenderTargetBinding();
	PassParameters->PS.RenderTargets[2] = FRenderTargetBinding();
	PassParameters->PS.RenderTargets.DepthStencil = FDepthStencilBinding(nullptr, ERenderTargetLoadAction::ENoAction, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthNop_StencilNop);

	FDeferredLightPS::FPermutationDomain PermutationVector;
	PermutationVector.Set< FDeferredLightPS::FLightingChannelsDim >(View.bUsesLightingChannels);
	PermutationVector.Set< FDeferredLightPS::FVisualizeCullingDim >(false);
	PermutationVector.Set< FDeferredLightPS::FTransmissionDim >(false);
	PermutationVector.Set< FDeferredLightPS::FHairLighting>(1);
	PermutationVector.Set< FDeferredLightPS::FHairComplexTransmittance>(true);
	PermutationVector.Set< FDeferredLightPS::FVirtualShadowMapMask >(bUseVirtualShadowMapMask);
	PermutationVector.Set< Substrate::FSubstrateTileType>(Substrate::IsSubstrateEnabled() ? ESubstrateTileType::EComplex : 0);
	PermutationVector.Set< FDeferredLightPS::FLightFunctionAtlasDim >(
		LightFunctionAtlas::IsEnabled(View, ELightFunctionAtlasSystem::DeferredLighting) && LightSceneInfo->Proxy->HasValidLightFunctionAtlasSlot() &&
		LightSceneInfo->Proxy->GetLightFunctionMaterial() != nullptr && !View.Family->EngineShowFlags.VisualizeLightCulling && bCanLightUsesAtlasForUnbatchedLight);
	if (bIsDirectional)
	{
		PermutationVector.Set< FDeferredLightPS::FSourceShapeDim >(ELightSourceShape::Directional);
		PermutationVector.Set< FDeferredLightPS::FSourceTextureDim >(false);
		PermutationVector.Set< FDeferredLightPS::FIESProfileDim >(false);
		PermutationVector.Set< FDeferredLightPS::FAtmosphereTransmittance >(PassParameters->PS.bAtmosphereTransmittanceEnabled > 0.f);
		PermutationVector.Set< FDeferredLightPS::FCloudTransmittance >(PassParameters->PS.bCloudShadowEnabled > 0.f);
	}
	else
	{
		const bool bUseIESTexture = View.Family->EngineShowFlags.TexturedLightProfiles && LightSceneInfo->Proxy->GetIESTextureResource() != 0;
		PermutationVector.Set< FDeferredLightPS::FSourceShapeDim >(LightSceneInfo->Proxy->IsRectLight() ? ELightSourceShape::Rect : ELightSourceShape::Capsule);
		PermutationVector.Set< FDeferredLightPS::FSourceTextureDim >(LightSceneInfo->Proxy->IsRectLight() && LightSceneInfo->Proxy->HasSourceTexture());
		PermutationVector.Set< FDeferredLightPS::FIESProfileDim >(bUseIESTexture);
		PermutationVector.Set< FDeferredLightPS::FAtmosphereTransmittance >(false);
		PermutationVector.Set< FDeferredLightPS::FCloudTransmittance >(false);
	}

	PermutationVector = FDeferredLightPS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
	TShaderMapRef<FDeferredLightHairVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FDeferredLightPS> PixelShader(View.ShaderMap, PermutationVector);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("StandardDeferredLighting_Hair"),
		PassParameters,
		ERDGPassFlags::Raster,
		[this, VertexShader, PixelShader, PermutationVector, &View, PassParameters, SampleLightingViewportResolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(0, 0, 0.0f, SampleLightingViewportResolution.X, SampleLightingViewportResolution.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		
		SetupLightForHairGraphicsPSOState(GraphicsPSOInit);

		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

		TShaderMapRef< FDeferredLightPS > UsedPixelShader = PixelShader;
		if (!PrecachePSO(GraphicsPSOInit))
		{
			UsedPixelShader = GetUberDeferredLightPS(View.ShaderMap, View.GetShaderPlatform(), PermutationVector);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = UsedPixelShader.GetPixelShader();
			ValidatePSOPrecacheState(GraphicsPSOInit);
		}

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
		SetShaderParameters(RHICmdList, UsedPixelShader, UsedPixelShader.GetPixelShader(), PassParameters->PS);
		
		RHICmdList.SetStreamSource(0, nullptr, 0);
		RHICmdList.DrawPrimitive(0, 1, 1);
	});
}

// Forward lighting version for hair
void FDeferredShadingSceneRenderer::RenderLightsForHair(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FSortedLightSetSceneInfo& SortedLightSet,
	FRDGTextureRef ScreenShadowMaskSubPixelTexture,
	FRDGTextureRef LightingChannelsTexture)
{
	const TArray<FSortedLightSceneInfo, SceneRenderingAllocator>& SortedLights = SortedLightSet.SortedLights;
	const int32 UnbatchedLightStart = SortedLightSet.UnbatchedLightStart;
	const int32 SimpleLightsEnd = SortedLightSet.SimpleLightsEnd;

	if (ViewFamily.EngineShowFlags.DirectLighting)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "DirectLighting");

		for (int32 ViewIndex = 0, ViewCount = Views.Num(); ViewIndex < ViewCount; ++ViewIndex)
		{
			FViewInfo& View = Views[ViewIndex];

			if (!HairStrands::HasViewHairStrandsData(View))
			{
				continue;
			}

			FHairStrandsTransmittanceMaskData DummyTransmittanceMaskData = CreateDummyHairStrandsTransmittanceMaskData(GraphBuilder, View.ShaderMap);
			for (int32 LightIndex = UnbatchedLightStart; LightIndex < SortedLights.Num(); LightIndex++)
			{
				const FSortedLightSceneInfo& SortedLightInfo = SortedLights[LightIndex];
				const FLightSceneInfo& LightSceneInfo = *SortedLightInfo.LightSceneInfo;
				if (LightSceneInfo.Proxy)
				{
					const bool bDrawHairShadow = SortedLightInfo.SortKey.Fields.bShadowed;
					FHairStrandsTransmittanceMaskData TransmittanceMaskData = DummyTransmittanceMaskData;
					if (bDrawHairShadow)
					{
						TransmittanceMaskData = RenderHairStrandsTransmittanceMask(GraphBuilder, View, ViewIndex, &LightSceneInfo, true, ScreenShadowMaskSubPixelTexture);
					}

					RenderLightForHair(
						GraphBuilder,
						View,
						SceneTextures,
						&LightSceneInfo,
						ScreenShadowMaskSubPixelTexture,
						LightingChannelsTexture,
						TransmittanceMaskData,
						true /*bForwardRendering*/,
						SortedLightInfo.bIsCompatibleWithLightFunctionAtlas);
				}
			}
		}
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FSimpleLightsStandardDeferredParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightPS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightVS::FParameters, VS)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSimpleLightsForHairParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightPS::FParameters, PS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeferredLightHairVS::FParameters, VS)
END_SHADER_PARAMETER_STRUCT()

static FDeferredLightPS::FParameters GetRenderLightSimplePSParameters(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FMinimalSceneTextures& SceneTextures,
	const FSimpleLightEntry& SimpleLight,
	const FVector& SimpleLightPosition)
{
	FDeferredLightPS::FParameters Out;

	FRDGTextureRef WhiteDummy = GSystemTextures.GetWhiteDummy(GraphBuilder);
	FRDGBufferRef BufferDummy = GSystemTextures.GetDefaultBuffer(GraphBuilder, 4, 0u);
	FRDGBufferSRVRef BufferDummySRV = GraphBuilder.CreateSRV(BufferDummy, PF_R32_UINT);
	
	// PS - General parameters
	Out.SceneTextures = SceneTextures.UniformBuffer;
	Out.HairStrands = View.HairStrandsViewData.UniformBuffer;
	Out.ForwardLightStruct = View.ForwardLightingResources.ForwardLightUniformBuffer;
	Out.Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	Out.bLightingChannelEnabled = 0u;
	Out.LightingChannelsTexture = GSystemTextures.GetZeroUIntDummy(GraphBuilder);
	Out.LightingChannelsSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Out.CloudShadowAO = GetCloudShadowAOParameters(GraphBuilder, View, nullptr);
	Out.bCloudShadowEnabled = SetupLightCloudTransmittanceParameters(GraphBuilder, nullptr, View, nullptr, Out.CloudShadow);	
	Out.bAtmosphereTransmittanceEnabled = 0;
	Out.LightAttenuationTexture = WhiteDummy;
	Out.LightAttenuationTextureSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	Out.View = View.ViewUniformBuffer;
	Out.DeferredLight = CreateDeferredLightUniformBuffer(GraphBuilder, View, SimpleLight, SimpleLightPosition);
	// PS - Hair (default)
	Out.ScreenShadowMaskSubPixelTexture = WhiteDummy;
	Out.HairTransmittanceBuffer = BufferDummySRV;
	Out.HairTransmittanceBufferMaxCount = 0;
	Out.HairShadowMaskValid = false;
	Out.ShadowChannelMask = FVector4f(1, 1, 1, 1);
	// PS - One pass projection
	Out.VirtualShadowMap = FVirtualShadowMapArray::CreateDummySamplingParameters(GraphBuilder).VirtualShadowMap;
	Out.VirtualShadowMapId = INDEX_NONE;
	Out.ShadowMaskBits = GSystemTextures.GetZeroUIntDummy(GraphBuilder);

	// For ubershader permutation
	Out.LightFunctionAtlas = LightFunctionAtlas::BindGlobalParameters(GraphBuilder, View);

	// PS - RT/Depth
	Out.RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
	if (SceneTextures.Depth.Target)
	{
		Out.RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);
	}

	return Out;
}

static FSimpleLightsStandardDeferredParameters GetRenderLightSimpleParameters(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FMinimalSceneTextures& SceneTextures,
	const FSimpleLightEntry& SimpleLight,
	const FVector& SimpleLightPosition)
{
	FSimpleLightsStandardDeferredParameters Out;

	Out.PS = GetRenderLightSimplePSParameters(GraphBuilder, Scene, View, SceneTextures, SimpleLight, SimpleLightPosition);

	// VS - General parameters (dummy geometry, as the geometry is setup within the pass light loop)
	FSphere SphereLight;
	SphereLight.Center = SimpleLightPosition; // Should we account for LWC Position+Tile here?
	SphereLight.W = SimpleLight.Radius;
	Out.VS = FDeferredLightVS::GetParameters(View, SphereLight, false);
	return Out;
}
static FSimpleLightsForHairParameters GetRenderLightSimpleForHairParameters(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const FMinimalSceneTextures& SceneTextures,
	const FSimpleLightEntry& SimpleLight,
	const FVector& SimpleLightPosition)
{
	FSimpleLightsForHairParameters Out;

	Out.PS = GetRenderLightSimplePSParameters(GraphBuilder, Scene, View, SceneTextures, SimpleLight, SimpleLightPosition);

	// Hair VS
	Out.VS.HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);

	// Hair PS
	Out.PS.RenderTargets[0] = FRenderTargetBinding(View.HairStrandsViewData.VisibilityData.SampleLightingTexture, ERenderTargetLoadAction::ELoad);
	Out.PS.RenderTargets[1] = FRenderTargetBinding();
	Out.PS.RenderTargets[2] = FRenderTargetBinding();
	Out.PS.RenderTargets.DepthStencil = FDepthStencilBinding(nullptr, ERenderTargetLoadAction::ENoAction, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthNop_StencilNop);

	return Out;
}

static void InternalRenderSimpleLightsForHair(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const uint32 ViewIndex,
	const uint32 NumViews,
	const FMinimalSceneTextures& SceneTextures,
	const FSortedLightSetSceneInfo& SortedLightSet)
{
	FSimpleLightsForHairParameters* PassParameters = GraphBuilder.AllocParameters<FSimpleLightsForHairParameters>();
	*PassParameters = GetRenderLightSimpleForHairParameters(
		GraphBuilder,
		Scene,
		View,
		SceneTextures,
		SortedLightSet.SimpleLights.InstanceData[0], // Use a dummy light to create the PassParameter buffer. The light data will be
		FVector(0, 0, 0));		  // update dynamically with the pass light loop for efficiency purpose

	FDeferredLightPS::FPermutationDomain PermutationVector;
	PermutationVector.Set< FDeferredLightPS::FSourceShapeDim >(ELightSourceShape::Capsule);
	PermutationVector.Set< FDeferredLightPS::FIESProfileDim >(false);
	PermutationVector.Set< FDeferredLightPS::FLightFunctionAtlasDim>(false);
	PermutationVector.Set< FDeferredLightPS::FVisualizeCullingDim >(false);
	PermutationVector.Set< FDeferredLightPS::FLightingChannelsDim >(false);
	PermutationVector.Set< FDeferredLightPS::FAnistropicMaterials >(false);
	PermutationVector.Set< FDeferredLightPS::FTransmissionDim >(false);
	PermutationVector.Set< FDeferredLightPS::FHairLighting>(1);
	PermutationVector.Set< FDeferredLightPS::FHairComplexTransmittance>(true /*bNeedComplexTransmittanceSupport*/);
	PermutationVector.Set< FDeferredLightPS::FAtmosphereTransmittance >(false);
	PermutationVector.Set< FDeferredLightPS::FCloudTransmittance >(false);
	PermutationVector.Set< Substrate::FSubstrateTileType>(Substrate::IsSubstrateEnabled() ? ESubstrateTileType::EComplex : 0);

	PermutationVector = FDeferredLightPS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
	TShaderMapRef<FDeferredLightPS> PixelShader(View.ShaderMap, PermutationVector);
	TShaderMapRef<FDeferredLightHairVS> VertexShader(View.ShaderMap);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Light::DeferredSimpleLights(Hair)"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, &SortedLightSet, ViewIndex, NumViews, PassParameters, PixelShader, VertexShader, PermutationVector](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		SetupLightForHairGraphicsPSOState(GraphicsPSOInit);

		// Use additive blending for color
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		const int32 GlobalPSOCollectorIndex = FGlobalPSOCollectorManager::GetIndex(DeferredLightGlobalPSOCollectorName);
		const FIntPoint SampleLightingViewportResolution = View.HairStrandsViewData.VisibilityData.SampleLightingViewportResolution;
		RHICmdList.SetViewport(0, 0, 0.0f, SampleLightingViewportResolution.X, SampleLightingViewportResolution.Y, 1.0f);

		for (int32 SortedLightIndex = 0; SortedLightIndex < SortedLightSet.SimpleLightsEnd; SortedLightIndex++)
		{
			const FSortedLightSceneInfo& SortedLightInfo = SortedLightSet.SortedLights[SortedLightIndex];

			const int32 SimpleLightIndex = SortedLightInfo.SimpleLightIndex;
			const FSimpleLightEntry& SimpleLight = SortedLightSet.SimpleLights.InstanceData[SimpleLightIndex];

			const FSimpleLightPerViewEntry& SimpleLightPerViewData = SortedLightSet.SimpleLights.GetViewDependentData(SimpleLightIndex, ViewIndex, NumViews);
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
						
			TShaderMapRef< FDeferredLightPS > UsedPixelShader = PixelShader;
			if (!PrecachePSO(GraphicsPSOInit))
			{
				UsedPixelShader = GetUberDeferredLightPS(View.ShaderMap, View.GetShaderPlatform(), PermutationVector);
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = UsedPixelShader.GetPixelShader();
				ValidatePSOPrecacheState(GraphicsPSOInit);
			}

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0/*StencilRef*/);

			// Update the light parameters with a custom uniform buffer
			FDeferredLightUniformStruct DeferredLightUniformsValue = GetSimpleDeferredLightParameters(View, SimpleLight, SimpleLightPerViewData);

			SetShaderParametersMixed(RHICmdList, UsedPixelShader, UsedPixelShader.GetPixelShader(), PassParameters->PS, DeferredLightUniformsValue);
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);

			RHICmdList.SetStreamSource(0, nullptr, 0);
			RHICmdList.DrawPrimitive(0, 1, 1);
		}
	});
}

static void InternalRenderSimpleLightsStandardDeferred(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FViewInfo& View,
	const uint32 ViewIndex,
	const uint32 NumViews,
	const FMinimalSceneTextures& SceneTextures,
	const FSortedLightSetSceneInfo& SortedLightSet,
	ESubstrateTileType TileType)
{
	FSimpleLightsStandardDeferredParameters* PassParameters = GraphBuilder.AllocParameters<FSimpleLightsStandardDeferredParameters>();
	*PassParameters = GetRenderLightSimpleParameters(
		GraphBuilder,
		Scene,
		View,
		SceneTextures,
		SortedLightSet.SimpleLights.InstanceData[0], // Use a dummy light to create the PassParameter buffer. The light data will be
		FVector(0, 0, 0));		  // update dynamically with the pass light loop for efficiency purpose

	const bool bNeedComplexTransmittanceSupport = View.HairCardsMeshElements.Num() && FDeferredLightPS::SupportsHairComplexTransmittance(View.GetShaderPlatform());

	FDeferredLightPS::FPermutationDomain PermutationVector;
	PermutationVector.Set< FDeferredLightPS::FSourceShapeDim >(ELightSourceShape::Capsule);
	PermutationVector.Set< FDeferredLightPS::FIESProfileDim >(false);
	PermutationVector.Set< FDeferredLightPS::FLightFunctionAtlasDim>(false);
	PermutationVector.Set< FDeferredLightPS::FVisualizeCullingDim >(View.Family->EngineShowFlags.VisualizeLightCulling);
	PermutationVector.Set< FDeferredLightPS::FLightingChannelsDim >(false);
	PermutationVector.Set< FDeferredLightPS::FAnistropicMaterials >(false);
	PermutationVector.Set< FDeferredLightPS::FTransmissionDim >(false);
	PermutationVector.Set< FDeferredLightPS::FHairLighting>(0);
	PermutationVector.Set< FDeferredLightPS::FHairComplexTransmittance>(bNeedComplexTransmittanceSupport);
	PermutationVector.Set< FDeferredLightPS::FAtmosphereTransmittance >(false);
	PermutationVector.Set< FDeferredLightPS::FCloudTransmittance >(false);
	PermutationVector.Set< Substrate::FSubstrateTileType>(TileType != ESubstrateTileType::ECount ? TileType : 0);

	PermutationVector = FDeferredLightPS::RemapPermutation(PermutationVector, View.GetShaderPlatform());
	TShaderMapRef<FDeferredLightPS> PixelShader(View.ShaderMap, PermutationVector);

	FDeferredLightVS::FPermutationDomain PermutationVectorVS;
	PermutationVectorVS.Set<FDeferredLightVS::FRadialLight>(true);
	TShaderMapRef<FDeferredLightVS> VertexShader(View.ShaderMap, PermutationVectorVS);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Light::DeferredSimpleLights(Substrate:%s,Tile:%s)", Substrate::IsSubstrateEnabled() ? TEXT("True") : TEXT("False"), Substrate::IsSubstrateEnabled() ? ToString(TileType) : TEXT("None")),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, &SortedLightSet, ViewIndex, NumViews, PassParameters, PixelShader, VertexShader, PermutationVector, TileType](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		// Use additive blending for color
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		for (int32 SortedLightIndex = 0; SortedLightIndex < SortedLightSet.SimpleLightsEnd; SortedLightIndex++)
		{
			const FSortedLightSceneInfo& SortedLightInfo = SortedLightSet.SortedLights[SortedLightIndex];

			const int32 SimpleLightIndex = SortedLightInfo.SimpleLightIndex;
			const FSimpleLightEntry& SimpleLight = SortedLightSet.SimpleLights.InstanceData[SimpleLightIndex];

			const FSimpleLightPerViewEntry& SimpleLightPerViewData = SortedLightSet.SimpleLights.GetViewDependentData(SimpleLightIndex, ViewIndex, NumViews);
			const FSphere LightBounds(SimpleLightPerViewData.Position, SimpleLight.Radius);


			// Set the device viewport for the view.
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

			const bool bCameraInsideLightGeometry = ((FVector)View.ViewMatrices.GetViewOrigin() - LightBounds.Center).SizeSquared() < FMath::Square(LightBounds.W * 1.05f + View.NearClippingDistance * 2.0f)
							// Always draw backfaces in ortho
							//@todo - accurate ortho camera / light intersection
							|| !View.IsPerspectiveProjection();

			const uint32 StencilRef = SetBoundingGeometryRasterizerAndDepthState(GraphicsPSOInit, View.bReverseCulling, bCameraInsideLightGeometry, TileType);
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
						
			TShaderMapRef< FDeferredLightPS > UsedPixelShader = PixelShader;
			if (!PrecachePSO(GraphicsPSOInit))
			{
				UsedPixelShader = GetUberDeferredLightPS(View.ShaderMap, View.GetShaderPlatform(), PermutationVector);
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = UsedPixelShader.GetPixelShader();
				ValidatePSOPrecacheState(GraphicsPSOInit);
			}

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

			// Update the light parameters with a custom uniform buffer
			FDeferredLightUniformStruct DeferredLightUniformsValue = GetSimpleDeferredLightParameters(View, SimpleLight, SimpleLightPerViewData);

			SetShaderParametersMixed(RHICmdList, UsedPixelShader, UsedPixelShader.GetPixelShader(), PassParameters->PS, DeferredLightUniformsValue);

			// Update vertex shader parameters with custom parameters/uniform buffer
			FDeferredLightVS::FParameters ParametersVS = FDeferredLightVS::GetParameters(View, LightBounds);
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS /*PassParameters->VS*/);

			// Apply the point or spot light with some approximately bounding geometry,
			// So we can get speedups from depth testing and not processing pixels outside of the light's influence.
			StencilingGeometry::DrawSphere(RHICmdList);
		}
	});
}

void FDeferredShadingSceneRenderer::RenderSimpleLightsStandardDeferred(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	const FSortedLightSetSceneInfo& SortedLightSet,
	FSubstrateSceneData& SubstrateSceneData)
{
	const int32 NumSimpleLights = SortedLightSet.SimpleLightsEnd;

	if (NumSimpleLights == 0)
	{
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_DirectLightRenderingTime);
	INC_DWORD_STAT_BY(STAT_NumLightsUsingStandardDeferred, NumSimpleLights);

	// Update the stencil buffer, marking single/simple/complex Substrate material only if not already done.
	if (Substrate::IsSubstrateEnabled() && Substrate::GetStencilPassStage() == Substrate::EStencilPassStage::StandaloneAdaptive)
	{
		Substrate::AddSubstrateStencilPass(GraphBuilder, Scene, Views, SceneTextures, SubstrateSceneData);
	}

	// GBufffer lighting
	for (int32 ViewIndex = 0, NumViews = Views.Num(); ViewIndex < NumViews; ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (Substrate::IsSubstrateEnabled())
		{
			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplexSpecial))
			{
				InternalRenderSimpleLightsStandardDeferred(GraphBuilder, Scene, View, ViewIndex, NumViews, SceneTextures, SortedLightSet, ESubstrateTileType::EComplexSpecial);
			}
			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::EComplex))
			{
				InternalRenderSimpleLightsStandardDeferred(GraphBuilder, Scene, View, ViewIndex, NumViews, SceneTextures, SortedLightSet, ESubstrateTileType::EComplex);
			}
			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESingle))
			{
				InternalRenderSimpleLightsStandardDeferred(GraphBuilder, Scene, View, ViewIndex, NumViews, SceneTextures, SortedLightSet, ESubstrateTileType::ESingle);
			}
			if (Substrate::GetSubstrateUsesTileType(View, ESubstrateTileType::ESimple))
			{
				InternalRenderSimpleLightsStandardDeferred(GraphBuilder, Scene, View, ViewIndex, NumViews, SceneTextures, SortedLightSet, ESubstrateTileType::ESimple);
			}
		}
		else
		{
			InternalRenderSimpleLightsStandardDeferred(GraphBuilder, Scene, View, ViewIndex, NumViews, SceneTextures, SortedLightSet, ESubstrateTileType::ECount);
		}
	}

	// Hair lighting
	if (HairStrands::HasViewHairStrandsData(Views) && CVarHairStrandsSimpleLightEnable.GetValueOnRenderThread() > 0)
	{
		for (int32 ViewIndex = 0, NumViews = Views.Num(); ViewIndex < NumViews; ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			InternalRenderSimpleLightsForHair(GraphBuilder, Scene, View, ViewIndex, NumViews, SceneTextures, SortedLightSet);
		}
	}
}

class FCopyStencilToLightingChannelsPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyStencilToLightingChannelsPS);
	SHADER_USE_PARAMETER_STRUCT(FCopyStencilToLightingChannelsPS, FGlobalShader);

	class FNaniteCompositeDim : SHADER_PERMUTATION_BOOL("NANITE_COMPOSITE");
	using FPermutationDomain = TShaderPermutationDomain<FNaniteCompositeDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, SceneStencilTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, NaniteShadingMask)
		SHADER_PARAMETER(uint32, bStencilHasCastShadowBit)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (!DoesPlatformSupportNanite(Parameters.Platform) && PermutationVector.Get<FNaniteCompositeDim>() != 0)
		{
			return false;
		}

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("STENCIL_LIGHTING_CHANNELS_SHIFT"), STENCIL_LIGHTING_CHANNELS_BIT_ID);
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_R16_UINT);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCopyStencilToLightingChannelsPS, "/Engine/Private/DownsampleDepthPixelShader.usf", "CopyStencilToLightingChannelsPS", SF_Pixel);

FRDGTextureRef FDeferredShadingSceneRenderer::CopyStencilToLightingChannelTexture(
	FRDGBuilder& GraphBuilder,
	FRDGTextureSRVRef SceneStencilTexture,
	const TArrayView<FRDGTextureRef> NaniteShadingMasks
)
{
	bool bNeedToCopyStencilToTexture = false;

	for (const FViewInfo& View : Views)
	{
		if (View.bUsesLightingChannels
			|| (GetViewPipelineState(View).DiffuseIndirectMethod == EDiffuseIndirectMethod::Lumen && Lumen::IsUsingDistanceFieldRepresentationBit(View))
			|| (MegaLights::IsEnabled(*View.Family)))
		{
			bNeedToCopyStencilToTexture = true;
		}
	}

	FRDGTextureRef LightingChannelsTexture = nullptr;

	if (bNeedToCopyStencilToTexture)
	{
		{
			check(SceneStencilTexture && SceneStencilTexture->Desc.Texture);
			const FIntPoint TextureExtent = SceneStencilTexture->Desc.Texture->Desc.Extent;
			const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(TextureExtent, PF_R8_UINT, FClearValueBinding::None, TexCreate_RenderTargetable | TexCreate_ShaderResource);
			LightingChannelsTexture = GraphBuilder.CreateTexture(Desc, TEXT("LightingChannels"));
		}

		const ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ENoAction;

		const bool bNaniteComposite = NaniteShadingMasks.Num() == Views.Num();

		for (int32 ViewIndex = 0, ViewCount = Views.Num(); ViewIndex < ViewCount; ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			auto* PassParameters = GraphBuilder.AllocParameters<FCopyStencilToLightingChannelsPS::FParameters>();
			PassParameters->RenderTargets[0] = FRenderTargetBinding(LightingChannelsTexture, View.DecayLoadAction(LoadAction));
			PassParameters->SceneStencilTexture = SceneStencilTexture;
			PassParameters->NaniteShadingMask = bNaniteComposite ? NaniteShadingMasks[ViewIndex] : nullptr;
			PassParameters->View = View.GetShaderParameters();

			// Substrate with DBuffer reuses STENCIL_CAST_SHADOW_BIT_ID as STENCIL_SUBSTRATE_RECEIVE_DBUFFER_NORMAL_BIT_ID so we can't read that bit
			const static bool bSubstrateDGufferPassEnabled = Substrate::IsSubstrateEnabled() && Substrate::IsDBufferPassEnabled(GShaderPlatformForFeatureLevel[FeatureLevel]);
			PassParameters->bStencilHasCastShadowBit = bSubstrateDGufferPassEnabled ? 0 : 1;

			const FScreenPassTextureViewport Viewport(LightingChannelsTexture, View.ViewRect);

			FCopyStencilToLightingChannelsPS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FCopyStencilToLightingChannelsPS::FNaniteCompositeDim>(PassParameters->NaniteShadingMask != nullptr);
			TShaderMapRef<FCopyStencilToLightingChannelsPS> PixelShader(View.ShaderMap, PermutationVector);

			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("CopyStencilToLightingChannels View%d", ViewIndex), View, Viewport, Viewport, PixelShader, PassParameters);
		}
	}

	return LightingChannelsTexture;
}

void DeferredLightGlobalPSOCollector(const FSceneTexturesConfig& SceneTexturesConfig, int32 GlobalPSOCollectorIndex, TArray<FPSOPrecacheData>& PSOInitializers)
{
	EShaderPlatform ShaderPlatform = SceneTexturesConfig.ShaderPlatform;
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(ShaderPlatform);

	auto AddPSOInitializer = [&](bool bIsHairShader, bool bSimpleLight, bool bCameraInsideLightGeometry, bool bReverseCulling, bool bIsDirectional, ESubstrateTileType SubstrateTileMaterialType, FRHIPixelShader* PixelShaderRHI)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;	

		FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
		RenderTargetsInfo.NumSamples = 1;
		if (bIsHairShader)
		{
			SetupLightForHairGraphicsPSOState(GraphicsPSOInit);
			AddRenderTargetInfo(GetHairLightSampleFormat(), ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::UAV | ETextureCreateFlags::ShaderResource, RenderTargetsInfo);

			TShaderMapRef<FDeferredLightHairVS> VertexShader(GlobalShaderMap);
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShaderRHI;
		}
		else
		{
			SetupLightGraphicsPSOState(bIsDirectional, bCameraInsideLightGeometry, bReverseCulling, SubstrateTileMaterialType, GraphicsPSOInit, ShaderPlatform);

			if (bIsDirectional)
			{
				FDeferredLightVS::FPermutationDomain PermutationVectorVS;
				PermutationVectorVS.Set<FDeferredLightVS::FRadialLight>(false);
				TShaderMapRef<FDeferredLightVS> VertexShader(GlobalShaderMap, PermutationVectorVS);

				Substrate::FSubstrateTilePassVS::FPermutationDomain VSPermutationVector;
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableDebug >(false);
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FEnableTexCoordScreenVector >(true);
				VSPermutationVector.Set< Substrate::FSubstrateTilePassVS::FClassificationTileSize16 >(Substrate::UseTileSize16());
				TShaderMapRef<Substrate::FSubstrateTilePassVS> TileVertexShader(GlobalShaderMap, VSPermutationVector);

				const bool bEnableSubstrateTiledPass = SubstrateTileMaterialType != ESubstrateTileType::ECount && Substrate::IsSubstrateEnabled() && Substrate::AreLightingPassesTiled();
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = bEnableSubstrateTiledPass ? TileVertexShader.GetVertexShader() : VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShaderRHI;
			}
			else // Radial light (LightType_Point, LightType_Spot, LightType_Rect)
			{
				FDeferredLightVS::FPermutationDomain PermutationVectorVS;
				PermutationVectorVS.Set<FDeferredLightVS::FRadialLight>(true);
				TShaderMapRef<FDeferredLightVS> VertexShader(GlobalShaderMap, PermutationVectorVS);

				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShaderRHI;

				if (bSimpleLight)
				{
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
					GraphicsPSOInit.bDepthBounds = false;
				}
			}

			AddRenderTargetInfo(SceneTexturesConfig.ColorFormat, SceneTexturesConfig.ColorCreateFlags, RenderTargetsInfo);
			if (Substrate::IsOpaqueRoughRefractionEnabled(ShaderPlatform) && Substrate::UsesSubstrateMaterialBuffer(ShaderPlatform))
			{
				// TODO: find out the render target info for substrate here
				//Out.RenderTargets[1] = FRenderTargetBinding(Scene->SubstrateSceneData.SeparatedOpaqueRoughRefractionSceneColor, ERenderTargetLoadAction::ELoad);
				//Out.RenderTargets[2] = FRenderTargetBinding(Scene->SubstrateSceneData.SeparatedSubSurfaceSceneColor, ERenderTargetLoadAction::ELoad);
				return;
			}

			if (UseSeparatedSubsurfaceDiffuse(ShaderPlatform) && !IsExperimentalToonUnifiedDiffuseEnabled())
			{
				// Consistent to the texture SeparatedSubsurfaceDiffuse
				AddRenderTargetInfo(PF_R16F, TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV, RenderTargetsInfo);
			}

			ETextureCreateFlags DepthStencilCreateFlags = SceneTexturesConfig.DepthCreateFlags;
			SetupDepthStencilInfo(PF_DepthStencil, DepthStencilCreateFlags, ERenderTargetLoadAction::ELoad,
				ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite, RenderTargetsInfo);			
		}

		GraphicsPSOInit.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(GraphicsPSOInit);		
		ApplyTargetsInfo(GraphicsPSOInit, RenderTargetsInfo);

		FPSOPrecacheData PSOPrecacheData;
		PSOPrecacheData.bRequired = true;
		PSOPrecacheData.Priority = EPSOPrecachePriority::High;
		PSOPrecacheData.Type = FPSOPrecacheData::EType::Graphics;
		PSOPrecacheData.GraphicsPSOInitializer = GraphicsPSOInit;
#if PSO_PRECACHING_VALIDATE
		PSOPrecacheData.PSOCollectorIndex = GlobalPSOCollectorIndex;
		PSOPrecacheData.VertexFactoryType = nullptr;
#endif // PSO_PRECACHING_VALIDATE

		PSOInitializers.Add(MoveTemp(PSOPrecacheData));
	};

	auto PSOInitializerForSubstrateTileType = [ & ](FRHIPixelShader* RHIPixelShader, FDeferredLightPS::FPermutationDomain& PermutationVector, ESubstrateTileType TileType)
	{
		// Extract useful information from the permutation vector
		bool bIsDirectional = (PermutationVector.Get< FDeferredLightPS::FSourceShapeDim >() == ELightSourceShape::Directional);
		bool bIsHairShader = PermutationVector.Get<FDeferredLightPS::FHairLighting>() > 0;
				
		AddPSOInitializer(bIsHairShader, false /* bSimpleLight*/, true /*bCameraInsideLightGeometry*/, true /*bReverCulling*/, bIsDirectional, TileType, RHIPixelShader);
		AddPSOInitializer(bIsHairShader, false /* bSimpleLight*/, true /*bCameraInsideLightGeometry*/, false /*bReverCulling*/, bIsDirectional, TileType, RHIPixelShader);
		if (!bIsDirectional)
		{
			AddPSOInitializer(bIsHairShader, false /* bSimpleLight*/, false /*bCameraInsideLightGeometry*/, true /*bReverCulling*/, bIsDirectional, TileType, RHIPixelShader);
			AddPSOInitializer(bIsHairShader, false /* bSimpleLight*/, false /*bCameraInsideLightGeometry*/, false /*bReverCulling*/, bIsDirectional, TileType, RHIPixelShader);

			AddPSOInitializer(bIsHairShader, true /* bSimpleLight*/, false /*bCameraInsideLightGeometry*/, true /*bReverCulling*/, bIsDirectional, TileType, RHIPixelShader);
			AddPSOInitializer(bIsHairShader, true /* bSimpleLight*/, false /*bCameraInsideLightGeometry*/, false /*bReverCulling*/, bIsDirectional, TileType, RHIPixelShader);

			AddPSOInitializer(bIsHairShader, true /* bSimpleLight*/, true /*bCameraInsideLightGeometry*/, true /*bReverCulling*/, bIsDirectional, TileType, RHIPixelShader);
			AddPSOInitializer(bIsHairShader, true /* bSimpleLight*/, true /*bCameraInsideLightGeometry*/, false /*bReverCulling*/, bIsDirectional, TileType, RHIPixelShader);
		}
	};

	// Precache PSOs are never required
	const bool bRequired = false;

	EShaderPermutationFlags PermutationFlags = EShaderPermutationFlags::None;

	FShaderType* ShaderType = FShaderType::GetShaderTypeByName(FDeferredLightPS::GetStaticType().GetName());
	FGlobalShaderType* GlobalShaderType = ShaderType->GetGlobalShaderType();
	for (int32 PermutationId = 0; PermutationId < GlobalShaderType->GetPermutationCount(); PermutationId++)
	{
		if (GlobalShaderType->ShouldCompilePermutation(ShaderPlatform, PermutationId, PermutationFlags) &&
			GlobalShaderType->ShouldPrecachePermutation(ShaderPlatform, PermutationId, PermutationFlags) == EShaderPermutationPrecacheRequest::Precached)
		{
			TShaderRef<FShader> GlobalShader = GlobalShaderMap->GetShader(GlobalShaderType, PermutationId);
			
			FRHIPixelShader* RHIPixelShader = static_cast<FRHIPixelShader*>(GlobalShader.GetRHIShaderBase(SF_Pixel, bRequired));
			if (RHIPixelShader)
			{
				FDeferredLightPS::FPermutationDomain PermutationVector(PermutationId);

				// Only precache the uber shader versions
				if (!PermutationVector.Get < FDeferredLightPS::FUbershader > ())
				{
					continue;
				}

				if (Substrate::IsSubstrateEnabled())
				{
					for (int32 TypeIndex = 0; TypeIndex <= (int32)ESubstrateTileType::EComplexSpecial; ++TypeIndex)
					{
						PSOInitializerForSubstrateTileType(RHIPixelShader, PermutationVector, (ESubstrateTileType) TypeIndex);
					}
				}
				else
				{
					PSOInitializerForSubstrateTileType(RHIPixelShader, PermutationVector, ESubstrateTileType::ECount);
				}
			}
		}
	}
}

FRegisterGlobalPSOCollectorFunction RegisterDeferredLightGlobalPSOCollector(&DeferredLightGlobalPSOCollector, DeferredLightGlobalPSOCollectorName);
