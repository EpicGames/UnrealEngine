// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightRendering.h: Light rendering declarations.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHIResourceUtils.h"
#include "RenderResource.h"
#include "UniformBuffer.h"
#include "ShaderParameters.h"
#include "ShaderParameterUtils.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "SceneRendering.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "LightSceneInfo.h"

/** Uniform buffer for rendering deferred lights. */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FDeferredLightUniformStruct,)
	SHADER_PARAMETER(FVector4f,ShadowMapChannelMask)
	SHADER_PARAMETER(FVector2f,DistanceFadeMAD)
	SHADER_PARAMETER(float, ContactShadowLength)
	SHADER_PARAMETER(float, ContactShadowCastingIntensity)
	SHADER_PARAMETER(float, ContactShadowNonCastingIntensity)
	SHADER_PARAMETER(float, VolumetricScatteringIntensity)
	SHADER_PARAMETER(uint32,ShadowedBits)
	SHADER_PARAMETER(uint32,LightingChannelMask)
	SHADER_PARAMETER_STRUCT_INCLUDE(FLightShaderParameters, LightParameters)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

extern uint32 GetShadowQuality();

extern float GetLightFadeFactor(const FSceneView& View, const FLightSceneProxy* Proxy);

extern uint32 GetShadowedBits(const FSceneView& View, const FLightSceneInfo& LightSceneInfo, bool bUseLightFunctionAtlas = false);
extern FDeferredLightUniformStruct GetDeferredLightParameters(const FSceneView& View, const FLightSceneInfo& LightSceneInfo, bool bUseLightFunctionAtlas=false, uint32 LightFlags=0);

inline void SetDeferredLightParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const TShaderUniformBufferParameter<FDeferredLightUniformStruct>& DeferredLightUniformBufferParameter, 
	const FLightSceneInfo* LightSceneInfo,
	const FSceneView& View,
	bool bUseLightFunctionAtlas)
{
	SetUniformBufferParameterImmediate(BatchedParameters, DeferredLightUniformBufferParameter, GetDeferredLightParameters(View, *LightSceneInfo, bUseLightFunctionAtlas));
}

extern FDeferredLightUniformStruct GetSimpleDeferredLightParameters(
	const FSceneView& View,
	const FSimpleLightEntry& SimpleLight,
	const FSimpleLightPerViewEntry &SimpleLightPerViewData);

inline void SetSimpleDeferredLightParameters(
	FRHIBatchedShaderParameters& BatchedParameters,
	const TShaderUniformBufferParameter<FDeferredLightUniformStruct>& DeferredLightUniformBufferParameter, 
	const FSimpleLightEntry& SimpleLight,
	const FSimpleLightPerViewEntry &SimpleLightPerViewData,
	const FSceneView& View)
{
	FDeferredLightUniformStruct DeferredLightUniformsValue = GetSimpleDeferredLightParameters(View, SimpleLight, SimpleLightPerViewData);
	SetUniformBufferParameterImmediate(BatchedParameters, DeferredLightUniformBufferParameter, DeferredLightUniformsValue);
}

/** Shader parameters needed to render a light function. */
class FLightFunctionSharedParameters
{
	DECLARE_TYPE_LAYOUT(FLightFunctionSharedParameters, NonVirtual);
public:
	void Bind(const FShaderParameterMap& ParameterMap);

	static FVector4f GetLightFunctionSharedParameters(const FLightSceneInfo* LightSceneInfo, float ShadowFadeFraction);

	void Set(FRHIBatchedShaderParameters& BatchedParameters, const FLightSceneInfo* LightSceneInfo, float ShadowFadeFraction) const
	{
		SetShaderValue(BatchedParameters, LightFunctionParameters, GetLightFunctionSharedParameters(LightSceneInfo, ShadowFadeFraction));
	}

	/** Serializer. */ 
	friend FArchive& operator<<(FArchive& Ar,FLightFunctionSharedParameters& P)
	{
		Ar << P.LightFunctionParameters;
		return Ar;
	}

private:
	LAYOUT_FIELD(FShaderParameter, LightFunctionParameters)
};

/** Utility functions for drawing a sphere */
namespace StencilingGeometry
{
	/**
	* Draws a sphere using RHIDrawIndexedPrimitive, useful as approximate bounding geometry for deferred passes.
	* Note: The sphere will be of unit size unless transformed by the shader. 
	*/
	extern void DrawSphere(FRHICommandList& RHICmdList);
	/**
	* Draws a sphere using RHIDrawIndexedPrimitive, useful as approximate bounding geometry for deferred passes.
	* Compatible with instanced rendering.
	* Note: The sphere will be of unit size unless transformed by the shader.
	*/
	extern void DrawSphere(FRHICommandList& RHICmdList, uint32 InstanceCount);
	/** Draws exactly the same as above, but uses FVector rather than FVector4f vertex data. */
	extern void DrawVectorSphere(FRHICommandList& RHICmdList);
	/** Renders a cone with a spherical cap, used for rendering spot lights in deferred passes. */
	extern void DrawCone(FRHICommandList& RHICmdList);

	/** 
	* Vertex buffer for a sphere of unit size. Used for drawing a sphere as approximate bounding geometry for deferred passes.
	*/
	template<int32 NumSphereSides, int32 NumSphereRings, typename VectorType>
	class TStencilSphereVertexBuffer : public FVertexBuffer
	{
	public:
		static_assert(std::is_same_v<typename VectorType::FReal, float>, "Must be a float vector type");

		int32 GetNumRings() const
		{
			return NumSphereRings;
		}

		/** 
		* Initialize the RHI for this rendering resource 
		*/
		void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			const int32 NumSides = NumSphereSides;
			const int32 NumRings = NumSphereRings;
			const int32 NumVerts = (NumSides + 1) * (NumRings + 1);

			const float RadiansPerRingSegment = UE_PI / (float)NumRings;
			float Radius = 1;

			TArray<VectorType, TInlineAllocator<NumRings + 1> > ArcVerts;
			ArcVerts.Empty(NumRings + 1);
			// Calculate verts for one arc
			for (int32 i = 0; i < NumRings + 1; i++)
			{
				const float Angle = i * RadiansPerRingSegment;
				ArcVerts.Add(FVector3f(0.0f, FMath::Sin(Angle), FMath::Cos(Angle)));
			}

			TArray<VectorType> Verts;
			Verts.Empty(NumVerts);
			// Then rotate this arc NumSides + 1 times.
			const FVector3f Center = FVector3f(0,0,0);
			for (int32 s = 0; s < NumSides + 1; s++)
			{
				FRotator3f ArcRotator(0, 360.f * ((float)s / NumSides), 0);
				FRotationMatrix44f ArcRot( ArcRotator );

				for (int32 v = 0; v < NumRings + 1; v++)
				{
					const int32 VIx = (NumRings + 1) * s + v;
					Verts.Add(Center + Radius * ArcRot.TransformPosition(ArcVerts[v]));
				}
			}

			NumSphereVerts = Verts.Num();

			// Create vertex buffer. Fill buffer with initial data upon creation
			VertexBufferRHI = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("TStencilSphereVertexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Verts));
		}

		int32 GetVertexCount() const { return NumSphereVerts; }

	/** 
	* Calculates the world transform for a sphere.
	* @param OutTransform - The output world transform.
	* @param Sphere - The sphere to generate the transform for.
	* @param PreViewTranslation - The pre-view translation to apply to the transform.
	* @param bConservativelyBoundSphere - when true, the sphere that is drawn will contain all positions in the analytical sphere,
	*		 Otherwise the sphere vertices will lie on the analytical sphere and the positions on the faces will lie inside the sphere.
	*/
	void CalcTransform(FVector4f& OutPosAndScale, const FSphere& Sphere, const FVector& PreViewTranslation, bool bConservativelyBoundSphere = true)
	{
		float Radius = Sphere.W; // LWC_TODO: Precision loss
		if (bConservativelyBoundSphere)
		{
			const int32 NumRings = NumSphereRings;
			const float RadiansPerRingSegment = UE_PI / (float)NumRings;

			// Boost the effective radius so that the edges of the sphere approximation lie on the sphere, instead of the vertices
			Radius /= FMath::Cos(RadiansPerRingSegment);
		}

		const FVector3f Translate(Sphere.Center + PreViewTranslation);
		OutPosAndScale = FVector4f(Translate, Radius);
	}

	private:
		int32 NumSphereVerts;
	};

	/** 
	* Stenciling sphere index buffer
	*/
	template<int32 NumSphereSides, int32 NumSphereRings>
	class TStencilSphereIndexBuffer : public FIndexBuffer
	{
	public:
		/** 
		* Initialize the RHI for this rendering resource 
		*/
		void InitRHI(FRHICommandListBase& RHICmdList) override
		{
			const int32 NumSides = NumSphereSides;
			const int32 NumRings = NumSphereRings;
			TArray<uint16> Indices;
			Indices.Reserve(NumSides * NumRings * 6);

			// Add triangles for all the vertices generated
			for (int32 s = 0; s < NumSides; s++)
			{
				const int32 a0start = (s + 0) * (NumRings + 1);
				const int32 a1start = (s + 1) * (NumRings + 1);

				for (int32 r = 0; r < NumRings; r++)
				{
					Indices.Add(a0start + r + 0);
					Indices.Add(a1start + r + 0);
					Indices.Add(a0start + r + 1);
					Indices.Add(a1start + r + 0);
					Indices.Add(a1start + r + 1);
					Indices.Add(a0start + r + 1);
				}
			}

			NumIndices = Indices.Num();

			// Create index buffer. Fill buffer with initial data upon creation
			IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("TStencilSphereIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(Indices));
		}

		int32 GetIndexCount() const { return NumIndices; }; 
	
	private:
		int32 NumIndices;
	};

	extern TGlobalResource<TStencilSphereVertexBuffer<18, 12, FVector4f> >	GStencilSphereVertexBuffer;
	extern TGlobalResource<TStencilSphereVertexBuffer<18, 12, FVector3f> >	GStencilSphereVectorBuffer;
	extern TGlobalResource<TStencilSphereIndexBuffer<18, 12> >				GStencilSphereIndexBuffer;
	extern TGlobalResource<TStencilSphereVertexBuffer<4, 4, FVector4f> >		GLowPolyStencilSphereVertexBuffer;
	extern TGlobalResource<TStencilSphereIndexBuffer<4, 4> >				GLowPolyStencilSphereIndexBuffer;

}; //End StencilingGeometry

/** 
* Stencil geometry parameters used by multiple shaders. 
*/
class FStencilingGeometryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FStencilingGeometryShaderParameters, NonVirtual);

public:

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, StencilingGeometryPosAndScale)
		SHADER_PARAMETER(FVector4f, StencilingConeParameters)
		SHADER_PARAMETER(FMatrix44f, StencilingConeTransform)
	END_SHADER_PARAMETER_STRUCT()

	void Bind(const FShaderParameterMap& ParameterMap);
	void Set(FRHIBatchedShaderParameters& BatchedParameters, const FVector4f& InStencilingGeometryPosAndScale) const;
	void Set(FRHIBatchedShaderParameters& BatchedParameters, const FSceneView& View, const FLightSceneInfo* LightSceneInfo) const;

	static FParameters GetParameters(const FVector4f& InStencilingGeometryPosAndScale);
	static FParameters GetParameters(const FSceneView& View, const FLightSceneInfo* LightSceneInfo);

	/** Serializer. */ 
	friend FArchive& operator<<(FArchive& Ar,FStencilingGeometryShaderParameters& P)
	{
		Ar << P.StencilGeometryPosAndScale;
		Ar << P.StencilConeParameters;
		Ar << P.StencilConeTransform;
		return Ar;
	}

private:
	
	LAYOUT_FIELD(FShaderParameter, StencilGeometryPosAndScale)
	LAYOUT_FIELD(FShaderParameter, StencilConeParameters)
	LAYOUT_FIELD(FShaderParameter, StencilConeTransform)
};


BEGIN_SHADER_PARAMETER_STRUCT(FDrawFullScreenRectangleParameters, )
	SHADER_PARAMETER(FVector4f, PosScaleBias)
	SHADER_PARAMETER(FVector4f, UVScaleBias)
	SHADER_PARAMETER(FVector4f, InvTargetSizeAndTextureSize)
END_SHADER_PARAMETER_STRUCT()

/** A vertex shader for rendering the light in a deferred pass. */
class FDeferredLightVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FDeferredLightVS,Global);
	SHADER_USE_PARAMETER_STRUCT(FDeferredLightVS, FGlobalShader);

	class FRadialLight : SHADER_PERMUTATION_BOOL("SHADER_RADIAL_LIGHT");
	using FPermutationDomain = TShaderPermutationDomain<FRadialLight>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDrawFullScreenRectangleParameters, FullScreenRect)
		SHADER_PARAMETER_STRUCT_INCLUDE(FStencilingGeometryShaderParameters::FParameters, Geometry)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	static FDrawFullScreenRectangleParameters GetFullScreenRectParameters(
		float X, float Y,
		float SizeX, float SizeY,
		float U, float V,
		float SizeU, float SizeV,
		FIntPoint InTargetSize,
		FIntPoint InTextureSize);

	static FParameters GetParameters(const FViewInfo& View,
		float X, float Y,
		float SizeX, float SizeY,
		float U, float V,
		float SizeU, float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize,
		bool bBindViewUniform = true);

	static FParameters GetParameters(const FViewInfo& View, bool bBindViewUniform = true);

	static FParameters GetParameters(const FViewInfo& View, const FSphere& LightBounds, bool bBindViewUniform = true);

	static FParameters GetParameters(const FViewInfo& View, const FLightSceneInfo* LightSceneInfo, bool bBindViewUniform = true);
};

enum class ELightOcclusionType : uint8
{
	Shadowmap,
	Raytraced,
	MegaLights,			// Light handled through MegaLights raytracing
	MegaLightsVSM,		// Light projection handled through MegaLights using a VSM source data
};

ELightOcclusionType GetLightOcclusionType(const FLightSceneProxy& Proxy, const FSceneViewFamily& ViewFamily);
ELightOcclusionType GetLightOcclusionType(const FLightSceneInfoCompact& LightInfo, const FSceneViewFamily& ViewFamily);
