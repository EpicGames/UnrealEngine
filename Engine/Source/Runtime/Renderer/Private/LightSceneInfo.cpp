// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightSceneInfo.cpp: Light scene info implementation.
=============================================================================*/

#include "LightSceneInfo.h"
#include "Components/LightComponent.h"
#include "SceneCore.h"
#include "ScenePrivate.h"
#include "DistanceFieldLightingShared.h"
#include "Misc/LargeWorldRenderPosition.h"
#include "LocalLightSceneProxy.h"
#include "ShadowRendering.h"

int32 GWholeSceneShadowUnbuiltInteractionThreshold = 500;
static FAutoConsoleVariableRef CVarWholeSceneShadowUnbuiltInteractionThreshold(
	TEXT("r.Shadow.WholeSceneShadowUnbuiltInteractionThreshold"),
	GWholeSceneShadowUnbuiltInteractionThreshold,
	TEXT("How many unbuilt light-primitive interactions there can be for a light before the light switches to whole scene shadows"),
	ECVF_RenderThreadSafe
	);

static int32 GRecordInteractionShadowPrimitives = 1;
FAutoConsoleVariableRef CVarRecordInteractionShadowPrimitives(
	TEXT("r.Shadow.RecordInteractionShadowPrimitives"),
	GRecordInteractionShadowPrimitives,
	TEXT(""),
	ECVF_RenderThreadSafe);

static int32 GTestMobilityForStaticSceneMembership = 1;
FAutoConsoleVariableRef CVarTestMobilityForStaticSceneMembership(
	TEXT("r.Light.TestMobilityForStaticSceneMembership"),
	GTestMobilityForStaticSceneMembership,
	TEXT("Deprecated (UE5.5): Temporary flag to switch back to the old behavior (testing the cast static shadow flag)\n.")
	TEXT("  The old behavior checked the HasStaticShadowing flag on the proxy, but that is cleared for VSMs so the new behavior tests the mobility instead."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

void FLightSceneInfoCompact::Init(FLightSceneInfo* InLightSceneInfo)
{
	LightSceneInfo = InLightSceneInfo;
	FSphere BoundingSphere = InLightSceneInfo->Proxy->GetBoundingSphere();
	BoundingSphere.W = BoundingSphere.W > 0.0f ? BoundingSphere.W : FLT_MAX;
	FMemory::Memcpy(&BoundingSphereVector,&BoundingSphere,sizeof(BoundingSphereVector));
	Color = InLightSceneInfo->Proxy->GetColor();
	LightType = InLightSceneInfo->Proxy->GetLightType();

	bCastDynamicShadow = InLightSceneInfo->Proxy->CastsDynamicShadow();
	bCastStaticShadow = InLightSceneInfo->Proxy->CastsStaticShadow();
	bStaticLighting = InLightSceneInfo->Proxy->HasStaticLighting();
	bAffectReflection = InLightSceneInfo->Proxy->AffectReflection();
	bAffectGlobalIllumination = InLightSceneInfo->Proxy->AffectGlobalIllumination();
	bIsMovable = InLightSceneInfo->Proxy->IsMovable();
    CastRaytracedShadow = InLightSceneInfo->Proxy->CastsRaytracedShadow();
	bAllowMegaLights = InLightSceneInfo->Proxy->AllowMegaLights();
	MegaLightsShadowMethod = InLightSceneInfo->Proxy->GetMegaLightsShadowMethod();
}

FLightSceneInfo::FLightSceneInfo(FLightSceneProxy* InProxy, bool InbVisible)
	: bRecordInteractionShadowPrimitives(!!GRecordInteractionShadowPrimitives && InProxy->GetLightType() != ELightComponentType::LightType_Directional)
	, DynamicInteractionOftenMovingPrimitiveList(NULL)
	, DynamicInteractionStaticPrimitiveList(NULL)
	, Proxy(InProxy)
	, Type((ELightComponentType)InProxy->GetLightType())
	, DynamicShadowMapChannel(-1)
	, bPrecomputedLightingIsValid(InProxy->IsPrecomputedLightingValid())
	, bVisible(InbVisible)
	, bEnableLightShaftBloom(InProxy->IsLightShaftBloomEnabled())
	, BloomScale(InProxy->GetBloomScale())
	, BloomThreshold(InProxy->GetBloomThreshold())
	, BloomMaxBrightness(InProxy->GetBloomMaxBrightness())
	, BloomTint(InProxy->GetBloomTint())
	, NumUnbuiltInteractions(0)
	, bCreatePerObjectShadowsForDynamicObjects(Proxy->ShouldCreatePerObjectShadowsForDynamicObjects())
	, Scene(static_cast<FScene*>(InProxy->GetSceneInterface()))
{
}

void FLightSceneInfo::AddToScene()
{
	const FLightSceneInfoCompact& LightSceneInfoCompact = Scene->Lights[Id];

	bool bIsValidLightTypeMobile = false;
	if (Scene->GetShadingPath() == EShadingPath::Mobile && ShouldRenderLightViewIndependent())
	{
		const uint8 LightType = Proxy->GetLightType();
		bIsValidLightTypeMobile = LightType == LightType_Rect || LightType == LightType_Point || LightType == LightType_Spot;
	}

	// Only need to create light interactions for lights that can cast a shadow,
	// As deferred shading doesn't need to know anything about the primitives that a light affects
	if (Proxy->CastsDynamicShadow()
		|| Proxy->CastsStaticShadow()
		// Lights that should be baked need to check for interactions to track unbuilt state correctly
		|| Proxy->HasStaticLighting()
		// Mobile path supports dynamic point/spot lights in the base pass using forward rendering, so we need to know the primitives
		|| bIsValidLightTypeMobile)
	{
		// Directional lights have no finite extent and cannot meaningfully be in the LocalShadowCastingLightOctree
		if (LightSceneInfoCompact.LightType == LightType_Directional)
		{
			// 
			Scene->DirectionalShadowCastingLightIDs.Add(Id);

			// All primitives may interact with a directional light
			for (FPrimitiveSceneInfo *PrimitiveSceneInfo : Scene->Primitives)
			{
				CreateLightPrimitiveInteraction(LightSceneInfoCompact, PrimitiveSceneInfo);
			}
		}
		else
		{
			// Add the light to the scene's light octree.
			Scene->LocalShadowCastingLightOctree.AddElement(LightSceneInfoCompact);

			Scene->PrimitiveOctree.FindElementsWithBoundsTest(GetBoundingBox(), [&LightSceneInfoCompact, this](const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact)
			{
				CreateLightPrimitiveInteraction(LightSceneInfoCompact, PrimitiveSceneInfoCompact);
			});
		}
	}
}

bool FLightSceneInfo::ShouldCreateLightPrimitiveInteraction(const FLightSceneInfoCompact& LightSceneInfoCompact, const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact)
{
	if (LightSceneInfoCompact.AffectsPrimitive(FBoxSphereBounds(PrimitiveSceneInfoCompact.Bounds), PrimitiveSceneInfoCompact.Proxy))
	{
		// create light interaction and add to light/primitive lists
		return FLightPrimitiveInteraction::ShouldCreate(this, PrimitiveSceneInfoCompact.PrimitiveSceneInfo).bShouldCreate;
	}
	return false;
}

void FLightSceneInfo::CreateLightPrimitiveInteraction(const FLightSceneInfoCompact& LightSceneInfoCompact, const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact)
{
	if (LightSceneInfoCompact.AffectsPrimitive(FBoxSphereBounds(PrimitiveSceneInfoCompact.Bounds), PrimitiveSceneInfoCompact.Proxy))
	{
		// create light interaction and add to light/primitive lists
		FLightPrimitiveInteraction::Create(this,PrimitiveSceneInfoCompact.PrimitiveSceneInfo);
	}
}

void FLightSceneInfo::RemoveFromScene()
{
	if (OctreeId.IsValidId())
	{
		// Remove the light from the octree.
		Scene->LocalShadowCastingLightOctree.RemoveElement(OctreeId);
		OctreeId = FOctreeElementId2();
	}
	else
	{
		Scene->DirectionalShadowCastingLightIDs.RemoveSwap(Id);
	}

	// Detach the light from the primitives it affects.
	Detach();
}

void FLightSceneInfo::Detach()
{
	check(IsInRenderingThread());

	InteractionShadowPrimitives.Empty();

	// implicit linked list. The destruction will update this "head" pointer to the next item in the list.
	while(DynamicInteractionOftenMovingPrimitiveList)
	{
		FLightPrimitiveInteraction::Destroy(DynamicInteractionOftenMovingPrimitiveList);
	}

	while(DynamicInteractionStaticPrimitiveList)
	{
		FLightPrimitiveInteraction::Destroy(DynamicInteractionStaticPrimitiveList);
	}
}

FBoxCenterAndExtent FLightSceneInfo::GetBoundingBox() const
{
	FSphere BoundingSphere = Proxy->GetBoundingSphere();
	return FBoxCenterAndExtent(BoundingSphere.Center, FVector(BoundingSphere.W, BoundingSphere.W, BoundingSphere.W));
}

inline static bool IsInDesiredSceneSubset(const FViewInfo& View, FLightSceneProxy *Proxy)
{
	if (!View.bStaticSceneOnly)
	{
		return true;
	}

	if (GTestMobilityForStaticSceneMembership)
	{
		return !Proxy->IsMovable();
	}
	else
	{
		return Proxy->HasStaticShadowing();
	}
}

bool FLightSceneInfo::ShouldRenderLight(const FViewInfo& View, bool bOffscreen) const
{
	// Only render the light if it is in the view frustum
	bool bLocalVisible = bVisible && (bOffscreen ? View.VisibleLightInfos[Id].bInDrawRange : View.VisibleLightInfos[Id].bInViewFrustum);

	if (bLocalVisible)
	{
		switch(Type)
		{
			case LightType_Directional:
				if (!View.Family->EngineShowFlags.DirectionalLights) 
				{
					bLocalVisible = false;
				}
				break;
			case LightType_Point:
				if (!View.Family->EngineShowFlags.PointLights) 
				{
					bLocalVisible = false;
				}
				break;
			case LightType_Spot:
				if (!View.Family->EngineShowFlags.SpotLights)
				{
					bLocalVisible = false;
				}
				break;
			case LightType_Rect:
				if (!View.Family->EngineShowFlags.RectLights)
				{
					bLocalVisible = false;
				}
				break;
		}
	}

	return bLocalVisible
		// Only render lights with static shadowing for reflection captures, since they are only captured at edit time
		&& IsInDesiredSceneSubset(View, Proxy)
		&& (Proxy->GetViewLightingChannelMask() & View.ViewLightingChannelMask)
		// Only render lights in the default channel, or if there are any primitives outside the default channel
		&& (Proxy->GetLightingChannelMask() & GetDefaultLightingChannelMask() || View.bUsesLightingChannels || bOffscreen);
}

bool FLightSceneInfo::ShouldRenderLightViewIndependent() const
{
	return !Proxy->GetColor().IsAlmostBlack()
		// Only render lights with dynamic lighting or unbuilt static lights
		&& (!Proxy->HasStaticLighting() || !IsPrecomputedLightingValid());
}

bool FLightSceneInfo::ShouldRenderViewIndependentWholeSceneShadows() const
{
	bool bShouldRenderLight = ShouldRenderLightViewIndependent();
	bool bCastDynamicShadow = Proxy->CastsDynamicShadow();

	// Also create a whole scene shadow for lights with precomputed shadows that are unbuilt
	const bool bCreateShadowToPreviewStaticLight =
		Proxy->HasStaticShadowing()
		&& bCastDynamicShadow
		&& !IsPrecomputedLightingValid();

	bool bShouldRenderShadow = bShouldRenderLight && bCastDynamicShadow && (!Proxy->HasStaticLighting() || bCreateShadowToPreviewStaticLight);
	return bShouldRenderShadow;
}

bool FLightSceneInfo::IsPrecomputedLightingValid() const
{
	return (bPrecomputedLightingIsValid && NumUnbuiltInteractions < GWholeSceneShadowUnbuiltInteractionThreshold) || !Proxy->HasStaticShadowing();
}

void FLightSceneInfo::SetDynamicShadowMapChannel(int32 NewChannel)
{
	if (Proxy->HasStaticShadowing())
	{
		// This ensure would trigger if several static shadowing light intersects eachother and have the same channel.
		// ensure(Proxy->GetPreviewShadowMapChannel() == NewChannel);
	}
	else
	{
		DynamicShadowMapChannel = NewChannel;
	}
}

int32 FLightSceneInfo::GetDynamicShadowMapChannel() const
{
	if (Proxy->HasStaticShadowing())
	{
		// Stationary lights get a channel assigned by ReassignStationaryLightChannels
		return Proxy->GetPreviewShadowMapChannel();
	}

	// Movable lights get a channel assigned when they are added to the scene
	return DynamicShadowMapChannel;
}

const TArray<FLightPrimitiveInteraction*>* FLightSceneInfo::GetInteractionShadowPrimitives() const
{
	return bRecordInteractionShadowPrimitives ? &InteractionShadowPrimitives : nullptr;
}

FLightPrimitiveInteraction* FLightSceneInfo::GetDynamicInteractionOftenMovingPrimitiveList() const
{
	return DynamicInteractionOftenMovingPrimitiveList;
}

FLightPrimitiveInteraction* FLightSceneInfo::GetDynamicInteractionStaticPrimitiveList() const
{
	return DynamicInteractionStaticPrimitiveList;
}

bool FLightSceneInfo::IsAValidShadowPointLight(const FViewInfo& View, const FVisibleLightInfoArray& VisibleLightInfos, EShaderPlatform ShaderPlatform) const
{
	if (Proxy->GetLightType() != LightType_Point)
	{
		return false;
	}

	if (!FReadOnlyCVARCache::EnablePointLightShadows(ShaderPlatform))
	{
		return false;
	}

	bool bShouldCastPointShadow = View.Family->EngineShowFlags.DynamicShadows
		&& GetShadowQuality() > 0
		&& VisibleLightInfos[GetPersistentIndex()].AllProjectedShadows.Num() > 0
		&& VisibleLightInfos[GetPersistentIndex()].AllProjectedShadows.Last()->bAllocated;

	return bShouldCastPointShadow;
}

bool FLightSceneInfo::SetupMobileMovableLocalSpotLightShadowParameters(const FViewInfo& View, const FVisibleLightInfoArray& VisibleLightInfos, FMobileMovableLocalLightShadowParameters& MobileMovableLocalLightShadowParameters) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FLightSceneProxy_SetupMobileMovableLocalLightShadowParameters);

	check(Proxy->IsLocalLight());
	const uint8 LightType = Proxy->GetLightType();

	bool bShouldCastShadow = View.Family->EngineShowFlags.DynamicShadows
		&& GetShadowQuality() > 0
		&& LightType == LightType_Spot
		&& VisibleLightInfos[Id].AllProjectedShadows.Num() > 0
		&& VisibleLightInfos[Id].AllProjectedShadows.Last()->bAllocated;

	if (bShouldCastShadow)
	{
		FProjectedShadowInfo* ProjectedShadowInfo = VisibleLightInfos[Id].AllProjectedShadows.Last();
		checkSlow(ProjectedShadowInfo && ProjectedShadowInfo->CacheMode != SDCM_StaticPrimitivesOnly);

		const float TransitionSize = ProjectedShadowInfo->ComputeTransitionSize();
		checkSlow(TransitionSize > 0.0f);

		float SoftTransitionScale = 1.0f / TransitionSize;
		float ShadowFadeFraction = ProjectedShadowInfo->FadeAlphas[0];

		MobileMovableLocalLightShadowParameters.SpotLightShadowSharpenAndFadeFractionAndReceiverDepthBiasAndSoftTransitionScale = FVector4f(Proxy->GetShadowSharpen() * 7.0f + 1.0f, ShadowFadeFraction, ProjectedShadowInfo->GetShaderReceiverDepthBias(), SoftTransitionScale);

		const FMatrix WorldToShadowMatrix = ProjectedShadowInfo->GetWorldToShadowMatrix(MobileMovableLocalLightShadowParameters.SpotLightShadowmapMinMax);
		FLightRenderParameters LightParameters;
		Proxy->GetLightShaderParameters(LightParameters);

		MobileMovableLocalLightShadowParameters.SpotLightShadowWorldToShadowMatrix = FMatrix44f(FTranslationMatrix(LightParameters.WorldPosition) * WorldToShadowMatrix);

		const FIntPoint ShadowBufferResolution = ProjectedShadowInfo->GetShadowBufferResolution();
		MobileMovableLocalLightShadowParameters.LocalLightShadowBufferSize = FVector4f(ShadowBufferResolution.X, ShadowBufferResolution.Y, 1.0f / ShadowBufferResolution.X, 1.0f / ShadowBufferResolution.Y);
		MobileMovableLocalLightShadowParameters.LocalLightShadowTexture = ProjectedShadowInfo->RenderTargets.DepthTarget->GetRHI();
		MobileMovableLocalLightShadowParameters.LocalLightShadowSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}

	MobileMovableLocalLightShadowParameters.ShadowedBits = GetShadowedBits(View, *this);

	return bShouldCastShadow;
}

bool FLightSceneInfo::ShouldRecordShadowSubjectsForMobile() const
{
	if (Proxy == nullptr)
	{
		return false;
	}

	// record shadow casters if CSM culling is enabled for the light's mobility type and the culling mode requires the list of casters.
	const bool bCombinedStaticAndCSMEnabled = FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers();
	const bool bMobileEnableMovableLightCSMShaderCulling = FReadOnlyCVARCache::MobileEnableMovableLightCSMShaderCulling();
		
	static auto* CVarMobileCSMShaderCullingMethod = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.Shadow.CSMShaderCullingMethod"));
	const uint32 MobileCSMCullingMode = CVarMobileCSMShaderCullingMethod->GetValueOnAnyThread() & 0xF;

	const FLightSceneProxy* LightSceneProxy = Proxy;

	bool bRenderMovableDirectionalLightCSM = LightSceneProxy->IsMovable() && ShouldRenderViewIndependentWholeSceneShadows();

	bool bLightHasCombinedStaticAndCSMEnabled = bCombinedStaticAndCSMEnabled && LightSceneProxy->UseCSMForDynamicObjects();
	bool bMovableLightUsingCSM = bMobileEnableMovableLightCSMShaderCulling && bRenderMovableDirectionalLightCSM;

	bool bShouldRecordShadowSubjectsForMobile = (MobileCSMCullingMode == 2 || MobileCSMCullingMode == 3) && (bLightHasCombinedStaticAndCSMEnabled || bMovableLightUsingCSM);

	return bShouldRecordShadowSubjectsForMobile;
}

uint32 FLightSceneInfo::PackExtraData(bool bAllowStaticLighting, bool bLightFunction, bool bMegaLight, bool bClusteredDeferredSupported) const
{
	uint32 Result = 0;

	{
		int32 ShadowMapChannel = Proxy->GetShadowMapChannel();
		int32 CurrentDynamicShadowMapChannel = GetDynamicShadowMapChannel();

		if (!bAllowStaticLighting)
		{
			ShadowMapChannel = INDEX_NONE;
		}

		// Static shadowing uses ShadowMapChannel, dynamic shadows are packed into light attenuation using DynamicShadowMapChannel
		uint32 ShadowMapChannelMask =
			(ShadowMapChannel == 0 ? 1 : 0) |
			(ShadowMapChannel == 1 ? 2 : 0) |
			(ShadowMapChannel == 2 ? 4 : 0) |
			(ShadowMapChannel == 3 ? 8 : 0) |
			(CurrentDynamicShadowMapChannel == 0 ? 16 : 0) |
			(CurrentDynamicShadowMapChannel == 1 ? 32 : 0) |
			(CurrentDynamicShadowMapChannel == 2 ? 64 : 0) |
			(CurrentDynamicShadowMapChannel == 3 ? 128 : 0);

		Result |= ShadowMapChannelMask << LIGHT_EXTRA_DATA_BIT_OFFSET_SHADOW_MAP_CHANNEL_MASK;
	}

	Result |= Proxy->GetLightingChannelMask() << LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHTING_CHANNEL_MASK;

	// pack light type in this uint32 as well
	Result |= ((uint32)Proxy->GetLightType()) << LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHT_TYPE;

	const uint32 CastShadows = Proxy->CastsDynamicShadow() ? 1 : 0;
	Result |= CastShadows << LIGHT_EXTRA_DATA_BIT_OFFSET_CAST_SHADOW;

	uint32 HasLightFunction = bLightFunction ? 1 : 0;
	Result |= HasLightFunction << LIGHT_EXTRA_DATA_BIT_OFFSET_HAS_LIGHT_FUNCTION;

	Result |= Proxy->LightFunctionAtlasLightIndex << LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHT_FUNCTION_ATLAS_LIGHT_INDEX;

	uint32 AffectsTranslucentLighting = Proxy->AffectsTranslucentLighting() ? 1 : 0;
	Result |= AffectsTranslucentLighting << LIGHT_EXTRA_DATA_BIT_OFFSET_AFFECT_TRANSLUCENT_LIGHTING;

	uint32 MegaLight = bMegaLight ? 1 : 0;
	Result |= MegaLight << LIGHT_EXTRA_DATA_BIT_OFFSET_MEGA_LIGHT;

	uint32 ClusteredDeferredSupported = bClusteredDeferredSupported ? 1 : 0;
	Result |= ClusteredDeferredSupported << LIGHT_EXTRA_DATA_BIT_OFFSET_CLUSTERED_DEFERRED_SUPPORTED;

	const uint32 CastVolumetricShadows = Proxy->CastsVolumetricShadow() ? 1 : 0;
	Result |= CastVolumetricShadows << LIGHT_EXTRA_DATA_BIT_OFFSET_CAST_VOLUMETRIC_SHADOWS;

	const uint32 UseTransmission = Proxy->Transmission() ? 1 : 0;
	Result |= UseTransmission << LIGHT_EXTRA_DATA_BIT_OFFSET_USE_TRANSMISSION;

	return Result;
}

/** Determines whether two bounding spheres intersect. */
FORCEINLINE bool AreSpheresNotIntersecting(
	const VectorRegister& A_XYZ,
	const VectorRegister& A_Radius,
	const VectorRegister& B_XYZ,
	const VectorRegister& B_Radius
	)
{
	const VectorRegister DeltaVector = VectorSubtract(A_XYZ,B_XYZ);
	const VectorRegister DistanceSquared = VectorDot3(DeltaVector,DeltaVector);
	const VectorRegister MaxDistance = VectorAdd(A_Radius,B_Radius);
	const VectorRegister MaxDistanceSquared = VectorMultiply(MaxDistance,MaxDistance);
	return !!VectorAnyGreaterThan(DistanceSquared,MaxDistanceSquared);
}

/**
* Tests whether this light affects the given primitive.  This checks both the primitive and light settings for light relevance
* and also calls AffectsBounds.
*
* @param CompactPrimitiveSceneInfo - The primitive to test.
* @return True if the light affects the primitive.
*/
bool FLightSceneInfoCompact::AffectsPrimitive(const FBoxSphereBounds& PrimitiveBounds, const FPrimitiveSceneProxy* PrimitiveSceneProxy) const
{
	// Check if the light's bounds intersect the primitive's bounds.
	// Directional lights reach everywhere (the hacky world max radius does not work for large worlds)
	if(LightType != LightType_Directional && AreSpheresNotIntersecting(
		BoundingSphereVector,
		VectorReplicate(BoundingSphereVector,3),
		VectorLoadFloat3(&PrimitiveBounds.Origin),
		VectorLoadFloat1(&PrimitiveBounds.SphereRadius)
		))
	{
		return false;
	}

	// Cull based on information in the full scene infos.

	if(!LightSceneInfo->Proxy->AffectsBounds(PrimitiveBounds))
	{
		return false;
	}

	if (LightSceneInfo->Proxy->CastsShadowsFromCinematicObjectsOnly() && !PrimitiveSceneProxy->CastsCinematicShadow())
	{
		return false;
	}

	if (!(LightSceneInfo->Proxy->GetLightingChannelMask() & PrimitiveSceneProxy->GetLightingChannelMask()))
	{
		return false;
	}

	return true;
}
