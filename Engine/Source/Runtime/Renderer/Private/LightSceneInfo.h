// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LightSceneInfo.h: Light scene info definitions.
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Math/Color.h"
#include "Math/GenericOctree.h"
#include "Math/GenericOctreePublic.h"
#include "Math/UnrealMath.h"
#include "PrimitiveSceneProxy.h"
#include "SceneTypes.h"
#include "SceneView.h"
#include "ScenePrivateBase.h"

class FLightPrimitiveInteraction;
class FLightSceneInfo;
class FMobileMovableLocalLightShadowParameters;
class FPrimitiveSceneInfoCompact;
class FPrimitiveSceneProxy;
class FScene;
class FViewInfo;

namespace ECastRayTracedShadow
{
	enum Type : int;
}

/**
 * The information needed to cull a light-primitive interaction.
 */
class FLightSceneInfoCompact
{
public:
	// XYZ: origin, W:sphere radius
	VectorRegister BoundingSphereVector;
	FLinearColor Color;
	// must not be 0
	FLightSceneInfo* LightSceneInfo;
	// e.g. LightType_Directional, LightType_Point or LightType_Spot
	uint32 LightType : LightType_NumBits;	
	uint32 bCastDynamicShadow : 1;
	uint32 bCastStaticShadow : 1;
	uint32 bStaticLighting : 1;
	uint32 bAffectReflection : 1;
	uint32 bAffectGlobalIllumination : 1;
	uint32 bIsMovable : 1;
	uint32 bAllowMegaLights : 1;
	TEnumAsByte<EMegaLightsShadowMethod::Type> MegaLightsShadowMethod;
    TEnumAsByte<ECastRayTracedShadow::Type> CastRaytracedShadow;

	/** Initializes the compact scene info from the light's full scene info. */
	void Init(FLightSceneInfo* InLightSceneInfo);

	/** Default constructor. */
	FLightSceneInfoCompact():
		LightSceneInfo(NULL)
	{}

	/** Initialization constructor. */
	FLightSceneInfoCompact(FLightSceneInfo* InLightSceneInfo)
	{
		Init(InLightSceneInfo);
	}

	/**
	 * Tests whether this light affects the given primitive.  This checks both the primitive and light settings for light relevance
	 * and also calls AffectsBounds.
	 *
	 * @param CompactPrimitiveSceneInfo - The primitive to test.
	 * @return True if the light affects the primitive.
	 */
	bool AffectsPrimitive(const FBoxSphereBounds& PrimitiveBounds, const FPrimitiveSceneProxy* PrimitiveSceneProxy) const;
};

/** Information for sorting lights. */
struct FSortedLightSceneInfo
{
	union
	{
		struct
		{
			// Note: the order of these members controls the light sort order!
			// Currently bHandledByMegaLights is the MSB and LightType is LSB
			/** The type of light. */
			uint32 LightType : LightType_NumBits;
			/** Whether the light uses IES Profile */
			uint32 bIESProfile : 1;
			/** Whether the light uses a light function. */
			uint32 bLightFunction : 1;
			/** Whether the light uses lighting channels. */
			uint32 bUsesLightingChannels : 1;
			/** Whether the light casts shadows. */
			uint32 bShadowed : 1;
			/** Whether the light is NOT a simple light - they always support tiled/clustered but may want to be selected separately. */
			uint32 bIsNotSimpleLight : 1;
			/** 
			 * True if the light doesn't support clustered deferred, logic is inverted so that lights that DO support clustered deferred will sort first in list 
			 * Super-set of lights supporting tiled, so the tiled lights will end up in the first part of this range.
			 */
			uint32 bClusteredDeferredNotSupported : 1;
			/** Mega Lights must be sorted by GPU Scene Light Index - this field will only be set for MegaLights which don't rely on ordering of the other fields */
			uint32 LightSceneId : LIGHT_ID_NUM_BITS;
			/** Whether the light should be handled by Mega Lights, these will be sorted to the end so they can be skipped */
			uint32 bHandledByMegaLights : 1;
		} Fields;
		/** Sort key bits packed into an integer. */
		uint32 Packed;
		static_assert(sizeof(Packed) >= sizeof(Fields), "SortKey Packed representation must include all bits in Fields");
	} SortKey;

	const FLightSceneInfo* LightSceneInfo;
	int32 SimpleLightIndex;
	bool bIsCompatibleWithLightFunctionAtlas;

	/** Initialization constructor. */
	explicit FSortedLightSceneInfo(const FLightSceneInfo* InLightSceneInfo)
		: LightSceneInfo(InLightSceneInfo),
		SimpleLightIndex(-1),
		bIsCompatibleWithLightFunctionAtlas(false)
	{
		SortKey.Packed = 0;
		SortKey.Fields.bIsNotSimpleLight = 1;
	}

	explicit FSortedLightSceneInfo(int32 InSimpleLightIndex)
		: LightSceneInfo(nullptr),
		SimpleLightIndex(InSimpleLightIndex),
		bIsCompatibleWithLightFunctionAtlas(false)
	{
		SortKey.Packed = 0;
		SortKey.Fields.bIsNotSimpleLight = 0;
	}
};

/** 
 * Stores info about sorted lights and ranges. 
 * The sort-key in FSortedLightSceneInfo gives rise to the following order:
 *  [SimpleLights,Clustered,UnbatchedLights,LumenLights]
 * Note that some shadowed lights can be included in the clustered pass when virtual shadow maps and one pass projection are used.
 */
struct FSortedLightSetSceneInfo
{
	// One past last simple light excluding those handled by Mega Lights
	int32 SimpleLightsEnd;
	int32 ClusteredSupportedEnd;

	/** First light with shadow map or */
	int32 UnbatchedLightStart;

	// First light handled by Mega Lights
	int32 MegaLightsLightStart;
	// One past last simple light handled by Mega Lights
	int32 MegaLightsSimpleLightsEnd;

	bool bHasRectLights = false;
	bool bHasLightFunctions = false;
	bool bHasLightChannels = false;
	bool bHasFirstPersonSelfShadowLights = false;

	FSimpleLightArray SimpleLights;
	TArray<FSortedLightSceneInfo, SceneRenderingAllocator> SortedLights;
};

template <>
struct TUseBitwiseSwap<FSortedLightSceneInfo>
{
	enum { Value = false };
};

/** The type of the octree used by FScene to find lights. */
typedef TOctree2<FLightSceneInfoCompact,struct FLightOctreeSemantics> FSceneLightOctree;

struct FPersistentShadowStateKey
{
	int32 AtlasIndex = -1;
	int32 ProjectionId = -1;
	int32 SubjectPrimitiveComponentIndex = -1;
};

inline uint32 GetTypeHash(const FPersistentShadowStateKey& Key)
{
	return HashCombine(HashCombine((uint32)Key.AtlasIndex, (uint32)Key.ProjectionId), (uint32)Key.SubjectPrimitiveComponentIndex);
}

inline bool operator==(const FPersistentShadowStateKey& A, const FPersistentShadowStateKey& B)
{
	return A.AtlasIndex == B.AtlasIndex && A.ProjectionId == B.ProjectionId && A.SubjectPrimitiveComponentIndex == B.SubjectPrimitiveComponentIndex;
}

class FPersistentShadowState
{
public:
	FViewMatrices						ViewMatrices;

	FIntRect							HZBTestViewRect;
	TRefCountPtr<IPooledRenderTarget>	HZB;				// Direct HZB. nullptr for Atlas rendering.
};

/**
 * The information used to render a light.  This is the rendering thread's mirror of the game thread's ULightComponent.
 * FLightSceneInfo is internal to the renderer module and contains internal scene state.
 */
class FLightSceneInfo
{
	friend class FLightPrimitiveInteraction;

	bool bRecordInteractionShadowPrimitives;
	TArray<FLightPrimitiveInteraction*> InteractionShadowPrimitives;

	/** The list of dynamic primitives affected by the light. */
	FLightPrimitiveInteraction* DynamicInteractionOftenMovingPrimitiveList;

	FLightPrimitiveInteraction* DynamicInteractionStaticPrimitiveList;

public:
	/** The light's scene proxy. */
	FLightSceneProxy* Proxy;

	ELightComponentType Type;

	/** If bVisible == true, this is the index of the primitive in Scene->Lights. */
	FLightSceneId Id;

	/** The identifier for the primitive in Scene->PrimitiveOctree. */
	FOctreeElementId2 OctreeId;

	/** Persistent shadow state used for HZB occlusion culling. */
	TMap<FPersistentShadowStateKey, FPersistentShadowState> PrevPersistentShadows;
	TMap<FPersistentShadowStateKey, FPersistentShadowState> PersistentShadows;

	// Note: these are provided for TSceneUpdateCommandQueue compatibility
	using FPersistentId = FLightSceneId;
	FORCEINLINE FPersistentId GetPersistentIndex() const { return Id; }
protected:

	/** 
	 * ShadowMap channel assigned in the forward renderer when a movable shadow casting light is added to the scene. 
	 * Used to pack shadow projections into channels of the light attenuation texture which is read in the base pass.
	 */
	int32 DynamicShadowMapChannel;

	/** True if the light is built. */
	uint32 bPrecomputedLightingIsValid : 1;

public:

	/** 
	 * True if the light is visible.  
	 * False if the light is invisible but still needed for previewing, which can only happen in the editor.
	 */
	uint32 bVisible : 1;

	/** 
	 * Whether to render light shaft bloom from this light. 
	 * For directional lights, the color around the light direction will be blurred radially and added back to the scene.
	 * for point lights, the color on pixels closer than the light's SourceRadius will be blurred radially and added back to the scene.
	 */
	uint32 bEnableLightShaftBloom : 1;

	/** Scales the additive color. */
	float BloomScale;

	/** Scene color must be larger than this to create bloom in the light shafts. */
	float BloomThreshold;

	/** After exposure is applied, scene color brightness larger than BloomMaxBrightness will be rescaled down to BloomMaxBrightness. */
	float BloomMaxBrightness;

	/** Multiplies against scene color to create the bloom color. */
	FColor BloomTint;

	/** Number of dynamic interactions with statically lit primitives. */
	int32 NumUnbuiltInteractions;

	/** Cached value from the light proxy's virtual function, since it is checked many times during shadow setup. */
	bool bCreatePerObjectShadowsForDynamicObjects;

	/** The scene the light is in. */
	FScene* Scene;

	/** Initialization constructor. */
	FLightSceneInfo(FLightSceneProxy* InProxy, bool InbVisible);

	/** Adds the light to the scene. */
	void AddToScene();

	/**
	 * Returns true if the light affects the primitive
	 * @param LightSceneInfoCompact Compact representation of the light
	 * @param PrimitiveSceneInfoCompact Compact representation of the primitive
	 */
	bool ShouldCreateLightPrimitiveInteraction(const FLightSceneInfoCompact& LightSceneInfoCompact, const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact);

	/**
	 * If the light affects the primitive, create an interaction, and process children 
	 * @param LightSceneInfoCompact Compact representation of the light
	 * @param PrimitiveSceneInfoCompact Compact representation of the primitive
	 */
	void CreateLightPrimitiveInteraction(const FLightSceneInfoCompact& LightSceneInfoCompact, const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact);

	/** Removes the light from the scene. */
	void RemoveFromScene();

	/** Detaches the light from the primitives it affects. */
	void Detach();

	/** Octree bounds setup. */
	FBoxCenterAndExtent GetBoundingBox() const;

	bool ShouldRenderLight(const FViewInfo& View, bool bOffscreen = false) const;

	/** Encapsulates all View-Independent reasons to have this light render. */
	bool ShouldRenderLightViewIndependent() const;

	/** Encapsulates all View-Independent reasons to render ViewIndependentWholeSceneShadows for this light */
	bool ShouldRenderViewIndependentWholeSceneShadows() const;

	bool IsPrecomputedLightingValid() const;

	void SetDynamicShadowMapChannel(int32 NewChannel);

	int32 GetDynamicShadowMapChannel() const;

	const TArray<FLightPrimitiveInteraction*>* GetInteractionShadowPrimitives() const;

	FLightPrimitiveInteraction* GetDynamicInteractionOftenMovingPrimitiveList() const;

	FLightPrimitiveInteraction* GetDynamicInteractionStaticPrimitiveList() const;

	/** Hash function. */
	friend uint32 GetTypeHash(const FLightSceneInfo* LightSceneInfo)
	{
		return GetTypeHash(LightSceneInfo->Id);
	}

	bool IsAValidShadowPointLight(const FViewInfo& View, const FVisibleLightInfoArray& VisibleLightInfos, EShaderPlatform ShaderPlatform) const;

	bool SetupMobileMovableLocalSpotLightShadowParameters(const FViewInfo& View, const FVisibleLightInfoArray& VisibleLightInfos, FMobileMovableLocalLightShadowParameters& MobileMovableLocalLightShadowParameters) const;

	bool ShouldRecordShadowSubjectsForMobile() const;

	// Extra data such as shadow channel masks, light type and flags packed into a uint32.
	// Unpacked in shader using helper functions defined in LightData.ush
	uint32 PackExtraData(bool bAllowStaticLighting, bool bLightFunction, bool bMegaLight, bool bClusteredDeferredSupported) const;
};

/** Defines how the light is stored in the scene's light octree. */
struct FLightOctreeSemantics
{
	enum { MaxElementsPerLeaf = 16 };
	enum { MinInclusiveElementsPerNode = 7 };
	enum { MaxNodeDepth = 12 };

	typedef TInlineAllocator<MaxElementsPerLeaf, TAlignedHeapAllocator<alignof(FLightSceneInfoCompact)>> ElementAllocator;

	FORCEINLINE static FBoxCenterAndExtent GetBoundingBox(const FLightSceneInfoCompact& Element)
	{
		return Element.LightSceneInfo->GetBoundingBox();
	}

	FORCEINLINE static bool AreElementsEqual(const FLightSceneInfoCompact& A,const FLightSceneInfoCompact& B)
	{
		return A.LightSceneInfo == B.LightSceneInfo;
	}
	
	FORCEINLINE static void SetElementId(const FLightSceneInfoCompact& Element,FOctreeElementId2 Id)
	{
		Element.LightSceneInfo->OctreeId = Id;
	}

	FORCEINLINE static void ApplyOffset(FLightSceneInfoCompact& Element, FVector Offset)
	{
		VectorRegister OffsetReg = VectorLoadFloat3_W0(&Offset);
		Element.BoundingSphereVector = VectorAdd(Element.BoundingSphereVector, OffsetReg);
	}
};

/**
 * Array type (sparse) for storing FLightSceneInfoCompact in the renderer.
 */
using FSceneLightInfoArray = TLightSparseArray<FLightSceneInfoCompact, TAlignedSparseArrayAllocator<alignof(FLightSceneInfoCompact)>>;

