// Copyright Epic Games, Inc. All Rights Reserved.

#include "LightShaftRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "LightSceneProxy.h"
#include "ScreenPass.h"
#include "PipelineStateCache.h"
#include "PostProcess/SceneRenderTargets.h"
#include "PostProcess/TemporalAA.h"
#include "TranslucentRendering.h"
#include "SceneTextureParameters.h"
#include "DeferredShadingRenderer.h"
#include "RenderCore.h"
#include "SceneViewState.h"
#include "LightSceneInfo.h"
#include "ScenePrivate.h"

int32 GLightShafts = 1;
static FAutoConsoleVariableRef CVarLightShaftQuality(
	TEXT("r.LightShaftQuality"),
	GLightShafts,
	TEXT("Defines the light shaft quality (mobile and non mobile).\n")
	TEXT("  0: off\n")
	TEXT("  1: on (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

int32 GLightShaftAllowTAA = 1;
static FAutoConsoleVariableRef CVarLightAllowTAA(
	TEXT("r.LightShaftAllowTAA"),
	GLightShaftAllowTAA,
	TEXT("Allows temporal filtering for lightshafts.\n")
	TEXT("  0: off\n")
	TEXT("  1: on (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

int32 GLightShaftDownsampleFactor = 2;
static FAutoConsoleVariableRef CVarCacheLightShaftDownsampleFactor(
	TEXT("r.LightShaftDownSampleFactor"),
	GLightShaftDownsampleFactor,
	TEXT("Downsample factor for light shafts. range: 1..8"),
	ECVF_RenderThreadSafe);

int32 GLightShaftRenderAfterDOF = 0;
static FAutoConsoleVariableRef CVarRenderLightshaftsAfterDOF(
	TEXT("r.LightShaftRenderToSeparateTranslucency"),
	GLightShaftRenderAfterDOF,
	TEXT("If enabled, light shafts will be rendered to the separate translucency buffer.\n")
	TEXT("This ensures postprocess materials with BL_BeforeTranslucnecy are applied before light shafts"),
	ECVF_RenderThreadSafe);

int32 GLightShaftBlurPasses = 3;
static FAutoConsoleVariableRef CVarCacheLightShaftBlurPasses(
	TEXT("r.LightShaftBlurPasses"),
	GLightShaftBlurPasses,
	TEXT("Number of light shaft blur passes."),
	ECVF_RenderThreadSafe);

float GLightShaftFirstPassDistance = .1f;
static FAutoConsoleVariableRef CVarCacheLightShaftFirstPassDistance(
	TEXT("r.LightShaftFirstPassDistance"),
	GLightShaftFirstPassDistance,
	TEXT("Fraction of the distance to the light to blur on the first radial blur pass."),
	ECVF_RenderThreadSafe);

// Must touch LightShaftShader.usf to propagate a change
int32 GLightShaftBlurNumSamples = 12;
static FAutoConsoleVariableRef CVarCacheLightShaftNumSamples(
	TEXT("r.LightShaftNumSamples"),
	GLightShaftBlurNumSamples,
	TEXT("Number of samples per light shaft radial blur pass.  Also affects how quickly the blur distance increases with each pass."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

int32 GetLightShaftDownsampleFactor()
{
	return FMath::Clamp(GLightShaftDownsampleFactor, 1, 8);
}

FVector4f GetLightScreenPosition(const FViewInfo& View, const FLightSceneProxy& LightSceneProxy)
{
	return (FVector4f)View.WorldToScreen(LightSceneProxy.GetLightPositionForLightShafts(View.ViewMatrices.GetViewOrigin()));
}

FMobileLightShaftInfo GetMobileLightShaftInfo(const FViewInfo& View, const FLightSceneInfo& LightSceneInfo)
{
	const FLightSceneProxy& LightSceneProxy = *LightSceneInfo.Proxy;
	const FLinearColor BloomScale(LightSceneInfo.BloomScale, LightSceneInfo.BloomScale, LightSceneInfo.BloomScale, 1.0f);
	const FVector4f LightScreenPosition = GetLightScreenPosition(View, LightSceneProxy);

	FMobileLightShaftInfo LightShaft;
	LightShaft.Center = FVector2D(LightScreenPosition.X / LightScreenPosition.W, LightScreenPosition.Y / LightScreenPosition.W);
	LightShaft.ColorMask = LightSceneInfo.BloomTint;
	LightShaft.ColorApply = LightSceneInfo.BloomTint;
	LightShaft.ColorMask *= BloomScale;
	LightShaft.ColorApply *= BloomScale;
	LightShaft.BloomMaxBrightness = LightSceneInfo.BloomMaxBrightness;
	return LightShaft;
}

enum class ELightShaftBloomOutput
{
	SceneColor,
	SeparateTranslucency
};

ELightShaftBloomOutput GetLightShaftBloomOutput(const FSceneViewFamily& ViewFamily)
{
	return (ViewFamily.AllowTranslucencyAfterDOF() && GLightShaftRenderAfterDOF) ? ELightShaftBloomOutput::SeparateTranslucency : ELightShaftBloomOutput::SceneColor;
}

bool ShouldRenderLightShafts(const FSceneViewFamily& ViewFamily)
{
	return GLightShafts
		&& ViewFamily.EngineShowFlags.LightShafts
		&& ViewFamily.EngineShowFlags.Lighting
		&& !ViewFamily.UseDebugViewPS()
		&& !ViewFamily.EngineShowFlags.VisualizeDOF
		&& !ViewFamily.EngineShowFlags.VisualizeBuffer
		&& !ViewFamily.EngineShowFlags.VisualizeHDR
		&& !ViewFamily.EngineShowFlags.VisualizeMotionBlur;
}

bool ShouldRenderLightShaftsForLight(const FViewInfo& View, const FLightSceneProxy& LightSceneProxy)
{
	if (LightSceneProxy.GetLightType() == LightType_Directional)
	{
		// Don't render if the light's origin is behind the view.
		return GetLightScreenPosition(View, LightSceneProxy).W > 0.0f;
	}
	return false;
}

enum class ELightShaftTechnique
{
	Occlusion,
	Bloom
};

BEGIN_SHADER_PARAMETER_STRUCT(FLightShaftPixelShaderParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER(FVector4f, UVMinMax)
	SHADER_PARAMETER(FVector4f, AspectRatioAndInvAspectRatio)
	SHADER_PARAMETER(FVector4f, LightShaftParameters)
	SHADER_PARAMETER(FVector4f, BloomTintAndThreshold)
	SHADER_PARAMETER(FVector2f, TextureSpaceBlurOrigin)
	SHADER_PARAMETER(float, BloomMaxBrightness)
END_SHADER_PARAMETER_STRUCT()

FLightShaftPixelShaderParameters GetLightShaftParameters(
	const FViewInfo& View,
	const FLightSceneInfo& LightSceneInfo,
	const FScreenPassTextureViewport& SceneViewport,
	const FScreenPassTextureViewport& LightShaftViewport)
{
	const FLightSceneProxy& LightSceneProxy = *LightSceneInfo.Proxy;
	const FScreenPassTextureViewportParameters SceneViewportParameters = GetScreenPassTextureViewportParameters(SceneViewport);
	const FScreenPassTextureViewportParameters LightShaftParameters = GetScreenPassTextureViewportParameters(LightShaftViewport);

	const FVector2D LightShaftRectToExtentRatio = LightShaftViewport.GetRectToExtentRatio();
	const FVector2f LightShaftAspectRatio(LightShaftRectToExtentRatio.X, (float)LightShaftViewport.Extent.X * LightShaftRectToExtentRatio.Y / LightShaftViewport.Extent.Y);
	const FVector2f LightShaftAspectRatioInverse = FVector2f(1.0f) / LightShaftAspectRatio;

	FLightShaftPixelShaderParameters Parameters;
	Parameters.View = View.ViewUniformBuffer;
	Parameters.AspectRatioAndInvAspectRatio = FVector4f(LightShaftAspectRatio, LightShaftAspectRatioInverse);

	{
		const FVector4f LightScreenPosition = GetLightScreenPosition(View, LightSceneProxy);
		const float InvW = 1.0f / LightScreenPosition.W;
		const float Y = (GProjectionSignY > 0.0f) ? LightScreenPosition.Y : 1.0f - LightScreenPosition.Y;
		const FVector2f ScreenSpaceBlurOrigin(
			View.ViewRect.Min.X + (0.5f + LightScreenPosition.X * 0.5f * InvW) * View.ViewRect.Width(),
			View.ViewRect.Min.Y + (0.5f - Y * 0.5f * InvW) * View.ViewRect.Height());

		Parameters.TextureSpaceBlurOrigin = ScreenSpaceBlurOrigin * SceneViewportParameters.ExtentInverse * LightShaftAspectRatioInverse;
	}

	Parameters.UVMinMax = FVector4f(LightShaftParameters.UVViewportBilinearMin, LightShaftParameters.UVViewportBilinearMax);

	const FLinearColor BloomTint = LightSceneInfo.BloomTint;
	Parameters.BloomTintAndThreshold = FVector4f(BloomTint.R, BloomTint.G, BloomTint.B, LightSceneInfo.BloomThreshold);
	Parameters.BloomMaxBrightness = LightSceneInfo.BloomMaxBrightness;

	float OcclusionMaskDarkness;
	float OcclusionDepthRange;
	LightSceneProxy.GetLightShaftOcclusionParameters(OcclusionMaskDarkness, OcclusionDepthRange);

	Parameters.LightShaftParameters = FVector4f(1.0f / OcclusionDepthRange, LightSceneInfo.BloomScale, 1, OcclusionMaskDarkness);

	return Parameters;
}

class FDownsampleLightShaftsVertexShader : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDownsampleLightShaftsVertexShader);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FDownsampleLightShaftsVertexShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDownsampleLightShaftsVertexShader, "/Engine/Private/LightShaftShader.usf", "DownsampleLightShaftsVertexMain", SF_Vertex);

class FDownsampleLightShaftsPixelShader : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDownsampleLightShaftsPixelShader);
	SHADER_USE_PARAMETER_STRUCT(FDownsampleLightShaftsPixelShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightShaftPixelShaderParameters, LightShafts)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	class FOcclusionTerm : SHADER_PERMUTATION_BOOL("OCCLUSION_TERM");
	using FPermutationDomain = TShaderPermutationDomain<FOcclusionTerm>;
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5); 
	}
};

IMPLEMENT_GLOBAL_SHADER(FDownsampleLightShaftsPixelShader, "/Engine/Private/LightShaftShader.usf", "DownsampleLightShaftsPixelMain", SF_Pixel);

class FBlurLightShaftsPixelShader : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FBlurLightShaftsPixelShader);
	SHADER_USE_PARAMETER_STRUCT(FBlurLightShaftsPixelShader, FGlobalShader);
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightShaftPixelShaderParameters, LightShafts)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTexture)
		SHADER_PARAMETER(FVector4f, RadialBlurParameters)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5); 
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("NUM_SAMPLES"), GLightShaftBlurNumSamples);
	}
};

IMPLEMENT_GLOBAL_SHADER(FBlurLightShaftsPixelShader, "/Engine/Private/LightShaftShader.usf", "BlurLightShaftsMain", SF_Pixel);

class FFinishOcclusionPixelShader : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FFinishOcclusionPixelShader);
	SHADER_USE_PARAMETER_STRUCT(FFinishOcclusionPixelShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightShaftPixelShaderParameters, LightShafts)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5); 
	}
};

IMPLEMENT_GLOBAL_SHADER(FFinishOcclusionPixelShader, "/Engine/Private/LightShaftShader.usf", "FinishOcclusionMain", SF_Pixel);

class FApplyLightShaftsPixelShader : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FApplyLightShaftsPixelShader);
	SHADER_USE_PARAMETER_STRUCT(FApplyLightShaftsPixelShader, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FLightShaftPixelShaderParameters, LightShafts)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SourceTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FApplyLightShaftsPixelShader, "/Engine/Private/LightShaftShader.usf", "ApplyLightShaftsPixelMain", SF_Pixel);

FScreenPassRenderTarget CreateLightShaftTexture(FRDGBuilder& GraphBuilder, FScreenPassTextureViewport Viewport, const TCHAR* Name)
{
	const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Viewport.Extent, PF_FloatRGB, FClearValueBinding::White, TexCreate_ShaderResource | TexCreate_RenderTargetable);
	return FScreenPassRenderTarget(GraphBuilder.CreateTexture(Desc, Name), Viewport.Rect, ERenderTargetLoadAction::ENoAction);
}

FScreenPassTexture AddDownsamplePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightShaftPixelShaderParameters& LightShaftParameters,
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures,
	FScreenPassTextureViewport SceneViewport,
	FScreenPassTextureViewport OutputViewport,
	ELightComponentType LightComponentType,
	ELightShaftTechnique LightShaftTechnique)
{
	const FScreenPassRenderTarget Output = CreateLightShaftTexture(GraphBuilder, OutputViewport, TEXT("LightShaftDownsample"));

	auto* PassParameters = GraphBuilder.AllocParameters<FDownsampleLightShaftsPixelShader::FParameters>();
	PassParameters->LightShafts = LightShaftParameters;
	PassParameters->SceneTextures = SceneTextures;
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	FDownsampleLightShaftsPixelShader::FPermutationDomain PixelPermutationVector;
	PixelPermutationVector.Set<FDownsampleLightShaftsPixelShader::FOcclusionTerm>(LightShaftTechnique == ELightShaftTechnique::Occlusion);

	TShaderMapRef<FDownsampleLightShaftsPixelShader> PixelShader(View.ShaderMap, PixelPermutationVector);
	TShaderMapRef<FDownsampleLightShaftsVertexShader> VertexShader(View.ShaderMap);

	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("Downsample %dx%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
		View,
		OutputViewport,
		SceneViewport,
		FScreenPassPipelineState(VertexShader, PixelShader),
		PassParameters,
		[VertexShader, PixelShader, PassParameters](FRHICommandList& RHICmdList)
	{
		FDownsampleLightShaftsVertexShader::FParameters VertexParameters;
		VertexParameters.View = PassParameters->LightShafts.View;
		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VertexParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
	});

	return FScreenPassTexture(Output);
}

FScreenPassTexture AddTemporalAAPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures,
	FTemporalAAHistory* HistoryState,
	FScreenPassTexture LightShafts)
{
	if (IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod) && HistoryState && GLightShaftAllowTAA)
	{
		const FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures);

		FTAAPassParameters TAAParameters(View);
		TAAParameters.Pass = ETAAPassConfig::LightShaft;
		TAAParameters.SceneDepthTexture = SceneTextureParameters.SceneDepthTexture;
		TAAParameters.SceneVelocityTexture = SceneTextureParameters.GBufferVelocityTexture;
		TAAParameters.SetupViewRect(View, GetLightShaftDownsampleFactor());
		TAAParameters.SceneColorInput = LightShafts.Texture;

		LightShafts.Texture = AddTemporalAAPass(GraphBuilder, View, TAAParameters, *HistoryState, HistoryState).SceneColor;
	}
	return LightShafts;
}

FScreenPassTexture AddRadialBlurPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightShaftPixelShaderParameters& LightShaftParameters,
	FScreenPassTexture LightShafts)
{
	const uint32 NumPasses = FMath::Max(GLightShaftBlurPasses, 0);

	for (uint32 PassIndex = 0; PassIndex < NumPasses; PassIndex++)
	{
		const FScreenPassRenderTarget Output = CreateLightShaftTexture(GraphBuilder, FScreenPassTextureViewport(LightShafts), TEXT("LightShaftBlur"));

		auto* PassParameters = GraphBuilder.AllocParameters<FBlurLightShaftsPixelShader::FParameters>();
		PassParameters->LightShafts = LightShaftParameters;
		PassParameters->SourceTexture = LightShafts.Texture;
		PassParameters->RadialBlurParameters = FVector4f(GLightShaftBlurNumSamples, GLightShaftFirstPassDistance, PassIndex);
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		TShaderMapRef<FBlurLightShaftsPixelShader> PixelShader(View.ShaderMap);
		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("RadialBlur(%d)", PassIndex), View, FScreenPassTextureViewport(Output), FScreenPassTextureViewport(LightShafts), PixelShader, PassParameters);
		LightShafts = Output;
	}

	return LightShafts;
}

FScreenPassTexture AddLightShaftSetupPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightShaftPixelShaderParameters& LightShaftParameters,
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures,
	FScreenPassTextureViewport SceneViewport,
	FScreenPassTextureViewport LightShaftViewport,
	FTemporalAAHistory* LightShaftTemporalHistory,
	ELightComponentType LightComponentType,
	ELightShaftTechnique LightShaftTechnique)
{
	FScreenPassTexture LightShafts;
	LightShafts = AddDownsamplePass(GraphBuilder, View, LightShaftParameters, SceneTextures, SceneViewport, LightShaftViewport, LightComponentType, LightShaftTechnique);
	LightShafts = AddTemporalAAPass(GraphBuilder, View, SceneTextures, LightShaftTemporalHistory, LightShafts);
	LightShafts = AddRadialBlurPass(GraphBuilder, View, LightShaftParameters, LightShafts);
	return LightShafts;
}

void AddOcclusionTermPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightShaftPixelShaderParameters& LightShaftParameters,
	FScreenPassTexture LightShafts,
	FScreenPassRenderTarget Output)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FFinishOcclusionPixelShader::FParameters>();
	PassParameters->LightShafts = LightShaftParameters;
	PassParameters->SourceTexture = LightShafts.Texture;
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	TShaderMapRef<FFinishOcclusionPixelShader> PixelShader(View.ShaderMap);
	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("FinishOcclusionTerm"), View, FScreenPassTextureViewport(Output), FScreenPassTextureViewport(LightShafts), PixelShader, PassParameters);
}

FRDGTextureRef FDeferredShadingSceneRenderer::RenderLightShaftOcclusion(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures)
{
	FScreenPassRenderTarget Output;

	if (ShouldRenderLightShafts(ViewFamily))
	{
		RDG_EVENT_SCOPE(GraphBuilder, "LightShafts (Occlusion)");

		for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
		{
			const FLightSceneInfo& LightSceneInfo = *LightSceneInfoCompact.LightSceneInfo;
			const FLightSceneProxy& LightSceneProxy = *LightSceneInfo.Proxy;
			const ELightComponentType LightComponentType = static_cast<ELightComponentType>(LightSceneProxy.GetLightType());

			if (LightComponentType != LightType_Directional)
			{
				continue;
			}

			float OcclusionMaskDarkness;
			float OcclusionDepthRange;
			const bool bEnableOcclusion = LightSceneProxy.GetLightShaftOcclusionParameters(OcclusionMaskDarkness, OcclusionDepthRange);

			if (bEnableOcclusion)
			{
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					const FViewInfo& View = Views[ViewIndex];

					if (ShouldRenderLightShaftsForLight(View, LightSceneProxy))
					{
						RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
						RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
						INC_DWORD_STAT(STAT_LightShaftsLights);

						const FScreenPassTextureViewport SceneViewport(SceneTextures.Config.Extent, View.ViewRect);
						const FScreenPassTextureViewport OutputViewport(GetDownscaledViewport(SceneViewport, GetLightShaftDownsampleFactor()));
						const FLightShaftPixelShaderParameters LightShaftParameters = GetLightShaftParameters(View, LightSceneInfo, SceneViewport, OutputViewport);
						FTemporalAAHistory* TemporalHistory = nullptr;

						if (!Output.IsValid())
						{
							Output = CreateLightShaftTexture(GraphBuilder, OutputViewport, TEXT("LightShaftOcclusion"));
							// If there are multiple views, we want to clear the texture in case some of them are not going to render any occlusion this frame 
							// See ShouldRenderLightShaftsForLight for the skipping logic.
							Output.LoadAction = Views.Num() > 1 ? ERenderTargetLoadAction::EClear : Output.LoadAction;
						}
						else
						{
							Output.ViewRect = OutputViewport.Rect;
							Output.UpdateVisualizeTextureExtent();
						}

						if (View.State)
						{
							TemporalHistory = &static_cast<FSceneViewState*>(View.State)->LightShaftOcclusionHistory;
						}

						FScreenPassTexture LightShafts = AddLightShaftSetupPass(
							GraphBuilder,
							View,
							LightShaftParameters,
							SceneTextures.UniformBuffer,
							SceneViewport,
							OutputViewport,
							TemporalHistory,
							LightComponentType,
							ELightShaftTechnique::Occlusion);

						AddOcclusionTermPass(GraphBuilder, View, LightShaftParameters, LightShafts, Output);

						// Subsequent views are composited into the same output target.
						Output.LoadAction = ERenderTargetLoadAction::ELoad;
					}
				}
			}
		}
	}

	return Output.Texture;
}

void AddLightShaftBloomPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLightShaftPixelShaderParameters& LightShaftParameters,
	FScreenPassTexture LightShafts,
	FScreenPassTextureViewport OutputViewport,
	FRenderTargetBinding OutputBinding)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FApplyLightShaftsPixelShader::FParameters>();
	PassParameters->LightShafts = LightShaftParameters;
	PassParameters->SourceTexture = LightShafts.Texture;
	PassParameters->RenderTargets[0] = OutputBinding;
	PassParameters->RenderTargets.ResolveRect = FResolveRect(OutputViewport.Rect);

	const FScreenPassTextureViewport InputViewport(LightShafts);

	TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FApplyLightShaftsPixelShader> PixelShader(View.ShaderMap);
	FRHIBlendState* BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Bloom"), View, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, PassParameters);
}

void FDeferredShadingSceneRenderer::RenderLightShaftBloom(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FTranslucencyPassResourcesMap& OutTranslucencyResourceMap)
{
	if (ShouldRenderLightShafts(ViewFamily))
	{
		RDG_EVENT_SCOPE(GraphBuilder, "LightShafts (Bloom)");

		TBitArray<TInlineAllocator<1, SceneRenderingBitArrayAllocator>> ViewsToRender(false, Views.Num());

		for (const FLightSceneInfoCompact& LightSceneInfoCompact : Scene->Lights)
		{
			const FLightSceneInfo& LightSceneInfo = *LightSceneInfoCompact.LightSceneInfo;
			const FLightSceneProxy& LightSceneProxy = *LightSceneInfo.Proxy;
			const ELightComponentType LightComponentType = static_cast<ELightComponentType>(LightSceneProxy.GetLightType());

			if (LightSceneInfo.bEnableLightShaftBloom)
			{
				bool bWillRenderLightShafts = false;
				
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					const FViewInfo& View    = Views[ViewIndex];
					const bool bRenderLight  = ShouldRenderLightShaftsForLight(View, LightSceneProxy);
					ViewsToRender[ViewIndex] = bRenderLight;
					bWillRenderLightShafts  |= bRenderLight;
				}

				if (bWillRenderLightShafts)
				{
					// Default to scene color output.
					FRDGTextureMSAA OutputTexture(SceneTextures.Color.Target);
					ERenderTargetLoadAction OutputLoadAction = ERenderTargetLoadAction::ELoad;
					float OutputViewRectScale = 1.0f;

					// Render to separate translucency buffer instead of scene color if requested.
					const ELightShaftBloomOutput BloomOutput = GetLightShaftBloomOutput(ViewFamily);
					bool bUpdateViewsSeparateTranslucency = false;
					if (BloomOutput == ELightShaftBloomOutput::SeparateTranslucency)
					{
						FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap.Get(/* ViewIndex = */ 0, ETranslucencyPass::TPT_TranslucencyAfterDOF);

						if (!TranslucencyPassResources.IsValid())
						{
							OutputLoadAction = ERenderTargetLoadAction::EClear;

							const bool bIsModulate = false;
							OutputTexture = CreatePostDOFTranslucentTexture(GraphBuilder, ETranslucencyPass::TPT_TranslucencyAfterDOF, SeparateTranslucencyDimensions, bIsModulate, ShaderPlatform);

							// We will need to update views separate transluceny buffers if we have just created them.
							bUpdateViewsSeparateTranslucency = true;
						}
						else
						{
							OutputTexture = TranslucencyPassResources.ColorTexture;
						}
						OutputViewRectScale = SeparateTranslucencyDimensions.Scale;
					}

					const FRenderTargetBinding OutputBinding(OutputTexture.Target, OutputTexture.Resolve, OutputLoadAction);

					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
					{
						const FViewInfo& View = Views[ViewIndex];

						if (ViewsToRender[ViewIndex])
						{
							RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
							RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
							INC_DWORD_STAT(STAT_LightShaftsLights);

							const FScreenPassTextureViewport SceneViewport(SceneTextures.Config.Extent, View.ViewRect);
							const FScreenPassTextureViewport LightShaftViewport(GetDownscaledViewport(SceneViewport, GetLightShaftDownsampleFactor()));
							const FScreenPassTextureViewport OutputViewport(OutputTexture.Target, GetScaledRect(View.ViewRect, OutputViewRectScale));
							const FLightShaftPixelShaderParameters LightShaftParameters = GetLightShaftParameters(View, LightSceneInfo, SceneViewport, LightShaftViewport);
							FTemporalAAHistory* TemporalHistory = nullptr;

							if (View.State)
							{
								FSceneViewState* ViewState = static_cast<FSceneViewState*>(View.State);
								TUniquePtr<FTemporalAAHistory>* Entry = ViewState->LightShaftBloomHistoryRTs.Find(LightSceneProxy.GetLightComponentId());
								if (Entry == nullptr)
								{
									TemporalHistory = new FTemporalAAHistory;
									ViewState->LightShaftBloomHistoryRTs.Emplace(LightSceneProxy.GetLightComponentId(), TemporalHistory);
								}
								else
								{
									TemporalHistory = Entry->Get();
								}
							}

							FScreenPassTexture LightShafts = AddLightShaftSetupPass(
								GraphBuilder,
								View,
								LightShaftParameters,
								SceneTextures.UniformBuffer,
								SceneViewport,
								LightShaftViewport,
								TemporalHistory,
								LightComponentType,
								ELightShaftTechnique::Bloom);

							AddLightShaftBloomPass(GraphBuilder, View, LightShaftParameters, LightShafts, OutputViewport, OutputBinding);
							OutputLoadAction = ERenderTargetLoadAction::ELoad;

							FTranslucencyPassResources& TranslucencyPassResources = OutTranslucencyResourceMap.Get(ViewIndex, ETranslucencyPass::TPT_TranslucencyAfterDOF);
							if (bUpdateViewsSeparateTranslucency)
							{
								TranslucencyPassResources.ViewRect = OutputViewport.Rect;
								TranslucencyPassResources.ColorTexture = OutputTexture;
							}
							else if(TranslucencyPassResources.IsValid())
							{
								if (BloomOutput == ELightShaftBloomOutput::SeparateTranslucency)
								{
									ensure(TranslucencyPassResources.ViewRect == OutputViewport.Rect);
								}
								else
								{
									ensure(View.ViewRect == OutputViewport.Rect);
								}
							}
						}
					}
				}
			}
		}
	}
}

// InScene is passed in, as the Scene pointer in the class itself may be null, if it was allocated without a scene.
void FSceneViewState::TrimHistoryRenderTargets(const FScene* InScene)
{
	for (TMap<FLightComponentId, TUniquePtr<FTemporalAAHistory>>::TIterator It(LightShaftBloomHistoryRTs); It; ++It)
	{
		bool bLightIsUsed = false;

		for (const FLightSceneInfoCompact& LightSceneInfoCompact : InScene->Lights)
		{
			const FLightSceneInfo* const LightSceneInfo = LightSceneInfoCompact.LightSceneInfo;

			if (LightSceneInfo->Proxy->GetLightComponentId() == It.Key())
			{
				bLightIsUsed = true;
				break;
			}
		}

		if (!bLightIsUsed)
		{
			// Remove references to render targets for lights that are no longer in the scene
			// This has to be done every frame instead of at light deregister time because the view states are not known by FScene
			It.RemoveCurrent();
		}
	}
}
