// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Math/Color.h"
#include "Math/Vector2D.h"

class FViewInfo;
class FLightSceneInfo;
class FLightSceneProxy;
class FSceneViewFamily;

struct FMobileLightShaftInfo
{
	FVector2D Center = FVector2D::ZeroVector;
	FLinearColor ColorMask = FLinearColor::Transparent;
	FLinearColor ColorApply = FLinearColor::Transparent;
	float BloomMaxBrightness = 100.0f;
};

// Returns mobile light shaft info for the light.
extern FMobileLightShaftInfo GetMobileLightShaftInfo(const FViewInfo& View, const FLightSceneInfo& LightSceneInfo);

// Returns whether light shafts globally are enabled.
extern bool ShouldRenderLightShafts(const FSceneViewFamily& ViewFamily);

// Returns whether the light proxy is eligible for light shaft rendering. Assumes light shafts are enabled.
extern bool ShouldRenderLightShaftsForLight(const FViewInfo& View, const FLightSceneProxy& LightSceneProxy);

// Returns the current downsample factor for the light shaft render target.
extern int32 GetLightShaftDownsampleFactor();