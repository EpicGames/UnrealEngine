// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocalFogVolumeRendering.h"
#include "ScenePrivate.h"
#include "RendererUtils.h"
#include "ScreenPass.h"
#include "LocalFogVolumeSceneProxy.h"
#include "MobileBasePassRendering.h"
#include "PixelShaderUtils.h"


// The runtime ON/OFF toggle
static TAutoConsoleVariable<int32> CVarLocalFogVolume(
	TEXT("r.LocalFogVolume"), 1,
	TEXT("Project settings enabling the rendering of Local Fog Volumes."),
	ECVF_RenderThreadSafe);

// The project setting (disable runtime and shader code)
static TAutoConsoleVariable<int32> CVarSupportLocalFogVolumes(
	TEXT("r.SupportLocalFogVolumes"),
	1,
	TEXT("Enables local fog volume rendering and shader code."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalFogVolumeRenderDuringHeightFogPass(
	TEXT("r.LocalFogVolume.RenderDuringHeightFogPass"), 0,
	TEXT("LocalFogVolume are going to be rendered during the height fog pass, skipping the tiled rendering pass specific to them. Only work on the non mobile path as an experiment."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalFogVolumeRenderIntoVolumetricFog(
	TEXT("r.LocalFogVolume.RenderIntoVolumetricFog"), 1,
	TEXT("Enables the voxelization of local fog volumes into the volumetric fog rendering system. Otherwise, local fog volumes will remain isolated."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarLocalFogVolumeMaxDensityIntoVolumetricFog(
	TEXT("r.LocalFogVolume.MaxDensityIntoVolumetricFog"), 0.01f,
	TEXT("LocalFogVolume height fog mode can become exponentially dense in the bottom part. VolumetricFog temporal reprojection then can leak du to high density. Clamping density is a way to get that visual artefact under control."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalFogVolumeApplyOnTranslucent(
	TEXT("r.LocalFogVolume.ApplyOnTranslucent"), 0,
	TEXT("Project settings enabling the sampling of local fog volumes on translucent elements."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalFogVolumeTilePixelSize(
	TEXT("r.LocalFogVolume.TilePixelSize"), 128,
	TEXT("Tile size on screen in pixel at which we cull the local fog volumes."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalFogVolumeTileMaxInstanceCount(
	TEXT("r.LocalFogVolume.TileMaxInstanceCount"), 32,
	TEXT("Maximum number of local fog volumes to account for per view (and per tile or consistency)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalFogVolumeTileCullingUseAsync(
	TEXT("r.LocalFogVolume.TileCullingUseAsync"), 1,
	TEXT("Enables running the culling process on the async compute pipe."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalFogVolumeTileDebug(
	TEXT("r.LocalFogVolume.TileDebug"), 0,
	TEXT("Debug the tiled rendering data complexity. 1: show per tile LFV count as color ; 2: same as one but also show the effect of pixel discard/clipping."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarLocalFogVolumeGlobalStartDistance(
	TEXT("r.LocalFogVolume.GlobalStartDistance"), 1000.0f,
	TEXT("The start distance in centimeter from which local fog volumes starts to appear. Default to 10 meters. This will push away from the camera all the local fog volume visibility independenly of their own start distance."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalFogVolumeUseHZB(
	TEXT("r.LocalFogVolume.UseHZB"), 1,
	TEXT("Enables the use of the HZB to cull local fog volumes away.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLocalFogVolumeHalfResolution(
	TEXT("r.LocalFogVolume.HalfResolution"), 0,
	TEXT("Enables half resolution rendering of local fog volumes with an upsampling to full resolution later. Only works for the mobile path for now.\n"),
	ECVF_RenderThreadSafe);

// Example of tile setup
//  - 1920x1080 => 15x9 tiles
//  - Allowing max 32 volumes at once => culling list buffer = 15 * 9 * 32 * 1 byte = 4320 bytes = 4.3KB

static uint32 GetLocalFogVolumeTilePixelSize()
{
	return FMath::Max(8u, FMath::Min(512u, (uint32)CVarLocalFogVolumeTilePixelSize.GetValueOnRenderThread()));
}

static bool GetLocalFogVolumeTileCullingUseAsync()
{
	return GSupportsEfficientAsyncCompute && CVarLocalFogVolumeTileCullingUseAsync.GetValueOnRenderThread() > 0;
}

static float GetLocalFogVolumeMaxDensityIntoVolumetricFog()
{
	return FMath::Max(0.0f, CVarLocalFogVolumeMaxDensityIntoVolumetricFog.GetValueOnRenderThread());
}

static uint32 GetLocalFogVolumeTileMaxInstanceCount()
{
	// We do not allow more than 256 instances since culled indices might be stored a u8 values.
	return FMath::Max(1u, FMath::Min(256u, (uint32)CVarLocalFogVolumeTileMaxInstanceCount.GetValueOnRenderThread()));
}

static int32 GetLocalFogVolumeTileDebugMode(const FEngineShowFlags& EngineShowFlags)
{
	return EngineShowFlags.VisualizeLocalFogVolumes ? 2 : FMath::Clamp(CVarLocalFogVolumeTileDebug.GetValueOnRenderThread(), 0, 2);
}

bool ProjectSupportsLocalFogVolumes()
{
	return CVarSupportLocalFogVolumes.GetValueOnRenderThread() > 0;
}

bool ShouldRenderLocalFogVolume(const FScene* Scene, const FSceneViewFamily& SceneViewFamily)
{
	const FEngineShowFlags& EngineShowFlags = SceneViewFamily.EngineShowFlags;
	if (Scene && Scene->HasAnyLocalFogVolume() && EngineShowFlags.Fog && !SceneViewFamily.UseDebugViewPS())
	{
		return ProjectSupportsLocalFogVolumes() && (CVarLocalFogVolume.GetValueOnRenderThread() > 0);
	}
	return false;
}

bool ShouldRenderLocalFogVolumeDuringHeightFogPass(const FScene* Scene, const FSceneViewFamily& SceneViewFamily)
{
	if (ShouldRenderLocalFogVolume(Scene, SceneViewFamily))
	{
		return CVarLocalFogVolumeRenderDuringHeightFogPass.GetValueOnRenderThread() > 0;
	}
	return false;
}

bool ShouldRenderLocalFogVolumeInVolumetricFog(const FScene* Scene, const FSceneViewFamily& SceneViewFamily, bool bShouldRenderVolumetricFog)
{
	if (ShouldRenderLocalFogVolume(Scene, SceneViewFamily) && bShouldRenderVolumetricFog)
	{
		return CVarLocalFogVolumeRenderIntoVolumetricFog.GetValueOnRenderThread() > 0;
	}
	return false;
}

bool ShouldRenderLocalFogVolumeVisualizationPass(const FScene* Scene, const FSceneViewFamily& SceneViewFamily)
{
	if (ShouldRenderLocalFogVolume(Scene, SceneViewFamily) 
		&& !SceneViewFamily.EngineShowFlags.VisualizeLocalFogVolumes)	// When this is set, the debug is rendered using the post process chain.
	{
		const FEngineShowFlags& EngineShowFlags = SceneViewFamily.EngineShowFlags;
		return GetLocalFogVolumeTileDebugMode(EngineShowFlags) > 0;
	}
	return false;
}

float GetLocalFogVolumeGlobalStartDistance()
{
	return FMath::Max(10.0f, CVarLocalFogVolumeGlobalStartDistance.GetValueOnRenderThread());
}

float GetLocalFogVolumeViewStartDistance(const FViewInfo& View)
{
	const float FinalStartDistance = FMath::Max(GetLocalFogVolumeGlobalStartDistance(), View.LocalFogVolumeViewData.DepthBoundStartDistance);
	return FinalStartDistance;
}

bool IsLocalFogVolumeHalfResolution()
{
	return CVarLocalFogVolumeHalfResolution.GetValueOnRenderThread() > 0;
}

DECLARE_GPU_STAT(LocalFogVolumeVolumes);

static const uint32 SizeOfUintVec4 = sizeof(FUintVector4);
static const uint32 UintVec4CountInLocalFogVolumeGPUInstanceData = sizeof(FLocalFogVolumeGPUInstanceData) / SizeOfUintVec4;

void SetDummyLocalFogVolumeUniformParametersStruct(FRDGBuilder& GraphBuilder, FLocalFogVolumeUniformParameters& UniformParametersStruct)
{
	UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTileDataTextureResolution = FUintVector2(1, 1);
	UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeInstanceCount = 0;
	UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTilePixelSize = GetLocalFogVolumeTilePixelSize();
	UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeMaxDensityIntoVolumetricFog = GetLocalFogVolumeMaxDensityIntoVolumetricFog();
	UniformParametersStruct.LocalFogVolumeCommon.ShouldRenderLocalFogVolumeInVolumetricFog = 0;
	UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightColor = FVector3f::Zero();
	UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightDirection = FVector3f::Zero();
	UniformParametersStruct.LocalFogVolumeCommon.GlobalStartDistance = 0.0f;
	UniformParametersStruct.LocalFogVolumeCommon.HalfResTextureSizeAndInvSize = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);

	UniformParametersStruct.LocalFogVolumeTileDataTexture = GraphBuilder.CreateSRV(GSystemTextures.GetZeroUIntArrayDummy(GraphBuilder));

	static FLocalFogVolumeGPUInstanceData DummyData;	// Static data so ERDGInitialDataFlags::NoCopy can be used. 
	FRDGBufferRef GPUInstanceDataBuffer = CreateVertexBuffer(GraphBuilder, TEXT("DUMMYLocalFogVolumeGPUInstanceDataBuffer"),
		FRDGBufferDesc::CreateBufferDesc(SizeOfUintVec4, UintVec4CountInLocalFogVolumeGPUInstanceData), &DummyData, sizeof(FLocalFogVolumeGPUInstanceData) * 1, ERDGInitialDataFlags::NoCopy);
	UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeInstances = GraphBuilder.CreateSRV(GPUInstanceDataBuffer, PF_A32B32G32R32F);
}

void SetDummyLocalFogVolumeForView(FRDGBuilder& GraphBuilder, FViewInfo& View)
{
	// The size of the structure must be a multiple of FUintVector4.
	static_assert(sizeof(FLocalFogVolumeGPUInstanceData) == UintVec4CountInLocalFogVolumeGPUInstanceData * SizeOfUintVec4);

	View.LocalFogVolumeViewData.GPUInstanceCount			= 0;
	View.LocalFogVolumeViewData.DepthBoundStartDistance		= FLT_MAX;

	static FLocalFogVolumeGPUInstanceData DummyData;	// Static data so ERDGInitialDataFlags::NoCopy can be used. 
	View.LocalFogVolumeViewData.GPUInstanceDataBuffer		= CreateVertexBuffer(GraphBuilder, TEXT("DUMMYLocalFogVolumeGPUInstanceDataBuffer"),
		FRDGBufferDesc::CreateBufferDesc(SizeOfUintVec4, UintVec4CountInLocalFogVolumeGPUInstanceData), &DummyData, sizeof(FLocalFogVolumeGPUInstanceData) * 1, ERDGInitialDataFlags::NoCopy);
	View.LocalFogVolumeViewData.GPUInstanceDataBufferSRV	= GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.GPUInstanceDataBuffer, PF_A32B32G32R32F);

	static FVector4f DummyCullingData(EForceInit::ForceInitToZero);
	View.LocalFogVolumeViewData.GPUInstanceCullingDataBuffer	= CreateVertexBuffer(GraphBuilder, TEXT("DUMMYLocalFogVolumeGPUInstanceCullingDataBuffer"),
		FRDGBufferDesc::CreateBufferDesc(sizeof(FVector4f), 1), &DummyCullingData, sizeof(FVector4f) * 1, ERDGInitialDataFlags::NoCopy);
	View.LocalFogVolumeViewData.GPUInstanceCullingDataBufferSRV = GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.GPUInstanceCullingDataBuffer, PF_A32B32G32R32F);

	View.LocalFogVolumeViewData.TileDataTextureArray			= GSystemTextures.GetZeroUIntArrayDummy(GraphBuilder);
	View.LocalFogVolumeViewData.TileDataTextureArraySRV			= GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.TileDataTextureArray);
	View.LocalFogVolumeViewData.TileDataTextureArrayUAV			= nullptr;	// Should never be written by culling passes if there are no instances.

	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTileDataTextureResolution	= FUintVector2(1, 1);
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeInstanceCount				= View.LocalFogVolumeViewData.GPUInstanceCount;
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTilePixelSize				= GetLocalFogVolumeTilePixelSize();
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeMaxDensityIntoVolumetricFog	= GetLocalFogVolumeMaxDensityIntoVolumetricFog();
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.ShouldRenderLocalFogVolumeInVolumetricFog	= 0;
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeInstances					= View.LocalFogVolumeViewData.GPUInstanceDataBufferSRV;
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightColor						= FVector3f::Zero();
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightDirection					= FVector3f::Zero();
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.GlobalStartDistance						= 0.0f;
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.HalfResTextureSizeAndInvSize				= FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeTileDataTexture									= View.LocalFogVolumeViewData.TileDataTextureArraySRV;
	View.LocalFogVolumeViewData.UniformBuffer																			= GraphBuilder.CreateUniformBuffer(&View.LocalFogVolumeViewData.UniformParametersStruct);

	// This buffer must remain a basic vertex buffer for mobile to be able to read it from vertex shader
	View.LocalFogVolumeViewData.GPUTileDataBuffer		= CreateVertexBuffer(
		GraphBuilder, TEXT("LocalFogVolume.GPUTileDataBuffer"), FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1 * 1 * sizeof(uint32)), nullptr, 0);
	View.LocalFogVolumeViewData.GPUTileDataBufferSRV	= GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.GPUTileDataBuffer, PF_R32_UINT);
	View.LocalFogVolumeViewData.GPUTileDataBufferUAV	= nullptr;	// Should never be written by culling passes if there are no instances.

	View.LocalFogVolumeViewData.GPUTileDrawIndirectBuffer	= GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndirectParameters>(), TEXT("LocalFogVolume.DispatchIndirectBuffer"));
	View.LocalFogVolumeViewData.GPUTileDrawIndirectBufferUAV= nullptr;

	View.LocalFogVolumeViewData.bUseHalfResLocalFogVolume = false;
	View.LocalFogVolumeViewData.HalfResLocalFogVolumeView = GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder);
	View.LocalFogVolumeViewData.HalfResLocalFogVolumeViewSRV = GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.HalfResLocalFogVolumeView);
	View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepth = GSystemTextures.GetBlackDummy(GraphBuilder);
	View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepthSRV = GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepth);
}

void SetDummyLocalFogVolumeForViews(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views)
{
	for (FViewInfo& View : Views)
	{
		SetDummyLocalFogVolumeForView(GraphBuilder, View);
	}
}

void SetDummyLocalFogVolumeForViews(FRDGBuilder& GraphBuilder, TArray<FViewInfo*>& Views)
{
	for (FViewInfo* View : Views)
	{
		SetDummyLocalFogVolumeForView(GraphBuilder, *View);
	}
}


/*=============================================================================
	FScene functions
=============================================================================*/

void FScene::AddLocalFogVolume(class FLocalFogVolumeSceneProxy* FogProxy)
{
	check(FogProxy);
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FAddLocalFogVolumeCommand)(
		[Scene, FogProxy](FRHICommandListImmediate& RHICmdList)
		{
			check(!Scene->LocalFogVolumes.Contains(FogProxy));
			Scene->LocalFogVolumes.Push(FogProxy);
		} );
}

void FScene::RemoveLocalFogVolume(class FLocalFogVolumeSceneProxy* FogProxy)
{
	check(FogProxy);
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FRemoveLocalFogVolumeCommand)(
		[Scene, FogProxy](FRHICommandListImmediate& RHICmdList)
		{
			Scene->LocalFogVolumes.RemoveSingle(FogProxy);
		} );
}

bool FScene::HasAnyLocalFogVolume() const
{ 
	return LocalFogVolumes.Num() > 0;
}

/*=============================================================================
	Local height fog tiled culling
=============================================================================*/


class FLocalFogVolumeTiledCullingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLocalFogVolumeTiledCullingCS);
	SHADER_USE_PARAMETER_STRUCT(FLocalFogVolumeTiledCullingCS, FGlobalShader);

	class FUseHZB : SHADER_PERMUTATION_BOOL("USE_HZB");
	using FPermutationDomain = TShaderPermutationDomain<FUseHZB>;

public:
	const static uint32 GroupSize = 8;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FLocalFogVolumeCommonParameters, LFV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, LocalFogVolumeTileDataTextureUAV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, LocalFogVolumeCullingDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, TileDataBufferUAV)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, TileDrawIndirectBufferUAV)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER(FVector4f, LeftPlane)
		SHADER_PARAMETER(FVector4f, RightPlane)
		SHADER_PARAMETER(FVector4f, TopPlane)
		SHADER_PARAMETER(FVector4f, BottomPlane)
		SHADER_PARAMETER(FVector4f, NearPlane)
		SHADER_PARAMETER(FVector2f, ViewToTileSpaceRatio)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize);
	}
};
IMPLEMENT_GLOBAL_SHADER(FLocalFogVolumeTiledCullingCS, "/Engine/Private/LocalFogVolumes/LocalFogVolumeTiledCulling.usf", "LocalFogVolumeTiledCullingCS", SF_Compute);

static void LocalFogVolumeViewTiledCullingPass(FViewInfo& View, FRDGBuilder& GraphBuilder)
{
	check(View.LocalFogVolumeViewData.GPUInstanceCount > 0);
	FIntVector TileDataTextureSize = View.LocalFogVolumeViewData.TileDataTextureArray->Desc.GetSize();

	FLocalFogVolumeTiledCullingCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FLocalFogVolumeTiledCullingCS::FParameters>();
	PassParameters->View								= View.ViewUniformBuffer;
	PassParameters->LFV									= View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon;
	PassParameters->LocalFogVolumeCullingDataBuffer		= View.LocalFogVolumeViewData.GPUInstanceCullingDataBufferSRV;
	PassParameters->LocalFogVolumeTileDataTextureUAV	= View.LocalFogVolumeViewData.TileDataTextureArrayUAV;
	PassParameters->TileDataBufferUAV					= View.LocalFogVolumeViewData.GPUTileDataBufferUAV;
	PassParameters->TileDrawIndirectBufferUAV			= View.LocalFogVolumeViewData.GPUTileDrawIndirectBufferUAV;
	PassParameters->HZBParameters 						= GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);

	auto ConvertPlanToVector4f = [&](FVector4f& OutVec4f, auto& Plane, bool bFlipPlane)
	{
		OutVec4f.X = Plane.X;
		OutVec4f.Y = Plane.Y;
		OutVec4f.Z = Plane.Z;
		OutVec4f.W = Plane.W;
		if (bFlipPlane)
		{
			// We swap some of the planes normal so that they are lerpable while avoiding potential null normal and precision issue at the middle of the frustum.
			OutVec4f.X *= -1.0f;
			OutVec4f.Y *= -1.0f;
			OutVec4f.Z *= -1.0f;
			OutVec4f.W *= -1.0f;
		}
	};
	
	// Using world space plane for now. LFV_TODO: do computation in view space.
	if (View.ViewFrustum.Planes.Num() >= 4)
	{
		// We use the view frustum witch matches the rendering frustum even when in stereo mode (not monoscopic).
		ConvertPlanToVector4f(PassParameters->LeftPlane,	View.ViewFrustum.Planes[0],	false);
		ConvertPlanToVector4f(PassParameters->RightPlane,	View.ViewFrustum.Planes[1],	true);
		ConvertPlanToVector4f(PassParameters->TopPlane,		View.ViewFrustum.Planes[2],	true);
		ConvertPlanToVector4f(PassParameters->BottomPlane,	View.ViewFrustum.Planes[3],	false);
	}
	else
	{
		// Disable culling and make each volume visible.
		PassParameters->LeftPlane	= FVector4f::Zero();
		PassParameters->RightPlane	= FVector4f::Zero();
		PassParameters->TopPlane	= FVector4f::Zero();
		PassParameters->BottomPlane	= FVector4f::Zero();
	}
	ConvertPlanToVector4f(PassParameters->NearPlane,	View.NearClippingPlane,			false);

	float TileCoveredResolutionX = View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTilePixelSize * View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTileDataTextureResolution.X;
	float TileCoveredResolutionY = View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTilePixelSize * View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTileDataTextureResolution.Y;
	PassParameters->ViewToTileSpaceRatio = FVector2f(TileCoveredResolutionX * View.CachedViewUniformShaderParameters->ViewSizeAndInvSize.Z, TileCoveredResolutionY * View.CachedViewUniformShaderParameters->ViewSizeAndInvSize.W);
 
	ERDGPassFlags PassFlag = GetLocalFogVolumeTileCullingUseAsync() ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute;

	TileDataTextureSize.Z = 1;
	const FIntVector NumGroups = FIntVector::DivideAndRoundUp(TileDataTextureSize, FLocalFogVolumeTiledCullingCS::GroupSize);

	
	FLocalFogVolumeTiledCullingCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FLocalFogVolumeTiledCullingCS::FUseHZB>(IsHZBValid(View, EHZBType::FurthestHZB) && CVarLocalFogVolumeUseHZB.GetValueOnRenderThread());
	TShaderMapRef<FLocalFogVolumeTiledCullingCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("LocalFogVolume.TiledCulling"), PassFlag, ComputeShader, PassParameters, NumGroups);
}


/*=============================================================================
	Local height fog rendering common function
=============================================================================*/

// This is view data because the transform is expressed in translated world space
void GetLocalFogVolumeViewSortingData(const FScene* Scene, const FViewInfo& View, FRDGBuilder& GraphBuilder, FLocalFogVolumeSortingData& Out)
{
	check(Scene->LocalFogVolumes.Num() > 0); // We should not get there if there is not any local fog volume.

	// No culling as of today
	Out.LocalFogVolumeInstanceCount = Scene->LocalFogVolumes.Num();
	Out.LocalFogVolumeInstanceCountFinal = 0;
	Out.LocalFogVolumeGPUInstanceData = (FLocalFogVolumeGPUInstanceData*)GraphBuilder.Alloc(sizeof(FLocalFogVolumeGPUInstanceData) * Out.LocalFogVolumeInstanceCount, 16);
	Out.LocalFogVolumeCenterPos = (FVector*)GraphBuilder.Alloc(sizeof(FVector) * Out.LocalFogVolumeInstanceCount, 16);
	Out.LocalFogVolumeStartDistanceAndUniformScale = (FVector2f*)GraphBuilder.Alloc(sizeof(FVector2f) * Out.LocalFogVolumeInstanceCount, 16);
	Out.LocalFogVolumeSortKeys.SetNumUninitialized(Out.LocalFogVolumeInstanceCount);
	for (FLocalFogVolumeSceneProxy* LHF : Scene->LocalFogVolumes)
	{
		if (LHF->RadialFogExtinction <= 0.0f && LHF->HeightFogExtinction <= 0.0f)
		{
			continue; // this volume will never be visible
		}

		FLocalFogVolumeGPUInstanceData* LocalFogVolumeGPUInstanceDataIt = &Out.LocalFogVolumeGPUInstanceData[Out.LocalFogVolumeInstanceCountFinal];

		// Falloff needs to be made safe in order to avoid artifact when the camera is looking toward the horizon at the level of the offset.
		const float SafeFalloffThreshold = 1.0f;
		const float FalloffScaleUI = 0.01f;
		const float SafeFallOff = FMath::Max(LHF->HeightFogFalloff, SafeFalloffThreshold) * FalloffScaleUI;

		FTransform RotationScaleMatrix = LHF->FogTransform;
		RotationScaleMatrix.SetTranslation(FVector::ZeroVector);
		FMatrix44f InvTransform = FMatrix44f(RotationScaleMatrix.ToMatrixWithScale().Inverse());

		FVector3f XVec(InvTransform.M[0][0], InvTransform.M[0][1], InvTransform.M[0][2]);
		FVector3f YVec(InvTransform.M[1][0], InvTransform.M[1][1], InvTransform.M[1][2]);
		FVector3f TranslatedWorldPosition(View.ViewMatrices.GetPreViewTranslation() + LHF->FogTransform.GetTranslation());

		// Normalization requires small tolerance for large volumes.
		const float NormalizeTolerance = 1.e-32f;
		XVec.Normalize(NormalizeTolerance);
		YVec.Normalize(NormalizeTolerance);

		auto AsUint32 = [](float X)
		{
			union { float F; uint32 U; } FU = { X };
			return FU.U;
		};

		auto AsFloat111110 = [](float x, float y, float z)
		{
			FVector2DHalf HalfXY(x, y);
			FVector2DHalf HalfZ0(z, 0.0f);

			uint32 r = (uint32(HalfXY.X.Encoded) << 17) & 0xFFE00000;
			uint32 g = (uint32(HalfXY.Y.Encoded) <<  6) & 0x001FFC00;
			uint32 b = (uint32(HalfZ0.X.Encoded) >>  5) & 0x000003FF;
			return r | g | b;
		};

		auto AsUNorm8888 = [](float x, float y, float z, float w)
		{
			uint32 r = (uint32(FMath::Clamp(x * 255.0f, 0.0f, 255.0f)) & 0xFFu);
			uint32 g = (uint32(FMath::Clamp(y * 255.0f, 0.0f, 255.0f)) & 0xFFu) << 8u;
			uint32 b = (uint32(FMath::Clamp(z * 255.0f, 0.0f, 255.0f)) & 0xFFu) << 16u;
			uint32 a = (uint32(FMath::Clamp(w * 255.0f, 0.0f, 255.0f)) & 0xFFu) << 24u;
			return r | g | b | a;
		};

		// Translation and scale at fp32 for stability. Further optimization: we could use fp16 if translated/view space position would be sent.
		LocalFogVolumeGPUInstanceDataIt->Data0[0] = AsUint32(TranslatedWorldPosition.X);
		LocalFogVolumeGPUInstanceDataIt->Data0[1] = AsUint32(TranslatedWorldPosition.Y);
		LocalFogVolumeGPUInstanceDataIt->Data0[2] = AsUint32(TranslatedWorldPosition.Z);
		LocalFogVolumeGPUInstanceDataIt->Data0[3] = AsUint32(LHF->FogUniformScale);

		// Store X and Y from the rotation matrix and recover Z in the shader.
		LocalFogVolumeGPUInstanceDataIt->Data1[0] = FVector2DHalf(XVec.X, XVec.Y).AsUInt32();
		LocalFogVolumeGPUInstanceDataIt->Data1[1] = FVector2DHalf(XVec.Z, YVec.X).AsUInt32();
		LocalFogVolumeGPUInstanceDataIt->Data1[2] = FVector2DHalf(YVec.Y, YVec.Z).AsUInt32();
		LocalFogVolumeGPUInstanceDataIt->Data1[3] = AsUint32(LHF->FogStartDistance);

		// All the remaining data are packed as small as possible w.r.t. their range of value.
		LocalFogVolumeGPUInstanceDataIt->Data2[0] = AsFloat111110(	LHF->RadialFogExtinction,	LHF->HeightFogExtinction,	SafeFallOff);
		LocalFogVolumeGPUInstanceDataIt->Data2[1] = AsFloat111110(	LHF->FogEmissive.R,			LHF->FogEmissive.G,			LHF->FogEmissive.B);
		LocalFogVolumeGPUInstanceDataIt->Data2[2] = AsUNorm8888(	LHF->FogAlbedo.R,			LHF->FogAlbedo.G,			LHF->FogAlbedo.B,		LHF->FogPhaseG);
		LocalFogVolumeGPUInstanceDataIt->Data2[3] = AsUint32(		LHF->HeightFogOffset);

		// Register the sorting data
		Out.LocalFogVolumeCenterPos[Out.LocalFogVolumeInstanceCountFinal]					= LHF->FogTransform.GetTranslation();
		Out.LocalFogVolumeStartDistanceAndUniformScale[Out.LocalFogVolumeInstanceCountFinal]= FVector2f(LHF->FogStartDistance, LHF->FogUniformScale);

		FLocalFogVolumeSortKey* LocalFogVolumeSortKeysIt= &Out.LocalFogVolumeSortKeys[Out.LocalFogVolumeInstanceCountFinal];
		LocalFogVolumeSortKeysIt->FogVolume.Index		= Out.LocalFogVolumeInstanceCountFinal;
		LocalFogVolumeSortKeysIt->FogVolume.Distance	= 0;	// Filled up right before sorting according to a view
		LocalFogVolumeSortKeysIt->FogVolume.Priority	= LHF->FogSortPriority;

		Out.LocalFogVolumeInstanceCountFinal++;
	}
	// Shrink the array to only what is needed in order for the sort to correctly work on only what is needed.
	Out.LocalFogVolumeSortKeys.SetNum(Out.LocalFogVolumeInstanceCountFinal, EAllowShrinking::No);
}

void CreateViewLocalFogVolumeBufferSRV(const FScene* Scene, FViewInfo& View, FRDGBuilder& GraphBuilder, FLocalFogVolumeSortingData& SortingData, bool bShouldRenderLocalFogVolumeInVolumetricFog, bool bUseHalfResLocalFogVolume)
{
	if (SortingData.LocalFogVolumeInstanceCountFinal == 0)
	{
		SetDummyLocalFogVolumeForView(GraphBuilder, View);
		return;
	}

	// 1. Sort all the volumes
	const FVector ViewOrigin = View.ViewMatrices.GetViewOrigin();
	float DepthBoundStartDistance = FLT_MAX;
	for (uint32 i = 0; i < SortingData.LocalFogVolumeInstanceCountFinal; ++i)
	{
		const uint64 Index = SortingData.LocalFogVolumeSortKeys[i].FogVolume.Index;
		const FVector FogCenterPos = SortingData.LocalFogVolumeCenterPos[Index];	// Recovered form the original array via index because the sorting of the previous view might have changed the order.
		const FVector2f FogStartDistanceAndUniformScale = SortingData.LocalFogVolumeStartDistanceAndUniformScale[Index];
		const float FogStartDistance = float(FogStartDistanceAndUniformScale.X);
		const float FogUniformScale = float(FogStartDistanceAndUniformScale.Y);

		const float DistanceToView = float((FogCenterPos - ViewOrigin).Size());
		SortingData.LocalFogVolumeSortKeys[i].FogVolume.Distance = BitCast<uint32>(DistanceToView);

		// For a volume, if the camera is inside, the start closest start distance is FogStartDistance.
		// Otherwise it is the distance to the edge of the volume.
		DepthBoundStartDistance = FMath::Min(DepthBoundStartDistance, FMath::Max(FogStartDistance, DistanceToView - FogUniformScale));
	}
	SortingData.LocalFogVolumeSortKeys.Sort();

	// We limit the instance count to the maximum of instance we can have per tile
	const uint32 LocalFogVolumeTileMaxInstanceCount = GetLocalFogVolumeTileMaxInstanceCount();
	const uint32 DiscardedOffset = (uint32)FMath::Max(0, int32(SortingData.LocalFogVolumeInstanceCountFinal) - int32(LocalFogVolumeTileMaxInstanceCount));
	SortingData.LocalFogVolumeInstanceCountFinal = FMath::Min(SortingData.LocalFogVolumeInstanceCountFinal, LocalFogVolumeTileMaxInstanceCount);

	// 2. Create the buffer containing all the fog volume data instance sorted according to their key for the current view.
	FLocalFogVolumeGPUInstanceData* LocalFogVolumeGPUSortedInstanceData = (FLocalFogVolumeGPUInstanceData*)GraphBuilder.Alloc(sizeof(FLocalFogVolumeGPUInstanceData) * SortingData.LocalFogVolumeInstanceCountFinal, 16);
	FVector4f* LocalFogVolumeGPUSortedInstanceCullingData = (FVector4f*)GraphBuilder.Alloc(sizeof(FVector4f) * SortingData.LocalFogVolumeInstanceCountFinal, 16);
	for (uint32 i = 0; i < SortingData.LocalFogVolumeInstanceCountFinal; i++)
	{
		FLocalFogVolumeSortKey LFVKey = SortingData.LocalFogVolumeSortKeys[i + DiscardedOffset];

		// We could also have an indirection buffer on GPU but choosing to go with the sorting + copy on CPU since it is expected to not have many local height fog volumes.
		LocalFogVolumeGPUSortedInstanceData[i] = SortingData.LocalFogVolumeGPUInstanceData[LFVKey.FogVolume.Index];

		const FVector& LFVPosition = SortingData.LocalFogVolumeCenterPos[LFVKey.FogVolume.Index];
		LocalFogVolumeGPUSortedInstanceCullingData[i] = FVector4f(LFVPosition.X, LFVPosition.Y, LFVPosition.Z, LocalFogVolumeGPUSortedInstanceData[i].GetUniformScale());
	}

	// 3. Allocate buffer and initialize with sorted data to upload to GPU
	const uint32 AllLocalFogVolumeInstanceBytesFinal = sizeof(FLocalFogVolumeGPUInstanceData) * SortingData.LocalFogVolumeInstanceCountFinal;

	View.LocalFogVolumeViewData.GPUInstanceCount			= SortingData.LocalFogVolumeInstanceCountFinal;
	View.LocalFogVolumeViewData.DepthBoundStartDistance		= DepthBoundStartDistance;

	View.LocalFogVolumeViewData.GPUInstanceDataBuffer		= CreateVertexBuffer(
		GraphBuilder, TEXT("LocalFogVolume.GPUInstanceDataBuffer"),
		FRDGBufferDesc::CreateBufferDesc(SizeOfUintVec4, SortingData.LocalFogVolumeInstanceCountFinal * UintVec4CountInLocalFogVolumeGPUInstanceData),
		LocalFogVolumeGPUSortedInstanceData, AllLocalFogVolumeInstanceBytesFinal, ERDGInitialDataFlags::NoCopy);
	View.LocalFogVolumeViewData.GPUInstanceDataBufferSRV	= GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.GPUInstanceDataBuffer, PF_A32B32G32R32F);

	View.LocalFogVolumeViewData.GPUInstanceCullingDataBuffer = CreateVertexBuffer(
		GraphBuilder, TEXT("LocalFogVolume.GPUInstanceCullingDataBuffer"),
		FRDGBufferDesc::CreateBufferDesc(sizeof(FVector4f), SortingData.LocalFogVolumeInstanceCountFinal),
		LocalFogVolumeGPUSortedInstanceCullingData, SortingData.LocalFogVolumeInstanceCountFinal * sizeof(FVector4f), ERDGInitialDataFlags::NoCopy);
	View.LocalFogVolumeViewData.GPUInstanceCullingDataBufferSRV = GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.GPUInstanceCullingDataBuffer, PF_A32B32G32R32F); // LFV_TODO use byte buffer to leverage scalar pipe in the culling compute shader.

	// Create the texture that will contain the tiled culled result: count in the first slice and indices in the remaining slices
	const uint32 LocalFogVolumeTilePixelSize				= GetLocalFogVolumeTilePixelSize();

	const FIntPoint TileDataTextureResolutionUnsafe			= FIntPoint::DivideAndRoundUp(View.ViewRect.Size(), FIntPoint(LocalFogVolumeTilePixelSize, LocalFogVolumeTilePixelSize));
	const FIntPoint TileDataTextureResolution				= FIntPoint(FMath::Max(1, TileDataTextureResolutionUnsafe.X), FMath::Max(1, TileDataTextureResolutionUnsafe.Y));
	const uint32 TileDataTextureSliceCount					= LocalFogVolumeTileMaxInstanceCount + 1; // +1 because the first slice is the culled instance count

	EPixelFormat TileDataFormat = PF_R8_UINT;	
	if(!UE::PixelFormat::HasCapabilities(TileDataFormat, EPixelFormatCapabilities::UAV))
	{
		// Some mobile platforms do not support UAV onto R8. A 32bit format is required.
		TileDataFormat = PF_R8G8B8A8_UINT;
		check(UE::PixelFormat::HasCapabilities(TileDataFormat, EPixelFormatCapabilities::UAV));
	}

	FRDGTextureDesc Texture2DArrayDesc(FRDGTextureDesc::Create2DArray(TileDataTextureResolution, TileDataFormat, FClearValueBinding(EClearBinding::ENoneBound), TexCreate_ShaderResource | TexCreate_UAV | TexCreate_ReduceMemoryWithTilingMode | TexCreate_3DTiling, TileDataTextureSliceCount));
	// LFV TODO, consider FFastVramConfig onto Texture2DArrayDesc

	View.LocalFogVolumeViewData.TileDataTextureArray		= GraphBuilder.CreateTexture(Texture2DArrayDesc, TEXT("LocalFogVolume.CullingDataTexture"));
	View.LocalFogVolumeViewData.TileDataTextureArraySRV		= GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(View.LocalFogVolumeViewData.TileDataTextureArray));
	View.LocalFogVolumeViewData.TileDataTextureArrayUAV		= GraphBuilder.CreateUAV(FRDGTextureUAVDesc(View.LocalFogVolumeViewData.TileDataTextureArray));

	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTileDataTextureResolution	= FUintVector2(TileDataTextureResolution.X, TileDataTextureResolution.Y);
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeInstanceCount				= View.LocalFogVolumeViewData.GPUInstanceCount;
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTilePixelSize				= LocalFogVolumeTilePixelSize;
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeMaxDensityIntoVolumetricFog	= GetLocalFogVolumeMaxDensityIntoVolumetricFog();
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.ShouldRenderLocalFogVolumeInVolumetricFog	= bShouldRenderLocalFogVolumeInVolumetricFog ? 1 : 0;
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeInstances					= View.LocalFogVolumeViewData.GPUInstanceDataBufferSRV;
	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.GlobalStartDistance						= GetLocalFogVolumeGlobalStartDistance();
	const FLightSceneProxy* ApproximatePerPixelTransmittanceProxy = nullptr; // by default ignore
	if (IsMobilePlatform(View.GetShaderPlatform()))
	{
		// On mobile there is a separate FMobileDirectionalLightShaderParameters UB which holds all directional light data.
		// See SetupMobileDirectionalLightUniformParameters

		View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightColor					= FVector3f::Zero();
		View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightDirection				= FVector3f::Zero();
		for (uint32 ChannelIdx = 0; ChannelIdx < UE_ARRAY_COUNT(Scene->MobileDirectionalLights); ChannelIdx++)
		{
			FLightSceneInfo* Light = Scene->GetMobileDirectionalLightForView(ChannelIdx, View);
			if (Light != nullptr)
			{
				ApproximatePerPixelTransmittanceProxy = Light->Proxy;
				View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightColor			= FVector3f( Light->Proxy->GetSunIlluminanceAccountingForSkyAtmospherePerPixelTransmittance());
				View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightDirection		= FVector3f(-Light->Proxy->GetDirection());
				break;
			}
		}
	}
	else if (View.ForwardLightingResources.SelectedForwardDirectionalLightProxy)
	{
		const FLightSceneProxy* SelectedForwardDirectionalLightProxy = View.ForwardLightingResources.SelectedForwardDirectionalLightProxy;
		View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightColor					= FVector3f( SelectedForwardDirectionalLightProxy->GetSunIlluminanceAccountingForSkyAtmospherePerPixelTransmittance());
		View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightDirection				= FVector3f(-SelectedForwardDirectionalLightProxy->GetDirection());
		ApproximatePerPixelTransmittanceProxy = SelectedForwardDirectionalLightProxy;
	}
	else
	{
		View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightColor					= FVector3f(View.CachedViewUniformShaderParameters->DirectionalLightColor);
		View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightDirection				= View.CachedViewUniformShaderParameters->DirectionalLightDirection;
	}

	if (ApproximatePerPixelTransmittanceProxy && ApproximatePerPixelTransmittanceProxy->GetUsePerPixelAtmosphereTransmittance())
	{
		// When using PerPixelTransmittance, transmittance is evaluated per pixel by sampling the transmittance texture. It gives better gradient on large scale objects such as mountains.
		// However, to skip doing that texture sampling in Local Fog Splatting shader, we use the simple planet top ground transmittance as a simplification.
		// That will work for most of the cases for most of the map/terrain at the top of the virtual planet.
		View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.DirectionalLightColor *= FVector3f(ApproximatePerPixelTransmittanceProxy->GetAtmosphereTransmittanceTowardSun());
	}

	// This buffer must remain a basic vertex buffer for mobile to be able to read it from vertex shader
	View.LocalFogVolumeViewData.GPUTileDataBuffer = CreateVertexBuffer(
		GraphBuilder, TEXT("LocalFogVolume.GPUTileDataBuffer"),
		FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), TileDataTextureResolution.X * TileDataTextureResolution.Y * sizeof(uint32)), nullptr, 0);
	View.LocalFogVolumeViewData.GPUTileDataBufferSRV = GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.GPUTileDataBuffer, PF_R32_UINT);
	View.LocalFogVolumeViewData.GPUTileDataBufferUAV = GraphBuilder.CreateUAV(View.LocalFogVolumeViewData.GPUTileDataBuffer, PF_R32_UINT);

	View.LocalFogVolumeViewData.GPUTileDrawIndirectBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndirectParameters>(), TEXT("LocalFogVolume.DispatchIndirectBuffer"));
	View.LocalFogVolumeViewData.GPUTileDrawIndirectBufferUAV = GraphBuilder.CreateUAV(View.LocalFogVolumeViewData.GPUTileDrawIndirectBuffer, PF_R32_UINT);
	AddClearUAVPass(GraphBuilder, View.LocalFogVolumeViewData.GPUTileDrawIndirectBufferUAV, 0, GetLocalFogVolumeTileCullingUseAsync() ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute);

	View.LocalFogVolumeViewData.bUseHalfResLocalFogVolume = bUseHalfResLocalFogVolume;
	if (View.LocalFogVolumeViewData.bUseHalfResLocalFogVolume)
	{
		FIntRect ViewRectAtOrigin = View.ViewRect;
		ViewRectAtOrigin.Max -= ViewRectAtOrigin.Min;
		ViewRectAtOrigin.Min -= ViewRectAtOrigin.Min;
		FIntRect HalfResRect = GetDownscaledViewport(FScreenPassTextureViewport(ViewRectAtOrigin), FIntPoint(2, 2)).Rect;
		View.LocalFogVolumeViewData.HalfResResolution = HalfResRect.Max - HalfResRect.Min;

		const FIntPoint& HalfResSize = View.LocalFogVolumeViewData.HalfResResolution;
		View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.HalfResTextureSizeAndInvSize = FVector4f(HalfResSize.X, HalfResSize.Y, 1.0f/float(HalfResSize.X), 1.0f/float(HalfResSize.Y));

		FRDGTextureDesc Texture2DHalfResLFVDesc(FRDGTextureDesc::Create2D(View.LocalFogVolumeViewData.HalfResResolution, PF_FloatRGBA, FClearValueBinding(EClearBinding::ENoneBound), TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_ReduceMemoryWithTilingMode | TexCreate_NoFastClear));

		View.LocalFogVolumeViewData.HalfResLocalFogVolumeView = GraphBuilder.CreateTexture(Texture2DHalfResLFVDesc, TEXT("LocalFogVolume.HalfResLocalFogVolumeView"));
		View.LocalFogVolumeViewData.HalfResLocalFogVolumeViewSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(View.LocalFogVolumeViewData.HalfResLocalFogVolumeView));

		Texture2DHalfResLFVDesc.Format = PF_R16F;
		View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepth = GraphBuilder.CreateTexture(Texture2DHalfResLFVDesc, TEXT("LocalFogVolume.HalfResLocalFogVolumeDepth"));
		View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepthSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepth));
	}
	else
	{
		View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.HalfResTextureSizeAndInvSize = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
		View.LocalFogVolumeViewData.HalfResLocalFogVolumeView = GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder);
		View.LocalFogVolumeViewData.HalfResLocalFogVolumeViewSRV = GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.HalfResLocalFogVolumeView);
		View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepth = GSystemTextures.GetBlackDummy(GraphBuilder);
		View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepthSRV = GraphBuilder.CreateSRV(View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepth);
	}

	View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeTileDataTexture = View.LocalFogVolumeViewData.TileDataTextureArraySRV;
	View.LocalFogVolumeViewData.UniformBuffer = GraphBuilder.CreateUniformBuffer(&View.LocalFogVolumeViewData.UniformParametersStruct);
}

void InitLocalFogVolumesForViews(
	const FScene* Scene,
	TArray<FViewInfo>& Views,
	const FSceneViewFamily& SceneViewFamily,
	FRDGBuilder& GraphBuilder,
	bool bShouldRenderVolumetricFog,
	bool bUseHalfResLocalFogVolume)
{
	const uint32 LocalFogVolumeInstanceCount = Scene->LocalFogVolumes.Num();
	const bool bShouldRenderLocalFogVolume = ShouldRenderLocalFogVolume(Scene, SceneViewFamily);
	if (LocalFogVolumeInstanceCount > 0 && bShouldRenderLocalFogVolume)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, LocalFogVolumeVolumes, "LocalFogVolumeVolumes");

		for (FViewInfo& View : Views)
		{
			FLocalFogVolumeSortingData SortingData;
			GetLocalFogVolumeViewSortingData(Scene, View, GraphBuilder, SortingData);

			CreateViewLocalFogVolumeBufferSRV(Scene, View, GraphBuilder, SortingData, ShouldRenderLocalFogVolumeInVolumetricFog(Scene, SceneViewFamily, bShouldRenderVolumetricFog), bUseHalfResLocalFogVolume);

			if (View.LocalFogVolumeViewData.GPUInstanceCount > 0)
			{
				LocalFogVolumeViewTiledCullingPass(View, GraphBuilder);
			}
		}
	}
	else
	{
		SetDummyLocalFogVolumeForViews(GraphBuilder, Views);
	}
}

/*=============================================================================
	Local height fog rendering - non mobile
=============================================================================*/

class FLocalFogVolumeTiledRenderVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLocalFogVolumeTiledRenderVS);
	SHADER_USE_PARAMETER_STRUCT(FLocalFogVolumeTiledRenderVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileDataBuffer)
		SHADER_PARAMETER(float, StartDepthZ)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (IsMobilePlatform(Parameters.Platform))
		{
			return false;
		}
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("LFV_TILED_VS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FLocalFogVolumeTiledRenderVS, "/Engine/Private/LocalFogVolumes/LocalFogVolumeSplat.usf", "LocalFogVolumeTiledVS", SF_Vertex);

class FLocalFogVolumeTiledRenderPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLocalFogVolumeTiledRenderPS);
	SHADER_USE_PARAMETER_STRUCT(FLocalFogVolumeTiledRenderPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
		SHADER_PARAMETER(int32, LocalFogVolumeTileDebug)
	END_SHADER_PARAMETER_STRUCT()

	class FVisualizationPass : SHADER_PERMUTATION_BOOL("LFV_VISUALIZATION_PS");
	using FPermutationDomain = TShaderPermutationDomain<FVisualizationPass>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (IsMobilePlatform(Parameters.Platform))
		{
			return false;
		}
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("LFV_TILED_PS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FLocalFogVolumeTiledRenderPS, "/Engine/Private/LocalFogVolumes/LocalFogVolumeSplat.usf", "LocalFogVolumeTiledPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FLocalFogVolumeTiledPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FLocalFogVolumeTiledRenderVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FLocalFogVolumeTiledRenderPS::FParameters, PS)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	RDG_BUFFER_ACCESS(TileDrawIndirectBuffer, ERHIAccess::IndirectArgs)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

struct FDepthBoundSetup
{
	bool bEnabled = false;
	float FogClipDeviceZ = 0.0f;
	float MinDeviceZ = 0.0f;
	float MaxDeviceZ = 1.0f;
};

static FDepthBoundSetup GetDepthBoundSetup(float FogStartDistance, FMatrix ViewProjectionMatrix, FMatrix ViewInvProjectionMatrix)
{
	FDepthBoundSetup DepthBoundSetup;

	// Here we compute the nearest z value the fog can start
	// to skip shader execution on pixels that are closer.
	// This means with a bigger distance specified more pixels are
	// are culled and don't need to be rendered. This is faster if
	// there is opaque content nearer than the computed z.
	// This optimization is achieved using depth bound tests.
	// Mobile platforms typically does not support that feature 
	// but typically renders the world using forward shading 
	// with height fog evaluated as part of the material vertex or pixel shader.
	FVector ViewSpaceCorner = ViewInvProjectionMatrix.TransformFVector4(FVector4(1, 1, 1, 1));
	float Ratio = ViewSpaceCorner.Z / ViewSpaceCorner.Size();
	FVector ViewSpaceStartFogPoint(0.0f, 0.0f, FogStartDistance * Ratio);
	FVector4f ClipSpaceMaxDistance = (FVector4f)ViewProjectionMatrix.TransformPosition(ViewSpaceStartFogPoint); // LWC_TODO: precision loss
	float FogClipSpaceZ = ClipSpaceMaxDistance.Z / ClipSpaceMaxDistance.W;
	DepthBoundSetup.FogClipDeviceZ = FMath::Clamp(FogClipSpaceZ, 0.f, 1.f);

	DepthBoundSetup.bEnabled = GSupportsDepthBoundsTest;
	DepthBoundSetup.MinDeviceZ = 0.0f;
	DepthBoundSetup.MaxDeviceZ = DepthBoundSetup.FogClipDeviceZ;

	return DepthBoundSetup;
}

void RenderLocalFogVolumeForView(
	const FViewInfo& View,
	FRDGBuilder& GraphBuilder,
	int32 LocalFogVolumeTileDebugMode,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture,
	FRDGTextureRef LightShaftOcclusionTexture,
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures,
	bool bVisualizationPass)
{
	const bool bRenderLocalFogVolumes = View.LocalFogVolumeViewData.GPUInstanceCount == 0;
	if (bRenderLocalFogVolumes)
	{
		return;
	}

	RDG_EVENT_SCOPE_CONDITIONAL_STAT(GraphBuilder, bRenderLocalFogVolumes, LocalFogVolumeVolumes, "%s", bVisualizationPass ? TEXT("LocalFogVolume.Visualization") : TEXT("LocalFogVolume.Render"));

	// This distance is the start distance account for global and all local fog volume start distance from the camera.
	const float FinalStartDistance = GetLocalFogVolumeViewStartDistance(View);

	const FIntRect ViewRect = View.ViewRect;
	const FMatrix ViewProjectionMatrix = View.ViewMatrices.GetViewToClip();
	const FMatrix ViewInvProjectionMatrix = View.ViewMatrices.GetClipToView();
	FDepthBoundSetup DepthBoundSetup = GetDepthBoundSetup(FinalStartDistance, ViewProjectionMatrix, ViewInvProjectionMatrix);

	FLocalFogVolumeTiledPassParameters* PassParameters = GraphBuilder.AllocParameters<FLocalFogVolumeTiledPassParameters>();

	PassParameters->VS.View = GetShaderBinding(View.ViewUniformBuffer);
	PassParameters->VS.LFV = View.LocalFogVolumeViewData.UniformParametersStruct;
	PassParameters->VS.TileDataBuffer = View.LocalFogVolumeViewData.GPUTileDataBufferSRV;
	PassParameters->VS.StartDepthZ = DepthBoundSetup.FogClipDeviceZ;

	PassParameters->PS.View = GetShaderBinding(View.ViewUniformBuffer);
	PassParameters->PS.LFV = View.LocalFogVolumeViewData.UniformParametersStruct;
	PassParameters->PS.LocalFogVolumeTileDebug = LocalFogVolumeTileDebugMode;

	PassParameters->SceneTextures = SceneTextures;
	PassParameters->TileDrawIndirectBuffer = View.LocalFogVolumeViewData.GPUTileDrawIndirectBuffer;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ENoAction);
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);

	FLocalFogVolumeTiledRenderVS::FPermutationDomain VSPermutationVector;
	auto VertexShader = View.ShaderMap->GetShader< FLocalFogVolumeTiledRenderVS >(VSPermutationVector);

	FLocalFogVolumeTiledRenderPS::FPermutationDomain PsPermutationVector;
	PsPermutationVector.Set<FLocalFogVolumeTiledRenderPS::FVisualizationPass>(bVisualizationPass);
	auto PixelShader = View.ShaderMap->GetShader< FLocalFogVolumeTiledRenderPS >(PsPermutationVector);

	ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

	FUintVector2& LocalFogVolumeTileDataTextureResolution = View.LocalFogVolumeViewData.UniformParametersStruct.LocalFogVolumeCommon.LocalFogVolumeTileDataTextureResolution;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("LocalFogVolume.Tiled (%u x %u)", LocalFogVolumeTileDataTextureResolution.X, LocalFogVolumeTileDataTextureResolution.Y),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, PassParameters, ViewRect, DepthBoundSetup](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

			// Render back faces only since camera may intersect
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList; // LFV_TODO check if rects are supported and use them if so

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			GraphicsPSOInit.bDepthBounds = DepthBoundSetup.bEnabled;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			RHICmdList.SetDepthBounds(DepthBoundSetup.MinDeviceZ, DepthBoundSetup.MaxDeviceZ);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

			RHICmdList.SetStreamSource(0, nullptr, 0);
			RHICmdList.DrawPrimitiveIndirect(PassParameters->TileDrawIndirectBuffer->GetIndirectRHICallBuffer(), 0);
		});

	if (bVisualizationPass)
	{
		const float LFVGlobalStartDistance = GetLocalFogVolumeGlobalStartDistance();
		const float LFVInstancesStartDistance = View.LocalFogVolumeViewData.DepthBoundStartDistance;
		// Now debug print
		AddDrawCanvasPass(GraphBuilder, {}, View, FScreenPassRenderTarget(SceneColorTexture, ERenderTargetLoadAction::ELoad),
			[&View, LFVGlobalStartDistance, LFVInstancesStartDistance, FinalStartDistance](FCanvas& Canvas)
			{
				FString Text;
				const float DPIScale = Canvas.GetDPIScale();
				Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale) * Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

				Canvas.DrawShadowedString(50.0, 50.0, TEXT("LOCAL FOG VOLUME DEBUG VIEW"), GEngine->GetLargeFont(), FLinearColor::White);

				Canvas.DrawShadowedString(50.0, 70.0, TEXT("Global Start Distance"), GEngine->GetLargeFont(), FLinearColor::White);
				Text = FString::Printf(TEXT("= %.1f"), LFVGlobalStartDistance);
				Canvas.DrawShadowedString(200.0, 70.0, *Text, GEngine->GetLargeFont(), FLinearColor::White);

				Canvas.DrawShadowedString(50.0, 85.0, TEXT("Instances Start Distance"), GEngine->GetLargeFont(), FLinearColor::White);
				Text = FString::Printf(TEXT("= %.1f"), LFVInstancesStartDistance);
				Canvas.DrawShadowedString(200.0, 85.0, *Text, GEngine->GetLargeFont(), FLinearColor::White);

				Canvas.DrawShadowedString(50.0, 100.0, TEXT("Final Start Distance"), GEngine->GetLargeFont(), FLinearColor::White);
				Text = FString::Printf(TEXT("= %.1f"), FinalStartDistance);
				Canvas.DrawShadowedString(200.0, 100.0, *Text, GEngine->GetLargeFont(), FLinearColor::White);
			});
	}
}
 
void RenderLocalFogVolume(
	const FScene* Scene,
	TArray<FViewInfo>& Views,
	const FSceneViewFamily& SceneViewFamily,
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef LightShaftOcclusionTexture,
	bool bVisualizationPass)
{
	uint32 LocalFogVolumeInstanceCount = Scene->LocalFogVolumes.Num();
	if (LocalFogVolumeInstanceCount > 0 && ShouldRenderLocalFogVolume(Scene, SceneViewFamily))
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, LocalFogVolumeVolumes, "%s", bVisualizationPass ? TEXT("LocalFogVolume.Visualization") : TEXT("LocalFogVolume.Render"));

		FRDGTextureRef SceneColorTexture = SceneTextures.Color.Resolve;
		FRDGTextureRef SceneDepthTexture = SceneTextures.Depth.Target;

		for (FViewInfo& View : Views)
		{
			if (View.LocalFogVolumeViewData.GPUInstanceCount == 0)
			{
				continue;
			}

			RenderLocalFogVolumeForView(
				View,
				GraphBuilder,
				GetLocalFogVolumeTileDebugMode(SceneViewFamily.EngineShowFlags),
				SceneColorTexture,
				SceneDepthTexture,
				LightShaftOcclusionTexture,
				SceneTextures.UniformBuffer,
				bVisualizationPass);
		}
	}
}
 
void RenderLocalFogVolumeVisualization(
	const FScene* Scene,
	TArray<FViewInfo>& Views,
	const FSceneViewFamily& SceneViewFamily,
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures)
{
#if !UE_BUILD_SHIPPING
	const bool bVisualizationPass = true;
	FRDGTexture* LightShaftOcclusionTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
	if (ShouldRenderLocalFogVolume(Scene, SceneViewFamily))
	{
		RenderLocalFogVolume(Scene, Views, SceneViewFamily, GraphBuilder, SceneTextures, LightShaftOcclusionTexture, bVisualizationPass);
	}
#endif
}


FScreenPassTexture RenderLocalFogVolumeVisualizationPostProcess(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTexture& ScreenPassSceneColor,
	const FScreenPassTexture& SceneDepth,
	const FScreenPassRenderTarget& OverrideOutput)
{
	FScreenPassRenderTarget OutputRT = OverrideOutput;

#if WITH_EDITOR

	if (!OutputRT.IsValid())
	{
		OutputRT = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, ScreenPassSceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeLightFunctionAtlas"));
	}

	if (ScreenPassSceneColor.Texture != OutputRT.GetRenderTargetBinding().GetTexture())
	{
		// Copy the scene color because below we do not write and copy all the view pixel. We only stamp the atlas and its tiles on top.
		AddDrawTexturePass(
			GraphBuilder,
			View,
			ScreenPassSceneColor.Texture,
			OutputRT.GetRenderTargetBinding().GetTexture(),
			FIntPoint::ZeroValue,
			OutputRT.GetRenderTargetBinding().GetTexture()->Desc.Extent,
			FIntPoint::ZeroValue,
			OutputRT.GetRenderTargetBinding().GetTexture()->Desc.Extent);
	}

	check(View.Family);
	FRDGTexture* LightShaftOcclusionTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
	const bool bVisualizationPass = true;
	RenderLocalFogVolumeForView(
		View,
		GraphBuilder,
		GetLocalFogVolumeTileDebugMode(View.Family->EngineShowFlags),
		OutputRT.Texture,
		SceneDepth.Texture,
		LightShaftOcclusionTexture,
		View.GetSceneTextures().UniformBuffer,
		bVisualizationPass);

#endif

	return OutputRT;
}

/*=============================================================================
	Local height fog rendering - mobile
=============================================================================*/

class FMobileLocalFogVolumeTiledRenderVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMobileLocalFogVolumeTiledRenderVS);
	SHADER_USE_PARAMETER_STRUCT(FMobileLocalFogVolumeTiledRenderVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileDataBuffer)
		RDG_BUFFER_ACCESS(TileDrawIndirectBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER(float, StartDepthZ)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsMobilePlatform(Parameters.Platform))
		{
			return false;
		}
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("LFV_TILED_VS"), 1);
		OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEPTHREAD_SUBPASS"), bMobileForceDepthRead ? 0u : 1u);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileLocalFogVolumeTiledRenderVS, "/Engine/Private/LocalFogVolumes/LocalFogVolumeSplat.usf", "LocalFogVolumeTiledVS", SF_Vertex);

class FMobileLocalFogVolumeTiledRenderPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMobileLocalFogVolumeTiledRenderPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileLocalFogVolumeTiledRenderPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileBasePassUniformParameters, MobileBasePass)
		SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
		SHADER_PARAMETER(int32, LocalFogVolumeTileDebug)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsMobilePlatform(Parameters.Platform))
		{
			return false;
		}
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("LFV_TILED_PS"), 1);
		OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEPTHREAD_SUBPASS"), bMobileForceDepthRead ? 0u : 1u);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileLocalFogVolumeTiledRenderPS, "/Engine/Private/LocalFogVolumes/LocalFogVolumeSplat.usf", "LocalFogVolumeTiledPS", SF_Pixel);

class FMobileLocalFogVolumeTiledRenderHalfResPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMobileLocalFogVolumeTiledRenderHalfResPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileLocalFogVolumeTiledRenderHalfResPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileBasePassUniformParameters, MobileBasePass)
		//SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
		SHADER_PARAMETER(int32, LocalFogVolumeTileDebug)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsMobilePlatform(Parameters.Platform))
		{
			return false;
		}
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("LFV_TILED_PS"), 1);
		OutEnvironment.SetDefine(TEXT("LFV_HALFRES_PS"), 1);
		OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEPTHREAD_SUBPASS"), bMobileForceDepthRead ? 0u : 1u);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileLocalFogVolumeTiledRenderHalfResPS, "/Engine/Private/LocalFogVolumes/LocalFogVolumeSplat.usf", "LocalFogVolumeTiledPS", SF_Pixel);

void RenderLocalFogVolumeMobile(
	FRHICommandList& RHICmdList,
	const FViewInfo& View)
{
	if (View.LocalFogVolumeViewData.GPUInstanceCount == 0)
	{
		return;
	}

	SCOPED_DRAW_EVENT(RHICmdList, LocalFogVolumeVolumes);

	FMobileLocalFogVolumeTiledRenderVS::FPermutationDomain VSPermutationVector;
	auto VertexShader = View.ShaderMap->GetShader< FMobileLocalFogVolumeTiledRenderVS >(VSPermutationVector);

	FMobileLocalFogVolumeTiledRenderPS::FPermutationDomain PsPermutationVector;
	auto PixelShader = View.ShaderMap->GetShader< FMobileLocalFogVolumeTiledRenderPS >(PsPermutationVector);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	const FIntRect ViewRect = View.ViewRect;
	RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

	// Render back faces only since camera may intersect
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha,
		CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
		CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
		CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
		CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
		CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
		CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
		CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();

	GraphicsPSOInit.PrimitiveType = PT_TriangleList; // LFV_TODO check if rects are supported and use them if so

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

	// This distance is the start distance account for global and all local fog volume start distance from the camera.
	const float FinalStartDistance = GetLocalFogVolumeViewStartDistance(View);

	FDepthBoundSetup DepthBoundSetup = GetDepthBoundSetup(FinalStartDistance, View.ViewMatrices.GetViewToClip(), View.ViewMatrices.GetClipToView());
	GraphicsPSOInit.bDepthBounds = DepthBoundSetup.bEnabled;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	RHICmdList.SetDepthBounds(DepthBoundSetup.MinDeviceZ, DepthBoundSetup.MaxDeviceZ);

	FMobileLocalFogVolumeTiledRenderVS::FParameters VSParameters;
	VSParameters.View = View.GetShaderParameters();
	VSParameters.LFV = View.LocalFogVolumeViewData.UniformParametersStruct;
	VSParameters.TileDataBuffer = View.LocalFogVolumeViewData.GPUTileDataBufferSRV;
	VSParameters.TileDrawIndirectBuffer = View.LocalFogVolumeViewData.GPUTileDrawIndirectBuffer;
	VSParameters.StartDepthZ = DepthBoundSetup.FogClipDeviceZ;
	SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParameters);

	FMobileLocalFogVolumeTiledRenderPS::FParameters PSParameters;
	PSParameters.View = View.GetShaderParameters();
	PSParameters.LFV = View.LocalFogVolumeViewData.UniformParametersStruct;
	PSParameters.LocalFogVolumeTileDebug = GetLocalFogVolumeTileDebugMode(View.Family->EngineShowFlags);
	// PSParameters.MobileBasePass filled up by the RDG pass parameters.
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PSParameters);

	RHICmdList.SetStreamSource(0, nullptr, 0);
	RHICmdList.DrawPrimitiveIndirect(View.LocalFogVolumeViewData.GPUTileDrawIndirectBuffer->GetIndirectRHICallBuffer(), 0);
}

void RenderLocalFogVolumeHalfResMobile(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View)
{
	if (View.LocalFogVolumeViewData.GPUInstanceCount == 0)
	{
		return;
	}
	FMobileLocalFogVolumeTiledRenderHalfResPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileLocalFogVolumeTiledRenderHalfResPS::FParameters>();

	// When rendering the half resolution LFV texture, we are right after the depth prepass so it should not be bound (the texture is going to be generated now).
	// That is why we notify that we are still in the prepass stage.
	const EMobileBasePass BasePass = EMobileBasePass::DepthPrePass;

	PassParameters->View = View.GetShaderParameters();
	PassParameters->LFV = View.LocalFogVolumeViewData.UniformParametersStruct;
	PassParameters->LocalFogVolumeTileDebug = GetLocalFogVolumeTileDebugMode(View.Family->EngineShowFlags);
	PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, BasePass, EMobileSceneTextureSetupMode::SceneDepth);
	PassParameters->RenderTargets[0] = FRenderTargetBinding(View.LocalFogVolumeViewData.HalfResLocalFogVolumeView, ERenderTargetLoadAction::ENoAction);
	PassParameters->RenderTargets[1] = FRenderTargetBinding(View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepth, ERenderTargetLoadAction::ENoAction);

	FMobileLocalFogVolumeTiledRenderHalfResPS::FPermutationDomain PsPermutationVector;
	auto PixelShader = View.ShaderMap->GetShader< FMobileLocalFogVolumeTiledRenderHalfResPS >(PsPermutationVector);
	ClearUnusedGraphResources(PixelShader, PassParameters);

	const FIntPoint HalfResolution = View.LocalFogVolumeViewData.HalfResResolution;
	FPixelShaderUtils::AddFullscreenPass<FMobileLocalFogVolumeTiledRenderHalfResPS>(
		GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("LocalFogVolume.HalfRes"), PixelShader, PassParameters,
		FIntRect(0, 0, HalfResolution.X, HalfResolution.Y));
}
