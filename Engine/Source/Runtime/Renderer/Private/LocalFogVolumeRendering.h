// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"
#include "RenderGraph.h"
#include "RenderResource.h"
#include "RenderGraphResources.h"
#include "SceneView.h"
#include "Containers/Array.h"

class FScene;
class FViewInfo;
struct FMinimalSceneTextures;



/*=============================================================================
	Local height fog rendering GPU data
=============================================================================*/

BEGIN_SHADER_PARAMETER_STRUCT(FLocalFogVolumeCommonParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, LocalFogVolumeInstances)
	SHADER_PARAMETER(FUintVector2, LocalFogVolumeTileDataTextureResolution)
	SHADER_PARAMETER(uint32, LocalFogVolumeInstanceCount)
	SHADER_PARAMETER(uint32, LocalFogVolumeTilePixelSize)
	SHADER_PARAMETER(float,  LocalFogVolumeMaxDensityIntoVolumetricFog)
	SHADER_PARAMETER(uint32, ShouldRenderLocalFogVolumeInVolumetricFog)
	SHADER_PARAMETER(float,  GlobalStartDistance)
	SHADER_PARAMETER(FVector4f, HalfResTextureSizeAndInvSize)
	SHADER_PARAMETER(FVector3f, DirectionalLightColor)
	SHADER_PARAMETER(FVector3f, DirectionalLightDirection)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FLocalFogVolumeCommonParameters, LocalFogVolumeCommon)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<uint>, LocalFogVolumeTileDataTexture)
END_SHADER_PARAMETER_STRUCT()

/*=============================================================================
	Local height fog rendering common data
=============================================================================*/

class FLocalFogVolumeGPUInstanceData
{
public:

	uint32 Data0[4];
	uint32 Data1[4];
	uint32 Data2[4];

	float GetUniformScale()
	{
		union { uint32 U; float F; } FU = { Data0[3] };
		return FU.F;
	}
};

class FLocalFogVolumeSortKey
{
public:
	union
	{
		uint64 PackedData;

		struct
		{
			uint64 Index : 16; // Index of the volume
			uint64 Distance : 32; // then by distance
			uint64 Priority : 16; // First order by priority
		} FogVolume;
	};

	FORCEINLINE bool operator!=(FLocalFogVolumeSortKey B) const
	{
		return PackedData != B.PackedData;
	}

	FORCEINLINE bool operator<(FLocalFogVolumeSortKey B) const
	{
		return PackedData > B.PackedData;
	}
};

// This can be generated once for a scene and then shared between all the views to generate the GPU buffer data needed for rendering while accounting for sorting.
struct FLocalFogVolumeSortingData
{
	uint32 LocalFogVolumeInstanceCount;			// Total count in the scene
	uint32 LocalFogVolumeInstanceCountFinal;	// Count after culling on CPU has happened from GetLocalFogVolumeViewSortingData.
	FLocalFogVolumeGPUInstanceData* LocalFogVolumeGPUInstanceData;
	FVector* LocalFogVolumeCenterPos;
	FVector2f* LocalFogVolumeStartDistanceAndUniformScale;
	TArray<FLocalFogVolumeSortKey>	LocalFogVolumeSortKeys;
};

struct FLocalFogVolumeViewData
{
	uint32 GPUInstanceCount = 0;
	float DepthBoundStartDistance = 0.0f;
	FRDGBufferRef GPUInstanceDataBuffer = nullptr;
	FRDGBufferSRVRef GPUInstanceDataBufferSRV = nullptr;

	FRDGBufferRef GPUInstanceCullingDataBuffer = nullptr;
	FRDGBufferSRVRef GPUInstanceCullingDataBufferSRV = nullptr;

	FRDGBufferRef	 GPUTileDataBuffer = nullptr;
	FRDGBufferSRVRef GPUTileDataBufferSRV = nullptr;
	FRDGBufferUAVRef GPUTileDataBufferUAV = nullptr;
	FRDGBufferRef    GPUTileDrawIndirectBuffer = nullptr;
	FRDGBufferUAVRef GPUTileDrawIndirectBufferUAV = nullptr;

	TRDGUniformBufferRef<FLocalFogVolumeUniformParameters> UniformBuffer = nullptr;

	FLocalFogVolumeUniformParameters UniformParametersStruct;

	FRDGTextureRef		TileDataTextureArray = nullptr;		// First slice is the instance count, later slices are instance indices.
	FRDGTextureSRVRef	TileDataTextureArraySRV = nullptr;
	FRDGTextureUAVRef	TileDataTextureArrayUAV = nullptr;

	bool				bUseHalfResLocalFogVolume = false;	// Only for the mobile path for now
	FIntPoint			HalfResResolution;
	FRDGTextureRef		HalfResLocalFogVolumeView = nullptr;
	FRDGTextureSRVRef	HalfResLocalFogVolumeViewSRV = nullptr;
	FRDGTextureRef		HalfResLocalFogVolumeDepth = nullptr;
	FRDGTextureSRVRef	HalfResLocalFogVolumeDepthSRV = nullptr;
};


/*=============================================================================
	Local height fog rendering functions
=============================================================================*/

bool ProjectSupportsLocalFogVolumes();
bool ShouldRenderLocalFogVolume(const FScene* Scene, const FSceneViewFamily& SceneViewFamily);
bool ShouldRenderLocalFogVolumeDuringHeightFogPass(const FScene* Scene, const FSceneViewFamily& SceneViewFamily);
bool ShouldRenderLocalFogVolumeInVolumetricFog(const FScene* Scene, const FSceneViewFamily& SceneViewFamily, bool bShouldRenderVolumetricFog);
bool ShouldRenderLocalFogVolumeVisualizationPass(const FScene* Scene, const FSceneViewFamily& SceneViewFamily);
float GetLocalFogVolumeGlobalStartDistance();
float GetLocalFogVolumeViewStartDistance(const FViewInfo& View);	// Account for global and all local fog volumes start distances.
bool IsLocalFogVolumeHalfResolution();

void GetLocalFogVolumeViewSortingData(const FScene* Scene, const FViewInfo& View, FRDGBuilder& GraphBuilder, FLocalFogVolumeSortingData& Out);

void SetDummyLocalFogVolumeForViews(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views);
void SetDummyLocalFogVolumeForViews(FRDGBuilder& GraphBuilder, TArray<FViewInfo*>& Views);
void SetDummyLocalFogVolumeForView(FRDGBuilder& GraphBuilder, FViewInfo& View);
void SetDummyLocalFogVolumeUniformParametersStruct(FRDGBuilder& GraphBuilder, FLocalFogVolumeUniformParameters& UniformParametersStruct);

void InitLocalFogVolumesForViews(
	const FScene* Scene,
	TArray<FViewInfo>& Views,
	const FSceneViewFamily& SceneViewFamily,
	FRDGBuilder& GraphBuilder,
	bool bShouldRenderVolumetricFog,
	bool bUseHalfResLocalFogVolume);

void RenderLocalFogVolume(
	const FScene* Scene,
	TArray<FViewInfo>& Views,
	const FSceneViewFamily& SceneViewFamily,
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef LightShaftOcclusionTexture,
	bool bVisualizationPass = false);

void RenderLocalFogVolumeMobile(
	FRHICommandList& RHICmdList, 
	const FViewInfo& View);
void RenderLocalFogVolumeHalfResMobile(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View);

void RenderLocalFogVolumeVisualization(
	const FScene* Scene,
	TArray<FViewInfo>& Views,
	const FSceneViewFamily& SceneViewFamily,
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures);

struct FScreenPassTexture;
struct FScreenPassRenderTarget;

FScreenPassTexture RenderLocalFogVolumeVisualizationPostProcess(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTexture& ScreenPassSceneColor,
	const FScreenPassTexture& SceneDepth,
	const FScreenPassRenderTarget& OverrideOutput);

