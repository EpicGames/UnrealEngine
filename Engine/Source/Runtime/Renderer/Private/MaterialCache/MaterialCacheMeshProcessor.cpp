// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheMeshProcessor.h"
#include "LogRenderer.h"
#include "MeshPassUtils.h"
#include "NaniteSceneProxy.h"
#include "NaniteVertexFactory.h"
#include "ScenePrivate.h"
#include "MaterialCache/MaterialCacheShaders.h"
#include "Materials/MaterialRenderProxy.h"
#include "Nanite/NaniteShading.h"
#include "Nanite/NaniteShared.h"
#include "MeshPassProcessor.inl"
#include "MaterialCache/MaterialCacheStackProvider.h"
#include "MaterialCache/MaterialCacheVirtualTextureRenderProxy.h"
#include "Materials/Material.h"

extern bool GMaterialCacheStaticMeshEnableViewportFromVS;

static uint32 GetMaterialCacheTagShaderIndex(const FMaterial* Material, const FGuid& Guid)
{
	FMaterialShaderMap* ShaderMap = Material->GetRenderingThreadShaderMap();
	if (!ensureMsgf(ShaderMap, TEXT("Material without rendering thread shader map")))
	{
		return UINT32_MAX;
	}
	
	TConstArrayView<FMaterialCacheTagStack> Stacks = ShaderMap->GetUniformExpressionSet().GetMaterialCacheTagStacks();

	// Linear search for the right stack index
	for (int32 i = 0; i < Stacks.Num(); i++)
	{
		if (Stacks[i].TagGuid == Guid)
		{
			return i;
		}
	}

	// No valid stack for this material
	// This is not an error, may be rendering a mesh with multiple sections, only some relevant
	return UINT32_MAX;
}

template<bool bSupportsViewportFromVS>
static bool GetMaterialCacheShaders(
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	const FGuid& TagGuid,
	TShaderRef<FMaterialCacheUnwrapVSBase>& VertexShader,
	TShaderRef<FMaterialCacheUnwrapPS>& PixelShader)
{
	// Find the shader index
	uint32 ShaderTagIndex = GetMaterialCacheTagShaderIndex(&Material, TagGuid);
	if (ShaderTagIndex == UINT32_MAX)
	{
		return false;
	}
	
	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<FMaterialCacheUnwrapVS<bSupportsViewportFromVS>>(ShaderTagIndex);
	ShaderTypes.AddShaderType<FMaterialCacheUnwrapPS>(ShaderTagIndex);

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

template<typename T>
static bool LoadShadingMaterial(
	ERHIFeatureLevel::Type FeatureLevel,
	const FMaterialRenderProxy* MaterialProxy,
	const FVertexFactoryType* NaniteVertexFactoryType,
	uint32 ShaderTagIndex,
	TShaderRef<T>& ComputeShader)
{
	const FMaterial& ShadingMaterial = MaterialProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
	check(Nanite::IsSupportedMaterialDomain(ShadingMaterial.GetMaterialDomain()));
	check(Nanite::IsSupportedBlendMode(ShadingMaterial));

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<T>(ShaderTagIndex);

	FMaterialShaders Shaders;
	if (!ShadingMaterial.TryGetShaders(ShaderTypes, NaniteVertexFactoryType, Shaders))
	{
		return false;
	}

	return Shaders.TryGetComputeShader(ComputeShader);
}

template<typename T>
bool CreateMaterialCacheComputeLayerShadingCommand(
	const FScene& Scene,
	const FPrimitiveSceneProxy* SceneProxy,
	const FMaterialRenderProxy* Material,
	bool bAllowDefaultFallback,
	const FGuid& TagGuid,
	FRHICommandListBase& RHICmdList,
	FMaterialCacheLayerShadingCSCommand& OutShadingCommand)
{
	const ERHIFeatureLevel::Type FeatureLevel = Scene.GetFeatureLevel();

	const FNaniteVertexFactory* NaniteVertexFactory     = Nanite::GVertexFactoryResource.GetVertexFactory();
	const FVertexFactoryType*   NaniteVertexFactoryType = NaniteVertexFactory->GetType();

	// Get first available material
	const FMaterialRenderProxy* MaterialProxy = Material;
	const FMaterial* BaseMaterial = nullptr;
	while (MaterialProxy)
	{
		BaseMaterial = MaterialProxy->GetMaterialNoFallback(FeatureLevel);
		if (BaseMaterial)
		{
			break;
		}
		
		MaterialProxy = MaterialProxy->GetFallback(FeatureLevel);
	}

	if (!MaterialProxy)
	{
		UE_LOGF(LogRenderer, Error, "Failed to get material cache fallback proxy");
		return false;
	}

	// Find the shader index
	int32 ShaderTagIndex = GetMaterialCacheTagShaderIndex(BaseMaterial, TagGuid);
	if (ShaderTagIndex == UINT32_MAX)
	{
		return false;
	}

	TShaderRef<T> ShadeCS;
	if (!LoadShadingMaterial(FeatureLevel, MaterialProxy, NaniteVertexFactoryType, ShaderTagIndex, ShadeCS))
	{
		if (!bAllowDefaultFallback)
		{
			return false;
		}
		
		MaterialProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
		
		if (!LoadShadingMaterial(FeatureLevel, MaterialProxy, NaniteVertexFactoryType, ShaderTagIndex, ShadeCS))
		{
			return false;
		}
	}

	OutShadingCommand.ComputeShader = ShadeCS;
	
	MaterialProxy->UpdateUniformExpressionCacheIfNeeded(RHICmdList, FeatureLevel);

	UE::MeshPassUtils::SetupComputeBindings(
		ShadeCS, &Scene, FeatureLevel, SceneProxy, 
		*MaterialProxy, *MaterialProxy->GetMaterialNoFallback(FeatureLevel),
		OutShadingCommand.ShaderBindings
	);

	return true;
}

bool LoadMaterialCacheNaniteShadingPipeline(
	const FScene& Scene,
	const Nanite::FSceneProxyBase* SceneProxy,
	const Nanite::FSceneProxyBase::FMaterialSection& Section,
	uint32 ShaderTagIndex,
	FNaniteShadingPipeline& ShadingPipeline)
{
	const ERHIFeatureLevel::Type FeatureLevel = Scene.GetFeatureLevel();

	const FNaniteVertexFactory* NaniteVertexFactory     = Nanite::GVertexFactoryResource.GetVertexFactory();
	const FVertexFactoryType*   NaniteVertexFactoryType = NaniteVertexFactory->GetType();

	// Get first available material
	const FMaterialRenderProxy* MaterialProxy = Section.ShadingMaterialProxy;
	while (MaterialProxy)
	{
		if (MaterialProxy->GetMaterialNoFallback(FeatureLevel))
		{
			break;
		}
		
		MaterialProxy = MaterialProxy->GetFallback(FeatureLevel);
	}

	if (!MaterialProxy)
	{
		UE_LOGF(LogRenderer, Error, "Failed to get material cache fallback proxy");
		return false;
	}

	TShaderRef<FMaterialCacheNaniteShadeCS> ShadeCS;
	
	if (!LoadShadingMaterial(FeatureLevel, MaterialProxy, NaniteVertexFactoryType, ShaderTagIndex, ShadeCS))
	{
		MaterialProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();

		if (!LoadShadingMaterial(FeatureLevel, MaterialProxy, NaniteVertexFactoryType, ShaderTagIndex, ShadeCS))
		{
			return false;
		}
	}
	
	ShadingPipeline.MaterialProxy     = MaterialProxy;
	ShadingPipeline.Material          = MaterialProxy->GetMaterialNoFallback(FeatureLevel);
	ShadingPipeline.BoundTargetMask   = 0x0;
	ShadingPipeline.ComputeShader     = ShadeCS.GetComputeShader();
	ShadingPipeline.bIsTwoSided       = Section.MaterialRelevance.bTwoSided;
	ShadingPipeline.bIsMasked         = Section.MaterialRelevance.bMasked;
	ShadingPipeline.bNoDerivativeOps  = Nanite::HasNoDerivativeOps(ShadingPipeline.ComputeShader);
	ShadingPipeline.MaterialBitFlags  = Nanite::PackMaterialBitFlags(*ShadingPipeline.Material, ShadingPipeline.BoundTargetMask, ShadingPipeline.bNoDerivativeOps, false, false);
	ShadingPipeline.MaterialCacheData = MakePimpl<FNaniteMaterialCacheData, EPimplPtrMode::DeepCopy>();
	ShadingPipeline.ShaderBindings    = MakePimpl<FMeshDrawShaderBindings, EPimplPtrMode::DeepCopy>();
	
	ShadingPipeline.MaterialCacheData->TypedShader = ShadeCS;

	UE::MeshPassUtils::SetupComputeBindings(
		ShadeCS, &Scene, FeatureLevel, SceneProxy, 
		*MaterialProxy, *ShadingPipeline.Material,
		*ShadingPipeline.ShaderBindings
	);

	ShadingPipeline.ShaderBindingsHash = ShadingPipeline.ShaderBindings->GetDynamicInstancingHash();
	return true;
}

bool FMaterialCacheMeshProcessor::TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial* Material)
{
	TMeshProcessorShaders<FMaterialCacheUnwrapVSBase, FMaterialCacheUnwrapPS> PassShaders;

	if (GRHISupportsArrayIndexFromAnyShader && GMaterialCacheStaticMeshEnableViewportFromVS)
	{
		if (!GetMaterialCacheShaders<true>(*Material, MeshBatch.VertexFactory->GetType(), TagGuid, PassShaders.VertexShader, PassShaders.PixelShader))
		{
			return false;
		}
	}
	else
	{
		if (!GetMaterialCacheShaders<false>(*Material, MeshBatch.VertexFactory->GetType(), TagGuid, PassShaders.VertexShader, PassShaders.PixelShader))
		{
			return false;
		}
	}

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey            SortKey          = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode                MeshFillMode     = ComputeMeshFillMode(*Material, OverrideSettings);
	
	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		*MaterialRenderProxy,
		*Material,
		PassDrawRenderState,
		PassShaders,
		MeshFillMode,
		CM_None,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData
	);

	return true;
}

#if WITH_EDITOR
static bool QueryOrSignalMaterialCacheShaderMap(EShaderPlatform ShaderPlatform, const FMaterialRenderProxy* Proxy)
{
	UMaterialInterface* Interface = Proxy->GetMaterialInterface();
	if (!Interface)
	{
		return false;
	}

	FMaterialResource* Resource = Interface->GetMaterialResource(ShaderPlatform);
	if (!Resource)
	{
		return false;
	}

	if (!Resource->IsRenderingThreadShaderMapComplete())
	{
		Resource->SubmitCompileJobs_RenderThread(EShaderCompileJobPriority::High);
		return false;
	}

	return true;
}

bool IsMaterialCacheMaterialReady(EShaderPlatform InShaderPlatform, const FPrimitiveSceneProxy* Proxy)
{
	// Validate each contained render proxy
	for (FMaterialCacheVirtualTextureRenderProxy* RenderProxy : Proxy->MaterialCacheRenderProxies)
	{
		if (!RenderProxy)
		{
			return false;
		}

		// If there's a stack provider, make sure that the associated resources are ready
		if (RenderProxy->StackProviderRenderProxy && !RenderProxy->StackProviderRenderProxy->IsMaterialResourcesReady())
		{
			return false;
		}
	
		if (Proxy->IsNaniteMesh())
		{
			const Nanite::FSceneProxy* NaniteProxy = static_cast<const Nanite::FSceneProxy*>(Proxy);
		
			for (const Nanite::FSceneProxyBase::FMaterialSection& MaterialSection : NaniteProxy->GetMaterialSections())
			{
				if (!QueryOrSignalMaterialCacheShaderMap(InShaderPlatform, MaterialSection.RasterMaterialProxy))
				{
					return false;
				}

				if (!QueryOrSignalMaterialCacheShaderMap(InShaderPlatform, MaterialSection.ShadingMaterialProxy))
				{
					return false;
				}
			}
		}
		else
		{
			const FPrimitiveSceneInfo* PrimitiveSceneInfo = Proxy->GetPrimitiveSceneInfo();
			if (!PrimitiveSceneInfo)
			{
				return false;
			}

			for (const FStaticMeshBatch& StaticMesh : PrimitiveSceneInfo->StaticMeshes)
			{
				if (!QueryOrSignalMaterialCacheShaderMap(InShaderPlatform, StaticMesh.MaterialRenderProxy))
				{
					return false;
				}
			}
		}
	}

	return true;
}
#endif // WITH_EDITOR

void FMaterialCacheMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (!MeshBatch.bUseForMaterial)
	{
		return;
	}
	
	const FMaterialRenderProxy* MaterialRenderProxy = OverrideLayerMaterialProxy ? OverrideLayerMaterialProxy : MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		if (const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel))
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

void FMaterialCacheMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	if (!PreCacheParams.bRenderInMainPass)
	{
		return;
	}

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	const ERasterizerFillMode                MeshFillMode     = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode                MeshCullMode     = ComputeMeshCullMode(Material, OverrideSettings);

	TMeshProcessorShaders<FMaterialCacheUnwrapVSBase, FMaterialCacheUnwrapPS> PassShaders;

	if (GRHISupportsArrayIndexFromAnyShader && GMaterialCacheStaticMeshEnableViewportFromVS)
	{
		if (!GetMaterialCacheShaders<true>(Material, VertexFactoryData.VertexFactoryType, TagGuid, PassShaders.VertexShader, PassShaders.PixelShader))
		{
			return;
		}
	}
	else
	{
		if (!GetMaterialCacheShaders<false>(Material, VertexFactoryData.VertexFactoryType, TagGuid, PassShaders.VertexShader, PassShaders.PixelShader))
		{
			return;
		}
	}

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;
	RenderTargetsInfo.RenderTargetsEnabled = 1;

	// First exported attribute
	// TODO[MP]: Support multiple physical layers
	RenderTargetsInfo.RenderTargetFormats[0] = PF_R8G8B8A8;
	RenderTargetsInfo.RenderTargetFlags[0]   = TexCreate_ShaderResource | TexCreate_RenderTargetable;

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		Material,
		PassDrawRenderState,
		RenderTargetsInfo,
		PassShaders,
		MeshFillMode,
		MeshCullMode,
		static_cast<EPrimitiveType>(PreCacheParams.PrimitiveType),
		EMeshPassFeatures::Default, 
		true,
		PSOInitializers
	);
}

FMeshDrawCommand& FMaterialCacheMeshPassContext::AddCommand(FMeshDrawCommand& Initializer, uint32 NumElements)
{
	
	return Initializer;
}

void FMaterialCacheMeshPassContext::FinalizeCommand(
	const FMeshBatch& MeshBatch, int32 BatchElementIndex, const FMeshDrawCommandPrimitiveIdInfo& IdInfo,
	ERasterizerFillMode MeshFillMode, ERasterizerCullMode MeshCullMode, FMeshDrawCommandSortKey SortKey, EFVisibleMeshDrawCommandFlags Flags,
	const FGraphicsMinimalPipelineStateInitializer& PipelineState, const FMeshProcessorShaders* ShadersForDebugging,
	FMeshDrawCommand& MeshDrawCommand)
{
	FGraphicsMinimalPipelineStateId PipelineId = FGraphicsMinimalPipelineStateId::GetPersistentId(PipelineState);

	MeshDrawCommand.SetDrawParametersAndFinalize(MeshBatch, BatchElementIndex, PipelineId, ShadersForDebugging);

	Command.Command = MeshDrawCommand;
	Command.CommandInfo = FCachedMeshDrawCommandInfo(EMeshPass::Num);
	Command.CommandInfo.SortKey = SortKey;
	Command.CommandInfo.CullingPayload = CreateCullingPayload(MeshBatch, MeshBatch.Elements[BatchElementIndex]);
	Command.CommandInfo.MeshFillMode = MeshFillMode;
	Command.CommandInfo.MeshCullMode = MeshCullMode;
	Command.CommandInfo.Flags = Flags;
}

FMaterialCacheMeshProcessor::FMaterialCacheMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FGuid& TagGuid, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext, const FMaterialRenderProxy* OverrideLayerMaterialProxy)
	: FMeshPassProcessor(EMeshPass::Num, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, OverrideLayerMaterialProxy(OverrideLayerMaterialProxy)
	, TagGuid(TagGuid)
	, PassDrawRenderState(InPassDrawRenderState)
{
	
}

bool CreateMaterialCacheStaticLayerDrawCommand(
	FScene& Scene,
	const FPrimitiveSceneProxy* Proxy,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FStaticMeshBatch& MeshBatch,
	const FGuid& TagGuid,
	FMaterialCacheMeshDrawCommand& OutMeshCommand)
{
	FMaterialCacheMeshPassContext Context;

	// TODO[MP]: Fixed function blending is a developmental thing
	FMeshPassProcessorRenderState PassState;
	PassState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Replace, SO_Replace, false, CF_Always, SO_Keep, SO_Keep, SO_Keep, 255, 1>::GetRHI());
	PassState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One>::GetRHI());

	// Process the command
	// TODO[MP]: Consider instantiating once somewhere
	FMaterialCacheMeshProcessor Processor(&Scene, Scene.GetFeatureLevel(), TagGuid, nullptr, PassState, &Context, MaterialRenderProxy);
	Processor.AddMeshBatch(MeshBatch, ~0ull, Proxy);
	OutMeshCommand = Context.Command;

	// May have failed for a number of reasons
	return Context.Command.Command.CachedPipelineId.IsValid();
}

/** Instantiate per shading command */
template bool CreateMaterialCacheComputeLayerShadingCommand<FMaterialCacheShadeCS>(const FScene& Scene, const FPrimitiveSceneProxy* SceneProxy, const FMaterialRenderProxy* Material, bool bAllowDefaultFallback, const FGuid& TagGuid, FRHICommandListBase& RHICmdList, FMaterialCacheLayerShadingCSCommand& Out);
template bool CreateMaterialCacheComputeLayerShadingCommand<FMaterialCacheNaniteShadeCS>(const FScene& Scene, const FPrimitiveSceneProxy* SceneProxy, const FMaterialRenderProxy* Material, bool bAllowDefaultFallback, const FGuid& TagGuid, FRHICommandListBase& RHICmdList, FMaterialCacheLayerShadingCSCommand& Out);
