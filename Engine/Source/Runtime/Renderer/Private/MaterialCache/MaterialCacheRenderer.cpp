// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheRenderer.h"
#include "DeferredShadingRenderer.h"
#include "MeshPassProcessor.h"
#include "PrimitiveSceneInfo.h"
#include "ScenePrivate.h"
#include "BasePassRendering.h"
#include "ComponentRecreateRenderStateContext.h"
#include "MaterialCachedData.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "Nanite/NaniteRayTracing.h"
#include "Nanite/NaniteShading.h"
#include "Nanite/NaniteShared.h"
#include "MaterialCache/MaterialCacheShaders.h"
#include "Rendering/NaniteStreamingManager.h"
#include "MaterialCacheDefinitions.h"
#include "LogRenderer.h"
#include "MaterialCache/MaterialCache.h"
#include "MaterialCache/MaterialCacheAttribute.h"
#include "MaterialCache/MaterialCacheMeshProcessor.h"
#include "MaterialCache/MaterialCachePrimitiveData.h"
#include "MaterialCache/MaterialCacheSceneExtension.h"
#include "MaterialCache/MaterialCacheStackProvider.h"
#include "MaterialCache/MaterialCacheVirtualTextureRenderProxy.h"
#include "MaterialCache/IMaterialCacheVirtualTextureAllocation.h"
#include "Materials/MaterialRenderProxy.h"

static void MaterialCacheInvalidateRenderStates(IConsoleVariable*)
{
	FGlobalComponentRecreateRenderStateContext{}; //-V607
}

bool GMaterialCacheStaticMeshEnableViewportFromVS = true;
static FAutoConsoleVariableRef CVarMaterialCacheStaticMeshEnableViewportFromVS(
	TEXT("r.MaterialCache.StaticMesh.EnableViewportFromVS"),
	GMaterialCacheStaticMeshEnableViewportFromVS,
	TEXT("Enable sliced rendering of static unwrapping on platforms that support render target array index from vertex shaders"),
	FConsoleVariableDelegate::CreateStatic(MaterialCacheInvalidateRenderStates),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

bool GMaterialCacheVertexInvariantEnable = true;
static FAutoConsoleVariableRef CVarMaterialCacheEnableVertexInvariant(
	TEXT("r.MaterialCache.VertexInvariant.Enable"),
	GMaterialCacheVertexInvariantEnable,
	TEXT("Enable compute-only shading of materials that only use UV-derived (or vertex-invariant) data"),
	FConsoleVariableDelegate::CreateStatic(MaterialCacheInvalidateRenderStates),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

bool GMaterialCacheVertexInvariantForce = false;
static FAutoConsoleVariableRef CVarMaterialCacheForceVertexInvariant(
	TEXT("r.MaterialCache.VertexInvariant.Force"),
	GMaterialCacheVertexInvariantForce,
	TEXT("Force compute-only shading of materials"),
	FConsoleVariableDelegate::CreateStatic(MaterialCacheInvalidateRenderStates),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

bool GMaterialCacheCommandCaching = false;
static FAutoConsoleVariableRef CVarMaterialCacheCommandCaching(
	TEXT("r.MaterialCache.CommandCaching"),
	GMaterialCacheCommandCaching,
	TEXT("Enable caching of mesh commands and layer shading commands"),
	FConsoleVariableDelegate::CreateStatic(MaterialCacheInvalidateRenderStates),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static_assert(MaterialCacheMaxRuntimeLayers == 8, "Max runtime layers out of sync with FMaterialCacheABufferParameters");

BEGIN_SHADER_PARAMETER_STRUCT(FMaterialCacheABufferParameters, )
	/** Array declarations not supported for this type, lay them out manually */
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, RWABuffer_0)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, RWABuffer_1)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, RWABuffer_2)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, RWABuffer_3)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, RWABuffer_4)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, RWABuffer_5)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, RWABuffer_6)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, RWABuffer_7)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>,   RWABuffer_R8Mask)
END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FMaterialCacheUniformParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT_INCLUDE(FMaterialCacheABufferParameters, ABuffer)
	SHADER_PARAMETER_STRUCT(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint4>, ShadingBinData)
	SHADER_PARAMETER(uint32, SvPagePositionModMask)
	SHADER_PARAMETER(FUintVector4, TileParams)
	SHADER_PARAMETER(FUintVector4, TileOrderingParams)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMaterialCacheRastShadeParameters, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMaterialCacheUniformParameters, Pass)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMaterialCacheNaniteShadeParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRasterUniformParameters, NaniteRaster)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteShadingUniformParameters, NaniteShading)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FOpaqueBasePassUniformParameters, BasePass)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMaterialCacheNaniteStackShadeParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PageIndirections)
	SHADER_PARAMETER_STRUCT_INCLUDE(FMaterialCacheNaniteShadeParameters, Shade)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMaterialCacheUniformParameters, Pass)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMaterialCacheCSStackShadeParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PageIndirections)
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FOpaqueBasePassUniformParameters, BasePass)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMaterialCacheUniformParameters, Pass)
END_SHADER_PARAMETER_STRUCT()

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FMaterialCacheUniformParameters, "MaterialCachePass", SceneTextures);

DECLARE_GPU_STAT(MaterialCacheCompositePages);
DECLARE_GPU_STAT(MaterialCacheFinalize);

enum class EMaterialCacheRenderPath
{
	/**
	 * Standard hardware rasterization unwrap path
	 * Batches to a single mesh command set per layer
	 */
	HardwareRaster,

	/**
	 * Nanite rasterization unwrap path
	 * All pages shader the same rasterization context / vis-buffer, a single stack shares the same page vis-region
	 * Shading is parallel per layer, batched by material then primitive
	 */
	NaniteRaster,

	/**
	 * Shade-only path, enabled when the material doesn't make use of non-uv derived vertex data
	 */
	VertexInvariant,
	
	Count
};

struct FMaterialCacheGenericCSPrimitiveBatch
{
	const FPrimitiveSceneProxy* Proxy = nullptr;

	TArray<uint32, SceneRenderingAllocator> Pages;

	/** Start into indirection table for pages */
	uint32_t PageIndirectionOffset = 0;

	/** Optional, shading bin for Nanite */
	uint32 ShadingBin = UINT32_MAX;

	/** The coordinate used for unwrapping */
	uint32 UVCoordinateIndex = UINT32_MAX;
	
	FMaterialCacheVirtualTextureAllocation* Allocation = nullptr;

	FMaterialCacheLayerShadingCSCommand* ShadingCommand = nullptr;
};

struct FMaterialCacheGenericCSMaterialBatch
{
	const FMaterialRenderProxy* Material = nullptr;

	TArray<FMaterialCacheGenericCSPrimitiveBatch, SceneRenderingAllocator> PrimitiveBatches;
};

struct FMaterialCacheGenericCSBatch
{
	FRDGBufferRef PageIndirectionBuffer;

	uint32 PageCount = 0;

	TArray<FMaterialCacheGenericCSMaterialBatch, SceneRenderingAllocator> MaterialBatches;
};

struct FMaterialCacheStaticMeshCommand
{
	/** Actual page index, not indirection table */
	uint32 PageIndex = UINT32_MAX;

	/** The coordinate used for unwrapping */
	uint32 UVCoordinateIndex = UINT32_MAX;
	
	FVector4f UnwrapMinAndInvSize;
};

struct FMaterialCacheHardwareLayerRenderData
{
	TArray<FMaterialCacheStaticMeshCommand, SceneRenderingAllocator> MeshCommands;
	
	FMeshCommandOneFrameArray VisibleMeshCommands;
	
	TArray<int32, SceneRenderingAllocator> PrimitiveIds;
};

struct FMaterialCacheNaniteLayerRenderData
{
	FMaterialCacheGenericCSBatch GenericCSBatch;
};

struct FMaterialCacheNaniteRenderData
{
	TArray<Nanite::FInstanceDraw, SceneRenderingAllocator> InstanceDraws;

	TArray<FNaniteShadingBin, SceneRenderingAllocator> ShadingBins;

	FNaniteShadingCommands ShadingCommands;
};

struct FMaterialCacheVertexInvariantLayerRenderData
{
	FMaterialCacheGenericCSBatch GenericCSBatch;
};

struct FMaterialCachePageInfo
{
	FMaterialCachePageEntry Page;

	uint32_t ABufferPageIndex = 0;

	uint32_t SetupEntryIndex = 0;
};

struct FMaterialCachePageCollection
{
	TArray<FMaterialCachePageInfo, SceneRenderingAllocator> Pages;
};

struct FMaterialCacheLayerRenderData
{
	FMaterialCacheHardwareLayerRenderData Hardware;

	FMaterialCacheNaniteLayerRenderData Nanite;

	FMaterialCacheVertexInvariantLayerRenderData VertexInvariant;
};

enum class EMaterialCacheABufferTileLayout
{
	Horizontal,
	Sliced
};

struct FMaterialCacheABuffer
{
	EMaterialCacheABufferTileLayout Layout;

	TArray<FMaterialCachePageEntry> Pages;

	TArray<FRDGTextureRef, TInlineAllocator<MaterialCacheMaxRuntimeLayers>> ABufferTextures;
	
	/** Only used for hardware pages */
	FRDGTextureRef ABufferR8Mask;
};

struct FMaterialCacheRenderData
{
	FMaterialCachePendingTagBucket* Bucket = nullptr;
	
	FMaterialCachePageCollection PageCollections[static_cast<uint32>(EMaterialCacheRenderPath::Count)];
	
	FMaterialCacheABuffer ABuffer;

	FMaterialCacheNaniteRenderData Nanite;

	TArray<FMaterialCacheLayerRenderData, SceneRenderingAllocator> Layers;
};

struct FMaterialCacheHardwareContext
{
	FMaterialCacheUniformParameters* PassUniformParameters = nullptr;
};

struct FMaterialCacheNaniteContext
{
	FMaterialCacheNaniteShadeParameters* PassShadeParameters = nullptr;
	FMaterialCacheUniformParameters* PassUniformParameters = nullptr;
};

struct FMaterialCacheVertexInvariantContext
{
	FMaterialCacheUniformParameters* PassUniformParameters = nullptr;
};

static FMaterialCacheVirtualTextureRenderProxy* GetMaterialCacheRenderProxy(const FPrimitiveSceneProxy* Proxy, const FGuid& Tag)
{
	// Find the first render proxy that represents the tag
	for (FMaterialCacheVirtualTextureRenderProxy* RenderProxy : Proxy->MaterialCacheRenderProxies)
	{
		if (RenderProxy->Allocation->Description.TagLayout.Guid == Tag)
		{
			return RenderProxy;
		}
	}

	// Shouldn't happen, any primitive that's been pushed through for rendering must have a valid proxy for a given tag
	return nullptr;
}

static EMaterialCacheRenderPath GetMaterialCacheRenderPath(FSceneRendererBase* Renderer, const FPrimitiveSceneProxy* Proxy, const FMaterialCacheVirtualTextureRenderProxy* CacheProxy, const FGuid& TagGuid, const FMaterialCacheStackEntry& StackEntry)
{
	if (GMaterialCacheVertexInvariantEnable)
	{
		bool bMaterialCacheHasNonPrimaryUVDerivedData = false;
		for (const FMaterialRenderProxy* SectionMaterial : StackEntry.SectionMaterials)
		{
			if (FMaterialResource* Resource = SectionMaterial->GetMaterialInterface()->GetMaterialResource(Renderer->Scene->GetShaderPlatform()))
			{
				const FMaterialCachedExpressionData& ExpressionData = Resource->GetCachedExpressionData();

				// If there's any non-uv derived data, we can't
				bMaterialCacheHasNonPrimaryUVDerivedData |= ExpressionData.bMaterialCacheHasNonUVDerivedExpression;

				// If the material reads a UV channel that's different from the primary one, it's no longer implicit
				if (ExpressionData.MaterialCacheUVCoordinatesUsedMask != (1ull << CacheProxy->UVCoordinateIndex))
				{
					bMaterialCacheHasNonPrimaryUVDerivedData = true;
				}
			}
		}
		
		// If the material doesn't make use of non-uv derived expressions, push it through the vertex invariant path
		if (GMaterialCacheVertexInvariantForce || !bMaterialCacheHasNonPrimaryUVDerivedData)
		{
			return EMaterialCacheRenderPath::VertexInvariant;
		}
	}

	// Otherwise, we need to rasterize, select the appropriate path
	if (Proxy->IsNaniteMesh())
	{
		return EMaterialCacheRenderPath::NaniteRaster;
	}
	else
	{
		return EMaterialCacheRenderPath::HardwareRaster;
	}
}

static FMaterialCacheGenericCSPrimitiveBatch& GetOrCreateCSPrimitiveBatch(
	const FMaterialCachePendingEntry& Entry,
	FMaterialCacheGenericCSMaterialBatch& MaterialBatch,
	const FPrimitiveSceneProxy* Proxy)
{
	// If shared shading is supported, have the CS batch shade over the full page
	// with the same geometric data.
	if (Entry.Setup.Allocation->Description.bSharedShading)
	{
		for (FMaterialCacheGenericCSPrimitiveBatch& PrimitiveBatch : MaterialBatch.PrimitiveBatches)
		{
			if (PrimitiveBatch.Allocation == Entry.Setup.Allocation)
			{
				return PrimitiveBatch;
			}
		}
	}
	
	for (FMaterialCacheGenericCSPrimitiveBatch& PrimitiveBatch : MaterialBatch.PrimitiveBatches)
	{
		if (PrimitiveBatch.Proxy == Proxy)
		{
			return PrimitiveBatch;
		}
	}

	FMaterialCacheGenericCSPrimitiveBatch& PrimitiveBatch = MaterialBatch.PrimitiveBatches.Emplace_GetRef();
	PrimitiveBatch.Proxy = Proxy;
	PrimitiveBatch.Allocation = Entry.Setup.Allocation;
	return PrimitiveBatch;
}

static FMaterialCacheGenericCSMaterialBatch& GetOrCreateCSMaterialBatch(FMaterialCacheGenericCSBatch& LayerBatch, const FMaterialRenderProxy* Material)
{
	for (FMaterialCacheGenericCSMaterialBatch& MaterialBatch : LayerBatch.MaterialBatches)
	{
		if (MaterialBatch.Material == Material)
		{
			return MaterialBatch;
		}
	}

	FMaterialCacheGenericCSMaterialBatch& MaterialBatch = LayerBatch.MaterialBatches.Emplace_GetRef();
	MaterialBatch.Material = Material;
	return MaterialBatch;
}

struct FMaterialCachePageAllocation
{
	uint32 PageIndex;

	bool bAllocated = false;
};

FMaterialCacheGenericCSPrimitiveBatch& MaterialCacheAllocateGenericCSShadePage(
	const FMaterialCachePendingEntry& Entry,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	FMaterialCacheGenericCSBatch& RenderData,
	FMaterialCachePageAllocation PageAllocation)
{
	FMaterialCacheGenericCSMaterialBatch& MaterialBatch = GetOrCreateCSMaterialBatch(RenderData, MaterialRenderProxy);
	FMaterialCacheGenericCSPrimitiveBatch& PrimitiveBatch = GetOrCreateCSPrimitiveBatch(Entry, MaterialBatch, PrimitiveSceneProxy);
	
	int32 Num = PrimitiveBatch.Pages.Num();
	
	if (PrimitiveBatch.Pages.AddUnique(PageAllocation.PageIndex) == Num)
	{
		RenderData.PageCount++;
	}
	
	return PrimitiveBatch;
}

static FMaterialCachePrimitiveCachedLayerCommands& GetCachedLayerCommands(FMaterialCachePrimitiveData* PrimitiveData, const FGuid& TagGuid, const FMaterialRenderProxy* RenderProxy)
{
	FMaterialCachePrimitiveCachedTagCommands&               TagCache   = PrimitiveData->CachedCommands.Tags.FindOrAdd(TagGuid);
	TUniquePtr<FMaterialCachePrimitiveCachedLayerCommands>& LayerCache = TagCache.Layers.FindOrAdd(RenderProxy->GetMaterialInterface());

	// Layer command cache is persistent (until scene proxy invalidation)
	if (!LayerCache)
	{
		LayerCache = MakeUnique<FMaterialCachePrimitiveCachedLayerCommands>();
	}

	return *LayerCache.Get();
}

void MaterialCacheAllocateNaniteRasterPage(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, const FGuid& TagGuid, const FMaterialCachePendingEntry& Entry, const FMaterialCachePendingPageEntry& Page, FMaterialCacheStackEntry StackEntry, const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FMaterialCacheVirtualTextureRenderProxy* CacheProxy, const FPrimitiveSceneInfo* PrimitiveSceneInfo, FMaterialCachePrimitiveData* PrimitiveData, FMaterialCacheNaniteRenderData& RenderData, FMaterialCacheNaniteLayerRenderData& LayerRenderData, FMaterialCachePageAllocation PageAllocation)
{
	const int32 NumInstances = PrimitiveSceneInfo->GetNumInstanceSceneDataEntries();

	// Create vis-buffer view for all instances
	for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; InstanceIndex++)
	{
		RenderData.InstanceDraws.Add(Nanite::FInstanceDraw{
			static_cast<uint32>(PrimitiveSceneInfo->GetInstanceSceneDataOffset()) + InstanceIndex,
			PageAllocation.PageIndex
		});
	}

	// Batch up per section
	for (int32 SectionIndex = 0; SectionIndex < StackEntry.SectionMaterials.Num(); SectionIndex++)
	{
		const FMaterialRenderProxy* SectionMaterial = StackEntry.SectionMaterials[SectionIndex];
		
		FMaterialCacheGenericCSPrimitiveBatch& Batch = MaterialCacheAllocateGenericCSShadePage(Entry, SectionMaterial, PrimitiveSceneProxy, LayerRenderData.GenericCSBatch, PageAllocation);

		if (!Batch.ShadingCommand)
		{
			FMaterialCachePrimitiveCachedLayerCommands& LayerCache = GetCachedLayerCommands(PrimitiveData, TagGuid, SectionMaterial);

			if (!LayerCache.NaniteLayerShadingCommand.IsSet())
			{
				CreateMaterialCacheComputeLayerShadingCommand<FMaterialCacheNaniteShadeCS>(
					*Renderer->Scene,
					PrimitiveSceneProxy,
					SectionMaterial,
					false,
					TagGuid,
					GraphBuilder.RHICmdList,
					LayerCache.NaniteLayerShadingCommand.Emplace()
				);
			}

			Batch.ShadingCommand    = LayerCache.NaniteLayerShadingCommand.GetPtrOrNull();
			Batch.UVCoordinateIndex = CacheProxy->UVCoordinateIndex;

			// Assign shading bin by section
			const TArray<FNaniteShadingBin>& ShadingBins = PrimitiveSceneInfo->NaniteShadingBins[static_cast<uint32>(ENaniteMeshPass::MaterialCache)];
			if (ShadingBins.IsValidIndex(SectionIndex))
			{
				Batch.ShadingBin = ShadingBins[SectionIndex].BinIndex;
			}
			else
			{
				Batch.ShadingBin = 0;
			}
		}
	}
}

void MaterialCacheAllocateVertexInvariantPage(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, const FGuid& TagGuid, const FMaterialCachePendingEntry& Entry, const FMaterialCachePendingPageEntry& Page, FMaterialCacheStackEntry StackEntry, const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FPrimitiveSceneInfo* PrimitiveSceneInfo, FMaterialCachePrimitiveData* PrimitiveData, FMaterialCacheVertexInvariantLayerRenderData& RenderData, FMaterialCachePageAllocation PageAllocation)
{
	for (const FMaterialRenderProxy* SectionMaterial : StackEntry.SectionMaterials)
	{
		FMaterialCacheGenericCSPrimitiveBatch& Batch = MaterialCacheAllocateGenericCSShadePage(Entry, SectionMaterial, PrimitiveSceneProxy, RenderData.GenericCSBatch, PageAllocation);

		if (!Batch.ShadingCommand)
		{
			FMaterialCachePrimitiveCachedLayerCommands& LayerCache = GetCachedLayerCommands(PrimitiveData, TagGuid, SectionMaterial);

			if (!LayerCache.VertexInvariantShadingCommand.IsSet())
			{
				CreateMaterialCacheComputeLayerShadingCommand<FMaterialCacheShadeCS>(
					*Renderer->Scene,
					PrimitiveSceneProxy,
					SectionMaterial,
					false,
					TagGuid,
					GraphBuilder.RHICmdList,
					LayerCache.VertexInvariantShadingCommand.Emplace()
				);
			}

			Batch.ShadingCommand = LayerCache.VertexInvariantShadingCommand.GetPtrOrNull();
		}
	}
}

static FVector4f GetPageUnwrapMinAndInvSize(const FMaterialCachePageEntry& Page)
{
	return FVector4f{
		Page.UVRect.Min.X,
		Page.UVRect.Min.Y,
		1.0f / (Page.UVRect.Max.X - Page.UVRect.Min.X),
		1.0f / (Page.UVRect.Max.Y - Page.UVRect.Min.Y)
	};
}

void MaterialCacheAllocateHardwareRasterPage(FSceneRendererBase* Renderer, const FGuid& TagGuid, const FMaterialCachePendingEntry& Entry, const FMaterialCachePendingPageEntry& Page, FMaterialCacheStackEntry StackEntry, const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FMaterialCacheVirtualTextureRenderProxy* CacheProxy, const FPrimitiveSceneInfo* PrimitiveSceneInfo, FMaterialCachePrimitiveData* PrimitiveData, FMaterialCacheHardwareLayerRenderData& RenderData, FMaterialCachePageAllocation PageAllocation)
{
	for (const FMaterialRenderProxy* SectionMaterial : StackEntry.SectionMaterials)
	{
		FMaterialCachePrimitiveCachedLayerCommands& LayerCache = GetCachedLayerCommands(PrimitiveData, TagGuid, SectionMaterial);

		if (LayerCache.StaticMeshBatchCommands.IsEmpty())
		{
			for (int32 i = 0; i < PrimitiveSceneInfo->StaticMeshes.Num(); i++)
			{
				FMaterialCacheMeshDrawCommand Command;
		
				bool bResult = CreateMaterialCacheStaticLayerDrawCommand(
					*Renderer->Scene,
					PrimitiveSceneProxy,
					SectionMaterial,
					PrimitiveSceneInfo->StaticMeshes[i],
					TagGuid,
					Command
				);

				if (bResult)
				{
					LayerCache.StaticMeshBatchCommands.Emplace(MoveTemp(Command));
				}
			}
		}

		for (const FMaterialCacheMeshDrawCommand& MeshDrawCommand : LayerCache.StaticMeshBatchCommands)
		{
			FVisibleMeshDrawCommand Command;
			Command.Setup(
				&MeshDrawCommand.Command,
				PrimitiveSceneInfo->GetMDCIdInfo(),
				-1,
				MeshDrawCommand.CommandInfo.MeshFillMode,
				MeshDrawCommand.CommandInfo.MeshCullMode,
				MeshDrawCommand.CommandInfo.Flags,
				MeshDrawCommand.CommandInfo.SortKey,
				MeshDrawCommand.CommandInfo.CullingPayload,
				EMeshDrawCommandCullingPayloadFlags::NoScreenSizeCull,
				nullptr,
				0
			);

			FMaterialCacheStaticMeshCommand Cmd;
			Cmd.UnwrapMinAndInvSize = GetPageUnwrapMinAndInvSize(Page.Page);
			Cmd.PageIndex           = PageAllocation.PageIndex;
			Cmd.UVCoordinateIndex   = CacheProxy->UVCoordinateIndex;

			RenderData.MeshCommands.Add(Cmd);
			RenderData.VisibleMeshCommands.Add(Command);
			RenderData.PrimitiveIds.Add(PrimitiveSceneInfo->GetIndex());
		}
	}
}

static uint32_t AllocateMaterialCacheABufferPage(FMaterialCacheRenderData& RenderData, const FMaterialCachePageEntry& Page)
{
	RenderData.ABuffer.Pages.Add(Page);
	return RenderData.ABuffer.Pages.Num() - 1;
}

static FMaterialCachePageAllocation AllocateMaterialCacheRenderPathPage(FMaterialCacheRenderData& RenderData, FMaterialCachePendingPageEntry& Page, uint32_t EntryIndex, EMaterialCacheRenderPath RenderPath, uint32& PageAllocationSet)
{
	FMaterialCachePageCollection& Collection = RenderData.PageCollections[static_cast<uint32>(RenderPath)];

	uint32 RenderPathMask = 1u << static_cast<uint32>(RenderPath);

	FMaterialCachePageAllocation Allocation;

	if (!(PageAllocationSet & RenderPathMask))
	{
		FMaterialCachePageInfo Info;
		Info.Page = Page.Page;
		Info.ABufferPageIndex = Page.ABufferPageIndex;
		Info.SetupEntryIndex = EntryIndex;
		Collection.Pages.Add(Info);

		Allocation.bAllocated = true;

		PageAllocationSet |= RenderPathMask;
	}

	check(!Collection.Pages.IsEmpty());
	Allocation.PageIndex = Collection.Pages.Num() - 1;

	return Allocation;
}

void CreatePageIndirectionBuffer(FRDGBuilder& GraphBuilder, FMaterialCacheGenericCSBatch& Batch)
{
	FRDGUploadData<uint32> PageIndirectionsData(GraphBuilder, Batch.PageCount);

	uint32 IndirectionOffset = 0;

	for (FMaterialCacheGenericCSMaterialBatch& MaterialBatch : Batch.MaterialBatches)
	{
		for (FMaterialCacheGenericCSPrimitiveBatch& PrimitiveBatch : MaterialBatch.PrimitiveBatches)
		{
			PrimitiveBatch.PageIndirectionOffset = IndirectionOffset;
			FMemory::Memcpy(&PageIndirectionsData[IndirectionOffset], PrimitiveBatch.Pages.GetData(), PrimitiveBatch.Pages.NumBytes());
			IndirectionOffset += PrimitiveBatch.Pages.Num();
		}
	}

	check(IndirectionOffset == Batch.PageCount);

	Batch.PageIndirectionBuffer = CreateUploadBuffer(
		GraphBuilder,
		TEXT("MaterialCache.PageIndirection"),
		sizeof(uint32_t), PageIndirectionsData.Num(),
		PageIndirectionsData
	);
}

static void GetMaterialCacheDefaultMaterials(const FPrimitiveSceneProxy* Proxy, const FPrimitiveSceneInfo* SceneInfo, FMaterialCacheStackEntry& StackEntry)
{
	if (Proxy->IsNaniteMesh())
	{
		const Nanite::FSceneProxy* NaniteProxy = static_cast<const Nanite::FSceneProxy*>(Proxy);

		StackEntry.SectionMaterials.Reserve(NaniteProxy->GetMaterialSections().Num());
		
		for (const Nanite::FSceneProxyBase::FMaterialSection& MaterialSection : NaniteProxy->GetMaterialSections())
		{
			StackEntry.SectionMaterials.Add(MaterialSection.ShadingMaterialProxy);
		}
	}
	else
	{
		StackEntry.SectionMaterials.Reserve(SceneInfo->StaticMeshes.Num());

		for (const FStaticMeshBatch& Mesh : SceneInfo->StaticMeshes)
		{
			StackEntry.SectionMaterials.Add(Mesh.MaterialRenderProxy);
		}
	}
}

static uint32 MaterialCacheAllocateAndBatchPages(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheSceneExtension& SceneExtension, FMaterialCacheRenderData& RenderData, int32 BatchStart)
{
	int32 ProducedPages = 0;
	
	// Each batch is limited by the layers for hardware pages, and nanite views for nanite pages
	const int32 MaxProducedPages = FMath::Min(GRHIGlobals.MaxTextureArrayLayers, NANITE_MAX_VIEWS_PER_CULL_RASTERIZE_PASS);
	
	for (int32 EntryIndex = BatchStart; EntryIndex < RenderData.Bucket->PendingEntries.Num(); EntryIndex++)
	{
		FMaterialCachePendingEntry& Entry = RenderData.Bucket->PendingEntries[EntryIndex];

		FMaterialCacheAllocationSceneData* AllocationSceneData = SceneExtension.GetAllocationSceneData(Entry.Setup.Allocation);
		if (!AllocationSceneData)
		{
			UE_LOGF(LogRenderer, Error, "Failed to get allocation scene data");
			continue;
		}
		
		// Shouldn't happen
		if (Entry.Pages.Num() > MaxProducedPages)
		{
			checkf(false, TEXT("Entry page count exceeded internal limit"));
			continue;
		}
		
		// Out of pages? Resume at the current index
		if (ProducedPages + Entry.Pages.Num() > MaxProducedPages)
		{
			return EntryIndex;
		}
		
		ProducedPages += Entry.Pages.Num();

		for (FMaterialCachePendingPageEntry& Page : Entry.Pages)
		{
			Page.ABufferPageIndex = AllocateMaterialCacheABufferPage(RenderData, Page.Page);

			uint32 PageAllocationSet = 0x0;

			// Process all associated primitives
			for (FPrimitiveComponentId PrimitiveId : AllocationSceneData->Primitives)
			{
				// Get the render-thread safe primitive data
				FMaterialCachePrimitiveData* PrimitiveData = SceneExtension.GetPrimitiveData(PrimitiveId);
				if (!PrimitiveData)
				{
					UE_LOGF(LogRenderer, Error, "Failed to get primitive data");
					continue;
				}

				// Must have a scene info
				const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveData->Proxy->GetPrimitiveSceneInfo();
				if (!PrimitiveSceneInfo)
				{
					UE_LOGF(LogRenderer, Error, "Failed to get primitive scene info");
					continue;
				}

				// Try to find the render proxy for the tag
				FMaterialCacheVirtualTextureRenderProxy* RenderProxy = GetMaterialCacheRenderProxy(PrimitiveData->Proxy, RenderData.Bucket->TagLayout.Guid);
				if (!RenderProxy)
				{
					continue;
				}
				
				// Check if the UV ranges overlap
				if (!Page.Page.UVRect.Intersect(RenderProxy->UVRegion))
				{
					continue;
				}

				// Providers are optional, if none is supplied, just assume the primary material as a stack entry
				FMaterialCacheStack Stack;
				if (RenderProxy->StackProviderRenderProxy)
				{
					RenderProxy->StackProviderRenderProxy->Evaluate(&Stack);
				}
				else
				{
					FMaterialCacheStackEntry& StackEntry = Stack.Stack.Emplace_GetRef();
					GetMaterialCacheDefaultMaterials(PrimitiveData->Proxy, PrimitiveSceneInfo, StackEntry);
				}

				// Do not produce pages for empty stacks
				if (Stack.Stack.IsEmpty())
				{
					continue;
				}
				
				if (Stack.Stack.Num() > RenderData.Layers.Num())
				{
					RenderData.Layers.SetNum(Stack.Stack.Num());
				}

				for (int32 StackIndex = 0; StackIndex < Stack.Stack.Num(); StackIndex++)
				{
					const FMaterialCacheStackEntry& StackEntry = Stack.Stack[StackIndex];

					if (!StackEntry.SectionMaterials.Num())
					{
						UE_LOGF(LogRenderer, Error, "Invalid stack entry");
						continue;
					}

					FMaterialCacheLayerRenderData& Layer = RenderData.Layers[StackIndex];

					EMaterialCacheRenderPath RenderPath = GetMaterialCacheRenderPath(Renderer, PrimitiveData->Proxy, RenderProxy, RenderData.Bucket->TagLayout.Guid, StackEntry);

					const FMaterialCachePageAllocation RenderPathPageIndex = AllocateMaterialCacheRenderPathPage(RenderData, Page, EntryIndex, RenderPath, PageAllocationSet);
					
					switch (RenderPath)
					{
					default:
						checkNoEntry();
						break;
					case EMaterialCacheRenderPath::HardwareRaster:
						MaterialCacheAllocateHardwareRasterPage(Renderer, RenderData.Bucket->TagLayout.Guid, Entry, Page, StackEntry, PrimitiveData->Proxy, RenderProxy, PrimitiveSceneInfo, PrimitiveData, Layer.Hardware, RenderPathPageIndex);
						break;
					case EMaterialCacheRenderPath::NaniteRaster:
						MaterialCacheAllocateNaniteRasterPage(Renderer, GraphBuilder, RenderData.Bucket->TagLayout.Guid, Entry, Page, StackEntry, PrimitiveData->Proxy, RenderProxy, PrimitiveSceneInfo, PrimitiveData, RenderData.Nanite, Layer.Nanite, RenderPathPageIndex);
						break;
					case EMaterialCacheRenderPath::VertexInvariant:
						MaterialCacheAllocateVertexInvariantPage(Renderer, GraphBuilder, RenderData.Bucket->TagLayout.Guid, Entry, Page, StackEntry, PrimitiveData->Proxy, PrimitiveSceneInfo, PrimitiveData, Layer.VertexInvariant, RenderPathPageIndex);
						break;
					}
				}
			}
		}
	}

	for (FMaterialCacheLayerRenderData& LayerRenderData : RenderData.Layers)
	{
		CreatePageIndirectionBuffer(GraphBuilder, LayerRenderData.Nanite.GenericCSBatch);
		CreatePageIndirectionBuffer(GraphBuilder, LayerRenderData.VertexInvariant.GenericCSBatch);
	}
	
	// Last batch completed
	return RenderData.Bucket->PendingEntries.Num();
}

static FIntPoint GetMaterialCacheTileSize()
{
	static uint32 Width = GetMaterialCacheTileWidth();
	return FIntPoint(Width, Width);
}

static FIntPoint GetMaterialCacheRenderTileSize()
{
	static uint32 Width = GetMaterialCacheTileWidth() + GetMaterialCacheTileBorderWidth() * 2;
	return FIntPoint(Width, Width);
}

static void MaterialCacheCreateABuffer(FRDGBuilder& GraphBuilder, FMaterialCacheRenderData& RenderData)
{
	FRDGTextureDesc Desc;

	// Shared flags
	ETextureCreateFlags CommonFlags =
		ETextureCreateFlags::ShaderResource |
		ETextureCreateFlags::UAV |
		ETextureCreateFlags::RenderTargetable;

	// Setup the generic ABuffer description
	if (GRHISupportsArrayIndexFromAnyShader && GMaterialCacheStaticMeshEnableViewportFromVS)
	{
		Desc = FRDGTextureDesc::Create2DArray(
			GetMaterialCacheRenderTileSize(),
			PF_Unknown,
			FClearValueBinding::Black,
			CommonFlags | ETextureCreateFlags::TargetArraySlicesIndependently,
			RenderData.ABuffer.Pages.Num()
		);

		// Sliced layout (PageX, PageY, PageIndex)
		RenderData.ABuffer.Layout = EMaterialCacheABufferTileLayout::Sliced;
	}
	else
	{
		// TODO[MP]: This needs to be atlassed instead, we do have size limitations...
		Desc = FRDGTextureDesc::Create2DArray(
			GetMaterialCacheRenderTileSize() * FIntPoint(RenderData.ABuffer.Pages.Num(), 1),
			PF_Unknown,
			FClearValueBinding::Black,
			CommonFlags,
			1
		);

		// Horizontal layout (PageX + Stride * PageIndex, PageY)
		RenderData.ABuffer.Layout = EMaterialCacheABufferTileLayout::Horizontal;
	}

	// Names, must have static lifetimes
	static const TCHAR* ABufferNames[MaterialCacheMaxRuntimeLayers] = {
		TEXT("MaterialCache::ABuffer0"),
		TEXT("MaterialCache::ABuffer1"),
		TEXT("MaterialCache::ABuffer2"),
		TEXT("MaterialCache::ABuffer3"),
		TEXT("MaterialCache::ABuffer4"),
		TEXT("MaterialCache::ABuffer5"),
		TEXT("MaterialCache::ABuffer6"),
		TEXT("MaterialCache::ABuffer7")
	};

	// Create all ABuffers
	for (int32 ABufferIndex = 0; ABufferIndex < RenderData.Bucket->TagLayout.Layers.Num(); ABufferIndex++)
	{
		// Override the format
		Desc.Format = RenderData.Bucket->TagLayout.Layers[ABufferIndex].RenderFormat;

		// Create the texture
		FRDGTextureRef Texture = GraphBuilder.CreateTexture(Desc, ABufferNames[ABufferIndex]);
		RenderData.ABuffer.ABufferTextures.Add(Texture);
		
		// TODO[MP]: This is a clear per-slice, which is inefficient
		// There should be something better somewhere
		FRDGTextureClearInfo TextureClearInfo;
		TextureClearInfo.ClearColor = FLinearColor(0, 0, 0, 0);
		TextureClearInfo.NumSlices  = Desc.ArraySize;
		AddClearRenderTargetPass(GraphBuilder, Texture, TextureClearInfo);
	}
	
	// Create R8 mask if needed
	const FMaterialCachePageCollection& HardwareCollection = RenderData.PageCollections[static_cast<uint32>(EMaterialCacheRenderPath::HardwareRaster)];
	if (!HardwareCollection.Pages.IsEmpty())
	{
		Desc.Format = PF_R8_UINT;
		
		RenderData.ABuffer.ABufferR8Mask = GraphBuilder.CreateTexture(Desc, TEXT("MaterialCache::ABufferR8Mask"));
		
		// TODO[MP]: This is a clear per-slice, which is inefficient
		// There should be something better somewhere
		FRDGTextureClearInfo TextureClearInfo;
		TextureClearInfo.ClearColor = FLinearColor(0, 0, 0, 0);
		TextureClearInfo.NumSlices  = Desc.ArraySize;
		AddClearRenderTargetPass(GraphBuilder, RenderData.ABuffer.ABufferR8Mask, TextureClearInfo);
	}
}

static FUintVector3 GetMaterialCacheABufferTilePhysicalLocation(const FMaterialCacheRenderData& RenderData, uint32_t ABufferPageIndex)
{
	const FIntPoint RenderTileSize = GetMaterialCacheRenderTileSize();

	switch (RenderData.ABuffer.Layout)
	{
	default:
		checkNoEntry();
		return {};
	case EMaterialCacheABufferTileLayout::Horizontal:
		return FUintVector3(RenderTileSize.X * ABufferPageIndex, 0, 0);
	case EMaterialCacheABufferTileLayout::Sliced:
		return FUintVector3(0, 0, ABufferPageIndex);
	}
}

static void GetShadingBinData(const FMaterialCacheRenderData& RenderData, FMaterialCacheSceneExtension& SceneExtension, const FMaterialCachePageCollection& Collection, FRDGUploadData<UE::HLSL::FMaterialCacheBinData>& Out)
{
	static FIntPoint RenderTileSize = GetMaterialCacheRenderTileSize();
	
	for (int32 PageIndex = 0; PageIndex < Collection.Pages.Num(); PageIndex++)
	{
		const FMaterialCachePageInfo& Info = Collection.Pages[PageIndex];

		UE::HLSL::FMaterialCacheBinData& BinData = Out[PageIndex];

		BinData.ABufferPhysicalPosition = GetMaterialCacheABufferTilePhysicalLocation(RenderData, Info.ABufferPageIndex);

		BinData.UVMinAndInvSize = FVector4f{
			Info.Page.UVRect.Min.X,
			Info.Page.UVRect.Min.Y,
			1.0f / (Info.Page.UVRect.Max.X - Info.Page.UVRect.Min.X),
			1.0f / (Info.Page.UVRect.Max.Y - Info.Page.UVRect.Min.Y)
		};

		FVector2f UVRange = Info.Page.UVRect.Max - Info.Page.UVRect.Min;
		BinData.UVMinAndThreadAdvance = FVector4f(
			Info.Page.UVRect.Min,
			FVector2f(1.0f / RenderTileSize.X, 1.0f / RenderTileSize.Y) * UVRange
		);

		const FMaterialCachePendingEntry& Entry = RenderData.Bucket->PendingEntries[Info.SetupEntryIndex];
		
		// TODO: For vertex invariant pages, we don't really have a way to associate separate primitives
		if (const FMaterialCacheAllocationSceneData* AllocationSceneData = SceneExtension.GetAllocationSceneData(Entry.Setup.Allocation))
		{
			if (const FMaterialCachePrimitiveData* PrimitiveData = SceneExtension.GetPrimitiveData(AllocationSceneData->Primitives[0]))
			{
				BinData.PrimitiveData = PrimitiveData->Proxy->GetPrimitiveSceneInfo()->GetPersistentIndex().Index;
			}
		}
	}
}

static uint32 GetMaterialCacheTileThreadCount()
{
	uint32 TileWidth       = GetMaterialCacheTileWidth();
	uint32 TileBorderWidth = GetMaterialCacheTileBorderWidth();

	// Unaligned border width and total thread count (excl. last tile)
	uint32 UnalignedWidth           = TileBorderWidth * 2;
	uint32 UnalignedTileThreadCount = UnalignedWidth * UnalignedWidth;

	// Aligned (intra-tile) and unaligned thread count (inc. last tile)
	uint32 AlignedThreadCount   = TileWidth * TileWidth;
	uint32 UnalignedThreadCount = UnalignedWidth * TileWidth * 2 + UnalignedTileThreadCount;
	return AlignedThreadCount + UnalignedThreadCount;
}

static FUintVector4 GetMaterialCacheTileParams()
{
	uint32 TileWidth       = GetMaterialCacheTileWidth();
	uint32 TileBorderWidth = GetMaterialCacheTileBorderWidth();
	uint32 UnalignedWidth  = TileBorderWidth * 2;

	return FUintVector4(
		/* Standard width of a tile */
		TileWidth,

		/* The number of aligned threads  */
		TileWidth * TileWidth,

		/* The number of unaligned threads in a single axis */
		UnalignedWidth * TileWidth,

		/* The unaligned offset, same on each axis */
		TileWidth + UnalignedWidth
	);
}

static FUintVector4 GetMaterialCacheTileOrderingParams()
{
	uint32 TileWidth       = GetMaterialCacheTileWidth();
	uint32 TileBorderWidth = GetMaterialCacheTileBorderWidth();

	// Unaligned border width and total thread count (excl. last tile)
	uint32_t UnalignedWidth       = TileBorderWidth * 2;
	uint32_t UnalignedThreadCount = UnalignedWidth  * UnalignedWidth;

	return FUintVector4(
		/* Unaligned tile morton window bit-mask */
		(1u << FMath::FloorLog2NonZero(UnalignedThreadCount)) - 1,
		
		/* Unaligned tile divisor as SHR */
		FMath::FloorLog2NonZero(UnalignedThreadCount),
		
		/* Unaligned tile y-offset as SHL */
		FMath::FloorLog2NonZero(UnalignedWidth),
		
		/* Assumed border width, a bit out of place */
		TileBorderWidth
	);
}

static void MaterialCacheSetupHardwareContext(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheSceneExtension& SceneExtension, FMaterialCacheRenderData& RenderData, FMaterialCacheHardwareContext& Context)
{
	const FMaterialCachePageCollection& Collection = RenderData.PageCollections[static_cast<uint32>(EMaterialCacheRenderPath::HardwareRaster)];

	if (Collection.Pages.IsEmpty())
	{
		return;
	}
	
	// All shading data, one per page
	FRDGUploadData<UE::HLSL::FMaterialCacheBinData> ShadingDataArray(GraphBuilder, Collection.Pages.Num());
	GetShadingBinData(RenderData, SceneExtension, Collection, ShadingDataArray);

	FRDGBufferRef ShadingBinData = GraphBuilder.CreateBuffer(
		FRDGBufferDesc::CreateBufferDesc(sizeof(FUintVector4), ShadingDataArray.NumBytes() / sizeof(FUintVector4)),
		TEXT("MaterialCache.ShadingBinData")
	);
	
	GraphBuilder.QueueBufferUpload(ShadingBinData, ShadingDataArray.GetData(), ShadingDataArray.NumBytes(), ERDGInitialDataFlags::None);

	FMaterialCacheUniformParameters* PassUniformParameters = GraphBuilder.AllocParameters<FMaterialCacheUniformParameters>();
	PassUniformParameters->ShadingBinData = GraphBuilder.CreateSRV(ShadingBinData, PF_R32G32B32A32_UINT);
	PassUniformParameters->SvPagePositionModMask = GetMaterialCacheTileWidth() - 1u;
	PassUniformParameters->TileParams = GetMaterialCacheTileParams();
	PassUniformParameters->TileOrderingParams = GetMaterialCacheTileOrderingParams();
	SetupSceneTextureUniformParameters(GraphBuilder, nullptr, Renderer->Scene->GetFeatureLevel(), ESceneTextureSetupMode::None, PassUniformParameters->SceneTextures);

	Context.PassUniformParameters = PassUniformParameters;
}

static FUintVector4 GetMaterialCacheABufferTilePhysicalViewport(const FMaterialCacheRenderData& RenderData, uint32_t ABufferPageIndex)
{
	const FIntPoint RenderTileSize = GetMaterialCacheRenderTileSize();

	switch (RenderData.ABuffer.Layout)
	{
	default:
		checkNoEntry();
		return {};
	case EMaterialCacheABufferTileLayout::Horizontal:
		return FUintVector4(
			RenderTileSize.X * ABufferPageIndex, 0,
			RenderTileSize.X * (ABufferPageIndex + 1), RenderTileSize.Y
		);
	case EMaterialCacheABufferTileLayout::Sliced:
		return FUintVector4(0, 0, RenderTileSize.X, RenderTileSize.Y);
	}
}

static void MaterialCacheRenderHardwarePages(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheRenderData& RenderData, FMaterialCacheLayerRenderData& LayerRenderData, FMaterialCacheHardwareContext& Context, uint32 LayerBatchIndex)
{
	const FMaterialCachePageCollection& Collection = RenderData.PageCollections[static_cast<uint32>(EMaterialCacheRenderPath::HardwareRaster)];

	if (Collection.Pages.IsEmpty())
	{
		return;
	}
	
	const bool bUseArrayTargetablePages = GRHISupportsArrayIndexFromAnyShader && GMaterialCacheStaticMeshEnableViewportFromVS;

	const FIntPoint RenderTileSize = GetMaterialCacheRenderTileSize();

	FInstanceCullingResult   InstanceCullingResult;
	FInstanceCullingContext* InstanceCullingContext = nullptr;
	FRHIBuffer* PrimitiveIdVertexBuffer = nullptr;

	if (Renderer->Scene->GPUScene.IsEnabled())
	{
		InstanceCullingContext = GraphBuilder.AllocObject<FInstanceCullingContext>(
			TEXT("FInstanceCullingContext"),
			Renderer->Views[0].GetShaderPlatform(),
			nullptr,
			TArrayView<const int32>(&Renderer->Views[0].SceneRendererPrimaryViewId, 1),
			nullptr
		);

		int32 MaxInstances = 0;
		int32 VisibleMeshDrawCommandsNum = 0;
		int32 NewPassVisibleMeshDrawCommandsNum = 0;

		InstanceCullingContext->SetupDrawCommands(
			LayerRenderData.Hardware.VisibleMeshCommands,
			false,
			Renderer->Scene,
			MaxInstances,
			VisibleMeshDrawCommandsNum,
			NewPassVisibleMeshDrawCommandsNum
		);

		InstanceCullingContext->BuildRenderingCommands(
			GraphBuilder,
			Renderer->Scene->GPUScene,
			Renderer->Views[0].DynamicPrimitiveCollector.GetInstanceSceneDataOffset(),
			Renderer->Views[0].DynamicPrimitiveCollector.NumInstances(),
			InstanceCullingResult
		);
	}
	else
	{
		const uint32 PrimitiveIdBufferDataSize = LayerRenderData.Hardware.PrimitiveIds.Num() * sizeof(int32);

		FPrimitiveIdVertexBufferPoolEntry Entry = GPrimitiveIdVertexBufferPool.Allocate(GraphBuilder.RHICmdList, PrimitiveIdBufferDataSize);
		PrimitiveIdVertexBuffer = Entry.BufferRHI;

		// Copy over primitive ids
		void* RESTRICT PrimitiveData = GraphBuilder.RHICmdList.LockBuffer(PrimitiveIdVertexBuffer, 0, PrimitiveIdBufferDataSize, RLM_WriteOnly);
		FMemory::Memcpy(PrimitiveData, LayerRenderData.Hardware.PrimitiveIds.GetData(), PrimitiveIdBufferDataSize);
		GraphBuilder.RHICmdList.UnlockBuffer(PrimitiveIdVertexBuffer);

		GPrimitiveIdVertexBufferPool.ReturnToFreeList(Entry);
	}

	FMaterialCacheRastShadeParameters* MeshPassParameters = GraphBuilder.AllocParameters<FMaterialCacheRastShadeParameters>();
	MeshPassParameters->View = GraphBuilder.CreateUniformBuffer(GraphBuilder.AllocParameters(Renderer->Views[0].CachedViewUniformShaderParameters.Get()));
	MeshPassParameters->Pass = GraphBuilder.CreateUniformBuffer(Context.PassUniformParameters);
	MeshPassParameters->Scene = Renderer->Views[0].GetSceneUniforms().GetBuffer(GraphBuilder);
	InstanceCullingResult.GetDrawParameters(MeshPassParameters->InstanceCullingDrawParams);

	// Blend mode for development
	uint32 Flags = UE::HLSL::MatCache_None;
	if (!LayerBatchIndex)
	{
		Flags |= UE::HLSL::MatCache_DefaultBottomLayer;
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Hardware Batch (%u pages)", Collection.Pages.Num()),
		MeshPassParameters,
		ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass,
		[
			bUseArrayTargetablePages, Flags,
			Renderer, PrimitiveIdVertexBuffer, RenderTileSize, MeshPassParameters, InstanceCullingContext, &LayerRenderData, &Collection, &RenderData
		](FRDGAsyncTask, FRHICommandList& RHICmdList) mutable
		{
			FMeshDrawCommandStateCache StateCache;

			FMeshDrawCommandOverrideArgs OverrideArgs = GetMeshDrawCommandOverrideArgs(MeshPassParameters->InstanceCullingDrawParams);

			FMeshDrawCommandSceneArgs SceneArgs;
			
			if (IsUniformBufferStaticSlotValid(InstanceCullingContext->InstanceCullingStaticSlot))
			{
				if (InstanceCullingContext->bUsesUniformBufferView)
				{
					SceneArgs.BatchedPrimitiveSlot = InstanceCullingContext->InstanceCullingStaticSlot;
				}

				RHICmdList.SetStaticUniformBuffer(InstanceCullingContext->InstanceCullingStaticSlot, OverrideArgs.InstanceCullingStaticUB);
			}
			
			if (bUseArrayTargetablePages)
			{
				RHICmdList.SetViewport(0, 0, 0, RenderTileSize.X, RenderTileSize.Y, 1.0f);
			}

			for (int32 CommandIndex = 0; CommandIndex < LayerRenderData.Hardware.MeshCommands.Num(); CommandIndex++)
			{
				const FMaterialCacheStaticMeshCommand& Command = LayerRenderData.Hardware.MeshCommands[CommandIndex];

				const FMaterialCachePageInfo& PageInfo = Collection.Pages[Command.PageIndex];

				if (!bUseArrayTargetablePages)
				{
					const FUintVector4 Viewport = GetMaterialCacheABufferTilePhysicalViewport(RenderData, PageInfo.ABufferPageIndex);
					RHICmdList.SetViewport(
						Viewport.X, Viewport.Y, 0,
						Viewport.Z, Viewport.W, 1.0f
					);
				}

				FGraphicsMinimalPipelineStateSet GraphicsMinimalPipelineStateSet;

				check(GRHISupportsShaderRootConstants);
				SceneArgs.RootConstants = FUintVector4(
					Command.PageIndex,
					PageInfo.ABufferPageIndex,
					Flags,
					Command.UVCoordinateIndex
				);

				SceneArgs.PrimitiveIdOffset = CommandIndex * FInstanceCullingContext::GetInstanceIdBufferStride(Renderer->Scene->GetShaderPlatform());

				if (Renderer->Scene->GPUScene.IsEnabled())
				{
					const FInstanceCullingContext::FMeshDrawCommandInfo& DrawCommandInfo = InstanceCullingContext->MeshDrawCommandInfos[CommandIndex];

					SceneArgs.IndirectArgsByteOffset = 0u;
					SceneArgs.IndirectArgsBuffer = nullptr;

					if (DrawCommandInfo.bUseIndirect)
					{
						SceneArgs.IndirectArgsByteOffset = OverrideArgs.IndirectArgsByteOffset + DrawCommandInfo.IndirectArgsOffsetOrNumInstances;
						SceneArgs.IndirectArgsBuffer = OverrideArgs.IndirectArgsBuffer;
					}

					SceneArgs.PrimitiveIdOffset = OverrideArgs.InstanceDataByteOffset + DrawCommandInfo.InstanceDataByteOffset;
					SceneArgs.PrimitiveIdsBuffer = OverrideArgs.InstanceBuffer;

					FMeshDrawCommand::SubmitDraw(
						*LayerRenderData.Hardware.VisibleMeshCommands[CommandIndex].MeshDrawCommand,
						GraphicsMinimalPipelineStateSet,
						SceneArgs,
						1,
						RHICmdList,
						StateCache
					);
				}
				else
				{
					SceneArgs.PrimitiveIdsBuffer = PrimitiveIdVertexBuffer;

					FMeshDrawCommand::SubmitDraw(
						*LayerRenderData.Hardware.VisibleMeshCommands[CommandIndex].MeshDrawCommand,
						GraphicsMinimalPipelineStateSet,
						SceneArgs,
						1,
						RHICmdList,
						StateCache
					);
				}
			}
		}
	);
}

static void MaterialCacheRenderNanitePages(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheRenderData& RenderData, FMaterialCacheLayerRenderData& LayerRenderData, FMaterialCacheNaniteContext& Context, uint32 LayerBatchIndex)
{
	const FMaterialCachePageCollection& Collection = RenderData.PageCollections[static_cast<uint32>(EMaterialCacheRenderPath::NaniteRaster)];

	if (Collection.Pages.IsEmpty())
	{
		return;
	}
	
	FMaterialCacheNaniteStackShadeParameters* Params = GraphBuilder.AllocParameters<FMaterialCacheNaniteStackShadeParameters>();
	Params->Shade = *Context.PassShadeParameters;
	Params->PageIndirections = GraphBuilder.CreateSRV(LayerRenderData.Nanite.GenericCSBatch.PageIndirectionBuffer, PF_R32_UINT);
	Params->Pass = GraphBuilder.CreateUniformBuffer(Context.PassUniformParameters);
	
	// Blend mode for development
	uint32 Flags = UE::HLSL::MatCache_None;
	if (!LayerBatchIndex)
	{
		Flags |= UE::HLSL::MatCache_DefaultBottomLayer;
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Nanite Batch (%u pages)", Collection.Pages.Num()),
		Params,
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		[
			Params, &LayerRenderData, Flags
		](FRHICommandList& RHICmdList) mutable
		{
			// Subsequent batches can run in parallel without issue
			for (FMaterialCacheGenericCSMaterialBatch& MaterialBatch : LayerRenderData.Nanite.GenericCSBatch.MaterialBatches)
			{
				for (FMaterialCacheGenericCSPrimitiveBatch& PrimitiveBatch : MaterialBatch.PrimitiveBatches)
				{
					TShaderRefBase<FMaterialCacheNaniteShadeCS, FShaderMapPointerTable> Shader = TShaderRef<FMaterialCacheNaniteShadeCS>::Cast(PrimitiveBatch.ShadingCommand->ComputeShader);

					if (!Shader.IsValid())
					{
						UE_LOGF(LogRenderer, Error, "Invalid shading command");
						continue;
					}

					SetComputePipelineState(RHICmdList, Shader.GetComputeShader());

					// TODO: Case with no root support
					check(GRHISupportsShaderRootConstants);

					FUintVector4 RootData;
					RootData.X = PrimitiveBatch.PageIndirectionOffset;
					RootData.Y = PrimitiveBatch.ShadingBin;
					RootData.Z = static_cast<uint32>(ENaniteMeshPass::MaterialCache);
					RootData.W = Flags;
					RHICmdList.SetShaderRootConstants(RootData);

					// Bind parameters
					FRHIBatchedShaderParameters& ShadingParameters = RHICmdList.GetScratchShaderParameters();
					PrimitiveBatch.ShadingCommand->ShaderBindings.SetParameters(ShadingParameters);
					Shader->SetPassParameters(ShadingParameters, RootData, Params->PageIndirections->GetRHI());
					RHICmdList.SetBatchedShaderParameters(Shader.GetComputeShader(), ShadingParameters);
					
					// Dispatch the bin over all pages
					RHICmdList.DispatchComputeShader(
						FMath::DivideAndRoundUp(GetMaterialCacheTileThreadCount(), 64u),
						PrimitiveBatch.Pages.Num(),
						1
					);
				}
			}
		});
}

static void MaterialCacheSetupVertexInvariantContext(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheSceneExtension& SceneExtension, FMaterialCacheRenderData& RenderData, FMaterialCacheVertexInvariantContext& Context)
{
	const FMaterialCachePageCollection& Collection = RenderData.PageCollections[static_cast<uint32>(EMaterialCacheRenderPath::VertexInvariant)];

	if (Collection.Pages.IsEmpty())
	{
		return;
	}

	// All shading data, one per page
	FRDGUploadData<UE::HLSL::FMaterialCacheBinData> ShadingDataArray(GraphBuilder, Collection.Pages.Num());
	GetShadingBinData(RenderData, SceneExtension, Collection, ShadingDataArray);

	FRDGBufferRef ShadingBinData = CreateStructuredBuffer(
		GraphBuilder,
		TEXT("MaterialCache.ShadingBinData"),
		sizeof(UE::HLSL::FMaterialCacheBinData),
		ShadingDataArray.Num(), ShadingDataArray.GetData(),
		ShadingDataArray.NumBytes()
	);

	FMaterialCacheUniformParameters* PassUniformParameters = GraphBuilder.AllocParameters<FMaterialCacheUniformParameters>();
	PassUniformParameters->ShadingBinData = GraphBuilder.CreateSRV(ShadingBinData);
	PassUniformParameters->TileParams = GetMaterialCacheTileParams();
	PassUniformParameters->TileOrderingParams = GetMaterialCacheTileOrderingParams();
	SetupSceneTextureUniformParameters(GraphBuilder, nullptr, Renderer->Scene->GetFeatureLevel(), ESceneTextureSetupMode::None, PassUniformParameters->SceneTextures);

	Context.PassUniformParameters = PassUniformParameters;
}

static void MaterialCacheRenderVertexInvariantPages(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheRenderData& RenderData, FMaterialCacheLayerRenderData& LayerRenderData, FMaterialCacheVertexInvariantContext& Context, uint32 LayerBatchIndex)
{
	const FMaterialCachePageCollection& Collection = RenderData.PageCollections[static_cast<uint32>(EMaterialCacheRenderPath::VertexInvariant)];

	if (Collection.Pages.IsEmpty())
	{
		return;
	}
	
	FMaterialCacheCSStackShadeParameters* Params = GraphBuilder.AllocParameters<FMaterialCacheCSStackShadeParameters>();
	Params->View = Renderer->Views[0].GetShaderParameters();
	Params->Pass = GraphBuilder.CreateUniformBuffer(Context.PassUniformParameters);
	Params->Scene = Renderer->Views[0].GetSceneUniforms().GetBuffer(GraphBuilder);
	Params->PageIndirections = GraphBuilder.CreateSRV(LayerRenderData.VertexInvariant.GenericCSBatch.PageIndirectionBuffer, PF_R32_UINT);
	
	// Blend mode for development
	uint32 Flags = UE::HLSL::MatCache_None;
	if (!LayerBatchIndex)
	{
		Flags |= UE::HLSL::MatCache_DefaultBottomLayer;
	}
	
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Vertex-Invariant Batch (%u)", Collection.Pages.Num()),
		Params,
		ERDGPassFlags::Compute | ERDGPassFlags::NeverCull,
		[
			&LayerRenderData, Flags, Params
		](FRHICommandList& RHICmdList) mutable
		{
			// Subsequent batches can run in parallel without issue
			for (const FMaterialCacheGenericCSMaterialBatch& MaterialBatch : LayerRenderData.VertexInvariant.GenericCSBatch.MaterialBatches)
			{
				for (const FMaterialCacheGenericCSPrimitiveBatch& PrimitiveBatch : MaterialBatch.PrimitiveBatches)
				{
					TShaderRefBase<FMaterialCacheShadeCS, FShaderMapPointerTable> Shader = TShaderRef<FMaterialCacheShadeCS>::Cast(PrimitiveBatch.ShadingCommand->ComputeShader);
					if (!Shader.IsValid())
					{
						UE_LOGF(LogRenderer, Error, "Invalid shading command");
						continue;
					}

					SetComputePipelineState(RHICmdList, Shader.GetComputeShader());

					// TODO: Case with no root support
					check(GRHISupportsShaderRootConstants);

					FUintVector4 RootData;
					RootData.X = PrimitiveBatch.PageIndirectionOffset;
					RootData.Y = static_cast<uint32>(Flags);
					RootData.Z = PrimitiveBatch.UVCoordinateIndex;
					RHICmdList.SetShaderRootConstants(RootData);

					// Bind parameters
					FRHIBatchedShaderParameters& ShadingParameters = RHICmdList.GetScratchShaderParameters();
					PrimitiveBatch.ShadingCommand->ShaderBindings.SetParameters(ShadingParameters);
					Shader->SetPassParameters(ShadingParameters, RootData, Params->PageIndirections->GetRHI());
					RHICmdList.SetBatchedShaderParameters(Shader.GetComputeShader(), ShadingParameters);
					
					// Dispatch the bin over all pages
					RHICmdList.DispatchComputeShader(
						FMath::DivideAndRoundUp(GetMaterialCacheTileThreadCount(), 64u),
						PrimitiveBatch.Pages.Num(),
						1
					);
				}
			}
		}
	);
}

static void GetNaniteRectArray(const FMaterialCachePageCollection& Collection, const FIntPoint& RenderTileSize, const TArrayView<FIntPoint>& TileOffsets, FRDGUploadData<FUintVector4>& Out)
{
	for (int32 PageIndex = 0; PageIndex < Collection.Pages.Num(); PageIndex++)
	{
		const FIntPoint& TileOffset = TileOffsets[PageIndex];
				
		Out[PageIndex] = FUintVector4(
			TileOffset.X,
			TileOffset.Y,
			TileOffset.X + RenderTileSize.X,
			TileOffset.Y + RenderTileSize.Y
		);
	}
}

static FIntPoint MaterialCacheArrangeRenderTiles(const FMaterialCachePageCollection& Collection, const FIntPoint RenderTileSize, TArray<FIntPoint, SceneRenderingAllocator>& TileOffsets)
{
	FIntPoint ContextSize = FIntPoint::ZeroValue;
	TileOffsets.SetNumZeroed(Collection.Pages.Num());

	// Arrange horizontally, wrap around on limits
	FIntPoint ContextTileOffset = FIntPoint::ZeroValue;
	for (int32 PageIndex = 0; PageIndex < Collection.Pages.Num(); PageIndex++)
	{
		TileOffsets[PageIndex] = ContextTileOffset;
		ContextSize = ContextSize.ComponentMax(ContextTileOffset + RenderTileSize);

		// Advance X
		ContextTileOffset.X += RenderTileSize.X;

		// Wrap around if needed
		if (ContextTileOffset.X + RenderTileSize.X >= GRHIGlobals.MaxTextureDimensions)
		{
			ContextTileOffset.X = 0;
			ContextTileOffset.Y += RenderTileSize.Y;
		}
	}

	return ContextSize;
}

static void MaterialCacheSetupNaniteContext(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheSceneExtension& SceneExtension, FMaterialCacheRenderData& RenderData, FMaterialCacheNaniteContext& Context)
{
	const FIntPoint RenderTileSize = GetMaterialCacheRenderTileSize();
	
	const FMaterialCachePageCollection& Collection = RenderData.PageCollections[static_cast<uint32>(EMaterialCacheRenderPath::NaniteRaster)];

	if (Collection.Pages.IsEmpty())
	{
		return;
	}

	// TODO[MP]: Just need to split up the batches
	checkf(Collection.Pages.Num() <= NANITE_MAX_VIEWS_PER_CULL_RASTERIZE_PASS, TEXT("Pending support for > 128 pages per frame"));

	// Wait for all bins to finish
	Renderer->Scene->WaitForCacheNaniteMaterialBinsTask();

	// TODO[MP]: With the layering, we probably don't need this
	Nanite::BuildShadingCommands(
		GraphBuilder,
		*Renderer->Scene,
		ENaniteMeshPass::MaterialCache,
		RenderData.Nanite.ShadingCommands,
		Nanite::EBuildShadingCommandsMode::Custom
	);
	
	TArray<FIntPoint, SceneRenderingAllocator> TileOffsets;
	FIntPoint RasterContextSize = MaterialCacheArrangeRenderTiles(Collection, RenderTileSize, TileOffsets);;
	
	// Create a view per page, we render all views laid out horizontally across the vis-buffer
	Nanite::FPackedViewArray* NaniteViews = Nanite::FPackedViewArray::CreateWithSetupTask(
		GraphBuilder,
		Collection.Pages.Num(),
		[RenderTileSize, RasterContextSize, Renderer, &TileOffsets, &Collection](Nanite::FPackedViewArray::ArrayType& OutViews)
		{
			const FMatrix ProjectionMatrix = FReversedZOrthoMatrix(
				0, RenderTileSize.X,
				0, RenderTileSize.Y,
				1.0f,
				0
			);

			FViewMatrices::FMinimalInitializer Initializer;
			Initializer.ViewRotationMatrix = FMatrix::Identity;
			Initializer.ViewOrigin = FVector::Zero();
			Initializer.ProjectionMatrix = ProjectionMatrix;
			Initializer.ConstrainedViewRect = Renderer->Views[0].SceneViewInitOptions.GetConstrainedViewRect();
			Initializer.StereoPass = Renderer->Views[0].SceneViewInitOptions.StereoPass;
			FViewMatrices ViewMatrices = FViewMatrices(Initializer);

			// Shared view parameters
			Nanite::FPackedViewParams Params;
			Params.ViewMatrices = ViewMatrices;
			Params.PrevViewMatrices = ViewMatrices;
			Params.RasterContextSize = RasterContextSize;
			Params.Flags = 0x0;
			Params.StreamingPriorityCategory = 3;
			Params.MinBoundsRadius = 0;
			Params.ViewLODDistanceFactor = Renderer->Views[0].LODDistanceFactor;
			Params.HZBTestViewRect = Renderer->Views[0].PrevViewInfo.ViewRect;
			Params.MaxPixelsPerEdgeMultipler = 1.0f;
			Params.GlobalClippingPlane = Renderer->Views[0].GlobalClippingPlane;
			Params.SceneRendererPrimaryViewId = Renderer->Views[0].SceneRendererPrimaryViewId;

			// Setup pages
			for (int32 PageIndex = 0; PageIndex < Collection.Pages.Num(); PageIndex++)
			{
				const FMaterialCachePageInfo& PageInfo = Collection.Pages[PageIndex];

				const FIntPoint& TileOffset = TileOffsets[PageIndex];
				
				Params.ViewRect = FIntRect(
					TileOffset.X,
					TileOffset.Y,
					TileOffset.X + RenderTileSize.X,
					TileOffset.Y + RenderTileSize.Y
				);

				Nanite::FPackedView View = Nanite::CreatePackedView(Params);

				View.MaterialCacheUnwrapMinAndInvSize = FVector4f(
					PageInfo.Page.UVRect.Min.X,
					PageInfo.Page.UVRect.Min.Y,
					1.0f / (PageInfo.Page.UVRect.Max.X - PageInfo.Page.UVRect.Min.X),
					1.0f / (PageInfo.Page.UVRect.Max.Y - PageInfo.Page.UVRect.Min.Y)
				);

				View.MaterialCachePageOffsetAndInvSize = FVector4f(
					TileOffset.X / static_cast<float>(RasterContextSize.X),
					TileOffset.Y / static_cast<float>(RasterContextSize.Y),
					RenderTileSize.X / static_cast<float>(RasterContextSize.X),
					RenderTileSize.Y / static_cast<float>(RasterContextSize.Y)
				);

				OutViews.Add(MoveTemp(View));
			}
		});

	// Rasterization view rectangles, one per page
	FRDGUploadData<FUintVector4> RasterRectArray(GraphBuilder, Collection.Pages.Num());
	GetNaniteRectArray(Collection, RenderTileSize, TileOffsets, RasterRectArray);

	// All shading data, one per page
	FRDGUploadData<UE::HLSL::FMaterialCacheBinData> ShadingDataArray(GraphBuilder, Collection.Pages.Num());
	GetShadingBinData(RenderData, SceneExtension, Collection, ShadingDataArray);

	FRDGBufferRef RasterRectBuffer = CreateUploadBuffer(
		GraphBuilder,
		TEXT("MaterialCache.Rects"),
		sizeof(FUintVector4), FMath::RoundUpToPowerOfTwo(RasterRectArray.Num()),
		RasterRectArray
	);

	FRDGBuffer* PackedViewBuffer = CreateStructuredBuffer(
		GraphBuilder,
		TEXT("MaterialCache.PackedViews"),
		NaniteViews->GetViews().GetTypeSize(),
		NaniteViews->NumViews,
		NaniteViews->GetViews().GetData(),
		NaniteViews->GetViews().NumBytes()
	);

	FRDGBuffer* ShadingBinData = CreateByteAddressBuffer(
		GraphBuilder,
		TEXT("MaterialCache.ShadingBinData"),
		ShadingDataArray.NumBytes(), ShadingDataArray.GetData(),
		ShadingDataArray.NumBytes()
	);

	Nanite::FSharedContext SharedContext{};
	SharedContext.FeatureLevel = Renderer->Scene->GetFeatureLevel();
	SharedContext.ShaderMap = GetGlobalShaderMap(SharedContext.FeatureLevel);
	SharedContext.Pipeline = Nanite::ERasterPipeline::MaterialCache;

	// Create context, tile all pages horizontally
	const Nanite::FRasterContextInitParams RasterInitParams =
	{
		.TextureSize = RasterContextSize,
		.TextureRect = FIntRect(0, 0, RasterContextSize.X, RasterContextSize.Y),
		.RasterMode = Nanite::EOutputBufferMode::VisBuffer,
		.RectMinMaxBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(RasterRectBuffer, PF_R32G32B32A32_UINT)),
		.NumRects = uint32(Collection.Pages.Num()),
		.bAsyncCompute = false
	};
	Nanite::FRasterContext RasterContext = Nanite::InitRasterContext(
		GraphBuilder,
		SharedContext,
		*Renderer->GetViewFamily(),
		RasterInitParams		
	);

	// Setup object space config
	Nanite::FConfiguration CullingConfig = { 0 };
	CullingConfig.SetViewFlags(Renderer->Views[0]);
	CullingConfig.bIsMaterialCache = true;
	CullingConfig.bForceHWRaster = true;
	CullingConfig.bUpdateStreaming = true;

	TUniquePtr<Nanite::IRenderer> NaniteRenderer = Nanite::IRenderer::Create(
		GraphBuilder,
		*Renderer->Scene,
		Renderer->Views[0],
		Renderer->GetSceneUniforms(),
		SharedContext,
		RasterContext,
		CullingConfig,
		FIntRect(),
		nullptr
	);

	Nanite::FRasterResults RasterResults;

	NaniteRenderer->DrawGeometry(
		Renderer->Scene->NaniteRasterPipelines[ENaniteMeshPass::MaterialCache],
		RasterResults.VisibilityQuery,
		*NaniteViews,
		RenderData.Nanite.InstanceDraws
	);

	NaniteRenderer->ExtractResults(RasterResults);

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	FNaniteRasterUniformParameters* RasterUniformParameters = GraphBuilder.AllocParameters<FNaniteRasterUniformParameters>();
	RasterUniformParameters->PageConstants = RasterResults.PageConstants;
	RasterUniformParameters->MaxNodes = Nanite::FGlobalResources::GetMaxNodes();
	RasterUniformParameters->MaxVisibleClusters = Nanite::FGlobalResources::GetMaxVisibleClusters();
	RasterUniformParameters->MaxCandidatePatches = Nanite::FGlobalResources::GetMaxCandidatePatches();
	RasterUniformParameters->MaxPatchesPerGroup = RasterResults.MaxPatchesPerGroup;
	RasterUniformParameters->MeshPass = RasterResults.MeshPass;
	RasterUniformParameters->InvDiceRate = RasterResults.InvDiceRate;
	RasterUniformParameters->RenderFlags = RasterResults.RenderFlags;
	RasterUniformParameters->DebugFlags = RasterResults.DebugFlags;
	RasterUniformParameters->CurveSubdivisionThreshold = Nanite::FGlobalResources::GetCurveSubdivisionThreshold();
	RasterUniformParameters->CurveSubdivisionMaxSegments = Nanite::FGlobalResources::GetCurveSubdivisionMaxSegments();

	FNaniteShadingUniformParameters* ShadingUniformParameters = GraphBuilder.AllocParameters<FNaniteShadingUniformParameters>();
	ShadingUniformParameters->ClusterPageData = Nanite::GStreamingManager.GetClusterPageDataSRV(GraphBuilder);
	ShadingUniformParameters->HierarchyBuffer = Nanite::GStreamingManager.GetHierarchySRV(GraphBuilder);
	ShadingUniformParameters->VisibleClustersSWHW = GraphBuilder.CreateSRV(RasterResults.VisibleClustersSWHW);
	ShadingUniformParameters->AssemblyTransforms = GraphBuilder.CreateSRV(RasterResults.AssemblyTransforms);
	ShadingUniformParameters->VisBuffer64 = RasterContext.VisBuffer64;
	ShadingUniformParameters->DbgBuffer64 = SystemTextures.Black;
	ShadingUniformParameters->DbgBuffer32 = SystemTextures.Black;
	ShadingUniformParameters->ShadingMask = SystemTextures.Black;
	ShadingUniformParameters->ShadingBinData = GraphBuilder.CreateSRV(ShadingBinData);
	ShadingUniformParameters->MultiViewEnabled = 1;
	ShadingUniformParameters->MultiViewIndices = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer<uint32>(GraphBuilder));
	ShadingUniformParameters->MultiViewRectScaleOffsets = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer<FVector4>(GraphBuilder));
	ShadingUniformParameters->InViews = GraphBuilder.CreateSRV(PackedViewBuffer);
	ShadingUniformParameters->ThreadGroupData = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer<uint32>(GraphBuilder));

	FMaterialCacheNaniteShadeParameters* PassParameters = GraphBuilder.AllocParameters<FMaterialCacheNaniteShadeParameters>();
	PassParameters->NaniteRaster = GraphBuilder.CreateUniformBuffer(RasterUniformParameters);
	PassParameters->NaniteShading = GraphBuilder.CreateUniformBuffer(ShadingUniformParameters);
	PassParameters->View = Renderer->Views[0].GetShaderParameters();
	PassParameters->Scene = Renderer->Views[0].GetSceneUniforms().GetBuffer(GraphBuilder);
	Context.PassShadeParameters = PassParameters;

	FMaterialCacheUniformParameters* PassUniformParameters = GraphBuilder.AllocParameters<FMaterialCacheUniformParameters>();
	PassUniformParameters->ShadingBinData = GraphBuilder.CreateSRV(ShadingBinData);
	PassUniformParameters->TileParams = GetMaterialCacheTileParams();
	PassUniformParameters->TileOrderingParams = GetMaterialCacheTileOrderingParams();
	SetupSceneTextureUniformParameters(GraphBuilder, nullptr, Renderer->Scene->GetFeatureLevel(), ESceneTextureSetupMode::None, PassUniformParameters->SceneTextures);
	Context.PassUniformParameters = PassUniformParameters;
}

static void MaterialCacheBorderFillHardwarePages(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheSceneExtension& SceneExtension, FMaterialCacheRenderData& RenderData)
{
	const FMaterialCachePageCollection& Collection = RenderData.PageCollections[static_cast<uint32>(EMaterialCacheRenderPath::HardwareRaster)];
	if (Collection.Pages.IsEmpty())
	{
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "Border Fill Hardware Pages");
	
	FRDGUploadData<UE::HLSL::FMaterialCachePageWriteData> PageWriteDataArray(GraphBuilder, Collection.Pages.Num());
	for (int32 i = 0; i < Collection.Pages.Num(); i++)
	{
		const FMaterialCachePageInfo&  PageInfo = Collection.Pages[i];
		const FMaterialCachePageEntry& Page = RenderData.ABuffer.Pages[PageInfo.ABufferPageIndex];

		UE::HLSL::FMaterialCachePageWriteData& BinData = PageWriteDataArray[i];
		BinData.ABufferPhysicalPosition = GetMaterialCacheABufferTilePhysicalLocation(RenderData, PageInfo.ABufferPageIndex);
		BinData.VTPhysicalPosition = FUintVector2(Page.TileRect.Min.X, Page.TileRect.Min.Y);
	}

	FRDGBuffer* PageWriteData = CreateByteAddressBuffer(
		GraphBuilder,
		TEXT("MaterialCache.PageWriteData"),
		PageWriteDataArray.NumBytes(), PageWriteDataArray.GetData(),
		PageWriteDataArray.NumBytes()
	);

	const uint32 TexelThreadCountX = GetMaterialCacheTileWidth() + GetMaterialCacheTileBorderWidth() * 2;
	
	for (int32 ABufferIndex = 0; ABufferIndex < RenderData.ABuffer.ABufferTextures.Num(); ABufferIndex++)
	{
		FMaterialCacheABufferFillBorderHardwarePagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMaterialCacheABufferFillBorderHardwarePagesCS::FParameters>();
		PassParameters->PageWriteData = GraphBuilder.CreateSRV(PageWriteData);
		PassParameters->RWABuffer = GraphBuilder.CreateUAV(RenderData.ABuffer.ABufferTextures[ABufferIndex]);
		PassParameters->R8Mask = GraphBuilder.CreateSRV(RenderData.ABuffer.ABufferR8Mask);
		PassParameters->TextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->ThreadCount = TexelThreadCountX;
		PassParameters->BorderWidth = FMath::Min(GetMaterialCacheTileBorderWidth(), 4u);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("BorderFillHardwarePages"),
			PassParameters,
			ERDGPassFlags::Compute,
			[Renderer, &Collection, PassParameters](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			FComputeShaderUtils::Dispatch(
				RHICmdList,
				Renderer->Views[0].ShaderMap->GetShader<FMaterialCacheABufferFillBorderHardwarePagesCS>(),
				*PassParameters,
				FIntVector(
					FMath::DivideAndRoundUp(PassParameters->ThreadCount, 8u),
					FMath::DivideAndRoundUp(PassParameters->ThreadCount, 8u),
					Collection.Pages.Num()
				)
			);
		});
	}
}

static void MaterialCacheFinalizePages(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheSceneExtension& SceneExtension, FMaterialCacheRenderData& RenderData)
{
	RDG_EVENT_SCOPE(GraphBuilder, "Finalize Pages");

	if (!RenderData.ABuffer.Pages.Num())
	{
		return;
	}

	FRDGUploadData<UE::HLSL::FMaterialCachePageWriteData> PageWriteDataArray(GraphBuilder, RenderData.ABuffer.Pages.Num());

	for (int32 PageIndex = 0; PageIndex < RenderData.ABuffer.Pages.Num(); PageIndex++)
	{
		const FMaterialCachePageEntry& Page = RenderData.ABuffer.Pages[PageIndex];

		UE::HLSL::FMaterialCachePageWriteData& BinData = PageWriteDataArray[PageIndex];
		BinData.ABufferPhysicalPosition = GetMaterialCacheABufferTilePhysicalLocation(RenderData, PageIndex);
		BinData.VTPhysicalPosition = FUintVector2(Page.TileRect.Min.X, Page.TileRect.Min.Y);
	}

	FRDGBuffer* PageWriteData = CreateByteAddressBuffer(
		GraphBuilder,
		TEXT("MaterialCache.PageWriteData"),
		PageWriteDataArray.NumBytes(), PageWriteDataArray.GetData(),
		PageWriteDataArray.NumBytes()
	);

	const uint32 BlockSH = 2;
	const uint32 TexelThreadCountX = GetMaterialCacheTileWidth() + GetMaterialCacheTileBorderWidth() * 2;
	const uint32 BlockThreadCountX = TexelThreadCountX >> BlockSH;

	// The ABuffer layout is generated, to avoid generating a page writers potentially per-tag we instead
	// only permute on the render and compressed formats, and invoke it for each respective ABuffer.
	for (int32 ABufferIndex = 0; ABufferIndex < RenderData.ABuffer.ABufferTextures.Num(); ABufferIndex++)
	{
		// Destination target
		IPooledRenderTarget* RenderTarget = RenderData.Bucket->PendingEntries[0].Setup.PhysicalRenderTargets[ABufferIndex];

		// Are we writing to a compressed format?
		EPixelFormat Format = RenderTarget->GetDesc().Format;
		const bool bIsCompressed = IsBlockCompressedFormat(Format);
		
		FMaterialCacheABufferWritePagesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMaterialCacheABufferWritePagesCS::FParameters>();
		PassParameters->PageWriteData = GraphBuilder.CreateSRV(PageWriteData);
		PassParameters->ABuffer = GraphBuilder.CreateSRV(RenderData.ABuffer.ABufferTextures[ABufferIndex]);
		PassParameters->TextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		PassParameters->TexelSize = FVector2f(1.0f / RenderData.ABuffer.ABufferTextures[0]->Desc.Extent.X, 1.0f / RenderData.ABuffer.ABufferTextures[0]->Desc.Extent.Y);
		PassParameters->bYCoCg = RenderData.Bucket->TagLayout.Layers[ABufferIndex].bYCoCg;
		PassParameters->bSRGB = RenderData.Bucket->TagLayout.Layers[ABufferIndex].bIsSRGB;
		PassParameters->BlockOrThreadCount = bIsCompressed ? BlockThreadCountX : TexelThreadCountX;

		// 128 wide?
		const bool bAliasTo64bit = Format == PF_DXT1 || Format == PF_BC4;
		const EPixelFormat AliasFormat = bAliasTo64bit ? PF_R32G32_UINT : PF_R32G32B32A32_UINT;
		
		// If compressed, bind the compressed aliased format, otherwise the uncompressed
		if (bIsCompressed)
		{
			if (bAliasTo64bit)
			{
				PassParameters->RWVTLayerCompressed64 = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(RenderTarget, ERDGTextureFlags::ForceImmediateFirstBarrier), ERDGUnorderedAccessViewFlags::None, AliasFormat);
			}
			else
			{
				PassParameters->RWVTLayerCompressed128 = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(RenderTarget, ERDGTextureFlags::ForceImmediateFirstBarrier), ERDGUnorderedAccessViewFlags::None, AliasFormat);
			}
		}
		else
		{
			PassParameters->RWVTLayerUncompressed = GraphBuilder.CreateUAV(GraphBuilder.RegisterExternalTexture(RenderTarget, ERDGTextureFlags::ForceImmediateFirstBarrier),  ERDGUnorderedAccessViewFlags::None, Format);
		}

		FMaterialCacheABufferWritePagesCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FMaterialCacheABufferWritePagesCS::FCompressMode>(FMaterialCacheABufferWritePagesCS::GetCompressMode(Format));
		
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("WritePages"),
			PassParameters,
			ERDGPassFlags::Compute,
			[Renderer, &RenderData, PermutationVector, PassParameters](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
		{
			FComputeShaderUtils::Dispatch(
				RHICmdList,
				Renderer->Views[0].ShaderMap->GetShader<FMaterialCacheABufferWritePagesCS>(PermutationVector),
				*PassParameters,
				FIntVector(
					FMath::DivideAndRoundUp(PassParameters->BlockOrThreadCount, 8u),
					FMath::DivideAndRoundUp(PassParameters->BlockOrThreadCount, 8u),
					RenderData.ABuffer.Pages.Num()
				)
			);
		});
	}
}

static FRDGTextureRef GetMaterialCacheABufferTexture(FMaterialCacheRenderData& RenderData, int32 Index)
{
	if (!RenderData.ABuffer.ABufferTextures.IsValidIndex(Index))
	{
		return RenderData.ABuffer.ABufferTextures[0];
	}
	
	return RenderData.ABuffer.ABufferTextures[Index];
}

static void SetMaterialCacheABufferParameters(FRDGBuilder& GraphBuilder, FMaterialCacheRenderData& RenderData, FMaterialCacheHardwareContext& HardwareContext, FMaterialCacheNaniteContext& NaniteContext, FMaterialCacheVertexInvariantContext& VertexInvariantContext)
{
	FMaterialCacheABufferParameters PassParameters;
	PassParameters.RWABuffer_0 = GraphBuilder.CreateUAV(GetMaterialCacheABufferTexture(RenderData, 0), ERDGUnorderedAccessViewFlags::SkipBarrier);
	PassParameters.RWABuffer_1 = GraphBuilder.CreateUAV(GetMaterialCacheABufferTexture(RenderData, 1), ERDGUnorderedAccessViewFlags::SkipBarrier);
	PassParameters.RWABuffer_2 = GraphBuilder.CreateUAV(GetMaterialCacheABufferTexture(RenderData, 2), ERDGUnorderedAccessViewFlags::SkipBarrier);
	PassParameters.RWABuffer_3 = GraphBuilder.CreateUAV(GetMaterialCacheABufferTexture(RenderData, 3), ERDGUnorderedAccessViewFlags::SkipBarrier);
	PassParameters.RWABuffer_4 = GraphBuilder.CreateUAV(GetMaterialCacheABufferTexture(RenderData, 4), ERDGUnorderedAccessViewFlags::SkipBarrier);
	PassParameters.RWABuffer_5 = GraphBuilder.CreateUAV(GetMaterialCacheABufferTexture(RenderData, 5), ERDGUnorderedAccessViewFlags::SkipBarrier);
	PassParameters.RWABuffer_6 = GraphBuilder.CreateUAV(GetMaterialCacheABufferTexture(RenderData, 6), ERDGUnorderedAccessViewFlags::SkipBarrier);
	PassParameters.RWABuffer_7 = GraphBuilder.CreateUAV(GetMaterialCacheABufferTexture(RenderData, 7), ERDGUnorderedAccessViewFlags::SkipBarrier);

	if (RenderData.ABuffer.ABufferR8Mask)
	{
		PassParameters.RWABuffer_R8Mask = GraphBuilder.CreateUAV(RenderData.ABuffer.ABufferR8Mask, ERDGUnorderedAccessViewFlags::SkipBarrier);
	}
	else
	{
		PassParameters.RWABuffer_R8Mask = GraphBuilder.CreateUAV(RenderData.ABuffer.ABufferTextures[0], ERDGUnorderedAccessViewFlags::SkipBarrier);
	}
	
	if (HardwareContext.PassUniformParameters)
	{
		HardwareContext.PassUniformParameters->ABuffer = PassParameters;
	}
	
	if (NaniteContext.PassUniformParameters)
	{
		NaniteContext.PassUniformParameters->ABuffer = PassParameters;
	}
	
	if (VertexInvariantContext.PassUniformParameters)
	{
		VertexInvariantContext.PassUniformParameters->ABuffer = PassParameters;
	}
}

static void MaterialCacheRenderLayers(FSceneRendererBase* Renderer, FRDGBuilder& GraphBuilder, FMaterialCacheSceneExtension& SceneExtension, FMaterialCacheRenderData& RenderData)
{
	MaterialCacheCreateABuffer(GraphBuilder, RenderData);

	// Scope for timings, composite all pages
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, MaterialCacheCompositePages, "MaterialCacheCompositePages");

		FMaterialCacheHardwareContext HardwareContext;
		MaterialCacheSetupHardwareContext(Renderer, GraphBuilder, SceneExtension, RenderData, HardwareContext);

		FMaterialCacheNaniteContext NaniteContext;
		MaterialCacheSetupNaniteContext(Renderer, GraphBuilder, SceneExtension, RenderData, NaniteContext);

		FMaterialCacheVertexInvariantContext VertexInvariantContext;
		MaterialCacheSetupVertexInvariantContext(Renderer, GraphBuilder, SceneExtension, RenderData, VertexInvariantContext);

		for (int32 LayerIndex = 0; LayerIndex < RenderData.Layers.Num(); LayerIndex++)
		{
			FMaterialCacheLayerRenderData& Layer = RenderData.Layers[LayerIndex];
			RDG_EVENT_SCOPE(GraphBuilder, "Layer %u", LayerIndex);

			// Set the ABuffer, skips barriers within a layer on RW passes
			SetMaterialCacheABufferParameters(GraphBuilder, RenderData, HardwareContext, NaniteContext, VertexInvariantContext);

			// Render all pages for this layer
			MaterialCacheRenderHardwarePages(Renderer, GraphBuilder, RenderData, Layer, HardwareContext, LayerIndex);
			MaterialCacheRenderNanitePages(Renderer, GraphBuilder, RenderData, Layer, NaniteContext, LayerIndex);
			MaterialCacheRenderVertexInvariantPages(Renderer, GraphBuilder, RenderData, Layer, VertexInvariantContext, LayerIndex);
		
			// Hardware raster pages require manual border filling
			MaterialCacheBorderFillHardwarePages(Renderer, GraphBuilder, SceneExtension, RenderData);
		}
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, MaterialCacheFinalize, "MaterialCacheFinalize");

	MaterialCacheFinalizePages(Renderer, GraphBuilder, SceneExtension, RenderData);
}

void MaterialCacheEnqueuePages(
	FRDGBuilder& GraphBuilder,
	const FMaterialCacheTagLayout& TagLayout,
	const FMaterialCacheSetup& Setup,
	const TArrayView<FMaterialCachePageEntry>& Pages
)
{
	FSceneRendererBase* Renderer = FSceneRendererBase::GetActiveInstance(GraphBuilder);
	if (!Renderer || Pages.IsEmpty())
	{
		return;
	}

	FMaterialCacheSceneExtension& SceneExtension = Renderer->Scene->GetExtension<FMaterialCacheSceneExtension>();

	// Get or create a new bucket for the tag
	FMaterialCachePendingTagBucket* Bucket = SceneExtension.TagBuckets.Find(TagLayout.Guid);
	if (!Bucket)
	{
		Bucket = &SceneExtension.TagBuckets.Add(TagLayout.Guid);
		Bucket->TagLayout = TagLayout;
	}

	// Initialize entry
	FMaterialCachePendingEntry& Entry = Bucket->PendingEntries.Emplace_GetRef();
	Entry.Setup = Setup;
	Entry.Pages.SetNumUninitialized(Pages.Num());

	// Copy over page batch
	for (int32 PageIndex = 0; PageIndex < Pages.Num(); PageIndex++)
	{
		FMaterialCachePendingPageEntry& Page = Entry.Pages[PageIndex];
		Page.Page             = Pages[PageIndex];
		Page.ABufferPageIndex = ABufferPageIndexNotProduced;
	}
}

void MaterialCacheRenderPages(FRDGBuilder& GraphBuilder)
{
	FSceneRendererBase* Renderer = FSceneRendererBase::GetActiveInstance(GraphBuilder);
	if (!Renderer)
	{
		return;
	}

	FMaterialCacheSceneExtension& SceneExtension = Renderer->Scene->GetExtension<FMaterialCacheSceneExtension>();

	// TODO: We should just have a single finalizer, which avoids all the trouble around command invalidation
	if (SceneExtension.TagBuckets.IsEmpty())
	{
		return;
	}

	// If caching is disabled, always rebuild
	if (!GMaterialCacheCommandCaching)
	{
		SceneExtension.ClearCachedPrimitiveData();
	}
	
	// Render serially over the tag set
	// Tags can realistically never be batched, given that the ABuffer layout is different
	// TODO: Is is worth it to "try" to batch for matching tag layouts, over tag guids? It would increase complexity
	for (auto&& [Guid, Bucket] : SceneExtension.TagBuckets)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "MaterialCache");
		
		for (int32_t Offset = 0; Offset < Bucket.PendingEntries.Num();)
		{
			// Create render data on the graph's lifetime
			FMaterialCacheRenderData& RenderData = *GraphBuilder.AllocObject<FMaterialCacheRenderData>();
			RenderData.Bucket = &Bucket;

			// First, allocate and batch all pages
			Offset = MaterialCacheAllocateAndBatchPages(Renderer, GraphBuilder, SceneExtension, RenderData, Offset);

			// Then, render them with their allotted layers
			if (!RenderData.ABuffer.Pages.IsEmpty())
			{
				MaterialCacheRenderLayers(Renderer, GraphBuilder, SceneExtension, RenderData);
			}
		}

	}

	SceneExtension.TagBuckets.Empty();
}
