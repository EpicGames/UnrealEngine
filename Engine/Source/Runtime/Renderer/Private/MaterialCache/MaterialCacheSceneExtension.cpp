// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheSceneExtension.h"
#include "MaterialCache/MaterialCache.h"
#include "GlobalRenderResources.h"
#include "MaterialCache/MaterialCachePrimitiveData.h"
#include "MaterialCache/MaterialCacheTagProvider.h"
#include "MaterialCache/MaterialCacheVirtualTextureRenderProxy.h"
#include "MaterialCache/MaterialCacheVirtualTextureTag.h"
#include "ShaderParameterMacros.h"
#include "MaterialCacheDefinitions.h"
#include "ScenePrivate.h"
#include "MaterialCache/MaterialCacheVirtualTextureDescriptor.h"
#include "MaterialCache/MaterialCacheDescriptor.h"

IMPLEMENT_SCENE_EXTENSION(FMaterialCacheSceneExtension);

struct FMaterialCacheSceneExtensionData
{
	~FMaterialCacheSceneExtensionData()
	{
		checkf(SceneDataMap.IsEmpty(), TEXT("Released scene extension data with dangling references"));
	}
	
	FCriticalSection CriticalSection;

	/** Shared primitive data map */
	TMap<FPrimitiveComponentId, FMaterialCachePrimitiveData> SceneDataMap;
	
	/** Shared allocation data map */
	TMap<FMaterialCacheVirtualTextureAllocation*, FMaterialCacheAllocationSceneData> AllocationSceneDataMap;
};

class FMaterialCacheSceneExtensionUpdater : public ISceneExtensionUpdater
{
	DECLARE_SCENE_EXTENSION_UPDATER(FRenderer, FMaterialCacheSceneExtension);

public:
	FMaterialCacheSceneExtensionUpdater(FScene& InScene, FMaterialCacheSceneExtensionData& Data) : Scene(InScene), Data(Data)
	{
		
	}
	
	virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override
	{
		FMaterialCacheTagProvider& TagProvider = FMaterialCacheTagProvider::Get();

		// Process all remoted primitives
		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.RemovedPrimitiveSceneInfos)
		{
			FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;
			if (!Proxy || !Proxy->SupportsMaterialCache())
			{
				continue;
			}
			
			// Remove primitive associations
			for (FMaterialCacheVirtualTextureRenderProxy* MaterialCacheProxy : Proxy->MaterialCacheRenderProxies)
			{
				if (ensure(MaterialCacheProxy))
				{
					// Remove from tag provider
					TagProvider.Unregister(
						Scene.GetRenderScene(),
						PrimitiveSceneInfo->GetSceneData()->PrimitiveSceneId, 
						MaterialCacheProxy->Allocation
					);
					
					FMaterialCacheAllocationSceneData& SceneData = Data.AllocationSceneDataMap.FindOrAdd(MaterialCacheProxy->Allocation);
					SceneData.Primitives.Remove(PrimitiveSceneInfo->GetSceneData()->PrimitiveSceneId);
				}
			}

			// Free the primitive tag offset
			TagProvider.FreePrimitiveTagOffset(Proxy->MaterialCacheDescriptor.TagOffset);
			Proxy->MaterialCacheDescriptor.TagOffset = UINT16_MAX;

			// Remove from tracked primitives
			Data.SceneDataMap.Remove(Proxy->GetPrimitiveComponentId());
		}
	}

	virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override
	{
		FMaterialCacheTagProvider& TagProvider = FMaterialCacheTagProvider::Get();

		// Process all added primitives
		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : ChangeSet.AddedPrimitiveSceneInfos)
		{
			FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;
			if (!Proxy || !Proxy->SupportsMaterialCache())
			{
				continue;
			}

			// Allocate the primitive tag offset
			checkf(Proxy->MaterialCacheDescriptor.TagOffset == UINT16_MAX, TEXT("Primitive double-registration"));
			Proxy->MaterialCacheDescriptor.TagOffset = TagProvider.AllocatePrimitiveTagOffset();

			// Shouldn't be tracking this
			checkf(!Data.SceneDataMap.Contains(Proxy->GetPrimitiveComponentId()), TEXT("Dangling primitive scene data"));

			// Associate proxy to CID
			FMaterialCachePrimitiveData& PrimitiveData = Data.SceneDataMap.Add(Proxy->GetPrimitiveComponentId());
			PrimitiveData.Proxy = Proxy;

			// Register all tag entries for the primitive
			for (FMaterialCacheVirtualTextureRenderProxy* MaterialCacheProxy : Proxy->MaterialCacheRenderProxies)
			{
				if (ensure(MaterialCacheProxy))
				{
					FMaterialCacheAllocationSceneData& SceneData = Data.AllocationSceneDataMap.FindOrAdd(MaterialCacheProxy->Allocation);
					SceneData.Primitives.Add(PrimitiveSceneInfo->GetSceneData()->PrimitiveSceneId);
					
					// TODO: We do need to flush on new primitive associations, but, could this be done better?
					MaterialCacheProxy->Flush(&Scene);

					// Register against tag provider
					TagProvider.Register(
						Scene.GetRenderScene(),
						PrimitiveSceneInfo->GetSceneData()->PrimitiveSceneId, 
						MaterialCacheProxy->Allocation
					);
					
					UE::HLSL::FMaterialCacheTagEntry Entry;
					Entry.PackedUniform = PackMaterialCacheTextureDescriptor(MaterialCacheProxy->Allocation, MaterialCacheProxy->UVCoordinateIndex);
					TagProvider.SetTagEntry(Proxy->MaterialCacheDescriptor.TagOffset, MaterialCacheProxy->Allocation->Description.TagLayout.Guid, Entry);
				}
			}
		}
	}

private:
	FScene& Scene;
	
	FMaterialCacheSceneExtensionData& Data;
};

FMaterialCacheSceneExtension::FMaterialCacheSceneExtension(FScene& InScene) : ISceneExtension(InScene)
{
	Data = MakeUnique<FMaterialCacheSceneExtensionData>();
}

bool FMaterialCacheSceneExtension::ShouldCreateExtension(FScene& Scene)
{
	return IsMaterialCacheEnabled(Scene.GetShaderPlatform());
}

ISceneExtensionUpdater* FMaterialCacheSceneExtension::CreateUpdater()
{
	return new FMaterialCacheSceneExtensionUpdater(Scene, *Data);
}

FMaterialCachePrimitiveData* FMaterialCacheSceneExtension::GetPrimitiveData(FPrimitiveComponentId PrimitiveComponentId) const
{
	// Multi-consumer is fine
	check(IsInParallelRenderingThread());
	return Data->SceneDataMap.Find(PrimitiveComponentId);
}

FMaterialCacheAllocationSceneData* FMaterialCacheSceneExtension::GetAllocationSceneData(FMaterialCacheVirtualTextureAllocation* Allocation) const
{
	// Multi-consumer is fine
	check(IsInParallelRenderingThread());
	return Data->AllocationSceneDataMap.Find(Allocation);
}

void FMaterialCacheSceneExtension::UpdateTagUniforms(FPrimitiveComponentId PrimitiveComponentId)
{
	check(IsInRenderingThread());
	
	FMaterialCacheTagProvider& TagProvider = FMaterialCacheTagProvider::Get();
	
	if (FMaterialCachePrimitiveData* It = Data->SceneDataMap.Find(PrimitiveComponentId))
	{
		// Update all tag entries for the primitive
		for (FMaterialCacheVirtualTextureRenderProxy* MaterialCacheProxy : It->Proxy->MaterialCacheRenderProxies)
		{
			if (ensure(MaterialCacheProxy))
			{
				UE::HLSL::FMaterialCacheTagEntry Entry;
				Entry.PackedUniform = PackMaterialCacheTextureDescriptor(MaterialCacheProxy->Allocation, MaterialCacheProxy->UVCoordinateIndex);
				TagProvider.SetTagEntry(It->Proxy->MaterialCacheDescriptor.TagOffset, MaterialCacheProxy->Allocation->Description.TagLayout.Guid, Entry);
			}
		}
	}
}

void FMaterialCacheSceneExtension::UpdateTagUniforms(FMaterialCacheVirtualTextureAllocation* Allocation)
{
	check(IsInRenderingThread());
	
	if (FMaterialCacheAllocationSceneData* It = Data->AllocationSceneDataMap.Find(Allocation))
	{
		for (FPrimitiveComponentId Primitive : It->Primitives)
		{
			UpdateTagUniforms(Primitive);
		}
	}
}

void FMaterialCacheSceneExtension::Register(FMaterialCacheVirtualTextureAllocation* Allocation)
{
	check(IsInRenderingThread());
	check(!Data->AllocationSceneDataMap.Contains(Allocation));
	Data->AllocationSceneDataMap.Add(Allocation);
	
	FMaterialCacheTagProvider& TagProvider = FMaterialCacheTagProvider::Get();
	TagProvider.Register(Allocation);
}

void FMaterialCacheSceneExtension::Unregister(FMaterialCacheVirtualTextureAllocation* Allocation)
{
	check(IsInRenderingThread());
	
	auto AllocationIt = Data->AllocationSceneDataMap.Find(Allocation);
	if (!AllocationIt)
	{
		check(false);
		return;
	}
	
	// Cleanup pending proxy references
	for (FPrimitiveComponentId Primitive : AllocationIt->Primitives)
	{
		if (auto PrimitiveIt = Data->SceneDataMap.Find(Primitive))
		{
			for (FMaterialCacheVirtualTextureRenderProxy* MaterialCacheRenderProxy : PrimitiveIt->Proxy->MaterialCacheRenderProxies)
			{
				delete MaterialCacheRenderProxy;
			}
			
			PrimitiveIt->Proxy->MaterialCacheRenderProxies.Empty();
		}
	}
	
	Data->AllocationSceneDataMap.Remove(Allocation);
	
	FMaterialCacheTagProvider& TagProvider = FMaterialCacheTagProvider::Get();
	TagProvider.Unregister(Allocation);
}

void FMaterialCacheSceneExtension::ClearCachedPrimitiveData()
{
	// Remove all cached commands for all tags
	for (auto&& [_, PrimitiveData] : Data->SceneDataMap)
	{
		PrimitiveData.CachedCommands.Tags.Empty();
	}
}
