// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheShaders.h"
#include "MaterialCache/MaterialCache.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderCompilerCore.h"

using FMaterialCacheUnwrapVS0 = FMaterialCacheUnwrapVS<false>;
using FMaterialCacheUnwrapVS1 = FMaterialCacheUnwrapVS<true>;

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FMaterialCacheUnwrapVS0, TEXT("/Engine/Private/MaterialCache/MaterialCacheUnwrapVertexShader.usf"), TEXT("Main"), SF_Vertex);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FMaterialCacheUnwrapVS1, TEXT("/Engine/Private/MaterialCache/MaterialCacheUnwrapVertexShader.usf"), TEXT("Main"), SF_Vertex);

IMPLEMENT_MATERIAL_SHADER_TYPE(, FMaterialCacheUnwrapPS,      TEXT("/Engine/Private/MaterialCache/MaterialCacheUnwrapPixelShader.usf"),  TEXT("Main"), SF_Pixel);
IMPLEMENT_MATERIAL_SHADER_TYPE(, FMaterialCacheNaniteShadeCS, TEXT("/Engine/Private/MaterialCache/MaterialCacheUnwrapNaniteShade.usf"),  TEXT("Main"), SF_Compute);
IMPLEMENT_MATERIAL_SHADER_TYPE(, FMaterialCacheShadeCS,       TEXT("/Engine/Private/MaterialCache/MaterialCacheShade.usf"),              TEXT("Main"), SF_Compute);

IMPLEMENT_GLOBAL_SHADER(FMaterialCacheABufferFillBorderHardwarePagesCS, "/Engine/Private/MaterialCache/MaterialCacheABufferBorderFillHardwarePages.usf", "BorderFillHardwarePagesMain", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FMaterialCacheABufferWritePagesCS, "/Engine/Private/MaterialCache/MaterialCacheABufferPages.usf", "WritePagesMain", SF_Compute);

template<bool bSupportsViewportFromVS>
bool FMaterialCacheUnwrapVS<bSupportsViewportFromVS>::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	const bool bHasMaterialCacheOutput = Parameters.MaterialParameters.bHasMaterialCacheOutput || Parameters.MaterialParameters.bIsDefaultMaterial;
	const bool bIsValidTag            = !Parameters.PermutationId || Parameters.PermutationId < Parameters.MaterialParameters.NumMaterialCacheTags;
	return IsMaterialCacheSupported(Parameters.Platform) && bIsValidTag && bHasMaterialCacheOutput;
}

template<bool bSupportsViewportFromVS>
void FMaterialCacheUnwrapVS<bSupportsViewportFromVS>::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("MATERIAL_CACHE"), 1);
	OutEnvironment.SetDefine(TEXT("SUPPORTS_VIEWPORT_FROM_VS"), bSupportsViewportFromVS);
	OutEnvironment.SetDefine(TEXT("MATERIAL_CACHE_TAG_INDEX"), Parameters.PermutationId);

	// TODO[MP]: Add permutation for lack of support
	OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
}

bool FMaterialCacheUnwrapPS::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	const bool bHasMaterialCacheOutput = Parameters.MaterialParameters.bHasMaterialCacheOutput || Parameters.MaterialParameters.bIsDefaultMaterial;
	const bool bIsValidTag            = !Parameters.PermutationId || Parameters.PermutationId < Parameters.MaterialParameters.NumMaterialCacheTags;
	return IsMaterialCacheSupported(Parameters.Platform) && bIsValidTag && bHasMaterialCacheOutput;
}

void FMaterialCacheUnwrapPS::ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
	OutEnvironment.SetDefine(TEXT("SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL"), 1);
	OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1);
	OutEnvironment.SetDefine(TEXT("MATERIAL_CACHE"), 1);
	OutEnvironment.SetDefine(TEXT("MATERIAL_CACHE_TAG_INDEX"), Parameters.PermutationId);
}

bool FMaterialCacheNaniteShadeCS::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	const bool bHasMaterialCacheOutput = Parameters.MaterialParameters.bHasMaterialCacheOutput || Parameters.MaterialParameters.bIsDefaultMaterial;
	const bool bIsValidTag            = !Parameters.PermutationId || Parameters.PermutationId < Parameters.MaterialParameters.NumMaterialCacheTags;

	return
		IsMaterialCacheSupported(Parameters.Platform) &&
		Parameters.VertexFactoryType->SupportsNaniteRendering() &&
		Parameters.VertexFactoryType->SupportsComputeShading() &&
		bIsValidTag &&
		bHasMaterialCacheOutput;
}

FMaterialCacheNaniteShadeCS::FMaterialCacheNaniteShadeCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer): FMeshMaterialShader(Initializer)
{
	PassDataParam.Bind(Initializer.ParameterMap, TEXT("PassData"));
	PageIndirectionsParam.Bind(Initializer.ParameterMap, TEXT("PageIndirections"));
}

void FMaterialCacheNaniteShadeCS::ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
	OutEnvironment.SetDefine(TEXT("SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL"), 1);
	OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1);
	OutEnvironment.SetDefine(TEXT("MATERIAL_CACHE"), 1);
	OutEnvironment.SetDefine(TEXT("MATERIAL_CACHE_TAG_INDEX"), Parameters.PermutationId);
	
	// Force shader model 6.0+
	OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
	OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
	OutEnvironment.CompilerFlags.Add(CFLAG_CheckForDerivativeOps);
}

void FMaterialCacheNaniteShadeCS::SetPassParameters(FRHIBatchedShaderParameters& BatchedParameters, const FUintVector4& PassData, FRHIShaderResourceView* PageIndirections)
{
	SetShaderValue(BatchedParameters, PassDataParam, PassData);
	SetSRVParameter(BatchedParameters, PageIndirectionsParam, PageIndirections);
}

bool FMaterialCacheShadeCS::ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	const bool bHasMaterialCacheOutput = Parameters.MaterialParameters.bHasMaterialCacheOutput || Parameters.MaterialParameters.bIsDefaultMaterial;
	const bool bIsValidTag            = !Parameters.PermutationId || Parameters.PermutationId < Parameters.MaterialParameters.NumMaterialCacheTags;

	return
		IsMaterialCacheSupported(Parameters.Platform) &&
		Parameters.VertexFactoryType->SupportsComputeShading() &&
		bIsValidTag &&
		bHasMaterialCacheOutput;
}

FMaterialCacheShadeCS::FMaterialCacheShadeCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer): FMeshMaterialShader(Initializer)
{
	PassDataParam.Bind(Initializer.ParameterMap, TEXT("PassData"));
	PageIndirectionsParam.Bind(Initializer.ParameterMap, TEXT("PageIndirections"));
}

void FMaterialCacheShadeCS::ModifyCompilationEnvironment(const FMeshMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
	OutEnvironment.SetDefine(TEXT("SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL"), 1);
	OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1);
	OutEnvironment.SetDefine(TEXT("USE_ANALYTIC_DERIVATIVES"), 1);
	OutEnvironment.SetDefine(TEXT("MATERIAL_CACHE"), 1);

	// Force shader model 6.0+
	OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
	OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
	OutEnvironment.CompilerFlags.Add(CFLAG_CheckForDerivativeOps);
}

void FMaterialCacheShadeCS::SetPassParameters(FRHIBatchedShaderParameters& BatchedParameters, const FUintVector4& PassData, FRHIShaderResourceView* PageIndirections)
{
	SetShaderValue(BatchedParameters, PassDataParam, PassData);
	SetSRVParameter(BatchedParameters, PageIndirectionsParam, PageIndirections);
}

bool FMaterialCacheABufferFillBorderHardwarePagesCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsMaterialCacheSupported(Parameters.Platform);
}

void FMaterialCacheABufferFillBorderHardwarePagesCS::ModifyCompilationEnvironment(const FShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
}

int32 FMaterialCacheABufferWritePagesCS::GetCompressMode(EPixelFormat Format)
{
	switch (Format)
	{
	default:
		return 0;
	case PF_DXT1:
		return 1;
	case PF_DXT5:
		return 2;
	case PF_BC4:
		return 3;
	case PF_BC5:
		return 4;
	case PF_BC6H:
		return 5;
	case PF_BC7:
		return 6;
	}
}

bool FMaterialCacheABufferWritePagesCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsMaterialCacheSupported(Parameters.Platform);
}

void FMaterialCacheABufferWritePagesCS::ModifyCompilationEnvironment(const FShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("BLOCK_COMPRESS_SRC_TEXTURE_ARRAY"), 1);
	
	OutEnvironment.SetDefine(TEXT("BC_NONE"), 0);
	OutEnvironment.SetDefine(TEXT("BC1"),     1);
	OutEnvironment.SetDefine(TEXT("BC3"),     2);
	OutEnvironment.SetDefine(TEXT("BC4"),     3);
	OutEnvironment.SetDefine(TEXT("BC5"),     4);
	OutEnvironment.SetDefine(TEXT("BC6"),     5);
	OutEnvironment.SetDefine(TEXT("BC7"),     6);
	
	OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
}

/** Instantiations **/

template class FMaterialCacheUnwrapVS<false>;
template class FMaterialCacheUnwrapVS<true>;
