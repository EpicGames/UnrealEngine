// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheTagProvider.h"
#include "EngineModule.h"
#include "RendererPrivateUtils.h"
#include "MaterialCache/MaterialCacheAttribute.h"
#include "MaterialCacheDefinitions.h"
#include "ScenePrivate.h"
#include "MaterialCache/IMaterialCacheVirtualTextureAllocation.h"
#include "MaterialCache/MaterialCachePrimitiveData.h"
#include "MaterialCache/MaterialCacheVirtualProducer.h"

static FMaterialCacheTagProvider* GMaterialCacheTagProvider = nullptr;

struct FMaterialCacheTagBucket
{
	~FMaterialCacheTagBucket()
	{
		checkf(VirtualTextures.IsEmpty(), TEXT("Released scene extension data with dangling references"));
	}

	/** Layout of this tag bucket */
	FMaterialCacheTagLayout Layout;

	/** All virtual textures registered to this tag */
	TArray<FMaterialCacheVirtualTextureAllocation*> VirtualTextures;

	/** All host-side tag entries */
	TArray<UE::HLSL::FMaterialCacheTagEntry> TagEntries;

	/** GPU tag state */
	FBufferRHIRef             EntryBuffer;
	FRHIShaderResourceViewRef EntryBufferSRV;

	/** Does this bucket require an update? */
	bool bIsDirty = false;
};

void FMaterialCacheTagProvider::Initialize()
{
	if (!GMaterialCacheTagProvider)
	{
		GMaterialCacheTagProvider = new FMaterialCacheTagProvider();
	}
}

void FMaterialCacheTagProvider::Shutdown()
{
	if (GMaterialCacheTagProvider)
	{
		delete GMaterialCacheTagProvider;
		GMaterialCacheTagProvider = nullptr;
	}
}

FMaterialCacheTagProvider& FMaterialCacheTagProvider::Get()
{
	check(GMaterialCacheTagProvider);
	return *GMaterialCacheTagProvider;
}

void FMaterialCacheTagProvider::CreateDeviceBuffersOrResize(struct FMaterialCacheTagBucket& Bucket)
{
	const uint64 MinBufferSize = sizeof(UE::HLSL::FMaterialCacheTagEntry) * Bucket.TagEntries.Num();

	// Can we accomodate for the current entry count?
	if (Bucket.EntryBuffer.IsValid() && Bucket.EntryBuffer->GetDesc().Size >= MinBufferSize)
	{
		return;
	}

	// Out of entries, conservative estimation
	const uint32 ElementCount = FMath::Max(512u, static_cast<uint32>(Bucket.TagEntries.Num() * 1.5f));

	// Allocate backing buffer
	Bucket.EntryBuffer = FRHICommandListImmediate::Get().CreateBuffer(
		FRHIBufferCreateDesc::CreateStructured<UE::HLSL::FMaterialCacheTagEntry>(TEXT("MaterialCache.EntryBuffer"), ElementCount)
			.AddUsage(BUF_UnorderedAccess | BUF_ShaderResource)
			.DetermineInitialState()
	);

	// Structured view
	Bucket.EntryBufferSRV = FRHICommandListImmediate::Get().CreateShaderResourceView(Bucket.EntryBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(Bucket.EntryBuffer));
	
	// Inform all listeners that the bindings have changed
	NotifyTagSceneInvalidation(Bucket.Layout.Guid);
}

void FMaterialCacheTagProvider::NotifyTagSceneInvalidation(const FGuid& TagGuid)
{
	PendingTagSceneInvalidations.Add(TagGuid);
}

void FMaterialCacheTagProvider::Update(class FRDGBuilder& GraphBuilder)
{
	FScopeLock Lock(&Mutex);
	
	for (auto&& [_, Bucket] : TagBuckets)
	{
		// Has any relevant updates?
		if (!Bucket->TagEntries.Num() || !Bucket->bIsDirty)
		{
			continue;
		}

		/**
		 * Note: We're just updating the full tag buffer for now, this can easily be a scatter upload
		 * But let's keep it simple for now to get things going.
		 */

		// Host staging buffer setup
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured<UE::HLSL::FMaterialCacheTagEntry>(TEXT("MaterialCache::HostTagBuffer"), Bucket->TagEntries.Num())
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::CopySrc)
			.SetInitActionInitializer();

		// Copy over the host data to a staging area
		TRHIBufferInitializer<UE::HLSL::FMaterialCacheTagEntry> Initializer = GraphBuilder.RHICmdList.CreateBufferInitializer(CreateDesc);
		Initializer.WriteArray(MakeConstArrayView(Bucket->TagEntries));
		FBufferRHIRef StagingBufferRHI = Initializer.Finalize();

		// Copy to device
		GraphBuilder.RHICmdList.CopyBufferRegion(
			Bucket->EntryBuffer, 0,
			StagingBufferRHI, 0,
			sizeof(UE::HLSL::FMaterialCacheTagEntry) * Bucket->TagEntries.Num()
		);

		Bucket->bIsDirty = false;
	}
}

void FMaterialCacheTagProvider::CallPendingCallbacks()
{
	FScopeLock Lock(&Mutex);

	if (!PendingTagSceneInvalidations.Num())
	{
		return;
	}

	// We need to collect and invoke separately as the invalidation callbacks may re-subscribe to the tags
	TArray<FTagSceneListener, SceneRenderingAllocator> PendingCallbacks;

	// First, collect all the pending callbacks
	for (const FGuid& TagGuid : PendingTagSceneInvalidations)
	{
		for (decltype(TagCallbackIndices)::TKeyIterator It = TagCallbackIndices.CreateKeyIterator(TagGuid); It; ++It)
		{
			PendingCallbacks.Add(TagSceneInvalidationListeners[It->Value]);
		}
	}

	// Cleanup
	PendingTagSceneInvalidations.Empty();
	
	// Invoke all pending scene invalidations
	for (const FTagSceneListener& Listener : PendingCallbacks)
	{
		Listener.Delegate(Listener.Baton);
	}
}

void FMaterialCacheTagProvider::NotifyAllTagSceneInvalidation()
{
	for (auto&& [Key, _] : TagBuckets)
	{
		NotifyTagSceneInvalidation(Key);
	}
}

uint32 FMaterialCacheTagProvider::AllocatePrimitiveTagOffset()
{
	FScopeLock Lock(&Mutex);
	
	if (!FreeTagIndices.IsEmpty())
	{
		return FreeTagIndices.Pop();
	}
	else
	{
		return TagOffsetAllocator++;
	}
}

void FMaterialCacheTagProvider::SetTagEntry(uint32 TagOffset, const FGuid& TagGuid, const UE::HLSL::FMaterialCacheTagEntry& Entry)
{
	FScopeLock Lock(&Mutex);
	
	// Bucket must exist at this point
	FMaterialCacheTagBucket** It = TagBuckets.Find(TagGuid);
	if (!It)
	{
		checkf(false, TEXT("Registering tag entries without a bucket"));
		return;
	}
	
	FMaterialCacheTagBucket* Bucket = *It;

	// Grow the host backing if needed
	if (Bucket->TagEntries.Num() <= static_cast<int32>(TagOffset))
	{
		Bucket->TagEntries.SetNum(TagOffset + 1);
	}

	Bucket->TagEntries[TagOffset] = Entry;
	Bucket->bIsDirty = true;

	// Resize the device buffers if needed
	CreateDeviceBuffersOrResize(*Bucket);
}

void FMaterialCacheTagProvider::FreePrimitiveTagOffset(uint32 TagOffset)
{
	FScopeLock Lock(&Mutex);
	
	// We do not need to dirty the tag buffers
	check(TagOffset != UINT32_MAX);
	FreeTagIndices.Add(TagOffset);
}

static FMaterialCacheVirtualTextureAllocation* GetFirstAllocation(FMaterialCacheTagBucket* Bucket)
{
	/**
	 * TODO: This is quite horrible, across both the allocator and related it makes more sense
	 * to take ownership of a particular space and associated physical textures so we don't have to 
	 * constantly query this, or track it elsewhere.
	 */
	
	for (FMaterialCacheVirtualTextureAllocation* Allocation : Bucket->VirtualTextures)
	{
		if (Allocation->VirtualTexture)
		{
			return Allocation;
		}
	}

	return nullptr;
}

FMaterialCacheTagBindingData FMaterialCacheTagProvider::GetBindingData(const FGuid& Guid)
{
	FScopeLock Lock(&Mutex);
	
	FMaterialCacheTagBindingData Out;
	Out.TagBufferSRV = GEmptyStructuredBufferWithUAV->ShaderResourceViewRHI;
	Out.PageTableSRV = GBlackUintTexture->TextureRHI;

	// If there's no bucket, assign dummy values
	FMaterialCacheTagBucket** It = TagBuckets.Find(Guid);
	if (!It || (*It)->VirtualTextures.IsEmpty())
	{
		Out.PhysicalTextureSRVs.Add(GBlackTextureWithSRV->ShaderResourceViewRHI);
		return Out;
	}

	// All virtual textures in the bucket share the same page table
	FMaterialCacheVirtualTextureAllocation* ReferenceTexture = GetFirstAllocation(*It);
	if (!ReferenceTexture)
	{
		Out.PhysicalTextureSRVs.Add(GBlackTextureWithSRV->ShaderResourceViewRHI);
		return Out;
	}
	
	FMaterialCacheTagBucket* Bucket = *It;
	Out.TagBufferSRV = Bucket->EntryBufferSRV;
	check(Out.TagBufferSRV);
	
	Out.PageTableSRV = ReferenceTexture->VirtualTexture->GetPageTableTexture(0);
	check(Out.PageTableSRV);

	// And all physical textures
	for (uint32 i = 0; i < ReferenceTexture->VirtualTexture->GetNumTextureLayers(); i++)
	{
		Out.PhysicalTextureSRVs.Add(ReferenceTexture->VirtualTexture->GetPhysicalTextureSRV(i, Bucket->Layout.Layers[i].bIsSRGB));
	}

	return Out;
}

FMaterialCacheTagUniformData FMaterialCacheTagProvider::GetUniformData(const FGuid& Guid)
{
	FScopeLock Lock(&Mutex);
	
	FMaterialCacheTagUniformData Out;
	Out.PackedTableUniform = FUintVector4::ZeroValue;
	
	// If there's no bucket, assign dummy values
	FMaterialCacheTagBucket** It = TagBuckets.Find(Guid);
	if (!It || (*It)->VirtualTextures.IsEmpty())
	{
		return Out;
	}

	FMaterialCacheTagBucket* Bucket = *It;

	// All virtual textures in the bucket share the same physical uniforms
	FMaterialCacheVirtualTextureAllocation* Allocation = GetFirstAllocation(Bucket);
	if (!Allocation)
	{
		return Out;
	}
	
	IAllocatedVirtualTexture* ReferenceTexture = Allocation->VirtualTexture;
	
	const uint32 PageSize               = ReferenceTexture->GetVirtualTileSize();
	const uint32 PageBorderSize         = ReferenceTexture->GetTileBorderSize();
	const float  RcpPhysicalTextureSize = 1.0f / static_cast<float>(ReferenceTexture->GetPhysicalTextureSize(0));
	const uint32 PageSizeWithBorder     = PageSize + PageBorderSize * 2u;

	Out.PackedTableUniform.X  = 0xFFFFFFFF;
	Out.PackedTableUniform.Y  = FMath::AsUInt(static_cast<float>(PageSize) * RcpPhysicalTextureSize);
	Out.PackedTableUniform.Z  = FMath::AsUInt(static_cast<float>(PageBorderSize) * RcpPhysicalTextureSize);
	Out.PackedTableUniform.W  = FMath::AsUInt(static_cast<float>(PageSizeWithBorder) * RcpPhysicalTextureSize);
	
	return Out;
}

void FMaterialCacheTagProvider::Register(FSceneInterface* Scene, FPrimitiveComponentId PrimitiveComponentId, FMaterialCacheVirtualTextureAllocation* VirtualTexture)
{
	FScopeLock Lock(&Mutex);

	// Register texture
	VirtualTextures.Add({PrimitiveComponentId, VirtualTexture->Description.TagLayout.Guid}, VirtualTexture);
}

void FMaterialCacheTagProvider::Unregister(FSceneInterface* Scene, FPrimitiveComponentId PrimitiveComponentId, FMaterialCacheVirtualTextureAllocation* VirtualTexture)
{
	FScopeLock Lock(&Mutex);

	// Unregister association
	VirtualTextures.Remove({PrimitiveComponentId, VirtualTexture->Description.TagLayout.Guid });
}

void FMaterialCacheTagProvider::Register(FMaterialCacheVirtualTextureAllocation* VirtualTexture)
{
	const FMaterialCacheTagLayout& TagLayout = VirtualTexture->Description.TagLayout;
	
	// Try to find a bucket
	FMaterialCacheTagBucket* Bucket = nullptr;
	if (FMaterialCacheTagBucket** It = TagBuckets.Find(TagLayout.Guid))
	{
		Bucket = *It;
	}
	else
	{
		// New bucket, set it up
		Bucket = TagBuckets.Add(TagLayout.Guid, new FMaterialCacheTagBucket());
		Bucket->Layout = TagLayout;

		// If there's no layers, pack some dummy ones
		if (Bucket->Layout.Layers.IsEmpty())
		{
			PackMaterialCacheAttributeLayers(DefaultMaterialCacheAttributes, Bucket->Layout.Layers);
		}

		// Create the buffers
		CreateDeviceBuffersOrResize(*Bucket);
	}
	
	// Register scene texture set
	Bucket->VirtualTextures.AddUnique(VirtualTexture);
}

void FMaterialCacheTagProvider::Unregister(FMaterialCacheVirtualTextureAllocation* VirtualTexture)
{
	const FGuid& TagGuid = VirtualTexture->Description.TagLayout.Guid;
	
	// Shouldn't happen
	FMaterialCacheTagBucket** It = TagBuckets.Find(TagGuid);
	if (!It)
	{
		return;
	}
	
	// Remove texture from bucket
	FMaterialCacheTagBucket* Bucket = *It;
	uint32_t N = (*It)->VirtualTextures.Num();
	(*It)->VirtualTextures.RemoveSingle(VirtualTexture);

	// If this was the last texture, remove the bucket entirely
	if (Bucket->VirtualTextures.IsEmpty())
	{
		// Inform all listeners that the bucket is gone
		NotifyTagSceneInvalidation(TagGuid);
		
		TagBuckets.Remove(TagGuid);
		delete Bucket;
	}
}

void FMaterialCacheTagProvider::Flush(FSceneInterface* Scene, FPrimitiveComponentId PrimitiveComponentId, const FGuid& TagGuid)
{
	FScopeLock Lock(&Mutex);

	// Find the texture and flush it
	if (FMaterialCacheVirtualTextureAllocation** It = VirtualTextures.Find({PrimitiveComponentId, TagGuid}))
	{
		if ((*It)->VirtualTexture)
		{
			GetRendererModule().FlushVirtualTextureCache((*It)->VirtualTexture, FVector2f(0, 0), FVector2f(1, 1));
		}
	}
}

void FMaterialCacheTagProvider::AddTagSceneInvalidationCallback(const FGuid& TagGuid, FMaterialCacheTagProviderSceneInvalidationDelegate Delegate, void* Baton)
{
	FScopeLock Lock(&Mutex);

	// Check if already registered
	for (decltype(BatonCallbackIndices)::TKeyIterator It = BatonCallbackIndices.CreateKeyIterator(Baton); It; ++It)
	{
		if (It->Value.TagGuid == TagGuid)
		{
			return;
		}
	}
	
	FTagSceneListener Listener;
	Listener.Delegate = Delegate;
	Listener.Baton = Baton;

	// Add the listener
	int32 CallbackIndex;
	if (FreeTagSceneInvalidationCallbackIndices.IsEmpty())
	{
		CallbackIndex = TagSceneInvalidationListeners.Num();
		TagSceneInvalidationListeners.Add(Listener);
	}
	else
	{
		CallbackIndex = FreeTagSceneInvalidationCallbackIndices.Pop();
		TagSceneInvalidationListeners[CallbackIndex] = Listener;
	}

	// Add associations
	TagCallbackIndices.Add(TagGuid, CallbackIndex);
	BatonCallbackIndices.Add(Baton, FTagBatonEntry { CallbackIndex, TagGuid });
}

void FMaterialCacheTagProvider::RemoveTagSceneInvalidationCallbacks(void* Baton)
{
	FScopeLock Lock(&Mutex);
	
	// Remove all tag associations
	for (decltype(BatonCallbackIndices)::TKeyIterator It = BatonCallbackIndices.CreateKeyIterator(Baton); It; ++It)
	{
		TagCallbackIndices.RemoveSingle(It->Value.TagGuid, It->Value.CallbackIndex);

		// Mark the index as free
		FreeTagSceneInvalidationCallbackIndices.Add(It->Value.CallbackIndex);
	}

	// Remove all baton associations
	BatonCallbackIndices.Remove(Baton);
}
