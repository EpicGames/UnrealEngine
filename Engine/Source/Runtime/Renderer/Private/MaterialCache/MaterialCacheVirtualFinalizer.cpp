// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheVirtualFinalizer.h"
#include "MaterialCache/MaterialCacheRenderer.h"
#include "Components/PrimitiveComponent.h"
#include "Interfaces/ITargetPlatform.h"
#include "RendererInterface.h"
#include "RenderUtils.h"
#include "ShaderPlatformCachedIniValue.h"
#include "TextureResource.h"
#include "VirtualTexturing.h"
#include "MeshPassProcessor.inl"

FMaterialCacheVirtualFinalizer::FMaterialCacheVirtualFinalizer(FScene* InScene, FMaterialCacheVirtualTextureAllocation* Allocation, const FMaterialCacheTagLayout& TagLayout, const FVTProducerDescription& InProducerDesc)
	: Scene(InScene)
	, Allocation(Allocation)
	, TagLayout(TagLayout)
	, ProducerDesc(InProducerDesc)
{
	SourceFormat       = PF_B8G8R8A8;
	IntermediateFormat = PF_B8G8R8A8;
	DestFormat         = ProducerDesc.LayerFormat[0];
}

void FMaterialCacheVirtualFinalizer::AddTile(const FMaterialCacheTileEntry& InEntry)
{
	TilesToRender.Add(InEntry);
}

void FMaterialCacheVirtualFinalizer::RenderFinalize(FRDGBuilder& GraphBuilder, ISceneRenderer* SceneRenderingContext)
{
	TArray<FMaterialCachePageEntry, SceneRenderingAllocator> Pages;

	// Fill all tiles
	for (const FMaterialCacheTileEntry& Tile : TilesToRender)
	{
		const float X        = static_cast<float>(FMath::ReverseMortonCode2_64(Tile.Address));
		const float Y        = static_cast<float>(FMath::ReverseMortonCode2_64(Tile.Address >> 1));
		const float DivisorX = static_cast<float>(ProducerDesc.BlockWidthInTiles) / (float)(1 << Tile.Level);
		const float DivisorY = static_cast<float>(ProducerDesc.BlockHeightInTiles) / (float)(1 << Tile.Level);

		// Virtual UV range
		const FVector2f UV(X / DivisorX, Y / DivisorY);
		const FVector2f UVSize(1.0f / DivisorX, 1.0f / DivisorY);
		const FVector2f UVBorder = UVSize * ((float)ProducerDesc.TileBorderSize / (float)ProducerDesc.TileSize);
		const FBox2f    UVRect(UV - UVBorder, UV + UVSize + UVBorder);

		// Layers within the same space share the tile table, so just get the first one
		FIntVector PageLocation = Tile.TargetLayers[0].pPageLocation;

		// Physical tile location
		const int32     TileSize = ProducerDesc.TileSize + 2 * ProducerDesc.TileBorderSize;
		const FIntPoint DestinationPos(PageLocation.X * TileSize, PageLocation.Y * TileSize);
		const FIntRect  DestRect(DestinationPos, DestinationPos + FIntPoint(TileSize, TileSize));

		FMaterialCachePageEntry& Entry = Pages.Emplace_GetRef();
		Entry.TileRect = DestRect;
		Entry.UVRect = UVRect;
	}

	FMaterialCacheSetup Setup;
	Setup.Allocation = Allocation;
	Setup.TileSize   = ProducerDesc.TileSize + 2 * ProducerDesc.TileBorderSize;

	// All tiles share the same targets in a bucket
	for (int32 i = 0; i < TilesToRender[0].TargetLayers.Num(); i++)
	{
		Setup.PhysicalRenderTargets.Add(TilesToRender[0].TargetLayers[i].PooledRenderTarget);
	}
	
	MaterialCacheEnqueuePages(GraphBuilder, TagLayout, Setup, Pages);

	// Cleanup
	TilesToRender.Empty();
}

void FMaterialCacheVirtualFinalizer::Finalize(FRDGBuilder& GraphBuilder) 
{
	// TODO[MP]: Couple of things:
	//  1. Share the finalizer among all MCVT's
	//  2. Consider just having a RenderPages entry point, the enqueueing may be kind of moot at this point
	MaterialCacheRenderPages(GraphBuilder);
}
