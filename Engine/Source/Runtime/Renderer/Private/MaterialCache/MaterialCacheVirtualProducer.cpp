// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheVirtualProducer.h"
#include "ScenePrivate.h"
#include "MaterialCache/IMaterialCacheVirtualTextureAllocation.h"
#include "MaterialCache/MaterialCacheMeshProcessor.h"
#include "MaterialCache/MaterialCachePrimitiveData.h"
#include "MaterialCache/MaterialCacheSceneExtension.h"
#include "MaterialCache/MaterialCacheVirtualTextureAllocator.h"
#include "VirtualTextureDefinitions.h"

FMaterialCacheVirtualProducer::FMaterialCacheVirtualProducer(
	FScene* Scene,
	const FMaterialCacheTagLayout& TagLayout,
	const FVTProducerDescription& InProducerDesc,
	FMaterialCacheVirtualTextureAllocation* Allocation
) : Finalizer(Scene, Allocation, TagLayout, InProducerDesc)
  , Scene(Scene)
  , Allocation(Allocation)
  , ProducerDesc(InProducerDesc)
{
	
}

bool FMaterialCacheVirtualProducer::IsPageStreamed(uint8 vLevel, uint32 vAddress) const
{
	return false;
}

FVTRequestPageResult FMaterialCacheVirtualProducer::RequestPageData(FRHICommandListBase& RHICmdList,
	const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint64 vAddress, EVTRequestPagePriority Priority)
{
	// Handle all requests in the owning scene's rendering cycle
	if (!Scene->GPUScene.IsRendering())
	{
		return FVTRequestPageResult(EVTRequestPageStatus::Saturated, 0u);
	}
	
#if WITH_EDITOR
	FMaterialCacheSceneExtension* Extension = Scene->GetExtensionPtr<FMaterialCacheSceneExtension>();
	if (!Extension)
	{
		// Wait for the relevant extension to be installed
		return FVTRequestPageResult(EVTRequestPageStatus::Saturated, 0u);
	}

	// If any material is being cached, handle the request later
	// (Or if the proxy isn't ready, for any reason)
	if (FMaterialCacheAllocationSceneData* SceneData = Extension->GetAllocationSceneData(Allocation))
	{
		// Have to check all relevant primitives
		for (FPrimitiveComponentId Primitive : SceneData->Primitives)
		{
			FMaterialCachePrimitiveData* Data = Extension->GetPrimitiveData(Primitive);
			if (!Data || !IsMaterialCacheMaterialReady(Scene->GetShaderPlatform(), Data->Proxy))
			{
				// Note: Used Saturated as Pending may still be processed the same update
				return FVTRequestPageResult(EVTRequestPageStatus::Saturated, 0u);
			}
		}
	}
#endif // WITH_EDITOR
	
	// All pages are implicitly available
	return FVTRequestPageResult(EVTRequestPageStatus::Available, 0u);
}

IVirtualTextureFinalizer* FMaterialCacheVirtualProducer::ProducePageData(FRHICommandListBase& RHICmdList, ERHIFeatureLevel::Type FeatureLevel,
	EVTProducePageFlags Flags, const FVirtualTextureProducerHandle& ProducerHandle, uint8 LayerMask, uint8 vLevel, uint64 vAddress,
	uint64 RequestHandle, const FVTProduceTargetLayer* TargetLayers)
{
	FMaterialCacheTileEntry Tile;
	Tile.Address = vAddress;
	Tile.Level = vLevel;

	for (int32 LayerIndex = 0; LayerIndex < ProducerDesc.NumTextureLayers; LayerIndex++)
	{
		Tile.TargetLayers.Add(TargetLayers[LayerIndex]);
	}
		
	Finalizer.AddTile(Tile);
		
	return &Finalizer;
}
