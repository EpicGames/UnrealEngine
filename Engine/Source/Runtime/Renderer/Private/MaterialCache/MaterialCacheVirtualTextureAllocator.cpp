// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/MaterialCacheVirtualTextureAllocator.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "EngineModule.h"
#include "Engine/Engine.h"
#include "RendererPrivateUtils.h"
#include "MaterialCache/MaterialCacheAttribute.h"
#include "RendererModule.h"
#include "ScenePrivate.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Canvas.h"
#include "MaterialCache/IMaterialCacheVirtualTextureAllocation.h"
#include "MaterialCache/MaterialCacheSceneExtension.h"
#include "MaterialCache/MaterialCacheTagProvider.h"
#include "MaterialCache/MaterialCacheVirtualProducer.h"
#include "VT/AllocatedVirtualTexture.h"
#include "VT/VirtualTextureSpace.h"
#include "VT/VirtualTextureSystem.h"

static TAutoConsoleVariable CVarMCVTA_Show(
	TEXT("r.MaterialCache.VirtualTextureAllocator.Show"),
	0,
	TEXT("Show the virtual texturing allocator debug display"),
	ECVF_Default
);

static TAutoConsoleVariable CVarMCVTA_MaxReallocationsPerFrame(
	TEXT("r.MaterialCache.VirtualTextureAllocator.MaxReallocationsPerFrame"),
	4,
	TEXT("The max number of backing allocations per frame, missed allocations are deferred per feedback"),
	ECVF_Default
);

static TAutoConsoleVariable CVarMCVTA_MinEvictFrames(
	TEXT("r.MaterialCache.VirtualTextureAllocator.MinEvictFrames"),
	16,
	TEXT("The minimum number of frames an allocation stays alive for before we consider its eviction"),
	ECVF_Default
);

static TAutoConsoleVariable CVarMCVTA_EvictMipShift(
	TEXT("r.MaterialCache.VirtualTextureAllocator.EvictMipShift"),
	1,
	TEXT("The mip shift performed when an allocation is evicted"),
	ECVF_Default
);

static TAutoConsoleVariable CVarMCVTA_MaxTileMappings(
	TEXT("r.MaterialCache.VirtualTextureAllocator.MaxTileMappings"),
	1 << 23, // ~3k w/h page table
	TEXT("Maximum number of tiles allocated"),
	ECVF_Default
);

static TAutoConsoleVariable CVarMCVTA_MaxTileMappingsLRUTarget(
	TEXT("r.MaterialCache.VirtualTextureAllocator.MaxTileMappings.LRUTarget"),
	0.8f,
	TEXT("The tile mapping target, allows passive LRU to reach MaxTileMappings * LRUTarget"),
	ECVF_Default
);

static TAutoConsoleVariable CVarMCVTA_LRUMaxIterations(
	TEXT("r.MaterialCache.VirtualTextureAllocator.LRUMaxIterations"),
	10,
	TEXT("The maximum number of LRU evictions we can perform during allocations"),
	ECVF_Default
);

static TAutoConsoleVariable CVarMCVTA_MaxSpaceAllocations(
	TEXT("r.MaterialCache.VirtualTextureAllocator.MaxSpaceAllocations"),
	1 << 15,
	TEXT("The maximum number of allocations we allow within the shared space"),
	ECVF_Default
);

static FAutoConsoleCommand CCmdMCVTA_DebugRecreateCoarse(
	TEXT("r.MaterialCache.VirtualTextureAllocator.Debug.RecreateCoarse"),
	TEXT("Debug command, recreate all backing allocations with their most coarse producer"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		ENQUEUE_RENDER_COMMAND(LinkGlobalBoundShaderStateResource)([](FRHICommandList& RHICmdList)
		{
			FMaterialCacheVirtualTextureAllocator::Get().DebugRecreateCoarse(RHICmdList);
		});
	})
);

static TAutoConsoleVariable CVarMCVTA_DebugRecreateCoarseStressTestFrameInterval(
	TEXT("r.MaterialCache.VirtualTextureAllocator.Debug.RecreateCoarse.StressTestFrameInterval"),
	0,
	TEXT("Number of frames to wait before recreation, 0 is disabled"),
	ECVF_Default
);

static TAutoConsoleVariable CVarMCVTA_DebugMaxAllocatedLevel(
	TEXT("r.MaterialCache.VirtualTextureAllocator.Debug.MaxAllocatedLevel"),
	0,
	TEXT(""),
	ECVF_Default
);

static uint32 GetLRUKey(uint32 Level, uint32 Frame)
{
	uint32 Key = 0;
	Key |= Level;
	Key |= Frame << 4;
	return Key;
}

static uint32 GetLRUFrame(uint32 Key)
{
	return Key >> 4;
}

void FMaterialCacheVirtualTextureAllocator::Initialize()
{
	FMaterialCacheVirtualTextureAllocator& Self = Get();
	
#if !UE_BUILD_SHIPPING
	Self.DrawDelegate = UDebugDrawService::Register(
		TEXT("MaterialCacheVirtualTextureAllocator"),
		FDebugDrawDelegate::CreateRaw(&Self, &FMaterialCacheVirtualTextureAllocator::DrawDebugDisplay)
	);
#endif // !UE_BUILD_SHIPPING
}

void FMaterialCacheVirtualTextureAllocator::Shutdown()
{
	FMaterialCacheVirtualTextureAllocator& Self = Get();
	
#if !UE_BUILD_SHIPPING
	UDebugDrawService::Unregister(Self.DrawDelegate);
#endif // !UE_BUILD_SHIPPING
}

FMaterialCacheVirtualTextureAllocator& FMaterialCacheVirtualTextureAllocator::Get()
{
	static FMaterialCacheVirtualTextureAllocator Instance;
	return Instance;
}

void FMaterialCacheVirtualTextureAllocator::UpdateAllocations(FRHICommandListBase& RHICmdList, uint32 Frame)
{
	Stats.CycleStats();
	
	// Process the sampling feedback
	ProcessFeedback(RHICmdList, Frame);
	
	// Free up resources if we're nearing the mapping threshold
	ProcessPassiveLRUEvictions(RHICmdList, Frame);
	
	// Allocation attempts for prior failures
	ProcessPendingAllocations(RHICmdList);
	
#if !UE_BUILD_SHIPPING
	int32 StressInterval = CVarMCVTA_DebugRecreateCoarseStressTestFrameInterval->GetInt();
	if (StressInterval > 0 && Frame % StressInterval == 0)
	{
		DebugRecreateCoarse(RHICmdList);
	}
#endif // !UE_BUILD_SHIPPING
	
	Stats.SetStat(FMaterialCacheVirtualTextureAllocatorStats::TileMapping, TotalMappedTileCount);
}

void FMaterialCacheVirtualTextureAllocator::ProcessFeedback(FRHICommandListBase& RHICmdList, uint32 Frame)
{
	int32_t PendingAllocationsNum = CVarMCVTA_MaxReallocationsPerFrame->GetInt();
	
	auto &VirtualTextureSystem = FVirtualTextureSystem::Get();
	
	for (FAllocationEntry& Entry : AllocationEntries)
	{
		if (Entry.Allocation->VirtualTexture)
		{
			uint32 FinestMip = VirtualTextureSystem.GetFinestSampledMipInLastNFrames(
				static_cast<FAllocatedVirtualTexture*>(Entry.Allocation->VirtualTexture),
				10u
			);
		
			Entry.RequestedLevel = FMath::Min(Entry.RequestedLevel, FinestMip);
		}
		
#if !UE_BUILD_SHIPPING
		if (int32 DebugMaxLevel = CVarMCVTA_DebugMaxAllocatedLevel->GetInt(); DebugMaxLevel > 0)
		{
			Entry.RequestedLevel = FMath::Max<uint32>(
				Entry.RequestedLevel,
				Entry.Allocation->Description.ProducerDesc.MaxLevel - FMath::Min<uint32>(Entry.Allocation->Description.ProducerDesc.MaxLevel, DebugMaxLevel)
			);
		}
#endif // !UE_BUILD_SHIPPING
	
		// Update the allocation if needed
		if (!UpdateAllocation(RHICmdList, Entry, Frame))
		{
			continue;
		}
		
		// Update last used
		LRUHeap.Update(
			GetLRUKey(Entry.RequestedLevel, Frame),
			Entry.Allocation->AllocationIndex
		);
		
		Stats.AddStat(FMaterialCacheVirtualTextureAllocatorStats::Realloc);
		
		if (!--PendingAllocationsNum)
		{
			break;
		}
	}
}

void FMaterialCacheVirtualTextureAllocator::ProcessPassiveLRUEvictions(FRHICommandListBase& RHICmdList, uint32 Frame)
{
	for (int32_t i = 0; i < CVarMCVTA_LRUMaxIterations->GetInt(); i++)
	{
		// Within target?
		if (TotalMappedTileCount <= CVarMCVTA_MaxTileMappingsLRUTarget->GetFloat() * CVarMCVTA_MaxTileMappings->GetInt())
		{
			break;
		}
		
		// Just keep trying
		FreeLRU(RHICmdList, Frame);
	}
}

void FMaterialCacheVirtualTextureAllocator::ProcessPendingAllocations(FRHICommandListBase& RHICmdList)
{
	while (!PendingAllocations.IsEmpty())
	{
		FAllocationEntry& Entry = AllocationEntries[PendingAllocations.Pop()->AllocationIndex];
		
		// Try to allocate the backing memory, if failed, back on the queue it goes
		if (!AllocateBackingAllocation(RHICmdList, Entry))
		{
			// If one failed, the rest are likely to
			PendingAllocations.Add(Entry.Allocation);
			break;
		}
		
		Stats.AddStat(FMaterialCacheVirtualTextureAllocatorStats::LateAlloc);
	}
}

void FMaterialCacheVirtualTextureAllocator::DebugRecreateCoarse(FRHICommandListBase& RHICmdList)
{
	for (FAllocationEntry& Entry : AllocationEntries)
	{
		// Reset to N-1
		Entry.RequestedLevel = Entry.Allocation->Description.ProducerDesc.MaxLevel - 1;
		
		// Fully reallocate the backing memory
		DeallocateBackingAllocation(*Entry.Allocation);
		ReallocateBackingAllocation(RHICmdList, Entry, 0, false);
	}
	
	// Full scene invalidation
	FMaterialCacheTagProvider::Get().NotifyAllTagSceneInvalidation();
}

FMaterialCacheVirtualTextureAllocation* FMaterialCacheVirtualTextureAllocator::Allocate(
	FRHICommandListBase& RHICmdList,
	FSceneInterface* Scene,
	const FMaterialCacheVirtualTextureDescription& Desc
)
{
	// Must have scene, otherwise headless
	FScene* RenderScene = Scene->GetRenderScene();
	if (!RenderScene)
	{
		return nullptr;
	}

	// Local index
	int32 AllocationIndex = AllocationEntries.Num();
	
	// Add entry
	FAllocationEntry& Entry = AllocationEntries.Emplace_GetRef();
	Entry.Allocation = new FMaterialCacheVirtualTextureAllocation();
	Entry.Allocation->AllocationIndex = AllocationIndex;
	Entry.Allocation->Description = Desc;
	Entry.Scene = RenderScene;
	
	// Start at N-1
	Entry.RequestedLevel = Desc.ProducerDesc.MaxLevel - 1;
	
	// Start tracking usage
	LRUHeap.Add(0u, AllocationIndex);

	// Attempt to allocate backing memory
	// Note that this may fail
	AllocateBackingAllocation(RHICmdList, Entry);
	
	// Register to scene
	if (FMaterialCacheSceneExtension* SceneData = Entry.Scene->GetExtensionPtr<FMaterialCacheSceneExtension>())
	{
		SceneData->Register(Entry.Allocation);
	}
	
	return Entry.Allocation;
}

void FMaterialCacheVirtualTextureAllocator::Reallocate(FRHICommandListBase& RHICmdList, FMaterialCacheVirtualTextureAllocation* Allocation)
{
	FAllocationEntry& Entry = AllocationEntries[Allocation->AllocationIndex];
	
	// Reallocate without remapping
	ReallocateBackingAllocation(RHICmdList, Entry, 0, false);
}

void FMaterialCacheVirtualTextureAllocator::Deallocate(FMaterialCacheVirtualTextureAllocation* Allocation)
{
	FAllocationEntry& Entry = AllocationEntries[Allocation->AllocationIndex];
	
	// Remove from scene
	if (FMaterialCacheSceneExtension* SceneData = Entry.Scene->GetExtensionPtr<FMaterialCacheSceneExtension>())
	{
		SceneData->Unregister(Entry.Allocation);
	}
	
	// Deallocate the backing memory
	DeallocateBackingAllocation(*Entry.Allocation);
	
	// Swap with last
	if (Allocation->AllocationIndex != AllocationEntries.Num() - 1)
	{
		FAllocationEntry& Last = AllocationEntries.Last();
		Last.Allocation->AllocationIndex = Allocation->AllocationIndex;
		LRUHeap.Update(LRUHeap.GetKey(Last.Allocation->AllocationIndex), Allocation->AllocationIndex);
		AllocationEntries[Allocation->AllocationIndex] = Last;
	}
	
	// Remove in-flight pending allocations
	PendingAllocations.Remove(Allocation);
	
	// Remove LRU state
	LRUHeap.Remove(AllocationEntries.Num() - 1);

	AllocationEntries.Pop();
	delete Allocation;
}

void FMaterialCacheVirtualTextureAllocator::Flush(FMaterialCacheVirtualTextureAllocation* Allocation, const FVector2f& MinUV, const FVector2f& MaxUV)
{
	if (Allocation->VirtualTexture)
	{
		GetRendererModule().FlushVirtualTextureCache(Allocation->VirtualTexture, MinUV, MaxUV);
	}
}

bool FMaterialCacheVirtualTextureAllocator::UpdateAllocation(FRHICommandListBase& RHICmdList, FAllocationEntry& Entry, uint32 Frame)
{
	uint32 RequestedMaxLevel = FMath::Max(
		1u, 
		Entry.Allocation->Description.ProducerDesc.MaxLevel - FMath::Min<uint32>(
			Entry.Allocation->Description.ProducerDesc.MaxLevel,
			Entry.RequestedLevel
		)
	);
	
	// Can the current backing accomodate?
	if (Entry.Allocation->VirtualTexture && 
		Entry.Allocation->VirtualTexture->GetMaxLevel() >= RequestedMaxLevel)
	{
		return false;
	}
	
	uint32 RequiredTileCount = GetRequiredTileCount(Entry);
	
	// Keep freeing until we can allocate
	for (int32_t i = 0; !CanAllocate(RequiredTileCount) && i < CVarMCVTA_LRUMaxIterations->GetInt(); i++)
	{
		FreeLRU(RHICmdList, Frame);
	}
	
	// Still out of space?
	if (!CanAllocate(RequiredTileCount))
	{
		Stats.AddStat(FMaterialCacheVirtualTextureAllocatorStats::FailedAlloc);
		return false;
	}
	
	// Reallocate the backing with remapped pages
	ReallocateBackingAllocation(RHICmdList, Entry, Frame, true);
	return true;
}

void FMaterialCacheVirtualTextureAllocator::ReallocateBackingAllocation(FRHICommandListBase& RHICmdList, FAllocationEntry& Entry, uint32 Frame, bool bRemap)
{	
	// Allocate to new dimensions
	FMaterialCacheVirtualTextureAllocation OldAllocation = *Entry.Allocation;
	AllocateBackingAllocation(RHICmdList, Entry);
	
	// Copy over old page mappings if needed
	if (bRemap && OldAllocation.VirtualTexture && Entry.Allocation->VirtualTexture)
	{
		RemapBackingAllocation(OldAllocation, *Entry.Allocation, Frame);
	}
	
	// Cleanup
	DeallocateBackingAllocation(OldAllocation);
}

bool FMaterialCacheVirtualTextureAllocator::AllocateBackingAllocation(FRHICommandListBase& RHICmdList, FAllocationEntry& Entry)
{
	if (TryAllocateBackingAllocation(RHICmdList, Entry))
	{
		return true;
	}
	
	// Allocation failed, cleanup and try again later
	DeallocateBackingAllocation(*Entry.Allocation);
	PendingAllocations.Add(Entry.Allocation);
	return false;
}

bool FMaterialCacheVirtualTextureAllocator::TryAllocateBackingAllocation(FRHICommandListBase& RHICmdList, FAllocationEntry& Entry)
{
	Entry.Allocation->ResetHandles();
	
	// If we've got a designated space, make sure were within the allocation limit
	if (DesignatedSpace != VIRTUALTEXTURE_INVALID_FORCE_SPACEID)
	{
		FVirtualTextureSpace* Space = FVirtualTextureSystem::Get().GetSpace(DesignatedSpace);
		
		if (Space->GetAllocator().GetNumAllocations() >= static_cast<uint32>(CVarMCVTA_MaxSpaceAllocations->GetInt()))
		{
			Stats.AddStat(FMaterialCacheVirtualTextureAllocatorStats::FailedAllocSpace);
			return false;
		}
	}
	
	// Setup producer
	FVTProducerDescription ProducerDesc = Entry.Allocation->Description.ProducerDesc;
	ProducerDesc.bTrackSamplingHistory = true;
	
	// Allocate producer
	Entry.Allocation->Producer = new FMaterialCacheVirtualProducer(
		Entry.Scene,
		Entry.Allocation->Description.TagLayout,
		ProducerDesc,
		Entry.Allocation
	);
	
	// Register producer
	Entry.Allocation->ProducerHandle = GetRendererModule().RegisterVirtualTextureProducer(
		RHICmdList, 
		ProducerDesc,
		Entry.Allocation->Producer
	);
	
	// Setup virtual texture
	FAllocatedVTDescription VTDesc;
	VTDesc.Name = TEXT("MaterialCacheVirtualTexture");
	VTDesc.Dimensions = 2;
	VTDesc.TileSize = Entry.Allocation->Description.ProducerDesc.TileSize;
	VTDesc.TileBorderSize = Entry.Allocation->Description.ProducerDesc.TileBorderSize;
	VTDesc.NumTextureLayers = Entry.Allocation->Description.TagLayout.Layers.Num();
	VTDesc.AllocatedAdaptiveBias = Entry.RequestedLevel;
	VTDesc.bShareDuplicateLayers = true;
	
	// Force into a single space
	VTDesc.bPrivateSpace = true;
	VTDesc.ForceSpaceID = DesignatedSpace;

	// All layers share the producer, for now at least
	for (uint32 LayerIndex = 0u; LayerIndex < VTDesc.NumTextureLayers; ++LayerIndex)
	{
		VTDesc.ProducerHandle[LayerIndex]     = Entry.Allocation->ProducerHandle;
		VTDesc.ProducerLayerIndex[LayerIndex] = LayerIndex;
	}
	
	// Allocate the embodying virtual texture
	// Allocations are allowed to fail if the underlying address space is OOM
	Entry.Allocation->VirtualTexture = FVirtualTextureSystem::Get().AllocateVirtualTexture(RHICmdList, VTDesc);
	if (!Entry.Allocation->VirtualTexture)
	{
		return false;
	}
	
	// Update the scene data tag uniforms, required as the page table uniforms have changed
	if (FMaterialCacheSceneExtension* SceneData = Entry.Scene->GetExtensionPtr<FMaterialCacheSceneExtension>())
	{
		SceneData->UpdateTagUniforms(Entry.Allocation);
	}
	
	// Book-keeping
	TotalMappedTileCount += Entry.Allocation->VirtualTexture->GetBlockWidthInTiles() * Entry.Allocation->VirtualTexture->GetBlockHeightInTiles();
	LiveBackingAllocations++;

	// Make sure we're in the same space
	if (DesignatedSpace != VIRTUALTEXTURE_INVALID_FORCE_SPACEID && DesignatedSpace != Entry.Allocation->VirtualTexture->GetSpaceID())
	{
		return false;
	}

	DesignatedSpace = Entry.Allocation->VirtualTexture->GetSpaceID();
	return true;
}

void FMaterialCacheVirtualTextureAllocator::RemapBackingAllocation(
	const FMaterialCacheVirtualTextureAllocation& OldAllocation,
	const FMaterialCacheVirtualTextureAllocation& NewAllocation,
	uint32 Frame
)
{
	FVirtualTextureSystem& System = FVirtualTextureSystem::Get();
	
	// Get producer
	FVirtualTextureProducer* Producer = System.FindProducer(OldAllocation.ProducerHandle);
	
	// TODO: There's an underlying issue with single mip page remaps
	if (OldAllocation.VirtualTexture->GetMaxLevel() == 1)
	{
		return;
	}
	
	if (Producer->GetDescription().bPersistentHighestMip)
	{
		System.ForceUnlockAllTiles(OldAllocation.ProducerHandle, Producer);
	}
	
	uint32 SpaceID   = OldAllocation.VirtualTexture->GetSpaceID();
	int32  LevelBias = static_cast<int32>(NewAllocation.VirtualTexture->GetMaxLevel()) - static_cast<int32>(OldAllocation.VirtualTexture->GetMaxLevel());

	// Remap pages for all groups
	for (uint32 i = 0; i < Producer->GetNumPhysicalGroups(); i++)
	{
		FVirtualTexturePhysicalSpace* PhysicalSpace = Producer->GetPhysicalSpaceForPhysicalGroup(i);

		PhysicalSpace->GetPagePool().RemapPages(
			&System, SpaceID, PhysicalSpace,
			OldAllocation.ProducerHandle, OldAllocation.VirtualTexture->GetVirtualAddress(),
			NewAllocation.ProducerHandle, NewAllocation.VirtualTexture->GetVirtualAddress(),
			OldAllocation.VirtualTexture->GetAllocatedAdaptiveBias(),
			LevelBias, 0, Frame
		);
	}
}

uint32 FMaterialCacheVirtualTextureAllocator::GetRequiredTileCount(FAllocationEntry& Entry)
{
	uint32 BlockWidthInTiles = FMath::Max(1u, Entry.Allocation->Description.ProducerDesc.BlockWidthInTiles >> Entry.RequestedLevel);
	uint32 BlockHeightInTiles = FMath::Max(1u, Entry.Allocation->Description.ProducerDesc.BlockHeightInTiles >> Entry.RequestedLevel);
	return BlockWidthInTiles * BlockHeightInTiles;
}

bool FMaterialCacheVirtualTextureAllocator::CanAllocate(uint32 TileCount)
{
	// TODO: We could query the space allocator, but that doesn't address fragmentation wrt. address block splits
	//       The proper solution would be a safe alloc function, but until then.
	return TotalMappedTileCount + TileCount < static_cast<uint32>(CVarMCVTA_MaxTileMappings->GetInt());
}

void FMaterialCacheVirtualTextureAllocator::FreeLRU(FRHICommandListBase& RHICmdList, uint32 Frame)
{
	if (AllocationEntries.IsEmpty())
	{
		return;
	}

	uint32 TopIndex  = LRUHeap.Top();
	uint32 LastFrame = GetLRUFrame(LRUHeap.GetKey(TopIndex));
	
	// Resident for long enough?
	if (Frame - LastFrame < static_cast<uint32>(CVarMCVTA_MinEvictFrames->GetInt()))
	{
		return;
	}
	
	FAllocationEntry& Entry = AllocationEntries[TopIndex];
	
	// Eviction is just adjusting the allocated level, shift it to something appropriate
	uint32 AdjustedRequestedLevel = FMath::Min<uint32>(
		Entry.Allocation->Description.ProducerDesc.MaxLevel - 1,
		FMath::Max(1u, Entry.RequestedLevel) << CVarMCVTA_EvictMipShift->GetInt()
	);

	// Even if this wasn't evicted, mark its LRU to move on to the next one
	LRUHeap.Update(
		GetLRUKey(Entry.RequestedLevel, Frame),
		Entry.Allocation->AllocationIndex
	);
	
	if (AdjustedRequestedLevel == Entry.RequestedLevel)
	{
		return;
	}
	
	Entry.RequestedLevel = AdjustedRequestedLevel;

	// Reallocate against lower max level
	ReallocateBackingAllocation(RHICmdList, Entry, Frame, true);
	
	Stats.AddStat(FMaterialCacheVirtualTextureAllocatorStats::LRUEvict);
}

void FMaterialCacheVirtualTextureAllocator::DeallocateBackingAllocation(FMaterialCacheVirtualTextureAllocation& Allocation)
{
	// Release producer
	if (Allocation.Producer)
	{
		FVirtualTextureSystem::Get().ReleaseProducer(Allocation.ProducerHandle);
		Allocation.Producer = nullptr;
	}
	
	// Release VT
	if (Allocation.VirtualTexture)
	{
		// Book-keeping
		TotalMappedTileCount -= Allocation.VirtualTexture->GetBlockWidthInTiles() * Allocation.VirtualTexture->GetBlockHeightInTiles();
		LiveBackingAllocations--;

		checkf(LiveBackingAllocations >= 0, TEXT("Allocation double-free"));
		if (!LiveBackingAllocations)
		{
			DesignatedSpace = VIRTUALTEXTURE_INVALID_FORCE_SPACEID;
		}

		FVirtualTextureSystem::Get().DestroyVirtualTexture(Allocation.VirtualTexture);
		Allocation.VirtualTexture = nullptr;
	}
}

#if !UE_BUILD_SHIPPING
struct FDebugDisplayGraphSeries
{
	FString Name = TEXT("None");
	FLinearColor Color = FLinearColor::White;
	float* History = nullptr;
};

static void DrawDebugDisplayHistoryGraph(
	FCanvas* Canvas, FBox2D CanvasPosition,	const FString& Title,
	uint32 HistoryCount, uint32 RingIndex, float MaxY,
	std::initializer_list<FDebugDisplayGraphSeries> AllSeries
)
{
	static constexpr int32 BorderSize = 10;
	
	const FLinearColor BackgroundColor(0.0f, 0.0f, 0.0f, 0.7f);
	const FLinearColor GraphBorderColor(0.1f, 0.1f, 0.1f);
	
	FHitProxyId HitProxy = Canvas->GetHitProxyId();
	
	// Background
	FCanvasTileItem BackgroundTile(CanvasPosition.Min, CanvasPosition.GetSize(), BackgroundColor);
	BackgroundTile.BlendMode = SE_BLEND_AlphaBlend;
	Canvas->DrawItem(BackgroundTile);
	
	// Title
	Canvas->DrawShadowedString(
		CanvasPosition.Min.X + BorderSize, CanvasPosition.Min.Y,
		*Title, GEngine->GetSmallFont(), FLinearColor::White
	);

	// Names
	uint32 NameOffset = 0;
	for (const FDebugDisplayGraphSeries& Serie : AllSeries)
	{
		Canvas->DrawShadowedString(
			CanvasPosition.Min.X + NameOffset,
			CanvasPosition.Max.Y + 10,
			Serie.Name, GEngine->GetSmallFont(), 
			Serie.Color
		);
		NameOffset += 50;
	}
	
	CanvasPosition.Min += FVector2D(BorderSize, BorderSize);
	CanvasPosition.Max -= FVector2D(BorderSize, BorderSize);

	// Preallocate line set
	FBatchedElements* BatchedElements = Canvas->GetBatchedElements(FCanvas::ET_Line);
	BatchedElements->AddReserveLines(2 + HistoryCount * AllSeries.size());

	// Y-Axis
	BatchedElements->AddLine(
		FVector(CanvasPosition.Min.X - 1.0f, CanvasPosition.Max.Y, 0.0f),
		FVector(CanvasPosition.Min.X - 1.0f, CanvasPosition.Min.Y - 1.0f, 0.0f),
		GraphBorderColor,
		HitProxy
	);

	// X-Axis
	BatchedElements->AddLine(
		FVector(CanvasPosition.Min.X, CanvasPosition.Max.Y - 1.0f, 0.0f),
		FVector(CanvasPosition.Max.X, CanvasPosition.Max.Y - 1.0f, 0.0f),
		GraphBorderColor,
		HitProxy
	);

	// Automatic height?
	if (MaxY < 0.0f)
	{
		MaxY = 1.0f;
		
		for (const FDebugDisplayGraphSeries& Serie : AllSeries)
		{
			for (uint32 SampleIndex = 0; SampleIndex < HistoryCount; ++SampleIndex)
			{
				MaxY = FMath::Max(MaxY, Serie.History[SampleIndex]);
			}
		}
	}
	
	float SampleIndexFactor = (1.0f / static_cast<float>(HistoryCount)) * CanvasPosition.GetSize().X;
	float SampleValueFactor = (1.0f / MaxY) * CanvasPosition.GetSize().Y;

	// Draw lines
	for (const FDebugDisplayGraphSeries& Serie : AllSeries)
	{
		for (uint32 SampleIndex = 0; SampleIndex < HistoryCount - 1; ++SampleIndex)
		{
			float SampleAY = Serie.History[(RingIndex + SampleIndex + 0) % HistoryCount] * SampleValueFactor;
			float SampleBY = Serie.History[(RingIndex + SampleIndex + 1) % HistoryCount] * SampleValueFactor;

			BatchedElements->AddLine(
				FVector(CanvasPosition.Min.X + (SampleIndex + 0) * SampleIndexFactor, CanvasPosition.Max.Y - SampleAY, 0.0f),
				FVector(CanvasPosition.Min.X + (SampleIndex + 1) * SampleIndexFactor, CanvasPosition.Max.Y - SampleBY, 0.0f),
				Serie.Color,
				HitProxy
			);
		}
	}
}

void FMaterialCacheVirtualTextureAllocator::DrawDebugDisplay(UCanvas* Canvas, APlayerController*)
{
	constexpr uint32_t Padding    = 25;
	constexpr uint32_t GraphWidth = 275;
	
	if (!CVarMCVTA_Show->GetBool())
	{
		return;
	}
	
	FIntVector2 Offset(Padding, Padding);
	
	Canvas->Canvas->DrawShadowedString(
		Offset.X, Offset.Y,
		FString::Printf(TEXT("Alloc: %u, Live: %i"), AllocationEntries.Num(), LiveBackingAllocations),
		GEngine->GetMediumFont(),
		FLinearColor::White
	);
	
	Offset.Y += 25;
	
	DrawDebugDisplayHistoryGraph(
		Canvas->Canvas, FBox2D(FVector2D(Offset.X, Offset.Y), FVector2D(Offset.X + GraphWidth, 175 + Offset.Y)),
		TEXT("Stats"),
		FMaterialCacheVirtualTextureAllocatorStats::HistorySize, Stats.GetRingIndex(), -1.0f,
		{
			{
				TEXT("Failed"),
				FLinearColor(1, 0, 0, 1),
				Stats.GetHistoryBuffer(FMaterialCacheVirtualTextureAllocatorStats::FailedAlloc)
			},
			{
				TEXT("Failed AB"),
				FLinearColor(1, 1, 0, 1),
				Stats.GetHistoryBuffer(FMaterialCacheVirtualTextureAllocatorStats::FailedAllocSpace)
			},
			{
				TEXT("LRU"),
				FLinearColor(1, 0, 1, 1),
				Stats.GetHistoryBuffer(FMaterialCacheVirtualTextureAllocatorStats::LRUEvict)
			},
			{
				TEXT("Late"),
				FLinearColor(1, .5, 0, 1),
				Stats.GetHistoryBuffer(FMaterialCacheVirtualTextureAllocatorStats::LateAlloc)
			},
			{
				TEXT("Realloc"),
				FLinearColor(0, 0, 1, 1),
				Stats.GetHistoryBuffer(FMaterialCacheVirtualTextureAllocatorStats::Realloc)
			}
		}
	);
	
	Offset.X += Padding + GraphWidth;
	
	DrawDebugDisplayHistoryGraph(
		Canvas->Canvas, FBox2D(FVector2D(Offset.X, Offset.Y), FVector2D(Offset.X + GraphWidth, 175 + Offset.Y)),
		TEXT("Tiles"),
		FMaterialCacheVirtualTextureAllocatorStats::HistorySize, Stats.GetRingIndex(), CVarMCVTA_MaxTileMappings->GetInt(),
		{
			{
				TEXT("Total Mappings"),
				FLinearColor(1, 1, 1, 1),
				Stats.GetHistoryBuffer(FMaterialCacheVirtualTextureAllocatorStats::TileMapping)
			}
		}
	);
}
#endif
