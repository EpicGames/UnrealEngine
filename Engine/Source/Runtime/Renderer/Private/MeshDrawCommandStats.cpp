// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshDrawCommandStats.h"

#include "InstanceCulling/InstanceCullingContext.h"
#include "MeshDrawCommandStatsSettings.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RenderGraph.h"
#include "RendererOnScreenNotification.h"
#include "RHI.h"
#include "RHIGPUReadback.h"
#include "HAL/FileManager.h"

#if MESH_DRAW_COMMAND_STATS

FMeshDrawCommandStatsManager* FMeshDrawCommandStatsManager::Instance = nullptr;

void FMeshDrawCommandStatsManager::CreateInstance()
{
	check(Instance == nullptr);
	Instance = new FMeshDrawCommandStatsManager();
}

DECLARE_STATS_GROUP(TEXT("MeshDrawCommandStats"), STATGROUP_Culling, STATCAT_Advanced);

DECLARE_DWORD_COUNTER_STAT(TEXT("Total Rendered Primitives"), STAT_Culling_TotalNumPrimitives, STATGROUP_Culling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Total Rendered Vertices"), STAT_Culling_TotalNumVertices, STATGROUP_Culling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Total Rendered Instances"), STAT_Culling_TotalNumInstances, STATGROUP_Culling);
DECLARE_DWORD_COUNTER_STAT(TEXT("InstanceCulling Indirect Rendered Primitives"), STAT_Culling_InstanceCullingIndirectNumPrimitives, STATGROUP_Culling);
DECLARE_DWORD_COUNTER_STAT(TEXT("InstanceCulling Indirect Rendered Vertices"), STAT_Culling_InstanceCullingIndirectNumVertices, STATGROUP_Culling);
DECLARE_DWORD_COUNTER_STAT(TEXT("InstanceCulling Indirect Rendered Instances"), STAT_Culling_InstanceCullingIndirectNumInstances, STATGROUP_Culling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Custom Indirect Rendered Primitives"), STAT_Culling_CustomIndirectNumPrimitives, STATGROUP_Culling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Custom Indirect Rendered Vertices"), STAT_Culling_CustomIndirectNumVertices, STATGROUP_Culling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Custom Indirect Rendered Instances"), STAT_Culling_CustomIndirectNumInstances, STATGROUP_Culling);

CSV_DEFINE_CATEGORY(MeshDrawCommandStats, false);

enum class MeshDrawStatsCollection : int32
{
	None,
	Pass,
	User,
};

static TAutoConsoleVariable<int32> CVarMeshDrawCommandStats(
	TEXT("r.MeshDrawCommands.Stats"),
	0,
	TEXT("Show on screen mesh draw command stats.\n")
	TEXT("The stats for visible triangles are post GPU culling.\n")
	TEXT(" 1 = Show stats per pass.\n")
	TEXT(" 2...N = Show the collection of stats matching the 'Collection' parameter in the ini file.\n")
	TEXT("You can also use 'stat culling' to see global culling stats.\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMeshDrawCommandUseVertexStats(
	TEXT("r.MeshDrawCommands.UseVertexCount"),
	0,
	TEXT("Use vertex count instead of primitive count for budget reporting."),
	ECVF_RenderThreadSafe
);

FMeshDrawCommandStatsManager::FFrameData::~FFrameData()
{
	// Collect set of unique readback buffers for deletion (can be shared between MDCs and passes)
	TSet<FRHIGPUBufferReadback*> ReadbackBuffers(RDGIndirectArgsReadbackBuffers);
	for (FMeshDrawCommandPassStats* PassStats : PassData)
	{
		if (PassStats->InstanceCullingGPUBufferReadback)
		{
			check(ReadbackBuffers.Contains(PassStats->InstanceCullingGPUBufferReadback));
			PassStats->InstanceCullingGPUBufferReadback = nullptr;
		}
		delete PassStats;
	}
	for (auto Iter = CustomIndirectArgsBufferResults.CreateIterator(); Iter; ++Iter)
	{
		FIndirectArgsBufferResult& CustomArgsBufferResult = Iter.Value();
		if (CustomArgsBufferResult.GPUBufferReadback)
		{
			ReadbackBuffers.Add(CustomArgsBufferResult.GPUBufferReadback);
			CustomArgsBufferResult.GPUBufferReadback = nullptr;
		}
	}
	CustomIndirectArgsBufferResults.Empty();

	// delete all unique readback buffers
	for (FRHIGPUBufferReadback* ReadbackBuffer : ReadbackBuffers)
	{
		delete ReadbackBuffer;
	}
}

void FMeshDrawCommandStatsManager::FFrameData::Validate() const
{
	bool bHasIndirectArgs = false;

	// make sure that each pass which has indirect draws also has a gpu readback buffer to resolve the final used instance count
	for (const FMeshDrawCommandPassStats* PassStats : PassData)
	{
		if (PassStats->bBuildRenderingCommandsCalled)
		{
			bool bUsesInstantCullingIndirectBuffer = false;
			for (const FVisibleMeshDrawCommandStatsData& DrawData : PassStats->DrawData)
			{
				if (DrawData.UseInstantCullingIndirectBuffer > 0)
				{
					bUsesInstantCullingIndirectBuffer = true;
					bHasIndirectArgs = true;
				}

				if (DrawData.CustomIndirectArgsBuffer)
				{
					ensure(PassStats->CustomIndirectArgsBuffers.Contains(DrawData.CustomIndirectArgsBuffer));
					bHasIndirectArgs = true;
				}
			}

			// either we don't use draw indirect or we don't have a readback buffer
			check(!bUsesInstantCullingIndirectBuffer || PassStats->InstanceCullingGPUBufferReadback != nullptr);
		}
	}

	// Make sure readback has been requested
	check(!bHasIndirectArgs || bIndirectArgReadbackRequested);
}

/**
 * Make sure all GPU readback requests are finished before marking frame as complete
 */
bool FMeshDrawCommandStatsManager::FFrameData::IsCompleted()
{
	for (auto Iter = CustomIndirectArgsBufferResults.CreateIterator(); Iter; ++Iter)
	{
		if (!Iter.Value().GPUBufferReadback->IsReady())
		{
			return false;
		}
	}

	for (FMeshDrawCommandPassStats* PassStats : PassData)
	{
		if (PassStats->InstanceCullingGPUBufferReadback && !PassStats->InstanceCullingGPUBufferReadback->IsReady())
		{
			return false;
		}
	}
	return true;
}

FMeshDrawCommandStatsManager::FMeshDrawCommandStatsManager()
{
	// Tick on and of RT frame
	FCoreDelegates::OnEndFrameRT.AddRaw(this, &FMeshDrawCommandStatsManager::Update);

	// Is it fine to keep the screen message delegate always registered even if we are not showing anything?
	ScreenMessageDelegate = FRendererOnScreenNotification::Get().AddLambda([this](TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText >& OutMessages)
		{	
			const bool bShowPassNameStats = CVarMeshDrawCommandStats->GetInt() == (int)MeshDrawStatsCollection::Pass;
			auto NumberToK = [](uint64 InV) -> uint64 { return InV / 1000; };

			int64 TotalTracked = 0;
			
			const bool bShowStats = CVarMeshDrawCommandStats->GetInt() != (int)MeshDrawStatsCollection::None;
			if (bShowStats)
			{
				const bool bUseVertexCount = CVarMeshDrawCommandUseVertexStats->GetInt() > 0;
				if (bUseVertexCount)
				{
					OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("MeshDrawCommandStats (Vertices / Budget - Category)"))));
				}
				else
				{
					OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("MeshDrawCommandStats (Triangles / Budget - Category)"))));
				}

				int Collection = CVarMeshDrawCommandStats->GetInt();
				FStatCollection* StatCollection = StatCollections.Find(Collection);

				// Categories tracked by specific budgets
				if (StatCollection)
				{
					for (const FCollectionCategory& Category : StatCollection->Categories)
					{
						FDrawData* DrawData = BudgetedDrawData.Find(Category.Name);
						if (DrawData && DrawData->PrimitiveCount > 0)
						{
							uint64 Count = bUseVertexCount ? DrawData->VertexCount : DrawData->PrimitiveCount;
							if (Count > 0)
							{
								uint64 Budget = bUseVertexCount ? Category.DrawBudgets.VertexCount : Category.DrawBudgets.PrimitiveCount;
								FCoreDelegates::EOnScreenMessageSeverity Severity = Budget < Count ? FCoreDelegates::EOnScreenMessageSeverity::Warning : FCoreDelegates::EOnScreenMessageSeverity::Info;
								OutMessages.Add(Severity, FText::FromString(FString::Printf(TEXT("%5" UINT64_FMT "K / %5lldK - %s (%s)"),
									NumberToK(Count),
									NumberToK(Budget),
									*(Category.Name.ToString()),
									*Category.PassFriendlyName
									)));

								TotalTracked += Count;
							}
						}
					}

					if (TotalTracked > 0)
					{
						int32 CollectionBudget = StatCollection ? (bUseVertexCount ? StatCollection->DrawBudgets.VertexCount : StatCollection->DrawBudgets.PrimitiveCount) : INT32_MAX;

						FCoreDelegates::EOnScreenMessageSeverity Severity = TotalTracked < CollectionBudget ? FCoreDelegates::EOnScreenMessageSeverity::Info : FCoreDelegates::EOnScreenMessageSeverity::Warning;
						OutMessages.Add(Severity, FText::FromString(FString::Printf(TEXT("%5lldK - Total Tracked"), NumberToK(TotalTracked))));
					}
				}

				// Categories which are not tracked by specific budgets
				int32 TotalUntracked = 0;
				
				FString PassFriendlyNames = (StatCollection && StatCollection->Untracked.Passes.Num()) ? FString::Printf(TEXT(" (%s)"), *StatCollection->Untracked.PassFriendlyName) : TEXT(" (All Passes)");
				for (const TPair<FName, FDrawData>& Pair : UntrackedDrawData)
				{
					const FName& Name = Pair.Key;
					uint64 Count = bUseVertexCount ? Pair.Value.VertexCount : Pair.Value.PrimitiveCount;
					if (Count > 0)
					{
						OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("\t%5lldK - %s%s"), NumberToK(Count), *(Name.ToString()), *PassFriendlyNames)));
						TotalUntracked += Count;
					}					
				}

				if (TotalUntracked > 0 && !bShowPassNameStats)
				{
					OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(FString::Printf(TEXT("%5lldK - Total Untracked"), NumberToK(TotalUntracked))));
				}

				int32 TotalBudget = StatCollection ? (bUseVertexCount ? StatCollection->DrawBudgets.VertexCount : StatCollection->DrawBudgets.PrimitiveCount) : 0;
				int32 TotalCount = bUseVertexCount ? Stats.TotalDrawData.VertexCount : Stats.TotalDrawData.PrimitiveCount;
				if (TotalBudget)
				{
					FCoreDelegates::EOnScreenMessageSeverity Severity = TotalBudget < TotalCount ? FCoreDelegates::EOnScreenMessageSeverity::Warning : FCoreDelegates::EOnScreenMessageSeverity::Info;
					OutMessages.Add(Severity, FText::FromString(FString::Printf(TEXT("%5lldK / %5lldK - Total"), NumberToK(TotalTracked + TotalUntracked), NumberToK(TotalBudget))));
				}
				else
				{
					OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("\t%5lldK - Total"), NumberToK(TotalCount))));
				}
			}
		});
}

FMeshDrawCommandPassStats* FMeshDrawCommandStatsManager::CreatePassStats(const TCHAR* PassName)
{
	if (!bCollectStats)
	{
		return nullptr;
	}

	FScopeLock ScopeLock(&FrameDataCS);

	FFrameData* FrameData = GetOrAddFrameData();
	FMeshDrawCommandPassStats* PassStats = new FMeshDrawCommandPassStats(PassName);
	FrameData->PassData.Add(PassStats);
	return PassStats;
}

FRHIGPUBufferReadback* FMeshDrawCommandStatsManager::QueueDrawRDGIndirectArgsReadback(FRDGBuilder& GraphBuilder, FRDGBuffer* DrawIndirectArgsRDG)
{
	// TODO: pool the readback buffers
	FRHIGPUBufferReadback* GPUBufferReadback = new FRHIGPUBufferReadback(TEXT("InstanceCulling.StatsReadbackQuery"));
	AddReadbackBufferPass(GraphBuilder, RDG_EVENT_NAME("ReadbackIndirectArgs"), DrawIndirectArgsRDG,
		[GPUBufferReadback, DrawIndirectArgsRDG](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			GPUBufferReadback->EnqueueCopy(RHICmdList, DrawIndirectArgsRDG->GetRHI(), 0u);
		});

	// Make sure the readback buffer is stored for later deletion because the batch could be empty and then readback buffer might never be deleted
	{
		FScopeLock ScopeLock(&FrameDataCS);	
		FFrameData* FrameData = GetOrAddFrameData();
		FrameData->RDGIndirectArgsReadbackBuffers.Add(GPUBufferReadback);
	}

	return GPUBufferReadback;
}

void FMeshDrawCommandStatsManager::QueueCustomDrawIndirectArgsReadback(FRHICommandListImmediate& CommandList)
{
	if (!bCollectStats)
	{
		return;
	}

	FScopeLock ScopeLock(&FrameDataCS);

	FFrameData* FrameData = GetOrAddFrameData();
	FrameData->bIndirectArgReadbackRequested = true;

	// Collect set of all unique custom indirect arg buffers
	TSet<FRHIBuffer*> CustomIndirectArgsBuffers;
	for (FMeshDrawCommandPassStats* PassStats : FrameData->PassData)
	{
		CustomIndirectArgsBuffers.Append(PassStats->CustomIndirectArgsBuffers);
	}

	for (FRHIBuffer* CustomIndirectArgsBuffer : CustomIndirectArgsBuffers)
	{
		FRHIGPUBufferReadback* GPUBufferReadback = new FRHIGPUBufferReadback(TEXT("CustomIndirectArgs.StatsReadbackQuery"));
		GPUBufferReadback->EnqueueCopy(CommandList, CustomIndirectArgsBuffer, 0u);

		FIndirectArgsBufferResult IndirectArgsBufferResult;
		IndirectArgsBufferResult.GPUBufferReadback = GPUBufferReadback;
		FrameData->CustomIndirectArgsBufferResults.Add(CustomIndirectArgsBuffer, IndirectArgsBufferResult);
	}
}

void FMeshDrawCommandStatsManager::Update()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshDrawCommandStatsManager::Update);

	++CurrentFrameNumber;

	const bool bShowPassNameStats = CVarMeshDrawCommandStats->GetInt() == (int)MeshDrawStatsCollection::Pass;

	FScopeLock ScopeLock(&FrameDataCS);

	bool bHasProcessedFrame = false;

	// TODO: might be more than one from a given frame. E.g., if it was using a scene capture, need to filter out those, or perhaps record them as a group actually.
	for (int32 Index = Frames.Num() - 1; Index >= 0; --Index)
	{
		FFrameData* FrameData = Frames[Index];
		if (FrameData->IsCompleted())
		{
			if (!bHasProcessedFrame)
			{
				bHasProcessedFrame = true;	
				
				// TODO: offload processing to async task to offload the rendering thread and time the FrameDataCS lock is taken

				Stats.Reset();

				// Get custom indirect args data
				for (auto Iter = FrameData->CustomIndirectArgsBufferResults.CreateIterator(); Iter; ++Iter)
				{
					FIndirectArgsBufferResult& CustomArgsBufferResult = Iter.Value();
					CustomArgsBufferResult.DrawIndexedIndirectParameters = reinterpret_cast<const FRHIDrawIndexedIndirectParameters*>(CustomArgsBufferResult.GPUBufferReadback->Lock(CustomArgsBufferResult.GPUBufferReadback->GetGPUSizeBytes()));
				}

				using PassCategoryStats = TMap<FName, FDrawData>;
				TMap<FName, PassCategoryStats> Passes;

				for (FMeshDrawCommandPassStats* PassStats : FrameData->PassData)
				{
					// make sure the pass was kicked
					if (!PassStats->bBuildRenderingCommandsCalled)
					{
						continue;
					}

					PassCategoryStats& CategoryStats = Passes.FindOrAdd(PassStats->PassName);

					const uint8* InstanceCullingReadBackData = PassStats->InstanceCullingGPUBufferReadback ? reinterpret_cast<const uint8*>(PassStats->InstanceCullingGPUBufferReadback->Lock(PassStats->DrawData.Num())) : nullptr;
					const FRHIDrawIndexedIndirectParameters* IndirectArgsPtr = reinterpret_cast<const FRHIDrawIndexedIndirectParameters*>(InstanceCullingReadBackData);
					
					for (int32 CmdIndex = 0; CmdIndex < PassStats->DrawData.Num(); ++CmdIndex)
					{
						FVisibleMeshDrawCommandStatsData& DrawData = PassStats->DrawData[CmdIndex];
						check(DrawData.StatsData.NumVertices > 0);
						int32 IndirectCommandIndex = DrawData.IndirectArgsOffset / (FInstanceCullingContext::IndirectArgsNumWords * sizeof(uint32));
						if (DrawData.CustomIndirectArgsBuffer)
						{
							ensure(DrawData.PrimitiveCount == 0);

							FIndirectArgsBufferResult* IndirectArgsBufferResult = FrameData->CustomIndirectArgsBufferResults.Find(DrawData.CustomIndirectArgsBuffer);
							if (ensure(IndirectArgsBufferResult))
							{
								const FRHIDrawIndexedIndirectParameters& IndirectArgs = IndirectArgsBufferResult->DrawIndexedIndirectParameters[IndirectCommandIndex];
								DrawData.PrimitiveCount = IndirectArgs.IndexCountPerInstance / 3; //< Assume triangles here for now - primitive count is empty so can't be used
								DrawData.VisibleInstanceCount = IndirectArgs.InstanceCount;
								DrawData.TotalInstanceCount = FMath::Max(DrawData.TotalInstanceCount, DrawData.VisibleInstanceCount);

								Stats.CustomIndirectInstances += DrawData.VisibleInstanceCount;
								Stats.CustomIndirectDrawData.VertexCount += DrawData.VisibleInstanceCount * DrawData.StatsData.NumVertices;
								Stats.CustomIndirectDrawData.PrimitiveCount += DrawData.VisibleInstanceCount * DrawData.PrimitiveCount;
							}
						}
						else if (IndirectArgsPtr && DrawData.UseInstantCullingIndirectBuffer > 0 && InstanceCullingReadBackData)
						{
							ensure(DrawData.NumBatches > 0);
							DrawData.VisibleInstanceCount = 0;
							for (int32 BatchIndex = 0; BatchIndex < DrawData.NumBatches; BatchIndex++)
							{
								const FRHIDrawIndexedIndirectParameters& IndirectArgs = IndirectArgsPtr[PassStats->IndirectArgParameterOffset + IndirectCommandIndex + BatchIndex];
								ensure(DrawData.PrimitiveCount == IndirectArgs.IndexCountPerInstance / 3);
								DrawData.VisibleInstanceCount += IndirectArgs.InstanceCount;								
							}
							ensure(DrawData.VisibleInstanceCount <= DrawData.TotalInstanceCount);
							Stats.InstanceCullingIndirectInstances += DrawData.VisibleInstanceCount;
							Stats.InstanceCullingIndirectDrawData.VertexCount += DrawData.VisibleInstanceCount * DrawData.StatsData.NumVertices;
							Stats.InstanceCullingIndirectDrawData.PrimitiveCount += DrawData.VisibleInstanceCount * DrawData.PrimitiveCount;
						}

						Stats.TotalInstances += DrawData.VisibleInstanceCount;
						Stats.TotalDrawData.VertexCount += DrawData.VisibleInstanceCount * DrawData.StatsData.NumVertices;
						Stats.TotalDrawData.PrimitiveCount += DrawData.VisibleInstanceCount * DrawData.PrimitiveCount;

						FName StatName = bShowPassNameStats ? PassStats->PassName : DrawData.StatsData.CategoryName;
						FDrawData& CategoryDrawData = CategoryStats.FindOrAdd(StatName);
						CategoryDrawData.PrimitiveCount += DrawData.VisibleInstanceCount * DrawData.PrimitiveCount;
						CategoryDrawData.VertexCount += DrawData.VisibleInstanceCount * DrawData.StatsData.NumVertices;
					}

					if (IndirectArgsPtr)
					{
						PassStats->InstanceCullingGPUBufferReadback->Unlock();
					}
				}

				for (auto Iter = FrameData->CustomIndirectArgsBufferResults.CreateIterator(); Iter; ++Iter)
				{
					FIndirectArgsBufferResult& CustomArgsBufferResult = Iter.Value();
					CustomArgsBufferResult.GPUBufferReadback->Unlock();
					CustomArgsBufferResult.DrawIndexedIndirectParameters = nullptr;
				}

				for (auto PassIter = Passes.CreateConstIterator(); PassIter; ++PassIter)
				{
					PassCategoryStats CatMap = PassIter.Value();
					for (auto CatIter = CatMap.CreateConstIterator(); CatIter; ++CatIter)
					{
						Stats.CategoryStats.Add(FStats::FCategoryStats(PassIter.Key(), CatIter.Key(), CatIter.Value()));
					}
				}
				Algo::Sort(Stats.CategoryStats, [this](FStats::FCategoryStats& LHS, FStats::FCategoryStats& RHS) { return LHS.CategoryName.ToString() < RHS.CategoryName.ToString(); });

				// Got new stats, so can dump them if requested
				static bool bDumpStats = false;
				if (bDumpStats || bRequestDumpStats)
				{
					DumpStats(FrameData);
					bDumpStats = false;
					bRequestDumpStats = false;
				}
			}

			// Could pool the frames for allocation effeciency
			delete FrameData;

			// Ok, since we're interating backwards - must not use RemoveAtSwap because we depend on the order being the most recent last.
			// there may be older frames further up that were not completed last frame, but we want to clear them out now.
			Frames.RemoveAt(Index);
		}
	}

	// We keep and set the value from the previous frame in case there are no readback, this avoids alternating values if for example two queries were consumed in one frame
	// might be able to do this better perhaps.
	SET_DWORD_STAT(STAT_Culling_TotalNumPrimitives, Stats.TotalDrawData.PrimitiveCount);
	SET_DWORD_STAT(STAT_Culling_TotalNumVertices, Stats.TotalDrawData.VertexCount);
	SET_DWORD_STAT(STAT_Culling_TotalNumInstances, Stats.TotalInstances);
	SET_DWORD_STAT(STAT_Culling_InstanceCullingIndirectNumPrimitives, Stats.InstanceCullingIndirectDrawData.PrimitiveCount);
	SET_DWORD_STAT(STAT_Culling_InstanceCullingIndirectNumVertices, Stats.InstanceCullingIndirectDrawData.VertexCount);
	SET_DWORD_STAT(STAT_Culling_InstanceCullingIndirectNumInstances, Stats.InstanceCullingIndirectInstances);
	SET_DWORD_STAT(STAT_Culling_CustomIndirectNumPrimitives, Stats.CustomIndirectDrawData.PrimitiveCount);
	SET_DWORD_STAT(STAT_Culling_CustomIndirectNumVertices, Stats.CustomIndirectDrawData.VertexCount);
	SET_DWORD_STAT(STAT_Culling_CustomIndirectNumInstances, Stats.CustomIndirectInstances);

	// Collect stats during the next frame (check if STATGROUP_Culling is also visible somehow)
	const bool bShowStats = CVarMeshDrawCommandStats->GetInt() != (int)MeshDrawStatsCollection::None;
	bCollectStats = bShowStats || bRequestDumpStats;

#if CSV_PROFILER_STATS
	const bool bCsvExport = FCsvProfiler::Get()->IsCapturing_Renderthread() && FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(MeshDrawCommandStats));
	bCollectStats |= bCsvExport;
#endif

	if (bCollectStats)
	{
		// First time - Build associative map for quick Stat -> Budget lookup
		if (!StatCollections.Num())
		{
			const UMeshDrawCommandStatsSettings* Settings = GetDefault<UMeshDrawCommandStatsSettings>();	
			for (const FMeshDrawCommandStatsBudget& CategoryBudget : Settings->Budgets)
			{
				FStatCollection& Collection = StatCollections.FindOrAdd(CategoryBudget.Collection);

				FCollectionCategory* Category;

				if (CategoryBudget.CategoryName == TEXT("Untracked"))
				{
					Category = &Collection.Untracked;
				}
				else
				{
					Category = &Collection.Categories.AddDefaulted_GetRef();
				}

				Category->DrawBudgets.PrimitiveCount = CategoryBudget.PrimitiveBudget;
				Category->DrawBudgets.VertexCount = CategoryBudget.VertexBudget;
				Category->Name = CategoryBudget.CategoryName;

				FString& FriendlyNames = Category->PassFriendlyName;

				if (CategoryBudget.Passes.Num())
				{
					for (int i = 0; i < CategoryBudget.Passes.Num(); i++)
					{
						const FName& Pass = CategoryBudget.Passes[i];
						Category->Passes.Add(Pass);

						FriendlyNames += i ? " | " : "";
						FriendlyNames += Pass.ToString();
					}
				}
				else
				{
					FriendlyNames = "All Passes";
				}

				Category->LinkedNames.Add(CategoryBudget.CategoryName);

				for (FName Name : CategoryBudget.LinkedStatNames)
				{
					Category->LinkedNames.Add(Name);
				}
			}
			
			for (const FMeshDrawCommandStatsBudgetTotals& BudgetTotal : Settings->BudgetTotals)
			{
				FStatCollection* Collection = StatCollections.Find(BudgetTotal.Collection);
				if (Collection)
				{
					Collection->DrawBudgets.PrimitiveCount = BudgetTotal.PrimitiveBudget;
					Collection->DrawBudgets.VertexCount = BudgetTotal.VertexBudget;
				}
			}
			
			for (TPair<int32, FStatCollection>& Pair : StatCollections)
			{
				Pair.Value.Finish();
			}
		}

		// Total up Primitive across stats to their respective Budgets
		BudgetedDrawData.Reset();
		UntrackedDrawData.Reset();

		int CollectionIdx = CVarMeshDrawCommandStats->GetInt();

	#if CSV_PROFILER_STATS // If capturing for CSV, override the collection to the one requested in the ini file
		if (FCsvProfiler::Get()->IsCapturing_Renderthread() && FCsvProfiler::Get()->IsCategoryEnabled(CSV_CATEGORY_INDEX(MeshDrawCommandStats)))
		{
			CollectionIdx = GetDefault<UMeshDrawCommandStatsSettings>()->CollectionForCsvProfiler;
		}
	#endif

		FStatCollection* Collection = StatCollections.Find(CollectionIdx);

		if (Collection || CollectionIdx == (int)MeshDrawStatsCollection::Pass)
		{
			for (const FStats::FCategoryStats& CategoryStat : Stats.CategoryStats)
			{ 
				TArray<int>* CategoryIndices = nullptr;

				if (Collection)
				{
					CategoryIndices = Collection->CategoriesThatLinkStat(CategoryStat.CategoryName);
				}
				
				if (CategoryIndices)
				{
					for (int CategoryIndex : *CategoryIndices)
					{ 
						FCollectionCategory& Category = Collection->Categories[CategoryIndex];

						if (!Category.Passes.Num() || Category.Passes.Contains(CategoryStat.PassName))
						{
							FDrawData& DrawData = BudgetedDrawData.FindOrAdd(Category.Name);
							DrawData.PrimitiveCount += CategoryStat.DrawData.PrimitiveCount;
							DrawData.VertexCount += CategoryStat.DrawData.VertexCount;
						}
					}
				}
				// Collect untracked from a specific pass or all passes if there is no collection or the Untracked collection doesn't exist
				else if (!Collection || !Collection->Untracked.Passes.Num() || Collection->Untracked.Passes.Contains(CategoryStat.PassName))
				{
					FDrawData& DrawData = UntrackedDrawData.FindOrAdd(CategoryStat.CategoryName);
					DrawData.PrimitiveCount += CategoryStat.DrawData.PrimitiveCount;
					DrawData.VertexCount += CategoryStat.DrawData.VertexCount;
				}
			}
		}
	}

#if CSV_PROFILER_STATS
	if (bCsvExport)
	{
		// Output Budget totals
		for (const TPair<FName, FDrawData>& Pair : BudgetedDrawData)
		{
			TRACE_CSV_PROFILER_INLINE_STAT(TCHAR_TO_ANSI(*Pair.Key.ToString()), CSV_CATEGORY_INDEX(MeshDrawCommandStats));
			FCsvProfiler::RecordCustomStat(Pair.Key, CSV_CATEGORY_INDEX(MeshDrawCommandStats), IntCastChecked<int32>(Pair.Value.PrimitiveCount), ECsvCustomStatOp::Set);
		}

		// Output Untracked totals as a single bucket
		uint64 TotalUntracked = 0;

		for (const TPair<FName, FDrawData>& Pair : UntrackedDrawData)
		{
			TotalUntracked += Pair.Value.PrimitiveCount;
		}

		const char* Name = "Untracked";
		TRACE_CSV_PROFILER_INLINE_STAT(Name, CSV_CATEGORY_INDEX(MeshDrawCommandStats));
		FCsvProfiler::RecordCustomStat(Name, CSV_CATEGORY_INDEX(MeshDrawCommandStats), IntCastChecked<int32>(TotalUntracked), ECsvCustomStatOp::Set);
	}
#endif
}

void FMeshDrawCommandStatsManager::DumpStats(FFrameData* FrameData)
{
	const FString Category = OptionalCategory.IsEmpty() ? TEXT("") : FString::Printf(TEXT("%s-"), *OptionalCategory);
	const FString Filename = FString::Printf(TEXT("%sMeshDrawCommandStats-%s%s.csv"), *FPaths::ProfilingDir(), *Category, *FDateTime::Now().ToString());
	FArchive* CSVFile = IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_AllowRead);
	if (CSVFile == nullptr)
	{
		return;
	}

	struct FStatEntry
	{
		FName PassName;
		int32 VisibilePrimitiveCount = 0;
		int32 VisibileVertexCount = 0;
		int32 VisibleInstance = 0;
		FName CategoryName;
		FName ResourceName;
		int32 LODIndex = 0;
		int32 SegmentIndex = 0;
		FString MaterialName;
		int32 PrimitiveCount = 0;
		int32 VertexCount = 0;
		int32 TotalInstanceCount = 0;
		int32 TotalPrimitiveCount = 0;
		int32 TotalVertexCount = 0;
	};
	TArray<FStatEntry> StatEntries;

	for (FMeshDrawCommandPassStats* PassStats : FrameData->PassData)
	{
		for (FVisibleMeshDrawCommandStatsData& DrawData : PassStats->DrawData)
		{
			if (DrawData.VisibleInstanceCount > 0)
			{
				FStatEntry& StatEntry = StatEntries.Add_GetRef(FStatEntry());
				StatEntry.PassName = PassStats->PassName;
				StatEntry.VisibilePrimitiveCount = DrawData.VisibleInstanceCount * DrawData.PrimitiveCount;
				StatEntry.VisibileVertexCount = DrawData.VisibleInstanceCount * DrawData.StatsData.NumVertices;
				StatEntry.VisibleInstance = DrawData.VisibleInstanceCount;
				StatEntry.PrimitiveCount = DrawData.PrimitiveCount;
				StatEntry.VertexCount = DrawData.StatsData.NumVertices;
				StatEntry.TotalInstanceCount = DrawData.TotalInstanceCount;
				StatEntry.TotalPrimitiveCount = DrawData.TotalInstanceCount * DrawData.PrimitiveCount;
				StatEntry.TotalVertexCount = DrawData.TotalInstanceCount * DrawData.StatsData.NumVertices;
				StatEntry.CategoryName = DrawData.StatsData.CategoryName;
			#if MESH_DRAW_COMMAND_DEBUG_DATA
				StatEntry.LODIndex = DrawData.LODIndex;
				StatEntry.SegmentIndex = DrawData.SegmentIndex;
				StatEntry.ResourceName = DrawData.ResourceName;
				StatEntry.MaterialName = DrawData.MaterialName;
			#endif
			}
		}
	}

	Algo::Sort(StatEntries, [this](FStatEntry& LHS, FStatEntry& RHS)
		{
			// first by pass
			if (LHS.PassName != RHS.PassName)
			{
				return LHS.PassName.ToString() < RHS.PassName.ToString();
			}

			// then by visible primitive count
			return LHS.VisibilePrimitiveCount > RHS.VisibilePrimitiveCount;
		});

	const TCHAR* Header = TEXT("Pass,VisiblePrimitiveCount,VisibleVertices,VisibleInstances,Category,ResourceName,LODIndex,SegmentIndex,MaterialName,PrimitiveCount,VertexCount,TotalInstanceCount,TotalPrimitiveCount,TotalVertexCount\n");
	CSVFile->Serialize(TCHAR_TO_ANSI(Header), FPlatformString::Strlen(Header));

	TCHAR PassNameBuffer[FName::StringBufferSize];
	TCHAR ResourceNameBuffer[FName::StringBufferSize];
	TCHAR CategoryBuffer[FName::StringBufferSize];	

	for (FStatEntry& StatEntry : StatEntries)
	{
		StatEntry.PassName.ToString(PassNameBuffer);
		StatEntry.CategoryName.ToString(CategoryBuffer);
		StatEntry.ResourceName.ToString(ResourceNameBuffer);

		FString Row = FString::Printf(TEXT("%s,%d,%d,%d,%s,%s,%d,%d,%s,%d,%d,%d,%d,%d\n"),
			PassNameBuffer,
			StatEntry.VisibilePrimitiveCount,
			StatEntry.VisibileVertexCount,
			StatEntry.VisibleInstance,
			CategoryBuffer,
			ResourceNameBuffer,
			StatEntry.LODIndex,
			StatEntry.SegmentIndex,
			*StatEntry.MaterialName,
			StatEntry.PrimitiveCount,
			StatEntry.VertexCount,
			StatEntry.TotalInstanceCount,
			StatEntry.TotalPrimitiveCount,
			StatEntry.TotalVertexCount);
		CSVFile->Serialize(TCHAR_TO_ANSI(*Row), Row.Len());
	}

	delete CSVFile;
	CSVFile = nullptr;
}

static FAutoConsoleCommand GDumpMeshDrawCommandStatsCmd(
	TEXT("r.MeshDrawCommands.DumpStats"),
	TEXT("Dumps all of the Mesh Draw Command stats for a single frame to a csv file in the saved profile directory.\nOptionally pass in a category string to be appened to the filename"),
	FConsoleCommandWithArgsDelegate::CreateStatic([](const TArray<FString>& Args)
{
	const FString& OptionalCategory = Args.Num() == 1 ? Args[0] : FString();	
		
	if (FMeshDrawCommandStatsManager* Instance = FMeshDrawCommandStatsManager::Get())
	{
		Instance->RequestDumpStats(OptionalCategory);
	}
}));

#endif  // MESH_DRAW_COMMAND_STATS
