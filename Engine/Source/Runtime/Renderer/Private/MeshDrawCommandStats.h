// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MeshPassProcessor.h"

class FRDGBuilder;
class FRHIGPUBufferReadback;

#if MESH_DRAW_COMMAND_STATS

/**
 * Contains all the required data per mesh draw command which is needed for stat collection - cached locally because lifetime of MDC is unknown by the time indirect args are resolved
 */
struct FVisibleMeshDrawCommandStatsData
{
	FVisibleMeshDrawCommandStatsData() : IndirectArgsOffset(0), UseInstantCullingIndirectBuffer(0) { }

	FMeshDrawCommandStatsData StatsData;				//< Extracted stats data from the MDC
	int32 PrimitiveCount = 0;							//< Primitice count of a single instance
	int32 TotalInstanceCount = 0;						//< Total instance count if no per instance culling would be done
	int16 VisibleInstanceCount = 0;						//< Visisble instances (read back from indirect args if needed)
	int16 NumBatches = 0;								//< Total number of batched in this specific draw command

	uint32 IndirectArgsOffset : 31;						//< Offset in indirect arg buffer where results can be read (when using instance culling indirect args the pass offset needs to be applied on top of this to get the correct offset)
	uint32 UseInstantCullingIndirectBuffer : 1;			//< Is the draw command using the shared GPU scene instance culling indirect arg buffer?
	FRHIBuffer* CustomIndirectArgsBuffer = nullptr;		//< Optional custom indirect arg buffer which was provided to the MDC at draw time

#if MESH_DRAW_COMMAND_DEBUG_DATA
	int32 LODIndex = 0;									//< LOD index in draw command.
	int32 SegmentIndex = 0;								//< Segment index in draw command.
	FName ResourceName;									//< Minimal resource name 
	FString MaterialName;								//< Material name used during draw event
#endif // MESH_DRAW_COMMAND_DEBUG_DATA
};

/**
 * Contains all the draw data for a single pas
 */
struct FMeshDrawCommandPassStats
{
	friend class FMeshDrawCommandStatsManager;
	
	const TCHAR* PassName;										//< Name of the pass
	bool bBuildRenderingCommandsCalled = false;			//< Have the final render commands been build and is the passed used
	TArray<FVisibleMeshDrawCommandStatsData> DrawData;	//< All the draw commands
	TSet<FRHIBuffer*> CustomIndirectArgsBuffers;		//< Set of all the custom indirect args used by the draw commands - needs manual readback requests

	FMeshDrawCommandPassStats(const TCHAR* InPassName) : PassName(InPassName)
	{
	}

	/** Set the shared instance culling read back buffer and the base offset into the buffer for the indirect arg results of this pass */
	void SetInstanceCullingGPUBufferReadback(FRHIGPUBufferReadback* Buffer, int32 Offset)
	{
		InstanceCullingGPUBufferReadback = Buffer;
		IndirectArgParameterOffset = Offset;
	}

private:

	FRHIGPUBufferReadback* InstanceCullingGPUBufferReadback = nullptr; //< Possible shared instance culling readback buffer
	int32 IndirectArgParameterOffset = 0; //< Base offset into the readback buffer when shared with other passes
};

/**
 * Collects all mesh draw command stats for all passes for a certain frame - collection is only done when a CSV dump is requested or on screen stats are active
 */
class FMeshDrawCommandStatsManager
{
public:

	static void CreateInstance();
	static FMeshDrawCommandStatsManager* Get() { return Instance; }

	FMeshDrawCommandStatsManager();

	/** Create the pass stats - object will only be returned when stat collection for this frame is enabled */
	FMeshDrawCommandPassStats* CreatePassStats(const TCHAR* PassName);
	/** Queue readback from GPU for given RDG managed indirect arg buffer */
	FRHIGPUBufferReadback* QueueDrawRDGIndirectArgsReadback(FRDGBuilder& GraphBuilder, FRDGBuffer* DrawIndirectArgsRDG);
	/** Queue readback from GPU for all custom indirect args buffers used in current frame */
	void QueueCustomDrawIndirectArgsReadback(FRHICommandListImmediate& CommandList);
	
	bool CollectStats() const { return bCollectStats; }
	void RequestDumpStats(const FString& InOptionalCategory)
	{
		bRequestDumpStats = true;
		OptionalCategory = InOptionalCategory;
	}	
	void Update();
	
private:
	
	struct FIndirectArgsBufferResult
	{
		FRHIGPUBufferReadback* GPUBufferReadback = nullptr;
		const FRHIDrawIndexedIndirectParameters* DrawIndexedIndirectParameters = nullptr;
	};

	/**
	 * Contains all data for a single frame
	 */
	struct FFrameData
	{
		FFrameData(int32 InFrameNumber) : FrameNumber(InFrameNumber) {}
		~FFrameData();

		void Validate() const;
		bool IsCompleted();

		int32 FrameNumber = 0;							//< Unique ever incrementing frame number
		TArray<FMeshDrawCommandPassStats*> PassData;	//< Stats for all the MDC passes running during this frame
		TMap<FRHIBuffer*, FIndirectArgsBufferResult> CustomIndirectArgsBufferResults; //< Custom indirect arg readback result buffer lookup map
		TArray<FRHIGPUBufferReadback*> RDGIndirectArgsReadbackBuffers; //< All indirect args buffers requested via RDG passes
		bool bIndirectArgReadbackRequested = false;		
	};

	struct FDrawData
	{
		FDrawData() = default;
		FDrawData(uint64 InPrimitiveCount, uint64 InVertexCount) : PrimitiveCount(InPrimitiveCount), VertexCount(InVertexCount) {}

		void Reset()
		{
			PrimitiveCount = 0;
			VertexCount = 0;
		}

		uint64 PrimitiveCount = 0;
		uint64 VertexCount = 0;
	};

	/**
	 * Last updated per frame stats
	 */
	struct FStats
	{
		void Reset()
		{
			TotalDrawData.Reset();
			TotalInstances = 0;

			InstanceCullingIndirectDrawData.Reset();
			InstanceCullingIndirectInstances = 0;

			CustomIndirectDrawData.Reset();
			CustomIndirectInstances = 0;

			CategoryStats.Empty();
		}

		FDrawData TotalDrawData;
		int32 TotalInstances = 0;					

		FDrawData InstanceCullingIndirectDrawData;
		int32 InstanceCullingIndirectInstances = 0;		

		FDrawData CustomIndirectDrawData;
		int32 CustomIndirectInstances = 0;

		/**
		 * Aggregated stats of all mesh draw command with shared category
		 */
		struct FCategoryStats
		{
			FCategoryStats(FName InPassName, FName InCategoryName, FDrawData InDrawData)
				: PassName(InPassName), CategoryName(InCategoryName), DrawData(InDrawData) {}
			FName PassName;
			FName CategoryName;
			FDrawData DrawData;
		};
		TArray<FCategoryStats> CategoryStats;
	};

	FFrameData* GetOrAddFrameData()
	{
		// Need to add a new frame?
		if (Frames.IsEmpty() || Frames.Last()->FrameNumber != CurrentFrameNumber)
		{
			if (!Frames.IsEmpty())
			{
				Frames.Last()->Validate();
			}
			Frames.Emplace(new FFrameData(CurrentFrameNumber));
		}
		return Frames.Last();
	}

	struct FCollectionCategory
	{
		FName Name;					// User supplied name for this category
		FString PassFriendlyName;	// Pipe delimited names of the passes for this category (FString because append to it as it's being created)
		TSet<FName> Passes;			// Passes this category cares about
		TSet<FName> LinkedNames;	// LinkedStats this collection cares about
		FDrawData DrawBudgets; 		// Draw budget for this category
	};

	struct FStatCollection
	{
		TArray<int>* CategoriesThatLinkStat(FName Stat)
		{
			return StatToCategoryIndices.Find(Stat);
		}

		void Finish()
		{
			for (int i = 0; i < Categories.Num(); i++)
			{
				for (FName& LinkedName : Categories[i].LinkedNames)
				{
					StatToCategoryIndices.FindOrAdd(LinkedName).Add(i);
				}
			}
		}

		TArray<FCollectionCategory> Categories;
		TMap<FName, TArray<int>> StatToCategoryIndices;
		FCollectionCategory Untracked;
		FDrawData DrawBudgets;
	};

	using FStatCollectionMap = TMap<int32, FStatCollection>;
	 
	/** Dump given frame data stats to csv file on disc in profiling folder */
	void DumpStats(FFrameData* FrameData);

	int32 CurrentFrameNumber = 0;				//< Ever incrementing frame number
	bool bRequestDumpStats = false;				//< Dump stats requested via command?
	bool bCollectStats = false;					//< Collect stats during next frame?

	FString OptionalCategory;					//< Optional category to append to stats filename. Empty string if no category.
	FCriticalSection FrameDataCS;			
	TArray<FFrameData*> Frames;					//< All active frames (contains the frame for which we are collecting stats now and all frames waiting for GPU readback)
	FStats Stats;								//< Last updated frame stats
	FStatCollectionMap StatCollections; 		//< Per Collection LinkedStatName to Budget CategoryName

	TMap<FName, FDrawData> BudgetedDrawData; 	//< Budget CategoryName to Total Draw data Count
	TMap<FName, FDrawData> UntrackedDrawData;  	//< Draw data stats which aren't tracked by any Budgets

	FDelegateHandle ScreenMessageDelegate;		//< Delegate used to render optional screen stats

	static FMeshDrawCommandStatsManager* Instance;
};

#endif // MESH_DRAW_COMMAND_STATS