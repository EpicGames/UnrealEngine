// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
MeshDrawCommandSetup.cpp: Mesh draw command setup.
=============================================================================*/

#include "MeshDrawCommands.h"
#include "BasePassRendering.h"
#include "LogRenderer.h"
#include "ScenePrivate.h"
#include "TranslucentRendering.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "StaticMeshBatch.h"
#include "SceneDefinitions.h"
#include "MeshDrawCommandStats.h"
#if WITH_ODSC
#include "ODSC/ODSCManager.h"
#endif

TGlobalResource<FPrimitiveIdVertexBufferPool> GPrimitiveIdVertexBufferPool;

static TAutoConsoleVariable<int32> CVarMeshDrawCommandsParallelPassSetup(
	TEXT("r.MeshDrawCommands.ParallelPassSetup"),
	1,
	TEXT("Whether to setup mesh draw command pass in parallel."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMeshSortingMethodWithoutEarlyZ(
	TEXT("r.MeshSortingMethodWithoutEarlyZ"),
	0,
	TEXT("How to sort mesh commands:\n")
	TEXT("\t0: Sort by state, roughly front to back (Default).\n")
	TEXT("\t1: Strict front to back sorting.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileMeshSortingMethod(
	TEXT("r.Mobile.MeshSortingMethod"),
	-1,
	TEXT("How to sort mesh commands on mobile:\n")
	TEXT("\t-1: Use r.MeshSortingMethodWithoutEarlyZ (Default).\n")
	TEXT("\t0: Sort by state, roughly front to back.\n")
	TEXT("\t1: Strict front to back sorting.\n"),
	ECVF_RenderThreadSafe);

static int32 GAllowOnDemandShaderCreation = 1;
static FAutoConsoleVariableRef CVarAllowOnDemandShaderCreation(
	TEXT("r.MeshDrawCommands.AllowOnDemandShaderCreation"),
	GAllowOnDemandShaderCreation,
	TEXT("How to create RHI shaders:\n")
	TEXT("\t0: Always create them on a Rendering Thread, before executing other MDC tasks.\n")
	TEXT("\t1: If RHI supports multi-threaded shader creation, create them on demand on tasks threads, at the time of submitting the draws.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDeferredMeshPassSetupTaskSync(
	TEXT("r.DeferredMeshPassSetupTaskSync"),
	1,
	TEXT("If enabled, the sync point of the mesh pass setup task is deferred until RDG execute (from during RDG setup) significantly increasing the overlap possible."),
	ECVF_RenderThreadSafe);

FPrimitiveIdVertexBufferPool::FPrimitiveIdVertexBufferPool()
	: DiscardId(0)
{
}

FPrimitiveIdVertexBufferPool::~FPrimitiveIdVertexBufferPool()
{
	check(!Entries.Num());
}

FPrimitiveIdVertexBufferPoolEntry FPrimitiveIdVertexBufferPool::Allocate(FRHICommandList& RHICmdList, int32 BufferSize)
{
	FScopeLock Lock(&AllocationCS);

	BufferSize = Align(BufferSize, 1024);

	// First look for a smallest unused one.
	int32 BestFitBufferIndex = -1;
	for (int32 Index = 0; Index < Entries.Num(); ++Index)
	{
		// Unused and fits?
		if (Entries[Index].LastDiscardId != DiscardId && Entries[Index].BufferSize >= BufferSize)
		{
			// Is it a bet fit than current BestFitBufferIndex?
			if (BestFitBufferIndex == -1 || Entries[Index].BufferSize < Entries[BestFitBufferIndex].BufferSize)
			{
				BestFitBufferIndex = Index;
				
				if (Entries[BestFitBufferIndex].BufferSize == BufferSize)
				{
					break;
				}
			}
		}
	}
	
	if (BestFitBufferIndex >= 0)
	{
		// Reuse existing buffer.
		FPrimitiveIdVertexBufferPoolEntry ReusedEntry = MoveTemp(Entries[BestFitBufferIndex]);
		ReusedEntry.LastDiscardId = DiscardId;
		Entries.RemoveAt(BestFitBufferIndex);
		return ReusedEntry;
	}
	else
	{
		// Allocate new one.
		FPrimitiveIdVertexBufferPoolEntry NewEntry;
		NewEntry.LastDiscardId = DiscardId;
		NewEntry.BufferSize = BufferSize;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("FPrimitiveIdVertexBufferPool"), BufferSize)
			.AddUsage(EBufferUsageFlags::Volatile | EBufferUsageFlags::UniformBuffer | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::ByteAddressBuffer)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer);

		NewEntry.BufferRHI = RHICmdList.CreateBuffer(CreateDesc);

		return NewEntry;
	}
}

void FPrimitiveIdVertexBufferPool::ReturnToFreeList(FPrimitiveIdVertexBufferPoolEntry Entry)
{
	// Entries can be returned from RHIT or RT, depending on if FParallelMeshDrawCommandPass::DispatchDraw() takes the parallel path
	FScopeLock Lock(&AllocationCS);

	Entries.Add(MoveTemp(Entry));
}

void FPrimitiveIdVertexBufferPool::DiscardAll()
{
	FScopeLock Lock(&AllocationCS);

	++DiscardId;

	// Remove old unused pool entries.
	for (int32 Index = 0; Index < Entries.Num();)
	{
		if (DiscardId - Entries[Index].LastDiscardId > 1000u)
		{
			Entries.RemoveAtSwap(Index);
		}
		else
		{
			++Index;
		}
	}
}

void FPrimitiveIdVertexBufferPool::ReleaseRHI()
{
	Entries.Empty();
}

uint32 BitInvertIfNegativeFloat(uint32 f)
{
	unsigned mask = -int32(f >> 31) | 0x80000000;
	return f ^ mask;
}

/**
* Update mesh sort keys with view dependent data.
*/
void UpdateTranslucentMeshSortKeys(
	ETranslucentSortPolicy::Type TranslucentSortPolicy,
	const FVector& TranslucentSortAxis,
	const FVector& ViewOrigin,
	const FMatrix& ViewMatrix,
	const TScenePrimitiveArray<FPrimitiveBounds>& PrimitiveBounds,
	ETranslucencyPass::Type TranslucencyPass, 
	bool bInverseSorting,
	FMeshCommandOneFrameArray& VisibleMeshCommands
	)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateTranslucentMeshSortKeys);

	for (int32 CommandIndex = 0; CommandIndex < VisibleMeshCommands.Num(); ++CommandIndex)
	{
		FVisibleMeshDrawCommand& VisibleCommand = VisibleMeshCommands[CommandIndex];

		const int32 PrimitiveIndex = VisibleCommand.PrimitiveIdInfo.ScenePrimitiveId;
		const FVector BoundsOrigin = PrimitiveIndex >= 0 ? PrimitiveBounds[PrimitiveIndex].BoxSphereBounds.Origin : FVector::ZeroVector;

		float Distance = 0.0f;
		if (TranslucentSortPolicy == ETranslucentSortPolicy::SortByDistance)
		{
			//sort based on distance to the view position, view rotation is not a factor
			Distance = (BoundsOrigin - ViewOrigin).Size();
		}
		else if (TranslucentSortPolicy == ETranslucentSortPolicy::SortAlongAxis)
		{
			// Sort based on enforced orthogonal distance
			const FVector CameraToObject = BoundsOrigin - ViewOrigin;
			Distance = FVector::DotProduct(CameraToObject, TranslucentSortAxis);
		}
		else
		{
			// Sort based on projected Z distance
			check(TranslucentSortPolicy == ETranslucentSortPolicy::SortByProjectedZ);
			Distance = ViewMatrix.TransformPosition(BoundsOrigin).Z;
		}

		// Apply distance offset from the primitive
		const uint32 PackedOffset = VisibleCommand.SortKey.Translucent.Distance;
		const float DistanceOffset = *((float*)&PackedOffset);
		Distance += DistanceOffset;

		// Sort front-to-back instead of back-to-front
		if (bInverseSorting)
		{
			const float MaxSortingDistance = 1000000.f; // 100km, arbitrary
			Distance = MaxSortingDistance - Distance;
		}

		// Patch distance inside translucent mesh sort key.
		FMeshDrawCommandSortKey SortKey;
		SortKey.PackedData = VisibleCommand.SortKey.PackedData;
		SortKey.Translucent.Distance = (uint32)~BitInvertIfNegativeFloat(*((uint32*)&Distance));
		VisibleCommand.SortKey.PackedData = SortKey.PackedData;
	}
}

/**
* Merge mobile BasePass with BasePassCSM based on CSM visibility in order to select appropriate shader for given command.
*/
void MergeMobileBasePassMeshDrawCommands(
	const FMobileCSMVisibilityInfo& MobileCSMVisibilityInfo,
	int32 ScenePrimitiveNum,
	FMeshCommandOneFrameArray& MeshCommands,
	FMeshCommandOneFrameArray& MeshCommandsCSM
	)
{
	if (MobileCSMVisibilityInfo.bMobileDynamicCSMInUse)
	{
		// determine per view CSM visibility
		if (MobileCSMVisibilityInfo.bAlwaysUseCSM)
		{
#if DO_CHECK
			check(!MeshCommandsCSM.Num() || MeshCommands.Num() == MeshCommandsCSM.Num());
			for (int32 Index = 0; Index < MeshCommandsCSM.Num(); ++Index)
			{
				check(MeshCommands[Index].PrimitiveIdInfo.ScenePrimitiveId == MeshCommandsCSM[Index].PrimitiveIdInfo.ScenePrimitiveId);
			}
#endif
			if (MeshCommandsCSM.Num() > 0)
			{
				MeshCommands = MoveTemp(MeshCommandsCSM);
			}
		}
		else
		{
			checkf(MeshCommands.Num() == MeshCommandsCSM.Num(), TEXT("VisibleMeshDrawCommands of BasePass and MobileBasePassCSM are expected to match."));
			for (int32 i = MeshCommands.Num() - 1; i >= 0; --i)
			{
				FVisibleMeshDrawCommand& MeshCommand = MeshCommands[i];
				FVisibleMeshDrawCommand& MeshCommandCSM = MeshCommandsCSM[i];

				const int32 CSMBit = 1 << 30;
				checkf(MeshCommand.StateBucketId == -1 || (MeshCommand.StateBucketId & CSMBit) == 0, TEXT("Non-CSM draw command's StateBucketID should not have the CSM bit set."));

				if (MeshCommand.PrimitiveIdInfo.ScenePrimitiveId < ScenePrimitiveNum && MobileCSMVisibilityInfo.MobilePrimitiveCSMReceiverVisibilityMap[MeshCommand.PrimitiveIdInfo.ScenePrimitiveId])
				{
					checkf(MeshCommand.PrimitiveIdInfo.ScenePrimitiveId == MeshCommandCSM.PrimitiveIdInfo.ScenePrimitiveId, TEXT("VisibleMeshDrawCommands of BasePass and MobileBasePassCSM are expected to match."));
					// Use CSM's VisibleMeshDrawCommand.
					MeshCommand = MeshCommandCSM;

					// Make sure that CSM and non-CSM draw commands don't have the same StateBucketId, which would potentially get them merged later.
					MeshCommand.StateBucketId |= CSMBit;
				}
			}
			MeshCommandsCSM.Reset();
		}
	}
}

static uint64 GetSortKey_FrontToBack(bool bMasked, bool bBackground, uint32 PipelineId, int32 StateBucketId, float PrimitiveDistance)
{
	union
	{
		uint64 PackedInt;
		struct
		{
			uint64 StateBucketId : 25; 		// Order by state bucket
			uint64 PipelineId : 22;			// Order by PSO
			uint64 DepthBits : 15;			// Order by primitive depth
			uint64 Background : 1;			// Non-background meshes first 
			uint64 Masked : 1;				// Non-masked first
		} Fields;
	} Key;

	union FFloatToInt { float F; uint32 I; };
	FFloatToInt F2I;

	Key.Fields.Masked = bMasked;
	Key.Fields.Background = bBackground;
	F2I.F = PrimitiveDistance;
	Key.Fields.DepthBits = ((-int32(F2I.I >> 31) | 0x80000000) ^ F2I.I) >> 17;
	Key.Fields.PipelineId = PipelineId;
	Key.Fields.StateBucketId = StateBucketId;

	return Key.PackedInt;
}

static uint32 DepthBuckets_4(float Distance)
{
	union FFloatToInt { float F; uint32 I; };
	FFloatToInt F2I;

	F2I.F = Distance;
	// discard distance shorter than 64
	int32 Exp = ((F2I.I >> 23u) & 0xff) - (127 + 6);
	// 16 buckets, ranging from 64 to 2km
	return (uint32)FMath::Clamp<int32>(Exp, 0, 15);
}

static uint64 GetSortKey_ByState(bool bMasked, bool bBackground, uint32 PipelineId, uint32 StateBucketId, float PipelineDistance, float StateBucketDistance, float PrimitiveDistance)
{
	// maximum primitive distance for bucketing, aprox 10.5km 
	constexpr float PrimitiveMaxDepth = (1 << 20);
	constexpr float PrimitiveDepthQuantization = ((1 << 12) - 1);

	union
	{
		uint64 PackedInt;
		struct
		{
			uint64 DepthBits : 12;			// Order by primitive depth
			uint64 StateBucketId : 20; 		// Order by state bucket
			uint64 StateBucketDepthBits : 4;// Order state buckets front to back
			uint64 PipelineId : 22;			// Order by PSO
			uint64 PipelineDepthBits : 4;	// Order PSOs front to back
			uint64 Background : 1;			// Non-background meshes first 
			uint64 Masked : 1;				// Non-masked first
		} Fields;
	} Key;

	Key.PackedInt = 0;
	Key.Fields.Masked = bMasked;
	Key.Fields.Background = bBackground;
	Key.Fields.PipelineDepthBits = DepthBuckets_4(PipelineDistance);
	Key.Fields.PipelineId = PipelineId;
	Key.Fields.StateBucketDepthBits = DepthBuckets_4(StateBucketDistance);
	Key.Fields.StateBucketId = StateBucketId;
	Key.Fields.DepthBits = uint32((FMath::Min<float>(PrimitiveDistance, PrimitiveMaxDepth) / PrimitiveMaxDepth) * PrimitiveDepthQuantization);

	return Key.PackedInt;
}

/**
 * Compute mesh sort keys to reduce overdraw
 */
static void UpdateMeshSortKeys(
	bool bMobile,
	const FVector& ViewOrigin,
	const TScenePrimitiveArray<FPrimitiveBounds>& ScenePrimitiveBounds,
	FMeshCommandOneFrameArray& VisibleMeshCommands
)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateMeshSortKeys);

	// Object radius past which we treat object as part of 'background'
	constexpr float MIN_BACKGROUND_OBJECT_RADIUS = 500000;

	const int32 NumCmds = VisibleMeshCommands.Num();
	const int32 MobileMeshSortingMethod = bMobile ? CVarMobileMeshSortingMethod.GetValueOnAnyThread() : -1;
	const int32 MeshSortingMethod = (MobileMeshSortingMethod >= 0)
		? MobileMeshSortingMethod
		: CVarMeshSortingMethodWithoutEarlyZ.GetValueOnAnyThread();

	if (MeshSortingMethod == 1) //strict front to back sorting
	{
		// compute sort key for each mesh command
		for (int32 CmdIdx = 0; CmdIdx < NumCmds; ++CmdIdx)
		{
			FVisibleMeshDrawCommand& Cmd = VisibleMeshCommands[CmdIdx];
			bool bMasked = Cmd.SortKey.BasePass.Masked == 1;
			bool bBackground = Cmd.SortKey.BasePass.Background == 1;
			float PrimitiveDistance = 0;
			if (Cmd.PrimitiveIdInfo.ScenePrimitiveId < ScenePrimitiveBounds.Num())
			{
				const FPrimitiveBounds& PrimitiveBounds = ScenePrimitiveBounds[Cmd.PrimitiveIdInfo.ScenePrimitiveId];
				PrimitiveDistance = (PrimitiveBounds.BoxSphereBounds.Origin - ViewOrigin).Size();
				bBackground |= (PrimitiveBounds.BoxSphereBounds.SphereRadius > MIN_BACKGROUND_OBJECT_RADIUS);
			}

			uint32 PipelineId = Cmd.MeshDrawCommand->CachedPipelineId.GetId();
			// use state bucket if dynamic instancing is enabled, otherwise identify same meshes by index buffer resource
			uint32 StateBucketId = Cmd.StateBucketId >= 0 ? Cmd.StateBucketId : PointerHash(Cmd.MeshDrawCommand->IndexBuffer);
			Cmd.SortKey.PackedData = GetSortKey_FrontToBack(bMasked, bBackground, PipelineId, StateBucketId, PrimitiveDistance);
		}
	}
	else
	{
		struct FPipelineDistance
		{
			double DistanceSum = 0.0;
			int32 Num = 0;
		};

		TMap<uint32, FPipelineDistance> PipelineDistances;
		PipelineDistances.Reserve(256);

		TMap<uint64, float> StateBucketDistances;
		StateBucketDistances.Reserve(512);

		// pre-compute a distance to a group of meshes that share same PSO and state
		for (int32 CmdIdx = 0; CmdIdx < NumCmds; ++CmdIdx)
		{
			const FVisibleMeshDrawCommand& Cmd = VisibleMeshCommands[CmdIdx];
			float PrimitiveDistance = 0.f;
			if (Cmd.PrimitiveIdInfo.ScenePrimitiveId < ScenePrimitiveBounds.Num())
			{
				const FPrimitiveBounds& PrimitiveBounds = ScenePrimitiveBounds[Cmd.PrimitiveIdInfo.ScenePrimitiveId];
				PrimitiveDistance = (PrimitiveBounds.BoxSphereBounds.Origin - ViewOrigin).Size();
			}

			// group meshes by PSO and find distance to each group
			uint32 PipelineId = Cmd.MeshDrawCommand->CachedPipelineId.GetId();
			FPipelineDistance& PipelineDistance = PipelineDistances.FindOrAdd(PipelineId);
			PipelineDistance.DistanceSum += PrimitiveDistance;
			PipelineDistance.Num += 1;

			// group meshes by StateBucketId or index buffer resource and find minimum distance to each group 
			uint32 StateBucketId = Cmd.StateBucketId >= 0 ? Cmd.StateBucketId : PointerHash(Cmd.MeshDrawCommand->IndexBuffer);
			uint64 PipelineAndStateBucketId = ((uint64)PipelineId << 32u) | StateBucketId;
			float& StateBucketDistance = StateBucketDistances.FindOrAdd(PipelineAndStateBucketId, BIG_NUMBER);
			StateBucketDistance = FMath::Min(StateBucketDistance, PrimitiveDistance);
		}

		// compute sort key for each mesh command
		for (int32 CmdIdx = 0; CmdIdx < NumCmds; ++CmdIdx)
		{
			FVisibleMeshDrawCommand& Cmd = VisibleMeshCommands[CmdIdx];
			bool bMasked = Cmd.SortKey.BasePass.Masked == 1;
			bool bBackground = Cmd.SortKey.BasePass.Background == 1;
			float PrimitiveDistance = 0;
			if (Cmd.PrimitiveIdInfo.ScenePrimitiveId < ScenePrimitiveBounds.Num())
			{
				const FPrimitiveBounds& PrimitiveBounds = ScenePrimitiveBounds[Cmd.PrimitiveIdInfo.ScenePrimitiveId];
				PrimitiveDistance = (PrimitiveBounds.BoxSphereBounds.Origin - ViewOrigin).Size();
				bBackground |= (PrimitiveBounds.BoxSphereBounds.SphereRadius > MIN_BACKGROUND_OBJECT_RADIUS);
			}

			uint32 PipelineId = Cmd.MeshDrawCommand->CachedPipelineId.GetId();
			FPipelineDistance PipelineDistance = PipelineDistances.FindRef(PipelineId);
			float MeanPipelineDistance = PipelineDistance.DistanceSum / PipelineDistance.Num;

			uint32 StateBucketId = Cmd.StateBucketId >= 0 ? Cmd.StateBucketId : PointerHash(Cmd.MeshDrawCommand->IndexBuffer);
			uint64 PipelineAndStateBucketId = ((uint64)PipelineId << 32u) | StateBucketId;
			float StateBucketDistance = StateBucketDistances.FindRef(PipelineAndStateBucketId);

			Cmd.SortKey.PackedData = GetSortKey_ByState(bMasked, bBackground, PipelineId, StateBucketId, MeanPipelineDistance, StateBucketDistance, PrimitiveDistance);
		}
	}
}

FORCEINLINE int32 TranslatePrimitiveId(int32 DrawPrimitiveIdIn, int32 DynamicPrimitiveIdOffset, int32 DynamicPrimitiveIdMax)
{
	// INDEX_NONE means we defer the translation to later
	if (DynamicPrimitiveIdOffset == INDEX_NONE)
	{
		return DrawPrimitiveIdIn;
	}

	int32 DrawPrimitiveId = DrawPrimitiveIdIn;

	if ((DrawPrimitiveIdIn & GPrimIDDynamicFlag) != 0)
	{
		int32 DynamicPrimitiveIndex = DrawPrimitiveIdIn & (~GPrimIDDynamicFlag);
		DrawPrimitiveId = DynamicPrimitiveIdOffset + DynamicPrimitiveIndex;
		checkSlow(DrawPrimitiveId < DynamicPrimitiveIdMax);
	}

	// Append flag to mark this as a non-instance data index.
	// This value is treated as a primitive ID in the SceneData.ush loading.
	return DrawPrimitiveId |= VF_TREAT_INSTANCE_ID_OFFSET_AS_PRIMITIVE_ID_FLAG;
}

/**
 * Build mesh draw command primitive Id buffer for instancing.
 * TempVisibleMeshDrawCommands must be presized for NewPassVisibleMeshDrawCommands.
 */
static void BuildMeshDrawCommandPrimitiveIdBuffer(
	bool bDynamicInstancing,
	FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	FDynamicMeshDrawCommandStorage& MeshDrawCommandStorage,
	FMeshCommandOneFrameArray& TempVisibleMeshDrawCommands,
	int32& MaxInstances,
	int32& VisibleMeshDrawCommandsNum,
	int32& NewPassVisibleMeshDrawCommandsNum,
	EShaderPlatform ShaderPlatform,
	uint32 InstanceFactor,
	TFunctionRef<void(int32, int32, int32)> WritePrimitiveDataFn)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BuildMeshDrawCommandPrimitiveIdBuffer);

	const FVisibleMeshDrawCommand* RESTRICT PassVisibleMeshDrawCommands = VisibleMeshDrawCommands.GetData();
	const int32 NumDrawCommands = VisibleMeshDrawCommands.Num();
	uint32 PrimitiveIdIndex = 0;

	if (bDynamicInstancing)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_DynamicInstancingOfVisibleMeshDrawCommands);
		check(VisibleMeshDrawCommands.Num() <= TempVisibleMeshDrawCommands.Max() && TempVisibleMeshDrawCommands.Num() == 0);

		int32 CurrentStateBucketId = -1;
		uint32* RESTRICT CurrentDynamicallyInstancedMeshCommandNumInstances = nullptr;
		MaxInstances = 1;

		for (int32 DrawCommandIndex = 0; DrawCommandIndex < NumDrawCommands; DrawCommandIndex++)
		{
			const FVisibleMeshDrawCommand& RESTRICT VisibleMeshDrawCommand = PassVisibleMeshDrawCommands[DrawCommandIndex];

			if (VisibleMeshDrawCommand.StateBucketId == CurrentStateBucketId && VisibleMeshDrawCommand.StateBucketId != -1)
			{
				if (CurrentDynamicallyInstancedMeshCommandNumInstances)
				{
					const int32 CurrentNumInstances = *CurrentDynamicallyInstancedMeshCommandNumInstances;
					*CurrentDynamicallyInstancedMeshCommandNumInstances = CurrentNumInstances + 1;
					MaxInstances = FMath::Max(MaxInstances, CurrentNumInstances + 1);
				}
				else
				{
					FVisibleMeshDrawCommand NewVisibleMeshDrawCommand = VisibleMeshDrawCommand;
					NewVisibleMeshDrawCommand.PrimitiveIdBufferOffset = PrimitiveIdIndex;
					TempVisibleMeshDrawCommands.Emplace(MoveTemp(NewVisibleMeshDrawCommand));
				}
			}
			else
			{
				// First time state bucket setup
				CurrentStateBucketId = VisibleMeshDrawCommand.StateBucketId;

				if (VisibleMeshDrawCommand.StateBucketId != INDEX_NONE
					&& VisibleMeshDrawCommand.MeshDrawCommand->PrimitiveIdStreamIndex >= 0
					&& VisibleMeshDrawCommand.MeshDrawCommand->NumInstances == 1
					// Don't create a new FMeshDrawCommand for the last command and make it safe for us to look at the next command
					&& DrawCommandIndex + 1 < NumDrawCommands
					// Only create a new FMeshDrawCommand if more than one draw in the state bucket
					&& CurrentStateBucketId == PassVisibleMeshDrawCommands[DrawCommandIndex + 1].StateBucketId)
				{
					const int32 Index = MeshDrawCommandStorage.MeshDrawCommands.AddElement(*VisibleMeshDrawCommand.MeshDrawCommand);
					FMeshDrawCommand& NewCommand = MeshDrawCommandStorage.MeshDrawCommands[Index];
					FVisibleMeshDrawCommand NewVisibleMeshDrawCommand;

					NewVisibleMeshDrawCommand.Setup(
						&NewCommand,
						VisibleMeshDrawCommand.PrimitiveIdInfo,
						VisibleMeshDrawCommand.StateBucketId,
						VisibleMeshDrawCommand.MeshFillMode,
						VisibleMeshDrawCommand.MeshCullMode,
						VisibleMeshDrawCommand.Flags,
						VisibleMeshDrawCommand.SortKey,
						VisibleMeshDrawCommand.CullingPayload,
						VisibleMeshDrawCommand.CullingPayloadFlags,
						VisibleMeshDrawCommand.RunArray,
						VisibleMeshDrawCommand.NumRuns);

					NewVisibleMeshDrawCommand.PrimitiveIdBufferOffset = PrimitiveIdIndex;
					TempVisibleMeshDrawCommands.Emplace(MoveTemp(NewVisibleMeshDrawCommand));

					CurrentDynamicallyInstancedMeshCommandNumInstances = &NewCommand.NumInstances;
				}
				else
				{
					CurrentDynamicallyInstancedMeshCommandNumInstances = nullptr;
					FVisibleMeshDrawCommand NewVisibleMeshDrawCommand = VisibleMeshDrawCommand;
					NewVisibleMeshDrawCommand.PrimitiveIdBufferOffset = PrimitiveIdIndex;
					TempVisibleMeshDrawCommands.Emplace(MoveTemp(NewVisibleMeshDrawCommand));
				}
			}

			//@todo - refactor into instance step rate in the RHI
			for (uint32 InstanceFactorIndex = 0; InstanceFactorIndex < InstanceFactor; InstanceFactorIndex++, PrimitiveIdIndex++)
			{
				WritePrimitiveDataFn(PrimitiveIdIndex, VisibleMeshDrawCommand.PrimitiveIdInfo.DrawPrimitiveId, VisibleMeshDrawCommand.PrimitiveIdInfo.ScenePrimitiveId);
			}
		}

		// Setup instancing stats for logging.
		VisibleMeshDrawCommandsNum = VisibleMeshDrawCommands.Num();
		NewPassVisibleMeshDrawCommandsNum = TempVisibleMeshDrawCommands.Num();

		// Replace VisibleMeshDrawCommands
		Swap(VisibleMeshDrawCommands, TempVisibleMeshDrawCommands);
		TempVisibleMeshDrawCommands.Reset();
	}
	else
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_BuildVisibleMeshDrawCommandPrimitiveIdBuffers);

		for (int32 DrawCommandIndex = 0; DrawCommandIndex < NumDrawCommands; DrawCommandIndex++)
		{
			const FVisibleMeshDrawCommand& VisibleMeshDrawCommand = VisibleMeshDrawCommands[DrawCommandIndex];
			for (uint32 InstanceFactorIndex = 0; InstanceFactorIndex < InstanceFactor; InstanceFactorIndex++, PrimitiveIdIndex++)
			{
				WritePrimitiveDataFn(PrimitiveIdIndex, VisibleMeshDrawCommand.PrimitiveIdInfo.DrawPrimitiveId, VisibleMeshDrawCommand.PrimitiveIdInfo.ScenePrimitiveId);
			}
		}
	}
}

/**
 * Converts each FMeshBatch into a set of FMeshDrawCommands for a specific mesh pass type.
 */
void GenerateDynamicMeshDrawCommands(
	const FViewInfo& View,
	EShadingPath ShadingPath,
	EMeshPass::Type PassType,
	FMeshPassProcessor* PassMeshProcessor,
	const TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>& DynamicMeshElements,
	const TArray<FMeshPassMask, SceneRenderingAllocator>* DynamicMeshElementsPassRelevance,
	int32 MaxNumDynamicMeshElements,
	const TArray<const FStaticMeshBatch*, SceneRenderingAllocator>& DynamicMeshCommandBuildRequests,
	const TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator> DynamicMeshCommandBuildFlags,
	int32 MaxNumBuildRequestElements,
	FMeshCommandOneFrameArray& VisibleCommands,
	FDynamicMeshDrawCommandStorage& MeshDrawCommandStorage,
	FGraphicsMinimalPipelineStateSet& MinimalPipelineStatePassSet,
	bool& NeedsShaderInitialisation
)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GenerateDynamicMeshDrawCommands);
	check(PassMeshProcessor);
	check((PassType == EMeshPass::Num) == (DynamicMeshElementsPassRelevance == nullptr));

	FDynamicPassMeshDrawListContext DynamicPassMeshDrawListContext(
		MeshDrawCommandStorage,
		VisibleCommands,
		MinimalPipelineStatePassSet,
		NeedsShaderInitialisation
	);
	PassMeshProcessor->SetDrawListContext(&DynamicPassMeshDrawListContext);

	{
		const int32 NumCommandsBefore = VisibleCommands.Num();
		const int32 NumDynamicMeshBatches = DynamicMeshElements.Num();

		for (int32 MeshIndex = 0; MeshIndex < NumDynamicMeshBatches; MeshIndex++)
		{
			if (!DynamicMeshElementsPassRelevance || (*DynamicMeshElementsPassRelevance)[MeshIndex].Get(PassType))
			{
				const FMeshBatchAndRelevance& MeshAndRelevance = DynamicMeshElements[MeshIndex];
				const uint64 BatchElementMask = ~0ull;
#if WITH_ODSC
				FODSCPrimitiveSceneInfoScope ODSCPrimitiveSceneInfoScope(MeshAndRelevance.PrimitiveSceneProxy ? MeshAndRelevance.PrimitiveSceneProxy->GetPrimitiveSceneInfo() : nullptr);
#endif

				PassMeshProcessor->AddMeshBatch(*MeshAndRelevance.Mesh, BatchElementMask, MeshAndRelevance.PrimitiveSceneProxy);
			}
		}

		const int32 NumCommandsGenerated = VisibleCommands.Num() - NumCommandsBefore;
		checkf(NumCommandsGenerated <= MaxNumDynamicMeshElements,
			TEXT("Generated %d mesh draw commands for DynamicMeshElements, while preallocating resources only for %d of them."), NumCommandsGenerated, MaxNumDynamicMeshElements);
	}

	{
		const int32 NumCommandsBefore = VisibleCommands.Num();
		const int32 NumStaticMeshBatches = DynamicMeshCommandBuildRequests.Num();

		for (int32 MeshIndex = 0; MeshIndex < NumStaticMeshBatches; MeshIndex++)
		{
			const FStaticMeshBatch* StaticMeshBatch = DynamicMeshCommandBuildRequests[MeshIndex];
			const uint64 DefaultBatchElementMask = ~0ul;
			const int32 StartCommandIndex = VisibleCommands.Num();

#if WITH_ODSC
			FODSCPrimitiveSceneInfoScope ODSCPrimitiveSceneInfoScope(StaticMeshBatch->PrimitiveSceneInfo);
#endif

			if (StaticMeshBatch->bViewDependentArguments)
			{
				FMeshBatch ViewDepenedentMeshBatch(*StaticMeshBatch);
				StaticMeshBatch->PrimitiveSceneInfo->Proxy->ApplyViewDependentMeshArguments(View, ViewDepenedentMeshBatch);
				PassMeshProcessor->AddMeshBatch(ViewDepenedentMeshBatch, DefaultBatchElementMask, StaticMeshBatch->PrimitiveSceneInfo->Proxy, StaticMeshBatch->Id);
			}
			else
			{
				PassMeshProcessor->AddMeshBatch(*StaticMeshBatch, DefaultBatchElementMask, StaticMeshBatch->PrimitiveSceneInfo->Proxy, StaticMeshBatch->Id);
			}

			// Patch the culling payload flags for the generated visible mesh commands.
			// Might be better to pass CullingPayloadFlags through AddMeshBatch() but that will involve a lot of plumbing.
			const EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags = DynamicMeshCommandBuildFlags.IsValidIndex(MeshIndex) ? DynamicMeshCommandBuildFlags[MeshIndex] : EMeshDrawCommandCullingPayloadFlags::NoScreenSizeCull;
			if (CullingPayloadFlags != EMeshDrawCommandCullingPayloadFlags::NoScreenSizeCull)
			{
				const int32 EndCommandIndex = VisibleCommands.Num();
				for (int32 CommandIndex = StartCommandIndex; CommandIndex < EndCommandIndex; ++CommandIndex)
				{
					VisibleCommands[CommandIndex].CullingPayloadFlags = CullingPayloadFlags;
				}
			}
		}

		const int32 NumCommandsGenerated = VisibleCommands.Num() - NumCommandsBefore;
		checkf(NumCommandsGenerated <= MaxNumBuildRequestElements,
			TEXT("Generated %d mesh draw commands for DynamicMeshCommandBuildRequests, while preallocating resources only for %d of them."), NumCommandsGenerated, MaxNumBuildRequestElements);
	}
}

/**
 * Special version of GenerateDynamicMeshDrawCommands for the mobile base pass.
 * Based on CSM visibility it will generate mesh draw commands using either normal base pass processor or CSM base pass processor.
*/
void GenerateMobileBasePassDynamicMeshDrawCommands(
	const FViewInfo& View,
	EShadingPath ShadingPath,
	EMeshPass::Type PassType,
	FMeshPassProcessor* PassMeshProcessor,
	FMeshPassProcessor* MobilePassCSMPassMeshProcessor,
	const TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>& DynamicMeshElements,
	const TArray<FMeshPassMask, SceneRenderingAllocator>* DynamicMeshElementsPassRelevance,
	int32 MaxNumDynamicMeshElements,
	const TArray<const FStaticMeshBatch*, SceneRenderingAllocator>& DynamicMeshCommandBuildRequests,
	const TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator> DynamicMeshCommandBuildFlags,
	int32 MaxNumBuildRequestElements,
	FMeshCommandOneFrameArray& VisibleCommands,
	FDynamicMeshDrawCommandStorage& MeshDrawCommandStorage,
	FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	bool& NeedsShaderInitialisation
)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GenerateMobileBasePassDynamicMeshDrawCommands);
	check(PassMeshProcessor && MobilePassCSMPassMeshProcessor);
	check((PassType == EMeshPass::Num) == (DynamicMeshElementsPassRelevance == nullptr));

	FDynamicPassMeshDrawListContext DynamicPassMeshDrawListContext(
		MeshDrawCommandStorage,
		VisibleCommands,
		GraphicsMinimalPipelineStateSet,
		NeedsShaderInitialisation
	);
	PassMeshProcessor->SetDrawListContext(&DynamicPassMeshDrawListContext);
	MobilePassCSMPassMeshProcessor->SetDrawListContext(&DynamicPassMeshDrawListContext);

	const FMobileCSMVisibilityInfo& MobileCSMVisibilityInfo = View.MobileCSMVisibilityInfo;
	const bool bSkipCSMShaderCulling = MobileBasePassAlwaysUsesCSM(View.GetShaderPlatform());
	
	{
		const int32 NumCommandsBefore = VisibleCommands.Num();
		const int32 NumDynamicMeshBatches = DynamicMeshElements.Num();

		for (int32 MeshIndex = 0; MeshIndex < NumDynamicMeshBatches; MeshIndex++)
		{
			if (!DynamicMeshElementsPassRelevance || (*DynamicMeshElementsPassRelevance)[MeshIndex].Get(PassType))
			{
				const FMeshBatchAndRelevance& MeshAndRelevance = DynamicMeshElements[MeshIndex];
				const uint64 BatchElementMask = ~0ull;

				const int32 PrimitiveIndex = MeshAndRelevance.PrimitiveSceneProxy->GetPrimitiveSceneInfo()->GetIndex();
				if (!bSkipCSMShaderCulling
					&& MobileCSMVisibilityInfo.bMobileDynamicCSMInUse
					&& (MobileCSMVisibilityInfo.bAlwaysUseCSM || MobileCSMVisibilityInfo.MobilePrimitiveCSMReceiverVisibilityMap[PrimitiveIndex]))
				{
					MobilePassCSMPassMeshProcessor->AddMeshBatch(*MeshAndRelevance.Mesh, BatchElementMask, MeshAndRelevance.PrimitiveSceneProxy);
				}
				else
				{
					PassMeshProcessor->AddMeshBatch(*MeshAndRelevance.Mesh, BatchElementMask, MeshAndRelevance.PrimitiveSceneProxy);
				}
			}
		}

		const int32 NumCommandsGenerated = VisibleCommands.Num() - NumCommandsBefore;
		checkf(NumCommandsGenerated <= MaxNumDynamicMeshElements,
			TEXT("Generated %d mesh draw commands for DynamicMeshElements, while preallocating resources only for %d of them."), NumCommandsGenerated, MaxNumDynamicMeshElements);
	}

	{
		const int32 NumCommandsBefore = VisibleCommands.Num();
		const int32 NumStaticMeshBatches = DynamicMeshCommandBuildRequests.Num();
		FMeshBatch ViewDependentMeshBatch{};

		for (int32 MeshIndex = 0; MeshIndex < NumStaticMeshBatches; MeshIndex++)
		{
			const FStaticMeshBatch* StaticMeshBatch = DynamicMeshCommandBuildRequests[MeshIndex];
			const int32 StaticMeshBatchId = StaticMeshBatch->Id;
			const FPrimitiveSceneProxy* Proxy = StaticMeshBatch->PrimitiveSceneInfo->Proxy;
			const int32 StartCommandIndex = VisibleCommands.Num();

			const FMeshBatch* MeshBatch = StaticMeshBatch;
			if (MeshBatch->bViewDependentArguments)
			{
				ViewDependentMeshBatch = *MeshBatch;
				Proxy->ApplyViewDependentMeshArguments(View, ViewDependentMeshBatch);
				MeshBatch = &ViewDependentMeshBatch;
			}

			const int32 PrimitiveIndex = Proxy->GetPrimitiveSceneInfo()->GetIndex();
			if (!bSkipCSMShaderCulling
				&& MobileCSMVisibilityInfo.bMobileDynamicCSMInUse
				&& (MobileCSMVisibilityInfo.bAlwaysUseCSM || MobileCSMVisibilityInfo.MobilePrimitiveCSMReceiverVisibilityMap[PrimitiveIndex]))
			{
				const uint64 DefaultBatchElementMask = ~0ul;
				MobilePassCSMPassMeshProcessor->AddMeshBatch(*MeshBatch, DefaultBatchElementMask, Proxy, StaticMeshBatchId);
			}
			else
			{
				const uint64 DefaultBatchElementMask = ~0ul;
				PassMeshProcessor->AddMeshBatch(*MeshBatch, DefaultBatchElementMask, Proxy, StaticMeshBatchId);
			}

			// Patch the culling payload flags for the generated visible mesh commands.
			// Might be better to pass CullingPayloadFlags through AddMeshBatch() but that will involve a lot of plumbing.
			const EMeshDrawCommandCullingPayloadFlags CullingPayloadFlags = DynamicMeshCommandBuildFlags.IsValidIndex(MeshIndex) ? DynamicMeshCommandBuildFlags[MeshIndex] : EMeshDrawCommandCullingPayloadFlags::NoScreenSizeCull;
			if (CullingPayloadFlags != EMeshDrawCommandCullingPayloadFlags::NoScreenSizeCull)
			{
				const int32 EndCommandIndex = VisibleCommands.Num();
				for (int32 CommandIndex = StartCommandIndex; CommandIndex < EndCommandIndex; ++CommandIndex)
				{
					VisibleCommands[CommandIndex].CullingPayloadFlags = CullingPayloadFlags;
				}
			}
		}

		const int32 NumCommandsGenerated = VisibleCommands.Num() - NumCommandsBefore;
		checkf(NumCommandsGenerated <= MaxNumBuildRequestElements,
			TEXT("Generated %d mesh draw commands for DynamicMeshCommandBuildRequests, while preallocating resources only for %d of them."), NumCommandsGenerated, MaxNumBuildRequestElements);
	}
}

/**
* Apply view overrides to existing mesh draw commands (e.g. reverse culling mode for rendering planar reflections).
* TempVisibleMeshDrawCommands must be presized for NewPassVisibleMeshDrawCommands.
*/
void ApplyViewOverridesToMeshDrawCommands(
	EShadingPath ShadingPath,
	EMeshPass::Type PassType,
	bool bReverseCulling,
	bool bRenderSceneTwoSided,
	FExclusiveDepthStencil::Type BasePassDepthStencilAccess,
	FExclusiveDepthStencil::Type DefaultBasePassDepthStencilAccess,
	FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	FDynamicMeshDrawCommandStorage& MeshDrawCommandStorage,
	FGraphicsMinimalPipelineStateSet& MinimalPipelineStatePassSet,
	bool& NeedsShaderInitialisation,
	FMeshCommandOneFrameArray& TempVisibleMeshDrawCommands
	)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ApplyViewOverridesToMeshDrawCommands);
	check(VisibleMeshDrawCommands.Num() <= TempVisibleMeshDrawCommands.Max() && TempVisibleMeshDrawCommands.Num() == 0 && PassType != EMeshPass::Num);

	if ((FPassProcessorManager::GetPassFlags(ShadingPath, PassType) & EMeshPassFlags::MainView) != EMeshPassFlags::None)
	{
		if (bReverseCulling || bRenderSceneTwoSided || (BasePassDepthStencilAccess != DefaultBasePassDepthStencilAccess && PassType == EMeshPass::BasePass))
		{
			for (int32 MeshCommandIndex = 0; MeshCommandIndex < VisibleMeshDrawCommands.Num(); MeshCommandIndex++)
			{
				MeshDrawCommandStorage.MeshDrawCommands.Add(1);
				FMeshDrawCommand& NewMeshCommand = MeshDrawCommandStorage.MeshDrawCommands[MeshDrawCommandStorage.MeshDrawCommands.Num() - 1];

				const FVisibleMeshDrawCommand& VisibleMeshDrawCommand = VisibleMeshDrawCommands[MeshCommandIndex];
				const FMeshDrawCommand& MeshCommand = *VisibleMeshDrawCommand.MeshDrawCommand;
				NewMeshCommand = MeshCommand;

				const ERasterizerCullMode LocalCullMode = bRenderSceneTwoSided ? CM_None : bReverseCulling ? FMeshPassProcessor::InverseCullMode(VisibleMeshDrawCommand.MeshCullMode) : VisibleMeshDrawCommand.MeshCullMode;

				FGraphicsMinimalPipelineStateInitializer PipelineState = MeshCommand.CachedPipelineId.GetPipelineState(MinimalPipelineStatePassSet);
				PipelineState.RasterizerState = GetStaticRasterizerState<true>(VisibleMeshDrawCommand.MeshFillMode, LocalCullMode);

				if (BasePassDepthStencilAccess != DefaultBasePassDepthStencilAccess && PassType == EMeshPass::BasePass)
				{
					FMeshPassProcessorRenderState PassDrawRenderState;
					SetupBasePassState(BasePassDepthStencilAccess, false, PassDrawRenderState);
					PipelineState.DepthStencilState = PassDrawRenderState.GetDepthStencilState();
				}

				const FGraphicsMinimalPipelineStateId PipelineId = FGraphicsMinimalPipelineStateId::GetPipelineStateId(PipelineState, MinimalPipelineStatePassSet, NeedsShaderInitialisation);
				NewMeshCommand.Finalize(PipelineId, nullptr);

				FVisibleMeshDrawCommand NewVisibleMeshDrawCommand;

				NewVisibleMeshDrawCommand.Setup(
					&NewMeshCommand,
					VisibleMeshDrawCommand.PrimitiveIdInfo,
					VisibleMeshDrawCommand.StateBucketId,
					VisibleMeshDrawCommand.MeshFillMode,
					VisibleMeshDrawCommand.MeshCullMode,
					VisibleMeshDrawCommand.Flags,
					VisibleMeshDrawCommand.SortKey,
					VisibleMeshDrawCommand.CullingPayload,
					VisibleMeshDrawCommand.CullingPayloadFlags,
					VisibleMeshDrawCommand.RunArray,
					VisibleMeshDrawCommand.NumRuns);

				TempVisibleMeshDrawCommands.Add(NewVisibleMeshDrawCommand);
			}

			// Replace VisibleMeshDrawCommands
			Swap(VisibleMeshDrawCommands, TempVisibleMeshDrawCommands);
			TempVisibleMeshDrawCommands.Reset();
		}
	}
}

void CollectMeshDrawCommandPassStats(
	FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	FInstanceCullingContext& InstanceCullingContext)
{	
#if MESH_DRAW_COMMAND_STATS
	FMeshDrawCommandPassStats* PassStats = InstanceCullingContext.MeshDrawCommandPassStats;
	if (PassStats == nullptr)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(CollectMeshDrawCommandPassStats);

	check(VisibleMeshDrawCommands.Num() == InstanceCullingContext.MeshDrawCommandInfos.Num());
		
	PassStats->DrawData.SetNum(VisibleMeshDrawCommands.Num(), EAllowShrinking::No);

	// See the InterlockedAdd on DrawIndirectArgsBufferOut in InstanceCullBuildInstanceIdBufferCS
	int32 InstanceCountMultiplier = (InstanceCullingContext.InstanceCullingMode == EInstanceCullingMode::Stereo) ? 2 : InstanceCullingContext.ViewIds.Num();

	for (int32 DrawCommandIndex = 0; DrawCommandIndex < VisibleMeshDrawCommands.Num(); ++DrawCommandIndex)
	{
		const FVisibleMeshDrawCommand& RESTRICT VisibleMeshDrawCommand = VisibleMeshDrawCommands[DrawCommandIndex];
		const FMeshDrawCommand* RESTRICT MeshDrawCommand = VisibleMeshDrawCommand.MeshDrawCommand;
		const FInstanceCullingContext::FMeshDrawCommandInfo& RESTRICT MeshDrawCommandInfo = InstanceCullingContext.MeshDrawCommandInfos[DrawCommandIndex];

		const bool bSupportsGPUSceneInstancing = EnumHasAnyFlags(VisibleMeshDrawCommand.Flags, EFVisibleMeshDrawCommandFlags::HasPrimitiveIdStreamIndex);

		FVisibleMeshDrawCommandStatsData& DrawData = PassStats->DrawData[DrawCommandIndex];
		MeshDrawCommand->GetStatsData(DrawData);

		bool bUseIndirect = MeshDrawCommandInfo.bUseIndirect;

		// Force disable draw indirect when num primitives is provided and MeshDrawCommand.PrimitiveIdStreamIndex < 0;
		int32 NumInstances = MeshDrawCommandInfo.IndirectArgsOffsetOrNumInstances;
		if (bUseIndirect && MeshDrawCommand->NumPrimitives > 0 && MeshDrawCommand->PrimitiveIdStreamIndex < 0)
		{
			bUseIndirect = false;
			NumInstances = 1;
		}

		if (bUseIndirect || MeshDrawCommand->NumPrimitives == 0)
		{
			// Setup indirect args buffer & offset if provided otherwise use the one from the gpuscene instance culling
			if (MeshDrawCommand->NumPrimitives == 0)
			{
				check(MeshDrawCommand->IndirectArgs.Buffer != nullptr);
				DrawData.CustomIndirectArgsBuffer = MeshDrawCommand->IndirectArgs.Buffer;
				DrawData.IndirectArgsOffset = MeshDrawCommand->IndirectArgs.Offset;

				PassStats->CustomIndirectArgsBuffers.Add(DrawData.CustomIndirectArgsBuffer);
			}
			else if (bSupportsGPUSceneInstancing)
			{
				DrawData.UseInstantCullingIndirectBuffer = 1;
				DrawData.IndirectArgsOffset = MeshDrawCommandInfo.IndirectArgsOffsetOrNumInstances;
				DrawData.NumBatches = MeshDrawCommandInfo.NumBatches;
			}

			// Find out the total instance count
			if (VisibleMeshDrawCommand.RunArray && VisibleMeshDrawCommand.NumRuns > 0)
			{
				DrawData.TotalInstanceCount = 0;
				for (int32 Run = 0; Run < VisibleMeshDrawCommand.NumRuns; Run++)
				{
					DrawData.TotalInstanceCount += (VisibleMeshDrawCommand.RunArray[Run * 2 + 1] - VisibleMeshDrawCommand.RunArray[Run * 2] + 1);
				}
			}
			else
			{
				DrawData.TotalInstanceCount = MeshDrawCommand->NumInstances;
			}

			DrawData.TotalInstanceCount *= InstanceCountMultiplier;

			// By default mark all invisible
			DrawData.VisibleInstanceCount = 0;
		}
		else
		{
			check(VisibleMeshDrawCommand.RunArray == nullptr);
			DrawData.VisibleInstanceCount = MeshDrawCommand->NumInstances * NumInstances;
			DrawData.TotalInstanceCount = MeshDrawCommand->NumInstances * NumInstances;
		}
	}
#endif // MESH_DRAW_COMMAND_STATS
}

FAutoConsoleTaskPriority CPrio_FMeshDrawCommandPassSetupTask(
	TEXT("TaskGraph.TaskPriorities.FMeshDrawCommandPassSetupTask"),
	TEXT("Task and thread priority for FMeshDrawCommandPassSetupTask."),
	ENamedThreads::NormalThreadPriority,
	ENamedThreads::HighTaskPriority
);

FMeshDrawCommandPassSetupTaskContext::FMeshDrawCommandPassSetupTaskContext()
	: View(nullptr)
	, Scene(nullptr)
	, ShadingPath(EShadingPath::Num)
	, PassType(EMeshPass::Num)
	, bUseGPUScene(false)
	, bDynamicInstancing(false)
	, bReverseCulling(false)
	, bRenderSceneTwoSided(false)
	, BasePassDepthStencilAccess(FExclusiveDepthStencil::DepthNop_StencilNop)
	, MeshPassProcessor(nullptr)
	, MobileBasePassCSMMeshPassProcessor(nullptr)
	, DynamicMeshElements(nullptr)
	, InstanceFactor(1)
	, NumDynamicMeshElements(0)
	, NumDynamicMeshCommandBuildRequestElements(0)
	, NeedsShaderInitialisation(false)
	, PrimitiveBounds(nullptr)
	, VisibleMeshDrawCommandsNum(0)
	, NewPassVisibleMeshDrawCommandsNum(0)
	, MaxInstances(1)
{
}

/**
 * Task for a parallel setup of mesh draw commands. Includes generation of dynamic mesh draw commands, sorting, merging etc.
 */
class FMeshDrawCommandPassSetupTask
{
public:
	FMeshDrawCommandPassSetupTask(FMeshDrawCommandPassSetupTaskContext& InContext)
		: Context(InContext)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMeshDrawCommandPassSetupTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return CPrio_FMeshDrawCommandPassSetupTask.Get();
	}

	static ESubsequentsMode::Type GetSubsequentsMode() 
	{ 
		return ESubsequentsMode::TrackSubsequents; 
	}

	void AnyThreadTask()
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
		SCOPED_NAMED_EVENT(MeshDrawCommandPassSetupTask, FColor::Magenta);
		// On SM5 Mobile platform, still want the same sorting
		const bool bMobile = Context.ShadingPath == EShadingPath::Mobile || IsVulkanMobileSM5Platform(Context.ShaderPlatform);
		// Mobile base pass is a special case, as final lists is created from two mesh passes based on CSM visibility.
		const bool bMobileShadingBasePass = Context.ShadingPath == EShadingPath::Mobile && Context.PassType == EMeshPass::BasePass;
		// On SM5 Mobile platform, still want the same sorting
		// Mobile pass expects mesh sorting
		// Also sort if EarlyZPassMode is disabled to reduce overdraw
		const bool bNeedsUpdateMeshSortKeys =
			(bMobile || (Context.Scene->EarlyZPassMode == DDM_None)) &&
			((Context.PassType == EMeshPass::DepthPass) ||
				((Context.PassType == EMeshPass::BasePass) && Context.Scene->EarlyZPassMode != DDM_AllOpaque));

		if (bMobileShadingBasePass)
		{
			MergeMobileBasePassMeshDrawCommands(
				Context.View->MobileCSMVisibilityInfo,
				Context.PrimitiveBounds->Num(),
				Context.MeshDrawCommands,
				Context.MobileBasePassCSMMeshDrawCommands
			);

			GenerateMobileBasePassDynamicMeshDrawCommands(
				*Context.View,
				Context.ShadingPath,
				Context.PassType,
				Context.MeshPassProcessor,
				Context.MobileBasePassCSMMeshPassProcessor,
				*Context.DynamicMeshElements,
				Context.DynamicMeshElementsPassRelevance,
				Context.NumDynamicMeshElements,
				Context.DynamicMeshCommandBuildRequests,
				Context.DynamicMeshCommandBuildFlags,
				Context.NumDynamicMeshCommandBuildRequestElements,
				Context.MeshDrawCommands,
				Context.MeshDrawCommandStorage,
				Context.MinimalPipelineStatePassSet,
				Context.NeedsShaderInitialisation
			);
		}
		else
		{
			GenerateDynamicMeshDrawCommands(
				*Context.View,
				Context.ShadingPath,
				Context.PassType,
				Context.MeshPassProcessor,
				*Context.DynamicMeshElements,
				Context.DynamicMeshElementsPassRelevance,
				Context.NumDynamicMeshElements,
				Context.DynamicMeshCommandBuildRequests,
				Context.DynamicMeshCommandBuildFlags,
				Context.NumDynamicMeshCommandBuildRequestElements,
				Context.MeshDrawCommands,
				Context.MeshDrawCommandStorage,
				Context.MinimalPipelineStatePassSet,
				Context.NeedsShaderInitialisation
			);
		}

		if (Context.MeshDrawCommands.Num() > 0)
		{
			if (Context.PassType != EMeshPass::Num)
			{
				ApplyViewOverridesToMeshDrawCommands(
					Context.ShadingPath,
					Context.PassType,
					Context.bReverseCulling,
					Context.bRenderSceneTwoSided,
					Context.BasePassDepthStencilAccess,
					Context.DefaultBasePassDepthStencilAccess,
					Context.MeshDrawCommands,
					Context.MeshDrawCommandStorage,
					Context.MinimalPipelineStatePassSet,
					Context.NeedsShaderInitialisation,
					Context.TempVisibleMeshDrawCommands
				);
			}

			if (bNeedsUpdateMeshSortKeys)
			{
				UpdateMeshSortKeys(
					bMobile,
					Context.ViewOrigin,
					*Context.PrimitiveBounds,
					Context.MeshDrawCommands
					);
			}
			else if (Context.TranslucencyPass != ETranslucencyPass::TPT_MAX)
			{
				// When per-pixel OIT is enabled, sort primitive from front to back ensure avoid 
				// constantly resorting front-to-back samples list.
				bool bInverseSorting = OIT::IsSortedPixelsEnabled(*Context.View);
				if (Context.TranslucencyPass == ETranslucencyPass::TPT_AllTranslucency ||
					Context.TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandard ||
					Context.TranslucencyPass == ETranslucencyPass::TPT_TranslucencyStandardModulate)
				{
					bInverseSorting &= OIT::IsSortedPixelsEnabledForPass(OITPass_RegularTranslucency);
				}
				else
				{
					bInverseSorting &= OIT::IsSortedPixelsEnabledForPass(OITPass_SeparateTranslucency);
				}

				UpdateTranslucentMeshSortKeys(
					Context.TranslucentSortPolicy,
					Context.TranslucentSortAxis,
					Context.ViewOrigin,
					Context.ViewMatrix,
					*Context.PrimitiveBounds,
					Context.TranslucencyPass,
					bInverseSorting,
					Context.MeshDrawCommands
				);
			}

			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_SortVisibleMeshDrawCommands);
				Context.MeshDrawCommands.Sort(FCompareFMeshDrawCommands());
			}

			if (Context.bUseGPUScene)
			{
				Context.InstanceCullingContext.SetupDrawCommands(
					Context.MeshDrawCommands, 
					true, 
					Context.Scene,
					Context.MaxInstances, 
					Context.VisibleMeshDrawCommandsNum, 
					Context.NewPassVisibleMeshDrawCommandsNum);

				CollectMeshDrawCommandPassStats(Context.MeshDrawCommands, Context.InstanceCullingContext);
			}
		}
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		AnyThreadTask();
	}


private:
	FMeshDrawCommandPassSetupTaskContext& Context;
};

/**
 * Task for shader initialization. This will run on the RenderThread after Commands have been generated.
 */
class FMeshDrawCommandInitResourcesTask
{
public:
	FMeshDrawCommandInitResourcesTask(FMeshDrawCommandPassSetupTaskContext& InContext)
		: Context(InContext)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMeshDrawCommandInitResourcesTask, STATGROUP_TaskGraphTasks);
	}

	ENamedThreads::Type GetDesiredThread()
	{
		return ENamedThreads::GetRenderThread_Local();
	}

	static ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}

	void AnyThreadTask()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshDrawCommandInitResourcesTask);
		if (Context.NeedsShaderInitialisation)
		{
			for (const FGraphicsMinimalPipelineStateInitializer& Initializer : Context.MinimalPipelineStatePassSet)
			{
				Initializer.BoundShaderState.LazilyInitShaders();
			}
		}
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		AnyThreadTask();
	}


private:
	FMeshDrawCommandPassSetupTaskContext& Context;
};

/*
 * Used by various dynamic passes to sort/merge mesh draw commands immediately on a rendering thread.
 */
void SortAndMergeDynamicPassMeshDrawCommands(
	const FSceneView& SceneView,
	FRHICommandList& RHICmdList,
	FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	FDynamicMeshDrawCommandStorage& MeshDrawCommandStorage,
	FRHIBuffer*& OutPrimitiveIdVertexBuffer,
	uint32 InstanceFactor,
	const FGPUScenePrimitiveCollector* DynamicPrimitiveCollector)
{
	const ERHIFeatureLevel::Type FeatureLevel = SceneView.GetFeatureLevel();
	const bool bUseGPUScene = UseGPUScene(SceneView.GetShaderPlatform());

	const int32 NumDrawCommands = VisibleMeshDrawCommands.Num();
	if (NumDrawCommands > 0)
	{
		FMeshCommandOneFrameArray NewPassVisibleMeshDrawCommands;
		int32 MaxInstances = 1;
		int32 VisibleMeshDrawCommandsNum = 0;
		int32 NewPassVisibleMeshDrawCommandsNum = 0;

		VisibleMeshDrawCommands.Sort(FCompareFMeshDrawCommands());

		if (bUseGPUScene)
		{
			// GPUCULL_TODO: workaround for the fact that DrawDynamicMeshPassPrivate et al. don't work with GPU-Scene instancing
			//               we don't support dynamic instancing for this path since we require one primitive per draw command
			//               This is because the stride on the instance data buffer is set to 0 so only the first will ever be fetched.
			const bool bDynamicInstancing = false;

			if (bDynamicInstancing)
			{
				NewPassVisibleMeshDrawCommands.Empty(NumDrawCommands);
			}

			// INDEX_NONE used in TranslatePrimitiveId to defer id translation
			int32 DynamicPrimitiveIdOffset = INDEX_NONE;
			int32 DynamicPrimitiveIdMax = 0;
			if (DynamicPrimitiveCollector)
			{
				const TRange<int32> DynamicPrimitiveIdRange = DynamicPrimitiveCollector->GetPrimitiveIdRange();
				DynamicPrimitiveIdOffset = DynamicPrimitiveIdRange.GetLowerBoundValue();
				DynamicPrimitiveIdMax = DynamicPrimitiveIdRange.GetUpperBoundValue();
			}

			const bool bUsesUniformView = PlatformGPUSceneUsesUniformBufferView(SceneView.GetShaderPlatform());

			const uint32 PrimitiveIdBufferStride = FInstanceCullingContext::GetInstanceIdBufferStride(SceneView.GetShaderPlatform());
			const int32 MaxNumPrimitives = InstanceFactor * NumDrawCommands;
			const int32 PrimitiveIdBufferDataSize = MaxNumPrimitives * PrimitiveIdBufferStride + (bUsesUniformView ? (PLATFORM_MAX_UNIFORM_BUFFER_RANGE - PrimitiveIdBufferStride) : 0);
			FPrimitiveIdVertexBufferPoolEntry Entry = GPrimitiveIdVertexBufferPool.Allocate(RHICmdList, PrimitiveIdBufferDataSize);
			OutPrimitiveIdVertexBuffer = Entry.BufferRHI;
			void* PrimitiveIdBufferData = RHICmdList.LockBuffer(OutPrimitiveIdVertexBuffer, 0, PrimitiveIdBufferDataSize, RLM_WriteOnly);

			if (bUsesUniformView)
			{
				check(PrimitiveIdBufferStride == sizeof(FBatchedPrimitiveShaderData));
				const FPrimitiveUniformShaderParameters* IdentityShaderParams = (const FPrimitiveUniformShaderParameters*)GIdentityPrimitiveUniformBuffer.GetContents();
			
				auto WritePrimitiveDataFn = [&](int32 PrimitiveIndex, int32 DrawPrimitiveId, int32 ScenePrimitiveId)
				{
					checkSlow(PrimitiveIndex < MaxNumPrimitives);
					FBatchedPrimitiveShaderData* PrimitiveData = reinterpret_cast<FBatchedPrimitiveShaderData*>(PrimitiveIdBufferData) + PrimitiveIndex;

					if ((DrawPrimitiveId & GPrimIDDynamicFlag) != 0)
					{
						const FPrimitiveUniformShaderParameters* ShaderParams = DynamicPrimitiveCollector->GetPrimitiveShaderParameters(DrawPrimitiveId);
						if (ShaderParams == nullptr)
						{
							ShaderParams = IdentityShaderParams;
						}
						FBatchedPrimitiveShaderData::Emplace(PrimitiveData, *ShaderParams);
					}
					else
					{
						const FScene* Scene = SceneView.Family->Scene->GetRenderScene();
						if (ScenePrimitiveId >= 0 && ScenePrimitiveId < Scene->Primitives.Num())
						{
							FPrimitiveSceneProxy* PrimitiveProxy = Scene->PrimitiveSceneProxies[ScenePrimitiveId];
							FBatchedPrimitiveShaderData::Emplace(PrimitiveData, PrimitiveProxy);
						}
						else
						{
							FBatchedPrimitiveShaderData::Emplace(PrimitiveData, *IdentityShaderParams);
						}
					}
				};
				
				BuildMeshDrawCommandPrimitiveIdBuffer(
					bDynamicInstancing,
					VisibleMeshDrawCommands,
					MeshDrawCommandStorage,
					NewPassVisibleMeshDrawCommands,
					MaxInstances,
					VisibleMeshDrawCommandsNum,
					NewPassVisibleMeshDrawCommandsNum,
					GShaderPlatformForFeatureLevel[FeatureLevel],
					InstanceFactor,
					WritePrimitiveDataFn);
			}
			else
			{
				int32* RESTRICT PrimitiveIds = reinterpret_cast<int32*>(PrimitiveIdBufferData);
			
				auto WritePrimitiveDataFn = [&](int32 PrimitiveIndex, int32 DrawPrimitiveId, int32 ScenePrimitiveId) 
				{
					checkSlow(PrimitiveIndex < MaxNumPrimitives);
					PrimitiveIds[PrimitiveIndex] = TranslatePrimitiveId(DrawPrimitiveId, DynamicPrimitiveIdOffset, DynamicPrimitiveIdMax);
				};

				
				BuildMeshDrawCommandPrimitiveIdBuffer(
					bDynamicInstancing,
					VisibleMeshDrawCommands,
					MeshDrawCommandStorage,
					NewPassVisibleMeshDrawCommands,
					MaxInstances,
					VisibleMeshDrawCommandsNum,
					NewPassVisibleMeshDrawCommandsNum,
					GShaderPlatformForFeatureLevel[FeatureLevel],
					InstanceFactor,
					WritePrimitiveDataFn);
			}

			RHICmdList.UnlockBuffer(OutPrimitiveIdVertexBuffer);
			GPrimitiveIdVertexBufferPool.ReturnToFreeList(Entry);
		}
	}
}



void FParallelMeshDrawCommandPass::DispatchPassSetup(
	FScene* Scene,
	const FViewInfo& View,
	FInstanceCullingContext&& InstanceCullingContext,
	EMeshPass::Type PassType,
	FExclusiveDepthStencil::Type BasePassDepthStencilAccess,
	FMeshPassProcessor* MeshPassProcessor,
	const TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>& DynamicMeshElements,
	const TArray<FMeshPassMask, SceneRenderingAllocator>* DynamicMeshElementsPassRelevance,
	int32 NumDynamicMeshElements,
	TArray<const FStaticMeshBatch*, SceneRenderingAllocator>& InOutDynamicMeshCommandBuildRequests,
	TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator> InOutDynamicMeshCommandBuildFlags,
	int32 NumDynamicMeshCommandBuildRequestElements,
	FMeshCommandOneFrameArray& InOutMeshDrawCommands,
	FMeshPassProcessor* MobileBasePassCSMMeshPassProcessor,
	FMeshCommandOneFrameArray* InOutMobileBasePassCSMMeshDrawCommands
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParallelMdcDispatchPassSetup);
	check(!TaskEventRef.IsValid() && MeshPassProcessor != nullptr);
	check((PassType == EMeshPass::Num) == (DynamicMeshElementsPassRelevance == nullptr));

	MaxNumDraws = InOutMeshDrawCommands.Num() + NumDynamicMeshElements + NumDynamicMeshCommandBuildRequestElements;

	TaskContext.MeshPassProcessor = MeshPassProcessor;
	TaskContext.MobileBasePassCSMMeshPassProcessor = MobileBasePassCSMMeshPassProcessor;
	TaskContext.DynamicMeshElements = &DynamicMeshElements;
	TaskContext.DynamicMeshElementsPassRelevance = DynamicMeshElementsPassRelevance;

	TaskContext.View = &View;
	TaskContext.Scene = Scene;
	TaskContext.ShadingPath = GetFeatureLevelShadingPath(View.GetFeatureLevel());
	TaskContext.ShaderPlatform = Scene->GetShaderPlatform();
	TaskContext.PassType = PassType;
	TaskContext.bUseGPUScene = UseGPUScene(View.GetShaderPlatform());
	TaskContext.bDynamicInstancing = IsDynamicInstancingEnabled(View.GetFeatureLevel());
	TaskContext.bReverseCulling = View.bReverseCulling;
	TaskContext.bRenderSceneTwoSided = View.bRenderSceneTwoSided;
	TaskContext.BasePassDepthStencilAccess = BasePassDepthStencilAccess;
	TaskContext.DefaultBasePassDepthStencilAccess = Scene->DefaultBasePassDepthStencilAccess;
	TaskContext.NumDynamicMeshElements = NumDynamicMeshElements;
	TaskContext.NumDynamicMeshCommandBuildRequestElements = NumDynamicMeshCommandBuildRequestElements;

	// Only apply instancing for ISR to main view passes

	const bool bIsMainViewPass = PassType != EMeshPass::Num && (FPassProcessorManager::GetPassFlags(TaskContext.ShadingPath, TaskContext.PassType) & EMeshPassFlags::MainView) != EMeshPassFlags::None;
	// GPUCULL_TODO: Note the InstanceFactor is ignored by the GPU-Scene supported instances, but is used for legacy primitives.
	TaskContext.InstanceFactor = (bIsMainViewPass && View.IsInstancedStereo() && IStereoRendering::IsAPrimaryView(View)) ? 2 : 1;

	TaskContext.InstanceCullingContext = MoveTemp(InstanceCullingContext); 

	// Setup translucency sort key update pass based on view.
	TaskContext.TranslucencyPass = ETranslucencyPass::TPT_MAX;
	TaskContext.TranslucentSortPolicy = View.TranslucentSortPolicy;
	TaskContext.TranslucentSortAxis = View.TranslucentSortAxis;
	TaskContext.ViewOrigin = View.ViewMatrices.GetViewOrigin();
	TaskContext.ViewMatrix = View.ViewMatrices.GetWorldToView();
	TaskContext.PrimitiveBounds = &Scene->PrimitiveBounds;

	switch (PassType)
	{
		case EMeshPass::TranslucencyStandard: TaskContext.TranslucencyPass			= ETranslucencyPass::TPT_TranslucencyStandard; break;
		case EMeshPass::TranslucencyStandardModulate: TaskContext.TranslucencyPass	= ETranslucencyPass::TPT_TranslucencyStandardModulate; break;
		case EMeshPass::TranslucencyAfterDOF: TaskContext.TranslucencyPass			= ETranslucencyPass::TPT_TranslucencyAfterDOF; break;
		case EMeshPass::TranslucencyAfterDOFModulate: TaskContext.TranslucencyPass	= ETranslucencyPass::TPT_TranslucencyAfterDOFModulate; break;
		case EMeshPass::TranslucencyAfterMotionBlur: TaskContext.TranslucencyPass	= ETranslucencyPass::TPT_TranslucencyAfterMotionBlur; break;
		case EMeshPass::TranslucencyHoldout: TaskContext.TranslucencyPass			= ETranslucencyPass::TPT_TranslucencyHoldout; break;
		case EMeshPass::TranslucencyAll: TaskContext.TranslucencyPass				= ETranslucencyPass::TPT_AllTranslucency; break;
	}

	Swap(TaskContext.MeshDrawCommands, InOutMeshDrawCommands);
	Swap(TaskContext.DynamicMeshCommandBuildRequests, InOutDynamicMeshCommandBuildRequests);
	Swap(TaskContext.DynamicMeshCommandBuildFlags, InOutDynamicMeshCommandBuildFlags);

	if (TaskContext.ShadingPath == EShadingPath::Mobile && TaskContext.PassType == EMeshPass::BasePass)
	{
		Swap(TaskContext.MobileBasePassCSMMeshDrawCommands, *InOutMobileBasePassCSMMeshDrawCommands);
	}
	else
	{
		check(MobileBasePassCSMMeshPassProcessor == nullptr && InOutMobileBasePassCSMMeshDrawCommands == nullptr);
	}

	if (MaxNumDraws > 0)
	{
		// Preallocate resources on rendering thread based on MaxNumDraws.
		TaskContext.MeshDrawCommands.Reserve(MaxNumDraws);
		TaskContext.TempVisibleMeshDrawCommands.Reserve(MaxNumDraws);

		const bool bExecuteInParallel = FApp::ShouldUseThreadingForPerformance()
			&& CVarMeshDrawCommandsParallelPassSetup.GetValueOnRenderThread() > 0
			&& GIsThreadedRendering; // Rendering thread is required to safely use rendering resources in parallel.

		if (bExecuteInParallel)
		{
			if (IsOnDemandShaderCreationEnabled())
			{
				TaskEventRef = TGraphTask<FMeshDrawCommandPassSetupTask>::CreateTask().ConstructAndDispatchWhenReady(TaskContext);
			}
			else
			{
				FGraphEventArray DependentGraphEvents;
				DependentGraphEvents.Add(TGraphTask<FMeshDrawCommandPassSetupTask>::CreateTask().ConstructAndDispatchWhenReady(TaskContext));
				TaskEventRef = TGraphTask<FMeshDrawCommandInitResourcesTask>::CreateTask(&DependentGraphEvents).ConstructAndDispatchWhenReady(TaskContext);
			}
		}
		else
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_MeshPassSetupImmediate);
			FMeshDrawCommandPassSetupTask Task(TaskContext);
			Task.AnyThreadTask();
			if (!IsOnDemandShaderCreationEnabled())
			{
				FMeshDrawCommandInitResourcesTask DependentTask(TaskContext);
				DependentTask.AnyThreadTask();
			}
		}

		// This work needs to be deferred until at least BuildRenderingCommands (to ensure the DynamicPrimitiveCollector is uploaded), so we use the async mechanism either way 
		auto FinalizeInstanceCullingSetup = [this, Scene](FInstanceCullingContext& InstanceCullingContext)
		{
			WaitForSetupTask();

#if DO_CHECK
			for (const FVisibleMeshDrawCommand& VisibleMeshDrawCommand : TaskContext.MeshDrawCommands)
			{
				if (VisibleMeshDrawCommand.PrimitiveIdInfo.bIsDynamicPrimitive)
				{
					uint32 PrimitiveIndex = VisibleMeshDrawCommand.PrimitiveIdInfo.DrawPrimitiveId & ~GPrimIDDynamicFlag;
					TaskContext.View->DynamicPrimitiveCollector.CheckPrimitiveProcessed(PrimitiveIndex, Scene->GPUScene);
				}
			}
#endif
			InstanceCullingContext.SetDynamicPrimitiveInstanceOffsets(TaskContext.View->DynamicPrimitiveCollector.GetInstanceSceneDataOffset(), TaskContext.View->DynamicPrimitiveCollector.NumInstances());
		};
		TaskContext.InstanceCullingContext.BeginAsyncSetup(FinalizeInstanceCullingSetup);
	}
}

bool FParallelMeshDrawCommandPass::IsOnDemandShaderCreationEnabled()
{
	// GL rhi does not support multithreaded shader creation, however the engine can be configured to not run mesh drawing tasks in threads other than the RT 
	// (see FRHICommandListExecutor::UseParallelAlgorithms()): if this condition is true, on demand shader creation can be enabled.
	const bool bIsMobileRenderer = GetFeatureLevelShadingPath(GMaxRHIFeatureLevel) == EShadingPath::Mobile;
	return GAllowOnDemandShaderCreation && 
		(GRHISupportsMultithreadedShaderCreation || (bIsMobileRenderer && (!GSupportsParallelRenderingTasksWithSeparateRHIThread && IsRunningRHIInSeparateThread())));
}

void FParallelMeshDrawCommandPass::WaitForSetupTask()
{
	if (TaskEventRef.IsValid())
	{
		// Need to wait on GetRenderThread_Local, as mesh pass setup task can wait on rendering thread inside InitResourceFromPossiblyParallelRendering().
		QUICK_SCOPE_CYCLE_COUNTER(STAT_WaitForMeshPassSetupTask);
		TaskEventRef->Wait(IsInActualRenderingThread() ? ENamedThreads::GetRenderThread_Local() : ENamedThreads::AnyThread);
		TaskEventRef = nullptr;
	}
}

void FParallelMeshDrawCommandPass::Cleanup()
{
	check(TaskEventRef == nullptr);
	DumpInstancingStats();

	if (TaskContext.MeshPassProcessor)
	{
		delete TaskContext.MeshPassProcessor;
		TaskContext.MeshPassProcessor = nullptr;
	}
	if (TaskContext.MobileBasePassCSMMeshPassProcessor)
	{
		delete TaskContext.MobileBasePassCSMMeshPassProcessor;
		TaskContext.MobileBasePassCSMMeshPassProcessor = nullptr;
	}

	MaxNumDraws = 0;
	PassNameForStats.Empty();

	TaskContext.DynamicMeshElements = nullptr;
	TaskContext.DynamicMeshElementsPassRelevance = nullptr;
	TaskContext.MeshDrawCommands.Empty();
	TaskContext.MeshDrawCommandStorage.MeshDrawCommands.Empty();
	FGraphicsMinimalPipelineStateId::AddSizeToLocalPipelineIdTableSize(TaskContext.MinimalPipelineStatePassSet.GetAllocatedSize());
	TaskContext.MinimalPipelineStatePassSet.Empty();
	TaskContext.MobileBasePassCSMMeshDrawCommands.Empty();
	TaskContext.DynamicMeshCommandBuildRequests.Empty();
	TaskContext.TempVisibleMeshDrawCommands.Empty();
}

FParallelMeshDrawCommandPass::~FParallelMeshDrawCommandPass()
{
	check(TaskEventRef == nullptr);
}

class FDrawVisibleMeshCommandsAnyThreadTask : public FRenderTask
{
	FRHICommandList& RHICmdList;
	const FInstanceCullingContext& InstanceCullingContext;
	const FMeshCommandOneFrameArray& VisibleMeshDrawCommands;
	const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet;
	const FMeshDrawCommandOverrideArgs OverrideArgs;
	uint32 InstanceFactor;
	int32 TaskIndex;
	int32 TaskNum;

public:

	FDrawVisibleMeshCommandsAnyThreadTask(
		FRHICommandList& InRHICmdList,
		const FInstanceCullingContext& InInstanceCullingContext,
		const FMeshCommandOneFrameArray& InVisibleMeshDrawCommands,
		const FGraphicsMinimalPipelineStateSet& InGraphicsMinimalPipelineStateSet,
		const FMeshDrawCommandOverrideArgs& InOverrideArgs,
		uint32 InInstanceFactor,
		int32 InTaskIndex,
		int32 InTaskNum
	)
		: RHICmdList(InRHICmdList)
		, InstanceCullingContext(InInstanceCullingContext)
		, VisibleMeshDrawCommands(InVisibleMeshDrawCommands)
		, GraphicsMinimalPipelineStateSet(InGraphicsMinimalPipelineStateSet)
		, OverrideArgs(InOverrideArgs)
		, InstanceFactor(InInstanceFactor)
		, TaskIndex(InTaskIndex)
		, TaskNum(InTaskNum)
	{
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FDrawVisibleMeshCommandsAnyThreadTask, STATGROUP_TaskGraphTasks);
	}

	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
		SCOPED_NAMED_EVENT_TEXT("DrawVisibleMeshCommandsAnyThreadTask", FColor::Magenta);
		
		checkSlow(RHICmdList.IsInsideRenderPass() || RHICmdList.IsSubCommandList());

		// check for the multithreaded shader creation has been moved to FShaderCodeArchive::CreateShader() 
		{
			SCOPED_DRAW_EVENTF(RHICmdList, ParallelDraw, TEXT("ParallelDraw (Index: %d, Num: %d)"), TaskIndex, TaskNum);

			// Recompute draw range.
			const int32 DrawNum = VisibleMeshDrawCommands.Num();
			const int32 NumDrawsPerTask = TaskIndex < DrawNum ? FMath::DivideAndRoundUp(DrawNum, TaskNum) : 0;
			const int32 StartIndex = TaskIndex * NumDrawsPerTask;
			const int32 NumDraws = FMath::Min(NumDrawsPerTask, DrawNum - StartIndex);

			InstanceCullingContext.SubmitDrawCommands(
				VisibleMeshDrawCommands,
				GraphicsMinimalPipelineStateSet,
				OverrideArgs,
				StartIndex,
				NumDraws,
				InstanceFactor,
				RHICmdList);
		}

		if(!RHICmdList.IsSubCommandList())
		{
			RHICmdList.EndRenderPass();
		}
		RHICmdList.FinishRecording();
	}
};

void FParallelMeshDrawCommandPass::BuildRenderingCommands(
	FRDGBuilder& GraphBuilder,
	const FGPUScene& GPUScene,
	FInstanceCullingDrawParams& OutInstanceCullingDrawParams)
{
	if (TaskContext.InstanceCullingContext.IsEnabled())
	{
		check(!bHasInstanceCullingDrawParameters);

		if (CVarDeferredMeshPassSetupTaskSync.GetValueOnRenderThread() != 0)
		{
			TaskContext.InstanceCullingContext.BuildRenderingCommands(GraphBuilder, GPUScene, &OutInstanceCullingDrawParams);
		}
		else
		{
			TaskContext.InstanceCullingContext.WaitForSetupTask();
			TaskContext.InstanceCullingContext.BuildRenderingCommands(GraphBuilder, GPUScene, &OutInstanceCullingDrawParams);
		}

		bHasInstanceCullingDrawParameters = true;
		return;
	}
	OutInstanceCullingDrawParams.DrawIndirectArgsBuffer = nullptr;
	OutInstanceCullingDrawParams.InstanceIdOffsetBuffer = nullptr;
	OutInstanceCullingDrawParams.InstanceDataByteOffset = 0U;
	OutInstanceCullingDrawParams.IndirectArgsByteOffset = 0U;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void FParallelMeshDrawCommandPass::DispatchDraw(FParallelCommandListSet* ParallelCommandListSet, FRHICommandList& RHICmdList, const FInstanceCullingDrawParams* InstanceCullingDrawParams) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParallelMdcDispatchDraw);
	if (MaxNumDraws <= 0)
	{
		return;
	}

	if (ParallelCommandListSet)
	{
		FMeshDrawCommandOverrideArgs OverrideArgs; 
		if (InstanceCullingDrawParams)
		{
			OverrideArgs = GetMeshDrawCommandOverrideArgs(*InstanceCullingDrawParams);
		}

		const ENamedThreads::Type RenderThread = ENamedThreads::GetRenderThread();

		FGraphEventArray Prereqs;
		if (ParallelCommandListSet->GetPrereqs())
		{
			Prereqs.Append(*ParallelCommandListSet->GetPrereqs());
		}
		if (TaskEventRef.IsValid())
		{
			Prereqs.Add(TaskEventRef);
		}

		// Distribute work evenly to the available task graph workers based on NumEstimatedDraws.
		// Every task will then adjust it's working range based on FVisibleMeshDrawCommandProcessTask results.
		const int32 NumThreads = FMath::Min<int32>(FTaskGraphInterface::Get().GetNumWorkerThreads(), ParallelCommandListSet->Width);
		const int32 NumTasks = FMath::Min<int32>(NumThreads, FMath::DivideAndRoundUp(MaxNumDraws, ParallelCommandListSet->MinDrawsPerCommandList));
		const int32 NumDrawsPerTask = FMath::DivideAndRoundUp(MaxNumDraws, NumTasks);

		for (int32 TaskIndex = 0; TaskIndex < NumTasks; TaskIndex++)
		{
			const int32 StartIndex = TaskIndex * NumDrawsPerTask;
			const int32 NumDraws = FMath::Min(NumDrawsPerTask, MaxNumDraws - StartIndex);
			checkSlow(NumDraws > 0);

			FRHICommandList* CmdList = ParallelCommandListSet->NewParallelCommandList();

			TGraphTask<FDrawVisibleMeshCommandsAnyThreadTask>::CreateTask(&Prereqs, RenderThread)
				.ConstructAndDispatchWhenReady(*CmdList, TaskContext.InstanceCullingContext, TaskContext.MeshDrawCommands, TaskContext.MinimalPipelineStatePassSet,
					OverrideArgs,
					TaskContext.InstanceFactor,
					TaskIndex, NumTasks);

			ParallelCommandListSet->AddParallelCommandList(CmdList);
		}
	}
	else
	{
		Draw(RHICmdList, InstanceCullingDrawParams);
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FParallelMeshDrawCommandPass::Draw(FRHICommandList& RHICmdList, const FInstanceCullingDrawParams* InstanceCullingDrawParams) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParallelMdcDispatchDraw);
	if (MaxNumDraws <= 0)
	{
		return;
	}

	FMeshDrawCommandOverrideArgs OverrideArgs; 
	if (InstanceCullingDrawParams)
	{
		OverrideArgs = GetMeshDrawCommandOverrideArgs(*InstanceCullingDrawParams);
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_MeshPassDrawImmediate);

	check(TaskEventRef == nullptr);

	if (TaskContext.bUseGPUScene)
	{
		if (TaskContext.MeshDrawCommands.Num() > 0)
		{
			TaskContext.InstanceCullingContext.SubmitDrawCommands(
				TaskContext.MeshDrawCommands,
				TaskContext.MinimalPipelineStatePassSet,
				OverrideArgs,
				0,
				TaskContext.MeshDrawCommands.Num(),
				TaskContext.InstanceFactor,
				RHICmdList);
		}
	}
	else
	{
		FMeshDrawCommandSceneArgs SceneArgs;
		SubmitMeshDrawCommandsRange(TaskContext.MeshDrawCommands, TaskContext.MinimalPipelineStatePassSet, SceneArgs, 0, TaskContext.bDynamicInstancing, 0, TaskContext.MeshDrawCommands.Num(), TaskContext.InstanceFactor, RHICmdList);
	}
}

void FParallelMeshDrawCommandPass::Dispatch(FRDGDispatchPassBuilder& DispatchPassBuilder, const FInstanceCullingDrawParams* InstanceCullingDrawParams, float ViewportScale) const
{
	Dispatch(DispatchPassBuilder, InstanceCullingDrawParams, [View = TaskContext.View, ViewportScale] (FRHICommandList& RHICmdList)
	{
		FSceneRenderer::SetStereoViewport(RHICmdList, *View, ViewportScale);
	});
}

void FParallelMeshDrawCommandPass::Dispatch(FRDGDispatchPassBuilder& DispatchPassBuilder, const FInstanceCullingDrawParams* InstanceCullingDrawParams, TFunctionRef<void(FRHICommandList&)> SetupCommandListFunction) const
{
	extern TAutoConsoleVariable<int32> CVarRHICmdMinDrawsPerParallelCmdList;

	TRACE_CPUPROFILER_EVENT_SCOPE(ParallelMdcDispatchDraw);
	if (MaxNumDraws <= 0)
	{
		return;
	}

	check(TaskContext.View);

	FMeshDrawCommandOverrideArgs OverrideArgs; 
	if (InstanceCullingDrawParams)
	{
		OverrideArgs = GetMeshDrawCommandOverrideArgs(*InstanceCullingDrawParams);
	}

	FGraphEventArray Prereqs;
	if (TaskEventRef.IsValid())
	{
		Prereqs.Add(TaskEventRef);
	}

	// Distribute work evenly to the available task graph workers based on NumEstimatedDraws.
	// Every task will then adjust it's working range based on FVisibleMeshDrawCommandProcessTask results.
	const int32 NumThreads = FMath::Min<int32>(FTaskGraphInterface::Get().GetNumWorkerThreads(), CVarRHICmdWidth.GetValueOnRenderThread());
	const int32 NumTasks = FMath::Min<int32>(NumThreads, FMath::DivideAndRoundUp(MaxNumDraws, CVarRHICmdMinDrawsPerParallelCmdList.GetValueOnRenderThread()));
	const int32 NumDrawsPerTask = FMath::DivideAndRoundUp(MaxNumDraws, NumTasks);

	for (int32 TaskIndex = 0; TaskIndex < NumTasks; TaskIndex++)
	{
		const int32 StartIndex = TaskIndex * NumDrawsPerTask;
		const int32 NumDraws = FMath::Min(NumDrawsPerTask, MaxNumDraws - StartIndex);
		checkSlow(NumDraws > 0);

		FRHICommandList* RHICmdList = DispatchPassBuilder.CreateCommandList();
		SetupCommandListFunction(*RHICmdList);

		TGraphTask<FDrawVisibleMeshCommandsAnyThreadTask>::CreateTask(&Prereqs)
			.ConstructAndDispatchWhenReady(*RHICmdList, TaskContext.InstanceCullingContext, TaskContext.MeshDrawCommands, TaskContext.MinimalPipelineStatePassSet,
				OverrideArgs,
				TaskContext.InstanceFactor,
				TaskIndex, NumTasks);
	}
}

void FParallelMeshDrawCommandPass::DumpInstancingStats() const
{
	if (!PassNameForStats.IsEmpty() && TaskContext.VisibleMeshDrawCommandsNum > 0)
	{
		UE_LOGF(LogRenderer, Log, "Instancing stats for %ls", *PassNameForStats);
		UE_LOGF(LogRenderer, Log, "   %i Mesh Draw Commands in %i instancing state buckets", TaskContext.VisibleMeshDrawCommandsNum, TaskContext.NewPassVisibleMeshDrawCommandsNum);
		UE_LOGF(LogRenderer, Log, "   Largest %i", TaskContext.MaxInstances);
		UE_LOGF(LogRenderer, Log, "   %.1f Dynamic Instancing draw call reduction factor", TaskContext.VisibleMeshDrawCommandsNum / (float)TaskContext.NewPassVisibleMeshDrawCommandsNum);
	}
}

void FParallelMeshDrawCommandPass::SetDumpInstancingStats(const FString& InPassNameForStats)
{
	PassNameForStats = InPassNameForStats;
}
