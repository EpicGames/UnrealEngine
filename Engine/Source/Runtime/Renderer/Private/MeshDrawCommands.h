// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
MeshDrawCommands.h: Mesh draw commands.
=============================================================================*/

#pragma once

#include "MeshPassProcessor.h"
#include "TranslucentPassResource.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "ScenePrivateBase.h"

struct FMeshBatchAndRelevance;
class FStaticMeshBatch;
class FParallelCommandListSet;
class FInstanceCullingManager;

/**
 * Global vertex buffer pool used for GPUScene primitive id arrays.
 */

struct FPrimitiveIdVertexBufferPoolEntry
{
	int32 BufferSize = 0;
	uint32 LastDiscardId = 0;
	FBufferRHIRef BufferRHI;
};

class FPrimitiveIdVertexBufferPool : public FRenderResource
{
public:
	FPrimitiveIdVertexBufferPool();
	~FPrimitiveIdVertexBufferPool();

	FPrimitiveIdVertexBufferPoolEntry Allocate(FRHICommandList& RHICmdList, int32 BufferSize);
	void ReturnToFreeList(FPrimitiveIdVertexBufferPoolEntry Entry);
	RENDERER_API void DiscardAll();

	virtual void ReleaseRHI() override;

private:
	uint32 DiscardId;
	TArray<FPrimitiveIdVertexBufferPoolEntry> Entries;
	FCriticalSection AllocationCS;
};

extern RENDERER_API TGlobalResource<FPrimitiveIdVertexBufferPool> GPrimitiveIdVertexBufferPool;

/**	
 * Parallel mesh draw command pass setup task context.
 */
class FMeshDrawCommandPassSetupTaskContext
{
public:
	FMeshDrawCommandPassSetupTaskContext();

	const FViewInfo* View;
	const FScene* Scene;
	EShadingPath ShadingPath;
	EShaderPlatform ShaderPlatform;
	EMeshPass::Type PassType;
	bool bUseGPUScene;
	bool bDynamicInstancing;
	bool bReverseCulling;
	bool bRenderSceneTwoSided;
	FExclusiveDepthStencil::Type BasePassDepthStencilAccess;
	FExclusiveDepthStencil::Type DefaultBasePassDepthStencilAccess;

	// Mesh pass processor.
	FMeshPassProcessor* MeshPassProcessor;
	FMeshPassProcessor* MobileBasePassCSMMeshPassProcessor;
	const TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>* DynamicMeshElements;
	const TArray<FMeshPassMask, SceneRenderingAllocator>* DynamicMeshElementsPassRelevance;

	// Commands.
	int32 InstanceFactor;
	int32 NumDynamicMeshElements;
	int32 NumDynamicMeshCommandBuildRequestElements;
	FMeshCommandOneFrameArray MeshDrawCommands;
	FMeshCommandOneFrameArray MobileBasePassCSMMeshDrawCommands;
	TArray<const FStaticMeshBatch*, SceneRenderingAllocator> DynamicMeshCommandBuildRequests;
	TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator> DynamicMeshCommandBuildFlags;
	TArray<const FStaticMeshBatch*, SceneRenderingAllocator> MobileBasePassCSMDynamicMeshCommandBuildRequests;
	FDynamicMeshDrawCommandStorage MeshDrawCommandStorage;
	FGraphicsMinimalPipelineStateSet MinimalPipelineStatePassSet;
	bool NeedsShaderInitialisation;

	// Resources preallocated on rendering thread.
	FMeshCommandOneFrameArray TempVisibleMeshDrawCommands;

	// For UpdateTranslucentMeshSortKeys.
	ETranslucencyPass::Type TranslucencyPass;
	ETranslucentSortPolicy::Type TranslucentSortPolicy;
	FVector TranslucentSortAxis;
	FVector ViewOrigin;
	FMatrix ViewMatrix;
	const TScenePrimitiveArray<struct FPrimitiveBounds>* PrimitiveBounds;

	// For logging instancing stats.
	int32 VisibleMeshDrawCommandsNum;
	int32 NewPassVisibleMeshDrawCommandsNum;
	int32 MaxInstances;

	FInstanceCullingContext InstanceCullingContext;
};

/**
 * Parallel mesh draw command processing and rendering. 
 * Encapsulates two parallel tasks - mesh command setup task and drawing task.
 */
class FParallelMeshDrawCommandPass
{
public:
	FParallelMeshDrawCommandPass()
		: bHasInstanceCullingDrawParameters(false)
		, MaxNumDraws(0)
	{
	}

	~FParallelMeshDrawCommandPass();

	/**
	 * Dispatch visible mesh draw command process task, which prepares this pass for drawing.
	 * This includes generation of dynamic mesh draw commands, draw sorting and draw merging.
	 */
	void DispatchPassSetup(
		FScene* Scene,
		const FViewInfo& View, 
		FInstanceCullingContext &&InstanceCullingContext,
		EMeshPass::Type PassType, 
		FExclusiveDepthStencil::Type BasePassDepthStencilAccess,
		FMeshPassProcessor* MeshPassProcessor,
		const TArray<FMeshBatchAndRelevance, SceneRenderingAllocator>& DynamicMeshElements,
		const TArray<FMeshPassMask, SceneRenderingAllocator>* DynamicMeshElementsPassRelevance,
		int32 NumDynamicMeshElements,
		TArray<const FStaticMeshBatch*, SceneRenderingAllocator>& InOutDynamicMeshCommandBuildRequests,
		TArray<EMeshDrawCommandCullingPayloadFlags, SceneRenderingAllocator> InOutDynamicMeshCommandBuildFlags,
		int32 NumDynamicMeshCommandBuildRequestElements,
		FMeshCommandOneFrameArray& InOutMeshDrawCommands,
		FMeshPassProcessor* MobileBasePassCSMMeshPassProcessor = nullptr, // Required only for the mobile base pass.
		FMeshCommandOneFrameArray* InOutMobileBasePassCSMMeshDrawCommands = nullptr // Required only for the mobile base pass.
	);

	/**
	 * Sync with setup task and run post-instance culling job to create the render commands and instance ID lists and optionally vertex instance data.
	 * Needs to happen after DispatchPassSetup and before DispatchDraw, but not before global instance culling has been done.
	 */
	void BuildRenderingCommands(
		FRDGBuilder& GraphBuilder,
		const FGPUScene& GPUScene,
		FInstanceCullingDrawParams& OutInstanceCullingDrawParams);

	/**
	 * Sync with setup task.
	 */
	void WaitForSetupTask();

	/**
	 * Dispatch visible mesh draw command draw task.
	 */
	void Draw(FRHICommandList& RHICmdList, const FInstanceCullingDrawParams* InstanceCullingDrawParams = nullptr) const;
	void Dispatch(FRDGDispatchPassBuilder& DispatchPassBuilder, const FInstanceCullingDrawParams* InstanceCullingDrawParams = nullptr, float ViewportScale = 1.0f) const;
	void Dispatch(FRDGDispatchPassBuilder& DispatchPassBuilder, const FInstanceCullingDrawParams* InstanceCullingDrawParams, TFunctionRef<void(FRHICommandList&)> SetupCommandListFunction) const;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "Use Draw / Dispatch instead.")
	void DispatchDraw(FParallelCommandListSet* ParallelCommandListSet, FRHICommandList& RHICmdList, const FInstanceCullingDrawParams* InstanceCullingDrawParams = nullptr) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void Cleanup();
	void SetDumpInstancingStats(const FString& InPassName);
	bool HasAnyDraw() const { return MaxNumDraws > 0; }

	static bool IsOnDemandShaderCreationEnabled();

	FInstanceCullingContext* GetInstanceCullingContext() { return &TaskContext.InstanceCullingContext; }
	const FGraphEventRef& GetTaskEvent() const { return TaskEventRef; }

	FGraphEventRef&& AcquireTaskEvent() { return MoveTemp(TaskEventRef); }

	// NOTE: It is only safe to access mesh draw commands after the setup task is complete (use WaitForSetupTask). 
	// Only access the data late in the frame to allow as much time as possible for async tasks to complete.
	const FMeshCommandOneFrameArray& GetMeshDrawCommands() const
	{
		return TaskContext.MeshDrawCommands;
	}

private:
	FMeshDrawCommandPassSetupTaskContext TaskContext;
	FGraphEventRef TaskEventRef;
	FString PassNameForStats;

	bool bHasInstanceCullingDrawParameters;

	// Maximum number of draws for this pass. Used to prealocate resources on rendering thread. 
	// Has a guarantee that if there won't be any draws, then MaxNumDraws = 0;
	int32 MaxNumDraws;

	void DumpInstancingStats() const;
};

inline bool HasAnyDraw(const FParallelMeshDrawCommandPass* Pass)
{
	return Pass && Pass->HasAnyDraw();
}

RENDERER_API extern void SortAndMergeDynamicPassMeshDrawCommands(
	const FSceneView& SceneView,
	FRHICommandList& RHICmdList,
	FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	FDynamicMeshDrawCommandStorage& MeshDrawCommandStorage,
	FRHIBuffer*& OutPrimitiveIdVertexBuffer,
	uint32 InstanceFactor,
	const FGPUScenePrimitiveCollector* DynamicPrimitiveCollector);
