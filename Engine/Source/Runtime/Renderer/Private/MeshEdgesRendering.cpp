// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshEdgesRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Scene.h"
#include "SceneInterface.h"
#include "RenderingThread.h"
#include "RHIStaticStates.h"
#include "RendererInterface.h"
#include "Camera/CameraTypes.h"
#include "Shader.h"
#include "TextureResource.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "ScenePrivateBase.h"
#include "PostProcess/SceneRenderTargets.h"
#include "GlobalShader.h"
#include "SceneRenderTargetParameters.h"
#include "SceneRendering.h"
#include "DeferredShadingRenderer.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessing.h"
#include "LightRendering.h"
#include "Materials/MaterialRenderProxy.h"
#include "Components/SceneCaptureComponent.h"
#include "Components/PlanarReflectionComponent.h"
#include "Containers/ArrayView.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "SceneTextureParameters.h"
#include "SceneViewExtension.h"
#include "PixelShaderUtils.h"
#include "LegacyScreenPercentageDriver.h"
#include "ScreenPass.h"
#include "PostProcess/TemporalAA.h"
#include "SceneRenderBuilder.h"

class FComposeMeshEdgesPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComposeMeshEdgesPS);
	SHADER_USE_PARAMETER_STRUCT(FComposeMeshEdgesPS, FGlobalShader)

	static const uint32 kMSAASampleCountMaxLog2 = 3; // = log2(MSAASampleCountMax)
	static const uint32 kMSAASampleCountMax = 1 << kMSAASampleCountMaxLog2;
	class FSampleCountDimension : SHADER_PERMUTATION_RANGE_INT("MSAA_SAMPLE_COUNT_LOG2", 0, kMSAASampleCountMaxLog2 + 1);
	using FPermutationDomain = TShaderPermutationDomain<FSampleCountDimension>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsDebugViewShaders(Parameters.Platform);
	}
	
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const int32 SampleCount = 1 << PermutationVector.Get<FSampleCountDimension>();
		OutEnvironment.SetDefine(TEXT("MSAA_SAMPLE_COUNT"), SampleCount);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WireframeColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WireframeDepthTexture)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Wireframe)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Depth)
		SHADER_PARAMETER(FVector2f, DepthTextureJitter)
		SHADER_PARAMETER_ARRAY(FVector4f, SampleOffsetArray, [FComposeMeshEdgesPS::kMSAASampleCountMax])

		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		
		SHADER_PARAMETER(float, Gamma)
		SHADER_PARAMETER(float, Opacity)
		
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FComposeMeshEdgesPS, "/Engine/Private/MeshEdges.usf", "ComposeMeshEdgesPS", SF_Pixel);

class FRenderTargetTexture : public FTexture, public FRenderTarget
{
public:

	FRenderTargetTexture(FRHITextureCreateDesc InDesc)
	: Desc(InDesc)
	{}

	virtual void InitRHI(FRHICommandListBase& RHICmdList)
	{
		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer
		(
			SF_Bilinear,
			AM_Clamp,
			AM_Clamp,
			AM_Clamp
		);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		RenderTargetTextureRHI = TextureRHI = RHICmdList.CreateTexture(Desc);
	}

	virtual uint32 GetSizeX() const
	{
		return Desc.GetSize().X;
	}

	virtual uint32 GetSizeY() const
	{
		return Desc.GetSize().Y;
	}

	virtual FIntPoint GetSizeXY() const { return FIntPoint(GetSizeX(), GetSizeY()); }

	virtual float GetDisplayGamma() const { return 1.0f; }

	virtual FString GetFriendlyName() const override { return Desc.DebugName; }

private:
	FRHITextureCreateDesc Desc;
};

class FMeshEdgesViewFamilyData : public ISceneViewFamilyExtentionData
{
public:
	~FMeshEdgesViewFamilyData()
	{
		if (WireframeColor)
		{
			WireframeColor->ReleaseResource();
		}

		if (WireframeDepth)
		{
			WireframeDepth->ReleaseResource();
		}
	}

	inline static const TCHAR* const GSubclassIdentifier = TEXT("FMeshEdgesViewFamilyData");
	virtual const TCHAR* GetSubclassIdentifier() const override
	{
		return GSubclassIdentifier;
	}

	void CreateRenderTargets(ERHIFeatureLevel::Type FeatureLevel, FIntPoint DesiredBufferSize)
	{
		int NumMSAASamples = FSceneTexturesConfig::GetEditorPrimitiveNumSamples(FeatureLevel);

		FRHITextureCreateDesc ColorDesc = FRHITextureCreateDesc::Create2D(TEXT("MeshEdgesRenderTarget"))
			.SetExtent(DesiredBufferSize)
			.SetFormat(PF_B8G8R8A8)
			.SetClearValue(FClearValueBinding::Transparent)
			.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetNumSamples(NumMSAASamples);

		FRHITextureCreateDesc DepthDesc = FRHITextureCreateDesc::Create2D(TEXT("MeshEdgesDepthRenderTarget"))
			.SetExtent(DesiredBufferSize)
			.SetFormat(PF_DepthStencil)
			.SetClearValue(FClearValueBinding::DepthFar)
			.SetFlags(ETextureCreateFlags::DepthStencilTargetable | ETextureCreateFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVMask)
			.SetNumSamples(NumMSAASamples);

		WireframeColor = MakeUnique<FRenderTargetTexture>(ColorDesc);
		WireframeDepth = MakeUnique<FRenderTargetTexture>(DepthDesc);
	}

	TUniquePtr<FRenderTargetTexture> WireframeColor;
	TUniquePtr<FRenderTargetTexture> WireframeDepth;
	TArray<FIntRect> ViewRects;
	TArray<FMeshEdgesViewSettings> ViewSettings;
	FMeshEdgesViewFamilySettings ViewFamilySettings = {};
};

const FMeshEdgesViewSettings& GetMeshEdgesViewSettings(const FSceneViewFamily& ViewFamily, int ViewIndex)
{
	FMeshEdgesViewFamilyData* FamilyData = const_cast<FSceneViewFamily&>(ViewFamily).GetOrCreateExtentionData<FMeshEdgesViewFamilyData>();

	// Init on first access, and handles edge-case where viewcount has changed since last access.
	if (ViewIndex >= FamilyData->ViewSettings.Num())
	{
		FamilyData->ViewSettings.SetNum(ViewFamily.Views.Num());
	}

	return FamilyData->ViewSettings[ViewIndex];
}

static int FindViewIndex(const FSceneView& View)
{
	if (!View.Family)
	{
		return INDEX_NONE;
	}

	for (int ViewIndex = 0; ViewIndex < View.Family->Views.Num(); ViewIndex++)
	{
		if (View.Family->Views[ViewIndex] == &View)
		{
			return ViewIndex;
		}
	}
	return INDEX_NONE;
}

const FMeshEdgesViewSettings& GetMeshEdgesViewSettings(const FSceneView& View)
{
	check(View.Family);

	int ViewIndex = FindViewIndex(View);
	if (ViewIndex == INDEX_NONE) { ViewIndex = 0; }

	return GetMeshEdgesViewSettings(*View.Family, ViewIndex);
}

FMeshEdgesViewSettings& GetMeshEdgesViewSettings(FSceneView& View)
{
	return const_cast<FMeshEdgesViewSettings&>(GetMeshEdgesViewSettings(AsConst(View)));
}

const FMeshEdgesViewFamilySettings& GetMeshEdgesViewFamilySettings(const FSceneViewFamily& ViewFamily)
{
	FMeshEdgesViewFamilyData* FamilyData = const_cast<FSceneViewFamily&>(ViewFamily).GetOrCreateExtentionData<FMeshEdgesViewFamilyData>();
	return FamilyData->ViewFamilySettings;
}

FMeshEdgesViewFamilySettings& GetMeshEdgesViewFamilySettings(FSceneViewFamily& ViewFamily)
{
	return const_cast<FMeshEdgesViewFamilySettings&>(GetMeshEdgesViewFamilySettings(AsConst(ViewFamily)));
}

void RenderMeshEdges(FSceneViewFamily& InViewFamily);

class FMeshEdgesExtension : public FSceneViewExtensionBase
{
public:
	FMeshEdgesExtension( const FAutoRegister& AutoRegister)
	: FSceneViewExtensionBase( AutoRegister )
	{
		FCoreDelegates::OnEnginePreExit.AddLambda([]()
			{ 
				Instance = nullptr;
			});
	}

	virtual void PostCreateSceneRenderer(const FSceneViewFamily& InViewFamily, ISceneRenderer* Renderer) override
	{
		RenderMeshEdges(static_cast<FSceneRenderer*>(Renderer)->ViewFamily);
	}

	inline static TSharedPtr<FMeshEdgesExtension,ESPMode::ThreadSafe> Instance = nullptr;
};

void InitMeshEdgesViewExtension()
{
	FMeshEdgesExtension::Instance = FSceneViewExtensions::NewExtension<FMeshEdgesExtension>(); 
}

void CopyViewFamily(const FSceneViewFamily& SrcViewFamily, FSceneViewFamily& ViewFamily)
{
	ViewFamily.FrameNumber = SrcViewFamily.FrameNumber;
	ViewFamily.FrameCounter = SrcViewFamily.FrameCounter;
	ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(SrcViewFamily.Scene));

	for (int32 ViewIndex = 0; ViewIndex < SrcViewFamily.Views.Num(); ++ViewIndex)
	{
		const FSceneView* SrcSceneView = SrcViewFamily.Views[ViewIndex];
		if (ensure(SrcSceneView))
		{
			FSceneViewInitOptions ViewInitOptions = SrcSceneView->SceneViewInitOptions;
			ViewInitOptions.ViewFamily = &ViewFamily;
			ViewInitOptions.ViewLocation = SrcSceneView->ViewLocation;
			ViewInitOptions.ViewRotation = SrcSceneView->ViewRotation;

			// Reset to avoid incorrect culling problems
			ViewInitOptions.SceneViewStateInterface = FSceneViewInitOptions{}.SceneViewStateInterface;
		
			FSceneView* View = new FSceneView(ViewInitOptions);
			ViewFamily.Views.Emplace(View);
		}
	}
}

void RenderMeshEdges(FSceneViewFamily& ViewFamily)
{
	if (!ViewFamily.EngineShowFlags.MeshEdges || ViewFamily.EngineShowFlags.HitProxies)
	{
		return;
	}

	FMeshEdgesViewFamilyData* ViewFamilyData = ViewFamily.GetOrCreateExtentionData<FMeshEdgesViewFamilyData>();
	const FMeshEdgesViewFamilySettings& Settings = ViewFamilyData->ViewFamilySettings;

	ERHIFeatureLevel::Type FeatureLevel = ViewFamily.GetFeatureLevel();
	FIntPoint DesiredBufferSize = FSceneRenderer::GetDesiredInternalBufferSize(ViewFamily);
	ViewFamilyData->CreateRenderTargets(FeatureLevel, DesiredBufferSize);

	FEngineShowFlags WireframeShowFlags = ViewFamily.EngineShowFlags;
	{
		// Render a wireframe view
		WireframeShowFlags.SetWireframe(true);

		// Copy the MSAA wireframe view only, don't copy other scene elements
		WireframeShowFlags.SetSceneCaptureCopySceneDepth(false);

		// Disable rendering of elements that are not needed
		WireframeShowFlags.SetMeshEdges(false);
		WireframeShowFlags.SetLighting(false);
		WireframeShowFlags.SetLightFunctions(false);
		WireframeShowFlags.SetGlobalIllumination(false);
		WireframeShowFlags.SetLumenGlobalIllumination(false);
		WireframeShowFlags.SetLumenReflections(false);
		WireframeShowFlags.SetDynamicShadows(false);
		WireframeShowFlags.SetCapsuleShadows(false);
		WireframeShowFlags.SetDistanceFieldAO(false);
		WireframeShowFlags.SetFog(false);
		WireframeShowFlags.SetVolumetricFog(false);
		WireframeShowFlags.SetCloud(false);
		WireframeShowFlags.SetDecals(false);
		WireframeShowFlags.SetAtmosphere(false);
		WireframeShowFlags.SetPostProcessing(false);
		WireframeShowFlags.SetCompositeDebugPrimitives(false);
		WireframeShowFlags.SetCompositeEditorPrimitives(false);
		WireframeShowFlags.SetGrid(false);
		WireframeShowFlags.SetShaderPrint(false);
		WireframeShowFlags.SetLensDistortion(false);
		//WireframeShowFlags.SetScreenPercentage(false);
		//WireframeShowFlags.SetTranslucency(false);
        WireframeShowFlags.SetPathTracing(false);
		WireframeShowFlags.SetRayTracingDebug(false);
	}
	
	FSceneViewFamilyContext CaptureViewFamily(
		FSceneViewFamily::ConstructionValues(
			ViewFamilyData->WireframeColor.Get(),
			ViewFamily.Scene,
			WireframeShowFlags)
		.SetRenderTargetDepth(ViewFamilyData->WireframeDepth.Get())
		.SetResolveScene(true)
		.SetRealtimeUpdate(true)
		.SetTime(ViewFamily.Time)
	);

	{
		CopyViewFamily(ViewFamily, CaptureViewFamily);

		CaptureViewFamily.SceneCaptureSource = SCS_SceneColorSceneDepth;
	
		// Use the same resolution scale as main view, so the buffers align pixel-perfect.
		// If the main view is low-res this affects the wireframe quality, so the main view should be 100% ideally
		CaptureViewFamily.SetScreenPercentageInterface(ViewFamily.GetScreenPercentageInterface()->Fork_GameThread(CaptureViewFamily));
	}

	Settings.OnBeforeWireframeRender(CaptureViewFamily);

	FSceneRenderBuilder SceneRenderBuilder(ViewFamily.Scene);

	FSceneRenderer* SceneRenderer = SceneRenderBuilder.CreateSceneRenderer(&CaptureViewFamily);

	for (const FSceneViewExtensionRef& Extension : CaptureViewFamily.ViewExtensions)
	{
		Extension->SetupViewFamily(CaptureViewFamily);
	}

	for (int32 ViewIndex = 0; ViewIndex < SceneRenderer->Views.Num(); ++ViewIndex)
	{
		FViewInfo& ViewInfo = SceneRenderer->Views[ViewIndex];
		ViewInfo.bAllowTemporalJitter = false;
		ViewInfo.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::RawOutput;

		for (const FSceneViewExtensionRef& Extension : CaptureViewFamily.ViewExtensions)
		{
			Extension->SetupView(CaptureViewFamily, ViewInfo);
		}
	}

	ON_SCOPE_EXIT
	{
		SceneRenderBuilder.Execute();
	};

	ViewFamilyData->ViewRects.SetNumZeroed(ViewFamily.Views.Num());

	SceneRenderBuilder.AddRenderer(SceneRenderer, TEXT("RenderMeshEdges"),
		[ViewFamilyData] (FRDGBuilder& GraphBuilder, const FSceneRenderFunctionInputs& Inputs)
	{
		ViewFamilyData->WireframeColor->InitResource(GraphBuilder.RHICmdList);
		ViewFamilyData->WireframeDepth->InitResource(GraphBuilder.RHICmdList);

		Inputs.Renderer->Render(GraphBuilder, Inputs.SceneUpdateInputs);

		for (const FViewInfo& ViewInfo : Inputs.Renderer->Views)
		{
			int ViewIndex = FindViewIndex(ViewInfo);
			if (ViewIndex != INDEX_NONE)
			{
				ViewFamilyData->ViewRects[ViewIndex] = ViewInfo.ViewRect;
			}
		}

		return true;
	});
}

void ComposeMeshEdges(FRDGBuilder& GraphBuilder,
                        const FViewInfo& View,
                        FScreenPassRenderTarget& EditorPrimitivesColor,
                        FScreenPassRenderTarget& EditorPrimitivesDepth)
{
	check(View.Family);
	const FSceneViewFamily& ViewFamily = *View.Family;
	if (!ViewFamily.EngineShowFlags.MeshEdges)
	{
		return;
	}

	int ViewIndex = FindViewIndex(View);
	if (ViewIndex == INDEX_NONE)
	{
		// Could not find view
		return;
	}

	if (ViewIndex >= ViewFamily.Views.Num())
	{
		// View was not found in its own ViewFamily (?), let's abort.
		return;
	}

	const FMeshEdgesViewFamilyData* ViewFamilyData = ViewFamily.GetExtentionData<FMeshEdgesViewFamilyData>();
	check(ViewFamilyData); // should have been created in RenderMeshEdges
	if (ViewFamilyData->ViewRects.Num() == 0)
	{
		// Cannot compose if the prepass did not render
		return;
	}
	check(ViewFamilyData->ViewRects.Num() > ViewIndex);
	check(ViewFamilyData->WireframeColor);
	check(ViewFamilyData->WireframeDepth);
	const FMeshEdgesViewSettings& ViewSettings = GetMeshEdgesViewSettings(ViewFamily, ViewIndex);

	FRenderTargetTexture& WireframeTextureColor = *ViewFamilyData->WireframeColor;
	FRenderTargetTexture& WireframeTextureDepth = *ViewFamilyData->WireframeDepth;
	const FIntRect& WireframeViewRect = ViewFamilyData->ViewRects[ViewIndex];
	const FSceneTextures& SceneTextures = View.GetSceneTextures();
	FScreenPassTexture SceneDepth(SceneTextures.Depth.Resolve, View.ViewRect);
	const uint32 NumMSAASamples = SceneTextures.Config.EditorPrimitiveNumSamples;
	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FComposeMeshEdgesPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComposeMeshEdgesPS::FParameters>();
	
	PassParameters->WireframeColorTexture = RegisterExternalTexture(GraphBuilder, WireframeTextureColor.TextureRHI, *WireframeTextureColor.GetFriendlyName());
	PassParameters->WireframeDepthTexture = RegisterExternalTexture(GraphBuilder, WireframeTextureDepth.TextureRHI, *WireframeTextureDepth.GetFriendlyName());
	PassParameters->Wireframe = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(WireframeViewRect));
	PassParameters->Depth = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(SceneDepth));
	PassParameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(EditorPrimitivesColor));
	PassParameters->DepthTexture = SceneDepth.Texture;
	PassParameters->DepthSampler = PointClampSampler;
	PassParameters->DepthTextureJitter = FVector2f(View.TemporalJitterPixels);
	PassParameters->Gamma = GEngine->DisplayGamma;
	PassParameters->Opacity = ViewSettings.Opacity;

	for (int32 i = 0; i < int32(NumMSAASamples); i++)
	{
		PassParameters->SampleOffsetArray[i].X = GetMSAASampleOffsets(NumMSAASamples, i).X;
		PassParameters->SampleOffsetArray[i].Y = GetMSAASampleOffsets(NumMSAASamples, i).Y;
	}

	PassParameters->RenderTargets[0] = EditorPrimitivesColor.GetRenderTargetBinding();
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(EditorPrimitivesDepth.Texture, EditorPrimitivesDepth.LoadAction, EditorPrimitivesDepth.LoadAction, FExclusiveDepthStencil::DepthWrite);

	const int MSAASampleCountDim = FMath::FloorLog2(NumMSAASamples);

	FComposeMeshEdgesPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FComposeMeshEdgesPS::FSampleCountDimension>(MSAASampleCountDim);

	check(View.ShaderMap);
	const TShaderRef<FComposeMeshEdgesPS> PixelShader = View.ShaderMap->GetShader<FComposeMeshEdgesPS>(PermutationVector);

	FIntRect Viewport = EditorPrimitivesColor.ViewRect;

	FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA>::GetRHI();
	FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI();

	FPixelShaderUtils::AddFullscreenPass(GraphBuilder, View.ShaderMap, RDG_EVENT_NAME("ComposeMeshEdges"), PixelShader, PassParameters, Viewport, BlendState, nullptr, DepthStencilState);
}

