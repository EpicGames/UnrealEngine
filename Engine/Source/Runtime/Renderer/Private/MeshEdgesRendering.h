// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshEdges.h"

class FRDGBuilder;
class FViewInfo;
struct FCompositePrimitiveInputs;
struct FScreenPassRenderTarget;

void InitMeshEdgesViewExtension();

void ComposeMeshEdges(FRDGBuilder& GraphBuilder,
						const FViewInfo& View,
						FScreenPassRenderTarget& EditorPrimitivesColor,
						FScreenPassRenderTarget& EditorPrimitivesDepth);
