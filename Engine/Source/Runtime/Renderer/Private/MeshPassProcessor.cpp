// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshPassProcessor.cpp: 
=============================================================================*/

#include "MeshPassProcessor.h"
#include "SceneUtils.h"
#include "SceneRendering.h"
#include "Logging/LogMacros.h"
#include "LogRenderer.h"
#include "SceneCore.h"
#include "ScenePrivate.h"
#include "SceneInterface.h"
#include "MeshPassProcessor.inl"
#include "PipelineStateCache.h"
#include "RayTracing/RayTracingMaterialHitShaders.h"
#include "Hash/CityHash.h"
#include "ComponentRecreateRenderStateContext.h"
#include "RenderCore.h"
#include "UnrealEngine.h"
#include "SceneUniformBuffer.h"
#include "MeshDrawCommandStats.h"
#include "VariableRateShadingImageManager.h"

FRWLock FGraphicsMinimalPipelineStateId::PersistentIdTableLock;
FGraphicsMinimalPipelineStateId::PersistentTableType FGraphicsMinimalPipelineStateId::PersistentIdTable;

#if MESH_DRAW_COMMAND_DEBUG_DATA
std::atomic<int32> FGraphicsMinimalPipelineStateId::DebugSaltAllocationIndex;
std::atomic<int32> FGraphicsMinimalPipelineStateId::LocalPipelineIdTableSize(0);
std::atomic<int32> FGraphicsMinimalPipelineStateId::CurrentLocalPipelineIdTableSize(0);
#endif //MESH_DRAW_COMMAND_DEBUG_DATA

bool FGraphicsMinimalPipelineStateId::NeedsShaderInitialisation = true;
bool FGraphicsMinimalPipelineStateId::bIsIdTableFrozen = false;
std::atomic<int32> FGraphicsMinimalPipelineStateId::ReffedItemCount(0);

const FMeshDrawCommandSortKey FMeshDrawCommandSortKey::Default = { {0} };

int32 GEmitMeshDrawEvent = 0;
static FAutoConsoleVariableRef CVarEmitMeshDrawEvent(
	TEXT("r.EmitMeshDrawEvents"),
	GEmitMeshDrawEvent,
	TEXT("Emits a GPU event around each drawing policy draw call.  /n")
	TEXT("Useful for seeing stats about each draw call, however it greatly distorts total time and time per draw call."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarSafeStateLookup(
	TEXT("r.SafeStateLookup"),
	1,
	TEXT("Forces new-style safe state lookup for easy runtime perf comparison\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

int32 GSkipDrawOnPSOPrecaching = 0;
static FAutoConsoleVariableRef CVarSkipDrawOnPSOPrecaching(
	TEXT("r.SkipDrawOnPSOPrecaching"),
	GSkipDrawOnPSOPrecaching,
	TEXT("Skips mesh draw call when the PSO is still compiling (default 0)."),
	ECVF_RenderThreadSafe
);

static bool GSkipUnloadedShaders = false;
static FAutoConsoleVariableRef CVarAllowSkipUnloadedShaders(
	TEXT("r.SkipUnloadedShaders"),
	GSkipUnloadedShaders,
	TEXT("Skip the draw call when the shaders are not fully preloaded. Useful for debugging"),
	ECVF_Default
);

#if WITH_EDITORONLY_DATA

int32 GNaniteIsolateInvalidCoarseMesh = 0;
static FAutoConsoleVariableRef CVarNaniteIsolateInvalidCoarseMesh(
	TEXT("r.Nanite.IsolateInvalidCoarseMesh"),
	GNaniteIsolateInvalidCoarseMesh,
	TEXT("Debug mode to render only non-Nanite proxies that incorrectly reference coarse static mesh assets."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		// Needed to force a recache of all the static mesh draw commands
		FGlobalComponentRecreateRenderStateContext Context;
	})
);

#endif

inline void SetTextureParameter(FRHIBatchedShaderParameters& BatchedParameters, const FShaderResourceParameterInfo& Parameter, FRHITexture* TextureRHI)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (Parameter.Type == EShaderParameterType::BindlessSRV)
	{
		BatchedParameters.SetBindlessTexture(Parameter.BaseIndex, TextureRHI);
	}
	else
#endif
	{
		BatchedParameters.SetShaderTexture(Parameter.BaseIndex, TextureRHI);
	}
}

inline void SetSrvParameter(FRHIBatchedShaderParameters& BatchedParameters, const FShaderResourceParameterInfo& Parameter, FRHIShaderResourceView* SrvRHI)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (Parameter.Type == EShaderParameterType::BindlessSRV)
	{
		BatchedParameters.SetBindlessResourceView(Parameter.BaseIndex, SrvRHI);
	}
	else
#endif
	{
		BatchedParameters.SetShaderResourceViewParameter(Parameter.BaseIndex, SrvRHI);
	}
}

inline void SetSamplerParameter(FRHIBatchedShaderParameters& BatchedParameters, const FShaderResourceParameterInfo& Parameter, FRHISamplerState* SamplerStateRHI)
{
#if PLATFORM_SUPPORTS_BINDLESS_RENDERING
	if (Parameter.Type == EShaderParameterType::BindlessSampler)
	{
		BatchedParameters.SetBindlessSampler(Parameter.BaseIndex, SamplerStateRHI);
	}
	else
#endif
	{
		BatchedParameters.SetShaderSampler(Parameter.BaseIndex, SamplerStateRHI);
	}
}

inline void SetLooseParameters(FRHIBatchedShaderParameters& BatchedParameters, const FShaderParameterMapInfo& ParameterMapInfo, const uint8* LooseDataStart)
{
	for (const FShaderLooseParameterBufferInfo& LooseParameterBuffer : ParameterMapInfo.LooseParameterBuffers)
	{
		for (const FShaderLooseParameterInfo& Parameter : LooseParameterBuffer.Parameters)
		{
			BatchedParameters.SetShaderParameter(
				LooseParameterBuffer.BaseIndex,
				Parameter.BaseIndex,
				Parameter.Size,
				LooseDataStart
			);

			LooseDataStart += Parameter.Size;
		}
	}
}

void FReadOnlyMeshDrawSingleShaderBindings::SetShaderBindings(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FReadOnlyMeshDrawSingleShaderBindings& RESTRICT SingleShaderBindings,
	FShaderBindingState& RESTRICT ShaderBindingState)
{
	FRHIUniformBuffer* const* RESTRICT UniformBufferBindings = SingleShaderBindings.GetUniformBufferStart();
	const FShaderUniformBufferParameterInfo* RESTRICT UniformBufferParameters = SingleShaderBindings.ParameterMapInfo.UniformBuffers.GetData();
	const int32 NumUniformBuffers = SingleShaderBindings.ParameterMapInfo.UniformBuffers.Num();

	for (int32 UniformBufferIndex = 0; UniformBufferIndex < NumUniformBuffers; UniformBufferIndex++)
	{
		FShaderUniformBufferParameterInfo Parameter = UniformBufferParameters[UniformBufferIndex];
		check(Parameter.BaseIndex < FShaderBindingState::MAX_UNIFORM_BUFFERS_PER_STAGE);
		FRHIUniformBuffer* UniformBuffer = UniformBufferBindings[UniformBufferIndex];

		if (UniformBuffer != ShaderBindingState.UniformBuffers[Parameter.BaseIndex])
		{
			BatchedParameters.SetShaderUniformBuffer(Parameter.BaseIndex, UniformBuffer);
			ShaderBindingState.UniformBuffers[Parameter.BaseIndex] = UniformBuffer;
			ShaderBindingState.MaxUniformBufferUsed = FMath::Max((int32)Parameter.BaseIndex, ShaderBindingState.MaxUniformBufferUsed);
		}
	}

	FRHISamplerState* const* RESTRICT SamplerBindings = SingleShaderBindings.GetSamplerStart();
	const FShaderResourceParameterInfo* RESTRICT TextureSamplerParameters = SingleShaderBindings.ParameterMapInfo.TextureSamplers.GetData();
	const int32 NumTextureSamplers = SingleShaderBindings.ParameterMapInfo.TextureSamplers.Num();

	for (int32 SamplerIndex = 0; SamplerIndex < NumTextureSamplers; SamplerIndex++)
	{
		FShaderResourceParameterInfo Parameter = TextureSamplerParameters[SamplerIndex];
		FRHISamplerState* Sampler = (FRHISamplerState*)SamplerBindings[SamplerIndex];

		SetSamplerParameter(BatchedParameters, Parameter, Sampler);
	}

	const uint8* RESTRICT SRVType = SingleShaderBindings.GetSRVTypeStart();
	FRHIResource* const* RESTRICT SRVBindings = SingleShaderBindings.GetSRVStart();
	const FShaderResourceParameterInfo* RESTRICT SRVParameters = SingleShaderBindings.ParameterMapInfo.SRVs.GetData();
	const uint32 NumSRVs = SingleShaderBindings.ParameterMapInfo.SRVs.Num();

	for (uint32 SRVIndex = 0; SRVIndex < NumSRVs; SRVIndex++)
	{
		FShaderResourceParameterInfo Parameter = SRVParameters[SRVIndex];

		uint32 TypeByteIndex = SRVIndex / 8;
		uint32 TypeBitIndex = SRVIndex % 8;

		if (SRVType[TypeByteIndex] & (1 << TypeBitIndex))
		{
			FRHIShaderResourceView* SRV = (FRHIShaderResourceView*)SRVBindings[SRVIndex];
			SetSrvParameter(BatchedParameters, Parameter, SRV);
		}
		else
		{
			FRHITexture* Texture = (FRHITexture*)SRVBindings[SRVIndex];
			SetTextureParameter(BatchedParameters, Parameter, Texture);
		}
	}

	SetLooseParameters(BatchedParameters, SingleShaderBindings.ParameterMapInfo, SingleShaderBindings.GetLooseDataStart());
}

void FReadOnlyMeshDrawSingleShaderBindings::SetShaderBindings(
	FRHIBatchedShaderParameters& BatchedParameters,
	const FReadOnlyMeshDrawSingleShaderBindings& RESTRICT SingleShaderBindings)
{
	FRHIUniformBuffer* const* RESTRICT UniformBufferBindings = SingleShaderBindings.GetUniformBufferStart();
	const FShaderUniformBufferParameterInfo* RESTRICT UniformBufferParameters = SingleShaderBindings.ParameterMapInfo.UniformBuffers.GetData();
	const int32 NumUniformBuffers = SingleShaderBindings.ParameterMapInfo.UniformBuffers.Num();

	for (int32 UniformBufferIndex = 0; UniformBufferIndex < NumUniformBuffers; UniformBufferIndex++)
	{
		FShaderUniformBufferParameterInfo Parameter = UniformBufferParameters[UniformBufferIndex];
		FRHIUniformBuffer* UniformBuffer = UniformBufferBindings[UniformBufferIndex];

		if (UniformBuffer)
		{
			BatchedParameters.SetShaderUniformBuffer(Parameter.BaseIndex, UniformBuffer);
		}
	}

	FRHISamplerState* const* RESTRICT SamplerBindings = SingleShaderBindings.GetSamplerStart();
	const FShaderResourceParameterInfo* RESTRICT TextureSamplerParameters = SingleShaderBindings.ParameterMapInfo.TextureSamplers.GetData();
	const int32 NumTextureSamplers = SingleShaderBindings.ParameterMapInfo.TextureSamplers.Num();

	for (int32 SamplerIndex = 0; SamplerIndex < NumTextureSamplers; SamplerIndex++)
	{
		FShaderResourceParameterInfo Parameter = TextureSamplerParameters[SamplerIndex];
		FRHISamplerState* Sampler = (FRHISamplerState*)SamplerBindings[SamplerIndex];

		if (Sampler)
		{
			SetSamplerParameter(BatchedParameters, Parameter, Sampler);
		}
	}

	const uint8* RESTRICT SRVType = SingleShaderBindings.GetSRVTypeStart();
	FRHIResource* const* RESTRICT SRVBindings = SingleShaderBindings.GetSRVStart();
	const FShaderResourceParameterInfo* RESTRICT SRVParameters = SingleShaderBindings.ParameterMapInfo.SRVs.GetData();
	const uint32 NumSRVs = SingleShaderBindings.ParameterMapInfo.SRVs.Num();

	for (uint32 SRVIndex = 0; SRVIndex < NumSRVs; SRVIndex++)
	{
		FShaderResourceParameterInfo Parameter = SRVParameters[SRVIndex];

		uint32 TypeByteIndex = SRVIndex / 8;
		uint32 TypeBitIndex = SRVIndex % 8;

		if (SRVType[TypeByteIndex] & (1 << TypeBitIndex))
		{
			FRHIShaderResourceView* SRV = (FRHIShaderResourceView*)SRVBindings[SRVIndex];

			if (SRV)
			{
				SetSrvParameter(BatchedParameters, Parameter, SRV);
			}
		}
		else
		{
			FRHITexture* Texture = (FRHITexture*)SRVBindings[SRVIndex];

			if (Texture)
			{
				SetTextureParameter(BatchedParameters, Parameter, Texture);
			}
		}
	}

	SetLooseParameters(BatchedParameters, SingleShaderBindings.ParameterMapInfo, SingleShaderBindings.GetLooseDataStart());
}

#if RHI_RAYTRACING

FRayTracingLocalShaderBindings* FMeshDrawShaderBindings::SetRayTracingShaderBindings(
	FRayTracingLocalShaderBindingWriter* BindingWriter, 
	uint32 ShaderIndexInPipeline, 
	uint32 RecordIndex, 
	uint32 UserData, 
	ERayTracingLocalShaderBindingType BindingType) const
{
	check(ShaderLayouts.Num() == 1);

	FReadOnlyMeshDrawSingleShaderBindings SingleShaderBindings(ShaderLayouts[0], GetData());

	FRHIUniformBuffer* const* RESTRICT UniformBufferBindings = SingleShaderBindings.GetUniformBufferStart();
	const FShaderUniformBufferParameterInfo* RESTRICT UniformBufferParameters = SingleShaderBindings.ParameterMapInfo.UniformBuffers.GetData();
	const int32 NumUniformBufferParameters = SingleShaderBindings.ParameterMapInfo.UniformBuffers.Num();

	checkf(SingleShaderBindings.ParameterMapInfo.TextureSamplers.Num() == 0, TEXT("Texture sampler parameters are not supported for ray tracing. UniformBuffers must be used for all resource binding."));
	checkf(SingleShaderBindings.ParameterMapInfo.SRVs.Num() == 0, TEXT("SRV parameters are not supported for ray tracing. UniformBuffers must be used for all resource binding."));

	// Measure parameter memory requirements

	int32 MaxUniformBufferIndex = -1;
	for (int32 UniformBufferIndex = 0; UniformBufferIndex < NumUniformBufferParameters; UniformBufferIndex++)
	{
		FShaderUniformBufferParameterInfo Parameter = UniformBufferParameters[UniformBufferIndex];
		const FRHIUniformBuffer* UniformBuffer = UniformBufferBindings[UniformBufferIndex];
		MaxUniformBufferIndex = FMath::Max((int32)Parameter.BaseIndex, MaxUniformBufferIndex);
	}

	const uint32 NumUniformBuffersToSet = MaxUniformBufferIndex + 1;

	const TMemoryImageArray<FShaderLooseParameterBufferInfo>& LooseParameterBuffers = SingleShaderBindings.ParameterMapInfo.LooseParameterBuffers;
	uint32 LooseParameterDataSize = 0;

	if (LooseParameterBuffers.Num())
	{
		check(LooseParameterBuffers.Num() <= 1);

		const FShaderLooseParameterBufferInfo& LooseParameterBuffer = SingleShaderBindings.ParameterMapInfo.LooseParameterBuffers[0];
		MaxUniformBufferIndex = FMath::Max((int32)LooseParameterBuffer.BaseIndex, MaxUniformBufferIndex);

		for (int32 LooseParameterIndex = 0; LooseParameterIndex < LooseParameterBuffer.Parameters.Num(); LooseParameterIndex++)
		{
			FShaderLooseParameterInfo LooseParameter = LooseParameterBuffer.Parameters[LooseParameterIndex];
			LooseParameterDataSize = FMath::Max<uint32>(LooseParameterDataSize, LooseParameter.BaseIndex + LooseParameter.Size);
		}
	}

	// Vulkan places loose parameter data in the shader record which does not add to UB count
	checkf(MaxUniformBufferIndex + 1 == NumUniformBufferParameters + (GRHIGlobals.RayTracing.SupportsLooseParamsInShaderRecord ? 0 : LooseParameterBuffers.Num()),
		TEXT("Highest index of a uniform buffer was %d, but there were %d uniform buffer parameters and %d loose parameters"),
		MaxUniformBufferIndex,
		NumUniformBufferParameters,
		LooseParameterBuffers.Num());

	// Allocate and fill bindings

	FRayTracingLocalShaderBindings& Bindings = BindingWriter->AddWithInlineParameters(NumUniformBuffersToSet, LooseParameterDataSize);
	Bindings.RecordIndex = RecordIndex;
	Bindings.ShaderIndexInPipeline = ShaderIndexInPipeline;
	Bindings.UserData = UserData; // UserData could be used to store material ID or any other kind of per-material constant. This can be retrieved in hit shaders via GetHitGroupUserData().
	Bindings.BindingType = BindingType;

	for (int32 UniformBufferIndex = 0; UniformBufferIndex < NumUniformBufferParameters; UniformBufferIndex++)
	{
		FShaderUniformBufferParameterInfo Parameter = UniformBufferParameters[UniformBufferIndex];
		const FRHIUniformBuffer* UniformBuffer = UniformBufferBindings[UniformBufferIndex];
		Bindings.UniformBuffers[Parameter.BaseIndex] = const_cast<FRHIUniformBuffer*>(UniformBuffer);
	}

	if (LooseParameterBuffers.Num())
	{
		const FShaderLooseParameterBufferInfo& LooseParameterBuffer = SingleShaderBindings.ParameterMapInfo.LooseParameterBuffers[0];
		const uint8* LooseDataOffset = SingleShaderBindings.GetLooseDataStart();
#if DO_CHECK
		// Memzero the loose parameter data because it's used for validation on persistent SBT entries to make sure binding data is always the same
		FMemory::Memzero(Bindings.LooseParameterData, LooseParameterDataSize);
#endif // DO_CHECK
		for (int32 LooseParameterIndex = 0; LooseParameterIndex < LooseParameterBuffer.Parameters.Num(); LooseParameterIndex++)
		{
			FShaderLooseParameterInfo LooseParameter = LooseParameterBuffer.Parameters[LooseParameterIndex];
			FMemory::Memcpy(Bindings.LooseParameterData + LooseParameter.BaseIndex, LooseDataOffset, LooseParameter.Size);
			LooseDataOffset += LooseParameter.Size;
		}
	}

	return &Bindings;
}

FRayTracingLocalShaderBindings* FMeshDrawShaderBindings::SetRayTracingShaderBindingsForHitGroup(
	FRayTracingLocalShaderBindingWriter* BindingWriter,
	uint32 RecordIndex,
	const FRHIRayTracingGeometry* Geometry,
	uint32 GeometrySegmentIndex,
	uint32 HitGroupIndexInPipeline, 
	ERayTracingLocalShaderBindingType BindingType) const
{
	FRayTracingLocalShaderBindings* Bindings = SetRayTracingShaderBindings(BindingWriter, HitGroupIndexInPipeline, RecordIndex, 0, BindingType);
	Bindings->Geometry = Geometry;
	Bindings->SegmentIndex = GeometrySegmentIndex;

	return Bindings;
}

void FMeshDrawShaderBindings::SetRayTracingShaderBindingsForMissShader(
	FRHICommandList& RHICmdList,
	FRHIShaderBindingTable* SBT, uint32 RecordIndex,
	FRayTracingPipelineState* PipelineState, uint32 ShaderIndexInPipeline) const
{
	check(ShaderLayouts.Num() == 1);

	FReadOnlyMeshDrawSingleShaderBindings SingleShaderBindings(ShaderLayouts[0], GetData());

	FShaderBindingState BindingState;

	const int32 MaxUniformBuffers = UE_ARRAY_COUNT(BindingState.UniformBuffers);

	FRHIUniformBuffer* const* RESTRICT UniformBufferBindings = SingleShaderBindings.GetUniformBufferStart();
	const FShaderUniformBufferParameterInfo* RESTRICT UniformBufferParameters = SingleShaderBindings.ParameterMapInfo.UniformBuffers.GetData();
	const int32 NumUniformBufferParameters = SingleShaderBindings.ParameterMapInfo.UniformBuffers.Num();

	checkf(SingleShaderBindings.ParameterMapInfo.TextureSamplers.Num() == 0, TEXT("Texture sampler parameters are not supported for ray tracing. UniformBuffers must be used for all resource binding."));
	checkf(SingleShaderBindings.ParameterMapInfo.SRVs.Num() == 0, TEXT("SRV parameters are not supported for ray tracing. UniformBuffers must be used for all resource binding."));

	// Measure parameter memory requirements

	int32 MaxUniformBufferUsed = -1;
	for (int32 UniformBufferIndex = 0; UniformBufferIndex < NumUniformBufferParameters; UniformBufferIndex++)
	{
		FShaderUniformBufferParameterInfo Parameter = UniformBufferParameters[UniformBufferIndex];
		checkSlow(Parameter.BaseIndex < MaxUniformBuffers);
		FRHIUniformBuffer* UniformBuffer = UniformBufferBindings[UniformBufferIndex];
		if (Parameter.BaseIndex < MaxUniformBuffers)
		{
			BindingState.UniformBuffers[Parameter.BaseIndex] = UniformBuffer;
			MaxUniformBufferUsed = FMath::Max((int32)Parameter.BaseIndex, MaxUniformBufferUsed);
		}
	}

	checkf(SingleShaderBindings.ParameterMapInfo.TextureSamplers.Num() == 0, TEXT("Texture sampler parameters are not supported for ray tracing. UniformBuffers must be used for all resource binding."));
	checkf(SingleShaderBindings.ParameterMapInfo.SRVs.Num() == 0, TEXT("SRV parameters are not supported for ray tracing. UniformBuffers must be used for all resource binding."));
	checkf(SingleShaderBindings.ParameterMapInfo.LooseParameterBuffers.Num() == 0, TEXT("Ray tracing miss shaders may not have loose parameters"));

	uint32 NumUniformBuffersToSet = MaxUniformBufferUsed + 1;
	const uint32 UserData = 0; // UserData could be used to store material ID or any other kind of per-material constant. This can be retrieved in hit shaders via GetHitGroupUserData().
	RHICmdList.SetRayTracingMissShader(
		SBT, RecordIndex,
		PipelineState, ShaderIndexInPipeline,
		NumUniformBuffersToSet, BindingState.UniformBuffers,
		UserData);
}
#endif // RHI_RAYTRACING

FGraphicsMinimalPipelineStateId FGraphicsMinimalPipelineStateId::GetPersistentId(const FGraphicsMinimalPipelineStateInitializer& InPipelineState)
{
	Experimental::FHashElementId TableId;
#if MESH_DRAW_COMMAND_DEBUG_DATA
	int32 DebugSalt;
#endif
	auto hash = PersistentIdTable.ComputeHash(InPipelineState);
	{
		FRWScopeLock Lock(PersistentIdTableLock, SLT_ReadOnly);

#if UE_BUILD_DEBUG
		FGraphicsMinimalPipelineStateInitializer PipelineStateDebug = FGraphicsMinimalPipelineStateInitializer(InPipelineState);
		check(GetTypeHash(PipelineStateDebug) == GetTypeHash(InPipelineState));
		check(PipelineStateDebug == InPipelineState);
#endif

		TableId = PersistentIdTable.FindIdByHash(hash, InPipelineState);


		if (!TableId.IsValid())
		{
			Lock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();

			TableId = PersistentIdTable.FindOrAddIdByHash(hash, InPipelineState, FRefCountedGraphicsMinimalPipelineState());
		}
		
		FRefCountedGraphicsMinimalPipelineState& Value = PersistentIdTable.GetByElementId(TableId).Value;

		// Needs to be atomic, so condition is only true on first increment
		if (Value.RefNum.fetch_add(1) == 0)
		{
			if (!NeedsShaderInitialisation)
			{
				NeedsShaderInitialisation = true;
			}
			ReffedItemCount++;
		}

#if MESH_DRAW_COMMAND_DEBUG_DATA
		DebugSalt = Value.DebugSalt;
#endif
	}

	checkf(TableId.GetIndex() < (MAX_uint32 >> 2), TEXT("Persistent FGraphicsMinimalPipelineStateId table overflow!"));

	FGraphicsMinimalPipelineStateId Ret;
	Ret.bValid = 1;
	Ret.bComesFromLocalPipelineStateSet = 0;
	Ret.SetElementIndex = TableId.GetIndex();
#if MESH_DRAW_COMMAND_DEBUG_DATA
	Ret.DebugSalt = DebugSalt;
#endif
	return Ret;
}


void FGraphicsMinimalPipelineStateId::InitializePersistentIds()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(InitializePersistentMdcIds);

	FRWScopeLock WriteLock(PersistentIdTableLock, SLT_Write);
	if (NeedsShaderInitialisation)
	{
		for (TPair<const FGraphicsMinimalPipelineStateInitializer, FRefCountedGraphicsMinimalPipelineState>& Element : PersistentIdTable)
		{
			Element.Key.BoundShaderState.LazilyInitShaders();
		}
		NeedsShaderInitialisation = false;
	}
}

void FGraphicsMinimalPipelineStateId::RemovePersistentId(FGraphicsMinimalPipelineStateId Id)
{
	check(!Id.bComesFromLocalPipelineStateSet && Id.bValid);

	{
		FRWScopeLock WriteLock(PersistentIdTableLock, SLT_Write);
		FRefCountedGraphicsMinimalPipelineState& RefCountedStateInitializer = PersistentIdTable.GetByElementId(Id.SetElementIndex).Value;

		check(RefCountedStateInitializer.RefNum > 0);
		--RefCountedStateInitializer.RefNum;
		if (RefCountedStateInitializer.RefNum == 0)
		{
			ReffedItemCount--;
			if (bIsIdTableFrozen == false)
			{
				PersistentIdTable.RemoveByElementId(Id.SetElementIndex);
			}
		}
	}
}

void FGraphicsMinimalPipelineStateId::FreezeIdTable(bool bFreeze)
{
	if (bFreeze != bIsIdTableFrozen)
	{
		FRWScopeLock WriteLock(PersistentIdTableLock, SLT_Write);
		bIsIdTableFrozen = bFreeze;

		// When set back to false, do a pass through the table and clean up any zero ref items.  Zero ref items usually don't happen
		// in practice given the context of how this function is used during calls to FPrimitiveSceneInfo::UpdateStaticMeshes where
		// multiple scene renderers are running (i.e. nDisplay multi-view).  That function removes and re-creates draw commands for a
		// set of render proxies, which we expect to produce the same set of pipeline cache IDs, given that no game logic can run
		// to change any state in the render proxies between renders of the same scene.  We could assert, but cleaning up the zero
		// reference items isn't an unreasonable option either (in case someone tries to use this function for other purposes)...
		if (bFreeze == false)
		{
			// We can detect whether we need to scan the table for zero ref items, by checking whether the number of items with
			// non-zero ref count equals the number of items in the table.
			if (PersistentIdTable.Num() != ReffedItemCount)
			{
				// There's a bug in our reference counting if the reffed item count is more than the items actually in the table
				check(PersistentIdTable.Num() > ReffedItemCount);

				// Remove zero reffed items
				for (auto PersistentIdIterator = PersistentIdTable.begin(); PersistentIdIterator != PersistentIdTable.end(); ++PersistentIdIterator)
				{
					if (PersistentIdTable.GetByElementId(PersistentIdIterator.GetElementId()).Value.RefNum == 0)
					{
						PersistentIdTable.RemoveByElementId(PersistentIdIterator.GetElementId());
					}
				}
			}
		}
	}
}

FGraphicsMinimalPipelineStateId FGraphicsMinimalPipelineStateId::GetPipelineStateId(const FGraphicsMinimalPipelineStateInitializer& InPipelineState, FGraphicsMinimalPipelineStateSet& InOutPassSet, bool& InNeedsShaderInitialisation)
{
	FGraphicsMinimalPipelineStateId Ret;
	Ret.bValid = 1;
	Ret.bComesFromLocalPipelineStateSet = 1;
#if UE_BUILD_DEBUG
	FGraphicsMinimalPipelineStateInitializer PipelineStateDebug = FGraphicsMinimalPipelineStateInitializer(InPipelineState);
	check(GetTypeHash(PipelineStateDebug) == GetTypeHash(InPipelineState));
	check(PipelineStateDebug == InPipelineState);
#endif
	Experimental::FHashElementId TableIndex = InOutPassSet.FindOrAddId(InPipelineState);
#if UE_BUILD_DEBUG
	check(InOutPassSet.GetByElementId(TableIndex) == InPipelineState);
#endif
	InNeedsShaderInitialisation = InNeedsShaderInitialisation || InPipelineState.BoundShaderState.NeedsShaderInitialisation();

	checkf(TableIndex.GetIndex() < (MAX_uint32 >> 2), TEXT("One frame FGraphicsMinimalPipelineStateId table overflow!"));

	Ret.SetElementIndex = TableIndex.GetIndex();
#if MESH_DRAW_COMMAND_DEBUG_DATA
	Ret.DebugSalt = 0;		// Salt is ignored for pipelines from local pipeline state set, just initialize to zero
#endif
	return Ret;
}

void FGraphicsMinimalPipelineStateId::ResetLocalPipelineIdTableSize()
{
#if MESH_DRAW_COMMAND_DEBUG_DATA
	int32 CapturedPipelineIdTableSize;
	do
	{
		CapturedPipelineIdTableSize = CurrentLocalPipelineIdTableSize;
	}while (!CurrentLocalPipelineIdTableSize.compare_exchange_strong(CapturedPipelineIdTableSize, 0));

	LocalPipelineIdTableSize = CapturedPipelineIdTableSize;
#endif //MESH_DRAW_COMMAND_DEBUG_DATA
}

void FGraphicsMinimalPipelineStateId::AddSizeToLocalPipelineIdTableSize(SIZE_T Size)
{
#if MESH_DRAW_COMMAND_DEBUG_DATA
	CurrentLocalPipelineIdTableSize += int32(Size);
#endif
}

FMeshDrawShaderBindings::~FMeshDrawShaderBindings()
{
	Release();
}

void FMeshDrawShaderBindings::Initialize(const FMeshProcessorShaders& Shaders)
{
	const int32 NumShaderFrequencies = 
		(Shaders.VertexShader.IsValid() ? 1 : 0) +
		(Shaders.MeshShader.IsValid() ? 1 : 0) +
		(Shaders.PixelShader.IsValid() ? 1 : 0) +
		(Shaders.GeometryShader.IsValid() ? 1 : 0) +
		(Shaders.ComputeShader.IsValid() ? 1 : 0) +
		(Shaders.WorkGraphShader.IsValid() ? 1 : 0)
#if RHI_RAYTRACING
		+ (Shaders.RayTracingShader.IsValid() ? 1 : 0)
#endif
		;

	ShaderLayouts.Empty(NumShaderFrequencies);
	int32 ShaderBindingDataSize = 0;

	if (Shaders.VertexShader.IsValid())
	{
		ShaderLayouts.Add(FMeshDrawShaderBindingsLayout(Shaders.VertexShader));
		ShaderBindingDataSize += ShaderLayouts.Last().GetDataSizeBytes();
		check(ShaderFrequencyBits < (1 << SF_Vertex));
		ShaderFrequencyBits |= (1 << SF_Vertex);
	}

#if PLATFORM_SUPPORTS_MESH_SHADERS
	if (Shaders.MeshShader.IsValid())
	{
		ShaderLayouts.Add(FMeshDrawShaderBindingsLayout(Shaders.MeshShader));
		ShaderBindingDataSize += ShaderLayouts.Last().GetDataSizeBytes();
		check(ShaderFrequencyBits < (1 << SF_Mesh));
		ShaderFrequencyBits |= (1 << SF_Mesh);
	}
#endif

	if (Shaders.PixelShader.IsValid())
	{
		ShaderLayouts.Add(FMeshDrawShaderBindingsLayout(Shaders.PixelShader));
		ShaderBindingDataSize += ShaderLayouts.Last().GetDataSizeBytes();
		check(ShaderFrequencyBits < (1 << SF_Pixel));
		ShaderFrequencyBits |= (1 << SF_Pixel);
	}

	if (Shaders.GeometryShader.IsValid())
	{
		ShaderLayouts.Add(FMeshDrawShaderBindingsLayout(Shaders.GeometryShader));
		ShaderBindingDataSize += ShaderLayouts.Last().GetDataSizeBytes();
		check(ShaderFrequencyBits < (1 << SF_Geometry));
		ShaderFrequencyBits |= (1 << SF_Geometry);
	}

	if (Shaders.ComputeShader.IsValid())
	{
		ShaderLayouts.Add(FMeshDrawShaderBindingsLayout(Shaders.ComputeShader));
		ShaderBindingDataSize += ShaderLayouts.Last().GetDataSizeBytes();
		check(ShaderFrequencyBits < (1 << SF_Compute));
		ShaderFrequencyBits |= (1 << SF_Compute);
	}

	if (Shaders.WorkGraphShader.IsValid())
	{
		ShaderLayouts.Add(FMeshDrawShaderBindingsLayout(Shaders.WorkGraphShader));
		ShaderBindingDataSize += ShaderLayouts.Last().GetDataSizeBytes();
		check(ShaderFrequencyBits < (1 << SF_WorkGraphComputeNode));
		ShaderFrequencyBits |= (1 << SF_WorkGraphComputeNode);
	}

#if RHI_RAYTRACING
	if (Shaders.RayTracingShader.IsValid())
	{
		ShaderLayouts.Add(FMeshDrawShaderBindingsLayout(Shaders.RayTracingShader));
		ShaderBindingDataSize += ShaderLayouts.Last().GetDataSizeBytes();

		const EShaderFrequency Frequency = Shaders.RayTracingShader->GetFrequency();
		check(ShaderFrequencyBits < (1 << Frequency));
		ShaderFrequencyBits |= (1 << Frequency);
	}
#endif

	checkSlow(ShaderLayouts.Num() == NumShaderFrequencies);

	if (ShaderBindingDataSize > 0)
	{
		AllocateZeroed(ShaderBindingDataSize);
	}
}

void FMeshDrawShaderBindings::Initialize(const TShaderRef<FShader>& Shader)
{
	const int32 NumShaderFrequencies = (Shader.IsValid() ? 1 : 0);

	ShaderLayouts.Empty(NumShaderFrequencies);
	int32 ShaderBindingDataSize = 0;

	if (Shader.IsValid())
	{
		ShaderLayouts.Add(FMeshDrawShaderBindingsLayout(Shader));
		ShaderBindingDataSize += ShaderLayouts.Last().GetDataSizeBytes();

		const EShaderFrequency Frequency = Shader->GetFrequency();
		check(ShaderFrequencyBits < (1 << Frequency));
		ShaderFrequencyBits |= (1 << Frequency);
	}

	checkSlow(ShaderLayouts.Num() == NumShaderFrequencies);

	if (ShaderBindingDataSize > 0)
	{
		AllocateZeroed(ShaderBindingDataSize);
	}
}

void FMeshDrawShaderBindings::Finalize(const FMeshProcessorShaders* ShadersForDebugging)
{
#if VALIDATE_MESH_COMMAND_BINDINGS
	if (!ShadersForDebugging)
	{
		return;
	}

	const uint8* ShaderBindingDataPtr = GetData();
	uint32 ShaderFrequencyBitIndex = ~0;
	for (int32 ShaderBindingsIndex = 0; ShaderBindingsIndex < ShaderLayouts.Num(); ShaderBindingsIndex++)
	{
		EShaderFrequency Frequency = SF_NumFrequencies;
		while (true)
		{
			ShaderFrequencyBitIndex++;
			if ((ShaderFrequencyBits & (1 << ShaderFrequencyBitIndex)) != 0)
			{
				Frequency = EShaderFrequency(ShaderFrequencyBitIndex);
				break;
			}
		}
		check(Frequency < SF_NumFrequencies);

		const FMeshDrawShaderBindingsLayout& ShaderLayout = ShaderLayouts[ShaderBindingsIndex];

		TShaderRef<FShader> Shader = ShadersForDebugging->GetShader(Frequency);
		check(Shader.IsValid());
		const FVertexFactoryType* VFType = Shader.GetVertexFactoryType();

		FReadOnlyMeshDrawSingleShaderBindings SingleShaderBindings(ShaderLayout, ShaderBindingDataPtr);

		FRHIUniformBuffer* const* UniformBufferBindings = SingleShaderBindings.GetUniformBufferStart();

		for (int32 BindingIndex = 0; BindingIndex < ShaderLayout.ParameterMapInfo.UniformBuffers.Num(); BindingIndex++)
		{
			FShaderUniformBufferParameterInfo ParameterInfo = ShaderLayout.ParameterMapInfo.UniformBuffers[BindingIndex];

			FRHIUniformBuffer* UniformBufferValue = UniformBufferBindings[BindingIndex];

			if (!UniformBufferValue)
			{
				// Search the automatically bound uniform buffers for more context if available
				const FShaderParametersMetadata* AutomaticallyBoundUniformBufferStruct = Shader->FindAutomaticallyBoundUniformBufferStruct(ParameterInfo.BaseIndex);

				if (AutomaticallyBoundUniformBufferStruct)
				{
					ensureMsgf(
						UniformBufferValue || EnumHasAnyFlags(AutomaticallyBoundUniformBufferStruct->GetBindingFlags(), EUniformBufferBindingFlags::Static) || EnumHasAnyFlags(AutomaticallyBoundUniformBufferStruct->GetUsageFlags(), FShaderParametersMetadata::EUsageFlags::ManuallyBoundByPass),
						TEXT("Shader %s with vertex factory %s never set automatically bound uniform buffer at BaseIndex %i.  Expected buffer of type %s.  This can cause GPU hangs, depending on how the shader uses it."),
						Shader.GetType()->GetName(), 
						VFType ? VFType->GetName() : TEXT("nullptr"),
						ParameterInfo.BaseIndex,
						AutomaticallyBoundUniformBufferStruct->GetStructTypeName());
				}
				else
				{
					ensureMsgf(UniformBufferValue, TEXT("Shader %s with vertex factory %s never set uniform buffer at BaseIndex %i.  This can cause GPU hangs, depending on how the shader uses it."), 
						VFType ? VFType->GetName() : TEXT("nullptr"),
						Shader.GetType()->GetName(), 
						ParameterInfo.BaseIndex);
				}
			}
		}

		FRHISamplerState* const* SamplerBindings = SingleShaderBindings.GetSamplerStart();

		for (int32 BindingIndex = 0; BindingIndex < ShaderLayout.ParameterMapInfo.TextureSamplers.Num(); BindingIndex++)
		{
			FShaderResourceParameterInfo ParameterInfo = ShaderLayout.ParameterMapInfo.TextureSamplers[BindingIndex];
			const FRHISamplerState* SamplerValue = SamplerBindings[BindingIndex];
			ensureMsgf(SamplerValue, TEXT("Shader %s with vertex factory %s never set sampler at BaseIndex %u.  This can cause GPU hangs, depending on how the shader uses it."), 
				Shader.GetType()->GetName(), 
				VFType ? VFType->GetName() : TEXT("nullptr"),
				ParameterInfo.BaseIndex);
		}

		const uint8* RESTRICT SRVType = SingleShaderBindings.GetSRVTypeStart();
		FRHIResource* const* RESTRICT SRVBindings = SingleShaderBindings.GetSRVStart();
		const FShaderResourceParameterInfo* RESTRICT SRVParameters = SingleShaderBindings.ParameterMapInfo.SRVs.GetData();
		const uint32 NumSRVs = SingleShaderBindings.ParameterMapInfo.SRVs.Num();

		for (uint32 SRVIndex = 0; SRVIndex < NumSRVs; SRVIndex++)
		{
			FShaderResourceParameterInfo Parameter = SRVParameters[SRVIndex];

			uint32 TypeByteIndex = SRVIndex / 8;
			uint32 TypeBitIndex = SRVIndex % 8;

			if (SRVType[TypeByteIndex] & (1 << TypeBitIndex))
			{
				FRHIShaderResourceView* SRV = (FRHIShaderResourceView*)SRVBindings[SRVIndex];

				ensureMsgf(SRV, TEXT("Shader %s with vertex factory %s never set SRV at BaseIndex %u.  This can cause GPU hangs, depending on how the shader uses it."), 
					Shader.GetType()->GetName(), 
					VFType ? VFType->GetName() : TEXT("nullptr"),
					Parameter.BaseIndex);
			}
			else
			{
				FRHITexture* Texture = (FRHITexture*)SRVBindings[SRVIndex];

				ensureMsgf(Texture, TEXT("Shader %s with vertex factory %s never set texture at BaseIndex %u.  This can cause GPU hangs, depending on how the shader uses it."), 
					Shader.GetType()->GetName(), 
					VFType ? VFType->GetName() : TEXT("nullptr"),
					Parameter.BaseIndex);
			}
		}

		ShaderBindingDataPtr += ShaderLayout.GetDataSizeBytes();
	}
#endif
}

void FMeshDrawShaderBindings::CopyFrom(const FMeshDrawShaderBindings& Other)
{
	Release();
	ShaderLayouts = Other.ShaderLayouts;
	ShaderFrequencyBits = Other.ShaderFrequencyBits;

	Allocate(Other.Size);

	if (Other.UsesInlineStorage())
	{
		Data = Other.Data;
	}
	else
	{
		FPlatformMemory::Memcpy(GetData(), Other.GetData(), Size);
	}
}

void FMeshDrawShaderBindings::Release()
{
	if (Size > sizeof(FData))
	{
		delete[] Data.GetHeapData();
	}
	Size = 0;
	Data.SetHeapData(nullptr);
}

bool FMinimalBoundShaderStateInput::AllowSkipUnloadedShaders() const 
{
	return GSkipUnloadedShaders;
}

void FGraphicsMinimalPipelineStateInitializer::SetupBoundShaderState(FRHIVertexDeclaration* VertexDeclaration, const FMeshProcessorShaders& Shaders)
{
	BoundShaderState.~FMinimalBoundShaderStateInput();
	new (&BoundShaderState) FMinimalBoundShaderStateInput();
	BoundShaderState.VertexDeclarationRHI = VertexDeclaration;

#if PLATFORM_SUPPORTS_MESH_SHADERS
	checkf(Shaders.VertexShader.IsValid() || Shaders.MeshShader.IsValid(), TEXT("Can't render without a vertex or mesh shader"));
#else
	checkf(Shaders.VertexShader.IsValid(), TEXT("Can't render without a vertex shader"));
#endif

	if(Shaders.VertexShader.IsValid())
	{
		checkSlow(Shaders.VertexShader->GetFrequency() == SF_Vertex);
		BoundShaderState.VertexShaderResource = Shaders.VertexShader.GetResource();
		BoundShaderState.VertexShaderIndex = Shaders.VertexShader->GetResourceIndex();
		check(BoundShaderState.VertexShaderResource->IsValidShaderIndex(BoundShaderState.VertexShaderIndex));
	}
#if PLATFORM_SUPPORTS_MESH_SHADERS
	if (Shaders.MeshShader.IsValid())
	{
		checkSlow(Shaders.MeshShader->GetFrequency() == SF_Mesh);
		BoundShaderState.MeshShaderResource = Shaders.MeshShader.GetResource();
		BoundShaderState.MeshShaderIndex = Shaders.MeshShader->GetResourceIndex();
		check(BoundShaderState.MeshShaderResource->IsValidShaderIndex(BoundShaderState.MeshShaderIndex));
	}
#endif
	if (Shaders.PixelShader.IsValid())
	{
		checkSlow(Shaders.PixelShader->GetFrequency() == SF_Pixel);
		BoundShaderState.PixelShaderResource = Shaders.PixelShader.GetResource();
		BoundShaderState.PixelShaderIndex = Shaders.PixelShader->GetResourceIndex();
		check(BoundShaderState.PixelShaderResource->IsValidShaderIndex(BoundShaderState.PixelShaderIndex));
	}
#if PLATFORM_SUPPORTS_GEOMETRY_SHADERS
	if (Shaders.GeometryShader.IsValid())
	{
		checkSlow(Shaders.GeometryShader->GetFrequency() == SF_Geometry);
		BoundShaderState.GeometryShaderResource = Shaders.GeometryShader.GetResource();
		BoundShaderState.GeometryShaderIndex = Shaders.GeometryShader->GetResourceIndex();
		check(BoundShaderState.GeometryShaderResource->IsValidShaderIndex(BoundShaderState.GeometryShaderIndex));
	}
#endif // PLATFORM_SUPPORTS_GEOMETRY_SHADERS
}

void FGraphicsMinimalPipelineStateInitializer::ComputeStatePrecachePSOHash()
{
	if(StatePrecachePSOHash == 0)
	{
		StatePrecachePSOHash = AsGraphicsPipelineStateInitializer().StatePrecachePSOHash;
		check(StatePrecachePSOHash);
	}
	else
	{
		checkSlow(StatePrecachePSOHash == AsGraphicsPipelineStateInitializer().StatePrecachePSOHash);
	}
}

void FMeshDrawCommand::SetDrawParametersAndFinalize(
	const FMeshBatch& MeshBatch,
	int32 BatchElementIndex,
	FGraphicsMinimalPipelineStateId PipelineId,
	const FMeshProcessorShaders* ShadersForDebugging)
{
	const FMeshBatchElement& BatchElement = MeshBatch.Elements[BatchElementIndex];

	check(!BatchElement.IndexBuffer || (BatchElement.IndexBuffer && BatchElement.IndexBuffer->IsInitialized() && BatchElement.IndexBuffer->IndexBufferRHI));
	IndexBuffer = BatchElement.IndexBuffer ? BatchElement.IndexBuffer->IndexBufferRHI.GetReference() : nullptr;
	FirstIndex = BatchElement.FirstIndex;
	NumPrimitives = BatchElement.NumPrimitives;
	// Note: we override the instance count to ensure we hash to the same state bucket for MDCs where we know it will be changed at draw time anyway.
	NumInstances = BatchElement.bFetchInstanceCountFromScene ? 1 : BatchElement.NumInstances;

	// If the mesh batch has a valid dynamic index buffer, use it instead
	if (BatchElement.DynamicIndexBuffer.IsValid() &&  BatchElement.DynamicIndexBuffer.IndexBuffer->IsInitialized())
	{
		check(!BatchElement.DynamicIndexBuffer.IndexBuffer || (BatchElement.DynamicIndexBuffer.IndexBuffer && BatchElement.DynamicIndexBuffer.IndexBuffer->IsInitialized() && BatchElement.DynamicIndexBuffer.IndexBuffer->IndexBufferRHI));
		IndexBuffer = BatchElement.DynamicIndexBuffer.IndexBuffer ? BatchElement.DynamicIndexBuffer.IndexBuffer->IndexBufferRHI.GetReference() : nullptr;
		FirstIndex = BatchElement.DynamicIndexBuffer.FirstIndex;
		PrimitiveType = EPrimitiveType(BatchElement.DynamicIndexBuffer.PrimitiveType);
	}

	if (NumPrimitives > 0)
	{
		VertexParams.BaseVertexIndex = BatchElement.BaseVertexIndex;
		VertexParams.NumVertices = BatchElement.MaxVertexIndex - BatchElement.MinVertexIndex + 1;
		checkf(!BatchElement.IndirectArgsBuffer, TEXT("FMeshBatchElement::NumPrimitives must be set to 0 when a IndirectArgsBuffer is used"));
	}
	else
	{
		checkf(BatchElement.IndirectArgsBuffer, TEXT("It is only valid to set BatchElement.NumPrimitives == 0 when a IndirectArgsBuffer is used"));
		IndirectArgs.Buffer = BatchElement.IndirectArgsBuffer;
		IndirectArgs.Offset = BatchElement.IndirectArgsOffset;
	}

#if MESH_DRAW_COMMAND_STATS
	// Store the num vertices for the MDC stats - with indirect args they are otherwise lost because not really needed for draw but handy for stat tracking
	// (can't be retrieved from draw data and indirect args)
	StatsData.NumVertices = BatchElement.MaxVertexIndex - BatchElement.MinVertexIndex + 1;
#endif

	Finalize(PipelineId, ShadersForDebugging);
}

void FMeshDrawShaderBindings::SetOnCommandList(FRHICommandList& RHICmdList, const FBoundShaderStateInput& Shaders, FShaderBindingState* StateCacheShaderBindings) const
{
	const uint8* ShaderBindingDataPtr = GetData();
	uint32 ShaderFrequencyBitIndex = ~0;
	for (int32 ShaderBindingsIndex = 0; ShaderBindingsIndex < ShaderLayouts.Num(); ShaderBindingsIndex++)
	{
		EShaderFrequency Frequency = SF_NumFrequencies;
		while (true)
		{
			ShaderFrequencyBitIndex++;
			if ((ShaderFrequencyBits & (1 << ShaderFrequencyBitIndex)) != 0)
			{
				Frequency = EShaderFrequency(ShaderFrequencyBitIndex);
				break;
			}
		}
		check(Frequency < SF_NumFrequencies);

		FReadOnlyMeshDrawSingleShaderBindings SingleShaderBindings(ShaderLayouts[ShaderBindingsIndex], ShaderBindingDataPtr);
		FShaderBindingState& ShaderBindingState = StateCacheShaderBindings[Frequency];

		if (Frequency == SF_Vertex)
		{
			FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
			FReadOnlyMeshDrawSingleShaderBindings::SetShaderBindings(BatchedParameters, SingleShaderBindings, ShaderBindingState);
			RHICmdList.SetBatchedShaderParameters(Shaders.VertexShaderRHI, BatchedParameters);
		}
	#if PLATFORM_SUPPORTS_MESH_SHADERS
		else if (Frequency == SF_Mesh)
		{
			FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
			FReadOnlyMeshDrawSingleShaderBindings::SetShaderBindings(BatchedParameters, SingleShaderBindings, ShaderBindingState);
			RHICmdList.SetBatchedShaderParameters(Shaders.GetMeshShader(), BatchedParameters);
		}
	#endif
		else if (Frequency == SF_Pixel)
		{
			FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
			FReadOnlyMeshDrawSingleShaderBindings::SetShaderBindings(BatchedParameters, SingleShaderBindings, ShaderBindingState);
			RHICmdList.SetBatchedShaderParameters(Shaders.PixelShaderRHI, BatchedParameters);
		}
		else if (Frequency == SF_Geometry)
		{
			FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
			FReadOnlyMeshDrawSingleShaderBindings::SetShaderBindings(BatchedParameters, SingleShaderBindings, ShaderBindingState);
			RHICmdList.SetBatchedShaderParameters(Shaders.GetGeometryShader(), BatchedParameters);
		}
		else
		{
			checkf(0, TEXT("Unknown shader frequency"));
		}

		ShaderBindingDataPtr += ShaderLayouts[ShaderBindingsIndex].GetDataSizeBytes();
	}
}

void FMeshDrawShaderBindings::SetParameters(FRHIBatchedShaderParameters& BatchedParameters, class FShaderBindingState* StateCacheShaderBindings) const
{
	check(ShaderLayouts.Num() == 1);
	FReadOnlyMeshDrawSingleShaderBindings SingleShaderBindings(ShaderLayouts[0], GetData());

	if (StateCacheShaderBindings != nullptr)
	{
		FReadOnlyMeshDrawSingleShaderBindings::SetShaderBindings(BatchedParameters, SingleShaderBindings, *StateCacheShaderBindings);
	}
	else
	{
		FReadOnlyMeshDrawSingleShaderBindings::SetShaderBindings(BatchedParameters, SingleShaderBindings);
	}
}

void FMeshDrawShaderBindings::SetOnCommandList(FRHIComputeCommandList& RHICmdList, FRHIComputeShader* Shader, FShaderBindingState* StateCacheShaderBindings) const
{
	FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
	SetParameters(BatchedParameters, StateCacheShaderBindings);
	RHICmdList.SetBatchedShaderParameters(Shader, BatchedParameters);
}

bool FMeshDrawShaderBindings::MatchesForDynamicInstancing(const FMeshDrawShaderBindings& Rhs) const
{
	if (ShaderFrequencyBits != Rhs.ShaderFrequencyBits)
	{
		return false;
	}

	if (ShaderLayouts.Num() != Rhs.ShaderLayouts.Num())
	{
		return false;
	}

	for (int Index = 0; Index < ShaderLayouts.Num(); Index++)
	{
		if (!(ShaderLayouts[Index] == Rhs.ShaderLayouts[Index]))
		{
			return false;
		}
	}

	const uint8* ShaderBindingDataPtr = GetData();
	const uint8* OtherShaderBindingDataPtr = Rhs.GetData();

	for (int32 ShaderBindingsIndex = 0; ShaderBindingsIndex < ShaderLayouts.Num(); ShaderBindingsIndex++)
	{
		FReadOnlyMeshDrawSingleShaderBindings SingleShaderBindings(ShaderLayouts[ShaderBindingsIndex], ShaderBindingDataPtr);
		FReadOnlyMeshDrawSingleShaderBindings OtherSingleShaderBindings(Rhs.ShaderLayouts[ShaderBindingsIndex], OtherShaderBindingDataPtr);

		if (SingleShaderBindings.ParameterMapInfo.LooseParameterBuffers.Num())
		{
			const uint8* LooseBindings = SingleShaderBindings.GetLooseDataStart();
			const uint8* OtherLooseBindings = OtherSingleShaderBindings.GetLooseDataStart();
			const uint32 LooseLength = SingleShaderBindings.GetLooseDataSizeBytes();
			const uint32 OtherLength = OtherSingleShaderBindings.GetLooseDataSizeBytes();

			if (LooseLength != OtherLength)
			{
				return false;
			}

			if (memcmp(LooseBindings, OtherLooseBindings, LooseLength) != 0)
			{
				return false;
			}
		}

		FRHISamplerState* const* SamplerBindings = SingleShaderBindings.GetSamplerStart();
		FRHISamplerState* const* OtherSamplerBindings = OtherSingleShaderBindings.GetSamplerStart();
		for (int32 SamplerIndex = 0; SamplerIndex < SingleShaderBindings.ParameterMapInfo.TextureSamplers.Num(); SamplerIndex++)
		{
			const FRHIResource* Sampler = SamplerBindings[SamplerIndex];
			const FRHIResource* OtherSampler = OtherSamplerBindings[SamplerIndex];
			if (Sampler != OtherSampler)
			{
				return false;
			}
		}

		FRHIResource* const* SrvBindings = SingleShaderBindings.GetSRVStart();
		FRHIResource* const* OtherSrvBindings = OtherSingleShaderBindings.GetSRVStart();
		for (int32 SrvIndex = 0; SrvIndex < SingleShaderBindings.ParameterMapInfo.SRVs.Num(); SrvIndex++)
		{
			const FRHIResource* Srv = SrvBindings[SrvIndex];
			const FRHIResource* OtherSrv = OtherSrvBindings[SrvIndex];
			if (Srv != OtherSrv)
			{
				return false;
			}
		}

		FRHIUniformBuffer* const* UniformBufferBindings = SingleShaderBindings.GetUniformBufferStart();
		FRHIUniformBuffer* const* OtherUniformBufferBindings = OtherSingleShaderBindings.GetUniformBufferStart();
		for (int32 UniformBufferIndex = 0; UniformBufferIndex < SingleShaderBindings.ParameterMapInfo.UniformBuffers.Num(); UniformBufferIndex++)
		{
			const FRHIUniformBuffer* UniformBuffer = UniformBufferBindings[UniformBufferIndex];
			const FRHIUniformBuffer* OtherUniformBuffer = OtherUniformBufferBindings[UniformBufferIndex];
			
			if (UniformBuffer != OtherUniformBuffer)
			{
				return false;
			}
		}

		ShaderBindingDataPtr += ShaderLayouts[ShaderBindingsIndex].GetDataSizeBytes();
		OtherShaderBindingDataPtr += Rhs.ShaderLayouts[ShaderBindingsIndex].GetDataSizeBytes();
	}

	return true;
}

uint32 FMeshDrawShaderBindings::GetDynamicInstancingHash() const
{
	//add and initialize any leftover padding within the struct to avoid unstable keys
	struct FHashKey
	{
		uint32 LooseParametersHash = 0;
		uint32 UniformBufferHash = 0;
		uint16 Size;
		uint16 Frequencies;
		static inline uint32 PointerHash(const void* Key)
		{
			// Ignoring the lower 4 bits since they are likely zero anyway.
			// Higher bits are more significant in 64 bit builds.
			return reinterpret_cast<UPTRINT>(Key) >> 4;
		};

		static inline uint32 HashCombine(uint32 A, uint32 B)
	{
			return A ^ (B + 0x9e3779b9 + (A << 6) + (A >> 2));
	}
	} HashKey;

	HashKey.Size = Size;
	HashKey.Frequencies = ShaderFrequencyBits;

	const uint8* ShaderBindingDataPtr = GetData();
	for (int32 ShaderBindingsIndex = 0; ShaderBindingsIndex < ShaderLayouts.Num(); ShaderBindingsIndex++)
	{
		FReadOnlyMeshDrawSingleShaderBindings SingleShaderBindings(ShaderLayouts[ShaderBindingsIndex], ShaderBindingDataPtr);

		if (SingleShaderBindings.ParameterMapInfo.LooseParameterBuffers.Num())
		{
			const uint8* LooseBindings = SingleShaderBindings.GetLooseDataStart();
			uint32 Length = SingleShaderBindings.GetLooseDataSizeBytes();
			HashKey.LooseParametersHash = uint32(CityHash64((const char*)LooseBindings, Length));
		}

		FRHISamplerState* const* SamplerBindings = SingleShaderBindings.GetSamplerStart();
		for (int32 SamplerIndex = 0; SamplerIndex < SingleShaderBindings.ParameterMapInfo.TextureSamplers.Num(); SamplerIndex++)
		{
			const FRHIResource* Sampler = SamplerBindings[SamplerIndex];
			HashKey.LooseParametersHash = FHashKey::HashCombine(FHashKey::PointerHash(Sampler), HashKey.LooseParametersHash);
		}

		FRHIResource* const* SrvBindings = SingleShaderBindings.GetSRVStart();
		for (int32 SrvIndex = 0; SrvIndex < SingleShaderBindings.ParameterMapInfo.SRVs.Num(); SrvIndex++)
		{
			const FRHIResource* Srv = SrvBindings[SrvIndex];
			HashKey.LooseParametersHash = FHashKey::HashCombine(FHashKey::PointerHash(Srv), HashKey.LooseParametersHash);
		}

		FRHIUniformBuffer* const* UniformBufferBindings = SingleShaderBindings.GetUniformBufferStart();
		for (int32 UniformBufferIndex = 0; UniformBufferIndex < SingleShaderBindings.ParameterMapInfo.UniformBuffers.Num(); UniformBufferIndex++)
		{
			const FRHIUniformBuffer* UniformBuffer = UniformBufferBindings[UniformBufferIndex];
			HashKey.UniformBufferHash = FHashKey::HashCombine(FHashKey::PointerHash(UniformBuffer), HashKey.UniformBufferHash);
		}

		ShaderBindingDataPtr += ShaderLayouts[ShaderBindingsIndex].GetDataSizeBytes();
	}

	return uint32(CityHash64((char*)&HashKey, sizeof(FHashKey)));
}

static EPSOPrecacheResult RetrieveAndCachePSOPrecacheResult(
	const FGraphicsMinimalPipelineStateInitializer& MeshPipelineState,
	const FGraphicsPipelineStateInitializer& GraphicsPSOInit,
	bool bAllowSkipDrawCommand)
{
	if (!PipelineStateCache::IsPSOPrecachingEnabled())
	{
		return EPSOPrecacheResult::Unknown;
	}

	EPSOPrecacheResult PSOPrecacheResult = MeshPipelineState.PSOPrecacheState;
	bool bShouldCheckPrecacheResult = false;

	// If PSO precache validation is on, we need to check the state for stats tracking purposes.
#if PSO_PRECACHING_VALIDATE
	if (PSOCollectorStats::IsPrecachingValidationEnabled() && PSOPrecacheResult == EPSOPrecacheResult::Unknown)
	{
		bShouldCheckPrecacheResult = true;
	}
#endif

	// If we are skipping draws when the PSO is being precached but is not ready, we
	// need to keep checking the state until it's not marked active anymore.
	if (bAllowSkipDrawCommand && GSkipDrawOnPSOPrecaching)
	{
		if (PSOPrecacheResult == EPSOPrecacheResult::Unknown ||
			PSOPrecacheResult == EPSOPrecacheResult::Active)
		{
			bShouldCheckPrecacheResult = true;
		}
	}

	if (bShouldCheckPrecacheResult)
	{
		// Cache the state so that it's only checked again if necessary.
		PSOPrecacheResult = PipelineStateCache::CheckPipelineStateInCache(GraphicsPSOInit);
		MeshPipelineState.PSOPrecacheState = PSOPrecacheResult;
	}

	return PSOPrecacheResult;
}

bool FMeshDrawCommand::SubmitDrawBegin(
	const FMeshDrawCommand& RESTRICT MeshDrawCommand, 
	const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	const FMeshDrawCommandSceneArgs& SceneArgs,
	uint32 InstanceFactor,
	FRHICommandList& RHICmdList,
	FMeshDrawCommandStateCache& RESTRICT StateCache,
	bool bAllowSkipDrawCommand)
{
	checkSlow(MeshDrawCommand.CachedPipelineId.IsValid());
	// GPUCULL_TODO: Can't do this check as the VFs are created with GMaxRHIFeatureLevel (so may support PrimitiveIdStreamIndex even for preview platforms)
	// Want to be sure that we supply GPU-scene instance data if required.
	// checkSlow(MeshDrawCommand.PrimitiveIdStreamIndex == -1 || ScenePrimitiveIdsBuffer != nullptr);
	
	const FGraphicsMinimalPipelineStateInitializer& MeshPipelineState = MeshDrawCommand.CachedPipelineId.GetPipelineState(GraphicsMinimalPipelineStateSet);

	if (MeshDrawCommand.CachedPipelineId.GetId() != StateCache.PipelineId)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit = MeshPipelineState.AsGraphicsPipelineStateInitializer();
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		EPSOPrecacheResult PSOPrecacheResult = RetrieveAndCachePSOPrecacheResult(MeshPipelineState, GraphicsPSOInit, bAllowSkipDrawCommand);

#if MESH_DRAW_COMMAND_DEBUG_DATA && MESH_DRAW_COMMAND_STATS
		// Try and skip draw if the shaders are not completly loaded yet.
		if (!MeshPipelineState.BoundShaderState.IsShaderAllLoaded())
		{
			UE_LOGF(LogRenderer, Log, "Missing Preload shaders. Material Friendly: %ls Material Proxy: %ls Static: %ls", *MeshDrawCommand.DebugData.Material->GetFriendlyName(), *MeshDrawCommand.DebugData.MaterialRenderProxyName, *MeshDrawCommand.StatsData.CategoryName.ToString());
			MeshPipelineState.BoundShaderState.ForceShaderReload();
			GraphicsPSOInit.BoundShaderState = MeshPipelineState.BoundShaderState.AsBoundShaderState();
		}
#endif // MESH_DRAW_COMMAND_DEBUG_DATA

#if PSO_PRECACHING_VALIDATE
#if MESH_DRAW_COMMAND_DEBUG_DATA
		PSOCollectorStats::CheckFullPipelineStateInCache(GraphicsPSOInit, PSOPrecacheResult, MeshDrawCommand.DebugData.Material,
			MeshDrawCommand.DebugData.VertexFactoryType, MeshDrawCommand.DebugData.PrimitiveSceneProxyIfNotUsingStateBuckets, MeshDrawCommand.DebugData.PSOCollectorIndex);		
#else
		PSOCollectorStats::CheckFullPipelineStateInCache(GraphicsPSOInit, PSOPrecacheResult, (const FMaterial*)nullptr, nullptr, nullptr, INDEX_NONE);
#endif // MESH_DRAW_COMMAND_DEBUG_DATA
#endif // PSO_PRECACHING_VALIDATE

		// Try and skip draw if the PSO is not precached yet.
		if (bAllowSkipDrawCommand && GSkipDrawOnPSOPrecaching && PSOPrecacheResult == EPSOPrecacheResult::Active)
		{
			return false;
		}

		// We can set the new StencilRef here to avoid the set below
		SetGraphicsPipelineStateCheckApply(RHICmdList, GraphicsPSOInit, MeshDrawCommand.StencilRef);
		StateCache.SetPipelineState(MeshDrawCommand.CachedPipelineId.GetId());
		StateCache.StencilRef = MeshDrawCommand.StencilRef;
	}

	if (MeshDrawCommand.StencilRef != StateCache.StencilRef)
	{
		RHICmdList.SetStencilRef(MeshDrawCommand.StencilRef);
		StateCache.StencilRef = MeshDrawCommand.StencilRef;
	}

	// Platforms that use global UB binding don't need to set PrimitiveIdStream
	const int8 PrimitiveIdStreamIndex = (IsUniformBufferStaticSlotValid(SceneArgs.BatchedPrimitiveSlot) ? -1 : MeshDrawCommand.PrimitiveIdStreamIndex);

	for (int32 VertexBindingIndex = 0; VertexBindingIndex < MeshDrawCommand.VertexStreams.Num(); VertexBindingIndex++)
	{
		const FVertexInputStream& Stream = MeshDrawCommand.VertexStreams[VertexBindingIndex];

		if (PrimitiveIdStreamIndex != -1 && Stream.StreamIndex == PrimitiveIdStreamIndex)
		{
			RHICmdList.SetStreamSource(Stream.StreamIndex, SceneArgs.PrimitiveIdsBuffer, SceneArgs.PrimitiveIdOffset);
			StateCache.VertexStreams[Stream.StreamIndex] = Stream;
		}
		else if (StateCache.VertexStreams[Stream.StreamIndex] != Stream)
		{
			Stream.SetOnRHICommandList(RHICmdList);
			StateCache.VertexStreams[Stream.StreamIndex] = Stream;
		}
	}

	MeshDrawCommand.ShaderBindings.SetOnCommandList(RHICmdList, MeshPipelineState.BoundShaderState.AsBoundShaderState(), StateCache.ShaderBindings);

	return true;
}

void FMeshDrawCommand::SubmitDrawEnd(const FMeshDrawCommand& MeshDrawCommand, const FMeshDrawCommandSceneArgs& SceneArgs, uint32 InstanceFactor, FRHICommandList& RHICmdList)
{
	const bool bDoOverrideArgs = SceneArgs.IndirectArgsBuffer != nullptr && MeshDrawCommand.PrimitiveIdStreamIndex >= 0;

	if (IsUniformBufferStaticSlotValid(SceneArgs.BatchedPrimitiveSlot))
	{
		RHICmdList.SetUniformBufferDynamicOffset(SceneArgs.BatchedPrimitiveSlot, SceneArgs.PrimitiveIdOffset);
	}

	if (SceneArgs.RootConstants.IsSet())
	{
		RHICmdList.SetShaderRootConstants(SceneArgs.RootConstants.GetValue());
	}

	if (MeshDrawCommand.IndexBuffer)
	{
		if (MeshDrawCommand.NumPrimitives > 0 && !bDoOverrideArgs)
		{
			RHICmdList.DrawIndexedPrimitive(
				MeshDrawCommand.IndexBuffer,
				MeshDrawCommand.VertexParams.BaseVertexIndex,
				0,
				MeshDrawCommand.VertexParams.NumVertices,
				MeshDrawCommand.FirstIndex,
				MeshDrawCommand.NumPrimitives,
				MeshDrawCommand.NumInstances * InstanceFactor
			);
		}
		else
		{
			RHICmdList.DrawIndexedPrimitiveIndirect(
				MeshDrawCommand.IndexBuffer,
				bDoOverrideArgs ? SceneArgs.IndirectArgsBuffer : MeshDrawCommand.IndirectArgs.Buffer,
				bDoOverrideArgs ? SceneArgs.IndirectArgsByteOffset : MeshDrawCommand.IndirectArgs.Offset
			);
		}
	}
	else
	{
		if (MeshDrawCommand.NumPrimitives > 0 && !bDoOverrideArgs)
		{
			RHICmdList.DrawPrimitive(
				MeshDrawCommand.VertexParams.BaseVertexIndex + MeshDrawCommand.FirstIndex,
				MeshDrawCommand.NumPrimitives,
				MeshDrawCommand.NumInstances * InstanceFactor);
		}
		else
		{
			RHICmdList.DrawPrimitiveIndirect(
				bDoOverrideArgs ? SceneArgs.IndirectArgsBuffer : MeshDrawCommand.IndirectArgs.Buffer,
				bDoOverrideArgs ? SceneArgs.IndirectArgsByteOffset : MeshDrawCommand.IndirectArgs.Offset
			);
		}
	}
}

bool FMeshDrawCommand::SubmitDrawIndirectBegin(
	const FMeshDrawCommand& RESTRICT MeshDrawCommand,
	const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	const FMeshDrawCommandSceneArgs& SceneArgs,
	uint32 InstanceFactor,
	FRHICommandList& RHICmdList,
	FMeshDrawCommandStateCache& RESTRICT StateCache,
	bool bAllowSkipDrawCommand)
{
	return SubmitDrawBegin(
		MeshDrawCommand,
		GraphicsMinimalPipelineStateSet,
		SceneArgs,
		InstanceFactor,
		RHICmdList,
		StateCache,
		bAllowSkipDrawCommand
	);
}

void FMeshDrawCommand::SubmitDrawIndirectEnd(
	const FMeshDrawCommand& MeshDrawCommand,
	const FMeshDrawCommandSceneArgs& SceneArgs,
	uint32 InstanceFactor,
	FRHICommandList& RHICmdList)
{
	FRHIBuffer* IndirectArgsBuffer = nullptr;
	uint32		IndirectArgsOffset = 0;

	if (MeshDrawCommand.NumPrimitives == 0)
	{
		IndirectArgsBuffer = MeshDrawCommand.IndirectArgs.Buffer;
		IndirectArgsOffset = MeshDrawCommand.IndirectArgs.Offset;
	}

	if (SceneArgs.IndirectArgsBuffer != nullptr)
	{
		IndirectArgsBuffer = SceneArgs.IndirectArgsBuffer;
		IndirectArgsOffset = SceneArgs.IndirectArgsByteOffset;
	}
	
	if (IsUniformBufferStaticSlotValid(SceneArgs.BatchedPrimitiveSlot))
	{
		RHICmdList.SetUniformBufferDynamicOffset(SceneArgs.BatchedPrimitiveSlot, SceneArgs.PrimitiveIdOffset);
	}

	if (IndirectArgsBuffer != nullptr)
	{
		if (MeshDrawCommand.IndexBuffer)
		{
			RHICmdList.DrawIndexedPrimitiveIndirect(
				MeshDrawCommand.IndexBuffer,
				IndirectArgsBuffer,
				IndirectArgsOffset
			);
		}
		else
		{
			RHICmdList.DrawPrimitiveIndirect(
				IndirectArgsBuffer,
				IndirectArgsOffset
			);
		}
	}
	else if (MeshDrawCommand.NumPrimitives > 0)
	{
		if (MeshDrawCommand.IndexBuffer)
		{
			RHICmdList.DrawIndexedPrimitive(
				MeshDrawCommand.IndexBuffer,
				MeshDrawCommand.VertexParams.BaseVertexIndex,
				0,
				MeshDrawCommand.VertexParams.NumVertices,
				MeshDrawCommand.FirstIndex,
				MeshDrawCommand.NumPrimitives,
				MeshDrawCommand.NumInstances * InstanceFactor
			);
		}
		else
		{
			RHICmdList.DrawPrimitive(
				MeshDrawCommand.VertexParams.BaseVertexIndex + MeshDrawCommand.FirstIndex,
				MeshDrawCommand.NumPrimitives,
				MeshDrawCommand.NumInstances * InstanceFactor
			);
		}
	}
}

bool FMeshDrawCommand::SubmitDraw(
	const FMeshDrawCommand& RESTRICT MeshDrawCommand,
	const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	const FMeshDrawCommandSceneArgs& SceneArgs,
	uint32 InstanceFactor,
	FRHICommandList& RHICmdList,
	FMeshDrawCommandStateCache& RESTRICT StateCache)
{
#if WANTS_DRAW_MESH_EVENTS
	RHI_BREADCRUMB_EVENT_CONDITIONAL_F(RHICmdList, GShowMaterialDrawEvents != 0, "MaterialDrawEvent", "%s %s (%u instances)"
		, MeshDrawCommand.DebugData.MaterialRenderProxyName
		, MeshDrawCommand.DebugData.ResourceName
		, MeshDrawCommand.NumInstances * InstanceFactor
	);
#endif

	bool bAllowSkipDrawCommand = true;
	if (SubmitDrawBegin(MeshDrawCommand, GraphicsMinimalPipelineStateSet, SceneArgs, InstanceFactor, RHICmdList, StateCache, bAllowSkipDrawCommand))
	{
		SubmitDrawEnd(MeshDrawCommand, SceneArgs, InstanceFactor, RHICmdList);
		return true;
	}

	return false;
}

void ApplyTargetsInfo(FGraphicsPipelineStateInitializer& GraphicsPSOInit, const FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo)
{
	GraphicsPSOInit.RenderTargetsEnabled = RenderTargetsInfo.RenderTargetsEnabled;
	GraphicsPSOInit.RenderTargetFormats = RenderTargetsInfo.RenderTargetFormats;
	GraphicsPSOInit.RenderTargetFlags = RenderTargetsInfo.RenderTargetFlags;
	GraphicsPSOInit.NumSamples = RenderTargetsInfo.NumSamples;

	GraphicsPSOInit.DepthStencilTargetFormat = RenderTargetsInfo.DepthStencilTargetFormat;
	GraphicsPSOInit.DepthStencilTargetFlag = RenderTargetsInfo.DepthStencilTargetFlag;

	GraphicsPSOInit.DepthTargetLoadAction = RenderTargetsInfo.DepthTargetLoadAction;
	GraphicsPSOInit.DepthTargetStoreAction = RenderTargetsInfo.DepthTargetStoreAction;
	GraphicsPSOInit.StencilTargetLoadAction = RenderTargetsInfo.StencilTargetLoadAction;
	GraphicsPSOInit.StencilTargetStoreAction = RenderTargetsInfo.StencilTargetStoreAction;
	GraphicsPSOInit.DepthStencilAccess = RenderTargetsInfo.DepthStencilAccess;

	GraphicsPSOInit.MultiViewCount = RenderTargetsInfo.MultiViewCount;
	GraphicsPSOInit.bHasFragmentDensityAttachment = RenderTargetsInfo.bHasFragmentDensityAttachment;
}

ESubpassHint GetSubpassHint(const FStaticShaderPlatform Platform, bool bIsUsingGBuffers, bool bMultiViewRendering, uint32 NumSamples)
{
	ESubpassHint SubpassHint = ESubpassHint::None;
	
	if (IsMobilePlatform(Platform) && MobileAllowFramebufferFetch(Platform))
	{
		if (bIsUsingGBuffers)
		{
			SubpassHint = ESubpassHint::DeferredShadingSubpass;
		}
		else
		{
			SubpassHint = IsMobileTonemapSubpassEnabledInline(Platform, bMultiViewRendering, NumSamples) ? ESubpassHint::CustomResolveSubpass : ESubpassHint::DepthReadSubpass;
		}
	}

	return SubpassHint;
}

uint64 FMeshDrawCommand::GetPipelineStateSortingKey(FRHICommandList& RHICmdList, const FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo) const
{
	// Default fallback sort key
	uint64 SortKey = CachedPipelineId.GetId();

	if (GRHISupportsPipelineStateSortKey)
	{
		FGraphicsMinimalPipelineStateSet PipelineStateSet;
		FGraphicsPipelineStateInitializer GraphicsPSOInit = CachedPipelineId.GetPipelineState(PipelineStateSet).AsGraphicsPipelineStateInitializer();
		ApplyTargetsInfo(GraphicsPSOInit, RenderTargetsInfo);

		// PSO is retrieved here already. This is currently only used by Nanite::DrawLumenMeshCards - can this also used the version without command list?
		const FGraphicsPipelineState* PipelineState = PipelineStateCache::GetAndOrCreateGraphicsPipelineState(RHICmdList, GraphicsPSOInit, EApplyRendertargetOption::DoNothing);
		if (PipelineState)
		{
			const uint64 StateSortKey = PipelineStateCache::RetrieveGraphicsPipelineStateSortKey(PipelineState);
			if (StateSortKey != 0) // 0 on the first occurrence (prior to caching), so these commands will fall back on shader id for sorting.
			{
				SortKey = StateSortKey;
			}
		}
	}

	return SortKey;
}

uint64 FMeshDrawCommand::GetPipelineStateSortingKey(const FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo) const
{
	// Default fallback sort key
	uint64 SortKey = CachedPipelineId.GetId();

	if (GRHISupportsPipelineStateSortKey)
	{
		FGraphicsMinimalPipelineStateSet PipelineStateSet;
		FGraphicsPipelineStateInitializer GraphicsPSOInit = CachedPipelineId.GetPipelineState(PipelineStateSet).AsGraphicsPipelineStateInitializer();
		ApplyTargetsInfo(GraphicsPSOInit, RenderTargetsInfo);

		const FGraphicsPipelineState* PipelineState = PipelineStateCache::FindGraphicsPipelineState(GraphicsPSOInit, false /* bVerifyUse */);
		if (PipelineState)
		{
			const uint64 StateSortKey = PipelineStateCache::RetrieveGraphicsPipelineStateSortKey(PipelineState);
			if (StateSortKey != 0) // 0 on the first occurrence (prior to caching), so these commands will fall back on shader id for sorting.
			{
				SortKey = StateSortKey;
			}
		}
	}

	return SortKey;
}

#if MESH_DRAW_COMMAND_DEBUG_DATA
void FMeshDrawCommand::SetDebugData(const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FMaterial* Material, const FMaterialRenderProxy* MaterialRenderProxy, const FMeshProcessorShaders& UntypedShaders, const FVertexFactory* VertexFactory, const FMeshBatch& MeshBatch, int32 PSOCollectorIndex)
{
	// The following members are potentially invalid when using bucketing (since multiple proxies can share the same MDC, even in the case of materials)
	DebugData.PrimitiveSceneProxyIfNotUsingStateBuckets = PrimitiveSceneProxy;
	DebugData.MaterialRenderProxyIfNotUsingStateBuckets = MaterialRenderProxy;

	DebugData.Material = Material;
	DebugData.MaterialRenderProxyName = MaterialRenderProxy ? MaterialRenderProxy->GetMaterialName() : TEXT(""); 
	DebugData.VertexShader = UntypedShaders.VertexShader;
	DebugData.PixelShader = UntypedShaders.PixelShader;
	DebugData.LODIndex = MeshBatch.LODIndex;
	DebugData.SegmentIndex = MeshBatch.SegmentIndex;
	DebugData.ResourceName =  PrimitiveSceneProxy ? PrimitiveSceneProxy->GetResourceName() : FName();
#if PSO_PRECACHING_VALIDATE
	DebugData.VertexFactory = VertexFactory;
	DebugData.VertexFactoryType = VertexFactory->GetType();
	DebugData.PSOCollectorIndex = PSOCollectorIndex;
#endif
}
#endif

#if MESH_DRAW_COMMAND_STATS
void FMeshDrawCommand::SetStatsData(const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	StatsData.CategoryName = PrimitiveSceneProxy ? PrimitiveSceneProxy->GetMeshDrawCommandStatsCategory() : FName();
}

void FMeshDrawCommand::GetStatsData(FVisibleMeshDrawCommandStatsData& OutVisibleStatsData) const
{
	OutVisibleStatsData.StatsData = StatsData;
	OutVisibleStatsData.PrimitiveCount = NumPrimitives;
#if MESH_DRAW_COMMAND_DEBUG_DATA
	OutVisibleStatsData.LODIndex = DebugData.LODIndex;
	OutVisibleStatsData.SegmentIndex = DebugData.SegmentIndex;
	OutVisibleStatsData.ResourceName = DebugData.ResourceName;
	OutVisibleStatsData.MaterialName = DebugData.MaterialRenderProxyName;
#endif
}
#endif // MESH_DRAW_COMMAND_STATS

void SubmitMeshDrawCommands(
	const FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	const FMeshDrawCommandSceneArgs& SceneArgs,
	uint32 PrimitiveIdBufferStride,
	bool bDynamicInstancing,
	uint32 InstanceFactor,
	FRHICommandList& RHICmdList)
{
	SubmitMeshDrawCommandsRange(VisibleMeshDrawCommands, GraphicsMinimalPipelineStateSet, SceneArgs, PrimitiveIdBufferStride, bDynamicInstancing, 0, VisibleMeshDrawCommands.Num(), InstanceFactor, RHICmdList);
}

void SubmitMeshDrawCommandsRange(
	const FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	const FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	const FMeshDrawCommandSceneArgs& InSceneArgs,
	uint32 PrimitiveIdBufferStride,
	bool bDynamicInstancing,
	int32 StartIndex,
	int32 NumMeshDrawCommands,
	uint32 InstanceFactor,
	FRHICommandList& RHICmdList)
{
	// GPUCULL_TODO: workaround for the fact that DrawDynamicMeshPassPrivate et al. don't work with GPU-Scene instancing
	//               we don't support dynamic instancing for this path since we require one primitive per draw command
	//               This is because the stride on the instance data buffer is set to 0 so only the first will ever be fetched.
	checkSlow(!bDynamicInstancing);
	bDynamicInstancing = false;

	FMeshDrawCommandStateCache StateCache;
	FMeshDrawCommandSceneArgs LocalSceneArgs = InSceneArgs;
	
	INC_DWORD_STAT_BY(STAT_MeshDrawCalls, NumMeshDrawCommands);

	for (int32 DrawCommandIndex = StartIndex; DrawCommandIndex < StartIndex + NumMeshDrawCommands; DrawCommandIndex++)
	{
		SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, MeshEvent, GEmitMeshDrawEvent != 0, TEXT("Mesh Draw"));

		const FVisibleMeshDrawCommand& VisibleMeshDrawCommand = VisibleMeshDrawCommands[DrawCommandIndex];
		LocalSceneArgs.PrimitiveIdOffset = InSceneArgs.PrimitiveIdOffset + (bDynamicInstancing ? VisibleMeshDrawCommand.PrimitiveIdBufferOffset : DrawCommandIndex) * PrimitiveIdBufferStride;
		checkSlow(!bDynamicInstancing || VisibleMeshDrawCommand.PrimitiveIdBufferOffset >= 0);
		FMeshDrawCommand::SubmitDraw(*VisibleMeshDrawCommand.MeshDrawCommand, GraphicsMinimalPipelineStateSet, LocalSceneArgs, InstanceFactor, RHICmdList, StateCache);
	}
}

void ApplyViewOverridesToMeshDrawCommands(const FSceneView& View, FMeshCommandOneFrameArray& VisibleMeshDrawCommands, FDynamicMeshDrawCommandStorage& DynamicMeshDrawCommandStorage, FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet, bool& InNeedsShaderInitialisation)
{
	if (View.bReverseCulling || View.bRenderSceneTwoSided)
	{
		const FMeshCommandOneFrameArray& PassVisibleMeshDrawCommands = VisibleMeshDrawCommands;

		FMeshCommandOneFrameArray ViewOverriddenMeshCommands;
		ViewOverriddenMeshCommands.Empty(PassVisibleMeshDrawCommands.Num());

		for (int32 MeshCommandIndex = 0; MeshCommandIndex < PassVisibleMeshDrawCommands.Num(); MeshCommandIndex++)
		{
			DynamicMeshDrawCommandStorage.MeshDrawCommands.Add(1);
			FMeshDrawCommand& NewMeshCommand = DynamicMeshDrawCommandStorage.MeshDrawCommands[DynamicMeshDrawCommandStorage.MeshDrawCommands.Num() - 1];

			const FVisibleMeshDrawCommand& VisibleMeshDrawCommand = PassVisibleMeshDrawCommands[MeshCommandIndex];
			const FMeshDrawCommand& MeshCommand = *VisibleMeshDrawCommand.MeshDrawCommand;
			NewMeshCommand = MeshCommand;

			const ERasterizerCullMode LocalCullMode = View.bRenderSceneTwoSided ? CM_None : View.bReverseCulling ? FMeshPassProcessor::InverseCullMode(VisibleMeshDrawCommand.MeshCullMode) : VisibleMeshDrawCommand.MeshCullMode;

			FGraphicsMinimalPipelineStateInitializer PipelineState = MeshCommand.CachedPipelineId.GetPipelineState(GraphicsMinimalPipelineStateSet);
			PipelineState.RasterizerState = GetStaticRasterizerState<true>(VisibleMeshDrawCommand.MeshFillMode, LocalCullMode);

			const FGraphicsMinimalPipelineStateId PipelineId = FGraphicsMinimalPipelineStateId::GetPipelineStateId(PipelineState, GraphicsMinimalPipelineStateSet, InNeedsShaderInitialisation);
			NewMeshCommand.Finalize(PipelineId, nullptr);

			FVisibleMeshDrawCommand NewVisibleMeshDrawCommand;

			NewVisibleMeshDrawCommand.Setup(
				&NewMeshCommand,
				VisibleMeshDrawCommand.PrimitiveIdInfo,
				VisibleMeshDrawCommand.StateBucketId,
				VisibleMeshDrawCommand.MeshFillMode,
				VisibleMeshDrawCommand.MeshCullMode,
				VisibleMeshDrawCommand.Flags,
				VisibleMeshDrawCommand.SortKey,
				VisibleMeshDrawCommand.CullingPayload,
				VisibleMeshDrawCommand.CullingPayloadFlags,
				VisibleMeshDrawCommand.RunArray,
				VisibleMeshDrawCommand.NumRuns);

			ViewOverriddenMeshCommands.Add(NewVisibleMeshDrawCommand);
		}

		// Replace VisibleMeshDrawCommands
		Swap(VisibleMeshDrawCommands, ViewOverriddenMeshCommands);
	}
}

class FDynamicBatchedPrimitiveLayout : public FRenderResource
{
public:
	void InitRHI(FRHICommandListBase& RHICmdList) override 
	{
		// This Layout fully replicates BatchedPrimitive UB
		// we replace RDG_SRV with a regular SRV to be able to update UB inside RDG passes
		const static FName BatchedPrimitiveSlotName = "BatchedPrimitive";
		FRHIUniformBufferLayoutInitializer Initializer(TEXT("FBatchedPrimitiveParameters"), 16u);
		EnumAddFlags(Initializer.Flags, ERHIUniformBufferFlags::UniformView);
		EnumAddFlags(Initializer.Flags, ERHIUniformBufferFlags::NoEmulatedUniformBuffer); // implicit for UniformView
		Initializer.Resources.Add({0, UBMT_RDG_BUFFER_SRV});
		Initializer.StaticSlot = FUniformBufferStaticSlotRegistry::Get().FindSlotByName(BatchedPrimitiveSlotName);
		Initializer.BindingFlags = EUniformBufferBindingFlags::StaticAndShader;
		Initializer.ComputeHash();
		// set view source to a regular SRV after hash computation, to make sure hash matches BatchedPrimitive layout
		Initializer.Resources[0].MemberType = UBMT_SRV;

		LayoutRHI = RHICreateUniformBufferLayout(Initializer);
	}

	void ReleaseRHI() override 
	{
		LayoutRHI = nullptr;
	}

	FUniformBufferLayoutRHIRef LayoutRHI;
};

TGlobalResource<FDynamicBatchedPrimitiveLayout> GDynamicBatchedPrimitiveLayout;

void DrawDynamicMeshPassPrivate(
	const FSceneView& View,
	FRHICommandList& RHICmdList,
	FMeshCommandOneFrameArray& VisibleMeshDrawCommands,
	FDynamicMeshDrawCommandStorage& DynamicMeshDrawCommandStorage,
	FGraphicsMinimalPipelineStateSet& GraphicsMinimalPipelineStateSet,
	bool& InNeedsShaderInitialisation,
	uint32 InstanceFactor)
{
	if (VisibleMeshDrawCommands.Num() > 0)
	{
		// GPUCULL_TODO: workaround for the fact that DrawDynamicMeshPassPrivate et al. don't work with GPU-Scene instancing
		//               we don't support dynamic instancing for this path since we require one primitive per draw command
		//               This is because the stride on the instance data buffer is set to 0 so only the first will ever be fetched.
		const bool bDynamicInstancing = false;
		const ERHIFeatureLevel::Type FeatureLevel = View.GetFeatureLevel();
		const EShaderPlatform ShaderPlatform = View.GetShaderPlatform();
		const bool bUseGPUScene = UseGPUScene(ShaderPlatform);
		
		ApplyViewOverridesToMeshDrawCommands(View, VisibleMeshDrawCommands, DynamicMeshDrawCommandStorage, GraphicsMinimalPipelineStateSet, InNeedsShaderInitialisation);

		check(View.bIsViewInfo);
		const FViewInfo* ViewInfo = static_cast<const FViewInfo*>(&View);

		FMeshDrawCommandSceneArgs SceneArgs;
		SortAndMergeDynamicPassMeshDrawCommands(View, RHICmdList, VisibleMeshDrawCommands, DynamicMeshDrawCommandStorage, SceneArgs.PrimitiveIdsBuffer, InstanceFactor, &ViewInfo->DynamicPrimitiveCollector);

		if (bUseGPUScene && PlatformGPUSceneUsesUniformBufferView(ShaderPlatform))
		{
			FShaderResourceViewRHIRef BatchedPrimitiveSRV = RHICmdList.CreateShaderResourceView(SceneArgs.PrimitiveIdsBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(SceneArgs.PrimitiveIdsBuffer));
			SceneArgs.PrimitiveIdsBuffer = nullptr;
			FRHIShaderResourceView* SRV = BatchedPrimitiveSRV.GetReference();
			FUniformBufferRHIRef UBRef = RHICreateUniformBuffer(&SRV, GDynamicBatchedPrimitiveLayout.LayoutRHI, UniformBuffer_SingleFrame, EUniformBufferValidation::None);
			SceneArgs.BatchedPrimitiveSlot = FInstanceCullingContext::GetStaticUniformBufferSlot(ShaderPlatform);
			RHICmdList.SetStaticUniformBuffer(SceneArgs.BatchedPrimitiveSlot, UBRef);
		}
		
		const uint32 PrimitiveIdBufferStride = FInstanceCullingContext::GetInstanceIdBufferStride(ShaderPlatform);
		SubmitMeshDrawCommandsRange(VisibleMeshDrawCommands, GraphicsMinimalPipelineStateSet, SceneArgs, PrimitiveIdBufferStride, bDynamicInstancing, 0, VisibleMeshDrawCommands.Num(), InstanceFactor, RHICmdList);
	}
}


FMeshDrawCommandSortKey CalculateMeshStaticSortKey(const FMeshMaterialShader* VertexShader, const FMeshMaterialShader* PixelShader)
{
	FMeshDrawCommandSortKey SortKey;
	SortKey.Generic.VertexShaderHash = VertexShader ? VertexShader->GetSortKey() : 0;
	SortKey.Generic.PixelShaderHash = PixelShader ? PixelShader->GetSortKey() : 0;

	return SortKey;
}

FMeshPassProcessor::FMeshPassProcessor(EMeshPass::Type InMeshPassType, const FScene* InScene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext) 
	: IPSOCollector(FPassProcessorManager::GetPSOCollectorIndex(GetFeatureLevelShadingPath(InFeatureLevel), InMeshPassType))
	, MeshPassType(InMeshPassType)
	, Scene(InScene)
	, FeatureLevel(InFeatureLevel)
	, ViewIfDynamicMeshCommand(InViewIfDynamicMeshCommand)
	, DrawListContext(InDrawListContext)
{	
}

FMeshPassProcessor::FMeshPassProcessor(const TCHAR* InMeshPassName, const FScene* InScene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	: IPSOCollector(FPSOCollectorCreateManager::GetIndex(GetFeatureLevelShadingPath(InFeatureLevel), InMeshPassName))
	, MeshPassType(EMeshPass::Num)
	, Scene(InScene)
	, FeatureLevel(InFeatureLevel)
	, ViewIfDynamicMeshCommand(InViewIfDynamicMeshCommand)
	, DrawListContext(InDrawListContext)
{
}

FMeshPassProcessor::FMeshDrawingPolicyOverrideSettings FMeshPassProcessor::ComputeMeshOverrideSettings(const FPSOPrecacheParams& PrecachePSOParams)
{
	FMeshDrawingPolicyOverrideSettings OverrideSettings;
	OverrideSettings.MeshPrimitiveType = PT_TriangleList;
		
	OverrideSettings.MeshOverrideFlags |= PrecachePSOParams.bDisableBackFaceCulling ? EDrawingPolicyOverrideFlags::TwoSided : EDrawingPolicyOverrideFlags::None;
	OverrideSettings.MeshOverrideFlags |= PrecachePSOParams.bReverseCulling ? EDrawingPolicyOverrideFlags::ReverseCullMode : EDrawingPolicyOverrideFlags::None;
	return OverrideSettings;
}

FMeshPassProcessor::FMeshDrawingPolicyOverrideSettings FMeshPassProcessor::ComputeMeshOverrideSettings(const FMeshBatch& Mesh)
{
	FMeshDrawingPolicyOverrideSettings OverrideSettings;
	OverrideSettings.MeshPrimitiveType = (EPrimitiveType)Mesh.Type;

	OverrideSettings.MeshOverrideFlags |= Mesh.bDisableBackfaceCulling ? EDrawingPolicyOverrideFlags::TwoSided : EDrawingPolicyOverrideFlags::None;
	OverrideSettings.MeshOverrideFlags |= Mesh.bDitheredLODTransition ? EDrawingPolicyOverrideFlags::DitheredLODTransition : EDrawingPolicyOverrideFlags::None;
	OverrideSettings.MeshOverrideFlags |= Mesh.bWireframe ? EDrawingPolicyOverrideFlags::Wireframe : EDrawingPolicyOverrideFlags::None;
	OverrideSettings.MeshOverrideFlags |= Mesh.ReverseCulling ? EDrawingPolicyOverrideFlags::ReverseCullMode : EDrawingPolicyOverrideFlags::None;
	return OverrideSettings;
}

ERasterizerFillMode FMeshPassProcessor::ComputeMeshFillMode(const FMaterial& InMaterialResource, const FMeshDrawingPolicyOverrideSettings& InOverrideSettings)
{
	const bool bMaterialResourceIsTwoSided = InMaterialResource.IsTwoSided();
	const bool bIsWireframeMaterial = InMaterialResource.IsWireframe() || !!(InOverrideSettings.MeshOverrideFlags & EDrawingPolicyOverrideFlags::Wireframe);
	return bIsWireframeMaterial ? FM_Wireframe : FM_Solid;
}

ERasterizerCullMode FMeshPassProcessor::ComputeMeshCullMode(const FMaterial& InMaterialResource, const FMeshDrawingPolicyOverrideSettings& InOverrideSettings)
{
	const bool bMaterialResourceIsTwoSided = InMaterialResource.IsTwoSided();
	const bool bInTwoSidedOverride = !!(InOverrideSettings.MeshOverrideFlags & EDrawingPolicyOverrideFlags::TwoSided);
	const bool bInReverseCullModeOverride = !!(InOverrideSettings.MeshOverrideFlags & EDrawingPolicyOverrideFlags::ReverseCullMode);
	const bool bIsTwoSided = (bMaterialResourceIsTwoSided || bInTwoSidedOverride);
	const bool bMeshRenderTwoSided = bIsTwoSided || bInTwoSidedOverride;
	return bMeshRenderTwoSided ? CM_None : (bInReverseCullModeOverride ? CM_CCW : CM_CW);
}

FMeshDrawCommandPrimitiveIdInfo FMeshPassProcessor::GetDrawCommandPrimitiveId(
	const FPrimitiveSceneInfo* RESTRICT PrimitiveSceneInfo,
	const FMeshBatchElement& BatchElement) const
{
	FMeshDrawCommandPrimitiveIdInfo PrimitiveIdInfo = FMeshDrawCommandPrimitiveIdInfo(0, FPersistentPrimitiveIndex { 0 }, -1);

	if (UseGPUScene(GShaderPlatformForFeatureLevel[FeatureLevel]))
	{
		if (BatchElement.PrimitiveIdMode == PrimID_FromPrimitiveSceneInfo)
		{
			ensureMsgf(BatchElement.PrimitiveUniformBufferResource == nullptr, TEXT("PrimitiveUniformBufferResource should not be setup when PrimitiveIdMode == PrimID_FromPrimitiveSceneInfo"));
			check(PrimitiveSceneInfo);
			PrimitiveIdInfo.DrawPrimitiveId = PrimitiveSceneInfo->GetPersistentIndex().Index;
			PrimitiveIdInfo.InstanceSceneDataOffset = PrimitiveSceneInfo->GetInstanceSceneDataOffset();
			PrimitiveIdInfo.bIsDynamicPrimitive = 0U;
		}
		else if (BatchElement.PrimitiveIdMode == PrimID_DynamicPrimitiveShaderData && ViewIfDynamicMeshCommand != nullptr)
		{
			// Mark using GPrimIDDynamicFlag (top bit) as we defer this to later.
			PrimitiveIdInfo.DrawPrimitiveId = BatchElement.DynamicPrimitiveIndex | GPrimIDDynamicFlag;
			PrimitiveIdInfo.InstanceSceneDataOffset = BatchElement.DynamicPrimitiveInstanceSceneDataOffset;
			PrimitiveIdInfo.bIsDynamicPrimitive = 1U;
		}
		else
		{
			check(BatchElement.PrimitiveIdMode == PrimID_ForceZero);
		}
	}

	PrimitiveIdInfo.ScenePrimitiveId = PrimitiveSceneInfo ? PrimitiveSceneInfo->GetIndex() : -1;

	return PrimitiveIdInfo;
}

bool FMeshPassProcessor::ShouldSkipMeshDrawCommand(const FMeshBatch& RESTRICT MeshBatch, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy) const
{
	bool bSkipMeshDrawCommand = false;

#if WITH_EDITORONLY_DATA
	// Support debug mode to render only non-Nanite proxies that incorrectly reference coarse mesh static mesh assets.
	if (GNaniteIsolateInvalidCoarseMesh != 0)
	{
		// Skip everything by default
		bSkipMeshDrawCommand = true;

		const bool bNaniteProxy = PrimitiveSceneProxy != nullptr && PrimitiveSceneProxy->IsNaniteMesh();
		if (!bNaniteProxy && MeshBatch.VertexFactory != nullptr)
		{
			// Only skip if the referenced static mesh is not a generated Nanite coarse mesh
			const bool bIsCoarseProxy = MeshBatch.VertexFactory->IsCoarseProxyMesh();
			if (bIsCoarseProxy)
			{
				bSkipMeshDrawCommand = false;
			}
		}
	}
#endif

	return bSkipMeshDrawCommand;
}

bool FMeshPassProcessor::PipelineVariableRateShadingEnabled() const
{
	return GVRSImageManager.IsPipelineVRSEnabled();
}

bool FMeshPassProcessor::HardwareVariableRateShadingSupportedByScene() const
{
	if (Scene)
	{
		return HardwareVariableRateShadingSupportedByPlatform(Scene->GetShaderPlatform());
	}
	else if (ViewIfDynamicMeshCommand)
	{
		// When applying changes to a material in the editor, we take this pathway
		return HardwareVariableRateShadingSupportedByPlatform(ViewIfDynamicMeshCommand->GetShaderPlatform());
	}
	else
	{
		return false;
	}
}

FCachedPassMeshDrawListContext::FCachedPassMeshDrawListContext(FScene& InScene)
	: Scene(InScene)
	, bUseGPUScene(UseGPUScene(GMaxRHIShaderPlatform))
{
}

FMeshDrawCommand& FCachedPassMeshDrawListContext::AddCommand(FMeshDrawCommand& Initializer, uint32 NumElements)
{
	checkf(CurrMeshPass < EMeshPass::Num, TEXT("BeginMeshPass() must be called before adding commands to this context"));
	ensureMsgf(CommandInfo.CommandIndex == -1 && CommandInfo.StateBucketId == -1, TEXT("GetCommandInfoAndReset() wasn't called since the last command was added"));

	if (NumElements == 1)
	{
		return Initializer;
	}
	else
	{
		MeshDrawCommandForStateBucketing = Initializer;
		return MeshDrawCommandForStateBucketing;
	}
}

void FCachedPassMeshDrawListContext::BeginMeshPass(EMeshPass::Type MeshPass)
{
	checkf(CurrMeshPass == EMeshPass::Num, TEXT("BeginMeshPass() was called without a matching EndMeshPass()"));
	check(MeshPass < EMeshPass::Num);
	CurrMeshPass = MeshPass;
}

void FCachedPassMeshDrawListContext::EndMeshPass()
{
	checkf(CurrMeshPass < EMeshPass::Num, TEXT("EndMeshPass() was called without matching BeginMeshPass()"));
	CurrMeshPass = EMeshPass::Num;
}

FCachedMeshDrawCommandInfo FCachedPassMeshDrawListContext::GetCommandInfoAndReset()
{
	FCachedMeshDrawCommandInfo Ret = CommandInfo;
	CommandInfo.CommandIndex = -1;
	CommandInfo.StateBucketId = -1;
	return Ret;
}

void FCachedPassMeshDrawListContext::FinalizeCommandCommon(
	const FMeshBatch& MeshBatch, 
	int32 BatchElementIndex,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	FMeshDrawCommandSortKey SortKey,
	EFVisibleMeshDrawCommandFlags Flags,
	const FGraphicsMinimalPipelineStateInitializer& PipelineState,
	const FMeshProcessorShaders* ShadersForDebugging,
	FMeshDrawCommand& MeshDrawCommand)
{
	FGraphicsMinimalPipelineStateId PipelineId = FGraphicsMinimalPipelineStateId::GetPersistentId(PipelineState);

	MeshDrawCommand.SetDrawParametersAndFinalize(MeshBatch, BatchElementIndex, PipelineId, ShadersForDebugging);

	CommandInfo = FCachedMeshDrawCommandInfo(CurrMeshPass);
	CommandInfo.SortKey = SortKey;
	CommandInfo.CullingPayload = CreateCullingPayload(MeshBatch, MeshBatch.Elements[BatchElementIndex]);
	CommandInfo.MeshFillMode = MeshFillMode;
	CommandInfo.MeshCullMode = MeshCullMode;
	CommandInfo.Flags = Flags;

#if MESH_DRAW_COMMAND_DEBUG_DATA
	if (bUseGPUScene)
	{
		MeshDrawCommand.ClearDebugPrimitiveSceneProxy(); //When using State Buckets multiple PrimitiveSceneProxies use the same MeshDrawCommand, so The PrimitiveSceneProxy pointer can't be stored.
	}
#endif

#if DO_GUARD_SLOW
	if (bUseGPUScene)
	{
		FMeshDrawCommand MeshDrawCommandDebug = FMeshDrawCommand(MeshDrawCommand);
		check(MeshDrawCommandDebug.ShaderBindings.GetDynamicInstancingHash() == MeshDrawCommand.ShaderBindings.GetDynamicInstancingHash());
		check(MeshDrawCommandDebug.GetDynamicInstancingHash() == MeshDrawCommand.GetDynamicInstancingHash());
	}
	if (Scene.GetShadingPath() == EShadingPath::Deferred)
	{
		ensureMsgf(MeshDrawCommand.VertexStreams.GetAllocatedSize() == 0, TEXT("Cached Mesh Draw command overflows VertexStreams. VertexStream inline size should be tweaked."));

		if (   CurrMeshPass == EMeshPass::BasePass
			|| CurrMeshPass == EMeshPass::DepthPass
			|| CurrMeshPass == EMeshPass::SecondStageDepthPass
			|| CurrMeshPass == EMeshPass::OnePassPointLightShadowDepth
			|| CurrMeshPass == EMeshPass::CSMShadowDepth
			|| CurrMeshPass == EMeshPass::VSMShadowDepth)
		{
			TArray<EShaderFrequency, TInlineAllocator<SF_NumFrequencies>> ShaderFrequencies;
			MeshDrawCommand.ShaderBindings.GetShaderFrequencies(ShaderFrequencies);

			int32 DataOffset = 0;
			for (int32 i = 0; i < ShaderFrequencies.Num(); i++)
			{
				FMeshDrawSingleShaderBindings SingleShaderBindings = MeshDrawCommand.ShaderBindings.GetSingleShaderBindings(ShaderFrequencies[i], DataOffset);
				if (SingleShaderBindings.GetParameterMapInfo().LooseParameterBuffers.Num() != 0)
				{
					bAnyLooseParameterBuffers = true;
				}
				ensureMsgf(SingleShaderBindings.GetParameterMapInfo().SRVs.Num() == 0, TEXT("Cached Mesh Draw command uses individual SRVs.  This will break dynamic instancing in performance critical pass.  Use Uniform Buffers instead."));
				ensureMsgf(SingleShaderBindings.GetParameterMapInfo().TextureSamplers.Num() == 0, TEXT("Cached Mesh Draw command uses individual Texture Samplers.  This will break dynamic instancing in performance critical pass.  Use Uniform Buffers instead."));
			}
		}
	}
#endif
}

void FCachedPassMeshDrawListContextImmediate::FinalizeCommand(
		const FMeshBatch& MeshBatch, 
		int32 BatchElementIndex,
		const FMeshDrawCommandPrimitiveIdInfo& IdInfo,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		FMeshDrawCommandSortKey SortKey,
		EFVisibleMeshDrawCommandFlags Flags,
		const FGraphicsMinimalPipelineStateInitializer& PipelineState,
		const FMeshProcessorShaders* ShadersForDebugging,
		FMeshDrawCommand& MeshDrawCommand)
{
	// disabling this by default as it incurs a high cost in perf captures due to sheer volume.  Recommendation is to re-enable locally if you need to profile this particular code.
	// QUICK_SCOPE_CYCLE_COUNTER(STAT_FinalizeCachedMeshDrawCommand);

	FinalizeCommandCommon(
		MeshBatch, 
		BatchElementIndex,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		Flags,
		PipelineState,
		ShadersForDebugging,
		MeshDrawCommand
	);

	if (bUseGPUScene)
	{
		Experimental::FHashElementId SetId;
		FStateBucketMap& BucketMap = Scene.CachedMeshDrawCommandStateBuckets[CurrMeshPass];
		auto hash = BucketMap.ComputeHash(MeshDrawCommand);
		{
			SetId = BucketMap.FindOrAddIdByHash(hash, MeshDrawCommand, FMeshDrawCommandCount());
			FMeshDrawCommandCount& DrawCount = BucketMap.GetByElementId(SetId).Value;
			DrawCount.Num++;
		}

		CommandInfo.StateBucketId = SetId.GetIndex();
	}
	else
	{
		// Only one FMeshDrawCommand supported per FStaticMesh in a pass
		// Allocate at lowest free index so that 'r.DoLazyStaticMeshUpdate' can shrink the TSparseArray more effectively
		FCachedPassMeshDrawList& CachedDrawLists = Scene.CachedDrawLists[CurrMeshPass];
		CommandInfo.CommandIndex = CachedDrawLists.MeshDrawCommands.EmplaceAtLowestFreeIndex(CachedDrawLists.LowestFreeIndexSearchStart, MeshDrawCommand);
	}	
}

void FCachedPassMeshDrawListContextDeferred::FinalizeCommand(
		const FMeshBatch& MeshBatch, 
		int32 BatchElementIndex,
		const FMeshDrawCommandPrimitiveIdInfo& IdInfo,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		FMeshDrawCommandSortKey SortKey,
		EFVisibleMeshDrawCommandFlags Flags,
		const FGraphicsMinimalPipelineStateInitializer& PipelineState,
		const FMeshProcessorShaders* ShadersForDebugging,
		FMeshDrawCommand& MeshDrawCommand)
{
	// disabling this by default as it incurs a high cost in perf captures due to sheer volume.  Recommendation is to re-enable locally if you need to profile this particular code.
	// QUICK_SCOPE_CYCLE_COUNTER(STAT_FinalizeCachedMeshDrawCommand);

	FinalizeCommandCommon(
		MeshBatch, 
		BatchElementIndex,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		Flags,
		PipelineState,
		ShadersForDebugging,
		MeshDrawCommand
	);

	const int32 Index = DeferredCommands.Add(MeshDrawCommand);

	if (bUseGPUScene)
	{
		// Cache the hash here to make the deferred finalize less expensive
		DeferredCommandHashes.Add(FStateBucketMap::ComputeHash(MeshDrawCommand));

		CommandInfo.StateBucketId = Index;
	}
	else
	{
		CommandInfo.CommandIndex = Index;
	}
}

void FCachedPassMeshDrawListContextDeferred::DeferredFinalizeMeshDrawCommands(const TArrayView<FPrimitiveSceneInfo*>& SceneInfos, int32 Start, int32 End)
{
	if (bUseGPUScene)
	{
		for (int32 SceneInfoIndex = Start; SceneInfoIndex < End; ++SceneInfoIndex)
		{
			FPrimitiveSceneInfo* SceneInfo = SceneInfos[SceneInfoIndex];
			for (auto& CmdInfo : SceneInfo->StaticMeshCommandInfos)
			{
				check(CmdInfo.MeshPass < EMeshPass::Num);
				FStateBucketMap& BucketMap = Scene.CachedMeshDrawCommandStateBuckets[CmdInfo.MeshPass];
				int32 DeferredIndex = CmdInfo.StateBucketId;

				check(DeferredIndex >= 0 && DeferredIndex < DeferredCommands.Num());
				check(CmdInfo.CommandIndex == -1);
				FMeshDrawCommand& Command = DeferredCommands[DeferredIndex];
				const Experimental::FHashType CommandHash = DeferredCommandHashes[DeferredIndex];

				Experimental::FHashElementId SetId = BucketMap.FindOrAddIdByHash(CommandHash, MoveTemp(Command), FMeshDrawCommandCount());
				FMeshDrawCommandCount& DrawCount = BucketMap.GetByElementId(SetId).Value;
				DrawCount.Num++;

				CmdInfo.StateBucketId = SetId.GetIndex();
			}
		}
	}
	else
	{
		for (int32 SceneInfoIndex = Start; SceneInfoIndex < End; ++SceneInfoIndex)
		{
			FPrimitiveSceneInfo* SceneInfo = SceneInfos[SceneInfoIndex];
			for (auto& CmdInfo : SceneInfo->StaticMeshCommandInfos)
			{				
				check(CmdInfo.MeshPass < EMeshPass::Num);
				FCachedPassMeshDrawList& CachedDrawLists = Scene.CachedDrawLists[CmdInfo.MeshPass];

				check(CmdInfo.CommandIndex >= 0 && CmdInfo.CommandIndex < DeferredCommands.Num());
				check(CmdInfo.StateBucketId == -1);
				FMeshDrawCommand& Command = DeferredCommands[CmdInfo.CommandIndex];
				
				CmdInfo.CommandIndex = CachedDrawLists.MeshDrawCommands.EmplaceAtLowestFreeIndex(CachedDrawLists.LowestFreeIndexSearchStart, MoveTemp(Command));
			}
		}
	}

	DeferredCommands.Reset();
	DeferredCommandHashes.Reset();
}

PassProcessorCreateFunction FPassProcessorManager::JumpTable[(int32)EShadingPath::Num][EMeshPass::Num] = {};
DeprecatedPassProcessorCreateFunction FPassProcessorManager::DeprecatedJumpTable[(int32)EShadingPath::Num][EMeshPass::Num] = {};
EMeshPassFlags FPassProcessorManager::Flags[(int32)EShadingPath::Num][EMeshPass::Num] = {};
int32 FPassProcessorManager::PSOCollectorIndex[(int32)EShadingPath::Num][EMeshPass::Num] = {};

static_assert(EMeshPass::Num <= FPSOCollectorCreateManager::MaxPSOCollectorCount);

void FPassProcessorManager::SetPassFlags(EShadingPath ShadingPath, EMeshPass::Type PassType, EMeshPassFlags NewFlags)
{
	check(IsInGameThread());
	FGlobalComponentRecreateRenderStateContext Context;
	if (JumpTable[(uint32)ShadingPath][PassType])
	{
		Flags[(uint32)ShadingPath][PassType] = NewFlags;
	}
}

void AddRenderTargetInfo(
	EPixelFormat PixelFormat,
	ETextureCreateFlags CreateFlags,
	FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo)
{
	RenderTargetsInfo.RenderTargetFormats[RenderTargetsInfo.RenderTargetsEnabled] = PixelFormat;
	RenderTargetsInfo.RenderTargetFlags[RenderTargetsInfo.RenderTargetsEnabled] = CreateFlags;
	RenderTargetsInfo.RenderTargetsEnabled++;
}

void SetupDepthStencilInfo(
	EPixelFormat DepthStencilFormat,
	ETextureCreateFlags DepthStencilCreateFlags,
	ERenderTargetLoadAction DepthTargetLoadAction,
	ERenderTargetLoadAction StencilTargetLoadAction,
	FExclusiveDepthStencil DepthStencilAccess,
	FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo)
{
	// Setup depth stencil state
	RenderTargetsInfo.DepthStencilTargetFormat = DepthStencilFormat;
	RenderTargetsInfo.DepthStencilTargetFlag = DepthStencilCreateFlags;

	RenderTargetsInfo.DepthTargetLoadAction = DepthTargetLoadAction;
	RenderTargetsInfo.StencilTargetLoadAction = StencilTargetLoadAction;
	RenderTargetsInfo.DepthStencilAccess = DepthStencilAccess;

	const ERenderTargetStoreAction StoreAction = EnumHasAnyFlags(RenderTargetsInfo.DepthStencilTargetFlag, TexCreate_Memoryless) ? ERenderTargetStoreAction::ENoAction : ERenderTargetStoreAction::EStore;
	RenderTargetsInfo.DepthTargetStoreAction = RenderTargetsInfo.DepthStencilAccess.IsUsingDepth() ? StoreAction : ERenderTargetStoreAction::ENoAction;
	RenderTargetsInfo.StencilTargetStoreAction = RenderTargetsInfo.DepthStencilAccess.IsUsingStencil() ? StoreAction : ERenderTargetStoreAction::ENoAction;
}

void SetupGBufferRenderTargetInfo(const FSceneTexturesConfig& SceneTexturesConfig, FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo, bool bSetupDepthStencil, EGBufferLayout Layout)
{
	SceneTexturesConfig.GetGBufferRenderTargetsInfo(RenderTargetsInfo, Layout);
	if (bSetupDepthStencil)
	{
		SetupDepthStencilInfo(PF_DepthStencil, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite, RenderTargetsInfo);
	}
}


#if PSO_PRECACHING_VALIDATE

FGraphicsMinimalPipelineStateInitializer PSOCollectorStats::GetShadersOnlyInitializer(const FGraphicsMinimalPipelineStateInitializer& Initializer)
{
	FGraphicsMinimalPipelineStateInitializer ShadersOnlyInitializer(Initializer);
	ShadersOnlyInitializer.BoundShaderState.VertexDeclarationRHI = nullptr;
	ShadersOnlyInitializer.ComputeStatePrecachePSOHash();

	return ShadersOnlyInitializer;
}

FGraphicsMinimalPipelineStateInitializer PSOCollectorStats::PatchMinimalPipelineStateToCheck(const FGraphicsMinimalPipelineStateInitializer& Initializer)
{
	FGraphicsMinimalPipelineStateInitializer PatchedInitializer = Initializer;
	//PatchedInitializer.DepthStencilState = nullptr;
	//PatchedInitializer.RasterizerState = nullptr;
	//PatchedInitializer.BlendState = nullptr;
	//PatchedInitializer.PrimitiveType = PT_TriangleList;
	//PatchedInitializer.BoundShaderState.VertexDeclarationRHI = nullptr;
	//PatchedInitializer.UniqueEntry = false;

	// Recompute the hash when disabling certain states for checks.
	// PatchedInitializer.StatePrecachePSOHash = 0;
	PatchedInitializer.ComputeStatePrecachePSOHash();

	return PatchedInitializer;
}

uint64 PSOCollectorStats::GetPSOPrecacheHash(const FGraphicsMinimalPipelineStateInitializer& GraphicsPSOInitializer)
{
	return GraphicsPSOInitializer.StatePrecachePSOHash;
}

void PSOCollectorStats::CheckShaderOnlyStateInCache(
	const FGraphicsMinimalPipelineStateInitializer& Initializer,
	const FMaterial& Material,
	const FVertexFactoryType* VFType,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	int32 PSOCollectorIndex)
{
	FGraphicsMinimalPipelineStateInitializer ShadersOnlyInitializer = GetShadersOnlyInitializer(Initializer);

	EPSOPrecacheResult Result = EPSOPrecacheResult::Unknown;
	bool bStatsUpdated = PSOCollectorStats::GetShadersOnlyPSOPrecacheStatsCollector().CheckStateInCacheByHash(EPSOPrecacheType::MeshPass, ShadersOnlyInitializer.StatePrecachePSOHash, PSOCollectorIndex, VFType, Result);
	if (IsFullPrecachingValidationEnabled() && bStatsUpdated && Result != EPSOPrecacheResult::Unknown && Result != EPSOPrecacheResult::Complete)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInitializer = ShadersOnlyInitializer.AsGraphicsPipelineStateInitializer();
		LogPSOMissInfo(GraphicsPSOInitializer, EPSOPrecacheMissType::ShadersOnly, Result, &Material, VFType, PrimitiveSceneProxy, EPSOPrecacheType::MeshPass, PSOCollectorIndex, ShadersOnlyInitializer.StatePrecachePSOHash);
	}
}

void PSOCollectorStats::CheckMinimalPipelineStateInCache(
	const FGraphicsMinimalPipelineStateInitializer& Initializer, 
	const FMaterial& Material, 
	const FVertexFactoryType* VFType, 
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	int32 PSOCollectorIndex)
{
	FGraphicsMinimalPipelineStateInitializer PatchedMinimalInitializer = PSOCollectorStats::PatchMinimalPipelineStateToCheck(Initializer);

	EPSOPrecacheResult Result = EPSOPrecacheResult::Unknown;
	bool bStatsUpdated = PSOCollectorStats::GetMinimalPSOPrecacheStatsCollector().CheckStateInCacheByHash(EPSOPrecacheType::MeshPass, PatchedMinimalInitializer.StatePrecachePSOHash, PSOCollectorIndex, VFType, Result);
	if (IsFullPrecachingValidationEnabled() && bStatsUpdated && Result != EPSOPrecacheResult::Unknown && Result != EPSOPrecacheResult::Complete)
	{
		FGraphicsMinimalPipelineStateInitializer ShadersOnlyInitializer = GetShadersOnlyInitializer(Initializer);
		bool bShaderOnlyPrecached = PSOCollectorStats::GetShadersOnlyPSOPrecacheStatsCollector().IsPrecached(ShadersOnlyInitializer.StatePrecachePSOHash);
		if (bShaderOnlyPrecached)
		{
			check(Result == EPSOPrecacheResult::Missed);
			FGraphicsPipelineStateInitializer GraphicsPSOInitializer = PatchedMinimalInitializer.AsGraphicsPipelineStateInitializer();
			LogPSOMissInfo(GraphicsPSOInitializer, EPSOPrecacheMissType::MinimalPSOState, Result, &Material, VFType, PrimitiveSceneProxy, EPSOPrecacheType::MeshPass, PSOCollectorIndex, ShadersOnlyInitializer.StatePrecachePSOHash);
		}
	}
}

#endif // PSO_PRECACHING_VALIDATE
