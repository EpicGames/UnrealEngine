// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MobileBasePassRendering.cpp: Base pass rendering implementation.
=============================================================================*/

#include "MobileBasePassRendering.h"
#include "TranslucentRendering.h"
#include "DynamicPrimitiveDrawing.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneProxies/ReflectionCaptureProxy.h"
#include "ShaderPlatformQualitySettings.h"
#include "MaterialShaderQualitySettings.h"
#include "PrimitiveSceneInfo.h"
#include "MeshPassProcessor.inl"
#include "Engine/TextureCube.h"
#include "ShaderPlatformCachedIniValue.h"
#include "StereoRenderUtils.h"
#include "VariableRateShadingImageManager.h"
#include "SceneCore.h"

int32 GMobileForwardLocalLightsSinglePermutation = 0;
FAutoConsoleVariableRef CVarMobileForwardLocalLightsSinglePermutation(
	TEXT("r.Mobile.Forward.LocalLightsSinglePermutation"),
	GMobileForwardLocalLightsSinglePermutation,
	TEXT("Whether to use the same permutation regardless of local lights state. This may improve RT time at expense of some GPU time"),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

bool MobileLocalLightsUseSinglePermutation(EShaderPlatform ShaderPlatform)
{ 
	return GMobileForwardLocalLightsSinglePermutation != 0 || MobileForwardEnableParticleLights(ShaderPlatform);
}

EMobileLocalLightSetting GetMobileForwardLocalLightSetting(EShaderPlatform ShaderPlatform)
{
	const int32 MobileForwardLocalLightsIniValue = FReadOnlyCVARCache::MobileForwardLocalLights(ShaderPlatform);
	return MobileForwardLocalLightsIniValue > 0 ? EMobileLocalLightSetting::LOCAL_LIGHTS_ENABLED: EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED;
}

uint8 GetMobileShadingModelStencilValue(FMaterialShadingModelField ShadingModel)
{
	if (ShadingModel.HasOnlyShadingModel(MSM_DefaultLit))
	{
		return GET_STENCIL_BIT_MASK(MOBILE_DEFAULTLIT, 1);
	}
	else if (ShadingModel.HasOnlyShadingModel(MSM_Unlit))
	{
		return 0u;
	}

	// mark everyhing as MSM_DefaultLit if GBuffer CustomData is not supported
	return MobileUsesGBufferCustomData(GMaxRHIShaderPlatform) ? GET_STENCIL_BIT_MASK(MOBILE_SHADINGMODELS, 1) : GET_STENCIL_BIT_MASK(MOBILE_DEFAULTLIT, 1);
}

void SetMobileBasePassDepthState(
	FMeshPassProcessorRenderState& DrawRenderState, 
	const FPrimitiveSceneProxy* PrimitiveSceneProxy, 
	const FMaterial& Material, 
	FMaterialShadingModelField ShadingModels, 
	bool bUsesDeferredShading,
	TOptional<uint32> DitheringFlag,
	bool bKeepStencilDither)
{
	uint8 StencilValue = 0u;
	
	if (DitheringFlag.IsSet())
	{
		StencilValue = DitheringFlag.GetValue();

		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
			true, CF_DepthNearOrEqual,
			true, CF_Equal, SO_Keep, SO_Keep, SO_Replace,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			STENCIL_SANDBOX_MASK, 0xFF
		>::GetRHI());
	}
	else if (bKeepStencilDither && IsStencilForLODDitherEnabled(Material.GetShaderPlatform()))
	{
		// If the object is not transitioning and StencilLODDither is enabled, we need to preserve the dither pattern, 
		// as well as the other stencil bits (RECEIVE_DECAL, lighting channels, etc.).
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
			true, CF_DepthNearOrEqual,
			true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			0x00, (0xff & ~STENCIL_SANDBOX_MASK) >::GetRHI());
	}
	else
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
			true, CF_DepthNearOrEqual,
			true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			// don't use masking as it has significant performance hit on Mali GPUs (T860MP2)
			0x00, 0xff >::GetRHI());
	}

	uint8 ReceiveDecals = (PrimitiveSceneProxy && PrimitiveSceneProxy->ReceivesDecals() ? 0x01 : 0x00);
	StencilValue |= GET_STENCIL_BIT_MASK(RECEIVE_DECAL, ReceiveDecals);

	if (bUsesDeferredShading)
	{
		StencilValue |= GetMobileShadingModelStencilValue(ShadingModels);
		StencilValue |= STENCIL_LIGHTING_CHANNELS_MASK(PrimitiveSceneProxy ? PrimitiveSceneProxy->GetLightingChannelStencilValue() : 0x00);
	}
	else
	{
		// TODO: ContactShadows do not work with deferred shading single pass
		uint8 CastContactShadows = (PrimitiveSceneProxy && PrimitiveSceneProxy->CastsContactShadow() ? 0x01 : 0x00);
		StencilValue |= GET_STENCIL_BIT_MASK(MOBILE_CAST_CONTACT_SHADOW, CastContactShadows);
	}

	DrawRenderState.SetStencilRef(StencilValue); 
}

template <ELightMapPolicyType Policy, EMobileLocalLightSetting LocalLightSetting>
bool GetUniformMobileBasePassShaders(
	const FMaterial& Material, 
	const FVertexFactoryType* VertexFactoryType, 
	EMobileTranslucentColorTransmittanceMode ColoredTransmittanceFallback,
	TShaderRef<TMobileBasePassVSPolicyParamType<FUniformLightMapPolicy>>& VertexShader,
	TShaderRef<TMobileBasePassPSPolicyParamType<FUniformLightMapPolicy>>& PixelShader
	)
{
	using FVertexShaderType = TMobileBasePassVSPolicyParamType<FUniformLightMapPolicy>;
	using FPixelShaderType = TMobileBasePassPSPolicyParamType<FUniformLightMapPolicy>;

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<TMobileBasePassVS<TUniformLightMapPolicy<Policy>>>();

	switch (ColoredTransmittanceFallback)
	{
	default:
	case EMobileTranslucentColorTransmittanceMode::DEFAULT:
		ShaderTypes.AddShaderType<TMobileBasePassPS<TUniformLightMapPolicy<Policy>, LocalLightSetting, EMobileTranslucentColorTransmittanceMode::DEFAULT>>();
		break;
	case EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING:
		ShaderTypes.AddShaderType<TMobileBasePassPS<TUniformLightMapPolicy<Policy>, LocalLightSetting, EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING>>();
		break;
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

template <EMobileLocalLightSetting LocalLightSetting>
bool GetMobileBasePassShaders(
	const FUniformLightMapPolicy& RESTRICT LightMapPolicy,
	const FMaterial& Material, 
	const FVertexFactoryType* VertexFactoryType, 
	EMobileTranslucentColorTransmittanceMode ColoredTransmittanceFallback,
	TShaderRef<TMobileBasePassVSPolicyParamType<FUniformLightMapPolicy>>& VertexShader,
	TShaderRef<TMobileBasePassPSPolicyParamType<FUniformLightMapPolicy>>& PixelShader
	)
{
	switch (LightMapPolicy.GetIndirectPolicy())
	{
	case LMP_NO_LIGHTMAP:
		return GetUniformMobileBasePassShaders<LMP_NO_LIGHTMAP, LocalLightSetting>(Material, VertexFactoryType, ColoredTransmittanceFallback, VertexShader, PixelShader);
	case LMP_LQ_LIGHTMAP:
		return GetUniformMobileBasePassShaders<LMP_LQ_LIGHTMAP, LocalLightSetting>(Material, VertexFactoryType, ColoredTransmittanceFallback, VertexShader, PixelShader);
	case LMP_MOBILE_DISTANCE_FIELD_SHADOWS_AND_LQ_LIGHTMAP:
		return GetUniformMobileBasePassShaders<LMP_MOBILE_DISTANCE_FIELD_SHADOWS_AND_LQ_LIGHTMAP, LocalLightSetting>(Material, VertexFactoryType, ColoredTransmittanceFallback, VertexShader, PixelShader);
	case LMP_MOBILE_DISTANCE_FIELD_SHADOWS_LIGHTMAP_AND_CSM:
		return GetUniformMobileBasePassShaders<LMP_MOBILE_DISTANCE_FIELD_SHADOWS_LIGHTMAP_AND_CSM, LocalLightSetting>(Material, VertexFactoryType, ColoredTransmittanceFallback, VertexShader, PixelShader);
	case LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_LIGHTMAP:
		return GetUniformMobileBasePassShaders<LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_LIGHTMAP, LocalLightSetting>(Material, VertexFactoryType, ColoredTransmittanceFallback, VertexShader, PixelShader);
	case LMP_MOBILE_DIRECTIONAL_LIGHT_AND_SH_INDIRECT:
		return GetUniformMobileBasePassShaders<LMP_MOBILE_DIRECTIONAL_LIGHT_AND_SH_INDIRECT, LocalLightSetting>(Material, VertexFactoryType, ColoredTransmittanceFallback, VertexShader, PixelShader);
	case LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_SH_INDIRECT:
		return GetUniformMobileBasePassShaders<LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_SH_INDIRECT, LocalLightSetting>(Material, VertexFactoryType, ColoredTransmittanceFallback, VertexShader, PixelShader);
	case LMP_MOBILE_DIRECTIONAL_LIGHT_CSM:
		return GetUniformMobileBasePassShaders<LMP_MOBILE_DIRECTIONAL_LIGHT_CSM, LocalLightSetting>(Material, VertexFactoryType, ColoredTransmittanceFallback, VertexShader, PixelShader);
	default:										
		check(false);
		return true;
	}
}

template <typename LightMapPolicyType>
bool MobileBasePass::GetShaders(
	const LightMapPolicyType& RESTRICT LightMapPolicy,
	EMobileLocalLightSetting LocalLightSetting,
	const FMaterial& MaterialResource,
	const FVertexFactoryType* VertexFactoryType,
	TShaderRef<TMobileBasePassVSPolicyParamType<LightMapPolicyType>>& VertexShader,
	TShaderRef<TMobileBasePassPSPolicyParamType<LightMapPolicyType>>& PixelShader)
{
	EMobileTranslucentColorTransmittanceMode ColoredTransmittanceFallback = EMobileTranslucentColorTransmittanceMode::DEFAULT;
	if (MaterialRequiresColorTransmittanceBlending(MaterialResource))
	{
		const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(MaterialResource.GetFeatureLevel());
		ColoredTransmittanceFallback = MobileActiveTranslucentColorTransmittanceMode(ShaderPlatform, false);
	}

	switch (LocalLightSetting)
	{
		case EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED:
		{
			return GetMobileBasePassShaders<EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED>(
				LightMapPolicy,
				MaterialResource,
				VertexFactoryType,
				ColoredTransmittanceFallback,
				VertexShader,
				PixelShader
				);
		}
		case EMobileLocalLightSetting::LOCAL_LIGHTS_ENABLED:
		{
			return GetMobileBasePassShaders<EMobileLocalLightSetting::LOCAL_LIGHTS_ENABLED>(
				LightMapPolicy,
				MaterialResource,
				VertexFactoryType,
				ColoredTransmittanceFallback,
				VertexShader,
				PixelShader
				);
		}

		default:
			check(false);
			return false;
	}
}

// Force instantination for FMobileSingleLayerWaterPolicy as it's used in a different translation unit
template bool MobileBasePass::GetShaders<FMobileSingleLayerWaterPolicy>(
	const FMobileSingleLayerWaterPolicy& RESTRICT LightMapPolicy,
	EMobileLocalLightSetting LocalLightSetting,
	const FMaterial& MaterialResource,
	const FVertexFactoryType* VertexFactoryType,
	TShaderRef<TMobileBasePassVSPolicyParamType<FMobileSingleLayerWaterPolicy>>& VertexShader,
	TShaderRef<TMobileBasePassPSPolicyParamType<FMobileSingleLayerWaterPolicy>>& PixelShader);

// Force instantination for FUniformLightMapPolicy as it's used in a different translation unit
template bool MobileBasePass::GetShaders<FUniformLightMapPolicy>(
	const FUniformLightMapPolicy& RESTRICT LightMapPolicy,
	EMobileLocalLightSetting LocalLightSetting,
	const FMaterial& MaterialResource,
	const FVertexFactoryType* VertexFactoryType,
	TShaderRef<TMobileBasePassVSPolicyParamType<FUniformLightMapPolicy>>& VertexShader,
	TShaderRef<TMobileBasePassPSPolicyParamType<FUniformLightMapPolicy>>& PixelShader);

static bool UseSkyReflectionCapture(const FScene* RenderScene)
{
	return RenderScene
		&& RenderScene->SkyLight
		&& ((RenderScene->SkyLight->ProcessedTexture && 
			RenderScene->SkyLight->ProcessedTexture->TextureRHI) 
			|| RenderScene->CanSampleSkyLightRealTimeCaptureData());
}

const FLightSceneInfo* MobileBasePass::GetDirectionalLightInfo(const FScene* Scene, const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	const FLightSceneInfo* MobileDirectionalLight = nullptr;
	if (PrimitiveSceneProxy && Scene)
	{
		const int32 LightChannel = GetFirstLightingChannelFromMask(PrimitiveSceneProxy->GetLightingChannelMask());
		MobileDirectionalLight = LightChannel >= 0 ? Scene->MobileDirectionalLights[LightChannel] : nullptr;
	}
	return MobileDirectionalLight;
}

bool MobileBasePass::StaticCanReceiveCSM(const FLightSceneInfo* LightSceneInfo, const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	// For movable directional lights, when CSM culling is disabled the default behavior is to receive CSM.
	if (LightSceneInfo && LightSceneInfo->Proxy->IsMovable() && !FReadOnlyCVARCache::MobileEnableMovableLightCSMShaderCulling())
	{		
		return true;
	}

	// If culling is enabled then CSM receiving is determined during InitDynamicShadows.
	// If culling is disabled then stationary directional lights default to no CSM. 
	return false; 
}

ELightMapPolicyType MobileBasePass::SelectMeshLightmapPolicy(
	const FScene* Scene, 
	const FMeshBatch& Mesh, 
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,	
	bool bPrimReceivesCSM,
	bool bUsesDeferredShading,
	bool bIsLitMaterial,
	bool bIsTranslucent) // not used
{
	// Unlit uses NoLightmapPolicy with 0 point lights
	ELightMapPolicyType SelectedLightmapPolicy = LMP_NO_LIGHTMAP;
	
	if (bIsLitMaterial)
	{
		constexpr ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::ES3_1;
		
		if (!IsStaticLightingAllowed())
		{
			// no precomputed lighting
			if (bUsesDeferredShading)
			{
				SelectedLightmapPolicy = LMP_NO_LIGHTMAP;
			}
			else
			{
				if (!bPrimReceivesCSM || MobileUseCSMShaderBranch())
				{
					SelectedLightmapPolicy = LMP_NO_LIGHTMAP;
				}
				else
				{
					SelectedLightmapPolicy = LMP_MOBILE_DIRECTIONAL_LIGHT_CSM;
				}
			}
		}
		else
		{
			// Check for a cached light-map.
			const FLightMapInteraction LightMapInteraction = (Mesh.LCI != nullptr)
				? Mesh.LCI->GetLightMapInteraction(FeatureLevel)
				: FLightMapInteraction();

			const FLightSceneInfo* MobileDirectionalLight = MobileBasePass::GetDirectionalLightInfo(Scene, PrimitiveSceneProxy);
		
			// Primitive can receive both pre-computed and CSM shadows
			const bool bPrimReceivesStaticAndCSM = 
				MobileDirectionalLight 
				&& PrimitiveSceneProxy
				&& bPrimReceivesCSM
				&& FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers()
				&& MobileDirectionalLight->ShouldRenderViewIndependentWholeSceneShadows()
				&& PrimitiveSceneProxy->ShouldReceiveMobileCSMShadows(); // receive CSM shadow enabled on meshcomponent

			const bool bPrimitiveUsesILC = 
				PrimitiveSceneProxy
				&& (PrimitiveSceneProxy->IsMovable() || PrimitiveSceneProxy->NeedsUnbuiltPreviewLighting() || PrimitiveSceneProxy->GetLightmapType() == ELightmapType::ForceVolumetric)
				&& PrimitiveSceneProxy->WillEverBeLit()
				&& PrimitiveSceneProxy->GetIndirectLightingCacheQuality() != ILCQ_Off;

			const bool bHasValidVLM = Scene && Scene->VolumetricLightmapSceneData.HasData();
			const bool bHasValidILC = Scene && Scene->PrecomputedLightVolumes.Num() > 0	&& IsIndirectLightingCacheAllowed(FeatureLevel);

			if (LightMapInteraction.GetType() == LMIT_Texture && FReadOnlyCVARCache::EnableLowQualityLightmaps())
			{
				const FShadowMapInteraction ShadowMapInteraction = Mesh.LCI != nullptr
					? Mesh.LCI->GetShadowMapInteraction(FeatureLevel)
					: FShadowMapInteraction();

				if (ShadowMapInteraction.GetType() == SMIT_Texture && FReadOnlyCVARCache::MobileAllowDistanceFieldShadows())
				{
					SelectedLightmapPolicy = (bPrimReceivesStaticAndCSM && !bUsesDeferredShading) ?  
						LMP_MOBILE_DISTANCE_FIELD_SHADOWS_LIGHTMAP_AND_CSM : 
						LMP_MOBILE_DISTANCE_FIELD_SHADOWS_AND_LQ_LIGHTMAP;
				}
				else
				{
					SelectedLightmapPolicy = (bPrimReceivesStaticAndCSM && !bUsesDeferredShading) ? 
						LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_LIGHTMAP : 
						LMP_LQ_LIGHTMAP;
				}
			}
			else if ((bHasValidVLM || bHasValidILC) && bPrimitiveUsesILC)
			{
				if (bPrimReceivesStaticAndCSM && !bUsesDeferredShading)
				{
					SelectedLightmapPolicy = LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_SH_INDIRECT;
				}
				else
				{
					SelectedLightmapPolicy = LMP_MOBILE_DIRECTIONAL_LIGHT_AND_SH_INDIRECT;
				}
			}
			else if (bPrimReceivesStaticAndCSM && !bUsesDeferredShading)
			{
				SelectedLightmapPolicy = LMP_MOBILE_DIRECTIONAL_LIGHT_CSM;
			}
		}
	}
		
	return SelectedLightmapPolicy;
}

FMobileLightMapPolicyTypeList MobileBasePass::GetUniformLightMapPolicyTypeForPSOCollection(bool bLitMaterial, bool bTranslucent/*not used*/, bool bUsesDeferredShading, bool bCanReceiveCSM, bool bMovable)
{
	FMobileLightMapPolicyTypeList Result;
	
	if (bLitMaterial)
	{
		if (!IsStaticLightingAllowed())
		{
			Result.Add(LMP_NO_LIGHTMAP);
						
			if (!bUsesDeferredShading && !MobileUseCSMShaderBranch())
			{
				// permutation that can receive CSM
				Result.Add(LMP_MOBILE_DIRECTIONAL_LIGHT_CSM);
			}
		}
		else
		{
			if (!bMovable && FReadOnlyCVARCache::EnableLowQualityLightmaps())
			{
				if (FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers() && !bUsesDeferredShading && bCanReceiveCSM)
				{
					if (FReadOnlyCVARCache::MobileAllowDistanceFieldShadows())
					{
						Result.Add(LMP_MOBILE_DISTANCE_FIELD_SHADOWS_LIGHTMAP_AND_CSM);
					}

					Result.Add(LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_LIGHTMAP);
				}

				if (FReadOnlyCVARCache::MobileAllowDistanceFieldShadows() && !bCanReceiveCSM)
				{
					Result.Add(LMP_MOBILE_DISTANCE_FIELD_SHADOWS_AND_LQ_LIGHTMAP);
				}
				
				Result.Add(LMP_LQ_LIGHTMAP);
			}
						
			// ILC/LVM
			if (bMovable)
			{
				if (!bUsesDeferredShading && FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers() && bCanReceiveCSM)
				{
					Result.Add(LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_SH_INDIRECT);
				}
				else
				{
					Result.Add(LMP_MOBILE_DIRECTIONAL_LIGHT_AND_SH_INDIRECT);
				}

				// in case there is no valid ILC/VLM
				if (bCanReceiveCSM) 
				{
					Result.Add(LMP_MOBILE_DIRECTIONAL_LIGHT_CSM);
				}
				else
				{
					Result.Add(LMP_NO_LIGHTMAP); 
				}
			}
		}
	}
	else
	{
		// Unlit materials
		Result.Add(LMP_NO_LIGHTMAP);
	}

	return Result;
}

void MobileBasePass::SetOpaqueRenderState(FMeshPassProcessorRenderState& DrawRenderState, const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FMaterial& Material, FMaterialShadingModelField ShadingModels, bool bCanUseDepthStencil, bool bUsesDeferredShading)
{
	if (bCanUseDepthStencil)
	{
		SetMobileBasePassDepthState(DrawRenderState, PrimitiveSceneProxy, Material, ShadingModels, bUsesDeferredShading, TOptional<uint32>(), false);
	}
	else
	{
		// default depth state should be already set
	}
	
	const bool bIsMasked = IsMaskedBlendMode(Material);
	if (bIsMasked && Material.IsUsingAlphaToCoverage())
	{
		DrawRenderState.SetBlendState(TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero, 
														CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
														CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero, 
														CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero, 
														CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero, 
														CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
														CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero, 
														CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero, 
														true>::GetRHI());
	}
}

static FRHIBlendState* GetBlendStateForColorTransmittanceBlending(const EShaderPlatform ShaderPlatform)
{
	switch (MobileActiveTranslucentColorTransmittanceMode(ShaderPlatform, true))
	{
	case EMobileTranslucentColorTransmittanceMode::DUAL_SRC_BLENDING:
		// Blend by putting add in target 0 and multiply by background in target 1.
		return TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Source1Color, BO_Add, BF_One, BF_Source1Alpha,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
	case EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING:
		// If a material was requesting dual source blending, the shader will use static platform knowledge to convert colored transmittance to a grey scale transmittance.
		return TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
	default:
	case EMobileTranslucentColorTransmittanceMode::PROGRAMMABLE_BLENDING:
		// Blending is done in shader
		return TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
	}
}

void MobileBasePass::SetTranslucentRenderState(FMeshPassProcessorRenderState& DrawRenderState, const FMaterial& Material, FMaterialShadingModelField ShadingModels)
{
	constexpr ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::ES3_1;
	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);

	if (Material.IsSubstrateMaterial())
	{
		if (Material.IsDualBlendingEnabled() || Material.GetBlendMode() == BLEND_TranslucentColoredTransmittance)
		{
			DrawRenderState.SetBlendState(GetBlendStateForColorTransmittanceBlending(ShaderPlatform));
		}
		else if (Material.GetBlendMode() == BLEND_ColoredTransmittanceOnly)
		{
			// Modulate with the existing scene color, preserve destination alpha.
			DrawRenderState.SetBlendState(TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_Zero>::GetRHI());
		}
		else if (Material.GetBlendMode() == BLEND_AlphaHoldout)
		{
			// Blend by holding out the matte shape of the source alpha
			DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI());
		}
		else
		{
			// We always use premultiplied alpha for translucent rendering.
			DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI());
		}
	}
	else if (ShadingModels.HasShadingModel(MSM_ThinTranslucent))
	{
		DrawRenderState.SetBlendState(GetBlendStateForColorTransmittanceBlending(ShaderPlatform));
	}
	else
	{
		switch (Material.GetBlendMode())
		{
		case BLEND_Translucent:
		case BLEND_TranslucentColoredTransmittance:	// When Substrate is disabled, this falls back to simple Translucency.
			if (Material.ShouldWriteOnlyAlpha())
			{
				DrawRenderState.SetBlendState(TStaticBlendState<CW_ALPHA, BO_Add, BF_Zero, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI());
			}
			else
			{
				DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI());
			}
			break;
		case BLEND_Additive:
			// Add to the existing scene color
			DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_InverseSourceAlpha,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI());
			break;
		case BLEND_Modulate:
			// Modulate with the existing scene color, preserve destination alpha.
			DrawRenderState.SetBlendState(TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_Zero, BO_Add, BF_Zero, BF_One,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI());
			break;
		case BLEND_AlphaComposite:
			// Blend with existing scene color. New color is already pre-multiplied by alpha.
			DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI());
			break;
		case BLEND_AlphaHoldout:
			// Blend by holding out the matte shape of the source alpha
			DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
															CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI());
			break;
		default:
			if (ShadingModels.HasShadingModel(MSM_SingleLayerWater))
			{
				// Single layer water is an opaque marerial rendered as translucent on Mobile. We force pre-multiplied alpha to achieve water depth based transmittance.
				DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
																CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI());
			}
			else
			{
				check(0);
			}
		};
	}

	if (Material.ShouldDisableDepthTest())
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
	}
}

static FMeshDrawCommandSortKey GetBasePassStaticSortKey(EDepthDrawingMode EarlyZPassMode, const bool bIsMasked, bool bBackground, const FMeshMaterialShader* VertexShader, const FMeshMaterialShader* PixelShader)
{
	FMeshDrawCommandSortKey SortKey;
	SortKey.BasePass.Background = (bBackground ? 1 : 0); // background flag in second bit
	SortKey.BasePass.VertexShaderHash = (VertexShader ? VertexShader->GetSortKey() : 0) & 0xFFFF;
	SortKey.BasePass.PixelShaderHash = PixelShader ? PixelShader->GetSortKey() : 0;
	if (EarlyZPassMode != DDM_None)
	{
		SortKey.BasePass.Masked = bIsMasked ? 0 : 1;
	}
	else
	{
		SortKey.BasePass.Masked = bIsMasked ? 1 : 0;
	}
	return SortKey;
}

template <typename LightMapPolicyType>
void TMobileBasePassPSPolicyParamType<LightMapPolicyType>::GetShaderBindings(
	const FScene* Scene,
	const FStaticFeatureLevel FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	const TMobileBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings) const
{
	FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

	LightMapPolicyType::GetPixelShaderBindings(
		PrimitiveSceneProxy,
		ShaderElementData.LightMapPolicyElementData,
		this,
		ShaderBindings);

	if (Scene)
	{
		if (ReflectionParameter.IsBound())
		{
			FRHIUniformBuffer* ReflectionUB = GDefaultMobileReflectionCaptureUniformBuffer.GetUniformBufferRHI();
			FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy ? PrimitiveSceneProxy->GetPrimitiveSceneInfo() : nullptr;
			if (PrimitiveSceneInfo && PrimitiveSceneInfo->CachedReflectionCaptureProxy)
			{
				ReflectionUB = PrimitiveSceneInfo->CachedReflectionCaptureProxy->MobileUniformBuffer;
			}
			// If no reflection captures are available then attempt to use sky light's texture.
			else if (UseSkyReflectionCapture(Scene))
			{
				ReflectionUB = Scene->UniformBuffers.MobileSkyReflectionUniformBuffer;
			}
			ShaderBindings.Add(ReflectionParameter, ReflectionUB);
		}
	}
	else
	{
		ensure(!ReflectionParameter.IsBound());
	}

	// Set directional light UB
	if (MobileDirectionLightBufferParam.IsBound() && Scene)
	{
		int32 UniformBufferIndex = PrimitiveSceneProxy ? GetFirstLightingChannelFromMask(PrimitiveSceneProxy->GetLightingChannelMask()) + 1 : 0;
		ShaderBindings.Add(MobileDirectionLightBufferParam, Scene->UniformBuffers.MobileDirectionalLightUniformBuffers[UniformBufferIndex]);
	}

	if (UseCSMParameter.IsBound())
	{
		ShaderBindings.Add(UseCSMParameter, ShaderElementData.bCanReceiveCSM ? 1 : 0);
	}
}

// Force instantination for FMobileSingleLayerWaterPolicy as it's used in a different translation unit
template void TMobileBasePassPSPolicyParamType<FMobileSingleLayerWaterPolicy>::GetShaderBindings(
	const FScene* Scene,
	const FStaticFeatureLevel FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	const TMobileBasePassShaderElementData<FMobileSingleLayerWaterPolicy>& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings) const;

// Force instantination for FUniformLightMapPolicy as it's used in a different translation unit
template void TMobileBasePassPSPolicyParamType<FUniformLightMapPolicy>::GetShaderBindings(
	const FScene* Scene,
	const FStaticFeatureLevel FeatureLevel,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material,
	const TMobileBasePassShaderElementData<FUniformLightMapPolicy>& ShaderElementData,
	FMeshDrawSingleShaderBindings& ShaderBindings) const;


FMobileBasePassMeshProcessor::FMobileBasePassMeshProcessor(
	EMeshPass::Type InMeshPassType,
	const FScene* Scene,
	const FSceneView* InViewIfDynamicMeshCommand,
	const FMeshPassProcessorRenderState& InDrawRenderState,
	FMeshPassDrawListContext* InDrawListContext,
	EFlags InFlags,
	ETranslucencyPass::Type InTranslucencyPassType)
	: FMeshPassProcessor(InMeshPassType, Scene, ERHIFeatureLevel::ES3_1, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InDrawRenderState)
	, TranslucencyPassType(InTranslucencyPassType)
	, Flags(InFlags)
	, bTranslucentBasePass(InTranslucencyPassType != ETranslucencyPass::TPT_MAX)
	, bDeferredShading(IsMobileDeferredShadingEnabled(GetFeatureLevelShaderPlatform(ERHIFeatureLevel::ES3_1)))
	, bPassUsesDeferredShading(bDeferredShading && !bTranslucentBasePass)
	, EarlyZPassMode(Scene ? Scene->EarlyZPassMode : DDM_None)
{
}

bool FMobileBasePassMeshProcessor::ShouldDraw(const FMaterial& Material) const
{
	bool bShouldDraw = ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain());
	const FMaterialShadingModelField ShadingModels = Material.GetShadingModels();
	const bool bIsSLW = ShadingModels.HasShadingModel(MSM_SingleLayerWater);
	const bool bIsTranslucentSLW = bIsSLW && IsUsingSingleLayerWaterFallback(GetFeatureLevelShaderPlatform(ERHIFeatureLevel::ES3_1));
	const bool bIsTranslucent = IsTranslucentBlendMode(Material.GetBlendMode()) || bIsTranslucentSLW; // Water goes into the translucent pass;
	if (bTranslucentBasePass)
	{
		// Skipping TPT_TranslucencyAfterDOFModulate. That pass is only needed for Dual Blending, which is not supported on Mobile.
		bShouldDraw &= bIsTranslucent && !Material.IsDeferredDecal() &&
			(TranslucencyPassType == ETranslucencyPass::TPT_AllTranslucency
				|| (TranslucencyPassType == ETranslucencyPass::TPT_TranslucencyStandard && !Material.IsMobileSeparateTranslucencyEnabled())
				|| (TranslucencyPassType == ETranslucencyPass::TPT_TranslucencyAfterDOF && Material.IsMobileSeparateTranslucencyEnabled()));

	}
	else
	{
		// opaque materials.
		bShouldDraw &= !bIsTranslucent && !bIsSLW;
	}
	return bShouldDraw;
}
static bool MaterialCanReceiveCSM(const FMaterial& Material)
{
	const EBlendMode BlendMode = Material.GetBlendMode();
	const FMaterialShadingModelField ShadingModels = Material.GetShadingModels();
	const bool bIsTranslucent = IsTranslucentBlendMode(BlendMode);
	const bool bIsSLW = ShadingModels.HasShadingModel(MSM_SingleLayerWater);
	const bool IsTranslucentPerPixelLighting = bIsTranslucent && Material.GetTranslucencyLightingMode() == ETranslucencyLightingMode::TLM_SurfacePerPixelLighting;
	return !bIsTranslucent || bIsSLW || IsTranslucentPerPixelLighting;
}

bool FMobileBasePassMeshProcessor::TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material)
{
	if (ShouldDraw(Material))
	{
		FMaterialShadingModelField ShadingModels = Material.GetShadingModels();
		if ((uint16(Material.MaterialGetSubstrateMaterialBsdfFeatures_RenderThread()) & uint16(ESubstrateBsdfFeature::Toon)) > 0)
		{
			ShadingModels.AddShadingModel(EMaterialShadingModel::MSM_Strata); // This model is not only a DefaultLit model and will be schedule in the "all shading model" lighting pass.
		}

		const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
#if WITH_EDITOR
		// Non-Editor builds filter out shading models on a material load
		UMaterialInterface::FilterOutPlatformShadingModels(ShaderPlatform, ShadingModels);
#endif
		const bool bIsSLW = ShadingModels.HasShadingModel(MSM_SingleLayerWater);
		// Only translucent SLW fallback takes this path. Opaque SLW must be handled by FMobileSingleLayerWaterPassMeshProcessor 
		check(!bIsSLW || IsUsingSingleLayerWaterFallback(ShaderPlatform)); 

		const EBlendMode BlendMode = Material.GetBlendMode();
		const bool bIsLitMaterial = ShadingModels.IsLit();
		const bool bIsTranslucent = IsTranslucentBlendMode(BlendMode) || bIsSLW; // Water fallback goes into the translucent pass;
		const bool bIsMasked = IsMaskedBlendMode(Material);		
		const bool bCanReceiveCSM = ((Flags & EFlags::CanReceiveCSM) == EFlags::CanReceiveCSM) && MaterialCanReceiveCSM(Material);
		ELightMapPolicyType Policy = MobileBasePass::SelectMeshLightmapPolicy(Scene, MeshBatch, PrimitiveSceneProxy, bCanReceiveCSM, bPassUsesDeferredShading, bIsLitMaterial, bIsTranslucent);

		return Process<FUniformLightMapPolicy>(
			MeshBatch,
			BatchElementMask,
			StaticMeshId,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			Material,
			bIsMasked,
			ShadingModels,
			FUniformLightMapPolicy(Policy),
			MeshBatch.LCI,
			bCanReceiveCSM);
	}
	return true;
}

void FMobileBasePassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (!MeshBatch.bUseForMaterial || 
		(Flags & FMobileBasePassMeshProcessor::EFlags::DoNotCache) == FMobileBasePassMeshProcessor::EFlags::DoNotCache ||
		(PrimitiveSceneProxy && !PrimitiveSceneProxy->ShouldRenderInMainPass()))
	{
		return;
	}
	
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material && Material->GetRenderingThreadShaderMap())
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

template <typename LightMapPolicyType>
bool FMobileBasePassMeshProcessor::Process(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		const bool bIsMasked,
		FMaterialShadingModelField ShadingModels,
		const LightMapPolicyType& RESTRICT LightMapPolicy,
		const typename LightMapPolicyType::ElementDataType& RESTRICT LightMapElementData,
		const bool bCanReceiveCSM)
{
	TMeshProcessorShaders<
		TMobileBasePassVSPolicyParamType<LightMapPolicyType>,
		TMobileBasePassPSPolicyParamType<LightMapPolicyType>> BasePassShaders;

	EMobileLocalLightSetting LocalLightSetting = EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED;
	if (Scene && PrimitiveSceneProxy && ShadingModels.IsLit())
	{
		if (!bPassUsesDeferredShading &&
			// we can choose to use a single permutation regarless of local light state
			// this is to avoid re-caching MDC on light state changes
			(MobileLocalLightsUseSinglePermutation(Scene->GetShaderPlatform()) || PrimitiveSceneProxy->GetPrimitiveSceneInfo()->NumMobileDynamicLocalLights > 0))
		{
			LocalLightSetting = GetMobileForwardLocalLightSetting(Scene->GetShaderPlatform());
		}
	}

	if (!MobileBasePass::GetShaders(
		LightMapPolicy,
		LocalLightSetting,
		MaterialResource,
		MeshBatch.VertexFactory->GetType(),
		BasePassShaders.VertexShader,
		BasePassShaders.PixelShader))
	{
		return false;
	}

	const bool bMaskedInEarlyPass = (MaterialResource.IsMasked() || MeshBatch.bDitheredLODTransition) && Scene && MaskedInEarlyPass(Scene->GetShaderPlatform());
	const bool bForcePassDrawRenderState = ((Flags & EFlags::ForcePassDrawRenderState) == EFlags::ForcePassDrawRenderState);
	const bool bIsFullDepthPrepassEnabled = Scene && (Scene->EarlyZPassMode == DDM_AllOpaque || Scene->EarlyZPassMode == DDM_AllOpaqueNoVelocity);

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);
	if (!bForcePassDrawRenderState)
	{
		if (bTranslucentBasePass)
		{
			MobileBasePass::SetTranslucentRenderState(DrawRenderState, MaterialResource, ShadingModels);
		}
		else if((MeshBatch.bUseForDepthPass && bIsFullDepthPrepassEnabled) || bMaskedInEarlyPass)
		{
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Equal>::GetRHI());
		}
		else
		{
			const bool bCanUseDepthStencil = ((Flags & EFlags::CanUseDepthStencil) == EFlags::CanUseDepthStencil);
			MobileBasePass::SetOpaqueRenderState(DrawRenderState, PrimitiveSceneProxy, MaterialResource, ShadingModels, bCanUseDepthStencil, bPassUsesDeferredShading);
		}
	}

	FMeshDrawCommandSortKey SortKey; 
	if (bTranslucentBasePass)
	{
		SortKey = CalculateTranslucentMeshStaticSortKey(PrimitiveSceneProxy, MeshBatch.MeshIdInPrimitive);
		// We always want water to be rendered first on mobile in order to mimic other renderers where it is opaque. We shift the other priorities by 1.
		// And we also want to render the meshes used for mobile pixel projected reflection first if it is opaque.
		SortKey.Translucent.Priority = ShadingModels.HasShadingModel(MSM_SingleLayerWater) ? uint16(0) : uint16(FMath::Clamp(uint32(SortKey.Translucent.Priority) + 1, 0u, uint32(USHRT_MAX)));
	}
	else
	{
		// Background primitives will be rendered last in masked/non-masked buckets
		bool bBackground = PrimitiveSceneProxy ? PrimitiveSceneProxy->TreatAsBackgroundForOcclusion() : false;
		// Default static sort key separates masked and non-masked geometry, generic mesh sorting will also sort by PSO
		// if platform wants front to back sorting, this key will be recomputed in InitViews
		SortKey = GetBasePassStaticSortKey(EarlyZPassMode, bIsMasked, bBackground, BasePassShaders.VertexShader.GetShader(), BasePassShaders.PixelShader.GetShader());
	}
	
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(MaterialResource, OverrideSettings);
	ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(MaterialResource, OverrideSettings);

	TMobileBasePassShaderElementData<LightMapPolicyType> ShaderElementData(LightMapElementData, bCanReceiveCSM);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		DrawRenderState,
		BasePassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);
	return true;
}

void FMobileBasePassMeshProcessor::CollectPSOInitializersForLMPolicy(
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMeshPassProcessorRenderState& RESTRICT DrawRenderState,
	const FGraphicsPipelineRenderTargetsInfo& RESTRICT RenderTargetsInfo,
	const FMaterial& RESTRICT MaterialResource,
	EMobileLocalLightSetting LocalLightSetting,
	const ELightMapPolicyType LightMapPolicyType,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	EPrimitiveType PrimitiveType,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	TMeshProcessorShaders<
		TMobileBasePassVSPolicyParamType<FUniformLightMapPolicy>,
		TMobileBasePassPSPolicyParamType<FUniformLightMapPolicy>> BasePassShaders;
	
	FUniformLightMapPolicy UniformLightmapPolicy(LightMapPolicyType);

	if (!MobileBasePass::GetShaders(
		UniformLightmapPolicy,
		LocalLightSetting,
		MaterialResource,
		VertexFactoryData.VertexFactoryType,
		BasePassShaders.VertexShader,
		BasePassShaders.PixelShader))
	{
		return;
	}

	// subpass info set during the submission of the draws in mobile deferred renderer.
	uint8 SubpassIndex = 0;
	ESubpassHint SubpassHint = GetSubpassHint(GMaxRHIShaderPlatform, bDeferredShading, RenderTargetsInfo.MultiViewCount > 1, RenderTargetsInfo.NumSamples);
	if (bTranslucentBasePass && MobileAllowFramebufferFetch(GMaxRHIShaderPlatform))
	{
		if (MeshPassType == EMeshPass::TranslucencyAfterDOF)
		{
			// Separate translucency in subpass 0
			SubpassIndex = 0;
			SubpassHint = ESubpassHint::None;
		}
		else
		{
			SubpassIndex = bDeferredShading ? 2 : 1;
		}
	}
	
	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		MaterialResource,
		DrawRenderState,
		RenderTargetsInfo,
		BasePassShaders,
		MeshFillMode,
		MeshCullMode,
		PrimitiveType,
		EMeshPassFeatures::Default,
		SubpassHint,
		SubpassIndex,
		true /*bRequired*/,
		PSOCollectorIndex,
		PSOInitializers);
}

static void SetupMultiViewInfo(FGraphicsPipelineRenderTargetsInfo& RenderTargetsInfo)
{
	const static UE::StereoRenderUtils::FStereoShaderAspects Aspects(GMaxRHIShaderPlatform);
	// If mobile multiview is enabled we expect it will be used with a native MMV, no pre-caching for fallbacks 
	RenderTargetsInfo.MultiViewCount = Aspects.IsMobileMultiViewEnabled() ? (GSupportsMobileMultiView ? 2 : 1) : 0;
	// FIXME: Need to figure out if renderer will use shading rate texture or not
	RenderTargetsInfo.bHasFragmentDensityAttachment = GVRSImageManager.IsAttachmentVRSEnabled();
}

void FMobileBasePassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	EMaterialDomain MaterialDomain = Material.GetMaterialDomain();
	// Precache only Surface and Decal materials (other domains are handled separately) for BasePass rendering
	const bool bPrecacheMaterialDomain = (MaterialDomain == MD_Surface || MaterialDomain == MD_DeferredDecal);
	if (!bPrecacheMaterialDomain)
	{
		return;
	}
		
	if (bTranslucentBasePass)
	{
		static IConsoleVariable* CVarSeparateTranslucency = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SeparateTranslucency"));
		if (CVarSeparateTranslucency->GetInt() == 0)
		{
			if (MeshPassType != EMeshPass::TranslucencyAll)
			{
				// Precache only TranslucencyAll when SeparateTranslucency is not active
				return;
			}
		}
		else if (MeshPassType == EMeshPass::TranslucencyAll && !UPSOPrecacheSettingsManager::GetSettings().bTranslucencyAll)
		{
			// PSO precaching is disabled for TranslucencyAll while SeparateTranslucency is active
			return;
		}
	}
	
	// Check if material should be rendered
	if (!PreCacheParams.bRenderInMainPass || !ShouldDraw(Material))
	{
		return;
	}

	// Determine the mesh's material and blend mode.
	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
	const EBlendMode BlendMode = Material.GetBlendMode();
	const FMaterialShadingModelField ShadingModels = Material.GetShadingModels();
	const bool bLitMaterial = ShadingModels.IsLit();

	bool bMovable = 
		PreCacheParams.Mobility == EComponentMobility::Movable || 
		PreCacheParams.Mobility == EComponentMobility::Stationary || 
		PreCacheParams.bUsesIndirectLightingCache; // ILC uses movable path

	// Setup the draw state
	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);
	
	const bool bMaskedInEarlyPass = MaskedInEarlyPass(ShaderPlatform);
	FExclusiveDepthStencil ExclusiveDepthStencil = (bTranslucentBasePass || bMaskedInEarlyPass) ? 
		FExclusiveDepthStencil::DepthRead_StencilRead : 
		FExclusiveDepthStencil::DepthWrite_StencilWrite;

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;
	if (MobileAllowFramebufferFetch(ShaderPlatform) || !bTranslucentBasePass)
	{		
		SetupGBufferRenderTargetInfo(SceneTexturesConfig, RenderTargetsInfo, false /*bSetupDepthStencil*/);
	}
	else
	{
		EPixelFormat SceneColorFormat = SceneTexturesConfig.ColorFormat;
		ETextureCreateFlags SceneColorCreateFlags = SceneTexturesConfig.ColorCreateFlags;
		AddRenderTargetInfo(SceneColorFormat, SceneColorCreateFlags, RenderTargetsInfo);
		ExclusiveDepthStencil = FExclusiveDepthStencil::DepthRead_StencilWrite;
	}
	SetupDepthStencilInfo(PF_DepthStencil, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad, ExclusiveDepthStencil, RenderTargetsInfo);
	SetupMultiViewInfo(RenderTargetsInfo);
					
	if (bTranslucentBasePass)
	{
		MobileBasePass::SetTranslucentRenderState(DrawRenderState, Material, ShadingModels);
	}
	//else if((MeshBatch.bUseForDepthPass && Scene->EarlyZPassMode == DDM_AllOpaque) || bMaskedInEarlyPass)
	//{
	//	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Equal>::GetRHI());
	//}
	else
	{
		MobileBasePass::SetOpaqueRenderState(DrawRenderState, nullptr, Material, ShadingModels, true, bPassUsesDeferredShading);
	}

	EMobileLocalLightSetting LocalLightSetting = EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED;
	if (bLitMaterial && !bPassUsesDeferredShading)
	{
		LocalLightSetting = GetMobileForwardLocalLightSetting(ShaderPlatform);
	}
		
	const bool bUseLocalLightPermutation = (LocalLightSetting != EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED);
	const bool bUseNoLocalLightPermutation = !bUseLocalLightPermutation || !MobileLocalLightsUseSinglePermutation(ShaderPlatform);
	check(bUseLocalLightPermutation || bUseNoLocalLightPermutation);

	const bool bCanReceiveCSM = ((Flags & EFlags::CanReceiveCSM) == EFlags::CanReceiveCSM) && MaterialCanReceiveCSM(Material);

	FMobileLightMapPolicyTypeList UniformLightMapPolicyTypes = MobileBasePass::GetUniformLightMapPolicyTypeForPSOCollection(bLitMaterial, bTranslucentBasePass, bPassUsesDeferredShading, bCanReceiveCSM, bMovable);
	
	for (ELightMapPolicyType LightMapPolicyType : UniformLightMapPolicyTypes)
	{
		if (bUseNoLocalLightPermutation)
		{
			CollectPSOInitializersForLMPolicy(VertexFactoryData, DrawRenderState, RenderTargetsInfo, Material, EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED, LightMapPolicyType, MeshFillMode, MeshCullMode, (EPrimitiveType)PreCacheParams.PrimitiveType, PSOInitializers);
		}
		if (bUseLocalLightPermutation)
		{
			CollectPSOInitializersForLMPolicy(VertexFactoryData, DrawRenderState, RenderTargetsInfo, Material, LocalLightSetting, LightMapPolicyType, MeshFillMode, MeshCullMode, (EPrimitiveType)PreCacheParams.PrimitiveType, PSOInitializers);
		}
	}
}

FMeshPassProcessor* CreateMobileBasePassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA>::GetRHI());
	const FExclusiveDepthStencil::Type DefaultBasePassDepthStencilAccess = FScene::GetDefaultBasePassDepthStencilAccess(FeatureLevel);
	PassDrawRenderState.SetDepthStencilAccess(DefaultBasePassDepthStencilAccess);
	const bool bDepthWrite = (DefaultBasePassDepthStencilAccess & FExclusiveDepthStencil::Type::DepthWrite) != 0;
	PassDrawRenderState.SetDepthStencilState(bDepthWrite ? TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI() : TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	const FMobileBasePassMeshProcessor::EFlags Flags = FMobileBasePassMeshProcessor::EFlags::CanUseDepthStencil
		| (MobileBasePassAlwaysUsesCSM(GShaderPlatformForFeatureLevel[FeatureLevel]) ? FMobileBasePassMeshProcessor::EFlags::CanReceiveCSM : FMobileBasePassMeshProcessor::EFlags::None);

	return new FMobileBasePassMeshProcessor(EMeshPass::BasePass, Scene, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags);
}

FMeshPassProcessor* CreateMobileBasePassCSMProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	const FExclusiveDepthStencil::Type DefaultBasePassDepthStencilAccess = FScene::GetDefaultBasePassDepthStencilAccess(FeatureLevel);

	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA>::GetRHI());
	PassDrawRenderState.SetDepthStencilAccess(DefaultBasePassDepthStencilAccess);
	const bool bDepthWrite = (DefaultBasePassDepthStencilAccess & FExclusiveDepthStencil::Type::DepthWrite) != 0;
	PassDrawRenderState.SetDepthStencilState(bDepthWrite ? TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI() : TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());

	// By default this processor will not cache anything. Only enable it when CSM culling is active
	FMobileBasePassMeshProcessor::EFlags Flags = FMobileBasePassMeshProcessor::EFlags::DoNotCache;
	if (!MobileBasePassAlwaysUsesCSM(GShaderPlatformForFeatureLevel[FeatureLevel]))
	{
		Flags = FMobileBasePassMeshProcessor::EFlags::CanReceiveCSM | FMobileBasePassMeshProcessor::EFlags::CanUseDepthStencil;
	}
	
	return new FMobileBasePassMeshProcessor(EMeshPass::MobileBasePassCSM, Scene, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags);
}

FMeshPassProcessor* CreateMobileTranslucencyStandardPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
	PassDrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthRead_StencilRead);
	

	const FMobileBasePassMeshProcessor::EFlags Flags = FMobileBasePassMeshProcessor::EFlags::CanUseDepthStencil | FMobileBasePassMeshProcessor::EFlags::CanReceiveCSM;

	return new FMobileBasePassMeshProcessor(EMeshPass::TranslucencyStandard, Scene, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_TranslucencyStandard);
}

FMeshPassProcessor* CreateMobileTranslucencyAfterDOFProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
	PassDrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthRead_StencilRead);

	const FMobileBasePassMeshProcessor::EFlags Flags = FMobileBasePassMeshProcessor::EFlags::CanUseDepthStencil | FMobileBasePassMeshProcessor::EFlags::CanReceiveCSM;

	return new FMobileBasePassMeshProcessor(EMeshPass::TranslucencyAfterDOF, Scene, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_TranslucencyAfterDOF);
}

FMeshPassProcessor* CreateMobileTranslucencyAllPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
	PassDrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthRead_StencilRead);

	const FMobileBasePassMeshProcessor::EFlags Flags = FMobileBasePassMeshProcessor::EFlags::CanUseDepthStencil | FMobileBasePassMeshProcessor::EFlags::CanReceiveCSM;

	return new FMobileBasePassMeshProcessor(EMeshPass::TranslucencyAll, Scene, InViewIfDynamicMeshCommand, PassDrawRenderState, InDrawListContext, Flags, ETranslucencyPass::TPT_AllTranslucency);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileBasePass, 			CreateMobileBasePassProcessor, 			EShadingPath::Mobile, EMeshPass::BasePass, 		EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileBasePassCSM,			CreateMobileBasePassCSMProcessor,		EShadingPath::Mobile, EMeshPass::MobileBasePassCSM, 	EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileTranslucencyAllPass,		CreateMobileTranslucencyAllPassProcessor,	EShadingPath::Mobile, EMeshPass::TranslucencyAll, 	EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileTranslucencyStandardPass,	CreateMobileTranslucencyStandardPassProcessor,	EShadingPath::Mobile, EMeshPass::TranslucencyStandard, 	EMeshPassFlags::MainView);
REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileTranslucencyAfterDOFPass,	CreateMobileTranslucencyAfterDOFProcessor,	EShadingPath::Mobile, EMeshPass::TranslucencyAfterDOF, 	EMeshPassFlags::MainView);
// Skipping EMeshPass::TranslucencyAfterDOFModulate because dual blending is not supported on mobile
// Skipping EMeshPass::TranslucencyHoldout, it is not supported on mobile.
