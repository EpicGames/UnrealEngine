// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MobileBasePassRendering.cpp: Base pass rendering implementation.
=============================================================================*/

#include "MobileBasePassRendering.h"
#include "DynamicPrimitiveDrawing.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneTextureParameters.h"
#include "ShaderPlatformQualitySettings.h"
#include "MaterialShaderQualitySettings.h"
#include "PrimitiveSceneInfo.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "EditorPrimitivesRendering.h"

#include "FramePro/FrameProProfiler.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/SpecularProfile.h"
#include "LocalLightSceneProxy.h"
#include "ReflectionEnvironment.h"
#include "RenderCore.h"
#include "LocalFogVolumeRendering.h"
#include "DistortionRendering.h"
#include "SceneViewState.h"

// Changing this causes a full shader recompile
static TAutoConsoleVariable<int32> CVarMobileDisableVertexFog(
	TEXT("r.Mobile.DisableVertexFog"),
	1,
	TEXT("If true, vertex fog will be omitted from the most of the mobile base pass shaders. Instead, fog will be applied in a separate pass and only when scene has a fog component."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileEnableMovableSpotLightShadows(
	TEXT("r.Mobile.EnableMovableSpotlightsShadow"),
	0,
	TEXT("If 1 then enable movable spotlight shadow support"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileMaxVisibleMovableSpotLightShadows(
	TEXT("r.Mobile.MaxVisibleMovableSpotLightShadows"),
	8,
	TEXT("The max number of visible spotlights can cast shadow sorted by screen size, should be as less as possible for performance reason"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileEnableMovablePointLightShadows(
	TEXT("r.Mobile.EnableMovablePointLightsShadows"),
	0,
	TEXT("If 1 then enable movable point light shadow support"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileSceneDepthAux(
	TEXT("r.Mobile.SceneDepthAux"),
	1,
	TEXT("1: 16F SceneDepthAux Format")
	TEXT("2: 32F SceneDepthAux Format"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobilePropagateAlpha(
	TEXT("r.Mobile.PropagateAlpha"),
	0,
	TEXT("0: Disabled")
	TEXT("1: Propagate Full Alpha Propagate"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileTonemapSubpass(
	TEXT("r.Mobile.TonemapSubpass"),
	0,
	TEXT(" Whether to enable mobile tonemap subpass \n")
	TEXT(" 0 = Off [default]\n")
	TEXT(" 1 = On"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileEnableCapsuleShadows(
	TEXT("r.Mobile.EnableCapsuleShadows"),
	0,
	TEXT("0: Capsule shadows are disabled in the mobile renderer")
	TEXT("1: Enables capsule shadowing on skinned components with bCastCapsuleDirectShadow or bCastCapsuleIndirectShadow enabled with the mobile renderer"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileEnableCapsuleDirectShadows(
	TEXT("r.Mobile.EnableCapsuleDirectShadows"),
	0,
	TEXT("0: Capsule direct shadows are disabled in the mobile renderer")
	TEXT("1: Enables capsule direct shadowing on skinned components with bCastCapsuleDirectShadow enabled with the mobile renderer"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);


IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FMobileBasePassUniformParameters, "MobileBasePass", SceneTextures);

static TAutoConsoleVariable<int32> CVarMobileUseHWsRGBEncoding(
	TEXT("r.Mobile.UseHWsRGBEncoding"),
	0,
	TEXT("0: Write sRGB encoding in the shader\n")
	TEXT("1: Use GPU HW to convert linear to sRGB automatically (device must support sRGB write control)\n"),
	ECVF_RenderThreadSafe);

EMobileTranslucentColorTransmittanceMode MobileDefaultTranslucentColorTransmittanceMode(EShaderPlatform Platform)
{
	if (FDataDrivenShaderPlatformInfo::GetSupportsDualSourceBlending(Platform) || IsSimulatedPlatform(Platform))
	{
		return EMobileTranslucentColorTransmittanceMode::DUAL_SRC_BLENDING;
	}
	if ((IsMetalMobilePlatform(Platform) || IsAndroidOpenGLESPlatform(Platform)) && MobileAllowFramebufferFetch(Platform))
	{
		return EMobileTranslucentColorTransmittanceMode::PROGRAMMABLE_BLENDING;
	}
	return EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING;
}

static bool SupportsTranslucentColorTransmittanceFallback(EShaderPlatform Platform, EMobileTranslucentColorTransmittanceMode Fallback)
{
	switch (Fallback)
	{
	default:
		return true;
	case EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING:
		return IsSimulatedPlatform(Platform) || IsAndroidPlatform(Platform);
	}
}

EMobileTranslucentColorTransmittanceMode MobileActiveTranslucentColorTransmittanceMode(EShaderPlatform Platform, bool bExplicitDefaultMode)
{
	const EMobileTranslucentColorTransmittanceMode DefaultMode = MobileDefaultTranslucentColorTransmittanceMode(Platform);

	if (DefaultMode == EMobileTranslucentColorTransmittanceMode::DUAL_SRC_BLENDING)
	{
		if (!GSupportsDualSrcBlending)
		{
			if (SupportsTranslucentColorTransmittanceFallback(Platform, EMobileTranslucentColorTransmittanceMode::PROGRAMMABLE_BLENDING) && GSupportsShaderFramebufferFetch)
			{
				return EMobileTranslucentColorTransmittanceMode::PROGRAMMABLE_BLENDING;
			}
			else
			{
				check(SupportsTranslucentColorTransmittanceFallback(Platform, EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING));
				return EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING;
			}
		}
	}
	else if (DefaultMode == EMobileTranslucentColorTransmittanceMode::PROGRAMMABLE_BLENDING)
	{
		if (!GSupportsShaderFramebufferFetch || !GSupportsShaderFramebufferFetchProgrammableBlending)
		{
			check(SupportsTranslucentColorTransmittanceFallback(Platform, EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING));
			return EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING;
		}
	}

	return bExplicitDefaultMode ? DefaultMode : EMobileTranslucentColorTransmittanceMode::DEFAULT;
}

#define IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_VERTEX_SHADER_TYPE(LightMapPolicyType,LightMapPolicyName) \
	typedef TMobileBasePassVS< LightMapPolicyType > TMobileBasePassVS##LightMapPolicyName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMobileBasePassVS##LightMapPolicyName, TEXT("/Engine/Private/MobileBasePassVertexShader.usf"), TEXT("Main"), SF_Vertex); \

#define IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_PIXEL_SHADER_TYPE(LightMapPolicyType, LightMapPolicyName, LocalLightSettingType, LocalLightSettingName) \
	typedef TMobileBasePassPS< LightMapPolicyType, LocalLightSettingType, EMobileTranslucentColorTransmittanceMode::DEFAULT > TMobileBasePassPS##LightMapPolicyName##LocalLightSettingName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMobileBasePassPS##LightMapPolicyName##LocalLightSettingName, TEXT("/Engine/Private/MobileBasePassPixelShader.usf"), TEXT("Main"), SF_Pixel); \
	typedef TMobileBasePassPS< LightMapPolicyType, LocalLightSettingType, EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING > TMobileBasePassPS##LightMapPolicyName##LocalLightSettingName##ThinTranslGrey; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMobileBasePassPS##LightMapPolicyName##LocalLightSettingName##ThinTranslGrey, TEXT("/Engine/Private/MobileBasePassPixelShader.usf"), TEXT("Main"), SF_Pixel); \

#define IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(LightMapPolicyType, LightMapPolicyName) \
	IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_VERTEX_SHADER_TYPE(LightMapPolicyType, LightMapPolicyName) \
	IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_PIXEL_SHADER_TYPE(LightMapPolicyType, LightMapPolicyName, LOCAL_LIGHTS_DISABLED, ) \
	IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_PIXEL_SHADER_TYPE(LightMapPolicyType, LightMapPolicyName, LOCAL_LIGHTS_ENABLED, LocalLights)

// Implement shader types per lightmap policy 
IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_NO_LIGHTMAP>, FNoLightMapPolicy);
IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_LQ_LIGHTMAP>, TLightMapPolicyLQ);
IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_MOBILE_DISTANCE_FIELD_SHADOWS_AND_LQ_LIGHTMAP>, FMobileDistanceFieldShadowsAndLQLightMapPolicy);
IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_MOBILE_DISTANCE_FIELD_SHADOWS_LIGHTMAP_AND_CSM>, FMobileDistanceFieldShadowsLightMapAndCSMLightingPolicy);
IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_LIGHTMAP>, FMobileDirectionalLightCSMAndLightMapPolicy);
IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_MOBILE_DIRECTIONAL_LIGHT_AND_SH_INDIRECT>, FMobileDirectionalLightAndSHIndirectPolicy);
IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_MOBILE_DIRECTIONAL_LIGHT_CSM_AND_SH_INDIRECT>, FMobileDirectionalLightCSMAndSHIndirectPolicy);
IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(TUniformLightMapPolicy<LMP_MOBILE_DIRECTIONAL_LIGHT_CSM>, FMobileDirectionalLightAndCSMPolicy);
IMPLEMENT_MOBILE_SHADING_BASEPASS_LIGHTMAPPED_SHADER_TYPE(FMobileSingleLayerWaterPolicy, FMobileSingleLayerWaterPolicy);

bool MaterialRequiresColorTransmittanceBlending(const FMaterial& MaterialResource)
{
	return MaterialResource.GetShadingModels().HasShadingModel(MSM_ThinTranslucent) || MaterialResource.GetBlendMode() == BLEND_TranslucentColoredTransmittance;
}

bool MaterialRequiresColorTransmittanceBlending(const FMaterialShaderParameters& MaterialParameters)
{
	return MaterialParameters.ShadingModels.HasShadingModel(MSM_ThinTranslucent) || MaterialParameters.BlendMode == BLEND_TranslucentColoredTransmittance;
}

bool ShouldCacheShaderForColorTransmittanceFallback(const FMaterialShaderPermutationParameters& Parameters, const EMobileTranslucentColorTransmittanceMode TranslucentColorTransmittanceFallback)
{
	if (TranslucentColorTransmittanceFallback == EMobileTranslucentColorTransmittanceMode::DEFAULT)
	{
		return true;
	}
	
	if (!MaterialRequiresColorTransmittanceBlending(Parameters.MaterialParameters))
	{
		return false;
	}

	return SupportsTranslucentColorTransmittanceFallback(Parameters.Platform, TranslucentColorTransmittanceFallback);
}

// shared defines for mobile base pass VS and PS
void MobileBasePassModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	static auto* MobileUseHWsRGBEncodingCVAR = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.UseHWsRGBEncoding"));
	const bool bMobileUseHWsRGBEncoding = (MobileUseHWsRGBEncodingCVAR && MobileUseHWsRGBEncodingCVAR->GetValueOnAnyThread() == 1);
	const bool bMobileHDR = IsMobileHDR();
	OutEnvironment.SetDefine( TEXT("OUTPUT_GAMMA_SPACE"), !bMobileHDR && !bMobileUseHWsRGBEncoding);
	OutEnvironment.SetDefine( TEXT("OUTPUT_MOBILE_HDR"), bMobileHDR ? 1u : 0u);
	
	const bool bTranslucentMaterial = 
		IsTranslucentBlendMode(Parameters.MaterialParameters) ||
		Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);

	// This define simply lets the compilation environment know that we are using a Base Pass PixelShader.
	OutEnvironment.SetDefine(TEXT("IS_BASE_PASS"), 1u);
	OutEnvironment.SetDefine(TEXT("IS_MOBILE_BASE_PASS"), 1u);
		
	const bool bDeferredShadingEnabled = IsMobileDeferredShadingEnabled(Parameters.Platform);
	if (bDeferredShadingEnabled)
	{
		OutEnvironment.SetDefine(TEXT("ENABLE_SHADINGMODEL_SUPPORT_MOBILE_DEFERRED"), MobileUsesGBufferCustomData(Parameters.Platform));
	}

	const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
	// separate translucency must sample depth texture, instead of using FBF
	OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEPTHREAD_SUBPASS"), bTranslucentMaterial && !bMobileForceDepthRead && !Parameters.MaterialParameters.bIsMobileSeparateTranslucencyEnabled ? 1u : 0u);
	// translucency is in the same subpass with deferred shading shaders, so it has access to GBuffer
	const bool bDeferredShadingSubpass = (bDeferredShadingEnabled && bTranslucentMaterial && !Parameters.MaterialParameters.bIsMobileSeparateTranslucencyEnabled);
	OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEFERREDSHADING_SUBPASS"), bDeferredShadingSubpass ? 1u : 0u);
}

template<typename LightMapPolicyType>
bool TMobileBasePassPSPolicyParamType<LightMapPolicyType>::ModifyCompilationEnvironmentForQualityLevel(EShaderPlatform Platform, EMaterialQualityLevel::Type QualityLevel, FShaderCompilerEnvironment& OutEnvironment)
{
	// Get quality settings for shader platform
	const UShaderPlatformQualitySettings* MaterialShadingQuality = UMaterialShaderQualitySettings::Get()->GetShaderPlatformQualitySettings(Platform);
	const FMaterialQualityOverrides& QualityOverrides = MaterialShadingQuality->GetQualityOverrides(QualityLevel);

	// the point of this check is to keep the logic between enabling overrides here and in UMaterial::GetQualityLevelUsage() in sync
	checkf(QualityOverrides.CanOverride(Platform), TEXT("ShaderPlatform %d was not marked as being able to use quality overrides! Include it in CanOverride() and recook."), static_cast<int32>(Platform));
	OutEnvironment.SetDefine(TEXT("MOBILE_QL_FORCE_FULLY_ROUGH"), QualityOverrides.bEnableOverride && QualityOverrides.bForceFullyRough != 0 ? 1u : 0u);
	OutEnvironment.SetDefine(TEXT("MOBILE_QL_FORCE_NONMETAL"), QualityOverrides.bEnableOverride && QualityOverrides.bForceNonMetal != 0 ? 1u : 0u);
	OutEnvironment.SetDefine(TEXT("QL_FORCEDISABLE_LM_DIRECTIONALITY"), QualityOverrides.bEnableOverride && QualityOverrides.bForceDisableLMDirectionality != 0 ? 1u : 0u);
	OutEnvironment.SetDefine(TEXT("MOBILE_QL_FORCE_DISABLE_PREINTEGRATEDGF"), QualityOverrides.bEnableOverride && QualityOverrides.bForceDisablePreintegratedGF != 0 ? 1u : 0u);
	OutEnvironment.SetDefine(TEXT("MOBILE_SHADOW_QUALITY"), (uint32)QualityOverrides.MobileShadowQuality);
	OutEnvironment.SetDefine(TEXT("MOBILE_QL_DISABLE_MATERIAL_NORMAL"), QualityOverrides.bEnableOverride && QualityOverrides.bDisableMaterialNormalCalculation);
	return true;
}

extern void SetForwardLightUniformParametersFromView(FRDGBuilder& GraphBuilder, const FViewInfo& View, FForwardLightUniformParameters& ForwardLightUniformParameters);

void SetupMobileBasePassUniformParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, 
	EMobileBasePass BasePass,
	EMobileSceneTextureSetupMode SetupMode,
	FMobileBasePassUniformParameters& BasePassParameters,
	const FMobileBasePassTextures& MobileBasePassTextures,
	bool bForRealtimeSkyCapture,
	bool bIsRenderingDeferredBasepass)
{
	const FViewInfo* InstancedView = View.GetInstancedView();

	SetupFogUniformParameters(GraphBuilder, View, BasePassParameters.Fog, bForRealtimeSkyCapture);
	if (InstancedView && View.UsesMobileMultiViewShaders())
	{
		SetupFogUniformParameters(GraphBuilder, *InstancedView, BasePassParameters.FogMMV, bForRealtimeSkyCapture);
	}
	else
	{
		BasePassParameters.FogMMV = BasePassParameters.Fog;
	}

	SetForwardLightUniformParametersFromView(GraphBuilder, View, BasePassParameters.Forward);

	const FScene* Scene = View.Family->Scene ? View.Family->Scene->GetRenderScene() : nullptr;
	const FPlanarReflectionSceneProxy* ReflectionSceneProxy = Scene ? Scene->GetForwardPassGlobalPlanarReflection() : nullptr;
	SetupPlanarReflectionUniformParameters(View, ReflectionSceneProxy, BasePassParameters.PlanarReflection);
	if (BasePassParameters.PlanarReflection.PlanarReflectionTexture == nullptr)
	{
		BasePassParameters.PlanarReflection.PlanarReflectionTexture = GBlackTexture->TextureRHI;
		BasePassParameters.PlanarReflection.PlanarReflectionSampler = GBlackTexture->SamplerStateRHI;
	}

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
	const FSceneTextures* SceneTextures = View.GetSceneTexturesChecked();

	SetupMobileSceneTextureUniformParameters(GraphBuilder, SceneTextures, SetupMode, BasePassParameters.SceneTextures);

	BasePassParameters.PreIntegratedGFTexture = GSystemTextures.PreintegratedGF->GetRHI();
	BasePassParameters.PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	BasePassParameters.EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));

	const bool bIsMobileMultiViewEnabled = View.UsesMobileMultiViewShaders();
	BasePassParameters.DBuffer = GetDBufferParameters(GraphBuilder, MobileBasePassTextures.DBufferTextures, View.GetShaderPlatform(), bIsMobileMultiViewEnabled);

	// Don't bind the SSAO texture if we're rendering the deferred basepass. It's not needed and will result in additional barriers
	if (!bIsRenderingDeferredBasepass && HasBeenProduced(MobileBasePassTextures.AmbientOcclusionTexture))
	{
		BasePassParameters.AmbientOcclusionTexture = MobileBasePassTextures.AmbientOcclusionTexture;
	}
	else
	{
		BasePassParameters.AmbientOcclusionTexture = SystemTextures.White;
	}
	BasePassParameters.AmbientOcclusionSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	BasePassParameters.AmbientOcclusionStaticFraction = FMath::Clamp(View.FinalPostProcessSettings.AmbientOcclusionStaticFraction, 0.0f, 1.0f);
	
	if (!bIsRenderingDeferredBasepass && HasBeenProduced(MobileBasePassTextures.ScreenSpaceShadowMaskTexture))
	{
		BasePassParameters.ScreenSpaceShadowMaskTexture = MobileBasePassTextures.ScreenSpaceShadowMaskTexture;
		BasePassParameters.ScreenSpaceShadowMaskTextureArray = MobileBasePassTextures.ScreenSpaceShadowMaskTexture;
	}
	else
	{
		BasePassParameters.ScreenSpaceShadowMaskTexture = SystemTextures.White;
		BasePassParameters.ScreenSpaceShadowMaskTextureArray = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, PF_DepthStencil, FClearValueBinding::White);
	}
	BasePassParameters.ScreenSpaceShadowMaskSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Substrate
	Substrate::BindSubstrateMobileForwardPasslUniformParameters(GraphBuilder, View, BasePassParameters.Substrate);

	if (bForRealtimeSkyCapture)
	{
		// LFV are not allowed in real time capture since they are local effects.
		SetDummyLocalFogVolumeUniformParametersStruct(GraphBuilder, BasePassParameters.LFV);
	}
	else
	{
		BasePassParameters.LFV = View.LocalFogVolumeViewData.UniformParametersStruct;
	}

	// We need to compose the half resolution LFV texture when rendering mesh with Sky materials. So that the Fog passes remains cheap and we can keep the stencil test on the fog pass.
	if (BasePass > EMobileBasePass::DepthPrePass && !bForRealtimeSkyCapture)
	{
		// HalfResLocalFogVolumeView is rendered after the depth pre pass so we only bind it after the depth pre pass
		BasePassParameters.bApplyHalfResLocalFogToSkyMeshes = View.LocalFogVolumeViewData.bUseHalfResLocalFogVolume ? 1 : 0;
		BasePassParameters.HalfResLocalFogVolumeViewTexture = View.LocalFogVolumeViewData.HalfResLocalFogVolumeView;
	}
	else
	{
		BasePassParameters.bApplyHalfResLocalFogToSkyMeshes = 0;
		BasePassParameters.HalfResLocalFogVolumeViewTexture = SystemTextures.BlackAlphaOne;
	}
	BasePassParameters.HalfResLocalFogVolumeViewSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	SetupReflectionUniformParameters(GraphBuilder, View, BasePassParameters.ReflectionsParameters);

	SetupMobileSSRParameters(GraphBuilder, View, BasePassParameters.SSRParams);

	BasePassParameters.SceneColorCopyTexture = SystemTextures.White;
	BasePassParameters.SceneColorCopySampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	BasePassParameters.SceneDepthCopyTexture = SystemTextures.DepthDummy;
	BasePassParameters.SceneDepthCopySampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	SetupDistortionParams(BasePassParameters.DistortionParams, View);

#if WITH_DEBUG_VIEW_MODES
	if (View.Family->UseDebugViewPS())
	{
		SetupDebugViewModePassUniformBufferConstants(View, BasePassParameters.DebugViewMode);
	}
#endif
	
	// We need some kind of "Dummy" type resource here, to be fully correct.
	// Otherwise UAVs break RDG pass auto-merging
	if (BasePass <= EMobileBasePass::Opaque || GMaxRHIFeatureLevel > ERHIFeatureLevel::ES3_1)
	{
		// QuadOverdraw is a UAV so it needs to be initialized even if not used
		FRDGTextureRef QuadOverdrawTexture = nullptr;
		EDebugViewShaderMode DebugViewMode = View.Family->GetDebugViewShaderMode();
		if ((DebugViewMode == EDebugViewShaderMode::DVSM_QuadComplexity || DebugViewMode == DVSM_ShaderComplexityContainedQuadOverhead) && SceneTextures)
		{
			QuadOverdrawTexture = SceneTextures->DebugAux;
		}

		if (!QuadOverdrawTexture)
		{
			QuadOverdrawTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_R32_UINT, FClearValueBinding::None, TexCreate_UAV), TEXT("DummyOverdrawUAV"));
		}
		BasePassParameters.QuadOverdraw = GraphBuilder.CreateUAV(QuadOverdrawTexture);

		// Occlussion Feedback
		FRDGBuffer* OcclusionBuffer = View.ViewState ? View.ViewState->OcclusionFeedback.GetGPUFeedbackBuffer() : nullptr;
		if (!OcclusionBuffer)
		{
			OcclusionBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 1), TEXT("OcclusionBufferFallback"));

		}
		BasePassParameters.RWOcclusionBufferUAV = GraphBuilder.CreateUAV(OcclusionBuffer);
	}
}

TRDGUniformBufferRef<FMobileBasePassUniformParameters> CreateMobileBasePassUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	EMobileBasePass BasePass,
	EMobileSceneTextureSetupMode SetupMode,
	const FMobileBasePassTextures& MobileBasePassTextures,
	bool bForRealtimeSkyCapture,
	bool bIsRenderingDeferredBasepass)
{
	FMobileBasePassUniformParameters* BasePassParameters = GraphBuilder.AllocParameters<FMobileBasePassUniformParameters>();
	SetupMobileBasePassUniformParameters(GraphBuilder, View, BasePass, SetupMode, *BasePassParameters, MobileBasePassTextures, bForRealtimeSkyCapture, bIsRenderingDeferredBasepass);

	return GraphBuilder.CreateUniformBuffer(BasePassParameters);
}

void SetupMobileDirectionalLightUniformParameters(
	const FScene& Scene,
	const FViewInfo& SceneView,
	const FVisibleLightInfoArray& VisibleLightInfos,
	int32 ChannelIdx,
	bool bDynamicShadows,
	FMobileDirectionalLightShaderParameters& Params)
{
	ERHIFeatureLevel::Type FeatureLevel = Scene.GetFeatureLevel();
	FLightSceneInfo* Light = Scene.GetMobileDirectionalLightForView(ChannelIdx, SceneView);
	if (Light)
	{
		Params.DirectionalLightColor = Light->Proxy->GetSunIlluminanceAccountingForSkyAtmospherePerPixelTransmittance();
		Params.DirectionalLightDirectionAndShadowTransition = FVector4f((FVector3f)-Light->Proxy->GetDirection(), 0.f);

		const FVector2D FadeParams = Light->Proxy->GetDirectionalLightDistanceFadeParameters(FeatureLevel, Light->IsPrecomputedLightingValid(), SceneView.MaxShadowCascades);
		Params.DirectionalLightDistanceFadeMADAndSpecularScale.X = FadeParams.Y;
		Params.DirectionalLightDistanceFadeMADAndSpecularScale.Y = -FadeParams.X * FadeParams.Y;
		Params.DirectionalLightDistanceFadeMADAndSpecularScale.Z = FMath::Clamp(Light->Proxy->GetSpecularScale(), 0.f, 1.f);
		Params.DirectionalLightDistanceFadeMADAndSpecularScale.W = FMath::Clamp(Light->Proxy->GetDiffuseScale(), 0.f, 1.f);

		const int32 ShadowMapChannel = IsStaticLightingAllowed() ? Light->Proxy->GetShadowMapChannel() : INDEX_NONE;
		int32 DynamicShadowMapChannel = Light->GetDynamicShadowMapChannel();

		// Static shadowing uses ShadowMapChannel, dynamic shadows are packed into light attenuation using DynamicShadowMapChannel
		Params.DirectionalLightShadowMapChannelMask =
			(ShadowMapChannel == 0 ? 1 : 0) |
			(ShadowMapChannel == 1 ? 2 : 0) |
			(ShadowMapChannel == 2 ? 4 : 0) |
			(ShadowMapChannel == 3 ? 8 : 0) |
			// The dynamic shadow map channel needs to be set to 16 even when not present
			// or the light's contribution will be zeroed out in the shader.
			(DynamicShadowMapChannel <= 0 ? 16 : 0) |
			(DynamicShadowMapChannel == 1 ? 32 : 0) |
			(DynamicShadowMapChannel == 2 ? 64 : 0) |
			(DynamicShadowMapChannel == 3 ? 128 : 0);

		if (bDynamicShadows && VisibleLightInfos.IsValidIndex(Light->Id) && VisibleLightInfos[Light->Id].AllProjectedShadows.Num() > 0)
		{
			const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& DirectionalLightShadowInfos = VisibleLightInfos[Light->Id].AllProjectedShadows;
			static_assert(MAX_MOBILE_SHADOWCASCADES <= 4, "more than 4 cascades not supported by the shader and uniform buffer");

			const int32 NumShadowsToCopy = DirectionalLightShadowInfos.Num();
			int32_t OutShadowIndex = 0;
			for (int32 i = 0; i < NumShadowsToCopy && OutShadowIndex < SceneView.MaxShadowCascades; ++i)
			{
				const FProjectedShadowInfo* ShadowInfo = DirectionalLightShadowInfos[i];

				const FViewInfo* DependentSceneView = IStereoRendering::IsAPrimaryView(SceneView) ? &SceneView : SceneView.GetPrimaryView();
				if (ShadowInfo->ShadowDepthView && !ShadowInfo->bRayTracedDistanceField && ShadowInfo->CacheMode != SDCM_StaticPrimitivesOnly && ShadowInfo->DependentView == DependentSceneView)
				{
					if (OutShadowIndex == 0)
					{
						const FIntPoint ShadowBufferResolution = ShadowInfo->GetShadowBufferResolution();
						const FVector4f ShadowBufferSizeValue((float)ShadowBufferResolution.X, (float)ShadowBufferResolution.Y, 1.0f / (float)ShadowBufferResolution.X, 1.0f / (float)ShadowBufferResolution.Y);

						Params.DirectionalLightShadowTexture = ShadowInfo->RenderTargets.DepthTarget->GetRHI();
						Params.DirectionalLightDirectionAndShadowTransition.W = 1.0f / ShadowInfo->ComputeTransitionSize();
						Params.DirectionalLightShadowSize = ShadowBufferSizeValue;
					}
					Params.DirectionalLightScreenToShadow[OutShadowIndex] = FMatrix44f(ShadowInfo->GetScreenToShadowMatrix(SceneView));		// LWC_TODO: Precision loss?
					Params.DirectionalLightShadowDistances[OutShadowIndex] = ShadowInfo->CascadeSettings.SplitFar;
					Params.DirectionalLightShadowDepthRange[OutShadowIndex] = ShadowInfo->MaxSubjectZ;										// TODO: Depth range is not MaxSubjectZ - MinSubjectZ as on the non-mobile it seems?
					Params.DirectionalLightNumCascades++;
					OutShadowIndex++;
				}
			}
		}

		Params.ShadowedBits = GetShadowedBits(SceneView, *Light);
	}
}

void SetupMobileSkyReflectionUniformParameters(const FScene* Scene, FSkyLightSceneProxy* SkyLight, FMobileReflectionCaptureShaderParameters& Parameters)
{
	Parameters.Texture				= GBlackTextureCube->TextureRHI;
	Parameters.TextureSampler		= GBlackTextureCube->SamplerStateRHI;
	Parameters.TextureBlend			= GBlackTextureCube->TextureRHI;
	Parameters.TextureBlendSampler	= GBlackTextureCube->SamplerStateRHI;
 
	bool bSkyLightIsDynamic = false;
	float Brightness = 0.f;
	float BlendFraction = 0.f;
	bool bIsRealTimeCapture = Scene && Scene->CanSampleSkyLightRealTimeCaptureData();

	if (bIsRealTimeCapture)
	{
		bSkyLightIsDynamic			= SkyLight && !SkyLight->bHasStaticLighting && !SkyLight->bWantsStaticShadowing;
		Parameters.Texture			= Scene->ConvolvedSkyRenderTarget[Scene->ConvolvedSkyRenderTargetReadyIndex]->GetRHI();
		Parameters.TextureSampler	= TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
	else
	{
		if (SkyLight && SkyLight->ProcessedTexture)
		{
			check(SkyLight->ProcessedTexture->IsInitialized());
			Parameters.Texture			= SkyLight->ProcessedTexture->TextureRHI;
			Parameters.TextureSampler	= SkyLight->ProcessedTexture->SamplerStateRHI;
			Brightness					= SkyLight->AverageBrightness;
			bSkyLightIsDynamic			= SkyLight && !SkyLight->bHasStaticLighting && !SkyLight->bWantsStaticShadowing;

			BlendFraction = SkyLight->BlendFraction;
			if (BlendFraction > 0.0 && SkyLight->BlendDestinationProcessedTexture)
			{
				Parameters.TextureBlend			= SkyLight->BlendDestinationProcessedTexture->TextureRHI;
				Parameters.TextureBlendSampler	= SkyLight->BlendDestinationProcessedTexture->SamplerStateRHI;
			}
		}
	}

	float SkyMaxMipIndex = FMath::Log2(static_cast<float>(Parameters.Texture->GetDesc().Extent.X));

	//To keep ImageBasedReflectionLighting coherence with PC, use AverageBrightness instead of InvAverageBrightness to calculate the IBL contribution
	Parameters.Params = FVector4f(Brightness, SkyMaxMipIndex, bIsRealTimeCapture ? 2.0 : (bSkyLightIsDynamic ? 1.0f : 0.0f), BlendFraction);
}

void FMobileSceneRenderer::RenderMobileBasePass(FRHICommandList& RHICmdList, const FViewInfo& View, const FInstanceCullingDrawParams* InstanceCullingDrawParams, const FInstanceCullingDrawParams* SkyPassInstanceCullingDrawParams)
{
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderBasePass);
	SCOPE_CYCLE_COUNTER(STAT_BasePassDrawTime);

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, Basepass, "MobileBasePass");

	RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1);

	if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::BasePass])
	{
		Pass->Draw(RHICmdList, InstanceCullingDrawParams);
	}

	if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::SkyPass]; Pass && View.Family->EngineShowFlags.Atmosphere)
	{
		Pass->Draw(RHICmdList, SkyPassInstanceCullingDrawParams);
	}

	// editor primitives
	FMeshPassProcessorRenderState DrawRenderState;
	DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA>::GetRHI());
	if (bIsFullDepthPrepassEnabled)
	{
		DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthRead_StencilWrite);
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false>::GetRHI());
	}
	else
	{
		DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite);
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true>::GetRHI());
	}
	RenderMobileEditorPrimitives(RHICmdList, View, DrawRenderState, InstanceCullingDrawParams);
}

void FMobileSceneRenderer::RenderMobileEditorPrimitives(FRHICommandList& RHICmdList, const FViewInfo& View, const FMeshPassProcessorRenderState& DrawRenderState, const FInstanceCullingDrawParams* InstanceCullingDrawParams)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EditorDynamicPrimitiveDrawTime);
	SCOPED_DRAW_EVENT(RHICmdList, DynamicEd);

	View.SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_World);
	View.SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_Foreground);

	if (!View.Family->EngineShowFlags.CompositeEditorPrimitives)
	{
		DrawDynamicMeshPass(View, RHICmdList,
			[&View, &DrawRenderState](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FEditorPrimitivesBasePassMeshProcessor PassMeshProcessor(
					View.Family->Scene->GetRenderScene(),
					View.GetFeatureLevel(),
					&View,
					DrawRenderState,
					false,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;
					
				for (int32 MeshIndex = 0; MeshIndex < View.ViewMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatch& MeshBatch = View.ViewMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
				}
			});

		// Draw the view's batched simple elements(lines, sprites, etc).
		View.BatchedViewElements.Draw(RHICmdList, DrawRenderState, FeatureLevel, View, false);

		DrawDynamicMeshPass(View, RHICmdList,
			[&View, &DrawRenderState](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
		{
			FEditorPrimitivesBasePassMeshProcessor PassMeshProcessor(
				View.Family->Scene->GetRenderScene(),
				View.GetFeatureLevel(),
				&View,
				DrawRenderState,
				false,
				DynamicMeshPassContext);

			const uint64 DefaultBatchElementMask = ~0ull;

			for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
			{
				const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
				PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
			}
		});
		
		// DrawDynamicMeshPass may change global InstanceCulling binding, so we need to restore it
		if (UseGPUScene(View.GetShaderPlatform()))
		{
			FRHIUniformBuffer* InstanceCullingBufferRHI = nullptr;
			if (PlatformGPUSceneUsesUniformBufferView(View.GetShaderPlatform()) && InstanceCullingDrawParams->BatchedPrimitive.GetUniformBuffer())
			{
				InstanceCullingBufferRHI = InstanceCullingDrawParams->BatchedPrimitive.GetUniformBuffer()->GetRHI();
			}
			else
			{
				InstanceCullingBufferRHI = InstanceCullingDrawParams->InstanceCulling.GetUniformBuffer()->GetRHI();
			}
			check(InstanceCullingBufferRHI);
			
			FUniformBufferStaticSlot InstanceCullingStaticSlot = FInstanceCullingContext::GetStaticUniformBufferSlot(View.GetShaderPlatform());
			RHICmdList.SetStaticUniformBuffer(InstanceCullingStaticSlot, InstanceCullingBufferRHI);
		}

		// Draw the view's batched simple elements(lines, sprites, etc).
		View.TopBatchedViewElements.Draw(RHICmdList, DrawRenderState, FeatureLevel, View, false);
	}
}

#if UE_ENABLE_DEBUG_DRAWING
void FMobileSceneRenderer::RenderMobileDebugPrimitives(FRHICommandList& RHICmdList, const FViewInfo& View)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_DebugDynamicPrimitiveDrawTime);
	SCOPED_DRAW_EVENT(RHICmdList, DynamicDebug);

	if (View.DebugSimpleElementCollector.HasAnyPrimitives())
	{
		FMeshPassProcessorRenderState DrawRenderState;

		const bool bDepthWrite = !bIsFullDepthPrepassEnabled;
		if (bDepthWrite)
		{
			DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite);
		}
		else
		{
			DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthRead_StencilWrite);
		}
		DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI());
			
		DrawRenderState.SetDepthStencilState(bDepthWrite ? TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI() : TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI());
		View.DebugSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_World);

		DrawRenderState.SetDepthStencilState(bDepthWrite ? TStaticDepthStencilState<true, CF_Always>::GetRHI() : TStaticDepthStencilState<false, CF_Always>::GetRHI());
		View.DebugSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_Foreground);
	}
}
#endif
