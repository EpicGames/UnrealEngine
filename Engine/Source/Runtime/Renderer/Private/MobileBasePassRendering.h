// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MobileBasePassRendering.h: base pass rendering definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "HitProxies.h"
#include "RHIStaticStates.h"
#include "SceneManagement.h"
#include "PrimitiveSceneInfo.h"
#include "PostProcess/SceneRenderTargets.h"
#include "LightMapRendering.h"
#include "MeshMaterialShaderType.h"
#include "MeshMaterialShader.h"
#include "FogRendering.h"
#include "PlanarReflectionRendering.h"
#include "BasePassRendering.h"
#include "SkyAtmosphereRendering.h"
#include "RenderUtils.h"
#include "DebugViewModeRendering.h"
#include "LocalFogVolumeRendering.h"
#include "DBufferTextures.h"
#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "MobileSSR.h"

struct FMobileBasePassTextures
{
	FDBufferTextures DBufferTextures = {};
	FRDGTextureRef ScreenSpaceShadowMaskTexture = nullptr;
	FRDGTextureRef AmbientOcclusionTexture = nullptr;
};

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FMobileBasePassUniformParameters, )
	SHADER_PARAMETER(float, AmbientOcclusionStaticFraction)
	SHADER_PARAMETER_STRUCT(FFogUniformParameters, Fog)
	SHADER_PARAMETER_STRUCT(FFogUniformParameters, FogMMV)
	SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
	SHADER_PARAMETER_STRUCT(FForwardLightUniformParameters, Forward)
	SHADER_PARAMETER_STRUCT(FPlanarReflectionUniformParameters, PlanarReflection) // Single global planar reflection for the forward pass.
	SHADER_PARAMETER_STRUCT(FMobileSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT(FSubstrateMobileForwardPassUniformParameters, Substrate)
	SHADER_PARAMETER_STRUCT(FDebugViewModeUniformParameters, DebugViewMode)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, QuadOverdraw)
	SHADER_PARAMETER_STRUCT(FReflectionUniformParameters, ReflectionsParameters)
	SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGFTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, RWOcclusionBufferUAV)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AmbientOcclusionTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, AmbientOcclusionSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ScreenSpaceShadowMaskTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, ScreenSpaceShadowMaskTextureArray)
	SHADER_PARAMETER_SAMPLER(SamplerState, ScreenSpaceShadowMaskSampler)
	SHADER_PARAMETER(uint32, bApplyHalfResLocalFogToSkyMeshes)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HalfResLocalFogVolumeViewTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, HalfResLocalFogVolumeViewSampler)
	SHADER_PARAMETER_STRUCT_INCLUDE(FDBufferParameters, DBuffer)
	SHADER_PARAMETER_STRUCT(FMobileScreenSpaceReflectionParams, SSRParams)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorCopyTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorCopySampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthCopyTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthCopySampler)
	SHADER_PARAMETER(FVector4f, DistortionParams)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

enum class EMobileBasePass
{
	DepthPrePass,
	Opaque,
	Translucent
};

extern void SetupMobileBasePassUniformParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	EMobileBasePass BasePass,
	EMobileSceneTextureSetupMode SetupMode,
	FMobileBasePassUniformParameters& BasePassParameters, 
	const FMobileBasePassTextures& MobileBasePassTextures = {},
	bool bForRealtimeSkyCapture = false,
	bool bIsRenderingDeferredBasepass = false);

extern TRDGUniformBufferRef<FMobileBasePassUniformParameters> CreateMobileBasePassUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	EMobileBasePass BasePass,
	EMobileSceneTextureSetupMode SetupMode,
	const FMobileBasePassTextures& MobileBasePassTextures = {},
	bool bForRealtimeSkyCapture = false,
	bool bIsRenderingDeferredBasepass = false);

extern void SetupMobileDirectionalLightUniformParameters(
	const FScene& Scene,
	const FViewInfo& View,
	const FVisibleLightInfoArray& VisibleLightInfos,
	int32 ChannelIdx,
	bool bDynamicShadows,
	FMobileDirectionalLightShaderParameters& Parameters);

extern void SetupMobileSkyReflectionUniformParameters(
	const FScene* Scene,
	class FSkyLightSceneProxy* SkyLight,
	FMobileReflectionCaptureShaderParameters& Parameters);



class FPlanarReflectionSceneProxy;
class FScene;

EMobileLocalLightSetting GetMobileForwardLocalLightSetting(EShaderPlatform ShaderPlatform);

enum class EMobileTranslucentColorTransmittanceMode
{
	DEFAULT,
	DUAL_SRC_BLENDING,
	PROGRAMMABLE_BLENDING,
	SINGLE_SRC_BLENDING, // Grey transmittance
};

EMobileTranslucentColorTransmittanceMode MobileDefaultTranslucentColorTransmittanceMode(EShaderPlatform Platform);
EMobileTranslucentColorTransmittanceMode MobileActiveTranslucentColorTransmittanceMode(EShaderPlatform Platform, bool bExplicitDefaultMode);
bool MaterialRequiresColorTransmittanceBlending(const FMaterial& MaterialResource);
bool MaterialRequiresColorTransmittanceBlending(const FMaterialShaderParameters& MaterialParameters);
bool ShouldCacheShaderForColorTransmittanceFallback(const FMaterialShaderPermutationParameters& Parameters, EMobileTranslucentColorTransmittanceMode TranslucentColorTransmittanceFallback);

// shared defines for mobile base pass VS and PS
void MobileBasePassModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

template<typename LightMapPolicyType>
class TMobileBasePassShaderElementData : public FMeshMaterialShaderElementData
{
public:
	TMobileBasePassShaderElementData(const typename LightMapPolicyType::ElementDataType& InLightMapPolicyElementData, bool bInCanReceiveCSM)
		: LightMapPolicyElementData(InLightMapPolicyElementData)
		, bCanReceiveCSM(bInCanReceiveCSM)
	{}

	typename LightMapPolicyType::ElementDataType LightMapPolicyElementData;

	const bool bCanReceiveCSM;
};

/**
 * The base shader type for vertex shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 */

template<typename LightMapPolicyType>
class TMobileBasePassVSPolicyParamType : public FMeshMaterialShader, public LightMapPolicyType::VertexParametersType
{
	DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(TMobileBasePassVSPolicyParamType, NonVirtual, FMeshMaterialShader, typename LightMapPolicyType::VertexParametersType);
protected:

	TMobileBasePassVSPolicyParamType() {}
	TMobileBasePassVSPolicyParamType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer):
		FMeshMaterialShader(Initializer)
	{
		LightMapPolicyType::VertexParametersType::Bind(Initializer.ParameterMap);
		PassUniformBuffer.Bind(Initializer.ParameterMap, FMobileBasePassUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	}

public:

	// static bool ShouldCompilePermutation(EShaderPlatform Platform,const FMaterial* Material,const FVertexFactoryType* VertexFactoryType)

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	void GetShaderBindings(
		const FScene* Scene,
		const FStaticFeatureLevel FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const TMobileBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		LightMapPolicyType::GetVertexShaderBindings(
			PrimitiveSceneProxy,
			ShaderElementData.LightMapPolicyElementData,
			this,
			ShaderBindings);
	}
};

template<typename LightMapPolicyType>
class TMobileBasePassVSBaseType : public TMobileBasePassVSPolicyParamType<LightMapPolicyType>
{
	typedef TMobileBasePassVSPolicyParamType<LightMapPolicyType> Super;
	DECLARE_INLINE_TYPE_LAYOUT(TMobileBasePassVSBaseType, NonVirtual);
protected:

	TMobileBasePassVSBaseType() {}
	TMobileBasePassVSBaseType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer) : Super(Initializer) {}

public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform) && LightMapPolicyType::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		LightMapPolicyType::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

template< typename LightMapPolicyType >
class TMobileBasePassVS : public TMobileBasePassVSBaseType<LightMapPolicyType>
{
	DECLARE_SHADER_TYPE(TMobileBasePassVS,MeshMaterial);
public:
	
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{		
		return TMobileBasePassVSBaseType<LightMapPolicyType>::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		MobileBasePassModifyCompilationEnvironment(Parameters, OutEnvironment);
		TMobileBasePassVSBaseType<LightMapPolicyType>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
	
	/** Initialization constructor. */
	TMobileBasePassVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		TMobileBasePassVSBaseType<LightMapPolicyType>(Initializer)
	{}

	/** Default constructor. */
	TMobileBasePassVS() {}
};

/**
 * The base type for pixel shaders that render the emissive color, and light-mapped/ambient lighting of a mesh.
 */

template<typename LightMapPolicyType>
class TMobileBasePassPSPolicyParamType : public FMeshMaterialShader, public LightMapPolicyType::PixelParametersType
{
	DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(TMobileBasePassPSPolicyParamType, NonVirtual, FMeshMaterialShader, typename LightMapPolicyType::PixelParametersType);
public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		// Modify compilation environment depending upon material shader quality level settings.
		ModifyCompilationEnvironmentForQualityLevel(Parameters.Platform, Parameters.MaterialParameters.QualityLevel, OutEnvironment);
	}

	/** Initialization constructor. */
	TMobileBasePassPSPolicyParamType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		LightMapPolicyType::PixelParametersType::Bind(Initializer.ParameterMap);
		PassUniformBuffer.Bind(Initializer.ParameterMap, FMobileBasePassUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		
		MobileDirectionLightBufferParam.Bind(Initializer.ParameterMap, FMobileDirectionalLightShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		ReflectionParameter.Bind(Initializer.ParameterMap, FMobileReflectionCaptureShaderParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
				
		UseCSMParameter.Bind(Initializer.ParameterMap, TEXT("UseCSM"));
	}

	TMobileBasePassPSPolicyParamType() {}

private:
	LAYOUT_FIELD(FShaderUniformBufferParameter, MobileDirectionLightBufferParam);
	LAYOUT_FIELD(FShaderUniformBufferParameter, ReflectionParameter);
	LAYOUT_FIELD(FShaderParameter, UseCSMParameter);
	
public:
	void GetShaderBindings(
		const FScene* Scene,
		const FStaticFeatureLevel FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const TMobileBasePassShaderElementData<LightMapPolicyType>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const;

private:
	static bool ModifyCompilationEnvironmentForQualityLevel(EShaderPlatform Platform, EMaterialQualityLevel::Type QualityLevel, FShaderCompilerEnvironment& OutEnvironment);
};

template<typename LightMapPolicyType>
class TMobileBasePassPSBaseType : public TMobileBasePassPSPolicyParamType<LightMapPolicyType>
{
	typedef TMobileBasePassPSPolicyParamType<LightMapPolicyType> Super;
	DECLARE_INLINE_TYPE_LAYOUT(TMobileBasePassPSBaseType, NonVirtual);
public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return LightMapPolicyType::ShouldCompilePermutation(Parameters)
			&& Super::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		LightMapPolicyType::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		Super::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	/** Initialization constructor. */
	TMobileBasePassPSBaseType(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer) : Super(Initializer) {}
	TMobileBasePassPSBaseType() {}
};

typedef TArray<ELightMapPolicyType, TInlineAllocator<4>> FMobileLightMapPolicyTypeList;

namespace MobileBasePass
{
	ELightMapPolicyType SelectMeshLightmapPolicy(
		const FScene* Scene, 
		const FMeshBatch& MeshBatch, 
		const FPrimitiveSceneProxy* PrimitiveSceneProxy, 
		bool bPrimReceivesCSM, 
		bool bUsedDeferredShading,
		bool bIsLitMaterial,
		bool bIsTranslucent);

	FMobileLightMapPolicyTypeList GetUniformLightMapPolicyTypeForPSOCollection(bool bLitMaterial, bool bTranslucent/*not used*/, bool bUsesDeferredShading, bool bCanReceiveCSM, bool bMovable);

	template <typename LightMapPolicyType>
	bool GetShaders(
		const LightMapPolicyType& RESTRICT LightMapPolicy,
		EMobileLocalLightSetting LocalLightSetting,
		const FMaterial& MaterialResource,
		const FVertexFactoryType* VertexFactoryType,
		TShaderRef<TMobileBasePassVSPolicyParamType<LightMapPolicyType>>& VertexShader,
		TShaderRef<TMobileBasePassPSPolicyParamType<LightMapPolicyType>>& PixelShader);

	const FLightSceneInfo* GetDirectionalLightInfo(const FScene* Scene, const FPrimitiveSceneProxy* PrimitiveSceneProxy);

	bool StaticCanReceiveCSM(const FLightSceneInfo* LightSceneInfo, const FPrimitiveSceneProxy* PrimitiveSceneProxy);

	void SetOpaqueRenderState(FMeshPassProcessorRenderState& DrawRenderState, const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FMaterial& Material, FMaterialShadingModelField ShadingModels, bool bEnableReceiveDecalOutput, bool bUsesDeferredShading);
	void SetTranslucentRenderState(FMeshPassProcessorRenderState& DrawRenderState, const FMaterial& Material, FMaterialShadingModelField ShadingModels);
};

template< typename LightMapPolicyType, EMobileLocalLightSetting LocalLightSetting, EMobileTranslucentColorTransmittanceMode TranslucentColorTransmittanceFallback = EMobileTranslucentColorTransmittanceMode::DEFAULT>
class TMobileBasePassPS : public TMobileBasePassPSBaseType<LightMapPolicyType>
{
	DECLARE_SHADER_TYPE(TMobileBasePassPS,MeshMaterial);
public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{		
		// We compile the point light shader combinations based on the project settings
		const bool bIsLit = Parameters.MaterialParameters.ShadingModels.IsLit();
		const bool bDeferredShadingEnabled = IsMobileDeferredShadingEnabled(Parameters.Platform);
		const bool bIsTranslucent = IsTranslucentBlendMode(Parameters.MaterialParameters);
		const bool bIsSLW = Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);
		const bool bMaterialUsesForwardShading = bIsLit && (bIsTranslucent || bIsSLW);
		// Translucent materials always support clustered shading on mobile deferred
		const bool bForwardShading = !bDeferredShadingEnabled || bMaterialUsesForwardShading;

		EMobileLocalLightSetting SupportedLocalLightsType = EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED;
		if (bForwardShading && bIsLit)
		{
			SupportedLocalLightsType = GetMobileForwardLocalLightSetting(Parameters.Platform);
		}
		// Deferred shading does not need SkyLight and LocalLight permutations
		// TODO: skip skylight permutations for deferred	
		bool bEnableLocalLights = LocalLightSetting != EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED;
		const bool bShouldCacheByLocalLights = !bEnableLocalLights || (bIsLit && (SupportedLocalLightsType == LocalLightSetting));

		return TMobileBasePassPSBaseType<LightMapPolicyType>::ShouldCompilePermutation(Parameters) && 
				bShouldCacheByLocalLights && 
				ShouldCacheShaderForColorTransmittanceFallback(Parameters, TranslucentColorTransmittanceFallback);
	}
	
	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{		
		MobileBasePassModifyCompilationEnvironment(Parameters, OutEnvironment);

		const bool bMobileUsesShadowMaskTexture = MobileUsesShadowMaskTexture(Parameters.Platform);
		const bool bEnableClusteredReflections = MobileForwardEnableClusteredReflections(Parameters.Platform);
		const bool bDeferredShadingEnabled = IsMobileDeferredShadingEnabled(Parameters.Platform);
		const bool bTranslucentMaterial = IsTranslucentBlendMode(Parameters.MaterialParameters);
		const bool bIsSLW = Parameters.MaterialParameters.ShadingModels.HasShadingModel(MSM_SingleLayerWater);
		const bool bTranslucentOrSLWMaterial = bTranslucentMaterial || bIsSLW;
		const bool bIsLit = Parameters.MaterialParameters.ShadingModels.IsLit();
		const bool bMaterialUsesForwardShading = bIsLit && bTranslucentOrSLWMaterial;
		// Translucent materials always support clustered shading on mobile deferred
		const bool bForwardShading = !bDeferredShadingEnabled || bMaterialUsesForwardShading;
				
		// Only non-static skylights contribute into forward basepass
		const bool bProjectSupportsNonStaticSkyLights = FReadOnlyCVARCache::EnableStationarySkylight() || !IsStaticLightingAllowed();
		OutEnvironment.SetDefine(TEXT("ENABLE_SKY_LIGHT"), bIsLit && bForwardShading && bProjectSupportsNonStaticSkyLights);
		OutEnvironment.SetDefine(TEXT("ENABLE_AMBIENT_OCCLUSION"), bForwardShading && IsMobileAmbientOcclusionEnabled(Parameters.Platform) ? 1u : 0u);
		
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("ENABLE_CLUSTERED_LIGHTS"), (LocalLightSetting == EMobileLocalLightSetting::LOCAL_LIGHTS_ENABLED) ? 1u : 0u);
		OutEnvironment.SetDefine(TEXT("ENABLE_CLUSTERED_REFLECTION"), bEnableClusteredReflections ? 1u : 0u);
		OutEnvironment.SetDefine(TEXT("ENABLE_SHADOWMASKTEXTURE"), bMobileUsesShadowMaskTexture && !bTranslucentOrSLWMaterial ? 1u : 0u);
		OutEnvironment.SetDefine(TEXT("ENABLE_DBUFFER_TEXTURES"), Parameters.MaterialParameters.MaterialDomain == MD_Surface ? 1u : 0u);
		EMobileTranslucentColorTransmittanceMode TranslucentColorTransmittanceMode = EMobileTranslucentColorTransmittanceMode::DEFAULT;
		if (MaterialRequiresColorTransmittanceBlending(Parameters.MaterialParameters))
		{
			TranslucentColorTransmittanceMode = (TranslucentColorTransmittanceFallback == EMobileTranslucentColorTransmittanceMode::DEFAULT) ? MobileDefaultTranslucentColorTransmittanceMode(Parameters.Platform) : TranslucentColorTransmittanceFallback;
		}
		OutEnvironment.SetDefine(TEXT("MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_DUAL_SRC_BLENDING"), TranslucentColorTransmittanceMode == EMobileTranslucentColorTransmittanceMode::DUAL_SRC_BLENDING ? 1u : 0u);
		OutEnvironment.SetDefine(TEXT("MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_PROGRAMMABLE_BLENDING"), TranslucentColorTransmittanceMode == EMobileTranslucentColorTransmittanceMode::PROGRAMMABLE_BLENDING ? 1u : 0u);
		OutEnvironment.SetDefine(TEXT("MOBILE_TRANSLUCENT_COLOR_TRANSMITTANCE_SINGLE_SRC_BLENDING"), TranslucentColorTransmittanceMode == EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING ? 1u : 0u);
		OutEnvironment.SetDefine(TEXT("MOBILE_SSR_ENABLED"), (!bDeferredShadingEnabled && AreMobileScreenSpaceReflectionsEnabled(Parameters.Platform)) ? 1u : 0u);
			
		TMobileBasePassPSBaseType<LightMapPolicyType>::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
	
	/** Initialization constructor. */
	TMobileBasePassPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		TMobileBasePassPSBaseType<LightMapPolicyType>(Initializer)
	{}

	/** Default constructor. */
	TMobileBasePassPS() {}
};

class FMobileBasePassMeshProcessor : public FSceneRenderingAllocatorObject<FMobileBasePassMeshProcessor>, public FMeshPassProcessor
{
public:
	enum class EFlags
	{
		None = 0,

		// Informs the processor whether a depth-stencil target is bound when processed draw commands are issued.
		CanUseDepthStencil = (1 << 0),

		// Informs the processor whether primitives can receive shadows from cascade shadow maps.
		CanReceiveCSM = (1 << 1),

		// Informs the processor to use PassDrawRenderState for all mesh commands
		ForcePassDrawRenderState = (1 << 2),

		// Informs the processor to not cache any mesh commands
		DoNotCache = (1 << 3)
	};

	FMobileBasePassMeshProcessor(
		EMeshPass::Type InMeshPassType,
		const FScene* InScene,
		const FSceneView* InViewIfDynamicMeshCommand,
		const FMeshPassProcessorRenderState& InDrawRenderState,
		FMeshPassDrawListContext* InDrawListContext,
		EFlags Flags,
		ETranslucencyPass::Type InTranslucencyPassType = ETranslucencyPass::TPT_MAX);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;

	FMeshPassProcessorRenderState PassDrawRenderState;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override;

	void CollectPSOInitializersForLMPolicy(
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMeshPassProcessorRenderState& RESTRICT DrawRenderState,
		const FGraphicsPipelineRenderTargetsInfo& RESTRICT RenderTargetsInfo,
		const FMaterial& RESTRICT MaterialResource,
		EMobileLocalLightSetting LocalLightSetting,
		const ELightMapPolicyType LightMapPolicyType,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		EPrimitiveType PrimitiveType,
		TArray<FPSOPrecacheData>& PSOInitializers);

private:
	FMaterialShadingModelField FilterShadingModelsMask(const FMaterialShadingModelField& ShadingModels) const;
	bool ShouldDraw(const class FMaterial& Material) const;

	bool TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy& MaterialRenderProxy, const FMaterial& Material);

	template <typename LightMapPolicyType>
	bool Process(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		const bool bIsMasked,
		FMaterialShadingModelField ShadingModels,
		const LightMapPolicyType& RESTRICT LightMapPolicy,
		const typename LightMapPolicyType::ElementDataType& RESTRICT LightMapElementData,
		const bool bCanReceiveCSM);

	const ETranslucencyPass::Type TranslucencyPassType;
	const EFlags Flags;
	const bool bTranslucentBasePass;
	// Whether renderer uses deferred shading
	const bool bDeferredShading; 
	// Whether this pass uses deferred shading
	const bool bPassUsesDeferredShading; 
	EDepthDrawingMode EarlyZPassMode;
};

ENUM_CLASS_FLAGS(FMobileBasePassMeshProcessor::EFlags);


template <EMobileLocalLightSetting LocalLightSetting, typename LightMapPolicyType>
bool GetMobileBasePassShaders(
	const LightMapPolicyType& RESTRICT LightMapPolicy,
	const FMaterial& Material,
	const FVertexFactoryType* VertexFactoryType,
	EMobileTranslucentColorTransmittanceMode ColoredTransmittanceFallback,
	TShaderRef<TMobileBasePassVSPolicyParamType<LightMapPolicyType>>& VertexShader,
	TShaderRef<TMobileBasePassPSPolicyParamType<LightMapPolicyType>>& PixelShader
)
{
	using FVertexShaderType = TMobileBasePassVSPolicyParamType<LightMapPolicyType>;
	using FPixelShaderType = TMobileBasePassPSPolicyParamType<LightMapPolicyType>;

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<TMobileBasePassVS<LightMapPolicyType>>();

	switch (ColoredTransmittanceFallback)
	{
	default:
	case EMobileTranslucentColorTransmittanceMode::DEFAULT:
		ShaderTypes.AddShaderType<TMobileBasePassPS<LightMapPolicyType, LocalLightSetting, EMobileTranslucentColorTransmittanceMode::DEFAULT>>();
		break;
	case EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING:
		ShaderTypes.AddShaderType<TMobileBasePassPS<LightMapPolicyType, LocalLightSetting, EMobileTranslucentColorTransmittanceMode::SINGLE_SRC_BLENDING>>();
		break;
	}

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}
