// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MobileDecalRendering.cpp: Decals for mobile renderer
=============================================================================*/

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "RHIDefinitions.h"
#include "RHI.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "PostProcess/SceneRenderTargets.h"
#include "SceneRendering.h"
#include "SceneProxies/DeferredDecalProxy.h"
#include "DecalRenderingCommon.h"
#include "DecalRenderingShared.h"
#include "RenderCore.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "CompositionLighting/PostProcessDeferredDecals.h"
#include "DBufferTextures.h"




static bool DoesPlatformSupportDecals(EShaderPlatform ShaderPlatform)
{
	if (!IsMobileHDR())
	{
		// Vulkan uses sub-pass to fetch SceneDepth
		if (IsVulkanPlatform(ShaderPlatform) ||
			IsSimulatedPlatform(ShaderPlatform) ||
			// Some Androids support SceneDepth fetch
			(IsAndroidOpenGLESPlatform(ShaderPlatform) && GSupportsShaderDepthStencilFetch))
		{
			return true;
		}

		// Metal needs DepthAux to fetch depth, and its not availle in LDR mode
		return false;
	}

	// HDR always supports decals
	return true;
}

int32 FMobileSceneRenderer::GetDecalViewOffset(TConstArrayView<FViewInfo> InViews) const
{
	return DecalVisibilityTaskData ? DecalVisibilityTaskData->GetViewOffset(InViews) : INDEX_NONE;
}

TConstArrayView<const FVisibleDecal*> FMobileSceneRenderer::GetRelevantDeferredDecals(int32 DecalViewOffset, int32 ViewIndex, EDecalRenderStage DecalRenderStage)
{
	if (DecalVisibilityTaskData != nullptr && DecalViewOffset != INDEX_NONE)
	{
		return DecalVisibilityTaskData->FinishRelevantDecals(DecalViewOffset, ViewIndex, DecalRenderStage);
	}

	return TConstArrayView<const FVisibleDecal*>();
}

void FMobileSceneRenderer::RenderDecals(FRHICommandList& RHICmdList, FViewInfo& View, const FDecalViewInfo& DecalViewInfo, const FInstanceCullingDrawParams* InstanceCullingDrawParams)
{
	if (!DoesPlatformSupportDecals(View.GetShaderPlatform()) || !View.Family->EngineShowFlags.Decals || View.bIsPlanarReflection)
	{
		return;
	}

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderDecals);
	SCOPE_CYCLE_COUNTER(STAT_DecalsDrawTime);

	// Mesh decals
	EMeshPass::Type DecalMeshPassType = DecalRendering::GetMeshPassType(DecalViewInfo.RenderTargetMode);
	if (HasAnyDraw(View.ParallelMeshDrawCommandPasses[DecalMeshPassType]))
	{
		RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1);
		View.ParallelMeshDrawCommandPasses[DecalMeshPassType]->Draw(RHICmdList, InstanceCullingDrawParams);
	}

	// Deferred decals
	if (DecalViewInfo.RelevantDeferredDecals.Num() > 0)
	{
		SCOPED_DRAW_EVENT(RHICmdList, Decals);
		RenderDeferredDecalsMobile(RHICmdList, View, DecalViewInfo);
	}
}

void FMobileSceneRenderer::RenderDeferredDecalsMobile(FRHICommandList& RHICmdList, const FViewInfo& View, const FDecalViewInfo& DecalViewInfo)
{
	FRelevantDecalList SortedDecals;

	for (const FVisibleDecal* VisibleDecal : DecalViewInfo.RelevantDeferredDecals)
	{
		if (DecalViewInfo.RenderTargetMode == DecalRendering::GetRenderTargetMode(View.GetShaderPlatform(), VisibleDecal->BlendDesc, DecalViewInfo.RenderStage))
		{
			SortedDecals.Add(VisibleDecal);
		}
	}
	INC_DWORD_STAT_BY(STAT_Decals, SortedDecals.Num());
	

	if (SortedDecals.Num() > 0)
	{
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1);
		RHICmdList.SetStreamSource(0, GetUnitCubeVertexBuffer(), 0);

		for (int32 DecalIndex = 0; DecalIndex < SortedDecals.Num(); DecalIndex++)
		{
			const FVisibleDecal& VisibleDecal = *SortedDecals[DecalIndex];
			const FMatrix ComponentToWorldMatrix = VisibleDecal.ComponentTrans.ToMatrixWithScale();
			const FMatrix FrustumComponentToClip = DecalRendering::ComputeComponentToClipMatrix(View, ComponentToWorldMatrix);

			const float ConservativeRadius = VisibleDecal.ConservativeRadius;
			const bool bInsideDecal = ((FVector)View.ViewMatrices.GetViewOrigin() - ComponentToWorldMatrix.GetOrigin()).SizeSquared() < FMath::Square(ConservativeRadius * 1.05f + View.NearClippingDistance * 2.0f);
			bool bReverseHanded = false;
			{
				// Account for the reversal of handedness caused by negative scale on the decal
				const auto& Scale3d = VisibleDecal.ComponentTrans.GetScale3D();
				bReverseHanded = Scale3d[0] * Scale3d[1] * Scale3d[2] < 0.f;
			}
			EDecalRasterizerState DecalRasterizerState = DecalRendering::GetDecalRasterizerState(bInsideDecal, bReverseHanded, View.bReverseCulling);
			GraphicsPSOInit.RasterizerState = DecalRendering::GetDecalRasterizerState(DecalRasterizerState);

			constexpr uint32 StencilRef = GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1);

			if (bInsideDecal)
			{
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
					false, CF_Always,
					true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
					false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
					StencilRef, 0x00>::GetRHI();
			}
			else
			{
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
					false, CF_DepthNearOrEqual,
					true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
					false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
					StencilRef, 0x00>::GetRHI();
			}

			GraphicsPSOInit.BlendState = DecalRendering::GetDecalBlendState(VisibleDecal.BlendDesc, DecalViewInfo.RenderStage, DecalViewInfo.RenderTargetMode);

			// Set shader params
			DecalRendering::SetShader(RHICmdList, GraphicsPSOInit, StencilRef, View, VisibleDecal, DecalViewInfo.RenderStage, FrustumComponentToClip, Scene);

			RHICmdList.DrawIndexedPrimitive(GetUnitCubeIndexBuffer(), 0, 0, 8, 0, UE_ARRAY_COUNT(GCubeIndices) / 3, View.GetStereoPassInstanceFactor());
		}
	}
}

void FMobileSceneRenderer::RenderDBuffer(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& InViews, FSceneTextures& SceneTextures, FDBufferTextures& DBufferTextures, FInstanceCullingManager& InstanceCullingManager)
{
	RDG_EVENT_SCOPE(GraphBuilder, "RenderDBuffer");
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RenderDBuffer);

	const EShaderPlatform Platform = GetViewFamilyInfo(InViews).GetShaderPlatform();

	int32 DecalViewOffset = GetDecalViewOffset(InViews);

	if (DecalViewOffset != INDEX_NONE)
	{
		for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
		{
			FViewInfo& View = InViews[ViewIndex];

			if (!View.ShouldRenderView())
			{
				continue;
			}

			TConstArrayView<const FVisibleDecal*> SortedDecals = DecalVisibilityTaskData->FinishRelevantDecals(DecalViewOffset, ViewIndex, EDecalRenderStage::BeforeBasePass);
			FDeferredDecalPassTextures DecalPassTextures = GetDeferredDecalPassTextures(GraphBuilder, View, SceneTextures, &DBufferTextures, EDecalRenderStage::BeforeBasePass);
			AddDeferredDecalPass(GraphBuilder, View, SortedDecals, DecalPassTextures, InstanceCullingManager, EDecalRenderStage::BeforeBasePass);
		}
	}
}