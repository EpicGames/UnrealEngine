// Copyright Epic Games, Inc. All Rights Reserved.

#include "MobileDeferredShadingPass.h"
#include "BasePassRendering.h"
#include "PSOPrecacheValidation.h"
#include "SceneView.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PipelineStateCache.h"
#include "PlanarReflectionRendering.h"
#include "LightFunctionRendering.h"
#include "LightRendering.h"
#include "LocalLightSceneProxy.h"
#include "Materials/MaterialRenderProxy.h"
#include "MobileSSR.h"
#include "DistanceFieldLightingShared.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "ClearQuad.h"

TAutoConsoleVariable<bool> CVarMobileDiffuseFromCapture(
	TEXT("r.Mobile.DiffuseFromCapture"),
	false,
	TEXT("Diffuse lighting comes from reflection captures on mobile, replacing sky lighting where present.  Requires\n")
	TEXT("r.ReflectionCapture.Runtime = 1.  Not modifiable at runtime, as it affects shader compilation.  Default = false."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

// Only runtime captures fill the low-mip diffuse irradiance that the FDiffuseFromCapture permutation samples,
// so that permutation's selection is gated on this being non-zero too.
static bool IsReflectionCaptureRuntimeModeEnabled()
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ReflectionCapture.Runtime"));
	return CVar && CVar->GetValueOnAnyThread() != 0;
}

static TAutoConsoleVariable<bool> CVarReflectionCaptureRuntimeSkyLightPixels(
	TEXT("r.ReflectionCapture.Runtime.SkyLightPixels"),
	true,
	TEXT("Whether the sky light cube map should be rendered to sky pixels.  Produces lighting similar to the existing sky\n")
	TEXT("light implementation, in areas where sky is visible.  Translucent sky may overwrite these sky light pixels\n")
	TEXT("if r.ReflectionCapture.Runtime.Translucency is enabled, which can be counteracted by setting a far clip plane\n")
	TEXT("that excludes translucent sky via Max View Distance and Finite Far Plane properties on the reflection capture.\n")
);

DECLARE_GPU_STAT(DeferredShading);

static const TCHAR* DeferredMobileLightMaterialPSOCollectorName = TEXT("DeferredMobileLightMaterialPSOCollector");

static FAutoConsoleVariableDeprecated CVarMobileUseClusteredDeferredShadingDep(TEXT("r.Mobile.UseClusteredDeferredShading"), TEXT("r.Mobile.UseClusteredDeferredShading_ToBeRemoved"), TEXT("5.7"));

int32 GMobileUseClusteredDeferredShading = 0;
static FAutoConsoleVariableRef CVarMobileUseClusteredDeferredShading(
	TEXT("r.Mobile.UseClusteredDeferredShading_ToBeRemoved"),
	GMobileUseClusteredDeferredShading,
	TEXT("NOTE: The mobile clustered deferred shading implementation will be removed in a future release due to low utility and use.\n")
	TEXT("Toggle use of clustered deferred shading for lights that support it. 0 is off (default), 1 is on. (requires LightGrid: r.Mobile.Forward.EnableLocalLights=1)"),
	ECVF_RenderThreadSafe
);

int32 GMobileEnableClusteredDeferredShadingPermutations = 0;
static FAutoConsoleVariableRef CVarMobileEnableClusteredDeferredShadingPermutations(
	TEXT("r.Mobile.EnableClusteredDeferredShadingPermutations"),
	GMobileEnableClusteredDeferredShadingPermutations,
	TEXT("Enable permutations for clustered shading and compile and cook them."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

static bool UseClusteredDeferredShading(const FStaticShaderPlatform Platform)
{
	// Needs LightGrid to function
	return GMobileUseClusteredDeferredShading != 0 && GMobileEnableClusteredDeferredShadingPermutations != 0 && MobileForwardEnableLocalLights(Platform);
}

static bool MobileDeferredEnableAmbientOcclusion(const FStaticShaderPlatform Platform)
{
	// AO requires a full depth before shading
	return MobileUsesFullDepthPrepass(Platform) || !MobileAllowFramebufferFetch(Platform);
}

int32 GMobileUseLightStencilCulling = 0;
static FAutoConsoleVariableRef CVarMobileUseLightStencilCulling(
	TEXT("r.Mobile.UseLightStencilCulling"),
	GMobileUseLightStencilCulling,
	TEXT("Whether to use stencil to cull local lights. 0 is off (default), 1 is on"),
	ECVF_RenderThreadSafe
);

int32 GMobileIgnoreDeferredShadingSkyLightChannels = 0;
static FAutoConsoleVariableRef CVarMobileIgnoreDeferredShadingSkyLightChannels(
	TEXT("r.Mobile.IgnoreDeferredShadingSkyLightChannels"),
	GMobileIgnoreDeferredShadingSkyLightChannels,
	TEXT("Whether to ignore primitive lighting channels when applying SkyLighting in a mobile deferred shading.\n" 
		 "This may improve GPU performance at the cost of incorrect lighting for a primitves with non-default lighting channels"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMobileAllowCapsuleLights(
	TEXT("r.Mobile.AllowCapsuleLights"),
	1,
	TEXT("Allow capsule lights in a mobile deferred shading (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

BEGIN_SHADER_PARAMETER_STRUCT(FMobileDeferredPassParameters, )
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileSceneTextureUniformParameters, MobileSceneTextures)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMobileDeferredCommonParameters, )
	SHADER_PARAMETER(FMatrix44f, TranslatedWorldToLight)
	SHADER_PARAMETER(FVector4f, LightFunctionParameters)
	SHADER_PARAMETER(FVector2f, LightFunctionParameters2)
	SHADER_PARAMETER(FVector3f, CameraRelativeLightPosition)
END_SHADER_PARAMETER_STRUCT()

class FMobileDirectionalLightFunctionPS : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FMobileDirectionalLightFunctionPS, Material);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileDirectionalLightFunctionPS, FMaterialShader)

	class FEnableShadingModelSupport	: SHADER_PERMUTATION_BOOL("ENABLE_SHADINGMODEL_SUPPORT_MOBILE_DEFERRED");
	class FEnableClustredLights			: SHADER_PERMUTATION_BOOL("ENABLE_CLUSTERED_LIGHTS");
	class FEnableSkyLight				: SHADER_PERMUTATION_BOOL("ENABLE_SKY_LIGHT");
	class FEnableScreenSpaceShadowMask  : SHADER_PERMUTATION_BOOL("ENABLE_SHADOWMASKTEXTURE");
	class FEnableCSM					: SHADER_PERMUTATION_BOOL("ENABLE_MOBILE_CSM");
	class FShadowQuality				: SHADER_PERMUTATION_RANGE_INT("MOBILE_SHADOW_QUALITY", 1, 3); // not using Quality=0
	class FSkyShadowing					: SHADER_PERMUTATION_BOOL("APPLY_SKY_SHADOWING");

	using FPermutationDomain = TShaderPermutationDomain<
		FEnableShadingModelSupport,
		FEnableClustredLights, 
		FEnableSkyLight,
		FEnableScreenSpaceShadowMask,
		FEnableCSM, 
		FShadowQuality,
		FSkyShadowing>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FMobileDirectionalLightShaderParameters, MobileDirectionalLight)
		SHADER_PARAMETER_STRUCT_INCLUDE(FMobileDeferredCommonParameters, MobileDeferredCommonParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDFAOUpsampleParameters, DFAOUpsampleParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSkyDiffuseLightingParameters, SkyDiffuseLighting)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_LIGHT_FUNCTION"), Parameters.MaterialParameters.bIsDefaultMaterial ? 0 : 1);
		OutEnvironment.SetDefine(TEXT("MATERIAL_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEFERREDSHADING_SUBPASS"), 1u);

		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), bMobileForceDepthRead ? 1u : 0u);
		OutEnvironment.SetDefine(TEXT("ENABLE_AMBIENT_OCCLUSION"), MobileDeferredEnableAmbientOcclusion(Parameters.Platform) ? 1u : 0u);
	}

	static FPermutationDomain RemapPermutationVector(FPermutationDomain PermutationVector, EShaderPlatform Platform)
	{
		if (MobileUsesShadowMaskTexture(Platform) || PermutationVector.Get<FEnableScreenSpaceShadowMask>() == true)
		{
			PermutationVector.Set<FEnableCSM>(false);
		}

		if (PermutationVector.Get<FEnableCSM>() == false)
		{
			PermutationVector.Set<FShadowQuality>(1);
		}

		if (!MobileUsesGBufferCustomData(Platform))
		{
			PermutationVector.Set<FEnableShadingModelSupport>(false);
		}

		if (!IsMobileDistanceFieldAOEnabled(Platform))
		{
			PermutationVector.Set<FSkyShadowing>(false);
		}

		if (GMobileEnableClusteredDeferredShadingPermutations == 0)
		{
			PermutationVector.Set<FEnableClustredLights>(false);
		}
		
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		if (Parameters.MaterialParameters.MaterialDomain != MD_LightFunction || 
			!IsMobilePlatform(Parameters.Platform) || 
			!IsMobileDeferredShadingEnabled(Parameters.Platform))
		{
			return false;
		}
		
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		// Compile out the shader if this permutation gets remapped.
		if (RemapPermutationVector(PermutationVector, Parameters.Platform) != PermutationVector)
		{
			return false;
		}
		
		return true;
	}

	static FPermutationDomain BuildPermutationVector(
		const FViewInfo& View, 
		bool bInlineReflectionAndSky, 
		bool bShadingModelSupport, 
		bool bDynamicShadows, 
		bool bSkyLight, 
		bool bScreenSpaceShadowMask,
		bool bApplySkyShadowing)
	{
		EShaderPlatform ShaderPlatform = View.GetShaderPlatform();
		bool bUseClusteredLights = UseClusteredDeferredShading(ShaderPlatform);
		bool bEnableSkyLight = bInlineReflectionAndSky && bSkyLight;
		int32 ShadowQuality = bDynamicShadows && !bScreenSpaceShadowMask ? (int32)GetShadowQuality() : 0;

		FPermutationDomain PermutationVector;
		PermutationVector.Set<FEnableShadingModelSupport>(bShadingModelSupport);
		PermutationVector.Set<FEnableClustredLights>(bUseClusteredLights);
		PermutationVector.Set<FEnableSkyLight>(bEnableSkyLight);
		PermutationVector.Set<FEnableScreenSpaceShadowMask>(bScreenSpaceShadowMask);
		PermutationVector.Set<FEnableCSM>(ShadowQuality > 0);
		PermutationVector.Set<FShadowQuality>(FMath::Clamp(ShadowQuality, 1, 3));
		PermutationVector.Set<FSkyShadowing>(bInlineReflectionAndSky && bApplySkyShadowing);
		return PermutationVector;
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FViewInfo& View, const FMaterialRenderProxy* Proxy, const FMaterial& Material)
	{
		FMaterialShader::SetParameters(BatchedParameters, Proxy, Material, View);

		// LightFunctions can use primitive data, set identity so we do not crash on a missing binding
		auto& PrimitivePS = GetUniformBufferParameter<FPrimitiveUniformShaderParameters>();
		SetUniformBufferParameter(BatchedParameters, PrimitivePS, GIdentityPrimitiveUniformBuffer);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FMobileDirectionalLightFunctionPS, TEXT("/Engine/Private/MobileDeferredShading.usf"), TEXT("MobileDirectionalLightPS"), SF_Pixel);

/**
 * A pixel shader for projecting a light function onto the scene.
 */
class FMobileRadialLightFunctionPS : public FMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FMobileRadialLightFunctionPS,Material);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileRadialLightFunctionPS, FMaterialShader)

	class FEnableShadingModelSupport		: SHADER_PERMUTATION_BOOL("ENABLE_SHADINGMODEL_SUPPORT_MOBILE_DEFERRED");
	class FRadialLightTypeDim				: SHADER_PERMUTATION_RANGE_INT("RADIAL_LIGHT_TYPE", LIGHT_TYPE_POINT, LIGHT_TYPE_RECT);
	class FIESProfileDim					: SHADER_PERMUTATION_BOOL("USE_IES_PROFILE");
	class FSupportRadialLightShadowDim		: SHADER_PERMUTATION_BOOL("SUPPORT_RADIAL_LIGHT_SHADOW");
	class FCapsuleLightDim					: SHADER_PERMUTATION_BOOL("MOBILE_SHADING_PATH_SUPPORT_CAPSULE_LIGHT");
	class FSimpleLightDim					: SHADER_PERMUTATION_BOOL("SIMPLE_LIGHT");
	using FPermutationDomain = TShaderPermutationDomain<FEnableShadingModelSupport, FRadialLightTypeDim, FIESProfileDim, FSupportRadialLightShadowDim, FCapsuleLightDim, FSimpleLightDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FLightShaderParameters, Light)
		SHADER_PARAMETER_STRUCT_INCLUDE(FMobileMovableLocalLightShadowParameters, MobileMovableLocalLightShadow)
		SHADER_PARAMETER_STRUCT_INCLUDE(FMobileDeferredCommonParameters, MobileDeferredCommonParameters)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		if (Parameters.MaterialParameters.MaterialDomain != MD_LightFunction || 
			!IsMobilePlatform(Parameters.Platform) || 
			!IsMobileDeferredShadingEnabled(Parameters.Platform))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		
		if (PermutationVector.Get<FSimpleLightDim>() == true && !Parameters.MaterialParameters.bIsDefaultMaterial)
		{
			return false;
		}
				
		// Compile out the shader if this permutation gets remapped.
		if (RemapPermutationVector(PermutationVector, Parameters.Platform) != PermutationVector)
		{
			return false;
		}

		return true;
	}

	static FPermutationDomain RemapPermutationVector(FPermutationDomain PermutationVector, EShaderPlatform Platform)
	{
		if (PermutationVector.Get<FSupportRadialLightShadowDim>() &&
			((IsMobileMovableSpotlightShadowsEnabled(Platform) && PermutationVector.Get<FRadialLightTypeDim>() == LIGHT_TYPE_SPOT) ||
			(MobileUsesShadowMaskTexture(Platform) && FReadOnlyCVARCache::EnablePointLightShadows(Platform) && PermutationVector.Get<FRadialLightTypeDim>() == LIGHT_TYPE_POINT))
			)
		{
			PermutationVector.Set<FSupportRadialLightShadowDim>(true);
		}
		else
		{
			PermutationVector.Set<FSupportRadialLightShadowDim>(false);
		}

		if (!MobileUsesGBufferCustomData(Platform))
		{
			PermutationVector.Set<FEnableShadingModelSupport>(false);
		}

		if (PermutationVector.Get<FSimpleLightDim>() == true)
		{
			PermutationVector.Set<FEnableShadingModelSupport>(false);
			PermutationVector.Set<FRadialLightTypeDim>(LIGHT_TYPE_POINT);
			PermutationVector.Set<FIESProfileDim>(false);
			PermutationVector.Set<FSupportRadialLightShadowDim>(false);
			PermutationVector.Set<FCapsuleLightDim>(false);
		}

		if (PermutationVector.Get<FRadialLightTypeDim>() == LIGHT_TYPE_RECT)
		{
			PermutationVector.Set<FCapsuleLightDim>(false);
		}

		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_LIGHT_FUNCTION"), Parameters.MaterialParameters.bIsDefaultMaterial ? 0 : 1);
		OutEnvironment.SetDefine(TEXT("MATERIAL_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("ENABLE_CLUSTERED_LIGHTS"), 0);
		OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEFERREDSHADING_SUBPASS"), 1u);

		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), bMobileForceDepthRead ? 1u : 0u);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		bool bPointLightCastShadow = (PermutationVector.Get<FSupportRadialLightShadowDim>() && PermutationVector.Get<FRadialLightTypeDim>() == LIGHT_TYPE_POINT);
		OutEnvironment.SetDefine(TEXT("ENABLE_SHADOWMASKTEXTURE"), bPointLightCastShadow ? 1 : 0);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FViewInfo& View, const FMaterialRenderProxy* Proxy, const FMaterial& Material)
	{
		FMaterialShader::SetParameters(BatchedParameters, Proxy, Material, View);
		
		// LightFunctions can use primitive data, set identity so we do not crash on a missing binding
		auto& PrimitivePS = GetUniformBufferParameter<FPrimitiveUniformShaderParameters>();
		SetUniformBufferParameter(BatchedParameters, PrimitivePS, GIdentityPrimitiveUniformBuffer);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(,FMobileRadialLightFunctionPS, TEXT("/Engine/Private/MobileDeferredShading.usf"), TEXT("MobileRadialLightPS"), SF_Pixel);


/**
 * A pixel shader for reflection env and sky lighting. 
 */
class FMobileReflectionEnvironmentSkyLightingPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileReflectionEnvironmentSkyLightingPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileReflectionEnvironmentSkyLightingPS, FGlobalShader);
	
	class FEnableShadingModelSupport	: SHADER_PERMUTATION_BOOL("ENABLE_SHADINGMODEL_SUPPORT_MOBILE_DEFERRED");
	class FEnableClustredReflection		: SHADER_PERMUTATION_BOOL("ENABLE_CLUSTERED_REFLECTION");
	class FEnablePlanarReflection		: SHADER_PERMUTATION_BOOL("ENABLE_PLANAR_REFLECTION");
	class FEnableSkyLight				: SHADER_PERMUTATION_BOOL("ENABLE_SKY_LIGHT");
	class FMobileSSRQuality				: SHADER_PERMUTATION_ENUM_CLASS("MOBILE_SSR_QUALITY", EMobileSSRQuality);
	class FSkyShadowing					: SHADER_PERMUTATION_BOOL("APPLY_SKY_SHADOWING");
	class FDiffuseFromCapture			: SHADER_PERMUTATION_BOOL("DIFFUSE_FROM_CAPTURE");

	using FPermutationDomain = TShaderPermutationDomain<
		FEnableShadingModelSupport, 
		FEnableClustredReflection, 
		FEnablePlanarReflection,
		FEnableSkyLight,
		FMobileSSRQuality,
		FSkyShadowing,
		FDiffuseFromCapture
	>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_STRUCT_REF(FMobileReflectionCaptureShaderData, ReflectionCapture)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDFAOUpsampleParameters, DFAOUpsampleParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSkyDiffuseLightingParameters, SkyDiffuseLighting)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsMobilePlatform(Parameters.Platform) ||
			!IsMobileDeferredShadingEnabled(Parameters.Platform))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (!MobileUsesGBufferCustomData(Parameters.Platform) && PermutationVector.Get<FEnableShadingModelSupport>())
		{
			return false;
		}

		if (PermutationVector.Get<FSkyShadowing>() && !IsMobileDistanceFieldAOEnabled(Parameters.Platform))
		{
			return false;
		}

		if (PermutationVector.Get<FDiffuseFromCapture>() && !CVarMobileDiffuseFromCapture.GetValueOnAnyThread())
		{
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FForwardLightingParameters::ModifyCompilationEnvironment(Parameters.Platform, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEFERREDSHADING_SUBPASS"), 1u);

		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("FORCE_DEPTH_TEXTURE_READS"), bMobileForceDepthRead ? 1u : 0u);
		OutEnvironment.SetDefine(TEXT("ENABLE_AMBIENT_OCCLUSION"), MobileDeferredEnableAmbientOcclusion(Parameters.Platform) ? 1u : 0u);
		OutEnvironment.SetDefine(TEXT("MOBILE_SSR_ENABLED"), PermutationVector.Get<FMobileSSRQuality>() != EMobileSSRQuality::Disabled ? 1u : 0u);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileReflectionEnvironmentSkyLightingPS, "/Engine/Private/MobileDeferredShading.usf", "MobileReflectionEnvironmentSkyLightingPS", SF_Pixel);

class FMobileSkyLightToSkyPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileSkyLightToSkyPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileSkyLightToSkyPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER(FLinearColor, SkylightUnscaledColor)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileSkyLightToSkyPS, "/Engine/Private/MobileDeferredShading.usf", "MobileSkyLightToSkyPS", SF_Pixel);


constexpr uint32 GetLightingChannel(uint32 LightingChannelMask)
{
	return (LightingChannelMask & 0x1) ? 0u : ((LightingChannelMask & 0x2) ? 1u : 2u);
}

constexpr uint8 GetLightingChannelStencilValue(uint32 LightingChannel)
{
	// LightingChannel_0 has an inverted bit in the stencil. 0 - means LightingChannel_0 is enabled. See FPrimitiveSceneProxy::GetLightingChannelStencilValue()
	return (LightingChannel == 0u ? 0u : (1u << LightingChannel));
}

constexpr bool IsOnlyDefaultLitShadingModel(uint32 ShadingModelMask)
{
	constexpr uint32 LitOpaqueMask = ~(1u << MSM_Unlit | 1u << MSM_SingleLayerWater | 1u << MSM_ThinTranslucent);
	constexpr uint32 DefaultLitMask = (1u << MSM_DefaultLit);
	return (ShadingModelMask & LitOpaqueMask) == DefaultLitMask;
}

struct FCachedLightMaterial
{
	const FMaterial* Material;
	const FMaterialRenderProxy* MaterialProxy;
};

template<class ShaderType>
static void GetLightMaterial(const FCachedLightMaterial& DefaultLightMaterial, const FMaterialRenderProxy* MaterialProxy, int32 PermutationId, FCachedLightMaterial& OutLightMaterial, TShaderRef<ShaderType>& OutShader)
{
	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<ShaderType>(PermutationId);
	FMaterialShaders Shaders;

	if (MaterialProxy)
	{
		const FMaterial* Material = MaterialProxy->GetMaterialNoFallback(ERHIFeatureLevel::ES3_1);
		if (Material && Material->IsLightFunction())
		{
			OutLightMaterial.Material = Material;
			OutLightMaterial.MaterialProxy = MaterialProxy;
			if (Material->TryGetShaders(ShaderTypes, nullptr, Shaders))
			{
				Shaders.TryGetPixelShader(OutShader);
				return;
			}
		}
	}

	// use default material
	OutLightMaterial.Material = DefaultLightMaterial.Material;
	OutLightMaterial.MaterialProxy = DefaultLightMaterial.MaterialProxy;

	// Perform a TryGetShaders to allow ODSC to record a shader recompile request when enabled
	if (DefaultLightMaterial.Material->TryGetShaders(ShaderTypes, nullptr, Shaders))
	{
		Shaders.TryGetPixelShader(OutShader);
		return;
	}

	const FMaterialShaderMap* MaterialShaderMap = OutLightMaterial.Material->GetRenderingThreadShaderMap();
	OutShader = MaterialShaderMap->GetShader<ShaderType>(PermutationId);
}

uint8 PassShadingModelStencilValue(bool bEnableShadingModelSupport)
{
	return bEnableShadingModelSupport ? GET_STENCIL_BIT_MASK(MOBILE_SHADINGMODELS, 1) : GET_STENCIL_BIT_MASK(MOBILE_DEFAULTLIT, 1);
}

static void RenderSkyLightToSky(
	FRHICommandList& RHICmdList,
	const FScene& Scene,
	const FViewInfo& View)
{
	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
		false, CF_Always,
		true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		GET_STENCIL_MOBILE_SM_MASK(0xff), 0x00>::GetRHI();

	TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FMobileSkyLightToSkyPS> PixelShader(View.ShaderMap);

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	FMobileSkyLightToSkyPS::FParameters PassParameters;
	PassParameters.View = GetShaderBinding(View.ViewUniformBuffer);

	float SkyPreExposureInv = 1.0f;
	if (Scene.SkyLight->bRealTimeCaptureEnabled)
	{
		TConsoleVariableData<float>* PreExposureCVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.SkyLight.RealTimeReflectionCapture.PreExposure"));
		if (PreExposureCVar)
		{
			SkyPreExposureInv = FMath::Exp2(PreExposureCVar->GetValueOnRenderThread());
		}
	}
	PassParameters.SkylightUnscaledColor = Scene.SkyLight->GetEffectiveLightColor() * SkyPreExposureInv;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters);

	const FIntPoint TargetSize = View.GetSceneTexturesConfig().Extent;

	DrawRectangle(
		RHICmdList,
		0, 0,
		View.ViewRect.Width(), View.ViewRect.Height(),
		View.ViewRect.Min.X, View.ViewRect.Min.Y,
		View.ViewRect.Width(), View.ViewRect.Height(),
		FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()),
		TargetSize,
		VertexShader);
}

void RenderReflectionEnvironmentSkyLighting(
	FRHICommandList& RHICmdList, 
	const FScene& Scene, 
	const FViewInfo& View, 
	const EMobileSSRQuality MobileSSRQuality,
	FRDGTextureRef DynamicBentNormalAOTexture)
{
	// Skylights with static lighting already had their diffuse contribution baked into lightmaps
	const bool bDynamicSkyLight = Scene.SkyLight && (!Scene.SkyLight->bHasStaticLighting || !IsStaticLightingAllowed());
	const bool bEnableSkyLight = bDynamicSkyLight && View.Family->EngineShowFlags.SkyLighting;
	const bool bClustredReflection = (View.NumBoxReflectionCaptures + View.NumSphereReflectionCaptures) > 0;
	const bool bPlanarReflection = Scene.GetForwardPassGlobalPlanarReflection() != nullptr;
	if (!(bEnableSkyLight || bClustredReflection || bPlanarReflection || MobileSSRQuality != EMobileSSRQuality::Disabled))
	{
		return;
	}

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, ReflectionEnvironment, "ReflectionEnvironment");
	SCOPED_DRAW_EVENT(RHICmdList, ReflectionEnvironmentSkyLighting);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	// Add to emissive in SceneColor
	if (!bDynamicSkyLight)
	{
		// pre-multiply SceneColor with AO. Only need it for a static skylights
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha>::GetRHI();
	}
	else
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
	}
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

	FMobileReflectionEnvironmentSkyLightingPS::FParameters PassParameters;
	PassParameters.View = View.GetShaderParameters();
	PassParameters.ReflectionCapture = View.MobileReflectionCaptureUniformBuffer;

	// DFAO
	if (DynamicBentNormalAOTexture != nullptr)
	{
		PassParameters.DFAOUpsampleParameters = DistanceField::SetupAOUpsampleParameters(View, DynamicBentNormalAOTexture);
		float DynamicBentNormalAO = 1.0f;
		PassParameters.SkyDiffuseLighting = GetSkyDiffuseLightingParameters(Scene.SkyLight, DynamicBentNormalAO);
	}

	TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);

	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
		false, CF_Always,
		true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		GET_STENCIL_MOBILE_SM_MASK(0xff), 0x00>::GetRHI();
	uint8 StencilRef = PassShadingModelStencilValue(/*bEnableShadingModelSupport=*/false);

	bool bEnableShadingModelSupport = false;
	if (!IsOnlyDefaultLitShadingModel(View.ShadingModelMaskInView) && MobileUsesGBufferCustomData(Scene.GetShaderPlatform()))
	{
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
			false, CF_Always,
			true, CF_NotEqual, SO_Keep, SO_Keep, SO_Keep,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			GET_STENCIL_MOBILE_SM_MASK(0xff), 0x00>::GetRHI();
		// Apply all shading models
		StencilRef = 0;
		bEnableShadingModelSupport = true;
	}

	FMobileReflectionEnvironmentSkyLightingPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FMobileReflectionEnvironmentSkyLightingPS::FEnableShadingModelSupport>(bEnableShadingModelSupport);
	PermutationVector.Set<FMobileReflectionEnvironmentSkyLightingPS::FEnableClustredReflection>(bClustredReflection);
	PermutationVector.Set<FMobileReflectionEnvironmentSkyLightingPS::FEnablePlanarReflection>(bPlanarReflection);
	PermutationVector.Set<FMobileReflectionEnvironmentSkyLightingPS::FEnableSkyLight>(bEnableSkyLight);
	PermutationVector.Set<FMobileReflectionEnvironmentSkyLightingPS::FMobileSSRQuality>(MobileSSRQuality);
	extern bool UseDistanceFieldAO();
	PermutationVector.Set<FMobileReflectionEnvironmentSkyLightingPS::FSkyShadowing>(DynamicBentNormalAOTexture && UseDistanceFieldAO() && IsMobileDistanceFieldAOEnabled(View.GetShaderPlatform()));

	// Diffuse irradiance is only generated in the cubemap's low mips when runtime captures are active.  With baked captures those mips aren't
	// populated, so fall back to the no-diffuse path whenever r.ReflectionCapture.Runtime is 0, even if r.Mobile.DiffuseFromCapture is on.
	const bool bDiffuseFromCapture = CVarMobileDiffuseFromCapture.GetValueOnRenderThread() && IsReflectionCaptureRuntimeModeEnabled();
	PermutationVector.Set<FMobileReflectionEnvironmentSkyLightingPS::FDiffuseFromCapture>(bDiffuseFromCapture);

	TShaderMapRef<FMobileReflectionEnvironmentSkyLightingPS> PixelShader(View.ShaderMap, PermutationVector);
		
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters);

	const FIntPoint TargetSize = View.GetSceneTexturesConfig().Extent;

	DrawRectangle(
		RHICmdList,
		0, 0,
		View.ViewRect.Width(), View.ViewRect.Height(),
		View.ViewRect.Min.X, View.ViewRect.Min.Y,
		View.ViewRect.Width(), View.ViewRect.Height(),
		FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()),
		TargetSize,
		VertexShader);
}

template<uint32 LightingChannelIdx>
static void SetDirectionalLightDepthStencilState(FGraphicsPipelineStateInitializer& GraphicsPSOInit)
{
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
		false, CF_Always,
		true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		GET_STENCIL_MOBILE_SM_MASK(0xff) | STENCIL_LIGHTING_CHANNELS_MASK(1u << LightingChannelIdx), 0x00>::GetRHI();
}

static void SetDirectionalLightDepthStencilState(FGraphicsPipelineStateInitializer& GraphicsPSOInit, uint32 LightingChannelIdx)
{
	switch (LightingChannelIdx)
	{
	default:
		SetDirectionalLightDepthStencilState<0>(GraphicsPSOInit);
		break;
	case 1:
		SetDirectionalLightDepthStencilState<1>(GraphicsPSOInit);
		break;
	case 2:
		SetDirectionalLightDepthStencilState<2>(GraphicsPSOInit);
		break;
	}
}

static void RenderDirectionalLight(
	FRHICommandList& RHICmdList, 
	const FScene& Scene, 
	const FViewInfo& View, 
	const FCachedLightMaterial& DefaultLightMaterial,
	const FLightSceneInfo& DirectionalLight, 
	uint32 LightingChannel, 
	bool bInlineReflectionAndSky, 
	FRDGTextureRef DynamicBentNormalAOTexture)
{
	FString LightNameWithLevel;
	FSceneRenderer::GetLightNameForDrawEvent(DirectionalLight.Proxy, LightNameWithLevel);
	SCOPED_DRAW_EVENTF(RHICmdList, DirectionalLight, TEXT("%s"), LightNameWithLevel);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

	TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);
	
	const FMaterialRenderProxy* LightFunctionMaterialProxy = nullptr;
	if (View.Family->EngineShowFlags.LightFunctions)
	{
		LightFunctionMaterialProxy = DirectionalLight.Proxy->GetLightFunctionMaterial();
	}

	FMobileDirectionalLightFunctionPS::FParameters PassParameters;
	PassParameters.MobileDirectionalLight = Scene.UniformBuffers.MobileDirectionalLightUniformBuffers[LightingChannel + 1];
	PassParameters.MobileDeferredCommonParameters.LightFunctionParameters = FVector4f(1.0f, 1.0f, 0.0f, 0.0f);
	PassParameters.MobileDeferredCommonParameters.CameraRelativeLightPosition = GetCamRelativeLightPosition(View.ViewMatrices, DirectionalLight);
	{
		PassParameters.MobileDeferredCommonParameters.LightFunctionParameters2 = FVector2f(DirectionalLight.Proxy->GetLightFunctionFadeDistance(), DirectionalLight.Proxy->GetLightFunctionDisabledBrightness());
		const FVector Scale = DirectionalLight.Proxy->GetLightFunctionScale();
		// Switch x and z so that z of the user specified scale affects the distance along the light direction
		const FVector InverseScale = FVector(1.f / Scale.Z, 1.f / Scale.Y, 1.f / Scale.X);
		const FMatrix WorldToLight = DirectionalLight.Proxy->GetWorldToLight() * FScaleMatrix(FVector(InverseScale));
		PassParameters.MobileDeferredCommonParameters.TranslatedWorldToLight = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToLight);
	}

	// DFAO
	if (DynamicBentNormalAOTexture != nullptr)
	{
		PassParameters.DFAOUpsampleParameters = DistanceField::SetupAOUpsampleParameters(View, DynamicBentNormalAOTexture);
		float DynamicBentNormalAO = 1.0f;
		PassParameters.SkyDiffuseLighting = GetSkyDiffuseLightingParameters(Scene.SkyLight, DynamicBentNormalAO);
	}

	// Skylights with static lighting already had their diffuse contribution baked into lightmaps
	const bool bDynamicSkyLight = Scene.SkyLight && (!Scene.SkyLight->bHasStaticLighting || !IsStaticLightingAllowed());
	const bool bEnableSkyLight = bDynamicSkyLight && View.Family->EngineShowFlags.SkyLighting;
	const bool bDynamicShadows = DirectionalLight.Proxy->CastsDynamicShadow() && View.Family->EngineShowFlags.DynamicShadows;

	// Add to emissive in SceneColor
	if (bInlineReflectionAndSky && !bDynamicSkyLight)
	{
		// pre-multiply SceneColor with AO
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha>::GetRHI();
	}
	else
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
	}
	
	// Do two passes, first masking DefautLit, second masking all other shading models
	const bool bOnlyDefaultLitInView = IsOnlyDefaultLitShadingModel(View.ShadingModelMaskInView);
	int32 NumPasses = 1;
	uint32 PassEnableShadingModelSupport = 0;
	uint32 ShadingModelStencilRef[2] = {};
	ShadingModelStencilRef[0] = PassShadingModelStencilValue(/*bEnableShadingModelSupport=*/false);

	if (!bOnlyDefaultLitInView && MobileUsesGBufferCustomData(Scene.GetShaderPlatform()))
	{
		const int32 PassIndex = NumPasses++;
		PassEnableShadingModelSupport |= (1 << PassIndex);
		ShadingModelStencilRef[PassIndex] = PassShadingModelStencilValue(/*bEnableShadingModelSupport=*/true);
	}

	static auto CVarMobileSupportInsetShadows = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.SupportInsetShadows"));
	const bool bInsetShadows = (CVarMobileSupportInsetShadows != nullptr && CVarMobileSupportInsetShadows->GetInt() != 0);
	const bool bMobileUsesShadowMaskTexture = (MobileUsesShadowMaskTexture(View.GetShaderPlatform()) || bInsetShadows);

	const uint8 LightingChannelStencilValue = GetLightingChannelStencilValue(LightingChannel);
	SetDirectionalLightDepthStencilState(GraphicsPSOInit, LightingChannel);
	
	for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
	{
		FMobileDirectionalLightFunctionPS::FPermutationDomain PermutationVector = FMobileDirectionalLightFunctionPS::BuildPermutationVector(
			View,
			bInlineReflectionAndSky,
			PassEnableShadingModelSupport & (1 << PassIndex),
			bDynamicShadows,
			bEnableSkyLight,
			bMobileUsesShadowMaskTexture,
			DynamicBentNormalAOTexture && UseDistanceFieldAO() && IsMobileDistanceFieldAOEnabled(View.GetShaderPlatform())
		);
		FCachedLightMaterial LightMaterial;
		TShaderRef<FMobileDirectionalLightFunctionPS> PixelShader;
		GetLightMaterial(DefaultLightMaterial, LightFunctionMaterialProxy, PermutationVector.ToDimensionValueId(), LightMaterial, PixelShader);

		const uint8 StencilRef = ShadingModelStencilRef[PassIndex] | STENCIL_LIGHTING_CHANNELS_MASK(LightingChannelStencilValue);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

#if PSO_PRECACHING_VALIDATE
		if (PSOCollectorStats::IsFullPrecachingValidationEnabled())
		{
			static const int32 MaterialPSOCollectorIndex = FPSOCollectorCreateManager::GetIndex(GetFeatureLevelShadingPath(GMaxRHIFeatureLevel), DeferredMobileLightMaterialPSOCollectorName);
			PSOCollectorStats::CheckFullPipelineStateInCache(GraphicsPSOInit, EPSOPrecacheResult::Unknown, LightMaterial.Material, nullptr, nullptr, MaterialPSOCollectorIndex);
		}
#endif // PSO_PRECACHING_VALIDATE

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

		SetShaderParametersMixedPS(RHICmdList, PixelShader, PassParameters, View, LightMaterial.MaterialProxy, *LightMaterial.Material);

		const FIntPoint TargetSize = View.GetSceneTexturesConfig().Extent;

		DrawRectangle(
			RHICmdList,
			0, 0,
			View.ViewRect.Width(), View.ViewRect.Height(),
			View.ViewRect.Min.X, View.ViewRect.Min.Y,
			View.ViewRect.Width(), View.ViewRect.Height(),
			FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()),
			TargetSize,
			VertexShader);
	}
}

static int RenderDirectionalLights(
	FRHICommandList& RHICmdList, 
	const FScene& Scene, 
	const FViewInfo& View, 
	const FCachedLightMaterial& DefaultLightMaterial, 
	EMobileSSRQuality MobileSSRQuality,
	FRDGTextureRef DynamicBentNormalAOTexture)
{
	uint32 NumLights = 0;
	for (uint32 ChannelIdx = 0; ChannelIdx < UE_ARRAY_COUNT(Scene.MobileDirectionalLights); ChannelIdx++)
	{
		NumLights += (Scene.GetMobileDirectionalLightForView(ChannelIdx, View) ? 1 : 0);
	}
	// We can merge reflection and skylight pass with a sole directional light pass and if all primitives and the directional light use the default lighting channel
	bool bPrimitivesUseLightingChannels = (View.bUsesLightingChannels && GMobileIgnoreDeferredShadingSkyLightChannels == 0);
	bool bPlanarReflection = Scene.GetForwardPassGlobalPlanarReflection() != nullptr;
	bool bClusteredReflection = (View.NumBoxReflectionCaptures + View.NumSphereReflectionCaptures) > 0;
	bool bSSR = MobileSSRQuality != EMobileSSRQuality::Disabled;

	bool bInlineReflectionAndSky = (NumLights == 1) && !bPrimitivesUseLightingChannels && (Scene.GetMobileDirectionalLightForView(0, View) != nullptr) && !(bPlanarReflection || bClusteredReflection || bSSR);
	if (!bInlineReflectionAndSky)
	{
		RenderReflectionEnvironmentSkyLighting(RHICmdList, Scene, View, MobileSSRQuality, DynamicBentNormalAOTexture);
	}

	for (uint32 ChannelIdx = 0; ChannelIdx < UE_ARRAY_COUNT(Scene.MobileDirectionalLights); ChannelIdx++)
	{
		FLightSceneInfo* DirectionalLight = Scene.GetMobileDirectionalLightForView(ChannelIdx, View);
		if (DirectionalLight)
		{
			RenderDirectionalLight(RHICmdList, Scene, View, DefaultLightMaterial, *DirectionalLight, ChannelIdx, bInlineReflectionAndSky, DynamicBentNormalAOTexture);
		}
	}

	if (View.IsRuntimeReflectionCapture() && View.Family->EngineShowFlags.SkyLighting && Scene.SkyLight && CVarReflectionCaptureRuntimeSkyLightPixels.GetValueOnRenderThread())
	{
		RenderSkyLightToSky(RHICmdList, Scene, View);
	}

	return NumLights;
}

template<uint32 LightingChannel, bool bEnableShadingModelSupport>
static void SetLocalLightRasterizerAndDepthState(FGraphicsPipelineStateInitializer& GraphicsPSOInit, bool bReverseCulling, bool bCameraInsideLightGeometry)
{
	if (GMobileUseLightStencilCulling != 0)
	{
		// Render backfaces with depth and stencil tests
		// and clear stencil to zero for next light mask
		GraphicsPSOInit.RasterizerState = bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
			false, CF_LessEqual,
			false, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
			true, CF_Equal, SO_Zero, SO_Keep, SO_Zero,
			GET_STENCIL_MOBILE_SM_MASK(0xff) | STENCIL_LIGHTING_CHANNELS_MASK(1u << LightingChannel) | STENCIL_SANDBOX_MASK,
			STENCIL_SANDBOX_MASK
		>::GetRHI();
	}
	else
	{
		if (bCameraInsideLightGeometry)
		{
			// Render backfaces with depth tests disabled since the camera is inside (or close to inside) the light geometry
			GraphicsPSOInit.RasterizerState = bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
				false, CF_Always,
				true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
				false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
				GET_STENCIL_MOBILE_SM_MASK(0xff) | STENCIL_LIGHTING_CHANNELS_MASK(1u << LightingChannel), 0x00>::GetRHI();
		}
		else
		{
			// Render frontfaces with depth tests on to get the speedup from HiZ since the camera is outside the light geometry
			GraphicsPSOInit.RasterizerState = bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
				false, CF_DepthNearOrEqual,
				true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
				false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
				GET_STENCIL_MOBILE_SM_MASK(0xff) | STENCIL_LIGHTING_CHANNELS_MASK(1u << LightingChannel), 0x00>::GetRHI();
		}
	}
}

template <bool bEnableShadingModelSupport>
static void SetLocalLightRasterizerAndDepthState(FGraphicsPipelineStateInitializer& GraphicsPSOInit, bool bReverseCulling, bool bCameraInsideLightGeometry, uint32 LightingChannel)
{
	// TODO: support multi-channel lights?
	switch (LightingChannel)
	{
	default:
		SetLocalLightRasterizerAndDepthState<0, bEnableShadingModelSupport>(GraphicsPSOInit, bReverseCulling, bCameraInsideLightGeometry);
		break;
	case 1:
		SetLocalLightRasterizerAndDepthState<1, bEnableShadingModelSupport>(GraphicsPSOInit, bReverseCulling, bCameraInsideLightGeometry);
		break;
	case 2:
		SetLocalLightRasterizerAndDepthState<2, bEnableShadingModelSupport>(GraphicsPSOInit, bReverseCulling, bCameraInsideLightGeometry);
		break;
	}
}

static void SetLocalLightRasterizerAndDepthState(FGraphicsPipelineStateInitializer& GraphicsPSOInit, bool bReverseCulling, bool bCameraInsideLightGeometry, uint32 LightingChannel, bool bEnableShadingModelSupport)
{
	if (bEnableShadingModelSupport)
	{
		SetLocalLightRasterizerAndDepthState<true>(GraphicsPSOInit, bReverseCulling, bCameraInsideLightGeometry, LightingChannel);
	}
	else
	{
		SetLocalLightRasterizerAndDepthState<false>(GraphicsPSOInit, bReverseCulling, bCameraInsideLightGeometry, LightingChannel);
	}
}

static void SetLocalLightRasterizerAndDepthState(FGraphicsPipelineStateInitializer& GraphicsPSOInit, const FViewInfo& View, const FSphere& LightBounds, uint32 LightingChannel, bool bEnableShadingModelSupport)
{
	const bool bCameraInsideLightGeometry =
		(GMobileUseLightStencilCulling == 0) &&
		(((FVector)View.ViewMatrices.GetViewOrigin() - LightBounds.Center).SizeSquared() < FMath::Square(LightBounds.W * 1.05f + View.NearClippingDistance * 2.0f)
			// Always draw backfaces in ortho
			//@todo - accurate ortho camera / light intersection
			|| !View.IsPerspectiveProjection());

	if (bEnableShadingModelSupport)
	{
		SetLocalLightRasterizerAndDepthState<true>(GraphicsPSOInit, View.bReverseCulling, bCameraInsideLightGeometry, LightingChannel);
	}
	else
	{
		SetLocalLightRasterizerAndDepthState<false>(GraphicsPSOInit, View.bReverseCulling, bCameraInsideLightGeometry, LightingChannel);
	}
}

static void RenderLocalLight_StencilMask(FRHICommandList& RHICmdList, const FScene& Scene, const FViewInfo& View, const FLightSceneInfo& LightSceneInfo)
{
	const uint8 LightType = LightSceneInfo.Proxy->GetLightType();

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
	GraphicsPSOInit.RasterizerState = View.bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();
	// set stencil to 1 where depth test fails
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
		false, CF_DepthNearOrEqual,
		true, CF_Always, SO_Keep, SO_Replace, SO_Keep,		
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		0x00, STENCIL_SANDBOX_MASK>::GetRHI();

	FDeferredLightVS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FDeferredLightVS::FRadialLight>(true);
	TShaderMapRef<FDeferredLightVS> VertexShader(View.ShaderMap, PermutationVector);
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = nullptr;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 1);

	FDeferredLightVS::FParameters ParametersVS = FDeferredLightVS::GetParameters(View, &LightSceneInfo);
	SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);

	if (LightType == LightType_Point || LightType == LightType_Rect)
	{
		StencilingGeometry::DrawSphere(RHICmdList);
	}
	else // LightType_Spot
	{
		StencilingGeometry::DrawCone(RHICmdList);
	}
}

static void RenderLocalLight(
	FRHICommandList& RHICmdList, 
	const FScene& Scene, 
	const FViewInfo& View, 
	const FLightSceneInfo& LightSceneInfo, 
	const FCachedLightMaterial& DefaultLightMaterial,
	const FVisibleLightInfoArray& VisibleLightInfos)
{
	if (!LightSceneInfo.ShouldRenderLight(View) || LightSceneInfo.Proxy->GetLightingChannelMask() == 0)
	{
		return;
	}
	
	const uint8 LightType = LightSceneInfo.Proxy->GetLightType();
	const bool bIsSpotLight = LightType == LightType_Spot;
	const bool bIsPointLight = LightType == LightType_Point;
	const bool bIsRectLight = LightType == LightType_Rect;
	if (!bIsSpotLight && !bIsPointLight && !bIsRectLight)
	{
		return;
	}

	FString LightNameWithLevel;
	FSceneRenderer::GetLightNameForDrawEvent(LightSceneInfo.Proxy, LightNameWithLevel);
	SCOPED_DRAW_EVENTF(RHICmdList, LocalLight, TEXT("%s"), LightNameWithLevel);
	check(LightSceneInfo.Proxy->IsLocalLight());
	
	if (GMobileUseLightStencilCulling != 0)
	{
		RenderLocalLight_StencilMask(RHICmdList, Scene, View, LightSceneInfo);
	}

	const bool bUseIESTexture = View.Family->EngineShowFlags.TexturedLightProfiles && LightSceneInfo.Proxy->GetIESTextureResource();

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	const FSphere LightBounds = LightSceneInfo.Proxy->GetBoundingSphere();

	FDeferredLightVS::FPermutationDomain PermutationVectorVS;
	PermutationVectorVS.Set<FDeferredLightVS::FRadialLight>(true);
	TShaderMapRef<FDeferredLightVS> VertexShader(View.ShaderMap, PermutationVectorVS);
	FDeferredLightVS::FParameters ParametersVS = FDeferredLightVS::GetParameters(View, &LightSceneInfo);
		
	const FMaterialRenderProxy* LightFunctionMaterialProxy = nullptr;
	if (View.Family->EngineShowFlags.LightFunctions)
	{
		LightFunctionMaterialProxy = LightSceneInfo.Proxy->GetLightFunctionMaterial();
	}

	FMobileRadialLightFunctionPS::FParameters PassParameters;
	bool bShouldCastLocalLightShadow = LightSceneInfo.SetupMobileMovableLocalSpotLightShadowParameters(View, VisibleLightInfos, PassParameters.MobileMovableLocalLightShadow);
	bShouldCastLocalLightShadow |= LightSceneInfo.IsAValidShadowPointLight(View, VisibleLightInfos, View.GetShaderPlatform());

	PassParameters.Light = GetDeferredLightParameters(View, LightSceneInfo).LightParameters;
	const float TanOuterAngle = bIsSpotLight ? FMath::Tan(LightSceneInfo.Proxy->GetOuterConeAngle()) : 1.0f;
	PassParameters.MobileDeferredCommonParameters.LightFunctionParameters = FVector4f(TanOuterAngle, 1.0f /*ShadowFadeFraction*/, bIsSpotLight ? 1.0f : 0.0f, bIsPointLight ? 1.0f : 0.0f);
	PassParameters.MobileDeferredCommonParameters.LightFunctionParameters2 = FVector2f(LightSceneInfo.Proxy->GetLightFunctionFadeDistance(), LightSceneInfo.Proxy->GetLightFunctionDisabledBrightness());
	const FVector Scale = LightSceneInfo.Proxy->GetLightFunctionScale();
	// Switch x and z so that z of the user specified scale affects the distance along the light direction
	const FVector InverseScale = FVector(1.f / Scale.Z, 1.f / Scale.Y, 1.f / Scale.X);
	const FMatrix WorldToLight = LightSceneInfo.Proxy->GetWorldToLight() * FScaleMatrix(FVector(InverseScale));
	PassParameters.MobileDeferredCommonParameters.TranslatedWorldToLight = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToLight);
	PassParameters.MobileDeferredCommonParameters.CameraRelativeLightPosition = GetCamRelativeLightPosition(View.ViewMatrices, LightSceneInfo);

	const bool bCapsuleLight = CVarMobileAllowCapsuleLights.GetValueOnAnyThread() != 0 && !bIsRectLight &&
								(PassParameters.Light.SourceRadius != 0	|| PassParameters.Light.SoftSourceRadius != 0 || PassParameters.Light.SourceLength != 0);
	
	// Do two passes, first masking DefautLit, second masking all other shading models
	const bool bOnlyDefaultLitInView = IsOnlyDefaultLitShadingModel(View.ShadingModelMaskInView);
	int32 NumPasses = !bOnlyDefaultLitInView && MobileUsesGBufferCustomData(Scene.GetShaderPlatform()) ? 2 : 1;

	for (int32 PassIndex = 0; PassIndex < NumPasses; PassIndex++)
	{
		// Clamp mask to 3 channels
		uint8 LightingChannelMask = (LightSceneInfo.Proxy->GetLightingChannelMask() & 0x7);
		while (LightingChannelMask != 0)
		{
			const uint32 LightingChannel = GetLightingChannel(LightingChannelMask);
			const uint8 LightingChannelStencilValue = GetLightingChannelStencilValue(LightingChannel);
			LightingChannelMask &= ~(1u << LightingChannel);

			const bool bEnableShadingModelSupport = (PassIndex > 0);
			SetLocalLightRasterizerAndDepthState(GraphicsPSOInit, View, LightBounds, LightingChannel, bEnableShadingModelSupport);

			FMobileRadialLightFunctionPS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FMobileRadialLightFunctionPS::FEnableShadingModelSupport>(bEnableShadingModelSupport);
			PermutationVector.Set<FMobileRadialLightFunctionPS::FRadialLightTypeDim>(LightType);
			PermutationVector.Set<FMobileRadialLightFunctionPS::FIESProfileDim>(bUseIESTexture);
			PermutationVector.Set<FMobileRadialLightFunctionPS::FSupportRadialLightShadowDim>(bShouldCastLocalLightShadow);
			PermutationVector.Set<FMobileRadialLightFunctionPS::FCapsuleLightDim>(bCapsuleLight);
			FCachedLightMaterial LightMaterial;
			TShaderRef<FMobileRadialLightFunctionPS> PixelShader;
			GetLightMaterial(DefaultLightMaterial, LightFunctionMaterialProxy, PermutationVector.ToDimensionValueId(), LightMaterial, PixelShader);

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

			const uint8 StencilRef = PassShadingModelStencilValue(bEnableShadingModelSupport) | STENCIL_LIGHTING_CHANNELS_MASK(LightingChannelStencilValue);

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, StencilRef);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);

			SetShaderParametersMixedPS(RHICmdList, PixelShader, PassParameters, View, LightMaterial.MaterialProxy, *LightMaterial.Material);

			if (LightType == LightType_Point || LightType == LightType_Rect)
			{
				StencilingGeometry::DrawSphere(RHICmdList);
			}
			else // LightType_Spot
			{
				StencilingGeometry::DrawCone(RHICmdList);
			}
		}
	}
}

static void RenderSimpleLights(
	FRHICommandList& RHICmdList, 
	const FScene& Scene, 
	int32 ViewIndex,
	int32 NumViews,
	const FViewInfo& View,
	const FSortedLightSetSceneInfo &SortedLightSet, 
	const FCachedLightMaterial& DefaultMaterial)
{
	const FSimpleLightArray& SimpleLights = SortedLightSet.SimpleLights;
	if (SimpleLights.InstanceData.Num() == 0)
	{
		return;
	}

	SCOPED_DRAW_EVENT(RHICmdList, SimpleLights);

	FDeferredLightVS::FPermutationDomain PermutationVectorVS;
	PermutationVectorVS.Set<FDeferredLightVS::FRadialLight>(true);
	TShaderMapRef<FDeferredLightVS> VertexShader(View.ShaderMap, PermutationVectorVS);

	FGraphicsPipelineStateInitializer GraphicsPSOLight;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOLight);
	// Use additive blending for color
	GraphicsPSOLight.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
	GraphicsPSOLight.PrimitiveType = PT_TriangleList;
	GraphicsPSOLight.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
	GraphicsPSOLight.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOLight.RasterizerState = View.bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();
	GraphicsPSOLight.DepthStencilState = TStaticDepthStencilState<
		false, CF_DepthNearOrEqual,
		true, CF_NotEqual, SO_Keep, SO_Keep, SO_Keep, // Render where ShadingModel Mask is not zero
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		GET_STENCIL_MOBILE_SM_MASK(0xff), 0x00>::GetRHI();

	TShaderRef<FMobileRadialLightFunctionPS> PixelShader;
	FMobileRadialLightFunctionPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FMobileRadialLightFunctionPS::FSimpleLightDim>(true);
	{
		FCachedLightMaterial LightMaterial;
		GetLightMaterial(DefaultMaterial, nullptr, PermutationVector.ToDimensionValueId(), LightMaterial, PixelShader);
	}
	GraphicsPSOLight.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	const uint8 StencilRef = 0;
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOLight, StencilRef);

	if (NumViews > 1)
	{
		// set viewports only we we have more than one 
		// otherwise it is set at the start of the pass
		RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
	}
	
	for (int32 LightIndex = 0; LightIndex < SimpleLights.InstanceData.Num(); LightIndex++)
	{
		const FSimpleLightEntry& SimpleLight = SimpleLights.InstanceData[LightIndex];
		const FSimpleLightPerViewEntry& SimpleLightPerViewData = SimpleLights.GetViewDependentData(LightIndex, ViewIndex, NumViews);
		const FSphere LightBounds(SimpleLightPerViewData.Position, SimpleLight.Radius);
		FDeferredLightVS::FParameters ParametersVS = FDeferredLightVS::GetParameters(View, LightBounds);

		// Render light
		FMobileRadialLightFunctionPS::FParameters ParametersPS;
		ParametersPS.Light = GetSimpleDeferredLightParameters(View, SimpleLight, SimpleLightPerViewData).LightParameters;
		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);
		SetShaderParametersMixedPS(RHICmdList, PixelShader, ParametersPS, View, DefaultMaterial.MaterialProxy, *DefaultMaterial.Material);

		// Apply the point or spot light with some approximately bounding geometry,
		// So we can get speedups from depth testing and not processing pixels outside of the light's influence.
		StencilingGeometry::DrawSphere(RHICmdList);
	}
}

void MobileDeferredRenderLocalLight(
	FRHICommandList& RHICmdList,
	const FScene& Scene,
	const FViewInfo& View,
	const FLightSceneInfo& LightSceneInfo,
	const FVisibleLightInfoArray& VisibleLightInfos)
{
	// Default material for light rendering
	FCachedLightMaterial DefaultMaterial;
	DefaultMaterial.MaterialProxy = UMaterial::GetDefaultMaterial(MD_LightFunction)->GetRenderProxy();
	DefaultMaterial.Material = DefaultMaterial.MaterialProxy->GetMaterialNoFallback(ERHIFeatureLevel::ES3_1);
	check(DefaultMaterial.Material);
	
	RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
	RenderLocalLight(RHICmdList, Scene, View, LightSceneInfo, DefaultMaterial, VisibleLightInfos);
}

void MobileDeferredShadingPass(
	FRHICommandList& RHICmdList,
	int32 ViewIndex,
	int32 NumViews,
	const FViewInfo& View,
	const FScene& Scene, 
	const FSortedLightSetSceneInfo& SortedLightSet,
	const FVisibleLightInfoArray& VisibleLightInfos,
	EMobileSSRQuality MobileSSRQuality,
	FRDGTextureRef DynamicBentNormalAOTexture)
{
	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, DeferredShading, "DeferredShading");
	
	RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

	// Default material for light rendering
	FCachedLightMaterial DefaultMaterial;
	DefaultMaterial.MaterialProxy = UMaterial::GetDefaultMaterial(MD_LightFunction)->GetRenderProxy();
	DefaultMaterial.Material = DefaultMaterial.MaterialProxy->GetMaterialNoFallback(ERHIFeatureLevel::ES3_1);
	check(DefaultMaterial.Material);

	int NumDirLights = RenderDirectionalLights(RHICmdList, Scene, View, DefaultMaterial, MobileSSRQuality, DynamicBentNormalAOTexture);
	
	EShaderPlatform ShaderPlatform = View.GetShaderPlatform();
	const bool bMobileUseClusteredDeferredShading = UseClusteredDeferredShading(ShaderPlatform) && NumDirLights > 0;
	if (!bMobileUseClusteredDeferredShading)
	{
		// Render non-clustered simple lights
		RenderSimpleLights(RHICmdList, Scene, ViewIndex, NumViews, View, SortedLightSet, DefaultMaterial);
	}

	// Render non-clustered local lights
	int32 NumLights = SortedLightSet.SortedLights.Num();
	const int32 UnbatchedLightStart = SortedLightSet.UnbatchedLightStart;
	int32 StandardDeferredStart = SortedLightSet.SimpleLightsEnd;
	if (bMobileUseClusteredDeferredShading)
	{
		StandardDeferredStart = SortedLightSet.ClusteredSupportedEnd;
	}

	// Draw non-shadowed non-light function lights
	for (int32 LightIdx = StandardDeferredStart; LightIdx < UnbatchedLightStart; ++LightIdx)
	{
		const FSortedLightSceneInfo& SortedLight = SortedLightSet.SortedLights[LightIdx];
		const FLightSceneInfo& LightSceneInfo = *SortedLight.LightSceneInfo;
		RenderLocalLight(RHICmdList, Scene, View, LightSceneInfo, DefaultMaterial, VisibleLightInfos);
	}

	// Draw shadowed and light function lights
	for (int32 LightIdx = UnbatchedLightStart; LightIdx < NumLights; ++LightIdx)
	{
		const FSortedLightSceneInfo& SortedLight = SortedLightSet.SortedLights[LightIdx];
		const FLightSceneInfo& LightSceneInfo = *SortedLight.LightSceneInfo;

		if (LightSceneInfo.IsAValidShadowPointLight(View, VisibleLightInfos, ShaderPlatform))
		{
			// Skip point light shadow as they 2 need RDG Passes (render point shadow to shadowscreenspace and renderlight).
			// they'll be rendered after "the main pass"
			continue;
		}

		RenderLocalLight(RHICmdList, Scene, View, LightSceneInfo, DefaultMaterial, VisibleLightInfos);
	}
}

class FDeferredMobileLightMaterialPSOCollector : public IPSOCollector
{
public:
	FDeferredMobileLightMaterialPSOCollector(ERHIFeatureLevel::Type InFeatureLevel) :
		IPSOCollector(FPSOCollectorCreateManager::GetIndex(GetFeatureLevelShadingPath(InFeatureLevel), DeferredMobileLightMaterialPSOCollectorName))
	{
	}

	virtual void CollectPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		TArray<FPSOPrecacheData>& PSOInitializers
	) override final;

private:
	void CollectPSOInitializersDirectional(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		TArray<FPSOPrecacheData>& PSOInitializers);
	void CollectPSOInitializersLocal(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		TArray<FPSOPrecacheData>& PSOInitializers);
};

void FDeferredMobileLightMaterialPSOCollector::CollectPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	if (Material.GetMaterialDomain() == MD_LightFunction)
	{
		CollectPSOInitializersDirectional(SceneTexturesConfig, Material, PSOInitializers);
		CollectPSOInitializersLocal(SceneTexturesConfig, Material, PSOInitializers);
	}
}

void FDeferredMobileLightMaterialPSOCollector::CollectPSOInitializersDirectional(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	FMaterialShaderTypes ShaderTypesToGetAnyPermutation;
	ShaderTypesToGetAnyPermutation.AddShaderType<FMobileDirectionalLightFunctionPS>();

	FMaterialShaders ShadersAnyPermutation;
	if (!Material.TryGetShaders(ShaderTypesToGetAnyPermutation, nullptr, ShadersAnyPermutation))
	{
		return;
	}
	
	static auto CVarMobileSupportInsetShadows = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.SupportInsetShadows"));
	const bool bInsetShadows = (CVarMobileSupportInsetShadows != nullptr && CVarMobileSupportInsetShadows->GetInt() != 0);
	const bool bMobileUsesShadowMaskTexture = (MobileUsesShadowMaskTexture(GMaxRHIShaderPlatform) || bInsetShadows);

	auto AddPSOInitializer =
		[&](int32 PassIndex, bool bInlineReflectionAndSky, bool bOnlyDefaultLitInView, int32 ShadowQuality, bool bHasBoxSphere)
		{
			const uint32 CustomPassIndex = 1;
			const int PassEnableShadingModelSupport = (!bOnlyDefaultLitInView && MobileUsesGBufferCustomData(GMaxRHIShaderPlatform)) ? (1 << CustomPassIndex) : 0;
			const bool bDynamicSkyLight = true;
			const bool bShadingModelSupport = PassEnableShadingModelSupport & (1 << PassIndex);
			const bool bUseClusteredLights = UseClusteredDeferredShading(GMaxRHIShaderPlatform);
			const bool bClustredReflection = bInlineReflectionAndSky && bHasBoxSphere;
			const bool bPlanarReflection = false;
			const bool bEnableSkyLight = bInlineReflectionAndSky && bDynamicSkyLight;
			const bool bApplySkyShadowing = false;
			const int32 LightingChannel = 0;
						
			FMobileDirectionalLightFunctionPS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FMobileDirectionalLightFunctionPS::FEnableShadingModelSupport>(bShadingModelSupport);
			PermutationVector.Set<FMobileDirectionalLightFunctionPS::FEnableClustredLights>(bUseClusteredLights);
			PermutationVector.Set<FMobileDirectionalLightFunctionPS::FEnableSkyLight>(bEnableSkyLight);
			PermutationVector.Set<FMobileDirectionalLightFunctionPS::FEnableScreenSpaceShadowMask>(bMobileUsesShadowMaskTexture);
			PermutationVector.Set<FMobileDirectionalLightFunctionPS::FEnableCSM>(ShadowQuality > 0);
			PermutationVector.Set<FMobileDirectionalLightFunctionPS::FShadowQuality>(FMath::Clamp(ShadowQuality, 1, 3));
			PermutationVector.Set<FMobileDirectionalLightFunctionPS::FSkyShadowing>(bInlineReflectionAndSky && bApplySkyShadowing);
			FMaterialShaderTypes ShaderTypesToGet;
			ShaderTypesToGet.AddShaderType<FMobileDirectionalLightFunctionPS>(PermutationVector.ToDimensionValueId());

			FMaterialShaders Shaders;
			if (!Material.TryGetShaders(ShaderTypesToGet, nullptr, Shaders))
			{
				return;
			}

			TShaderRef<FMobileDirectionalLightFunctionPS> PixelShader;
			Shaders.TryGetPixelShader(PixelShader);
			if (!PixelShader.IsValid())
			{
				return;
			}

			const bool bRequired = false;
			FRHIPixelShader* RHIPixelShader = PixelShader.GetPixelShader(bRequired);
			if (RHIPixelShader == nullptr)
			{
				return;
			}

			TShaderMapRef<FPostProcessVS> VertexShader(GetGlobalShaderMap(GMaxRHIShaderPlatform));
			FRHIVertexShader* RHIVertexShader = VertexShader.GetVertexShader(bRequired);
			if (RHIVertexShader == nullptr)
			{
				return;
			}

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

			// Add to emissive in SceneColor
			if (bInlineReflectionAndSky && !bDynamicSkyLight)
			{
				// pre-multiply SceneColor with AO
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha>::GetRHI();
			}
			else
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
			}
			SetDirectionalLightDepthStencilState(GraphicsPSOInit, LightingChannel);

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = RHIVertexShader;
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = RHIPixelShader;
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
			RenderTargetsInfo.NumSamples = 1;
			if (MobileAllowFramebufferFetch(GMaxRHIShaderPlatform))
			{
				SetupGBufferRenderTargetInfo(SceneTexturesConfig, RenderTargetsInfo, false /*bSetupDepthStencil*/);
			}
			else
			{
				AddRenderTargetInfo(SceneTexturesConfig.ColorFormat, SceneTexturesConfig.ColorCreateFlags, RenderTargetsInfo);
			}

			SetupDepthStencilInfo(PF_DepthStencil, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
				ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite, RenderTargetsInfo);

			GraphicsPSOInit.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(GraphicsPSOInit);
			ApplyTargetsInfo(GraphicsPSOInit, RenderTargetsInfo);

			GraphicsPSOInit.SubpassIndex = 0;
			GraphicsPSOInit.SubpassHint = ESubpassHint::None;
			if (MobileAllowFramebufferFetch(GMaxRHIShaderPlatform))
			{
				GraphicsPSOInit.SubpassIndex = 2;
				GraphicsPSOInit.SubpassHint = ESubpassHint::DeferredShadingSubpass;
			}

			FPSOPrecacheData PSOPrecacheData;
			PSOPrecacheData.bRequired = true;
			PSOPrecacheData.Type = FPSOPrecacheData::EType::Graphics;
			PSOPrecacheData.GraphicsPSOInitializer = GraphicsPSOInit;
#if PSO_PRECACHING_VALIDATE
			PSOPrecacheData.PSOCollectorIndex = PSOCollectorIndex;
			PSOPrecacheData.VertexFactoryType = nullptr;
#endif // PSO_PRECACHING_VALIDATE

			PSOInitializers.Add(MoveTemp(PSOPrecacheData));
		};

	// int32 PassIndex, bool bInlineReflectionAndSky, bool bOnlyDefaultLitInView, int32 ShadowQuality, bool bHasBoxSphere
	AddPSOInitializer(0, true, false, 3, true);
	AddPSOInitializer(1, true, false, 3, true);
	AddPSOInitializer(0, false, true, 0, false);
	AddPSOInitializer(0, true, false, 0, false);
	AddPSOInitializer(1, true, false, 0, false);
	AddPSOInitializer(0, false, false, 0, false);
	AddPSOInitializer(1, false, false, 0, false);
	AddPSOInitializer(0, true, true, 0, false);
}

void FDeferredMobileLightMaterialPSOCollector::CollectPSOInitializersLocal(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	FMaterialShaderTypes ShaderTypesToGetAnyPermutation;
	ShaderTypesToGetAnyPermutation.AddShaderType<FMobileRadialLightFunctionPS>();

	FMaterialShaders ShadersAnyPermutation;
	if (!Material.TryGetShaders(ShaderTypesToGetAnyPermutation, nullptr, ShadersAnyPermutation))
	{
		return;
	}

	auto AddPSOInitializer =
		[&](int32 PassIndex, uint8 LightType, bool bUseIESTexture, bool bCameraInsideLightGeometry, bool bCapsuleLight)
		{
			const bool bEnableShadingModelSupport = (PassIndex > 0);
			const bool bShouldCastShadow = false;
			const int32 LightingChannel = 0;
			const bool bReverseCulling = false;

			FMobileRadialLightFunctionPS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FMobileRadialLightFunctionPS::FEnableShadingModelSupport>(bEnableShadingModelSupport);
			PermutationVector.Set<FMobileRadialLightFunctionPS::FRadialLightTypeDim>(LightType);
			PermutationVector.Set<FMobileRadialLightFunctionPS::FIESProfileDim>(bUseIESTexture);
			PermutationVector.Set<FMobileRadialLightFunctionPS::FSupportRadialLightShadowDim>(bShouldCastShadow);
			PermutationVector.Set<FMobileRadialLightFunctionPS::FCapsuleLightDim>(bCapsuleLight);
			FMaterialShaderTypes ShaderTypesToGet;
			ShaderTypesToGet.AddShaderType<FMobileRadialLightFunctionPS>(PermutationVector.ToDimensionValueId());

			FMaterialShaders Shaders;
			if (!Material.TryGetShaders(ShaderTypesToGet, nullptr, Shaders))
			{
				return;
			}

			TShaderRef<FMobileRadialLightFunctionPS> PixelShader;
			Shaders.TryGetPixelShader(PixelShader);
			if (!PixelShader.IsValid())
			{
				return;
			}

			const bool bRequired = false;
			FRHIPixelShader* RHIPixelShader = PixelShader.GetPixelShader(bRequired);
			if (RHIPixelShader == nullptr)
			{
				return;
			}

			FDeferredLightVS::FPermutationDomain PermutationVectorVS;
			PermutationVectorVS.Set<FDeferredLightVS::FRadialLight>(true);
			TShaderMapRef<FDeferredLightVS> VertexShader(GetGlobalShaderMap(GMaxRHIShaderPlatform), PermutationVectorVS);
			FRHIVertexShader* RHIVertexShader = VertexShader.GetVertexShader(bRequired);
			if (RHIVertexShader == nullptr)
			{
				return;
			}

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
			SetLocalLightRasterizerAndDepthState(GraphicsPSOInit, bReverseCulling, bCameraInsideLightGeometry, LightingChannel, bEnableShadingModelSupport);

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = RHIVertexShader;
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = RHIPixelShader;
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
			RenderTargetsInfo.NumSamples = 1;
			if (MobileAllowFramebufferFetch(GMaxRHIShaderPlatform))
			{
				SetupGBufferRenderTargetInfo(SceneTexturesConfig, RenderTargetsInfo, false /*bSetupDepthStencil*/);
			}
			else
			{
				AddRenderTargetInfo(SceneTexturesConfig.ColorFormat, SceneTexturesConfig.ColorCreateFlags, RenderTargetsInfo);
			}
			
			SetupDepthStencilInfo(PF_DepthStencil, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
				ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite, RenderTargetsInfo);

			GraphicsPSOInit.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(GraphicsPSOInit);
			ApplyTargetsInfo(GraphicsPSOInit, RenderTargetsInfo);

			GraphicsPSOInit.SubpassIndex = 0;
			GraphicsPSOInit.SubpassHint = ESubpassHint::None;
			if (MobileAllowFramebufferFetch(GMaxRHIShaderPlatform))
			{
				GraphicsPSOInit.SubpassIndex = 2;
				GraphicsPSOInit.SubpassHint = ESubpassHint::DeferredShadingSubpass;
			}

			FPSOPrecacheData PSOPrecacheData;
			PSOPrecacheData.bRequired = true;
			PSOPrecacheData.Type = FPSOPrecacheData::EType::Graphics;
			PSOPrecacheData.GraphicsPSOInitializer = GraphicsPSOInit;
#if PSO_PRECACHING_VALIDATE
			PSOPrecacheData.PSOCollectorIndex = PSOCollectorIndex;
			PSOPrecacheData.VertexFactoryType = nullptr;
#endif // PSO_PRECACHING_VALIDATE

			PSOInitializers.Add(MoveTemp(PSOPrecacheData));
		};

	bool bAllowCapsuleLights = (CVarMobileAllowCapsuleLights.GetValueOnAnyThread() != 0);

	// int32 PassIndex, uint8 LightType, bool bUseIESTexture, bool bCameraInsideLightGeometry
	AddPSOInitializer(0, LIGHT_TYPE_POINT, false, false, false);
	AddPSOInitializer(1, LIGHT_TYPE_POINT, false, false, false);
	AddPSOInitializer(0, LIGHT_TYPE_POINT, true, false, false);
	AddPSOInitializer(1, LIGHT_TYPE_POINT, true, false, false);
	AddPSOInitializer(0, LIGHT_TYPE_POINT, true, true, false);
	AddPSOInitializer(1, LIGHT_TYPE_POINT, true, true, false);
	AddPSOInitializer(0, LIGHT_TYPE_SPOT, false, false, false);
	AddPSOInitializer(1, LIGHT_TYPE_SPOT, false, false, false);

	if (bAllowCapsuleLights)
	{
		AddPSOInitializer(0, LIGHT_TYPE_POINT, false, false, true);
		AddPSOInitializer(1, LIGHT_TYPE_POINT, false, false, true);
		AddPSOInitializer(0, LIGHT_TYPE_POINT, false, true, true);
		AddPSOInitializer(1, LIGHT_TYPE_POINT, false, true, true);
		
		AddPSOInitializer(0, LIGHT_TYPE_SPOT, false, false, true);
		AddPSOInitializer(1, LIGHT_TYPE_SPOT, false, false, true);
	}
}

IPSOCollector* CreateDeferredMobileLightMaterialPSOCollector(ERHIFeatureLevel::Type FeatureLevel)
{
	return new FDeferredMobileLightMaterialPSOCollector(FeatureLevel);
}
FRegisterPSOCollectorCreateFunction RegisterDeferredMobileLightMaterialPSOCollector(&CreateDeferredMobileLightMaterialPSOCollector, EShadingPath::Mobile, DeferredMobileLightMaterialPSOCollectorName);
