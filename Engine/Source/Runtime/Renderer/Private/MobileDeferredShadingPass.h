// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphDefinitions.h"
#include "SceneRendering.h"

enum class EMobileSSRQuality;
class FMobileSceneTextureUniformParameters;
class FRDGBuilder;
class FRHICommandListImmediate;
class FScene;
class FViewInfo;
struct FRenderTargetBindingSlots;
struct FSortedLightSetSceneInfo;

extern int32 GMobileUseClusteredDeferredShading;

void MobileDeferredRenderLocalLight(
	FRHICommandList& RHICmdList,
	const FScene& Scene,
	const FViewInfo& View,
	const FLightSceneInfo& LightSceneInfo,
	const FVisibleLightInfoArray& VisibleLightInfos);

void MobileDeferredShadingPass(
	FRHICommandList& RHICmdList,
	int32 ViewIndex,
	int32 NumViews,
	const FViewInfo& View,
	const FScene& Scene,
	const FSortedLightSetSceneInfo& SortedLightSet,
	const FVisibleLightInfoArray& VisibleLightInfos,
	EMobileSSRQuality MobileSSRQuality,
	FRDGTextureRef DynamicBentNormalAOTexture = nullptr);
