// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
MobileDistortionPass.cpp - Mobile specific rendering of primtives with refraction
=============================================================================*/

#include "MobileDistortionPass.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "TranslucentRendering.h"
#include "DynamicPrimitiveDrawing.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PipelineStateCache.h"
#include "ScenePrivate.h"
#include "DistortionRendering.h"

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FMobileDistortionPassUniformParameters, RENDERER_API)
	SHADER_PARAMETER_STRUCT(FMobileSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER(FVector4f, DistortionParams)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_STATIC_UNIFORM_BUFFER_STRUCT(FMobileDistortionPassUniformParameters, "MobileDistortionPass", SceneTextures);



static TAutoConsoleVariable<int32> CVarMobileDistortionBeforeTranslucency(
	TEXT("r.Mobile.Distortion.BeforeTranslucency"),
	0,
	TEXT("0: Render distortion in the PostProcess chain (default).\n")
	TEXT("1: Render distortion before the translucent pass"),
	ECVF_RenderThreadSafe);

bool IsMobileDistortionBeforeTranslucency()
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.Distortion.BeforeTranslucency"));
	int32 DistortionBeforeTranslucency = CVar->GetInt();

	return DistortionBeforeTranslucency == 1;
}

bool IsMobileDistortionActive(const FViewInfo& View)
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DisableDistortion"));
	int32 DisableDistortion = CVar->GetInt();

	return
		View.Family->EngineShowFlags.Translucency &&
		View.bHasDistortionPrimitives &&
		FSceneRenderer::GetRefractionQuality(*View.Family) > 0 &&
		DisableDistortion == 0;
}

BEGIN_SHADER_PARAMETER_STRUCT(FMobileDistortionPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileDistortionPassUniformParameters, Pass)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

TRDGUniformBufferRef<FMobileDistortionPassUniformParameters> CreateMobileDistortionPassUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	auto* Parameters = GraphBuilder.AllocParameters<FMobileDistortionPassUniformParameters>();

	EMobileSceneTextureSetupMode SetupMode = EMobileSceneTextureSetupMode::None;
	if (View.bCustomDepthStencilValid)
	{
		SetupMode |= EMobileSceneTextureSetupMode::CustomDepth;
	}

	if (MobileRequiresSceneDepthAux(View.GetShaderPlatform()))
	{
		SetupMode |= EMobileSceneTextureSetupMode::SceneDepthAux;
	}
	else
	{
		SetupMode |= EMobileSceneTextureSetupMode::SceneDepth;
	}

	SetupMobileSceneTextureUniformParameters(GraphBuilder, View.GetSceneTexturesChecked(), SetupMode, Parameters->SceneTextures);

	SetupDistortionParams(Parameters->DistortionParams, View);

	return GraphBuilder.CreateUniformBuffer(Parameters);
}

FMobileDistortionAccumulateOutputs AddMobileDistortionAccumulatePass(FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View, const FMobileDistortionAccumulateInputs& Inputs)
{
	const EPixelFormat Format = PF_B8G8R8A8;
	ETextureCreateFlags TextureCreateFlags = TexCreate_ShaderResource | TexCreate_RenderTargetable;
	if (UE::PixelFormat::HasCapabilities(Format, EPixelFormatCapabilities::LossyCompressible))
	{
		TextureCreateFlags |= TexCreate_LossyCompressionLowBitrate;
	}

	FRDGTextureDesc DistortionAccumulateDesc = FRDGTextureDesc::Create2D(Inputs.SceneColor.Texture->Desc.Extent, Format, FClearValueBinding::Transparent, TextureCreateFlags);

	FScreenPassRenderTarget DistortionAccumulateOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(DistortionAccumulateDesc, TEXT("DistortionAccumulatePass")), Inputs.SceneColor.ViewRect, ERenderTargetLoadAction::EClear);

	FMobileDistortionPassParameters* PassParameters = GraphBuilder.AllocParameters<FMobileDistortionPassParameters>();
	PassParameters->View = View.GetShaderParameters();
	PassParameters->Pass = CreateMobileDistortionPassUniformBuffer(GraphBuilder, View);
	PassParameters->RenderTargets[0] = DistortionAccumulateOutput.GetRenderTargetBinding();

	const FScreenPassTextureViewport SceneColorViewport(Inputs.SceneColor);

	auto* Pass = const_cast<FViewInfo&>(View).ParallelMeshDrawCommandPasses[EMeshPass::Distortion];
	if (Pass)
	{
		Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);
	}

	// Pass always added due to clear action.
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("DistortionAccumulate %dx%d", SceneColorViewport.Rect.Width(), SceneColorViewport.Rect.Height()),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, Pass, SceneColorViewport, PassParameters](FRHICommandList& RHICmdList)
	{
		if (Pass)
		{
			RHICmdList.SetViewport(SceneColorViewport.Rect.Min.X, SceneColorViewport.Rect.Min.Y, 0.0f, SceneColorViewport.Rect.Max.X, SceneColorViewport.Rect.Max.Y, 1.0f);
			Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
		}
	});

	FMobileDistortionAccumulateOutputs Outputs;

	Outputs.DistortionAccumulate = DistortionAccumulateOutput;

	return MoveTemp(Outputs);
}

class FMobileDistortionMergePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileDistortionMergePS);
	SHADER_USE_PARAMETER_STRUCT(FMobileDistortionMergePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistortionAccumulateTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistortionAccumulateSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileDistortionMergePS, "/Engine/Private/DistortApplyScreenPS.usf", "Merge_Mobile", SF_Pixel);

FScreenPassTexture AddMobileDistortionMergePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDistortionMergeInputs& Inputs)
{
	FRDGTextureDesc DistortionMergeDesc = FRDGTextureDesc::Create2D(Inputs.DistortionAccumulate.Texture->Desc.Extent, Inputs.SceneColor.Texture->Desc.Format, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_RenderTargetable);

	FScreenPassRenderTarget DistortionMergeOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(DistortionMergeDesc, TEXT("DistortionMergePass")), Inputs.DistortionAccumulate.ViewRect, ERenderTargetLoadAction::EClear);

	TShaderMapRef<FMobileDistortionMergePS> PixelShader(View.ShaderMap);

	FMobileDistortionMergePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileDistortionMergePS::FParameters>();
	PassParameters->RenderTargets[0] = DistortionMergeOutput.GetRenderTargetBinding();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters->SceneColorTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->DistortionAccumulateTexture = Inputs.DistortionAccumulate.Texture;
	PassParameters->DistortionAccumulateSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(DistortionMergeOutput);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DistortionMerge"), View, OutputViewport, InputViewport, PixelShader, PassParameters);

	return MoveTemp(DistortionMergeOutput);
}

void FMobileSceneRenderer::RenderDistortion(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef SceneDepthTexture)
{
	check(SceneDepthTexture);
	check(SceneColorTexture);

	QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRenderer_RenderDistortion);
	SCOPED_NAMED_EVENT(RenderDistortion, FColor::Emerald);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, Distortion, "Distortion");

	FRDGTextureRef DistortionTexture = nullptr;
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);
	const FDepthStencilBinding StencilReadBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
	FDepthStencilBinding StencilWriteBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);

	// Use stencil mask to optimize cases with lower screen coverage.
	// Note: This adds an extra pass which is actually slower as distortion tends towards full-screen.
	//       It could be worth testing object screen bounds then reverting to a target flip and single pass.
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRenderer_RenderDistortion_Accumulate);
		RDG_EVENT_SCOPE(GraphBuilder, "Accumulate");

		// Use RGBA8 light target for accumulating distortion offsets.
		// R = positive X offset
		// G = positive Y offset
		// B = negative X offset
		// A = negative Y offset
		DistortionTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(
				SceneDepthTexture->Desc.Extent,
				PF_B8G8R8A8,
				FClearValueBinding::Transparent,
				GFastVRamConfig.Distortion | TexCreate_RenderTargetable | TexCreate_ShaderResource,
				1,
				SceneDepthTexture->Desc.NumSamples),
			TEXT("Distortion"));

		ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::EClear;

		if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::Distortion])
		{
			FMobileDistortionPassParameters* PassParameters = GraphBuilder.AllocParameters<FMobileDistortionPassParameters>();
			PassParameters->View = View.GetShaderParameters();
			PassParameters->Pass = CreateMobileDistortionPassUniformBuffer(GraphBuilder, View);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(DistortionTexture, LoadAction);
			PassParameters->RenderTargets.DepthStencil = StencilWriteBinding;

			Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

			GraphBuilder.AddPass(
				{},
				PassParameters,
				ERDGPassFlags::Raster,
				[&View, Pass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRender_RenderDistortion_Accumulate_Meshes);
					SetStereoViewport(RHICmdList, View);
					Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
				});

			LoadAction = ERenderTargetLoadAction::ELoad;
		}

		if (LoadAction == ERenderTargetLoadAction::EClear)
		{
			AddClearRenderTargetPass(GraphBuilder, DistortionTexture);
		}
	}

	TShaderMapRef<FScreenPassVS> VertexShader(ShaderMap);
	FScreenPassPipelineState PipelineState(VertexShader, {});
	FScreenPassTextureViewport Viewport(SceneColorTexture);
	TShaderMapRef<FMobileDistortionMergePS> MergePixelShader(ShaderMap);

	FRDGTextureDesc DistortedSceneColorDesc = SceneColorTexture->Desc;
	//Remove fast clear flag on the DistoredSceneColor which is used in the Apply and Merge passes. 
	// This can save the Fast clear eliminate in the Merge pass when the RTV is transient allocated.
	EnumAddFlags(DistortedSceneColorDesc.Flags, TexCreate_NoFastClear);
	EnumRemoveFlags(DistortedSceneColorDesc.Flags, TexCreate_FastVRAM);
	FRDGTextureRef DistortionSceneColorTexture = GraphBuilder.CreateTexture(DistortedSceneColorDesc, TEXT("DistortedSceneColor"));

	// Apply distortion and store off-screen.
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRenderer_RenderDistortion_Apply);

		// Test against stencil mask but don't clear.
		PipelineState.DepthStencilState = TStaticDepthStencilState<
			false, CF_Always,
			true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			DISTORTION_STENCIL_MASK_BIT, DISTORTION_STENCIL_MASK_BIT>::GetRHI();
		PipelineState.StencilRef = DISTORTION_STENCIL_MASK_BIT;
		PipelineState.PixelShader = MergePixelShader;
		ERenderTargetLoadAction LoadAction = ERenderTargetLoadAction::ENoAction;

		FMobileDistortionMergePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileDistortionMergePS::FParameters>();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(DistortionSceneColorTexture, LoadAction);
		PassParameters->RenderTargets.DepthStencil = StencilReadBinding;
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneColorTexture = SceneColorTexture;
		PassParameters->SceneColorTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->DistortionAccumulateTexture = DistortionTexture;
		PassParameters->DistortionAccumulateSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Distortion::Apply"), FScreenPassViewInfo(View, 1), Viewport, Viewport, PipelineState, PassParameters,
			[MergePixelShader, PassParameters](FRHICommandList& RHICmdList)
			{
				SetShaderParameters(RHICmdList, MergePixelShader, MergePixelShader.GetPixelShader(), *PassParameters);
			});

		LoadAction = ERenderTargetLoadAction::ELoad;
	}

	// Merge distortion back to scene color.
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FSceneRenderer_RenderDistortion_Merge);
			
		TShaderMapRef<FCopyRectPS> CopyPixelShader(ShaderMap);

		// Test against stencil mask and clear it.
		PipelineState.DepthStencilState = TStaticDepthStencilState<
			false, CF_Always,
			true, CF_Equal, SO_Keep, SO_Keep, SO_Zero,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			DISTORTION_STENCIL_MASK_BIT, DISTORTION_STENCIL_MASK_BIT>::GetRHI();
		PipelineState.StencilRef = DISTORTION_STENCIL_MASK_BIT;
		PipelineState.PixelShader = CopyPixelShader;

		FCopyRectPS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
		Parameters->InputTexture = DistortionSceneColorTexture;
		Parameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ENoAction);
		Parameters->RenderTargets.DepthStencil = StencilWriteBinding;
		Parameters->InputSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Distortion::Merge"), FScreenPassViewInfo(View, 1), Viewport, Viewport, PipelineState, Parameters,
			[CopyPixelShader, Parameters](FRHICommandList& RHICmdList)
			{
				SetShaderParameters(RHICmdList, CopyPixelShader, CopyPixelShader.GetPixelShader(), *Parameters);
			});
	}
}