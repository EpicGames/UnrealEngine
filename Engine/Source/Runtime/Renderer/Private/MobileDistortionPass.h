// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "ScreenPass.h"

class FScene;

struct FMobileDistortionAccumulateInputs
{
	FScreenPassTexture SceneColor;
};

struct FMobileDistortionAccumulateOutputs
{
	FScreenPassTexture DistortionAccumulate;
};

FMobileDistortionAccumulateOutputs AddMobileDistortionAccumulatePass(FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View, const FMobileDistortionAccumulateInputs& Inputs);

struct FMobileDistortionMergeInputs
{
	FScreenPassTexture SceneColor;
	FScreenPassTexture DistortionAccumulate;
};

FScreenPassTexture AddMobileDistortionMergePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDistortionMergeInputs& Inputs);

// Returns whether distortion is enabled and there primitives to draw
bool IsMobileDistortionActive(const FViewInfo& View);

// Returns if the distortion is render before the translucent pass
bool IsMobileDistortionBeforeTranslucency();
