// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MobileTranslucentRendering.cpp: translucent rendering implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "RenderUtils.h"
#include "SceneRendering.h"
#include "ScenePrivate.h"
#include "FogRendering.h"
#include "PostProcess/SceneRenderTargets.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "MobileBasePassRendering.h"
#include "SkyAtmosphereRendering.h"
#include "LocalFogVolumeRendering.h"

DECLARE_GPU_STAT(MobileFog);

static TAutoConsoleVariable<int32> CVarPixelFogDepthTest(
	TEXT("r.Mobile.PixelFogDepthTest"),
	1,
	TEXT("Whether to use depth and stencil tests for fog rendering"),
	ECVF_RenderThreadSafe);


extern bool PlatformUsesBasicFogFeatures(EShaderPlatform ShaderPlatform);

class FMobileFogVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMobileFogVS);
	SHADER_USE_PARAMETER_STRUCT(FMobileFogVS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(float, StartDepthZ)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FMobileFogVS, "/Engine/Private/MobileFog.usf", "MobileFogVS", SF_Vertex);

class FMobileFogPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMobileFogPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileFogPS, FGlobalShader);
	
	class FSupportHeightFog							: SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_HEIGHT_FOG");
	class FSupportFogStartDistance					: SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_FOG_START_DISTANCE");
	class FSupportFogInScatteringTexture			: SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_FOG_INSCATTERING_TEXTURE");
	class FSupportFogSecondTerm						: SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_FOG_SECOND_TERM");
	class FSupportFogDirectionalLightInScattering	: SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_FOG_DIRECTIONAL_LIGHT_INSCATTERING");
	class FSupportAerialPerspective					: SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_AERIAL_PERSPECTIVE");
	class FSupportVolumetricFog						: SHADER_PERMUTATION_BOOL("PERMUTATION_SUPPORT_VOLUMETRIC_FOG");
	class FSupportLocalFogVolume					: SHADER_PERMUTATION_INT("PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME", 3); // 0 disabled, 1 sample LFVs, 2 compose halfres LFV texture
	
	using FPermutationDomain = TShaderPermutationDomain< 
		FSupportHeightFog, 
		FSupportFogStartDistance,
		FSupportFogInScatteringTexture, 
		FSupportFogSecondTerm,
		FSupportFogDirectionalLightInScattering,
		FSupportAerialPerspective,
		FSupportVolumetricFog,
		FSupportLocalFogVolume
	>;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FSupportHeightFog>() == false)
		{
			PermutationVector.Set<FSupportFogInScatteringTexture>(false);
			PermutationVector.Set<FSupportFogDirectionalLightInScattering>(false);
			PermutationVector.Set<FSupportFogStartDistance>(false);
			PermutationVector.Set<FSupportFogSecondTerm>(false);
		}
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsMobilePlatform(Parameters.Platform))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}
		
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const bool bMobileForceDepthRead = MobileUsesFullDepthPrepass(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("IS_MOBILE_DEPTHREAD_SUBPASS"), bMobileForceDepthRead ? 0u : 1u);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FLocalFogVolumeUniformParameters, LFV)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, HalfResLocalFogVolumeViewSRV)
		SHADER_PARAMETER_SAMPLER(SamplerState, HalfResLocalFogVolumeViewSRVSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, HalfResLocalFogVolumeDepthSRV)
		SHADER_PARAMETER_SAMPLER(SamplerState, HalfResLocalFogVolumeDepthSRVSampler)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_SHADER_TYPE(, FMobileFogPS, TEXT("/Engine/Private/MobileFog.usf"), TEXT("MobileFogPS"), SF_Pixel);

void FMobileSceneRenderer::RenderFog(FRHICommandList& RHICmdList, const FViewInfo& View)
{
	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, MobileFog, "MobileFog");

	// RenderFog has some extra logic to skip the rendering of fog. So we account for that inside this function using a lambda.
	bool bFogHasComposedLocalFogVolumes = false;
	auto RenderLocalFogVolumeMobileLambda = [&]()
	{
		if (!bFogHasComposedLocalFogVolumes)
		{
			RenderLocalFogVolumeMobile(RHICmdList, View);
		}
	};

	static const auto* CVarDisableVertexFog = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.DisableVertexFog"));
	if (CVarDisableVertexFog && CVarDisableVertexFog->GetValueOnRenderThread() == 0)
	{
		// Project uses only vertex fogging
		return;
	}

	const bool bSimpleFog = PlatformUsesBasicFogFeatures(View.GetShaderPlatform());
	const bool bUseHeightFog = Scene->ExponentialFogs.Num() > 0 && ShouldRenderFog(*View.Family);
	const bool bUseAerialPerspective = !bSimpleFog && ShouldRenderSkyAtmosphere(Scene, View.Family->EngineShowFlags);

	if (!bUseAerialPerspective && !bUseHeightFog)
	{
		RenderLocalFogVolumeMobileLambda();
		return;
	}

	const bool bUseDepthTest = CVarPixelFogDepthTest.GetValueOnRenderThread() != 0;
		
	SCOPED_DRAW_EVENT(RHICmdList, Fog);
		
	RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
	
	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	
	if (bUseDepthTest)
	{
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<
			false, CF_DepthNearOrEqual,
			true, CF_NotEqual, SO_Keep, SO_Keep, SO_Keep,
			false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
			STENCIL_MOBILE_SKY_MASK, 0x00>::GetRHI();
	}
	else
	{
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	}
	
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One,
													CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
													CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
													CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
													CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
													CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
													CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
													CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
		
	TShaderMapRef<FMobileFogVS> VertexShader(View.ShaderMap);

	const bool bUseFogStartDistance = View.ExponentialFogParameters.W > 0;
	const bool bUseFogInscatteringColorCubemap = View.FogInscatteringColorCubemap != nullptr;
	const bool bUseFogSecondTerm = !bSimpleFog && (View.ExponentialFogParameters2.Z > 0); // Fog Density in Z
	const bool bUseFogDirectionalInscatering = !bSimpleFog && !bUseFogInscatteringColorCubemap && (View.DirectionalInscatteringColor.GetLuminance() > 0 || View.bUseDirectionalInscattering);
	
	bFogHasComposedLocalFogVolumes = bShouldRenderVolumetricFog && ShouldRenderLocalFogVolume(Scene, *View.Family) && ShouldRenderLocalFogVolumeInVolumetricFog(Scene, *View.Family, bShouldRenderVolumetricFog);
	const bool bComposeHalfResLFV = View.LocalFogVolumeViewData.bUseHalfResLocalFogVolume;
	bFogHasComposedLocalFogVolumes |= bComposeHalfResLFV;

	FMobileFogPS::FPermutationDomain PsPermutationVector;
	PsPermutationVector.Set<FMobileFogPS::FSupportHeightFog>(bUseHeightFog);
	PsPermutationVector.Set<FMobileFogPS::FSupportFogStartDistance>(bUseFogStartDistance);
	PsPermutationVector.Set<FMobileFogPS::FSupportFogInScatteringTexture>(bUseFogInscatteringColorCubemap);
	PsPermutationVector.Set<FMobileFogPS::FSupportFogSecondTerm>(bUseFogSecondTerm);
	PsPermutationVector.Set<FMobileFogPS::FSupportAerialPerspective>(bUseAerialPerspective);
	PsPermutationVector.Set<FMobileFogPS::FSupportFogDirectionalLightInScattering>(bUseFogDirectionalInscatering);
	PsPermutationVector.Set<FMobileFogPS::FSupportVolumetricFog>(bShouldRenderVolumetricFog);
	PsPermutationVector.Set<FMobileFogPS::FSupportLocalFogVolume>(bFogHasComposedLocalFogVolumes ? (bComposeHalfResLFV ? 2 : 1) : 0);
	
	TShaderMapRef<FMobileFogPS> PixelShader(View.ShaderMap, PsPermutationVector);
		
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GTileVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	
	if (bUseDepthTest)
	{
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, STENCIL_MOBILE_SKY_MASK);
	}
	else
	{
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
	}
	
	// Use height fog start distance by default and fallback to AP distance
	float FogStartDistance = GetFogDefaultStartDistance();
	{
		if (bUseHeightFog)
		{
			FogStartDistance = GetViewFogCommonStartDistance(View, bShouldRenderVolumetricFog, bFogHasComposedLocalFogVolumes);
		}
		if (bUseAerialPerspective)
		{
			FSkyAtmosphereRenderSceneInfo& SkyInfo = *Scene->GetSkyAtmosphereSceneInfo();
			const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyInfo.GetSkyAtmosphereSceneProxy();
			const float AerialPerspectiveStartDepthInCm = GetValidAerialPerspectiveStartDepthInCm(View, SkyAtmosphereSceneProxy);
			FogStartDistance = bUseHeightFog ? FMath::Min(FogStartDistance, AerialPerspectiveStartDepthInCm) : AerialPerspectiveStartDepthInCm;
		}
	}
		
	float StartDepthZ = 0.1f;
	if (bUseDepthTest && FogStartDistance > 0.f)
	{
		const FMatrix ProjectionMatrix = View.ViewMatrices.GetViewToClip();
		float HalfHorizontalFOV = FMath::Atan(1.0f / ProjectionMatrix.M[0][0]);
		float HalfVerticalFOV = FMath::Atan(1.0f / ProjectionMatrix.M[1][1]);
		float StartDepthViewCm = FMath::Cos(FMath::Max(HalfHorizontalFOV, HalfVerticalFOV)) * FogStartDistance;
		StartDepthViewCm = FMath::Max(StartDepthViewCm, View.NearClippingDistance); // In any case, we need to limit the distance to frustum near plane to not be clipped away.
		const FVector4 Projected = ProjectionMatrix.TransformFVector4(FVector4(0.0f, 0.0f, StartDepthViewCm, 1.0f));
		StartDepthZ = (float)(Projected.Z / Projected.W); // LWC_TODO: precision loss
	}

	FMobileFogVS::FParameters VSParameters;
	VSParameters.View = View.GetShaderParameters();
	VSParameters.StartDepthZ = StartDepthZ;
	SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParameters);

	FMobileFogPS::FParameters PSParameters;
	PSParameters.View = View.GetShaderParameters();
	PSParameters.LFV = View.LocalFogVolumeViewData.UniformParametersStruct;
	PSParameters.HalfResLocalFogVolumeViewSRV = View.LocalFogVolumeViewData.HalfResLocalFogVolumeViewSRV;
	PSParameters.HalfResLocalFogVolumeDepthSRV = View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepthSRV;
	PSParameters.HalfResLocalFogVolumeViewSRVSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSParameters.HalfResLocalFogVolumeDepthSRVSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PSParameters);

	// Draw a quad covering the view.
	RHICmdList.SetStreamSource(0, GScreenSpaceVertexBuffer.VertexBufferRHI, 0);
	RHICmdList.DrawIndexedPrimitive(GTwoTrianglesIndexBuffer.IndexBufferRHI, 0, 0, 4, 0, 2, View.GetStereoPassInstanceFactor());

	RenderLocalFogVolumeMobileLambda();
}
