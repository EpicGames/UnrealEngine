// Copyright Epic Games, Inc. All Rights Reserved.

#include "MobileReflectionEnvironmentCapture.h"
#include "ReflectionEnvironmentCapture.h"
#include "ShaderParameterUtils.h"
#include "RHIStaticStates.h"
#include "PostProcess/SceneRenderTargets.h"
#include "SceneUtils.h"
#include "ScreenRendering.h"
#include "PipelineStateCache.h"
#include "PostProcess/SceneFilterRendering.h"
#include "OneColorShader.h"
#include "PixelShaderUtils.h"

/** Computes the average brightness of the given reflection capture and stores it in the scene. */
extern void ComputeSingleAverageBrightnessFromCubemap(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, float* OutAverageBrightness);

extern FRDGTexture* FilterCubeMap(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* SourceTexture);

extern void PremultiplyCubeMipAlpha(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, int32 MipIndex);

namespace MobileReflectionEnvironmentCapture
{
	void ComputeAverageBrightness(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, float* OutAverageBrightness)
	{
		CreateCubeMips(GraphBuilder, ShaderMap, CubemapTexture);
		ComputeSingleAverageBrightnessFromCubemap(GraphBuilder, ShaderMap, CubemapTexture, OutAverageBrightness);
	}

	/** Generates mips for glossiness and filters the cubemap for a given reflection. */
	FRDGTexture* FilterReflectionEnvironment(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, FSHVectorRGB3* OutIrradianceEnvironmentMap)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "FilterReflectionEnvironment");

		PremultiplyCubeMipAlpha(GraphBuilder, ShaderMap, CubemapTexture, 0);
		CreateCubeMips(GraphBuilder, ShaderMap, CubemapTexture);

		if (OutIrradianceEnvironmentMap)
		{
			ComputeDiffuseIrradiance(GraphBuilder, ShaderMap, CubemapTexture, OutIrradianceEnvironmentMap);
		}

		return FilterCubeMap(GraphBuilder, ShaderMap, CubemapTexture);
	}
}
