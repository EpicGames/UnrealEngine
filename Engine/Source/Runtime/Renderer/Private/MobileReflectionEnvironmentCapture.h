// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FGlobalShaderMap;
class FRDGBuilder;
class FRDGTexture;
class FScene;
class FTexture;

template<int32 MaxSHOrder> class TSHVectorRGB;
typedef TSHVectorRGB<3> FSHVectorRGB3;

namespace MobileReflectionEnvironmentCapture
{
	void ComputeAverageBrightness(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, float* OutAverageBrightness);
	FRDGTexture* FilterReflectionEnvironment(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, FSHVectorRGB3* OutIrradianceEnvironmentMap);
	void CopyToSkyTexture(FRDGBuilder& GraphBuilder, FScene* Scene, FRDGTexture* CubemapTexture, FTexture* ProcessedTexture);
}
