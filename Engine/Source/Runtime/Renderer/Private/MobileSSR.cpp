// Copyright Epic Games, Inc. All Rights Reserved.

#include "MobileSSR.h"

#include "RenderGraph.h"
#include "PixelShaderUtils.h"
#include "ScreenPass.h"
#include "PostProcess/SceneRenderTargets.h"
#include "SystemTextures.h"
#include "SceneViewState.h"

int32 GMobileScreenSpaceReflectionsSupported = 0;
static FAutoConsoleVariableRef CVarMobileScreenSpaceReflections(
	TEXT("r.Mobile.ScreenSpaceReflections"),
	GMobileScreenSpaceReflectionsSupported,
	TEXT("Caution: Affects only mobile Forward shading. Screen Space Reflections on mobile require TAA.\n")
	TEXT("0: Mobile Renderer Screen Space Reflections disabled (default)\n"
		 "1: Mobile Renderer Screen Space Reflections enabled\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

float GMobileMobileSSRIntensity = -1.0f;
static FAutoConsoleVariableRef CVarMobileMobileSSRIntensity(
	TEXT("r.Mobile.ScreenSpaceReflections.Intensity"),
	GMobileMobileSSRIntensity,
	TEXT("Scale factor to adjust the intensity of mobile screen space reflections in the range [0.0, 1.0] or -1. (default: -1, ignores this setting)\n"),
	ECVF_RenderThreadSafe);

bool IsMobileSSREnabled(const FViewInfo& View)
{
	const EShaderPlatform Platform = View.GetShaderPlatform();
	
	bool bDeferredWithFullDepth = false;
	if (IsMobileDeferredShadingEnabled(Platform))
	{
		bDeferredWithFullDepth = (MobileUsesFullDepthPrepass(Platform) || !MobileAllowFramebufferFetch(Platform));
	}
	return (GMobileScreenSpaceReflectionsSupported || bDeferredWithFullDepth) && ScreenSpaceRayTracing::ShouldRenderScreenSpaceReflections(View) && (GMobileMobileSSRIntensity < 0.0f || GMobileMobileSSRIntensity > 0.0f);
}

EMobileSSRQuality ActiveMobileSSRQuality(const FViewInfo& View, bool bHasVelocityTexture)
{
	if (!IsMobileSSREnabled(View))
	{
		return EMobileSSRQuality::Disabled;
	}
	ESSRQuality SSRQuality;
	IScreenSpaceDenoiser::FReflectionsRayTracingConfig DenoiserConfig;
	ScreenSpaceRayTracing::GetSSRQualityForView(View, &SSRQuality, &DenoiserConfig);
	if (SSRQuality < ESSRQuality::Low)
	{
		return EMobileSSRQuality::Disabled;
	}
	if (!View.PrevViewInfo.TemporalAAHistory.RT[0])
	{
		return EMobileSSRQuality::Disabled;
	}

	if (SSRQuality >= ESSRQuality::Medium && bHasVelocityTexture)
	{
		return EMobileSSRQuality::Medium;
	}

	return  EMobileSSRQuality::Low;
}

void SetupMobileSSRParameters(FRDGBuilder& GraphBuilder,const FViewInfo& View, FMobileScreenSpaceReflectionParams& Params)
{
	if (!IsMobileSSREnabled(View) || !View.PrevViewInfo.TemporalAAHistory.RT[0] || !(IsHZBValid(View, EHZBType::FurthestHZB) || IsPreviousHZBValid(View, EHZBType::FurthestHZB)))
	{
		const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
		Params.HZBParameters = GetDummyHZBParameters(GraphBuilder);
		Params.SceneColor = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(SystemTextures.Black));
		Params.SceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Params.PrevSceneColorBilinearUVMinMax = FVector4f(0.0f, 0.0f, 1.0f, 1.0f);
		Params.IntensityAndExposureCorrection = FVector4f(ForceInitToZero);
		return;
	}
	FRDGTextureRef SceneColor = GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.TemporalAAHistory.RT[0]);
	Params.SceneColor = GraphBuilder.CreateSRV(SceneColor->Desc.IsTextureArray()
		? FRDGTextureSRVDesc::CreateForSlice(SceneColor, View.PrevViewInfo.TemporalAAHistory.OutputSliceIndex)
		: FRDGTextureSRVDesc(SceneColor));
	Params.SceneColorSampler = TStaticSamplerState<SF_Point>::GetRHI();

	Params.HZBParameters = GetHZBParameters(GraphBuilder, View, true /*bUsePreviousHZBAsFallback*/);

	{
		ensure(View.PrevViewInfo.TemporalAAHistory.IsValid());
		FIntPoint ViewportOffset = View.PrevViewInfo.TemporalAAHistory.ViewportRect.Min;
		FIntPoint ViewportExtent = View.PrevViewInfo.TemporalAAHistory.ViewportRect.Size();
		FIntPoint BufferSize = View.PrevViewInfo.TemporalAAHistory.ReferenceBufferSize;
		ensure(ViewportExtent.X > 0 && ViewportExtent.Y > 0);
		ensure(BufferSize.X > 0 && BufferSize.Y > 0);

		FVector2D InvBufferSize(1.0f / float(BufferSize.X), 1.0f / float(BufferSize.Y));
		Params.PrevScreenPositionScaleBias = FVector4f(
			ViewportExtent.X * 0.5f * InvBufferSize.X,
			-ViewportExtent.Y * 0.5f * InvBufferSize.Y,
			(ViewportExtent.X * 0.5f + ViewportOffset.X) * InvBufferSize.X,
			(ViewportExtent.Y * 0.5f + ViewportOffset.Y) * InvBufferSize.Y);
	}

	{
		ESSRQuality SSRQuality;
		IScreenSpaceDenoiser::FReflectionsRayTracingConfig DenoiserConfig;
		ScreenSpaceRayTracing::GetSSRQualityForView(View, &SSRQuality, &DenoiserConfig);

		const float MobileMobileSSRIntensity = GMobileMobileSSRIntensity >= 0.0f ? GMobileMobileSSRIntensity : 1.0f;
		Params.IntensityAndExposureCorrection.X = SSRQuality > ESSRQuality::VisualizeSSR ? FMath::Clamp(View.FinalPostProcessSettings.ScreenSpaceReflectionIntensity * 0.01f * MobileMobileSSRIntensity, 0.0f, 1.0f) : 0.0f;
		Params.IntensityAndExposureCorrection.Y = 1.f / View.PrevViewInfo.SceneColorPreExposure;
		float MaxRoughness = FMath::Clamp(View.FinalPostProcessSettings.ScreenSpaceReflectionMaxRoughness, 0.01f, 0.6f);
		MaxRoughness *= 0.5;
		Params.IntensityAndExposureCorrection.Z = MaxRoughness;
		Params.IntensityAndExposureCorrection.W = 2.0 / MaxRoughness;
	}

	{
		FScreenPassTextureViewportParameters PrevSceneColorParameters = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(SceneColor->Desc.Extent, View.PrevViewInfo.TemporalAAHistory.ViewportRect));
		Params.PrevSceneColorBilinearUVMinMax.X = PrevSceneColorParameters.UVViewportBilinearMin.X;
		Params.PrevSceneColorBilinearUVMinMax.Y = PrevSceneColorParameters.UVViewportBilinearMin.Y;
		Params.PrevSceneColorBilinearUVMinMax.Z = PrevSceneColorParameters.UVViewportBilinearMax.X;
		Params.PrevSceneColorBilinearUVMinMax.W = PrevSceneColorParameters.UVViewportBilinearMax.Y;
	}

	switch (View.AntiAliasingMethod)
	{
	default:
		// Without TAA disable temporal noise and reduce intensity of SSR to hide the noise.
		Params.NoiseIndex = 0;
		if (GMobileMobileSSRIntensity < 0.0f)
		{
			Params.IntensityAndExposureCorrection.X = FMath::Min(Params.IntensityAndExposureCorrection.X, 0.4f);
		}
		break;
	case AAM_TemporalAA:
	case AAM_TSR:
		Params.NoiseIndex = View.ViewState ? View.ViewState->GetFrameIndex() % 8 : 0;
		break;
	}
}

