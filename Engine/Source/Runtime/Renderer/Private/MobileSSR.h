// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphDefinitions.h"
#include "SceneRendering.h"
#include "ScreenSpaceRayTracing.h"
#include "HZB.h"

BEGIN_SHADER_PARAMETER_STRUCT(FMobileScreenSpaceReflectionParams, )
SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SceneColor)
SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
SHADER_PARAMETER(FVector4f, PrevScreenPositionScaleBias)
SHADER_PARAMETER(FVector4f, PrevSceneColorBilinearUVMinMax)
SHADER_PARAMETER(FVector4f, IntensityAndExposureCorrection) // .x = Intensity, .y = PrevSceneColorPreExposureInv, .z = View.FinalPostProcessSettings.ScreenSpaceReflectionMaxRoughness, .w = 2DivMaxRoughness
SHADER_PARAMETER(uint32, NoiseIndex)
END_SHADER_PARAMETER_STRUCT()

enum class EMobileSSRQuality
{
	Disabled,
	Low,
	Medium,
	MAX
};

bool IsMobileSSREnabled(const FViewInfo& View);
EMobileSSRQuality ActiveMobileSSRQuality(const FViewInfo& View, bool bHasVelocityTexture);
void SetupMobileSSRParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FMobileScreenSpaceReflectionParams& Params);
