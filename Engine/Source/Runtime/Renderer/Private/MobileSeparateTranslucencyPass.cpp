// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
MobileSeparateTranslucencyPass.cpp - Mobile specific separate translucency pass
=============================================================================*/

#include "MobileSeparateTranslucencyPass.h"
#include "TranslucentRendering.h"
#include "DynamicPrimitiveDrawing.h"
#include "MobileBasePassRendering.h"
#include "ScenePrivate.h"

BEGIN_SHADER_PARAMETER_STRUCT(FMobileSeparateTranslucencyPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileBasePassUniformParameters, MobileBasePass)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

bool IsMobileSeparateTranslucencyActive(const FViewInfo* Views, int32 NumViews)
{
	for (int32 ViewIdx = 0; ViewIdx < NumViews; ++ViewIdx)
	{
		if (IsMobileSeparateTranslucencyActive(Views[ViewIdx]))
		{
			return true;
		}
	}
	return false;
}

bool IsMobileSeparateTranslucencyActive(const FViewInfo& View)
{
	return HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::TranslucencyAfterDOF]);
}

void AddMobileSeparateTranslucencyPass(FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View, const FMobileSeparateTranslucencyInputs& Inputs)
{
	FMobileSeparateTranslucencyPassParameters* PassParameters = GraphBuilder.AllocParameters<FMobileSeparateTranslucencyPassParameters>();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(Inputs.SceneColor.Texture, ERenderTargetLoadAction::ELoad);
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Inputs.SceneDepth.Texture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
	PassParameters->View = View.ViewUniformBuffer;
	EMobileSceneTextureSetupMode SetupMode = EMobileSceneTextureSetupMode::SceneDepth | EMobileSceneTextureSetupMode::SceneDepthAux | EMobileSceneTextureSetupMode::CustomDepth;
	PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, EMobileBasePass::Translucent, SetupMode);

	auto* Pass = const_cast<FViewInfo&>(View).ParallelMeshDrawCommandPasses[EMeshPass::TranslucencyAfterDOF];
	if (Pass)
	{
		Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("SeparateTranslucency %dx%d", View.ViewRect.Width(), View.ViewRect.Height()),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, Pass, PassParameters](FRHICommandList& RHICmdList)
			{
				// Set the view family's render target/viewport.
				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
				Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
			});
	}

}