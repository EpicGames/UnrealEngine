// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "ScreenPass.h"

class FScene;

struct FMobileSeparateTranslucencyInputs
{
	FScreenPassTexture SceneColor;
	FScreenPassTexture SceneDepthAux;
	FScreenPassTexture SceneDepth;
};

void AddMobileSeparateTranslucencyPass(FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View, const FMobileSeparateTranslucencyInputs& Inputs);

// Returns whether separate translucency is enabled and there primitives to draw in the view
bool IsMobileSeparateTranslucencyActive(const FViewInfo& View);
bool IsMobileSeparateTranslucencyActive(const FViewInfo* Views, int32 NumViews);