// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MobileShadingRenderer.cpp: Scene rendering code for ES3/3.1 feature level.
=============================================================================*/

#include "CoreMinimal.h"
#include "BlendableManager.h"
#include "ClearQuad.h"
#include "CommonRenderResources.h"
#include "Containers/ArrayView.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "Containers/StridedView.h"
#include "CustomRenderPassSceneCapture.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DBufferTextures.h"
#include "DebugViewModeRendering.h"
#include "DecalRenderingCommon.h"
#include "DecalRenderingShared.h"
#include "DepthCopy.h"
#include "DepthRendering.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "DistanceFieldLightingShared.h"
#include "Engine/BlendableInterface.h"
#include "Engine/Engine.h"
#include "Engine/Scene.h"
#include "Engine/SpecularProfile.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/ToonProfile.h"
#include "Engine/World.h"
#include "EngineModule.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "FinalPostProcessSettings.h"
#include "FXSystem.h"
#include "GBufferInfo.h"
#include "GenerateMips.h"
#include "GlobalShader.h"
#include "GPUMessaging.h"
#include "GPUScene.h"
#include "GPUSortManager.h"
#include "HairStrands/HairStrandsData.h"
#include "HairStrands/HairStrandsRendering.h"
#include "HairStrandsInterface.h"
#include "HAL/IConsoleManager.h"
#include "HighResScreenshot.h"
#include "HitProxies.h"
#include "IESTextureManager.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "InstanceCulling/InstanceCullingOcclusionQuery.h"
#include "LightFunctionAtlas.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "LocalFogVolumeRendering.h"
#include "Logging/LogMacros.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialSceneTextureId.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "MeshDrawCommands.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.h"
#include "Misc/ScopeExit.h"
#include "MobileBasePassRendering.h"
#include "MobileDeferredShadingPass.h"
#include "MobileDistortionPass.h"
#include "MobileSeparateTranslucencyPass.h"
#include "MobileSSR.h"
#include "MultiGPU.h"
#include "OIT/OIT.h"
#include "PipelineStateCache.h"
#include "PlanarReflectionSceneProxy.h"
#include "PostProcess/AlphaInvert.h"
#include "PostProcess/PostProcessAmbientOcclusionMobile.h"
#include "PostProcess/PostProcessCombineLUTs.h"
#include "PostProcess/PostProcessCompositeDebugPrimitives.h"
#include "PostProcess/PostProcessEyeAdaptation.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessTonemap.h"
#include "PostProcess/SceneFilterRendering.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "RectLightTextureManager.h"
#include "ReflectionEnvironment.h"
#include "RenderCore.h"
#include "RendererInterface.h"
#include "RendererModule.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/CustomRenderPass.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneCaptureRendering.h"
#include "SceneCulling/SceneCullingRenderer.h"
#include "SceneExtensions.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "SceneOcclusion.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextureReductions.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneUniformBuffer.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "SceneVisibility.h"
#include "ScreenPass.h"
#include "ScreenRendering.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderPrint.h"
#include "ShadingEnergyConservation.h"
#include "ShowFlags.h"
#include "SingleLayerWaterRendering.h"
#include "SkyAtmosphereRendering.h"
#include "Stats/Stats.h"
#include "StereoRenderUtils.h"
#include "Substrate/Glint/GlintShadingLUTs.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "TranslucencyPass.h"
#include "TranslucentPassResource.h"
#include "TranslucentRendering.h"
#include "UnrealClient.h"
#include "VariableRateShadingImageManager.h"
#include "VelocityRendering.h"
#include "ViewData.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VT/VirtualTextureFeedbackResource.h"
#include "VT/VirtualTextureSystem.h"
#include "WaterInfoTextureRendering.h"
#include "SceneViewState.h"

CSV_DECLARE_CATEGORY_EXTERN(LightCount);

uint32 GetShadowQuality();

static TAutoConsoleVariable<int32> CVarMobileForceDepthResolve(
	TEXT("r.Mobile.ForceDepthResolve"),
	0,
	TEXT("0: Depth buffer is resolved by switching out render targets. (Default)\n")
	TEXT("1: Depth buffer is resolved by switching out render targets and drawing with the depth texture.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileAdrenoOcclusionMode(
	TEXT("r.Mobile.AdrenoOcclusionMode"),
	0,
	TEXT("0: Render occlusion queries after the base pass (default).\n")
	TEXT("1: Render occlusion queries after translucency and a flush, which can help Adreno devices in GL mode."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileCustomDepthForTranslucency(
	TEXT("r.Mobile.CustomDepthForTranslucency"),
	1,
	TEXT(" Whether to render custom depth/stencil if any tranclucency in the scene uses it. \n")
	TEXT(" 0 = Off \n")
	TEXT(" 1 = On [default]"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileXRMSAAMode(
	TEXT("r.Mobile.XRMSAAMode"),
	0,
	TEXT(" Whether to modify how mobile XR msaa support works\n")
	TEXT(" 0 = Standard depth pass/swapchain mode [default]\n")
	TEXT(" 1 = Perform a copy of depth to the depth resolve target")
	TEXT(" 2 = Make the depth swap chain be MSAA and use it directly as scene depth"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> GAdrenoOcclusionUseFDM(
	TEXT("r.AdrenoOcclusionUseFDM"),
	0,
	TEXT("Use FDM with adreno occlusion mode"),
	ECVF_RenderThreadSafe);

DECLARE_GPU_STAT_NAMED(MobileSceneRender, TEXT("Mobile Scene Render"));
DECLARE_GPU_STAT_NAMED(CapMobileSceneRender, TEXT("Mobile Scene Render (Cap)"));

extern bool IsMobileEyeAdaptationEnabled(const FViewInfo& View);

struct FMobileCustomDepthStencilUsage
{
	bool bUsesCustomDepthStencil = false;
	// whether CustomStencil is sampled as a textures
	bool bSamplesCustomStencil = false;
};

static FMobileCustomDepthStencilUsage GetCustomDepthStencilUsage(const FViewInfo& View)
{
	FMobileCustomDepthStencilUsage CustomDepthStencilUsage;

	// Find out whether there are primitives will render in custom depth pass or just always render custom depth
	if ((View.bHasCustomDepthPrimitives || GetCustomDepthMode() == ECustomDepthMode::EnabledWithStencil))
	{
		// Find out whether CustomDepth/Stencil used in translucent materials
		if (CVarMobileCustomDepthForTranslucency.GetValueOnAnyThread() != 0)
		{
			CustomDepthStencilUsage.bUsesCustomDepthStencil = View.bUsesCustomDepth || View.bUsesCustomStencil;
			CustomDepthStencilUsage.bSamplesCustomStencil = View.bUsesCustomStencil;
		}

		if (!CustomDepthStencilUsage.bSamplesCustomStencil)
		{
			// Find out whether post-process materials use CustomDepth/Stencil lookups
			const FBlendableManager& BlendableManager = View.FinalPostProcessSettings.BlendableManager;
			FBlendableEntry* BlendableIt = nullptr;
			while (FPostProcessMaterialNode* DataPtr = BlendableManager.IterateBlendables<FPostProcessMaterialNode>(BlendableIt))
			{
				if (DataPtr->IsValid())
				{
					FMaterialRenderProxy* Proxy = DataPtr->GetMaterialInterface()->GetRenderProxy();
					check(Proxy);

					const FMaterial& Material = Proxy->GetIncompleteMaterialWithFallback(View.GetFeatureLevel());
					const FMaterialShaderMap* MaterialShaderMap = Material.GetRenderingThreadShaderMap();
					bool bUsesCustomDepth = MaterialShaderMap->UsesSceneTexture(PPI_CustomDepth);
					bool bUsesCustomStencil = MaterialShaderMap->UsesSceneTexture(PPI_CustomStencil);
					if (Material.IsStencilTestEnabled() || bUsesCustomDepth || bUsesCustomStencil)
					{
						CustomDepthStencilUsage.bUsesCustomDepthStencil |= true;
					}

					if (bUsesCustomStencil)
					{
						CustomDepthStencilUsage.bSamplesCustomStencil |= true;
						break;
					}
				}
			}
		}
		
		FHighResScreenshotConfig& Config = GetHighResScreenshotConfig();

		// If the highres screenshot UI is open and we're not taking a highres screenshot this frame
		if (
	#if WITH_EDITOR
			Config.bDisplayCaptureRegion ||
	#endif	
			GIsHighResScreenshot)
		{
			CustomDepthStencilUsage.bUsesCustomDepthStencil |= Config.bMaskEnabled;
			CustomDepthStencilUsage.bSamplesCustomStencil |= Config.bMaskEnabled;
		}
	}

	return CustomDepthStencilUsage;
}

static void RenderOpaqueFX(
	FRDGBuilder& GraphBuilder,
	TConstStridedView<FSceneView> Views,
	FSceneUniformBuffer &SceneUniformBuffer,
	FFXSystemInterface* FXSystem,
	TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> MobileSceneTexturesUniformBuffer)
{
	// Notify the FX system that opaque primitives have been rendered and we now have a valid depth buffer.
	if (FXSystem && Views.Num() > 0)
	{
		FXSystem->PostRenderOpaque(GraphBuilder, Views, SceneUniformBuffer, true /*bAllowGPUParticleUpdate*/);

		if (FGPUSortManager* GPUSortManager = FXSystem->GetGPUSortManager())
		{
			GPUSortManager->OnPostRenderOpaque(GraphBuilder);
		}
	}
}

static void BuildMeshRenderingCommands(FRDGBuilder& GraphBuilder, EMeshPass::Type MeshPass, FViewInfo& View, const FGPUScene& GPUScene, FInstanceCullingManager& InstanceCullingManager, FInstanceCullingDrawParams& OutParams)
{
	if (auto* Pass = View.ParallelMeshDrawCommandPasses[MeshPass])
	{
		Pass->BuildRenderingCommands(GraphBuilder, GPUScene, OutParams);

		// When batching is disabled instead of a single UniformBuffer we get a separate buffers for each mesh pass
		// Because mobile renderer manually merges several mesh passes into a single RDG pass we can't specify InstanceCullingDrawParams for each mesh pass through RDG pass parameters (only one)
		// We do these dummy RDG passes to make sure InstanceCullingDrawParams are initialized for each mesh pass
		if (!FInstanceCullingManager::AllowBatchedBuildRenderingCommands(GPUScene))
		{
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("SetupInstanceCullingDrawParams"),
				&OutParams,
				ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass | ERDGPassFlags::NeverCull, [](FRHICommandList&){});
		}
	}
	else
	{
		InstanceCullingManager.SetDummyCullingParams(GraphBuilder, OutParams);
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FMobileRenderPassParameters,)
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileBasePassUniformParameters, MobileBasePass)
	SHADER_PARAMETER_STRUCT_REF(FMobileReflectionCaptureShaderData, ReflectionCapture)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, LocalFogVolumeInstances)
	RDG_BUFFER_ACCESS(LocalFogVolumeTileDrawIndirectBuffer, ERHIAccess::IndirectArgs)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<uint>, LocalFogVolumeTileDataTexture)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, LocalFogVolumeTileDataBuffer)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, HalfResLocalFogVolumeViewSRV)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, HalfResLocalFogVolumeDepthSRV)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BentNormalAOTexture)
	RDG_TEXTURE_ACCESS(ColorGradingLUT, ERHIAccess::SRVGraphics)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static bool PostProcessUsesSceneDepth(const FViewInfo& View)
{
	if ((View.FinalPostProcessSettings.DepthOfFieldScale > 0.0f && View.Family->EngineShowFlags.DepthOfField)
		|| View.MobileLightShaft.IsSet())
	{
		return true;
	}
	
	// Find out whether post-process materials use CustomDepth/Stencil lookups
	const FBlendableManager& BlendableManager = View.FinalPostProcessSettings.BlendableManager;
	FBlendableEntry* BlendableIt = nullptr;

	while (FPostProcessMaterialNode* DataPtr = BlendableManager.IterateBlendables<FPostProcessMaterialNode>(BlendableIt))
	{
		if (DataPtr->IsValid())
		{
			FMaterialRenderProxy* Proxy = DataPtr->GetMaterialInterface()->GetRenderProxy();
			check(Proxy);

			const FMaterial& Material = Proxy->GetIncompleteMaterialWithFallback(View.GetFeatureLevel());
			const FMaterialShaderMap* MaterialShaderMap = Material.GetRenderingThreadShaderMap();
			if (MaterialShaderMap->UsesSceneTexture(PPI_SceneDepth))
			{
				return true;
			}
		}
	}
	
	return IsMobileDistortionActive(View) || View.bIsSceneCapture;
}

struct FRenderViewContext
{
	FViewInfo* ViewInfo;
	int32 ViewIndex;
	int32 DecalViewOffset;
	bool bIsFirstView;
	bool bIsLastView;
};
using FRenderViewContextArray = TArray<FRenderViewContext, TInlineAllocator<2, SceneRenderingAllocator>>;

static void GetRenderViews(TArrayView<FViewInfo> InViews, int32 DecalViewOffset, FRenderViewContextArray& RenderViews)
{
	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
	{
		FViewInfo& View = InViews[ViewIndex];
		if (View.ShouldRenderView())
		{
			FRenderViewContext RenderView;
			RenderView.ViewInfo = &View;
			RenderView.ViewIndex = ViewIndex;
			RenderView.DecalViewOffset = DecalViewOffset;
			RenderView.bIsFirstView = (RenderViews.Num() == 0);
			RenderView.bIsLastView = false;

			RenderViews.Add(RenderView);
		}
	}

	if (RenderViews.Num())
	{
		RenderViews.Last().bIsLastView = true;
	}
}

static bool IsShadowOnlyTimeslice_GT(const FSceneViewFamily* InViewFamily, FScene* Scene)
{
	check(IsInGameThread());
	if (!InViewFamily->Views.Num())
	{
		return false;
	}
	return InViewFamily->Views[0]->IsRuntimeReflectionCapture() && Scene->ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT == 0;
}

static bool IsShadowOnlyTimeslice_RT(const FSceneViewFamily* InViewFamily, FScene* Scene)
{
	check(IsInParallelRenderingThread());
	if (!InViewFamily->Views.Num())
	{
		return false;
	}
	return InViewFamily->Views[0]->IsRuntimeReflectionCapture() && Scene->ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_RT == 0;
}

FMobileSceneRenderer::FMobileSceneRenderer(const FSceneViewFamily* InViewFamily, FHitProxyConsumer* HitProxyConsumer)
	: FSceneRenderer(InViewFamily, HitProxyConsumer)
	, bGammaSpace(!IsMobileHDR())
	, bDeferredShading(IsMobileDeferredShadingEnabled(ShaderPlatform))
	, bRequiresDBufferDecals(IsUsingDBuffers(ShaderPlatform))
	, bUseVirtualTexturing(UseVirtualTexturing(ShaderPlatform) && GetRendererOutput() != FSceneRenderer::ERendererOutput::DepthPrepassOnly && !IsShadowOnlyTimeslice_GT(InViewFamily, Scene))
	, bSupportsSimpleLights(bDeferredShading || MobileForwardEnableParticleLights(ShaderPlatform))
{
	bRenderToSceneColor = false;
	bRequiresMultiPass = false;
	bKeepDepthContent = false;
	bModulatedShadowsInUse = false;
	bShouldRenderCustomDepth = false;
	bRequiresShadowProjections = false;
	bEnableDistanceFieldAO = false;
	bShouldRenderVolumetricFog = false;
	bIsFullDepthPrepassEnabled = (Scene->EarlyZPassMode == DDM_AllOpaque || Scene->EarlyZPassMode == DDM_AllOpaqueNoVelocity);
	bIsMaskedOnlyDepthPrepassEnabled = Scene->EarlyZPassMode == DDM_MaskedOnly;
	bEnableClusteredLocalLights = MobileForwardEnableLocalLights(ShaderPlatform);
	bEnableClusteredReflections = MobileForwardEnableClusteredReflections(ShaderPlatform);
		
	StandardTranslucencyPass = ViewFamily.AllowTranslucencyAfterDOF() ? ETranslucencyPass::TPT_TranslucencyStandard : ETranslucencyPass::TPT_AllTranslucency;
	StandardTranslucencyMeshPass = TranslucencyPassToMeshPass(StandardTranslucencyPass);

	// Don't do occlusion queries when doing scene captures
	for (FViewInfo& View : Views)
	{
		if (View.bIsSceneCapture)
		{
			View.bDisableQuerySubmissions = true;
			View.bIgnoreExistingQueries = true;
		}
	}

	NumMSAASamples = GetDefaultMSAACount(ERHIFeatureLevel::ES3_1);
	// As of UE 5.4 only vulkan supports inline (single pass) tonemap
	bTonemapSubpass = IsMobileTonemapSubpassEnabled(ShaderPlatform, ViewFamily.bRequireMultiView) && ViewFamily.bResolveScene && GetRendererOutput() != FSceneRenderer::ERendererOutput::DepthPrepassOnly;
	bTonemapSubpassInline = IsMobileTonemapSubpassEnabledInline(ShaderPlatform, ViewFamily.bRequireMultiView, NumMSAASamples) && bTonemapSubpass;
	bRequiresSceneDepthAux = MobileRequiresSceneDepthAux(ShaderPlatform) && !bTonemapSubpass;

	// Initialize scene renderer extensions here, after the rest of the renderer has been initialized
	InitSceneExtensionsRenderers(ViewFamily.EngineShowFlags, true);
}

class FMobileDirLightShaderParamsRenderResource : public FRenderResource
{
public:
	using MobileDirLightUniformBufferRef = TUniformBufferRef<FMobileDirectionalLightShaderParameters>;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		UniformBufferRHI =
			MobileDirLightUniformBufferRef::CreateUniformBufferImmediate(
				FMobileDirectionalLightShaderParameters(),
				UniformBuffer_MultiFrame);
	}

	virtual void ReleaseRHI() override
	{
		UniformBufferRHI.SafeRelease();
	}

	MobileDirLightUniformBufferRef UniformBufferRHI;
};

TUniformBufferRef<FMobileDirectionalLightShaderParameters>& GetNullMobileDirectionalLightShaderParameters()
{
	static TGlobalResource<FMobileDirLightShaderParamsRenderResource>* NullLightParams;
	if (!NullLightParams)
	{
		NullLightParams = new TGlobalResource<FMobileDirLightShaderParamsRenderResource>();
	}
	check(!!NullLightParams->UniformBufferRHI);
	return NullLightParams->UniformBufferRHI;
}

void FMobileSceneRenderer::PrepareViewVisibilityLists()
{
	// Prepare view's visibility lists.
	// TODO: only do this when CSM + static is required.
	for (auto& View : Views)
	{
		FMobileCSMVisibilityInfo& MobileCSMVisibilityInfo = View.MobileCSMVisibilityInfo;
		// Init list of primitives that can receive Dynamic CSM.
		MobileCSMVisibilityInfo.MobilePrimitiveCSMReceiverVisibilityMap.Init(false, View.PrimitiveVisibilityMap.Num());

		// Init static mesh visibility info for CSM drawlist
		MobileCSMVisibilityInfo.MobileCSMStaticMeshVisibilityMap.Init(false, View.StaticMeshVisibilityMap.Num());

		// Init static mesh visibility info for default drawlist that excludes meshes in CSM only drawlist.
		MobileCSMVisibilityInfo.MobileNonCSMStaticMeshVisibilityMap = View.StaticMeshVisibilityMap;
	}
}

void FMobileSceneRenderer::SetupMobileBasePassAfterShadowInit(FExclusiveDepthStencil::Type BasePassDepthStencilAccess, TArrayView<FViewCommands> ViewCommandsPerView, FInstanceCullingManager& InstanceCullingManager)
{
	// Sort front to back on all platforms, even HSR benefits from it
	//const bool bWantsFrontToBackSorting = (GHardwareHiddenSurfaceRemoval == false);

	// compute keys for front to back sorting and dispatch pass setup.
	for (int32 ViewIndex = 0; ViewIndex < AllViews.Num(); ++ViewIndex)
	{
		FViewInfo& View = *AllViews[ViewIndex];
		FViewCommands& ViewCommands = ViewCommandsPerView[ViewIndex];

		if (ViewCommands.MeshCommands[EMeshPass::BasePass].IsEmpty() && ViewCommands.MeshCommands[EMeshPass::MobileBasePassCSM].IsEmpty() && View.NumVisibleDynamicMeshElements[EMeshPass::BasePass] == 0 && ViewCommands.NumDynamicMeshCommandBuildRequestElements[EMeshPass::BasePass] == 0)
		{
			continue;
		}

		FMeshPassProcessor* MeshPassProcessor = FPassProcessorManager::CreateMeshPassProcessor(EShadingPath::Mobile, EMeshPass::BasePass, Scene->GetFeatureLevel(), Scene, &View, nullptr);

		FMeshPassProcessor* BasePassCSMMeshPassProcessor = FPassProcessorManager::CreateMeshPassProcessor(EShadingPath::Mobile, EMeshPass::MobileBasePassCSM, Scene->GetFeatureLevel(), Scene, &View, nullptr);

		TArray<int32, TInlineAllocator<2> > ViewIds;
		ViewIds.Add(View.SceneRendererPrimaryViewId);
		// Only apply instancing for ISR to main view passes
		EInstanceCullingMode InstanceCullingMode = View.IsInstancedStereo() ? EInstanceCullingMode::Stereo : EInstanceCullingMode::Normal;
		if (InstanceCullingMode == EInstanceCullingMode::Stereo)
		{
			check(View.GetInstancedView() != nullptr);
			ViewIds.Add(View.GetInstancedView()->SceneRendererPrimaryViewId);
		}

		// Run sorting on BasePass, as it's ignored inside FSceneRenderer::SetupMeshPass, so it can be done after shadow init on mobile.
		FParallelMeshDrawCommandPass& Pass = *View.CreateMeshPass(EMeshPass::BasePass);
		if (ShouldDumpMeshDrawCommandInstancingStats())
		{
			Pass.SetDumpInstancingStats(GetMeshPassName(EMeshPass::BasePass));
		}

		FSceneCullingRenderer* SceneCullingRenderer = GetSceneExtensionsRenderers().GetRendererPtr<FSceneCullingRenderer>();
		FSceneCullingRendererHierarchicalCullingQuery* HierarchicalCullingQuery = SceneCullingRenderer ? SceneCullingRenderer->GetPrimaryViewHierarchicalCullingQuery(View.SceneRendererPrimaryViewId) : nullptr;
		Pass.DispatchPassSetup(
			Scene,
			View,
			FInstanceCullingContext(GetMeshPassName(EMeshPass::BasePass), ShaderPlatform, &InstanceCullingManager, ViewIds, View.PrevViewInfo.HZB, InstanceCullingMode, EInstanceCullingFlags::None, EBatchProcessingMode::UnCulled, HierarchicalCullingQuery),
			EMeshPass::BasePass,
			BasePassDepthStencilAccess,
			MeshPassProcessor,
			View.DynamicMeshElements,
			&View.DynamicMeshElementsPassRelevance,
			View.NumVisibleDynamicMeshElements[EMeshPass::BasePass],
			ViewCommands.DynamicMeshCommandBuildRequests[EMeshPass::BasePass],
			ViewCommands.DynamicMeshCommandBuildFlags[EMeshPass::BasePass],
			ViewCommands.NumDynamicMeshCommandBuildRequestElements[EMeshPass::BasePass],
			ViewCommands.MeshCommands[EMeshPass::BasePass],
			BasePassCSMMeshPassProcessor,
			&ViewCommands.MeshCommands[EMeshPass::MobileBasePassCSM]);
	}
}

/**
 * Initialize scene's views.
 * Check visibility, sort translucent items, etc.
 */
void FMobileSceneRenderer::InitViews(
	FRDGBuilder& GraphBuilder,
	FSceneTexturesConfig& SceneTexturesConfig,
	FInstanceCullingManager& InstanceCullingManager,
	FVirtualTextureUpdater* VirtualTextureUpdater,
	FInitViewTaskDatas& TaskDatas)
{
	FRHICommandListImmediate& RHICmdList = GraphBuilder.RHICmdList;
	
	SCOPED_DRAW_EVENT(RHICmdList, InitViews);

	SCOPE_CYCLE_COUNTER(STAT_InitViewsTime);
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, InitViews_Scene);

	check(Scene);

	const bool bShadowOnlyTimeslice = IsShadowOnlyTimeslice_RT(&ViewFamily, Scene);
	const bool bRendererOutputFinalSceneColor = (GetRendererOutput() != ERendererOutput::DepthPrepassOnly) && !bShadowOnlyTimeslice;

	PreVisibilityFrameSetup(GraphBuilder);

	if (InstanceCullingManager.IsEnabled()
		&& Scene->InstanceCullingOcclusionQueryRenderer
		&& Scene->InstanceCullingOcclusionQueryRenderer->InstanceOcclusionQueryBuffer)
	{
		InstanceCullingManager.InstanceOcclusionQueryBuffer = GraphBuilder.RegisterExternalBuffer(Scene->InstanceCullingOcclusionQueryRenderer->InstanceOcclusionQueryBuffer);
		InstanceCullingManager.InstanceOcclusionQueryBufferFormat = Scene->InstanceCullingOcclusionQueryRenderer->InstanceOcclusionQueryBufferFormat;
	}

	FILCUpdatePrimTaskData* ILCTaskData = nullptr;

	const FExclusiveDepthStencil::Type BasePassDepthStencilAccess = FExclusiveDepthStencil::DepthWrite_StencilWrite;

	if (FXSystem && FXSystem->RequiresEarlyViewUniformBuffer() && Views.IsValidIndex(0) && bRendererOutputFinalSceneColor)
	{
		// This is to init the ViewUniformBuffer before rendering for the Niagara compute shader.
		// This needs to run before ComputeViewVisibility() is called, but the views normally initialize the ViewUniformBuffer after that (at the end of this method).

		// during ISR, instanced view RHI resources need to be initialized first.
		if (FViewInfo* InstancedView = const_cast<FViewInfo*>(Views[0].GetInstancedView()))
		{
			InstancedView->InitRHIResources();
		}
		Views[0].InitRHIResources();
		FXSystem->PostInitViews(GraphBuilder, GetSceneViews(), !ViewFamily.EngineShowFlags.HitProxies);
	}

	TaskDatas.VisibilityTaskData->ProcessRenderThreadTasks();
	TaskDatas.VisibilityTaskData->FinishGatherDynamicMeshElements(BasePassDepthStencilAccess, InstanceCullingManager, VirtualTextureUpdater);

	// Volumetric fog requires a full depth before lighting
	bShouldRenderVolumetricFog = ShouldRenderVolumetricFog() && (bIsFullDepthPrepassEnabled || (bDeferredShading && !MobileAllowFramebufferFetch(ShaderPlatform)));
	if (bShouldRenderVolumetricFog && bRendererOutputFinalSceneColor)
	{
		SetupVolumetricFog();
	}
	PostVisibilityFrameSetup(ILCTaskData);

	FIntPoint RenderTargetSize = ViewFamily.RenderTarget->GetSizeXY();
	EPixelFormat RenderTargetPixelFormat = PF_Unknown;
	if (ViewFamily.RenderTarget->GetRenderTargetTexture().IsValid())
	{
		RenderTargetSize = ViewFamily.RenderTarget->GetRenderTargetTexture()->GetSizeXY();
		RenderTargetPixelFormat = ViewFamily.RenderTarget->GetRenderTargetTexture()->GetFormat();
	}

	// When using raw output, primary scaling renders to the upper left of a fixed size render target for percentages <100, leaving some of the render target unused.
	// The render target and accompanying view rect are then submitted to an external compositor (such as an OpenXR runtime) which will perform spatial scaling independently.
	const bool bUsingRawOutput = Views[0].PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::RawOutput;
	const bool bRequiresUpscale = !bUsingRawOutput && ((int32)RenderTargetSize.X > FamilySize.X || (int32)RenderTargetSize.Y > FamilySize.Y);

	// ES requires that the back buffer and depth match dimensions.
	// For the most part this is not the case when using scene captures. Thus scene captures always render to scene color target.
	const bool bShouldCompositeEditorPrimitives = FSceneRenderer::ShouldCompositeEditorPrimitives(Views[0]);
	const bool bShouldRenderHMDDistortion = ViewFamily.EngineShowFlags.StereoRendering && ViewFamily.EngineShowFlags.HMDDistortion;
	bRenderToSceneColor = !bGammaSpace 
						|| bShouldRenderHMDDistortion 
						|| bRequiresUpscale 
						|| bShouldCompositeEditorPrimitives 
						|| Views[0].bIsSceneCapture 
						|| Views[0].bIsReflectionCapture 
						// If the resolve texture is not the same as the MSAA texture, we need to render to scene color and copy to back buffer.
						|| (NumMSAASamples > 1 && !RHISupportsSeparateMSAAAndResolveTextures(ShaderPlatform))
						|| (NumMSAASamples > 1 && (RenderTargetPixelFormat != PF_Unknown && RenderTargetPixelFormat != SceneTexturesConfig.ColorFormat))
						|| bIsFullDepthPrepassEnabled;

	bool bSceneDepthCapture = (
		ViewFamily.SceneCaptureSource == SCS_SceneColorSceneDepth ||
		ViewFamily.SceneCaptureSource == SCS_SceneDepth ||
		ViewFamily.SceneCaptureSource == SCS_DeviceDepth);
	// Check if any of the custom render passes outputs depth texture, used to decide whether to enable bPreciseDepthAux.
	for (FViewInfo* View : AllViews)
	{
		if (View->CustomRenderPass)
		{
			ESceneCaptureSource CaptureSource = View->CustomRenderPass->GetSceneCaptureSource();
			if (CaptureSource == SCS_SceneColorSceneDepth ||
				CaptureSource == SCS_SceneDepth ||
				CaptureSource == SCS_DeviceDepth)
			{
				bSceneDepthCapture = true;
				break;
			}
		}
	}

	const FPlanarReflectionSceneProxy* PlanarReflectionSceneProxy = Scene ? Scene->GetForwardPassGlobalPlanarReflection() : nullptr;

	bShouldRenderVelocities = ShouldRenderVelocities();

	static auto CVarMobileSupportInsetShadows = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.SupportInsetShadows"));
	const bool bInsetShadows = (CVarMobileSupportInsetShadows != nullptr && CVarMobileSupportInsetShadows->GetInt() != 0);

	// Runtime reflection captures should support shadows.  Offline reflection captures do not.
	bRequiresShadowProjections = (MobileUsesShadowMaskTexture(ShaderPlatform) || bInsetShadows)
		&& ViewFamily.EngineShowFlags.Lighting
		&& (!Views[0].bIsReflectionCapture || Views[0].IsRuntimeReflectionCapture())
		&& !Views[0].bIsPlanarReflection
		&& !ViewFamily.EngineShowFlags.HitProxies
		&& !ViewFamily.EngineShowFlags.VisualizeLightCulling
		&& !ViewFamily.UseDebugViewPS()
		&& (bRendererOutputFinalSceneColor || bShadowOnlyTimeslice);

	bShouldRenderHZB = ShouldRenderHZB(Views) && bRendererOutputFinalSceneColor;

	// Wait for visibilityTaskData to finish as IsMobileSeparateTranslucencyActive depends on results from SetupMeshPasses.
	TaskDatas.VisibilityTaskData->Finish();
	
	bool bUsingOcclusionFeedback = Views[0].ViewState && Views[0].ViewState->OcclusionFeedback.IsInitialized();
	bAdrenoOcclusionMode = (DoOcclusionQueries() && !bUsingOcclusionFeedback && !Views[0].bDisableQuerySubmissions && CVarMobileAdrenoOcclusionMode.GetValueOnAnyThread() != 0);

	// Whether we need to store depth for post-processing
	// On PowerVR we see flickering of shadows and depths not updating correctly if targets are discarded.
	const bool bForceDepthResolve = (CVarMobileForceDepthResolve.GetValueOnRenderThread() == 1);
	const bool bSeparateTranslucencyActive = IsMobileSeparateTranslucencyActive(Views.GetData(), Views.Num()); 
	const bool bPostProcessUsesSceneDepth = PostProcessUsesSceneDepth(Views[0]);
	const bool bRequireSeparateViewPass = Views.Num() > 1 && !Views[0].bIsMobileMultiViewEnabled;
	bRequiresMultiPass = RequiresMultiPass(NumMSAASamples, ShaderPlatform);

	bKeepDepthContent =
		bRequiresMultiPass ||
		bForceDepthResolve ||
		bSeparateTranslucencyActive ||
		Views[0].bIsReflectionCapture ||
		(bDeferredShading && bPostProcessUsesSceneDepth) ||
		(bDeferredShading && bSceneDepthCapture) ||
		Views[0].AntiAliasingMethod == AAM_TemporalAA ||
		bRequireSeparateViewPass ||
		bIsFullDepthPrepassEnabled ||
		bShouldRenderHZB ||
		(bAdrenoOcclusionMode && IsVulkanPlatform(ShaderPlatform)) ||
		GraphBuilder.IsDumpingFrame();
	// never keep MSAA depth if SceneDepthAux is enabled
	bKeepDepthContent = ((NumMSAASamples > 1) && bRequiresSceneDepthAux) ? false : bKeepDepthContent;

	// Runtime reflection captures should support DFAO.  Offline reflection captures do not.
	extern int32 GDistanceFieldAOApplyToStaticIndirect;
	bEnableDistanceFieldAO =
		Scene->SkyLight && Scene->SkyLight->bCastShadows
		&& ShouldRenderDeferredDynamicSkyLight(Scene, ViewFamily)
		&& AnyViewHasGIMethodSupportingDFAO()
		&& (Views[0].GlobalDistanceFieldInfo.Clipmaps.Num() > 0)
		&& !GDistanceFieldAOApplyToStaticIndirect
		&& ShouldRenderDistanceFieldAO(Views, ViewFamily.EngineShowFlags, AnyViewHasGIMethodSupportingDFAO())
		&& ShouldRenderDistanceFieldLighting(Scene->DistanceFieldSceneData, Views)
		&& ViewFamily.EngineShowFlags.AmbientOcclusion
		&& (!Views[0].bIsReflectionCapture || Views[0].IsRuntimeReflectionCapture());
	
	// Depth is needed for Editor Primitives
	if (bShouldCompositeEditorPrimitives)
	{
		bKeepDepthContent = true;
	}

	// Depth is needed for the mobile DebugView render pass which runs separately from the base pass.
	if (ViewFamily.UseDebugViewPS())
	{
		bKeepDepthContent = true;
	}

	// In the editor RHIs may split a render-pass into several cmd buffer submissions, so all targets need to Store
	if (IsSimulatedPlatform(ShaderPlatform))
	{
		bKeepDepthContent = true;
	}
	// Update the bKeepDepthContent based on the mobile renderer status.
	SceneTexturesConfig.bKeepDepthContent = bKeepDepthContent;
	// If we render in a single pass MSAA targets can be memoryless
    SceneTexturesConfig.bMemorylessMSAA = !(bRequiresMultiPass || bShouldCompositeEditorPrimitives || bRequireSeparateViewPass);
    SceneTexturesConfig.NumSamples = NumMSAASamples;
	SceneTexturesConfig.ExtraSceneColorCreateFlags |= (bTonemapSubpassInline ? TexCreate_InputAttachmentRead : TexCreate_None);
    SceneTexturesConfig.BuildSceneColorAndDepthFlags();
	if (bDeferredShading) 
	{
		SceneTexturesConfig.SetupMobileGBufferFlags(bRequiresMultiPass || GraphBuilder.IsDumpingFrame() || bRequireSeparateViewPass);
	}

	SceneTexturesConfig.bRequiresDepthAux = bRequiresSceneDepthAux;
	// When we capturing scene depth, use a more precise format for SceneDepthAux as it will be used as a source DepthTexture
	if (bSceneDepthCapture)
	{
		SceneTexturesConfig.bPreciseDepthAux = true;
	}
	
	// Find out whether custom depth pass should be rendered.
	{
		bool bCouldUseCustomDepthStencil = (!Scene->World || (Scene->World->WorldType != EWorldType::Inactive));
		FMobileCustomDepthStencilUsage CustomDepthStencilUsage;
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			CustomDepthStencilUsage = GetCustomDepthStencilUsage(Views[ViewIndex]);
			Views[ViewIndex].bCustomDepthStencilValid = bCouldUseCustomDepthStencil && CustomDepthStencilUsage.bUsesCustomDepthStencil;
			bShouldRenderCustomDepth |= Views[ViewIndex].bCustomDepthStencilValid;
			SceneTexturesConfig.bSamplesCustomStencil |= bShouldRenderCustomDepth && CustomDepthStencilUsage.bSamplesCustomStencil;
		}
	}
	
	// Finalize and set the scene textures config.
	FSceneTexturesConfig::Set(SceneTexturesConfig);

	bool bShouldRenderSkyAtmosphere = false;
	if (bRendererOutputFinalSceneColor)
	{
		// This must happen before we start initialising and using views (allocating Scene->SkyIrradianceEnvironmentMap).
		UpdateSkyIrradianceGpuBuffer(GraphBuilder, ViewFamily.EngineShowFlags, Scene->SkyLight, Scene->SkyIrradianceEnvironmentMap);

		// Initialise Sky/View resources before the view global uniform buffer is built.
		bShouldRenderSkyAtmosphere = ShouldRenderSkyAtmosphere(Scene, ViewFamily.EngineShowFlags);
		if (bShouldRenderSkyAtmosphere)
		{
			InitSkyAtmosphereForViews(RHICmdList, GraphBuilder);
		}
	}
		
	FRDGExternalAccessQueue ExternalAccessQueue;

	// initialize per-view uniform buffer.  Pass in shadow info as necessary.
	for (int32 ViewIndex = Views.Num() - 1; ViewIndex >= 0; --ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];
		
		View.UpdatePreExposure();

		// Initialize the view's RHI resources.
		View.InitRHIResources();
	}

	for (int32 i = 0; i < CustomRenderPassInfos.Num(); ++i)
	{
		for (FViewInfo& View : CustomRenderPassInfos[i].Views)
		{
			View.InitRHIResources();
		}		
	}

	if (bRendererOutputFinalSceneColor || bShadowOnlyTimeslice)
	{
		const bool bDynamicShadows = ViewFamily.EngineShowFlags.DynamicShadows;
		if (bDynamicShadows)
		{
			// Setup dynamic shadows.
			TaskDatas.DynamicShadows = InitDynamicShadows(GraphBuilder, InstanceCullingManager);
		}
		else
		{
			// TODO: only do this when CSM + static is required.
			PrepareViewVisibilityLists();
		}
	}

	if (bRendererOutputFinalSceneColor)
	{
		SetupMobileBasePassAfterShadowInit(BasePassDepthStencilAccess, TaskDatas.VisibilityTaskData->GetViewCommandsPerView(), InstanceCullingManager);

		// if we kicked off ILC update via task, wait and finalize.
		if (ILCTaskData)
		{
			Scene->IndirectLightingCache.FinalizeCacheUpdates(RHICmdList, Scene, *this, *ILCTaskData);
		}
	}

	ExternalAccessQueue.Submit(GraphBuilder);

	extern TSet<IPersistentViewUniformBufferExtension*> PersistentViewUniformBufferExtensions;

	for (IPersistentViewUniformBufferExtension* Extension : PersistentViewUniformBufferExtensions)
	{
		Extension->BeginFrame();

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			// Must happen before RHI thread flush so any tasks we dispatch here can land in the idle gap during the flush
			Extension->PrepareView(&Views[ViewIndex]);
		}
	}

	if (bRendererOutputFinalSceneColor)
	{
		if (bDeferredShading ||
			bEnableClusteredLocalLights || 
			bEnableClusteredReflections)
		{
			SetupSceneReflectionCaptureBuffer(RHICmdList);
		}
		UpdateSkyReflectionUniformBuffer(RHICmdList);

		// Now that the indirect lighting cache is updated, we can update the uniform buffers.
		UpdatePrimitiveIndirectLightingCacheBuffers(RHICmdList);

		UpdateDirectionalLightUniformBuffers(GraphBuilder, Views[0]);
	}

	if (!bShadowOnlyTimeslice)
	{
		DecalVisibilityTaskData = FDecalVisibilityTaskData::Launch(GraphBuilder, *Scene, AllViews);
	}
}

static void BeginOcclusionScope(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];
		if (View.ShouldRenderView() && View.ViewState && View.ViewState->OcclusionFeedback.IsInitialized())
		{
			View.ViewState->OcclusionFeedback.BeginOcclusionScope(GraphBuilder);
		}
	}
}

static void EndOcclusionScope(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& Views)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];
		if (View.ShouldRenderView() && View.ViewState && View.ViewState->OcclusionFeedback.IsInitialized())
		{
			View.ViewState->OcclusionFeedback.EndOcclusionScope(GraphBuilder);
		}
	}
}

/*
* Renders the Full Depth Prepass
*/
void FMobileSceneRenderer::RenderFullDepthPrepass(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager, bool bIsSceneCaptureRenderPass)
{
	FRenderViewContextArray RenderViews;
	GetRenderViews(InViews, GetDecalViewOffset(InViews), RenderViews);

	FRDGTextureRef DepthTargetTexture = SceneTextures.Depth.Target;

#if WITH_EDITOR
	if (ViewFamily.EngineShowFlags.Wireframe)
	{
		DepthTargetTexture = SceneTextures.EditorPrimitiveDepth;
	}
#endif

	TRDGUniformBufferBinding<FMobileBasePassUniformParameters> LastViewMobileBasePassUB;
	if (DepthPass.IsRasterStencilDitherEnabled())
	{
		AddDitheredStencilFillPass(GraphBuilder, InViews, DepthTargetTexture, DepthPass);
	}

	for (FRenderViewContext& ViewContext : RenderViews)
	{
		FViewInfo& View = *ViewContext.ViewInfo;

		View.BeginRenderView();

		const bool bRenderVelocityInDepthPrePass = (Scene->EarlyZPassMode == DDM_AllOpaqueNoVelocity);

		struct FFullDepthPrepassParameterCollection
		{
			FMobileRenderPassParameters PassParameters;
			FInstanceCullingDrawParams VelocityInstanceCullingDrawParams;
		};
		auto ParameterCollection = GraphBuilder.AllocParameters<FFullDepthPrepassParameterCollection>();

		auto PassParameters = &ParameterCollection->PassParameters;
		if (bShouldRenderVelocities)
		{
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Velocity, ERenderTargetLoadAction::EClear);
		}
		PassParameters->RenderTargets.DepthStencil = DepthPass.IsRasterStencilDitherEnabled() ?
			FDepthStencilBinding(DepthTargetTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite):
			FDepthStencilBinding(DepthTargetTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);

		PassParameters->View = View.GetShaderParameters();
		PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, EMobileBasePass::DepthPrePass, EMobileSceneTextureSetupMode::None);
		//if the scenecolor isn't multiview but the app is, need to render as a single-view multiview due to shaders
		PassParameters->RenderTargets.MultiViewCount = (View.bIsMobileMultiViewEnabled) ? 2 : (View.Aspects.IsMobileMultiViewEnabled() ? 1 : 0);
		if (ViewContext.bIsLastView)
		{
			LastViewMobileBasePassUB = PassParameters->MobileBasePass;
		}

		if (!ViewContext.bIsFirstView)
		{
			PassParameters->RenderTargets.DepthStencil.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);
			PassParameters->RenderTargets.DepthStencil.SetStencilLoadAction(ERenderTargetLoadAction::ELoad);
			PassParameters->RenderTargets.DepthStencil.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite);
			if (bShouldRenderVelocities)
			{
				PassParameters->RenderTargets[0].SetLoadAction(ERenderTargetLoadAction::ELoad);
			}
		}

		BuildMeshRenderingCommands(GraphBuilder, EMeshPass::DepthPass, View, Scene->GPUScene, InstanceCullingManager, PassParameters->InstanceCullingDrawParams);
		FInstanceCullingDrawParams* VelocityInstanceCullingDrawParams = nullptr;
		if (bRenderVelocityInDepthPrePass)
		{
			VelocityInstanceCullingDrawParams = View.ParallelMeshDrawCommandPasses[EMeshPass::DepthPass] ? &ParameterCollection->VelocityInstanceCullingDrawParams : &PassParameters->InstanceCullingDrawParams;
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::Velocity, View, Scene->GPUScene, InstanceCullingManager, *VelocityInstanceCullingDrawParams);		}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("FullDepthPrepass"),
				PassParameters,
				ERDGPassFlags::Raster,
				[this, ParameterCollection, &View, VelocityInstanceCullingDrawParams](FRHICommandList& RHICmdList)
				{
					auto PassParameters = &ParameterCollection->PassParameters;
					RenderPrePass(RHICmdList, View, &PassParameters->InstanceCullingDrawParams);
					if (VelocityInstanceCullingDrawParams)
					{
						RenderVelocityPass(RHICmdList, View, VelocityInstanceCullingDrawParams);
					}
				});
	}

	// Dithered transition stencil mask clear, accounting for all active viewports
	if (DepthPass.bDitheredLODTransitionsUseStencil)
	{
		AddDitheredStencilClearPass(GraphBuilder, DepthTargetTexture, ShaderPlatform, false);
	}

	FViewOcclusionQueriesPerView QueriesPerView;
	bool bDoOcclusionQueries = DoOcclusionQueries() && !bIsSceneCaptureRenderPass;
	if (bDoOcclusionQueries)
	{
		AllocateOcclusionTests(Scene, QueriesPerView, Views, VisibleLightInfos);
		bDoOcclusionQueries = (QueriesPerView.Num() > 0);
	}

	if (bDoOcclusionQueries)
	{
		for (FRenderViewContext& ViewContext : RenderViews)
		{
			FViewInfo& View = *ViewContext.ViewInfo;

			auto* PassParameters = GraphBuilder.AllocParameters<FMobileRenderPassParameters>();
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthTargetTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilRead);
			PassParameters->View = View.GetShaderParameters();
			PassParameters->MobileBasePass = LastViewMobileBasePassUB;
			PassParameters->RenderTargets.NumOcclusionQueries = ComputeNumOcclusionQueriesToBatch();
			PassParameters->RenderTargets.MultiViewCount = (View.bIsMobileMultiViewEnabled) ? 2 : (View.Aspects.IsMobileMultiViewEnabled() ? 1 : 0);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("RenderOcclusion"),
				PassParameters,
				ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
				[this, ViewContext, ViewQueries = bDoOcclusionQueries ? MoveTemp(QueriesPerView[ViewContext.ViewIndex]) : FViewOcclusionQueries()](FRHICommandList& RHICmdList)
				{
					FViewInfo& View = *ViewContext.ViewInfo;
					RenderOcclusion(RHICmdList, View, ViewQueries);
				});
		}
	}

	if (DoOcclusionQueries() && !bIsSceneCaptureRenderPass)
	{
		EndOcclusionScope(GraphBuilder, Views);
		FenceOcclusionTests(GraphBuilder);
	}
}

void FMobileSceneRenderer::RenderMaskedPrePass(FRHICommandList& RHICmdList, const FViewInfo& View, const FInstanceCullingDrawParams* DepthPassInstanceCullingDrawParams)
{
	if (bIsMaskedOnlyDepthPrepassEnabled)
	{
		RenderPrePass(RHICmdList, View, DepthPassInstanceCullingDrawParams);
	}
}

void FMobileSceneRenderer::RenderCustomRenderPassBasePass(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FRDGTextureRef ViewFamilyTexture, FSceneTextures& SceneTextures, FMobileBasePassTextures& BasePassTextures, FInstanceCullingManager& InstanceCullingManager, bool bIncludeTranslucent)
{
	FRenderTargetBindingSlots BasePassRenderTargets;
	// Use the same subpass hints as main render, to avoid generating new PSOs 
	int32 NumAdditionalSubpasses = 0;
	if (bDeferredShading)
	{
		FColorTargets ColorTargets = GetColorTargets_Deferred(SceneTextures);
		BasePassRenderTargets = InitRenderTargetBindings_Deferred(SceneTextures, ColorTargets);
		if (MobileAllowFramebufferFetch(ShaderPlatform))
		{
			BasePassRenderTargets.SubpassHint = ESubpassHint::DeferredShadingSubpass;
			NumAdditionalSubpasses = 2;
		}
	}
	else
	{
		BasePassRenderTargets = InitRenderTargetBindings_Forward(ViewFamilyTexture, SceneTextures);
		BasePassRenderTargets.SubpassHint = ESubpassHint::DepthReadSubpass;
		NumAdditionalSubpasses = 1;
	}

	FRenderViewContextArray RenderViews;
	GetRenderViews(InViews, GetDecalViewOffset(InViews), RenderViews);

	for (FRenderViewContext& ViewContext : RenderViews)
	{
		FViewInfo& View = *ViewContext.ViewInfo;

		struct FCustomPassParameterCollection
		{
			FMobileRenderPassParameters PassParameters;
			FInstanceCullingDrawParams SkyPassInstanceCullingDrawParams;
			FInstanceCullingDrawParams MeshDecalInstanceCullingDrawParams;
		};
		auto ParameterCollection = GraphBuilder.AllocParameters<FCustomPassParameterCollection>();

		EMobileSceneTextureSetupMode SetupMode = bIsFullDepthPrepassEnabled ? 
			(EMobileSceneTextureSetupMode::SceneDepth | EMobileSceneTextureSetupMode::SceneVelocity) : EMobileSceneTextureSetupMode::None;
		auto PassParameters = &ParameterCollection->PassParameters;
		PassParameters->View = View.GetShaderParameters();
		PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, EMobileBasePass::Opaque, SetupMode, BasePassTextures);
		PassParameters->RenderTargets = BasePassRenderTargets;

		FMobileRenderPassParameters* TranslucencyPassParameters = nullptr;
		if (bIncludeTranslucent)
		{
			TranslucencyPassParameters = GraphBuilder.AllocParameters<FMobileRenderPassParameters>();
			TranslucencyPassParameters->View = View.GetShaderParameters();
			TranslucencyPassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, EMobileBasePass::Translucent, SetupMode);
			TranslucencyPassParameters->RenderTargets[0] = BasePassRenderTargets[0];
			TranslucencyPassParameters->RenderTargets[0].SetLoadAction(ERenderTargetLoadAction::ELoad);
			TranslucencyPassParameters->RenderTargets.DepthStencil = BasePassRenderTargets.DepthStencil;
			TranslucencyPassParameters->RenderTargets.DepthStencil.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);
		}

		if (Scene->GPUScene.IsEnabled())
		{
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::BasePass, View, Scene->GPUScene, InstanceCullingManager, PassParameters->InstanceCullingDrawParams);
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::SkyPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->SkyPassInstanceCullingDrawParams);

			if (NumAdditionalSubpasses > 0)
			{
				BuildMeshRenderingCommands(
					GraphBuilder,
					IsForwardShadingEnabled(View.Family->GetShaderPlatform()) ? EMeshPass::MeshDecal_SceneColor : EMeshPass::MeshDecal_SceneColorAndGBuffer,
					View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->MeshDecalInstanceCullingDrawParams);
			}

			if (bIncludeTranslucent)
			{
				BuildMeshRenderingCommands(GraphBuilder, EMeshPass::TranslucencyAll, View, Scene->GPUScene, InstanceCullingManager, TranslucencyPassParameters->InstanceCullingDrawParams);
			}
		}

		EDecalRenderStage DecalStage = bRequiresDBufferDecals ? EDecalRenderStage::Emissive : EDecalRenderStage::Mobile;
		FDecalViewInfo DecalViewInfo = FDecalViewInfo(ViewContext.DecalViewOffset, ViewContext.ViewIndex,
													  DecalStage,
													  bDeferredShading ? EDecalRenderTargetMode::SceneColorAndGBuffer : EDecalRenderTargetMode::SceneColor,
													  GetRelevantDeferredDecals(ViewContext.DecalViewOffset, ViewContext.ViewIndex, DecalStage));

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RenderMobileBasePass"),
			PassParameters,
			ERDGPassFlags::Raster,
			[this, DecalViewInfo, ParameterCollection, ViewContext, NumAdditionalSubpasses](FRHICommandList& RHICmdList)
			{
				FViewInfo& View = *ViewContext.ViewInfo;
				auto PassParameters = &ParameterCollection->PassParameters;
				RenderMobileBasePass(RHICmdList, View, &PassParameters->InstanceCullingDrawParams, &ParameterCollection->SkyPassInstanceCullingDrawParams);

				for (int32 AddSubpass = 0; AddSubpass < NumAdditionalSubpasses; ++AddSubpass)
				{
					RHICmdList.NextSubpass();
					if (AddSubpass == 0)
					{
						RenderDecals(RHICmdList, View, DecalViewInfo, &ParameterCollection->MeshDecalInstanceCullingDrawParams);
					}
				}
			});

		// Decals are rendered as the first subpass above -- if no subpasses, render them as a separate pass.
		if (NumAdditionalSubpasses == 0)
		{
			BasePassRenderTargets.Enumerate([](FRenderTargetBinding& RenderTarget) {
				RenderTarget.SetLoadAction(ERenderTargetLoadAction::ELoad);
			});
			BasePassRenderTargets.DepthStencil.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);
			BasePassRenderTargets.DepthStencil.SetStencilLoadAction(ERenderTargetLoadAction::ELoad);
			BasePassRenderTargets.DepthStencil.SetDepthStencilAccess(FExclusiveDepthStencil::DepthRead_StencilWrite);

			RenderDeferredMultiPassDecals(GraphBuilder, View, ViewContext.DecalViewOffset, ViewContext.ViewIndex, SceneTextures, InstanceCullingManager);
		}

		if (bIncludeTranslucent)
		{
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("RenderMobileTranslucentPass"),
				TranslucencyPassParameters,
				ERDGPassFlags::Raster,
				[this, TranslucencyPassParameters, ViewContext, InViews, &SceneTextures](FRHICommandList& RHICmdList)
				{
					// Custom render passes run all translucency in a single pass
					FViewInfo& View = *ViewContext.ViewInfo;
					RenderTranslucency(RHICmdList, View, InViews, ETranslucencyPass::TPT_AllTranslucency, EMeshPass::TranslucencyAll, &TranslucencyPassParameters->InstanceCullingDrawParams);
				});
		}

		if (!bIsFullDepthPrepassEnabled)
		{
			AddResolveSceneDepthPass(GraphBuilder, View, SceneTextures.Depth);
		}
		if (bRequiresSceneDepthAux)
		{
			AddResolveSceneColorPass(GraphBuilder, View, SceneTextures.DepthAux);
		}
	}
}

void FMobileSceneRenderer::Render(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs)
{
	const ERendererOutput RendererOutput = GetRendererOutput();
	const bool bShadowOnlyTimeslice = IsShadowOnlyTimeslice_RT(&ViewFamily, Scene);
	const bool bRendererOutputFinalSceneColor = (RendererOutput != ERendererOutput::DepthPrepassOnly) && !bShadowOnlyTimeslice;

	// Allow GPU cost of scene captures to be distinguished from main views, useful for profiling the amortized GPU cost of runtime reflection captures.
	RDG_RHI_EVENT_SCOPE_CONDITIONAL_STAT(GraphBuilder, MobileSceneRender,    MobileSceneRender,    !Views[0].bIsSceneCapture);
	RDG_RHI_EVENT_SCOPE_CONDITIONAL_STAT(GraphBuilder, CapMobileSceneRender, CapMobileSceneRender,  Views[0].bIsSceneCapture);

	// Shadow state swapping (to a transform cached in the scene, rather than the current transform) is necessary for timesliced shadows, in case the
	// shadow casting actor moved since timeslicing started (say there is a time of day progression, or it is moved in the editor).  Even small changes
	// to the transform will cause shadows to be radically incorrect, making this a necessity.
	bool bShadowStateSwapped = false;
	if (Views[0].IsRuntimeReflectionCapture() && Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame)
	{
		Scene->SwapMobileReflectionCaptureCachedShadowState();
		bShadowStateSwapped = true;
	}

	IVisibilityTaskData* VisibilityTaskData = OnRenderBegin(GraphBuilder, SceneUpdateInputs);

	FUpdateExposureCompensationCurveLUTTaskData UpdateExposureCompensationCurveLUTTaskData;
	BeginUpdateExposureCompensationCurveLUT(Views, &UpdateExposureCompensationCurveLUTTaskData);

	FRDGExternalAccessQueue ExternalAccessQueue;

	// If we're rendering with async compute then render shadows after the prepass
	// We don't do this in the general case because it is slower on some platforms
	extern int32 GDFShadowAsyncCompute;
	const bool bRenderShadowsAfterPrepass = !!GDFShadowAsyncCompute && bIsFullDepthPrepassEnabled;

	if (SceneUpdateInputs)
	{
		static auto CVarDistanceFieldShadowQuality = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DFShadowQuality"));

		if (IsMobileDistanceFieldEnabled(ShaderPlatform)
			&& CVarDistanceFieldShadowQuality != nullptr
			&& CVarDistanceFieldShadowQuality->GetInt() > 0)
		{
			// It's necessary to check all scene renderers when deciding whether to update distance field scene data, as depth
			// only scene captures and shadow only timeslices don't use distance fields, and would otherwise skip their update.
			// The first renderer in a batch is responsible for scene updates required by other renderers.
			for (const FSceneRenderer* Renderer : SceneUpdateInputs->Renderers)
			{
				if (Renderer->GetRendererOutput() != ERendererOutput::DepthPrepassOnly && !IsShadowOnlyTimeslice_RT(&Renderer->ViewFamily, Scene))
				{
					const FSceneViewFamily& Family = Renderer->ViewFamily;
					const FEngineShowFlags& EngineShowFlags = Family.EngineShowFlags;
					const FSceneView& View = *Family.Views[0];

					// Runtime reflection captures should support distance field shadows.  Offline reflection captures do not
					if (EngineShowFlags.Lighting && !EngineShowFlags.VisualizeLightCulling && !Family.UseDebugViewPS() && (!View.bIsReflectionCapture || View.IsRuntimeReflectionCapture()) && !View.bIsPlanarReflection)
					{
						UpdateGlobalDistanceFieldViewOrigin(*SceneUpdateInputs);
						PrepareDistanceFieldScene(GraphBuilder, ExternalAccessQueue, *SceneUpdateInputs);
						break;
					}
				}
			}
		}
	}

	ExternalAccessQueue.Submit(GraphBuilder);

	GPU_MESSAGE_SCOPE(GraphBuilder);

	// Establish scene primitive count (must be done after UpdateAllPrimitiveSceneInfos)
	FGPUSceneScopeBeginEndHelper GPUSceneScopeBeginEndHelper(GraphBuilder, Scene->GPUScene, GPUSceneDynamicContext);

	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderOther);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FMobileSceneRenderer_Render);

	FSceneTexturesConfig& SceneTexturesConfig = GetActiveSceneTexturesConfig();

	// Initialize global system textures (pass-through if already initialized).
	GSystemTextures.InitializeTextures(GraphBuilder.RHICmdList, FeatureLevel);

	FRDGSystemTextures::Create(GraphBuilder);

	ShaderPrint::BeginViews(GraphBuilder, Views);

	ON_SCOPE_EXIT
	{
		ShaderPrint::EndViews(Views);
	};

	TUniquePtr<FVirtualTextureUpdater> VirtualTextureUpdater;

	if (bUseVirtualTexturing)
	{
		FVirtualTextureUpdateSettings Settings(ViewFamily);

		// Disable VT feedback for runtime reflection captures.  These are low resolution (usually) one-time
		// renders at a given location, so it's not worth the overhead.
		if (Views.Num() && Views[0].IsRuntimeReflectionCapture())
		{
			Settings.EnableFeedback(false);
		}

		VirtualTextureUpdater = FVirtualTextureSystem::Get().BeginUpdate(GraphBuilder, FeatureLevel, this, Settings);
		VirtualTextureFeedbackBegin(GraphBuilder, Views, SceneTexturesConfig.Extent);
	}

	// Substrate initialization is always run even when not enabled.
	ShadingEnergyConservation::Init(GraphBuilder, Views);
	if (Substrate::IsSubstrateEnabled())
	{
		for (FViewInfo& View : Views)
		{
			FGlintShadingLUTsStateData::Init(GraphBuilder, View);
		}
	}

	if (bRendererOutputFinalSceneColor)
	{
		// Force the subsurface profile & specular profile textures to be updated.
		SubsurfaceProfile::UpdateSubsurfaceProfileTexture(GraphBuilder, ShaderPlatform);
		SpecularProfile::UpdateSpecularProfileTextureAtlas(GraphBuilder, ShaderPlatform);
		ToonProfile::UpdateToonProfileTextureAtlas(GraphBuilder, ShaderPlatform);

		if (bDeferredShading)
		{
			RectLightAtlas::UpdateAtlasTexture(GraphBuilder, FeatureLevel);
		}
		IESAtlas::UpdateAtlasTexture(GraphBuilder, ShaderPlatform);

		// Important that this uses consistent logic throughout the frame, so evaluate once and pass in the flag from here
		// NOTE: Must be done after  system texture initialization
		VirtualShadowMapArray.Initialize(GraphBuilder, Scene->GetVirtualShadowMapCache(), UseVirtualShadowMaps(ShaderPlatform, FeatureLevel), ViewFamily.EngineShowFlags);
	}

	GetSceneExtensionsRenderers().PreInitViews(GraphBuilder);

	FInitViewTaskDatas InitViewTaskDatas(VisibilityTaskData);

	FRendererViewDataManager& ViewDataManager = *GraphBuilder.AllocObject<FRendererViewDataManager>(GraphBuilder, *Scene, GetSceneUniforms(), AllViews);
	FInstanceCullingManager& InstanceCullingManager = *GraphBuilder.AllocObject<FInstanceCullingManager>(GraphBuilder, *Scene, GetSceneUniforms(), ViewDataManager);

	// Find the visible primitives and prepare targets and buffers for rendering
	InitViews(GraphBuilder, SceneTexturesConfig, InstanceCullingManager, VirtualTextureUpdater.Get(), InitViewTaskDatas);

	if (bRendererOutputFinalSceneColor && DoOcclusionQueries())
	{
		BeginOcclusionScope(GraphBuilder, Views);
	}

	// Notify the FX system that the scene is about to be rendered.
	// TODO: These should probably be moved to scene extensions
	if (FXSystem)
	{
		FXSystem->PreRender(GraphBuilder, GetSceneViews(), GetSceneUniforms(), true /*bAllowGPUParticleUpdate*/);
		if (FGPUSortManager* GPUSortManager = FXSystem->GetGPUSortManager())
		{
			// if GPUSortManager::OnPostRenderOpaque is called below (from RenderOpaqueFX) we must also call OnPreRender (as it sets up
			// the internal state of the GPUSortManager).  Any optimization to skip this block needs to take that into consideration.
			GPUSortManager->OnPreRender(GraphBuilder);
		}
	}

	{
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, UpdateGPUScene);

		for (int32 ViewIndex = 0; ViewIndex < AllViews.Num(); ViewIndex++)
		{
			FViewInfo& View = *AllViews[ViewIndex];
			Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, View);
			Scene->GPUScene.DebugRender(GraphBuilder, GetSceneUniforms(), View);
		}
	}
	
	GetSceneExtensionsRenderers().UpdateViewData(GraphBuilder, ViewDataManager);

	// Allow scene extensions to affect the scene uniform buffer
	GetSceneExtensionsRenderers().UpdateSceneUniformBuffer(GraphBuilder, GetSceneUniforms());

	InstanceCullingManager.BeginDeferredCulling(GraphBuilder);

	GetSceneExtensionsRenderers().PreRender(GraphBuilder);
	GEngine->GetPreRenderDelegateEx().Broadcast(GraphBuilder);

	FSceneTextures::InitializeViewFamily(GraphBuilder, ViewFamily, FamilySize);
	FSceneTextures& SceneTextures = GetActiveSceneTextures();

	Substrate::InitialiseSubstrateFrameSceneData(GraphBuilder, *Scene, ViewFamily, AllViews, ShaderPlatform);

	FSortedLightSetSceneInfo& SortedLightSet = *GraphBuilder.AllocObject<FSortedLightSetSceneInfo>();

	SceneTextures.MobileSetupMode = EMobileSceneTextureSetupMode::None;
	SceneTextures.MobileUniformBuffer = CreateMobileSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, SceneTextures.MobileSetupMode);

	// We must have a full depth buffer in order to render half res and upsample
	const bool bUseHalfResLocalFogVolume = bIsFullDepthPrepassEnabled && IsLocalFogVolumeHalfResolution();

	if (bRendererOutputFinalSceneColor)
	{
		ClearDebugAux(GraphBuilder, ViewFamily, SceneTextures.DebugAux);

		if (bUseVirtualTexturing)
		{
			FVirtualTextureSystem::Get().EndUpdate(GraphBuilder, MoveTemp(VirtualTextureUpdater), FeatureLevel);
			FVirtualTextureSystem::Get().FinalizeRequests(GraphBuilder, this);
		}

		if (bDeferredShading ||
			bEnableClusteredLocalLights ||
			bEnableClusteredReflections)
		{
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, SortLights);
			// Shadows are applied in clustered shading on mobile forward and separately on mobile deferred.
			bool bShadowedLightsInClustered = bRequiresShadowProjections && !bDeferredShading;

			// This task needs to run before any other functions gathering lights for upload on GPU, for light function indices to be assigned to lights.
			UpdateLightFunctionAtlasTaskFunction();

			GatherAndSortLights(SortedLightSet, bShadowedLightsInClustered);

			const int32 NumReflectionCaptures = Views[0].NumBoxReflectionCaptures + Views[0].NumSphereReflectionCaptures;
			const bool bCullLightsToGrid = (((bEnableClusteredReflections || bDeferredShading) && NumReflectionCaptures > 0) || bEnableClusteredLocalLights);
			PrepareForwardLightData(GraphBuilder, bCullLightsToGrid, SortedLightSet);

			LightFunctionAtlas.RenderLightFunctionAtlas(GraphBuilder, Views);
		}
		else
		{
			SetDummyForwardLightUniformBufferOnViews(GraphBuilder, ShaderPlatform, Views);
		}

		CSV_CUSTOM_STAT(LightCount, All, float(SortedLightSet.SortedLights.Num()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(LightCount, Batched, float(SortedLightSet.UnbatchedLightStart), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT(LightCount, Unbatched, float(SortedLightSet.SortedLights.Num()) - float(SortedLightSet.UnbatchedLightStart), ECsvCustomStatOp::Set);

		// Generate the Sky/Atmosphere look up tables
		const bool bShouldRenderSkyAtmosphere = ShouldRenderSkyAtmosphere(Scene, ViewFamily.EngineShowFlags);
		if (bShouldRenderSkyAtmosphere)
		{
			FSkyAtmospherePendingRDGResources PendingRDGResources;
			RenderSkyAtmosphereLookUpTables(GraphBuilder, /* out */ PendingRDGResources);
			PendingRDGResources.CommitToSceneAndViewUniformBuffers(GraphBuilder, /* out */ ExternalAccessQueue);
		}

		// Run local fog volume initialization before base pass and volumetric fog for all the culled instance instance data to be ready.
		InitLocalFogVolumesForViews(Scene, Views, ViewFamily, GraphBuilder, bShouldRenderVolumetricFog, bUseHalfResLocalFogVolume);

		if (bRendererOutputFinalSceneColor && Views.Num() > 0)
		{
			FViewInfo& MainView = Views[0];
			const bool bRealTimeSkyCaptureEnabled = (bShouldRenderSkyAtmosphere || MainView.SkyMeshBatches.Num() > 0) && Scene->SkyLight && Scene->SkyLight->bRealTimeCaptureEnabled && ViewFamily.EngineShowFlags.SkyLighting;
			if (bRealTimeSkyCaptureEnabled)
			{
				// We must execute the submit for transition of SkyAtmosphere resources to happen (see CommitToSceneAndViewUniformBuffers) and avoid validation error.
				ExternalAccessQueue.Submit(GraphBuilder);

				const bool bShouldRenderVolumetricCloud = false; // Not supported on the mobile renderer.
				Scene->AllocateAndCaptureFrameSkyEnvMap(GraphBuilder, *this, MainView, bShouldRenderSkyAtmosphere, bShouldRenderVolumetricCloud, InstanceCullingManager, ExternalAccessQueue);

				UpdateSkyReflectionUniformBuffer(GraphBuilder.RHICmdList);
			}
		}

		// Hair update
		if (IsHairStrandsEnabled(EHairStrandsShaderType::All, Scene->GetShaderPlatform()) && RendererOutput != ERendererOutput::DepthPrepassOnly)
		{
			FHairStrandsBookmarkParameters& HairStrandsBookmarkParameters = *GraphBuilder.AllocObject<FHairStrandsBookmarkParameters>();
			CreateHairStrandsBookmarkParameters(Scene, Views, AllViews, HairStrandsBookmarkParameters);
			check(Scene->HairStrandsSceneData.TransientResources);
			HairStrandsBookmarkParameters.TransientResources = Scene->HairStrandsSceneData.TransientResources;

			// Not need for hair uniform buffer, as this is only used for strands rendering
			// If some shader refers to it, we can create a default one with HairStrands::CreateDefaultHairStrandsViewUniformBuffer(GraphBuilder, View);
			for (FViewInfo& View : Views)
			{
				View.HairStrandsViewData.UniformBuffer = nullptr;
			}

			// Interpolation needs to happen after the skin cache run as there is a dependency 
			// on the skin cache output.
			const bool bRunHairStrands = HairStrandsBookmarkParameters.HasInstances() && (Views.Num() > 0);
			if (bRunHairStrands)
			{
				// 1. Update groom visible in primary views
				RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessCardsAndMeshesInterpolation_PrimaryView, HairStrandsBookmarkParameters);

				// 2. Update groom only visible in shadow 
				// For now, not running on mobile to keep computation light
				// UpdateHairStrandsBookmarkParameters(Scene, Views, HairStrandsBookmarkParameters);
				// RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessCardsAndMeshesInterpolation_ShadowView, HairStrandsBookmarkParameters);
			}
		}
		if (!bRenderShadowsAfterPrepass)
		{
			RenderShadowDepthMaps(GraphBuilder, nullptr, InstanceCullingManager, ExternalAccessQueue);
			GraphBuilder.AddDispatchHint();
		}
		ExternalAccessQueue.Submit(GraphBuilder);

		// Custom depth
		// bShouldRenderCustomDepth has been initialized in InitViews on mobile platform
		if (bShouldRenderCustomDepth)
		{
			RenderCustomDepthPass(GraphBuilder, SceneTextures.CustomDepth, SceneTextures.GetSceneTextureShaderParameters(FeatureLevel), {}, {});
		}
	}
	else
	{
		if (bShadowOnlyTimeslice)
		{
			RenderShadowDepthMaps(GraphBuilder, nullptr, InstanceCullingManager, ExternalAccessQueue);
			ExternalAccessQueue.Submit(GraphBuilder);
		}

		SetDummyLocalFogVolumeForViews(GraphBuilder, Views);
	}
	
	// Sort objects' triangles
	for (FViewInfo& View : Views)
	{
		if (View.ShouldRenderView() && OIT::IsSortedTrianglesEnabled(View.GetShaderPlatform()))
		{
			OIT::AddSortTrianglesPass(GraphBuilder, View, Scene->OITSceneData, FTriangleSortingOrder::BackToFront);
		}
	}

	FRDGTextureRef ViewFamilyTexture = TryCreateViewFamilyTexture(GraphBuilder, ViewFamily);
	FRDGTextureRef ViewFamilyDepthTexture = TryCreateViewFamilyDepthTexture(GraphBuilder, ViewFamily);

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	RenderWaterInfoTexture(GraphBuilder, *this, Scene);

	FViewInfo* MainView = Views.Num() > 0 ? &Views[0] : nullptr;
	const bool bIsMobileMultiView = SceneTextures.Config.bRequireMultiView || (MainView && MainView->Aspects.IsMobileMultiViewEnabled());

	FMobileBasePassTextures BasePassTextures{};

	if (CustomRenderPassInfos.Num() > 0)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_CustomRenderPasses);
		RDG_EVENT_SCOPE_STAT(GraphBuilder, CustomRenderPasses, "CustomRenderPasses");

		for (int32 i = 0; i < CustomRenderPassInfos.Num(); ++i)
		{
			FCustomRenderPassBase* CustomRenderPass = CustomRenderPassInfos[i].CustomRenderPass;
			TArray<FViewInfo>& CustomRenderPassViews = CustomRenderPassInfos[i].Views;
			check(CustomRenderPass);

			const FSceneCaptureCustomRenderPassUserData& SceneCaptureUserData = FSceneCaptureCustomRenderPassUserData::Get(CustomRenderPass);
			bool bNeedGBufferC = !SceneCaptureUserData.UserSceneTextureBaseColor.IsNone() || CustomRenderPass->GetSceneCaptureSource() == SCS_BaseColor;
			bool bNeedGBufferA = !SceneCaptureUserData.UserSceneTextureNormal.IsNone() || CustomRenderPass->GetSceneCaptureSource() == SCS_Normal;

			// Initialize scene textures if owned by this Custom Render Pass (not reusing the main view family).
			FViewFamilyInfo& CustomRenderPassViewFamily = CustomRenderPassInfos[i].ViewFamily;
			if (CustomRenderPassViewFamily.IsSceneTexturesOwner())
			{
				// If we are reading back GBuffer textures, they can't be memoryless.
				for (uint32 Layout = 0; Layout < GBL_Num; ++Layout)
				{
					FGBufferBindings& Bindings = CustomRenderPassViewFamily.SceneTexturesConfig.GBufferBindings[Layout];
					if (bNeedGBufferC)
					{
						Bindings.GBufferC.Flags &= ~TexCreate_Memoryless;
					}
					if (bNeedGBufferA)
					{
						Bindings.GBufferA.Flags &= ~TexCreate_Memoryless;
					}
				}

				FSceneTextures::InitializeViewFamily(GraphBuilder, CustomRenderPassViewFamily, FamilySize);
				Substrate::InitialiseSubstrateFrameSceneData(GraphBuilder, *Scene, CustomRenderPassViewFamily, AllViews, ShaderPlatform);
			}
			FSceneTextures& CustomRenderPassSceneTextures = CustomRenderPassViewFamily.GetSceneTextures();

			// We want to reset the scene texture uniform buffer to its original state after custom render passes,
			// so they can't affect downstream rendering.
			EMobileSceneTextureSetupMode OriginalSceneTextureSetupMode = CustomRenderPassSceneTextures.MobileSetupMode;
			TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> OriginalSceneTextureUniformBuffer = CustomRenderPassSceneTextures.MobileUniformBuffer;

			CustomRenderPass->BeginPass(GraphBuilder);

			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_CustomRenderPass);
				RDG_EVENT_SCOPE(GraphBuilder, "CustomRenderPass[%d] %s", i, *CustomRenderPass->GetDebugName());

				CustomRenderPass->PreRender(GraphBuilder);

				// Setup dummy uniform buffer parameters for fog volume.
				SetDummyLocalFogVolumeForViews(GraphBuilder, CustomRenderPassViews);

				if (bIsFullDepthPrepassEnabled)
				{
					RenderFullDepthPrepass(GraphBuilder, CustomRenderPassViews, CustomRenderPassSceneTextures, InstanceCullingManager, true);
					if (!bRequiresSceneDepthAux)
					{
						AddResolveSceneDepthPass(GraphBuilder, CustomRenderPassViews, CustomRenderPassSceneTextures.Depth);
					}
				}

				// Render base pass if the custom pass requires it. Otherwise if full depth prepass is not enabled, then depth is generated in the base pass.
				if (CustomRenderPass->GetRenderMode() == FCustomRenderPassBase::ERenderMode::DepthAndBasePass || (CustomRenderPass->GetRenderMode() == FCustomRenderPassBase::ERenderMode::DepthPass && !bIsFullDepthPrepassEnabled))
				{
					FMobileBasePassTextures CustomRenderPassBasePassTextures{};

					if (CustomRenderPass->GetRenderMode() == FCustomRenderPassBase::ERenderMode::DepthAndBasePass && bRequiresDBufferDecals)
					{
						// If mobile multi-view is enabled, force separate DBuffer textures to be allocated.  Multi-view causes the DBuffer textures to be allocated as array
						// textures, and the concern is that could cause issues with Custom Render Passes.  We also need separate DBuffer textures if this view allocates its
						// own scene textures, as the scene texture resolution will be different than the main view in that case.
						if (CustomRenderPassViewFamily.IsSceneTexturesOwner() || bIsMobileMultiView)
						{
							CustomRenderPassBasePassTextures.DBufferTextures = CreateDBufferTextures(GraphBuilder, CustomRenderPassSceneTextures.Config.Extent, ShaderPlatform);
						}
						else
						{
							// Sharing main renderer DBuffer textures, allocate them if not yet allocated.  Note that scene texture extents for custom render passes are the
							// same as the main view(s), as upstream scene texture extent calculation logic ensures the extent is a union of all views rendered in the frame,
							// including Scene Captures and Custom Render Passes.  The exception is if bExcludeFromSceneTextureExtents is set, which excludes the given
							// Scene Capture (or Custom Render Pass) from extent calculations.  In that case, IsSceneTexturesOwner() will be true, and the code above runs.
							if (!BasePassTextures.DBufferTextures.DBufferA && !BasePassTextures.DBufferTextures.DBufferATexArray)
							{
								BasePassTextures.DBufferTextures = CreateDBufferTextures(GraphBuilder, SceneTextures.Config.Extent, ShaderPlatform, bIsMobileMultiView);
							}
							CustomRenderPassBasePassTextures.DBufferTextures = BasePassTextures.DBufferTextures;
						}
						RenderDBuffer(GraphBuilder, CustomRenderPassViews, CustomRenderPassSceneTextures, CustomRenderPassBasePassTextures.DBufferTextures, InstanceCullingManager);
					}

					RenderCustomRenderPassBasePass(GraphBuilder, CustomRenderPassViews, ViewFamilyTexture, CustomRenderPassSceneTextures, CustomRenderPassBasePassTextures, InstanceCullingManager, CustomRenderPass->IsTranslucentIncluded());
				}

				CustomRenderPassSceneTextures.MobileSetupMode = EMobileSceneTextureSetupMode::SceneColor | EMobileSceneTextureSetupMode::SceneDepth | EMobileSceneTextureSetupMode::SceneDepthAux;

				if (!IsForwardShadingEnabled(ViewFamily.GetShaderPlatform()))
				{
					if (bNeedGBufferC)
					{
						CustomRenderPassSceneTextures.MobileSetupMode |= EMobileSceneTextureSetupMode::GBufferC;
					}
					if (bNeedGBufferA)
					{
						CustomRenderPassSceneTextures.MobileSetupMode |= EMobileSceneTextureSetupMode::GBufferA;
					}
				}

				CustomRenderPassSceneTextures.MobileUniformBuffer = CreateMobileSceneTextureUniformBuffer(GraphBuilder, &CustomRenderPassSceneTextures, CustomRenderPassSceneTextures.MobileSetupMode);

				CopySceneCaptureComponentToTarget(GraphBuilder, CustomRenderPassSceneTextures, CustomRenderPass->GetRenderTargetTexture(), ViewFamily, CustomRenderPassViews);

				if (!IsForwardShadingEnabled(ViewFamily.GetShaderPlatform()))
				{
					if (!SceneCaptureUserData.UserSceneTextureBaseColor.IsNone())
					{
						// User Scene Textures are stored to "SceneTextures" for downstream use, not in "CustomRenderPassSceneTextures", only used during Custom Render Pass rendering
						bool bFirstRender;
						FRDGTextureRef BaseColorSceneTexture = SceneTextures.FindOrAddUserSceneTexture(GraphBuilder, 0, SceneCaptureUserData.UserSceneTextureBaseColor, SceneCaptureUserData.SceneTextureDivisor, bFirstRender, nullptr, CustomRenderPassViews[0].ViewRect);
#if !(UE_BUILD_SHIPPING)
						SceneTextures.UserSceneTextureEvents.Add({ EUserSceneTextureEvent::CustomRenderPass, NAME_None, (uint16)FCustomRenderPassBase::ERenderOutput::BaseColor, 0, (const UMaterialInterface*)CustomRenderPass });
#endif

						CustomRenderPass->OverrideRenderOutput(FCustomRenderPassBase::ERenderOutput::BaseColor);
						CopySceneCaptureComponentToTarget(GraphBuilder, CustomRenderPassSceneTextures, BaseColorSceneTexture, ViewFamily, CustomRenderPassViews);
					}

					if (!SceneCaptureUserData.UserSceneTextureNormal.IsNone())
					{
						bool bFirstRender;
						FRDGTextureRef NormalSceneTexture = SceneTextures.FindOrAddUserSceneTexture(GraphBuilder, 0, SceneCaptureUserData.UserSceneTextureNormal, SceneCaptureUserData.SceneTextureDivisor, bFirstRender, nullptr, CustomRenderPassViews[0].ViewRect);
#if !(UE_BUILD_SHIPPING)
						SceneTextures.UserSceneTextureEvents.Add({ EUserSceneTextureEvent::CustomRenderPass, NAME_None, (uint16)FCustomRenderPassBase::ERenderOutput::Normal, 0, (const UMaterialInterface*)CustomRenderPass });
#endif

						CustomRenderPass->OverrideRenderOutput(FCustomRenderPassBase::ERenderOutput::Normal);
						CopySceneCaptureComponentToTarget(GraphBuilder, CustomRenderPassSceneTextures, NormalSceneTexture, ViewFamily, CustomRenderPassViews);
					}
				}

				if (!SceneCaptureUserData.UserSceneTextureSceneColor.IsNone())
				{
					bool bFirstRender;
					FRDGTextureRef SceneColorSceneTexture = SceneTextures.FindOrAddUserSceneTexture(GraphBuilder, 0, SceneCaptureUserData.UserSceneTextureSceneColor, SceneCaptureUserData.SceneTextureDivisor, bFirstRender, nullptr, CustomRenderPassViews[0].ViewRect);
#if !(UE_BUILD_SHIPPING)
					SceneTextures.UserSceneTextureEvents.Add({ EUserSceneTextureEvent::CustomRenderPass, NAME_None, (uint16)FCustomRenderPassBase::ERenderOutput::SceneColorAndAlpha, 0, (const UMaterialInterface*)CustomRenderPass });
#endif

					CustomRenderPass->OverrideRenderOutput(FCustomRenderPassBase::ERenderOutput::SceneColorAndAlpha);
					CopySceneCaptureComponentToTarget(GraphBuilder, CustomRenderPassSceneTextures, SceneColorSceneTexture, ViewFamily, CustomRenderPassViews);
				}

				CustomRenderPass->PostRender(GraphBuilder);

				// Mips are normally generated in UpdateSceneCaptureContentMobile_RenderThread, but that doesn't run when the
				// scene capture runs as a custom render pass.  The function does nothing if the render target doesn't have mips.
				if (CustomRenderPassViews[0].bIsSceneCapture)
				{
					FGenerateMips::Execute(GraphBuilder, FeatureLevel, CustomRenderPass->GetRenderTargetTexture(), FGenerateMipsParams());
				}
			}

			CustomRenderPass->EndPass(GraphBuilder);

			CustomRenderPassSceneTextures.MobileSetupMode = OriginalSceneTextureSetupMode;
			CustomRenderPassSceneTextures.MobileUniformBuffer = OriginalSceneTextureUniformBuffer;

			// Clear DBuffer produced flags after Custom Render Pass rendering, so they will be cleared when they render again
			CustomRenderPassSceneTextures.DBufferTextures->bDBufferProduced = false;
			CustomRenderPassSceneTextures.DBufferTextures->bDBufferTexArrayProduced = false;

			// Call finish on decal visibility task data.  Note that this is handled by the FCompositionLighting destructor for Custom Render Passes
			// in the deferred renderer, but the mobile renderer doesn't use that class.
			int32 DecalViewOffset = GetDecalViewOffset(CustomRenderPassViews);
			if (DecalViewOffset != INDEX_NONE)
			{
				for (int32 ViewIndex = 0; ViewIndex < CustomRenderPassViews.Num(); ViewIndex++)
				{
					DecalVisibilityTaskData->Finish(DecalViewOffset, ViewIndex);
				}
			}
		}
	}

	if (bIsFullDepthPrepassEnabled && !bShadowOnlyTimeslice)
	{
		RenderFullDepthPrepass(GraphBuilder, Views, SceneTextures, InstanceCullingManager);

		if (!bRequiresSceneDepthAux)
		{
			AddResolveSceneDepthPass(GraphBuilder, Views, SceneTextures.Depth);
		}

		SceneTextures.MobileSetupMode = EMobileSceneTextureSetupMode::SceneDepth | EMobileSceneTextureSetupMode::SceneVelocity;
		SceneTextures.MobileUniformBuffer = CreateMobileSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, SceneTextures.MobileSetupMode);
	}

	if (bRendererOutputFinalSceneColor)
	{
		if (bRenderShadowsAfterPrepass)
		{
			// Kick async DF shadows to run concurrently with shadow depths
			// Note: this is a no-op if async DF shadows is disabled
			BeginAsyncDistanceFieldShadowProjections(GraphBuilder, SceneTextures, InitViewTaskDatas.DynamicShadows);

			// Render shadows after the prepass so we can overap async compute work with shadows
			RenderShadowDepthMaps(GraphBuilder, nullptr, InstanceCullingManager, ExternalAccessQueue);
			GraphBuilder.AddDispatchHint();
			ExternalAccessQueue.Submit(GraphBuilder);
		}
	}

	if (bIsFullDepthPrepassEnabled)
	{
		// When renderer is in ERendererOutput::DepthPrepassOnly mode, bShouldRenderHZB is set to false in InitViews()
		if (bShouldRenderHZB)
		{
			RenderHZB(GraphBuilder, SceneTextures.Depth.Resolve);
		}

		if (bShouldRenderVolumetricFog && bRendererOutputFinalSceneColor)
		{
			ComputeVolumetricFog(GraphBuilder, SceneTextures);
		}

		// When renderer is in ERendererOutput::DepthPrepassOnly mode, bRequiresAmbientOcclusionPass is set to false in InitViews()
		// Render SSAO after base pass when possible, for GBuffer access. AsyncCompute is suspposed to run with a BasePass
		if (bRendererOutputFinalSceneColor && ((!bDeferredShading || !bRequiresMultiPass) || MobileSSAOUsesCompute()))
		{
			BasePassTextures.AmbientOcclusionTexture = RenderAmbientOcclusion(GraphBuilder, SceneTextures);
		}

		if (bEnableDistanceFieldAO && (!bDeferredShading || !bRequiresMultiPass))
		{
			TArray<FRDGTextureRef> DynamicBentNormalAOTextures;
			const float OcclusionMaxDistance = Scene->SkyLight && !Scene->SkyLight->bWantsStaticShadowing ? Scene->SkyLight->OcclusionMaxDistance : Scene->DefaultMaxDistanceFieldOcclusionDistance;
			RenderDistanceFieldLighting(GraphBuilder, SceneTextures, FDistanceFieldAOParameters(OcclusionMaxDistance), DynamicBentNormalAOTextures, false, false, true);
		}

		// When renderer is in ERendererOutput::DepthPrepassOnly mode, bRequiresShadowProjections is set to false in InitViews()
		if (bRequiresShadowProjections)
		{
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderMobileShadowProjections);
			RDG_EVENT_SCOPE_STAT(GraphBuilder, ShadowProjection, "ShadowProjection");

			BasePassTextures.ScreenSpaceShadowMaskTexture = RenderMobileShadowProjections(GraphBuilder, SceneTextures.Config.Extent, bIsMobileMultiView);
			// In mobile deferred we always render screen space shadows after the base pass
			// TODO: ContactShadows do not work with deferred shading single pass
			if (!bDeferredShading)
			{
				RenderMobileScreenSpaceShadows(GraphBuilder, SceneTextures.Config.Extent, BasePassTextures.ScreenSpaceShadowMaskTexture);
			}
		}

		if (bRendererOutputFinalSceneColor)
		{
			if (bRequiresDBufferDecals)
			{
				// DBufferTextures may already have been created for Custom Render Passes above
				if (!BasePassTextures.DBufferTextures.DBufferA && !BasePassTextures.DBufferTextures.DBufferATexArray)
				{
					BasePassTextures.DBufferTextures = CreateDBufferTextures(GraphBuilder, SceneTextures.Config.Extent, ShaderPlatform, bIsMobileMultiView);
				}
				RenderDBuffer(GraphBuilder, Views, SceneTextures, BasePassTextures.DBufferTextures, InstanceCullingManager);
			}
		}

		// Render half res local fog volume here
		for (FViewInfo& View : Views)
		{
			if (View.LocalFogVolumeViewData.bUseHalfResLocalFogVolume)
			{
				RenderLocalFogVolumeHalfResMobile(GraphBuilder, View);
			}
		}
	}

	for (FSceneViewExtensionRef& ViewExtension : ViewFamily.ViewExtensions)
	{
		ViewExtension->PreRenderBasePass_RenderThread(GraphBuilder, bIsFullDepthPrepassEnabled /*bDepthBufferIsPopulated*/);
	}

	if (bRendererOutputFinalSceneColor)
	{
		if (bDeferredShading)
		{
			if (bRequiresMultiPass)
			{
				RenderDeferredMultiPass(GraphBuilder, SceneTextures, SortedLightSet, BasePassTextures, InstanceCullingManager);
			}
			else
			{
				RenderDeferredSinglePass(GraphBuilder, SceneTextures, SortedLightSet, BasePassTextures, InstanceCullingManager);
			}
		}
		else
		{
			RenderForward(GraphBuilder, ViewFamilyTexture, SceneTextures, BasePassTextures, InstanceCullingManager);
		}

		int32 DecalViewOffset = GetDecalViewOffset(Views);
		if (DecalViewOffset != INDEX_NONE)
		{
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				DecalVisibilityTaskData->Finish(DecalViewOffset, ViewIndex);
			}
		}

		if (DoOcclusionQueries() && !bIsFullDepthPrepassEnabled)
		{
			EndOcclusionScope(GraphBuilder, Views);
		    FenceOcclusionTests(GraphBuilder);
		}

		SceneTextures.MobileSetupMode = EMobileSceneTextureSetupMode::All;
		SceneTextures.MobileUniformBuffer = CreateMobileSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, SceneTextures.MobileSetupMode);

		if (bShouldRenderVelocities)
		{
			// Render the velocities of movable objects
			EDepthDrawingMode EarlyZPassMode = Scene ? Scene->EarlyZPassMode : DDM_None;
			if (EarlyZPassMode != DDM_AllOpaqueNoVelocity)
			{ 
				RenderVelocities(GraphBuilder, Views, SceneTextures, EVelocityPass::Opaque, false);
			}
			RenderVelocities(GraphBuilder, Views, SceneTextures, EVelocityPass::Translucent, false);

			SceneTextures.MobileSetupMode = EMobileSceneTextureSetupMode::All;
			SceneTextures.MobileUniformBuffer = CreateMobileSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, SceneTextures.MobileSetupMode);
		}

		if (!(bDeferredShading && bRequiresMultiPass))
		{
			FRendererModule& RendererModule = static_cast<FRendererModule&>(GetRendererModule());
			RendererModule.RenderPostOpaqueExtensions(GraphBuilder, Views, SceneTextures);
			
			RenderOpaqueFX(GraphBuilder, GetSceneViews(), GetSceneUniforms(), FXSystem, SceneTextures.MobileUniformBuffer);
		}

		if (Views[0].IsRuntimeReflectionCapture())
		{
			// Needs to run before CopySceneCaptureComponentToTarget, as it disables the scene color copy that occurs there,
			// if a post pass has already handled copying to the final output.
			AddRuntimeReflectionCapturePostPass(GraphBuilder, ViewFamily, ViewFamilyTexture, ViewFamilyDepthTexture, Views);
		}

		CopySceneCaptureComponentToTarget(GraphBuilder, SceneTextures, ViewFamilyTexture, ViewFamilyDepthTexture, ViewFamily, Views);
		
		if (ViewFamily.EngineShowFlags.VisualizeMeshDistanceFields || ViewFamily.EngineShowFlags.VisualizeGlobalDistanceField)
		{
			RenderMeshDistanceFieldVisualization(GraphBuilder, SceneTextures);
		}

		if (ViewFamily.bResolveScene)
		{
			if (bRenderToSceneColor && !bTonemapSubpassInline)
			{
				// Finish rendering for each view, or the full stereo buffer if enabled
				{
					RDG_EVENT_SCOPE_STAT(GraphBuilder, Postprocessing, "PostProcessing");
					SCOPE_CYCLE_COUNTER(STAT_FinishRenderViewTargetTime);

					FinishUpdateExposureCompensationCurveLUT(GraphBuilder.RHICmdList, &UpdateExposureCompensationCurveLUTTaskData);

					FMobilePostProcessingInputs PostProcessingInputs;
					PostProcessingInputs.ViewFamilyTexture = ViewFamilyTexture;
					PostProcessingInputs.ViewFamilyDepthTexture = ViewFamilyDepthTexture;
					PostProcessingInputs.SceneTextures = CreateMobileSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, EMobileSceneTextureSetupMode::All);

					for (int32 ViewExt = 0; ViewExt < ViewFamily.ViewExtensions.Num(); ++ViewExt)
					{
						for (int32 ViewIndex = 0; ViewIndex < ViewFamily.Views.Num(); ++ViewIndex)
						{
							FViewInfo& View = Views[ViewIndex];
							RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
							ViewFamily.ViewExtensions[ViewExt]->PrePostProcessPassMobile_RenderThread(GraphBuilder, View, PostProcessingInputs);
						}
					}

					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
					{
						if (Views[ViewIndex].ShouldRenderView())
						{
							RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
							if (bTonemapSubpass)
							{
								AddMobileCustomResolvePass(GraphBuilder, Views[ViewIndex], SceneTextures, ViewFamilyTexture);
							}
							else
							{ 
								AddMobilePostProcessingPasses(GraphBuilder, Scene, Views[ViewIndex], ViewIndex, GetSceneUniforms(), PostProcessingInputs, InstanceCullingManager);
							}

							if (CVarMobileXRMSAAMode.GetValueOnAnyThread() == 1)
							{
								AddDrawTexturePass(GraphBuilder, Views[ViewIndex], SceneTextures.Depth.Target, SceneTextures.Depth.Resolve);
							}
						}
					}
				}
			}

			if (ViewFamilyTexture)
			{
				GVRSImageManager.DrawDebugPreview(GraphBuilder, ViewFamily, ViewFamilyTexture);
			}
		}
	}

	GEngine->GetPostRenderDelegateEx().Broadcast(GraphBuilder);
	GetSceneExtensionsRenderers().PostRender(GraphBuilder);

	FinishUpdateExposureCompensationCurveLUT(GraphBuilder.RHICmdList, &UpdateExposureCompensationCurveLUTTaskData);

	if (bUseVirtualTexturing)
	{
		VirtualTexture::EndFeedback(GraphBuilder);
	}

	if (bRendererOutputFinalSceneColor && bShouldRenderHZB && !bRequiresMultiPass)
	{
		RenderHZB(GraphBuilder, SceneTextures.Depth.Resolve);
	}

	OnRenderFinish(GraphBuilder, ViewFamilyTexture);

	QueueSceneTextureExtractions(GraphBuilder, SceneTextures);

	if (Scene->InstanceCullingOcclusionQueryRenderer)
	{
		Scene->InstanceCullingOcclusionQueryRenderer->EndFrame(GraphBuilder);
	}

	if (bShadowStateSwapped)
	{
		Scene->SwapMobileReflectionCaptureCachedShadowState();
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FMotionVectorPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FMobileSceneRenderer::RenderStereoMotionVectors(FRDGBuilder& GraphBuilder, FViewInfo& View, const FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager)
{
	SCOPED_NAMED_EVENT(RenderStereoMotionVectors, FColor::Magenta);

	check(PlatformSupportsOpenXRMotionVectors(View.GetShaderPlatform()));

	bool bClearedRTs = false;
	if (View.ShouldRenderView())
	{
		constexpr int32 TotalMotionVectorPasses = 2;
		for (int32 PassIndex = 0; PassIndex < TotalMotionVectorPasses; ++PassIndex)
		{
			const EVelocityPass VelocityPass = PassIndex == 0 ? EVelocityPass::Opaque : EVelocityPass::Translucent;
			const EMeshPass::Type MeshPass = GetMeshPassFromVelocityPass(VelocityPass);
			FParallelMeshDrawCommandPass* ParallelMeshPass = View.ParallelMeshDrawCommandPasses[MeshPass];

			if (!ParallelMeshPass || !ParallelMeshPass->HasAnyDraw())
			{
				continue;
			}

			View.BeginRenderView();

			FRenderTargetBindingSlots MotionVectorRenderTargets;
			const ERenderTargetLoadAction LoadAction = bClearedRTs ? ERenderTargetLoadAction::ELoad : ERenderTargetLoadAction::EClear;
			MotionVectorRenderTargets[0] = FRenderTargetBinding(SceneTextures.StereoMotionVectors, LoadAction);
			MotionVectorRenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.StereoMotionVectorDepth, LoadAction, FExclusiveDepthStencil::DepthWrite_StencilWrite);

			// If the main view isn't multiview but the app is, need to render as a single-view multiview due to shaders
			MotionVectorRenderTargets.MultiViewCount = View.bIsMobileMultiViewEnabled ? 2 : (View.Aspects.IsMobileMultiViewEnabled() ? 1 : 0);

			MotionVectorRenderTargets.SubpassHint = ESubpassHint::None;
			bClearedRTs = true;

			EMobileSceneTextureSetupMode SetupMode = EMobileSceneTextureSetupMode::SceneVelocity;
			FMotionVectorPassParameters* PassParameters = GraphBuilder.AllocParameters<FMotionVectorPassParameters>();
			PassParameters->View = View.GetShaderParameters();
			PassParameters->SceneTextures = SceneTextures.UniformBuffer;
			PassParameters->RenderTargets = MotionVectorRenderTargets;

			BuildMeshRenderingCommands(GraphBuilder, MeshPass, View, Scene->GPUScene, InstanceCullingManager, PassParameters->InstanceCullingDrawParams);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("StereoMotionVectors"),
				PassParameters,
				ERDGPassFlags::Raster | ERDGPassFlags::NeverMerge,
				[this, PassParameters, MeshPass, &View, &SceneTextures](FRHICommandListImmediate& RHICmdList)
			{
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderStereoMotionVectors);
				SCOPED_DRAW_EVENT(RHICmdList, RenderStereoMotionVectors);
				SCOPE_CYCLE_COUNTER(STAT_BasePassDrawTime);
				SCOPED_GPU_MASK(RHICmdList, View.GPUMask);

				if (auto* Pass = View.ParallelMeshDrawCommandPasses[MeshPass])
				{
					// Always use full resolution velocity buffer
					FIntPoint VelocityBufferSize(SceneTextures.StereoMotionVectors->GetRHI()->GetSizeXYZ().X, SceneTextures.StereoMotionVectors->GetRHI()->GetSizeXYZ().Y);
					RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, VelocityBufferSize.X, VelocityBufferSize.Y, 1.0f);
					
					Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
				}
			});
		}
	}
}

bool FMobileSceneRenderer::HasStereoMotionVectorRenderTargets(const FSceneTextures& SceneTextures)
{
	if (SceneTextures.StereoMotionVectors != nullptr && SceneTextures.StereoMotionVectorDepth != nullptr)
	{
		return true;
	}
	return false;
}

FRenderTargetBindingSlots FMobileSceneRenderer::InitRenderTargetBindings_Forward(FRDGTextureRef ViewFamilyTexture, FSceneTextures& SceneTextures)
{
	FRDGTextureRef SceneColor = nullptr;
	FRDGTextureRef SceneColorResolve = nullptr;
	FRDGTextureRef SceneDepth = nullptr;
	FRDGTextureRef SceneDepthResolve = nullptr;

	// Verify using both MSAA sample count AND the scene color surface sample count, since on GLES you can't have MSAA color targets,
	// so the color target would be created without MSAA, and MSAA is achieved through magical means (the framebuffer, being MSAA,
	// tells the GPU "execute this renderpass as MSAA, and when you're done, automatically resolve and copy into this non-MSAA texture").
	bool bMobileMSAA = NumMSAASamples > 1;

	if (!bRenderToSceneColor)
	{
		if (bMobileMSAA)
		{
			SceneColor = SceneTextures.Color.Target;
			SceneColorResolve = ViewFamilyTexture;
		}
		else
		{
			SceneColor = ViewFamilyTexture;
		}
	}
	else
	{
		SceneColor = SceneTextures.Color.Target;
		SceneColorResolve = bMobileMSAA ? SceneTextures.Color.Resolve : nullptr;
	}
	SceneDepth = SceneTextures.Depth.Target;
	SceneDepthResolve = GRHISupportsDepthStencilResolve && bMobileMSAA && SceneTextures.Depth.IsSeparate() ? SceneTextures.Depth.Resolve : nullptr;
	
	FRenderTargetBindingSlots BasePassRenderTargets;
	BasePassRenderTargets[0] = FRenderTargetBinding(SceneColor, SceneColorResolve, ERenderTargetLoadAction::EClear);
	if (bRequiresSceneDepthAux)
	{
		BasePassRenderTargets[1] = FRenderTargetBinding(SceneTextures.DepthAux.Target, SceneTextures.DepthAux.Resolve, ERenderTargetLoadAction::EClear);
	}
		
	if (bTonemapSubpassInline)
	{
		// DepthAux is not used with tonemap subpass, since there are no post-processing passes
		// Backbuffer surface provided as a second render target instead of resolve target.
		BasePassRenderTargets[0].SetResolveTexture(nullptr);
		BasePassRenderTargets[1] = FRenderTargetBinding(ViewFamilyTexture, nullptr, ERenderTargetLoadAction::EClear);
	}
	
	BasePassRenderTargets.DepthStencil = bIsFullDepthPrepassEnabled ?
		FDepthStencilBinding(SceneDepth, SceneDepthResolve, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite) :
		FDepthStencilBinding(SceneDepth, SceneDepthResolve, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);
	BasePassRenderTargets.SubpassHint = ESubpassHint::None;
	BasePassRenderTargets.NumOcclusionQueries = 0u;

#if WITH_EDITOR
	if (ViewFamily.EngineShowFlags.Wireframe)
	{
		BasePassRenderTargets[0].SetTexture(SceneTextures.EditorPrimitiveColor);
		BasePassRenderTargets[0].SetResolveTexture(nullptr);
		BasePassRenderTargets.DepthStencil.SetTexture(SceneTextures.EditorPrimitiveDepth);
		BasePassRenderTargets.DepthStencil.SetResolveTexture(nullptr);
	}
#endif

	return BasePassRenderTargets;
}

void FMobileSceneRenderer::RenderForward(FRDGBuilder& GraphBuilder, FRDGTextureRef ViewFamilyTexture, FSceneTextures& SceneTextures, FMobileBasePassTextures& BasePassTextures, FInstanceCullingManager& InstanceCullingManager)
{
	const FViewInfo& MainView = Views[0];

	GVRSImageManager.PrepareImageBasedVRS(GraphBuilder, ViewFamily, SceneTextures);
	FRDGTextureRef NewShadingRateTarget = GVRSImageManager.GetVariableRateShadingImage(GraphBuilder, MainView, FVariableRateShadingImageManager::EVRSPassType::BasePass);

	FRenderTargetBindingSlots BasePassRenderTargets = InitRenderTargetBindings_Forward(ViewFamilyTexture, SceneTextures);
	BasePassRenderTargets.ShadingRateTexture = (!MainView.bIsSceneCapture && !MainView.bIsReflectionCapture && (NewShadingRateTarget != nullptr)) ? NewShadingRateTarget : nullptr;

	//if the scenecolor isn't multiview but the app is, need to render as a single-view multiview due to shaders
	BasePassRenderTargets.MultiViewCount = (MainView.bIsMobileMultiViewEnabled) ? 2 : (MainView.Aspects.IsMobileMultiViewEnabled() ? 1 : 0);

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	FRenderViewContextArray RenderViews;
	GetRenderViews(Views, GetDecalViewOffset(Views), RenderViews);

	bool bDoOcclusionQueries = !bIsFullDepthPrepassEnabled && DoOcclusionQueries();
	FViewOcclusionQueriesPerView QueriesPerView;
	if (bDoOcclusionQueries)
	{
		AllocateOcclusionTests(Scene, QueriesPerView, Views, VisibleLightInfos);
		bDoOcclusionQueries = (QueriesPerView.Num() > 0);
	}

	for (FRenderViewContext& ViewContext : RenderViews)
	{
		FViewInfo& View = *ViewContext.ViewInfo;

		SCOPED_GPU_MASK(GraphBuilder.RHICmdList, !View.IsInstancedStereo() ? View.GPUMask : (View.GPUMask | View.GetInstancedView()->GPUMask));
		SCOPED_CONDITIONAL_DRAW_EVENTF(GraphBuilder.RHICmdList, EventView, RenderViews.Num() > 1, TEXT("View%d"), ViewContext.ViewIndex);

		if (View.Family->EngineShowFlags.StereoMotionVectors
			&& HasStereoMotionVectorRenderTargets(SceneTextures) 
			&& !View.bIsSceneCapture && !View.bIsReflectionCapture)
		{
			RenderStereoMotionVectors(GraphBuilder, View, SceneTextures, InstanceCullingManager);
		}

		if (!ViewContext.bIsFirstView)
		{
			BasePassRenderTargets[0].SetLoadAction(ERenderTargetLoadAction::ELoad);
			if (bRequiresSceneDepthAux)
			{
				BasePassRenderTargets[1].SetLoadAction(ERenderTargetLoadAction::ELoad);
			}
			BasePassRenderTargets.DepthStencil.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);
			BasePassRenderTargets.DepthStencil.SetStencilLoadAction(ERenderTargetLoadAction::ELoad);
			BasePassRenderTargets.DepthStencil.SetDepthStencilAccess(bIsFullDepthPrepassEnabled ? FExclusiveDepthStencil::DepthRead_StencilWrite : FExclusiveDepthStencil::DepthWrite_StencilWrite);
		}

		View.BeginRenderView();

		UpdateDirectionalLightUniformBuffers(GraphBuilder, View);
			
		EMobileSceneTextureSetupMode SetupMode = (bIsFullDepthPrepassEnabled ? EMobileSceneTextureSetupMode::SceneDepth : EMobileSceneTextureSetupMode::None) | EMobileSceneTextureSetupMode::CustomDepth;
		FMobileRenderPassParameters* PassParameters = GraphBuilder.AllocParameters<FMobileRenderPassParameters>();
		PassParameters->View = View.GetShaderParameters();
		PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, EMobileBasePass::Opaque, SetupMode, BasePassTextures);
		PassParameters->ReflectionCapture = View.MobileReflectionCaptureUniformBuffer;
		PassParameters->RenderTargets = BasePassRenderTargets;
		PassParameters->LocalFogVolumeInstances = View.LocalFogVolumeViewData.GPUInstanceDataBufferSRV;
		PassParameters->LocalFogVolumeTileDrawIndirectBuffer = View.LocalFogVolumeViewData.GPUTileDrawIndirectBuffer;
		PassParameters->LocalFogVolumeTileDataTexture = View.LocalFogVolumeViewData.TileDataTextureArraySRV;
		PassParameters->LocalFogVolumeTileDataBuffer = View.LocalFogVolumeViewData.GPUTileDataBufferSRV;
		PassParameters->HalfResLocalFogVolumeViewSRV = View.LocalFogVolumeViewData.HalfResLocalFogVolumeViewSRV;
		PassParameters->HalfResLocalFogVolumeDepthSRV = View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepthSRV;

		FViewOcclusionQueries* ViewQueries = bDoOcclusionQueries ? &QueriesPerView[ViewContext.ViewIndex] : nullptr;
		
		// Split if we need to render translucency in a separate render pass
		if (bRequiresMultiPass)
		{
			RenderForwardMultiPass(GraphBuilder, PassParameters, ViewContext, SceneTextures, InstanceCullingManager, ViewQueries);
		}
		else
		{
			RenderForwardSinglePass(GraphBuilder, PassParameters, ViewContext, SceneTextures, InstanceCullingManager, ViewQueries);
		}
	}
	
	if (ViewFamily.EngineShowFlags.AlphaInvert)
	{
		AlphaInvert::AddAlphaInvertPass(GraphBuilder, MainView, SceneTextures);
	}
}

void FMobileSceneRenderer::RenderForwardSinglePass(FRDGBuilder& GraphBuilder, FMobileRenderPassParameters* PassParameters, FRenderViewContext& ViewContext, FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager, FViewOcclusionQueries* ViewOcclusionQueries)
{
	struct FForwardSinglePassParameterCollection
	{
		FInstanceCullingDrawParams DepthPassInstanceCullingDrawParams;
		FInstanceCullingDrawParams SkyPassInstanceCullingDrawParams;
		FInstanceCullingDrawParams MeshDecalSceneColorInstanceCullingDrawParams;
		FInstanceCullingDrawParams TranslucencyInstanceCullingDrawParams;
	};
	auto ParameterCollection = GraphBuilder.AllocParameters<FForwardSinglePassParameterCollection>();

	FViewInfo& View = *ViewContext.ViewInfo;

	if (Scene->GPUScene.IsEnabled())
	{
		if (!bIsFullDepthPrepassEnabled)
		{
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::DepthPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->DepthPassInstanceCullingDrawParams);
		}
		BuildMeshRenderingCommands(GraphBuilder, EMeshPass::BasePass, View, Scene->GPUScene, InstanceCullingManager, PassParameters->InstanceCullingDrawParams);
		BuildMeshRenderingCommands(GraphBuilder, EMeshPass::SkyPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->SkyPassInstanceCullingDrawParams);
		BuildMeshRenderingCommands(GraphBuilder, EMeshPass::MeshDecal_SceneColor, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->MeshDecalSceneColorInstanceCullingDrawParams);
		BuildMeshRenderingCommands(GraphBuilder, StandardTranslucencyMeshPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->TranslucencyInstanceCullingDrawParams);
	}

	if (bTonemapSubpassInline)
	{
		// tonemapping LUT pass before we start main render pass. The texture is needed by the custom resolve pass which does tonemapping
		PassParameters->ColorGradingLUT = AddCombineLUTPass(GraphBuilder, *ViewContext.ViewInfo);
	}
		
	PassParameters->RenderTargets.SubpassHint = bTonemapSubpassInline ? ESubpassHint::CustomResolveSubpass : ESubpassHint::DepthReadSubpass;
	const bool bDoOcclusionQueries = (ViewOcclusionQueries != nullptr);
	const int32 NumOcclusionQueries = bDoOcclusionQueries ? ComputeNumOcclusionQueriesToBatch() : 0u;
	const bool bVulkanAdrenoOcclusionMode = bAdrenoOcclusionMode && IsVulkanPlatform(ShaderPlatform);
	const bool bDoOcclusionInMainPass = bDoOcclusionQueries && !bVulkanAdrenoOcclusionMode;
	PassParameters->RenderTargets.NumOcclusionQueries = bVulkanAdrenoOcclusionMode ? 0u : NumOcclusionQueries;
	
	EDecalRenderStage DecalStage = bRequiresDBufferDecals ? EDecalRenderStage::Emissive : EDecalRenderStage::Mobile;
	FDecalViewInfo DecalViewInfo = FDecalViewInfo(ViewContext.DecalViewOffset, ViewContext.ViewIndex,
												  DecalStage,
												  EDecalRenderTargetMode::SceneColor,
												  GetRelevantDeferredDecals(ViewContext.DecalViewOffset, ViewContext.ViewIndex, DecalStage));

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("SceneColorRendering"),
		PassParameters,
		// the second view pass should not be merged with the first view pass on mobile since the subpass would not work properly.
		ERDGPassFlags::Raster | ERDGPassFlags::NeverMerge,
		[this, DecalViewInfo, PassParameters, ParameterCollection, ViewContext, bDoOcclusionInMainPass, &SceneTextures, ViewQueries = bDoOcclusionInMainPass ? MoveTemp(*ViewOcclusionQueries) : FViewOcclusionQueries()](FRHICommandList& RHICmdList)
	{
		FViewInfo& View = *ViewContext.ViewInfo;
			
		if (GIsEditor && !View.bIsSceneCapture && ViewContext.bIsFirstView)
		{
			DrawClearQuad(RHICmdList, View.BackgroundColor);
		}

		// Depth pre-pass
		RenderMaskedPrePass(RHICmdList, View, &ParameterCollection->DepthPassInstanceCullingDrawParams);
		// Opaque and masked
		RenderMobileBasePass(RHICmdList, View, &PassParameters->InstanceCullingDrawParams, &ParameterCollection->SkyPassInstanceCullingDrawParams);

		PostRenderBasePass(RHICmdList, View);
		// scene depth is read only and can be fetched
		RHICmdList.NextSubpass();
		RenderDecals(RHICmdList, View, DecalViewInfo, &ParameterCollection->MeshDecalSceneColorInstanceCullingDrawParams);
		RenderModulatedShadowProjections(RHICmdList, ViewContext.ViewIndex, View);
		if (GMaxRHIShaderPlatform != SP_METAL_SIM)
		{
			RenderFog(RHICmdList, View);
		}
		// Draw translucency.
		RenderTranslucency(RHICmdList, View, Views, StandardTranslucencyPass, StandardTranslucencyMeshPass, &ParameterCollection->TranslucencyInstanceCullingDrawParams);
		
#if UE_ENABLE_DEBUG_DRAWING
		if ((!IsMobileHDR() || bTonemapSubpass) && IsDebugPrimitivePassEnabled(View))
		{
			// Draw debug primitives after translucency for LDR as we do not have a post processing pass
			RenderMobileDebugPrimitives(RHICmdList, View);
		}
#endif

		if (bDoOcclusionInMainPass)
		{
			RenderOcclusion(RHICmdList, View, ViewQueries);
		}

		// Pre-tonemap before MSAA resolve (iOS only)
		PreTonemapMSAA(RHICmdList, SceneTextures);
		if (bTonemapSubpassInline)
		{
			RHICmdList.NextSubpass();
			RenderMobileCustomResolve(RHICmdList, View, NumMSAASamples, SceneTextures);
		}
	});

	RenderMobileDebugViewMode(GraphBuilder, View, SceneTextures, InstanceCullingManager, *PassParameters);

#if UE_ENABLE_DEBUG_DRAWING
	if (ViewFamily.EngineShowFlags.VisualizeInstanceOcclusionQueries && Scene->InstanceCullingOcclusionQueryRenderer)
	{
		Scene->InstanceCullingOcclusionQueryRenderer->RenderDebug(GraphBuilder, Scene->GPUScene, View, View.ViewRect, SceneTextures.Color.Resolve, nullptr);
	}
#endif
	
	// resolve MSAA depth
	if (!GRHISupportsDepthStencilResolve && !bIsFullDepthPrepassEnabled)
	{
		AddResolveSceneDepthPass(GraphBuilder, *ViewContext.ViewInfo, SceneTextures.Depth);
	}

	if (bDoOcclusionQueries && bVulkanAdrenoOcclusionMode)
	{
		FMobileRenderPassParameters* OcclusionPassParams = GraphBuilder.AllocParameters<FMobileRenderPassParameters>();
		OcclusionPassParams->View = PassParameters->View;
		OcclusionPassParams->RenderTargets.DepthStencil = FDepthStencilBinding(SceneTextures.Depth.Resolve, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilNop);
		OcclusionPassParams->RenderTargets.MultiViewCount = PassParameters->RenderTargets.MultiViewCount;
		OcclusionPassParams->RenderTargets.NumOcclusionQueries = NumOcclusionQueries;
		if (GAdrenoOcclusionUseFDM.GetValueOnRenderThread())
		{
			OcclusionPassParams->RenderTargets.ShadingRateTexture = PassParameters->RenderTargets.ShadingRateTexture;
		}

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("VulkanAdrenoOcclusion"),
			OcclusionPassParams,
			// The occlusion pass needs to be unique to be optimized properly by the driver, don't merge it.
			// This pass has no observable outputs on the RenderGraph, so it needs to be marked as NeverCull.
			ERDGPassFlags::Raster | ERDGPassFlags::NeverMerge | ERDGPassFlags::NeverCull,
			[this, ViewContext, ViewQueries = MoveTemp(*ViewOcclusionQueries)](FRHICommandListImmediate& RHICmdList)
		{
			FViewInfo& View = *ViewContext.ViewInfo;
			RenderOcclusion(RHICmdList, View, ViewQueries);
		});
	}
}

void FMobileSceneRenderer::RenderForwardMultiPass(FRDGBuilder& GraphBuilder, FMobileRenderPassParameters* PassParameters, FRenderViewContext& ViewContext, FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager, FViewOcclusionQueries* ViewOcclusionQueries)
{
	struct FForwardFirstPassParameterCollection
	{
		FInstanceCullingDrawParams DepthPassInstanceCullingDrawParams;
		FInstanceCullingDrawParams SkyPassInstanceCullingDrawParams;
	};
	auto ParameterCollection = GraphBuilder.AllocParameters<FForwardFirstPassParameterCollection>();

	FViewInfo& View = *ViewContext.ViewInfo;

	if (Scene->GPUScene.IsEnabled())
	{
		if (!bIsFullDepthPrepassEnabled)
		{
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::DepthPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->DepthPassInstanceCullingDrawParams);
		}
		BuildMeshRenderingCommands(GraphBuilder, EMeshPass::BasePass, View, Scene->GPUScene, InstanceCullingManager, PassParameters->InstanceCullingDrawParams);
		BuildMeshRenderingCommands(GraphBuilder, EMeshPass::SkyPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->SkyPassInstanceCullingDrawParams);
	}

	EDecalRenderStage DecalStage = bRequiresDBufferDecals ? EDecalRenderStage::Emissive : EDecalRenderStage::Mobile;
	FDecalViewInfo DecalViewInfo = FDecalViewInfo(ViewContext.DecalViewOffset, ViewContext.ViewIndex, 
												  DecalStage,
												  EDecalRenderTargetMode::SceneColor,
												  GetRelevantDeferredDecals(ViewContext.DecalViewOffset, ViewContext.ViewIndex, DecalStage));

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("SceneColorRendering"),
		PassParameters,
		ERDGPassFlags::Raster,
		[this, PassParameters, ParameterCollection, ViewContext, &SceneTextures](FRHICommandList& RHICmdList)
	{
		FViewInfo& View = *ViewContext.ViewInfo;
			
		if (GIsEditor && !View.bIsSceneCapture && ViewContext.bIsFirstView)
		{
			DrawClearQuad(RHICmdList, View.BackgroundColor);
		}

		// Depth pre-pass
		RenderMaskedPrePass(RHICmdList, View, &ParameterCollection->DepthPassInstanceCullingDrawParams);
		// Opaque and masked
		RenderMobileBasePass(RHICmdList, View, &PassParameters->InstanceCullingDrawParams, &ParameterCollection->SkyPassInstanceCullingDrawParams);

		PostRenderBasePass(RHICmdList, View);
	});

	RenderMobileDebugViewMode(GraphBuilder, View, SceneTextures, InstanceCullingManager, *PassParameters);

	// resolve MSAA depth
	if (!bIsFullDepthPrepassEnabled)
	{
		AddResolveSceneDepthPass(GraphBuilder, View, SceneTextures.Depth);
	}
	if (bRequiresSceneDepthAux)
	{
		AddResolveSceneColorPass(GraphBuilder, View, SceneTextures.DepthAux);
	}
	if (bShouldRenderHZB && !bIsFullDepthPrepassEnabled)
	{
		RenderHZB(GraphBuilder, SceneTextures.Depth.Resolve);
	}

	FExclusiveDepthStencil::Type ExclusiveDepthStencil = FExclusiveDepthStencil::DepthRead_StencilRead;
	if (bModulatedShadowsInUse)
	{
		// FIXME: modulated shadows write to stencil
		ExclusiveDepthStencil = FExclusiveDepthStencil::DepthRead_StencilWrite;
	}

	EMobileSceneTextureSetupMode SetupMode = EMobileSceneTextureSetupMode::SceneDepth | EMobileSceneTextureSetupMode::SceneDepthAux | EMobileSceneTextureSetupMode::CustomDepth;

	struct FForwardSecondPassParameterCollection
	{
		FMobileRenderPassParameters PassParameters;
		FInstanceCullingDrawParams MeshDecalSceneColorInstanceCullingDrawParams;
	};
	auto SecondParameterCollection = GraphBuilder.AllocParameters<FForwardSecondPassParameterCollection>();

	auto SecondPassParameters = &SecondParameterCollection->PassParameters;
	*SecondPassParameters = *PassParameters;
	SecondPassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, EMobileBasePass::Translucent, SetupMode);
	SecondPassParameters->ReflectionCapture = View.MobileReflectionCaptureUniformBuffer;
	SecondPassParameters->RenderTargets[0].SetLoadAction(ERenderTargetLoadAction::ELoad);
	SecondPassParameters->RenderTargets[1] = FRenderTargetBinding();
	SecondPassParameters->RenderTargets.DepthStencil.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);
	SecondPassParameters->RenderTargets.DepthStencil.SetStencilLoadAction(ERenderTargetLoadAction::ELoad);
	SecondPassParameters->RenderTargets.DepthStencil.SetDepthStencilAccess(ExclusiveDepthStencil);
	
	const bool bDoOcclusionQueries = (ViewOcclusionQueries != nullptr);
	SecondPassParameters->RenderTargets.NumOcclusionQueries = bDoOcclusionQueries ? ComputeNumOcclusionQueriesToBatch() : 0u;

	if (Scene->GPUScene.IsEnabled())
	{
		BuildMeshRenderingCommands(GraphBuilder, EMeshPass::MeshDecal_SceneColor, View, Scene->GPUScene, InstanceCullingManager, SecondParameterCollection->MeshDecalSceneColorInstanceCullingDrawParams);
		BuildMeshRenderingCommands(GraphBuilder, StandardTranslucencyMeshPass, View, Scene->GPUScene, InstanceCullingManager, SecondPassParameters->InstanceCullingDrawParams);
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("DecalsAndTranslucency"),
		SecondPassParameters,
		ERDGPassFlags::Raster,
		[this, SecondParameterCollection, ViewContext, bDoOcclusionQueries, &SceneTextures, ViewQueries = (bDoOcclusionQueries) ? MoveTemp(*ViewOcclusionQueries) : FViewOcclusionQueries(), DecalViewInfo](FRHICommandList& RHICmdList)
	{
		FViewInfo& View = *ViewContext.ViewInfo;
		auto SecondPassParameters = &SecondParameterCollection->PassParameters;
			
		// scene depth is read only and can be fetched
		RenderDecals(RHICmdList, View, DecalViewInfo, &SecondParameterCollection->MeshDecalSceneColorInstanceCullingDrawParams);
		RenderModulatedShadowProjections(RHICmdList, ViewContext.ViewIndex, View);
		RenderFog(RHICmdList, View);
		// Draw translucency.
		RenderTranslucency(RHICmdList, View, Views, StandardTranslucencyPass, StandardTranslucencyMeshPass, &SecondPassParameters->InstanceCullingDrawParams);

		if (bDoOcclusionQueries)
		{
			// Issue occlusion queries
			RenderOcclusion(RHICmdList, View, ViewQueries);
		}

		// Pre-tonemap before MSAA resolve (iOS only)
		PreTonemapMSAA(RHICmdList, SceneTextures);
	});

	AddResolveSceneColorPass(GraphBuilder, View, SceneTextures.Color);
}

class FMobileDeferredCopyPLSPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FMobileDeferredCopyPLSPS, Global);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform) && IsMobileDeferredShadingEnabled(Parameters.Platform);
	}

	/** Default constructor. */
	FMobileDeferredCopyPLSPS() {}

	/** Initialization constructor. */
	FMobileDeferredCopyPLSPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}
};

IMPLEMENT_SHADER_TYPE(, FMobileDeferredCopyPLSPS, TEXT("/Engine/Private/MobileDeferredUtils.usf"), TEXT("MobileDeferredCopyPLSPS"), SF_Pixel);

class FMobileDeferredCopyDepthPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FMobileDeferredCopyDepthPS, Global);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform) && IsMobileDeferredShadingEnabled(Parameters.Platform);
	}

	/** Default constructor. */
	FMobileDeferredCopyDepthPS() {}

	/** Initialization constructor. */
	FMobileDeferredCopyDepthPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}
};

IMPLEMENT_SHADER_TYPE(, FMobileDeferredCopyDepthPS, TEXT("/Engine/Private/MobileDeferredUtils.usf"), TEXT("MobileDeferredCopyDepthPS"), SF_Pixel);

template<class T>
void MobileDeferredCopyBuffer(FRHICommandList& RHICmdList, const FViewInfo& View)
{
	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

	TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);
	TShaderMapRef<T> PixelShader(View.ShaderMap);

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0u);

	DrawRectangle(
		RHICmdList,
		0, 0,
		View.ViewRect.Width(), View.ViewRect.Height(),
		View.ViewRect.Min.X, View.ViewRect.Min.Y,
		View.ViewRect.Width(), View.ViewRect.Height(),
		FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()),
		View.GetSceneTexturesConfig().Extent,
		VertexShader);
}

static bool UsingPixelLocalStorage(const FStaticShaderPlatform ShaderPlatform)
{
	return IsAndroidOpenGLESPlatform(ShaderPlatform) && GSupportsPixelLocalStorage && GSupportsShaderDepthStencilFetch;
}

FColorTargets FMobileSceneRenderer::GetColorTargets_Deferred(FSceneTextures& SceneTextures)
{
	FColorTargets ColorTargets;

	// If we are using GL and don't have FBF support, use PLS
	bool bUsingPixelLocalStorage = UsingPixelLocalStorage(ShaderPlatform);

	if (bUsingPixelLocalStorage)
	{
		ColorTargets.Add(SceneTextures.Color.Target);
	}
	else
	{
		ColorTargets.Add(SceneTextures.Color.Target);
		ColorTargets.Add(SceneTextures.GBufferA);
		ColorTargets.Add(SceneTextures.GBufferB);
		ColorTargets.Add(SceneTextures.GBufferC);
		
		if (MobileUsesExtendedGBuffer(ShaderPlatform))
		{
			ColorTargets.Add(SceneTextures.GBufferD);
		}

		if (bRequiresSceneDepthAux)
		{
			ColorTargets.Add(SceneTextures.DepthAux.Target);
		}
	}

	return ColorTargets;
}

FRenderTargetBindingSlots FMobileSceneRenderer::InitRenderTargetBindings_Deferred(FSceneTextures& SceneTextures, TArray<FRDGTextureRef, TInlineAllocator<6>>& ColorTargets)
{
	TArrayView<FRDGTextureRef> BasePassTexturesView = MakeArrayView(ColorTargets);
	FRenderTargetBindingSlots BasePassRenderTargets = GetRenderTargetBindings(ERenderTargetLoadAction::EClear, BasePassTexturesView);
	BasePassRenderTargets.DepthStencil = bIsFullDepthPrepassEnabled ? 
		FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite) : 
		FDepthStencilBinding(SceneTextures.Depth.Target, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);
	BasePassRenderTargets.SubpassHint = ESubpassHint::None;
	BasePassRenderTargets.NumOcclusionQueries = 0u;
	BasePassRenderTargets.ShadingRateTexture = nullptr;
	
	//if the scenecolor isn't multiview but the app is, need to render as a single-view multiview due to shaders
	BasePassRenderTargets.MultiViewCount = (Views[0].bIsMobileMultiViewEnabled) ? 2 : (Views[0].Aspects.IsMobileMultiViewEnabled() ? 1 : 0);

#if WITH_EDITOR
	if (ViewFamily.EngineShowFlags.Wireframe)
	{
		BasePassRenderTargets[0].SetTexture(SceneTextures.EditorPrimitiveColor);
		BasePassRenderTargets[0].SetResolveTexture(nullptr);
		BasePassRenderTargets.DepthStencil.SetTexture(SceneTextures.EditorPrimitiveDepth);
		BasePassRenderTargets.DepthStencil.SetResolveTexture(nullptr);
	}
#endif

	return BasePassRenderTargets;
}

void FMobileSceneRenderer::RenderDeferredSinglePass(FRDGBuilder& GraphBuilder, FSceneTextures& SceneTextures, const FSortedLightSetSceneInfo& SortedLightSet, FMobileBasePassTextures& BasePassTextures, FInstanceCullingManager& InstanceCullingManager)
{
	bool bUsingPixelLocalStorage = UsingPixelLocalStorage(ShaderPlatform);
	FColorTargets ColorTargets = GetColorTargets_Deferred(SceneTextures);
	FRenderTargetBindingSlots BasePassRenderTargets = InitRenderTargetBindings_Deferred(SceneTextures, ColorTargets);
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
	
	FRenderViewContextArray RenderViews;
	GetRenderViews(Views, GetDecalViewOffset(Views), RenderViews);

	bool bDoOcclusionQueries = (!bIsFullDepthPrepassEnabled && DoOcclusionQueries());
	FViewOcclusionQueriesPerView QueriesPerView;
	if (bDoOcclusionQueries)
	{
		AllocateOcclusionTests(Scene, QueriesPerView, Views, VisibleLightInfos);
		bDoOcclusionQueries = QueriesPerView.Num() > 0;
	}

	bool bRenderDeferredLighting = ViewFamily.EngineShowFlags.Lighting && ViewFamily.EngineShowFlags.DeferredLighting;

	for (FRenderViewContext& ViewContext : RenderViews)
	{
		FViewInfo& View = *ViewContext.ViewInfo;

		SCOPED_GPU_MASK(GraphBuilder.RHICmdList, !View.IsInstancedStereo() ? View.GPUMask : (View.GPUMask | View.GetInstancedView()->GPUMask));
		SCOPED_CONDITIONAL_DRAW_EVENTF(GraphBuilder.RHICmdList, EventView, RenderViews.Num() > 1, TEXT("View%d"), ViewContext.ViewIndex);

		if (!ViewContext.bIsFirstView)
		{
			// Load targets for a non-first view 
			for (int32 i = 0; i < ColorTargets.Num(); ++i)
			{
				BasePassRenderTargets[i].SetLoadAction(ERenderTargetLoadAction::ELoad);
			}
			BasePassRenderTargets.DepthStencil.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);
			BasePassRenderTargets.DepthStencil.SetStencilLoadAction(ERenderTargetLoadAction::ELoad);
			BasePassRenderTargets.DepthStencil.SetDepthStencilAccess(bIsFullDepthPrepassEnabled ? FExclusiveDepthStencil::DepthRead_StencilWrite : FExclusiveDepthStencil::DepthWrite_StencilWrite);
		}

		View.BeginRenderView();

		UpdateDirectionalLightUniformBuffers(GraphBuilder, View);

		EMobileSceneTextureSetupMode SetupMode = (bIsFullDepthPrepassEnabled ? EMobileSceneTextureSetupMode::SceneDepth : EMobileSceneTextureSetupMode::None) | EMobileSceneTextureSetupMode::CustomDepth;

		struct FDeferredSinglePassParameterCollection
		{
			FMobileRenderPassParameters PassParameters;
			FInstanceCullingDrawParams DepthPassInstanceCullingDrawParams;
			FInstanceCullingDrawParams SkyPassInstanceCullingDrawParams;
			FInstanceCullingDrawParams MeshDecalSceneColorAndGBufferInstanceCullingDrawParams;
			FInstanceCullingDrawParams TranslucencyInstanceCullingDrawParams;
		};
		auto ParameterCollection = GraphBuilder.AllocParameters<FDeferredSinglePassParameterCollection>();

		auto PassParameters = &ParameterCollection->PassParameters;
		PassParameters->View = View.GetShaderParameters();
		PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, EMobileBasePass::Opaque, SetupMode, BasePassTextures);
		PassParameters->ReflectionCapture = View.MobileReflectionCaptureUniformBuffer;
		PassParameters->LocalFogVolumeInstances = View.LocalFogVolumeViewData.GPUInstanceDataBufferSRV;
		PassParameters->LocalFogVolumeTileDrawIndirectBuffer = View.LocalFogVolumeViewData.GPUTileDrawIndirectBuffer;
		PassParameters->LocalFogVolumeTileDataTexture = View.LocalFogVolumeViewData.TileDataTextureArraySRV;
		PassParameters->LocalFogVolumeTileDataBuffer = View.LocalFogVolumeViewData.GPUTileDataBufferSRV;
		PassParameters->HalfResLocalFogVolumeViewSRV = View.LocalFogVolumeViewData.HalfResLocalFogVolumeViewSRV;
		PassParameters->HalfResLocalFogVolumeDepthSRV = View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepthSRV;
		PassParameters->RenderTargets = BasePassRenderTargets;
		PassParameters->RenderTargets.SubpassHint = ESubpassHint::DeferredShadingSubpass;
		const EMobileSSRQuality MobileSSRQuality = ActiveMobileSSRQuality(View, bShouldRenderVelocities);
		PassParameters->RenderTargets.NumOcclusionQueries = bDoOcclusionQueries ? ComputeNumOcclusionQueriesToBatch() : 0u;

		if (Scene->GPUScene.IsEnabled())
		{
			if (!bIsFullDepthPrepassEnabled)
			{
				BuildMeshRenderingCommands(GraphBuilder, EMeshPass::DepthPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->DepthPassInstanceCullingDrawParams);
			}
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::BasePass, View, Scene->GPUScene, InstanceCullingManager, PassParameters->InstanceCullingDrawParams);
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::SkyPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->SkyPassInstanceCullingDrawParams);
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::MeshDecal_SceneColorAndGBuffer, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->MeshDecalSceneColorAndGBufferInstanceCullingDrawParams);
			BuildMeshRenderingCommands(GraphBuilder, StandardTranslucencyMeshPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->TranslucencyInstanceCullingDrawParams);
		}

		EDecalRenderStage DecalStage = bRequiresDBufferDecals ? EDecalRenderStage::Emissive : EDecalRenderStage::MobileBeforeLighting;
		FDecalViewInfo DecalViewInfo = FDecalViewInfo(ViewContext.DecalViewOffset, ViewContext.ViewIndex,
													  DecalStage,
													  EDecalRenderTargetMode::SceneColorAndGBuffer,
													  GetRelevantDeferredDecals(ViewContext.DecalViewOffset, ViewContext.ViewIndex, DecalStage));

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("SceneColorRendering"),
			PassParameters,
			// the second view pass should not be merged with the first view pass on mobile since the subpass would not work properly.
			ERDGPassFlags::Raster | ERDGPassFlags::NeverMerge,
			[this, 
			ParameterCollection, 
			ViewContext, 
			MobileSSRQuality, 
			&SortedLightSet, 
			bUsingPixelLocalStorage,
			bDoOcclusionQueries, 
			bRenderDeferredLighting,
			DecalViewInfo,
			ViewQueries = (bDoOcclusionQueries) ? MoveTemp(QueriesPerView[ViewContext.ViewIndex]) : FViewOcclusionQueries()](FRHICommandList& RHICmdList)
			{
				FViewInfo& View = *ViewContext.ViewInfo;
				auto PassParameters = &ParameterCollection->PassParameters;

				// Depth pre-pass
				RenderMaskedPrePass(RHICmdList, View, &ParameterCollection->DepthPassInstanceCullingDrawParams);
				// Opaque and masked
				RenderMobileBasePass(RHICmdList, View, &PassParameters->InstanceCullingDrawParams, &ParameterCollection->SkyPassInstanceCullingDrawParams);

				PostRenderBasePass(RHICmdList, View);
				// SceneColor + GBuffer write, SceneDepth is read only
				RHICmdList.NextSubpass();
				RenderDecals(RHICmdList, View, DecalViewInfo, &ParameterCollection->MeshDecalSceneColorAndGBufferInstanceCullingDrawParams);
				// SceneColor write, SceneDepth is read only
				RHICmdList.NextSubpass();
				if (bRenderDeferredLighting)
				{
					MobileDeferredShadingPass(RHICmdList, ViewContext.ViewIndex, Views.Num(), View, *Scene, SortedLightSet, VisibleLightInfos, MobileSSRQuality);
				}

				if (bUsingPixelLocalStorage)
				{
					MobileDeferredCopyBuffer<FMobileDeferredCopyPLSPS>(RHICmdList, View);
				}
				RenderFog(RHICmdList, View);
				// Draw translucency.
				RenderTranslucency(RHICmdList, View, Views, StandardTranslucencyPass, StandardTranslucencyMeshPass, &ParameterCollection->TranslucencyInstanceCullingDrawParams);

				if (bDoOcclusionQueries)
				{
					// Issue occlusion queries
					RenderOcclusion(RHICmdList, View, ViewQueries);
				}
			});

		RenderMobileDebugViewMode(GraphBuilder, View, SceneTextures, InstanceCullingManager, *PassParameters);
	}
}

void FMobileSceneRenderer::RenderDeferredMultiPassDecals(FRDGBuilder& GraphBuilder, FViewInfo& View, int32 DecalViewOffset, uint32 ViewIndex, FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager)
{
	if (DecalViewOffset == INDEX_NONE ||
		(Scene->Decals.Num() == 0 && !HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::MeshDecal_SceneColorAndGBuffer])
								  && !HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::MeshDecal_SceneColorAndGBufferNoNormal])))
	{
		// Skip decal render-pass if there is nothing to draw
		return;
	}

	View.BeginRenderView();

	const EDecalRenderTargetMode DecalRenderTargetModes[] = { EDecalRenderTargetMode::SceneColorAndGBuffer, EDecalRenderTargetMode::SceneColorAndGBufferNoNormal };
	const EDecalRenderStage DecalRenderStage = bRequiresDBufferDecals ? EDecalRenderStage::Emissive : EDecalRenderStage::MobileBeforeLighting;
	constexpr const TCHAR* PassNames[2] = { TEXT("SceneColorAndGBuffer"), TEXT("SceneColorAndGBufferNoNormal")};
	TConstArrayView<const FVisibleDecal*> RelevantDeferredDecals = GetRelevantDeferredDecals(DecalViewOffset, ViewIndex, DecalRenderStage);

	for (const EDecalRenderTargetMode DecalRenderTargetMode : DecalRenderTargetModes)
	{
		EMeshPass::Type MeshDecalsPass = DecalRendering::GetMeshPassType(DecalRenderTargetMode);

		const FVisibleDecal* const* FirstDecal = RelevantDeferredDecals.FindByPredicate( [&View, &DecalRenderTargetMode, &DecalRenderStage](const FVisibleDecal* Decal) { return DecalRenderTargetMode == DecalRendering::GetRenderTargetMode(View.GetShaderPlatform(), Decal->BlendDesc, DecalRenderStage); });
		if (FirstDecal == nullptr && !HasAnyDraw(View.ParallelMeshDrawCommandPasses[MeshDecalsPass]))
		{
			continue;
		}

		auto PassParameters = GraphBuilder.AllocParameters<FMobileRenderPassParameters>();
		PassParameters->View = View.GetShaderParameters();
		
		uint32 ColorTargetIndex = 0;
		FRenderTargetBindingSlots& RenderTargets = PassParameters->RenderTargets;
		RenderTargets[ColorTargetIndex++] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
		if (DecalRenderTargetMode != EDecalRenderTargetMode::SceneColorAndGBufferNoNormal)
		{
			RenderTargets[ColorTargetIndex++] = FRenderTargetBinding(SceneTextures.GBufferA, ERenderTargetLoadAction::ELoad);
		}
		RenderTargets[ColorTargetIndex++] = FRenderTargetBinding(SceneTextures.GBufferB, ERenderTargetLoadAction::ELoad);
		RenderTargets[ColorTargetIndex++] = FRenderTargetBinding(SceneTextures.GBufferC, ERenderTargetLoadAction::ELoad);
		RenderTargets.DepthStencil = FDepthStencilBinding(
			SceneTextures.Depth.Target,
			ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad,
			FExclusiveDepthStencil::DepthRead_StencilWrite);

		EMobileSceneTextureSetupMode TextureReadAccess = EMobileSceneTextureSetupMode::SceneDepth;
		if (DecalRenderTargetMode == EDecalRenderTargetMode::SceneColorAndGBufferNoNormal)
		{
			TextureReadAccess |= EMobileSceneTextureSetupMode::GBufferA;
		}
		PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, EMobileBasePass::Opaque, TextureReadAccess);
		PassParameters->RenderTargets = RenderTargets;

		if (Scene->GPUScene.IsEnabled())
		{
			BuildMeshRenderingCommands(GraphBuilder, MeshDecalsPass, View, Scene->GPUScene, InstanceCullingManager, PassParameters->InstanceCullingDrawParams);
		}

		FDecalViewInfo DecalViewInfo = FDecalViewInfo(DecalViewOffset, 
			ViewIndex,
			DecalRenderStage,
			DecalRenderTargetMode,
			RelevantDeferredDecals);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Decals_%s", DecalRendering::GetRenderTargetModeName(DecalRenderTargetMode)),
			PassParameters,
			ERDGPassFlags::Raster,
			[this, DecalViewOffset, DecalViewInfo, PassParameters, &View](FRHICommandList& RHICmdList)
			{
				RenderDecals(RHICmdList, View, DecalViewInfo, &PassParameters->InstanceCullingDrawParams);
			});
	}
}

void FMobileSceneRenderer::RenderDeferredMultiPass(FRDGBuilder& GraphBuilder, FSceneTextures& SceneTextures, const FSortedLightSetSceneInfo& SortedLightSet, FMobileBasePassTextures& BasePassTextures, FInstanceCullingManager& InstanceCullingManager)
{
	FColorTargets ColorTargets = GetColorTargets_Deferred(SceneTextures);
	FRenderTargetBindingSlots BasePassRenderTargets = InitRenderTargetBindings_Deferred(SceneTextures, ColorTargets);
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	FRenderViewContextArray RenderViews;
	GetRenderViews(Views, GetDecalViewOffset(Views), RenderViews);

	bool bDoOcclusionQueries = (!bIsFullDepthPrepassEnabled && DoOcclusionQueries());
	FViewOcclusionQueriesPerView QueriesPerView;
	if (bDoOcclusionQueries)
	{
		AllocateOcclusionTests(Scene, QueriesPerView, Views, VisibleLightInfos);
		bDoOcclusionQueries = QueriesPerView.Num() > 0;
	}

	for (FRenderViewContext& ViewContext : RenderViews)
	{
		FViewInfo& View = *ViewContext.ViewInfo;

		SCOPED_GPU_MASK(GraphBuilder.RHICmdList, !View.IsInstancedStereo() ? View.GPUMask : (View.GPUMask | View.GetInstancedView()->GPUMask));
		SCOPED_CONDITIONAL_DRAW_EVENTF(GraphBuilder.RHICmdList, EventView, RenderViews.Num() > 1, TEXT("View%d"), ViewContext.ViewIndex);
		
		View.BeginRenderView();

		struct FDeferredMultiPassParameterCollection
		{
			FMobileRenderPassParameters PassParameters;
			FInstanceCullingDrawParams DepthPassInstanceCullingDrawParams;
			FInstanceCullingDrawParams SkyPassInstanceCullingDrawParams;
			FInstanceCullingDrawParams MeshDecalSceneColorAndGBufferInstanceCullingDrawParams;
		};
		auto ParameterCollection = GraphBuilder.AllocParameters<FDeferredMultiPassParameterCollection>();

		auto PassParameters = &ParameterCollection->PassParameters;
		PassParameters->View = View.GetShaderParameters();
		EMobileSceneTextureSetupMode SetupMode = bIsFullDepthPrepassEnabled ? EMobileSceneTextureSetupMode::SceneDepth : EMobileSceneTextureSetupMode::None;
		PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, View, EMobileBasePass::Opaque, SetupMode, BasePassTextures, false, true);
		PassParameters->RenderTargets = BasePassRenderTargets;
		if (!ViewContext.bIsFirstView)
		{
			// Load targets for a non-first view 
			for (int32 i = 0; i < ColorTargets.Num(); ++i)
			{
				PassParameters->RenderTargets[i].SetLoadAction(ERenderTargetLoadAction::ELoad);
			}
			PassParameters->RenderTargets.DepthStencil.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);
			PassParameters->RenderTargets.DepthStencil.SetStencilLoadAction(ERenderTargetLoadAction::ELoad);
			PassParameters->RenderTargets.DepthStencil.SetDepthStencilAccess(bIsFullDepthPrepassEnabled ? FExclusiveDepthStencil::DepthRead_StencilWrite : FExclusiveDepthStencil::DepthWrite_StencilWrite);
		}

		PassParameters->RenderTargets.NumOcclusionQueries = bDoOcclusionQueries ? ComputeNumOcclusionQueriesToBatch() : 0u;

		if (Scene->GPUScene.IsEnabled())
		{
			if (!bIsFullDepthPrepassEnabled)
			{
				BuildMeshRenderingCommands(GraphBuilder, EMeshPass::DepthPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->DepthPassInstanceCullingDrawParams);
			}
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::BasePass, View, Scene->GPUScene, InstanceCullingManager, PassParameters->InstanceCullingDrawParams);
			BuildMeshRenderingCommands(GraphBuilder, EMeshPass::SkyPass, View, Scene->GPUScene, InstanceCullingManager, ParameterCollection->SkyPassInstanceCullingDrawParams);

		}

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("BasePass"),
			PassParameters,
			ERDGPassFlags::Raster,
			[this, 
			ParameterCollection, 
			ViewContext, 
			&SceneTextures, 
			bDoOcclusionQueries, 
			ViewQueries = (bDoOcclusionQueries) ? MoveTemp(QueriesPerView[ViewContext.ViewIndex]) : FViewOcclusionQueries()]
			(FRHICommandList& RHICmdList)
			{
				auto PassParameters = &ParameterCollection->PassParameters;
				FViewInfo& View = *ViewContext.ViewInfo;

				// Depth pre-pass
				RenderMaskedPrePass(RHICmdList, View, &ParameterCollection->DepthPassInstanceCullingDrawParams);
				// Opaque and masked
				RenderMobileBasePass(RHICmdList, View, &PassParameters->InstanceCullingDrawParams, &ParameterCollection->SkyPassInstanceCullingDrawParams);

				PostRenderBasePass(RHICmdList, View);

				if (bDoOcclusionQueries)
				{
					// Issue occlusion queries
					RenderOcclusion(RHICmdList, View, ViewQueries);
				}

			});

		RenderMobileDebugViewMode(GraphBuilder, View, SceneTextures, InstanceCullingManager, *PassParameters);
	}

	SceneTextures.MobileSetupMode = EMobileSceneTextureSetupMode::SceneDepth | EMobileSceneTextureSetupMode::GBuffers | EMobileSceneTextureSetupMode::SceneVelocity;
	SceneTextures.MobileUniformBuffer = CreateMobileSceneTextureUniformBuffer(GraphBuilder, &SceneTextures, SceneTextures.MobileSetupMode);

	if (bShouldRenderHZB && !bIsFullDepthPrepassEnabled)
	{
		RenderHZB(GraphBuilder, SceneTextures.Depth.Target);
	}

	FRendererModule& RendererModule = static_cast<FRendererModule&>(GetRendererModule());
	RendererModule.RenderPostOpaqueExtensions(GraphBuilder, Views, SceneTextures);

	RenderOpaqueFX(GraphBuilder, GetSceneViews(), GetSceneUniforms(), FXSystem, SceneTextures.MobileUniformBuffer);

	if (bShouldRenderVolumetricFog && !bIsFullDepthPrepassEnabled)
	{
		ComputeVolumetricFog(GraphBuilder, SceneTextures);
	}

	BasePassRenderTargets.Enumerate([](FRenderTargetBinding& RenderTarget) {
		RenderTarget.SetLoadAction(ERenderTargetLoadAction::ELoad);
		});
	BasePassRenderTargets.DepthStencil.SetDepthLoadAction(ERenderTargetLoadAction::ELoad);
	BasePassRenderTargets.DepthStencil.SetStencilLoadAction(ERenderTargetLoadAction::ELoad);
	BasePassRenderTargets.DepthStencil.SetDepthStencilAccess(FExclusiveDepthStencil::DepthRead_StencilWrite);

	// Decals
	for (FRenderViewContext& ViewContext : RenderViews)
	{
		FViewInfo& View = *ViewContext.ViewInfo;
	
		
		SCOPED_GPU_MASK(GraphBuilder.RHICmdList, !View.IsInstancedStereo() ? View.GPUMask : (View.GPUMask | View.GetInstancedView()->GPUMask));
		SCOPED_CONDITIONAL_DRAW_EVENTF(GraphBuilder.RHICmdList, EventView, RenderViews.Num() > 1, TEXT("View%d"), ViewContext.ViewIndex);

		RenderDeferredMultiPassDecals(GraphBuilder, View, ViewContext.DecalViewOffset, ViewContext.ViewIndex, SceneTextures, InstanceCullingManager);
	}

	if (!HasBeenProduced(BasePassTextures.AmbientOcclusionTexture))
	{
		BasePassTextures.AmbientOcclusionTexture = RenderAmbientOcclusion(GraphBuilder, SceneTextures);
	}

	TArray<FRDGTextureRef> DynamicBentNormalAOTextures;
	if (bEnableDistanceFieldAO)
	{
		FDistanceFieldAOParameters Parameters(Scene->SkyLight->OcclusionMaxDistance, Scene->SkyLight->Contrast);
		RenderDistanceFieldLighting(GraphBuilder, SceneTextures, Parameters, DynamicBentNormalAOTextures, false, false);
	}

	if (bRequiresShadowProjections)
	{
		if (!bIsFullDepthPrepassEnabled)
		{
			FViewInfo* MainView = Views.Num() > 0 ? &Views[0] : nullptr;
			const bool bIsMobileMultiView = SceneTextures.Config.bRequireMultiView || (MainView && MainView->Aspects.IsMobileMultiViewEnabled());

			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderMobileShadowProjections);
			RDG_EVENT_SCOPE_STAT(GraphBuilder, ShadowProjection, "ShadowProjection");

			BasePassTextures.ScreenSpaceShadowMaskTexture = RenderMobileShadowProjections(GraphBuilder, SceneTextures.Config.Extent, bIsMobileMultiView);
		}
		RenderMobileScreenSpaceShadows(GraphBuilder, SceneTextures.Config.Extent, BasePassTextures.ScreenSpaceShadowMaskTexture);
	}

	const bool bShouldRenderSingleLayerWater = ShouldRenderSingleLayerWater(Views);
	const bool bNeedsDepthCopy = bShouldRenderSingleLayerWater;
	FRDGTextureRef SceneDepthCopyTexture = nullptr;
	if (bNeedsDepthCopy)
	{
		AddResolveSceneDepthPass(GraphBuilder, Views, SceneTextures.Depth);

		SceneDepthCopyTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(SceneTextures.Config.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource), TEXT("SceneDepthCopy"));

		for (FRenderViewContext& ViewContext : RenderViews)
		{
			FViewInfo& View = *ViewContext.ViewInfo;
			DepthCopy::AddViewDepthCopyPSPass(GraphBuilder, View, SceneTextures.Depth.Resolve, SceneDepthCopyTexture);
		}
	}

	// Lighting and translucency
	uint32 ViewIndex = 0;
	for (FRenderViewContext& ViewContext : RenderViews)
	{
		FViewInfo& View = *ViewContext.ViewInfo;
		const uint32 CurrentViewIndex = ViewIndex++;
		SCOPED_GPU_MASK(GraphBuilder.RHICmdList, !View.IsInstancedStereo() ? View.GPUMask : (View.GPUMask | View.GetInstancedView()->GPUMask));
		SCOPED_CONDITIONAL_DRAW_EVENTF(GraphBuilder.RHICmdList, EventView, RenderViews.Num() > 1, TEXT("View%d"), ViewContext.ViewIndex);

		View.BeginRenderView();
		UpdateDirectionalLightUniformBuffers(GraphBuilder, View);

		FRDGTextureRef DynamicBentNormalAOTexture = DynamicBentNormalAOTextures.IsEmpty() ? nullptr : DynamicBentNormalAOTextures[CurrentViewIndex];;
		auto* PassParameters = GraphBuilder.AllocParameters<FMobileRenderPassParameters>();
		PassParameters->View = View.GetShaderParameters();

		EMobileSceneTextureSetupMode SetupMode = 
			EMobileSceneTextureSetupMode::SceneDepth | 
			EMobileSceneTextureSetupMode::CustomDepth |
			EMobileSceneTextureSetupMode::SceneVelocity |
			EMobileSceneTextureSetupMode::GBuffers;

		FMobileBasePassUniformParameters* BasePassParameters = GraphBuilder.AllocParameters<FMobileBasePassUniformParameters>();
		SetupMobileBasePassUniformParameters(GraphBuilder, View, EMobileBasePass::Translucent, SetupMode, *BasePassParameters, BasePassTextures);
		PassParameters->MobileBasePass = GraphBuilder.CreateUniformBuffer(BasePassParameters);

		PassParameters->ReflectionCapture = View.MobileReflectionCaptureUniformBuffer;
		PassParameters->LocalFogVolumeInstances = View.LocalFogVolumeViewData.GPUInstanceDataBufferSRV;
		PassParameters->LocalFogVolumeTileDrawIndirectBuffer = View.LocalFogVolumeViewData.GPUTileDrawIndirectBuffer;
		PassParameters->LocalFogVolumeTileDataTexture = View.LocalFogVolumeViewData.TileDataTextureArraySRV;
		PassParameters->LocalFogVolumeTileDataBuffer = View.LocalFogVolumeViewData.GPUTileDataBufferSRV;
		PassParameters->HalfResLocalFogVolumeViewSRV = View.LocalFogVolumeViewData.HalfResLocalFogVolumeViewSRV;
		PassParameters->HalfResLocalFogVolumeDepthSRV = View.LocalFogVolumeViewData.HalfResLocalFogVolumeDepthSRV;
		PassParameters->BentNormalAOTexture = DynamicBentNormalAOTexture;
		// Only SceneColor and Depth
		PassParameters->RenderTargets[0] = BasePassRenderTargets[0];
		PassParameters->RenderTargets.DepthStencil = BasePassRenderTargets.DepthStencil;

		if (Scene->GPUScene.IsEnabled())
		{
			BuildMeshRenderingCommands(GraphBuilder, StandardTranslucencyMeshPass, View, Scene->GPUScene, InstanceCullingManager, PassParameters->InstanceCullingDrawParams);
		}

		const EMobileSSRQuality MobileSSRQuality = ActiveMobileSSRQuality(View, bShouldRenderVelocities);
		const bool bRenderDeferredLighting = ViewFamily.EngineShowFlags.Lighting && ViewFamily.EngineShowFlags.DeferredLighting;

		// Determine if a shadow point light is present.
		TArray<int32, TInlineAllocator<8>> ShadowPointLightIndexes;
		GatherShadowPointLights(View, SortedLightSet, ShadowPointLightIndexes);
		const bool bSeparateShadowPointLights = (ShadowPointLightIndexes.Num() > 0);

		{
			RDG_EVENT_SCOPE(GraphBuilder, "LightingAndFog", ViewIndex);

			if (bRenderDeferredLighting)
			{
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("Lighting"),
					PassParameters,
					ERDGPassFlags::Raster,
					[this, PassParameters, ViewContext, MobileSSRQuality, &SortedLightSet](FRHICommandList& RHICmdList)
					{
						FViewInfo& View = *ViewContext.ViewInfo;
						MobileDeferredShadingPass(RHICmdList, ViewContext.ViewIndex, Views.Num(), View, *Scene, SortedLightSet, VisibleLightInfos, MobileSSRQuality, PassParameters->BentNormalAOTexture);
					});


				if (bSeparateShadowPointLights)
				{
					for (int32 Index = 0; Index < ShadowPointLightIndexes.Num(); ++Index)
					{
						const FSortedLightSceneInfo& SortedLight = SortedLightSet.SortedLights[ShadowPointLightIndexes[Index]];
						const FLightSceneInfo& LightSceneInfo = *SortedLight.LightSceneInfo;

						RDG_EVENT_SCOPE(GraphBuilder, "PointLightShadow_%s", LightSceneInfo.Proxy->GetOwnerNameOrLabel());
						{
							ClearMobileScreenSpaceShadows(GraphBuilder, View, LightSceneInfo.Proxy, BasePassTextures.ScreenSpaceShadowMaskTexture);
							RenderShadowProjections(GraphBuilder, SceneTextures, BasePassTextures.ScreenSpaceShadowMaskTexture, nullptr, &LightSceneInfo, false);

							GraphBuilder.AddPass(
								RDG_EVENT_NAME("PointLightShadowPass"),
								PassParameters,
								ERDGPassFlags::Raster,
								[this, &View, LightSceneInfo](FRHICommandList& RHICmdList)
								{
									MobileDeferredRenderLocalLight(RHICmdList, *Scene, View, LightSceneInfo, VisibleLightInfos);
								});
						}
					}
				}
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("Fog"),
				PassParameters,
				ERDGPassFlags::Raster,
				[this, &View](FRHICommandList& RHICmdList)
				{
					RenderFog(RHICmdList, View);
				});
		}

		if (IsMobileDistortionBeforeTranslucency())
		{
			RenderDistortion(GraphBuilder, View, SceneTextures.Color.Target, SceneTextures.Depth.Target);
		}
			
		const bool bRenderTranslucency = ShouldRenderTranslucency(StandardTranslucencyPass, Views) && ViewFamily.EngineShowFlags.Translucency && HasAnyDraw(View.ParallelMeshDrawCommandPasses[StandardTranslucencyMeshPass]); 
		const bool bRenderSingleLayerWater = bShouldRenderSingleLayerWater && View.bHasSingleLayerWaterMaterial && HasAnyDraw(View.ParallelMeshDrawCommandPasses[EMeshPass::SingleLayerWaterPass]);
		ETranslucencyView TranslucencyViewsToRender = bRenderTranslucency ? GetTranslucencyViews(Views) : ETranslucencyView::None;
		const bool bIsCameraUnderWater = EnumHasAnyFlags(TranslucencyViewsToRender, ETranslucencyView::UnderWater);
		
		// - Above water : render water then translucencies
		// - Under water : render translucencies then water
		auto AddTranslucencyPass = [&]()
		{
			check(bRenderTranslucency);

			// No need to copy SceneColor before translucencies when underwater
			if (View.TranslucentPrimCount.UseSceneColorCopy(StandardTranslucencyPass) && (!bRenderSingleLayerWater || !bIsCameraUnderWater))
			{
				FRDGTextureRef SceneColorCopyTexture = AddCopySceneColorPass(GraphBuilder, Views, SceneTextures.Color, false, false, true);
				if (SceneColorCopyTexture != nullptr)
				{
					BasePassParameters->SceneColorCopyTexture = SceneColorCopyTexture;
				}
				if (SceneDepthCopyTexture != nullptr)
				{
					BasePassParameters->SceneDepthCopyTexture = SceneDepthCopyTexture;
				}

				PassParameters->MobileBasePass = GraphBuilder.CreateUniformBuffer(BasePassParameters);
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("Translucency"),
				PassParameters,
				ERDGPassFlags::Raster,
				[this, &View, PassParameters](FRHICommandList& RHICmdList)
				{
					RenderTranslucency(RHICmdList, View, Views, StandardTranslucencyPass, StandardTranslucencyMeshPass, &PassParameters->InstanceCullingDrawParams);
				});
		};

		if (bIsCameraUnderWater && bRenderTranslucency)
		{
			AddTranslucencyPass();
		}
			
		if (bRenderSingleLayerWater)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "SingleLayerWater");

			auto* SLWPassParameters = GraphBuilder.AllocParameters<FMobileRenderPassParameters>();
			*SLWPassParameters = *PassParameters;
			SLWPassParameters->RenderTargets[0] = BasePassRenderTargets[0];
			SLWPassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(BasePassRenderTargets.DepthStencil.GetTexture(), ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);

			FMobileBasePassUniformParameters* SLWBasePassParameters = GraphBuilder.AllocParameters<FMobileBasePassUniformParameters>();
			SetupMobileBasePassUniformParameters(GraphBuilder, View, EMobileBasePass::Translucent, SetupMode, *SLWBasePassParameters, BasePassTextures);
			
			FRDGTextureRef SceneColorCopyTexture = AddCopySceneColorPass(GraphBuilder, Views, SceneTextures.Color, false, false, false);
			if (SceneColorCopyTexture != nullptr)
			{
				SLWBasePassParameters->SceneColorCopyTexture = SceneColorCopyTexture;
			}
			if (SceneDepthCopyTexture != nullptr)
			{
				SLWBasePassParameters->SceneDepthCopyTexture = SceneDepthCopyTexture;
			}
			SLWPassParameters->MobileBasePass = GraphBuilder.CreateUniformBuffer(SLWBasePassParameters);

			if (Scene->GPUScene.IsEnabled())
			{
				BuildMeshRenderingCommands(GraphBuilder, EMeshPass::SingleLayerWaterPass, View, Scene->GPUScene, InstanceCullingManager, SLWPassParameters->InstanceCullingDrawParams);
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("SingleLayerWater"),
				SLWPassParameters,
				ERDGPassFlags::Raster,
				[this, &View, SLWPassParameters](FRHICommandList& RHICmdList)
				{
					RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

					if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::SingleLayerWaterPass])
					{
						Pass->Draw(RHICmdList, &SLWPassParameters->InstanceCullingDrawParams);
					}
				});
		}

		if (!bIsCameraUnderWater && bRenderTranslucency)
		{
			AddTranslucencyPass();
		}
	}
}

void FMobileSceneRenderer::RenderMobileDebugViewMode(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FSceneTextures& SceneTextures,
	FInstanceCullingManager& InstanceCullingManager,
	const FMobileRenderPassParameters& BasePassParameters)
{
#if WITH_DEBUG_VIEW_MODES
	if (!ViewFamily.UseDebugViewPS())
	{
		return;
	}

	auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::DebugViewMode];
	if (!Pass)
	{
		return;
	}

	FRDGTextureRef SceneColorTexture = BasePassParameters.RenderTargets[0].GetTexture();
	if (!SceneColorTexture)
	{
		return;
	}
	FRDGTextureRef SceneColorResolveTexture = BasePassParameters.RenderTargets[0].GetResolveTexture();

	auto* PassParameters = GraphBuilder.AllocParameters<FMobileRenderPassParameters>();
	PassParameters->View = BasePassParameters.View;
	PassParameters->MobileBasePass = BasePassParameters.MobileBasePass;
	PassParameters->ReflectionCapture = BasePassParameters.ReflectionCapture;
	PassParameters->LocalFogVolumeInstances = BasePassParameters.LocalFogVolumeInstances;
	PassParameters->LocalFogVolumeTileDrawIndirectBuffer = BasePassParameters.LocalFogVolumeTileDrawIndirectBuffer;
	PassParameters->LocalFogVolumeTileDataTexture = BasePassParameters.LocalFogVolumeTileDataTexture;
	PassParameters->LocalFogVolumeTileDataBuffer = BasePassParameters.LocalFogVolumeTileDataBuffer;
	PassParameters->HalfResLocalFogVolumeViewSRV = BasePassParameters.HalfResLocalFogVolumeViewSRV;
	PassParameters->HalfResLocalFogVolumeDepthSRV = BasePassParameters.HalfResLocalFogVolumeDepthSRV;
	// Shader-complexity output is additive over black; folding the clear into the load action avoids
	// an extra render-pass scope on tile GPUs.
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, SceneColorResolveTexture, ERenderTargetLoadAction::EClear);
	{
		FRDGTextureDesc DummyDesc = SceneColorTexture->Desc;
		DummyDesc.Format = PF_R8_UINT;
		DummyDesc.Flags = TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV;
		DummyDesc.ClearValue = FClearValueBinding::Black;
		FRDGTextureRef DummyDebugAux = GraphBuilder.CreateTexture(DummyDesc, TEXT("DummyDebugAux"));
		PassParameters->RenderTargets[1] = FRenderTargetBinding(DummyDebugAux, ERenderTargetLoadAction::ENoAction);
	}

	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
		SceneTextures.Depth.Target,
		ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad,
		FExclusiveDepthStencil::DepthRead_StencilRead);
	PassParameters->RenderTargets.MultiViewCount = BasePassParameters.RenderTargets.MultiViewCount;

	if (Scene->GPUScene.IsEnabled())
	{
		BuildMeshRenderingCommands(
			GraphBuilder, EMeshPass::DebugViewMode, View, Scene->GPUScene,
			InstanceCullingManager, PassParameters->InstanceCullingDrawParams);
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("MobileDebugViewMode"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, Pass, PassParameters](FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1);
			Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
		});
#endif
}

void FMobileSceneRenderer::PostRenderBasePass(FRHICommandList& RHICmdList, FViewInfo& View)
{
	if (ViewFamily.ViewExtensions.Num() > 1)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(ViewExtensionPostRenderBasePass);
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FMobileSceneRenderer_ViewExtensionPostRenderBasePass);
		for (int32 ViewExt = 0; ViewExt < ViewFamily.ViewExtensions.Num(); ++ViewExt)
		{
			ViewFamily.ViewExtensions[ViewExt]->PostRenderBasePassMobile_RenderThread(RHICmdList, View);
		}
	}
}

int32 FMobileSceneRenderer::ComputeNumOcclusionQueriesToBatch() const
{
	int32 NumQueriesForBatch = 0;
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		const FSceneViewState* ViewState = (FSceneViewState*)View.State;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (!ViewState || (!ViewState->bIsFrozen))
#endif
		{
			NumQueriesForBatch += View.IndividualOcclusionQueries.GetNumBatchOcclusionQueries();
			NumQueriesForBatch += View.GroupedOcclusionQueries.GetNumBatchOcclusionQueries();
		}
	}
	
	return NumQueriesForBatch;
}

bool FMobileSceneRenderer::ShouldRenderCustomDepth(const FViewInfo& View)
{
	return GetCustomDepthStencilUsage(View).bUsesCustomDepthStencil;
}

// Whether we need a separate render-passes for translucency, decals etc
bool FMobileSceneRenderer::RequiresMultiPass(int32 NumMSAASamples, EShaderPlatform ShaderPlatform)
{
	if (!MobileAllowFramebufferFetch(ShaderPlatform))
	{
		return true;
	}
	
	// Vulkan uses subpasses
	if (IsVulkanPlatform(ShaderPlatform))
	{
		return false;
	}

	// All iOS support frame_buffer_fetch
	if (IsMetalMobilePlatform(ShaderPlatform) && GSupportsShaderFramebufferFetch)
	{
		return false;
	}

	// Some Androids support frame_buffer_fetch
	if (IsAndroidOpenGLESPlatform(ShaderPlatform) && (GSupportsShaderFramebufferFetch || GSupportsShaderDepthStencilFetch))
	{
		return false;
	}

	// Only Vulkan, iOS and some GL can do a single pass deferred shading, otherwise multipass
	if (IsMobileDeferredShadingEnabled(ShaderPlatform))
	{
		return true;
	}
		
	// Always render LDR in single pass
	if (!IsMobileHDR() && !IsSimulatedPlatform(ShaderPlatform))
	{
		return false;
	}

	// MSAA depth can't be sampled or resolved, unless we are on PC (no vulkan)
	if (NumMSAASamples > 1 && !IsSimulatedPlatform(ShaderPlatform))
	{
		return false;
	}

	return true;
}

void FMobileSceneRenderer::UpdateDirectionalLightUniformBuffers(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	if (CachedView == &View)
	{
		return;
	}
	CachedView = &View;

	AddPass(GraphBuilder, RDG_EVENT_NAME("UpdateDirectionalLightUniformBuffers"), [this, &View](FRHICommandListImmediate& RHICmdList)
	{
		const bool bRenderDirectionalLights = ViewFamily.EngineShowFlags.Lighting && ViewFamily.EngineShowFlags.DirectLighting && ViewFamily.EngineShowFlags.DirectionalLights;
		const bool bDynamicShadows = ViewFamily.EngineShowFlags.DynamicShadows;
		// Fill in the other entries based on the lights
		for (int32 ChannelIdx = 0; ChannelIdx < UE_ARRAY_COUNT(Scene->MobileDirectionalLights); ChannelIdx++)
		{
			FMobileDirectionalLightShaderParameters Params;
			if (bRenderDirectionalLights)
			{
				SetupMobileDirectionalLightUniformParameters(*Scene, View, VisibleLightInfos, ChannelIdx, bDynamicShadows, Params);
			}
			Scene->UniformBuffers.MobileDirectionalLightUniformBuffers[ChannelIdx + 1].UpdateUniformBufferImmediate(RHICmdList, Params);
		}
	});
}

void FMobileSceneRenderer::UpdateSkyReflectionUniformBuffer(FRHICommandListBase& RHICmdList)
{
	FSkyLightSceneProxy* SkyLight = nullptr;
	if (Scene->SkyLight
		&& 
		(
			(
				Scene->SkyLight->ProcessedTexture && Scene->SkyLight->ProcessedTexture->TextureRHI
				// Don't use skylight reflection if it is a static sky light for keeping coherence with PC.
				&& !Scene->SkyLight->bHasStaticLighting
			)
			||
			Scene->CanSampleSkyLightRealTimeCaptureData()
		))
	{
		SkyLight = Scene->SkyLight;
	}

    // Make sure we don't try to use the skylight when doing a scene capture since it might contain uninitialized data
	if (ViewFamily.EngineShowFlags.SkyLighting == 0 && Views.Num() > 0 && Views[0].bIsReflectionCapture)
	{
		SkyLight = nullptr;
	}

	FMobileReflectionCaptureShaderParameters Parameters;
	SetupMobileSkyReflectionUniformParameters(Scene, SkyLight, Parameters);
	Scene->UniformBuffers.MobileSkyReflectionUniformBuffer.UpdateUniformBufferImmediate(RHICmdList, Parameters);
}

class FPreTonemapMSAA_Mobile : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FPreTonemapMSAA_Mobile, Global);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMetalMobilePlatform(Parameters.Platform);
	}	

	FPreTonemapMSAA_Mobile() {}

public:
	FPreTonemapMSAA_Mobile(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}
};

IMPLEMENT_SHADER_TYPE(, FPreTonemapMSAA_Mobile,TEXT("/Engine/Private/PostProcessMobile.usf"),TEXT("PreTonemapMSAA_Mobile"),SF_Pixel);

void FMobileSceneRenderer::PreTonemapMSAA(FRHICommandList& RHICmdList, const FMinimalSceneTextures& SceneTextures)
{
	// iOS only
	bool bOnChipPP = GSupportsRenderTargetFormat_PF_FloatRGBA && GSupportsShaderFramebufferFetch &&	ViewFamily.EngineShowFlags.PostProcessing;
	bool bOnChipPreTonemapMSAA = bOnChipPP && IsMetalMobilePlatform(ViewFamily.GetShaderPlatform()) && (NumMSAASamples > 1);
	if (!bOnChipPreTonemapMSAA || bGammaSpace)
	{
		return;
	}

	const FIntPoint TargetSize = SceneTextures.Config.Extent;

	const auto ShaderMap = GetGlobalShaderMap(FeatureLevel);
	TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
	TShaderMapRef<FPreTonemapMSAA_Mobile> PixelShader(ShaderMap);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero, CW_NONE>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	RHICmdList.SetViewport(0, 0, 0.0f, TargetSize.X, TargetSize.Y, 1.0f);

	DrawRectangle(
		RHICmdList,
		0, 0,
		TargetSize.X, TargetSize.Y,
		0, 0,
		TargetSize.X, TargetSize.Y,
		TargetSize,
		TargetSize,
		VertexShader,
		EDRF_UseTriangleOptimization);
}

bool FMobileSceneRenderer::ShouldRenderHZB(TArrayView<FViewInfo> InViews)
{
	// Mobile SSAO requests HZB
	bool bIsFeatureRequested =  RequiresMobileAmbientOcclusionPass(Scene, InViews[0]);

	// Instance occlusion culling requires HZB
	if (FInstanceCullingContext::IsOcclusionCullingEnabled())
	{
		bIsFeatureRequested = true;
	}

	bool bNeedsHZB = bIsFeatureRequested;

	if (!bNeedsHZB)
	{
		for (const FViewInfo& View : InViews)
		{
			if (IsMobileSSREnabled(View))
			{
				bNeedsHZB = true;
				break;
			}
		}
	}

	return bNeedsHZB;
}

void FMobileSceneRenderer::RenderHZB(FRHICommandListImmediate& RHICmdList, const TRefCountPtr<IPooledRenderTarget>& SceneDepthZ)
{
	checkSlow(bShouldRenderHZB);

	FRDGBuilder GraphBuilder(RHICmdList);
	{
		FRDGTextureRef SceneDepthTexture = GraphBuilder.RegisterExternalTexture(SceneDepthZ, TEXT("SceneDepthTexture"));

		RenderHZB(GraphBuilder, SceneDepthTexture);
	}
	GraphBuilder.Execute();
}

void FMobileSceneRenderer::RenderHZB(FRDGBuilder& GraphBuilder, FRDGTextureRef SceneDepthTexture)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, HZB, "HZB");

	if (!HasBeenProduced(SceneDepthTexture))
	{
		return;
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];
		if (View.ShouldRenderView())
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
			{
				RDG_EVENT_SCOPE(GraphBuilder, "BuildHZB(ViewId=%d)", ViewIndex);

				FRDGTextureRef FurthestHZBTexture = nullptr;

				BuildHZBFurthest(
					GraphBuilder,
					SceneDepthTexture,
					/* VisBufferTexture = */ nullptr,
					GetHZBViewRect(View),
					View.GetFeatureLevel(),
					View.GetShaderPlatform(),
					TEXT("MobileHZBFurthest"),
					&FurthestHZBTexture);

				View.HZBMipmap0Size = FurthestHZBTexture->Desc.Extent;
				View.HZB = FurthestHZBTexture;

				if (View.ViewState)
				{
					if (FInstanceCullingContext::IsOcclusionCullingEnabled() || (AreMobileScreenSpaceReflectionsEnabled(ShaderPlatform) && !bIsFullDepthPrepassEnabled))
					{
						GraphBuilder.QueueTextureExtraction(FurthestHZBTexture, &View.ViewState->PrevFrameViewInfo.HZB);
					}
					else
					{
						View.ViewState->PrevFrameViewInfo.HZB = nullptr;
					}
				}
			}

			if (Scene->InstanceCullingOcclusionQueryRenderer && View.ViewState)
			{
				// Render per-instance occlusion queries and save the mask to interpret results on the next frame
				const uint32 OcclusionQueryMaskForThisView = Scene->InstanceCullingOcclusionQueryRenderer->Render(GraphBuilder, Scene->GPUScene, View);
				View.ViewState->PrevFrameViewInfo.InstanceOcclusionQueryMask = OcclusionQueryMaskForThisView;
			}
		}
	}
}

bool FMobileSceneRenderer::AllowSimpleLights() const
{
	return FSceneRenderer::AllowSimpleLights() && bSupportsSimpleLights;
}
