// Copyright Epic Games, Inc. All Rights Reserved.

#include "MobileBasePassRendering.h"
#include "TranslucentRendering.h"
#include "RenderCore.h"
#include "SceneRendering.h"
#include "LightMapRendering.h"
#include "MeshPassProcessor.inl"
#include "RenderGraphBuilder.h"

extern bool MobileLocalLightsUseSinglePermutation(EShaderPlatform ShaderPlatform);

static FMeshDrawCommandSortKey GetMobileSingleLayerWaterSortKey(bool bIsMasked, bool bIsBackground, const FMeshMaterialShader* VertexShader, const FMeshMaterialShader* PixelShader)
{
	FMeshDrawCommandSortKey SortKey;
	SortKey.BasePass.Masked = (bIsMasked ? 1 : 0);
	SortKey.BasePass.Background = (bIsBackground ? 1 : 0); // background flag in second bit
	SortKey.BasePass.VertexShaderHash = (VertexShader ? VertexShader->GetSortKey() : 0) & 0xFFFF;
	SortKey.BasePass.PixelShaderHash = PixelShader ? PixelShader->GetSortKey() : 0;
	return SortKey;
};

static void SetMobileSingleLayerWaterRenderState(FMeshPassProcessorRenderState& DrawRenderState, const FMaterial& Material)
{
	constexpr ERHIFeatureLevel::Type FeatureLevel = ERHIFeatureLevel::ES3_1;
	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);

	DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilNop);
	DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA, CW_RGBA, CW_RGBA, CW_RGBA>::GetRHI());
	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
}

class FMobileSingleLayerWaterPassMeshProcessor : public FSceneRenderingAllocatorObject<FMobileSingleLayerWaterPassMeshProcessor>, public FMeshPassProcessor
{
public:
	FMobileSingleLayerWaterPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		bool bIsMasked,
		FMaterialShadingModelField ShadingModels);

	void CollectPSOInitializersForLMPolicy(
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FMeshPassProcessorRenderState& RESTRICT DrawRenderState,
		const FGraphicsPipelineRenderTargetsInfo& RESTRICT RenderTargetsInfo,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode,
		EPrimitiveType PrimitiveType,
		TArray<FPSOPrecacheData>& PSOInitializers);

	FMeshPassProcessorRenderState PassDrawRenderState;
};

FMobileSingleLayerWaterPassMeshProcessor::FMobileSingleLayerWaterPassMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type FeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, const FMeshPassProcessorRenderState& InPassDrawRenderState, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::SingleLayerWaterPass, Scene, FeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InPassDrawRenderState)
{
}

void FMobileSingleLayerWaterPassMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	// This MeshProcessor is only used on the mobile SM5 path, otherwise all SLW is handled by the translucent path.
	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	if (IsUsingSingleLayerWaterFallback(ShaderPlatform))
	{
		return;
	}
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, *MaterialRenderProxy, *Material))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

bool FMobileSingleLayerWaterPassMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material)
{
	FMaterialShadingModelField ShadingModels = Material.GetShadingModels();
	if (ShadingModels.HasShadingModel(MSM_SingleLayerWater) && IsOpaqueOrMaskedBlendMode(Material))
	{
		const bool bIsMasked = IsMaskedBlendMode(Material);
		return Process(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material, bIsMasked, ShadingModels);
	}

	return true;
}

bool FMobileSingleLayerWaterPassMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	bool bIsMasked,
	FMaterialShadingModelField ShadingModels)
{
	TMeshProcessorShaders<
		TMobileBasePassVSPolicyParamType<FMobileSingleLayerWaterPolicy>,
		TMobileBasePassPSPolicyParamType<FMobileSingleLayerWaterPolicy>> BasePassShaders;

	if (!MobileBasePass::GetShaders<FMobileSingleLayerWaterPolicy>(
		FMobileSingleLayerWaterPolicy(),
		EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED,
		MaterialResource,
		MeshBatch.VertexFactory->GetType(),
		BasePassShaders.VertexShader,
		BasePassShaders.PixelShader))
	{
		return false;
	}

	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);
	SetMobileSingleLayerWaterRenderState(DrawRenderState, MaterialResource);

	// Background primitives will be rendered last in masked/non-masked buckets
	bool bIsBackground = PrimitiveSceneProxy ? PrimitiveSceneProxy->TreatAsBackgroundForOcclusion() : false;
	// Default static sort key separates masked and non-masked geometry, generic mesh sorting will also sort by PSO
	// if platform wants front to back sorting, this key will be recomputed in InitViews
	FMeshDrawCommandSortKey SortKey = GetMobileSingleLayerWaterSortKey(bIsMasked, bIsBackground, BasePassShaders.VertexShader.GetShader(), BasePassShaders.PixelShader.GetShader());

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(MaterialResource, OverrideSettings);
	ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(MaterialResource, OverrideSettings);

	TMobileBasePassShaderElementData<FMobileSingleLayerWaterPolicy> ShaderElementData(nullptr, true /*bCanReceiveCSM*/);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		DrawRenderState,
		BasePassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);
	return true;
}

void FMobileSingleLayerWaterPassMeshProcessor::CollectPSOInitializersForLMPolicy(
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FMeshPassProcessorRenderState& RESTRICT DrawRenderState,
	const FGraphicsPipelineRenderTargetsInfo& RESTRICT RenderTargetsInfo,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode,
	EPrimitiveType PrimitiveType,
	TArray<FPSOPrecacheData>& PSOInitializers)
{
	TMeshProcessorShaders<
		TMobileBasePassVSPolicyParamType<FMobileSingleLayerWaterPolicy>,
		TMobileBasePassPSPolicyParamType<FMobileSingleLayerWaterPolicy>> BasePassShaders;

	if (!MobileBasePass::GetShaders<FMobileSingleLayerWaterPolicy>(
		FMobileSingleLayerWaterPolicy(),
		EMobileLocalLightSetting::LOCAL_LIGHTS_DISABLED,
		MaterialResource,
		VertexFactoryData.VertexFactoryType,
		BasePassShaders.VertexShader,
		BasePassShaders.PixelShader))
	{
		return;
	}

	// subpass info set during the submission of the draws in mobile deferred renderer.
	uint8 SubpassIndex = 0;
	ESubpassHint SubpassHint = GetSubpassHint(GMaxRHIShaderPlatform, false /*bIsUsingGBuffers*/, RenderTargetsInfo.MultiViewCount > 1, RenderTargetsInfo.NumSamples);

	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		MaterialResource,
		DrawRenderState,
		RenderTargetsInfo,
		BasePassShaders,
		MeshFillMode,
		MeshCullMode,
		PrimitiveType,
		EMeshPassFeatures::Default,
		SubpassHint,
		SubpassIndex,
		true /*bRequired*/,
		PSOCollectorIndex,
		PSOInitializers);
}

void FMobileSingleLayerWaterPassMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	// Fallback SLW rendering is handled by the translucent path.
	EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(FeatureLevel);
	if (IsUsingSingleLayerWaterFallback(ShaderPlatform))
	{
		return;
	}
	const FMaterialShadingModelField ShadingModels = Material.GetShadingModels();
	if (!ShadingModels.HasShadingModel(MSM_SingleLayerWater) || !IsOpaqueOrMaskedBlendMode(Material))
	{
		return;
	}

	// Determine the mesh's material and blend mode.
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(Material, OverrideSettings);
	const EBlendMode BlendMode = Material.GetBlendMode();
	const bool bLitMaterial = ShadingModels.IsLit();

	bool bMovable =
		PreCacheParams.Mobility == EComponentMobility::Movable ||
		PreCacheParams.Mobility == EComponentMobility::Stationary ||
		PreCacheParams.bUsesIndirectLightingCache; // ILC uses movable path

	// Setup the draw state
	FMeshPassProcessorRenderState DrawRenderState(PassDrawRenderState);

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	SceneTexturesConfig.GetGBufferRenderTargetsInfo(RenderTargetsInfo, GBL_Default);
	
	SetupDepthStencilInfo(PF_DepthStencil, SceneTexturesConfig.DepthCreateFlags, ERenderTargetLoadAction::ELoad,
		ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilNop, RenderTargetsInfo);
	
	{
		const static UE::StereoRenderUtils::FStereoShaderAspects Aspects(GMaxRHIShaderPlatform);
		// If mobile multiview is enabled we expect it will be used with a native MMV, no pre-caching for fallbacks 
		RenderTargetsInfo.MultiViewCount = Aspects.IsMobileMultiViewEnabled() ? (GSupportsMobileMultiView ? 2 : 1) : 0;
		// FIXME: Need to figure out if renderer will use shading rate texture or not
		RenderTargetsInfo.bHasFragmentDensityAttachment = GVRSImageManager.IsAttachmentVRSEnabled();
	}

	SetMobileSingleLayerWaterRenderState(DrawRenderState, Material);
	CollectPSOInitializersForLMPolicy(VertexFactoryData, DrawRenderState, RenderTargetsInfo, Material,  MeshFillMode, MeshCullMode, (EPrimitiveType)PreCacheParams.PrimitiveType, PSOInitializers);
}

FMeshPassProcessor* CreateMobileSingleLayerWaterPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState DrawRenderState;
	DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilNop);
	DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA, CW_RGBA, CW_RGBA, CW_RGBA>::GetRHI());
	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());

	return new FMobileSingleLayerWaterPassMeshProcessor(Scene, FeatureLevel, InViewIfDynamicMeshCommand, DrawRenderState, InDrawListContext);
}

REGISTER_MESHPASSPROCESSOR_AND_PSOCOLLECTOR(MobileSingleLayerWater, CreateMobileSingleLayerWaterPassProcessor, EShadingPath::Mobile, EMeshPass::SingleLayerWaterPass, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
