// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneRendering.h"
#include "RenderCore.h"
#include "TranslucentRendering.h"

void FMobileSceneRenderer::RenderTranslucency(FRHICommandList& RHICmdList, const FViewInfo& View, TArrayView<FViewInfo> FamilyViewInfos, ETranslucencyPass::Type InStandardTranslucencyPass, EMeshPass::Type InStandardTranslucencyMeshPass, const FInstanceCullingDrawParams* InTranslucencyInstanceCullingDrawParams)
{
	const FSceneViewFamily& ViewFamily = *View.Family;

	const bool bShouldRenderTranslucency = !ViewFamily.UseDebugViewPS() && ShouldRenderTranslucency(InStandardTranslucencyPass, FamilyViewInfos) && ViewFamily.EngineShowFlags.Translucency;
	if (bShouldRenderTranslucency)
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RenderTranslucency);
		SCOPE_CYCLE_COUNTER(STAT_TranslucencyDrawTime);

		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, Translucency, "Translucency");

		RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

		if (auto* Pass = View.ParallelMeshDrawCommandPasses[InStandardTranslucencyMeshPass])
		{
			Pass->Draw(RHICmdList, InTranslucencyInstanceCullingDrawParams);
		}
	}
}
