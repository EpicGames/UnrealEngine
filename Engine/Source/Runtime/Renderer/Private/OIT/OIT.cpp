// Copyright Epic Games, Inc. All Rights Reserved.

#include "OIT.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "LocalVertexFactory.h"
#include "MeshBatch.h"
#include "OITParameters.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "PrimitiveSceneProxy.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "SceneTextureParameters.h"
#include "SceneRendering.h"
#include "ShaderCompilerCore.h"
#include "ShaderPrintParameters.h"
#include "ShaderPrint.h"
#include "ScreenPass.h"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Variables: sorted triangles

static TAutoConsoleVariable<int32> CVarOIT_SortedTriangles_Enable_Project(
	TEXT("r.OIT.SortedTriangles"), 
	1, 
	TEXT("Enable per-instance triangle sorting to avoid invalid triangle ordering."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOIT_SortedTriangles_Debug(
	TEXT("r.OIT.SortedTriangles.Debug"), 
	0, 
	TEXT("Enable per-instance triangle sorting debug rendering."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOIT_SortedTriangles_Pool(
	TEXT("r.OIT.SortedTriangles.Pool"), 
	0, 
	TEXT("Enable index buffer pool allocation which reduce creation/deletion time by re-use buffers."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOIT_SortedTriangles_PoolReleaseThreshold(
	TEXT("r.OIT.SortedTriangles.Pool.ReleaseFrameThreshold"), 
	100, 
	TEXT("Number of frame after which unused buffer are released."),
	ECVF_RenderThreadSafe);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Variables: sorted pixels

// Referenced in RendererSettings.h, as it is a project settings
static TAutoConsoleVariable<int32> CVarOIT_SortedPixels_Enable_Project(
	TEXT("r.OIT.SortedPixels"),
	0,
	TEXT("Enable OIT rendering (project settings, can't be changed at runtime)"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOIT_SortedPixels_Enable_Runtime(
	TEXT("r.OIT.SortedPixels.Enable"),
	1,
	TEXT("Enable OIT rendering (runtime setting, selects shader permutation)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOIT_SortedPixels_PassType(
	TEXT("r.OIT.SortedPixels.PassType"),
	3,
	TEXT("Enable OIT rendering. 0: disable 1: enable OIT for std. translucency 2: enable OIT for separated translucency 3: enable for both std. and separated translucency (default)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOIT_SortedPixels_SampleCount(
	TEXT("r.OIT.SortedPixels.MaxSampleCount"),
	4,
	TEXT("Max sample count."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOIT_SortedPixels_Debug(
	TEXT("r.OIT.SortedPixels.Debug"),
	0,
	TEXT("Enable debug rendering for OIT. 1: Enable debug for std. translucency 2: Enable for separated translucency."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarOIT_SortedPixels_Method(
	TEXT("r.OIT.SortedPixels.Method"),
	1,
	TEXT("Toggle OIT methods 0: Regular alpha-blending (i.e., no OIT) 1: MLAB"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarOIT_SortedPixels_TransmittanceThreshold(
	TEXT("r.OIT.SortedPixels.TransmittanceThreshold"),
	0.05f,
	TEXT("Remove translucent rendering surfaces when the accumulated thransmittance is below this threshold"),
	ECVF_RenderThreadSafe);

///////////////////////////////////////////////////////////////////////////////////////////////////

static bool IsOITSortedPixelsSupported(EShaderPlatform InShaderPlatform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsROV(InShaderPlatform) && FDataDrivenShaderPlatformInfo::GetSupportsOIT(InShaderPlatform);
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Debug OIT
class FOITPixelDebugCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOITPixelDebugCS);
	SHADER_USE_PARAMETER_STRUCT(FOITPixelDebugCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, CursorCoord)
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER(uint32, MaxSideSampleCount)
		SHADER_PARAMETER(uint32, MaxSampleCount)
		SHADER_PARAMETER(uint32, Method)
		SHADER_PARAMETER(uint32, PassType)
		SHADER_PARAMETER(uint32, SupportedPass)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, SampleDataTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, SampleCountTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		END_SHADER_PARAMETER_STRUCT()
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsOITSortedPixelsSupported(Parameters.Platform) && ShaderPrint::IsSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_OIT_DEBUG"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FOITPixelDebugCS, "/Engine/Private/OITCombine.usf", "MainCS", SF_Compute);

static void AddOITPixelDebugPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FOITData& OITData)
{
	if (!OITData.SampleDataTexture ||
		!OITData.SampleCountTexture || 
		!ShaderPrint::IsSupported(View.GetShaderPlatform()))
		return;

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForCharacters(512);

	FOITPixelDebugCS::FParameters* Parameters = GraphBuilder.AllocParameters<FOITPixelDebugCS::FParameters>();
	Parameters->PassType = OITData.PassType;
	Parameters->SupportedPass = OITData.SupportedPass;
	Parameters->Resolution = View.ViewRect.Size();
	Parameters->CursorCoord = View.CursorPos;
	Parameters->MaxSampleCount = OITData.MaxSamplePerPixel;
	Parameters->Method = OITData.Method;
	Parameters->MaxSideSampleCount = OITData.MaxSideSamplePerPixel;
	Parameters->SampleDataTexture  = OITData.SampleDataTexture;
	Parameters->SampleCountTexture = OITData.SampleCountTexture;
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintUniformBuffer);

	FOITPixelDebugCS::FPermutationDomain PermutationVector;
	TShaderMapRef<FOITPixelDebugCS> ComputeShader(View.ShaderMap, PermutationVector);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("Translucency::OITDebug(Pixel,%s)", !!(OITData.PassType & OITPass_SeparateTranslucency) ? TEXT("SeparateTransluency") : TEXT("Regular")),
		ComputeShader,
		Parameters,
		FIntVector(1, 1, 1));
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Combine OIT samples
class FOITPixelCombineCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOITPixelCombineCS);
	SHADER_USE_PARAMETER_STRUCT(FOITPixelCombineCS, FGlobalShader);

	class FModulate : SHADER_PERMUTATION_BOOL("PERMUTATION_MODULATE");
	using FPermutationDomain = TShaderPermutationDomain<FModulate>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER(uint32, MaxSideSampleCount)
		SHADER_PARAMETER(uint32, MaxSampleCount)
		SHADER_PARAMETER(uint32, Method)
		SHADER_PARAMETER(uint32, PassType)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, SampleDataTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, SampleCountTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutLightingTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutTransmittanceTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return IsOITSortedPixelsSupported(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		OutEnvironment.SetDefine(TEXT("SHADER_OIT_COMBINE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FOITPixelCombineCS, "/Engine/Private/OITCombine.usf", "MainCS", SF_Compute);

static void AddInternalOITComposePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FOITData& OITData,
	FRDGTextureRef OutColorTexture,
	FRDGTextureRef OutLightingTexture,
	FRDGTextureRef OutTransmittanceTexture)
{
	if (!OutColorTexture && !OutLightingTexture) return;

	const bool bValidSceneColor = OutColorTexture && OutColorTexture->Desc.NumSamples == 1;
	const bool bModulate = OutLightingTexture && OutLightingTexture->Desc.NumSamples == 1 && OutTransmittanceTexture && OutTransmittanceTexture->Desc.NumSamples == 1;

	if (!(bValidSceneColor || bModulate) ||
		!OITData.SampleDataTexture ||
		!OITData.SampleCountTexture)
		return;

	FIntPoint InResolution = OITData.SampleDataTexture->Desc.Extent;
	FIntPoint OutResolution = bModulate ? OutLightingTexture->Desc.Extent : OutColorTexture->Desc.Extent; //-V522
	FIntPoint Resolution = FIntPoint(FMath::Min(InResolution.X, OutResolution.X), FMath::Min(InResolution.Y, OutResolution.Y));

	FOITPixelCombineCS::FParameters* Parameters = GraphBuilder.AllocParameters<FOITPixelCombineCS::FParameters>();
	Parameters->PassType = OITData.PassType;
	Parameters->Resolution = Resolution;
	Parameters->Method = OITData.Method;
	Parameters->MaxSampleCount = OITData.MaxSamplePerPixel;
	Parameters->MaxSideSampleCount = OITData.MaxSideSamplePerPixel;
	Parameters->SampleDataTexture = OITData.SampleDataTexture;
	Parameters->SampleCountTexture = OITData.SampleCountTexture;
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	if (bModulate)
	{
		Parameters->OutLightingTexture = GraphBuilder.CreateUAV(OutLightingTexture);
		Parameters->OutTransmittanceTexture = GraphBuilder.CreateUAV(OutTransmittanceTexture);
	}
	else
	{
		Parameters->OutColorTexture = GraphBuilder.CreateUAV(OutColorTexture);
	}

	FOITPixelCombineCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FOITPixelCombineCS::FModulate>(bModulate);
	TShaderMapRef<FOITPixelCombineCS > ComputeShader(View.ShaderMap, PermutationVector);

	// Add 64 threads permutation
	const int32 GroupSize = 8;
	const FIntVector DispatchCount = FIntVector(FMath::DivideAndRoundUp(Resolution.X, GroupSize), FMath::DivideAndRoundUp(Resolution.Y, GroupSize), 1);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("Translucency::OITCombine(%s)", !!(OITData.PassType & OITPass_SeparateTranslucency) ? TEXT("SeparateTransluency") : TEXT("Regular")),
		ComputeShader,
		Parameters,
		DispatchCount);

	// Add debug pass
	const uint32 ActiveDebugPass = CVarOIT_SortedPixels_Debug.GetValueOnRenderThread();
	if (ActiveDebugPass == OITData.PassType)
	{
		AddOITPixelDebugPass(GraphBuilder, View, OITData);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// OIT Index buffer

class FSortedIndexBuffer : public FIndexBuffer
{
public:
	static const uint32 SliceCount = 256u;

	FSortedIndexBuffer(uint32 InId, const FIndexBuffer* InSourceIndexBuffer, uint32 InNumIndices, const TCHAR* InDebugName)
	: SourceIndexBuffer(InSourceIndexBuffer)
	, NumIndices(InNumIndices)
	, Id(InId)
	, DebugName(InDebugName) { }

	bool CanBeInitialized() const
	{
		return SourceIndexBuffer && SourceIndexBuffer->IndexBufferRHI && SourceIndexBuffer->IndexBufferRHI->GetStride() > 0;
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		check(SourceIndexBuffer && SourceIndexBuffer->IndexBufferRHI);
		const uint32 BytesPerElement = SourceIndexBuffer->IndexBufferRHI->GetStride();
		check(BytesPerElement == 2 || BytesPerElement == 4);
		const EPixelFormat Format = BytesPerElement == 2 ? PF_R16_UINT : PF_R32_UINT;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex(DebugName, NumIndices * BytesPerElement, BytesPerElement)
			.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer);

		IndexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		SortedIndexUAV = RHICmdList.CreateUnorderedAccessView(
			IndexBufferRHI,
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format));
		SourceIndexSRV = RHICmdList.CreateShaderResourceView(
			SourceIndexBuffer->IndexBufferRHI, 
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format));
	}

	virtual void ReleaseRHI() override
	{
		IndexBufferRHI.SafeRelease();
		SortedIndexUAV.SafeRelease();
		SourceIndexSRV.SafeRelease();
	}

	static constexpr uint32 InvalidId = ~0;

	const FIndexBuffer* SourceIndexBuffer = nullptr;
	uint32 NumIndices = 0;
	uint32 Id = FSortedIndexBuffer::InvalidId;
	uint32 LastUsedFrameId = 0;
	const TCHAR* DebugName = nullptr;

	FShaderResourceViewRHIRef  SourceIndexSRV = nullptr;
	FUnorderedAccessViewRHIRef SortedIndexUAV = nullptr;
};

static void TrimSortedIndexBuffers(TArray<FSortedIndexBuffer*>& FreeBuffers, TQueue<FSortedIndexBuffer*>& DeleteQueue, uint32 FrameId)
{
	uint32 FreeCount = FreeBuffers.Num();
	for (uint32 FreeIt = 0; FreeIt < FreeCount;)
	{
		check(FreeBuffers[FreeIt]);
		const uint32 LastFrameId = FreeBuffers[FreeIt]->LastUsedFrameId;
		if (LastFrameId != 0)
		{
			const int32 ElapsedFrame = FMath::Abs(int32(FrameId) - int32(LastFrameId));
			if (ElapsedFrame > CVarOIT_SortedTriangles_PoolReleaseThreshold.GetValueOnRenderThread())
			{
				DeleteQueue.Enqueue(FreeBuffers[FreeIt]);
				FreeBuffers[FreeIt] = FreeBuffers[FreeCount - 1];
				--FreeCount;
				FreeBuffers.SetNum(FreeCount);
				continue;
			}
		}
		++FreeIt;
	}
}

void FOITSceneData::Allocate(
	FRHICommandListBase& RHICmdList, 
	EPrimitiveType InPrimitiveType, 
	const FMeshBatchElement& InMeshElement, 
	FMeshBatchElementDynamicIndexBuffer& OutMeshElement)
{
	check(InMeshElement.IndexBuffer && InMeshElement.IndexBuffer->IndexBufferRHI);
	check(InPrimitiveType == PT_TriangleList || InPrimitiveType == PT_TriangleStrip);

	// Find a free slot, or create a new one
	FSortedTriangleData* Out = nullptr;
	uint32 FreeSlot = FSortedIndexBuffer::InvalidId;
	FreeSlots.Dequeue(FreeSlot);
	if (FreeSlot != FSortedIndexBuffer::InvalidId)
	{
		Out = &Allocations[FreeSlot];
	}
	else
	{
		FreeSlot = Allocations.Num();
		Out = &Allocations.AddDefaulted_GetRef();
	}

	// Linear scan if there are some free resource which are large enough
	const uint32 NumIndices = InMeshElement.NumPrimitives * 3; // Sorted index always has triangle list topology
	FSortedIndexBuffer* OITIndexBuffer = nullptr;
	if (CVarOIT_SortedTriangles_Pool.GetValueOnRenderThread() > 0)
	{
		for (uint32 FreeIt=0,FreeCount=FreeBuffers.Num(); FreeIt<FreeCount; ++FreeIt)
		{		
			FSortedIndexBuffer* FreeBuffer = FreeBuffers[FreeIt];
			// TODO: check for format as well + more robust comparison
			if (FreeBuffer != nullptr && FreeBuffer->NumIndices >= NumIndices && FreeBuffer->Id == FSortedIndexBuffer::InvalidId) 
			{			
				OITIndexBuffer = FreeBuffer;
				OITIndexBuffer->Id = FreeSlot;
				FreeBuffers[FreeIt] = FreeBuffers[FreeCount - 1];
				FreeBuffers.SetNum(FreeCount - 1);
				break;
			}
		}
	}

	// Otherwise create a new one
	if (OITIndexBuffer == nullptr)
	{
		OITIndexBuffer = new FSortedIndexBuffer(FreeSlot, InMeshElement.IndexBuffer, NumIndices, TEXT("OIT::SortedIndexBuffer"));

		// It's possible the index buffer isn't ready yet (data not streamed-in yet). In such case we post-pone OITIndexBuffer creation.
		if (OITIndexBuffer->CanBeInitialized())
		{
			OITIndexBuffer->InitResource(RHICmdList);
		}
	}
	Out->NumPrimitives = InMeshElement.NumPrimitives;
	Out->NumIndices = NumIndices;
	Out->SourceFirstIndex = InMeshElement.FirstIndex;
	Out->SourceBaseVertexIndex = InMeshElement.BaseVertexIndex;
	Out->SourceMinVertexIndex = InMeshElement.MinVertexIndex;
	Out->SourceMaxVertexIndex = InMeshElement.MaxVertexIndex;
	Out->SortedFirstIndex = 0u;
	Out->SourcePrimitiveType = InPrimitiveType;
	Out->SortedPrimitiveType = PT_TriangleList;
	Out->SourceIndexBuffer = InMeshElement.IndexBuffer;
	Out->SortedIndexBuffer = OITIndexBuffer;

	OutMeshElement.IndexBuffer 	= Out->SortedIndexBuffer;
	OutMeshElement.FirstIndex 	= Out->SortedFirstIndex;
	OutMeshElement.PrimitiveType= Out->SortedPrimitiveType;
}

void FOITSceneData::Deallocate(FMeshBatchElement& OutMeshElement)
{
	if (FSortedIndexBuffer* OITIndexBuffer = (FSortedIndexBuffer*)OutMeshElement.DynamicIndexBuffer.IndexBuffer)
	{
		const uint32 Slot = OITIndexBuffer->Id;
		if (Slot < uint32(Allocations.Num()))
		{
			if (CVarOIT_SortedTriangles_Pool.GetValueOnAnyThread() > 0)
			{
				OITIndexBuffer->Id = FSortedIndexBuffer::InvalidId;
				OITIndexBuffer->LastUsedFrameId = FrameIndex;
				FreeBuffers.Add(OITIndexBuffer);
				Allocations[Slot] = FSortedTriangleData();
			}
			else
			{
				FSortedTriangleData& In = Allocations[Slot];
				PendingDeletes.Enqueue((FSortedIndexBuffer*)In.SortedIndexBuffer);
				In = FSortedTriangleData();
			}
			FreeSlots.Enqueue(Slot);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Sort triangle indices to order them front-to-back or back-to-front

class FOITSortTriangleIndex_ScanCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOITSortTriangleIndex_ScanCS);
	SHADER_USE_PARAMETER_STRUCT(FOITSortTriangleIndex_ScanCS, FGlobalShader);

	class FDebug : SHADER_PERMUTATION_BOOL("PERMUTATION_DEBUG");
	using FPermutationDomain = TShaderPermutationDomain<FDebug>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters,)

	// For Debug
	SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
	SHADER_PARAMETER(FMatrix44f, ViewToWorld)
	SHADER_PARAMETER(FVector3f, WorldBound_Min)
	SHADER_PARAMETER(FVector3f, WorldBound_Max)
	SHADER_PARAMETER(FVector3f, ViewBound_Min)
	SHADER_PARAMETER(FVector3f, ViewBound_Max)

	SHADER_PARAMETER(FMatrix44f, LocalToWorld)
	SHADER_PARAMETER(FMatrix44f, LocalToView)

	SHADER_PARAMETER(uint32, SourcePrimitiveType)
	SHADER_PARAMETER(uint32, SourceFirstIndex)
	SHADER_PARAMETER(uint32, SourceBaseVertexIndex)
	SHADER_PARAMETER(uint32, SourceMinVertexIndex)
	SHADER_PARAMETER(uint32, SourceMaxVertexIndex)

	SHADER_PARAMETER(uint32, NumPrimitives)
	SHADER_PARAMETER(uint32, NumIndices)
	SHADER_PARAMETER(uint32, SortType)
	SHADER_PARAMETER(uint32, SortedIndexBufferSizeInByte)
		
	SHADER_PARAMETER(float, ViewBoundMinZ)
	SHADER_PARAMETER(float, ViewBoundMaxZ)

	SHADER_PARAMETER_SRV(Buffer < float > , PositionBuffer)
	SHADER_PARAMETER_SRV(Buffer < uint > , IndexBuffer)
	SHADER_PARAMETER_UAV(Buffer < uint > , OutIndexBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer < uint2 > , OutSliceCounterBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer < uint > , OutPrimitiveSliceBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer < uint > , OutDebugData)


	END_SHADER_PARAMETER_STRUCT()
public:

	static bool SupportDebugMode(EShaderPlatform InPlatform)
	{
		return ShaderPrint::IsSupported(InPlatform);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDebug>() == 1 && !SupportDebugMode(Parameters.Platform))
		{
			return false;
		}
		return OIT::IsSortedTrianglesEnabled(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_SCAN"), 1);
		OutEnvironment.SetDefine(TEXT("SORTING_SLICE_COUNT"), FSortedIndexBuffer::SliceCount);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDebug>())
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}
		return FGlobalShader::ShouldPrecachePermutation(Parameters);
	}
};

IMPLEMENT_GLOBAL_SHADER(FOITSortTriangleIndex_ScanCS, "/Engine/Private/OIT/OITSorting.usf", "MainCS", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////

class FOITSortTriangleIndex_AllocateCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOITSortTriangleIndex_AllocateCS);
	SHADER_USE_PARAMETER_STRUCT(FOITSortTriangleIndex_AllocateCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SliceCounterBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, SliceOffsetsBuffer)
	END_SHADER_PARAMETER_STRUCT()
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return OIT::IsSortedTrianglesEnabled(Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ALLOCATE"), 1);
		OutEnvironment.SetDefine(TEXT("SORTING_SLICE_COUNT"), FSortedIndexBuffer::SliceCount);
	}
};

IMPLEMENT_GLOBAL_SHADER(FOITSortTriangleIndex_AllocateCS, "/Engine/Private/OIT/OITSorting.usf", "MainCS", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////

class FOITSortTriangleIndex_WriteOutCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOITSortTriangleIndex_WriteOutCS);
	SHADER_USE_PARAMETER_STRUCT(FOITSortTriangleIndex_WriteOutCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, SourcePrimitiveType)
		SHADER_PARAMETER(uint32, NumPrimitives)
		SHADER_PARAMETER(uint32, NumIndices)
		SHADER_PARAMETER(uint32, SrcFirstIndex)
		SHADER_PARAMETER(uint32, DstFirstIndex)

		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SliceOffsetsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, PrimitiveSliceBuffer)

		SHADER_PARAMETER_SRV(Buffer<uint>, IndexBuffer)
		SHADER_PARAMETER_UAV(RWBuffer<uint>, OutIndexBuffer)
	END_SHADER_PARAMETER_STRUCT()
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return OIT::IsSortedTrianglesEnabled(Parameters.Platform);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_WRITE"), 1);
		OutEnvironment.SetDefine(TEXT("SORTING_SLICE_COUNT"), FSortedIndexBuffer::SliceCount);
	}
};

IMPLEMENT_GLOBAL_SHADER(FOITSortTriangleIndex_WriteOutCS, "/Engine/Private/OIT/OITSorting.usf", "MainCS", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////

class FOITSortTriangleIndex_Debug : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOITSortTriangleIndex_Debug);
	SHADER_USE_PARAMETER_STRUCT(FOITSortTriangleIndex_Debug, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER(uint32, VisibleInstances)
		SHADER_PARAMETER(uint32, VisiblePrimitives)
		SHADER_PARAMETER(uint32, VisibleIndexSizeInBytes)
		SHADER_PARAMETER(uint32, AllocatedBuffers)
		SHADER_PARAMETER(uint32, AllocatedIndexSizeInBytes)
		SHADER_PARAMETER(uint32, UnusedBuffers)
		SHADER_PARAMETER(uint32, UnusedIndexSizeInBytes)
		SHADER_PARAMETER(uint32, TotalEntries)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, DebugData)
	END_SHADER_PARAMETER_STRUCT()
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return FOITSortTriangleIndex_ScanCS::SupportDebugMode(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_DEBUG"), 1);
		OutEnvironment.SetDefine(TEXT("SORTING_SLICE_COUNT"), FSortedIndexBuffer::SliceCount);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FOITSortTriangleIndex_Debug, "/Engine/Private/OIT/OITSorting.usf", "MainCS", SF_Compute);

///////////////////////////////////////////////////////////////////////////////////////////////////

struct FOITDebugData
{
	static const EPixelFormat Format = PF_R32_UINT;

	FRDGBufferRef Buffer = nullptr; // First element is counter, then element are: (NumPrim/Type/Size)

	uint32 VisibleInstances = 0;
	uint32 VisiblePrimitives = 0;
	uint32 VisibleIndexSizeInBytes = 0;

	uint32 AllocatedBuffers = 0;
	uint32 AllocatedIndexSizeInBytes = 0;

	uint32 UnusedBuffers = 0;
	uint32 UnusedIndexSizeInBytes = 0;

	uint32 TotalEntries = 0;

	bool IsValid() const { return Buffer != nullptr; }
};

static void AddOITSortTriangleIndexPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FOITSceneData& OITSceneData,
	const FSortedTrianglesMeshBatch& MeshBatch,
	FTriangleSortingOrder SortType,
	FOITDebugData& DebugData)
{
	// Fat format: PF_R32G32_UINT | Compact format: PF_R32_UINT
	const EPixelFormat PackedFormat = PF_R32_UINT;
	const uint32 PackedFormatInBytes = 4;

	const bool bIsValid = 
		MeshBatch.Mesh != nullptr && 
		MeshBatch.Mesh->VertexFactory != nullptr &&
		MeshBatch.Mesh->VertexFactory->SupportsTriangleSorting() &&
		MeshBatch.Mesh->Elements.Num() > 0 && 
		MeshBatch.Mesh->Elements[0].DynamicIndexBuffer.IndexBuffer != nullptr;
	if (!bIsValid)
	{
		return;
	}

	// If the index buffer hasn't been initialized yet (e.g., data are not streamed in yet), run initialization
	FSortedIndexBuffer* SortedIndexBuffer = (FSortedIndexBuffer*)MeshBatch.Mesh->Elements[0].DynamicIndexBuffer.IndexBuffer;
	if (!SortedIndexBuffer->IsInitialized())
	{
		if (!SortedIndexBuffer->CanBeInitialized())
		{
			// Data are still not ready yet
			return;
		}
		SortedIndexBuffer->InitResource(GraphBuilder.RHICmdList);
	}

	const FShaderResourceViewRHIRef VertexPosition = MeshBatch.Mesh->VertexFactory->GetTriangleSortingPositionSRV();
	if (!VertexPosition) { return; }

	const FSortedIndexBuffer* OITIndexBuffer = (const FSortedIndexBuffer*)MeshBatch.Mesh->Elements[0].DynamicIndexBuffer.IndexBuffer;
	check(OITIndexBuffer->Id < uint32(OITSceneData.Allocations.Num()));
	const FSortedTriangleData& Allocation = OITSceneData.Allocations[OITIndexBuffer->Id];
	
	check(Allocation.IsValid());
	check(Allocation.SourcePrimitiveType == PT_TriangleList || Allocation.SourcePrimitiveType == PT_TriangleStrip);
	check(Allocation.SortedPrimitiveType == PT_TriangleList);

	FRDGBufferRef PrimitiveSliceBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(PackedFormatInBytes, Allocation.NumPrimitives), TEXT("OIT.TriangleSortingSliceIndex"));
	FRDGBufferRef SliceCounterBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, FSortedIndexBuffer::SliceCount), TEXT("OIT.SliceCounters"));
	FRDGBufferRef SliceOffsetsBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4, FSortedIndexBuffer::SliceCount), TEXT("OIT.SliceOffsets"));
	FRDGBufferUAVRef SliceCounterUAV = GraphBuilder.CreateUAV(SliceCounterBuffer, PF_R32_UINT);
	AddClearUAVPass(GraphBuilder, SliceCounterUAV, 0u);

	const EShaderPlatform Platform = View.Family->GetShaderPlatform();
	const bool bDebugEnable = DebugData.IsValid() && FOITSortTriangleIndex_ScanCS::SupportDebugMode(Platform) && ShaderPrint::IsValid(View.ShaderPrintData);
	FRHIBuffer* SortedIndexBufferRHI = Allocation.SortedIndexBuffer->IndexBufferRHI;

	// 1. Scan the primitive and assign each primitive to a slice
	{
		// Compute the primitive Min/Max-Z value in view space. This domain is sliced for sorting
		const FBoxSphereBounds& Bounds = MeshBatch.Proxy->GetBounds();
		const FBoxSphereBounds& ViewBounds = Bounds.TransformBy(View.ViewMatrices.GetWorldToView());
		const float ViewBoundMinZ = ViewBounds.GetBox().Min.Z;
		const float ViewBoundMaxZ = ViewBounds.GetBox().Max.Z;

		FOITSortTriangleIndex_ScanCS::FParameters* Parameters = GraphBuilder.AllocParameters<FOITSortTriangleIndex_ScanCS::FParameters>();
		Parameters->LocalToView				= FMatrix44f(MeshBatch.Proxy->GetLocalToWorld() * View.ViewMatrices.GetWorldToView());
		Parameters->LocalToWorld 			= FMatrix44f(MeshBatch.Proxy->GetLocalToWorld());
		Parameters->SourcePrimitiveType		= Allocation.SourcePrimitiveType == PT_TriangleStrip ? 1u : 0u;
		Parameters->NumPrimitives			= Allocation.NumPrimitives;
		Parameters->NumIndices				= Allocation.NumIndices;
		Parameters->ViewBoundMinZ			= ViewBoundMinZ;
		Parameters->ViewBoundMaxZ			= ViewBoundMaxZ;
		Parameters->SortType				= SortType == FTriangleSortingOrder::BackToFront ? 0 : 1;
		Parameters->SortedIndexBufferSizeInByte = Allocation.SortedIndexBuffer->IndexBufferRHI->GetSize();
		Parameters->PositionBuffer			= VertexPosition;
		Parameters->SourceFirstIndex		= Allocation.SourceFirstIndex;
		Parameters->SourceBaseVertexIndex	= Allocation.SourceBaseVertexIndex;
		Parameters->SourceMinVertexIndex	= Allocation.SourceMinVertexIndex;
		Parameters->SourceMaxVertexIndex	= Allocation.SourceMaxVertexIndex;
		Parameters->IndexBuffer				= Allocation.SortedIndexBuffer->SourceIndexSRV;
		Parameters->OutIndexBuffer			= Allocation.SortedIndexBuffer->SortedIndexUAV;
		Parameters->OutSliceCounterBuffer	= SliceCounterUAV;
		Parameters->OutPrimitiveSliceBuffer = GraphBuilder.CreateUAV(PrimitiveSliceBuffer, PackedFormat);

		// Debug
		if (bDebugEnable)
		{
			Parameters->ViewToWorld		= FMatrix44f(View.ViewMatrices.GetWorldToView().Inverse());	 // LWC_TODO: Precision loss?
			Parameters->WorldBound_Min	= (FVector3f)Bounds.GetBox().Min;
			Parameters->WorldBound_Max	= (FVector3f)Bounds.GetBox().Max;
			Parameters->ViewBound_Min	= (FVector3f)ViewBounds.GetBox().Min;
			Parameters->ViewBound_Max	= (FVector3f)ViewBounds.GetBox().Max;
			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);

			check(DebugData.Buffer);

			++DebugData.VisibleInstances;
			DebugData.VisiblePrimitives += Parameters->NumPrimitives;
			DebugData.VisibleIndexSizeInBytes += Allocation.SortedIndexBuffer->IndexBufferRHI->GetSize();
			Parameters->OutDebugData = GraphBuilder.CreateUAV(DebugData.Buffer, PF_R32_UINT);
		}

		FOITSortTriangleIndex_ScanCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FOITSortTriangleIndex_ScanCS::FDebug>(bDebugEnable ? 1 : 0);
		TShaderMapRef<FOITSortTriangleIndex_ScanCS> ComputeShader(View.ShaderMap, PermutationVector);
		
		const uint32 GroupSize = FSortedIndexBuffer::SliceCount;
		const FIntVector DispatchCount = FIntVector(FMath::DivideAndRoundUp(Parameters->NumPrimitives, GroupSize), 1u, 1u);
		check(DispatchCount.X < GRHIMaxDispatchThreadGroupsPerDimension.X);
		ClearUnusedGraphResources(ComputeShader, Parameters);
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("OIT::SortTriangleIndices(Scan)"),
			Parameters,
			ERDGPassFlags::Compute,
			[Parameters, ComputeShader, DispatchCount, SortedIndexBufferRHI](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
			{
				RHICmdList.Transition(FRHITransitionInfo(SortedIndexBufferRHI, ERHIAccess::VertexOrIndexBuffer, ERHIAccess::UAVCompute));
				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *Parameters, DispatchCount);
			});
	}

	// 2. Pre-fix sum onto the slices count to allocate each bucket
	{
		FOITSortTriangleIndex_AllocateCS::FParameters* Parameters = GraphBuilder.AllocParameters<FOITSortTriangleIndex_AllocateCS::FParameters>();
		Parameters->SliceCounterBuffer = GraphBuilder.CreateSRV(SliceCounterBuffer, PF_R32_UINT);
		Parameters->SliceOffsetsBuffer = GraphBuilder.CreateUAV(SliceOffsetsBuffer, PF_R32_UINT);
		TShaderMapRef<FOITSortTriangleIndex_AllocateCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("OIT::SortTriangleIndices(PrefixedSum)"),
			ComputeShader,
			Parameters,
			FIntVector(1u, 1u, 1u));
	}

	// 3. Write-out the sorted indices
	{
		FOITSortTriangleIndex_WriteOutCS::FParameters* Parameters = GraphBuilder.AllocParameters<FOITSortTriangleIndex_WriteOutCS::FParameters>();
		Parameters->SourcePrimitiveType		= Allocation.SourcePrimitiveType == PT_TriangleStrip ? 1u : 0u;
		Parameters->NumPrimitives			= Allocation.NumPrimitives;
		Parameters->NumIndices				= Allocation.NumIndices;
		Parameters->SrcFirstIndex			= Allocation.SourceFirstIndex;
		Parameters->DstFirstIndex			= 0u;

		Parameters->SliceOffsetsBuffer		= GraphBuilder.CreateSRV(SliceOffsetsBuffer, PF_R32_UINT);
		Parameters->PrimitiveSliceBuffer	= GraphBuilder.CreateSRV(PrimitiveSliceBuffer, PackedFormat);

		Parameters->IndexBuffer				= Allocation.SortedIndexBuffer->SourceIndexSRV;
		Parameters->OutIndexBuffer			= Allocation.SortedIndexBuffer->SortedIndexUAV;

		TShaderMapRef<FOITSortTriangleIndex_WriteOutCS> ComputeShader(View.ShaderMap);

		const uint32 GroupSize = 256;
		const FIntVector DispatchCount = FIntVector(FMath::DivideAndRoundUp(Parameters->NumPrimitives, GroupSize), 1u, 1u);
		ClearUnusedGraphResources(ComputeShader, Parameters);
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("OIT::SortTriangleIndices(Write)"),
			Parameters,
			ERDGPassFlags::Compute,
			[Parameters, ComputeShader, DispatchCount, SortedIndexBufferRHI](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
			{
				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *Parameters, DispatchCount);

				RHICmdList.Transition(FRHITransitionInfo(SortedIndexBufferRHI, ERHIAccess::UAVCompute, ERHIAccess::VertexOrIndexBuffer));
			});
	}

	// Next todos
	// * Merge several meshes together (not clear out to do the mapping thread->mesh info)
	// * Batch Scan/Alloc/Write of several primitive, so that we have better overlapping
}

static void AddOITTriangleDebugPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FOITDebugData& DebugData)
{
	const EShaderPlatform Platform = View.Family->GetShaderPlatform();
	if (!DebugData.IsValid() || !ShaderPrint::IsSupported(Platform))
	{
		return;
	}

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);
	ShaderPrint::RequestSpaceForCharacters(DebugData.VisibleInstances * 256 + 512);
	ShaderPrint::RequestSpaceForLines(DebugData.VisiblePrimitives * 8 + DebugData.VisibleInstances * 32);

	FOITSortTriangleIndex_Debug::FParameters* Parameters = GraphBuilder.AllocParameters<FOITSortTriangleIndex_Debug::FParameters>();
	Parameters->VisibleInstances = DebugData.VisibleInstances;
	Parameters->VisiblePrimitives = DebugData.VisiblePrimitives;
	Parameters->VisibleIndexSizeInBytes = DebugData.VisibleIndexSizeInBytes;
	Parameters->UnusedBuffers = DebugData.UnusedBuffers;
	Parameters->UnusedIndexSizeInBytes = DebugData.UnusedIndexSizeInBytes;
	Parameters->AllocatedBuffers = DebugData.AllocatedBuffers;
	Parameters->AllocatedIndexSizeInBytes = DebugData.AllocatedIndexSizeInBytes;
	Parameters->TotalEntries = DebugData.TotalEntries;
	Parameters->DebugData = GraphBuilder.CreateSRV(DebugData.Buffer, FOITDebugData::Format);
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);

	TShaderMapRef<FOITSortTriangleIndex_Debug> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("OIT::Debug(Triangle)"),
		ComputeShader,
		Parameters,
		FIntVector(1u, 1u, 1u));
}

///////////////////////////////////////////////////////////////////////////////////////////////////

namespace OIT
{
	bool IsSortedTrianglesEnabled(EShaderPlatform InPlatform)
	{
		return CVarOIT_SortedTriangles_Enable_Project.GetValueOnAnyThread() > 0 && !IsOpenGLPlatform(InPlatform);
	}

	bool IsSortedPixelsEnabledForProject(EShaderPlatform InPlatform)
	{
		//const bool bMSAAEnabled = GetDefaultAntiAliasingMethod(GetMaxSupportedFeatureLevel(InPlatform)) != EAntiAliasingMethod::AAM_MSAA;
		const bool bPixelOIT = CVarOIT_SortedPixels_Enable_Project.GetValueOnAnyThread() > 0 && FDataDrivenShaderPlatformInfo::GetSupportsOIT(EShaderPlatform(InPlatform));
		return bPixelOIT;
	}

	bool InternalIsSortedPixelsEnabled(EShaderPlatform InPlatform, bool bMSAA)
	{
		return IsSortedPixelsEnabledForProject(InPlatform) && GRHISupportsRasterOrderViews && !!CVarOIT_SortedPixels_Enable_Runtime.GetValueOnRenderThread() && !bMSAA;
	}
	bool IsSortedPixelsEnabled(const FViewInfo& InView) 	{ return InternalIsSortedPixelsEnabled(InView.GetShaderPlatform(), InView.AntiAliasingMethod == EAntiAliasingMethod::AAM_MSAA); }
	bool IsSortedPixelsEnabled(EShaderPlatform InPlatform)	{ return InternalIsSortedPixelsEnabled(InPlatform, false /*bMSAA*/); }
	
	bool IsSortedPixelsEnabledForPass(EOITPassType PassType)
	{
		const uint32 PassTypeBits = FMath::Clamp(CVarOIT_SortedPixels_PassType.GetValueOnRenderThread(), 0, 3);
		return !!(PassTypeBits & PassType);
	}

	bool IsCompatible(const FMeshBatch& InMesh, ERHIFeatureLevel::Type InFeatureLevel)
	{
		return InMesh.IsTranslucent(InFeatureLevel) && InMesh.VertexFactory && InMesh.VertexFactory->SupportsTriangleSorting();
	}

	void AddSortTrianglesPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FOITSceneData& OITSceneData, FTriangleSortingOrder SortType)
	{
		if (!IsSortedTrianglesEnabled(View.GetShaderPlatform()))
		{
			return;
		}

		RDG_EVENT_SCOPE(GraphBuilder, "OIT::IndexSorting");

		const bool bDebugEnable = CVarOIT_SortedTriangles_Debug.GetValueOnRenderThread() > 0;
		FOITDebugData DebugData;
		if (bDebugEnable)
		{
			const uint32 ValidAllocationCount = OITSceneData.Allocations.Num();
			DebugData.Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(4u, 10*ValidAllocationCount + 1), TEXT("OIT.DebugData"));
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(DebugData.Buffer, FOITDebugData::Format), 0u);

			// Allocated/used
			uint32 AllocatedNumPrimitives = 0;
			for (const FSortedTriangleData& Allocated : OITSceneData.Allocations)
			{
				if (Allocated.IsValid())
				{
					++DebugData.AllocatedBuffers;
					DebugData.AllocatedIndexSizeInBytes += Allocated.SortedIndexBuffer->IndexBufferRHI->GetSize();
					AllocatedNumPrimitives += Allocated.NumPrimitives;
				}
			}

			ShaderPrint::SetEnabled(true);
			ShaderPrint::RequestSpaceForCharacters(DebugData.AllocatedBuffers * 256 + 512);
			ShaderPrint::RequestSpaceForLines(AllocatedNumPrimitives * 8 + DebugData.AllocatedBuffers * 32);

			// Unused
			for (const FSortedIndexBuffer* FreeBuffer : OITSceneData.FreeBuffers)
			{
				++DebugData.UnusedBuffers;
				DebugData.UnusedIndexSizeInBytes = FreeBuffer->IndexBufferRHI->GetSize();
			}

			DebugData.TotalEntries = OITSceneData.Allocations.Num();
		}

		for (const FSortedTrianglesMeshBatch& MeshBatch : View.SortedTrianglesMeshBatches)		
		{
			AddOITSortTriangleIndexPass(GraphBuilder, View, OITSceneData, MeshBatch, SortType, DebugData);
		}

		if (DebugData.IsValid())
		{
			AddOITTriangleDebugPass(GraphBuilder, View, DebugData);
		}

		// Trim unused buffers
		OITSceneData.FrameIndex = View.Family->FrameNumber;
		if (CVarOIT_SortedTriangles_Pool.GetValueOnRenderThread() > 0 && CVarOIT_SortedTriangles_PoolReleaseThreshold.GetValueOnRenderThread() > 0)
		{
			TrimSortedIndexBuffers(OITSceneData.FreeBuffers, OITSceneData.PendingDeletes, OITSceneData.FrameIndex);
		}
	}

	FOITData CreateOITData(FRDGBuilder& GraphBuilder, const FViewInfo& View, EOITPassType PassType)
	{
		const bool bOIT = IsSortedPixelsEnabled(View);
		const uint32 PassTypeBits = FMath::Clamp(CVarOIT_SortedPixels_PassType.GetValueOnRenderThread(), 0, 3);
		const bool bPassValid = !!(PassTypeBits & PassType);
		const uint32 LayerCount = 3u; /*Depth/Color/Trans*/

		FOITData Out;
		if (!bOIT || !bPassValid)
		{
			Out.MaxSideSamplePerPixel = 0;
			Out.MaxSamplePerPixel = 0;
			Out.Method = 0;
			Out.TransmittanceThreshold = 0;

			Out.SampleDataTexture  = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2DArray(FIntPoint(1, 1), PF_R32_UINT, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV, LayerCount), TEXT("OIT.SampleData"));
			Out.SampleCountTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_R32_UINT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_UAV), TEXT("OIT.SampleCount"));
			return Out;
		}

		// Scene render targets might not exist yet; avoids NaNs.
		FIntPoint EffectiveBufferSize = View.GetSceneTexturesConfig().Extent;
		EffectiveBufferSize.X = FMath::Max(EffectiveBufferSize.X, 1);
		EffectiveBufferSize.Y = FMath::Max(EffectiveBufferSize.Y, 1);

		// Allocate OIT data		
		Out.PassType = PassTypeBits & PassType ? PassType : OITPass_None;
		if (PassTypeBits & OITPass_RegularTranslucency)  Out.SupportedPass |= OITPass_RegularTranslucency;
		if (PassTypeBits & OITPass_SeparateTranslucency) Out.SupportedPass |= OITPass_SeparateTranslucency;

		if (Out.PassType != OITPass_None)
		{
			// Round sample to be square to store them into square tile
			uint32 MaxSamplePerPixel = FMath::Clamp(CVarOIT_SortedPixels_SampleCount.GetValueOnRenderThread(), 1, 16);
			uint32 MaxSideSamplePerPixel = FMath::FloorToInt(FMath::Sqrt(float(MaxSamplePerPixel)));
			MaxSamplePerPixel = MaxSideSamplePerPixel * MaxSideSamplePerPixel;

			Out.MaxSideSamplePerPixel = MaxSideSamplePerPixel;
			Out.MaxSamplePerPixel = MaxSamplePerPixel;
			Out.Method = FMath::Clamp(CVarOIT_SortedPixels_Method.GetValueOnRenderThread(), 0, 1);
			Out.TransmittanceThreshold = FMath::Clamp(CVarOIT_SortedPixels_TransmittanceThreshold.GetValueOnRenderThread(), 0.f, 1.f);

			Out.SampleDataTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2DArray(EffectiveBufferSize * MaxSideSamplePerPixel, PF_R32_UINT, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV, LayerCount), TEXT("OIT.SampleData"));
			Out.SampleCountTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(EffectiveBufferSize, PF_R32_UINT, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV), TEXT("OIT.SampleCount"));
		}

		// TODO: Add tile clear based on the coarse translucent raster AABB buffer
		//AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Out.SampleDataTexture), 0u);
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(Out.SampleCountTexture), 0u);

		return Out;
	}

	void SetOITParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FOITBasePassUniformParameters& OutOIT, const FOITData& InOITData)
	{
		// Always set the parameters, even when the OIT is disabled, as Uniform buffer validation complain about null resources otherwise
		OutOIT.OITMethod = InOITData.Method;
		OutOIT.bOITEnable = InOITData.PassType != OITPass_None ? 1 : 0;
		OutOIT.MaxSideSamplePerPixel = InOITData.MaxSideSamplePerPixel;
		OutOIT.MaxSamplePerPixel = InOITData.MaxSamplePerPixel;
		OutOIT.TransmittanceThreshold = InOITData.TransmittanceThreshold;
		OutOIT.OutOITSampleData  = GraphBuilder.CreateUAV(InOITData.SampleDataTexture);
		OutOIT.OutOITSampleCount = GraphBuilder.CreateUAV(InOITData.SampleCountTexture);
	}

	void AddOITComposePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FOITData& OITData, FRDGTextureRef SceneColorTexture)
	{
		AddInternalOITComposePass(GraphBuilder, View, OITData, SceneColorTexture, nullptr, nullptr);
	}

	void AddOITComposePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FOITData& OITData, FRDGTextureRef LightingTexture, FRDGTextureRef ModulateTexture)
	{
		AddInternalOITComposePass(GraphBuilder, View, OITData, nullptr, LightingTexture, ModulateTexture);
	}

	void OnRenderBegin(FOITSceneData& OITSceneData)
	{
		// Delete all enqueue buffer for deletion
		FSortedIndexBuffer* Buffer = nullptr;
		while (OITSceneData.PendingDeletes.Dequeue(Buffer))
		{
			if (Buffer)
			{
				Buffer->ReleaseResource();
				delete Buffer;
				Buffer = nullptr;
			}
		}
	}
}
