// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "RenderResource.h"

class FViewInfo;
class FSortedIndexBuffer;
struct FMeshBatchElement;
struct FMeshBatchElementDynamicIndexBuffer;
class FViewInfo;

enum EOITPassType
{
	OITPass_None = 0,
	OITPass_RegularTranslucency = 1,
	OITPass_SeparateTranslucency = 2
};

enum class FTriangleSortingOrder
{
	FrontToBack,
	BackToFront,
};

// Sorted triangles data
struct FSortedTriangleData
{
	const FIndexBuffer* SourceIndexBuffer = nullptr;
	FSortedIndexBuffer* SortedIndexBuffer = nullptr;

	uint32 SortedFirstIndex = 0;
	uint32 SourceFirstIndex	= 0;
	uint32 SourceBaseVertexIndex = 0;
	uint32 SourceMinVertexIndex = 0;
	uint32 SourceMaxVertexIndex = 0;
	uint32 NumPrimitives = 0;
	uint32 NumIndices = 0;

	EPrimitiveType SourcePrimitiveType = PT_TriangleList;
	EPrimitiveType SortedPrimitiveType = PT_TriangleList;

	bool IsValid() const { return SortedIndexBuffer != nullptr; }
};

// Order Independent Translucency pass data for a given translucent pass
struct FOITData
{
	EOITPassType PassType = OITPass_None;
	uint32 SupportedPass = 0;
	uint32 Method = 0;
	uint32 MaxSamplePerPixel = 0;
	uint32 MaxSideSamplePerPixel = 0;
	float TransmittanceThreshold = 0.f;

	FRDGTextureRef SampleDataTexture = nullptr;
	FRDGTextureRef SampleCountTexture = nullptr;
};

// Order Independent Translucency scene data
struct FOITSceneData
{
	/* Allocate sorted-triangle data for a instance */
	void Allocate(FRHICommandListBase& RHICmdList, EPrimitiveType PrimitiveType, const FMeshBatchElement& InMeshElement, FMeshBatchElementDynamicIndexBuffer& OutMeshElement);

	/* Deallocate sorted-triangle data */
	void Deallocate(FMeshBatchElement& OutMeshElement);

	TArray<FSortedTriangleData> Allocations;
	TArray<FSortedIndexBuffer*> FreeBuffers;
	TQueue<FSortedIndexBuffer*> PendingDeletes;
	TQueue<uint32> FreeSlots;
	uint32 FrameIndex = 0;
};

namespace OIT
{
	/* Return true if OIT sorted triangles is enabled/supported */
	bool IsSortedTrianglesEnabled(EShaderPlatform InPlatform);

	/* Return true if the current MeshBatch is compatible with per-instance sorted triangle */
	bool IsCompatible(const FMeshBatch& Mesh, ERHIFeatureLevel::Type InFeatureLevel);

	/* Return if OIT sorted pixel is enabled for the project*/
	bool IsSortedPixelsEnabledForProject(EShaderPlatform InPlatform);

	/* Return if OIT is enabled for the current runtime */
	bool IsSortedPixelsEnabled(const FViewInfo& InView);
	bool IsSortedPixelsEnabled(EShaderPlatform InPlatform); 

	/* Return if OIT is enabled for the translucent pass current pass */
	bool IsSortedPixelsEnabledForPass(EOITPassType PassType);

	/* Sort triangles of all instances whose has the sorted triangle option enabled */
	void AddSortTrianglesPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FOITSceneData& OITSceneData, FTriangleSortingOrder SortType);

	/* Create OIT data for translucent pass */
	FOITData CreateOITData(FRDGBuilder& GraphBuilder, const FViewInfo& View, EOITPassType PassType);

	/* Compose all OIT samples into the target color buffer */
	void AddOITComposePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FOITData& OITData, FRDGTextureRef SceneColorTexture);
	void AddOITComposePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FOITData& OITData, FRDGTextureRef LightingTexture, FRDGTextureRef ModulateTexture);

	/* Call on SceneRenderer OnRenderBegin */
	void OnRenderBegin(FOITSceneData& OITSceneData);
}
