// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "ShaderParameters.h"
#include "RenderGraphResources.h"

// Uniform buffer used for translucent material shader
BEGIN_SHADER_PARAMETER_STRUCT(FOITBasePassUniformParameters, )
	SHADER_PARAMETER(uint32, bOITEnable)
	SHADER_PARAMETER(uint32, OITMethod)
	SHADER_PARAMETER(uint32, MaxSideSamplePerPixel)
	SHADER_PARAMETER(uint32, MaxSamplePerPixel)
	SHADER_PARAMETER(float, TransmittanceThreshold)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RasterizerOrderedTexture2D<uint>, OutOITSampleCount)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, OutOITSampleData)
END_SHADER_PARAMETER_STRUCT()

class FViewInfo;
struct FOITData;

namespace OIT
{
	void SetOITParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FOITBasePassUniformParameters& OutOIT, const FOITData& InOITData);
}