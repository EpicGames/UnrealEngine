// Copyright Epic Games, Inc. All Rights Reserved.

#include "PathTracing.h"
#include "RHI.h"
#include "HAL/IConsoleManager.h"

TAutoConsoleVariable<int32> CVarPathTracing(
	TEXT("r.PathTracing"),
	1,
	TEXT("Enables the path tracing renderer (to guard the compilation of path tracer specific material permutations)"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

#if RHI_RAYTRACING

#include "CoreMinimal.h"
#include "PathTracingDefinitions.h"
#include "BasePassRendering.h"
#include "CanvasTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/SparseSet.h"
#include "Containers/StringView.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DecalRenderingShared.h"
#include "DeferredShadingRenderer.h"
#include "DistanceFieldLightingShared.h"
#include "DynamicRHI.h"
#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "EnvironmentComponentsFlags.h"
#include "ExtensibleUniformBuffer.h"
#include "FinalPostProcessSettings.h"
#include "FogRendering.h"
#include "GenerateMips.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GenericPlatform/GenericPlatformMemory.h"
#include "GlobalShader.h"
#include "HairStrands/HairStrandsData.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HeterogeneousVolumeInterface.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#include "LightFunctionRendering.h"
#include "LightMapRendering.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "LocalVertexFactory.h"
#include "Logging/LogMacros.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "MeshBatch.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshPassProcessor.h"
#include "MeshPassUtils.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Modules/ModuleManager.h"
#include "MultiGPU.h"
#include "Nanite/NaniteRayTracing.h"
#include "PathTracingDenoiser.h"
#include "PathTracingOutputInvalidateReason.h"
#include "PathTracingResources.h"
#include "PathTracingSpatialTemporalDenoising.h"
#include "PipelineStateCache.h"
#include "PostProcess/DiaphragmDOF.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveUniformShaderParameters.h"
#include "RayTracing/RayTracingDecals.h"
#include "RayTracing/RayTracingInstanceMask.h"
#include "RayTracing/RayTracingLighting.h"
#include "RayTracing/RayTracingMaterialHitShaders.h"
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracing/RayTracingShaderBindingTable.h"
#include "RayTracingMeshDrawCommands.h"
#include "RayTracingPayloadType.h"
#include "RayTracingTypes.h"
#include "ReflectionEnvironment.h"
#include "RenderCommandTag.h"
#include "RendererInterface.h"
#include "LogRenderer.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/SkyAtmosphereCommonData.h"
#include "Rendering/SubstrateMaterialShared.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIContext.h"
#include "RHIResources.h"
#include "RHIShaderParameters.h"
#include "RHIStaticStates.h"
#include "SceneCore.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyAtmosphereSceneProxy.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneRendering.h"
#include "SceneTextures.h"
#include "SceneTypes.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShaderPrintParameters.h"
#include "ShowFlags.h"
#include "SkyAtmosphereRendering.h"
#include "Stats/Stats.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"
#include "VolumetricCloudProxy.h"
#include "VolumetricCloudRendering.h"
#include "SceneViewState.h"
#include <limits>

TAutoConsoleVariable<bool> CVarPathTracingExperimental(
	TEXT("r.PathTracing.Experimental"),
	false,
	TEXT("Enables experimental features or rarely used modes of the path tracing renderer that require compiling additional shader permutations. Enabling this will increase startup time by compiling additional shaders."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);



struct FPathTracingDebugModeInfo
{
	FPathTracingDebugModeInfo()
	{
		CVarHelpText = TEXT("Enables a debug mode for the path tracer which can visualize payload fields and perform basic lighting without sample accumulation.\n")
					   TEXT("Allowed values:\n");
#define REGISTER_VIZ_TYPE(name, value)	Map.Emplace(FName(name), value); CVarHelpText += TEXT(" - ") name TEXT("\n")

		REGISTER_VIZ_TYPE(TEXT("Radiance") 				, PATH_TRACER_DEBUG_VIZ_RADIANCE				);
		REGISTER_VIZ_TYPE(TEXT("WorldNormal") 			, PATH_TRACER_DEBUG_VIZ_WORLD_NORMAL			);
		REGISTER_VIZ_TYPE(TEXT("WorldSmoothNormal") 	, PATH_TRACER_DEBUG_VIZ_WORLD_SMOOTH_NORMAL		);
		REGISTER_VIZ_TYPE(TEXT("WorldGeoNormal") 		, PATH_TRACER_DEBUG_VIZ_WORLD_GEO_NORMAL		);
		REGISTER_VIZ_TYPE(TEXT("BaseColor") 	    	, PATH_TRACER_DEBUG_VIZ_BASE_COLOR				);
		REGISTER_VIZ_TYPE(TEXT("DiffuseColor") 	   		, PATH_TRACER_DEBUG_VIZ_DIFFUSE_COLOR			);
		REGISTER_VIZ_TYPE(TEXT("SpecularColor") 	   	, PATH_TRACER_DEBUG_VIZ_SPECULAR_COLOR			);
		REGISTER_VIZ_TYPE(TEXT("TransparencyColor")		, PATH_TRACER_DEBUG_VIZ_TRANSPARENCY_COLOR		);
		REGISTER_VIZ_TYPE(TEXT("Metallic") 	     		, PATH_TRACER_DEBUG_VIZ_METALLIC				);
		REGISTER_VIZ_TYPE(TEXT("Specular") 	     		, PATH_TRACER_DEBUG_VIZ_SPECULAR				);
		REGISTER_VIZ_TYPE(TEXT("Roughness") 	     	, PATH_TRACER_DEBUG_VIZ_ROUGHNESS				);
		REGISTER_VIZ_TYPE(TEXT("IOR") 	    	 		, PATH_TRACER_DEBUG_VIZ_IOR						);
		REGISTER_VIZ_TYPE(TEXT("ShadingModel")   	 	, PATH_TRACER_DEBUG_VIZ_SHADING_MODEL			);
		REGISTER_VIZ_TYPE(TEXT("LightingChannelMask") 	, PATH_TRACER_DEBUG_VIZ_LIGHTING_CHANNEL_MASK	);
		REGISTER_VIZ_TYPE(TEXT("CustomData0") 			, PATH_TRACER_DEBUG_VIZ_CUSTOM_DATA0 			);
		REGISTER_VIZ_TYPE(TEXT("CustomData1") 			, PATH_TRACER_DEBUG_VIZ_CUSTOM_DATA1 			);
		REGISTER_VIZ_TYPE(TEXT("WorldPosition") 		, PATH_TRACER_DEBUG_VIZ_WORLD_POSITION 			);
		REGISTER_VIZ_TYPE(TEXT("PrimaryRays") 			, PATH_TRACER_DEBUG_VIZ_PRIMARY_RAYS 			);
		REGISTER_VIZ_TYPE(TEXT("WorldTangent") 			, PATH_TRACER_DEBUG_VIZ_WORLD_TANGENT 			);
		REGISTER_VIZ_TYPE(TEXT("Anisotropy") 	    	, PATH_TRACER_DEBUG_VIZ_ANISOTROPY 				);
		REGISTER_VIZ_TYPE(TEXT("LightGridCount") 	   	, PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_COUNT		);
		REGISTER_VIZ_TYPE(TEXT("LightGridAxis") 	   	, PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_AXIS			);
        REGISTER_VIZ_TYPE(TEXT("LightGridListHash")	   	, PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_LIST_HASH    );
        REGISTER_VIZ_TYPE(TEXT("LightGridSingular")     , PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_SINGULAR     );
		REGISTER_VIZ_TYPE(TEXT("DecalGridCount")	   	, PATH_TRACER_DEBUG_VIZ_DECAL_GRID_COUNT		);
		REGISTER_VIZ_TYPE(TEXT("DecalGridAxis") 	   	, PATH_TRACER_DEBUG_VIZ_DECAL_GRID_AXIS			);
        REGISTER_VIZ_TYPE(TEXT("DecalGridListHash")     , PATH_TRACER_DEBUG_VIZ_DECAL_GRID_LIST_HASH    );
		REGISTER_VIZ_TYPE(TEXT("VolumeLightCount") 		, PATH_TRACER_DEBUG_VIZ_VOLUME_LIGHT_COUNT		);
		REGISTER_VIZ_TYPE(TEXT("HitKind")            	, PATH_TRACER_DEBUG_VIZ_HITKIND             	);
		REGISTER_VIZ_TYPE(TEXT("TransparencyCount")     , PATH_TRACER_DEBUG_VIZ_TRANSPARENCY_COUNT		);
		REGISTER_VIZ_TYPE(TEXT("SSSColor")     			, PATH_TRACER_DEBUG_VIZ_SSS_COLOR				);
		REGISTER_VIZ_TYPE(TEXT("SSSRadius")   			, PATH_TRACER_DEBUG_VIZ_SSS_RADIUS				);
		REGISTER_VIZ_TYPE(TEXT("SSSWeight")   			, PATH_TRACER_DEBUG_VIZ_SSS_WEIGHT				);
		REGISTER_VIZ_TYPE(TEXT("SSSPhase")   			, PATH_TRACER_DEBUG_VIZ_SSS_PHASE				);
		REGISTER_VIZ_TYPE(TEXT("FuzzColor")   			, PATH_TRACER_DEBUG_VIZ_FUZZ_COLOR				);
		REGISTER_VIZ_TYPE(TEXT("FuzzRoughness") 		, PATH_TRACER_DEBUG_VIZ_FUZZ_ROUGHNESS			);
		REGISTER_VIZ_TYPE(TEXT("FuzzAmount")   			, PATH_TRACER_DEBUG_VIZ_FUZZ_AMOUNT				);
		REGISTER_VIZ_TYPE(TEXT("BSDFOpacity")   		, PATH_TRACER_DEBUG_VIZ_BSDF_OPACITY			);
		REGISTER_VIZ_TYPE(TEXT("SubstrateWeightV") 		, PATH_TRACER_DEBUG_VIZ_SUBSTRATE_WEIGHT_V		);
		REGISTER_VIZ_TYPE(TEXT("SubstrateTransmittanceN"), PATH_TRACER_DEBUG_VIZ_SUBSTRATE_TRANSMITTANCE_N);
		REGISTER_VIZ_TYPE(TEXT("SubstrateCoverageAboveN"), PATH_TRACER_DEBUG_VIZ_SUBSTRATE_COVERAGE_ABOVE_N);
		REGISTER_VIZ_TYPE(TEXT("SubstrateF90")           , PATH_TRACER_DEBUG_VIZ_SUBSTRATE_F90			);
		REGISTER_VIZ_TYPE(TEXT("EyeIrisMask")           , PATH_TRACER_DEBUG_VIZ_EYE_IRIS_MASK           );
		REGISTER_VIZ_TYPE(TEXT("EyeIrisNormal")         , PATH_TRACER_DEBUG_VIZ_EYE_IRIS_NORMAL         );
		REGISTER_VIZ_TYPE(TEXT("EyeCausticNormal")      , PATH_TRACER_DEBUG_VIZ_EYE_CAUSTIC_NORMAL      );
		REGISTER_VIZ_TYPE(TEXT("BackfacingNormal")      , PATH_TRACER_DEBUG_VIZ_BACKFACING_NORMAL       );
		REGISTER_VIZ_TYPE(TEXT("HairPrimitiveUV")		, PATH_TRACER_DEBUG_VIZ_HAIR_PRIMITIVE_UV		);
        REGISTER_VIZ_TYPE(TEXT("TriangleID")            , PATH_TRACER_DEBUG_VIZ_TRIANGLE_ID             );
        REGISTER_VIZ_TYPE(TEXT("InstanceID")            , PATH_TRACER_DEBUG_VIZ_INSTANCE_ID             );
        REGISTER_VIZ_TYPE(TEXT("ClusterID")             , PATH_TRACER_DEBUG_VIZ_CLUSTER_ID              );
        REGISTER_VIZ_TYPE(TEXT("DecalCoverage")         , PATH_TRACER_DEBUG_VIZ_DECAL_COVERAGE          );
        REGISTER_VIZ_TYPE(TEXT("DecalColor")            , PATH_TRACER_DEBUG_VIZ_DECAL_COLOR             );
        REGISTER_VIZ_TYPE(TEXT("DecalNormal")           , PATH_TRACER_DEBUG_VIZ_DECAL_NORMAL            );
        REGISTER_VIZ_TYPE(TEXT("DecalRoughMetalSpec")   , PATH_TRACER_DEBUG_VIZ_DECAL_RMS               );
        REGISTER_VIZ_TYPE(TEXT("DecalRadiance")         , PATH_TRACER_DEBUG_VIZ_DECAL_RADIANCE          );
        REGISTER_VIZ_TYPE(TEXT("DecalResponseMask")     , PATH_TRACER_DEBUG_VIZ_DECAL_RESPONSE_MASK     );
        REGISTER_VIZ_TYPE(TEXT("DecalUseDBuffer")       , PATH_TRACER_DEBUG_VIZ_DECAL_USE_DBUFFER       );

#undef REGISTER_VIZ_TYPE
	}

	static FPathTracingDebugModeInfo& Get()
	{
		static FPathTracingDebugModeInfo Singleton;
		return Singleton;
	}

	TMap<FName, uint32> Map;
	FString CVarHelpText;
};


static FName GPathTracingVisualizeMode = NAME_None;
FAutoConsoleVariableRef CVarPathTracingVisualize(
	TEXT("r.PathTracing.Visualize"),
	GPathTracingVisualizeMode,
	*FPathTracingDebugModeInfo::Get().CVarHelpText,
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingVisualizeIncludeTranslucent(
	TEXT("r.PathTracing.Visualize.IncludeTranslucent"),
	true,
	TEXT("Indicates if translucent materials should be included in selected visualization mode."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingVisualizeSamplesPerPixel(
	TEXT("r.PathTracing.Visualize.SamplesPerPixel"),
	1,
	TEXT("Number of samples for primary rays visualization mode"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingVisualizeAmbientOcclusionMaxDistance(
	TEXT("r.PathTracing.Visualize.AmbientOcclusionMaxDistance"),
	200.0f,
	TEXT("How far to trace ambient occlusion rays for fake shading effect? This only applies to some of the visualization modes. Negative values revert to a simple facing ratio shading."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingVisualizeMaxCount(
	TEXT("r.PathTracing.Visualize.MaxCount"),
	16,
	TEXT("Number of lights (or decals) to use for the visualization modes involving turning a count into a color. The color ramp is blue to red, with pink indicating a cell beyond the target count. "),
	ECVF_RenderThreadSafe
);


TAutoConsoleVariable<bool> CVarPathTracingVisualizeIndirectGuiding(
	TEXT("r.PathTracing.Visualize.IndirectGuiding"),
	false,
	TEXT("Should indirect rays use path guiding to improve quality?"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingVisualizeIndirectGuidingHashSize(
	TEXT("r.PathTracing.Visualize.IndirectGuiding.HashSize"),
	1024 * 1024,
	TEXT("Size of the hash grid to use for storing path guiding information (automatically rounded up to nearest power of 2)"),
	ECVF_RenderThreadSafe
);


TAutoConsoleVariable<int32> CVarPathTracingCompactionDepth(
	TEXT("r.PathTracing.CompactionDepth"),
	6,
	TEXT("Enables path compaction to improve GPU occupancy for the path tracer. The value sets the bounce up to which compaction will happen, beyond that point the path tracer will handle bounces within the dispatch. Values <= 0 apply compaction to all bounces."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingShaderExecutionReordering(
	TEXT("r.PathTracing.ShaderExecutionReordering"),
	true,
	TEXT("Enables Shader Execution Reordering to improve shader coherence for the path tracer. This variable only has effect if the current hardware supports it."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingIndirectDispatch(
	TEXT("r.PathTracing.IndirectDispatch"),
	false,
	TEXT("Enables indirect dispatch (if supported by the hardware) for compacted path tracing."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingFlushDispatch(
	TEXT("r.PathTracing.FlushDispatch"),
	2,
	TEXT("Enables flushing of the command list after dispatch to reduce the likelyhood of TDRs on Windows\n")
	TEXT("0: off\n")
	TEXT("1: flush after each dispatch\n")
	TEXT("2: flush after each tile\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingDispatchSize(
	TEXT("r.PathTracing.DispatchSize"),
	2048,
	TEXT("Controls the tile size used when rendering the image. Reducing this value may prevent GPU timeouts for heavy renders."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingMaxBounces(
	TEXT("r.PathTracing.MaxBounces"),
	-1,
	TEXT("Sets the maximum number of path tracing bounces (default = -1 (driven by postprocesing volume))"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingSamplesPerPixel(
	TEXT("r.PathTracing.SamplesPerPixel"),
	-1,
	TEXT("Sets the maximum number of samples per pixel (default = -1 (driven by postprocesing volume))"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingFilterWidth(
	TEXT("r.PathTracing.FilterWidth"),
	3.0,
	TEXT("Sets the anti-aliasing filter width (default = 3.0 which corresponds to a gaussian with standard deviation of a 1/2 pixel)"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingMISMode(
	TEXT("r.PathTracing.MISMode"),
	2,
	TEXT("Selects the sampling technique for light integration.\n")
	TEXT("0: Material sampling\n")
	TEXT("1: Light sampling\n")
	TEXT("2: MIS betwen material and light sampling (default)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingVolumeMISMode(
	TEXT("r.PathTracing.VolumeMISMode"),
	1,
	TEXT("Selects the sampling technique for volumetric integration of local lighting\n")
	TEXT("0: Density sampling\n")
	TEXT("1: Light sampling (default)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingMaxRaymarchSteps(
	TEXT("r.PathTracing.MaxRaymarchSteps"),
	768,
	TEXT("Upper limit on the number of ray marching steps in volumes. This limit should not be hit in most cases, but raising it can reduce bias in case it is."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingMISCompensation(
	TEXT("r.PathTracing.MISCompensation"),
	1,
	TEXT("Activates MIS compensation for skylight importance sampling.\n")
	TEXT("This option only takes effect when r.PathTracing.MISMode = 2\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingSkylightCaching(
	TEXT("r.PathTracing.SkylightCaching"),
	1,
	TEXT("Attempts to re-use skylight data between frames.\n")
	TEXT("When set to 0, the skylight texture and importance samping data will be regenerated every frame. This is mainly intended as a benchmarking and debugging aid\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingVisibleLights(
	TEXT("r.PathTracing.VisibleLights"),
	0,
	TEXT("Should light sources be visible to camera rays?\n")
	TEXT("0: Hide lights from camera rays (default)\n")
	TEXT("1: Make all lights visible to camera\n")
	TEXT("2: Make skydome only visible to camera\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingMaxSSSBounces(
	TEXT("r.PathTracing.MaxSSSBounces"),
	256,
	TEXT("Sets the maximum number of bounces inside subsurface materials. Lowering this value can make subsurface scattering render too dim, while setting it too high can cause long render times."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingSSSGuidingRatio(
	TEXT("r.PathTracing.SSSGuidingRatio"),
	0.5f,
	TEXT("Sets the ratio between classical random walks and walks guided towards the surface. A value of 0.0 corresponds to a purely classical random walk, while a value of 1.0 is fully guided towards the surface (at the expense of fireflies in non-flat regions of the model."),
	ECVF_RenderThreadSafe
);


TAutoConsoleVariable<float> CVarPathTracingMaxPathIntensity(
	TEXT("r.PathTracing.MaxPathIntensity"),
	-1,
	TEXT("When positive, light paths greater that this amount are clamped to prevent fireflies (default = -1 (driven by postprocesing volume))"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingApproximateCaustics(
	TEXT("r.PathTracing.ApproximateCaustics"),
	true,
	TEXT("When true, the path tracer will approximate caustic paths to reduce noise. This reduces speckles and noise from low-roughness glass and metals."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingEnableEmissive(
	TEXT("r.PathTracing.EnableEmissive"),
	-1,
	TEXT("Indicates if emissive materials should contribute to scene lighting (default = -1 (driven by postprocesing volume)"),
	ECVF_RenderThreadSafe
);

static int32 GEnableReferenceDOF = -1;
FAutoConsoleVariableRef CVarPathTracingEnableReferenceDOF(
	TEXT("r.PathTracing.EnableReferenceDOF"),
	GEnableReferenceDOF,
	TEXT("Should the path tracer ray trace the depth-of-field effect instead of the post-processed effect?\n")
	TEXT("-1: Inherit from PostProcess settings (default)\n")
	TEXT(" 0: Disabled\n")
	TEXT(" 1: Enabled\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingEnableReferenceAtmosphere(
	TEXT("r.PathTracing.EnableReferenceAtmosphere"),
	-1,
	TEXT("Should the path tracer use a volumetric calculation to represent the sky atmosphere?\n")
	TEXT("-1: Inherit from PostProcess settings (default)\n")
	TEXT(" 0: Disabled\n")
	TEXT(" 1: Enabled\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingEnableReferenceClouds(
	TEXT("r.PathTracing.EnableReferenceClouds"),
	1,
	TEXT("Should the path tracer use a volumetric calculation to represent volumetric clouds? (This requires Reference Atmosphere to be enabled)\n")
	TEXT(" 0: Disabled\n")
	TEXT(" 1: Enabled (default)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingAtmosphereOpticalDepthLutResolution(
	TEXT("r.PathTracing.AtmosphereOpticalDepthLUTResolution"),
	512,
	TEXT("Size of the square lookup texture used for transmittance calculations by the path tracer in reference atmosphere mode."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingAtmosphereOpticalDepthLutNumSamples(
	TEXT("r.PathTracing.AtmosphereOpticalDepthLUTNumSamples"),
	16384,
	TEXT("Number of ray marching samples used when building the transmittance lookup texture used for transmittance calculations by the path tracer in reference atmosphere mode."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingEnableAtmosphereGround(
	TEXT("r.PathTracing.EnableAtmosphereGround"),
	false,
	TEXT("Should the planet ground surface of the atmosphere model be visible by the path tracer?\n")
	TEXT("The planet ground is always visible to volume bounces to influence the color of the atmosphere\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingCloudAccelerationMapResolution(
	TEXT("r.PathTracing.CloudAccelerationMap.Resolution"),
	512,
	TEXT("Size of the square texture used to accelerate cloud ray marching for the path tracer in reference atmosphere mode."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingCloudAccelerationMapNumSamples(
	TEXT("r.PathTracing.CloudAccelerationMap.NumSamples"),
	64,
	TEXT("Number of ray marching samples used when building the cloud acceleration map for the path tracer in reference atmosphere mode."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingCloudAccelerationMapVisualize(
	TEXT("r.PathTracing.CloudAccelerationMap.Visualize"),
	false,
	TEXT("If true, replace clouds with a visualization of the acceleration map to help visualize it and fine tune its resolution."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int> CVarPathTracingCloudMultipleScatterMode(
	TEXT("r.PathTracing.CloudMultipleScatterMode"),
	1,
	TEXT("Selects the multiple scattering mode for rendering of volumetric clouds in the path tracer.\n")
	TEXT("  0: None      - multiple scattering settings inside the material are ignored, CloudRoughnessCutoff is applied\n")
	TEXT("  1: Approx    - multiple scattering settings inside the material are used, CloudRoughnessCutoff is applied (default)\n")
	TEXT("  2: Reference - multiple scattering settings inside the material are ignored, CloudRoughnessCutoff is not applied\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingCloudRoughnessCutoff(
	TEXT("r.PathTracing.CloudRoughnessCutoff"),
	0.05f,
	TEXT("Do not evaluate volumetric clouds beyond this roughness level to improve performance."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingFogDensityClamp(
	TEXT("r.PathTracing.FogDensityClamp"),
	8.0f,
	TEXT("Limit the density growth in exponential heightfog (default = 8)\n")
	TEXT("Instead of allowing the exponential density to increase to infinity vertically, clamp it to some multiplier of the overall density."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingFrameIndependentTemporalSeed(
	TEXT("r.PathTracing.FrameIndependentTemporalSeed"),
	1,
	TEXT("Indicates to use different temporal seed for each sample across frames rather than resetting the sequence at the start of each frame\n")
	TEXT("0: off\n")
	TEXT("1: on (default)\n"),
	ECVF_RenderThreadSafe
);

// See PATHTRACER_SAMPLER_* defines
TAutoConsoleVariable<int32> CVarPathTracingSamplerType(
	TEXT("r.PathTracing.SamplerType"),
	PATHTRACER_SAMPLER_DEFAULT,
	TEXT("Controls the way the path tracer generates its random numbers\n")
	TEXT("0: use a different high quality random sequence per pixel (default)\n")
	TEXT("1: optimize the random sequence across pixels to reduce visible error at the target sample count\n"),
	ECVF_RenderThreadSafe
);

#if WITH_MGPU
TAutoConsoleVariable<bool> CVarPathTracingMultiGPU(
	TEXT("r.PathTracing.MultiGPU"),
	false,
	TEXT("Run the path tracer using all available GPUs when enabled\n")
	TEXT("Using this functionality in the editor requires -MaxGPUCount=N setting on the command line."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingAdjustMultiGPUPasses(
	TEXT("r.PathTracing.AdjustMultiGPUPasses"),
	true,
	TEXT("Run extra passes per frame when multiple GPUs are active, to improve perf scaling as GPUs are added."),
	ECVF_RenderThreadSafe
);
#endif  // WITH_MGPU

TAutoConsoleVariable<bool> CVarPathTracingWiperMode(
	TEXT("r.PathTracing.WiperMode"),
	false,
	TEXT("Enables wiper mode to render using the path tracer only in a region of the screen for debugging purposes."),
	ECVF_RenderThreadSafe 
);

TAutoConsoleVariable<bool> CVarPathTracingProgressDisplay(
	TEXT("r.PathTracing.ProgressDisplay"),
	true,
	TEXT("Enables an in-frame display of progress towards the defined sample per pixel limit. The indicator dissapears when the maximum is reached and sample accumulation has stopped\n")
	TEXT(" false: off\n")
	TEXT(" true : on (default)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingLightGridResolution(
	TEXT("r.PathTracing.LightGridResolution"),
	256,
	TEXT("Controls the resolution of the 2D light grid used to cull irrelevant lights from lighting calculations."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingLightGridMaxCount(
	TEXT("r.PathTracing.LightGridMaxCount"),
	128,
	TEXT("Controls the maximum number of lights per cell in the 2D light grid. The minimum of this value and the number of lights in the scene is used."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingLightGridAxis(
	TEXT("r.PathTracing.LightGridAxis"),
	-1,
	TEXT("Choose the coordinate axis along which to project the light grid (default = -1, automatic)"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingUseDBuffer(
	TEXT("r.PathTracing.UseDBuffer"),
	true,
	TEXT("When enabled, materials are allowed to read from d-buffer nodes. This carries a cost as the primary ray must be traced again to re-evaluate the material after the decal information has been gathered."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingDecalRoughnessCutoff(
	TEXT("r.PathTracing.DecalRoughnessCutoff"),
	0.15f,
	TEXT("Do not evaluate decals beyond this roughness level to improve performance."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingMeshDecalRoughnessCutoff(
	TEXT("r.PathTracing.MeshDecalRoughnessCutoff"),
	0.15f,
	TEXT("Do not evaluate mesh decals beyond this roughness level to improve performance."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingMeshDecalBias(
	TEXT("r.PathTracing.MeshDecalBias"),
	1.0f,
	TEXT("Bias applied to mesh decal rays to avoid intersection with geometry."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingLightFunctionColor(
	TEXT("r.PathTracing.LightFunctionColor"),
	true,
	TEXT("Enables colored light function output\n")
	TEXT("0: off (light function material output is converted to grayscale)\n")
	TEXT("1: on (light function material output is used directly)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingHeterogeneousVolumesRebuildEveryFrame(
	TEXT("r.PathTracing.HeterogeneousVolumes.RebuildEveryFrame"),
	true,
	TEXT("Rebuilds volumetric acceleration structures every frame."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingHeterogeneousVolumesPipeline(
	TEXT("r.PathTracing.HeterogeneousVolumes.Pipeline"),
	0,
	TEXT("Determines which pipeline to invoke when rendering Heterogeneous Volumes. The first method resamples the volume into a global voxel buffer. The second method isolates each volume, and invokes an individual callable shader to perform rendering options.\n")
	TEXT("0: Global voxel buffers (default)\n")
	TEXT("1: Callable shading\n")
	TEXT("Requires r.PathTracing.Experimental=true to modify.\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingCameraMediumTracking(
	TEXT("r.PathTracing.CameraMediumTracking"),
	true,
	TEXT("Enables automatic camera medium tracking to detect when a camera starts inside water or solid glass automatically\n")
	TEXT(" false: off\n")
	TEXT(" true : on (default)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingOutputPostProcessResources(
	TEXT("r.PathTracing.OutputPostProcessResources"),
	true,
	TEXT("Output the pathtracing resources to the postprocess passes\n")
	TEXT(" false: off\n")
	TEXT(" true : on (Buffers including, raw/denoised radiance, albedo, normal, and variance)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingSubstrateUseSimplifiedMaterial(
	TEXT("r.PathTracing.Substrate.UseSimplifiedMaterials"),
	false,
	TEXT("Instead of evaluating all layers, use an optimized material in which all slabs have been merged.\n")
	TEXT(" false: off (default)\n")
	TEXT(" true : on\n")
	TEXT("Requires r.PathTracing.Substrate.CompileSimplifiedMaterials=true to be set.\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<bool> CVarPathTracingSubstrateCompileSimplifiedMaterial(
	TEXT("r.PathTracing.Substrate.CompileSimplifiedMaterials"),
	false,
	TEXT("Compile a simplified representation of Substrate materials which merges all slabs into one. This is mainly intended for debugging purposes. Enabling this double the number of path tracing shader permutations.\n")
	TEXT(" false: off (default)\n")
	TEXT(" true : on\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

TAutoConsoleVariable<bool> CVarPathTracingUseAnalyticTransmittance(
	TEXT("r.PathTracing.UseAnalyticTransmittance"),
	true,
	TEXT("Determines use of analytical or null-tracking estimation when evaluating transmittance\n")
	TEXT(" false: off (uses null-tracking estimation)\n")
	TEXT(" true : on (uses analytical estimation when possible) (default)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingAdaptiveSampling(
	TEXT("r.PathTracing.AdaptiveSampling"),
	0,
	TEXT("Determines if adaptive sampling is enabled. When non-zero, the path tracer will try to skip calculation of pixels below the specified error threshold.\n")
	TEXT("0: off (uniform sampling - default)\n")
	TEXT("1: on (adaptive sampling)\n")
	TEXT("Requires r.PathTracing.Experimental=true to modify.\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingAdaptiveSamplingErrorThreshold(
	TEXT("r.PathTracing.AdaptiveSampling.ErrorThreshold"),
	0.001f,
	TEXT("This is the target perceptual error threshold. Once a pixel's error falls below this value, it will not be sampled again.\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingAdaptiveSamplingVisualize(
	TEXT("r.PathTracing.AdaptiveSampling.Visualize"),
	0,
	TEXT("Select a visualization mode to help understand how adaptive sampling is working.\n")
	TEXT("0: off\n")
	TEXT("1: Visualize active pixels with heatmap (converged pixels are displayed as is)\n")
	TEXT("2: Visualize sample count heatmap (against current max samples)\n")
	TEXT("3-7: Visualize variance mip levels\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<float> CVarPathTracingBackgroundAlpha(
	TEXT("r.PathTracing.BackgroundAlpha"),
	0.0f,
	TEXT("Value of the alpha channel for pixels that do hit anything (default 0.0)\n")
	TEXT("Note that this refers to the normal interpretation of alpha which the path tracer uses internally, so 0 corresponds to a transparent pixel while 1 refers to a solid pixel."),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarPathTracingDebug(
	TEXT("r.PathTracing.Debug"),
	0,
	TEXT("Enable debug rendering for path tracer. Used for only development and needs to be enabled before starting the engine.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

TAutoConsoleVariable<bool> CVarPathTracingInvalidateOnMaterialParameterChange(
	TEXT("r.PathTracing.InvalidateOnMaterialParameterChange"),
	true,
	TEXT("Whether to invalidate path tracer output and restart accumulaion when a material parameter changes."),
	ECVF_RenderThreadSafe
);

bool PathTracing::IsOutputInvalidateAllowed(EInvalidateReason InvalidateReason)
{
	if (InvalidateReason == EInvalidateReason::UpdateMaterialParameter)
	{
		return CVarPathTracingInvalidateOnMaterialParameterChange.GetValueOnAnyThread();
	}

	return true;
}

// Placeholder for experimenting with newer versions of external denoisers
RENDERER_API FPathTracingRealtimeDenoiserResources GPathTracingRealtimeDenoiserResources;

BEGIN_SHADER_PARAMETER_STRUCT(FPathTracingData, )
	SHADER_PARAMETER(float, BlendFactor)
	SHADER_PARAMETER(float, BaseExposure)
	SHADER_PARAMETER(uint32, Iteration)
	SHADER_PARAMETER(uint32, TemporalSeed)
	SHADER_PARAMETER(uint32, MaxSamples)
	SHADER_PARAMETER(uint32, MaxBounces)
	SHADER_PARAMETER(uint32, MaxSSSBounces)
	SHADER_PARAMETER(float, BackgroundAlpha)
	SHADER_PARAMETER(float , SSSGuidingRatio)
	SHADER_PARAMETER(uint32, MISMode)
	SHADER_PARAMETER(uint32, VolumeMISMode)
	SHADER_PARAMETER(float, ApproximateCaustics)
	SHADER_PARAMETER(uint32, SamplerType)
	SHADER_PARAMETER(uint32, EnableDBuffer)
	SHADER_PARAMETER(uint32, VolumeFlags)
	SHADER_PARAMETER(uint32, EnabledDirectLightingContributions)   // PATHTRACER_CONTRIBUTION_*
	SHADER_PARAMETER(uint32, EnabledIndirectLightingContributions) // PATHTRACER_CONTRIBUTION_*
	SHADER_PARAMETER(uint32, ApplyDiffuseSpecularOverrides)
	SHADER_PARAMETER(int32, MaxRaymarchSteps)
	SHADER_PARAMETER(float, MaxPathIntensity)
	SHADER_PARAMETER(float, MaxNormalBias)
	SHADER_PARAMETER(float, FilterWidth)
	SHADER_PARAMETER(float, CloudRoughnessCutoff)
	SHADER_PARAMETER(float, DecalRoughnessCutoff)
	SHADER_PARAMETER(float, MeshDecalRoughnessCutoff)
	SHADER_PARAMETER(float, MeshDecalBias)
	SHADER_PARAMETER(float, CameraFocusDistance)
	SHADER_PARAMETER(FVector2f, CameraLensRadius)
	SHADER_PARAMETER(float, Petzval)
	SHADER_PARAMETER(float, PetzvalFalloffPower)
	SHADER_PARAMETER(FVector2f, PetzvalExclusionBoxExtents)
	SHADER_PARAMETER(float, PetzvalExclusionBoxRadius)
END_SHADER_PARAMETER_STRUCT()


// Store the rendering options used on the previous frame so we can correctly invalidate when things change
struct FPathTracingConfig
{
	FPathTracingData PathTracingData;
	FIntRect ViewRect;
	int LightShowFlags;
	int LightGridResolution;
	int LightGridMaxCount;
	bool VisibleLights;
	bool UseMISCompensation;
	bool LockedSamplingPattern;
	bool UseCameraMediumTracking;
	bool UseAdaptiveSampling;
	bool UseMultiGPU; // NOTE: Requires invalidation because the buffer layout changes
	int DenoiserMode; // NOTE: does not require path tracing invalidation
	float AdaptiveSamplingThreshold;
	int CloudAccelerationMapNumSamples;
	int CloudAccelerationMapResolution;
	bool CloudAccelerationMapVisualize;
	int CloudMultipleScatterMode;

	bool IsDifferent(const FPathTracingConfig& Other) const
	{
		// If any of these parameters if different, we will need to restart path tracing accuulation
		return
			PathTracingData.MaxSamples != Other.PathTracingData.MaxSamples ||
			PathTracingData.MaxBounces != Other.PathTracingData.MaxBounces ||
			PathTracingData.BackgroundAlpha != Other.PathTracingData.BackgroundAlpha ||
			PathTracingData.MaxSSSBounces != Other.PathTracingData.MaxSSSBounces ||
			PathTracingData.SSSGuidingRatio != Other.PathTracingData.SSSGuidingRatio ||
			PathTracingData.MISMode != Other.PathTracingData.MISMode ||
			PathTracingData.VolumeMISMode != Other.PathTracingData.VolumeMISMode ||
			PathTracingData.SamplerType != Other.PathTracingData.SamplerType ||
			PathTracingData.ApproximateCaustics != Other.PathTracingData.ApproximateCaustics ||
			PathTracingData.EnableDBuffer != Other.PathTracingData.EnableDBuffer ||
			PathTracingData.MaxPathIntensity != Other.PathTracingData.MaxPathIntensity ||
			PathTracingData.FilterWidth != Other.PathTracingData.FilterWidth ||
			PathTracingData.VolumeFlags != Other.PathTracingData.VolumeFlags ||
			PathTracingData.ApplyDiffuseSpecularOverrides != Other.PathTracingData.ApplyDiffuseSpecularOverrides ||
			PathTracingData.EnabledDirectLightingContributions != Other.PathTracingData.EnabledDirectLightingContributions ||
			PathTracingData.EnabledIndirectLightingContributions != Other.PathTracingData.EnabledIndirectLightingContributions ||
			PathTracingData.CloudRoughnessCutoff != Other.PathTracingData.CloudRoughnessCutoff ||
			PathTracingData.DecalRoughnessCutoff != Other.PathTracingData.DecalRoughnessCutoff ||
			PathTracingData.MeshDecalRoughnessCutoff != Other.PathTracingData.MeshDecalRoughnessCutoff ||
			PathTracingData.MeshDecalBias != Other.PathTracingData.MeshDecalBias ||
			PathTracingData.MaxRaymarchSteps != Other.PathTracingData.MaxRaymarchSteps ||
			ViewRect != Other.ViewRect ||
			LightShowFlags != Other.LightShowFlags ||
			LightGridResolution != Other.LightGridResolution ||
			LightGridMaxCount != Other.LightGridMaxCount ||
			VisibleLights != Other.VisibleLights ||
			UseMISCompensation != Other.UseMISCompensation ||
			LockedSamplingPattern != Other.LockedSamplingPattern ||
			UseCameraMediumTracking != Other.UseCameraMediumTracking ||
			UseAdaptiveSampling != Other.UseAdaptiveSampling ||
			AdaptiveSamplingThreshold != Other.AdaptiveSamplingThreshold ||
			CloudAccelerationMapNumSamples != Other.CloudAccelerationMapNumSamples ||
			CloudAccelerationMapResolution != Other.CloudAccelerationMapResolution ||
			CloudAccelerationMapVisualize != Other.CloudAccelerationMapVisualize ||
			CloudMultipleScatterMode != Other.CloudMultipleScatterMode ||
			UseMultiGPU != Other.UseMultiGPU;
	}

	bool IsExposureDifferentEnough(const FPathTracingConfig& Other) const
	{
		const float ExposureA = PathTracingData.BaseExposure;
		const float ExposureB = Other.PathTracingData.BaseExposure;
		return FMath::Max(ExposureA, ExposureB) > 16.0f * FMath::Min(ExposureA, ExposureB);
	}

	bool IsDOFDifferent(const FPathTracingConfig& Other) const
	{
		return PathTracingData.CameraFocusDistance != Other.PathTracingData.CameraFocusDistance ||
			   PathTracingData.CameraLensRadius != Other.PathTracingData.CameraLensRadius ||
			   PathTracingData.Petzval != Other.PathTracingData.Petzval ||
			   PathTracingData.PetzvalFalloffPower != Other.PathTracingData.PetzvalFalloffPower ||
			   PathTracingData.PetzvalExclusionBoxExtents != Other.PathTracingData.PetzvalExclusionBoxExtents ||
			   PathTracingData.PetzvalExclusionBoxRadius != Other.PathTracingData.PetzvalExclusionBoxRadius;
	}
};

struct FAtmosphereConfig
{
	FAtmosphereConfig() = default;
	FAtmosphereConfig(const FAtmosphereUniformShaderParameters& Parameters) :
		AtmoParameters(Parameters),
		NumSamples(CVarPathTracingAtmosphereOpticalDepthLutNumSamples.GetValueOnRenderThread()),
		Resolution(CVarPathTracingAtmosphereOpticalDepthLutResolution.GetValueOnRenderThread()) {}

	bool IsDifferent(const FAtmosphereConfig& Other) const
	{
		// Compare only those parameters which impact the LUT construction
		return
			AtmoParameters.BottomRadiusKm != Other.AtmoParameters.BottomRadiusKm ||
			AtmoParameters.TopRadiusKm != Other.AtmoParameters.TopRadiusKm ||
			AtmoParameters.RayleighDensityExpScale != Other.AtmoParameters.RayleighDensityExpScale ||
			AtmoParameters.RayleighScattering != Other.AtmoParameters.RayleighScattering ||
			AtmoParameters.MieScattering != Other.AtmoParameters.MieScattering ||
			AtmoParameters.MieDensityExpScale != Other.AtmoParameters.MieDensityExpScale ||
			AtmoParameters.MieExtinction != Other.AtmoParameters.MieExtinction ||
			AtmoParameters.MieAbsorption != Other.AtmoParameters.MieAbsorption ||
			AtmoParameters.AbsorptionDensity0LayerWidth != Other.AtmoParameters.AbsorptionDensity0LayerWidth ||
			AtmoParameters.AbsorptionDensity0ConstantTerm != Other.AtmoParameters.AbsorptionDensity0ConstantTerm ||
			AtmoParameters.AbsorptionDensity0LinearTerm != Other.AtmoParameters.AbsorptionDensity0LinearTerm ||
			AtmoParameters.AbsorptionDensity1ConstantTerm != Other.AtmoParameters.AbsorptionDensity1ConstantTerm ||
			AtmoParameters.AbsorptionDensity1LinearTerm != Other.AtmoParameters.AbsorptionDensity1LinearTerm ||
			AtmoParameters.AbsorptionExtinction != Other.AtmoParameters.AbsorptionExtinction ||
			NumSamples != Other.NumSamples ||
			Resolution != Other.Resolution;
	}

	// hold a copy of the parameters that influence LUT construction so we can detect when they change
	FAtmosphereUniformShaderParameters AtmoParameters;

	// parameters for the LUT itself
	uint32 NumSamples;
	uint32 Resolution;
};

struct FPathTracingState
{
	FPathTracingConfig LastConfig;
	// Textures holding onto the accumulated frame data
	TRefCountPtr<IPooledRenderTarget> RadianceRT;
	TRefCountPtr<IPooledRenderTarget> VarianceRT;
	TRefCountPtr<IPooledRenderTarget> AlbedoRT;
	TRefCountPtr<IPooledRenderTarget> NormalRT;
	TRefCountPtr<IPooledRenderTarget> DepthRT;
	TRefCountPtr<FRDGPooledBuffer> VarianceBuffer;
	TRefCountPtr<IPooledRenderTarget> CloudAccelerationMap;
	TRefCountPtr<FRDGPooledBuffer> MarkovChainStateGrid;

	// Cache to improve the stability when frame denoising (SPP=r.pathtracing.SamplesPerPixel) is used in animation rendering
	TRefCountPtr<IPooledRenderTarget> LastDenoisedRadianceRT;
	TRefCountPtr<IPooledRenderTarget> LastRadianceRT;
	TRefCountPtr<IPooledRenderTarget> LastAlbedoRT;
	TRefCountPtr<IPooledRenderTarget> LastNormalRT;
	TRefCountPtr<IPooledRenderTarget> LastDepthRT;
	TRefCountPtr<FRDGPooledBuffer> LastVarianceBuffer;

	// Volume acceleration structures
	FAdaptiveOrthoGridParameterCache AdaptiveOrthoGridParameterCache;
	FAdaptiveFrustumGridParameterCache AdaptiveFrustumGridParameterCache;

	TMap<FHeterogeneousVolumesPrimitiveInstanceKey, TRefCountPtr<IPooledRenderTarget> > HeterogeneousVolumesPersistentMajorantRT;
	TArray<FPathTracingHeterogeneousVolumesCallableInfo> HeterogeneousVolumesCallableInfo;

	// Texture holding onto the precomputed atmosphere data
	TRefCountPtr<IPooledRenderTarget> AtmosphereOpticalDepthLUT;
	FAtmosphereConfig LastAtmosphereConfig;

	// Buffer containing the starting medium coefficients
	TRefCountPtr<FRDGPooledBuffer>    StartingMediumData;

	// Custom path tracing spacial temporal denoiser result, used by plugins
	TRefCountPtr<UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser::IHistory> SpatialTemporalDenoiserHistory;

	// Current sample index to be rendered by the path tracer - this gets incremented each time the path tracer accumulates a frame of samples
	uint32 SampleIndex = 0;

	// Path tracer frame index, not reset on invalidation unlike SampleIndex to avoid
	// the "screen door" effect and reduce temporal aliasing
	uint32_t FrameIndex = 0;
};

FPathTracingState* GetPathTracingStateFromView(const FViewInfo& View)
{
	if (!View.ViewState->PathTracingState.IsValid())
	{
		View.ViewState->PathTracingState = MakePimpl<FPathTracingState>();
	}
	check(View.ViewState->PathTracingState.IsValid());
	return View.ViewState->PathTracingState.Get();
}


int GetPathTracingVisualizationMode()
{
	return GPathTracingVisualizeMode.IsNone() ? -1 : FPathTracingDebugModeInfo::Get().Map.FindRef(GPathTracingVisualizeMode, -1);
}

namespace PathTracing
{
	bool ShouldCompilePathTracingShadersForProject(EShaderPlatform ShaderPlatform)
	{
		return ShouldCompileRayTracingShadersForProject(ShaderPlatform) &&
			FDataDrivenShaderPlatformInfo::GetSupportsPathTracing(ShaderPlatform) &&
			CVarPathTracing.GetValueOnAnyThread() != 0;
	}

	bool UsesDecals(const FSceneViewFamily& ViewFamily)
	{
		return ViewFamily.EngineShowFlags.Decals;
	}

	bool UsesReferenceAtmosphere(const FViewInfo& View)
	{
		const int32 EnableReferenceAtmosphereCVar = CVarPathTracingEnableReferenceAtmosphere.GetValueOnRenderThread();
		return EnableReferenceAtmosphereCVar < 0 ? View.FinalPostProcessSettings.PathTracingEnableReferenceAtmosphere != 0 : EnableReferenceAtmosphereCVar != 0;
	}

	bool UsesReferenceDOF(const FViewInfo& View)
	{
		return GEnableReferenceDOF < 0 ? View.FinalPostProcessSettings.PathTracingEnableReferenceDOF != 0 : GEnableReferenceDOF != 0;
	}

	bool NeedsAntiAliasing(const FViewInfo& View)
	{
		return GetPathTracingVisualizationMode() >= 0;
	}

	bool NeedsTonemapping()
	{
		const int DebugMode = GetPathTracingVisualizationMode();
		return DebugMode < 0 ||
			   DebugMode == PATH_TRACER_DEBUG_VIZ_RADIANCE ||
			   DebugMode == PATH_TRACER_DEBUG_VIZ_PRIMARY_RAYS;
	}
}

// This function prepares the portion of shader arguments that may involve invalidating the path traced state
static void PreparePathTracingData(const FScene* Scene, const FViewInfo& View, FPathTracingData& PathTracingData)
{
	const FFinalPostProcessSettings& PPV = View.FinalPostProcessSettings;
	const FEngineShowFlags& ShowFlags = View.Family->EngineShowFlags;

	// Capture the current exposure (NOTE: This is overwritten later so we maintain the exposure that was used on the first sample)
	PathTracingData.BaseExposure = View.PreExposure;

	int32 MaxBounces = CVarPathTracingMaxBounces.GetValueOnRenderThread();
	if (MaxBounces < 0)
	{
		MaxBounces = PPV.PathTracingMaxBounces;
	}

	PathTracingData.MaxBounces = MaxBounces;
	PathTracingData.BackgroundAlpha = FMath::Clamp(CVarPathTracingBackgroundAlpha.GetValueOnRenderThread(), 0.0f, 1.0f);
	PathTracingData.MaxSSSBounces = ShowFlags.SubsurfaceScattering ? CVarPathTracingMaxSSSBounces.GetValueOnRenderThread() : 0;
	PathTracingData.SSSGuidingRatio = FMath::Clamp(CVarPathTracingSSSGuidingRatio.GetValueOnRenderThread(), 0.0f, 1.0f);
	PathTracingData.MaxNormalBias = GetRaytracingMaxNormalBias();
	PathTracingData.MISMode = CVarPathTracingMISMode.GetValueOnRenderThread();
	PathTracingData.VolumeMISMode = CVarPathTracingVolumeMISMode.GetValueOnRenderThread();
	PathTracingData.MaxPathIntensity = CVarPathTracingMaxPathIntensity.GetValueOnRenderThread();
	if (PathTracingData.MaxPathIntensity <= 0)
	{
		// cvar clamp disabled, use PPV value instad
		PathTracingData.MaxPathIntensity = PPV.PathTracingMaxPathIntensity;
	}
	PathTracingData.MaxPathIntensity = FFloat16(PathTracingData.MaxPathIntensity).GetClampedNonNegativeAndFinite().GetFloat(); // Clip to half precision
	PathTracingData.ApproximateCaustics = CVarPathTracingApproximateCaustics.GetValueOnRenderThread() ? 1.0f : 0.0f;
	PathTracingData.SamplerType = CVarPathTracingSamplerType.GetValueOnRenderThread();
	PathTracingData.FilterWidth = CVarPathTracingFilterWidth.GetValueOnRenderThread();
	PathTracingData.CameraFocusDistance = 0;
	PathTracingData.CameraLensRadius = FVector2f::ZeroVector;
	PathTracingData.Petzval = 0;
	PathTracingData.PetzvalFalloffPower = 0;
	PathTracingData.PetzvalExclusionBoxExtents = FVector2f::ZeroVector;
	PathTracingData.PetzvalExclusionBoxRadius = 0;
	if (ShowFlags.DepthOfField &&
		PathTracing::UsesReferenceDOF(View) &&
		PPV.DepthOfFieldFocalDistance > 0)
	{
		DiaphragmDOF::FPhysicalCocModel CocModel;
		CocModel.Compile(View);
		PathTracingData.CameraFocusDistance 			= CocModel.FocusDistance;
		PathTracingData.CameraLensRadius    			= CocModel.GetLensRadius();
		PathTracingData.Petzval	 						= CocModel.Petzval;
		PathTracingData.PetzvalFalloffPower 			= CocModel.PetzvalFalloffPower;
		PathTracingData.PetzvalExclusionBoxExtents		= CocModel.PetzvalExclusionBoxExtents;
		PathTracingData.PetzvalExclusionBoxRadius		= CocModel.PetzvalExclusionBoxRadius;
	}

	const bool bUseReferenceAtmosphere = ShouldRenderSkyAtmosphere(Scene, ShowFlags) && 
		View.SkyAtmosphereUniformShaderParameters != nullptr &&
		PathTracing::UsesReferenceAtmosphere(View);
	
	// NOTE: the callable shader is populated only when clouds are active, so no need to check PPV/cvar again
	const FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();
	const bool bVolumeCloudsVisible = uint32(View.PathTracingVolumetricCloudCallableShaderIndex) < Scene->RayTracingSBT.NumCallableShaderSlots;

	// Merge all volume flags into one uint
	PathTracingData.VolumeFlags = 0;
	PathTracingData.VolumeFlags |= bUseReferenceAtmosphere ? PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE : 0;
	PathTracingData.VolumeFlags |= bVolumeCloudsVisible ? PATH_TRACER_VOLUME_ENABLE_CLOUDS : 0;
	PathTracingData.VolumeFlags |= ShouldRenderFog(*View.Family)
		&& Scene->ExponentialFogs.Num() > 0
		&& Scene->ExponentialFogs[0].bEnableVolumetricFog
		&& Scene->ExponentialFogs[0].VolumetricFogDistance > 0
		&& Scene->ExponentialFogs[0].VolumetricFogExtinctionScale > 0
		&& (Scene->ExponentialFogs[0].FogData[0].Density > 0 ||
			Scene->ExponentialFogs[0].FogData[1].Density > 0) ? PATH_TRACER_VOLUME_ENABLE_FOG : 0;
	PathTracingData.VolumeFlags |= ShouldRenderHeterogeneousVolumesForView(View) ? PATH_TRACER_VOLUME_ENABLE_HETEROGENEOUS_VOLUMES : 0;
	if (View.CachedViewUniformShaderParameters->bPrimitiveAlphaHoldoutEnabled)
	{
		PathTracingData.VolumeFlags |= ShouldRenderHeterogeneousVolumesAsHoldoutForView(View) ? PATH_TRACER_VOLUME_HOLDOUT_HETEROGENEOUS_VOLUMES : 0;
		PathTracingData.VolumeFlags |= View.SkyAtmosphereUniformShaderParameters != nullptr && IsSkyAtmosphereHoldout(View.CachedViewUniformShaderParameters->EnvironmentComponentsFlags) ? PATH_TRACER_VOLUME_HOLDOUT_ATMOSPHERE : 0;
		PathTracingData.VolumeFlags |= bVolumeCloudsVisible && IsVolumetricCloudHoldout(View.CachedViewUniformShaderParameters->EnvironmentComponentsFlags) ? PATH_TRACER_VOLUME_HOLDOUT_CLOUDS : 0;
		PathTracingData.VolumeFlags |= Scene->ExponentialFogs.Num() > 0 && IsExponentialFogHoldout(View.CachedViewUniformShaderParameters->EnvironmentComponentsFlags) ? PATH_TRACER_VOLUME_HOLDOUT_FOG : 0;
	}
	PathTracingData.VolumeFlags |= CVarPathTracingUseAnalyticTransmittance.GetValueOnRenderThread() ? PATH_TRACER_VOLUME_USE_ANALYTIC_TRANSMITTANCE : 0;
	PathTracingData.VolumeFlags |= CVarPathTracingEnableAtmosphereGround.GetValueOnRenderThread() ? PATH_TRACER_VOLUME_SHOW_PLANET_GROUND : 0;

	PathTracingData.CloudRoughnessCutoff = bVolumeCloudsVisible ? CVarPathTracingCloudRoughnessCutoff.GetValueOnRenderThread() : -1.0f;
	if (bVolumeCloudsVisible &&
		PathTracingData.CloudRoughnessCutoff > 0 &&
		PathTracingData.CloudRoughnessCutoff < 1 &&
		CVarPathTracingCloudMultipleScatterMode.GetValueOnRenderThread() == 2)
	{
		// User had clouds visible, but wants multiple scattering to be done by brute force -- enable this here
		PathTracingData.CloudRoughnessCutoff = 1.0f;
	}

	PathTracingData.EnableDBuffer = CVarPathTracingUseDBuffer.GetValueOnRenderThread();

	PathTracingData.DecalRoughnessCutoff = PathTracing::UsesDecals(*View.Family) && View.bHasRayTracingDecals ? CVarPathTracingDecalRoughnessCutoff.GetValueOnRenderThread() : -1.0f;

	PathTracingData.MeshDecalRoughnessCutoff = PathTracing::UsesDecals(*View.Family) && Scene->RayTracingScene.GetNumNativeInstances(ERayTracingSceneLayer::Decals, View.GetRayTracingSceneViewHandle()) > 0 ? CVarPathTracingMeshDecalRoughnessCutoff.GetValueOnRenderThread() : -1.0f;
	PathTracingData.MeshDecalBias = CVarPathTracingMeshDecalBias.GetValueOnRenderThread();

	PathTracingData.MaxRaymarchSteps = CVarPathTracingMaxRaymarchSteps.GetValueOnRenderThread();

	// NOTE: Diffuse and Specular show flags also modify the override colors, but we prefer to tie those to the lighting contribution mechanism below which is more principled
	PathTracingData.ApplyDiffuseSpecularOverrides =
		ShowFlags.LightingOnlyOverride       != 0 ||
		ShowFlags.OverrideDiffuseAndSpecular != 0 ||
		ShowFlags.ReflectionOverride         != 0;

	PathTracingData.EnabledDirectLightingContributions = 0;
	if (ShowFlags.DirectLighting != 0)
	{
		PathTracingData.EnabledDirectLightingContributions |= (PPV.PathTracingIncludeEmissive != 0                           ) ? PATHTRACER_CONTRIBUTION_EMISSIVE : 0;
		PathTracingData.EnabledDirectLightingContributions |= (PPV.PathTracingIncludeDiffuse  != 0 && ShowFlags.Diffuse  != 0) ? PATHTRACER_CONTRIBUTION_DIFFUSE  : 0;
		PathTracingData.EnabledDirectLightingContributions |= (PPV.PathTracingIncludeSpecular != 0 && ShowFlags.Specular != 0) ? PATHTRACER_CONTRIBUTION_SPECULAR : 0;
		PathTracingData.EnabledDirectLightingContributions |= (PPV.PathTracingIncludeVolume   != 0                           ) ? PATHTRACER_CONTRIBUTION_VOLUME   : 0;
	}
	PathTracingData.EnabledIndirectLightingContributions = 0;
	if (ShowFlags.GlobalIllumination != 0)
	{
		const int EnableEmissiveCVar = CVarPathTracingEnableEmissive.GetValueOnRenderThread();
		const bool bEnableEmissive = EnableEmissiveCVar < 0 ? PPV.PathTracingEnableEmissiveMaterials : EnableEmissiveCVar != 0;
		PathTracingData.EnabledIndirectLightingContributions |= (bEnableEmissive                                                       ) ? PATHTRACER_CONTRIBUTION_EMISSIVE : 0;
		PathTracingData.EnabledIndirectLightingContributions |= (PPV.PathTracingIncludeIndirectDiffuse  != 0 && ShowFlags.Diffuse  != 0) ? PATHTRACER_CONTRIBUTION_DIFFUSE  : 0;
		PathTracingData.EnabledIndirectLightingContributions |= (PPV.PathTracingIncludeIndirectSpecular != 0 && ShowFlags.Specular != 0) ? PATHTRACER_CONTRIBUTION_SPECULAR : 0;
		PathTracingData.EnabledIndirectLightingContributions |= (PPV.PathTracingIncludeIndirectVolume   != 0                           ) ? PATHTRACER_CONTRIBUTION_VOLUME   : 0;
	}
}

static bool ShouldCompileGPULightmassShadersForProject(EShaderPlatform ShaderPlatform)
{
#if WITH_EDITOR
	if (!ShouldCompileRayTracingShadersForProject(ShaderPlatform))
	{
		return false;
	}
	// NOTE: cache on first use as this won't change
	static const bool bIsGPULightmassLoaded = FModuleManager::Get().IsModuleLoaded(TEXT("GPULightmass"));
	return bIsGPULightmassLoaded;
#else
	// GPULightmass is an editor only plugin, so don't compile any of its permutations otherwise
	return false;
#endif
}

static bool ShouldCompileGPULightmassShadersForProject(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	return ShouldCompileGPULightmassShadersForProject(Parameters.Platform) &&
		EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData) &&
		Parameters.VertexFactoryType->SupportsLightmapBaking();
}

class FPathTracingSkylightPrepareCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingSkylightPrepareCS)
	SHADER_USE_PARAMETER_STRUCT(FPathTracingSkylightPrepareCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// NOTE: skylight code is shared with RT passes
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		//OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(TextureCube, SkyLightCubemap0)
		SHADER_PARAMETER_TEXTURE(TextureCube, SkyLightCubemap1)
		SHADER_PARAMETER_SAMPLER(SamplerState, SkyLightCubemapSampler0)
		SHADER_PARAMETER_SAMPLER(SamplerState, SkyLightCubemapSampler1)
		SHADER_PARAMETER(float, SkylightBlendFactor)
		SHADER_PARAMETER(float, SkylightInvResolution)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, SkylightTextureOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, SkylightTexturePdf)
		SHADER_PARAMETER(FVector3f, SkyColor)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FPathTracingSkylightPrepareCS, TEXT("/Engine/Private/PathTracing/PathTracingSkylightPrepare.usf"), TEXT("PathTracingSkylightPrepareCS"), SF_Compute);

class FPathTracingSkylightMISCompensationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingSkylightMISCompensationCS)
	SHADER_USE_PARAMETER_STRUCT(FPathTracingSkylightMISCompensationCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// NOTE: skylight code is shared with RT passes
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		//OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SkylightTexturePdfAverage)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, SkylightTextureOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, SkylightTexturePdf)
		SHADER_PARAMETER(FVector3f, SkyColor)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FPathTracingSkylightMISCompensationCS, TEXT("/Engine/Private/PathTracing/PathTracingSkylightMISCompensation.usf"), TEXT("PathTracingSkylightMISCompensationCS"), SF_Compute);

// this struct holds a light grid for both building or rendering
BEGIN_SHADER_PARAMETER_STRUCT(FPathTracingLightGrid, RENDERER_API)
	SHADER_PARAMETER(uint32, SceneInfiniteLightCount)
	SHADER_PARAMETER(FVector3f, SceneLightsTranslatedBoundMin)
	SHADER_PARAMETER(FVector3f, SceneLightsTranslatedBoundMax)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, LightGrid)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, LightGridData)
	SHADER_PARAMETER(unsigned, LightGridResolution)
	SHADER_PARAMETER(unsigned, LightGridMaxCount)
	SHADER_PARAMETER(int, LightGridAxis)
END_SHADER_PARAMETER_STRUCT()

class FPathTracingBuildLightGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingBuildLightGridCS)
	SHADER_USE_PARAMETER_STRUCT(FPathTracingBuildLightGridCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform) ||
			   ShouldCompileGPULightmassShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPathTracingLight>, SceneLights)
		SHADER_PARAMETER(uint32, SceneLightCount)
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingLightGrid, LightGridParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, RWLightGrid)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWLightGridData)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FPathTracingBuildLightGridCS, TEXT("/Engine/Private/PathTracing/PathTracingBuildLightGrid.usf"), TEXT("PathTracingBuildLightGridCS"), SF_Compute);

// make a small custom struct to represent fog, because we need a more physical approach than the rest of the engine
BEGIN_SHADER_PARAMETER_STRUCT(FPathTracingFogParameters, )
	SHADER_PARAMETER(FVector2f, FogDensity)
	SHADER_PARAMETER(FVector2f, FogHeight)
	SHADER_PARAMETER(FVector2f, FogFalloff)
	SHADER_PARAMETER(FLinearColor, FogAlbedo)
	SHADER_PARAMETER(float, FogPhaseG)
	SHADER_PARAMETER(FVector2f, FogCenter)
	SHADER_PARAMETER(float, FogMinZ)
	SHADER_PARAMETER(float, FogMaxZ)
	SHADER_PARAMETER(float, FogRadius)
	SHADER_PARAMETER(float, FogFalloffClamp)
END_SHADER_PARAMETER_STRUCT()

static FPathTracingFogParameters PrepareFogParameters(const FViewInfo& View, const FExponentialHeightFogSceneInfo& FogInfo)
{
	static_assert(FExponentialHeightFogSceneInfo::NumFogs == 2, "Path tracing code assumes a fixed number of fogs");
	FPathTracingFogParameters Parameters = {};

	const FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();

	// See VolumetricFog.usf - the factor of .5 is needed for a better match to HeightFog behavior
	const float MatchHeightFogFactor = .5f;
	Parameters.FogDensity.X = MatchHeightFogFactor * FogInfo.FogData[0].Density * FogInfo.VolumetricFogExtinctionScale;
	Parameters.FogDensity.Y = MatchHeightFogFactor * FogInfo.FogData[1].Density * FogInfo.VolumetricFogExtinctionScale;
	Parameters.FogHeight.X = FogInfo.FogData[0].Height + PreViewTranslation.Z;
	Parameters.FogHeight.Y = FogInfo.FogData[1].Height + PreViewTranslation.Z;
	// Clamp to UI limit to avoid division by 0 in the transmittance calculations
	// Note that we have to adjust by factor of 1000.0 that is applied in FExponentialHeightFogSceneInfo()
	Parameters.FogFalloff.X = FMath::Max(FogInfo.FogData[0].HeightFalloff, 0.001f / 1000.0f);
	Parameters.FogFalloff.Y = FMath::Max(FogInfo.FogData[1].HeightFalloff, 0.001f / 1000.0f);
	Parameters.FogAlbedo = FogInfo.VolumetricFogAlbedo;
	Parameters.FogPhaseG = FogInfo.VolumetricFogScatteringDistribution;

	const float DensityEpsilon = 1e-6f;
	const float Radius = FogInfo.VolumetricFogDistance;
	// compute the value of Z at which the density becomes negligible (but don't go beyond the radius)
	const float ZMax0 = Parameters.FogHeight.X + FMath::Min(Radius, FMath::Log2(FMath::Max(Parameters.FogDensity.X, DensityEpsilon) / DensityEpsilon) / Parameters.FogFalloff.X);
	const float ZMax1 = Parameters.FogHeight.Y + FMath::Min(Radius, FMath::Log2(FMath::Max(Parameters.FogDensity.Y, DensityEpsilon) / DensityEpsilon) / Parameters.FogFalloff.Y);
	// lowest point is just defined by the radius (fog is homogeneous below the height)
	const float ZMin0 = Parameters.FogHeight.X - Radius;
	const float ZMin1 = Parameters.FogHeight.Y - Radius;

	// center X,Y around the current view point
	// NOTE: this can lead to "sliding" when the view distance is low, would it be better to just use the component center instead?
	// NOTE: the component position is not available here, would need to be added to FogInfo ...
	const FVector O = View.ViewMatrices.GetViewOrigin() + PreViewTranslation;
	Parameters.FogCenter = FVector2f(O.X, O.Y);
	Parameters.FogMinZ = FMath::Min(ZMin0, ZMin1);
	Parameters.FogMaxZ = FMath::Max(ZMax0, ZMax1);
	Parameters.FogRadius = Radius;
	Parameters.FogFalloffClamp = -FMath::Log2(FMath::Clamp(CVarPathTracingFogDensityClamp.GetValueOnRenderThread(), 1.0f, 256.0f));
	return Parameters;
}

// Take a double as input as modify it to a FDFScalar that represents the same value.
// TODO: Do we still need this? Could unify with FDFScalar C++ type
static void SplitDouble(double x, float* hi, float* lo)
{
	const double SPLIT = 134217729.0; // 2^27+1
	double temp = SPLIT * x;
	*hi = static_cast<float>(temp - (temp - x));
	*lo = static_cast<float>(x - *hi);
}

static void PreparePlanetCenter(const FViewInfo& View, const FSkyAtmosphereRenderSceneInfo* SkyAtmosphereSceneInfo, FVector3f* PlanetCenterTranslatedWorldHi, FVector3f* PlanetCenterTranslatedWorldLo)
{
	if (SkyAtmosphereSceneInfo != nullptr)
	{
		FVector PlanetCenterTranslatedWorld = SkyAtmosphereSceneInfo->GetSkyAtmosphereSceneProxy().GetAtmosphereSetup().PlanetCenterKm * double(FAtmosphereSetup::SkyUnitToCm) + View.ViewMatrices.GetPreViewTranslation();
		SplitDouble(PlanetCenterTranslatedWorld.X, &PlanetCenterTranslatedWorldHi->X, &PlanetCenterTranslatedWorldLo->X);
		SplitDouble(PlanetCenterTranslatedWorld.Y, &PlanetCenterTranslatedWorldHi->Y, &PlanetCenterTranslatedWorldLo->Y);
		SplitDouble(PlanetCenterTranslatedWorld.Z, &PlanetCenterTranslatedWorldHi->Z, &PlanetCenterTranslatedWorldLo->Z);
	}
	else
	{
		*PlanetCenterTranslatedWorldHi = FVector3f(0);
		*PlanetCenterTranslatedWorldLo = FVector3f(0);
	}
}


BEGIN_SHADER_PARAMETER_STRUCT(FPathTracingCloudParameters, )
	// coordinate frame for the cloud acceleration map
	SHADER_PARAMETER(FVector3f, CloudClipX)	// Right
	SHADER_PARAMETER(FVector3f, CloudClipY) // Forward
	SHADER_PARAMETER(FVector3f, CloudClipZ) // Up

	SHADER_PARAMETER(FVector3f, CloudClipCenterKm) // Planet center in Km

	SHADER_PARAMETER(float, CloudLayerBotKm)
	SHADER_PARAMETER(float, CloudLayerTopKm)
	SHADER_PARAMETER(float, CloudClipDistKm) // distance in x,y in cloud clip space
	SHADER_PARAMETER(float, CloudClipRadiusKm) // distance from origin to planet center in Km

	SHADER_PARAMETER(float, CloudTracingMaxDistance) // limit ray lengths (to avoid slowing down when a ray crosses all clouds)
	SHADER_PARAMETER(float, CloudVoxelWidth)
	SHADER_PARAMETER(float, CloudInvVoxelWidth)
	SHADER_PARAMETER(int32, CloudAccelMapResolution)
	SHADER_PARAMETER(int32, CloudCallableShaderId)
END_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FPathTracingCloudParameterGlobals,)
	SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingCloudParameters, CloudParameters)
	SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingFogParameters, FogParameters)
	SHADER_PARAMETER_STRUCT_INCLUDE(FAtmosphereUniformShaderParameters, AtmosphereParameters)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CloudAccelerationMap)
	SHADER_PARAMETER_SAMPLER(SamplerState, CloudAccelerationMapSampler)
	SHADER_PARAMETER(FVector3f, PlanetCenterTranslatedWorldHi)
	SHADER_PARAMETER(FVector3f, PlanetCenterTranslatedWorldLo)
	SHADER_PARAMETER(uint32, MaxRaymarchSteps)
	SHADER_PARAMETER(int32, CloudShaderMultipleScatterApproxEnabled)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FPathTracingCloudParameterGlobals, "PathTracingCloudParameters");

IMPLEMENT_RT_PAYLOAD_TYPE(ERayTracingPayloadType::PathTracingMaterial, 76);
IMPLEMENT_RT_PAYLOAD_TYPE(ERayTracingPayloadType::GPULightmass, 32);

class FPathTracingRG : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingRG)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FPathTracingRG, FGlobalShader)

	class FKernelType : SHADER_PERMUTATION_INT("PATH_TRACER_KERNEL_TYPE", 3);
	class FAdaptiveSampling : SHADER_PERMUTATION_BOOL("PATH_TRACER_USE_ADAPTIVE_SAMPLING");
	class FCloudShader : SHADER_PERMUTATION_BOOL("PATH_TRACER_USE_CLOUD_SHADER");
	class FSubstrateComplexSpecialMaterial : SHADER_PERMUTATION_BOOL("PATH_TRACER_USE_SUBSTRATE_SPECIAL_COMPLEX_MATERIAL");
	class FUseSER : SHADER_PERMUTATION_BOOL("PATH_TRACER_USE_SER");
	class FNeedTMinWorkaround : SHADER_PERMUTATION_BOOL("NEED_TMIN_WORKAROUND");
	class FHeterogeneousVolumesPipeline : SHADER_PERMUTATION_INT("PATH_TRACER_HETEROGENEOUS_VOLUMES_PIPELINE", 2);
	using FPermutationDomain = TShaderPermutationDomain<FKernelType, FAdaptiveSampling, FCloudShader, FSubstrateComplexSpecialMaterial, FUseSER, FNeedTMinWorkaround, FHeterogeneousVolumesPipeline>;

    static bool IsPermutationExperimental(const FGlobalShaderPermutationParameters& Parameters)
    {
        const FPermutationDomain PermutationVector(Parameters.PermutationId);
        return PermutationVector.Get<FAdaptiveSampling>() ||
               PermutationVector.Get<FHeterogeneousVolumesPipeline>();
    }

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const bool bUseExperimental = CVarPathTracingExperimental.GetValueOnAnyThread();
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (bUseExperimental == false && IsPermutationExperimental(Parameters))
		{
            // do not compile experimental features if they are not enabled
            return false;
		}

		if (PermutationVector.Get<FCloudShader>())
		{
			// the cloud shader version can only be supported if the platform supports callable shaders
			if (!ShouldCompileRayTracingCallableShadersForProject(Parameters.Platform))
			{
				return false;
			}
		}

		// Only compile SER extensions on platforms that could support it
		if (PermutationVector.Get<FUseSER>() && !FDataDrivenShaderPlatformInfo::GetSupportsShaderExecutionReordering(Parameters.Platform))
		{
			return false;
		}

		if (!Substrate::IsSubstrateEnabled())
		{
			// If we aren't using Substrate, no need to compile the complex material path
			if (PermutationVector.Get<FSubstrateComplexSpecialMaterial>())
			{
				return false;
			}
		}
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_RECT_LIGHT_TEXTURES"), 1);
		OutEnvironment.SetDefine(TEXT("DEBUG_ENABLE"), CVarPathTracingDebug.GetValueOnAnyThread() > 0 ? 1u : 0u);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::PathTracingMaterial | ERayTracingPayloadType::Decals;
	}
			
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	static EShaderCompileJobPriority GetOverrideJobPriority()
	{
		// FPathTracingRG takes up to 20s on average on D3D SM6, and 30s on Vulkan SM6
		return EShaderCompileJobPriority::ExtraHigh;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RadianceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float2>, VarianceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, AlbedoTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, NormalTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, DepthTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, DecalTLAS)
		
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingData, PathTracingData)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, RandomSequenceSpaceFillingCurve)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrint)

		// scene lights
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPathTracingLight>, SceneLights)
		SHADER_PARAMETER(uint32, SceneLightCount)
		SHADER_PARAMETER(uint32, SceneVisibleLightCount)
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingLightGrid, LightGridParameters)

		// Skylight
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingSkylight, SkylightParameters)

		// sky atmosphere
		SHADER_PARAMETER_STRUCT_REF(FAtmosphereUniformShaderParameters, Atmosphere)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AtmosphereOpticalDepthLUT)
		SHADER_PARAMETER_SAMPLER(SamplerState, AtmosphereOpticalDepthLUTSampler)
		SHADER_PARAMETER(FVector3f, PlanetCenterTranslatedWorldHi)
		SHADER_PARAMETER(FVector3f, PlanetCenterTranslatedWorldLo)

		// clouds
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingCloudParameters, CloudParameters)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, CloudAccelerationMap)
		SHADER_PARAMETER_SAMPLER(SamplerState, CloudAccelerationMapSampler)

		// exponential height fog
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingFogParameters, FogParameters)

		// Heterogeneous volumes adaptive voxel grid
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FOrthoVoxelGridUniformBufferParameters, OrthoGridUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFrustumVoxelGridUniformBufferParameters, FrustumGridUniformBuffer)
		// NOTE: Remove in separate submission
		//SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FPreshadingUniformBufferParameters, PreshadingUniformBuffer)

        // Heterogeneous volumes callable interface
		SHADER_PARAMETER(int32, HeterogeneousVolumesNum)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, HeterogeneousVolumesCallableShaderIdIntersect)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, HeterogeneousVolumesCallableShaderIdGetDensityBounds)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, HeterogeneousVolumesCallableShaderIdGetDensity)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, HeterogeneousVolumesCallableShaderIdSampleDistance)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<int>, HeterogeneousVolumesCallableShaderIdGetTransmittance)

		// scene decals
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRayTracingDecals, DecalParameters)

		// camera ray starting medium coefficients
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, StartingMediumData)

		// Used by multi-GPU rendering and TDR-avoidance tiling
		SHADER_PARAMETER(FIntPoint, TilePixelOffset)
		SHADER_PARAMETER(FIntPoint, TileTextureOffset)
		SHADER_PARAMETER(int32, ScanlineStride)
		SHADER_PARAMETER(int32, ScanlineWidth)

		// extra parameters required for path compacting kernel
		SHADER_PARAMETER(int32, FirstBounce)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPathTracingPackedPathState>, PathStateData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, ActivePaths)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, NextActivePaths)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, NumPathStates)

		RDG_BUFFER_ACCESS(PathTracingIndirectArgs, ERHIAccess::IndirectArgs | ERHIAccess::SRVCompute)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FPathTracingRG, "/Engine/Private/PathTracing/PathTracing.usf", "PathTracingMainRG", SF_RayGen);

class FPathTracingDebugRG : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingDebugRG)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FPathTracingDebugRG, FGlobalShader)

	class FUseSER : SHADER_PERMUTATION_BOOL("PATH_TRACER_USE_SER");
	class FUsePrimaryRays : SHADER_PERMUTATION_BOOL("PATH_TRACER_PRIMARY_RAYS");
	class FIncludeTranslucent : SHADER_PERMUTATION_BOOL("PATH_TRACER_INCLUDE_TRANSLUCENT");
	class FKernelType : SHADER_PERMUTATION_INT("PATH_TRACER_KERNEL_TYPE", 3);
	using FPermutationDomain = TShaderPermutationDomain<FUseSER, FUsePrimaryRays, FIncludeTranslucent, FKernelType>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FUseSER>())
		{
			// SER only makes sense in the PrimaryRays mode
			if (!PermutationVector.Get<FUsePrimaryRays>())
			{
				return false;
			}
			// Only compile SER extensions on platforms that could support it
			if (!FDataDrivenShaderPlatformInfo::GetSupportsShaderExecutionReordering(Parameters.Platform))
			{
				return false;
			}
		}

		// If we are one of the debug modes (i.e.: not "PrimaryRays" mode)
		if (!PermutationVector.Get<FUsePrimaryRays>())
		{
			// then only the CAMERA kernel type applies
			if (PermutationVector.Get<FKernelType>() != PATH_TRACER_KERNEL_CAMERA)
			{
				return false;
			}
		}

		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("USE_RECT_LIGHT_TEXTURES"), 1);
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::PathTracingMaterial | ERayTracingPayloadType::Decals;
	}
		
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWSceneColor)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWSceneDiffuseColor)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWSceneSpecularColor)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWSceneNormal)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>,  RWSceneRoughness)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>,  RWSceneLinearDepth)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>,  RWSceneDepth)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, RandomSequenceSpaceFillingCurve)
		SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
		SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)
		SHADER_PARAMETER(int32, DebugMode)
		SHADER_PARAMETER(int32, SamplesPerPixel)
		SHADER_PARAMETER(int32, SampleIndex)
		SHADER_PARAMETER(float, SampleBlendFactor)
		SHADER_PARAMETER(int32, MaxBounces)
		SHADER_PARAMETER(int32, EnableEmissive)
        SHADER_PARAMETER(float, MaxPathIntensity)
        SHADER_PARAMETER(float, AmbientOcclusionMaxDistance)
        SHADER_PARAMETER(uint32, VisualizeMaxCount)

		// scene lights
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPathTracingLight>, SceneLights)
		SHADER_PARAMETER(uint32, SceneLightCount)
		SHADER_PARAMETER(uint32, SceneVisibleLightCount)
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingLightGrid, LightGridParameters)

	
		// scene decals
        SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, DecalTLAS)
        SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRayTracingDecals, DecalParameters)
        SHADER_PARAMETER(float, DecalRoughnessCutoff)
        SHADER_PARAMETER(float, MeshDecalRoughnessCutoff)
        SHADER_PARAMETER(float, MeshDecalBias)
        SHADER_PARAMETER(uint32, EnableDBuffer)

		// scene uniform buffer
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)

		// Skylight
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingSkylight, SkylightParameters)

		// extra parameters required for path compacting kernel
		SHADER_PARAMETER(int32, FirstBounce)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPathTracingDebugPackedPathState>, PathStateData)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, ActivePaths)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, NextActivePaths)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, NumPathStates)

	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FPathTracingDebugRG, "/Engine/Private/PathTracing/PathTracingDebug.usf", "PathTracingDebugRG", SF_RayGen);


class FPathTracingCopyDepthPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingCopyDepthPS)
	SHADER_USE_PARAMETER_STRUCT(FPathTracingCopyDepthPS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, DepthTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FPathTracingCopyDepthPS, TEXT("/Engine/Private/PathTracing/PathTracingCopyDepth.usf"), TEXT("CopyDepth"), SF_Pixel);


class FPathTracingInitExtinctionCoefficientRG : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingInitExtinctionCoefficientRG)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FPathTracingInitExtinctionCoefficientRG, FGlobalShader)

	class FNeedTMinWorkaround : SHADER_PERMUTATION_BOOL("NEED_TMIN_WORKAROUND");
	using FPermutationDomain = TShaderPermutationDomain<FNeedTMinWorkaround>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::PathTracingMaterial;
	}
			
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		RDG_BUFFER_ACCESS_ARRAY(SBTBuffers)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, RWStartingMediumData)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FPathTracingInitExtinctionCoefficientRG, "/Engine/Private/PathTracing/PathTracingInitExtinctionCoefficient.usf", "PathTracingInitExtinctionCoefficientRG", SF_RayGen);

class FPathTracingSwizzleScanlinesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingSwizzleScanlinesCS)
	SHADER_USE_PARAMETER_STRUCT(FPathTracingSwizzleScanlinesCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, DispatchDim)
		SHADER_PARAMETER(FIntPoint, TileSize)
		SHADER_PARAMETER(int32, ScanlineStride)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FPathTracingSwizzleScanlinesCS, TEXT("/Engine/Private/PathTracing/PathTracingSwizzleScanlines.usf"), TEXT("PathTracingSwizzleScanlinesCS"), SF_Compute);


class FPathTracingBuildAtmosphereOpticalDepthLUTCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingBuildAtmosphereOpticalDepthLUTCS)
	SHADER_USE_PARAMETER_STRUCT(FPathTracingBuildAtmosphereOpticalDepthLUTCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumSamples)
		SHADER_PARAMETER(uint32, Resolution)
		SHADER_PARAMETER_STRUCT_REF(FAtmosphereUniformShaderParameters, Atmosphere)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, AtmosphereOpticalDepthLUT)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FPathTracingBuildAtmosphereOpticalDepthLUTCS, TEXT("/Engine/Private/PathTracing/PathTracingBuildAtmosphereLUT.usf"), TEXT("PathTracingBuildAtmosphereOpticalDepthLUTCS"), SF_Compute);


FPathTracingCloudParameters PrepareCloudParameters(const FScene* Scene, const FViewInfo& View, const int CloudAccelerationMapResolution)
{
	check(Scene != nullptr);;
	check(Scene->GetVolumetricCloudSceneInfo() != nullptr);


	const FSkyAtmosphereRenderSceneInfo* SkyInfo = Scene->GetSkyAtmosphereSceneInfo();
	const FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();
	const FVolumetricCloudSceneProxy& CloudProxy = CloudInfo->GetVolumetricCloudSceneProxy();

	float PlanetRadiusKm = CloudProxy.PlanetRadiusKm;
	FVector CloudCenterKm = FVector(0, 0, -CloudProxy.PlanetRadiusKm);
	if (SkyInfo != nullptr)
	{
		const FAtmosphereSetup& AtmosphereSetup = SkyInfo->GetSkyAtmosphereSceneProxy().GetAtmosphereSetup();
		PlanetRadiusKm = AtmosphereSetup.BottomRadiusKm;
		CloudCenterKm = AtmosphereSetup.PlanetCenterKm;
	}

	FVector PlanetUp = View.ViewMatrices.GetViewOrigin() - CloudCenterKm * double(FAtmosphereSetup::SkyUnitToCm);
	double ViewToPlanet = PlanetUp.Length();
	PlanetUp.Normalize();

	FPathTracingCloudParameters Params = {};

	// Make a coordinate frame for our cloud acceleration map -- we want it to be stable with respect
	// to camera rotations to minimize aliasing as the camera moves. PlanetUp will generally be quite
	// stable when moving about the planet surface, so using this as the only input minimizes resampling artifacts
	// TODO: Figure out a stable scheme for views from space ...
	// See GetTangentBasis() in MonteCarlo.ush
	// TODO: Should probably be turned into a utility on TVector?
	{
		const FVector TangentZ = PlanetUp;
		const double Sign = TangentZ.Z >= 0 ? 1 : -1;
		const double a = -1 / (Sign + TangentZ.Z);
		const double b = TangentZ.X * TangentZ.Y * a;
		FVector TangentX = { 1 + Sign * a * (TangentZ.X * TangentZ.X), Sign * b, -Sign * TangentZ.X };
		FVector TangentY = { b,  Sign + a * (TangentZ.Y * TangentZ.Y), -TangentZ.Y };

		Params.CloudClipX = FVector3f(TangentX);
		Params.CloudClipY = FVector3f(TangentY);
		Params.CloudClipZ = FVector3f(TangentZ);
	}
	Params.CloudClipCenterKm = FVector3f(CloudCenterKm); // LWC_TODO: Pass this in as high/low for better precision

	Params.CloudLayerBotKm = PlanetRadiusKm + CloudProxy.LayerBottomAltitudeKm;
	Params.CloudLayerTopKm = Params.CloudLayerBotKm + CloudProxy.LayerHeightKm;
	if (CloudProxy.TracingMaxDistanceMode == 0)
	{
		Params.CloudClipDistKm = FMath::Min(CloudProxy.TracingStartMaxDistance, Params.CloudLayerTopKm);
		Params.CloudTracingMaxDistance = CloudProxy.TracingMaxDistance * FAtmosphereSetup::SkyUnitToCm;
	}
	else
	{
		Params.CloudClipDistKm = FMath::Min(FMath::Min(CloudProxy.TracingStartMaxDistance, CloudProxy.TracingMaxDistance), Params.CloudLayerTopKm);
		Params.CloudTracingMaxDistance = 2 * Params.CloudClipDistKm * FAtmosphereSetup::SkyUnitToCm; // full diagonal for this
	}
	
	Params.CloudClipRadiusKm = float(ViewToPlanet * double(FAtmosphereSetup::CmToSkyUnit));
	Params.CloudCallableShaderId = -1;
	Params.CloudAccelMapResolution = CloudAccelerationMapResolution;
	Params.CloudVoxelWidth = float(2.0 * Params.CloudClipDistKm * FAtmosphereSetup::SkyUnitToCm / CloudAccelerationMapResolution);
	Params.CloudInvVoxelWidth = 1.0f / Params.CloudVoxelWidth;
	return Params;
}

class FPathTracingBuildCloudAccelerationMapCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FPathTracingBuildCloudAccelerationMapCS, MeshMaterial)
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FPathTracingBuildCloudAccelerationMapCS, FMeshMaterialShader) // TODO: following pattern used in VolumetricCloudRender.cpp -- is there a proper modern alternative?

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform) &&
			Parameters.MaterialParameters.bIsUsedWithVolumetricCloud &&
			Parameters.MaterialParameters.MaterialDomain == MD_Volume;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("CLOUD_LAYER_PIXEL_SHADER"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumSamples)
		SHADER_PARAMETER(uint32, Iteration)
		SHADER_PARAMETER(uint32, TemporalSeed)
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingCloudParameters, CloudParameters)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, CloudAccelerationMap)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_MATERIAL_SHADER_TYPE(, FPathTracingBuildCloudAccelerationMapCS, TEXT("/Engine/Private/PathTracing/PathTracingBuildCloudAccelerationMap.usf"), TEXT("PathTracingBuildCloudAccelerationMapCS"), SF_Compute);

class FPathTracingVolumetricCloudMaterial : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FPathTracingVolumetricCloudMaterial, Material);
public:
	FPathTracingVolumetricCloudMaterial() = default;

	FPathTracingVolumetricCloudMaterial(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FMaterialShader(Initializer)
	{
		CloudParameter.Bind(Initializer.ParameterMap, TEXT("PathTracingCloudParameters"));
	}

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform) &&
			ShouldCompileRayTracingCallableShadersForProject(Parameters.Platform) &&
			Parameters.MaterialParameters.bIsUsedWithVolumetricCloud &&
			Parameters.MaterialParameters.MaterialDomain == MD_Volume;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		check(Parameters.MaterialParameters.MaterialDomain == MD_Volume);
		OutEnvironment.SetDefine(TEXT("CLOUD_LAYER_PIXEL_SHADER"), 1);
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		if (ParameterMap.ContainsParameterAllocation(FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
		{
			OutError.Add(TEXT("Ray tracing callable shaders cannot read from the SceneTexturesStruct."));
			return false;
		}

		for (const auto& It : ParameterMap.GetParameterMap())
		{
			const FParameterAllocation& ParamAllocation = It.Value;
			if (ParamAllocation.Type != EShaderParameterType::UniformBuffer
				&& ParamAllocation.Type != EShaderParameterType::LooseData)
			{
				OutError.Add(FString::Printf(TEXT("Invalid ray tracing shader parameter '%s'. Only uniform buffers and loose data parameters are supported."), *(It.Key)));
				return false;
			}
		}

		return true;
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		// TODO: This isn't the payload we use, but the logic in RayTracingMaterialHitShaders.cpp needs to assume a consistent payload ID for all callable shaders
		// The simplest solution is probably to remove FDecalShaderPayload and use the PT payload everywhere
		return ERayTracingPayloadType::Decals;
	}
			
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FViewInfo& View,
		const TUniformBufferRef<FPathTracingCloudParameterGlobals>& CloudParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMaterialShader::GetShaderBindings(Scene, FeatureLevel, MaterialRenderProxy, Material, ShaderBindings);

		ShaderBindings.Add(GetUniformBufferParameter<FViewUniformShaderParameters>(), View.ViewUniformBuffer);
		// Use GIdentityPrimitiveUniformBuffer just like in the decal handling code
		// We could potentially bind the actual primitive uniform buffer
		ShaderBindings.Add(GetUniformBufferParameter<FPrimitiveUniformShaderParameters>(), GIdentityPrimitiveUniformBuffer);
		ShaderBindings.Add(CloudParameter, CloudParameters);
	}

private:
	LAYOUT_FIELD(FShaderUniformBufferParameter, CloudParameter);
};
IMPLEMENT_SHADER_TYPE(, FPathTracingVolumetricCloudMaterial, TEXT("/Engine/Private/PathTracing/PathTracingVolumetricCloudMaterialShader.usf"), TEXT("PathTracingVolumetricCloudMaterialShader"), SF_RayCallable);

enum class ECallableType
{
	Intersect,
	GetDensityBounds,
	GetDensity,
	SampleDistance,
	GetTransmittance
};

template <enum ECallableType>
class FPathTracingHeterogeneousVolumesMaterial : public FGlobalShader
//class FPathTracingHeterogeneousVolumesMaterial : public FMaterialShader
{
	//DECLARE_SHADER_TYPE(FPathTracingHeterogeneousVolumesMaterial, Material);
	DECLARE_GLOBAL_SHADER(FPathTracingHeterogeneousVolumesMaterial)
public:
	FPathTracingHeterogeneousVolumesMaterial() = default;

	FPathTracingHeterogeneousVolumesMaterial(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		//: FMaterialShader(Initializer)
		: FGlobalShader(Initializer)
	{
		PreshadingUniformBuffer.Bind(Initializer.ParameterMap, TEXT("PreshadingUniformBuffer"));
	}

	//static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform) &&
			ShouldCompileRayTracingCallableShadersForProject(Parameters.Platform)
			//Parameters.MaterialParameters.bIsUsedWithHeterogeneousVolumes &&
			//Parameters.MaterialParameters.MaterialDomain == MD_Volume
			;
	}

	//static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		//FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		//check(Parameters.MaterialParameters.MaterialDomain == MD_Volume);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		if (ParameterMap.ContainsParameterAllocation(FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
		{
			OutError.Add(TEXT("Ray tracing callable shaders cannot read from the SceneTexturesStruct."));
			return false;
		}

		for (const auto& It : ParameterMap.GetParameterMap())
		{
			const FParameterAllocation& ParamAllocation = It.Value;
			if (ParamAllocation.Type != EShaderParameterType::UniformBuffer
				&& ParamAllocation.Type != EShaderParameterType::LooseData)
			{
				OutError.Add(FString::Printf(TEXT("Invalid ray tracing shader parameter '%s'. Only uniform buffers and loose data parameters are supported."), *(It.Key)));
				return false;
			}
		}

		return true;
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		// TODO: This isn't the payload we use, but the logic in RayTracingMaterialHitShaders.cpp needs to assume a consistent payload ID for all callable shaders
		// The simplest solution is probably to remove FDecalShaderPayload and use the PT payload everywhere
		return ERayTracingPayloadType::Decals;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	void GetShaderBindings(
		const FViewInfo& View,
		const TUniformBufferRef<FPreshadingUniformBufferParameters>& PreshadingUniformBufferParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		//FMaterialShader::GetShaderBindings(Scene, FeatureLevel, MaterialRenderProxy, Material, ShaderBindings);

		ShaderBindings.Add(GetUniformBufferParameter<FViewUniformShaderParameters>(), View.ViewUniformBuffer);
		ShaderBindings.Add(GetUniformBufferParameter<FPreshadingUniformBufferParameters>(), PreshadingUniformBufferParameters);
	}

private:
	LAYOUT_FIELD(FShaderUniformBufferParameter, PreshadingUniformBuffer);
};

using FPathTracingHeterogeneousVolumesIntersectCS = FPathTracingHeterogeneousVolumesMaterial<ECallableType::Intersect>;
using FPathTracingHeterogeneousVolumesGetDensityBoundsCS = FPathTracingHeterogeneousVolumesMaterial<ECallableType::GetDensityBounds>;
using FPathTracingHeterogeneousVolumesGetDensityCS = FPathTracingHeterogeneousVolumesMaterial<ECallableType::GetDensity>;
using FPathTracingHeterogeneousVolumesSampleDistanceCS = FPathTracingHeterogeneousVolumesMaterial<ECallableType::SampleDistance>;
using FPathTracingHeterogeneousVolumesGetTransmittanceCS = FPathTracingHeterogeneousVolumesMaterial<ECallableType::GetTransmittance>;
IMPLEMENT_SHADER_TYPE(template<>, FPathTracingHeterogeneousVolumesIntersectCS, TEXT("/Engine/Private/PathTracing/PathTracingHeterogeneousVolumeMaterialShader.usf"), TEXT("PathTracingHeterogeneousVolumesIntersectCS"), SF_RayCallable);
IMPLEMENT_SHADER_TYPE(template<>, FPathTracingHeterogeneousVolumesGetDensityBoundsCS, TEXT("/Engine/Private/PathTracing/PathTracingHeterogeneousVolumeMaterialShader.usf"), TEXT("PathTracingHeterogeneousVolumesGetDensityBoundsCS"), SF_RayCallable);
IMPLEMENT_SHADER_TYPE(template<>, FPathTracingHeterogeneousVolumesGetDensityCS, TEXT("/Engine/Private/PathTracing/PathTracingHeterogeneousVolumeMaterialShader.usf"), TEXT("PathTracingHeterogeneousVolumesGetDensityCS"), SF_RayCallable);
IMPLEMENT_SHADER_TYPE(template<>, FPathTracingHeterogeneousVolumesSampleDistanceCS, TEXT("/Engine/Private/PathTracing/PathTracingHeterogeneousVolumeMaterialShader.usf"), TEXT("PathTracingHeterogeneousVolumesSampleDistanceCS"), SF_RayCallable);
IMPLEMENT_SHADER_TYPE(template<>, FPathTracingHeterogeneousVolumesGetTransmittanceCS, TEXT("/Engine/Private/PathTracing/PathTracingHeterogeneousVolumeMaterialShader.usf"), TEXT("PathTracingHeterogeneousVolumesGetTransmittanceCS"), SF_RayCallable);

class FPathTracingVolumetricCloudMaterialVisualize : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingVolumetricCloudMaterialVisualize)
public:

	FPathTracingVolumetricCloudMaterialVisualize() = default;
	FPathTracingVolumetricCloudMaterialVisualize(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FGlobalShader(Initializer)
	{
		CloudParameter.Bind(Initializer.ParameterMap, TEXT("PathTracingCloudParameters"));
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform) &&
			   ShouldCompileRayTracingCallableShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("CLOUD_LAYER_PIXEL_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("CLOUD_VISUALIZATION_SHADER"), 1);
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		if (ParameterMap.ContainsParameterAllocation(FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
		{
			OutError.Add(TEXT("Ray tracing callable shaders cannot read from the SceneTexturesStruct."));
			return false;
		}

		for (const auto& It : ParameterMap.GetParameterMap())
		{
			const FParameterAllocation& ParamAllocation = It.Value;
			if (ParamAllocation.Type != EShaderParameterType::UniformBuffer
				&& ParamAllocation.Type != EShaderParameterType::LooseData)
			{
				OutError.Add(FString::Printf(TEXT("Invalid ray tracing shader parameter '%s'. Only uniform buffers and loose data parameters are supported."), *(It.Key)));
				return false;
			}
		}

		return true;
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		// TODO: This isn't the payload we use, but the logic in RayTracingMaterialHitShaders.cpp needs to assume a consistent payload ID for all callable shaders
		// The simplest solution is probably to remove FDecalShaderPayload and use the PT payload everywhere
		return ERayTracingPayloadType::Decals;
	}
			
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	void GetShaderBindings(
		const FViewInfo& View,
		const TUniformBufferRef<FPathTracingCloudParameterGlobals>& CloudParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		ShaderBindings.Add(GetUniformBufferParameter<FViewUniformShaderParameters>(), View.ViewUniformBuffer);
		ShaderBindings.Add(CloudParameter, CloudParameters);
	}

private:
	LAYOUT_FIELD(FShaderUniformBufferParameter, CloudParameter);
};
IMPLEMENT_SHADER_TYPE(, FPathTracingVolumetricCloudMaterialVisualize, TEXT("/Engine/Private/PathTracing/PathTracingVolumetricCloudMaterialShader.usf"), TEXT("PathTracingVolumetricCloudMaterialShader"), SF_RayCallable);


void PreparePathTracingCloudMaterial(FRDGBuilder& GraphBuilder, FScene* Scene, TArrayView<FViewInfo> Views)
{
	// make sure all views have an invalid callable shader index (unless proven otherwise below)
	for (FViewInfo& View : Views)
	{
		View.PathTracingVolumetricCloudCallableShaderIndex = -1;
	}

	if (!ShouldCompileRayTracingCallableShadersForProject(Scene->GetShaderPlatform()))
	{
		return;
	}

	if (CVarPathTracingEnableReferenceClouds.GetValueOnRenderThread() == 0)
	{
		return;
	}

	FVolumetricCloudRenderSceneInfo* CloudRenderSceneInfo = Scene->GetVolumetricCloudSceneInfo();
	if (CloudRenderSceneInfo == nullptr)
	{
		return;
	}

	UMaterialInterface* CloudMaterialInterface = CloudRenderSceneInfo->GetVolumetricCloudSceneProxy().GetCloudVolumeMaterial();
	if (CloudMaterialInterface == nullptr)
	{
		return;
	}
	const FMaterialRenderProxy* CloudVolumeMaterialProxy = CloudMaterialInterface->GetRenderProxy();
	if (CloudVolumeMaterialProxy == nullptr)
	{
		return;
	}
	const FMaterial* MaterialResource = &CloudVolumeMaterialProxy->GetMaterialWithFallback(Scene->GetFeatureLevel(), CloudVolumeMaterialProxy);
	if (MaterialResource->GetMaterialDomain() != MD_Volume)
	{
		return;
	}

	const FMaterialShaderMap* MaterialShaderMap = MaterialResource->GetRenderingThreadShaderMap();
	auto CallableShader = MaterialShaderMap->GetShader<FPathTracingVolumetricCloudMaterial>();
	if (!CallableShader.IsValid())
	{
		return;
	}

	const FVolumetricCloudRenderSceneInfo* CloudInfo = Scene->GetVolumetricCloudSceneInfo();

	const bool bCloudAccelerationMapVisualize = CVarPathTracingCloudAccelerationMapVisualize.GetValueOnRenderThread();

	TShaderRef<FPathTracingVolumetricCloudMaterialVisualize> CallableShaderVisualize;
	if (bCloudAccelerationMapVisualize)
	{
		CallableShaderVisualize = GetGlobalShaderMap(Scene->GetShaderPlatform())->GetShader<FPathTracingVolumetricCloudMaterialVisualize>();
		if (!CallableShaderVisualize.IsValid())
		{
			// asking for visualization, but visualization shader is not ready
			return;
		}
	}

	for (FViewInfo& View : Views)
	{
		if (!PathTracing::UsesReferenceAtmosphere(View) || !ShouldRenderVolumetricCloud(Scene, View.Family->EngineShowFlags))
		{
			// reference atmosphere mode disabled for this view, or clouds disabled for this view
			continue;
		}

		uint32 BaseCallableSlotIndex = Scene->RayTracingSBT.NumCallableShaderSlots;
		FRayTracingShaderCommand& Command = Scene->RayTracingSBT.CallableCommands.AddDefaulted_GetRef();

		if (bCloudAccelerationMapVisualize)
		{
			Command.SetShader(CallableShaderVisualize);
		}
		else
		{
			Command.SetShader(CallableShader);
		}
		Command.SlotInScene = BaseCallableSlotIndex;

		View.PathTracingVolumetricCloudCallableShaderIndex = BaseCallableSlotIndex;

		const int CloudAccelMapResolution = CVarPathTracingCloudAccelerationMapResolution.GetValueOnRenderThread();

		FPathTracingCloudParameterGlobals Params = {};
		Params.CloudParameters = PrepareCloudParameters(Scene, View, CloudAccelMapResolution);
		if (!Scene->ExponentialFogs.IsEmpty())
		{
			Params.FogParameters = PrepareFogParameters(View, Scene->ExponentialFogs[0]);
		}
		if (Scene->GetSkyAtmosphereSceneInfo() != nullptr)
		{
			Params.AtmosphereParameters = *Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereShaderParameters();
		}

		const int Resolution = Params.CloudParameters.CloudAccelMapResolution;
		FPathTracingState* PathTracingState = GetPathTracingStateFromView(View);
		if (PathTracingState->CloudAccelerationMap && PathTracingState->CloudAccelerationMap->GetDesc().Extent.X == Resolution)
		{
			// we already have a map from a previous iteration, re-use it
		}
		else
		{
			// Either we don't have a map yet, or the resolution cvar changed
			PathTracingState->CloudAccelerationMap.SafeRelease();
			EPixelFormat CloudAccelerationMapFormat = PF_FloatRGBA; // 16 bit should be good enough for typical density/z ranges
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("PathTracer.CloudAccelerationMap"), Resolution, Resolution, CloudAccelerationMapFormat)
			.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV);

			PathTracingState->CloudAccelerationMap = CreateRenderTarget(GraphBuilder.RHICmdList.CreateTexture(Desc), Desc.DebugName);
		}

		Params.CloudAccelerationMap = GraphBuilder.RegisterExternalTexture(PathTracingState->CloudAccelerationMap, TEXT("PathTracer.CloudAccelerationMap"));
		Params.CloudAccelerationMapSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		PreparePlanetCenter(View, Scene->GetSkyAtmosphereSceneInfo(), &Params.PlanetCenterTranslatedWorldHi, &Params.PlanetCenterTranslatedWorldLo);
		Params.MaxRaymarchSteps = CVarPathTracingMaxRaymarchSteps.GetValueOnRenderThread();
		Params.CloudShaderMultipleScatterApproxEnabled = CVarPathTracingCloudMultipleScatterMode.GetValueOnRenderThread() == 1;

		TUniformBufferRef<FPathTracingCloudParameterGlobals> CloudParametersUB = CreateUniformBufferImmediate(Params, EUniformBufferUsage::UniformBuffer_SingleFrame);
		Scene->RayTracingSBT.TransientUniformBuffers.Add(CloudParametersUB); // Hold uniform buffer ref in RayTracingSBT since FMeshDrawSingleShaderBindings doesn't

		FMeshDrawSingleShaderBindings SingleShaderBindings = Command.ShaderBindings.GetSingleShaderBindings(SF_RayCallable);
		
		if (bCloudAccelerationMapVisualize)
		{
			CallableShaderVisualize->GetShaderBindings(View, CloudParametersUB, SingleShaderBindings);
		}
		else
		{
			CallableShader->GetShaderBindings(Scene, Scene->GetFeatureLevel(), *CloudVolumeMaterialProxy, *MaterialResource, View, CloudParametersUB, SingleShaderBindings);
		}
		
		Scene->RayTracingSBT.NumCallableShaderSlots++;
	}
}

template <class CallableShaderType>
void BindCallableShader(
	FScene* Scene,
	const FViewInfo& View,
	const FMaterialRenderProxy* MaterialRenderProxy,
	const FMaterial* MaterialResource,
	const FMaterialShaderMap* MaterialShaderMap,
	TUniformBufferRef<FPreshadingUniformBufferParameters> PreshadingUniformBuffer,
	int32& CallableShaderIndex
)
{
	//auto CallableShader = MaterialShaderMap->GetShader<CallableShaderType>();
	auto CallableShader = GetGlobalShaderMap(Scene->GetShaderPlatform())->GetShader<CallableShaderType>();
	if (!CallableShader.IsValid())
	{
		return;
	}

	CallableShaderIndex = Scene->RayTracingSBT.NumCallableShaderSlots++;

	FRayTracingShaderCommand& Command = Scene->RayTracingSBT.CallableCommands.AddDefaulted_GetRef();
	Command.SetShader(CallableShader);
	Command.SlotInScene = CallableShaderIndex;

	// Bind resource to shader
	FMeshDrawSingleShaderBindings SingleShaderBindings = Command.ShaderBindings.GetSingleShaderBindings(SF_RayCallable);
	CallableShader->GetShaderBindings(View, PreshadingUniformBuffer, SingleShaderBindings);
}

void PreparePathTracingHeterogeneousVolumeMaterial(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const FViewInfo& View,
	FVolumetricMeshBatch& VolumetricMeshBatch,
	int32 VolumeInstanceIndex,
	const FPreshadingUniformBufferParameters& PreshadingUniformBufferParameters,
	FPathTracingHeterogeneousVolumesCallableInfo& CallableInfo
)
{
	check(View.ViewState);

	if (!ShouldCompileRayTracingCallableShadersForProject(Scene->GetShaderPlatform()))
	{
		return;
	}

	if (!ShouldRenderHeterogeneousVolumes(Scene))
	{
		return;
	}

	const FPrimitiveSceneProxy* PrimitiveSceneProxy = VolumetricMeshBatch.Proxy;
	if (!PrimitiveSceneProxy->IsHeterogeneousVolume())
	{
		return;
	}

	const FMaterialRenderProxy* MaterialRenderProxy = VolumetricMeshBatch.Mesh->MaterialRenderProxy;
	if (MaterialRenderProxy == nullptr)
	{
		return;
	}

	const FMaterial* MaterialResource = &MaterialRenderProxy->GetMaterialWithFallback(Scene->GetFeatureLevel(), MaterialRenderProxy);
	if (MaterialResource->GetMaterialDomain() != MD_Volume)
	{
		return;
	}

	const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface = (IHeterogeneousVolumeInterface*)VolumetricMeshBatch.Mesh->Elements[VolumeInstanceIndex].UserData;
	if (HeterogeneousVolumeInterface == nullptr)
	{
		return;
	}

	// Add preshading uniform buffer resource to SBT
	TUniformBufferRef<FPreshadingUniformBufferParameters> PreshadingUniformBuffer = CreateUniformBufferImmediate(PreshadingUniformBufferParameters, EUniformBufferUsage::UniformBuffer_SingleFrame);
	Scene->RayTracingSBT.TransientUniformBuffers.Add(PreshadingUniformBuffer);

	const FMaterialShaderMap* MaterialShaderMap = MaterialResource->GetRenderingThreadShaderMap();
	BindCallableShader<FPathTracingHeterogeneousVolumesIntersectCS>(
		Scene,
		View,
		MaterialRenderProxy,
		MaterialResource,
		MaterialShaderMap,
		PreshadingUniformBuffer,
		CallableInfo.IntersectShaderId
	);

	BindCallableShader<FPathTracingHeterogeneousVolumesGetDensityBoundsCS>(
		Scene,
		View,
		MaterialRenderProxy,
		MaterialResource,
		MaterialShaderMap,
		PreshadingUniformBuffer,
		CallableInfo.GetDensityBoundsShaderId
	);

	BindCallableShader<FPathTracingHeterogeneousVolumesGetDensityCS>(
		Scene,
		View,
		MaterialRenderProxy,
		MaterialResource,
		MaterialShaderMap,
		PreshadingUniformBuffer,
		CallableInfo.GetDensityShaderId
	);

	BindCallableShader<FPathTracingHeterogeneousVolumesSampleDistanceCS>(
		Scene,
		View,
		MaterialRenderProxy,
		MaterialResource,
		MaterialShaderMap,
		PreshadingUniformBuffer,
		CallableInfo.SampleDistanceShaderId
	);

	BindCallableShader<FPathTracingHeterogeneousVolumesGetTransmittanceCS>(
		Scene,
		View,
		MaterialRenderProxy,
		MaterialResource,
		MaterialShaderMap,
		PreshadingUniformBuffer,
		CallableInfo.GetTransmittanceShaderId
	);

	// TODO: check that callable shader id index corresponds to volume id
}

enum class EPathTracingHeterogeneousVolumesPipeline
{
	Megavoxel,
	Callable
};

EPathTracingHeterogeneousVolumesPipeline GetPathTracingHeterogeneousVolumesPipeline()
{
	EPathTracingHeterogeneousVolumesPipeline Pipeline = EPathTracingHeterogeneousVolumesPipeline::Megavoxel;

	const bool bUseExperimental = CVarPathTracingExperimental.GetValueOnRenderThread();
	if (bUseExperimental &&
		CVarPathTracingHeterogeneousVolumesPipeline.GetValueOnRenderThread() == 1)
	{
		Pipeline = EPathTracingHeterogeneousVolumesPipeline::Callable;
	}
	return Pipeline;
}

static TAutoConsoleVariable<int32> CVarHeterogeneousVolumesMajorantFlush(
	TEXT("r.HeterogeneousVolumes.Majorant.Flush"),
	0,
	TEXT("Flushes the majorant cache (Default = 0)"),
	ECVF_RenderThreadSafe
);

void PreparePathTracingHeterogeneousVolumes(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const TArrayView<FViewInfo>& Views
)
{
	if (!ShouldCompileRayTracingCallableShadersForProject(Scene->GetShaderPlatform()))
	{
		return;
	}

	if (GetPathTracingHeterogeneousVolumesPipeline() == EPathTracingHeterogeneousVolumesPipeline::Megavoxel)
	{
		return;
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = Views[ViewIndex];
		if (View.ViewState == nullptr)
		{
			continue;
		}

		FPathTracingState* PathTracingState = GetPathTracingStateFromView(View);

		// Empty shader ids
		PathTracingState->HeterogeneousVolumesCallableInfo.Empty();

		if (CVarHeterogeneousVolumesMajorantFlush.GetValueOnAnyThread())
		{
			PathTracingState->HeterogeneousVolumesPersistentMajorantRT.Empty();
		}

		if (ShouldRenderHeterogeneousVolumesForView(View))
		{
			int32 VolumeId = 0;
			for (int32 MeshBatchIndex = 0; MeshBatchIndex < View.HeterogeneousVolumesMeshBatches.Num(); ++MeshBatchIndex)
			{
				FVolumetricMeshBatch VolumetricMeshBatch = View.HeterogeneousVolumesMeshBatches[MeshBatchIndex];
				const FMeshBatch* Mesh = VolumetricMeshBatch.Mesh;
				const FPrimitiveSceneProxy* PrimitiveSceneProxy = VolumetricMeshBatch.Proxy;
				const FMaterialRenderProxy* MaterialRenderProxy = Mesh->MaterialRenderProxy;
				if (!ShouldRenderMeshBatchWithHeterogeneousVolumes(Mesh, PrimitiveSceneProxy, View.GetFeatureLevel()))
				{
					continue;
				}

				for (int32 VolumeInstanceIndex = 0; VolumeInstanceIndex < Mesh->Elements.Num(); ++VolumeInstanceIndex)
				{
					const IHeterogeneousVolumeInterface* HeterogeneousVolumeInterface = (IHeterogeneousVolumeInterface*)Mesh->Elements[VolumeInstanceIndex].UserData;
					//check(HeterogeneousVolumeInterface != nullptr);
					if (HeterogeneousVolumeInterface == nullptr)
					{
						continue;
					}

					FHeterogeneousVolumesPrimitiveInstanceKey Key;
					Key.PersistentPrimitiveIndex = PrimitiveSceneProxy->GetPrimitiveSceneInfo()->GetPersistentIndex().Index;
					Key.InstanceIndex = VolumeInstanceIndex;

					TRefCountPtr<IPooledRenderTarget>& PersistentMajorantRT = PathTracingState->HeterogeneousVolumesPersistentMajorantRT.FindOrAdd(Key);
					FPathTracingHeterogeneousVolumesCallableInfo& CallableInfo = PathTracingState->HeterogeneousVolumesCallableInfo.AddDefaulted_GetRef();

					FPreshadingUniformBufferParameters PreshadingUniformBufferParameters;
					BuildPreshadingGrid(GraphBuilder, Scene, View, HeterogeneousVolumeInterface, MaterialRenderProxy,
						PersistentMajorantRT,
						CallableInfo,
						PreshadingUniformBufferParameters);
					PreparePathTracingHeterogeneousVolumeMaterial(
						GraphBuilder,
						Scene,
						View,
						VolumetricMeshBatch,
						VolumeInstanceIndex,
						PreshadingUniformBufferParameters,
						CallableInfo);
				}
			}
		}

		// Initialize shader ids if there are no volumes..
		if (PathTracingState->HeterogeneousVolumesCallableInfo.IsEmpty())
		{
			FPathTracingHeterogeneousVolumesCallableInfo& CallableInfo = PathTracingState->HeterogeneousVolumesCallableInfo.AddDefaulted_GetRef();
			CallableInfo.IntersectShaderId = -1;
			CallableInfo.GetDensityBoundsShaderId = -1;
			CallableInfo.GetDensityShaderId = -1;
			CallableInfo.SampleDistanceShaderId = -1;
			CallableInfo.GetTransmittanceShaderId = -1;
		}
	}
}

class FPathTracingBuildAdaptiveErrorTextureCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingBuildAdaptiveErrorTextureCS)
	SHADER_USE_PARAMETER_STRUCT(FPathTracingBuildAdaptiveErrorTextureCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, InputResolution)
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputMipSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputMip)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputMip)

	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FPathTracingBuildAdaptiveErrorTextureCS, TEXT("/Engine/Private/PathTracing/PathTracingBuildAdaptiveError.usf"), TEXT("PathTracingBuildAdaptiveErrorTextureCS"), SF_Compute);

class FPathTracingAdaptiveStartCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingAdaptiveStartCS)
	SHADER_USE_PARAMETER_STRUCT(FPathTracingAdaptiveStartCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, VarianceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, VarianceSampler)
		SHADER_PARAMETER(FIntVector, VarianceTextureDims)
		SHADER_PARAMETER(float, AdaptiveSamplingErrorThreshold)
		SHADER_PARAMETER(FIntPoint, TileTextureOffset)
		SHADER_PARAMETER(FIntPoint, DispatchDim)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>, NextActivePaths)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>, NumPathStates)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FPathTracingAdaptiveStartCS, TEXT("/Engine/Private/PathTracing/PathTracingAdaptiveStart.usf"), TEXT("PathTracingAdaptiveStartCS"), SF_Compute);



// Default miss shader (using the path tracing payload)
template <bool IsGPULightmass>
class TPathTracingDefaultMS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(TPathTracingDefaultMS, Global, );
public:

	TPathTracingDefaultMS() = default;
	TPathTracingDefaultMS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (IsGPULightmass)
		{
			return ShouldCompileGPULightmassShadersForProject(Parameters.Platform);
		}
		else
		{
			return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
		}
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		if (IsGPULightmass)
		{
			return ERayTracingPayloadType::GPULightmass;
		}
		else
		{
			return ERayTracingPayloadType::PathTracingMaterial;
		}
	}
			
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};

using FPathTracingDefaultMS  = TPathTracingDefaultMS<false>;
using FGPULightmassDefaultMS = TPathTracingDefaultMS<true>;
IMPLEMENT_SHADER_TYPE(template<>, FPathTracingDefaultMS , TEXT("/Engine/Private/PathTracing/PathTracingMissShader.usf"), TEXT("PathTracingDefaultMS"), SF_RayMiss);
IMPLEMENT_SHADER_TYPE(template<>, FGPULightmassDefaultMS, TEXT("/Engine/Private/PathTracing/PathTracingMissShader.usf"), TEXT("PathTracingDefaultMS"), SF_RayMiss);

FRHIRayTracingShader* GetPathTracingDefaultMissShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FPathTracingDefaultMS>().GetRayTracingShader();
}

FRHIRayTracingShader* GetGPULightmassDefaultMissShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FGPULightmassDefaultMS>().GetRayTracingShader();
}

void FDeferredShadingSceneRenderer::SetupPathTracingDefaultMissShader(FRHICommandList& RHICmdList, const FViewInfo& View)
{
	int32 MissShaderPipelineIndex = FindRayTracingMissShaderIndex(View.MaterialRayTracingData.PipelineState, GetPathTracingDefaultMissShader(View.ShaderMap), true);

	RHICmdList.SetRayTracingMissShader(
		View.MaterialRayTracingData.ShaderBindingTable,
		RAY_TRACING_MISS_SHADER_SLOT_DEFAULT,
		View.MaterialRayTracingData.PipelineState,
		MissShaderPipelineIndex,
		0, nullptr, 0);
}


BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLightFunctionParametersPathTracing, )
	SHADER_PARAMETER(FMatrix44f, LightFunctionTranslatedWorldToLight)
	SHADER_PARAMETER(FVector4f, LightFunctionParameters)
	SHADER_PARAMETER(FVector3f, LightFunctionParameters2)
	SHADER_PARAMETER(FVector3f, CameraRelativeLightPosition)
	SHADER_PARAMETER(int32    , EnableColoredLightFunctions)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLightFunctionParametersPathTracing, "PathTracingLightFunctionParameters");

static TUniformBufferRef<FLightFunctionParametersPathTracing> CreateLightFunctionParametersBufferPT(
	const FLightSceneInfo* LightSceneInfo,
	const FSceneView& View,
	EUniformBufferUsage Usage)
{
	FLightFunctionParametersPathTracing LightFunctionParameters;

	const FVector Scale = LightSceneInfo->Proxy->GetLightFunctionScale();
	// Switch x and z so that z of the user specified scale affects the distance along the light direction
	const FVector InverseScale = FVector(1.f / Scale.Z, 1.f / Scale.Y, 1.f / Scale.X);
	const FMatrix WorldToLight = LightSceneInfo->Proxy->GetWorldToLight() * FScaleMatrix(FVector(InverseScale));

	LightFunctionParameters.LightFunctionTranslatedWorldToLight = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToLight);

	const bool bIsSpotLight = LightSceneInfo->Proxy->GetLightType() == LightType_Spot;
	const bool bIsPointLight = LightSceneInfo->Proxy->GetLightType() == LightType_Point;
	const float TanOuterAngle = bIsSpotLight ? FMath::Tan(LightSceneInfo->Proxy->GetOuterConeAngle()) : 1.0f;

	const float ShadowFadeFraction = 1.0f;

	LightFunctionParameters.LightFunctionParameters = FVector4f(TanOuterAngle, ShadowFadeFraction, bIsSpotLight ? 1.0f : 0.0f, bIsPointLight ? 1.0f : 0.0f);

	const bool bRenderingPreviewShadowIndicator = false;

	LightFunctionParameters.LightFunctionParameters2 = FVector3f(
		LightSceneInfo->Proxy->GetLightFunctionFadeDistance(),
		LightSceneInfo->Proxy->GetLightFunctionDisabledBrightness(),
		bRenderingPreviewShadowIndicator ? 1.0f : 0.0f);

	LightFunctionParameters.CameraRelativeLightPosition = GetCamRelativeLightPosition(View.ViewMatrices, *LightSceneInfo);

	LightFunctionParameters.EnableColoredLightFunctions = CVarPathTracingLightFunctionColor.GetValueOnRenderThread();

	return CreateUniformBufferImmediate(LightFunctionParameters, Usage);
}

// Miss Shader implementing light functions
class FPathTracingLightingMS : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FPathTracingLightingMS, Material);
	LAYOUT_FIELD(FShaderUniformBufferParameter, LightMaterialsParameter);

public:
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.MaterialParameters.MaterialDomain == MD_LightFunction && PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
	}

	FPathTracingLightingMS() {}
	FPathTracingLightingMS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMaterialShader(Initializer)
	{
		LightMaterialsParameter.Bind(Initializer.ParameterMap, TEXT("PathTracingLightFunctionParameters"));
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FViewInfo& View,
		const TUniformBufferRef<FLightFunctionParametersPathTracing>& LightFunctionParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMaterialShader::GetShaderBindings(Scene, FeatureLevel, MaterialRenderProxy, Material, ShaderBindings);
		ShaderBindings.Add(GetUniformBufferParameter<FViewUniformShaderParameters>(), View.ViewUniformBuffer);
		ShaderBindings.Add(LightMaterialsParameter, LightFunctionParameters);

		// LightFunctions can use primitive data, set identity so we do not crash on a missing binding
		ShaderBindings.Add(GetUniformBufferParameter<FPrimitiveUniformShaderParameters>(), GIdentityPrimitiveUniformBuffer);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::PathTracingMaterial;
	}
			
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FPathTracingLightingMS, TEXT("/Engine/Private/PathTracing/PathTracingLightingMissShader.usf"), TEXT("PathTracingLightingMS"), SF_RayMiss);


static void BindLightFunction(
	FRHICommandList& RHICmdList,
	const FScene* Scene,
	const FViewInfo& View,
	const FMaterial& Material,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const TUniformBufferRef<FLightFunctionParametersPathTracing>& LightFunctionParameters,
	int32 Index
)
{
	FRHIShaderBindingTable* SBT = View.MaterialRayTracingData.ShaderBindingTable;
	FRayTracingPipelineState* Pipeline = View.MaterialRayTracingData.PipelineState;
	const FMaterialShaderMap* MaterialShaderMap = Material.GetRenderingThreadShaderMap();

	TShaderRef<FPathTracingLightingMS> Shader = MaterialShaderMap->GetShader<FPathTracingLightingMS>();

	FMeshDrawShaderBindings ShaderBindings;
	ShaderBindings.Initialize(Shader);

	FMeshDrawSingleShaderBindings SingleShaderBindings = ShaderBindings.GetSingleShaderBindings(SF_RayMiss);

	Shader->GetShaderBindings(Scene, Scene->GetFeatureLevel(), MaterialRenderProxy, Material, View, LightFunctionParameters, SingleShaderBindings);

	int32 MissShaderPipelineIndex = FindRayTracingMissShaderIndex(View.MaterialRayTracingData.PipelineState, Shader.GetRayTracingShader(), false);

	if (MissShaderPipelineIndex == INDEX_NONE)
	{
		MissShaderPipelineIndex = FindRayTracingMissShaderIndex(View.MaterialRayTracingData.PipelineState, GetPathTracingDefaultMissShader(View.ShaderMap), true);
	}

	ShaderBindings.SetRayTracingShaderBindingsForMissShader(RHICmdList, SBT, Index, Pipeline, MissShaderPipelineIndex);
}

void BindLightFunctionShadersPathTracing(
	FRHICommandList& RHICmdList,
	const FScene* Scene,
	const FRayTracingLightFunctionMap* RayTracingLightFunctionMap,
	const class FViewInfo& View)
{
	if (RayTracingLightFunctionMap == nullptr)
	{
		return;
	}
	for (const FRayTracingLightFunctionMap::ElementType& LightAndIndex : *RayTracingLightFunctionMap)
	{
		const FLightSceneInfo* LightSceneInfo = LightAndIndex.Key;

		const FMaterialRenderProxy* MaterialProxy = LightSceneInfo->Proxy->GetLightFunctionMaterial();
		check(MaterialProxy != nullptr);
		// Catch the fallback material case
		const FMaterialRenderProxy* FallbackMaterialRenderProxyPtr = nullptr;
		const FMaterial& Material = MaterialProxy->GetMaterialWithFallback(Scene->GetFeatureLevel(), FallbackMaterialRenderProxyPtr);

		check(Material.IsLightFunction());

		const FMaterialRenderProxy& MaterialRenderProxy = FallbackMaterialRenderProxyPtr ? *FallbackMaterialRenderProxyPtr : *MaterialProxy;

		TUniformBufferRef<FLightFunctionParametersPathTracing> LightFunctionParameters = CreateLightFunctionParametersBufferPT(LightSceneInfo, View, EUniformBufferUsage::UniformBuffer_SingleFrame);

		int32 MissIndex = LightAndIndex.Value;
		BindLightFunction(RHICmdList, Scene, View, Material, MaterialRenderProxy, LightFunctionParameters, MissIndex);
	}
}


FRayTracingLightFunctionMap GatherLightFunctionLightsPathTracing(FScene* Scene, const FEngineShowFlags EngineShowFlags, ERHIFeatureLevel::Type InFeatureLevel)
{
	checkf(EngineShowFlags.LightFunctions, TEXT("This function should not be called if light functions are disabled"));
	FRayTracingLightFunctionMap RayTracingLightFunctionMap;
	for (const FLightSceneInfoCompact& Light : Scene->Lights)
	{
		FLightSceneInfo* LightSceneInfo = Light.LightSceneInfo;
		auto MaterialProxy = LightSceneInfo->Proxy->GetLightFunctionMaterial();
		if (MaterialProxy)
		{
			const FMaterialRenderProxy* FallbackMaterialRenderProxyPtr = nullptr;
			const FMaterial& Material = MaterialProxy->GetMaterialWithFallback(InFeatureLevel, FallbackMaterialRenderProxyPtr);
			if (Material.IsLightFunction())
			{
				const FMaterialShaderMap* MaterialShaderMap = Material.GetRenderingThreadShaderMap();
				// Getting the shader here has the side-effect of populating the raytracing miss shader library which is used when building the raytracing pipeline
				MaterialShaderMap->GetShader<FPathTracingLightingMS>().GetRayTracingShader();

				int32 Index = Scene->RayTracingSBT.NumMissShaderSlots;
				Scene->RayTracingSBT.NumMissShaderSlots++;
				RayTracingLightFunctionMap.Add(LightSceneInfo, Index);
			}
		}
	}
	return RayTracingLightFunctionMap;
}

static bool NeedsAnyHitShader(EBlendMode BlendMode)
{
	switch (BlendMode)
	{
		case BLEND_Opaque: 							return false; // always hit
		case BLEND_Masked: 							return true;  // runs shader (NOTE: dithered masking gets turned into translucent for the path tracer)
		case BLEND_Translucent: 					return true;  // casts transparent (colored) shadows depending on the shading model setup (fake caustics or transparent shadows)
		case BLEND_Additive: 						return false; // never hit for shadows, goes through the default shader instead, so no need to use AHS for primary rays
		case BLEND_Modulate: 						return true;  // casts colored shadows
		case BLEND_AlphaComposite: 					return true;
		case BLEND_AlphaHoldout: 					return false; // treat as opaque for shadows
		case BLEND_TranslucentColoredTransmittance: return true;  // NOTE: Substrate only
		default: checkf(false, TEXT("Unhandled blend mode %d"), int(BlendMode)); return false;
	}

}

static bool NeedsAnyHitShader(const FMaterial& RESTRICT MaterialResource)
{
	return NeedsAnyHitShader(MaterialResource.GetBlendMode());
}

template<bool UseAnyHitShader, bool UseIntersectionShader, bool IsGPULightmass, bool SimplifySubstrate>
class TPathTracingMaterial : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(TPathTracingMaterial, MeshMaterial);
public:
	TPathTracingMaterial() = default;

	TPathTracingMaterial(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		if (!ShouldCompileRayTracingShadersForProject(Parameters.Platform))
		{
			// is raytracing enabled at all?
			return false;
		}
		if (!Parameters.VertexFactoryType->SupportsRayTracing())
		{
			// does the VF support ray tracing at all?
			return false;
		}
		if (Parameters.MaterialParameters.MaterialDomain != MD_Surface)
		{
			// This material is only for surfaces at the moment
			return false;
		}
		if (NeedsAnyHitShader(Parameters.MaterialParameters.BlendMode) != UseAnyHitShader)
		{
			return false;
		}
		const bool bUseProceduralPrimitive = Parameters.VertexFactoryType->SupportsRayTracingProceduralPrimitive() && FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(Parameters.Platform);
		if (UseIntersectionShader != bUseProceduralPrimitive)
		{
			// only need to compile the intersection shader permutation if the VF actually requires it
			return false;
		}		
		if (IsGPULightmass)
		{
			return ShouldCompileGPULightmassShadersForProject(Parameters);
		}
		else
		{
			if (SimplifySubstrate && (!Substrate::IsSubstrateEnabled() || CVarPathTracingSubstrateCompileSimplifiedMaterial.GetValueOnAnyThread() == false))
			{
				// don't compile the extra Substrate permutation if:
				//    Substrate is not enabled on this project
				// or the user did not request the extra permutations to be compiled (default)
				return false;
			}
			return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
		}
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("USE_MATERIAL_CLOSEST_HIT_SHADER"), 1);
		OutEnvironment.SetDefine(TEXT("USE_MATERIAL_ANY_HIT_SHADER"), UseAnyHitShader ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("USE_MATERIAL_INTERSECTION_SHADER"), UseIntersectionShader ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("USE_RAYTRACED_TEXTURE_RAYCONE_LOD"), 0);
		OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1);
		OutEnvironment.SetDefine(TEXT("SIMPLIFIED_MATERIAL_SHADER"), IsGPULightmass);
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_USE_FULLYSIMPLIFIED_MATERIAL"), IsGPULightmass || SimplifySubstrate);
		OutEnvironment.SetDefine(TEXT("HAIRSTRANDS_HAS_NORMAL_CONNECTED"), Parameters.MaterialParameters.bHasNormalConnected ? 1 : 0);
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		if (ParameterMap.ContainsParameterAllocation(FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
		{
			OutError.Add(TEXT("Ray tracing closest hit shaders cannot read from the SceneTexturesStruct."));
			return false;
		}

		for (const auto& It : ParameterMap.GetParameterMap())
		{
			const FParameterAllocation& ParamAllocation = It.Value;
			if (ParamAllocation.Type != EShaderParameterType::UniformBuffer
				&& ParamAllocation.Type != EShaderParameterType::LooseData)
			{
				OutError.Add(FString::Printf(TEXT("Invalid ray tracing shader parameter '%s'. Only uniform buffers and loose data parameters are supported."), *(It.Key)));
				return false;
			}
		}

		return true;
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		if (IsGPULightmass)
		{
			return ERayTracingPayloadType::GPULightmass;
		}
		else
		{
			return ERayTracingPayloadType::PathTracingMaterial;
		}
	}
			
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		// GPULM does not use shader binding layout
		return IsGPULightmass ? nullptr : RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};


// TODO: It would be nice to avoid this template boilerplate and just use ordinary permutations. This would require allowing the FunctionName for the material to be dependent on the permutation somehow
using FPathTracingMaterialCHS        = TPathTracingMaterial<false, false, false, false>;
using FPathTracingMaterialCHS_AHS    = TPathTracingMaterial<true , false, false, false>;
using FPathTracingMaterialCHS_IS     = TPathTracingMaterial<false, true , false, false>;
using FPathTracingMaterialCHS_AHS_IS = TPathTracingMaterial<true , true , false, false>;
using FPathTracingMaterialSimplifiedCHS        = TPathTracingMaterial<false, false, false, true>;
using FPathTracingMaterialSimplifiedCHS_AHS    = TPathTracingMaterial<true , false, false, true>;
using FPathTracingMaterialSimplifiedCHS_IS     = TPathTracingMaterial<false, true , false, true>;
using FPathTracingMaterialSimplifiedCHS_AHS_IS = TPathTracingMaterial<true , true , false, true>;


// NOTE: lightmass doesn't work with intersection shader VFs at the moment, so avoid instantiating permutations that will never generate any shaders
// Also lightmass is always using simplified Substrate mode.
using FGPULightmassCHS               = TPathTracingMaterial<false, false, true, true>;
using FGPULightmassCHS_AHS           = TPathTracingMaterial<true , false, true, true>;

IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FPathTracingMaterialCHS       , TEXT("/Engine/Private/PathTracing/PathTracingMaterialHitShader.usf"), TEXT("closesthit=PathTracingMaterialCHS"), SF_RayHitGroup);
IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FPathTracingMaterialCHS_AHS   , TEXT("/Engine/Private/PathTracing/PathTracingMaterialHitShader.usf"), TEXT("closesthit=PathTracingMaterialCHS anyhit=PathTracingMaterialAHS"), SF_RayHitGroup);
IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FPathTracingMaterialCHS_IS    , TEXT("/Engine/Private/PathTracing/PathTracingMaterialHitShader.usf"), TEXT("closesthit=PathTracingMaterialCHS intersection=MaterialIS"), SF_RayHitGroup);
IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FPathTracingMaterialCHS_AHS_IS, TEXT("/Engine/Private/PathTracing/PathTracingMaterialHitShader.usf"), TEXT("closesthit=PathTracingMaterialCHS anyhit=PathTracingMaterialAHS intersection=MaterialIS"), SF_RayHitGroup);
IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FPathTracingMaterialSimplifiedCHS       , TEXT("/Engine/Private/PathTracing/PathTracingMaterialHitShader.usf"), TEXT("closesthit=PathTracingMaterialCHS"), SF_RayHitGroup);
IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FPathTracingMaterialSimplifiedCHS_AHS   , TEXT("/Engine/Private/PathTracing/PathTracingMaterialHitShader.usf"), TEXT("closesthit=PathTracingMaterialCHS anyhit=PathTracingMaterialAHS"), SF_RayHitGroup);
IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FPathTracingMaterialSimplifiedCHS_IS    , TEXT("/Engine/Private/PathTracing/PathTracingMaterialHitShader.usf"), TEXT("closesthit=PathTracingMaterialCHS intersection=MaterialIS"), SF_RayHitGroup);
IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FPathTracingMaterialSimplifiedCHS_AHS_IS, TEXT("/Engine/Private/PathTracing/PathTracingMaterialHitShader.usf"), TEXT("closesthit=PathTracingMaterialCHS anyhit=PathTracingMaterialAHS intersection=MaterialIS"), SF_RayHitGroup);

IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FGPULightmassCHS              , TEXT("/Engine/Private/PathTracing/PathTracingGPULightmassMaterialHitShader.usf"), TEXT("closesthit=GPULightmassMaterialCHS"), SF_RayHitGroup);
IMPLEMENT_MATERIAL_SHADER_TYPE(template <>, FGPULightmassCHS_AHS          , TEXT("/Engine/Private/PathTracing/PathTracingGPULightmassMaterialHitShader.usf"), TEXT("closesthit=GPULightmassMaterialCHS anyhit=GPULightmassMaterialAHS"), SF_RayHitGroup);

template <bool IsGPULightmass, bool IsOpaque>
class TPathTracingDefaultHitGroup : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(TPathTracingDefaultHitGroup)
	SHADER_USE_ROOT_PARAMETER_STRUCT(TPathTracingDefaultHitGroup, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (IsGPULightmass)
		{
			return ShouldCompileGPULightmassShadersForProject(Parameters.Platform);
		}
		else
		{
			return PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform);
		}
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		if (IsGPULightmass)
		{
			return ERayTracingPayloadType::GPULightmass;
		}
		else
		{
			return ERayTracingPayloadType::PathTracingMaterial;
		}
	}
			
	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	using FParameters = FEmptyShaderParameters;
};

using FPathTracingDefaultOpaqueHitGroup  = TPathTracingDefaultHitGroup<false, true >;
using FPathTracingDefaultHiddenHitGroup  = TPathTracingDefaultHitGroup<false, false>;
using FGPULightmassDefaultOpaqueHitGroup = TPathTracingDefaultHitGroup<true , true >;
using FGPULightmassDefaultHiddenHitGroup = TPathTracingDefaultHitGroup<true , false>;

IMPLEMENT_SHADER_TYPE(template<>, FPathTracingDefaultOpaqueHitGroup , TEXT("/Engine/Private/PathTracing/PathTracingDefaultHitShader.usf"), TEXT("closesthit=PathTracingDefaultOpaqueCHS"), SF_RayHitGroup);
IMPLEMENT_SHADER_TYPE(template<>, FGPULightmassDefaultOpaqueHitGroup, TEXT("/Engine/Private/PathTracing/PathTracingDefaultHitShader.usf"), TEXT("closesthit=PathTracingDefaultOpaqueCHS"), SF_RayHitGroup);
IMPLEMENT_SHADER_TYPE(template<>, FPathTracingDefaultHiddenHitGroup , TEXT("/Engine/Private/PathTracing/PathTracingDefaultHitShader.usf"), TEXT("closesthit=PathTracingDefaultHiddenCHS anyhit=PathTracingDefaultHiddenAHS"), SF_RayHitGroup);
IMPLEMENT_SHADER_TYPE(template<>, FGPULightmassDefaultHiddenHitGroup, TEXT("/Engine/Private/PathTracing/PathTracingDefaultHitShader.usf"), TEXT("closesthit=PathTracingDefaultHiddenCHS anyhit=PathTracingDefaultHiddenAHS"), SF_RayHitGroup);

FRHIRayTracingShader* GetPathTracingDefaultOpaqueHitShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FPathTracingDefaultOpaqueHitGroup>().GetRayTracingShader();
}

FRHIRayTracingShader* GetGPULightmassDefaultOpaqueHitShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FGPULightmassDefaultOpaqueHitGroup>().GetRayTracingShader();
}

FRHIRayTracingShader* GetPathTracingDefaultHiddenHitShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FPathTracingDefaultHiddenHitGroup>().GetRayTracingShader();
}

FRHIRayTracingShader* GetGPULightmassDefaultHiddenHitShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FGPULightmassDefaultHiddenHitGroup>().GetRayTracingShader();
}

bool FRayTracingMeshProcessor::ProcessPathTracing(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource)
{
	FMaterialShaderTypes ShaderTypes;

	if (MaterialResource.GetMaterialDomain() == MD_DeferredDecal)
	{
		ShaderTypes.AddShaderType(GetRayTracingDecalMaterialShaderType(MaterialResource.GetBlendMode()));
	}
	else
	{
		const bool bUseProceduralPrimitive = MeshBatch.VertexFactory->GetType()->SupportsRayTracingProceduralPrimitive() &&
			FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(GMaxRHIShaderPlatform);
		switch (RayTracingType)
		{
			case ERayTracingType::PathTracing:
			{
				// In order to use Substrate simplified materials, Substrate has to be enabled, we have to have _compiled_ the extra permutations _and_ the runtime toggle must be true
				const bool bUseSimplifiedMaterial = Substrate::IsSubstrateEnabled() &&
					CVarPathTracingSubstrateCompileSimplifiedMaterial.GetValueOnRenderThread() &&
					CVarPathTracingSubstrateUseSimplifiedMaterial.GetValueOnRenderThread();
				if (NeedsAnyHitShader(MaterialResource))
				{
					if (bUseSimplifiedMaterial)
					{
						if (bUseProceduralPrimitive)
							ShaderTypes.AddShaderType<FPathTracingMaterialSimplifiedCHS_AHS_IS>();
						else
							ShaderTypes.AddShaderType<FPathTracingMaterialSimplifiedCHS_AHS>();
					}
					else
					{
						if (bUseProceduralPrimitive)
							ShaderTypes.AddShaderType<FPathTracingMaterialCHS_AHS_IS>();
						else
							ShaderTypes.AddShaderType<FPathTracingMaterialCHS_AHS>();
					}
				}
				else
				{
					if (bUseSimplifiedMaterial)
					{
						if (bUseProceduralPrimitive)
							ShaderTypes.AddShaderType<FPathTracingMaterialSimplifiedCHS_IS>();
						else
							ShaderTypes.AddShaderType<FPathTracingMaterialSimplifiedCHS>();
					}
					else
					{
						if (bUseProceduralPrimitive)
							ShaderTypes.AddShaderType<FPathTracingMaterialCHS_IS>();
						else
							ShaderTypes.AddShaderType<FPathTracingMaterialCHS>();
					}
				}
				break;
			}
			case ERayTracingType::LightMapTracing:
			{
				if (NeedsAnyHitShader(MaterialResource))
				{
					ShaderTypes.AddShaderType<FGPULightmassCHS_AHS>();
				}
				else
				{
					ShaderTypes.AddShaderType<FGPULightmassCHS>();
				}
				break;
			}
			default:
			{
				return false;
			}
		}
	}

	FMaterialShaders Shaders;
	if (!MaterialResource.TryGetShaders(ShaderTypes, MeshBatch.VertexFactory->GetType(), Shaders))
	{
		return false;
	}

	TShaderRef<FMeshMaterialShader> RayTracingShader;
	if (!Shaders.TryGetShader(SF_RayHitGroup, RayTracingShader))
	{
		return false;
	}

	TBasePassShaderElementData<FUniformLightMapPolicy> ShaderElementData(nullptr);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, -1, true);

	BuildRayTracingMeshCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		RayTracingShader,
		ShaderElementData);

	return true;
}

RENDERER_API void PrepareSkyTexture_Internal(
	FRDGBuilder& GraphBuilder,
	ERHIFeatureLevel::Type FeatureLevel,
	FReflectionUniformParameters& Parameters,
	uint32 Size,
	FLinearColor SkyColor,
	bool UseMISCompensation,

	// Out
	FRDGTextureRef& SkylightTexture,
	FRDGTextureRef& SkylightPdf,
	float& SkylightInvResolution,
	int32& SkylightMipCount
)
{
	FRDGTextureDesc SkylightTextureDesc = FRDGTextureDesc::Create2D(
		FIntPoint(Size, Size),
		PF_A32B32G32R32F, // Must use float as CubeMap * Color could have float range (could use half if we didn't include Color in the map)
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV);

	SkylightTexture = GraphBuilder.CreateTexture(SkylightTextureDesc, TEXT("PathTracer.Skylight"), ERDGTextureFlags::None);

	FRDGTextureDesc SkylightPdfDesc = FRDGTextureDesc::Create2D(
		FIntPoint(Size, Size),
		PF_R32_FLOAT, // Must use float as CubeMap * Color could have float range (could use half if we didn't include Color in the map)
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV,
		FMath::CeilLogTwo(Size) + 1);

	SkylightPdf = GraphBuilder.CreateTexture(SkylightPdfDesc, TEXT("PathTracer.SkylightPdf"), ERDGTextureFlags::None);

	SkylightInvResolution = 1.0f / Size;
	SkylightMipCount = SkylightPdfDesc.NumMips;

	// run a simple compute shader to sample the cubemap and prep the top level of the mipmap hierarchy
	{
		TShaderMapRef<FPathTracingSkylightPrepareCS> ComputeShader(GetGlobalShaderMap(FeatureLevel));
		FPathTracingSkylightPrepareCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingSkylightPrepareCS::FParameters>();
		PassParameters->SkyColor = FVector3f(SkyColor.R, SkyColor.G, SkyColor.B);
		PassParameters->SkyLightCubemap0 = Parameters.SkyLightCubemap;
		PassParameters->SkyLightCubemap1 = Parameters.SkyLightBlendDestinationCubemap;
		PassParameters->SkyLightCubemapSampler0 = Parameters.SkyLightCubemapSampler;
		PassParameters->SkyLightCubemapSampler1 = Parameters.SkyLightBlendDestinationCubemapSampler;
		PassParameters->SkylightBlendFactor = Parameters.SkyLightParameters.W;
		PassParameters->SkylightInvResolution = SkylightInvResolution;
		PassParameters->SkylightTextureOutput = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SkylightTexture, 0));
		PassParameters->SkylightTexturePdf = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SkylightPdf, 0));
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("SkylightPrepare"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(FIntPoint(Size, Size), FComputeShaderUtils::kGolden2DGroupSize));
	}
	FGenerateMips::ExecuteCompute(GraphBuilder, FeatureLevel, SkylightPdf, TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI());

	if (UseMISCompensation)
	{
		TShaderMapRef<FPathTracingSkylightMISCompensationCS> ComputeShader(GetGlobalShaderMap(FeatureLevel));
		FPathTracingSkylightMISCompensationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingSkylightMISCompensationCS::FParameters>();
		PassParameters->SkylightTexturePdfAverage = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SkylightPdf, SkylightMipCount - 1));
		PassParameters->SkylightTextureOutput = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SkylightTexture, 0));
		PassParameters->SkylightTexturePdf = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(SkylightPdf, 0));
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("SkylightMISCompensation"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(FIntPoint(Size, Size), FComputeShaderUtils::kGolden2DGroupSize));
		FGenerateMips::ExecuteCompute(GraphBuilder, FeatureLevel, SkylightPdf, TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI());
	}
}

RDG_REGISTER_BLACKBOARD_STRUCT(FPathTracingSkylight)

bool PrepareSkyTexture(FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View, bool SkylightEnabled, bool UseMISCompensation, FPathTracingSkylight* SkylightParameters)
{
	SkylightParameters->SkylightTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FReflectionUniformParameters Parameters;
	SetupReflectionUniformParameters(GraphBuilder, View, Parameters);
	if (!SkylightEnabled || !(Parameters.SkyLightParameters.Y > 0))
	{
		// textures not ready, or skylight not active
		// just put in a placeholder
		SkylightParameters->SkylightTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		SkylightParameters->SkylightPdf = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		SkylightParameters->SkylightInvResolution = 0;
		SkylightParameters->SkylightMipCount = 0;
		return false;
	}

	// the sky is actually enabled, lets see if someone already made use of it for this frame
	const FPathTracingSkylight* PreviousSkylightParameters = GraphBuilder.Blackboard.Get<FPathTracingSkylight>();
	if (PreviousSkylightParameters != nullptr)
	{
		*SkylightParameters = *PreviousSkylightParameters;
		return true;
	}

	// should we remember the skylight prep for the next frame?
	const bool IsSkylightCachingEnabled = CVarPathTracingSkylightCaching.GetValueOnAnyThread() != 0;
	const FLinearColor SkyColor = View.CachedViewUniformShaderParameters->SkyLightColor;
	const bool bSkylightColorChanged = SkyColor != Scene->PathTracingSkylightColor;
	if (!IsSkylightCachingEnabled || bSkylightColorChanged)
	{
		// we don't want any caching (or the light color changed)
		// release what we might have been holding onto so we get the right texture for this frame
		Scene->PathTracingSkylightTexture.SafeRelease();
		Scene->PathTracingSkylightPdf.SafeRelease();
	}

	if (Scene->PathTracingSkylightTexture.IsValid() &&
		Scene->PathTracingSkylightPdf.IsValid())
	{
		// we already have a valid texture and pdf, just re-use them!
		// it is the responsability of code that may invalidate the contents to reset these pointers
		SkylightParameters->SkylightTexture = GraphBuilder.RegisterExternalTexture(Scene->PathTracingSkylightTexture, TEXT("PathTracer.Skylight"));
		SkylightParameters->SkylightPdf = GraphBuilder.RegisterExternalTexture(Scene->PathTracingSkylightPdf, TEXT("PathTracer.SkylightPdf"));
		SkylightParameters->SkylightInvResolution = 1.0f / SkylightParameters->SkylightTexture->Desc.GetSize().X;
		SkylightParameters->SkylightMipCount = SkylightParameters->SkylightPdf->Desc.NumMips;
		return true;
	}
	RDG_EVENT_SCOPE(GraphBuilder, "Path Tracing SkylightPrepare");
	Scene->PathTracingSkylightColor = SkyColor;
	// since we are resampled into an octahedral layout, we multiply the cubemap resolution by 2 to get roughly the same number of texels
	uint32 Size = FMath::RoundUpToPowerOfTwo(2 * Scene->SkyLight->CaptureCubeMapResolution);
	
	RDG_GPU_MASK_SCOPE(GraphBuilder, 
		IsSkylightCachingEnabled ? FRHIGPUMask::All() : GraphBuilder.RHICmdList.GetGPUMask());

	PrepareSkyTexture_Internal(
		GraphBuilder,
		View.FeatureLevel,
		Parameters,
		Size,
		SkyColor,
		UseMISCompensation,
		// Out
		SkylightParameters->SkylightTexture,
		SkylightParameters->SkylightPdf,
		SkylightParameters->SkylightInvResolution,
		SkylightParameters->SkylightMipCount
	);

	// hang onto these for next time (if caching is enabled)
	if (IsSkylightCachingEnabled)
	{
		GraphBuilder.QueueTextureExtraction(SkylightParameters->SkylightTexture, &Scene->PathTracingSkylightTexture);
		GraphBuilder.QueueTextureExtraction(SkylightParameters->SkylightPdf, &Scene->PathTracingSkylightPdf);
	}

	// remember the skylight parameters for future passes within this frame
	GraphBuilder.Blackboard.Create<FPathTracingSkylight>() = *SkylightParameters;

	return true;
}

RENDERER_API void PrepareLightGrid(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FPathTracingLightGrid* LightGridParameters, const FPathTracingLight* Lights, uint32 NumLights, uint32 NumInfiniteLights, FRDGBufferSRV* LightsSRV)
{
	const float Inf = std::numeric_limits<float>::infinity();
	LightGridParameters->SceneInfiniteLightCount = NumInfiniteLights;
	LightGridParameters->SceneLightsTranslatedBoundMin = FVector3f(+Inf, +Inf, +Inf);
	LightGridParameters->SceneLightsTranslatedBoundMax = FVector3f(-Inf, -Inf, -Inf);
	LightGridParameters->LightGrid = nullptr;
	LightGridParameters->LightGridData = nullptr;

	int NumFiniteLights = NumLights - NumInfiniteLights;
	// if we have some finite lights -- build a light grid
	if (NumFiniteLights > 0)
	{
		// get bounding box of all finite lights
		const FPathTracingLight* FiniteLights = Lights + NumInfiniteLights;
		for (int Index = 0; Index < NumFiniteLights; Index++)
		{
			const FPathTracingLight& Light = FiniteLights[Index];
			FBox3f Box;

			const float Radius = 1.0f / Light.Attenuation;
			const FVector3f Center = Light.TranslatedWorldPosition;
			const FVector3f Normal = Light.Normal;
			switch (Light.Flags & PATHTRACER_FLAG_TYPE_MASK)
			{
				case PATHTRACING_LIGHT_POINT:
				{
					Box = GetPointLightBounds(Center, Radius);
					break;
				}
				case PATHTRACING_LIGHT_SPOT:
				{
					Box = GetSpotLightBounds(Center, Normal, Radius, Light.Shaping.X);
					break;
				}
				case PATHTRACING_LIGHT_RECT:
				{
					Box = GetRectLightBounds(Center, Normal, Light.Tangent, Light.Dimensions.X * 0.5f, Light.Dimensions.Y * 0.5f, Radius, Light.Shaping.X, Light.Shaping.Y);
					break;
				}
				default:
				{
					// non-finite lights should not appear in this case
					checkNoEntry();
					break;
				}
			}
			LightGridParameters->SceneLightsTranslatedBoundMin = FVector3f::Min(LightGridParameters->SceneLightsTranslatedBoundMin, Box.Min);
			LightGridParameters->SceneLightsTranslatedBoundMax = FVector3f::Max(LightGridParameters->SceneLightsTranslatedBoundMax, Box.Max);
		}

		const uint32 Resolution = FMath::Clamp(CVarPathTracingLightGridResolution.GetValueOnRenderThread(), 1, 2048);
		const uint32 MaxCount = FMath::Clamp(
			CVarPathTracingLightGridMaxCount.GetValueOnRenderThread(),
			1,
			FMath::Min(NumFiniteLights, RAY_TRACING_LIGHT_COUNT_MAXIMUM)
		);
		LightGridParameters->LightGridResolution = Resolution;
		LightGridParameters->LightGridMaxCount = MaxCount;

		LightGridParameters->LightGridAxis = CVarPathTracingLightGridAxis.GetValueOnRenderThread();
		FPathTracingBuildLightGridCS::FParameters* LightGridPassParameters = GraphBuilder.AllocParameters< FPathTracingBuildLightGridCS::FParameters>();

		FRDGTextureDesc LightGridDesc = FRDGTextureDesc::Create2DArray(
			FIntPoint(Resolution, Resolution),
			PF_R32_UINT,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV, 3);

		// jhoerner TODO 9/30/2022: Hack to work around MGPU resource transition architectural bug in RDG.  Mask PathTracer.LightGrid texture
		// to only be present on current GPU.  The bug is that RDG batches transitions, but the execution of batched transitions uses the
		// GPU Mask of the current Pass that's executing, not the GPU Mask that's relevant to the Passes where a given resource is used.  This
		// causes an assert due to a mismatch in the expected transition state on a specific GPU, when an intermediate transition was skipped
		// on that GPU, due to the arbitrary nature of the GPU mask when a transition batch is flushed.  The hack works by removing the
		// resource from GPUs it's not actually used on, where the intermediate transition gets skipped.
		LightGridDesc.GPUMask = GraphBuilder.RHICmdList.GetGPUMask();

		FRDGTexture* LightGridTexture = GraphBuilder.CreateTexture(LightGridDesc, TEXT("PathTracer.LightGrid"), ERDGTextureFlags::None);
		LightGridPassParameters->RWLightGrid = GraphBuilder.CreateUAV(LightGridTexture);

		EPixelFormat LightGridDataFormat = PF_R32_UINT;
		size_t LightGridDataNumBytes = sizeof(uint32);
		if (NumLights <= (MAX_uint8 + 1))
		{
			LightGridDataFormat = PF_R8_UINT;
			LightGridDataNumBytes = sizeof(uint8);
		}
		else if (NumLights <= (MAX_uint16 + 1))
		{
			LightGridDataFormat = PF_R16_UINT;
			LightGridDataNumBytes = sizeof(uint16);
		}
		FRDGBufferDesc LightGridDataDesc = FRDGBufferDesc::CreateBufferDesc(LightGridDataNumBytes, 3 * MaxCount * Resolution * Resolution);
		FRDGBuffer* LightGridData = GraphBuilder.CreateBuffer(LightGridDataDesc, TEXT("PathTracer.LightGridData"));
		LightGridPassParameters->RWLightGridData = GraphBuilder.CreateUAV(LightGridData, LightGridDataFormat);
		LightGridPassParameters->LightGridParameters = *LightGridParameters;
		LightGridPassParameters->SceneLights = LightsSRV;
		LightGridPassParameters->SceneLightCount = NumLights;

		TShaderMapRef<FPathTracingBuildLightGridCS> ComputeShader(GetGlobalShaderMap(FeatureLevel));
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Light Grid Create (%u lights)", NumFiniteLights),
			ComputeShader,
			LightGridPassParameters,
			FComputeShaderUtils::GetGroupCount(FIntVector(Resolution, Resolution, 3), FIntVector(FComputeShaderUtils::kGolden2DGroupSize, FComputeShaderUtils::kGolden2DGroupSize, 1)));

		// hookup to the actual rendering pass
		LightGridParameters->LightGrid = LightGridTexture;
		LightGridParameters->LightGridData = GraphBuilder.CreateSRV(LightGridData, LightGridDataFormat);


	}
	else
	{
		// light grid is not needed - just hookup dummy data
		LightGridParameters->LightGridResolution = 0;
		LightGridParameters->LightGridMaxCount = 0;
		LightGridParameters->LightGridAxis = 0;
		LightGridParameters->LightGrid = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		FRDGBufferDesc LightGridDataDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1);
		FRDGBuffer* LightGridData = GraphBuilder.CreateBuffer(LightGridDataDesc, TEXT("PathTracer.LightGridData"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(LightGridData, PF_R32_UINT), 0);
		LightGridParameters->LightGridData = GraphBuilder.CreateSRV(LightGridData, PF_R32_UINT);
	}
}

uint32 PackRG16(float In0, float In1);

void SetLightParameters(
	FRDGBuilder& GraphBuilder,
	FScene* Scene,
	const FViewInfo& View,
	const bool UseMISCompensation,
	const bool bUseAtmosphere,
	// output args
	FPathTracingSkylight* SkylightParameters,
	FPathTracingLightGrid* LightGridParameters,
	uint32* SceneVisibleLightCount,
	uint32* SceneLightCount,
	FRDGBufferSRVRef* SceneLights
)
{
	check(SkylightParameters != nullptr);
	check(SceneVisibleLightCount != nullptr);
	check(SceneLightCount != nullptr);
	check(SceneLights != nullptr);
	*SceneVisibleLightCount = 0;

	// Lights
	uint32 MaxNumLights = 1 + Scene->Lights.Num(); // upper bound
	// Allocate from the graph builder so that we don't need to copy the data again when queuing the upload
	FPathTracingLight* Lights = (FPathTracingLight*) GraphBuilder.Alloc(sizeof(FPathTracingLight) * MaxNumLights, 16);
	uint32 NumLights = 0;

	// Prepend SkyLight to light buffer since it is not part of the regular light list
	// skylight should be excluded if we are using the reference atmosphere calculation (don't bother checking again if an atmosphere is present)
	const bool bEnableSkydome = !bUseAtmosphere;
	if (PrepareSkyTexture(GraphBuilder, Scene, View, bEnableSkydome, UseMISCompensation, SkylightParameters))
	{
		check(Scene->SkyLight != nullptr);
		FPathTracingLight& DestLight = Lights[NumLights++];
		DestLight.Color = FVector3f(1, 1, 1); // not used (it is folded into the importance table directly)
		DestLight.Flags = Scene->SkyLight->bTransmission ? PATHTRACER_FLAG_TRANSMISSION_MASK : 0;
		DestLight.Flags |= PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK;
		DestLight.Flags |= PATHTRACING_LIGHT_SKY;
		DestLight.Flags |= Scene->SkyLight->bCastShadows ? PATHTRACER_FLAG_CAST_SHADOW_MASK : 0;
		DestLight.Flags |= Scene->SkyLight->bCastVolumetricShadow ? PATHTRACER_FLAG_CAST_VOL_SHADOW_MASK : 0;
		DestLight.DiffuseSpecularScale = PackRG16(1.f, 1.f);
		DestLight.IndirectLightingScale = Scene->SkyLight->IndirectLightingIntensity;
		DestLight.VolumetricScatteringIntensity = Scene->SkyLight->VolumetricScatteringIntensity;
		DestLight.IESAtlasIndex = INDEX_NONE;
		DestLight.MissShaderIndex = 0;
		DestLight.CameraColorScale = FVector3f(1, 1, 1);
		if ((Scene->SkyLight->bRealTimeCaptureEnabled && (View.SkyAtmosphereUniformShaderParameters == nullptr ||  !IsSkyAtmosphereHoldout(View.CachedViewUniformShaderParameters->EnvironmentComponentsFlags))) || CVarPathTracingVisibleLights.GetValueOnRenderThread() == 2)
		{
			// When using the realtime capture system, always make the skylight visible
			// because this is our only way of "seeing" the atmo/clouds at the moment
			// The one exception to this case is if the sky atmo has been marked as holdout.

			// Also allow seeing just the sky via a cvar for debugging purposes
			*SceneVisibleLightCount = 1;

			if (Scene->SkyLight->bRealTimeCaptureEnabled)
			{
				// NOTE: this color is already baked into the skylight texture so that importance sampling takes it into account, we pass it in here so that camera rays can factor it out
				// This is only for the realtime capture case, because otherwise (specified cube map case) we want the displayed texture and lighting to match
				// NOTE: We want to use the effective light color, not the exposed skylight color SkyLightColor from the View UniformBuffer so that the overall intensity is correct (see jira UE-280734)
				DestLight.CameraColorScale /= FVector3f(Scene->SkyLight->GetEffectiveLightColor());
			}
		}
	}

	const FRayTracingLightFunctionMap* RayTracingLightFunctionMap = GraphBuilder.Blackboard.Get<FRayTracingLightFunctionMap>();

	// Add directional lights next (all lights with infinite bounds should come first)
	if (View.Family->EngineShowFlags.DirectionalLights)
	{
		for (const FLightSceneInfoCompact& Light : Scene->Lights)
		{
			ELightComponentType LightComponentType = (ELightComponentType)Light.LightSceneInfo->Proxy->GetLightType();

			if (LightComponentType != LightType_Directional)
			{
				continue;
			}

			FLightRenderParameters LightParameters;
			Light.LightSceneInfo->Proxy->GetLightShaderParameters(LightParameters);

			if (FVector3f(LightParameters.Color).IsZero())
			{
				continue;
			}

			FPathTracingLight& DestLight = Lights[NumLights++];
			uint32 Transmission = Light.LightSceneInfo->Proxy->Transmission();
			uint8 LightingChannelMask = Light.LightSceneInfo->Proxy->GetLightingChannelMask();

			DestLight.Flags = Transmission ? PATHTRACER_FLAG_TRANSMISSION_MASK : 0;
			DestLight.Flags |= LightingChannelMask & PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK;
			DestLight.Flags |= Light.LightSceneInfo->Proxy->CastsDynamicShadow() ? PATHTRACER_FLAG_CAST_SHADOW_MASK : 0;
			DestLight.Flags |= Light.LightSceneInfo->Proxy->CastsVolumetricShadow() ? PATHTRACER_FLAG_CAST_VOL_SHADOW_MASK : 0;
			DestLight.Flags |= Light.LightSceneInfo->Proxy->GetCastCloudShadows() ? PATHTRACER_FLAG_CAST_CLOUD_SHADOW_MASK : 0;
			DestLight.IESAtlasIndex = INDEX_NONE;
			DestLight.MissShaderIndex = 0;
			DestLight.CameraColorScale = FVector3f(Light.LightSceneInfo->Proxy->GetAtmosphereSunDiskColorScale());

			if (RayTracingLightFunctionMap)
			{
				const int32* LightFunctionIndex = RayTracingLightFunctionMap->Find(Light.LightSceneInfo);
				if (LightFunctionIndex)
				{
					DestLight.MissShaderIndex = *LightFunctionIndex;
				}
			}

			// these mean roughly the same thing across all light types
			if (bUseAtmosphere)
			{
				// we want the sky color to come in _without_ the sky transmittance baked in, so need to go back to the proxy
				DestLight.Color = FVector3f(Light.LightSceneInfo->Proxy->GetOuterSpaceIlluminance());
			}
			else
			{
				// this color has the sky transmittance already factored in
				DestLight.Color = FVector3f(LightParameters.Color);
			}

			DestLight.Color *= LightParameters.GetLightExposureScale(View.GetLastEyeAdaptationExposure());
			DestLight.TranslatedWorldPosition = FVector3f(LightParameters.WorldPosition + View.ViewMatrices.GetPreViewTranslation());
			DestLight.Normal = -LightParameters.Direction;
			DestLight.Tangent = LightParameters.Tangent;
			DestLight.Shaping = FVector2f(0.0f, 0.0f);
			DestLight.DiffuseSpecularScale = PackRG16(LightParameters.DiffuseScale, LightParameters.SpecularScale);
			DestLight.IndirectLightingScale = Light.LightSceneInfo->Proxy->GetIndirectLightingScale();
			DestLight.Attenuation = LightParameters.InvRadius;
			DestLight.FalloffExponent = 0;
			DestLight.VolumetricScatteringIntensity = Light.LightSceneInfo->Proxy->GetVolumetricScatteringIntensity();
			DestLight.RectLightAtlasUVOffset = FVector2f(0.0f, 0.0f);
			DestLight.RectLightAtlasUVScale = FVector2f(0.0f, 0.0f);

			DestLight.Normal = LightParameters.Direction;
			DestLight.Dimensions = FVector2f(LightParameters.SourceRadius, 0.0f);
			DestLight.Flags |= PATHTRACING_LIGHT_DIRECTIONAL;
		}
	}

	if (bUseAtmosphere && (View.SkyAtmosphereUniformShaderParameters == nullptr ||  !IsSkyAtmosphereHoldout(View.CachedViewUniformShaderParameters->EnvironmentComponentsFlags)))
	{
		// show directional lights when atmosphere is enabled and not marked as holdout
		// NOTE: there cannot be any skydome in this case
		*SceneVisibleLightCount = NumLights;
	}

	uint32 NumInfiniteLights = NumLights;

	int32 NextRectTextureIndex = 0;

	for (const FLightSceneInfoCompact& Light : Scene->Lights)
	{
		ELightComponentType LightComponentType = (ELightComponentType)Light.LightSceneInfo->Proxy->GetLightType();

		if ( (LightComponentType == LightType_Directional) /* already handled by the loop above */  ||
			((LightComponentType == LightType_Rect       ) && !View.Family->EngineShowFlags.RectLights       ) ||
			((LightComponentType == LightType_Spot       ) && !View.Family->EngineShowFlags.SpotLights       ) ||
			((LightComponentType == LightType_Point      ) && !View.Family->EngineShowFlags.PointLights      ))
		{
			// This light type is not currently enabled
			continue;
		}

		FLightRenderParameters LightParameters;
		Light.LightSceneInfo->Proxy->GetLightShaderParameters(LightParameters);

		if (FVector3f(LightParameters.Color).IsZero())
		{
			continue;
		}

		FPathTracingLight& DestLight = Lights[NumLights++];

		uint32 Transmission = Light.LightSceneInfo->Proxy->Transmission();
		uint8 LightingChannelMask = Light.LightSceneInfo->Proxy->GetLightingChannelMask();

		DestLight.Flags = Transmission ? PATHTRACER_FLAG_TRANSMISSION_MASK : 0;
		DestLight.Flags |= LightingChannelMask & PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK;
		DestLight.Flags |= Light.LightSceneInfo->Proxy->CastsDynamicShadow() ? PATHTRACER_FLAG_CAST_SHADOW_MASK : 0;
		DestLight.Flags |= Light.LightSceneInfo->Proxy->CastsVolumetricShadow() ? PATHTRACER_FLAG_CAST_VOL_SHADOW_MASK : 0;
		DestLight.Flags |= Light.LightSceneInfo->Proxy->GetCastCloudShadows() ? PATHTRACER_FLAG_CAST_CLOUD_SHADOW_MASK : 0;
		DestLight.IESAtlasIndex = LightParameters.IESAtlasIndex;
		DestLight.MissShaderIndex = 0;

		// these mean roughly the same thing across all light types
		DestLight.Color = FVector3f(LightParameters.Color) * LightParameters.GetLightExposureScale(View.GetLastEyeAdaptationExposure());
		DestLight.TranslatedWorldPosition = FVector3f(LightParameters.WorldPosition + View.ViewMatrices.GetPreViewTranslation());
		DestLight.Normal = -LightParameters.Direction;
		DestLight.Tangent = LightParameters.Tangent;
		DestLight.Shaping = FVector2f(0.0f, 0.0f);
		DestLight.DiffuseSpecularScale = PackRG16(LightParameters.DiffuseScale, LightParameters.SpecularScale);
		DestLight.IndirectLightingScale = Light.LightSceneInfo->Proxy->GetIndirectLightingScale();
		DestLight.Attenuation = LightParameters.InvRadius;
		DestLight.FalloffExponent = 0;
		DestLight.VolumetricScatteringIntensity = Light.LightSceneInfo->Proxy->GetVolumetricScatteringIntensity();
		DestLight.RectLightAtlasUVOffset = FVector2f(0.0f, 0.0f);
		DestLight.RectLightAtlasUVScale = FVector2f(0.0f, 0.0f);

		DestLight.CameraColorScale = FVector3f(1, 1, 1);

		if (RayTracingLightFunctionMap)
		{
			const int32* LightFunctionIndex = RayTracingLightFunctionMap->Find(Light.LightSceneInfo);
			if (LightFunctionIndex)
			{
				DestLight.MissShaderIndex = *LightFunctionIndex;
			}
		}

		switch (LightComponentType)
		{
			case LightType_Rect:
			{
				DestLight.Dimensions = FVector2f(2.0f * LightParameters.SourceRadius, 2.0f * LightParameters.SourceLength);
				DestLight.Shaping = FVector2f(LightParameters.RectLightBarnCosAngle, LightParameters.RectLightBarnLength);
				DestLight.FalloffExponent = LightParameters.FalloffExponent;
				DestLight.Flags |= Light.LightSceneInfo->Proxy->IsInverseSquared() ? 0 : PATHTRACER_FLAG_NON_INVERSE_SQUARE_FALLOFF_MASK;
				DestLight.Flags |= PATHTRACING_LIGHT_RECT;


				// Rect light atlas UV transformation
				DestLight.RectLightAtlasUVOffset = LightParameters.RectLightAtlasUVOffset;
				DestLight.RectLightAtlasUVScale  = LightParameters.RectLightAtlasUVScale;
				if (LightParameters.RectLightAtlasMaxLevel < 16)
				{
					DestLight.Flags |= PATHTRACER_FLAG_HAS_RECT_TEXTURE_MASK;
				}
				break;
			}
			case LightType_Spot:
			{
				DestLight.Dimensions = FVector2f(LightParameters.SourceRadius, LightParameters.SourceLength);
				DestLight.Shaping = LightParameters.SpotAngles;
				DestLight.FalloffExponent = LightParameters.FalloffExponent;
				DestLight.Flags |= Light.LightSceneInfo->Proxy->IsInverseSquared() ? 0 : PATHTRACER_FLAG_NON_INVERSE_SQUARE_FALLOFF_MASK;
				DestLight.Flags |= PATHTRACING_LIGHT_SPOT;
				break;
			}
			case LightType_Point:
			{
				DestLight.Dimensions = FVector2f(LightParameters.SourceRadius, LightParameters.SourceLength);
				DestLight.FalloffExponent = LightParameters.FalloffExponent;
				DestLight.Flags |= Light.LightSceneInfo->Proxy->IsInverseSquared() ? 0 : PATHTRACER_FLAG_NON_INVERSE_SQUARE_FALLOFF_MASK;
				DestLight.Flags |= PATHTRACING_LIGHT_POINT;
				break;
			}
			default:
			{
				// Just in case someone adds a new light type one day ...
				checkNoEntry();
				break;
			}
		}
	}

	*SceneLightCount = NumLights;
	{
		// Upload the buffer of lights to the GPU
		uint32 NumCopyLights = FMath::Max(1u, NumLights); // need at least one since zero-sized buffers are not allowed
		size_t DataSize = sizeof(FPathTracingLight) * NumCopyLights;
		*SceneLights = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(CreateStructuredBuffer(GraphBuilder, TEXT("PathTracer.LightsBuffer"), sizeof(FPathTracingLight), NumCopyLights, Lights, DataSize, ERDGInitialDataFlags::NoCopy)));
	}

	if (CVarPathTracingVisibleLights.GetValueOnRenderThread() == 1)
	{
		// make all lights in the scene visible
		*SceneVisibleLightCount = *SceneLightCount;
	}

	PrepareLightGrid(GraphBuilder, View.FeatureLevel, LightGridParameters, Lights, NumLights, NumInfiniteLights, *SceneLights);
}

class FPathTracingCompositorPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPathTracingCompositorPS)
	SHADER_USE_PARAMETER_STRUCT(FPathTracingCompositorPS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SAMPLER(SamplerState, VarianceSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, RadianceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float2>, VarianceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, DepthTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(uint32, Iteration)
		SHADER_PARAMETER(uint32, MaxSamples)
		SHADER_PARAMETER(int, ProgressDisplayEnabled)
		SHADER_PARAMETER(float, AdaptiveSamplingErrorThreshold)
		SHADER_PARAMETER(int, AdaptiveSamplingVisualize)
		SHADER_PARAMETER(FIntVector, VarianceTextureDims)
		SHADER_PARAMETER(float, PreExposure)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FPathTracingCompositorPS, TEXT("/Engine/Private/PathTracing/PathTracingCompositingPixelShader.usf"), TEXT("CompositeMain"), SF_Pixel);

static bool GPUNeedsTMinWorkaround()
{
	// See JIRA UE-213077
	// Some AMD GPUs can return true for a hit exactly at TMin. This causes some of the loops in the path tracer that want to repeatedly trace the same ray to get stuck in infinite loops
	// This workaround will be fine tuned in the future as drivers that fix this issue get released.
	return IsRHIDeviceAMD();
}

static FPathTracingRG::FPermutationDomain GetPathTracingRGPermutation(const FViewInfo& View, const FScene& Scene, const bool bIsCameraRays, const bool bCompactKernel)
{
	const bool bUseExperimental = CVarPathTracingExperimental.GetValueOnRenderThread();
	const bool bUseAdaptiveSampling = bUseExperimental && CVarPathTracingAdaptiveSampling.GetValueOnRenderThread() != 0;
	// NOTE: the decision about when to enable clouds involves checking lots of things, so rely on the presence of a valid index to signify that clouds are ready
	const bool bUseCloudShader = uint32(View.PathTracingVolumetricCloudCallableShaderIndex) < Scene.RayTracingSBT.NumCallableShaderSlots;
	const bool bHasComplexSpecialRenderPath = Substrate::IsSubstrateEnabled() && UsesSubstrateTileType(View.GetSceneTextures().SubstrateSceneData->UsesTileTypeMask, ESubstrateTileType::EComplexSpecial);
	const bool bUseSER = GRHIGlobals.SupportsShaderExecutionReordering && CVarPathTracingShaderExecutionReordering.GetValueOnRenderThread();

	FPathTracingRG::FPermutationDomain Out;
	Out.Set<FPathTracingRG::FKernelType>(bIsCameraRays ? PATH_TRACER_KERNEL_CAMERA : (bCompactKernel ? PATH_TRACER_KERNEL_INNER : PATH_TRACER_KERNEL_TAIL));
	Out.Set<FPathTracingRG::FAdaptiveSampling>(bUseAdaptiveSampling);
	Out.Set<FPathTracingRG::FCloudShader>(bUseCloudShader);
	Out.Set<FPathTracingRG::FSubstrateComplexSpecialMaterial>(bHasComplexSpecialRenderPath);
	Out.Set<FPathTracingRG::FUseSER>(bUseSER);
	Out.Set<FPathTracingRG::FNeedTMinWorkaround>(GPUNeedsTMinWorkaround());
	Out.Set<FPathTracingRG::FHeterogeneousVolumesPipeline>(static_cast<int32>(GetPathTracingHeterogeneousVolumesPipeline()));
	return Out;
}

static FPathTracingDebugRG::FPermutationDomain GetPathTracingDebugRGPermutation(const FViewInfo& View, const FScene& Scene, const bool bIsCameraRays, const bool bCompactKernel, const int DebugMode)
{
	const bool bUsePrimaryRays = DebugMode == PATH_TRACER_DEBUG_VIZ_PRIMARY_RAYS;
	const bool bUseSER = GRHIGlobals.SupportsShaderExecutionReordering && CVarPathTracingShaderExecutionReordering.GetValueOnRenderThread();
	FPathTracingDebugRG::FPermutationDomain Out;
	Out.Set<FPathTracingDebugRG::FKernelType>((bIsCameraRays || !bUsePrimaryRays) ? PATH_TRACER_KERNEL_CAMERA : (bCompactKernel ? PATH_TRACER_KERNEL_INNER : PATH_TRACER_KERNEL_TAIL));
	Out.Set<FPathTracingDebugRG::FUseSER>(bUseSER && bUsePrimaryRays);
	Out.Set<FPathTracingDebugRG::FUsePrimaryRays>(bUsePrimaryRays);
	Out.Set<FPathTracingDebugRG::FIncludeTranslucent>(CVarPathTracingVisualizeIncludeTranslucent.GetValueOnRenderThread());
	return Out;
}

void FDeferredShadingSceneRenderer::PreparePathTracing(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders)
{
	check(View.Family != nullptr);

	const FSceneViewFamily& ViewFamily = *View.Family;
	if (ViewFamily.EngineShowFlags.PathTracing
		&& PathTracing::ShouldCompilePathTracingShadersForProject(ViewFamily.GetShaderPlatform()))
	{
		const int CompactionDepth = CVarPathTracingCompactionDepth.GetValueOnRenderThread();

		if (int DebugMode = GetPathTracingVisualizationMode(); DebugMode >= 0)
		{
			{
				FPathTracingDebugRG::FPermutationDomain PermutationVector = GetPathTracingDebugRGPermutation(View, Scene, true, false, DebugMode);
				auto RayGenShader = GetGlobalShaderMap(ViewFamily.GetShaderPlatform())->GetShader<FPathTracingDebugRG>(PermutationVector);
				OutRayGenShaders.Add(RayGenShader.GetRayTracingShader());
			}
			if (DebugMode == PATH_TRACER_DEBUG_VIZ_PRIMARY_RAYS)
			{
				if (CompactionDepth != 1)
				{
					FPathTracingDebugRG::FPermutationDomain PermutationVector = GetPathTracingDebugRGPermutation(View, Scene, false, true, DebugMode);
					auto RayGenShader = GetGlobalShaderMap(ViewFamily.GetShaderPlatform())->GetShader<FPathTracingDebugRG>(PermutationVector);
					OutRayGenShaders.Add(RayGenShader.GetRayTracingShader());
				}
				if (CompactionDepth > 0)
				{
					FPathTracingDebugRG::FPermutationDomain PermutationVector = GetPathTracingDebugRGPermutation(View, Scene, false, false, DebugMode);
					auto RayGenShader = GetGlobalShaderMap(ViewFamily.GetShaderPlatform())->GetShader<FPathTracingDebugRG>(PermutationVector);
					OutRayGenShaders.Add(RayGenShader.GetRayTracingShader());
				}
			}
		}
		else
		{
			
			// Declare all RayGen shaders that require material closest hit shaders to be bound
			{
				// camera ray kernel
				FPathTracingRG::FPermutationDomain PermutationVector = GetPathTracingRGPermutation(View, Scene, true, false);
				auto RayGenShader = GetGlobalShaderMap(ViewFamily.GetShaderPlatform())->GetShader<FPathTracingRG>(PermutationVector);
				OutRayGenShaders.Add(RayGenShader.GetRayTracingShader());
			}
			if (CompactionDepth != 1)
			{
				// at least one compacted innner bounce
				FPathTracingRG::FPermutationDomain PermutationVector = GetPathTracingRGPermutation(View, Scene, false, true);
				auto RayGenShader = GetGlobalShaderMap(ViewFamily.GetShaderPlatform())->GetShader<FPathTracingRG>(PermutationVector);
				OutRayGenShaders.Add(RayGenShader.GetRayTracingShader());
			}
			if (CompactionDepth > 0)
			{
				// non-compacted tail handling
				FPathTracingRG::FPermutationDomain PermutationVector = GetPathTracingRGPermutation(View, Scene, false, false);
				auto RayGenShader = GetGlobalShaderMap(ViewFamily.GetShaderPlatform())->GetShader<FPathTracingRG>(PermutationVector);
				OutRayGenShaders.Add(RayGenShader.GetRayTracingShader());
			}
			{
				FPathTracingInitExtinctionCoefficientRG::FPermutationDomain PermutationVector;
				PermutationVector.Set<FPathTracingInitExtinctionCoefficientRG::FNeedTMinWorkaround>(GPUNeedsTMinWorkaround());
				auto RayGenShader = GetGlobalShaderMap(ViewFamily.GetShaderPlatform())->GetShader<FPathTracingInitExtinctionCoefficientRG>(PermutationVector);
				OutRayGenShaders.Add(RayGenShader.GetRayTracingShader());
			}
		}
	}
}

void PreparePathTracingRTPSO()
{
	if (!IsRayTracingEnabled())
	{
		return;
	}
	ENQUEUE_RENDER_COMMAND(PreparePathTracingRTPSO)([](FRHICommandListImmediate& RHICmdList)
		{
			int NumValidPermutations = 0;
			for (int PermutationId = 0; PermutationId < FPathTracingRG::FPermutationDomain::PermutationCount; PermutationId++)
			{
				FGlobalShaderPermutationParameters Parameters(FPathTracingRG::GetStaticType().GetFName(), GMaxRHIShaderPlatform, PermutationId);
				if (!FPathTracingRG::ShouldCompilePermutation(Parameters))
				{
					// Permutation is not enabled, nothing to pre-compile
					continue;
				}
                if (FPathTracingRG::IsPermutationExperimental(Parameters))
                {
                    // Permutation is for an experimental feature, don't compile it on startup
                    continue;
                }
				FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

				FPathTracingRG::FPermutationDomain PermutationVector(PermutationId);

				if (PermutationVector.Get<FPathTracingRG::FNeedTMinWorkaround>() != GPUNeedsTMinWorkaround())
				{
					// only compile the version that will be used at runtime
					continue;
				}

				if (PermutationVector.Get<FPathTracingRG::FUseSER>() && !GRHIGlobals.SupportsShaderExecutionReordering)
				{
					// we can safely skip these if the current machine does not support SER
					// ShouldCompilePermutation will not take this into account because it could be called in a cooking context
					// where the machine that decides what to compile is not the machine that will run the code
					continue;
				}

				FPathTracingInitExtinctionCoefficientRG::FPermutationDomain PermutationVectorInitExtinctionCoeffs;
				PermutationVectorInitExtinctionCoeffs.Set<FPathTracingInitExtinctionCoefficientRG::FNeedTMinWorkaround>(GPUNeedsTMinWorkaround());
				
				FRHIRayTracingShader* RayGenShaderTable[] = {
					ShaderMap->GetShader<FPathTracingRG>(PermutationVector).GetRayTracingShader(),
					ShaderMap->GetShader<FPathTracingInitExtinctionCoefficientRG>(PermutationVectorInitExtinctionCoeffs).GetRayTracingShader(),
				};
				FRHIRayTracingShader* MissShaderTable[] = {
					GetPathTracingDefaultMissShader(ShaderMap),
				};
				FRHIRayTracingShader* HitGroupTable[] = {
					GetPathTracingDefaultOpaqueHitShader(ShaderMap),
					GetPathTracingDefaultHiddenHitShader(ShaderMap),
				};
				FRayTracingPipelineStateInitializer Initializer;
				Initializer.bPartial = true; // TODO: got a crash in some older nvidia drivers when false - need to find out which driver version has the fix
				Initializer.bBackgroundCompilation = true;
				Initializer.SetRayGenShaderTable(RayGenShaderTable);
				Initializer.SetMissShaderTable(MissShaderTable);
				Initializer.SetHitGroupTable(HitGroupTable);
				Initializer.MaxPayloadSizeInBytes = RayGenShaderTable[0]->RayTracingPayloadSize;

				const FShaderBindingLayout* ShaderBindingLayout = RayTracing::GetShaderBindingLayout(GMaxRHIShaderPlatform);
				if (ShaderBindingLayout)
				{
					Initializer.ShaderBindingLayout = &ShaderBindingLayout->RHILayout;
				}

				FRayTracingPipelineState* PipelineState = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(RHICmdList, Initializer, ERayTracingPipelineCacheFlags::NonBlocking);
				NumValidPermutations++;
			}
			UE_LOGF(LogRenderer, Log, "Requested compilation of Path Tracing RTPSOs (%d permutations).", NumValidPermutations);
		}
	);
}

void FSceneViewState::PathTracingInvalidate(bool InvalidateAnimationStates)
{
	FPathTracingState* State = PathTracingState.Get();
	if (State)
	{
		
		if (InvalidateAnimationStates)
		{
			State->LastDenoisedRadianceRT.SafeRelease();
			State->LastRadianceRT.SafeRelease();
			State->LastNormalRT.SafeRelease();
			State->LastAlbedoRT.SafeRelease();
			State->LastVarianceBuffer.SafeRelease();

			State->SpatialTemporalDenoiserHistory.SafeRelease();
		}

		State->RadianceRT.SafeRelease();
		State->VarianceRT.SafeRelease();
		State->AlbedoRT.SafeRelease();
		State->NormalRT.SafeRelease();
		State->DepthRT.SafeRelease();
		State->VarianceBuffer.SafeRelease();
		State->SampleIndex = 0;

		State->AdaptiveFrustumGridParameterCache.TopLevelGridBuffer.SafeRelease();
	}
}

uint32 FSceneViewState::GetPathTracingSampleIndex() const {
	const FPathTracingState* State = PathTracingState.Get();
	return State ? State->SampleIndex : 0;
}

uint32 FSceneViewState::GetPathTracingSampleCount() const {
	const FPathTracingState* State = PathTracingState.Get();
	return State ? State->LastConfig.PathTracingData.MaxSamples : 0;
}

#if WITH_MGPU
BEGIN_SHADER_PARAMETER_STRUCT(FMGPUTransferParameters, )
	RDG_TEXTURE_ACCESS(InputTexture, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(InputAlbedo , ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(InputNormal , ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(InputDepth  , ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()
#endif

DECLARE_GPU_STAT_NAMED(PathTracing, TEXT("Path Tracing"));
DECLARE_GPU_STAT_NAMED(PathTracingPost, TEXT("Path Tracing Post"));
#if WITH_MGPU
DECLARE_GPU_STAT_NAMED(PathTracingCopy, TEXT("Path Tracing Copy"));
#endif

TAutoConsoleVariable<int32> CVarPathTracingVolumeIndex(
	TEXT("r.PathTracing.VolumeIndex"),
	0,
	TEXT("Heterogeneous volume index for rendering"),
	ECVF_RenderThreadSafe
);

BEGIN_SHADER_PARAMETER_STRUCT(FHeterogeneousVolumesCallableShadersParameters, )
	RDG_BUFFER_ACCESS(IntersectShaderIds, ERHIAccess::CopyDest)
	RDG_BUFFER_ACCESS(GetDensityBoundsShaderIds, ERHIAccess::CopyDest)
	RDG_BUFFER_ACCESS(GetDensityShaderIds, ERHIAccess::CopyDest)
	RDG_BUFFER_ACCESS(SampleDistanceShaderIds, ERHIAccess::CopyDest)
	RDG_BUFFER_ACCESS(GetTransmittanceShaderIds, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

void FDeferredShadingSceneRenderer::RenderPathTracing(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformBuffer,
	FRDGTextureRef SceneColorOutputTexture,
	FRDGTextureRef SceneDepthOutputTexture,
	FPathTracingResources& PathTracingResources)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, PathTracing, "PathTracing");

	// To make the GPU profiler work for path tracing with multi-GPU, we need the root GPU profiling scope (marked as "Unaccounted") to be on all GPUs,
	// as the profiler discards events where any event in the hierarchy wasn't on a given GPU.  So in the parent scene render code, we set the GPU mask
	// to "All" when path tracing is enabled, instead of "AllViewsGPUMask".  Then we'll enable that scope inside the path tracer instead.  We also
	// subdivide the profiling scopes inside the path tracer, so the multi-GPU rendering and single-GPU post processing are separate scopes, instead of
	// a scope for the whole path tracer (which would create the same problem).
	RDG_GPU_MASK_SCOPE(GraphBuilder, AllViewsGPUMask);

	if (!PathTracing::ShouldCompilePathTracingShadersForProject(View.GetShaderPlatform()))
	{
		AddClearRenderTargetPass(GraphBuilder, SceneColorOutputTexture);
		OnGetOnScreenMessages.AddLambda([](FScreenMessageWriter& ScreenMessageWriter)->void
		{
			static const FText Message = NSLOCTEXT("Renderer", "PathTracingNotEnabled", "Path Tracing is not enabled or supported in the current configuration.");
			ScreenMessageWriter.DrawLine(Message);
		});
		return;
	}

	if (int DebugMode = GetPathTracingVisualizationMode(); DebugMode >= 0)
	{
		const int32 DispatchResX = View.ViewRect.Size().X;
		const int32 DispatchResY = View.ViewRect.Size().Y;

		// simplified pass for debugging purposes
		FRDGTextureDesc DepthDesc = FRDGTextureDesc::Create2D(
			View.ViewRect.Size(),
			PF_R32_FLOAT,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV);
		FRDGTextureRef DepthTexture    = GraphBuilder.CreateTexture(DepthDesc, TEXT("PathTracer.Depth"));
		
		FRDGTextureDesc AlbedoNormalDesc = FRDGTextureDesc::Create2D(
			View.ViewRect.Size(),
			PF_FloatRGBA,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV);
		
		FRDGTextureDesc RoughnessDesc(FRDGTextureDesc::Create2D(
			View.ViewRect.Size(),
			PF_R32_FLOAT, // TODO PF_B8G8R8A8 enough?
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV
		));

		FRDGTextureRef DiffuseColorTexture = GraphBuilder.CreateTexture(AlbedoNormalDesc, TEXT("PathTracer.DiffuseColor"));
		FRDGTextureRef SpecularColorTexture = GraphBuilder.CreateTexture(AlbedoNormalDesc, TEXT("PathTracer.SpecularColor"));
		FRDGTextureRef NormalTexture = GraphBuilder.CreateTexture(AlbedoNormalDesc, TEXT("PathTracer.WorldNormal"));
		FRDGTextureRef RoughnessTexture = GraphBuilder.CreateTexture(RoughnessDesc, TEXT("PathTracer.Roughness"));
		FRDGTextureRef LinearDepthTexture = GraphBuilder.CreateTexture(RoughnessDesc, TEXT("PathTracer.LinearDepth"));

		GPathTracingRealtimeDenoiserResources.DiffuseColor = DiffuseColorTexture;
		GPathTracingRealtimeDenoiserResources.SpecularColor = SpecularColorTexture;
		GPathTracingRealtimeDenoiserResources.Normal = NormalTexture;
		GPathTracingRealtimeDenoiserResources.Roughness = RoughnessTexture;
		GPathTracingRealtimeDenoiserResources.LinearDepth = LinearDepthTexture;		

		FPathTracingState* PathTracingState = GetPathTracingStateFromView(View);

		const bool bUsePrimaryRays = DebugMode == PATH_TRACER_DEBUG_VIZ_PRIMARY_RAYS;
		const int SamplesPerPixel = bUsePrimaryRays ? FMath::Max(CVarPathTracingVisualizeSamplesPerPixel.GetValueOnRenderThread(), 1) : 1;

		int32 MaxBounces = bUsePrimaryRays ? CVarPathTracingMaxBounces.GetValueOnRenderThread() : 0;
		if (MaxBounces < 0)
		{
			MaxBounces = View.FinalPostProcessSettings.PathTracingMaxBounces;
		}
		int32 CompactionDepth = bUsePrimaryRays ? CVarPathTracingCompactionDepth.GetValueOnRenderThread() : -1;
		if (CompactionDepth <= 0 || CompactionDepth > MaxBounces)
		{
			CompactionDepth = MaxBounces;
		}

		FRDGBuffer* ActivePaths[2] = {};
		FRDGBuffer* NumPathStates = nullptr;
		FRDGBuffer* PathStateData = nullptr;
		{
			const int32 NumPaths = DispatchResX * DispatchResY;
			ActivePaths[0] = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumPaths), TEXT("PathTracer.ActivePaths0"));
			ActivePaths[1] = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumPaths), TEXT("PathTracer.ActivePaths1"));
			NumPathStates = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(CompactionDepth + 1), TEXT("PathTracer.NumPathStates"));
			PathStateData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FPathTracingDebugPackedPathState), NumPaths), TEXT("PathTracer.PathStateData"));
		}

		FPathTracingDebugRG::FParameters* FirstPassParameters = nullptr;
		for (int SampleIndex = 0; SampleIndex < SamplesPerPixel; SampleIndex++)
		{
			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(NumPathStates, PF_R32_UINT), 0);
			for (int Bounce = 0; Bounce <= CompactionDepth; Bounce++)
			{
				// TODO: can we do better than making a new set of parameters per sample?
				FPathTracingDebugRG::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingDebugRG::FParameters>();
				if (FirstPassParameters == nullptr)
				{
					PassParameters->RWSceneColor = GraphBuilder.CreateUAV(SceneColorOutputTexture);
					PassParameters->RWSceneDepth = GraphBuilder.CreateUAV(DepthTexture);
					PassParameters->TLAS = Scene->RayTracingScene.GetLayerView(ERayTracingSceneLayer::Base, View.GetRayTracingSceneViewHandle());
					PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
					PassParameters->RandomSequenceSpaceFillingCurve = GSystemTextures.GetSpaceFillingCurveTexture(GraphBuilder);
					PassParameters->PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
					PassParameters->PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
					PassParameters->DebugMode = DebugMode;

					PassParameters->SamplesPerPixel = SamplesPerPixel;
					PassParameters->MaxBounces = MaxBounces;
					PassParameters->EnableEmissive = CVarPathTracingEnableEmissive.GetValueOnRenderThread();
					if (PassParameters->EnableEmissive < 0)
					{
						PassParameters->EnableEmissive = View.FinalPostProcessSettings.PathTracingEnableEmissiveMaterials;
					}
                    PassParameters->MaxPathIntensity = CVarPathTracingMaxPathIntensity.GetValueOnRenderThread();
                    if (PassParameters->MaxPathIntensity <= 0)
                    {
                        // cvar clamp disabled, use PPV value instad
                        PassParameters->MaxPathIntensity = View.FinalPostProcessSettings.PathTracingMaxPathIntensity;
                    }
                    PassParameters->AmbientOcclusionMaxDistance = CVarPathTracingVisualizeAmbientOcclusionMaxDistance.GetValueOnRenderThread();
                    PassParameters->VisualizeMaxCount = uint32(FMath::Max(1, CVarPathTracingVisualizeMaxCount.GetValueOnRenderThread()));
					{
						SetLightParameters(
							GraphBuilder,
							Scene,
							View,
							true,
							false,
							&PassParameters->SkylightParameters,
							&PassParameters->LightGridParameters,
							&PassParameters->SceneVisibleLightCount,
							&PassParameters->SceneLightCount,
							&PassParameters->SceneLights
						);
					}

					PassParameters->RWSceneDiffuseColor = GraphBuilder.CreateUAV(DiffuseColorTexture);
					PassParameters->RWSceneSpecularColor = GraphBuilder.CreateUAV(SpecularColorTexture);
					PassParameters->RWSceneNormal = GraphBuilder.CreateUAV(NormalTexture);
					PassParameters->RWSceneRoughness = GraphBuilder.CreateUAV(RoughnessTexture);
					PassParameters->RWSceneLinearDepth = GraphBuilder.CreateUAV(LinearDepthTexture);

                    // decal parameters
                    PassParameters->DecalParameters = View.RayTracingDecalUniformBuffer;
                    PassParameters->DecalTLAS = Scene->RayTracingScene.GetLayerView(ERayTracingSceneLayer::Decals, View.GetRayTracingSceneViewHandle());
                    PassParameters->DecalRoughnessCutoff = PathTracing::UsesDecals(ViewFamily) && View.bHasRayTracingDecals ? CVarPathTracingDecalRoughnessCutoff.GetValueOnRenderThread() : -1;
                    PassParameters->MeshDecalRoughnessCutoff = PathTracing::UsesDecals(ViewFamily) && Scene->RayTracingScene.GetNumNativeInstances(ERayTracingSceneLayer::Decals, View.GetRayTracingSceneViewHandle()) > 0 ? CVarPathTracingMeshDecalRoughnessCutoff.GetValueOnRenderThread() : -1;
                    PassParameters->MeshDecalBias = CVarPathTracingMeshDecalBias.GetValueOnRenderThread();
					PassParameters->EnableDBuffer = CVarPathTracingUseDBuffer.GetValueOnRenderThread();
	
					PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
					PassParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();

					FirstPassParameters = PassParameters;
				}
				else
				{
					// Already made one set of parameters, re-use it
					*PassParameters = *FirstPassParameters;
				}
				PassParameters->SampleIndex = SampleIndex;
				PassParameters->SampleBlendFactor = 1.0f / (SampleIndex + 1);
				PassParameters->FirstBounce = Bounce;
				PassParameters->NumPathStates = GraphBuilder.CreateUAV(NumPathStates, PF_R32_UINT);
				PassParameters->ActivePaths = GraphBuilder.CreateUAV(ActivePaths[Bounce & 1], PF_R32_UINT);
				PassParameters->NextActivePaths = GraphBuilder.CreateUAV(ActivePaths[(Bounce & 1) ^ 1], PF_R32_UINT);
				PassParameters->PathStateData = GraphBuilder.CreateUAV(PathStateData);

				const bool bUseCompaction = Bounce < CompactionDepth || Bounce == MaxBounces;
				FPathTracingDebugRG::FPermutationDomain PermutationVector = GetPathTracingDebugRGPermutation(View, *Scene, Bounce == 0, bUseCompaction, DebugMode);
				TShaderMapRef<FPathTracingDebugRG> RayGenShader(View.ShaderMap, PermutationVector);
				ClearUnusedGraphResources(RayGenShader, PassParameters);
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("Path Tracing Visualize [%d/%d] - Bounce %d%s", SampleIndex + 1, SamplesPerPixel, Bounce, bUseCompaction ? "" : "+"),
					PassParameters,
					ERDGPassFlags::Compute,
					[PassParameters, RayGenShader, &View, DispatchResX, DispatchResY](FRHICommandList& RHICmdList)
					{
						FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
						SetShaderParameters(GlobalResources, RayGenShader, *PassParameters);
				
						TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, PassParameters->Scene->GetRHI(), PassParameters->NaniteRayTracing->GetRHI(), RHICmdList);

						RHICmdList.RayTraceDispatch(
							View.MaterialRayTracingData.PipelineState,
							RayGenShader.GetRayTracingShader(),
							View.MaterialRayTracingData.ShaderBindingTable, GlobalResources,
							DispatchResX, DispatchResY
						);
					}
				);
			}
		}

		{
			FPathTracingCopyDepthPS::FParameters* DisplayParameters = GraphBuilder.AllocParameters<FPathTracingCopyDepthPS::FParameters>();
			DisplayParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			DisplayParameters->DepthTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DepthTexture));
			//DisplayParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorOutputTexture, ERenderTargetLoadAction::ELoad);
			DisplayParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthOutputTexture,  ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
			FScreenPassTextureViewport Viewport(SceneColorOutputTexture, View.ViewRect);
			TShaderMapRef<FPathTracingCopyDepthPS> PixelShader(View.ShaderMap);
			TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
			FRHIBlendState* BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
			FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState<true /* bEnableDepthWrite */, CF_Always>::GetRHI();
			AddDrawScreenPass(
				GraphBuilder,
				RDG_EVENT_NAME("Path Tracer Copy Depth", View.ViewRect.Size().X, View.ViewRect.Size().Y),
				View,
				Viewport,
				Viewport,
				VertexShader,
				PixelShader,
				BlendState,
				DepthStencilState,
				DisplayParameters);
		}
		return;
	}

	FPathTracingConfig Config = {};

	// Get current value of MaxSPP and reset render if it has changed
	// NOTE: we ignore the CVar when using offline rendering
	int32 SamplesPerPixelCVar = View.bIsOfflineRender ? -1 : CVarPathTracingSamplesPerPixel.GetValueOnRenderThread();
	uint32 MaxSPP = SamplesPerPixelCVar > -1 ? SamplesPerPixelCVar : View.FinalPostProcessSettings.PathTracingSamplesPerPixel;
	MaxSPP = FMath::Max(MaxSPP, 1u);

	int32 SeedOffset = View.PathTracerSeedOffset;

	const bool bUseExperimental = CVarPathTracingExperimental.GetValueOnRenderThread();

	Config.LockedSamplingPattern = CVarPathTracingFrameIndependentTemporalSeed.GetValueOnRenderThread() == 0;
	Config.UseCameraMediumTracking = CVarPathTracingCameraMediumTracking.GetValueOnRenderThread();
	Config.UseAdaptiveSampling = bUseExperimental && CVarPathTracingAdaptiveSampling.GetValueOnAnyThread() != 0;
	Config.AdaptiveSamplingThreshold = CVarPathTracingAdaptiveSamplingErrorThreshold.GetValueOnRenderThread();
	Config.CloudAccelerationMapNumSamples = FMath::Clamp(CVarPathTracingCloudAccelerationMapNumSamples.GetValueOnRenderThread(), 1, 65536);
	Config.CloudAccelerationMapResolution = FMath::Clamp(CVarPathTracingCloudAccelerationMapResolution.GetValueOnRenderThread(), 1, 4096);
	Config.CloudAccelerationMapVisualize = CVarPathTracingCloudAccelerationMapVisualize.GetValueOnRenderThread();
	Config.CloudMultipleScatterMode = CVarPathTracingCloudMultipleScatterMode.GetValueOnRenderThread();

	// compute an integer code of what show flags and booleans related to lights are currently enabled so we can detect changes
	Config.LightShowFlags = 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.SkyLighting           ? 1 << 0 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.DirectionalLights     ? 1 << 1 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.RectLights            ? 1 << 2 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.SpotLights            ? 1 << 3 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.PointLights           ? 1 << 4 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.TexturedLightProfiles ? 1 << 5 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.LightFunctions        ? 1 << 6 : 0;
	Config.LightShowFlags |= CVarPathTracingLightFunctionColor.GetValueOnRenderThread() ? 1 << 7 : 0;
	// the following flags all mess with diffuse/spec overrides and therefore change the image
	Config.LightShowFlags |= View.Family->EngineShowFlags.Diffuse                    ? 1 << 8 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.Specular                   ? 1 << 9 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.OverrideDiffuseAndSpecular ? 1 << 10 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.LightingOnlyOverride       ? 1 << 11 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.ReflectionOverride         ? 1 << 12 : 0;
	Config.LightShowFlags |= View.Family->EngineShowFlags.SubsurfaceScattering       ? 1 << 13 : 0;
	// the following affects which material shaders get used and therefore change the image
	if (Substrate::IsSubstrateEnabled() && CVarPathTracingSubstrateCompileSimplifiedMaterial.GetValueOnRenderThread())
	{
		Config.LightShowFlags |= CVarPathTracingSubstrateUseSimplifiedMaterial.GetValueOnRenderThread() ? 1 << 14 : 0;
	}

	PreparePathTracingData(Scene, View, Config.PathTracingData);

	Config.VisibleLights = CVarPathTracingVisibleLights.GetValueOnRenderThread() != 0;
	Config.UseMISCompensation = Config.PathTracingData.MISMode == 2 && CVarPathTracingMISCompensation.GetValueOnRenderThread() != 0;

	Config.ViewRect = View.ViewRect;

	Config.LightGridResolution = FMath::RoundUpToPowerOfTwo(CVarPathTracingLightGridResolution.GetValueOnRenderThread());
	Config.LightGridMaxCount = FMath::Clamp(CVarPathTracingLightGridMaxCount.GetValueOnRenderThread(), 1, RAY_TRACING_LIGHT_COUNT_MAXIMUM);

	Config.PathTracingData.MaxSamples = MaxSPP;

	FPathTracingState* PathTracingState = GetPathTracingStateFromView(View);
	const bool bFirstTime = !PathTracingState->RadianceRT.IsValid() && !PathTracingState->LastRadianceRT.IsValid(); // we just initialized (or reset) the option state for this view -- don't bother comparing in this case

	if (!bFirstTime && Config.UseMISCompensation != PathTracingState->LastConfig.UseMISCompensation)
	{
		// if the mode changes we need to rebuild the importance table
		Scene->PathTracingSkylightTexture.SafeRelease();
		Scene->PathTracingSkylightPdf.SafeRelease();
	}

	// if the skylight has changed colors, reset both the path tracer and the importance tables
	const FLinearColor SkyColor = View.CachedViewUniformShaderParameters->SkyLightColor;
	if (Scene->SkyLight && SkyColor != Scene->PathTracingSkylightColor)
	{
		Scene->PathTracingSkylightTexture.SafeRelease();
		Scene->PathTracingSkylightPdf.SafeRelease();
		// reset last color here as well in case we don't reach PrepareSkyLightTexture
		Scene->PathTracingSkylightColor = SkyColor;
		if (!View.bIsOfflineRender)
		{
			// reset accumulation, unless this is an offline render, in which case it is ok for the color to evolve
			// across temporal samples
			View.ViewState->PathTracingInvalidate();
		}
		
	}


	// If this is the first sample, recompute the initial medium
	// In this case of an offline render, do this every frame so that motion blur through a boundary is properly accounted for
	FRDGBufferRef StartingMediumData = nullptr;
	if (!Config.UseCameraMediumTracking)
	{
		PathTracingState->StartingMediumData.SafeRelease();
		// camera medium tracking is not enabled - just make a temp buffer and set it to 0
		StartingMediumData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(float), 7), TEXT("PathTracer.StartingMediumData"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(StartingMediumData, PF_R32_FLOAT), 0);
	}
	else if (!PathTracingState->StartingMediumData.IsValid() || PathTracingState->SampleIndex == 0 || View.bIsOfflineRender)
	{
		FPathTracingInitExtinctionCoefficientRG::FPermutationDomain PermutationVector;
		PermutationVector.Set<FPathTracingInitExtinctionCoefficientRG::FNeedTMinWorkaround>(GPUNeedsTMinWorkaround());

		auto RayGenShader = GetGlobalShaderMap(View.FeatureLevel)->GetShader<FPathTracingInitExtinctionCoefficientRG>(PermutationVector);

		// prepare extinction coefficient for camera rays
		StartingMediumData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(float), 7), TEXT("PathTracer.StartingMediumData"));

		FPathTracingInitExtinctionCoefficientRG::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingInitExtinctionCoefficientRG::FParameters>();
		PassParameters->TLAS = Scene->RayTracingScene.GetLayerView(ERayTracingSceneLayer::Base, View.GetRayTracingSceneViewHandle());
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
		PassParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->RWStartingMediumData = GraphBuilder.CreateUAV(StartingMediumData, PF_R32_FLOAT);

		for (FRDGBuffer* RDGBuffer : View.DynamicRayTracingRDGBuffers)
		{
			PassParameters->SBTBuffers.Emplace(RDGBuffer, ERHIAccess::SRVCompute);
		}

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Path Tracer Init Sigma"),
			PassParameters,
			ERDGPassFlags::Compute,
			[PassParameters, RayGenShader, &View](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
				SetShaderParameters(GlobalResources, RayGenShader, *PassParameters);

				TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, PassParameters->Scene->GetRHI(), PassParameters->NaniteRayTracing->GetRHI(), RHICmdList);

				RHICmdList.RayTraceDispatch(
					View.MaterialRayTracingData.PipelineState,
					RayGenShader.GetRayTracingShader(),
					View.MaterialRayTracingData.ShaderBindingTable,
					GlobalResources,
					1, 1
				);
			});
		GraphBuilder.QueueBufferExtraction(StartingMediumData, &PathTracingState->StartingMediumData);
	}
	else
	{
		check(PathTracingState->StartingMediumData.IsValid());
		StartingMediumData = GraphBuilder.RegisterExternalBuffer(PathTracingState->StartingMediumData, TEXT("PathTracer.StartingMediumData"));
	}

	// prepare atmosphere optical depth lookup texture (if needed)
	FRDGTexture* AtmosphereOpticalDepthLUT = nullptr;
	if ((Config.PathTracingData.VolumeFlags & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE) != 0)
	{
		check(Scene->GetSkyAtmosphereSceneInfo() != nullptr);
		check(Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereShaderParameters() != nullptr);
		FAtmosphereConfig AtmoConfig(*Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereShaderParameters());
		if (!PathTracingState->AtmosphereOpticalDepthLUT.IsValid() || PathTracingState->LastAtmosphereConfig.IsDifferent(AtmoConfig))
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());
			PathTracingState->LastAtmosphereConfig = AtmoConfig;
			// need to create a new LUT
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				FIntPoint(AtmoConfig.Resolution, AtmoConfig.Resolution),
				PF_A32B32G32R32F,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_UAV);
			AtmosphereOpticalDepthLUT = GraphBuilder.CreateTexture(Desc, TEXT("PathTracer.AtmosphereOpticalDepthLUT"), ERDGTextureFlags::MultiFrame);
			FPathTracingBuildAtmosphereOpticalDepthLUTCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingBuildAtmosphereOpticalDepthLUTCS::FParameters>();
			PassParameters->NumSamples = AtmoConfig.NumSamples;
			PassParameters->Resolution = AtmoConfig.Resolution;
			PassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
			PassParameters->AtmosphereOpticalDepthLUT = GraphBuilder.CreateUAV(AtmosphereOpticalDepthLUT);
			TShaderMapRef<FPathTracingBuildAtmosphereOpticalDepthLUTCS> ComputeShader(GetGlobalShaderMap(View.FeatureLevel));
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("Path Tracing Atmosphere Optical Depth LUT (Resolution=%u, NumSamples=%u)", AtmoConfig.Resolution, AtmoConfig.NumSamples),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(
					FIntPoint(AtmoConfig.Resolution, AtmoConfig.Resolution),
					FIntPoint(FComputeShaderUtils::kGolden2DGroupSize, FComputeShaderUtils::kGolden2DGroupSize))
			);
			GraphBuilder.QueueTextureExtraction(AtmosphereOpticalDepthLUT, &PathTracingState->AtmosphereOpticalDepthLUT);
		}
		else
		{
			AtmosphereOpticalDepthLUT = GraphBuilder.RegisterExternalTexture(PathTracingState->AtmosphereOpticalDepthLUT, TEXT("PathTracer.AtmosphereOpticalDepthLUT"));
		}
	}
	else
	{
		AtmosphereOpticalDepthLUT = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
	}

#if WITH_MGPU
	Config.UseMultiGPU = CVarPathTracingMultiGPU.GetValueOnRenderThread();
	// TODO: Figure out how to support adaptive sampling in multi-gpu cases (this is complicated due to the swizzled layout of the variance texture)
	Config.UseMultiGPU &= !Config.UseAdaptiveSampling;
#else
	Config.UseMultiGPU = false;
#endif

	// If the scene has changed in some way (camera move, object movement, etc ...)
	// we must invalidate the ViewState to start over from scratch
	// NOTE: only check things like hair position changes for interactive viewports, for offline renders we don't want any chance of mid-render invalidation
	// NOTE: same for DOF changes, these parameters could be animated which should not automatically invalidate a render in progress
	if (bFirstTime ||
		Config.IsDifferent(PathTracingState->LastConfig) ||
		(!View.bIsOfflineRender && Config.IsExposureDifferentEnough(PathTracingState->LastConfig)) ||
		(!View.bIsOfflineRender && Config.IsDOFDifferent(PathTracingState->LastConfig)) ||
		(!View.bIsOfflineRender && HairStrands::HasPositionsChanged(GraphBuilder, *Scene, View)))
	{
		// remember the options we used for next time
		PathTracingState->LastConfig = Config;
		View.ViewState->PathTracingInvalidate();
	}
	// copy the base exposure from last time, so we can have a consistent exposure when we accumulate samples
	Config.PathTracingData.BaseExposure = PathTracingState->LastConfig.PathTracingData.BaseExposure;

	// Declare heterogeneous volume buffers
	TRDGUniformBufferRef<FOrthoVoxelGridUniformBufferParameters> OrthoGridUniformBuffer;
	TRDGUniformBufferRef<FFrustumVoxelGridUniformBufferParameters> FrustumGridUniformBuffer;
	// NOTE: Remove in separate submission
	TRDGUniformBufferRef<FPreshadingUniformBufferParameters> PreshadingUniformBuffer = nullptr;
	bool bCreateVolumeGrids = false;

	// Upload heterogeneous volume callable interface
	bool bCallableShaderHeterogeneousVolumesPipeline = GetPathTracingHeterogeneousVolumesPipeline() == EPathTracingHeterogeneousVolumesPipeline::Callable;
	int32 ElementCount = bCallableShaderHeterogeneousVolumesPipeline ? FMath::Max(PathTracingState->HeterogeneousVolumesCallableInfo.Num(), 1) : 1;
	FRDGBufferDesc ShaderIdsDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(int32), ElementCount);
	FRDGBufferRef IntersectShaderIdsBuffer = GraphBuilder.CreateBuffer(ShaderIdsDesc, TEXT("IntersectShaderId"));
	FRDGBufferRef GetDensityBoundsShaderIdsBuffer = GraphBuilder.CreateBuffer(ShaderIdsDesc, TEXT("GetDensityBoundsShaderId"));
	FRDGBufferRef GetDensityShaderIdsBuffer = GraphBuilder.CreateBuffer(ShaderIdsDesc, TEXT("GetDensityShaderId"));
	FRDGBufferRef SampleDistanceShaderIdsBuffer = GraphBuilder.CreateBuffer(ShaderIdsDesc, TEXT("SampleDistanceShaderId"));
	FRDGBufferRef GetTransmittanceShaderIdsBuffer = GraphBuilder.CreateBuffer(ShaderIdsDesc, TEXT("GetTransmittanceShaderId"));
	if (bCallableShaderHeterogeneousVolumesPipeline)
	{
		FHeterogeneousVolumesCallableShadersParameters* HeterogeneousVolumesCallableShadersParameters = GraphBuilder.AllocParameters<FHeterogeneousVolumesCallableShadersParameters>();
		{
			HeterogeneousVolumesCallableShadersParameters->IntersectShaderIds = IntersectShaderIdsBuffer;
			HeterogeneousVolumesCallableShadersParameters->GetDensityBoundsShaderIds = GetDensityBoundsShaderIdsBuffer;
			HeterogeneousVolumesCallableShadersParameters->GetDensityShaderIds = GetDensityShaderIdsBuffer;
			HeterogeneousVolumesCallableShadersParameters->SampleDistanceShaderIds = SampleDistanceShaderIdsBuffer;
			HeterogeneousVolumesCallableShadersParameters->GetTransmittanceShaderIds = GetTransmittanceShaderIdsBuffer;
		}

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("UploadShaderIds"),
			HeterogeneousVolumesCallableShadersParameters,
			ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
			[
				ShaderIdsDesc,
				HeterogeneousVolumesCallableShadersParameters,
				PathTracingState
			](FRHICommandList& RHICmdList)
			{
				int32 SizeInBytes = ShaderIdsDesc.NumElements * ShaderIdsDesc.BytesPerElement;

				// Intersect
				{
					// TODO: Convert to SoA style
					TArray<int32> IntersectShaderIds;
					IntersectShaderIds.Reserve(ShaderIdsDesc.NumElements);
					for (uint32 i = 0; i < ShaderIdsDesc.NumElements; ++i)
					{
						IntersectShaderIds.Add(PathTracingState->HeterogeneousVolumesCallableInfo[i].IntersectShaderId);
					}

					void* Dest = RHICmdList.LockBuffer(HeterogeneousVolumesCallableShadersParameters->IntersectShaderIds->GetRHI(), 0, SizeInBytes, RLM_WriteOnly);
					FPlatformMemory::Memcpy(Dest, IntersectShaderIds.GetData(), SizeInBytes);
					RHICmdList.UnlockBuffer(HeterogeneousVolumesCallableShadersParameters->IntersectShaderIds->GetRHI());
				}

				// GetDensityBounds
				{
					// TODO: Convert to SoA style
					TArray<int32> GetDensityBoundsShaderId;
					GetDensityBoundsShaderId.Reserve(ShaderIdsDesc.NumElements);
					for (uint32 i = 0; i < ShaderIdsDesc.NumElements; ++i)
					{
						GetDensityBoundsShaderId.Add(PathTracingState->HeterogeneousVolumesCallableInfo[i].GetDensityBoundsShaderId);
					}

					void* Dest = RHICmdList.LockBuffer(HeterogeneousVolumesCallableShadersParameters->GetDensityBoundsShaderIds->GetRHI(), 0, SizeInBytes, RLM_WriteOnly);
					FPlatformMemory::Memcpy(Dest, GetDensityBoundsShaderId.GetData(), SizeInBytes);
					RHICmdList.UnlockBuffer(HeterogeneousVolumesCallableShadersParameters->GetDensityBoundsShaderIds->GetRHI());
				}

				// GetDensity
				{
					// TODO: Convert to SoA style
					TArray<int32> GetDensityShaderId;
					GetDensityShaderId.Reserve(ShaderIdsDesc.NumElements);
					for (uint32 i = 0; i < ShaderIdsDesc.NumElements; ++i)
					{
						GetDensityShaderId.Add(PathTracingState->HeterogeneousVolumesCallableInfo[i].GetDensityShaderId);
					}

					void* Dest = RHICmdList.LockBuffer(HeterogeneousVolumesCallableShadersParameters->GetDensityShaderIds->GetRHI(), 0, SizeInBytes, RLM_WriteOnly);
					FPlatformMemory::Memcpy(Dest, GetDensityShaderId.GetData(), SizeInBytes);
					RHICmdList.UnlockBuffer(HeterogeneousVolumesCallableShadersParameters->GetDensityShaderIds->GetRHI());
				}

				// SampleDistance
				{
					// TODO: Convert to SoA style
					TArray<int32> SampleDistanceShaderId;
					SampleDistanceShaderId.Reserve(ShaderIdsDesc.NumElements);
					for (uint32 i = 0; i < ShaderIdsDesc.NumElements; ++i)
					{
						SampleDistanceShaderId.Add(PathTracingState->HeterogeneousVolumesCallableInfo[i].SampleDistanceShaderId);
					}

					void* Dest = RHICmdList.LockBuffer(HeterogeneousVolumesCallableShadersParameters->SampleDistanceShaderIds->GetRHI(), 0, SizeInBytes, RLM_WriteOnly);
					FPlatformMemory::Memcpy(Dest, SampleDistanceShaderId.GetData(), SizeInBytes);
					RHICmdList.UnlockBuffer(HeterogeneousVolumesCallableShadersParameters->SampleDistanceShaderIds->GetRHI());
				}

				//  GetTransmittance
				{
					// TODO: Convert to SoA style
					TArray<int32> GetTransmittanceShaderId;
					GetTransmittanceShaderId.Reserve(ShaderIdsDesc.NumElements);
					for (uint32 i = 0; i < ShaderIdsDesc.NumElements; ++i)
					{
						GetTransmittanceShaderId.Add(PathTracingState->HeterogeneousVolumesCallableInfo[i].GetTransmittanceShaderId);
					}

					void* Dest = RHICmdList.LockBuffer(HeterogeneousVolumesCallableShadersParameters->GetTransmittanceShaderIds->GetRHI(), 0, SizeInBytes, RLM_WriteOnly);
					FPlatformMemory::Memcpy(Dest, GetTransmittanceShaderId.GetData(), SizeInBytes);
					RHICmdList.UnlockBuffer(HeterogeneousVolumesCallableShadersParameters->GetTransmittanceShaderIds->GetRHI());
				}
			}
		);
	}

	// Prepare radiance buffer (will be shared with display pass)
	FRDGTexture* RadianceTexture = nullptr;
	FRDGTexture* VarianceTexture = nullptr;
	FRDGTexture* AlbedoTexture   = nullptr;
	FRDGTexture* NormalTexture   = nullptr;
	FRDGTexture* DepthTexture    = nullptr;
	const int NumVarianceMips = FMath::Min(5u, 1 + FMath::FloorLog2(uint32(View.ViewRect.Size().GetMin())));
	if (PathTracingState->RadianceRT)
	{
		// we already have a valid radiance texture, re-use it
		RadianceTexture = GraphBuilder.RegisterExternalTexture(PathTracingState->RadianceRT, TEXT("PathTracer.Radiance"));
		AlbedoTexture   = GraphBuilder.RegisterExternalTexture(PathTracingState->AlbedoRT  , TEXT("PathTracer.Albedo"));
		NormalTexture   = GraphBuilder.RegisterExternalTexture(PathTracingState->NormalRT  , TEXT("PathTracer.Normal"));
		DepthTexture    = GraphBuilder.RegisterExternalTexture(PathTracingState->DepthRT   , TEXT("PathTracer.Depth"));
	}
	else
	{
		// First time through, need to make a new texture
		FRDGTextureDesc RadianceDesc = FRDGTextureDesc::Create2D(
			View.ViewRect.Size(),
			PF_A32B32G32R32F, // radiance accumulation must take place in floats to avoid quantization artifacts on smooth gradients
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV | GetExtraTextureCreateFlagsForDenoiser());
		FRDGTextureDesc AlbedoNormalDesc = FRDGTextureDesc::Create2D(
			View.ViewRect.Size(),
			PF_FloatRGBA,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV | GetExtraTextureCreateFlagsForDenoiser());
		FRDGTextureDesc DepthDesc = FRDGTextureDesc::Create2D(
			View.ViewRect.Size(),
			PF_R32_FLOAT,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV | GetExtraTextureCreateFlagsForDenoiser());
		RadianceTexture = GraphBuilder.CreateTexture(RadianceDesc    , TEXT("PathTracer.Radiance"), ERDGTextureFlags::MultiFrame);
		AlbedoTexture   = GraphBuilder.CreateTexture(AlbedoNormalDesc, TEXT("PathTracer.Albedo")  , ERDGTextureFlags::MultiFrame);
		NormalTexture   = GraphBuilder.CreateTexture(AlbedoNormalDesc, TEXT("PathTracer.Normal")  , ERDGTextureFlags::MultiFrame);
		DepthTexture    = GraphBuilder.CreateTexture(DepthDesc       , TEXT("PathTracer.Depth")   , ERDGTextureFlags::MultiFrame);
	}
	if (Config.UseAdaptiveSampling)
	{
		if (PathTracingState->VarianceRT)
		{
			VarianceTexture = GraphBuilder.RegisterExternalTexture(PathTracingState->VarianceRT, TEXT("PathTracer.Variance"));
		}
		else
		{
			// format stores Luminance,Luminance^2,NumSamples which can be used for error estimation
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				View.ViewRect.Size(),
				PF_A32B32G32R32F,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_UAV);
			Desc.NumMips = NumVarianceMips;
			VarianceTexture = GraphBuilder.CreateTexture(Desc, TEXT("PathTracer.Variance"), ERDGTextureFlags::MultiFrame);
		}
	}
	else
	{
		// If we are not using adaptive, make sure the old variance buffer doesn't stick around
		PathTracingState->VarianceRT.SafeRelease();
	}


	// should we use multiple GPUs to render the image?
	const FRHIGPUMask GPUMask = Config.UseMultiGPU ? FRHIGPUMask::All() : View.GPUMask;
	const int32 NumGPUs = GPUMask.GetNumActive();
	const int32 DispatchResX = View.ViewRect.Size().X;
	const int32 DispatchResY = View.ViewRect.Size().Y;
	const int32 DispatchSize = FMath::Max(CVarPathTracingDispatchSize.GetValueOnRenderThread(), 64);

	// When running with multiple GPUs, do that number of passes per frame, to keep the GPU work done per frame consistent
	// (given that each GPU processes a fraction of the pixels), but get the job done in fewer frames.
#if WITH_MGPU
	const int32 FramePassCount = !View.bIsOfflineRender && CVarPathTracingAdjustMultiGPUPasses.GetValueOnRenderThread() ? NumGPUs : 1;
#else
	const int32 FramePassCount = 1;
#endif

	bool bNeedsMoreRays = false;
	bool bNeedsTextureExtract = false;

	for (int32 FramePassIndex = 0; FramePassIndex < FramePassCount; FramePassIndex++)
	{
		// Setup temporal seed _after_ invalidation in case we got reset
		if (Config.LockedSamplingPattern)
		{
			// Count samples from 0 for deterministic results
			Config.PathTracingData.TemporalSeed = PathTracingState->SampleIndex;
		}
		else
		{
			// Count samples from an ever-increasing counter to avoid screen-door effect
			Config.PathTracingData.TemporalSeed = PathTracingState->FrameIndex;
		}
		Config.PathTracingData.TemporalSeed += SeedOffset;

		Config.PathTracingData.Iteration = PathTracingState->SampleIndex;
		Config.PathTracingData.BlendFactor = 1.0f / (Config.PathTracingData.Iteration + 1);

		bNeedsMoreRays = Config.PathTracingData.Iteration < MaxSPP;

		if (bNeedsMoreRays)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "Path Tracing Compute (%d x %d)", DispatchResX, DispatchResY);
			// Combined interface
			const bool bForceRebuild = CVarPathTracingHeterogeneousVolumesRebuildEveryFrame.GetValueOnRenderThread();
			bCreateVolumeGrids = bForceRebuild ||
				!PathTracingState->AdaptiveFrustumGridParameterCache.TopLevelGridBuffer ||
				!PathTracingState->AdaptiveOrthoGridParameterCache.TopLevelGridBuffer;
			if (bCreateVolumeGrids)
			{
				bool bMegavoxelPipeline = GetPathTracingHeterogeneousVolumesPipeline() == EPathTracingHeterogeneousVolumesPipeline::Megavoxel;

				FVoxelGridBuildOptions BuildOptions;
				BuildOptions.bBuildFrustumGrid &= bMegavoxelPipeline;
				BuildOptions.bBuildOrthoGrid &= bMegavoxelPipeline;
				BuildOrthoVoxelGrid(GraphBuilder, Scene, Views, VisibleLightInfos, BuildOptions, OrthoGridUniformBuffer);
				BuildFrustumVoxelGrid(GraphBuilder, Scene, Views[0], BuildOptions, FrustumGridUniformBuffer);
			}
			else
			{
				RegisterExternalOrthoVoxelGridUniformBuffer(GraphBuilder,
					PathTracingState->AdaptiveOrthoGridParameterCache,
					OrthoGridUniformBuffer
				);

				RegisterExternalFrustumVoxelGridUniformBuffer(GraphBuilder,
					PathTracingState->AdaptiveFrustumGridParameterCache,
					FrustumGridUniformBuffer
				);
			}

			// Preshading interface
			// NOTE: Removing in separate submission
			if (false)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "Heterogeneous Volumes: Preshading Pipeline");

				TArray<const IHeterogeneousVolumeInterface*> HeterogeneousVolumeInterfaces;
				TArray<const FMaterialRenderProxy*> MaterialRenderProxies;
				CollectRenderableHeterogeneousVolumes(
					GraphBuilder,
					Scene,
					View,
					HeterogeneousVolumeInterfaces,
					MaterialRenderProxies
				);

				if (!HeterogeneousVolumeInterfaces.IsEmpty())
				{
					FHeterogeneousVolumesPrimitiveInstanceKey Key;
					Key.PersistentPrimitiveIndex = 0;
					Key.InstanceIndex = 0;

					TRefCountPtr<IPooledRenderTarget>& PersistentMajorantRT = PathTracingState->HeterogeneousVolumesPersistentMajorantRT.FindOrAdd(Key);
					BuildPreshadingGrid(
						GraphBuilder,
						Scene,
						View,
						HeterogeneousVolumeInterfaces[0],
						MaterialRenderProxies[0],
						PersistentMajorantRT,
						PreshadingUniformBuffer
					);
				}
				else
				{
					BuildEmptyPreshadingUniformBuffer(GraphBuilder, PreshadingUniformBuffer);
				}
			}

			FRDGTexture* CloudAccelerationMap = PathTracingState->CloudAccelerationMap.IsValid() ? GraphBuilder.RegisterExternalTexture(PathTracingState->CloudAccelerationMap, TEXT("PathTracer.CloudAccelerationMap")) : nullptr;

			const bool bEnableClouds = (Config.PathTracingData.VolumeFlags & PATH_TRACER_VOLUME_ENABLE_CLOUDS) != 0 && CloudAccelerationMap != nullptr;

			if (bEnableClouds)
			{
				// clouds are enabled, build an accel map (do this every frame as clouds are usually animating, and so that the bounds improve during sampling)
				RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

				const int32 NumSamples = Config.CloudAccelerationMapNumSamples;
				const int32 Resolution = Config.CloudAccelerationMapResolution;

				// If we got here, the acceleration map texture should have already been created
				check(CloudAccelerationMap != nullptr);

				FVolumetricCloudRenderSceneInfo* CloudRenderSceneInfo = Scene->GetVolumetricCloudSceneInfo();
				check(CloudRenderSceneInfo != nullptr);
				UMaterialInterface* CloudMaterialInterface = CloudRenderSceneInfo->GetVolumetricCloudSceneProxy().GetCloudVolumeMaterial();
				check(CloudMaterialInterface != nullptr);
				const FMaterialRenderProxy* CloudVolumeMaterialProxy = CloudMaterialInterface->GetRenderProxy();
				check(CloudVolumeMaterialProxy != nullptr);
				const FMaterial* MaterialResource = &CloudVolumeMaterialProxy->GetMaterialWithFallback(Scene->GetFeatureLevel(), CloudVolumeMaterialProxy);
				

				const FPathTracingCloudParameters CloudParameters = PrepareCloudParameters(Scene, View, Config.CloudAccelerationMapResolution);
				// build cloud accel map
				{
					typename FPathTracingBuildCloudAccelerationMapCS::FPermutationDomain PermutationVector;
					TShaderRef<FPathTracingBuildCloudAccelerationMapCS> ComputeShader = MaterialResource->GetShader<FPathTracingBuildCloudAccelerationMapCS>(&FLocalVertexFactory::StaticType, PermutationVector, false);

					FPathTracingBuildCloudAccelerationMapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingBuildCloudAccelerationMapCS::FParameters>();
					PassParameters->NumSamples = NumSamples;
					PassParameters->Iteration = PathTracingState->SampleIndex;
					PassParameters->TemporalSeed = Config.PathTracingData.TemporalSeed;
					PassParameters->CloudParameters = CloudParameters;
					PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
					PassParameters->CloudAccelerationMap = GraphBuilder.CreateUAV(CloudAccelerationMap);

					FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(
						FIntPoint(Resolution, Resolution),
						FIntPoint(FComputeShaderUtils::kGolden2DGroupSize, FComputeShaderUtils::kGolden2DGroupSize));
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("Path Tracing Cloud Acceleration Map Build (Resolution=%u, NumSamples=%u)", Resolution, NumSamples),
						PassParameters,
						ERDGPassFlags::Compute,
						[Scene = Scene, CloudVolumeMaterialProxy, MaterialResource, PassParameters, ComputeShader, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
						{
							FMeshDrawShaderBindings ShaderBindings;
							UE::MeshPassUtils::SetupComputeBindings(ComputeShader, Scene, Scene->GetFeatureLevel(), nullptr, *CloudVolumeMaterialProxy, *MaterialResource, ShaderBindings);
							UE::MeshPassUtils::Dispatch(RHICmdList, ComputeShader, ShaderBindings, *PassParameters, GroupCount);
						});
					GraphBuilder.QueueTextureExtraction(CloudAccelerationMap, &PathTracingState->CloudAccelerationMap);
				}
			}
			if (CloudAccelerationMap == nullptr)
			{
				// Clouds are not enabled, make sure we don't accidentally try to use them during rendering
				CloudAccelerationMap = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
				Config.PathTracingData.VolumeFlags &= ~PATH_TRACER_VOLUME_ENABLE_CLOUDS;
			}

			// We are writing to the texture, we'll need to extract it...
			bNeedsTextureExtract = true;

			int32 CompactionDepth = CVarPathTracingCompactionDepth.GetValueOnRenderThread();
			if (CompactionDepth <= 0 || CompactionDepth > int32(Config.PathTracingData.MaxBounces))
			{
				CompactionDepth = Config.PathTracingData.MaxBounces;
			}

			const bool bUseIndirectDispatch = GRHISupportsRayTracingDispatchIndirect && CVarPathTracingIndirectDispatch.GetValueOnRenderThread();
			const int FlushRenderingCommands = CVarPathTracingFlushDispatch.GetValueOnRenderThread();

			FRDGBuffer* ActivePaths[2] = {};
			FRDGBuffer* NumPathStates = nullptr;
			FRDGBuffer* PathStateData = nullptr;
			{
				const int32 NumPaths = FMath::Min(
					DispatchSize * FMath::DivideAndRoundUp(DispatchSize, NumGPUs),
					DispatchResX * FMath::DivideAndRoundUp(DispatchResY, NumGPUs)
				);
				ActivePaths[0] = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumPaths), TEXT("PathTracer.ActivePaths0"));
				ActivePaths[1] = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumPaths), TEXT("PathTracer.ActivePaths1"));
				NumPathStates = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(CompactionDepth + 1), TEXT("PathTracer.NumPathStates"));
				PathStateData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(FPathTracingPackedPathState), NumPaths), TEXT("PathTracer.PathStateData"));
			}

			FPathTracingRG::FParameters* PreviousPassParameters = nullptr;
			// Divide each tile among all the active GPUs (interleaving scanlines)
			// The assumption is that the tiles are as big as possible, hopefully covering the entire screen
			// so rather than dividing tiles among GPUs, we divide each tile among all GPUs
			int32 CurrentGPU = 0; // keep our own counter so that we don't assume the assigned GPUs in the view mask are sequential
			for (int32 GPUIndex : GPUMask)
			{
				RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::FromIndex(GPUIndex));
				RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, NumGPUs >  1, "Path Tracing GPU%d", GPUIndex);
				for (int32 TileY = 0; TileY < DispatchResY; TileY += DispatchSize)
				{
					for (int32 TileX = 0; TileX < DispatchResX; TileX += DispatchSize)
					{
						const int32 DispatchSizeX = FMath::Min(DispatchSize, DispatchResX - TileX);
						const int32 DispatchSizeY = FMath::Min(DispatchSize, DispatchResY - TileY);

						const int32 DispatchSizeYSplit = FMath::DivideAndRoundUp(DispatchSizeY, NumGPUs);

						// Compute the dispatch size for just this set of scanlines
						const int32 DispatchSizeYLocal = FMath::Min(DispatchSizeYSplit, DispatchSizeY - CurrentGPU * DispatchSizeYSplit);

						RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, DispatchResX > DispatchSize || DispatchResY > DispatchSize, "Tile=(%d,%d - %dx%d)", TileX, TileY, DispatchSizeX, DispatchSizeYLocal);

						AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(NumPathStates, PF_R32_UINT), 0);
						if (Config.UseAdaptiveSampling && Config.PathTracingData.Iteration > 0)
						{
							// If we are using adaptive sampling, build a smaller list of active paths after the first iteration
							TShaderMapRef<FPathTracingAdaptiveStartCS> ComputeShader(GetGlobalShaderMap(View.FeatureLevel));

							FPathTracingAdaptiveStartCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingAdaptiveStartCS::FParameters>();

							PassParameters->VarianceTexture = GraphBuilder.CreateSRV(VarianceTexture);
							PassParameters->VarianceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
							PassParameters->VarianceTextureDims = FIntVector(DispatchResX, DispatchResY, NumVarianceMips);
							PassParameters->AdaptiveSamplingErrorThreshold = Config.AdaptiveSamplingThreshold;

							PassParameters->NextActivePaths = GraphBuilder.CreateUAV(ActivePaths[0], PF_R32_UINT);
							PassParameters->NumPathStates = GraphBuilder.CreateUAV(NumPathStates, PF_R32_UINT);

							PassParameters->TileTextureOffset.X = TileX;
							PassParameters->TileTextureOffset.Y = TileY + CurrentGPU * DispatchSizeYSplit;
							PassParameters->DispatchDim = FIntPoint(DispatchSizeX, DispatchSizeYLocal);

							FComputeShaderUtils::AddPass(
								GraphBuilder,
								RDG_EVENT_NAME("Prepare Adaptive Sampling Mask"),
								ComputeShader,
								PassParameters,
								FComputeShaderUtils::GetGroupCount(PassParameters->DispatchDim, FComputeShaderUtils::kGolden2DGroupSize));
						}

						// Run a pass per bounce, up until the compaction depth. Beyond that point, the path tracer will handle any remaining bounces.
						// Generally, since there is some launch overhead - it can be worthwhile to handle the "bounce tail" in a single loop because not too many paths survive
						for (int Bounce = 0; Bounce <= CompactionDepth; Bounce++)
						{
							FPathTracingRG::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingRG::FParameters>();
							PassParameters->TLAS = Scene->RayTracingScene.GetLayerView(ERayTracingSceneLayer::Base, View.GetRayTracingSceneViewHandle());
							PassParameters->DecalTLAS = Scene->RayTracingScene.GetLayerView(ERayTracingSceneLayer::Decals, View.GetRayTracingSceneViewHandle());
							PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
							PassParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();
							PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
							PassParameters->PathTracingData = Config.PathTracingData;
							PassParameters->RandomSequenceSpaceFillingCurve = GSystemTextures.GetSpaceFillingCurveTexture(GraphBuilder);
							PassParameters->StartingMediumData = GraphBuilder.CreateSRV(StartingMediumData, PF_R32_FLOAT);
							if (PreviousPassParameters == nullptr)
							{
								// upload sky/lights data
								RDG_GPU_MASK_SCOPE(GraphBuilder, GPUMask); // make sure this happens on all GPUs we will be rendering on
								SetLightParameters(
									GraphBuilder,
									Scene,
									View,
									Config.UseMISCompensation,
									(Config.PathTracingData.VolumeFlags & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE) != 0,
									&PassParameters->SkylightParameters,
									&PassParameters->LightGridParameters,
									&PassParameters->SceneVisibleLightCount,
									&PassParameters->SceneLightCount,
									&PassParameters->SceneLights
								);
							}
							else
							{
								// re-use from last iteration
								PassParameters->LightGridParameters = PreviousPassParameters->LightGridParameters;
								PassParameters->SceneLightCount = PreviousPassParameters->SceneLightCount;
								PassParameters->SceneVisibleLightCount = PreviousPassParameters->SceneVisibleLightCount;
								PassParameters->SceneLights = PreviousPassParameters->SceneLights;
								PassParameters->SkylightParameters = PreviousPassParameters->SkylightParameters;
							}
							PassParameters->DecalParameters = View.RayTracingDecalUniformBuffer;

							PassParameters->RadianceTexture = GraphBuilder.CreateUAV(RadianceTexture);
							PassParameters->AlbedoTexture = GraphBuilder.CreateUAV(AlbedoTexture);
							PassParameters->NormalTexture = GraphBuilder.CreateUAV(NormalTexture);
							PassParameters->DepthTexture = GraphBuilder.CreateUAV(DepthTexture);

							if (Config.UseAdaptiveSampling)
							{
								PassParameters->VarianceTexture = GraphBuilder.CreateUAV(VarianceTexture);
							}
							else
							{
								// this texture is not used in this case
								PassParameters->VarianceTexture = nullptr;
							}
							

							if (PreviousPassParameters != nullptr)
							{
								PassParameters->Atmosphere = PreviousPassParameters->Atmosphere;
								PassParameters->PlanetCenterTranslatedWorldHi = PreviousPassParameters->PlanetCenterTranslatedWorldHi;
								PassParameters->PlanetCenterTranslatedWorldLo = PreviousPassParameters->PlanetCenterTranslatedWorldLo;
							}
							else if ((Config.PathTracingData.VolumeFlags & PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE) != 0)
							{
								PassParameters->Atmosphere = Scene->GetSkyAtmosphereSceneInfo()->GetAtmosphereUniformBuffer();
								PreparePlanetCenter(View, Scene->GetSkyAtmosphereSceneInfo(), &PassParameters->PlanetCenterTranslatedWorldHi, &PassParameters->PlanetCenterTranslatedWorldLo);
							}
							else
							{
								FAtmosphereUniformShaderParameters AtmosphereParams = {};
								PassParameters->Atmosphere = CreateUniformBufferImmediate(AtmosphereParams, EUniformBufferUsage::UniformBuffer_SingleFrame);
								PassParameters->PlanetCenterTranslatedWorldHi = FVector3f(0);
								PassParameters->PlanetCenterTranslatedWorldLo = FVector3f(0);
							}
							PassParameters->AtmosphereOpticalDepthLUT = AtmosphereOpticalDepthLUT;
							PassParameters->AtmosphereOpticalDepthLUTSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

							if (PreviousPassParameters != nullptr)
							{
								PassParameters->CloudParameters = PreviousPassParameters->CloudParameters;
							}
							else if ((Config.PathTracingData.VolumeFlags & PATH_TRACER_VOLUME_ENABLE_CLOUDS) != 0)
							{
								PassParameters->CloudParameters = PrepareCloudParameters(Scene, View, Config.CloudAccelerationMapResolution);
							}
							else
							{
								PassParameters->CloudParameters = FPathTracingCloudParameters{};
							}
							PassParameters->CloudParameters.CloudCallableShaderId = View.PathTracingVolumetricCloudCallableShaderIndex;
							PassParameters->CloudAccelerationMap = CloudAccelerationMap;
							PassParameters->CloudAccelerationMapSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

							if ((Config.PathTracingData.VolumeFlags & PATH_TRACER_VOLUME_ENABLE_FOG) != 0)
							{
								PassParameters->FogParameters = PrepareFogParameters(View, Scene->ExponentialFogs[0]);
							}
							else
							{
								PassParameters->FogParameters = {};
							}

							// Heterogeneous volume bindings
							PassParameters->OrthoGridUniformBuffer = OrthoGridUniformBuffer;
							PassParameters->FrustumGridUniformBuffer = FrustumGridUniformBuffer;
							// NOTE: Remove in separate submission
							//PassParameters->PreshadingUniformBuffer = PreshadingUniformBuffer;

							// Heterogeneous volume callable interface
							if (bCallableShaderHeterogeneousVolumesPipeline)
							{
								PassParameters->HeterogeneousVolumesNum = PathTracingState->HeterogeneousVolumesCallableInfo.Num();
								PassParameters->HeterogeneousVolumesCallableShaderIdIntersect = GraphBuilder.CreateSRV(IntersectShaderIdsBuffer, PF_R32_SINT);
								PassParameters->HeterogeneousVolumesCallableShaderIdGetDensityBounds = GraphBuilder.CreateSRV(GetDensityBoundsShaderIdsBuffer, PF_R32_SINT);
								PassParameters->HeterogeneousVolumesCallableShaderIdGetDensity = GraphBuilder.CreateSRV(GetDensityShaderIdsBuffer, PF_R32_SINT);
								PassParameters->HeterogeneousVolumesCallableShaderIdSampleDistance = GraphBuilder.CreateSRV(SampleDistanceShaderIdsBuffer, PF_R32_SINT);
								PassParameters->HeterogeneousVolumesCallableShaderIdGetTransmittance = GraphBuilder.CreateSRV(GetTransmittanceShaderIdsBuffer, PF_R32_SINT);
							}

							PassParameters->TilePixelOffset.X = TileX;
							PassParameters->TilePixelOffset.Y = TileY + CurrentGPU;
							PassParameters->TileTextureOffset.X = TileX;
							PassParameters->TileTextureOffset.Y = TileY + CurrentGPU * DispatchSizeYSplit;
							PassParameters->ScanlineStride = NumGPUs;
							PassParameters->ScanlineWidth = DispatchSizeX;

							PassParameters->FirstBounce = Bounce;
							PassParameters->ActivePaths = GraphBuilder.CreateUAV(ActivePaths[Bounce & 1], PF_R32_UINT);
							PassParameters->NextActivePaths = GraphBuilder.CreateUAV(ActivePaths[(Bounce & 1) ^ 1], PF_R32_UINT);
							PassParameters->PathStateData = GraphBuilder.CreateUAV(PathStateData);
							PassParameters->NumPathStates = GraphBuilder.CreateUAV(NumPathStates, PF_R32_UINT);
							if (bUseIndirectDispatch)
							{
								PassParameters->PathTracingIndirectArgs = NumPathStates;
							}
							const bool bEnableDebug = CVarPathTracingDebug.GetValueOnRenderThread() > 0;
							if (bEnableDebug)
							{
								ShaderPrint::SetEnabled(true);
								ShaderPrint::RequestSpaceForCharacters(1024);
								ShaderPrint::RequestSpaceForLines(1024);
								ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrint);
							}


							const bool bUseCompaction = Bounce < CompactionDepth || Bounce == Config.PathTracingData.MaxBounces;
							TShaderMapRef<FPathTracingRG> RayGenShader(View.ShaderMap, GetPathTracingRGPermutation(View, *Scene, Bounce == 0, bUseCompaction));
							ClearUnusedGraphResources(RayGenShader, PassParameters, { PassParameters->PathTracingIndirectArgs });
							const bool bFlushRenderingCommands = FlushRenderingCommands == 1 || (FlushRenderingCommands == 2 && Bounce == CompactionDepth);
							const bool bUse1DDispatch = (Config.UseAdaptiveSampling && Config.PathTracingData.Iteration > 0) || Bounce > 0;
							GraphBuilder.AddPass(
								RDG_EVENT_NAME("Path Tracer Sample=%d/%d NumLights=%d (Bounce=%d%s%s)", PathTracingState->SampleIndex, MaxSPP, PassParameters->SceneLightCount, PassParameters->FirstBounce, bUseCompaction ? "" : "+", bUseIndirectDispatch && Bounce > 0 ? TEXT(" indirect") : TEXT("")),
								PassParameters,
								ERDGPassFlags::Compute,
								[PassParameters, RayGenShader, DispatchSizeX, DispatchSizeYLocal, bUseIndirectDispatch, bUse1DDispatch, GPUIndex, &View](FRDGAsyncTask, FRHICommandList& RHICmdList)
								{
									FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
									SetShaderParameters(GlobalResources, RayGenShader, *PassParameters);

									TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, PassParameters->Scene->GetRHI(), PassParameters->NaniteRayTracing->GetRHI(), RHICmdList);

									if (bUseIndirectDispatch && PassParameters->FirstBounce > 0)
									{
										PassParameters->PathTracingIndirectArgs->MarkResourceAsUsed();

										RHICmdList.RayTraceDispatchIndirect(
											View.MaterialRayTracingData.PipelineState,
											RayGenShader.GetRayTracingShader(),
											View.MaterialRayTracingData.ShaderBindingTable, GlobalResources,
											PassParameters->PathTracingIndirectArgs->GetIndirectRHICallBuffer(), PassParameters->FirstBounce * sizeof(FRHIDispatchIndirectParameters)
										);
									}
									else if (bUse1DDispatch)
									{
										RHICmdList.RayTraceDispatch(
											View.MaterialRayTracingData.PipelineState,
											RayGenShader.GetRayTracingShader(),
											View.MaterialRayTracingData.ShaderBindingTable, GlobalResources,
											DispatchSizeX * DispatchSizeYLocal, 1
										);
									}
									else
									{
										RHICmdList.RayTraceDispatch(
											View.MaterialRayTracingData.PipelineState,
											RayGenShader.GetRayTracingShader(),
											View.MaterialRayTracingData.ShaderBindingTable, GlobalResources,
											DispatchSizeX, DispatchSizeYLocal
										);
									}
								});

							if (bFlushRenderingCommands)
							{
								GraphBuilder.AddDispatchHint();
							}

							if (PreviousPassParameters == nullptr)
							{
								PreviousPassParameters = PassParameters;
							}
						}
					}
				}
				++CurrentGPU;
			}

			// Bump counters for next frame pass
			++PathTracingState->SampleIndex;
			++PathTracingState->FrameIndex;
		}
	}

	if (bNeedsTextureExtract)
	{
#if WITH_MGPU
		if (NumGPUs > 1)
		{
			// Need fences to prevent cross GPU copies from overlapping with rendering to the same buffers
			TArray<FTransferResourceFenceData*> CopyFenceDatas;
			CopyFenceDatas.AddUninitialized(NumGPUs - 1);
			for (int32 FenceIndex = 0; FenceIndex < NumGPUs - 1; FenceIndex++)
			{
				CopyFenceDatas[FenceIndex] = RHICreateTransferResourceFenceData();
			}

			{
				// Signal that the first GPU is done rendering, and other GPUs can copy to the buffer now.  Get all the GPUs
				// besides the first GPU into a mask.  These are the source GPUs for copies to the first GPU.
				FRHIGPUMask SrcGPUMask = FRHIGPUMask::FromIndex(GPUMask.GetLastIndex());
				for (uint32 SrcGPUIndex : GPUMask)
				{
					if (SrcGPUIndex != GPUMask.GetFirstIndex())
					{
						SrcGPUMask |= FRHIGPUMask::FromIndex(SrcGPUIndex);
					}
				}

				// Signal goes from first GPU (destination of copy), to remaining GPUs (sources of copy).
				RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::FromIndex(GPUMask.GetFirstIndex()));
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("Path Tracer Cross-GPU Signal (%d GPUs)", NumGPUs),
					ERDGPassFlags::None,
					[this, LocalCopyFenceDatas = CopyTemp(CopyFenceDatas), SrcGPUMask](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					RHICmdList.TransferResourceSignal(LocalCopyFenceDatas, SrcGPUMask);
				});
			}

			// Treat the cross GPU copy as occurring on all GPUs, for profiling purposes.  Internally, the cross GPU transfer doesn't
			// pay attention to the mask, so it has no effect on behavior.  Technically the work of the copy is done on the second GPU,
			// and the first GPU stalls waiting on that, so it's useful to show this interval on both GPUs.
			RDG_GPU_MASK_SCOPE(GraphBuilder, GPUMask);
			RDG_EVENT_SCOPE_STAT(GraphBuilder, PathTracingCopy, "PathTracingCopy");

			FMGPUTransferParameters* Parameters = GraphBuilder.AllocParameters<FMGPUTransferParameters>();
			Parameters->InputTexture = RadianceTexture;
			Parameters->InputAlbedo = AlbedoTexture;
			Parameters->InputNormal = NormalTexture;
			Parameters->InputDepth = DepthTexture;
			GraphBuilder.AddPass(RDG_EVENT_NAME("Path Tracer Cross-GPU Transfer (%d GPUs)", NumGPUs), Parameters, ERDGPassFlags::Readback,
				[Parameters, DispatchResX, DispatchResY, DispatchSize, GPUMask, MainGPUMask = View.GPUMask, CopyFenceDatas = MoveTemp(CopyFenceDatas)](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					const int32 FirstGPUIndex = MainGPUMask.GetFirstIndex();
					const int32 NumGPUs = GPUMask.GetNumActive();
					TArray<FTransferResourceParams> TransferParams;
					for (int32 TileY = 0; TileY < DispatchResY; TileY += DispatchSize)
					{
						for (int32 TileX = 0; TileX < DispatchResX; TileX += DispatchSize)
						{
							const int32 DispatchSizeX = FMath::Min(DispatchSize, DispatchResX - TileX);
							const int32 DispatchSizeY = FMath::Min(DispatchSize, DispatchResY - TileY);

							const int32 DispatchSizeYSplit = FMath::DivideAndRoundUp(DispatchSizeY, NumGPUs);

							// Divide each tile among all the active GPUs (interleaving scanlines)
							// The assumption is that the tiles are as big as possible, hopefully covering the entire screen
							// so rather than dividing tiles among GPUs, we divide each tile among all GPUs
							int32 CurrentGPU = 0; // keep our own counter so that we don't assume the assigned GPUs in the view mask are sequential
							for (int32 GPUIndex : GPUMask)
							{
								// Compute the dispatch size for just this set of scanlines
								const int32 DispatchSizeYLocal = FMath::Min(DispatchSizeYSplit, DispatchSizeY - CurrentGPU * DispatchSizeYSplit);
								// If this portion of the texture was not rendered by GPU0, transfer the rendered pixels there
								if (GPUIndex != FirstGPUIndex)
								{
									FIntRect TileToCopy;
									TileToCopy.Min.X = TileX;
									TileToCopy.Min.Y = TileY + CurrentGPU * DispatchSizeYSplit;
									TileToCopy.Max.X = TileX + DispatchSizeX;
									TileToCopy.Max.Y = TileToCopy.Min.Y + DispatchSizeYLocal;
									TransferParams.Emplace(Parameters->InputTexture->GetRHI(), TileToCopy, GPUIndex, FirstGPUIndex, true, true);
									TransferParams.Emplace(Parameters->InputAlbedo->GetRHI(), TileToCopy, GPUIndex, FirstGPUIndex, true, true);
									TransferParams.Emplace(Parameters->InputNormal->GetRHI(), TileToCopy, GPUIndex, FirstGPUIndex, true, true);
									TransferParams.Emplace(Parameters->InputDepth->GetRHI(), TileToCopy, GPUIndex, FirstGPUIndex, true, true);
								}
								++CurrentGPU;
							}
						}
					}

					// Include the fences we need to wait on in our list of transfers
					check(TransferParams.Num() >= CopyFenceDatas.Num());
					for (int32 FenceIndex = 0; FenceIndex < CopyFenceDatas.Num(); FenceIndex++)
					{
						TransferParams[FenceIndex].PreTransferFence = CopyFenceDatas[FenceIndex];
					}

					RHICmdList.TransferResources(TransferParams);
				}
			);
		}
#endif
		// After we are done, make sure we remember our texture for next time so that we can accumulate samples across frames
		GraphBuilder.QueueTextureExtraction(RadianceTexture, &PathTracingState->RadianceRT);
		GraphBuilder.QueueTextureExtraction(AlbedoTexture, &PathTracingState->AlbedoRT);
		GraphBuilder.QueueTextureExtraction(NormalTexture, &PathTracingState->NormalRT);
		GraphBuilder.QueueTextureExtraction(DepthTexture, &PathTracingState->DepthRT);
		if (Config.UseAdaptiveSampling)
		{
			check(VarianceTexture != nullptr);
			GraphBuilder.QueueTextureExtraction(VarianceTexture, &PathTracingState->VarianceRT);
		}
	}

	if (bCreateVolumeGrids)
	{
		ExtractOrthoVoxelGridUniformBuffer(GraphBuilder, OrthoGridUniformBuffer, PathTracingState->AdaptiveOrthoGridParameterCache);
		ExtractFrustumVoxelGridUniformBuffer(GraphBuilder, FrustumGridUniformBuffer, PathTracingState->AdaptiveFrustumGridParameterCache);
	}

	RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, PathTracingPost, "PathTracingPost");

	// Figure out if the denoiser is enabled and needs to run
	FRDGTexture* DenoisedRadianceTexture = nullptr;
	bool IsDenoiserEnabled = IsPathTracingDenoiserEnabled(View);
	int DenoiserMode = GetPathTracingDenoiserMode(View);

	// Request denoise if this is the last sample OR allow turning on the denoiser after the image has stopped accumulating samples
	const bool NeedsDenoise = IsDenoiserEnabled &&
		(((Config.PathTracingData.Iteration + 1) == MaxSPP) ||
		 (!bNeedsMoreRays && DenoiserMode != PathTracingState->LastConfig.DenoiserMode));

#if WITH_MGPU
	if (NumGPUs > 1)
	{
		// mGPU renders blocks of pixels that need to be mapped back into alternating scanlines
		// perform this swizzling now with a simple compute shader
		// NOTE: we only perform this swizzling for albedo/normals if we are going to use them for denoising

		TShaderMapRef<FPathTracingSwizzleScanlinesCS> ComputeShader(GetGlobalShaderMap(View.FeatureLevel));
		FRDGTexture* NewRadianceTexture = GraphBuilder.CreateTexture(RadianceTexture->Desc, TEXT("PathTracer.RadianceUnswizzled"));
		FRDGTexture* NewDepthTexture = GraphBuilder.CreateTexture(DepthTexture->Desc, TEXT("PathTracer.DepthUnswizzled"));
		FRDGTexture* NewNormalTexture = IsDenoiserEnabled ? GraphBuilder.CreateTexture(NormalTexture->Desc, TEXT("PathTracer.NormalUnswizzled")) : nullptr;
		FRDGTexture* NewAlbedoTexture = IsDenoiserEnabled ? GraphBuilder.CreateTexture(AlbedoTexture->Desc, TEXT("PathTracer.AlbedoUnswizzled")) : nullptr;

		FRDGTexture* InputTextures[4] = { RadianceTexture, NormalTexture, DepthTexture, AlbedoTexture};
		FRDGTexture* OutputTextures[4] = { NewRadianceTexture, NewNormalTexture, NewDepthTexture, NewAlbedoTexture};
		for (int Index = 0; Index < 4; Index++)
		{
			if (OutputTextures[Index] == nullptr)
			{
				// skip unused textures
				continue;
			}
			FPathTracingSwizzleScanlinesCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingSwizzleScanlinesCS::FParameters>();
			PassParameters->DispatchDim.X = DispatchResX;
			PassParameters->DispatchDim.Y = DispatchResY;
			PassParameters->TileSize.X = DispatchSize;
			PassParameters->TileSize.Y = DispatchSize;
			PassParameters->ScanlineStride = NumGPUs;
			PassParameters->InputTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(InputTextures[Index]));
			PassParameters->OutputTexture = GraphBuilder.CreateUAV(OutputTextures[Index]);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("UnswizzleScanlines(%d)", Index),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(FIntPoint(DispatchResX, DispatchResY), FComputeShaderUtils::kGolden2DGroupSize));
		}

		// let the remaining code operate on the unswizzled textures
		RadianceTexture = NewRadianceTexture;
		NormalTexture = NewNormalTexture;
		DepthTexture = NewDepthTexture;
		AlbedoTexture = NewAlbedoTexture;
	}
#endif

	// build adaptive sampling error map if we traced some rays
	if (Config.UseAdaptiveSampling && bNeedsMoreRays)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "Adaptive Sampling");
		FIntPoint BufferSize = View.ViewRect.Size();
		TShaderMapRef<FPathTracingBuildAdaptiveErrorTextureCS> ComputeShader(GetGlobalShaderMap(View.FeatureLevel));
		for (int MipLevel = 0; MipLevel < NumVarianceMips - 1; MipLevel++)
		{
			FPathTracingBuildAdaptiveErrorTextureCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPathTracingBuildAdaptiveErrorTextureCS::FParameters>();

			PassParameters->InputMipSampler = TStaticSamplerState<ESamplerFilter::SF_Bilinear>::CreateRHI();
			PassParameters->InputMip = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(VarianceTexture, MipLevel));
			PassParameters->OutputMip = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(VarianceTexture, MipLevel + 1));
			PassParameters->InputResolution = FIntPoint(
				FMath::Max(BufferSize.X >> MipLevel, 1),
				FMath::Max(BufferSize.Y >> MipLevel, 1));
			PassParameters->OutputResolution = FIntPoint(
				FMath::Max(BufferSize.X >> (MipLevel + 1), 1),
				FMath::Max(BufferSize.Y >> (MipLevel + 1), 1));
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("Build Error Estimation Mips (%d)", MipLevel),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(PassParameters->OutputResolution, FComputeShaderUtils::kGolden2DGroupSize));
		}
	}

	FPathTracingSpatialTemporalDenoisingContext DenoisingContext = {};
	DenoisingContext.SpatialTemporalDenoiserHistory = PathTracingState->SpatialTemporalDenoiserHistory;
	const bool EnablePathTracingDenoiserRealtimeDebug = ShouldEnablePathTracingDenoiserRealtimeDebug();

	if (IsDenoiserEnabled)
	{	
		if (PathTracingState->LastDenoisedRadianceRT)
		{
			// we already have a texture for this
			DenoisedRadianceTexture = GraphBuilder.RegisterExternalTexture(PathTracingState->LastDenoisedRadianceRT, TEXT("PathTracer.DenoisedRadiance"));
		}

		// 1. Prepass to estimate pixel variance
		FRDGBuffer* CurrentVarianceBufer = nullptr;
		{
			DenoisingContext.RadianceTexture = RadianceTexture;
			DenoisingContext.AlbedoTexture = AlbedoTexture;
			DenoisingContext.NormalTexture = NormalTexture;
			DenoisingContext.DepthTexture = DepthTexture;
			DenoisingContext.VarianceBuffer = PathTracingState->VarianceBuffer ? 
				GraphBuilder.RegisterExternalBuffer(PathTracingState->VarianceBuffer, TEXT("PathTracing.VarianceBuffer")) : nullptr;
			DenoisingContext.LastVarianceBuffer = PathTracingState->LastVarianceBuffer ?
				GraphBuilder.RegisterExternalBuffer(PathTracingState->LastVarianceBuffer, TEXT("PathTracing.LastVarianceBuffer")) : nullptr;

			PathTracingSpatialTemporalDenoisingPrePass(GraphBuilder, View, Config.PathTracingData.Iteration, MaxSPP, DenoisingContext);

			CurrentVarianceBufer = DenoisingContext.VarianceBuffer;
		}

		// 2. Denoising pass
		if (NeedsDenoise || EnablePathTracingDenoiserRealtimeDebug)
		{
			DenoisingContext.RadianceTexture = RadianceTexture;
			DenoisingContext.FrameIndex = PathTracingState->FrameIndex;
			DenoisingContext.VarianceBuffer = CurrentVarianceBufer;

			if (PathTracingState->LastDenoisedRadianceRT)
			{
				DenoisingContext.LastDenoisedRadianceTexture =
					GraphBuilder.RegisterExternalTexture(PathTracingState->LastDenoisedRadianceRT, TEXT("PathTracing.LastPreDenoisedRadiance"));
				DenoisingContext.LastRadianceTexture =
					GraphBuilder.RegisterExternalTexture(PathTracingState->LastRadianceRT, TEXT("PathTracing.LastRadianceTexture"));
				DenoisingContext.LastAlbedoTexture =
					GraphBuilder.RegisterExternalTexture(PathTracingState->LastAlbedoRT, TEXT("PathTracing.LastAlbedoTexture"));
				DenoisingContext.LastNormalTexture =
					GraphBuilder.RegisterExternalTexture(PathTracingState->LastNormalRT, TEXT("PathTracing.LastNormalTexture"));
				DenoisingContext.LastDepthTexture =
					GraphBuilder.RegisterExternalTexture(PathTracingState->LastDepthRT, TEXT("PathTracing.LastDepthTexture"));
			}

			PathTracingSpatialTemporalDenoising(GraphBuilder,
				View,
				DenoiserMode,
				DenoisedRadianceTexture,
				DenoisingContext);

			GraphBuilder.QueueTextureExtraction(DenoisedRadianceTexture, &PathTracingState->LastDenoisedRadianceRT);
			GraphBuilder.QueueTextureExtraction(AlbedoTexture, &PathTracingState->LastAlbedoRT);
			GraphBuilder.QueueTextureExtraction(NormalTexture, &PathTracingState->LastNormalRT);
			GraphBuilder.QueueTextureExtraction(DepthTexture, &PathTracingState->LastDepthRT);
			GraphBuilder.QueueTextureExtraction(RadianceTexture, &PathTracingState->LastRadianceRT);

			PathTracingState->SpatialTemporalDenoiserHistory = DenoisingContext.SpatialTemporalDenoiserHistory;
		}

		// 3. Update pixel variance
		if (CurrentVarianceBufer)
		{
			GraphBuilder.QueueBufferExtraction(CurrentVarianceBufer, 
				(NeedsDenoise || EnablePathTracingDenoiserRealtimeDebug) ?
				&PathTracingState->LastVarianceBuffer:
				&PathTracingState->VarianceBuffer);

			if (NeedsDenoise || EnablePathTracingDenoiserRealtimeDebug)
			{
				PathTracingState->VarianceBuffer = nullptr;
			}
		}
		
	}
	PathTracingState->LastConfig.DenoiserMode = DenoiserMode;

	// now add a pixel shader pass to display our Radiance buffer and write to the depth buffer

	FPathTracingCompositorPS::FParameters* DisplayParameters = GraphBuilder.AllocParameters<FPathTracingCompositorPS::FParameters>();
	DisplayParameters->Iteration = Config.PathTracingData.Iteration;
	DisplayParameters->MaxSamples = MaxSPP;
	DisplayParameters->ProgressDisplayEnabled = CVarPathTracingProgressDisplay.GetValueOnRenderThread();
	DisplayParameters->AdaptiveSamplingErrorThreshold = Config.AdaptiveSamplingThreshold;
	DisplayParameters->AdaptiveSamplingVisualize = Config.UseAdaptiveSampling ? CVarPathTracingAdaptiveSamplingVisualize.GetValueOnRenderThread() : 0;
	DisplayParameters->VarianceTextureDims = FIntVector(DispatchResX, DispatchResY, NumVarianceMips);
	DisplayParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	DisplayParameters->RadianceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DenoisedRadianceTexture ? DenoisedRadianceTexture : RadianceTexture));
	DisplayParameters->VarianceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(VarianceTexture ? VarianceTexture : GSystemTextures.GetBlackDummy(GraphBuilder)));
	DisplayParameters->DepthTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DepthTexture));
	DisplayParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorOutputTexture, ERenderTargetLoadAction::ELoad);
	DisplayParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthOutputTexture,  ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilNop);
	DisplayParameters->VarianceSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	DisplayParameters->PreExposure = View.PreExposure / Config.PathTracingData.BaseExposure;

	FScreenPassTextureViewport Viewport(SceneColorOutputTexture, View.ViewRect);

	const bool IsCursorInsideView = View.CursorPos.X != -1 || View.CursorPos.Y != -1;
	// wiper mode - reveals the render below the path tracing display
	// NOTE: we still path trace the full resolution even while wiping the cursor so that rendering does not get out of sync
	if (CVarPathTracingWiperMode.GetValueOnRenderThread())
	{
		float DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(View.CursorPos.X, View.CursorPos.Y);
		
		if (IsCursorInsideView)
		{
			Viewport.Rect.Min.X = View.CursorPos.X / DPIScale;
		}
		else
		{
			Viewport.Rect.Min.X = 0.5 * View.ViewRect.Min.X + 0.5 * View.ViewRect.Max.X;
		}
	}

	TShaderMapRef<FPathTracingCompositorPS> PixelShader(View.ShaderMap);
	TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
	FRHIBlendState* BlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
	FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState<true /* bEnableDepthWrite */, CF_Always>::GetRHI();

	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("Path Tracer Display (%d x %d)", View.ViewRect.Size().X, View.ViewRect.Size().Y),
		View,
		Viewport,
		Viewport,
		VertexShader,
		PixelShader,
		BlendState,
		DepthStencilState,
		DisplayParameters);

	// Setup the path tracing resources to be used by post process pass.
	if (CVarPathTracingOutputPostProcessResources.GetValueOnRenderThread())
	{
		PathTracingResources.bPostProcessEnabled = true;
		PathTracingResources.DenoisedRadiance = DenoisedRadianceTexture ? DenoisedRadianceTexture : RadianceTexture;
		PathTracingResources.Radiance = RadianceTexture;
		PathTracingResources.Albedo = AlbedoTexture;
		PathTracingResources.Normal = NormalTexture;
		PathTracingResources.Variance = DenoisingContext.VarianceTexture;
	}

	// Add a visualization path for denoising
	if (NeedsDenoise || EnablePathTracingDenoiserRealtimeDebug)
	{
		FVisualizePathTracingDenoisingInputs Inputs;
		Inputs.SceneColor =SceneColorOutputTexture;

		FScreenPassTextureViewport MotionVectorViewport(SceneColorOutputTexture, View.ViewRect);
		if (CVarPathTracingWiperMode.GetValueOnRenderThread())
		{
			float DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(View.CursorPos.X, View.CursorPos.Y);
			if (IsCursorInsideView)
			{
				MotionVectorViewport.Rect.Max.X = View.CursorPos.X / DPIScale;
			}
			else
			{
				MotionVectorViewport.Rect.Max.X = 0.5 * View.ViewRect.Min.X + 0.5 * View.ViewRect.Max.X;
			}
		}

		Inputs.Viewport = MotionVectorViewport;

		Inputs.DenoisingContext = DenoisingContext;
		Inputs.SceneTexturesUniformBuffer = SceneTexturesUniformBuffer;
		Inputs.DenoisedTexture = DenoisedRadianceTexture;

		AddVisualizePathTracingDenoisingPass(GraphBuilder, View, Inputs);
	}
}

#else // RHI_RAYTRACING == false

class FSceneViewFamily;
class FViewInfo;

namespace PathTracing
{
	bool ShouldCompilePathTracingShadersForProject(EShaderPlatform ShaderPlatform)
	{
		return false;
	}

	bool UsesDecals(const FSceneViewFamily& ViewFamily)
	{
		return false;
	}

	bool UsesReferenceAtmosphere(const FViewInfo& View)
	{
		return false;
	}

	bool UsesReferenceDOF(const FViewInfo& View)
	{
		return false;
	}

	bool NeedsAntiAliasing(const FViewInfo& View)
	{
		return false;
	}

	bool NeedsTonemapping()
	{
		return false;
	}
}

#endif
