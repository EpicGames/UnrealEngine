// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingResources.h"
#include "ShaderParameterMacros.h"
#include "PathTracingOutputInvalidateReason.h"

// this struct holds skylight parameters
BEGIN_SHADER_PARAMETER_STRUCT(FPathTracingSkylight, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SkylightTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SkylightPdf)
	SHADER_PARAMETER_SAMPLER(SamplerState, SkylightTextureSampler)
	SHADER_PARAMETER(float, SkylightInvResolution)
	SHADER_PARAMETER(int32, SkylightMipCount)
END_SHADER_PARAMETER_STRUCT()

class FRDGBuilder;
class FScene;
class FViewInfo;
class FSceneViewFamily;
class FRHIRayTracingShader;
class FGlobalShaderMap;

bool PrepareSkyTexture(FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View, bool SkylightEnabled, bool UseMISCompensation, FPathTracingSkylight* SkylightParameters);
RENDERER_API FRHIRayTracingShader* GetPathTracingDefaultMissShader(const FGlobalShaderMap* ShaderMap);
RENDERER_API FRHIRayTracingShader* GetPathTracingDefaultOpaqueHitShader(const FGlobalShaderMap* ShaderMap);
RENDERER_API FRHIRayTracingShader* GetPathTracingDefaultHiddenHitShader(const FGlobalShaderMap* ShaderMap);

RENDERER_API FRHIRayTracingShader* GetGPULightmassDefaultMissShader(const FGlobalShaderMap* ShaderMap);
RENDERER_API FRHIRayTracingShader* GetGPULightmassDefaultOpaqueHitShader(const FGlobalShaderMap* ShaderMap);
RENDERER_API FRHIRayTracingShader* GetGPULightmassDefaultHiddenHitShader(const FGlobalShaderMap* ShaderMap);

void PreparePathTracingRTPSO();
void PreparePathTracingCloudMaterial(FRDGBuilder& GraphBuilder, FScene* Scene, TArrayView<FViewInfo> Views);
void PreparePathTracingHeterogeneousVolumes(FRDGBuilder& GraphBuilder, FScene* Scene, const TArrayView<FViewInfo>& Views);

namespace PathTracing
{
	bool ShouldCompilePathTracingShadersForProject(EShaderPlatform ShaderPlatform);
	bool UsesDecals(const FSceneViewFamily& ViewFamily);
	bool UsesReferenceAtmosphere(const FViewInfo& View);
	bool UsesReferenceDOF(const FViewInfo& View);
	bool NeedsAntiAliasing(const FViewInfo& View);
	bool NeedsTonemapping();
	bool IsOutputInvalidateAllowed(EInvalidateReason InvalidateReason);
}
