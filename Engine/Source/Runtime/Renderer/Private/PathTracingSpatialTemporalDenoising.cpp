// Copyright Epic Games, Inc. All Rights Reserved.

#include "PathTracingSpatialTemporalDenoising.h"
#include "PathTracing.h"
#include "RHI.h"

class FDenoiserManager
{
public:
	static FDenoiserManager& Get()
	{
		static FDenoiserManager DenoiserManager;
		return DenoiserManager;
	}

	void RegisterSpatialDenoiser(TUniquePtr<UE::Renderer::Private::IPathTracingDenoiser> InDenoiser, FString Name)
	{
		check(!SpatialDenoiser.Contains(Name));

		bNeedTextureCreateExtraFlags |= InDenoiser->NeedTextureCreateExtraFlags();
		SpatialDenoiser.Add(Name, MoveTemp(InDenoiser));
	}
	void RegisterSpatialTemporalDenoiser(TUniquePtr<UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser> InDenoiser, FString Name)
	{
		check(!PathTracingSpatialTemporalDenoisers.Contains(Name));
		bNeedTextureCreateExtraFlags |= InDenoiser->NeedTextureCreateExtraFlags();
		PathTracingSpatialTemporalDenoisers.Add(Name, MoveTemp(InDenoiser));
	}

	bool HasSpatialDenoiser()const { return SpatialDenoiser.Num() > 0; }
	bool HasSpatialTemporalDenoiser()const { return PathTracingSpatialTemporalDenoisers.Num() > 0; }
	bool HasDenoiser()const { return HasSpatialDenoiser() || HasSpatialTemporalDenoiser();}

	UE::Renderer::Private::IPathTracingDenoiser* GetSpatialDenoiser(FString Name, bool bMatch)
	{
		if (SpatialDenoiser.Contains(Name))
		{
			return SpatialDenoiser[Name].Get();
		}
		else if (SpatialDenoiser.Num() > 0 && !bMatch)
		{
			return SpatialDenoiser.CreateConstIterator()->Value.Get();
		}

		return nullptr;
	}

	UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser* GetSpatialTemporalDenoiser(FString Name, bool bMatch)
	{
		if (PathTracingSpatialTemporalDenoisers.Contains(Name))
		{
			return PathTracingSpatialTemporalDenoisers[Name].Get();
		}
		else if (PathTracingSpatialTemporalDenoisers.Num() > 0 && !bMatch)
		{
			return PathTracingSpatialTemporalDenoisers.CreateConstIterator()->Value.Get();
		}

		return nullptr;
	}

	void UnregisterDenoiser(FString Name)
	{
		SpatialDenoiser.Remove(Name);
		PathTracingSpatialTemporalDenoisers.Remove(Name);
	}

	/** If any plugin needs extra creation flag*/
	bool NeedTextureCreateExtraFlags()
	{
		return bNeedTextureCreateExtraFlags;
	}

private:
	FDenoiserManager() = default;
	FDenoiserManager(const FDenoiserManager&) = delete;
	FDenoiserManager& operator=(const FDenoiserManager&) = delete;

	TMap<FString, TUniquePtr<UE::Renderer::Private::IPathTracingDenoiser> > SpatialDenoiser;

	/**Spatial-temporal denoiser can work as both spatial and temporal denoiser*/
	TMap<FString, TUniquePtr<UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser> > PathTracingSpatialTemporalDenoisers;

	/** Since all plugins can be dynamically switched, need to increase the compatibility*/
	bool bNeedTextureCreateExtraFlags = false;
};

void RegisterSpatialDenoiser(TUniquePtr<UE::Renderer::Private::IPathTracingDenoiser> PathTracingDenoiser, FString Name)
{
	FDenoiserManager::Get().RegisterSpatialDenoiser(MoveTemp(PathTracingDenoiser), Name);
}
void RegisterSpatialTemporalDenoiser(TUniquePtr<UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser> PathTracingDenoiser, FString Name)
{
	FDenoiserManager::Get().RegisterSpatialTemporalDenoiser(MoveTemp(PathTracingDenoiser), Name);
}

void UnregisterDenoiser(FString Name)
{
	FDenoiserManager::Get().UnregisterDenoiser(Name);
}

bool HasTemporalDenoiser()
{
	return FDenoiserManager::Get().HasSpatialTemporalDenoiser();
}

#if RHI_RAYTRACING

#include "DeferredShadingRenderer.h"
#include "GenerateMips.h"
#include "GlobalShader.h"
#include "HAL/PlatformApplicationMisc.h"
#include "PathTracingDefinitions.h"
#include "RayTracing/RayTracingMaterialHitShaders.h"
#include "RayTracingDefinitions.h"
#include "RayTracingTypes.h"
#include "RenderGraphDefinitions.h"
#include "PostProcess/PostProcessMaterial.h"
#include <limits>

DEFINE_LOG_CATEGORY_STATIC(LogPathTracingDenoising, Log, All);

namespace {

	TAutoConsoleVariable<int32> CVarPathTracingDenoiser(
		TEXT("r.PathTracing.Denoiser"),
		-1,
		TEXT("Enable denoising of the path traced output (if a denoiser plugin is active) (default = -1 (driven by postprocesing volume))\n")
		TEXT("-1: inherit from PostProcessVolume\n")
		TEXT("0: disable denoiser\n")
		TEXT("1: enable denoiser (if a denoiser plugin is active)\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingDenoiserNormalSpace(
		TEXT("r.PathTracing.Denoiser.NormalSpace"),
		0,
		TEXT("The space normal is in\n")
		TEXT("0: World space (default)\n")
		TEXT("1: Camera space. Some denoisers require camera space normal\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingDenoiserPrepassVarianceType(
		TEXT("r.PathTracing.Denoiser.Prepass.VarianceType"),
		1,
		TEXT("Select the per-pixel variance type:\n")
		TEXT("0: Multiple channel (RGB) variance for radiance\n")
		TEXT("1: Combined single channel variance for radiance, albedo and normal\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingDenoiserPrepassOutputVarianceTexture(
		TEXT("r.PathTracing.Denoiser.Prepass.OutputVarianceTexture"),
		0,
		TEXT("0: Output variance texture on demand\n")
		TEXT("1: Always output variance texture to denoisers, or the postprocess material usually used by MRQ\n")
	);

	TAutoConsoleVariable<int32> CVarPathTracingDenoiserPrepassRankedLuminanceVariance(
		TEXT("r.PathTracing.Denoiser.Prepass.RankedLuminanceVariance"),
		0,
		TEXT("Select the luminance type when calculating the variance:\n")
		TEXT("0: use default luminance to estimate variance (Most clarity but may preserve some noise if variance is too high).\n")
		TEXT("1: Use channel ranked luminance when calculating variance (better in high variance and low motion scene, e.g., dawn, dusk, or low lighting).\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingSpatialDenoiser(
		TEXT("r.PathTracing.SpatialDenoiser"),
		1,
		TEXT("Enable spatial denoising of the path traced output\n")
		TEXT("-1: inherit from PostProcessVolume\n")
		TEXT("0: disable denoiser\n")
		TEXT("1: enable denoiser (if a denoiser plugin is active)\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<FString> CVarPathTracingDenoiserName(
		TEXT("r.PathTracing.Denoiser.Name"),
		"NNEDenoiser",
		TEXT("Set the spatial denoiser name. It is the corresponding name registered by a denoiser plugin\n")
		TEXT("Any registered denoiser should be able to denoise a single frame spatially.\n")
	);

	TAutoConsoleVariable<FString> CVarPathTracingTemporalDenoiserName(
		TEXT("r.PathTracing.TemporalDenoiser.Name"),
		"NFOR",
		TEXT("Set the temporal denoiser name. It is the corresponding name registered by a denoiser plugin\n")
		TEXT("The temporal denoiser usually has better temporal stability when rendering offline.\n")
	);

	TAutoConsoleVariable<int32> CVarPathTracingSpatialDenoiserType(
		TEXT("r.PathTracing.SpatialDenoiser.Type"),
		0,
		TEXT("The type of spatial denoiser\n")
		TEXT("0: Use spatial denoiser only plugin\n")
		TEXT("1: Use spatial denoiser plugin that also provides temporal denoising\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiser(
		TEXT("r.PathTracing.TemporalDenoiser"),
		0,
		TEXT("Enable temporal denoising of the path traced output\n")
		TEXT("-1: inherit from PostProcessVolume (TODO when out of experimental phase)\n")
		TEXT("0: disable denoiser\n")
		TEXT("1: enable denoiser (if a denoiser plugin is active)\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserType(
		TEXT("r.PathTracing.TemporalDenoiser.Type"),
		0,
		TEXT("The type of temporal denoiser\n")
		TEXT("0: Use the built-in temporal denoiser\n")
		TEXT("1: Use the temporal denoiser from plugin\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserMotionVectorType(
		TEXT("r.PathTracing.TemporalDenoiser.MotionVector.Type"),
		0,
		TEXT("The type of motion vecotr estimation algorithm\n")
		TEXT("0: Built-in motion vector estimator\n")
		TEXT("1: Motion vector estimator from the plugin\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserVisualizeMotionVector(
		TEXT("r.PathTracing.TemporalDenoiser.VisualizeMotionVector"),
		0,
		TEXT("1: visualize the motion vector compared to the raster motion vector\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserEnableSubPixelOffset(
		TEXT("r.PathTracing.TemporalDenoiser.EnableSubPixelOffset"),
		1,
		TEXT("Enable subpixel offset when merging\n")
		TEXT("-1: inherit from PostProcessVolume\n")
		TEXT("0: disable denoiser\n")
		TEXT("1: enable denoiser (if a denoiser plugin is active)\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<float> CVarPathTracingTemporalDenoiserKappa(
		TEXT("r.PathTracing.TemporalDenoiser.kappa"),
		-1.0f,
		TEXT("Scaling parameter to determine how fast the history weight falls and the cutting point to zero. Use DeltaE to derive kappa if -1\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<float> CVarPathTracingTemporalDenoiserEta(
		TEXT("r.PathTracing.TemporalDenoiser.eta"),
		1.0f,
		TEXT("Eta param. Error distance below this will have max history weight. Use DeltaE to derive Eta if -1\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<float> CVarPathTracingTemporalDenoiserDeltaE(
		TEXT("r.PathTracing.TemporalDenoiser.DeltaE"),
		5.0f,
		TEXT("Cut off the history weight to zero at CIE DeltaE for low frequency.\n")
		TEXT("1.0 :the just noticeable difference (JND),\n")
		TEXT("2.0 :perceptible for close look\n")
		TEXT("10.0:Perceptible at a glance ")
		TEXT("This works as an alternative control instead of kappa. 2 as default."),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<float> CVarPathTracingTemporalDenoiserDeltaEHighFrequency(
		TEXT("r.PathTracing.TemporalDenoiser.DeltaE.HighFrequency"),
		2.1f,
		TEXT("Cut off the history weight to zero when using high frequency per pixel difference.\n")
		TEXT("It should only be enabled when the source image is smooth or denoised."),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<float> CVarPathTracingTemporalDenoiserAlpha(
		TEXT("r.PathTracing.TemporalDenoiser.alpha"),
		1.0f,
		TEXT("The weight of history in the exponential mean average\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserMode(
		TEXT("r.PathTracing.TemporalDenoiser.mode"),
		1,
		TEXT("0: disabled \n")
		TEXT("1: offline rendering only\n")
		TEXT("2: online rendering (for debug)\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserSource(
		TEXT("r.PathTracing.TemporalDenoiser.source"),
		0,
		TEXT("0: Denoised Radance when possible (Default) \n")
		TEXT("1: Normal\n")
		TEXT("2: Albedo\n")
		TEXT("3: Raw Radiance")
		TEXT("Otherwise: Feature Fusion (TODO)"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserDenoiseSourceImageFirst(
		TEXT("r.PathTracing.TemporalDenoiser.DenoiseSourceImageFirst"),
		0,
		TEXT("Denoise the source image with IntelImageDenoisier"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserSubPixelOffsetStartMip(
		TEXT("r.PathTracing.TemporalDenoiser.SubPixelOffset.StartMip"),
		8,
		TEXT("From 0 to this mip, we will perform subpixel offset"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserMotionOperation(
	   TEXT("r.PathTracing.TemporalDenoiser.MotionOperation"),
		1,
		TEXT("0: use the motion vector directly estimated")
		TEXT("1: subtract between the motion"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserVisWarp(
		TEXT("r.PathTracing.TemporalDenoiser.VisWarp"),
		0,
		TEXT("0: disable")
		TEXT("1: visualize warped source by the motion vector")
		TEXT("2: weights, warped source, and combined"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserDistanceMetrics(
		TEXT("r.PathTracing.TemporalDenoiser.DistanceMetrics"),
		2,
		TEXT("0: Luminance based metrics for distance estimation. Color with same luminance will create error in motion and history weights estimation.")
		TEXT("1: Direct color difference")
		TEXT("2: Visual color difference based on CIELAB2000 color difference."),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserTotalVariation(
		TEXT("r.PathTracing.TemporalDenoiser.TotalVariation"),
		1,
		TEXT("!=0: Use less history if the total variation is large in a local patch"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarPathTracingTemporalDenoiserPatchCount(
		TEXT("r.PathTracing.TemporalDenoiser.PatchCount"),
		1,
		TEXT("The number of similar patches found by Non-Local Mean to use for temporal denoising\n")
		TEXT("1: default. Accumulae the one with the minimal distance exponentially.")
		TEXT(">1 && < 16: Use bilaterial filtering to accumulate multiple patches."),
		ECVF_RenderThreadSafe
	);

	DECLARE_GPU_STAT(PathTracingSpatialTemporalDenoising);
}

int GetPathTracingDenoiserMode(const FViewInfo& View)
{
	int DenoiserMode = CVarPathTracingDenoiser.GetValueOnRenderThread();
	if (DenoiserMode < 0)
	{
		DenoiserMode = View.FinalPostProcessSettings.PathTracingEnableDenoiser;
	}
	return DenoiserMode;
}

bool IsPathTracingDenoiserEnabled(const FViewInfo& View)
{
	return GetPathTracingDenoiserMode(View) != 0 && FDenoiserManager::Get().HasDenoiser();
}

static bool ShouldDenoiseWithNormalInCameraSpace()
{
	int NormalSpace = CVarPathTracingDenoiserNormalSpace.GetValueOnRenderThread();
	return NormalSpace != 0;
}

enum class ESpatialDenoiserType : int32
{
	NONE = -1,
	SPATIAL_DENOISER_PLUGIN,
	SPATIAL_TEMPORAL_DENOISER_PLUGIN,
	MAX
};

enum class ETemporalDenoisingMode : uint32 
{
	ETDM_DISABLED,
	ETDM_OFFLINE,
	ETDM_ONLINE,
	ETDM_MAX
};

enum class ETemporalDenoiserType : int32
{
	NONE = -1,
	BUILTIN_TEMPORAL_DENOISER,
	SPATIAL_TEMPORAL_DENOISER_PLUGIN,
	MAX
};

enum class ETemporalDenoiserMotionVectorType : int32
{
	NONE = -1,
	BUILTIN,
	PLUGIN,
	MAX
};

bool ShouldEnablePathTracingDenoiserRealtimeDebug()
{
	int TemporalDenoiserMode = CVarPathTracingTemporalDenoiserMode.GetValueOnRenderThread();
	TemporalDenoiserMode = FMath::Clamp(TemporalDenoiserMode, 0, static_cast<int32>(ETemporalDenoisingMode::ETDM_MAX));
	ETemporalDenoisingMode Mode = static_cast<ETemporalDenoisingMode>(TemporalDenoiserMode);

	return Mode == ETemporalDenoisingMode::ETDM_ONLINE;
}

static bool ShouldApplySpatialDenoiser()
{
	return CVarPathTracingSpatialDenoiser.GetValueOnRenderThread() != 0;
}


TArray<ESpatialDenoiserType> GetAvailableSpatialDenoiserTypes()
{
	TArray<ESpatialDenoiserType> Types;
	if (FDenoiserManager::Get().HasSpatialTemporalDenoiser())
	{
		Types.Add(ESpatialDenoiserType::SPATIAL_TEMPORAL_DENOISER_PLUGIN);
	}

	if (FDenoiserManager::Get().HasSpatialDenoiser())
	{
		Types.Add(ESpatialDenoiserType::SPATIAL_DENOISER_PLUGIN);
	}

	if (Types.Num() == 0)
	{
		Types.Add(ESpatialDenoiserType::NONE);
	}
	return Types;
}

ESpatialDenoiserType GetSpatialDenoiserType()
{
	int32 Type = CVarPathTracingSpatialDenoiserType.GetValueOnRenderThread();

	Type = FMath::Clamp(Type,
		static_cast<int32>(ESpatialDenoiserType::NONE) + 1,
		static_cast<int32>(ESpatialDenoiserType::MAX) - 1);

	ESpatialDenoiserType DenoiserType = static_cast<ESpatialDenoiserType>(Type);

	if (DenoiserType == ESpatialDenoiserType::SPATIAL_DENOISER_PLUGIN && FDenoiserManager::Get().HasSpatialDenoiser())
	{
		return DenoiserType;
	}
	else if (DenoiserType == ESpatialDenoiserType::SPATIAL_TEMPORAL_DENOISER_PLUGIN && FDenoiserManager::Get().HasSpatialTemporalDenoiser())
	{
		return DenoiserType;
	}
	else
	{
		// Fallback to the one that is available instead of the requested one
		DenoiserType = GetAvailableSpatialDenoiserTypes()[0];
	}

	return DenoiserType;
}

static bool ShouldApplyTemporalDenoiser(const FPathTracingSpatialTemporalDenoisingContext& DenoisingContext, const FViewInfo& View)
{
	const bool bApplyTemporalDenoiser =
		(CVarPathTracingTemporalDenoiser.GetValueOnAnyThread() == 1) &&
		DenoisingContext.LastDenoisedRadianceTexture &&
		DenoisingContext.LastDenoisedRadianceTexture != DenoisingContext.RadianceTexture;

	return bApplyTemporalDenoiser;
}

ETemporalDenoiserType GetTemporalDenoiserType()
{

	ESpatialDenoiserType SpatialDenoiserType = GetSpatialDenoiserType();

	if ( SpatialDenoiserType == ESpatialDenoiserType::SPATIAL_DENOISER_PLUGIN)
	{
		return ETemporalDenoiserType::BUILTIN_TEMPORAL_DENOISER;
	}
	else if (SpatialDenoiserType == ESpatialDenoiserType::SPATIAL_TEMPORAL_DENOISER_PLUGIN)
	{
		return ETemporalDenoiserType::SPATIAL_TEMPORAL_DENOISER_PLUGIN;
	}

	return ETemporalDenoiserType::NONE;

}

ETemporalDenoiserMotionVectorType GetTemporalDenoiserMotionVectorType()
{
	int32 Type = CVarPathTracingTemporalDenoiserMotionVectorType.GetValueOnRenderThread();

	Type = FMath::Clamp(Type,
		static_cast<int32>(ETemporalDenoiserMotionVectorType::NONE) + 1,
		static_cast<int32>(ETemporalDenoiserMotionVectorType::MAX) - 1);

	return static_cast<ETemporalDenoiserMotionVectorType>(Type);
}

UE::Renderer::Private::IPathTracingDenoiser* GetActiveSpatialDenoiser(bool bMatch = false)
{
	FString DenoiserName = CVarPathTracingDenoiserName.GetValueOnRenderThread();
	return FDenoiserManager::Get().GetSpatialDenoiser(DenoiserName, bMatch);
}

UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser* GetActiveSpatialTemporalDenoiser(bool bMatch = false)
{
	FString DenoiserName = CVarPathTracingTemporalDenoiserName.GetValueOnRenderThread();
	return FDenoiserManager::Get().GetSpatialTemporalDenoiser(DenoiserName, bMatch);
}

ETextureCreateFlags GetExtraTextureCreateFlagsForDenoiser()
{
	ESpatialDenoiserType SpatialDenoiserType = GetSpatialDenoiserType();
	ETemporalDenoiserType TemporalDenoiserType = GetTemporalDenoiserType();
	ETextureCreateFlags TextureCreateFlags = ETextureCreateFlags::None;

	bool bThirdPartyDenoiserNeedsTextureCreateExtraFlags = FDenoiserManager::Get().NeedTextureCreateExtraFlags();

	if (SpatialDenoiserType == ESpatialDenoiserType::SPATIAL_TEMPORAL_DENOISER_PLUGIN ||
		TemporalDenoiserType == ETemporalDenoiserType::SPATIAL_TEMPORAL_DENOISER_PLUGIN ||
		bThirdPartyDenoiserNeedsTextureCreateExtraFlags)
	{
		TextureCreateFlags |= TexCreate_Shared | TexCreate_RenderTargetable;
	}

	return TextureCreateFlags;
}

static float GetHighFrequencyCutoffDeltaE()
{
	return FMath::Max(1.0f, CVarPathTracingTemporalDenoiserDeltaEHighFrequency.GetValueOnRenderThread());
}

static bool ShouldRemoveSelfSubpixelOffset(int MipLevel)
{
	bool bRemoveSelfSubpixelOffset = CVarPathTracingTemporalDenoiserMotionOperation.GetValueOnRenderThread() == 1 
		&& MipLevel <= CVarPathTracingTemporalDenoiserSubPixelOffsetStartMip.GetValueOnRenderThread()
		&& CVarPathTracingTemporalDenoiserEnableSubPixelOffset.GetValueOnAnyThread() != 0;

	return bRemoveSelfSubpixelOffset;
}

static float GetErrorDistanceBasedOnDeltaE(float DeltaE)
{
	const float NormalizeDeltaEAndEuclideanDistanceToSameMean = 0.0141f;
	float ErrorDistanceFromDeltaE = log2f(2 + DeltaE * NormalizeDeltaEAndEuclideanDistanceToSameMean);
	return ErrorDistanceFromDeltaE;
}

static void GetBlendingFactor(float& Kappa, float& Eta, float& Alpha)
{
	Kappa = CVarPathTracingTemporalDenoiserKappa.GetValueOnRenderThread();
	Eta = CVarPathTracingTemporalDenoiserEta.GetValueOnRenderThread();
	Alpha = CVarPathTracingTemporalDenoiserAlpha.GetValueOnRenderThread();

	if (Eta == -1)
	{
		// Eta is set to 1 JND (just noticeable difference).
		Eta = GetErrorDistanceBasedOnDeltaE(1.0f);
	}

	if (Kappa == -1)
	{
		// Use DeltaE to determine Kappa in our modified weight formula such that
		// the history weight falls off to zero when we reaches DeltaE.

		float DeltaE =CVarPathTracingTemporalDenoiserDeltaE.GetValueOnRenderThread();
		DeltaE = FMath::Max(DeltaE, 1.0f + 1e-6f);
		float FallOffToZeroErrorDistance = GetErrorDistanceBasedOnDeltaE(DeltaE);
		Kappa = 1 / (FallOffToZeroErrorDistance - Eta);
	}
}

static bool ShouldCompilePathTracingDenoiserShadersForProject(EShaderPlatform ShaderPlatform)
{
	static const auto CVarLocalVariablePathTracing = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PathTracing"));
	const int PathTracing = CVarLocalVariablePathTracing ? CVarLocalVariablePathTracing->GetValueOnAnyThread() : 0;

	return ShouldCompileRayTracingShadersForProject(ShaderPlatform) &&
		FDataDrivenShaderPlatformInfo::GetSupportsPathTracing(ShaderPlatform) &&
		PathTracing != 0;
}

static bool ShouldGenerateVarianceMap()
{
	int32 PathTracingTemporalDenoiserSource = CVarPathTracingTemporalDenoiserSource.GetValueOnRenderThread();
	return PathTracingTemporalDenoiserSource < 0 || PathTracingTemporalDenoiserSource > 3;
}

static bool ShouldVisualizePathTracingVelocityState(const FPathTracingSpatialTemporalDenoisingContext& DenoisingContext)
{
	bool bVisualizeMotionVector = CVarPathTracingTemporalDenoiserVisualizeMotionVector.GetValueOnRenderThread() != 0;

	return bVisualizeMotionVector && DenoisingContext.MotionVector;
}

static bool ShouldVisualizeWarping(const FPathTracingSpatialTemporalDenoisingContext& DenoisingContext)
{
	return CVarPathTracingTemporalDenoiserVisWarp.GetValueOnRenderThread() != 0 && 
		DenoisingContext.MotionVector && DenoisingContext.LastDenoisedRadianceTexture;
}

static int32 GetTemporalAccumulationPatchCount()
{
	return FMath::Clamp(CVarPathTracingTemporalDenoiserPatchCount.GetValueOnRenderThread(), 1, 16);
}

static bool ShouldUseTotalVariation(uint32 MipLevel)
{
	return CVarPathTracingTemporalDenoiserTotalVariation.GetValueOnRenderThread() != 0.0f &&
		(MipLevel == 0) &&
		CVarPathTracingTemporalDenoiserEnableSubPixelOffset.GetValueOnAnyThread() != 0.0f;
}

static bool ShouldEnableSubpixelOffset(uint32 MipLevel)
{
	uint32 PixelOffsetStartMip = CVarPathTracingTemporalDenoiserSubPixelOffsetStartMip.GetValueOnRenderThread();
	bool bShouldEnableSubpixelOffset = (MipLevel <= PixelOffsetStartMip) &&
		CVarPathTracingTemporalDenoiserEnableSubPixelOffset.GetValueOnAnyThread() != 0;

	return bShouldEnableSubpixelOffset;
}

static bool IsVarianceTextureRequiredForCurrentDenoiser()
{
	bool NeedVarianceTexture = false;
	using UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser;
	IPathTracingSpatialTemporalDenoiser* ActiveSpatialTemporalDenoiser = nullptr;

	if (GetSpatialDenoiserType() == ESpatialDenoiserType::SPATIAL_TEMPORAL_DENOISER_PLUGIN)
	{
		ActiveSpatialTemporalDenoiser = GetActiveSpatialTemporalDenoiser();
	}
	else
	{
		FString DenoiserName = CVarPathTracingDenoiserName.GetValueOnRenderThread();
		ActiveSpatialTemporalDenoiser = FDenoiserManager::Get().GetSpatialTemporalDenoiser(DenoiserName, /*bMatch*/true);
	}
	
	if (ActiveSpatialTemporalDenoiser)
	{
		NeedVarianceTexture = ActiveSpatialTemporalDenoiser->NeedVarianceTexture();
	}
	
	return NeedVarianceTexture;
}

static bool ShouldPrepassOutputVarianceTexture(const FViewInfo& View)
{
	static const auto CVarOutputPostProcessResources = 
		IConsoleManager::Get().FindTConsoleVariableDataBool(TEXT("r.PathTracing.OutputPostProcessResources"));
	const bool bOutputPostProcessResources = CVarOutputPostProcessResources ?
		CVarOutputPostProcessResources->GetValueOnRenderThread() : false;
	
	// Variance texture will be available if 
	// 1. when we always output so it can be accessed for debug
	// 2. post process requires and we allow output post process resource in path tracing
	// 3. the denoiser requires a variance texture.
	return CVarPathTracingDenoiserPrepassOutputVarianceTexture.GetValueOnRenderThread() != 0 ||
		(bOutputPostProcessResources && IsPathTracingVarianceTextureRequiredInPostProcessMaterial(View)) ||
		IsVarianceTextureRequiredForCurrentDenoiser();
}

static constexpr uint32 kMipDiffDelta = 2;

static constexpr uint32 kNumberOfPixelShifts = 25;
static constexpr uint32 kNumberOfShiftsPerTexture = 4;
static constexpr uint32 kNumberOfPasses = 1;
static constexpr uint32 kNumberOfShiftsPerPass = kNumberOfShiftsPerTexture * kNumberOfPasses;
static constexpr uint32 kNumberOfTexturesPerPass = (kNumberOfPixelShifts + kNumberOfShiftsPerPass - 1) / kNumberOfShiftsPerPass;

static constexpr uint32 kThreadSize = 8;

static_assert(kNumberOfTexturesPerPass <= 7, "The number of buffers for Pixel correspondence estimation can not exceed  seven per pass");

static const TCHAR* DistanceTextureNames[7] =
{
	TEXT("PathTracing.EstimateMotion.Disntace.0"),
	TEXT("PathTracing.EstimateMotion.Disntace.1"),
	TEXT("PathTracing.EstimateMotion.Disntace.2"),
	TEXT("PathTracing.EstimateMotion.Disntace.3"),
	TEXT("PathTracing.EstimateMotion.Disntace.4"),
	TEXT("PathTracing.EstimateMotion.Disntace.5"),
	TEXT("PathTracing.EstimateMotion.Disntace.6"),
};

BEGIN_SHADER_PARAMETER_STRUCT(FDenoisingCommonParameters, )
	// Constant variable in each pass
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, TargetViewport)
	SHADER_PARAMETER_SAMPLER(SamplerState, SharedTextureSampler)
	SHADER_PARAMETER(uint32, PatchCount)
	SHADER_PARAMETER(uint32, NumOfMips)

	// Dynamic variable in each subpass.
	SHADER_PARAMETER(uint32, PatchId)
	SHADER_PARAMETER(uint32, MipLevel)
END_SHADER_PARAMETER_STRUCT();


class FTemporalReprojectionAlignCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTemporalReprojectionAlignCS);
	SHADER_USE_PARAMETER_STRUCT(FTemporalReprojectionAlignCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDenoisingCommonParameters, DenoisingCommonParameters)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, PixelOffsetTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SourceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, TargetTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<float4>, RWDistanceTextures, [kNumberOfTexturesPerPass])
	END_SHADER_PARAMETER_STRUCT()

	enum class EDistanceMetrics : uint32 {
		METRICS_LUMINANCE,
		METRICS_EUCLIDEAN,
		METRICS_PERCEPTION,
		MAX
	};

	class FDistanceMetrics : SHADER_PERMUTATION_ENUM_CLASS("DISTANCE_METRICS", EDistanceMetrics);
	using FPermutationDomain = TShaderPermutationDomain<FDistanceMetrics>;

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TEMPORAL_REPROJECTION_ALIGN"), 1);
		OutEnvironment.SetDefine(TEXT("K_NUM_OF_TEXTURES_PER_PASS"), kNumberOfTexturesPerPass);
	}

	static EDistanceMetrics GetDistanceMetrics()
	{
		uint32 DistanceMetrics = CVarPathTracingTemporalDenoiserDistanceMetrics.GetValueOnRenderThread();
		DistanceMetrics = FMath::Clamp(DistanceMetrics, 
			static_cast<uint32>(EDistanceMetrics::METRICS_LUMINANCE),
			static_cast<uint32>(EDistanceMetrics::MAX) - 1);

		return static_cast<EDistanceMetrics>(DistanceMetrics);
	}

	static const TCHAR* GetEventName(EDistanceMetrics DistanceMetrics)
	{
		static const TCHAR* const kEventNames[] = {
			TEXT("Luminance"),
			TEXT("Euclidean"),
			TEXT("Perception")
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == int32(EDistanceMetrics::MAX), "Fix me");
		return kEventNames[static_cast<uint32>(DistanceMetrics)];
	}
};

IMPLEMENT_GLOBAL_SHADER(FTemporalReprojectionAlignCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "ReprojectionAlignCS", SF_Compute);

class FTemporalReprojectionBlurCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTemporalReprojectionBlurCS);
	SHADER_USE_PARAMETER_STRUCT(FTemporalReprojectionBlurCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDenoisingCommonParameters, DenoisingCommonParameters)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InputTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()
	
	class FDimensionDirectCopy : SHADER_PERMUTATION_BOOL("DIRECT_COPY");
	using FPermutationDomain = TShaderPermutationDomain<FDimensionDirectCopy>;

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TEMPORAL_REPROJECTION_BLUR"), 1);
		OutEnvironment.SetDefine(TEXT("K_NUM_OF_TEXTURES_PER_PASS"), kNumberOfTexturesPerPass);
	}
};

IMPLEMENT_GLOBAL_SHADER(FTemporalReprojectionBlurCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "ReprojectionBlurCS", SF_Compute);

class FTemporalReprojectionMergeCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTemporalReprojectionMergeCS);
	SHADER_USE_PARAMETER_STRUCT(FTemporalReprojectionMergeCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDenoisingCommonParameters, DenoisingCommonParameters)
		SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D<float4>, DistanceTextures, [kNumberOfTexturesPerPass])
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, PixelOffsetTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWPixelOffsetTexture)
	END_SHADER_PARAMETER_STRUCT()
	
	class FSubpixelOffset : SHADER_PERMUTATION_BOOL("SUBPIXEL_OFFSET");
	class FTotalVariation : SHADER_PERMUTATION_BOOL("TOTAL_VARIATION");
	using FPermutationDomain = TShaderPermutationDomain<FSubpixelOffset,FTotalVariation>;

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TEMPORAL_REPROJECTION_MERGE"), 1);
		OutEnvironment.SetDefine(TEXT("K_NUM_OF_TEXTURES_PER_PASS"), kNumberOfTexturesPerPass);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

};

IMPLEMENT_GLOBAL_SHADER(FTemporalReprojectionMergeCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "ReprojectionMergeCS", SF_Compute);

class FMotionVectorSubtractCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMotionVectorSubtractCS);
	SHADER_USE_PARAMETER_STRUCT(FMotionVectorSubtractCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDenoisingCommonParameters, DenoisingCommonParameters)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, Minuend)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Subtrahend)
	END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("MOTION_VECTOR_SUBTRACT"), 1);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMotionVectorSubtractCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "MotionVectorSubtractCS", SF_Compute);

// Warp the history texture, and use local high frequency to adjust the final blending factor. 
class FTemporalHighFrequencyRejectMapCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTemporalHighFrequencyRejectMapCS);
	SHADER_USE_PARAMETER_STRUCT(FTemporalHighFrequencyRejectMapCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, HighFrequencyCutoffDeltaE)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDenoisingCommonParameters, DenoisingCommonParameters)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SourceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, TargetTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, PixelOffsetTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TEMPORAL_HIGHFREQ_REJECT"), 1);
		OutEnvironment.SetDefine(TEXT("K_NUM_OF_TEXTURES_PER_PASS"), kNumberOfTexturesPerPass);
	}
};

IMPLEMENT_GLOBAL_SHADER(FTemporalHighFrequencyRejectMapCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "TemporalHighFrequencyRejectCS", SF_Compute);

enum class EFeatureFusionCategory : uint32
{
	SOURCE,
	TARGET,
	MAX
};

static constexpr uint32 kNumOfFeatureFusionCategory = static_cast<uint32>(EFeatureFusionCategory::MAX);

class FTemporalFeatureFusionCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTemporalFeatureFusionCS)
	SHADER_USE_PARAMETER_STRUCT(FTemporalFeatureFusionCS, FGlobalShader)

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D, AlbedoTexture,	[kNumOfFeatureFusionCategory])
		SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D, NormalTexture,	[kNumOfFeatureFusionCategory])
		SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D, RadianceTexture,	[kNumOfFeatureFusionCategory])
		SHADER_PARAMETER_RDG_BUFFER_SRV_ARRAY(StructuredBuffer<FPixelMaterialLightingFingerprint>, VarianceMap,[kNumOfFeatureFusionCategory])
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, LastDenoisedRadiance)

		SHADER_PARAMETER_SAMPLER(SamplerState, SharedTextureSampler)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, TargetViewport)

		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<float4>, OutputTexture,	[kNumOfFeatureFusionCategory])
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TEMPORAL_FEATURE_FUSION"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FTemporalFeatureFusionCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "TemporalFeatureFusionCS", SF_Compute);


class FTemporalResolveCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTemporalResolveCS);
	SHADER_USE_PARAMETER_STRUCT(FTemporalResolveCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, Kappa)
		SHADER_PARAMETER(float, Eta)
		SHADER_PARAMETER(float, Alpha)
		SHADER_PARAMETER(float, HighFrequencyCutoffDeltaE)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDenoisingCommonParameters, DenoisingCommonParameters)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SourceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, TargetTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, PixelOffsetTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, HighFrequencyRejectMap)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
		END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TEMPORAL_REPROJECTION_RESOLVE"), 1);
		OutEnvironment.SetDefine(TEXT("K_NUM_OF_TEXTURES_PER_PASS"), kNumberOfTexturesPerPass);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

IMPLEMENT_GLOBAL_SHADER(FTemporalResolveCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "TemporalResolveCS", SF_Compute);

// Add Spatial denoiser
class FSpatialDenoiserCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSpatialDenoiserCS);
	SHADER_USE_PARAMETER_STRUCT(FSpatialDenoiserCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputNormal)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputAlbedo)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, TargetViewport)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SPATIAL_DENOISING"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSpatialDenoiserCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "SpatialDenoiserCS", SF_Compute);

class FConvertWorldSpaceNormalToCameraSpaceCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FConvertWorldSpaceNormalToCameraSpaceCS);
	SHADER_USE_PARAMETER_STRUCT(FConvertWorldSpaceNormalToCameraSpaceCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, NormalTexture)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(float, Width)
		SHADER_PARAMETER(float, Height)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("PREPROCESS_BUFFER"), 1);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

IMPLEMENT_GLOBAL_SHADER(FConvertWorldSpaceNormalToCameraSpaceCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "ConvertWorldSpaceNormalToCameraSpaceCS", SF_Compute);

static void ConvertNormalSpace(FRDGBuilder& GraphBuilder,
	const FViewInfo& View, 
	FRDGTextureRef NormalTexture)
{
	typedef FConvertWorldSpaceNormalToCameraSpaceCS SHADER;
	SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
	PassParameters->NormalTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(NormalTexture));
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->Width = NormalTexture->Desc.GetSize().X;
	PassParameters->Height = NormalTexture->Desc.GetSize().Y;


	TShaderMapRef<SHADER> ComputeShader(View.ShaderMap);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("ConvertWorldSpaceNormalToCameraSpace %dx%d",
			View.ViewRect.Width(),
			View.ViewRect.Height()),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), 8));
}

static void PathTracingDenoiserPlugin(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	int DenoiserMode,
	FRDGTextureRef InputTexture, 
	FRDGTextureRef AlbedoTexture,
	FRDGTextureRef NormalTexture,
	FRDGTextureRef DepthTexture,
	FRDGTextureRef VarianceTexture,
	FRDGTextureRef OutputTexture)
{
	FRDGTextureRef ProcessedNormalTexture = NormalTexture;

	if (ShouldDenoiseWithNormalInCameraSpace())
	{
		const FRDGTextureDesc& Desc = NormalTexture->Desc;
		ProcessedNormalTexture = GraphBuilder.CreateTexture(Desc, TEXT("PathTracing.CameraSpaceNormal"));
		{
			FRHICopyTextureInfo CopyInfo;
			CopyInfo.Size.X = Desc.Extent.X;
			CopyInfo.Size.Y = Desc.Extent.Y;
			CopyInfo.Size.Z = 1;
			CopyInfo.NumMips = Desc.NumMips;

			AddCopyTexturePass(GraphBuilder, NormalTexture, ProcessedNormalTexture, CopyInfo);
		}
		ConvertNormalSpace(GraphBuilder, View, ProcessedNormalTexture);
	}

	/**
	* First try to use the matched denoiser to denoise.
	* If the requested denoiser is not available, fallback to the spatial denoiser.
	*/
	bool bMatch = true;
	if (UE::Renderer::Private::IPathTracingDenoiser* ActiveSpatialDenoiser = GetActiveSpatialDenoiser(bMatch); ActiveSpatialDenoiser)
	{
		ActiveSpatialDenoiser->AddPasses(GraphBuilder, View, { InputTexture, AlbedoTexture, NormalTexture, OutputTexture });
	}
	else
	{
		using UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser;

		FString DenoiserName = CVarPathTracingDenoiserName.GetValueOnRenderThread();
		IPathTracingSpatialTemporalDenoiser* ActiveSpatialTemporalDenoiser = FDenoiserManager::Get().GetSpatialTemporalDenoiser(DenoiserName,bMatch);
		
		if (ActiveSpatialTemporalDenoiser)
		{
			/**
			* Force the temporal denoiser to denoise a single frame.
			*/
			IPathTracingSpatialTemporalDenoiser::FInputs Inputs;
			Inputs.ColorTex = InputTexture;
			Inputs.AlbedoTex = AlbedoTexture;
			Inputs.NormalTex = NormalTexture;
			Inputs.DepthTex = DepthTexture;
			Inputs.VarianceTex = VarianceTexture;
			
			// Set flow texture to black
			Inputs.FlowTex = GraphBuilder.CreateTexture(InputTexture->Desc, TEXT("PathTracing.OpticalFlow"));
			AddClearRenderTargetPass(GraphBuilder, Inputs.FlowTex, FLinearColor::Black);
			
			Inputs.PreviousOutputTex = InputTexture;
			Inputs.OutputTex = OutputTexture;
			Inputs.DenoisingFrameId = 0;
			Inputs.bForceSpatialDenoiserOnly = true; // Force to use spatial denoiser.
			
			ActiveSpatialTemporalDenoiser->AddPasses(GraphBuilder, View, Inputs);
		}
		else
		{
			//fallback to the default spatial denoiser
			bMatch = false;
			GetActiveSpatialDenoiser(bMatch)->AddPasses(GraphBuilder, View, { InputTexture, AlbedoTexture, NormalTexture, OutputTexture });
		}
	}
}

static void PathTracingSpatialTemporalDenoiserPlugin(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	int DenoiserMode,
	FRDGTextureRef InputTexture,
	FRDGTextureRef AlbedoTexture,
	FRDGTextureRef NormalTexture,
	FRDGTextureRef DepthTexture,
	FRDGTextureRef FlowTexture,
	FRDGTextureRef PreviousOutputFrameTexture,
	FRDGTextureRef OutputTexture,
	int DenoisingFrameId,
	bool bForceSpatialDenoiserOnly,
	FPathTracingSpatialTemporalDenoisingContext& Context)
{
	check(GetActiveSpatialTemporalDenoiser());

	FRDGTextureRef ProcessedNormalTexture = NormalTexture;

	if (ShouldDenoiseWithNormalInCameraSpace())
	{
		const FRDGTextureDesc& Desc = NormalTexture->Desc;
		ProcessedNormalTexture = GraphBuilder.CreateTexture(Desc, TEXT("PathTracing.CameraSpaceNormal"));
		{
			FRHICopyTextureInfo CopyInfo;
			CopyInfo.Size.X = Desc.Extent.X;
			CopyInfo.Size.Y = Desc.Extent.Y;
			CopyInfo.Size.Z = 1;
			CopyInfo.NumMips = Desc.NumMips;

			AddCopyTexturePass(GraphBuilder, NormalTexture, ProcessedNormalTexture, CopyInfo);
		}
		ConvertNormalSpace(GraphBuilder, View, ProcessedNormalTexture);
	}

	using UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser;

	IPathTracingSpatialTemporalDenoiser::FInputs Inputs;
	Inputs.ColorTex = InputTexture;
	Inputs.AlbedoTex = AlbedoTexture;
	Inputs.NormalTex = NormalTexture;
	Inputs.DepthTex = DepthTexture;
	Inputs.VarianceTex = Context.VarianceTexture;
	Inputs.OutputTex = OutputTexture;
	Inputs.FlowTex = FlowTexture;
	Inputs.PreviousOutputTex = PreviousOutputFrameTexture;
	Inputs.DenoisingFrameId = DenoisingFrameId;
	Inputs.bForceSpatialDenoiserOnly = bForceSpatialDenoiserOnly;

	if (Context.SpatialTemporalDenoiserHistory &&
		Context.SpatialTemporalDenoiserHistory->GetDebugName() == GetActiveSpatialTemporalDenoiser()->GetDebugName())
	{
		Inputs.PrevHistory = Context.SpatialTemporalDenoiserHistory;
	}

	IPathTracingSpatialTemporalDenoiser::FOutputs Outputs = GetActiveSpatialTemporalDenoiser()->AddPasses(GraphBuilder, View, Inputs);
	if (Outputs.NewHistory)
	{
		Context.SpatialTemporalDenoiserHistory = Outputs.NewHistory;
	}
}

static bool ShouldApplyPreExposureToMotionVectorEstimation()
{
	int32 DenoiserSource = CVarPathTracingTemporalDenoiserSource.GetValueOnRenderThread();
	if (DenoiserSource == 0 || DenoiserSource == 3)
	{
		return true;
	}

	return false;
}

static void PathTracingMotionVectorPlugin(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef InputFrameTexture,
	FRDGTextureRef ReferenceFrameTexture,
	FRDGTextureRef OutputTexture)
{
	check(GetActiveSpatialTemporalDenoiser());

	bool bShouldApplyPreExposure = ShouldApplyPreExposureToMotionVectorEstimation();
	float PreExposure = bShouldApplyPreExposure ? View.PreExposure : 1.0f;

	GetActiveSpatialTemporalDenoiser()->AddMotionVectorPass(GraphBuilder, View, {InputFrameTexture, ReferenceFrameTexture, OutputTexture, PreExposure});
}

class FMotionVectorEstimationContext
{
public:

	const TCHAR* PixelOffsetTextureNames[2] = {
		TEXT("PathTracing.EstimateMotion.PixelOffset.Ping"),
		TEXT("PathTracing.EstimateMotion.PixelOffset.Pong"),
	};

	FRDGTextureRef PixelOffsetTextures[2];
	FScreenPassTextureViewport TargetViewport;

	FRDGTextureDesc TextureWithNMipsDescriptor;
	FRDGTextureDesc TextureDescriptor;
	FRDGTextureDesc TextureDescriptorFinalOutput;

	FRDGTextureRef SourceTexture;
	FRDGTextureRef TargetTexture;

	FRDGTextureRef SourceMipTexture;
	FRDGTextureRef TargetMipTexture;
	FRHISamplerState* BilinearClampSampler;

	FRDGTextureRef DistanceTextures[kNumberOfTexturesPerPass];
	FDenoisingCommonParameters DenoisingCommonParameters;

public:

	FMotionVectorEstimationContext(FRDGTextureRef InSourceTexture, 
		FRDGTextureRef InTargetTexture):
	SourceTexture(InSourceTexture),
	TargetTexture(InTargetTexture){}

	bool InitContext(FRDGBuilder& GraphBuilder,
		const FViewInfo& View)
	{
		TargetViewport = FScreenPassTextureViewport(View.ViewRect);
		uint32 NumOfMips = FMath::Min(7u, 1 + FMath::FloorLog2((uint32)TargetViewport.Extent.GetMin()));

		if (!ensureMsgf(NumOfMips == 7u, TEXT("The image is too small to estimate temporal reprojection NumOfMips(%d)< 7"), NumOfMips))
		{
			return false;
		}

		// Set up common denoising parameters for shader
		{
			DenoisingCommonParameters.ViewUniformBuffer = View.ViewUniformBuffer;
			DenoisingCommonParameters.TargetViewport = GetScreenPassTextureViewportParameters(TargetViewport);
			DenoisingCommonParameters.SharedTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			DenoisingCommonParameters.PatchCount = GetTemporalAccumulationPatchCount();
			DenoisingCommonParameters.NumOfMips = NumOfMips;
		}

		TextureWithNMipsDescriptor = FRDGTextureDesc::Create2D(
			TargetViewport.Extent,
			PF_A32B32G32R32F,
			FClearValueBinding(),
			TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV,
			NumOfMips);

		TextureDescriptor = FRDGTextureDesc::Create2D(
			TargetViewport.Extent,
			PF_A32B32G32R32F,
			FClearValueBinding(),
			TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV);

		TextureDescriptorFinalOutput = FRDGTextureDesc::Create2D(
			TargetViewport.Extent,
			PF_A32B32G32R32F,
			FClearValueBinding(),
			TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV | GetExtraTextureCreateFlagsForDenoiser());
		
		PixelOffsetTextures[0] = GraphBuilder.CreateTexture(TextureWithNMipsDescriptor, PixelOffsetTextureNames[0]);
		PixelOffsetTextures[1] = GraphBuilder.CreateTexture(TextureWithNMipsDescriptor, PixelOffsetTextureNames[1]);

		FRDGTextureDesc SourceDesc = SourceTexture->Desc;
		FRDGTextureDesc TargetDesc = TargetTexture->Desc;

		// Copy and create the mipmap for both source and target texture
		SourceMipTexture = GraphBuilder.CreateTexture(TextureWithNMipsDescriptor, TEXT("PathTracing.EstimateMotion.Source"));
		TargetMipTexture = GraphBuilder.CreateTexture(TextureWithNMipsDescriptor, TEXT("PathTracing.EstimateMotion.Target"));
		BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		{
			FRHICopyTextureInfo CopyInfo;
			CopyInfo.Size.X = FMath::Min(SourceDesc.Extent.X, TargetDesc.Extent.X);
			CopyInfo.Size.Y = FMath::Min(SourceDesc.Extent.Y, TargetDesc.Extent.Y);
			CopyInfo.Size.Z = 1;
			CopyInfo.NumMips = FMath::Min(SourceDesc.NumMips, TargetDesc.NumMips);

			AddCopyTexturePass(GraphBuilder, SourceTexture, SourceMipTexture, CopyInfo);
			FGenerateMips::Execute(GraphBuilder, View.FeatureLevel, SourceMipTexture, BilinearClampSampler);

			AddCopyTexturePass(GraphBuilder, TargetTexture, TargetMipTexture, CopyInfo);
			FGenerateMips::Execute(GraphBuilder, View.FeatureLevel, TargetMipTexture, BilinearClampSampler);
		}

		for (int TextureId = 0; TextureId < kNumberOfTexturesPerPass; ++TextureId)
		{
			DistanceTextures[TextureId] = GraphBuilder.CreateTexture(TextureWithNMipsDescriptor, DistanceTextureNames[TextureId]);
		}

		return true;
	}

	uint32 GetPatchCount() const
	{
		return DenoisingCommonParameters.PatchCount;
	}

	uint32 GetNumOfMips() const
	{
		return DenoisingCommonParameters.NumOfMips;
	}

	void UpdatePatchId(uint32 PatchId)
	{
		DenoisingCommonParameters.PatchId = PatchId;
	}

	void UpdateMipLevel(uint32 MipLevel)
	{
		DenoisingCommonParameters.MipLevel = MipLevel;
	}

	uint32 GetMipLevel() const
	{
		return DenoisingCommonParameters.MipLevel;
	}

	FIntVector GetAlignGroupCount() const
	{
		return FComputeShaderUtils::GetGroupCount(
			FIntVector(TargetViewport.Extent.X, TargetViewport.Extent.Y, kNumberOfTexturesPerPass),
			FIntVector(kThreadSize, kThreadSize, 1));
	}
	
};

static void AlignTexture(FRDGBuilder& GraphBuilder, 
	const FViewInfo& View,
	const FMotionVectorEstimationContext& Context,
	FRDGTextureSRVDesc PixelOffsetSRVDesc,
	FRDGTextureRef SourceMipTexture, 
	FRDGTextureRef TargetMipTexture)
{

	uint32 MipLevel = Context.GetMipLevel();

	typedef FTemporalReprojectionAlignCS SHADER;

	SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
	PassParameters->PixelOffsetTexture = GraphBuilder.CreateSRV(PixelOffsetSRVDesc);
	PassParameters->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceMipTexture, MipLevel));
	PassParameters->TargetTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(TargetMipTexture, MipLevel));
	PassParameters->DenoisingCommonParameters = Context.DenoisingCommonParameters;

	for (int TextureId = 0; TextureId < kNumberOfTexturesPerPass; ++TextureId)
	{
		PassParameters->RWDistanceTextures[TextureId] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Context.DistanceTextures[TextureId], MipLevel));
	}

	SHADER::EDistanceMetrics DistanceMetrics = SHADER::GetDistanceMetrics();
	SHADER::FPermutationDomain ComputeShaderPermutationVector;
	ComputeShaderPermutationVector.Set<SHADER::FDistanceMetrics>(DistanceMetrics);

	TShaderMapRef<SHADER> ComputeShader(View.ShaderMap,ComputeShaderPermutationVector);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("PathTracing::Denoising::Align %dx%d (%s)",
			Context.TargetViewport.Extent.X,
			Context.TargetViewport.Extent.Y,
			SHADER::GetEventName(DistanceMetrics)),
		ComputeShader,
		PassParameters,
		Context.GetAlignGroupCount());
}

// Determine the image blur size based on the current mip level and image size.
// TODO: scale blur size based on viewport size
static int32 GetBlurSize(int32 MipLevel, FIntPoint ViewportExtent)
{
	int BlurSize = 1;	

	switch (MipLevel)
	{
	case 0:
		BlurSize = 3; break;
	case 2:
		BlurSize = 2; break;
	default:
		BlurSize = 1; break;
	};

	return BlurSize;
}

static void BlurDistanceMetrics(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FMotionVectorEstimationContext& Context)
{	
	struct FReprojectionBulrPassInfo
	{
		FReprojectionBulrPassInfo(const TCHAR* InName, FRDGTextureRef  InInput, FRDGTextureRef InOutput)
			:Name(InName), Input(InInput), Output(InOutput)
		{}

		const TCHAR* Name;
		FRDGTextureRef Input;
		FRDGTextureRef Output;
	};

	FRDGTextureRef TempBuffer = GraphBuilder.CreateTexture(Context.TextureWithNMipsDescriptor, TEXT("PathTracing.EstimateMotion.TempBuffer"));
	uint32 MipLevel = Context.GetMipLevel();
	int BlurSize = GetBlurSize(MipLevel, Context.TargetViewport.Extent);

	for (int TextureId = 0; TextureId < kNumberOfTexturesPerPass; ++TextureId)
	{
		const int NumOfBlurPass = 2;
		const FReprojectionBulrPassInfo BlurPassInfos[NumOfBlurPass] =
		{
			{TEXT("Blur0"),	Context.DistanceTextures[TextureId],						  TempBuffer},
			{TEXT("Blur1"),							 TempBuffer, Context.DistanceTextures[TextureId]}
		};

		for (int i = 0; i < BlurSize; ++i) 
		{
			for (int PassIndex = 0; PassIndex < NumOfBlurPass; ++PassIndex)
			{

				FReprojectionBulrPassInfo PassInfo = BlurPassInfos[PassIndex];
				FRDGTextureSRVDesc InputSRVDesc = FRDGTextureSRVDesc::CreateForMipLevel(PassInfo.Input, MipLevel);
				FRDGTextureUAVDesc OutputUAVDesc(PassInfo.Output, MipLevel);

				typedef FTemporalReprojectionBlurCS SHADER;
				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				PassParameters->InputTexture = GraphBuilder.CreateSRV(InputSRVDesc);
				PassParameters->OutputTexture = GraphBuilder.CreateUAV(OutputUAVDesc);
				PassParameters->DenoisingCommonParameters = Context.DenoisingCommonParameters;

				SHADER::FPermutationDomain ComputeShaderPermutationVector;
				ComputeShaderPermutationVector.Set<SHADER::FDimensionDirectCopy>(false);

				TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, ComputeShaderPermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("PathTracing::Denoising::%s %dx%d Mip=%d (DistPatch %d/%d)",
						BlurPassInfos[PassIndex].Name,
						Context.TargetViewport.Extent.X,
						Context.TargetViewport.Extent.Y,
						MipLevel,
						TextureId + 1,
						kNumberOfTexturesPerPass),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(Context.TargetViewport.Extent, 8));
			}
		}
	}
}

static void MergeDistanceMetrics(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FMotionVectorEstimationContext& Context,
	const FRDGTextureSRVDesc& LastPixelOffsetSRVDesc,
	FRDGTextureRef TargetPixelOffsetTexture)
{
	uint32 MipLevel = Context.GetMipLevel();

	typedef FTemporalReprojectionMergeCS SHADER;
	SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
	for (int TextureId = 0; TextureId < kNumberOfTexturesPerPass; ++TextureId)
	{
		PassParameters->DistanceTextures[TextureId] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Context.DistanceTextures[TextureId], MipLevel));
	}
	PassParameters->PixelOffsetTexture = GraphBuilder.CreateSRV(LastPixelOffsetSRVDesc);
	PassParameters->RWPixelOffsetTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(TargetPixelOffsetTexture, MipLevel));
	PassParameters->DenoisingCommonParameters = Context.DenoisingCommonParameters;

	bool bShouldUseTotalVariation = ShouldUseTotalVariation(MipLevel);
	bool bShouldEnableSubpixelOffset = ShouldEnableSubpixelOffset(MipLevel);

	SHADER::FPermutationDomain ComputeShaderPermutationVector;
	ComputeShaderPermutationVector.Set<SHADER::FSubpixelOffset>(bShouldEnableSubpixelOffset);
	ComputeShaderPermutationVector.Set<SHADER::FTotalVariation>(bShouldUseTotalVariation);

	TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, ComputeShaderPermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("PathTracing::Denoising::Merge %dx%d",
			Context.TargetViewport.Extent.X,
			Context.TargetViewport.Extent.Y),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(Context.TargetViewport.Extent, 8));
}

// Estimate the motion vector (pixel correspondence offset)
static FRDGTextureRef EstimateMotionVector(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FMotionVectorEstimationContext& Context)
{
	int PixelOffsetIndex = 0;
	int NumOfMips = Context.GetNumOfMips();

	for (int MipLevel = NumOfMips - 1; MipLevel >= 0; MipLevel -= kMipDiffDelta)
	{
		Context.UpdateMipLevel(MipLevel);

		for (int Pass = 0; Pass < kNumberOfPasses; ++Pass)
		{
			const bool UseBlackDummpy = ((MipLevel == (NumOfMips - 1)) && Pass == 0);

			// Write the calibration subpixel offset to the source so that we can remove it.
			FRDGTextureSRVDesc SelfLastPixelOffsetSRVDesk = FRDGTextureSRVDesc::CreateForMipLevel(
				GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy, Context.PixelOffsetTextureNames[1]),
				0);

			FRDGTextureRef SelfTargetPixelOffsetTexture = Context.PixelOffsetTextures[PixelOffsetIndex % 2];

			FRDGTextureSRVDesc PixelOffsetSRVDesc = FRDGTextureSRVDesc::CreateForMipLevel(
				UseBlackDummpy ?
				GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy, Context.PixelOffsetTextureNames[0]) :
				Context.PixelOffsetTextures[PixelOffsetIndex % 2], UseBlackDummpy ? 0 : (MipLevel + kMipDiffDelta));
			
			FRDGTextureRef TargetPixelOffsetTexture = Context.PixelOffsetTextures[(++PixelOffsetIndex) % 2];

			// Calculate the subpixel offset for the source image, and write to the unused texture in the source mipmap
			if(ShouldRemoveSelfSubpixelOffset(MipLevel))
			{
				AlignTexture(GraphBuilder, View, Context,
					SelfLastPixelOffsetSRVDesk,
					Context.SourceMipTexture,
					Context.SourceMipTexture);

				BlurDistanceMetrics(GraphBuilder, View,
					Context);

				// Merge: Find the smallest distance within [-2,2]^2
				MergeDistanceMetrics(GraphBuilder, View,
					Context,
					SelfLastPixelOffsetSRVDesk,
					SelfTargetPixelOffsetTexture);
			}

			// Calculate the subpixel offset between the source and the target
			{
				AlignTexture(GraphBuilder, View, Context,
					PixelOffsetSRVDesc,
					Context.SourceMipTexture,
					Context.TargetMipTexture);

				BlurDistanceMetrics(GraphBuilder, View,
					Context);

				// Merge: Find the smallest distance within [-2,2]^2
				MergeDistanceMetrics(GraphBuilder, View,
					Context,
					PixelOffsetSRVDesc,
					TargetPixelOffsetTexture);
			}
			
			// Subtract the distance from source to target
			if (ShouldRemoveSelfSubpixelOffset(MipLevel))
			{
				typedef FMotionVectorSubtractCS SHADER;
				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				PassParameters->Minuend = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(TargetPixelOffsetTexture, MipLevel));
				PassParameters->Subtrahend = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SelfTargetPixelOffsetTexture, MipLevel));
				PassParameters->DenoisingCommonParameters = Context.DenoisingCommonParameters;

				TShaderMapRef<SHADER> ComputeShader(View.ShaderMap);
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("PathTracing::Denoising::MotionVectorDiff %dx%d (Mip=%d)",
						Context.TargetViewport.Extent.X,
						Context.TargetViewport.Extent.Y,
						MipLevel),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(Context.TargetViewport.Extent, 8));
			}

		}
	}

	return Context.PixelOffsetTextures[PixelOffsetIndex % 2];
}

// combine albedo, normal, and radiance into a single feature texture
static void FuseTemporalFeature(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FPathTracingSpatialTemporalDenoisingContext& DenoisingContext, 
	FRDGTextureRef& OutSourceTexture, 
	FRDGTextureRef& OutTargetTexture)
{
	const FScreenPassTextureViewport TargetViewport(View.ViewRect);
	const FScreenPassTextureViewportParameters TargetViewportParameters = GetScreenPassTextureViewportParameters(TargetViewport);

	const FRDGTextureDesc TextureDescriptor = FRDGTextureDesc::Create2D(
		TargetViewport.Extent,
		PF_A32B32G32R32F,
		FClearValueBinding(),
		TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV | GetExtraTextureCreateFlagsForDenoiser());

	OutSourceTexture = GraphBuilder.CreateTexture(TextureDescriptor, TEXT("PathTracing.Denoising.Feature.Source"));
	OutTargetTexture = GraphBuilder.CreateTexture(TextureDescriptor, TEXT("PathTracing.Denoising.Feature.Target"));

	typedef FTemporalFeatureFusionCS SHADER;
	SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
	{
		PassParameters->AlbedoTexture[0] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DenoisingContext.LastAlbedoTexture));
		PassParameters->AlbedoTexture[1] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DenoisingContext.AlbedoTexture));
		PassParameters->NormalTexture[0] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DenoisingContext.LastNormalTexture));
		PassParameters->NormalTexture[1] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DenoisingContext.NormalTexture));
		PassParameters->RadianceTexture[0] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DenoisingContext.LastRadianceTexture));
		PassParameters->RadianceTexture[1] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DenoisingContext.RadianceTexture));
		PassParameters->VarianceMap[0] = GraphBuilder.CreateSRV(DenoisingContext.LastVarianceBuffer ? 
											DenoisingContext.LastVarianceBuffer : DenoisingContext.VarianceBuffer,EPixelFormat::PF_R32_FLOAT);
		PassParameters->VarianceMap[1] = GraphBuilder.CreateSRV(DenoisingContext.VarianceBuffer, EPixelFormat::PF_R32_FLOAT);
		PassParameters->OutputTexture[0] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutSourceTexture));
		PassParameters->OutputTexture[1] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutTargetTexture));

		PassParameters->LastDenoisedRadiance = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DenoisingContext.LastDenoisedRadianceTexture));

		PassParameters->SharedTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->TargetViewport = TargetViewportParameters;
	}

	TShaderMapRef<SHADER> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("PathTracing::Denoising::FeatureFusion %dx%d",
			TargetViewport.Extent.X,
			TargetViewport.Extent.Y),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(TargetViewport.Extent, 8));

}

static bool SelectMotionSourceAndTargetTextures(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef InSourceTexture,
	FRDGTextureRef InTargetTexture,
	FPathTracingSpatialTemporalDenoisingContext& DenoisingContext,
	FRDGTextureRef& MotionEstimationSourceTexture,
	FRDGTextureRef& MotionEstimationTargetTexture)
{
	int32 MotionSource = CVarPathTracingTemporalDenoiserSource.GetValueOnAnyThread();

	switch (MotionSource)
	{
	case 3:
		MotionEstimationSourceTexture = DenoisingContext.LastRadianceTexture;
		MotionEstimationTargetTexture = DenoisingContext.RadianceTexture;
		break;
	case 2:
		MotionEstimationSourceTexture = DenoisingContext.LastAlbedoTexture;
		MotionEstimationTargetTexture = DenoisingContext.AlbedoTexture;
		break;
	case 1:
		MotionEstimationSourceTexture = DenoisingContext.LastNormalTexture;
		MotionEstimationTargetTexture = DenoisingContext.NormalTexture;
		break;
	case 0:
		MotionEstimationSourceTexture = InSourceTexture;
		MotionEstimationTargetTexture = InTargetTexture;
		break;
	default:
		FuseTemporalFeature(GraphBuilder, View, DenoisingContext, MotionEstimationSourceTexture, MotionEstimationTargetTexture);
		break;
	}

	return  MotionEstimationSourceTexture->Desc.Extent == MotionEstimationTargetTexture->Desc.Extent;
}

// Estimate the motion vector from source to target
static FRDGTextureRef TemporalReprojectionWithoutMotionVector(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef InSourceTexture,
	FRDGTextureRef InTargetTexture,
	FRDGTextureRef MotionEstimationSourceTexture,
	FRDGTextureRef MotionEstimationTargetTexture,
	FPathTracingSpatialTemporalDenoisingContext& DenoisingContext)
{

	FMotionVectorEstimationContext EstimationContext(MotionEstimationSourceTexture, MotionEstimationTargetTexture);

	if (!EstimationContext.InitContext(GraphBuilder, View))
	{
		return nullptr;
	}

	FRDGTextureRef  HighFrequencyRejectMap = GraphBuilder.CreateTexture(EstimationContext.TextureDescriptor, TEXT("PathTracing.EstimateMotion.HighFrequencyRejectMap"));
	FRDGTextureRef  FinalAccumulation = GraphBuilder.CreateTexture(EstimationContext.TextureDescriptorFinalOutput, TEXT("PathTracing.EstimateMotion.FinalAccumulation"));
	int PatchCount = EstimationContext.GetPatchCount();
	float HighFrequencyCutoffDeltaE = GetHighFrequencyCutoffDeltaE();

	FRDGTextureRef  TempAccumulation = nullptr;
	if (PatchCount > 1)
	{
		TempAccumulation = GraphBuilder.CreateTexture(EstimationContext.TextureDescriptor, TEXT("PathTracing.EstimateMotion.TempAccumulation"));
	}

	// Perform accumulation on Miplevel 0
	const uint32 AccumulationMipLevel = 0;
	float Kappa, Eta, Alpha;
	GetBlendingFactor(Kappa, Eta, Alpha);

	for (int PatchId = 0; PatchId < PatchCount; ++PatchId)
	{
		EstimationContext.UpdatePatchId(PatchId);

		FRDGTextureRef  PixelOffsetTexture = EstimateMotionVector(
			GraphBuilder,
			View,
			EstimationContext);

		DenoisingContext.MotionVector = PixelOffsetTexture; // Used for debugging

		// Final resolve, accumulate the temporal information based on the pixel offset texture
		FRDGTextureRef BlendSourceTexture = InSourceTexture;
		FRDGTextureRef BlendTargetTexture = (PatchId == 0) ? InTargetTexture : TempAccumulation;
		FRDGTextureRef BlendFinalTexture = (PatchId + 1 == PatchCount)? FinalAccumulation: TempAccumulation;
		
		EstimationContext.UpdateMipLevel(AccumulationMipLevel);

		// Warp the source image so that we can adjust the blending weight based on high frequency information.
		// E.g., if the source and target is the albedo. We can use visual perception difference of 1dE, 2dE or 10dE
		{
			typedef FTemporalHighFrequencyRejectMapCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			PassParameters->PixelOffsetTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(PixelOffsetTexture, AccumulationMipLevel));
			PassParameters->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(BlendSourceTexture, AccumulationMipLevel));
			PassParameters->TargetTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(InTargetTexture, AccumulationMipLevel));
			PassParameters->OutputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(HighFrequencyRejectMap));	// the difference of the optimal shift.
			PassParameters->DenoisingCommonParameters = EstimationContext.DenoisingCommonParameters;
			PassParameters->HighFrequencyCutoffDeltaE = HighFrequencyCutoffDeltaE;
			TShaderMapRef<SHADER> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("PathTracing::Denoising::RejectionMap %dx%d",
					EstimationContext.TargetViewport.Extent.X,
					EstimationContext.TargetViewport.Extent.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(EstimationContext.TargetViewport.Extent, 8));
		}

		{
			typedef FTemporalResolveCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			PassParameters->PixelOffsetTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(PixelOffsetTexture, AccumulationMipLevel));
			PassParameters->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(BlendSourceTexture, AccumulationMipLevel));
			PassParameters->TargetTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(BlendTargetTexture, AccumulationMipLevel));
			PassParameters->HighFrequencyRejectMap = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(HighFrequencyRejectMap, AccumulationMipLevel));
			PassParameters->OutputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(BlendFinalTexture));	// the difference of the optimal shift.
			PassParameters->DenoisingCommonParameters = EstimationContext.DenoisingCommonParameters;

			PassParameters->HighFrequencyCutoffDeltaE = HighFrequencyCutoffDeltaE;
			PassParameters->Alpha = Alpha;
			PassParameters->Kappa = Kappa;
			PassParameters->Eta = Eta;

			TShaderMapRef<SHADER> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("PathTracing::Denoising::TemporalResolve %dx%d",
					EstimationContext.TargetViewport.Extent.X,
					EstimationContext.TargetViewport.Extent.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(EstimationContext.TargetViewport.Extent, 8));
		}
	}

	return FinalAccumulation;
}

struct FPixelMaterialLightingFingerprint
{
	FVector4 Mean;
	FVector4 Var;
};

class FTemporalPrepassCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTemporalPrepassCS);
	SHADER_USE_PARAMETER_STRUCT(FTemporalPrepassCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, AlbedoTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, NormalTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FPixelMaterialLightingFingerprint>, RWVarianceMap)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, TargetViewport)
		SHADER_PARAMETER(int, Iteration)
	END_SHADER_PARAMETER_STRUCT()
	
	enum class EVarianceType : uint32
	{
		RadianceMultiChannel = 0,
		RadianceAlbedoNormalSingleChannel,
		MAX
	};

	class FPrepassPhase : SHADER_PERMUTATION_BOOL("PREPASS_PHASE");	// 0: initialize, 1: update
	class FVarianceType : SHADER_PERMUTATION_ENUM_CLASS("VARIANCE_TYPE", EVarianceType);
	class FRankedLuminanceVariance : SHADER_PERMUTATION_BOOL("RANKED_LUMINANCE_VARIANCE");
	using FPermutationDomain = TShaderPermutationDomain<FPrepassPhase,FVarianceType, FRankedLuminanceVariance>;

	static bool UseRankedLuminanceVariance()
	{
		return CVarPathTracingDenoiserPrepassRankedLuminanceVariance.GetValueOnRenderThread() != 0;
	}

	static EVarianceType GetVarianceType()
	{
		return static_cast<FTemporalPrepassCS::EVarianceType>(
			FMath::Clamp(CVarPathTracingDenoiserPrepassVarianceType.GetValueOnRenderThread(),
				0,
				static_cast<int32>(EVarianceType::MAX) - 1));
	}

	static const TCHAR* GetEventName(EVarianceType VarianceType)
	{
		static const TCHAR* const kEventName[] = {
			TEXT("Radiance"),
			TEXT("Rad,Albedo,Norm")
		};
		static_assert(UE_ARRAY_COUNT(kEventName) == int32(EVarianceType::MAX), "Fix me");
		return kEventName[int32(VarianceType)];
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("TEMPORAL_PREPASS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FTemporalPrepassCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "TemporalPrepassCS", SF_Compute);

class FPrepassGenerateTextureCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FPrepassGenerateTextureCS);
	SHADER_USE_PARAMETER_STRUCT(FPrepassGenerateTextureCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FPixelMaterialLightingFingerprint>, VarianceMap)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, TargetViewport)
		SHADER_PARAMETER(int32, Iteration)
	END_SHADER_PARAMETER_STRUCT()

	class FVarianceType : SHADER_PERMUTATION_ENUM_CLASS("VARIANCE_TYPE", FTemporalPrepassCS::EVarianceType);
	using FPermutationDomain = TShaderPermutationDomain<FVarianceType>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("PREPASS_GENERATE_TEXTURE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FPrepassGenerateTextureCS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "PrepassGenerateTextureCS", SF_Compute);

void PathTracingSpatialTemporalDenoisingPrePass(FRDGBuilder& GraphBuilder, const FViewInfo& View,
	int IterationNumber,
	int MaxSPP,
	FPathTracingSpatialTemporalDenoisingContext& SpatialTemporalDenoisingContext)
{
	bool bShouldPrepassOutputVarianceTexture = ShouldPrepassOutputVarianceTexture(View);
	bool bShouldGenerateVarianceMap = ShouldGenerateVarianceMap() || bShouldPrepassOutputVarianceTexture;
	if (bShouldGenerateVarianceMap)
	{
		bool bNeedToUpdateVariance = (IterationNumber < MaxSPP);
		const FScreenPassTextureViewport TargetViewport(View.ViewRect);
		const FScreenPassTextureViewportParameters TargetViewportParameters = GetScreenPassTextureViewportParameters(TargetViewport);


		if (bNeedToUpdateVariance)
		{
			if (!SpatialTemporalDenoisingContext.VarianceBuffer)
			{
				SpatialTemporalDenoisingContext.VarianceBuffer = GraphBuilder.CreateBuffer(
					FRDGBufferDesc::CreateStructuredDesc(sizeof(float) * 8, View.ViewRect.Area()), TEXT("PathTracing.VarianceBuffer"));
			}

			typedef FTemporalPrepassCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->InputTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SpatialTemporalDenoisingContext.RadianceTexture));
				PassParameters->AlbedoTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SpatialTemporalDenoisingContext.AlbedoTexture));
				PassParameters->NormalTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SpatialTemporalDenoisingContext.NormalTexture));
				PassParameters->RWVarianceMap = GraphBuilder.CreateUAV(SpatialTemporalDenoisingContext.VarianceBuffer, EPixelFormat::PF_R32_FLOAT);
				PassParameters->TargetViewport = TargetViewportParameters;
				PassParameters->Iteration = IterationNumber;
			}

			bool bUpdateVarianceMapPhase = (IterationNumber > 0);

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			ComputeShaderPermutationVector.Set<SHADER::FPrepassPhase>(bUpdateVarianceMapPhase);
			ComputeShaderPermutationVector.Set<SHADER::FVarianceType>(SHADER::GetVarianceType());
			ComputeShaderPermutationVector.Set<SHADER::FRankedLuminanceVariance>(SHADER::UseRankedLuminanceVariance());

			TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, ComputeShaderPermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("PathTracing::Denoising::Prepass(%s %dx%d)",
					SHADER::GetEventName(SHADER::GetVarianceType()),
					TargetViewport.Extent.X,
					TargetViewport.Extent.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(TargetViewport.Extent, 8));
		}
		else
		{
			SpatialTemporalDenoisingContext.VarianceBuffer = SpatialTemporalDenoisingContext.LastVarianceBuffer;
		}

		FRDGBufferRef VarianceBuffer = SpatialTemporalDenoisingContext.VarianceBuffer;

		if (bShouldPrepassOutputVarianceTexture && VarianceBuffer)
		{
			const FRDGTextureDesc TextureDescriptor = FRDGTextureDesc::Create2D(
				TargetViewport.Extent,
				PF_A32B32G32R32F,
				FClearValueBinding(),
				TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV | GetExtraTextureCreateFlagsForDenoiser());

			if (!SpatialTemporalDenoisingContext.VarianceTexture)
			{
				SpatialTemporalDenoisingContext.VarianceTexture = GraphBuilder.CreateTexture(TextureDescriptor, TEXT("PathTracing.VarianceTexture"));
			}

			typedef FPrepassGenerateTextureCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->OutputTexture = GraphBuilder.CreateUAV(SpatialTemporalDenoisingContext.VarianceTexture);
				PassParameters->VarianceMap	  = GraphBuilder.CreateSRV(VarianceBuffer, EPixelFormat::PF_R32_FLOAT);
				PassParameters->TargetViewport = TargetViewportParameters;
				PassParameters->Iteration = FMath::Min(IterationNumber, MaxSPP - 1);
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			ComputeShaderPermutationVector.Set<SHADER::FVarianceType>(FTemporalPrepassCS::GetVarianceType());

			TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, ComputeShaderPermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("PathTracing::Denoising::Prepass::Texture(Var[%s] %dx%d)",
					FTemporalPrepassCS::GetEventName(FTemporalPrepassCS::GetVarianceType()),
					TargetViewport.Extent.X,
					TargetViewport.Extent.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(TargetViewport.Extent, 8));
		}
	}
}

void PathTracingSpatialTemporalDenoising(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	int DenoiserMode,
	FRDGTexture*& SpatialTemporalDenoisedTexture,
	FPathTracingSpatialTemporalDenoisingContext& SpatialTemporalDenoisingContext)
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, PathTracingSpatialTemporalDenoising, "PathTracingSpatialTemporalDenoising");

	FRDGTextureDesc RadianceTextureDesc = SpatialTemporalDenoisingContext.RadianceTexture->Desc;

	const ESpatialDenoiserType SpatialDenoiserType = GetSpatialDenoiserType();
	const ETemporalDenoiserType TemporalDenoiserType = GetTemporalDenoiserType();
	const bool bApplySpatialDenoiser = ShouldApplySpatialDenoiser();
	const bool bApplyTemporalDenoiser = ShouldApplyTemporalDenoiser(SpatialTemporalDenoisingContext, View);

	FRDGTextureRef TargetTexture = SpatialTemporalDenoisingContext.RadianceTexture;
	FRDGTextureRef SourceTexture = SpatialTemporalDenoisingContext.LastDenoisedRadianceTexture;
	FRDGTextureRef TemporalDenoisedTexture = nullptr;

	if (SpatialDenoiserType == ESpatialDenoiserType::SPATIAL_DENOISER_PLUGIN)
	{
		if (bApplySpatialDenoiser)
		{
			TargetTexture = GraphBuilder.CreateTexture(RadianceTextureDesc, TEXT("PathTracer.SpatialDenoiser.Output"));
			PathTracingDenoiserPlugin(
				GraphBuilder,
				View,
				DenoiserMode,
				SpatialTemporalDenoisingContext.RadianceTexture,
				SpatialTemporalDenoisingContext.AlbedoTexture,
				SpatialTemporalDenoisingContext.NormalTexture,
				SpatialTemporalDenoisingContext.DepthTexture,
				SpatialTemporalDenoisingContext.VarianceTexture,
				TargetTexture);
		}

		if (bApplyTemporalDenoiser)
		{
			// Image space temporal and spatial denoising.
			// 
			// Select motion source and target for temporal denoising
			FRDGTextureRef MotionEstimationSourceTexture = nullptr;
			FRDGTextureRef MotionEstimationTargetTexture = nullptr;
			bool IsSourceTargetDimensionMatch = SelectMotionSourceAndTargetTextures(
					GraphBuilder,
					View,
					SourceTexture,
					TargetTexture, SpatialTemporalDenoisingContext, MotionEstimationSourceTexture, MotionEstimationTargetTexture);


			if (IsSourceTargetDimensionMatch)
			{
				UE_LOGF(LogPathTracingDenoising, Log, "Using temporal denoising for frame %d", SpatialTemporalDenoisingContext.FrameIndex);

				if (TemporalDenoiserType == ETemporalDenoiserType::BUILTIN_TEMPORAL_DENOISER)
				{
					TemporalDenoisedTexture = TemporalReprojectionWithoutMotionVector(GraphBuilder,
						View,
						SourceTexture,
						TargetTexture,
						MotionEstimationSourceTexture,
						MotionEstimationTargetTexture,
						SpatialTemporalDenoisingContext);
				}
				else
				{
					// not supported
				}
			}
		}
	}
	else if (SpatialDenoiserType == ESpatialDenoiserType::SPATIAL_TEMPORAL_DENOISER_PLUGIN)
	{
		FRDGTextureRef MotionTexture = nullptr;
		bool bIsInitialFrame = true;
		
		MotionTexture = GraphBuilder.CreateTexture(RadianceTextureDesc, TEXT("PathTracing.EstimateMotion.OpticalFlow"));

		if (bApplyTemporalDenoiser)
		{
			// Select motion source and target for temporal denoising
			FRDGTextureRef MotionEstimationSourceTexture = nullptr;
			FRDGTextureRef MotionEstimationTargetTexture = nullptr;
			bool IsSourceTargetDimensionMatch = SelectMotionSourceAndTargetTextures(
					GraphBuilder,
					View,
					SourceTexture,
					TargetTexture, SpatialTemporalDenoisingContext, MotionEstimationSourceTexture, MotionEstimationTargetTexture);

			if (IsSourceTargetDimensionMatch)
			{
				if (TemporalDenoiserType == ETemporalDenoiserType::SPATIAL_TEMPORAL_DENOISER_PLUGIN)
				{
					FScreenPassTextureViewport TargetViewport = FScreenPassTextureViewport(View.ViewRect);
					

					PathTracingMotionVectorPlugin(
						GraphBuilder,
						View,
						MotionEstimationSourceTexture,
						MotionEstimationTargetTexture,
						MotionTexture);
					bIsInitialFrame = false;
					
				}
			}
		}
		else
		{
			FLinearColor ClearColor = FLinearColor::Black;
			AddClearRenderTargetPass(GraphBuilder, MotionTexture, ClearColor);
		}

		SpatialTemporalDenoisingContext.MotionVector = MotionTexture;

		TemporalDenoisedTexture = GraphBuilder.CreateTexture(RadianceTextureDesc, TEXT("PathTracing.EstimateMotion.FinalAccumulation"));
		
		PathTracingSpatialTemporalDenoiserPlugin(
			GraphBuilder, View, DenoiserMode, TargetTexture,
			SpatialTemporalDenoisingContext.AlbedoTexture, SpatialTemporalDenoisingContext.NormalTexture,
			SpatialTemporalDenoisingContext.DepthTexture,
			MotionTexture, SourceTexture, TemporalDenoisedTexture,
			SpatialTemporalDenoisingContext.FrameIndex,
			bIsInitialFrame,
			SpatialTemporalDenoisingContext);	// zero frame will denoise without temporal

	}

	SpatialTemporalDenoisedTexture = TemporalDenoisedTexture ? TemporalDenoisedTexture : TargetTexture;
}

class FVisualizePathTracingMotionVectorPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizePathTracingMotionVectorPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizePathTracingMotionVectorPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, TemporalDenoisingMotionVector)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, RasterMotionVector)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, DenoisedTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("VISUALIZE_MOTIONVECTOR"), 1);
	}

};

IMPLEMENT_GLOBAL_SHADER(FVisualizePathTracingMotionVectorPS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "VisualizePathTracingMotionVector", SF_Pixel);

class FVisualizeWarpingPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeWarpingPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeWarpingPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, TemporalDenoisingMotionVector)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, SourceTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, DenoisedTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, TargetTexture)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, TargetViewport)
		SHADER_PARAMETER_SAMPLER(SamplerState, SharedTextureSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompilePathTracingDenoiserShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FShaderPermutationParameters&, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("VISUALIZE_WARPING"), 1);
	}

};

IMPLEMENT_GLOBAL_SHADER(FVisualizeWarpingPS, "/Engine/Private/PathTracing/PathTracingSpatialTemporalDenoising.usf", "FVisualizeWarpingPS", SF_Pixel);

FScreenPassTexture AddVisualizePathTracingDenoisingPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizePathTracingDenoisingInputs& Inputs)
{
	
	if (ShouldVisualizePathTracingVelocityState(Inputs.DenoisingContext))
	{
		typedef FVisualizePathTracingMotionVectorPS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->TemporalDenoisingMotionVector = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Inputs.DenoisingContext.MotionVector));
			PassParameters->SceneTextures = Inputs.SceneTexturesUniformBuffer;
			PassParameters->DenoisedTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Inputs.DenoisedTexture));
			PassParameters->RenderTargets[0] = FRenderTargetBinding(Inputs.SceneColor, ERenderTargetLoadAction::ELoad);
		}

		TShaderMapRef<SHADER> PixelShader(View.ShaderMap);
		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("Visualize Motion Vector (%d x %d)", View.ViewRect.Size().X, View.ViewRect.Size().Y),
			View,
			Inputs.Viewport,
			Inputs.Viewport,
			PixelShader,
			PassParameters
		);
	}

	if (ShouldVisualizeWarping(Inputs.DenoisingContext))
	{
		const FScreenPassTextureViewport TargetViewport(Inputs.DenoisedTexture, View.ViewRect);
		typedef FVisualizeWarpingPS SHADER;
		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		{
			const FScreenPassTextureViewportParameters TargetViewportParameters = GetScreenPassTextureViewportParameters(TargetViewport);

			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->TemporalDenoisingMotionVector = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Inputs.DenoisingContext.MotionVector));
			PassParameters->SceneTextures = Inputs.SceneTexturesUniformBuffer;
			PassParameters->DenoisedTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Inputs.DenoisedTexture));
			PassParameters->SourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(Inputs.DenoisingContext.LastDenoisedRadianceTexture));
			PassParameters->SharedTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->TargetViewport = TargetViewportParameters;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(Inputs.SceneColor, ERenderTargetLoadAction::ELoad);
		}

		TShaderMapRef<SHADER> PixelShader(View.ShaderMap);
		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("Visualize Source Warping (%d x %d)", View.ViewRect.Size().X, View.ViewRect.Size().Y),
			View,
			Inputs.Viewport,
			Inputs.Viewport,
			PixelShader,
			PassParameters
		);
	}

	return FScreenPassTexture();
}

#endif
