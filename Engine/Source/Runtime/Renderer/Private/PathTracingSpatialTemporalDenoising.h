// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PathTracingDenoiser.h"
#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#include "RenderGraphFwd.h"
#include "ScreenPass.h"

class FRDGBuilder;
class FScene;
class FViewInfo;
class FRDGTexture;

struct FPathTracingSpatialTemporalDenoisingContext
{
	FRDGTexture*	RadianceTexture = nullptr;
	FRDGTexture*	AlbedoTexture = nullptr;
	FRDGTexture*	NormalTexture = nullptr;
	FRDGTexture*	DepthTexture = nullptr;
	FRDGBuffer*		VarianceBuffer = nullptr;

	FRDGTexture*	LastDenoisedRadianceTexture = nullptr;
	FRDGTexture*	LastRadianceTexture = nullptr;
	FRDGTexture*	LastAlbedoTexture = nullptr;
	FRDGTexture*	LastNormalTexture = nullptr;
	FRDGTexture*	LastDepthTexture = nullptr;
	FRDGBuffer*		LastVarianceBuffer = nullptr;

	// Custom path tracing spacial temporal denoiser result, used by plugins
	TRefCountPtr<UE::Renderer::Private::IPathTracingSpatialTemporalDenoiser::IHistory> SpatialTemporalDenoiserHistory;

	int FrameIndex;

	static constexpr uint32 kNumberOfPixelShifts = 25;
	static constexpr uint32 kNumberOfShiftsPerTexture = 4;
	static constexpr uint32 kNumberOfPasses = 1;
	static constexpr uint32 kNumberOfTexturesPerPass =
		(kNumberOfPixelShifts + (kNumberOfShiftsPerTexture * kNumberOfPasses) - 1)
		/ (kNumberOfShiftsPerTexture * kNumberOfPasses);

	// Debug information
	FRDGTexture*	MotionVector = nullptr;
	FRDGTexture*	WarpedSource = nullptr;

	// Utility texture
	FRDGTexture* VarianceTexture = nullptr;
};


int	GetPathTracingDenoiserMode(const FViewInfo& View);
bool IsPathTracingDenoiserEnabled(const FViewInfo& View);
bool ShouldEnablePathTracingDenoiserRealtimeDebug();
ETextureCreateFlags GetExtraTextureCreateFlagsForDenoiser();

// Calculate the variance per-pixel.
void PathTracingSpatialTemporalDenoisingPrePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	int IterationNumber,
	int MaxSPP,
	FPathTracingSpatialTemporalDenoisingContext& SpatialTemporalDenoisingContext
);

void PathTracingSpatialTemporalDenoising(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	int DenoiserMode,
	FRDGTexture*& SpatialTemporalDenoisedTexture,
	FPathTracingSpatialTemporalDenoisingContext& SpatialTemporalDenoisingContext);

struct FVisualizePathTracingDenoisingInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;
	FRDGTextureRef SceneColor;
	FRDGTextureRef DenoisedTexture;
	FScreenPassTextureViewport Viewport;

	FPathTracingSpatialTemporalDenoisingContext DenoisingContext;

	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTexturesUniformBuffer;
};

FScreenPassTexture AddVisualizePathTracingDenoisingPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizePathTracingDenoisingInputs& Inputs);

#endif
