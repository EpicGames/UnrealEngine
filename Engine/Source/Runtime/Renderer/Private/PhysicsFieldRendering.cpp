// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsFieldRendering.h"
#include "PhysicsField/PhysicsFieldComponent.h"

#include "RHIStaticStates.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShaderCompilerCore.h"
#include "RenderGraphUtils.h"
#include "CanvasTypes.h"
#include "RenderGraphUtils.h"
#include "SceneTextureParameters.h"
#include "DynamicPrimitiveDrawing.h"
#include "ShaderPrint.h"

// Console variables
int32 GPhysicsFieldTargetType = EFieldPhysicsType::Field_PhysicsType_Max;
FAutoConsoleVariableRef CVarPhysicsFieldTargetType(
	TEXT("r.PhysicsField.Rendering.TargetType"),
	GPhysicsFieldTargetType,
	TEXT("Physics field target to be used in the viewport show options.\n"),
	ECVF_RenderThreadSafe);

int32 GPhysicsFieldEvalType = 0;
FAutoConsoleVariableRef CVarPhysicsFieldEvalType(
	TEXT("r.PhysicsField.Rendering.EvalType"),
	GPhysicsFieldEvalType,
	TEXT("Physics field boolean to check if we are evaluating exactly(0) or sampling(1) the field for visualisation.\n"),
	ECVF_RenderThreadSafe);

int32 GPhysicsFieldSystemType = 0;
FAutoConsoleVariableRef CVarPhysicsFieldSystemType(
	TEXT("r.PhysicsField.Rendering.SystemType"),
	GPhysicsFieldSystemType,
	TEXT("Physics field boolean to check if we want to display the CPU(0) or GPU(1) field.\n"),
	ECVF_RenderThreadSafe);

float GPhysicsFieldTransientLifetime = 0.0f;
FAutoConsoleVariableRef CVarPhysicsFieldTransientLifetime(
	TEXT("r.PhysicsField.Rendering.TransientLifetime"),
	GPhysicsFieldTransientLifetime,
	TEXT("Physics field transient commands lifetime for rendering.\n"),
	ECVF_RenderThreadSafe);

/**
 * Physics Field Rendering.
 */

class FPhysicsFieldRayMarchingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPhysicsFieldRayMarchingCS);
	SHADER_USE_PARAMETER_STRUCT(FPhysicsFieldRayMarchingCS, FGlobalShader);

	static const int32 VoxelResolution = 10;

	class FFieldType : SHADER_PERMUTATION_INT("PERMUTATION_FIELD", 4);
	class FEvalType : SHADER_PERMUTATION_INT("PERMUTATION_EVAL", 2);
	using FPermutationDomain = TShaderPermutationDomain<FFieldType, FEvalType>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_SRV(Buffer<float4>, BoundsMin)
		SHADER_PARAMETER_SRV(Buffer<float4>, BoundsMax)
		SHADER_PARAMETER(uint32, BoundsOffset)
		SHADER_PARAMETER(uint32, BoundsSize)
		SHADER_PARAMETER(uint32, LocalTarget)
		SHADER_PARAMETER(uint32, PrintOffset)
		SHADER_PARAMETER(uint32, GlobalTarget)
		SHADER_PARAMETER(FVector2f, OutputResolution)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		return ShaderPrint::IsSupported(Parameters.Platform) && IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// Skip optimization for avoiding long compilation time due to large UAV writes
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MAX_PHYSICS_FIELD_TARGETS"), MAX_PHYSICS_FIELD_TARGETS);
		OutEnvironment.SetDefine(TEXT("VOXEL_RESOLUTION"), VoxelResolution);
		OutEnvironment.CompilerFlags.Add(ECompilerFlags::CFLAG_AllowTypedUAVLoads);
	}
};

IMPLEMENT_GLOBAL_SHADER(FPhysicsFieldRayMarchingCS, "/Engine/Private/PhysicsFieldVisualizer.usf", "MainCS", SF_Compute);

static FPhysicsFieldRayMarchingCS::FParameters* CreateShaderParameters(FRDGBuilder& GraphBuilder,
	const FViewInfo& View, const FPhysicsFieldResource* PhysicsFieldResource,
	FRDGTextureRef& OutputTexture, const int32 TargetIndex, const int32 TargetType, const uint32 PrintOffset)
{
	const FIntPoint OutputResolution(OutputTexture->Desc.Extent);
	FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

	FPhysicsFieldRayMarchingCS::FParameters* Parameters = GraphBuilder.AllocParameters<FPhysicsFieldRayMarchingCS::FParameters>();
	Parameters->OutputTexture = GraphBuilder.CreateUAV(OutputTexture);
	Parameters->OutputResolution = OutputResolution;
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->SceneTextures = SceneTextures;

	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);

	Parameters->BoundsMin = PhysicsFieldResource->BoundsMin.SRV;
	Parameters->BoundsMax = PhysicsFieldResource->BoundsMax.SRV;
	Parameters->BoundsOffset = PhysicsFieldResource->FieldInfos.BoundsOffsets[TargetType];
	Parameters->BoundsSize = PhysicsFieldResource->FieldInfos.BoundsOffsets[TargetType + 1] - PhysicsFieldResource->FieldInfos.BoundsOffsets[TargetType];;

	Parameters->LocalTarget = TargetIndex;
	Parameters->GlobalTarget = TargetType;
	Parameters->PrintOffset = PrintOffset;

	return Parameters;
}

static void AddPhysicsFieldRayMarchingPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FPhysicsFieldResource* PhysicsFieldResource,
	FRDGTextureRef& OutputTexture)
{
	const FIntPoint OutputResolution(OutputTexture->Desc.Extent);
	auto RenderPhysicsField = [&GraphBuilder, &View, PhysicsFieldResource, &OutputTexture, &OutputResolution](const int32 TargetType, const bool bClearBackground, uint32& PrintOffset) {

		int32 TargetIndex = INDEX_NONE;
		EFieldOutputType FieldOutputType = EFieldOutputType::Field_Output_Max;
		GetFieldIndex(TargetType, TargetIndex, FieldOutputType);

		const int32 NumBounds = PhysicsFieldResource->FieldInfos.BoundsOffsets[TargetType + 1] - PhysicsFieldResource->FieldInfos.BoundsOffsets[TargetType];
		if (NumBounds > 0 && (!bClearBackground || (bClearBackground && (FieldOutputType == EFieldOutputType::Field_Output_Vector))))
		{
			const FIntVector BoundResolution(FPhysicsFieldRayMarchingCS::VoxelResolution * NumBounds, FPhysicsFieldRayMarchingCS::VoxelResolution, FPhysicsFieldRayMarchingCS::VoxelResolution);
			PrintOffset = !bClearBackground ? PrintOffset + 1 : PrintOffset;

			FPhysicsFieldRayMarchingCS::FParameters* Parameters = CreateShaderParameters(GraphBuilder, View, PhysicsFieldResource, OutputTexture, TargetIndex, TargetType, PrintOffset);

			FPhysicsFieldRayMarchingCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FPhysicsFieldRayMarchingCS::FFieldType>(bClearBackground ? 3 : FieldOutputType);
			PermutationVector.Set<FPhysicsFieldRayMarchingCS::FEvalType>(GPhysicsFieldEvalType);
			TShaderMapRef<FPhysicsFieldRayMarchingCS> ComputeShader(View.ShaderMap, PermutationVector);

			const FIntVector DispatchCount = (!bClearBackground && (FieldOutputType == EFieldOutputType::Field_Output_Vector)) ?
				FIntVector::DivideAndRoundUp(BoundResolution, FIntVector(4, 4, 4)) :
				FIntVector::DivideAndRoundUp(FIntVector(OutputResolution.X, OutputResolution.Y, 1), FIntVector(8, 8, 1));

			FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RenderPhysicsField"), ComputeShader, Parameters, DispatchCount);
		}
	};
	FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

	uint32 PrintOffset = 0;
	if(GPhysicsFieldTargetType < EFieldPhysicsType::Field_PhysicsType_Max)
	{
		RenderPhysicsField(GPhysicsFieldTargetType, true, PrintOffset);
		RenderPhysicsField(GPhysicsFieldTargetType, false, PrintOffset);
	}
	else
	{
		for (int32 TargetType = 0; TargetType < EFieldPhysicsType::Field_PhysicsType_Max; ++TargetType)
		{
			RenderPhysicsField(TargetType, true, PrintOffset);
		}

		for (int32 TargetType = 0; TargetType < EFieldPhysicsType::Field_PhysicsType_Max; ++TargetType)
		{
			RenderPhysicsField(TargetType, false, PrintOffset);
		}
	}
}

void RenderPhysicsField(
	FRDGBuilder& GraphBuilder,
	TArrayView<FViewInfo> Views,
	const FPhysicsFieldSceneProxy* PhysicsFieldProxy,
	FRDGTextureRef SceneColorTexture)
{
	if (PhysicsFieldProxy)
	{
		FPhysicsFieldResource* PhysicsFieldResource = (GPhysicsFieldSystemType == 0) ? PhysicsFieldProxy->DebugResource : PhysicsFieldProxy->FieldResource;
		if (Views.Num() > 0 && PhysicsFieldResource && ShaderPrint::IsEnabled(Views[0].ShaderPrintData) && UE::PixelFormat::HasCapabilities(PF_FloatRGBA, EPixelFormatCapabilities::TypedUAVLoad))
		{
			if (ShaderPrint::IsSupported(Views[0].GetShaderPlatform()) && IsFeatureLevelSupported(Views[0].GetShaderPlatform(), ERHIFeatureLevel::SM5))
			{
				AddPhysicsFieldRayMarchingPass(GraphBuilder, Views[0], PhysicsFieldResource, SceneColorTexture);
			}
		}
	}
}



