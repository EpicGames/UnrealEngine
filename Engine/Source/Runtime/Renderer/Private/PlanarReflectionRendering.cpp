// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 PlanarReflectionRendering.cpp
=============================================================================*/

#include "PlanarReflectionRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Scene.h"
#include "SceneInterface.h"
#include "RenderingThread.h"
#include "RHIStaticStates.h"
#include "RendererInterface.h"
#include "Camera/CameraTypes.h"
#include "Shader.h"
#include "TextureResource.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "ScenePrivateBase.h"
#include "PostProcess/SceneRenderTargets.h"
#include "GlobalShader.h"
#include "SceneRenderTargetParameters.h"
#include "SceneRendering.h"
#include "SceneRenderingUtils.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessing.h"
#include "LightRendering.h"
#include "Materials/MaterialRenderProxy.h"
#include "Components/SceneCaptureComponent.h"
#include "Components/PlanarReflectionComponent.h"
#include "PlanarReflectionSceneProxy.h"
#include "Containers/ArrayView.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "SceneTextureParameters.h"
#include "SceneViewExtension.h"
#include "SceneCaptureRendering.h"
#include "Substrate/Substrate.h"
#include "SceneRenderBuilder.h"
#include "SceneViewState.h"

void SetupPlanarReflectionUniformParameters(const class FSceneView& View, const FPlanarReflectionSceneProxy* ReflectionSceneProxy, FPlanarReflectionUniformParameters& OutParameters)
{
	// Degenerate plane causes shader to branch around the reflection lookup
	OutParameters.ReflectionPlane.Set(0.0f, 0.0f, 0.0f, 0.0f);
	FTexture* PlanarReflectionTextureValue = GBlackTexture;

	if (ReflectionSceneProxy && ReflectionSceneProxy->RenderTarget)
	{
		ensure(ReflectionSceneProxy->ViewRect[0].Min.X >= 0);

		const FVector PreViewTranslation = View.ViewMatrices.GetPreViewTranslation();
		const FPlane4f TranslatedReflectionPlane(ReflectionSceneProxy->ReflectionPlane.TranslateBy(PreViewTranslation));

		// Need to set W separately due to FVector = FPlane, which sets W to 1.0.
		OutParameters.ReflectionPlane = TranslatedReflectionPlane;
		OutParameters.ReflectionPlane.W = TranslatedReflectionPlane.W;

		PlanarReflectionTextureValue = ReflectionSceneProxy->RenderTarget;

		FIntPoint BufferSize = ReflectionSceneProxy->RenderTarget->GetSizeXY();
		float InvBufferSizeX = 1.0f / BufferSize.X;
		float InvBufferSizeY = 1.0f / BufferSize.Y;

		FVector2D PlanarReflectionScreenBoundValue(
			1 - 2 * 0.5 / ReflectionSceneProxy->ViewRect[0].Width(),
			1 - 2 * 0.5 / ReflectionSceneProxy->ViewRect[0].Height());

		// Uses hardware's texture unit to reliably clamp UV if the view fill the entire buffer.
		if (View.Family->Views.Num() == 1 &&
			ReflectionSceneProxy->ViewRect[0].Min == FIntPoint::ZeroValue &&
			ReflectionSceneProxy->ViewRect[0].Max == BufferSize)
		{
			PlanarReflectionScreenBoundValue = FVector2D(1, 1);
		}

		FVector4f ScreenScaleBiasValue[2] = {
			FVector4f(0, 0, 0, 0),
			FVector4f(0, 0, 0, 0),
		};
		for (int32 ViewIndex = 0; ViewIndex < FMath::Min(View.Family->Views.Num(), GMaxPlanarReflectionViews); ViewIndex++)
		{
			FIntRect ViewRect = ReflectionSceneProxy->ViewRect[ViewIndex];
			ScreenScaleBiasValue[ViewIndex] = FVector4f(
				ViewRect.Width() * InvBufferSizeX / +2.0f,
				ViewRect.Height() * InvBufferSizeY / (-2.0f * GProjectionSignY),
				(ViewRect.Width() / 2.0f + ViewRect.Min.X) * InvBufferSizeX,
				(ViewRect.Height() / 2.0f + ViewRect.Min.Y) * InvBufferSizeY);
		}

		OutParameters.PlanarReflectionOrigin = (FVector3f)(PreViewTranslation + ReflectionSceneProxy->PlanarReflectionOrigin); // LWC_TODO: precision loss
		OutParameters.PlanarReflectionXAxis = (FVector4f)ReflectionSceneProxy->PlanarReflectionXAxis; // LWC_TODO: precision loss
		OutParameters.PlanarReflectionYAxis = (FVector4f)ReflectionSceneProxy->PlanarReflectionYAxis; // LWC_TODO: precision loss
		OutParameters.InverseTransposeMirrorMatrix = ReflectionSceneProxy->InverseTransposeMirrorMatrix;
		OutParameters.PlanarReflectionParameters = (FVector3f)ReflectionSceneProxy->PlanarReflectionParameters;
		OutParameters.PlanarReflectionParameters2 = FVector2f(ReflectionSceneProxy->PlanarReflectionParameters2);	// LWC_TODO: Precision loss
		OutParameters.bIsStereo = ReflectionSceneProxy->bIsStereo;
		OutParameters.PlanarReflectionScreenBound = FVector2f(PlanarReflectionScreenBoundValue);	// LWC_TODO: Precision loss

		// Instanced stereo needs both view's values available at once
		if (ReflectionSceneProxy->bIsStereo || View.Family->Views.Num() == 1)
		{
			static_assert(UE_ARRAY_COUNT(ReflectionSceneProxy->ProjectionWithExtraFOV) == 2 
				&& GPlanarReflectionUniformMaxReflectionViews == 2, "Code assumes max 2 planar reflection views.");

			OutParameters.ProjectionWithExtraFOV[0] = FMatrix44f(ReflectionSceneProxy->ProjectionWithExtraFOV[0]);	// LWC_TODO: Precision loss
			OutParameters.ProjectionWithExtraFOV[1] = FMatrix44f(ReflectionSceneProxy->ProjectionWithExtraFOV[1]);

			OutParameters.PlanarReflectionScreenScaleBias[0] = ScreenScaleBiasValue[0];
			OutParameters.PlanarReflectionScreenScaleBias[1] = ScreenScaleBiasValue[1];
		}
		else
		{
			int32 ViewIndex = 0;

			for (int32 i = 0; i < View.Family->Views.Num(); i++)
			{
				if (&View == View.Family->Views[i])
				{
					ViewIndex = i;
					break;
				}
			}
			// Clamp the index to not go out of bounds (can happen for example in split screen with > 2 players).
			ViewIndex = FMath::Min(ViewIndex, GPlanarReflectionUniformMaxReflectionViews - 1);
			// Make sure the current view's value is at index 0
			OutParameters.ProjectionWithExtraFOV[0] = FMatrix44f(ReflectionSceneProxy->ProjectionWithExtraFOV[ViewIndex]);		// LWC_TODO: Precision loss?
			OutParameters.ProjectionWithExtraFOV[1] = FMatrix44f::Identity;
			OutParameters.PlanarReflectionScreenScaleBias[0] = ScreenScaleBiasValue[ViewIndex];
			OutParameters.PlanarReflectionScreenScaleBias[1] = FVector4f(0, 0, 0, 0);
		}
	}
	else
	{
		OutParameters.bIsStereo = false;
	}

	OutParameters.PlanarReflectionTexture = PlanarReflectionTextureValue->TextureRHI;
	OutParameters.PlanarReflectionSampler = PlanarReflectionTextureValue->SamplerStateRHI;
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FPlanarReflectionUniformParameters, "PlanarReflectionStruct");

template< bool bEnablePlanarReflectionPrefilter >
class TPrefilterPlanarReflectionPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(TPrefilterPlanarReflectionPS);
	SHADER_USE_PARAMETER_STRUCT(TPrefilterPlanarReflectionPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_REF(FPlanarReflectionUniformParameters, PlanarReflection)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER(float, KernelRadiusY)
		SHADER_PARAMETER(float, InvPrefilterRoughnessDistance)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorInputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorInputSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return bEnablePlanarReflectionPrefilter ? IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) : true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("ENABLE_PLANAR_REFLECTIONS_PREFILTER"), bEnablePlanarReflectionPrefilter);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_SHADER_TYPE(template<>, TPrefilterPlanarReflectionPS<false>, TEXT("/Engine/Private/PlanarReflectionShaders.usf"), TEXT("PrefilterPlanarReflectionPS"), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TPrefilterPlanarReflectionPS<true>, TEXT("/Engine/Private/PlanarReflectionShaders.usf"), TEXT("PrefilterPlanarReflectionPS"), SF_Pixel);

template<bool bEnablePlanarReflectionPrefilter>
void PrefilterPlanarReflection(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSceneTextureShaderParameters SceneTextures,
	const FPlanarReflectionSceneProxy* ReflectionSceneProxy,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef ViewFamilyTexture)
{
	using FPrefilterPlanarReflectionPS = TPrefilterPlanarReflectionPS<bEnablePlanarReflectionPrefilter>;

	if(View.FeatureLevel >= ERHIFeatureLevel::SM5)
	{
		SceneColorTexture = AddProcessPlanarReflectionPass(GraphBuilder, View, SceneColorTexture);
	}

	{
		RDG_EVENT_SCOPE(GraphBuilder, "PrefilterPlanarReflection");

		// Workaround for a possible driver bug on S7 Adreno, missing planar reflections
		const ERenderTargetLoadAction RTLoadAction = IsVulkanMobilePlatform(View.GetShaderPlatform()) ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ENoAction;

		const float FilterWidth = View.ViewRect.Width();

		auto* PassParameters = GraphBuilder.AllocParameters<typename FPrefilterPlanarReflectionPS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;

		{
			FPlanarReflectionUniformParameters PlanarReflectionUniformParameters;
			SetupPlanarReflectionUniformParameters(View, ReflectionSceneProxy, PlanarReflectionUniformParameters);
			PassParameters->PlanarReflection = TUniformBufferRef<FPlanarReflectionUniformParameters>::CreateUniformBufferImmediate(PlanarReflectionUniformParameters, UniformBuffer_SingleFrame);
		}

		PassParameters->SceneTextures = SceneTextures;

		PassParameters->KernelRadiusY = FMath::Clamp(ReflectionSceneProxy->PrefilterRoughness, 0.0f, 0.04f) * 0.5f * FilterWidth;
		PassParameters->InvPrefilterRoughnessDistance = 1.0f / FMath::Max(ReflectionSceneProxy->PrefilterRoughnessDistance, DELTA);
		PassParameters->SceneColorInputTexture = SceneColorTexture;
		PassParameters->SceneColorInputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(ViewFamilyTexture, RTLoadAction);

		FDeferredLightVS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FDeferredLightVS::FRadialLight>(false);
		TShaderMapRef<FDeferredLightVS> VertexShader(View.ShaderMap, PermutationVector);
		TShaderMapRef<FPrefilterPlanarReflectionPS> PixelShader(View.ShaderMap);

		const FIntPoint SceneColorExtent = SceneColorTexture->Desc.Extent;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("PrefilterPlanarReflections"),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, VertexShader, PixelShader, PassParameters, SceneColorExtent](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			FIntPoint UV = View.ViewRect.Min;
			FIntPoint UVSize = View.ViewRect.Size();

			FDeferredLightVS::FParameters ParametersVS = FDeferredLightVS::GetParameters(View, 
				0, 0,
				View.ViewRect.Width(), View.ViewRect.Height(),
				UV.X, UV.Y,
				UVSize.X, UVSize.Y,
				View.ViewRect.Size(),
				SceneColorExtent);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);

			DrawRectangle(
				RHICmdList,
				0, 0,
				View.ViewRect.Width(), View.ViewRect.Height(),
				UV.X, UV.Y,
				UVSize.X, UVSize.Y,
				View.ViewRect.Size(),
				SceneColorExtent,
				VertexShader,
				EDRF_UseTriangleOptimization);
		});
	}
}

struct FUpdatePlanarReflectionContentsInputs
{
	FSceneRenderer* ParentSceneRenderer;
	FPlanarReflectionSceneProxy* SceneProxy;
	FPlanarReflectionRenderTarget* RenderTarget;
	FPlane MirrorPlane;
};

static bool UpdatePlanarReflectionContents_RenderThread(
	FRDGBuilder& GraphBuilder,
	FSceneRenderer* SceneRenderer,
	const FSceneRenderUpdateInputs* SceneUpdateInputs,
	const FUpdatePlanarReflectionContentsInputs& Inputs)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RenderPlanarReflection);

	{
		FBox PlanarReflectionBounds = Inputs.SceneProxy->WorldBounds;
		bool bIsInAnyFrustum = false;

		for (int32 ViewIndex = 0; ViewIndex < Inputs.ParentSceneRenderer->Views.Num(); ++ViewIndex)
		{
			FViewInfo& View = Inputs.ParentSceneRenderer->Views[ViewIndex];
			if (Inputs.MirrorPlane.PlaneDot(View.ViewMatrices.GetViewOrigin()) > 0)
			{
				if (View.ViewFrustum.IntersectBox(PlanarReflectionBounds.GetCenter(), PlanarReflectionBounds.GetExtent()))
				{
					bIsInAnyFrustum = true;
					break;
				}
			}
		}

		if (!bIsInAnyFrustum)
		{
			return false;
		}

		bool bIsVisibleInAnyView = true;

		for (const FViewInfo& View : Inputs.ParentSceneRenderer->Views)
		{
			FSceneViewState* ViewState = View.ViewState;

			if (ViewState)
			{
				FIndividualOcclusionHistory& OcclusionHistory = ViewState->PlanarReflectionOcclusionHistories.FindOrAdd(Inputs.SceneProxy->PlanarReflectionId);

				// +1 to buffered frames because the query is submitted late into the main frame, but read at the beginning of a reflection capture frame
				const int32 NumBufferedFrames = FOcclusionQueryHelpers::GetNumBufferedFrames(SceneRenderer->FeatureLevel) + 1;
				// +1 to frame counter because we are operating before the main view's InitViews, which is where OcclusionFrameCounter is incremented
				uint32 OcclusionFrameCounter = ViewState->OcclusionFrameCounter + 1;
				FRHIRenderQuery* PastQuery = OcclusionHistory.GetPastQuery(OcclusionFrameCounter, NumBufferedFrames);

				if (PastQuery)
				{
					uint64 NumSamples = 0;
					QUICK_SCOPE_CYCLE_COUNTER(STAT_PlanarReflectionOcclusionQueryResults);

					if (RHIGetRenderQueryResult(PastQuery, NumSamples, true))
					{
						bIsVisibleInAnyView = NumSamples > 0;
						if (bIsVisibleInAnyView)
						{
							break;
						}
					}
				}
			}
		}

		if (!bIsVisibleInAnyView)
		{
			return false;
		}
	}

	// Clear the SRV prior to rendering so we don't attempt to read from the target we are writing to.
	Inputs.SceneProxy->RenderTarget = nullptr;

	// Assign the view rect to the resolved scene renderer view rect for the main scene render pass.
	// Do this at the end of the scope since we can early out for mobile pixel reflections. This needs
	// to be called after PrepareViewRectsForRendering (which is done within Render for the main path).
	ON_SCOPE_EXIT
	{
		// Update the view rects into the planar reflection proxy.
		for (int32 ViewIndex = 0; ViewIndex < SceneRenderer->Views.Num(); ++ViewIndex)
		{
			// Make sure screen percentage has correctly been set on render thread.
			check(SceneRenderer->Views[ViewIndex].ViewRect.Area() > 0);
			Inputs.SceneProxy->ViewRect[ViewIndex] = SceneRenderer->Views[ViewIndex].ViewRect;
		}

		// Re-assign the SRV after rendering.
		Inputs.SceneProxy->RenderTarget = Inputs.RenderTarget;
	};

	const ERHIFeatureLevel::Type FeatureLevel = SceneRenderer->FeatureLevel;

	// Make sure we render to the same set of GPUs as the main scene renderer.
	if (Inputs.ParentSceneRenderer->ViewFamily.RenderTarget != nullptr)
	{
		Inputs.RenderTarget->SetActiveGPUMask(Inputs.ParentSceneRenderer->ViewFamily.RenderTarget->GetGPUMask(GraphBuilder.RHICmdList));
	}
	else
	{
		Inputs.RenderTarget->SetActiveGPUMask(FRHIGPUMask::GPU0());
	}

	// Applies late update (if any) to view matrices and re-reflects
	if (SceneRenderer->Views.Num() > 1)
	{
		const FMirrorMatrix MirrorMatrix(Inputs.MirrorPlane);
		for (int32 ViewIndex = 0; ViewIndex < SceneRenderer->Views.Num(); ++ViewIndex)
		{
			FViewInfo& ReflectionViewToUpdate = SceneRenderer->Views[ViewIndex];

			// Updates view matrices to match new ViewLocation/ViewRotation, un-reflects
			// Normally performed in late update itself, delayed to here to ensure we don't ever re-reflect without first un-reflecting
			ReflectionViewToUpdate.UpdateViewMatrix(); 

			// Re-reflects view matrices
			ReflectionViewToUpdate.UpdatePlanarReflectionViewMatrix(ReflectionViewToUpdate, MirrorMatrix);
		}
	}

	UE_CLOGF(FSceneCaptureLogUtils::bEnableSceneCaptureLogging, LogSceneCapture, Log, "Running UpdatePlanarReflectionContents_RenderThread.");

	SceneRenderer->Render(GraphBuilder, SceneUpdateInputs);

	FRDGTextureRef ReflectionOutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Inputs.RenderTarget->TextureRHI, TEXT("ReflectionOutputTexture")));
	GraphBuilder.SetTextureAccessFinal(ReflectionOutputTexture, ERHIAccess::SRVGraphics);

	FSceneTextureShaderParameters SceneTextureParameters = CreateSceneTextureShaderParameters(GraphBuilder, &SceneRenderer->GetActiveSceneTextures(), SceneRenderer->FeatureLevel, ESceneTextureSetupMode::SceneDepth);
	const FMinimalSceneTextures& SceneTextures = SceneRenderer->GetActiveSceneTextures();

	for (FViewInfo& View : SceneRenderer->Views)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		if (Inputs.ParentSceneRenderer->Scene->GetShadingPath() == EShadingPath::Deferred)
		{
			PrefilterPlanarReflection<true>(GraphBuilder, View, SceneTextureParameters, Inputs.SceneProxy, SceneTextures.Color.Resolve, ReflectionOutputTexture);
		}
		else
		{
			PrefilterPlanarReflection<false>(GraphBuilder, View, SceneTextureParameters, Inputs.SceneProxy, SceneTextures.Color.Resolve, ReflectionOutputTexture);
		}
	}

	return true;
}

extern void BuildProjectionMatrix(FIntPoint RenderTargetSize, float FOV, float InNearClippingPlane, float InFarClippingPlane, FMatrix& ProjectionMatrix);

namespace UE::Renderer::SceneCapture::Private
{

extern TArray<FSceneView*> SetupViewFamilyForSceneCapture(
	FSceneViewFamily& ViewFamily,
	USceneCaptureComponent* SceneCaptureComponent,
	const TArrayView<const FSceneCaptureViewInfo> Views,
	float MaxViewDistance,
	ESetupViewFamilyFlags Flags,
	FPostProcessSettings* PostProcessSettings,
	const FFinalPostProcessSettings* InheritedMainViewPostProcessSettings,
	float PostProcessBlendWeight,
	const AActor* ViewActor,
	int32 CubemapFaceIndex,
	int32 CubemapFaceOffset);

extern void SetupSceneViewExtensionsForSceneCapture(
	FSceneViewFamily& ViewFamily,
	TConstArrayView<FSceneView*> Views);

} // namespace UE::Renderer::SceneCapture::Private

void FScene::UpdatePlanarReflectionContents(UPlanarReflectionComponent* CaptureComponent, FSceneRenderer& ParentSceneRenderer, ISceneRenderBuilder& SceneRenderBuilder)
{
	using namespace UE::Renderer::SceneCapture::Private;
	check(CaptureComponent);

	FIntPoint DesiredBufferSize = FSceneRenderer::GetDesiredInternalBufferSize(ParentSceneRenderer.ViewFamily);
	FVector2D DesiredPlanarReflectionTextureSizeFloat = FVector2D(DesiredBufferSize.X, DesiredBufferSize.Y) * FMath::Clamp(CaptureComponent->ScreenPercentage / 100.f, 0.25f, 1.f);
	FIntPoint DesiredPlanarReflectionTextureSize;
	DesiredPlanarReflectionTextureSize.X = FMath::Clamp(FMath::CeilToInt32(DesiredPlanarReflectionTextureSizeFloat.X), 1, static_cast<int32>(DesiredBufferSize.X));
	DesiredPlanarReflectionTextureSize.Y = FMath::Clamp(FMath::CeilToInt32(DesiredPlanarReflectionTextureSizeFloat.Y), 1, static_cast<int32>(DesiredBufferSize.Y));

	const bool bIsRenderTargetValid = CaptureComponent->RenderTarget != nullptr
								&& CaptureComponent->RenderTarget->GetSizeXY() == DesiredPlanarReflectionTextureSize
								&& (CaptureComponent->RenderTarget->TextureRHI.IsValid());

	FPlanarReflectionRenderTarget* RenderTargetToRelease = nullptr;
	FPlanarReflectionRenderTarget* RenderTargetToInit = nullptr;

	if (CaptureComponent->RenderTarget != nullptr && !bIsRenderTargetValid)
	{
		RenderTargetToRelease = CaptureComponent->RenderTarget;
		CaptureComponent->RenderTarget = nullptr;
	}

	if (CaptureComponent->RenderTarget == nullptr)
	{
		CaptureComponent->RenderTarget = new FPlanarReflectionRenderTarget(DesiredPlanarReflectionTextureSize);

		RenderTargetToInit = CaptureComponent->RenderTarget;
	}

	const FMatrix ComponentTransform = CaptureComponent->GetComponentTransform().ToMatrixWithScale();
	FPlane MirrorPlane = FPlane(ComponentTransform.TransformPosition(FVector::ZeroVector), ComponentTransform.TransformVector(FVector(0, 0, 1)));

	if (!MirrorPlane.Normalize())
	{
		MirrorPlane = FPlane(FVector(0, 0, 1), 0);
	}

	TArray<FSceneCaptureViewInfo, FConcurrentLinearArrayAllocator> SceneCaptureViewInfo;

	for (int32 ViewIndex = 0; ViewIndex < ParentSceneRenderer.Views.Num() && ViewIndex < GMaxPlanarReflectionViews; ++ViewIndex)
	{
		const FViewInfo& View = ParentSceneRenderer.Views[ViewIndex];
		FSceneCaptureViewInfo NewView;

		FVector2D ViewRectMin = FVector2D(View.UnscaledViewRect.Min.X, View.UnscaledViewRect.Min.Y);
		FVector2D ViewRectMax = FVector2D(View.UnscaledViewRect.Max.X, View.UnscaledViewRect.Max.Y);
		ViewRectMin *= FMath::Clamp(CaptureComponent->ScreenPercentage / 100.f, 0.25f, 1.f);
		ViewRectMax *= FMath::Clamp(CaptureComponent->ScreenPercentage / 100.f, 0.25f, 1.f);

		NewView.ViewRect.Min.X = FMath::TruncToInt(ViewRectMin.X);
		NewView.ViewRect.Min.Y = FMath::TruncToInt(ViewRectMin.Y);
		NewView.ViewRect.Max.X = FMath::CeilToInt(ViewRectMax.X);
		NewView.ViewRect.Max.Y = FMath::CeilToInt(ViewRectMax.Y);

		// Create a mirror matrix and premultiply the view transform by it
		const FMirrorMatrix MirrorMatrix(MirrorPlane);
		const FMatrix ViewMatrix(MirrorMatrix * View.ViewMatrices.GetWorldToView());
		const FVector ViewOrigin = ViewMatrix.InverseTransformPosition(FVector::ZeroVector);
		const FMatrix ViewRotationMatrix = ViewMatrix.RemoveTranslation();

		FMatrix ProjectionMatrix;
		if (!View.IsPerspectiveProjection() || (CaptureComponent->ExtraFOV == 0.f && ParentSceneRenderer.Views.Num() > 1))
		{
			// Prefer exact (potentially uneven) stereo projection matrices when no extra FOV is requested
			ProjectionMatrix = View.ViewMatrices.GetViewToClip();
		}
		else
		{
			const float HalfFOV = FMath::Atan(1.0f / View.ViewMatrices.GetViewToClip().M[0][0]);
			BuildProjectionMatrix(View.UnscaledViewRect.Size(), HalfFOV + FMath::DegreesToRadians(CaptureComponent->ExtraFOV), GNearClippingPlane, GNearClippingPlane, ProjectionMatrix);
		}

		NewView.ViewLocation = View.ViewLocation;
		NewView.ViewRotation = View.ViewRotation;
		NewView.ViewOrigin = ViewOrigin;
		NewView.ViewRotationMatrix = ViewRotationMatrix;
		NewView.ProjectionMatrix = ProjectionMatrix;
		NewView.StereoPass = View.StereoPass;
		NewView.StereoViewIndex = View.StereoViewIndex;
		NewView.FOV = View.FOV;

		SceneCaptureViewInfo.Add(NewView);
	}

	FPostProcessSettings PostProcessSettings;

	bool bIsMobileMultiViewEnabled = false;
	if (ParentSceneRenderer.ViewFamily.Views.Num() > 0)
	{
		bIsMobileMultiViewEnabled = ParentSceneRenderer.ViewFamily.Views[0]->Aspects.IsMobileMultiViewEnabled();
	}

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		CaptureComponent->RenderTarget,
		this,
		CaptureComponent->ShowFlags)
		.SetResolveScene(false)
		.SetRealtimeUpdate(true)
		.SetRequireMobileMultiView(bIsMobileMultiViewEnabled));

	FSceneViewExtensionContext ViewExtensionContext(this);
	ViewExtensionContext.bStereoEnabled = true;
	ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(ViewExtensionContext);

	ESetupViewFamilyFlags SetupViewFamilyFlags = ESetupViewFamilyFlags::CaptureSceneColor | ESetupViewFamilyFlags::IsPlanarReflection;
	if (CaptureComponent->bOverrideVirtualTextureThrottle)
	{
		SetupViewFamilyFlags |= ESetupViewFamilyFlags::OverrideVirtualTextureThrottle;
	}

	TArray<FSceneView*> Views = SetupViewFamilyForSceneCapture(
		ViewFamily,
		CaptureComponent,
		SceneCaptureViewInfo, CaptureComponent->MaxViewDistanceOverride,
		SetupViewFamilyFlags,
		&PostProcessSettings,
		/* InheritedMainViewPostProcessSettings = */ nullptr,
		1.0f,
		/*ViewActor =*/ nullptr,
		/*CubemapFaceIndex =*/ INDEX_NONE,
		/*CubemapFaceOffset =*/ 0);

	// Fork main renderer's screen percentage interface to have exactly same settings.
	ViewFamily.EngineShowFlags.ScreenPercentage = ParentSceneRenderer.ViewFamily.EngineShowFlags.ScreenPercentage;
	ViewFamily.SetScreenPercentageInterface(FSceneRenderer::ForkScreenPercentageInterface(
		ParentSceneRenderer.ViewFamily.GetScreenPercentageInterface(), ViewFamily));

	for (FSceneView* View : Views)
	{
		View->GlobalClippingPlane = MirrorPlane;
		// Jitter can't be removed completely due to the clipping plane
		// Also, this prevents the prefilter pass, which reads from jittered depth, from having to do special handling of it's depth-dependent input
		View->bAllowTemporalJitter = false;
		View->bRenderSceneTwoSided = CaptureComponent->bRenderSceneTwoSided;
	}

	const bool bIsStereo = IStereoRendering::IsStereoEyeView(ParentSceneRenderer.Views[0]);

	// Call SetupViewFamily & SetupView on scene view extensions before renderer creation
	SetupSceneViewExtensionsForSceneCapture(ViewFamily, Views);

	for (int32 ViewIndex = 0; ViewIndex < SceneCaptureViewInfo.Num(); ++ViewIndex)
	{
		CaptureComponent->ProjectionWithExtraFOV[ViewIndex] = SceneCaptureViewInfo[ViewIndex].ProjectionMatrix;
	}

	FString EventName = "PlanarReflection";

#if WANTS_DRAW_MESH_EVENTS
	if (CaptureComponent->GetOwner())
	{
		EventName = CaptureComponent->GetOwner()->GetFName().ToString();
	}
#endif

	SCENE_RENDER_GROUP_SCOPE(SceneRenderBuilder, MoveTemp(EventName), ESceneRenderGroupFlags::None);

	if (RenderTargetToRelease)
	{
		SceneRenderBuilder.AddRenderCommand([RenderTargetToRelease](FRHICommandListImmediate& RHICmdList)
		{
			RenderTargetToRelease->ReleaseResource();
			delete RenderTargetToRelease;
		});
		RenderTargetToRelease = nullptr;
	}

	if (RenderTargetToInit)
	{
		SceneRenderBuilder.AddRenderCommand([RenderTargetToInit] (FRHICommandListImmediate& RHICmdList)
		{
			RenderTargetToInit->InitResource(RHICmdList);
		});
	}

	const FUpdatePlanarReflectionContentsInputs UpdateInputs
	{
		  .ParentSceneRenderer = &ParentSceneRenderer
		, .SceneProxy = CaptureComponent->SceneProxy
		, .RenderTarget = CaptureComponent->RenderTarget
		, .MirrorPlane = MirrorPlane
	};

	FSceneRenderer* SceneRenderer = SceneRenderBuilder.CreateSceneRenderer(&ViewFamily);

	SceneRenderBuilder.AddRenderer(SceneRenderer,
		[
			  UpdateInputs
			, SceneCaptureViewInfo = MoveTemp(SceneCaptureViewInfo)
			, bIsStereo
		] (FRDGBuilder& GraphBuilder, const FSceneRenderFunctionInputs& Inputs)
	{
		// Remove the render target on the planar reflection proxy so that this planar reflection is not getting drawn in its own FSceneRenderer.
		UpdateInputs.SceneProxy->RenderTarget = nullptr;

		for (int32 ViewIndex = 0; ViewIndex < SceneCaptureViewInfo.Num(); ++ViewIndex)
		{
			UpdateInputs.SceneProxy->ProjectionWithExtraFOV[ViewIndex] = SceneCaptureViewInfo[ViewIndex].ProjectionMatrix;
			UpdateInputs.SceneProxy->bIsStereo = bIsStereo;
		}

		return UpdatePlanarReflectionContents_RenderThread(GraphBuilder, Inputs.Renderer, Inputs.SceneUpdateInputs, UpdateInputs);
	});
}

void FScene::AddPlanarReflection(UPlanarReflectionComponent* Component)
{
	check(Component->SceneProxy);
	PlanarReflections_GameThread.Add(Component);

	FPlanarReflectionSceneProxy* SceneProxy = Component->SceneProxy;
	FScene* Scene = this;
	ENQUEUE_RENDER_COMMAND(FAddPlanarReflectionCommand)(
		[SceneProxy, Scene](FRHICommandListImmediate& RHICmdList)
		{
			Scene->ReflectionSceneData.SetRegisteredReflectionCapturesHasChanged();
			Scene->PlanarReflections.Add(SceneProxy);
		});
}

void FScene::RemovePlanarReflection(UPlanarReflectionComponent* Component) 
{
	check(Component->SceneProxy);
	PlanarReflections_GameThread.Remove(Component);

	FPlanarReflectionSceneProxy* SceneProxy = Component->SceneProxy;
	FScene* Scene = this;
	ENQUEUE_RENDER_COMMAND(FRemovePlanarReflectionCommand)(
		[SceneProxy, Scene](FRHICommandListImmediate& RHICmdList)
		{
			Scene->ReflectionSceneData.SetRegisteredReflectionCapturesHasChanged();
			Scene->PlanarReflections.Remove(SceneProxy);

			// Need to clear out all planar reflection references on removal to avoid dangling pointers.
			for (int32 PrimitiveIndex = 0; PrimitiveIndex < Scene->Primitives.Num(); ++PrimitiveIndex)
			{
				Scene->Primitives[PrimitiveIndex]->RemoveCachedPlanarReflection(SceneProxy);
			}
		});
}

void FScene::UpdatePlanarReflectionTransform(UPlanarReflectionComponent* Component)
{	
	check(Component->SceneProxy);

	FPlanarReflectionSceneProxy* SceneProxy = Component->SceneProxy;
	FMatrix Transform = Component->GetComponentTransform().ToMatrixWithScale();
	FScene* Scene = this;
	ENQUEUE_RENDER_COMMAND(FUpdatePlanarReflectionCommand)(
		[SceneProxy, Transform, Scene](FRHICommandListImmediate& RHICmdList)
		{
			Scene->ReflectionSceneData.SetRegisteredReflectionCapturesHasChanged();
			SceneProxy->UpdateTransform(Transform);
		});
}

class FPlanarReflectionPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FPlanarReflectionPS);
	SHADER_USE_PARAMETER_STRUCT(FPlanarReflectionPS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_REF(FPlanarReflectionUniformParameters, PlanarReflectionParameters)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FPlanarReflectionPS, "/Engine/Private/PlanarReflectionShaders.usf", "PlanarReflectionPS", SF_Pixel);

bool FDeferredShadingSceneRenderer::HasDeferredPlanarReflections(const FViewInfo& View) const
{
	if (View.bIsPlanarReflection || View.bIsReflectionCapture)
	{
		return false;
	}

	// Prevent rendering unsupported views when ViewIndex >= GMaxPlanarReflectionViews
	// Planar reflections in those views will fallback to other reflection methods
	{
		int32 ViewIndex = INDEX_NONE;

		ViewFamily.Views.Find(&View, ViewIndex);

		if (ViewIndex >= GMaxPlanarReflectionViews)
		{
			return false;
		}
	}

	bool bAnyVisiblePlanarReflections = false;

	for (int32 PlanarReflectionIndex = 0; PlanarReflectionIndex < Scene->PlanarReflections.Num(); PlanarReflectionIndex++)
	{
		FPlanarReflectionSceneProxy* ReflectionSceneProxy = Scene->PlanarReflections[PlanarReflectionIndex];

		if (View.GetCullingFrustum().IntersectBox(ReflectionSceneProxy->WorldBounds.GetCenter(), ReflectionSceneProxy->WorldBounds.GetExtent()))
		{
			bAnyVisiblePlanarReflections = true;
			break;
		}
	}

	bool bComposePlanarReflections = Scene->PlanarReflections.Num() > 0 && bAnyVisiblePlanarReflections;

	return bComposePlanarReflections;
}

void FDeferredShadingSceneRenderer::RenderDeferredPlanarReflections(FRDGBuilder& GraphBuilder, const FSceneTextureParameters& SceneTextures, const FViewInfo& View, FRDGTextureRef& ReflectionsOutputTexture)
{
	check(HasDeferredPlanarReflections(View));

	// Allocate planar reflection texture
	bool bClearReflectionsOutputTexture = false;
	if (!ReflectionsOutputTexture)
	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			SceneTextures.SceneDepthTexture->Desc.Extent,
			PF_FloatRGBA, FClearValueBinding(FLinearColor(0, 0, 0, 0)),
			TexCreate_ShaderResource | TexCreate_RenderTargetable);

		ReflectionsOutputTexture = GraphBuilder.CreateTexture(Desc, TEXT("PlanarReflections"));
		bClearReflectionsOutputTexture = true;
	}

	FPlanarReflectionPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPlanarReflectionPS::FParameters>();
	PassParameters->SceneTextures.SceneDepthTexture = SceneTextures.SceneDepthTexture;
	PassParameters->SceneTextures.GBufferATexture = SceneTextures.GBufferATexture;
	PassParameters->SceneTextures.GBufferBTexture = SceneTextures.GBufferBTexture;

	PassParameters->SceneTextures.GBufferCTexture = SceneTextures.GBufferCTexture;
	PassParameters->SceneTextures.GBufferDTexture = SceneTextures.GBufferDTexture;
	PassParameters->SceneTextures.GBufferETexture = SceneTextures.GBufferETexture;
	PassParameters->SceneTextures.GBufferFTexture = SceneTextures.GBufferFTexture;
	PassParameters->SceneTextures.GBufferVelocityTexture = SceneTextures.GBufferVelocityTexture;

	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(
		ReflectionsOutputTexture, bClearReflectionsOutputTexture ? ERenderTargetLoadAction::EClear : ERenderTargetLoadAction::ELoad);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("CompositePlanarReflections"),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, &View, this](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		// Blend over previous reflections in the output target (SSR or planar reflections that have already been rendered)
		// Planar reflections win over SSR and reflection environment
		//@todo - this is order dependent blending, but ordering is coming from registration order
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Max, BF_One, BF_One>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		for (FPlanarReflectionSceneProxy* ReflectionSceneProxy : Scene->PlanarReflections)
		{
			if (!View.GetCullingFrustum().IntersectBox(ReflectionSceneProxy->WorldBounds.GetCenter(), ReflectionSceneProxy->WorldBounds.GetExtent()))
			{
				continue;
			}

			SCOPED_DRAW_EVENTF(RHICmdList, PlanarReflection, TEXT("PlanarReflection: %s"), ReflectionSceneProxy->OwnerName);

			FDeferredLightVS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FDeferredLightVS::FRadialLight>(false);
			TShaderMapRef<FDeferredLightVS> VertexShader(View.ShaderMap, PermutationVector);
			TShaderMapRef<FPlanarReflectionPS> PixelShader(View.ShaderMap);

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			FDeferredLightVS::FParameters ParametersVS = FDeferredLightVS::GetParameters(View);
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), ParametersVS);

			{
				FPlanarReflectionUniformParameters PlanarReflectionUniformParameters;
				SetupPlanarReflectionUniformParameters(View, ReflectionSceneProxy, PlanarReflectionUniformParameters);
		
				FPlanarReflectionPS::FParameters ShaderParameters = *PassParameters;
				ShaderParameters.PlanarReflectionParameters = CreateUniformBufferImmediate(PlanarReflectionUniformParameters, UniformBuffer_SingleDraw);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), ShaderParameters);
			}

			DrawRectangle(
				RHICmdList,
				0, 0,
				View.ViewRect.Width(), View.ViewRect.Height(),
				View.ViewRect.Min.X, View.ViewRect.Min.Y,
				View.ViewRect.Width(), View.ViewRect.Height(),
				View.ViewRect.Size(),
				View.GetSceneTexturesConfig().Extent,
				VertexShader,
				EDRF_UseTriangleOptimization);
		}
	});
}