// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
PlanarReflectionRendering.h: shared planar reflection rendering declarations.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "UniformBuffer.h"
#include "Matrix3x4.h"

class FShaderParameterMap;
class FSceneView;
class FMeshDrawSingleShaderBindings;
class FPlanarReflectionSceneProxy;

const int32 GPlanarReflectionUniformMaxReflectionViews = 2;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FPlanarReflectionUniformParameters, )
	SHADER_PARAMETER(FVector4f, ReflectionPlane)
	SHADER_PARAMETER(FVector4f, PlanarReflectionOrigin)
	SHADER_PARAMETER(FVector4f, PlanarReflectionXAxis)
	SHADER_PARAMETER(FVector4f, PlanarReflectionYAxis)
	SHADER_PARAMETER(FMatrix3x4, InverseTransposeMirrorMatrix)
	SHADER_PARAMETER(FVector3f, PlanarReflectionParameters)
	SHADER_PARAMETER(FVector2f, PlanarReflectionParameters2)
	SHADER_PARAMETER_ARRAY(FMatrix44f, ProjectionWithExtraFOV, [GPlanarReflectionUniformMaxReflectionViews])
	SHADER_PARAMETER_ARRAY(FVector4f, PlanarReflectionScreenScaleBias, [GPlanarReflectionUniformMaxReflectionViews])
	SHADER_PARAMETER(FVector2f, PlanarReflectionScreenBound)
	SHADER_PARAMETER(uint32, bIsStereo)
	SHADER_PARAMETER_TEXTURE(Texture2D, PlanarReflectionTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, PlanarReflectionSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

extern void SetupPlanarReflectionUniformParameters(const class FSceneView& View, const FPlanarReflectionSceneProxy* ReflectionSceneProxy, FPlanarReflectionUniformParameters& OutParameters);
