// Copyright Epic Games, Inc. All Rights Reserved.

#include "ACESUtils.h"

#include "GlobalRenderResources.h"
#include "RenderGraphBuilder.h"
#include "RenderResource.h"
#include "RHICommandList.h"
#include "RHIStaticStates.h"

namespace UE::Color::ACES
{
/**
* Source code adapted from OpenColorIO/src/OpenColorIO/ops/fixedfunction/ACES2 (v2.4.1)
*
* Copyright Contributors to the OpenColorIO Project.

Redistribution and use in source and binary forms, with or without
modification, are permitted provided that the following conditions are
met:

* Redistributions of source code must retain the above copyright
  notice, this list of conditions and the following disclaimer.
* Redistributions in binary form must reproduce the above copyright
  notice, this list of conditions and the following disclaimer in the
  documentation and/or other materials provided with the distribution.
* Neither the name of the copyright holder nor the names of its
  contributors may be used to endorse or promote products derived from
  this software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS
"AS IS" AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT
LIMITED TO, THE IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR
A PARTICULAR PURPOSE ARE DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT
HOLDER OR CONTRIBUTORS BE LIABLE FOR ANY DIRECT, INDIRECT, INCIDENTAL,
SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES (INCLUDING, BUT NOT
LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES; LOSS OF USE,
DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON ANY
THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

constexpr int32 TABLE_SIZE = 360;
constexpr int32 TABLE_ADDITION_ENTRIES = 2;
constexpr int32 TABLE_TotalSize = TABLE_SIZE + TABLE_ADDITION_ENTRIES;
constexpr int32 GAMUT_TABLE_BASE_INDEX = 1;

constexpr float ReferenceLuminance = 100.f;
constexpr float L_A = 100.f;
constexpr float Y_b = 20.f;
constexpr float AcResp = 1.f;
constexpr float Ra = 2.f * AcResp;
constexpr float Ba = 0.05f + (2.f - Ra);
constexpr float Surround[3] = { 0.9f, 0.59f, 0.9f }; // Dim surround

// Gamut compression
constexpr float SmoothCusps = 0.12f;
constexpr float SmoothM = 0.27f;
constexpr float CuspMidBlend = 1.3f;
constexpr float FocusGainBlend = 0.3f;
constexpr float FocusAdjustGain = 0.55f;
constexpr float FocusDistance = 1.35f;
constexpr float FocusDistanceScaling = 1.75f;
constexpr float CompressionThreshold = 0.75f;

// Table generation
constexpr float GammaMinimum = 0.0f;
constexpr float GammaMaximum = 5.0f;
constexpr float GammaSearchStep = 0.4f;
constexpr float GammaAccuracy = 1e-5f;

struct JMhParams
{
	float F_L;
	float z;
	float A_w;
	float A_w_J;
	FVector3f XYZ_w;
	FVector3f D_RGB;
	FMatrix44f MATRIX_RGB_to_CAM16;
	FMatrix44f MATRIX_CAM16_to_RGB;
};

struct ToneScaleParams
{
	float n;
	float n_r;
	float g;
	float t_1;
	float c_t;
	float s_2;
	float u_2;
	float m_2;
};

struct Table3D
{
	static constexpr int32 BaseIndex = GAMUT_TABLE_BASE_INDEX;
	static constexpr int32 Size = TABLE_SIZE;
	static constexpr int32 TotalSize = TABLE_TotalSize;
	FVector3f Data[TABLE_TotalSize];
};

struct Table1D
{
	static constexpr int32 BaseIndex = GAMUT_TABLE_BASE_INDEX;
	static constexpr int32 Size = TABLE_SIZE;
	static constexpr int32 TotalSize = TABLE_TotalSize;
	float Data[TABLE_TotalSize];
};

// Post adaptation non linear response compression
float PanlrcForward(float Value, float F_L)
{
	const float F_L_v = FMath::Pow(F_L * FMath::Abs(Value) / ReferenceLuminance, 0.42f);
	return (400.f * FMath::CopySign(1.f, Value) * F_L_v) / (27.13f + F_L_v);
}

float PanlrcInverse(float Value, float F_L)
{
	return FMath::CopySign(1.f, Value) * ReferenceLuminance / F_L * FMath::Pow((27.13f * FMath::Abs(Value) / (400.f - FMath::Abs(Value))), 1.f / 0.42f);
}

inline bool AnyBelowZero(const FVector3f& RGB)
{
	return (RGB[0] < 0. || RGB[1] < 0. || RGB[2] < 0.);
}

bool OutsideHull(const FVector3f& RGB)
{
	// limit value, once we cross this value, we are outside of the top gamut shell
	constexpr float MaxRGBtestVal = 1.0;
	return RGB[0] > MaxRGBtestVal || RGB[1] > MaxRGBtestVal || RGB[2] > MaxRGBtestVal;
}

// Optimization used during initialization
float Y_to_J(float Y, const JMhParams& Params)
{
	float F_L_Y = FMath::Pow(Params.F_L * FMath::Abs(Y) / ReferenceLuminance, 0.42f);
	return FMath::CopySign(1.f, Y) * ReferenceLuminance * FMath::Pow(((400.f * F_L_Y) / (27.13f + F_L_Y)) / Params.A_w_J, Surround[1] * Params.z);
}

float WrapTo360(float Hue)
{
	float Y = FMath::Fmod(Hue, 360.f);
	if (Y < 0.f)
	{
		Y = Y + 360.f;
	}

	return Y;
}

int32 HuePositionInUniformTable(float Hue, int32 TableSize)
{
	const float WrappedHue = WrapTo360(Hue);

	return int32(WrappedHue / 360.f * (float)TableSize);
}

int32 ClampToTableBounds(int32 Entry, int32 TableSize)
{
	return FMath::Min(TableSize - 1, FMath::Max(0, Entry));
}

float Smin(float a, float b, float s)
{
	const float h = FMath::Max(s - FMath::Abs(a - b), 0.f) / s;
	return FMath::Min(a, b) - h * h * h * s * (1.f / 6.f);
}

FVector3f JMh_to_RGB(const FVector3f& JMh, const JMhParams& Params)
{
	const float J = JMh[0];
	const float M = JMh[1];
	const float h = JMh[2];

	const float HRad = h * UE_PI / 180.f;

	const float Scale = M / (43.f * Surround[2]);
	const float A = Params.A_w * FMath::Pow(J / 100.f, 1.f / (Surround[1] * Params.z));
	const float a = Scale * FMath::Cos(HRad);
	const float b = Scale * FMath::Sin(HRad);

	const float RedA = (460.f * A + 451.f * a + 288.f * b) / 1403.f;
	const float GrnA = (460.f * A - 891.f * a - 261.f * b) / 1403.f;
	const float BluA = (460.f * A - 220.f * a - 6300.f * b) / 1403.f;

	FVector3f CamM;
	CamM.X = PanlrcInverse(RedA, Params.F_L) / Params.D_RGB[0];
	CamM.Y = PanlrcInverse(GrnA, Params.F_L) / Params.D_RGB[1];
	CamM.Z = PanlrcInverse(BluA, Params.F_L) / Params.D_RGB[2];

	return Params.MATRIX_CAM16_to_RGB.TransformVector(CamM);
}

JMhParams InitJMhParams(const FColorSpace& InColorSpace)
{
	static FColorSpace ColorSpaceCAM16 = FColorSpace(EColorSpace::ACESCAM16);

	const FVector XYZ_w = InColorSpace.GetRgbToXYZ().TransformVector(FVector::One() * ReferenceLuminance);
	const float Y_W = XYZ_w[1];
	const FVector RGB_w = ColorSpaceCAM16.GetXYZToRgb().TransformVector(XYZ_w);

	// Viewing condition dependent parameters
	const float K = 1.f / (5.f * L_A + 1.f);
	const float K4 = FMath::Pow(K, 4.f);
	const float N = Y_b / Y_W;
	const float F_L = 0.2f * K4 * (5.f * L_A) + 0.1f * FMath::Pow((1.f - K4), 2.f) * FMath::Pow(5.f * L_A, 1.f / 3.f);
	const float z = 1.48f + FMath::Sqrt(N);

	const FVector3f D_RGB = {
		Y_W / (float)RGB_w[0],
		Y_W / (float)RGB_w[1],
		Y_W / (float)RGB_w[2]
	};

	const FVector3f RGB_WC{
		D_RGB[0] * (float)RGB_w[0],
		D_RGB[1] * (float)RGB_w[1],
		D_RGB[2] * (float)RGB_w[2]
	};

	const FVector3f RGB_AW = {
		PanlrcForward(RGB_WC[0], F_L),
		PanlrcForward(RGB_WC[1], F_L),
		PanlrcForward(RGB_WC[2], F_L)
	};

	const float A_w = Ra * RGB_AW[0] + RGB_AW[1] + Ba * RGB_AW[2];
	const float F_L_W = FMath::Pow(F_L, 0.42f);
	const float A_w_J = (400.f * F_L_W) / (27.13f + F_L_W);

	JMhParams Params;
	Params.XYZ_w = FVector3f(FVector3d(XYZ_w));
	Params.F_L = F_L;
	Params.z = z;
	Params.D_RGB = D_RGB;
	Params.A_w = A_w;
	Params.A_w_J = A_w_J;

	FColorSpaceTransform ToCAM16T(InColorSpace, ColorSpaceCAM16, EChromaticAdaptationMethod::None);
	FMatrix ToCAM16 = ToCAM16T.ApplyScale(100.0);

	Params.MATRIX_RGB_to_CAM16 = FMatrix44f(ToCAM16);
	Params.MATRIX_CAM16_to_RGB = FMatrix44f(ToCAM16.Inverse());

	return Params;
}

ToneScaleParams InitToneScaleParams(float PeakLuminance)
{
	// Preset constants that set the desired behavior for the curve
	const float n = PeakLuminance;

	const float n_r = 100.0f;    // normalized white in nits (what 1.0 should be)
	const float g = 1.15f;       // surround / contrast
	const float c = 0.18f;       // anchor for 18% grey
	const float c_d = 10.013f;   // output luminance of 18% grey (in nits)
	const float w_g = 0.14f;     // change in grey between different peak luminance
	const float t_1 = 0.04f;     // shadow toe or flare/glare compensation
	const float r_hit_min = 128.f;   // scene-referred value "hitting the roof"
	const float r_hit_max = 896.f;   // scene-referred value "hitting the roof"

	// Calculate output constants
	const float r_hit = r_hit_min + (r_hit_max - r_hit_min) * (FMath::Loge(n / n_r) / FMath::Loge(10000.f / 100.f));
	const float m_0 = (n / n_r);
	const float m_1 = 0.5f * (m_0 + FMath::Sqrt(m_0 * (m_0 + 4.f * t_1)));
	const float u = FMath::Pow((r_hit / m_1) / ((r_hit / m_1) + 1.f), g);
	const float m = m_1 / u;
	const float w_i = FMath::Loge(n / 100.f) / FMath::Loge(2.f);
	const float c_t = c_d / n_r * (1.f + w_i * w_g);
	const float g_ip = 0.5f * (c_t + FMath::Sqrt(c_t * (c_t + 4.f * t_1)));
	const float g_ipp2 = -(m_1 * FMath::Pow((g_ip / m), (1.f / g))) / (FMath::Pow(g_ip / m, 1.f / g) - 1.f);
	const float w_2 = c / g_ipp2;
	const float s_2 = w_2 * m_1;
	const float u_2 = FMath::Pow((r_hit / m_1) / ((r_hit / m_1) + w_2), g);
	const float m_2 = m_1 / u_2;

	ToneScaleParams TonescaleParams = {
		n,
		n_r,
		g,
		t_1,
		c_t,
		s_2,
		u_2,
		m_2
	};

	return TonescaleParams;
}

Table1D MakeReachMTable(float PeakLuminance)
{
	static FColorSpace ColorSpaceAP1 = FColorSpace(EColorSpace::ACESAP1);

	const JMhParams Params = InitJMhParams(ColorSpaceAP1);
	const float LimitJMax = Y_to_J(PeakLuminance, Params);

	Table1D GamutReachTable{};

	for (int32 Index = 0; Index < GamutReachTable.Size; Index++)
	{
		const float Hue = (float)Index;
		const float SearchRange = 50.f;

		float Low = 0.;
		float High = Low + SearchRange;
		bool Outside = false;

		while ((Outside != true) && (High < 1300.f))
		{
			const FVector3f SearchJMh = FVector3f{ LimitJMax, High, Hue };
			const FVector3f NewLimitRGB = JMh_to_RGB(SearchJMh, Params);
			Outside = AnyBelowZero(NewLimitRGB);
			
			if (Outside == false)
			{
				Low = High;
				High = High + SearchRange;
			}
		}

		while (High - Low > 1e-2)
		{
			const float SampleM = (High + Low) / 2.f;
			const FVector3f SearchJMh = FVector3f{ LimitJMax, SampleM, Hue };
			const FVector3f NewLimitRGB = JMh_to_RGB(SearchJMh, Params);
			Outside = AnyBelowZero(NewLimitRGB);
			
			if (Outside)
			{
				High = SampleM;
			}
			else
			{
				Low = SampleM;
			}
		}

		GamutReachTable.Data[Index] = High;
	}

	return GamutReachTable;
}

FVector3f HSV_to_RGB(const FVector3f& HSV)
{
	const float C = HSV[2] * HSV[1];
	const float X = C * (1.f - FMath::Abs(FMath::Fmod(HSV[0] * 6.f, 2.f) - 1.f));
	const float m = HSV[2] - C;

	FVector3f RGB{};
	if (HSV[0] < 1.f / 6.f) {
		RGB = { C, X, 0.f };
	}
	else if (HSV[0] < 2. / 6.) {
		RGB = { X, C, 0.f };
	}
	else if (HSV[0] < 3. / 6.) {
		RGB = { 0.f, C, X };
	}
	else if (HSV[0] < 4. / 6.) {
		RGB = { 0.f, X, C };
	}
	else if (HSV[0] < 5. / 6.) {
		RGB = { X, 0.f, C };
	}
	else {
		RGB = { C, 0.f, X };
	}
	RGB = RGB + m;

	return RGB;
}

FVector3f RGB_to_JMh(const FVector3f& RGB, const JMhParams& Params)
{
	const FVector3f RGB_m = Params.MATRIX_RGB_to_CAM16.TransformVector(RGB);

	const float RedA = PanlrcForward(RGB_m[0] * Params.D_RGB[0], Params.F_L);
	const float GrnA = PanlrcForward(RGB_m[1] * Params.D_RGB[1], Params.F_L);
	const float BluA = PanlrcForward(RGB_m[2] * Params.D_RGB[2], Params.F_L);

	const float A = 2.f * RedA + GrnA + 0.05f * BluA;
	const float a = RedA - 12.f * GrnA / 11.f + BluA / 11.f;
	const float b = (RedA + GrnA - 2.f * BluA) / 9.f;

	const float J = 100.f * FMath::Pow(A / Params.A_w, Surround[1] * Params.z);

	const float M = J == 0.f ? 0.f : 43.f * Surround[2] * FMath::Sqrt(a * a + b * b);

	const float h_rad = FMath::Atan2(b, a);
	float h = FMath::Fmod(h_rad * 180.f / UE_PI, 360.f);
	if (h < 0.f)
	{
		h += 360.f;
	}

	return { J, M, h };
}

Table3D MakeGamutTable(const FColorSpace& InLimitingColorSpace, float PeakLuminance)
{
	const JMhParams Params = InitJMhParams(InLimitingColorSpace);

	Table3D GamutCuspTableUnsorted{};
	for (int32 i = 0; i < GamutCuspTableUnsorted.Size; i++)
	{
		const float hNorm = (float)i / GamutCuspTableUnsorted.Size;
		const FVector3f HSV = { hNorm, 1., 1. };
		const FVector3f RGB = HSV_to_RGB(HSV);
		const FVector3f ScaledRGB = (PeakLuminance / ReferenceLuminance) * RGB;
		const FVector3f JMh = RGB_to_JMh(ScaledRGB, Params);

		GamutCuspTableUnsorted.Data[i] = JMh;
	}

	int32 MinhIndex = 0;
	for (int32 i = 0; i < GamutCuspTableUnsorted.Size; i++)
	{
		if (GamutCuspTableUnsorted.Data[i][2] < GamutCuspTableUnsorted.Data[MinhIndex][2])
			MinhIndex = i;
	}

	Table3D GamutCuspTable{};
	for (int32 i = 0; i < GamutCuspTableUnsorted.Size; i++)
	{
		GamutCuspTable.Data[i + GamutCuspTable.BaseIndex] = GamutCuspTableUnsorted.Data[(MinhIndex + i) % GamutCuspTableUnsorted.Size];
	}

	// Copy last populated entry to first empty spot
	GamutCuspTable.Data[0] = GamutCuspTable.Data[GamutCuspTable.BaseIndex + GamutCuspTable.Size - 1];

	// Copy first populated entry to last empty spot
	GamutCuspTable.Data[GamutCuspTable.BaseIndex + GamutCuspTable.Size] = GamutCuspTable.Data[GamutCuspTable.BaseIndex];

	// Wrap the hues, to maintain monotonicity. These entries will fall outside [0.0, 360.0]
	GamutCuspTable.Data[0][2] = GamutCuspTable.Data[0][2] - 360.f;
	GamutCuspTable.Data[GamutCuspTable.Size + 1][2] = GamutCuspTable.Data[GamutCuspTable.Size + 1][2] + 360.f;

	return GamutCuspTable;
}

FVector2f CuspToTable(float h, const Table3D& Gt)
{
	int32 Idx_lo = 0;
	int32 Idx_hi = Gt.BaseIndex + Gt.Size; // allowed as we have an extra entry in the table
	int32 Idx = ClampToTableBounds(HuePositionInUniformTable(h, Gt.Size) + Gt.BaseIndex, Gt.TotalSize);

	while (Idx_lo + 1 < Idx_hi)
	{
		if (h > Gt.Data[Idx][2])
		{
			Idx_lo = Idx;
		}
		else
		{
			Idx_hi = Idx;
		}
		
		Idx = ClampToTableBounds((Idx_lo + Idx_hi) / 2, Gt.TotalSize);
	}

	Idx_hi = FMath::Max(1, Idx_hi);

	const FVector3f Lo{
		Gt.Data[Idx_hi - 1][0],
		Gt.Data[Idx_hi - 1][1],
		Gt.Data[Idx_hi - 1][2]
	};

	const FVector3f Hi{
		Gt.Data[Idx_hi][0],
		Gt.Data[Idx_hi][1],
		Gt.Data[Idx_hi][2]
	};

	const float t = (h - Lo[2]) / (Hi[2] - Lo[2]);
	const float CuspJ = FMath::Lerp(Lo[0], Hi[0], t);
	const float CuspM = FMath::Lerp(Lo[1], Hi[1], t);

	return FVector2f{ CuspJ, CuspM };
}

float GetFocusGain(float J, float CuspJ, float LimitJMax)
{
	const float Thr = FMath::Lerp(CuspJ, LimitJMax, FocusGainBlend);

	if (J > Thr)
	{
		// Approximate inverse required above threshold
		float Gain = (LimitJMax - Thr) / FMath::Max(0.0001f, (LimitJMax - FMath::Min(LimitJMax, J)));
		return FMath::Pow(log10(Gain), 1.f / FocusAdjustGain) + 1.f;
	}
	else
	{
		// Analytic inverse possible below cusp
		return 1.f;
	}
}

float SolveJIntersect(float J, float M, float FocusJ, float MaxJ, float SlopeGain)
{
	const float a = M / (FocusJ * SlopeGain);
	float b = 0.f;
	float c = 0.f;
	float IntersectJ = 0.f;

	if (J < FocusJ)
	{
		b = 1.f - M / SlopeGain;
	}
	else
	{
		b = -(1.f + M / SlopeGain + MaxJ * M / (FocusJ * SlopeGain));
	}

	if (J < FocusJ)
	{
		c = -J;
	}
	else
	{
		c = MaxJ * M / SlopeGain + J;
	}

	const float Root = FMath::Sqrt(b * b - 4.f * a * c);

	if (J < FocusJ)
	{
		IntersectJ = 2.f * c / (-b - Root);
	}
	else
	{
		IntersectJ = 2.f * c / (-b + Root);
	}

	return IntersectJ;
}

FVector3f FindGamutBoundaryIntersection(const FVector3f& JMh_s, const FVector2f& JM_cusp_in, float J_focus, float J_max, float SlopeGain, float gamma_top, float gamma_bottom)
{
	constexpr float s = FMath::Max(0.000001f, SmoothCusps);
	const FVector2f JM_cusp = {
		JM_cusp_in[0],
		JM_cusp_in[1] * (1.f + SmoothM * s)
	};

	const float J_intersect_source = SolveJIntersect(JMh_s[0], JMh_s[1], J_focus, J_max, SlopeGain);
	const float J_intersect_cusp = SolveJIntersect(JM_cusp[0], JM_cusp[1], J_focus, J_max, SlopeGain);

	float Slope = 0.f;
	if (J_intersect_source < J_focus)
	{
		Slope = J_intersect_source * (J_intersect_source - J_focus) / (J_focus * SlopeGain);
	}
	else
	{
		Slope = (J_max - J_intersect_source) * (J_intersect_source - J_focus) / (J_focus * SlopeGain);
	}

	const float M_boundary_lower = J_intersect_cusp * FMath::Pow(J_intersect_source / J_intersect_cusp, 1.f / gamma_bottom) / (JM_cusp[0] / JM_cusp[1] - Slope);
	const float M_boundary_upper = JM_cusp[1] * (J_max - J_intersect_cusp) * FMath::Pow((J_max - J_intersect_source) / (J_max - J_intersect_cusp), 1.f / gamma_top) / (Slope * JM_cusp[1] + J_max - JM_cusp[0]);
	const float M_boundary = JM_cusp[1] * Smin(M_boundary_lower / JM_cusp[1], M_boundary_upper / JM_cusp[1], s);
	const float J_boundary = J_intersect_source + Slope * M_boundary;

	return { J_boundary, M_boundary, J_intersect_source };
}

bool EvaluateGammaFit(
	const FVector2f& JMcusp,
	const FVector3f TestJMh[3],
	float TopGamma,
	float PeakLuminance,
	float LimitJMax,
	float Mid_J,
	float FocusDist,
	float LowerHullGamma,
	const JMhParams& LimitJMhParams)
{
	const float FocusJ = FMath::Lerp(JMcusp[0], Mid_J, FMath::Min(1.f, CuspMidBlend - (JMcusp[0] / LimitJMax)));

	for (size_t TestIndex = 0; TestIndex < 3; TestIndex++)
	{
		const float SlopeGain = LimitJMax * FocusDist * GetFocusGain(TestJMh[TestIndex][0], JMcusp[0], LimitJMax);
		const FVector3f ApproxLimit = FindGamutBoundaryIntersection(TestJMh[TestIndex], JMcusp, FocusJ, LimitJMax, SlopeGain, TopGamma, LowerHullGamma);
		const FVector3f Approximate_JMh = { ApproxLimit[0], ApproxLimit[1], TestJMh[TestIndex][2] };
		const FVector3f NewLimitRGB = JMh_to_RGB(Approximate_JMh, LimitJMhParams);
		const FVector3f NewLimitRGBScaled = (ReferenceLuminance / PeakLuminance) * NewLimitRGB;

		if (!OutsideHull(NewLimitRGBScaled))
		{
			return false;
		}
	}

	return true;
}

Table1D MakeUpperHullGamma(
	const Table3D& GamutCuspTable,
	float PeakLuminance,
	float LimitJMax,
	float Mid_J,
	float FocusDist,
	float LowerHullGamma,
	const JMhParams& LimitJMhParams)
{
	const int32 TestCount = 3;
	const float TestPositions[TestCount] = { 0.01f, 0.5f, 0.99f };

	Table1D GammaTable{};
	Table1D GamutTopGamma{};

	for (int32 Index = 0; Index < GammaTable.Size; Index++)
	{
		GammaTable.Data[Index] = -1.f;

		const float Hue = (float)Index;
		const FVector2f JMcusp = CuspToTable(Hue, GamutCuspTable);

		FVector3f TestJMh[TestCount]{};
		for (int32 TestIndex = 0; TestIndex < TestCount; TestIndex++)
		{
			const float TestJ = JMcusp[0] + ((LimitJMax - JMcusp[0]) * TestPositions[TestIndex]);
			TestJMh[TestIndex] = {
				TestJ,
				JMcusp[1],
				Hue
			};
		}

		const float SearchRange = GammaSearchStep;
		float Low = GammaMinimum;
		float High = Low + SearchRange;
		bool Outside = false;

		while (!(Outside) && (High < 5.f))
		{
			const bool bGammaFound = EvaluateGammaFit(JMcusp, TestJMh, High, PeakLuminance, LimitJMax, Mid_J, FocusDist, LowerHullGamma, LimitJMhParams);
			if (!bGammaFound)
			{
				Low = High;
				High = High + SearchRange;
			}
			else
			{
				Outside = true;
			}
		}

		float TestGamma = -1.f;
		while ((High - Low) > GammaAccuracy)
		{
			TestGamma = (High + Low) / 2.f;
			const bool bGammaFound = EvaluateGammaFit(JMcusp, TestJMh, TestGamma, PeakLuminance, LimitJMax, Mid_J, FocusDist, LowerHullGamma, LimitJMhParams);
			if (bGammaFound)
			{
				High = TestGamma;
				GammaTable.Data[Index] = High;
			}
			else
			{
				Low = TestGamma;
			}
		}

		// Duplicate gamma value to array, leaving empty entries at first and last position
		GamutTopGamma.Data[Index + GamutTopGamma.BaseIndex] = GammaTable.Data[Index];
	}

	// Copy last populated entry to first empty spot
	GamutTopGamma.Data[0] = GammaTable.Data[GammaTable.Size - 1];

	// Copy first populated entry to last empty spot
	GamutTopGamma.Data[GamutTopGamma.TotalSize - 1] = GammaTable.Data[0];

	return GamutTopGamma;
}

/* Base class for ACES 2.0 table texture resources. */
class FTextureLookupBase : public FTextureWithSRV
{
public:
	/* Constructor */
	FTextureLookupBase(const FString& InDebugName)
		: FTextureWithSRV()
		, DebugName(InDebugName)
	{
	}
	
	/* Destructor */
	virtual ~FTextureLookupBase() = default;

	virtual uint32 GetSizeY() const override
	{
		return 1;
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create2D(*DebugName, GetSizeX(), GetSizeY(), GetPixelFormat())
			.SetFlags(ETextureCreateFlags::ShaderResource);

		// Create the RHI texture
		TextureRHI = RHICmdList.CreateTexture(Desc);

		// Create the sampler state RHI resource.
		FSamplerStateInitializerRHI SamplerStateInitializer(SF_Point, AM_Clamp, AM_Clamp, AM_Clamp);
		SamplerStateRHI = GetOrCreateSamplerState(SamplerStateInitializer);

		// Create a view of the texture
		ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(TextureRHI, FRHIViewDesc::CreateTextureSRV().SetDimensionFromTexture(TextureRHI));
	}

	/* Update the texture table resource given a display-limiting color space & a peak luminance. */
	virtual void Update(FRHICommandListImmediate& RHICmdList, const FColorSpace& InLimitingColorSpace, float InPeakLuminance) = 0;

	/* Get the texture resource pixel format. */
	virtual EPixelFormat GetPixelFormat() const = 0;

private:
	/* Resource debug name */
	FString DebugName;
};

/* ACES 2.0 reach M value table. */
class FReachMTable : public FTextureLookupBase
{
public:
	FReachMTable()
		: FTextureLookupBase(TEXT("ACES_ReachMTable"))
	{
	}

	virtual ~FReachMTable() = default;

	virtual void Update(FRHICommandListImmediate& RHICmdList, const FColorSpace& InLimitingColorSpace, float InPeakLuminance) override
	{
		Table1D ReachTableData = MakeReachMTable(InPeakLuminance);
		check(GetSizeX() == ReachTableData.Size);
		const uint32 DataSize = GetSizeX() * sizeof(float);
		const FUpdateTextureRegion2D Region(0, 0, 0, 0, GetSizeX(), GetSizeY());
		
		RHICmdList.UpdateTexture2D(TextureRHI, 0, Region, DataSize, (const uint8*)ReachTableData.Data);
	}

	virtual uint32 GetSizeX() const override
	{
		return TABLE_SIZE;
	}

	virtual EPixelFormat GetPixelFormat() const override
	{
		return PF_R32_FLOAT;
	}
};

/* ACES 2.0 gamut cusp table. */
class FGamutCuspTable : public FTextureLookupBase
{
public:
	FGamutCuspTable()
		: FTextureLookupBase(TEXT("ACES_GamutCuspTable"))
	{
	}

	virtual ~FGamutCuspTable() = default;

	virtual void Update(FRHICommandListImmediate& RHICmdList, const FColorSpace& InLimitingColorSpace, float InPeakLuminance) override
	{
		Table3D GamutCuspTable = MakeGamutTable(InLimitingColorSpace, InPeakLuminance);
		check(GetSizeY() == GamutCuspTable.TotalSize);
		const FUpdateTextureRegion2D Region(0, 0, 0, 0, GetSizeX(), GetSizeY());

		RHICmdList.UpdateTexture2D(TextureRHI, 0, Region, sizeof(FVector3f), (const uint8*)GamutCuspTable.Data);
	}

	virtual uint32 GetSizeX() const override
	{
		// Store JMh channels to separate texels.
		return 3;
	}

	virtual uint32 GetSizeY() const override
	{
		return TABLE_TotalSize;
	}

	virtual EPixelFormat GetPixelFormat() const override
	{
		return PF_R32_FLOAT;
	}
};

/* ACES 2.0 upper hull gamma. */
class FUpperHullGammaTable : public FTextureLookupBase
{
public:
	FUpperHullGammaTable()
		: FTextureLookupBase(TEXT("ACES_UpperHullGammaTable"))
	{
	}

	virtual ~FUpperHullGammaTable() = default;

	virtual void Update(FRHICommandListImmediate& RHICmdList, const FColorSpace& InLimitingColorSpace, float InPeakLuminance) override
	{
		static const FColorSpace ColorSpaceAP0 = FColorSpace(EColorSpace::ACESAP0);

		Table3D GamutCuspTable = MakeGamutTable(InLimitingColorSpace, InPeakLuminance);

		const ToneScaleParams TsParams = InitToneScaleParams(InPeakLuminance);
		const JMhParams InputJMhParams = InitJMhParams(ColorSpaceAP0);

		float LimitJMax = Y_to_J(InPeakLuminance, InputJMhParams);
		float Mid_J = Y_to_J(TsParams.c_t * 100.f, InputJMhParams);

		// Calculated chroma compress variables
		const float LogPeak = log10(TsParams.n / TsParams.n_r);
		const float ModelGamma = 1.f / (Surround[1] * (1.48f + sqrt(Y_b / L_A)));
		const float FocusDist = FocusDistance + FocusDistance * FocusDistanceScaling * LogPeak;
		const float LowerHullGamma = 1.14f + 0.07f * LogPeak;

		const JMhParams LimitJMhParams = InitJMhParams(InLimitingColorSpace);

		Table1D UpperHullGammaTable = MakeUpperHullGamma(
			GamutCuspTable,
			InPeakLuminance,
			LimitJMax,
			Mid_J,
			FocusDist,
			LowerHullGamma,
			LimitJMhParams);

		check(GetSizeX() == UpperHullGammaTable.TotalSize);
		const uint32 DataSize = GetSizeX() * sizeof(float);
		const FUpdateTextureRegion2D Region(0, 0, 0, 0, GetSizeX(), GetSizeY());

		RHICmdList.UpdateTexture2D(TextureRHI, 0, Region, DataSize, (const uint8*)UpperHullGammaTable.Data);
	}

	virtual uint32 GetSizeX() const override
	{
		return TABLE_TotalSize;
	}
	
	virtual EPixelFormat GetPixelFormat() const override
	{
		return PF_R32_FLOAT;
	}
};

struct AcesContext
{
	FTextureLookupBase* ReachMTable = new TGlobalResource<FReachMTable>;
	FTextureLookupBase* GamutCuspTable = new TGlobalResource<FGamutCuspTable>;
	FTextureLookupBase* UpperHullGammaTable = new TGlobalResource<FUpperHullGammaTable>;
	float CachedPeakLuminance = 0.0f;
	UE::Color::FColorSpace CachedLimitingColorSpace;
};

static void GetTransformResourcesInternal(
	FRDGBuilder& GraphBuilder,
	float InPeakLuminance,
	const UE::Color::FColorSpace& LimitingColorSpace,
	AcesContext& Context,
	FRHIShaderResourceView*& OutReachMTable,
	FRHIShaderResourceView*& OutGamutCuspTable,
	FRHIShaderResourceView*& OutUpperHullGammaTable)
{
	static const auto CVarAcesVersion = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.Aces.Version"));
	
	if (CVarAcesVersion->GetValueOnRenderThread() > 1)
	{
		// ACES 2.0

		if (!FMath::IsNearlyEqual(Context.CachedPeakLuminance, InPeakLuminance) ||
			(Context.CachedLimitingColorSpace != LimitingColorSpace))
		{
			FRHICommandListImmediate& RHICmdListImmediate = GraphBuilder.RHICmdList.GetAsImmediate();

			Context.ReachMTable->Update(RHICmdListImmediate, LimitingColorSpace, InPeakLuminance);
			Context.GamutCuspTable->Update(RHICmdListImmediate, LimitingColorSpace, InPeakLuminance);
			Context.UpperHullGammaTable->Update(RHICmdListImmediate, LimitingColorSpace, InPeakLuminance);

			// Update static cached info.
			Context.CachedPeakLuminance = InPeakLuminance;
			Context.CachedLimitingColorSpace = LimitingColorSpace;
		}

		OutReachMTable = Context.ReachMTable->ShaderResourceViewRHI;
		OutGamutCuspTable = Context.GamutCuspTable->ShaderResourceViewRHI;
		OutUpperHullGammaTable = Context.UpperHullGammaTable->ShaderResourceViewRHI;
	}
	else
	{
		// ACES 1.3

		OutReachMTable = GBlackTextureWithSRV->ShaderResourceViewRHI;
		OutGamutCuspTable = GBlackTextureWithSRV->ShaderResourceViewRHI;
		OutUpperHullGammaTable = GBlackTextureWithSRV->ShaderResourceViewRHI;
	}
}

void GetTransformResources(FRDGBuilder& GraphBuilder, float InPeakLuminance, const UE::Color::FColorSpace& LimitingColorSpace, FRHIShaderResourceView*& OutReachMTable, FRHIShaderResourceView*& OutGamutCuspTable, FRHIShaderResourceView*& OutUpperHullGammaTable)
{
	static AcesContext Context;
	GetTransformResourcesInternal(GraphBuilder, InPeakLuminance, LimitingColorSpace, Context, OutReachMTable, OutGamutCuspTable, OutUpperHullGammaTable);
}

void GetTransformResourcesSDR(FRDGBuilder& GraphBuilder, FRHIShaderResourceView*& OutReachMTable, FRHIShaderResourceView*& OutGamutCuspTable, FRHIShaderResourceView*& OutUpperHullGammaTable)
{
	static AcesContext Context;
	GetTransformResourcesInternal(GraphBuilder, 100.f, UE::Color::FColorSpace::GetWorking(), Context, OutReachMTable, OutGamutCuspTable, OutUpperHullGammaTable);
}

}  // namespace UE::Color::ACES

