// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIFwd.h"
#include "ColorManagement/ColorSpace.h"

class FRDGBuilder;

namespace UE::Color::ACES
{
	/*
	* Get the ACES 2.0 transform table resources.
	* 
	* @param GraphBuilder			Frame render dependency graph.
	* @param InPeakLuminance		Peak luminance value in nits (cd/m^2).
	* @param LimitingColorSpace		Color space of the target display.
	* @param OutReachMTable			Reach M value table resource output.
	* @param OutGamutCuspTable		Gamut cusp table resource output.
	* @param OutUpperHullGammaTable	Upper hull gamma table resource output.
	*/
	void GetTransformResources(
		FRDGBuilder& GraphBuilder,
		float InPeakLuminance,
		const UE::Color::FColorSpace& LimitingColorSpace,
		FRHIShaderResourceView*& OutReachMTable,
		FRHIShaderResourceView*& OutGamutCuspTable,
		FRHIShaderResourceView*& OutUpperHullGammaTable
	);

	/*
	* Get the ACES 2.0 transform table resources.
	*
	* @param GraphBuilder			Frame render dependency graph.
	* @param InPeakLuminance		Peak luminance value in nits (cd/m^2).
	* @param LimitingColorSpace		Color space of the target display.
	* @param OutReachMTable			Reach M value table resource output.
	* @param OutGamutCuspTable		Gamut cusp table resource output.
	* @param OutUpperHullGammaTable	Upper hull gamma table resource output.
	*/
	void GetTransformResourcesSDR(
		FRDGBuilder& GraphBuilder,
		FRHIShaderResourceView*& OutReachMTable,
		FRHIShaderResourceView*& OutGamutCuspTable,
		FRHIShaderResourceView*& OutUpperHullGammaTable
	);
} // namespace UE::Color::ACES
