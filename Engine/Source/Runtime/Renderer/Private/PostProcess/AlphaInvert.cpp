// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AlphaInvert.cpp: Inverts alpha channel in a render pass.
=============================================================================*/

#include "AlphaInvert.h"
#include "PostProcess/SceneFilterRendering.h"
#include "ScreenRendering.h"
#include "SceneRendering.h"

namespace AlphaInvert
{
	static TAutoConsoleVariable<bool> CVarAlphaInvertPass(
		TEXT("r.AlphaInvertPass"),
		0,
		TEXT("Whether to run a render pass to un-invert the alpha value from unreal standard to the much more common standard where alpha 0 is fully transparent and alpha 1 is fully opaque.")
		TEXT("This cvar attempts to affect all renders, not only the main view.")
		TEXT("If your project does multiple renders which do not all need alpha inverted it would be more performant to find or implement a narrower version of it for your specific purpose (eg OpenXR.AlphaInvertPass)."),
		ECVF_RenderThreadSafe);

	class FAlphaInvertPS : public FScreenPS
	{
		DECLARE_SHADER_TYPE(FAlphaInvertPS, Global);
	public:
		FAlphaInvertPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
			FScreenPS(Initializer)
		{
		}
		FAlphaInvertPS() {}
	};
	IMPLEMENT_SHADER_TYPE(, FAlphaInvertPS, TEXT("/Engine/Private/PostProcessAlphaInvert.usf"), TEXT("AlphaInvert_MainPS"), SF_Pixel);

	void RenderAlphaInvertPass(FRHICommandList& RHICmdList, const FViewInfo& View, const FScreenPassTexture& Color)
	{
		// Part of scene rendering pass
		check(RHICmdList.IsInsideRenderPass());
		SCOPED_DRAW_EVENT(RHICmdList, AlphaInvert);

		const FIntPoint OutputTextureRect = Color.Texture->Desc.Extent;
		TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FAlphaInvertPS> PixelShader(View.ShaderMap);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		const FIntRect OutputViewRect(Color.ViewRect);

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
		SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI(), Color.Texture->GetRHI());
		RHICmdList.SetViewport(OutputViewRect.Min.X, OutputViewRect.Min.Y, 0.0f, OutputViewRect.Max.X, OutputViewRect.Max.Y, 1.0f);

		DrawRectangle(
			RHICmdList,
			// Output rect, relative to rhi viewport.
			0, 0, OutputViewRect.Width(), OutputViewRect.Height(),
			// Input rect, relative to the input texture
			OutputViewRect.Min.X, OutputViewRect.Min.Y, OutputViewRect.Width(), OutputViewRect.Height(),
			OutputViewRect.Size(),
			OutputTextureRect,
			VertexShader,
			EDRF_UseTriangleOptimization,
			1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FAlphaInvertParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
}

FScreenPassTexture AlphaInvert::AddAlphaInvertPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const struct FAlphaInvertInputs& Inputs)
{
	FAlphaInvertParameters* PassParameters = GraphBuilder.AllocParameters<FAlphaInvertParameters>();
	PassParameters->View = View.GetShaderParameters();
	PassParameters->ColorTexture = Inputs.SceneColor.Texture;

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, ERenderTargetLoadAction::ELoad, TEXT("AlphaInvert"));
	}

	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	FScreenPassTexture Color = Inputs.SceneColor;

	// We need to make sure that both input and output have alpha channels if not the pass is useless.
	const EPixelFormat InputPixelFormat = Color.Texture->Desc.Format;
	const bool bInputHasAlphaChannel = (GetPixelFormatValidChannels(InputPixelFormat) & EPixelFormatChannelFlags::A) != EPixelFormatChannelFlags::None;
	const EPixelFormat OutputPixelFormat = Output.Texture->Desc.Format;
	const bool bOutputHasAlphaChannel = (GetPixelFormatValidChannels(OutputPixelFormat) & EPixelFormatChannelFlags::A) != EPixelFormatChannelFlags::None;

	if (!(bInputHasAlphaChannel && bOutputHasAlphaChannel))
	{
		return Inputs.SceneColor;
	}

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("AlphaInvertPass"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, Color](FRHICommandListImmediate& RHICmdList)
		{
			AlphaInvert::RenderAlphaInvertPass(RHICmdList, View, Color);
		});

	return Inputs.SceneColor;
}

namespace AlphaInvert
{
	// In this version we will copy the entire input to the entire output, ignoring the view rect
	// This would do one pass instead of two when stereo rendering.
	void RenderAlphaInvertPassFullTexture(FRHICommandList& RHICmdList, const FViewInfo& View, FRDGTextureRef Color)
	{
		// Part of scene rendering pass
		check(RHICmdList.IsInsideRenderPass());
		SCOPED_DRAW_EVENT(RHICmdList, AlphaInvert);

		const FIntPoint TargetSize = Color->Desc.Extent;

		TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FAlphaInvertPS> PixelShader(View.ShaderMap);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
		SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI(), Color->GetRHI());
		RHICmdList.SetViewport(0, 0, 0.0f, TargetSize.X, TargetSize.Y, 1.0f);

		DrawRectangle(
			RHICmdList,
			0, 0, TargetSize.X, TargetSize.Y,
			0, 0, TargetSize.X, TargetSize.Y,
			TargetSize,
			TargetSize,
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	}
}

void AlphaInvert::AddAlphaInvertPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSceneTextures& SceneTextures)
{
	FAlphaInvertParameters* PassParameters = GraphBuilder.AllocParameters<FAlphaInvertParameters>();
	PassParameters->View = View.GetShaderParameters();
	PassParameters->ColorTexture = SceneTextures.Color.Resolve;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Resolve, ERenderTargetLoadAction::ELoad);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("AlphaInvertPass"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, &SceneTextures](FRHICommandListImmediate& RHICmdList)
		{
			AlphaInvert::RenderAlphaInvertPassFullTexture(RHICmdList, View, SceneTextures.Color.Resolve);
		});
}