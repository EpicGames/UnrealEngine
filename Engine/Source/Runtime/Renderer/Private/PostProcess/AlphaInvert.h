// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

namespace AlphaInvert
{

	// Pass per view version, used in the desktop renderer.
	struct FAlphaInvertInputs
	{
		// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
		FScreenPassRenderTarget OverrideOutput;

		// [Required] The input scene color and view rect.
		FScreenPassTexture SceneColor;
	};
	FScreenPassTexture AddAlphaInvertPass(class FRDGBuilder& GraphBuilder, const class FViewInfo& View, const struct FAlphaInvertInputs& Inputs);
	
	// Single pass version, used in the mobile renderer.
	void AddAlphaInvertPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSceneTextures& SceneTextures);
}
