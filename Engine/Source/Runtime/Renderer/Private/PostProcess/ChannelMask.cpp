// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChannelMask.h"

#include "SceneRendering.h"
#include "PostProcess/PostProcessMaterialInputs.h"

namespace ChannelMask
{
	BEGIN_SHADER_PARAMETER_STRUCT(FChannelMaskParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER(FVector4f, ChannelMask)
		SHADER_PARAMETER(uint32, bMaskChannels)
		SHADER_PARAMETER(uint32, bInvertAlphaChannelMask)
		SHADER_PARAMETER(uint32, bDisplayCheckerboard)
		SHADER_PARAMETER(FScreenTransform, ScreenPosToViewPos)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
	
	class FChannelMaskPS : public FGlobalShader
	{
	public:
		DECLARE_SHADER_TYPE(FChannelMaskPS, Global);
		SHADER_USE_PARAMETER_STRUCT(FChannelMaskPS, FGlobalShader);
		using FParameters = FChannelMaskParameters;
	};
	IMPLEMENT_SHADER_TYPE(, FChannelMaskPS, TEXT("/Engine/Private/PostProcessChannelMask.usf"), TEXT("ChannelMask_MainPS"), SF_Pixel);
}

FScreenPassTexture ChannelMask::AddChannelMaskPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FChannelMaskInputs& Inputs)
{
	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, ERenderTargetLoadAction::ELoad, TEXT("ChannelMask"));
	}

	const FScreenPassTextureViewport OutputViewport(Output);
	const FScreenPassTextureViewport ColorViewport(Inputs.SceneColor);

	const bool bMaskChannels = View.Family->ChannelMaskParams.ColorChannelMask != EColorChannelMask::All;
	FVector4f ChannelMask = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
	if (bMaskChannels)
	{
		ChannelMask[(int32)View.Family->ChannelMaskParams.ColorChannelMask] = 1.0f;
	}
	
	FChannelMaskParameters* PassParameters = GraphBuilder.AllocParameters<FChannelMaskParameters>();
	PassParameters->View = View.GetShaderParameters();
	PassParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters->SceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Border>::GetRHI();
	PassParameters->ChannelMask = ChannelMask;
	PassParameters->bMaskChannels = bMaskChannels;
	PassParameters->bInvertAlphaChannelMask = View.Family->ChannelMaskParams.bInvertAlphaChannelMask;
	PassParameters->bDisplayCheckerboard = View.Family->ChannelMaskParams.bDrawAlphaBlendedCheckerboard;
	PassParameters->ScreenPosToViewPos = (
		FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::ScreenPosition, FScreenTransform::ETextureBasis::ViewportUV) *
		FScreenTransform::ChangeTextureBasisFromTo(View.ViewRect.Size(), View.ViewRect, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TexelPosition));
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	
	TShaderMapRef<FChannelMaskPS> PixelShader(View.ShaderMap);

	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("ChannelMask"),
		View,
		OutputViewport,
		ColorViewport,
		PixelShader,
		PassParameters);
	
	return MoveTemp(Output);
}
