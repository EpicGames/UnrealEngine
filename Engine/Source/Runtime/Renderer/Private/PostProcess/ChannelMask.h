// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

namespace ChannelMask
{
	struct FChannelMaskInputs
	{
		// Input scene color
		FScreenPassTexture SceneColor;

		// Optional output render target, if not provided a new render target is created and returned
		FScreenPassRenderTarget OverrideOutput;
	};
	
	FScreenPassTexture AddChannelMaskPass(class FRDGBuilder& GraphBuilder, const class FViewInfo& View, const struct FChannelMaskInputs& Inputs);
}
