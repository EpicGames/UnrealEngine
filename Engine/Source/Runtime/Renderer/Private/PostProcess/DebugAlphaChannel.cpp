// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/DebugAlphaChannel.h"
#include "ScreenPass.h"
#include "PixelShaderUtils.h"
#include "SceneTextures.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"

#if DEBUG_ALPHA_CHANNEL

TAutoConsoleVariable<float> CVarTestAlphaOpaqueWorldDistance(
	TEXT("r.Test.Aplha.OpaqueWorldDistance"), 0.0f,
	TEXT("Sets the world distance beyond which the opaque pixel are lerped to translucent for testing purposes."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarTestAlphaOpaqueLerpWorldRange(
	TEXT("r.Test.Aplha.OpaqueLerpWorldRange"), 100.0f,
	TEXT("Sets the gradient length in world unit on which opaque pixel are lerped to translucent for testing purposes."),
	ECVF_RenderThreadSafe);

#endif // DEBUG_ALPHA_CHANNEL

class FDebugAlphaChannelCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDebugAlphaChannelCS);
	SHADER_USE_PARAMETER_STRUCT(FDebugAlphaChannelCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(int32, DebugModeId)
		SHADER_PARAMETER(float, OpaqueWorldDistance)
		SHADER_PARAMETER(float, OpaqueLerpWorldRange)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, SceneColorOutput)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
}; // class FDebugAlphaChannelCS

IMPLEMENT_GLOBAL_SHADER(FDebugAlphaChannelCS, "/Engine/Private/Tools/DebugAlphaChannel.usf", "MainCS", SF_Compute);

#if DEBUG_ALPHA_CHANNEL

bool ShouldMakeDistantGeometryTranslucent()
{
	return CVarTestAlphaOpaqueWorldDistance.GetValueOnRenderThread() > 0.0f;
}

FRDGTextureMSAA MakeDistanceGeometryTranslucent(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	FMinimalSceneTextures SceneTextures)
{
	FRDGTextureDesc Desc = SceneTextures.Color.Resolve->Desc;
	Desc.Flags |= TexCreate_UAV;

	FRDGTexture* NewSceneColorTexture = GraphBuilder.CreateTexture(Desc, TEXT("SceneColorWithAlpha"));

	for (const FViewInfo& View : Views)
	{
		FDebugAlphaChannelCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDebugAlphaChannelCS::FParameters>();
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->DebugModeId = 0; // MakeDistanceOpaqueTranslucent
		PassParameters->OpaqueWorldDistance = CVarTestAlphaOpaqueWorldDistance.GetValueOnRenderThread();
		PassParameters->OpaqueLerpWorldRange = CVarTestAlphaOpaqueLerpWorldRange.GetValueOnRenderThread();

		PassParameters->SceneColorTexture = SceneTextures.Color.Resolve;
		PassParameters->SceneDepthTexture = SceneTextures.Depth.Resolve;

		PassParameters->SceneColorOutput = GraphBuilder.CreateUAV(NewSceneColorTexture);

		TShaderMapRef<FDebugAlphaChannelCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("DebugAlphaChannel(MakeDistanceOpaqueTranslucent) %dx%d", View.ViewRect.Width(), View.ViewRect.Height()),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), 8));
	}

	return FRDGTextureMSAA(NewSceneColorTexture);
}

#endif // DEBUG_ALPHA_CHANNEL
