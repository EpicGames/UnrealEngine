// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "ScreenPass.h"

class FViewInfo;
struct FMinimalSceneTextures;

#if !UE_BUILD_SHIPPING
	#define DEBUG_ALPHA_CHANNEL 1
#else
	#define DEBUG_ALPHA_CHANNEL 0
#endif

#if DEBUG_ALPHA_CHANNEL

bool ShouldMakeDistantGeometryTranslucent();

FRDGTextureMSAA MakeDistanceGeometryTranslucent(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	FMinimalSceneTextures SceneTextures);

#endif // DEBUG_ALPHA_CHANNEL
