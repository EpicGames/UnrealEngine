// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DiaphragmDOFPasses.cpp: Implementations of all diaphragm DOF's passes.
=============================================================================*/

#include "PostProcess/DiaphragmDOF.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "PostProcess/SceneRenderTargets.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessBokehDOF.h"
#include "SceneRenderTargetParameters.h"
#include "PostProcess/PostProcessing.h"
#include "DeferredShadingRenderer.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "SpriteIndexBuffer.h"
#include "PostProcess/TemporalAA.h"
#include "SceneTextureParameters.h"
#include "TranslucentRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ScreenPass.h"
#include "SceneViewState.h"
#include "PathTracing.h"

// ---------------------------------------------------- Cvars

namespace
{

DECLARE_GPU_STAT(DepthOfField);

TAutoConsoleVariable<int32> CVarDOFGatherResDivisor(
	TEXT("r.DOF.Gather.ResolutionDivisor"), 2,
	TEXT("Selects the resolution divisor of the gather pass.\n")
	TEXT(" 1: Do gathering pass at full resolution;\n")
	TEXT(" 2: Do gathering pass at half resolution (default)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarAccumulatorQuality(
	TEXT("r.DOF.Gather.AccumulatorQuality"),
	1,
	TEXT("Controles the quality of the gathering accumulator.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarGatherDistinctionLimit(
	TEXT("r.DOF.Gather.DistinctionLimit"),
	0.0,
	TEXT("Bokeh that are not scattered but exceed this distinction limit are discarded instead of gathered.\n")
	TEXT("Distinction is calculcated based on perceived brightness and radius.\n")
	TEXT("This is used to reduce noise in gathered bokeh, without drawing additional scattered bokeh.\n")
	TEXT("This can cause bokeh to become darker than they should be, which is a compromise for noise reduction.\n")
	TEXT("See also: r.DOF.Scatter.NeighborCompareMaxColor \n")
	TEXT("Set to 0 to disable.\n")
	TEXT("(default: 0)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarEnableGatherBokehSettings(
	TEXT("r.DOF.Gather.EnableBokehSettings"),
	1,
	TEXT("Whether to applies bokeh settings on foreground and background gathering.\n")
	TEXT(" 0: Disable;\n 1: Enable (default)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarPostFilteringMethod(
	TEXT("r.DOF.Gather.PostfilterMethod"),
	1,
	TEXT("Method to use to post filter a gather pass.\n")
	TEXT(" 0: None;\n")
	TEXT(" 1: Per RGB channel median 3x3 (default);\n")
	TEXT(" 2: Per RGB channel max 3x3."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarRingCount(
	TEXT("r.DOF.Gather.RingCount"),
	5,
	TEXT("Number of rings for gathering kernels [[3; 5]]. Default to 5.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarGatherMaxMipMapCount(
	TEXT("r.DOF.Gather.MaxMipMapCount"),
	4,
	TEXT("Set a maximum amount of mipmaps to calculate in the reduction phase.\n")
	TEXT("These are then used in the gather step to reduce noise in areas with a large but sparse blurring kernel."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarHybridScatterForegroundMode(
	TEXT("r.DOF.Scatter.ForegroundCompositing"),
	1,
	TEXT("Compositing mode of the foreground hybrid scattering.\n")
	TEXT(" 0: Disabled;\n")
	TEXT(" 1: Additive (default)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarHybridScatterBackgroundMode(
	TEXT("r.DOF.Scatter.BackgroundCompositing"),
	2,
	TEXT("Compositing mode of the background hybrid scattering.\n")
	TEXT(" 0: Disabled;\n")
	TEXT(" 1: Additive;\n")
	TEXT(" 2: Gather occlusion (default)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarEnableScatterBokehSettings(
	TEXT("r.DOF.Scatter.EnableBokehSettings"),
	1,
	TEXT("Whether to enable bokeh settings on scattering.\n")
	TEXT(" 0: Disable;\n 1: Enable (default)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarScatterMinCocRadius(
	TEXT("r.DOF.Scatter.MinCocRadius"),
	3.0f,
	TEXT("Minimal Coc radius required to be scattered (default = 3)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarScatterMaxSpriteRatio(
	TEXT("r.DOF.Scatter.MaxSpriteRatio"),
	0.1f,
	TEXT("Maximum ratio of scattered pixel quad as sprite, usefull to control DOF's scattering upperbound. ")
	TEXT(" 1 will allow to scatter 100% pixel quads, whereas 0.2 will only allow 20% (default = 0.1)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarEnableRecombineBokehSettings(
	TEXT("r.DOF.Recombine.EnableBokehSettings"),
	1,
	TEXT("Whether to applies bokeh settings on slight out of focus done in recombine pass.\n")
	TEXT(" 0: Disable;\n 1: Enable (default)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarRecombineQuality(
	TEXT("r.DOF.Recombine.Quality"),
	2,
	TEXT("Configures the quality of the recombine pass.\n")
	TEXT(" 0: No slight out of focus;\n")
	TEXT(" 1: Slight out of focus 24spp;\n")
	TEXT(" 2: Slight out of focus 32spp (default)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarMinimalFullresBlurRadius(
	TEXT("r.DOF.Recombine.MinFullresBlurRadius"),
	0.1f,
	TEXT("Minimal blurring radius used in full resolution pixel width to actually do ")
	TEXT("DOF  when slight out of focus is enabled (default = 0.1)."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarDOFTemporalAAQuality(
	TEXT("r.DOF.TemporalAAQuality"),
	1,
	TEXT("Quality of temporal AA pass done in DOF.\n")
	TEXT(" 0: Faster but lower quality;")
	TEXT(" 1: Higher quality pass (default)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarScatterNeighborCompareMaxColor(
	TEXT("r.DOF.Scatter.NeighborCompareMaxColor"),
	10,
	TEXT("Threshold used when deciding if a pixel should get scattered if it has a neighbor that is already getting scattered. \n")
	TEXT("The linear color of the pixel and its neighbors are compared. If the difference is lower than this threshold, it is not scattered. \n")
	TEXT("Too low, and you may not scatter enough, causing quality issues. Too high you may scatter unnecessarily too much in highlights, causing overdraw. \n")
	TEXT("See also: r.DOF.Gather.DistinctionLimit \n")
	TEXT(" (Default: 10)."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarCoCBilateralFilterStrength(
	TEXT("r.DOF.TAA.CoCBilateralFilterStrength"),
	0.0,
	TEXT("If 1, a bilateral filter based on the circle-of-confusion is used to reject history in the pre-DoF TAA pass.\n")
	TEXT("If 0, this is disabled.\n")
	TEXT("When enabled, you may get stronger noise on edges of objects that separate foreground and background.\n")
	TEXT("When disabled, you may get stronger ghosting along those same edges.\n")
	TEXT(" (Default: 0)."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarSOFHistoryRejectionBoost(
	TEXT("r.DOF.TAA.SOFHistoryRejectionBoost"),
	0.0,
	TEXT("In the recombine pass, reduce the strength of denoising for slight-out-of-focus areas.\n")
	TEXT("Reasonable values are 0 to 0.125.\n")
	TEXT("If 0, this is disabled.\n")
	TEXT(" (Default: 0)."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarPreferLowerBitDepth(
	TEXT("r.DOF.PreferLowerBitDepth"),
	0,
	TEXT("When enabled, prefers intermediate textures with lower bit depth.\n")
	TEXT("This improves memory usage and bandwidth, at the cost of quality."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

} // namespace


// ---------------------------------------------------- COMMON

bool DiaphragmDOF::IsSupported(const FStaticShaderPlatform ShaderPlatform)
{
	// Only compile diaphragm DOF on platform it has been tested to ensure this is not blocking anyone else.
	return FDataDrivenShaderPlatformInfo::GetSupportsDiaphragmDOF(ShaderPlatform);
}

namespace
{
	
/** Defines which layer to process. */
enum class EDiaphragmDOFLayerProcessing
{
	// Foreground layer only.
	ForegroundOnly,

	// Foreground hole filling.
	ForegroundHoleFilling,

	// Background layer only.
	BackgroundOnly,

	// Both foreground and background layers.
	ForegroundAndBackground,

	// Slight out of focus layer.
	SlightOutOfFocus,

	MAX
};



/** Defines which layer to process. */
enum class EDiaphragmDOFPostfilterMethod
{
	// Disable post filtering.
	None,

	// Per RGB channel median on 3x3 neighborhood.
	RGBMedian3x3,

	// Per RGB channel max on 3x3 neighborhood.
	RGBMax3x3,

	MAX
};


/** Modes to simulate a bokeh. */
enum class EDiaphragmDOFBokehSimulation
{
	// No bokeh simulation.
	Disabled,

	// Symmetric bokeh (even number of blade).
	SimmetricBokeh,

	// Generic bokeh.
	GenericBokeh,

	MAX,
};

/** Dilate mode of the pass. */
enum class EDiaphragmDOFDilateCocMode
{
	// One single dilate pass.
	StandAlone,

	// Dilate min foreground and max background coc radius.
	MinForegroundAndMaxBackground,

	// Dilate everything else from dilated min foreground and max background coc radius.
	MinimalAbsoluteRadiuses,

	MAX
};

/** Quality configurations for gathering passes. */
enum class EDiaphragmDOFGatherQuality
{
	// Lower but faster accumulator.
	LowQualityAccumulator,

	// High quality accumulator.
	HighQuality,

	// High quality accumulator with hybrid scatter occlusion buffer output.
	// TODO: distinct shader permutation dimension for hybrid scatter occlusion?
	HighQualityWithHybridScatterOcclusion,

	// High quality accumulator, with layered full disks and hybrid scatter occlusion.
	Cinematic,

	MAX,
};

/** Quality configurations for scattering passes. */
enum class EDiaphragmDOFScatterQuality
{
	Medium,
	High,

	MAX,
};

/** Format of the LUT to generate. */
enum class EDiaphragmDOFBokehLUTFormat
{
	// Look up table that stores a factor to transform a CocRadius to a BokehEdge distance.
	// Used for scattering and low res focus gathering.
	CocRadiusToBokehEdgeFactor,

	// Look up table that stores Coc distance to compare against neighbor's CocRadius.
	// Used exclusively for full res gathering in recombine pass.
	FullResOffsetToCocDistance,


	// Look up table to stores the gathering sample pos within the kernel.
	// Used for low res back and foreground gathering.
	GatherSamplePos,

	MAX,
};


const int32 kDefaultGroupSize = 8;

/** Number of half res pixel are covered by a CocTile */
const int32 kCocTileSize = kDefaultGroupSize;

/** Resolution divisor of the Coc tiles. */
const int32 kMaxCocDilateSampleRadiusCount = 3;

/** Resolution divisor of the Coc tiles. */
const int32 kMaxMipLevelCount = 4;

/** Minimum number of ring. */
const int32 kMinGatheringRingCount = 3;

const int32 kMaxGatheringRingCount = 5;

/** Maximum number of ring for slight out of focus pass. Same as USH's MAX_RECOMBINE_ABS_COC_RADIUS. */
const int32 kMaxSlightOutOfFocusCocRadius = 3;

/** Maximum quality level of the recombine pass. */
const int32 kMaxRecombineQuality = 2;

/** Absolute minim coc radius required for a bokeh to be scattered */
const float kMinScatteringCocRadius = 3.0f;


FIntPoint CocTileGridSize(FIntPoint FullResSize)
{
	uint32 TilesX = FMath::DivideAndRoundUp(FullResSize.X, kCocTileSize);
	uint32 TilesY = FMath::DivideAndRoundUp(FullResSize.Y, kCocTileSize);
	return FIntPoint(TilesX, TilesY);
}


/** Returns the lower res's viewport from a given view size. */
FIntRect GetLowerResViewport(const FIntRect& ViewRect, int32 ResDivisor)
{
	check(ResDivisor >= 1);
	check(FMath::IsPowerOfTwo(ResDivisor));

	FIntRect DestViewport;

	// All diaphragm DOF's lower res viewports are top left cornered to only do a min(SampleUV, MaxUV) when doing convolution.
	DestViewport.Min = FIntPoint::ZeroValue;

	DestViewport.Max.X = FMath::DivideAndRoundUp(ViewRect.Width(), ResDivisor);
	DestViewport.Max.Y = FMath::DivideAndRoundUp(ViewRect.Height(), ResDivisor);
	return DestViewport;
}


EDiaphragmDOFPostfilterMethod GetPostfilteringMethod()
{
	int32 i = CVarPostFilteringMethod.GetValueOnRenderThread();
	if (i >= 0 && i < int32(EDiaphragmDOFPostfilterMethod::MAX))
	{
		return EDiaphragmDOFPostfilterMethod(i);
	}
	return EDiaphragmDOFPostfilterMethod::None;
}

enum class EHybridScatterMode
{
	Disabled,
	Additive,
	Occlusion,
};

const TCHAR* GetEventName(EDiaphragmDOFLayerProcessing e)
{
	static const TCHAR* const kArray[] = {
		TEXT("FgdOnly"),
		TEXT("FgdFill"),
		TEXT("BgdOnly"),
		TEXT("Fgd&Bgd"),
		TEXT("FocusOnly"),
	};
	int32 i = int32(e);
	check(i < UE_ARRAY_COUNT(kArray));
	return kArray[i];
}

const TCHAR* GetEventName(EDiaphragmDOFPostfilterMethod e)
{
	static const TCHAR* const kArray[] = {
		TEXT("Median3x3"),
		TEXT("Max3x3"),
	};
	int32 i = int32(e) - 1;
	check(i < UE_ARRAY_COUNT(kArray));
	return kArray[i];
}

const TCHAR* GetEventName(EDiaphragmDOFBokehSimulation e)
{
	static const TCHAR* const kArray[] = {
		TEXT("None"),
		TEXT("Symmetric"),
		TEXT("Generic"),
	};
	int32 i = int32(e);
	check(i < UE_ARRAY_COUNT(kArray));
	return kArray[i];
}

const TCHAR* GetEventName(EDiaphragmDOFBokehLUTFormat e)
{
	static const TCHAR* const kArray[] = {
		TEXT("Scatter"),
		TEXT("Recombine"),
		TEXT("Gather"),
	};
	int32 i = int32(e);
	check(i < UE_ARRAY_COUNT(kArray));
	return kArray[i];
}

const TCHAR* GetEventName(EDiaphragmDOFGatherQuality e)
{
	static const TCHAR* const kArray[] = {
		TEXT("LowQ"),
		TEXT("HighQ"),
		TEXT("ScatterOcclusion"),
		TEXT("Cinematic"),
	};
	int32 i = int32(e);
	check(i < UE_ARRAY_COUNT(kArray));
	return kArray[i];
}

const TCHAR* GetEventName(EDiaphragmDOFDilateCocMode e)
{
	static const TCHAR* const kArray[] = {
		TEXT("StandAlone"),
		TEXT("MinMax"),
		TEXT("MinAbs"),
	};
	int32 i = int32(e);
	check(i < UE_ARRAY_COUNT(kArray));
	return kArray[i];
}

// Returns X and Y for F(M) = saturate(M * X + Y) so that F(LowM) = 0 and F(HighM) = 1
FVector2f GenerateSaturatedAffineTransformation(float LowM, float HighM)
{
	float X = 1.0f / (HighM - LowM);
	return FVector2f(X, -X * LowM);
}

// Affine transformtations that always return 0 or 1.
const FVector2f kContantlyPassingAffineTransformation(0, 1);
const FVector2f kContantlyBlockingAffineTransformation(0, 0);

/** Base shader class for diaphragm DOF. */
class FDiaphragmDOFShader : public FGlobalShader
{
public:
	class FAlphaChannelDim : SHADER_PERMUTATION_BOOL("DIM_ALPHA_CHANNEL");

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DiaphragmDOF::IsSupported(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		bool bSM5OrPreview = Parameters.Platform == SP_PCD3D_SM5 || FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(Parameters.Platform);

		if (FDataDrivenShaderPlatformInfo::GetSupportsRealTypes(Parameters.Platform) == ERHIFeatureSupport::RuntimeGuaranteed)
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_AllowRealTypes);
		}
		if (!bSM5OrPreview && FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(Parameters.Platform) != ERHIFeatureSupport::Unsupported)
		{
			OutEnvironment.SetDefine(TEXT("WAVE_OPS"), 1);
		}

		OutEnvironment.SetDefine(TEXT("COC_TILE_SIZE"), kCocTileSize);

		// If on console, don't compile any dynamic CoC offset to avoid performance differences
		const bool bIsConsole = FDataDrivenShaderPlatformInfo::GetIsConsole(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("WITH_DYNAMIC_COC_OFFSET"), bIsConsole ? 0 : 1);

		OutEnvironment.SetDefine(TEXT("MAX_MATTE_BOX_FLAGS"), (int)DiaphragmDOF::FPhysicalCocModel::MaxMatteBoxFlags);
	}

	FDiaphragmDOFShader() {}
	FDiaphragmDOFShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{ }
};

} // namespace


// ---------------------------------------------------- Global resource

namespace
{

class FDOFGlobalResource : public FRenderResource
{
public:
	// Index buffer to have 4 vertex shader invocation per scatter group that is the most efficient in therms of vertex processing
	// using  if RHI does not support rect list topology.
	FSpriteIndexBuffer<16> ScatterIndexBuffer;


	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		if (!GRHISupportsRectTopology)
		{
			ScatterIndexBuffer.InitRHI(RHICmdList);
		}
	}

	virtual void ReleaseRHI() override
	{
		if (!GRHISupportsRectTopology)
		{
			ScatterIndexBuffer.ReleaseRHI();
		}
	}
};

TGlobalResource<FDOFGlobalResource> GDOFGlobalResource;

}


// ---------------------------------------------------- Shader permutation dimensions

namespace
{

class FDDOFDilateRadiusDim     : SHADER_PERMUTATION_RANGE_INT("DIM_DILATE_RADIUS", 1, 3);
class FDDOFDilateModeDim       : SHADER_PERMUTATION_ENUM_CLASS("DIM_DILATE_MODE", EDiaphragmDOFDilateCocMode);

class FDDOFLayerProcessingDim  : SHADER_PERMUTATION_ENUM_CLASS("DIM_LAYER_PROCESSING", EDiaphragmDOFLayerProcessing);
class FDDOFGatherRingCountDim  : SHADER_PERMUTATION_RANGE_INT("DIM_GATHER_RING_COUNT", kMinGatheringRingCount, kMaxGatheringRingCount - kMinGatheringRingCount + 1);
class FDDOFGatherQualityDim    : SHADER_PERMUTATION_ENUM_CLASS("DIM_GATHER_QUALITY", EDiaphragmDOFGatherQuality);
class FDDOFPostfilterMethodDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_POSTFILTER_METHOD", EDiaphragmDOFPostfilterMethod);
class FDDOFRGBColorBufferDim   : SHADER_PERMUTATION_BOOL("DIM_RGB_COLOR_BUFFER");

class FDDOFBokehSimulationDim  : SHADER_PERMUTATION_ENUM_CLASS("DIM_BOKEH_SIMULATION", EDiaphragmDOFBokehSimulation);
class FDDOFScatterOcclusionDim : SHADER_PERMUTATION_BOOL("DIM_SCATTER_OCCLUSION");

} // namespace


// ---------------------------------------------------- Shared shader parameters

namespace
{

struct FDOFGatherInputDescs
{
	FRDGTextureDesc SceneColor;
	FRDGTextureDesc SeparateCoc;
};

BEGIN_SHADER_PARAMETER_STRUCT(FDOFGatherInputTextures, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColor)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SeparateCoc)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FDOFGatherInputSRVs, )
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SceneColor)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SeparateCoc)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FDOFGatherInputUAVs, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SceneColor)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SeparateCoc)
END_SHADER_PARAMETER_STRUCT()

FDOFGatherInputTextures CreateTextures(FRDGBuilder& GraphBuilder, const FDOFGatherInputDescs& Descs, const TCHAR* DebugName)
{
	FDOFGatherInputTextures Textures;
	Textures.SceneColor = GraphBuilder.CreateTexture(Descs.SceneColor, DebugName);
	if (Descs.SeparateCoc.Format != PF_Unknown)
	{
		Textures.SeparateCoc = GraphBuilder.CreateTexture(Descs.SeparateCoc, DebugName);
	}
	return Textures;
}

FDOFGatherInputSRVs CreateSRVs(FRDGBuilder& GraphBuilder, const FDOFGatherInputTextures& Textures, uint8 MipLevel = 0)
{
	FDOFGatherInputSRVs SRVs;
	SRVs.SceneColor = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Textures.SceneColor, MipLevel));
	if (Textures.SeparateCoc)
	{
		SRVs.SeparateCoc = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Textures.SeparateCoc, MipLevel));
	}
	return SRVs;
}

FDOFGatherInputUAVs CreateUAVs(FRDGBuilder& GraphBuilder, const FDOFGatherInputTextures& Textures, uint8 MipLevel = 0)
{
	FDOFGatherInputUAVs UAVs;
	UAVs.SceneColor = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Textures.SceneColor, MipLevel));
	if (Textures.SeparateCoc)
	{
		UAVs.SeparateCoc = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Textures.SeparateCoc, MipLevel));
	}
	return UAVs;
}


struct FDOFConvolutionDescs
{
	FRDGTextureDesc SceneColor;
	FRDGTextureDesc SeparateAlpha;
};

BEGIN_SHADER_PARAMETER_STRUCT(FDOFConvolutionTextures, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColor)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SeparateAlpha)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FDOFConvolutionUAVs, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SceneColor)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SeparateAlpha)
END_SHADER_PARAMETER_STRUCT()

FDOFConvolutionTextures CreateTextures(FRDGBuilder& GraphBuilder, const FDOFConvolutionDescs& Descs, const TCHAR* DebugName)
{
	FDOFConvolutionTextures Textures;
	Textures.SceneColor = GraphBuilder.CreateTexture(Descs.SceneColor, DebugName);
	if (Descs.SeparateAlpha.Format != PF_Unknown)
	{
		Textures.SeparateAlpha = GraphBuilder.CreateTexture(Descs.SeparateAlpha, DebugName);
	}
	return Textures;
}

FDOFConvolutionUAVs CreateUAVs(FRDGBuilder& GraphBuilder, const FDOFConvolutionTextures& Textures)
{
	FDOFConvolutionUAVs UAVs;
	UAVs.SceneColor = GraphBuilder.CreateUAV(Textures.SceneColor);
	if (Textures.SeparateAlpha)
	{
		UAVs.SeparateAlpha = GraphBuilder.CreateUAV(Textures.SeparateAlpha);
	}
	return UAVs;
}


struct FDOFTileClassificationDescs
{
	FRDGTextureDesc Foreground;
	FRDGTextureDesc Background;
};

BEGIN_SHADER_PARAMETER_STRUCT(FDOFTileClassificationTextures, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Foreground)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Background)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FDOFTileClassificationUAVs, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Foreground)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Background)
END_SHADER_PARAMETER_STRUCT()

FDOFTileClassificationTextures CreateTextures(FRDGBuilder& GraphBuilder, const FDOFTileClassificationDescs& Descs, const TCHAR* DebugNames[2])
{
	FDOFTileClassificationTextures Textures;
	Textures.Foreground = GraphBuilder.CreateTexture(Descs.Foreground, DebugNames[0]);
	Textures.Background = GraphBuilder.CreateTexture(Descs.Background, DebugNames[1]);
	return Textures;
}

FDOFTileClassificationUAVs CreateUAVs(FRDGBuilder& GraphBuilder, const FDOFTileClassificationTextures& Textures)
{
	FDOFTileClassificationUAVs UAVs;
	UAVs.Foreground = GraphBuilder.CreateUAV(Textures.Foreground);
	UAVs.Background = GraphBuilder.CreateUAV(Textures.Background);
	return UAVs;
}


BEGIN_SHADER_PARAMETER_STRUCT(FDOFCommonShaderParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
END_SHADER_PARAMETER_STRUCT()


BEGIN_SHADER_PARAMETER_STRUCT(FDOFTileDecisionParameters, )
	SHADER_PARAMETER(float, MinGatherRadius)
	SHADER_PARAMETER(float, SlightOutOfFocusRadiusBoundary)
END_SHADER_PARAMETER_STRUCT()

} // namespace

namespace DiaphragmDOF
{
	void SetCocModelParameters(
		FRDGBuilder& GraphBuilder,
		FDOFCocModelShaderParameters* OutParameters,
		const FPhysicalCocModel& CocModel,
		float CocRadiusBasis)
	{
		OutParameters->CocInfinityRadius = CocRadiusBasis * CocModel.InfinityBackgroundCocRadius;
		OutParameters->CocInFocusRadius = CocRadiusBasis * CocModel.InFocusRadius;
		OutParameters->bCocEnableDynamicRadiusOffset = CocModel.bEnableDynamicOffset;
		OutParameters->CocMinRadius = CocRadiusBasis * CocModel.MinForegroundCocRadius;
		OutParameters->CocMaxRadius = CocRadiusBasis * CocModel.MaxBackgroundCocRadius;
		OutParameters->CocSqueeze = CocModel.Squeeze;
		OutParameters->CocInvSqueeze = 1.0f / CocModel.Squeeze;
		OutParameters->DepthBlurRadius = CocRadiusBasis * CocModel.MaxDepthBlurRadius;
		OutParameters->DepthBlurExponent = CocModel.DepthBlurExponent;

		if (CocModel.DynamicRadiusOffsetLUT)
		{
			OutParameters->DynamicRadiusOffsetLUT = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(CocModel.DynamicRadiusOffsetLUT, TEXT("DynamicRadiusOffsetLUT")));
		}
		else
		{
			const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

			OutParameters->DynamicRadiusOffsetLUT = SystemTextures.Black;
		}

		OutParameters->DynamicRadiusOffsetLUTSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Wrap>::GetRHI();
	}
}


// ---------------------------------------------------- Shaders

namespace
{

/** Returns whether hybrid scattering is supported. */
static FORCEINLINE bool SupportsHybridScatter(EShaderPlatform ShaderPlatform)
{
	return !!FDataDrivenShaderPlatformInfo::GetSupportsDOFHybridScattering(ShaderPlatform);
}


/** Returns the number maximum number of ring available. */
static FORCEINLINE int32 MaxGatheringRingCount(EShaderPlatform ShaderPlatform)
{
	if (IsPCPlatform(ShaderPlatform))
	{
		return kMaxGatheringRingCount;
	}
	return 4;
}

/** Returns whether the shader for bokeh simulation are compiled. */
static FORCEINLINE bool SupportsBokehSimmulation(EShaderPlatform ShaderPlatform)
{
	// Shaders of gathering pass are big, so only compile them on desktop.
	return IsPCPlatform(ShaderPlatform);
}

/** Returns whether separate coc buffer is supported. */
static FORCEINLINE bool SupportsRGBColorBuffer(EShaderPlatform ShaderPlatform)
{
	// There is no point when alpha channel is supported because needs 4 channel anyway for fast gathering tiles.
	if (IsPostProcessingWithAlphaChannelSupported())
	{
		return false;
	}

	// There is high number of UAV to write in reduce pass.
	return FDataDrivenShaderPlatformInfo::GetSupportsRGBColorBuffer(ShaderPlatform);
}


class FDiaphragmDOFSetupCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFSetupCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFSetupCS, FDiaphragmDOFShader);

	class FOutputResDivisor : SHADER_PERMUTATION_INT("DIM_OUTPUT_RES_DIVISOR", 3);

	using FPermutationDomain = TShaderPermutationDomain<FDiaphragmDOFShader::FAlphaChannelDim, FOutputResDivisor>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFCommonShaderParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(DiaphragmDOF::FDOFCocModelShaderParameters, CocModel)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Output0)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Output1)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Output2)
	END_SHADER_PARAMETER_STRUCT()
};

class FDiaphragmDOFCocFlattenCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFCocFlattenCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFCocFlattenCS, FDiaphragmDOFShader);

	class FDoCocGather4 : SHADER_PERMUTATION_BOOL("DIM_DO_COC_GATHER4");

	using FPermutationDomain = TShaderPermutationDomain<FDiaphragmDOFShader::FAlphaChannelDim, FDoCocGather4>;
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, Stride)
		SHADER_PARAMETER(FVector4f, GatherInputExtentAndInvExtent)
		SHADER_PARAMETER(FVector4f, GatherInputViewRectUV)

		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFCommonShaderParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT(FDOFGatherInputTextures, GatherInput)
		SHADER_PARAMETER_STRUCT(FDOFTileClassificationUAVs, TileOutput)
	END_SHADER_PARAMETER_STRUCT()

	static_assert(ISceneViewFamilyScreenPercentage::kMinTAAUpsampleResolutionFraction == 0.5f,
		"Gather4 shader permutation assumes with min TAAU screen percentage = 50%.");
	static_assert(ISceneViewFamilyScreenPercentage::kMaxTAAUpsampleResolutionFraction == 2.0f,
		"Gather4 shader permutation assumes with max TAAU screen percentage = 200%.");
};

class FDiaphragmDOFCocDilateCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFCocDilateCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFCocDilateCS, FDiaphragmDOFShader);

	using FPermutationDomain = TShaderPermutationDomain<FDDOFDilateRadiusDim, FDDOFDilateModeDim>;
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntRect, ViewportRect)
		SHADER_PARAMETER(int32, SampleOffsetMultipler)
		SHADER_PARAMETER(float, fSampleOffsetMultipler)
		SHADER_PARAMETER(float, CocRadiusToBucketDistanceUpperBound)
		SHADER_PARAMETER(float, BucketDistanceToCocRadius)
		
		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFCommonShaderParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT(FDOFTileClassificationTextures, TileInput)
		SHADER_PARAMETER_STRUCT(FDOFTileClassificationTextures, DilatedTileMinMax)
		SHADER_PARAMETER_STRUCT(FDOFTileClassificationUAVs, TileOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FDiaphragmDOFDownsampleCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFDownsampleCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFDownsampleCS, FDiaphragmDOFShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!SupportsHybridScatter(Parameters.Platform))
		{
			return false;
		}
		return FDiaphragmDOFShader::ShouldCompilePermutation(Parameters);
	}

	using FPermutationDomain = TShaderPermutationDomain<FDiaphragmDOFShader::FAlphaChannelDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntRect, ViewportRect)
		SHADER_PARAMETER(float, OutputCocRadiusMultiplier)

		SHADER_PARAMETER(FVector4f, GatherInputExtentAndInvExtent)
		SHADER_PARAMETER(FVector4f, GatherInputViewRectUV)
		SHADER_PARAMETER_STRUCT(FDOFGatherInputTextures, GatherInput)

		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFCommonShaderParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT(FDOFGatherInputUAVs, OutDownsampledGatherInput)
	END_SHADER_PARAMETER_STRUCT()
};

class FDiaphragmDOFReduceCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFReduceCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFReduceCS, FDiaphragmDOFShader);

	class FReduceMipCount : SHADER_PERMUTATION_RANGE_INT("DIM_REDUCE_MIP_COUNT", 0, kMaxMipLevelCount + 1);
	class FHybridScatterForeground : SHADER_PERMUTATION_BOOL("DIM_HYBRID_SCATTER_FGD");
	class FHybridScatterBackground : SHADER_PERMUTATION_BOOL("DIM_HYBRID_SCATTER_BGD");
	class FDisableOutputMip0 : SHADER_PERMUTATION_BOOL("DIM_DISABLE_OUTPUT_MIP0");

	using FPermutationDomain = TShaderPermutationDomain<
		FDiaphragmDOFShader::FAlphaChannelDim,
		FReduceMipCount,
		FHybridScatterForeground,
		FHybridScatterBackground,
		FDDOFRGBColorBufferDim,
		FDisableOutputMip0>;


	/** Returns the number of mip level the reduce pass is able to output. */
	static int32 GetMaxReductionMipLevelCount(bool bSupportAlphaChannel, bool bMip0Disabled)
	{
		// Can only have 8 UAVs, but need to output 3x UAV for hybird scatter + 2 UAV per mips for RGBA + SeparateCoc.
		int32 MaxLevelsWithAlphaChannel = bMip0Disabled ? 3 : 2;
		return bSupportAlphaChannel ? MaxLevelsWithAlphaChannel : kMaxMipLevelCount;
	}

	/** Returns the nearest number of mip levels the reduce pass is able to output */
	static int32 GetNearestMipLevelReductionCount(int ReductionCount, bool bSupportAlphaChannel, bool bMip0Disabled)
	{
		int32 MaxCount = GetMaxReductionMipLevelCount(bSupportAlphaChannel, bMip0Disabled);
		ReductionCount = FMath::Min(ReductionCount, MaxCount);

		return ReductionCount;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		bool bAlphaChannel = PermutationVector.Get<FDiaphragmDOFShader::FAlphaChannelDim>();
		bool bDisableOutputMip0 = PermutationVector.Get<FDisableOutputMip0>();

		if (PermutationVector.Get<FDisableOutputMip0>())
		{
			// Hybrid scatter is done only on first reduction, that needs to output mip0
			if (PermutationVector.Get<FHybridScatterForeground>() || PermutationVector.Get<FHybridScatterBackground>())
			{
				return false;
			}
		}
		
		// Do not output mip level that are more than supported.
		int32 ReduceMipCount = PermutationVector.Get<FReduceMipCount>();
		if (GetNearestMipLevelReductionCount(ReduceMipCount, bAlphaChannel, bDisableOutputMip0) != ReduceMipCount)
		{
			return false;
		}

		// Do not compile storing Coc independently of RGB if not supported.
		if (PermutationVector.Get<FDDOFRGBColorBufferDim>() && !SupportsRGBColorBuffer(Parameters.Platform))
		{
			return false;
		}

		if (!SupportsHybridScatter(Parameters.Platform))
		{
			if (PermutationVector.Get<FHybridScatterForeground>() || PermutationVector.Get<FHybridScatterBackground>())
			{
				return false;
			}
		}

		return FDiaphragmDOFShader::ShouldCompilePermutation(Parameters);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, PassViewSize)
		SHADER_PARAMETER(FVector4f, ScatteringViewportSize)
		SHADER_PARAMETER(float, ScatteringScaling)
		SHADER_PARAMETER(int32, MaxScatteringGroupCount)
		SHADER_PARAMETER(float, PreProcessingToProcessingCocRadiusFactor)
		SHADER_PARAMETER(float, BokehGatherDistinctionLimit)
		SHADER_PARAMETER(float, MinScatteringCocRadius)
		SHADER_PARAMETER(float, NeighborCompareMaxColor)

		SHADER_PARAMETER_STRUCT_INCLUDE(DiaphragmDOF::FDOFCocModelShaderParameters, CocModel)

		SHADER_PARAMETER(FVector2f, SensorSize)
		SHADER_PARAMETER(float, Aperture)
		SHADER_PARAMETER(float, LensImageDistance)
	
		SHADER_PARAMETER(float, BarrelRadius)
		SHADER_PARAMETER(float, BarrelLength)
		SHADER_PARAMETER_ARRAY(FUintVector4, MatteBoxPlanes, [DiaphragmDOF::FPhysicalCocModel::MaxMatteBoxFlags])
		
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFCommonShaderParameters, CommonParameters)

		SHADER_PARAMETER(FVector4f, GatherInputExtentAndInvExtent)
		SHADER_PARAMETER(FVector4f, GatherInputViewRectUV)
		SHADER_PARAMETER_STRUCT(FDOFGatherInputSRVs, GatherInput)
		
		SHADER_PARAMETER(FVector4f, QuarterResGatherInputExtentAndInvExtent)
		SHADER_PARAMETER(FVector4f, QuarterResGatherInputViewRectUV)
		SHADER_PARAMETER_STRUCT(FDOFGatherInputTextures, QuarterResGatherInput)

		SHADER_PARAMETER_STRUCT_ARRAY(FDOFGatherInputUAVs, OutputMips, [kMaxMipLevelCount])
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutScatterDrawIndirectParameters)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, OutForegroundScatterDrawList)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, OutBackgroundScatterDrawList)
	END_SHADER_PARAMETER_STRUCT()
};

class FDiaphragmDOFScatterGroupPackCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFScatterGroupPackCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFScatterGroupPackCS, FDiaphragmDOFShader);
	
	using FPermutationDomain = TShaderPermutationDomain<
		FDiaphragmDOFReduceCS::FHybridScatterForeground,
		FDiaphragmDOFReduceCS::FHybridScatterBackground>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!SupportsHybridScatter(Parameters.Platform))
		{
			return false;
		}
		
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// This shader is used there is at least foreground and/or background to scatter.
		if (!PermutationVector.Get<FDiaphragmDOFReduceCS::FHybridScatterForeground>() &&
			!PermutationVector.Get<FDiaphragmDOFReduceCS::FHybridScatterBackground>())
		{
			return false;
		}

		return FDiaphragmDOFShader::ShouldCompilePermutation(Parameters);
	}
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, MaxScatteringGroupCount)

		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutScatterDrawIndirectParameters)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, OutForegroundScatterDrawList)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, OutBackgroundScatterDrawList)
	END_SHADER_PARAMETER_STRUCT()
};

class FDiaphragmDOFBuildBokehLUTCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFBuildBokehLUTCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFBuildBokehLUTCS, FDiaphragmDOFShader);

	class FBokehSimulationDim : SHADER_PERMUTATION_BOOL("DIM_ROUND_BLADES");
	class FLUTFormatDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_LUT_FORMAT", EDiaphragmDOFBokehLUTFormat);

	using FPermutationDomain = TShaderPermutationDomain<FBokehSimulationDim, FLUTFormatDim>;
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, BladeCount)
		SHADER_PARAMETER(float, DiaphragmRotation)
		SHADER_PARAMETER(float, CocRadiusToCircumscribedRadius)
		SHADER_PARAMETER(float, CocRadiusToIncircleRadius)
		SHADER_PARAMETER(float, DiaphragmBladeRadius)
		SHADER_PARAMETER(float, DiaphragmBladeCenterOffset)
		SHADER_PARAMETER(float, CocSqueeze)
		SHADER_PARAMETER(float, CocInvSqueeze)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, BokehLUTOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FDiaphragmDOFGatherCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFGatherCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFGatherCS, FDiaphragmDOFShader);

	using FPermutationDomain = TShaderPermutationDomain<
		FDiaphragmDOFShader::FAlphaChannelDim,
		FDDOFLayerProcessingDim,
		FDDOFGatherRingCountDim,
		FDDOFBokehSimulationDim,
		FDDOFGatherQualityDim,
		FDDOFRGBColorBufferDim>;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		// There is a lot of permutation, so no longer compile permutation.
		if (1)
		{
			// Alway simulate bokeh generically.
			if (PermutationVector.Get<FDDOFBokehSimulationDim>() == EDiaphragmDOFBokehSimulation::SimmetricBokeh)
			{
				PermutationVector.Set<FDDOFBokehSimulationDim>(EDiaphragmDOFBokehSimulation::GenericBokeh);
			}
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// Do not compile this permutation if we know this is going to be remapped.
		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}

		// Some platforms might be to slow for even considering large number of gathering samples.
		if (PermutationVector.Get<FDDOFGatherRingCountDim>() > MaxGatheringRingCount(Parameters.Platform))
		{
			return false;
		}

		// Do not compile storing Coc independently of RGB.
		if (PermutationVector.Get<FDDOFRGBColorBufferDim>() && !SupportsRGBColorBuffer(Parameters.Platform))
		{
			return false;
		}

		// No point compiling gather pass with hybrid scatter occlusion if the shader platform doesn't support if.
		if (!SupportsHybridScatter(Parameters.Platform) &&
			PermutationVector.Get<FDDOFGatherQualityDim>() == EDiaphragmDOFGatherQuality::HighQualityWithHybridScatterOcclusion) return false;

		// Do not compile bokeh simulation shaders on platform that couldn't handle them anyway.
		if (!SupportsBokehSimmulation(Parameters.Platform) &&
			PermutationVector.Get<FDDOFBokehSimulationDim>() != EDiaphragmDOFBokehSimulation::Disabled) return false;

		if (PermutationVector.Get<FDDOFLayerProcessingDim>() == EDiaphragmDOFLayerProcessing::ForegroundOnly)
		{
			// Foreground does not support CocVariance output yet.
			if (PermutationVector.Get<FDDOFGatherQualityDim>() == EDiaphragmDOFGatherQuality::HighQualityWithHybridScatterOcclusion) return false;

			// Storing Coc independently of RGB is only supported for low gathering quality.
			if (PermutationVector.Get<FDDOFRGBColorBufferDim>() &&
				PermutationVector.Get<FDDOFGatherQualityDim>() != EDiaphragmDOFGatherQuality::LowQualityAccumulator)
				return false;
		}
		else if (PermutationVector.Get<FDDOFLayerProcessingDim>() == EDiaphragmDOFLayerProcessing::ForegroundHoleFilling)
		{
			// Foreground hole filling does not need to output CocVariance, since this is the job of foreground pass.
			if (PermutationVector.Get<FDDOFGatherQualityDim>() == EDiaphragmDOFGatherQuality::HighQualityWithHybridScatterOcclusion) return false;

			// Foreground hole filling doesn't have lower quality accumulator.
			if (PermutationVector.Get<FDDOFGatherQualityDim>() == EDiaphragmDOFGatherQuality::LowQualityAccumulator) return false;

			// Foreground hole filling doesn't need cinematic quality.
			if (PermutationVector.Get<FDDOFGatherQualityDim>() == EDiaphragmDOFGatherQuality::Cinematic) return false;

			// No bokeh simulation on hole filling, always use euclidian closest distance to compute opacity alpha channel.
			if (PermutationVector.Get<FDDOFBokehSimulationDim>() != EDiaphragmDOFBokehSimulation::Disabled) return false;

			// Storing Coc independently of RGB is only supported for RecombineQuality == 0.
			if (PermutationVector.Get<FDDOFRGBColorBufferDim>())
				return false;
		}
		else if (PermutationVector.Get<FDDOFLayerProcessingDim>() == EDiaphragmDOFLayerProcessing::SlightOutOfFocus)
		{
			// Slight out of focus don't need to output CocVariance since there is no hybrid scattering.
			if (PermutationVector.Get<FDDOFGatherQualityDim>() == EDiaphragmDOFGatherQuality::HighQualityWithHybridScatterOcclusion) return false;

			// Slight out of focus filling can't have lower quality accumulator since it needs to brute force the focus areas.
			if (PermutationVector.Get<FDDOFGatherQualityDim>() == EDiaphragmDOFGatherQuality::LowQualityAccumulator) return false;

			// Slight out of focus doesn't have cinematic quality, yet.
			if (PermutationVector.Get<FDDOFGatherQualityDim>() == EDiaphragmDOFGatherQuality::Cinematic) return false;

			// Number of rings is dynamic in the shader.
			if (PermutationVector.Get<FDDOFGatherRingCountDim>() != kMinGatheringRingCount) return false;

			// Storing Coc independently of RGB is only supported for RecombineQuality == 0.
			if (PermutationVector.Get<FDDOFRGBColorBufferDim>()) return false;
		}
		else if (PermutationVector.Get<FDDOFLayerProcessingDim>() == EDiaphragmDOFLayerProcessing::BackgroundOnly)
		{
			// There is no performance point doing high quality gathering without scattering occlusion.
			if (PermutationVector.Get<FDDOFGatherQualityDim>() == EDiaphragmDOFGatherQuality::HighQuality) return false;

			// Storing Coc independently of RGB is only supported for low gathering quality.
			if (PermutationVector.Get<FDDOFRGBColorBufferDim>() &&
				PermutationVector.Get<FDDOFGatherQualityDim>() != EDiaphragmDOFGatherQuality::LowQualityAccumulator)
				return false;
		}
		else if (PermutationVector.Get<FDDOFLayerProcessingDim>() == EDiaphragmDOFLayerProcessing::ForegroundAndBackground)
		{
			// Gathering foreground and backrgound at same time is not supported yet.
			return false;
		}
		else
		{
			check(0);
		}

		return FDiaphragmDOFShader::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FDiaphragmDOFShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
	}

	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, ViewportSize)
		SHADER_PARAMETER(FIntRect, ViewportRect)
		SHADER_PARAMETER(FVector2f, TemporalJitterPixels)
		SHADER_PARAMETER(FVector2f, DispatchThreadIdToInputBufferUV)
		SHADER_PARAMETER(FVector2f, ConsiderCocRadiusAffineTransformation0)
		SHADER_PARAMETER(FVector2f, ConsiderCocRadiusAffineTransformation1)
		SHADER_PARAMETER(FVector2f, ConsiderAbsCocRadiusAffineTransformation)
		SHADER_PARAMETER(FVector2f, InputBufferUVToOutputPixel)
		SHADER_PARAMETER(float, MipBias)
		SHADER_PARAMETER(float, MaxRecombineAbsCocRadius)
		SHADER_PARAMETER(float, CocSqueeze)
		SHADER_PARAMETER(float, CocInvSqueeze)
		
		SHADER_PARAMETER(float, Petzval)
		SHADER_PARAMETER(float, PetzvalFalloffPower)
		SHADER_PARAMETER(FVector2f, PetzvalExclusionBoxExtents)
		SHADER_PARAMETER(float, PetzvalExclusionBoxRadius)

		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFTileDecisionParameters, TileDecisionParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFCommonShaderParameters, CommonParameters)

		SHADER_PARAMETER(FVector4f, GatherInputSize)
		SHADER_PARAMETER(FVector2f, GatherInputViewportSize)
		SHADER_PARAMETER_STRUCT(FDOFGatherInputTextures, GatherInput)

		SHADER_PARAMETER_STRUCT(FDOFTileClassificationTextures, TileClassification)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BokehLUT)

		SHADER_PARAMETER_STRUCT(FDOFConvolutionUAVs, ConvolutionOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ScatterOcclusionOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
}; // class FDiaphragmDOFGatherCS

class FDiaphragmDOFPostfilterCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFPostfilterCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFPostfilterCS, FDiaphragmDOFShader);

	class FTileOptimization : SHADER_PERMUTATION_BOOL("DIM_TILE_PERMUTATION");

	using FPermutationDomain = TShaderPermutationDomain<
		FDiaphragmDOFShader::FAlphaChannelDim,
		FDDOFLayerProcessingDim,
		FDDOFPostfilterMethodDim,
		FTileOptimization>;

	static FPermutationDomain RemapPermutationVector(FPermutationDomain PermutationVector)
	{
		// Tile permutation optimisation is only for Max3x3 post filtering.
		if (PermutationVector.Get<FDDOFPostfilterMethodDim>() != EDiaphragmDOFPostfilterMethod::RGBMax3x3)
			PermutationVector.Set<FTileOptimization>(false);
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (RemapPermutationVector(PermutationVector) != PermutationVector) return false;
		if (PermutationVector.Get<FDDOFPostfilterMethodDim>() == EDiaphragmDOFPostfilterMethod::None) return false;
		if (PermutationVector.Get<FDDOFLayerProcessingDim>() != EDiaphragmDOFLayerProcessing::ForegroundOnly &&
			PermutationVector.Get<FDDOFLayerProcessingDim>() != EDiaphragmDOFLayerProcessing::BackgroundOnly) return false;

		return FDiaphragmDOFShader::ShouldCompilePermutation(Parameters);
	}
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntRect, ViewportRect)
		SHADER_PARAMETER(FVector2f, MaxInputBufferUV)
		
		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFTileDecisionParameters, TileDecisionParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFCommonShaderParameters, CommonParameters)

		SHADER_PARAMETER(FVector4f, ConvolutionInputSize)
		SHADER_PARAMETER_STRUCT(FDOFConvolutionTextures, ConvolutionInput)

		SHADER_PARAMETER_STRUCT(FDOFTileClassificationTextures, TileClassification)
		SHADER_PARAMETER_STRUCT(FDOFConvolutionUAVs, ConvolutionOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
}; // class FDiaphragmDOFPostfilterCS

BEGIN_SHADER_PARAMETER_STRUCT(FDOFHybridScatterParameters, )
	SHADER_PARAMETER(FVector4f, ViewportSize)
	SHADER_PARAMETER(float, CocRadiusToCircumscribedRadius)
	SHADER_PARAMETER(float, ScatteringScaling)
	
	SHADER_PARAMETER(float, Petzval)
	SHADER_PARAMETER(float, PetzvalFalloffPower)
	SHADER_PARAMETER(FVector2f, PetzvalExclusionBoxExtents)
	SHADER_PARAMETER(float, PetzvalExclusionBoxRadius)
	
	SHADER_PARAMETER_STRUCT_INCLUDE(FDOFCommonShaderParameters, CommonParameters)
	SHADER_PARAMETER_STRUCT_INCLUDE(DiaphragmDOF::FDOFCocModelShaderParameters, CocModel)
	
	SHADER_PARAMETER(FVector4f, ScatterOcclusionSize)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ScatterOcclusion)

	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BokehLUT)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, ScatterDrawList)
	RDG_BUFFER_ACCESS(IndirectDrawParameter, ERHIAccess::IndirectArgs)

	SHADER_PARAMETER(FVector2f, SensorSize)
	SHADER_PARAMETER(float, Aperture)
	SHADER_PARAMETER(float, LensImageDistance)
	
	SHADER_PARAMETER(float, BarrelRadius)
	SHADER_PARAMETER(float, BarrelLength)
	SHADER_PARAMETER_ARRAY(FUintVector4, MatteBoxPlanes, [DiaphragmDOF::FPhysicalCocModel::MaxMatteBoxFlags])

	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FDDOFScatterQualityDim	: SHADER_PERMUTATION_ENUM_CLASS("DIM_SCATTER_QUALITY", EDiaphragmDOFScatterQuality);

class FDiaphragmDOFHybridScatterVS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFHybridScatterVS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFHybridScatterVS, FDiaphragmDOFShader);

	using FPermutationDomain = TShaderPermutationDomain<FDDOFScatterQualityDim>;
	using FParameters = FDOFHybridScatterParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!SupportsHybridScatter(Parameters.Platform))
		{
			return false;
		}
		return FDiaphragmDOFShader::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FDiaphragmDOFShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SCATTER_QUALITY_MEDIUM"), (int)EDiaphragmDOFScatterQuality::Medium);
		OutEnvironment.SetDefine(TEXT("SCATTER_QUALITY_HIGH"), (int)EDiaphragmDOFScatterQuality::High);
	}
};

class FDiaphragmDOFHybridScatterPS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFHybridScatterPS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFHybridScatterPS, FDiaphragmDOFShader);

	class FBokehSimulationDim		: SHADER_PERMUTATION_BOOL("DIM_BOKEH_SIMULATION");

	using FPermutationDomain = TShaderPermutationDomain<FDDOFLayerProcessingDim, FBokehSimulationDim, FDDOFScatterOcclusionDim, FDDOFScatterQualityDim>;
	using FParameters = FDOFHybridScatterParameters;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		// Pixel shader are exactly the same between foreground and background when there is no bokeh LUT.
		if (PermutationVector.Get<FDDOFLayerProcessingDim>() == EDiaphragmDOFLayerProcessing::BackgroundOnly && 
			!PermutationVector.Get<FBokehSimulationDim>())
		{
			PermutationVector.Set<FDDOFLayerProcessingDim>(EDiaphragmDOFLayerProcessing::ForegroundOnly);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!SupportsHybridScatter(Parameters.Platform))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// Do not compile this permutation if it gets remapped at runtime.
		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}

		if (PermutationVector.Get<FDDOFLayerProcessingDim>() != EDiaphragmDOFLayerProcessing::ForegroundOnly &&
			PermutationVector.Get<FDDOFLayerProcessingDim>() != EDiaphragmDOFLayerProcessing::BackgroundOnly) return false;

		if (PermutationVector.Get<FDDOFScatterQualityDim>() != EDiaphragmDOFScatterQuality::Medium &&
			PermutationVector.Get<FDDOFScatterQualityDim>() != EDiaphragmDOFScatterQuality::High) return false;

		return FDiaphragmDOFShader::ShouldCompilePermutation(Parameters);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FDiaphragmDOFShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SCATTER_QUALITY_MEDIUM"), (int)EDiaphragmDOFScatterQuality::Medium);
		OutEnvironment.SetDefine(TEXT("SCATTER_QUALITY_HIGH"), (int)EDiaphragmDOFScatterQuality::High);
	}
};

class FDiaphragmDOFRecombineCS : public FDiaphragmDOFShader
{
	DECLARE_GLOBAL_SHADER(FDiaphragmDOFRecombineCS);
	SHADER_USE_PARAMETER_STRUCT(FDiaphragmDOFRecombineCS, FDiaphragmDOFShader);

	class FQualityDim : SHADER_PERMUTATION_INT("DIM_QUALITY", 3);

	using FPermutationDomain = TShaderPermutationDomain<
		FDiaphragmDOFShader::FAlphaChannelDim,
		FDDOFLayerProcessingDim,
		FDDOFBokehSimulationDim,
		FQualityDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDDOFLayerProcessingDim>() != EDiaphragmDOFLayerProcessing::ForegroundOnly &&
		    PermutationVector.Get<FDDOFLayerProcessingDim>() != EDiaphragmDOFLayerProcessing::BackgroundOnly &&
			PermutationVector.Get<FDDOFLayerProcessingDim>() != EDiaphragmDOFLayerProcessing::ForegroundAndBackground) return false;

		// Do not compile bokeh simulation shaders on platform that couldn't handle them anyway.
		if (!SupportsBokehSimmulation(Parameters.Platform) &&
			PermutationVector.Get<FDDOFBokehSimulationDim>() != EDiaphragmDOFBokehSimulation::Disabled) return false;

		return FDiaphragmDOFShader::ShouldCompilePermutation(Parameters);
	}
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDOFCommonShaderParameters, CommonParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(DiaphragmDOF::FDOFCocModelShaderParameters, CocModel)

		SHADER_PARAMETER(FIntRect, ViewportRect)
		SHADER_PARAMETER(FVector4f, ViewportSize)
		SHADER_PARAMETER(FScreenTransform, DispatchThreadIdToDOFBufferUV)
		SHADER_PARAMETER(FVector2f, DOFBufferUVMax)
		SHADER_PARAMETER(FVector4f, SeparateTranslucencyBilinearUVMinMax)
		SHADER_PARAMETER(int32, SeparateTranslucencyUpscaling)
		SHADER_PARAMETER(float, EncodedCocRadiusToRecombineCocRadius)
		SHADER_PARAMETER(float, MaxRecombineAbsCocRadius)
		SHADER_PARAMETER(float, SOFHistoryRejectionBoost)

		SHADER_PARAMETER(FVector4f, SceneColorExtentAndInvExtent)
		SHADER_PARAMETER(FVector4f, SceneColorViewRectUV)
		
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BokehLUT)

		// Full res textures.
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorInput)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneSeparateCoc)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneSeparateTranslucency)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneSeparateTranslucencyModulateColor)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, LowResDepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, FullResDepthTexture)

		// Half res convolution textures.
		SHADER_PARAMETER(FVector4f, ConvolutionInputSize)
		SHADER_PARAMETER_STRUCT(FDOFConvolutionTextures, ForegroundConvolution)
		SHADER_PARAMETER_STRUCT(FDOFConvolutionTextures, ForegroundHoleFillingConvolution)
		SHADER_PARAMETER_STRUCT(FDOFConvolutionTextures, SlightOutOfFocusConvolution)
		SHADER_PARAMETER_STRUCT(FDOFConvolutionTextures, BackgroundConvolution)
		SHADER_PARAMETER(FVector2f, SeparateTranslucencyTextureLowResExtentInverse)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SceneColorOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFSetupCS,            "/Engine/Private/DiaphragmDOF/DOFSetup.usf",                     "SetupCS",                SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFCocFlattenCS,       "/Engine/Private/DiaphragmDOF/DOFCocTileFlatten.usf",            "CocFlattenMainCS",       SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFCocDilateCS,        "/Engine/Private/DiaphragmDOF/DOFCocTileDilate.usf",             "CocDilateMainCS",        SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFDownsampleCS,       "/Engine/Private/DiaphragmDOF/DOFDownsample.usf",                "DownsampleCS",           SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFReduceCS,           "/Engine/Private/DiaphragmDOF/DOFReduce.usf",                    "ReduceCS",               SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFScatterGroupPackCS, "/Engine/Private/DiaphragmDOF/DOFHybridScatterCompilation.usf",  "ScatterGroupPackMainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFBuildBokehLUTCS,    "/Engine/Private/DiaphragmDOF/DOFBokehLUT.usf",                  "BuildBokehLUTMainCS",    SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFGatherCS,           "/Engine/Private/DiaphragmDOF/DOFGatherPass.usf",                "GatherMainCS",           SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFPostfilterCS,       "/Engine/Private/DiaphragmDOF/DOFPostfiltering.usf",             "PostfilterMainCS",       SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFHybridScatterVS,    "/Engine/Private/DiaphragmDOF/DOFHybridScatterVertexShader.usf", "ScatterMainVS",          SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFHybridScatterPS,    "/Engine/Private/DiaphragmDOF/DOFHybridScatterPixelShader.usf",  "ScatterMainPS",          SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FDiaphragmDOFRecombineCS,        "/Engine/Private/DiaphragmDOF/DOFRecombine.usf",                 "RecombineMainCS",        SF_Compute);

} // namespace

namespace DiaphragmDOF
{
	class FDOFViewState
	{
	public:
		TStaticArray<TRefCountPtr<IPooledRenderTarget>, (int)EDiaphragmDOFBokehLUTFormat::MAX> BokehLUTs;
		FBokehModel BokehModel;

		FDOFViewState()
			: BokehLUTs()
			, BokehModel()
		{
		}
	};
}

bool DiaphragmDOF::IsEnabled(const FViewInfo& View)
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DepthOfFieldQuality"));
	check(CVar);

	const bool bDepthOfFieldRequestedByCVar = CVar->GetValueOnAnyThread() > 0;

	return
		DiaphragmDOF::IsSupported(View.GetShaderPlatform()) &&
		View.Family->EngineShowFlags.DepthOfField &&
		bDepthOfFieldRequestedByCVar &&
		!(View.Family->EngineShowFlags.PathTracing && PathTracing::UsesReferenceDOF(View)) &&
		View.FinalPostProcessSettings.DepthOfFieldEnabled &&
		(View.FinalPostProcessSettings.DepthOfFieldFocalDistance > 0.f || View.FinalPostProcessSettings.DepthOfFieldDepthBlurRadius > 0.f);
}

bool DiaphragmDOF::AddPasses(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfo& View,
	FRDGTextureRef InputSceneColor,
	const FTranslucencyPassResources& TranslucencyPassResources,
	FRDGTextureRef& OutputColor,
	FRDGTextureRef OriginalSceneDepthForStencil)
{
	if (View.Family->EngineShowFlags.VisualizeDOF)
	{
		// no need for this pass
		OutputColor = InputSceneColor;
		return false;
	}

	// Format of the scene color.
	EPixelFormat SceneColorFormat = InputSceneColor->Desc.Format;
	
	// Whether should process alpha channel of the scene or not.
	const bool bProcessSceneAlpha = IsPostProcessingWithAlphaChannelSupported();

	const EShaderPlatform ShaderPlatform = View.GetShaderPlatform();

	// Number of sampling ring in the gathering kernel.
	const int32 HalfResRingCount = FMath::Clamp(CVarRingCount.GetValueOnRenderThread(), kMinGatheringRingCount, MaxGatheringRingCount(ShaderPlatform));

	// Post filtering method to do.
	const EDiaphragmDOFPostfilterMethod PostfilterMethod = GetPostfilteringMethod();

	// The mode for hybrid scattering.
	const EHybridScatterMode FgdHybridScatteringMode = EHybridScatterMode(CVarHybridScatterForegroundMode.GetValueOnRenderThread());
	const EHybridScatterMode BgdHybridScatteringMode = EHybridScatterMode(CVarHybridScatterBackgroundMode.GetValueOnRenderThread());
	
	const float MinScatteringCocRadius = FMath::Max(CVarScatterMinCocRadius.GetValueOnRenderThread(), kMinScatteringCocRadius);

	// Whether the platform support gather bokeh simmulation.
	const bool bSupportGatheringBokehSimulation = SupportsBokehSimmulation(ShaderPlatform);

	// Whether should use shade permutation that does lower quality accumulation.
	// TODO: this is becoming a mess.
	const bool bUseLowAccumulatorQuality = CVarAccumulatorQuality.GetValueOnRenderThread() == 0;
	const bool bUseCinematicAccumulatorQuality = CVarAccumulatorQuality.GetValueOnRenderThread() == 2;

	// Setting for scattering budget upper bound.
	const float MaxScatteringRatio = FMath::Clamp(CVarScatterMaxSpriteRatio.GetValueOnRenderThread(), 0.0f, 1.0f);

	// Slight out of focus is not supporting with DOF's TAA upsampling, because of the brute force
	// kernel used in GatherCS for slight out of focus stability buffer.
	const bool bSupportsSlightOutOfFocus = true; // View.PrimaryScreenPercentageMethod != EPrimaryScreenPercentageMethod::TemporalUpscale;

	// Quality setting for the recombine pass.
	const int32 RecombineQuality = bSupportsSlightOutOfFocus ? FMath::Clamp(CVarRecombineQuality.GetValueOnRenderThread(), 0, kMaxRecombineQuality) : 0;

	// Resolution divisor.
	const int32 PrefilteringResolutionDivisor = CVarDOFGatherResDivisor.GetValueOnRenderThread() >= 2 ? 2 : 1;

	// Minimal absolute Coc radius to spawn a gather pass. Blurring radius under this are considered not great looking.
	// This is assuming the pass is opacity blending with a ramp from 1 to 2. This can not be exposed as a cvar,
	// because the slight out focus's lower res pass uses for full res convolution stability depends on this.
	const float kMinimalAbsGatherPassCocRadius = 1;

	// Minimal CocRadius to wire lower res gathering passes.
	const float BackgroundCocRadiusMaximumForUniquePass = HalfResRingCount * 4.0; // TODO: polish that.

	// Whether the recombine pass does slight out of focus convolution.
	const bool bRecombineDoesSlightOutOfFocus = RecombineQuality > 0;

	// Whether the recombine pass wants separate input buffer for foreground hole filling.
	const bool bRecombineDoesSeparateForegroundHoleFilling = RecombineQuality > 0;

	// Compute the required blurring radius to actually perform depth of field, that depends on whether
	// doing slight out of focus convolution.
	const float MinRequiredBlurringRadius = bRecombineDoesSlightOutOfFocus
		? (CVarMinimalFullresBlurRadius.GetValueOnRenderThread() * 0.5f)
		: kMinimalAbsGatherPassCocRadius;

	// Whether to use R11G11B10 + separate CoC buffer.
	const bool bPreferLowerBitDepth = CVarPreferLowerBitDepth.GetValueOnRenderThread() != 0;
	const bool bRGBBufferSeparateCocBuffer = (
		// Can't use FloatR11G11B10 if also need to support alpha channel.
		!bProcessSceneAlpha &&
		// Platform must support it
		SupportsRGBColorBuffer(ShaderPlatform) &&
		// This is just to get the number of shader permutation down.
		RecombineQuality == 0 &&
		bUseLowAccumulatorQuality &&
		// Maintain quality
		(bPreferLowerBitDepth || SceneColorFormat == PF_FloatR11G11B10));

	// Derives everything needed from the view.
	FSceneViewState* const ViewState = View.ViewState;

	FPhysicalCocModel CocModel;
	CocModel.Compile(View);

	FBokehModel BokehModel;
	BokehModel.Compile(View, CocModel);

	FDOFViewState* DOFViewState = nullptr;
	TStaticArray<bool, (int)EDiaphragmDOFBokehLUTFormat::MAX> HasCachedBokehLUT {};
	if (View.ViewState)
	{
		if (!View.ViewState->DepthOfFieldState.IsValid())
		{
			View.ViewState->DepthOfFieldState = MakePimpl<FDOFViewState>();
		}
		DOFViewState = View.ViewState->DepthOfFieldState.Get();

		if (DOFViewState->BokehModel != BokehModel)
		{
			for (int Index = 0; Index < (int)EDiaphragmDOFBokehLUTFormat::MAX; ++Index)
			{
				DOFViewState->BokehLUTs[Index] = nullptr;
			}
			DOFViewState->BokehModel = BokehModel;
		}

		for (int Index = 0; Index < (int)EDiaphragmDOFBokehLUTFormat::MAX; ++Index)
		{
			HasCachedBokehLUT[Index] = DOFViewState->BokehLUTs[Index] != nullptr;
		}
	}

	// Prepare preprocessing TAA pass.
	FTAAPassParameters TAAParameters(View);
	{
		TAAParameters.Pass = ETAAPassConfig::DiaphragmDOF;

		// When using dynamic resolution, the blur introduce by TAA's history resolution changes is quite noticeable on DOF.
		// Therefore we switch for a temporal upsampling technic to maintain the same history resolution.
		if (View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale)
		{
			TAAParameters.Pass = ETAAPassConfig::DiaphragmDOFUpsampling;
		}
		
		TAAParameters.SetupViewRect(View, PrefilteringResolutionDivisor);

		TAAParameters.Quality = CVarDOFTemporalAAQuality.GetValueOnRenderThread() == 0 ? ETAAQuality::Medium : ETAAQuality::High;
	}

	// Size of the view in GatherColorSetup.
	FIntPoint FullResViewSize = View.ViewRect.Size();
	FIntPoint PreprocessViewSize = FIntPoint::DivideAndRoundUp(FullResViewSize, PrefilteringResolutionDivisor);
	FIntPoint GatheringViewSize = PreprocessViewSize;

	if (IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod) && ViewState)
	{
		PreprocessViewSize = FIntPoint::DivideAndRoundUp(TAAParameters.OutputViewRect.Size(), PrefilteringResolutionDivisor);
	}

	// Basis of the coc encoded in all buffer
	const float EncodedCocRadiusBasis = PreprocessViewSize.X;
	const float GatheringCocRadiusBasis = float(GatheringViewSize.X);

	const float MaxBackgroundCocRadius = CocModel.ComputeViewMaxBackgroundCocRadius(GatheringViewSize.X);
	const float MinForegroundCocRadius = CocModel.ComputeViewMinForegroundCocRadius(GatheringViewSize.X);
	const float AbsMaxForegroundCocRadius = FMath::Abs(MinForegroundCocRadius);
	const float MaxBluringRadius = FMath::Max(MaxBackgroundCocRadius, AbsMaxForegroundCocRadius);

	// Whether should hybrid scatter for foreground and background.
	bool bForegroundHybridScattering = FgdHybridScatteringMode != EHybridScatterMode::Disabled && AbsMaxForegroundCocRadius > MinScatteringCocRadius && MaxScatteringRatio > 0.0f;
	bool bBackgroundHybridScattering = BgdHybridScatteringMode != EHybridScatterMode::Disabled && MaxBackgroundCocRadius > MinScatteringCocRadius && MaxScatteringRatio > 0.0f;

	if (!SupportsHybridScatter(ShaderPlatform))
	{
		bForegroundHybridScattering = false;
		bBackgroundHybridScattering = false;
	}

	// Compute the reference buffer size for PrefilteringResolutionDivisor.
	FIntPoint RefBufferSize = FIntPoint::DivideAndRoundUp(InputSceneColor->Desc.Extent, PrefilteringResolutionDivisor);
	
	EDiaphragmDOFBokehSimulation BokehSimulation = EDiaphragmDOFBokehSimulation::Disabled;
	if (BokehModel.BokehShape != EBokehShape::Circle)
	{
		BokehSimulation = (BokehModel.DiaphragmBladeCount % 2)
			? EDiaphragmDOFBokehSimulation::GenericBokeh
			: EDiaphragmDOFBokehSimulation::SimmetricBokeh;
	}

	// If the max blurring radius is too small, do not wire any passes.
	if (MaxBluringRadius < MinRequiredBlurringRadius)
	{
		OutputColor = InputSceneColor;
		return false;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, DepthOfField, "DOF(Alpha=%s)", bProcessSceneAlpha ? TEXT("Yes") : TEXT("No"));

	bool bGatherBackground = MaxBackgroundCocRadius > kMinimalAbsGatherPassCocRadius;
	bool bGatherForeground = AbsMaxForegroundCocRadius > kMinimalAbsGatherPassCocRadius;
	
	const bool bEnableGatherBokehSettings = (
		bSupportGatheringBokehSimulation &&
		CVarEnableGatherBokehSettings.GetValueOnRenderThread() == 1);
	const bool bEnableScatterBokehSettings = CVarEnableScatterBokehSettings.GetValueOnRenderThread() == 1;
	const bool bEnableSlightOutOfFocusBokeh = bSupportGatheringBokehSimulation && bRecombineDoesSlightOutOfFocus && CVarEnableRecombineBokehSettings.GetValueOnRenderThread();

	bool bEnableVignette = true;
		
	// Setup all the descriptors.
	FRDGTextureDesc FullResDesc;
	{
		FullResDesc = InputSceneColor->Desc;

		// Reset so that the number of samples of descriptor becomes 1, which is totally legal still with MSAA because
		// the scene color will already be resolved to ShaderResource texture that is always 1. This is to work around
		// hack that MSAA will have targetable texture with MSAA != shader resource, and still have descriptor indicating
		// the number of samples of the targetable resource.
		FullResDesc.Reset();

		FullResDesc.Format = PF_FloatRGBA;
		FullResDesc.Flags |= TexCreate_UAV;
		FullResDesc.Flags &= ~(TexCreate_RenderTargetable | TexCreate_FastVRAM);
	}
	
	FDOFGatherInputDescs FullResGatherInputDescs;
	{
		FullResGatherInputDescs.SceneColor = FullResDesc;
		FullResGatherInputDescs.SceneColor.Format = PF_FloatRGBA;
		
		FullResGatherInputDescs.SeparateCoc = FullResDesc;
		FullResGatherInputDescs.SeparateCoc.Format = PF_R16F;
	}

	FDOFGatherInputDescs HalfResGatherInputDescs;
	{
		HalfResGatherInputDescs.SceneColor = FullResDesc;
		HalfResGatherInputDescs.SceneColor.Extent /= PrefilteringResolutionDivisor;
		HalfResGatherInputDescs.SceneColor.Format = PF_FloatRGBA;
		HalfResGatherInputDescs.SceneColor.Flags |= GFastVRamConfig.DOFSetup;
		
		HalfResGatherInputDescs.SeparateCoc = FullResDesc;
		HalfResGatherInputDescs.SeparateCoc.Extent /= PrefilteringResolutionDivisor;
		HalfResGatherInputDescs.SeparateCoc.Format = PF_R16F;
		HalfResGatherInputDescs.SeparateCoc.Flags |= GFastVRamConfig.DOFSetup;
	}

	// Setup the shader parameter used in all shaders.
	FDOFCommonShaderParameters CommonParameters;
	CommonParameters.ViewUniformBuffer = View.ViewUniformBuffer;

	FDOFGatherInputTextures FullResGatherInputTextures;
	FIntRect FullResGatherInputViewRect = View.ViewRect;
	FDOFGatherInputTextures HalfResGatherInputTextures;
	FIntRect HalfResGatherInputViewRect = View.ViewRect / 2;

	// Setup at lower resolution from full resolution scene color and scene depth.
	{
		FullResGatherInputTextures = CreateTextures(GraphBuilder, FullResGatherInputDescs, TEXT("DOF.FullResSetup"));
		HalfResGatherInputTextures = CreateTextures(GraphBuilder, HalfResGatherInputDescs, TEXT("DOF.HalfResSetup"));

		bool bOutputFullResolution = (bRecombineDoesSlightOutOfFocus && !bProcessSceneAlpha) || PrefilteringResolutionDivisor == 1;
		bool bOutputHalfResolution = PrefilteringResolutionDivisor == 2;
		
		FDiaphragmDOFSetupCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FDiaphragmDOFShader::FAlphaChannelDim>(bProcessSceneAlpha);
	
		FIntPoint PassViewSize = FullResViewSize;
		if (bOutputFullResolution && bOutputHalfResolution)
		{
			PermutationVector.Set<FDiaphragmDOFSetupCS::FOutputResDivisor>(0);
		}
		else if (bOutputFullResolution)
		{
			PermutationVector.Set<FDiaphragmDOFSetupCS::FOutputResDivisor>(1);
		}
		else if (bOutputHalfResolution)
		{
			PermutationVector.Set<FDiaphragmDOFSetupCS::FOutputResDivisor>(2);
			PassViewSize = GatheringViewSize;
		}
		else
		{
			check(0);
		}
		
		FDiaphragmDOFSetupCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFSetupCS::FParameters>();
		{
			PassParameters->CommonParameters = CommonParameters;
			SetCocModelParameters(GraphBuilder, &PassParameters->CocModel, CocModel, EncodedCocRadiusBasis);
			PassParameters->SceneColorTexture = InputSceneColor;
			PassParameters->SceneDepthTexture = SceneTextures.SceneDepthTexture;
		
			if (!bOutputFullResolution)
			{
				FullResGatherInputTextures.SceneColor = InputSceneColor;
			}
			else if (bProcessSceneAlpha)
			{
				// No point passing through the full res scene color, the shader just output SeparateCoc.
				PassParameters->Output0 = CreateUAVs(GraphBuilder, FullResGatherInputTextures).SeparateCoc;
				FullResGatherInputTextures.SceneColor = InputSceneColor;
			}
			else
			{
				PassParameters->Output0 = CreateUAVs(GraphBuilder, FullResGatherInputTextures).SceneColor;
			}

			if (bOutputHalfResolution)
			{
				PassParameters->Output1 = CreateUAVs(GraphBuilder, HalfResGatherInputTextures).SceneColor;
				PassParameters->Output2 = CreateUAVs(GraphBuilder, HalfResGatherInputTextures).SeparateCoc;
			}
		}

		FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(PassViewSize, kDefaultGroupSize);

		TShaderMapRef<FDiaphragmDOFSetupCS> ComputeShader(View.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("DOF Setup(%s CoC=[%d;%d]) %dx%d",
				!bOutputFullResolution ? TEXT("HalfRes") : (!bOutputHalfResolution ? TEXT("FullRes") : TEXT("Full&HalfRes")),
				FMath::FloorToInt(CocModel.ComputeViewMinForegroundCocRadius(PassViewSize.X)),
				FMath::CeilToInt(CocModel.ComputeViewMaxBackgroundCocRadius(PassViewSize.X)),
				PassViewSize.X, PassViewSize.Y),
			ComputeShader,
			PassParameters,
			GroupCount);

		if (!bOutputFullResolution || bProcessSceneAlpha)
		{
			FullResGatherInputTextures.SceneColor = InputSceneColor;
		}

		if (!bOutputHalfResolution)
		{
			check(PrefilteringResolutionDivisor == 1);
			HalfResGatherInputTextures = FullResGatherInputTextures;
			HalfResGatherInputViewRect = FullResGatherInputViewRect;
		}
	}
	
	// TAA the setup for the convolution to be temporally stable.
	if (IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod) && ViewState)
	{
		TAAParameters.SceneDepthTexture = SceneTextures.SceneDepthTexture;
		TAAParameters.SceneVelocityTexture = SceneTextures.GBufferVelocityTexture;
		TAAParameters.SceneColorInput = HalfResGatherInputTextures.SceneColor;
		TAAParameters.SceneMetadataInput = HalfResGatherInputTextures.SeparateCoc;
		TAAParameters.CoCBilateralFilterStrength = FMath::Clamp(CVarCoCBilateralFilterStrength.GetValueOnRenderThread(), 0.0f, 1.0f);
		TAAParameters.StencilDepthTexture = OriginalSceneDepthForStencil;

		FTAAOutputs TAAOutputs = AddTemporalAAPass(
			GraphBuilder,
			View,
			TAAParameters,
			View.PrevViewInfo.DOFSetupHistory,
			&ViewState->PrevFrameViewInfo.DOFSetupHistory);
		
		HalfResGatherInputTextures.SceneColor = TAAOutputs.SceneColor;
		HalfResGatherInputTextures.SeparateCoc = TAAOutputs.SceneMetadata;
		HalfResGatherInputViewRect = TAAParameters.OutputViewRect;
		HalfResGatherInputViewRect.Min = FIntPoint::DivideAndRoundUp(HalfResGatherInputViewRect.Min, TAAParameters.ResolutionDivisor);
		HalfResGatherInputViewRect.Max = FIntPoint::DivideAndRoundUp(HalfResGatherInputViewRect.Max, TAAParameters.ResolutionDivisor);

		HalfResGatherInputDescs.SceneColor = TAAOutputs.SceneColor->Desc;
		if (TAAOutputs.SceneMetadata)
		{
			HalfResGatherInputDescs.SeparateCoc = TAAOutputs.SceneMetadata->Desc;
		}
	}

	// Tile classify work than needs to be done.
	FDOFTileClassificationTextures TileClassificationTextures;
	{
		// Setup the descriptors for tile classification.
		FDOFTileClassificationDescs TileClassificationDescs;
		{
			FIntPoint MaxTileCount = CocTileGridSize(HalfResGatherInputTextures.SceneColor->Desc.Extent);
			
			TileClassificationDescs.Foreground = FRDGTextureDesc::Create2D(
				MaxTileCount, PF_G16R16F, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV);

			TileClassificationDescs.Background = FRDGTextureDesc::Create2D(
				MaxTileCount, PF_FloatRGBA, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_UAV);
		}

		// Adds a coc flatten pass.
		FDOFTileClassificationTextures FlattenedTileClassificationTextures;
		{
			static const TCHAR* OutputDebugNames[2] = {
				TEXT("DOF.FlattenFgdCoc"),
				TEXT("DOF.FlattenBgdCoc"),
			};
			FlattenedTileClassificationTextures = CreateTextures(GraphBuilder, TileClassificationDescs, OutputDebugNames);

			const FIntPoint HalfResExtent = HalfResGatherInputDescs.SceneColor.Extent;
			const FVector4f HalfResExtentAndInvExtent
			{
				float(HalfResExtent.X), float(HalfResExtent.Y), 1.0f / HalfResExtent.X, 1.0f / HalfResExtent.Y
			};
			const FVector4f HalfResViewRectUV
			{
				FVector2f(HalfResGatherInputViewRect.Min) / HalfResExtent, 
				(FVector2f(HalfResGatherInputViewRect.Max) - 0.5f) / HalfResExtent
			};

			FDiaphragmDOFCocFlattenCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFCocFlattenCS::FParameters>();
			PassParameters->CommonParameters = CommonParameters;

			PassParameters->Stride = FVector2f(
				PreprocessViewSize.X / float(GatheringViewSize.X),
				PreprocessViewSize.Y / float(GatheringViewSize.Y));
			PassParameters->GatherInputExtentAndInvExtent = HalfResExtentAndInvExtent;
			PassParameters->GatherInputViewRectUV = HalfResViewRectUV;
			PassParameters->GatherInput = HalfResGatherInputTextures;
			PassParameters->TileOutput = CreateUAVs(GraphBuilder, FlattenedTileClassificationTextures);
			
			FDiaphragmDOFCocFlattenCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FDiaphragmDOFShader::FAlphaChannelDim>(bProcessSceneAlpha);
			PermutationVector.Set<FDiaphragmDOFCocFlattenCS::FDoCocGather4>(PreprocessViewSize != GatheringViewSize);

			TShaderMapRef<FDiaphragmDOFCocFlattenCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DOF FlattenCoc(Gather4=%s) %dx%d",
					PermutationVector.Get<FDiaphragmDOFCocFlattenCS::FDoCocGather4>() ? TEXT("Yes") : TEXT("No"),
					GatheringViewSize.X, GatheringViewSize.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(GatheringViewSize, kDefaultGroupSize));
		}
		
		// Error introduced by the random offset of the gathering kernel's center.
		const float BluringRadiusErrorMultiplier = 1.0f + 1.0f / (HalfResRingCount + 0.5f);

		// Number of group to dispatch for dilate passes.
		const FIntPoint DilatePassViewSize = FIntPoint::DivideAndRoundUp(GatheringViewSize, kCocTileSize);
		const FIntVector DilateGroupCount = FComputeShaderUtils::GetGroupCount(DilatePassViewSize, kDefaultGroupSize);

		// Add one coc dilate pass.
		auto AddCocDilatePass = [&](
			EDiaphragmDOFDilateCocMode Mode,
			const FDOFTileClassificationTextures& TileInput,
			const FDOFTileClassificationTextures& DilatedTileMinMax,
			int32 SampleRadiusCount, int32 SampleOffsetMultipler)
		{
			FDOFTileClassificationDescs OutputDescs = TileClassificationDescs;
			const TCHAR* OutputDebugNames[2] = {
				TEXT("DOF.DilateFgdCoc"),
				TEXT("DOF.DilateBgdCoc"),
			};
			if (Mode == EDiaphragmDOFDilateCocMode::MinForegroundAndMaxBackground)
			{
				OutputDebugNames[0] = TEXT("DOF.DilateMinFgdCoc");
				OutputDebugNames[1] = TEXT("DOF.DilateMaxBgdCoc");
				OutputDescs.Foreground.Format = PF_R16F;
				OutputDescs.Background.Format = PF_R16F;
			}

			FDOFTileClassificationTextures TileClassificationOutputTextures = CreateTextures(GraphBuilder, OutputDescs, OutputDebugNames);
			
			FDiaphragmDOFCocDilateCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFCocDilateCS::FParameters>();
			PassParameters->CommonParameters = CommonParameters;
			PassParameters->ViewportRect = FIntRect(0, 0, DilatePassViewSize.X, DilatePassViewSize.Y);
			PassParameters->SampleOffsetMultipler = SampleOffsetMultipler;
			PassParameters->fSampleOffsetMultipler = SampleOffsetMultipler;
			PassParameters->CocRadiusToBucketDistanceUpperBound = (GatheringCocRadiusBasis / EncodedCocRadiusBasis) * BluringRadiusErrorMultiplier;
			PassParameters->BucketDistanceToCocRadius = 1.0f / PassParameters->CocRadiusToBucketDistanceUpperBound;
			PassParameters->TileInput = TileInput;
			PassParameters->DilatedTileMinMax = DilatedTileMinMax;
			PassParameters->TileOutput = CreateUAVs(GraphBuilder, TileClassificationOutputTextures);
			
			FDiaphragmDOFCocDilateCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FDDOFDilateRadiusDim>(SampleRadiusCount);
			PermutationVector.Set<FDDOFDilateModeDim>(Mode);
			// TODO: permutation to do foreground and background separately, to have higher occupancy?

			TShaderMapRef<FDiaphragmDOFCocDilateCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DOF DilateCoc(1/16 %s radius=%d step=%d) %dx%d",
					GetEventName(Mode), SampleRadiusCount, SampleOffsetMultipler,
					DilatePassViewSize.X, DilatePassViewSize.Y),
				ComputeShader,
				PassParameters,
				DilateGroupCount);

			return TileClassificationOutputTextures;
		};
		
		// Parameters for the dilate Coc passes.
		int32 DilateCount = 1;
		int32 SampleRadiusCount[3];
		int32 SampleDistanceMultiplier[3];
		{
			const int32 MaxSampleRadiusCount = kMaxCocDilateSampleRadiusCount;

			// Compute the maximum tile dilation.
			int32 MaximumTileDilation = FMath::CeilToInt(
				(MaxBluringRadius * BluringRadiusErrorMultiplier) / kCocTileSize);

			// There is always at least one dilate pass so that even small Coc radius conservatively dilate on next neighboor.
			int32 CurrentConvolutionRadius = FMath::Min(
				MaximumTileDilation, MaxSampleRadiusCount);
			
			SampleDistanceMultiplier[0] = 1;
			SampleRadiusCount[0] = CurrentConvolutionRadius;

			// If the theoric radius is too big, setup more dilate passes.
			for (int32 i = 1; i < UE_ARRAY_COUNT(SampleDistanceMultiplier); i++)
			{
				if (MaximumTileDilation <= CurrentConvolutionRadius)
				{
					break;
				}

				// Highest upper bound possible for SampleDistanceMultiplier to not step over any tile.
				int32 HighestPossibleMultiplierUpperBound = CurrentConvolutionRadius + 1;

				// Find out how many step we need to do on dilate radius.
				SampleRadiusCount[i] = FMath::Min(
					MaximumTileDilation / HighestPossibleMultiplierUpperBound,
					MaxSampleRadiusCount);

				// Find out ideal multiplier to not dilate an area too large.
				// TODO: Could add control over the radius of the last.
				int32 IdealMultiplier = FMath::DivideAndRoundUp(MaximumTileDilation - CurrentConvolutionRadius, SampleRadiusCount[1]); // TODO: why 1?

				SampleDistanceMultiplier[i] = FMath::Min(IdealMultiplier, HighestPossibleMultiplierUpperBound);

				CurrentConvolutionRadius += SampleRadiusCount[i] * SampleDistanceMultiplier[i];

				DilateCount++;
			}
		}
		
		if (DilateCount > 1)
		{
			// TODO.
			FDOFTileClassificationTextures MinMaxTexture = FlattenedTileClassificationTextures;

			// Dilate min foreground and max background coc radii first.
			for (int32 i = 0; i < DilateCount; i++)
			{
				MinMaxTexture = AddCocDilatePass(
					EDiaphragmDOFDilateCocMode::MinForegroundAndMaxBackground,
					MinMaxTexture, FDOFTileClassificationTextures(),
					SampleRadiusCount[i], SampleDistanceMultiplier[i]);
			}

			TileClassificationTextures = FlattenedTileClassificationTextures;

			// Dilates everything else.
			for (int32 i = 0; i < DilateCount; i++)
			{
				TileClassificationTextures = AddCocDilatePass(
					EDiaphragmDOFDilateCocMode::MinimalAbsoluteRadiuses,
					TileClassificationTextures, MinMaxTexture,
					SampleRadiusCount[i], SampleDistanceMultiplier[i]);
			}
		}
		else
		{
			TileClassificationTextures = AddCocDilatePass(
				EDiaphragmDOFDilateCocMode::StandAlone,
				FlattenedTileClassificationTextures, FDOFTileClassificationTextures(),
				SampleRadiusCount[0], SampleDistanceMultiplier[0]);
		}
	}

	bool bMatteBoxActive = false;
	TStaticArray<UE::Core::TAlignedElement<FUintVector4, SHADER_PARAMETER_ARRAY_ELEMENT_ALIGNMENT>, FPhysicalCocModel::MaxMatteBoxFlags> MatteBoxPlanes;

	for (int Index = 0; Index < FPhysicalCocModel::MaxMatteBoxFlags; ++Index)
	{
		float PlanePitch = (CocModel.MatteBoxFlags[Index].Pitch + 90.0f) * PI / 180.0;
		float PlaneRoll = CocModel.MatteBoxFlags[Index].Roll * PI / 180.0;
		float PlaneLength = CocModel.MatteBoxFlags[Index].Length;

		float SinPlanePitch;
		float CosPlanePitch;
		float SinPlaneRoll;
		float CosPlaneRoll;
		float RcpSinPlanePitch;

		FMath::SinCos(&SinPlanePitch, &CosPlanePitch, PlanePitch);
		FMath::SinCos(&SinPlaneRoll, &CosPlaneRoll, PlaneRoll);
		RcpSinPlanePitch = 1.0f / SinPlanePitch;

		uint32 Packed0 = FVector2DHalf(SinPlanePitch, CosPlanePitch).AsUInt32();
		uint32 Packed1 = FVector2DHalf(SinPlaneRoll, CosPlaneRoll).AsUInt32();
		uint32 Packed2 = *reinterpret_cast<uint32*>(&RcpSinPlanePitch);
		uint32 Packed3 = *reinterpret_cast<uint32*>(&PlaneLength);

		MatteBoxPlanes[Index] = FUintVector4(Packed0, Packed1, Packed2, Packed3);
		bMatteBoxActive = bMatteBoxActive || PlaneLength > 0;
	}
	bool bVignetteActive = CocModel.BarrelLength > 0 || bMatteBoxActive;

	// Add the reduce pass
	FDOFGatherInputTextures ReducedGatherInputTextures;
	FRDGBufferRef DrawIndirectParametersBuffer = nullptr;
	FRDGBufferRef ForegroundScatterDrawListBuffer = nullptr;
	FRDGBufferRef BackgroundScatterDrawListBuffer = nullptr;
	{
		FIntPoint SrcSize = HalfResGatherInputDescs.SceneColor.Extent;
		
		const FIntPoint HalfResExtent = HalfResGatherInputDescs.SceneColor.Extent;
		const FVector4f HalfResExtentAndInvExtent
		{
			float(HalfResExtent.X), float(HalfResExtent.Y), 1.0f / HalfResExtent.X, 1.0f / HalfResExtent.Y
		};
		const FVector4f HalfResViewRectUV
		{
			FVector2f(HalfResGatherInputViewRect.Min) / HalfResExtent, 
			(FVector2f(HalfResGatherInputViewRect.Max) - 0.5f) / HalfResExtent
		};
		
		// Compute the number of mip level required by the gathering pass.
		const int32 MipLevelsRequested = FMath::CeilToInt(FMath::Log2(MaxBluringRadius * 0.5 / HalfResRingCount)) + (bUseLowAccumulatorQuality ? 1 : 0);
		const int32 MipLevelCount = FMath::Clamp(MipLevelsRequested, 1, CVarGatherMaxMipMapCount.GetValueOnRenderThread());

		// Maximum number of scattering group per draw instance.
		// TODO: depends.
		const uint32 kMaxScatteringGroupPerInstance = 21;

		// Maximum number of scattering group allowed per frame.
		const uint32 MaxScatteringGroupCount = FMath::Max(MaxScatteringRatio * 0.25f * SrcSize.X * SrcSize.Y - kMaxScatteringGroupPerInstance, float(kMaxScatteringGroupPerInstance));

		// Allocate the reduced gather input textures.
		{
			FDOFGatherInputDescs ReducedGatherInputDescs = HalfResGatherInputDescs;
			ReducedGatherInputDescs.SceneColor.NumMips = MipLevelCount;
			ReducedGatherInputDescs.SceneColor.Flags = (ReducedGatherInputDescs.SceneColor.Flags & ~(TexCreate_FastVRAM)) | (ETextureCreateFlags)GFastVRamConfig.DOFReduce;
			
			// Make sure the mip 0 is a multiple of 2^NumMips so there is no per mip level UV conversion to do in the gathering shader.
			// Also make sure it is a multiple of group size because reduce shader unconditionally output Mip0.
			int32 Multiple = FMath::Max(1 << (MipLevelCount - 1), kDefaultGroupSize);
			ReducedGatherInputDescs.SceneColor.Extent.X = Multiple * FMath::DivideAndRoundUp(ReducedGatherInputDescs.SceneColor.Extent.X, Multiple);
			ReducedGatherInputDescs.SceneColor.Extent.Y = Multiple * FMath::DivideAndRoundUp(ReducedGatherInputDescs.SceneColor.Extent.Y, Multiple);
			
			ReducedGatherInputDescs.SeparateCoc = ReducedGatherInputDescs.SceneColor;
			ReducedGatherInputDescs.SeparateCoc.Format = HalfResGatherInputDescs.SeparateCoc.Format;
			
			if (bRGBBufferSeparateCocBuffer)
			{
				ReducedGatherInputDescs.SceneColor.Format = PF_FloatR11G11B10;
				ReducedGatherInputDescs.SeparateCoc.Format = PF_R16F;
			}

			ReducedGatherInputTextures = CreateTextures(GraphBuilder, ReducedGatherInputDescs, TEXT("DOF.Reduce"));
		}

		// Downsample the gather color setup to have faster neighborhood comparisons.
		FDOFGatherInputTextures QuarterResGatherInputTextures;
		FDOFGatherInputDescs QuarterResGatherInputDescs;
		{
			QuarterResGatherInputDescs = HalfResGatherInputDescs;
			QuarterResGatherInputDescs.SceneColor.Extent /= 2;
			QuarterResGatherInputDescs.SeparateCoc.Extent /= 2;
		}
		const FIntPoint QuarterResExtent = QuarterResGatherInputDescs.SceneColor.Extent;
		const FVector4f QuarterResExtentAndInvExtent
		{
			float(QuarterResExtent.X), float(QuarterResExtent.Y), 1.0f / QuarterResExtent.X, 1.0f / QuarterResExtent.Y
		};
		const FIntRect QuarterResGatherInputViewRect{ FIntPoint::ZeroValue, HalfResGatherInputViewRect.Size() / 2 }; // rebase to (0,0) and halve size
		const FVector4f QuarterResViewRectUV
		{
			FVector2f(QuarterResGatherInputViewRect.Min) / QuarterResExtent, 
			(FVector2f(QuarterResGatherInputViewRect.Max) - 0.5f) / QuarterResExtent
		};
		if (bForegroundHybridScattering || bBackgroundHybridScattering)
		{
			// Allocate quarter res textures.
			{
				// Lower the bit depth to speed up texture fetches in reduce pass, that is ok since this is used only for comparison purposes.
				if (bRGBBufferSeparateCocBuffer && !bProcessSceneAlpha)
					QuarterResGatherInputDescs.SceneColor.Format = PF_FloatR11G11B10;
			
				QuarterResGatherInputTextures = CreateTextures(GraphBuilder, QuarterResGatherInputDescs, TEXT("DOF.Downsample"));
			}

			FIntPoint PassViewSize = FIntPoint::DivideAndRoundUp(PreprocessViewSize, 2);

			FDiaphragmDOFDownsampleCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FDiaphragmDOFShader::FAlphaChannelDim>(bProcessSceneAlpha);

			FDiaphragmDOFDownsampleCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFDownsampleCS::FParameters>();
			PassParameters->CommonParameters = CommonParameters;
			PassParameters->ViewportRect = FIntRect(0, 0, PassViewSize.X, PassViewSize.Y);
			PassParameters->OutputCocRadiusMultiplier = GatheringCocRadiusBasis / EncodedCocRadiusBasis;
			PassParameters->GatherInputExtentAndInvExtent = HalfResExtentAndInvExtent;
			PassParameters->GatherInputViewRectUV = HalfResViewRectUV;
			PassParameters->GatherInput = HalfResGatherInputTextures;

			PassParameters->OutDownsampledGatherInput = CreateUAVs(GraphBuilder, QuarterResGatherInputTextures);
			
			TShaderMapRef<FDiaphragmDOFDownsampleCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DOF Downsample %dx%d", PassViewSize.X, PassViewSize.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(PassViewSize, kDefaultGroupSize));
		}
		
		// Create and clears buffers for indirect scatter.
		if (bForegroundHybridScattering || bBackgroundHybridScattering)
		{
			DrawIndirectParametersBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc<FRHIDrawIndexedIndirectParameters>(2), TEXT("DOFIndirectDrawParameters"));

			const uint32 PixelCountPerScatterGroup = 2 * 2;
			const uint32 VignetteDataPerPixel = 3;
			const uint32 ScatterDrawListDataPerPixel = 1 + VignetteDataPerPixel;
			const uint32 ScatterDrawListGroupStride = 1 + (PixelCountPerScatterGroup * ScatterDrawListDataPerPixel);

			FRDGBufferDesc DrawListDescs = FRDGBufferDesc::CreateStructuredDesc(sizeof(float) * 4, ScatterDrawListGroupStride * MaxScatteringGroupCount);
			if (bForegroundHybridScattering)
				ForegroundScatterDrawListBuffer = GraphBuilder.CreateBuffer(DrawListDescs, TEXT("DOF.ForegroundDrawList"));
			if (bBackgroundHybridScattering)
				BackgroundScatterDrawListBuffer = GraphBuilder.CreateBuffer(DrawListDescs, TEXT("DOF.BackgroundDrawList"));
		}
		
		// Number of mip level that has been reduced.
		int32 ReducedMipLevelCount;

		// Adds the first reduce pass.
		{
			const bool bMip0Disabled = false;
			int32 ProcessingMipLevelCount = FDiaphragmDOFReduceCS::GetNearestMipLevelReductionCount(MipLevelCount, IsPostProcessingWithAlphaChannelSupported(), bMip0Disabled);

			FIntPoint PassViewSize = PreprocessViewSize;

			FDiaphragmDOFReduceCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FDiaphragmDOFShader::FAlphaChannelDim>(bProcessSceneAlpha);
			PermutationVector.Set<FDiaphragmDOFReduceCS::FReduceMipCount>(ProcessingMipLevelCount);
			PermutationVector.Set<FDiaphragmDOFReduceCS::FHybridScatterForeground>(bForegroundHybridScattering);
			PermutationVector.Set<FDiaphragmDOFReduceCS::FHybridScatterBackground>(bBackgroundHybridScattering);
			PermutationVector.Set<FDDOFRGBColorBufferDim>(bRGBBufferSeparateCocBuffer);

			FDiaphragmDOFReduceCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFReduceCS::FParameters>();
			PassParameters->PassViewSize = PassViewSize;
			PassParameters->ScatteringViewportSize = FVector4f(GatheringViewSize.X, GatheringViewSize.Y, 1.0f / GatheringViewSize.X, 1.0f / GatheringViewSize.Y);
			PassParameters->ScatteringScaling = float(GatheringViewSize.X) / float(PreprocessViewSize.X);
			PassParameters->MaxScatteringGroupCount = MaxScatteringGroupCount;
			PassParameters->PreProcessingToProcessingCocRadiusFactor = GatheringCocRadiusBasis / EncodedCocRadiusBasis;
			PassParameters->BokehGatherDistinctionLimit = CVarGatherDistinctionLimit.GetValueOnRenderThread();
			PassParameters->MinScatteringCocRadius = MinScatteringCocRadius;
			PassParameters->NeighborCompareMaxColor = CVarScatterNeighborCompareMaxColor.GetValueOnRenderThread();
			SetCocModelParameters(GraphBuilder, &PassParameters->CocModel, CocModel, float(GatheringViewSize.X));

			PassParameters->LensImageDistance = 1.0f / (1.0f/CocModel.VerticalFocalLength - 1.0f/CocModel.FocusDistance);
			PassParameters->SensorSize = FVector2f(CocModel.SensorWidth, CocModel.SensorHeight);
			PassParameters->Aperture = CocModel.GetLensRadius().Y * 2.0f;
			PassParameters->BarrelRadius = CocModel.BarrelRadius;
			PassParameters->BarrelLength = bVignetteActive ? CocModel.BarrelLength : -1.0f;
			PassParameters->MatteBoxPlanes = MatteBoxPlanes;
			
			PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));
			PassParameters->CommonParameters = CommonParameters;

			PassParameters->GatherInputExtentAndInvExtent = HalfResExtentAndInvExtent;
			PassParameters->GatherInputViewRectUV = HalfResViewRectUV;
			PassParameters->GatherInput = CreateSRVs(GraphBuilder, HalfResGatherInputTextures);
			
			PassParameters->QuarterResGatherInputExtentAndInvExtent = QuarterResExtentAndInvExtent;
			PassParameters->QuarterResGatherInputViewRectUV = QuarterResViewRectUV;
			PassParameters->QuarterResGatherInput = QuarterResGatherInputTextures;

			for (int32 MipLevel = 0; MipLevel < ProcessingMipLevelCount; MipLevel++)
			{
				PassParameters->OutputMips[MipLevel] = CreateUAVs(GraphBuilder, ReducedGatherInputTextures, MipLevel);
			}

			if (bForegroundHybridScattering || bBackgroundHybridScattering)
			{
				PassParameters->OutScatterDrawIndirectParameters = GraphBuilder.CreateUAV(DrawIndirectParametersBuffer);
				if (ForegroundScatterDrawListBuffer)
					PassParameters->OutForegroundScatterDrawList = GraphBuilder.CreateUAV(ForegroundScatterDrawListBuffer);
				if (BackgroundScatterDrawListBuffer)
					PassParameters->OutBackgroundScatterDrawList = GraphBuilder.CreateUAV(BackgroundScatterDrawListBuffer);

				AddClearUAVPass(GraphBuilder, PassParameters->OutScatterDrawIndirectParameters, 0);
			}

			TShaderMapRef<FDiaphragmDOFReduceCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DOF Reduce(Mips=[0;%d] FgdScatter=%s BgdScatter=%s%s) %dx%d",
					ProcessingMipLevelCount - 1,
					bForegroundHybridScattering ? TEXT("Yes") : TEXT("No"),
					bBackgroundHybridScattering ? TEXT("Yes") : TEXT("No"),
					bRGBBufferSeparateCocBuffer ? TEXT(" R11G11B10") : TEXT(""),
					PassViewSize.X, PassViewSize.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(PassViewSize, kDefaultGroupSize));

			ReducedMipLevelCount = ProcessingMipLevelCount;
		}

		// Complete the reduction.
		while (ReducedMipLevelCount < MipLevelCount)
		{
			int32 ProcessingMipLevelCount = FMath::Min(MipLevelCount - ReducedMipLevelCount, IsPostProcessingWithAlphaChannelSupported() ? 2 : (kMaxMipLevelCount - 1));
			int32 InputMipLevel = ReducedMipLevelCount - 1;

			FIntPoint GatherInputExtent = ReducedGatherInputTextures.SceneColor->Desc.Extent;
			FIntPoint PassViewSize = FIntPoint::DivideAndRoundUp(PreprocessViewSize, 1 << InputMipLevel);

			FDiaphragmDOFReduceCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FDiaphragmDOFShader::FAlphaChannelDim>(bProcessSceneAlpha);
			PermutationVector.Set<FDiaphragmDOFReduceCS::FReduceMipCount>(ProcessingMipLevelCount + 1);
			PermutationVector.Set<FDDOFRGBColorBufferDim>(bRGBBufferSeparateCocBuffer);
			PermutationVector.Set<FDiaphragmDOFReduceCS::FDisableOutputMip0>(true);

			FDiaphragmDOFReduceCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFReduceCS::FParameters>();
			PassParameters->PassViewSize = PassViewSize;

			PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));
			PassParameters->CommonParameters = CommonParameters;

			const FVector4f ViewRectUV
			{ 
				0.0f, 
				0.0f,
				(PreprocessViewSize.X - 0.5f) / GatherInputExtent.X,
				(PreprocessViewSize.Y - 0.5f) / GatherInputExtent.Y
			};

			float InputMipLevelPow2 = 1 << InputMipLevel;
			PassParameters->GatherInputExtentAndInvExtent = FVector4f(GatherInputExtent.X / InputMipLevelPow2, GatherInputExtent.Y / InputMipLevelPow2, InputMipLevelPow2 / GatherInputExtent.X, InputMipLevelPow2 / GatherInputExtent.Y);
			PassParameters->GatherInputViewRectUV = ViewRectUV;
			PassParameters->GatherInput = CreateSRVs(GraphBuilder, ReducedGatherInputTextures, InputMipLevel);

			for (int32 MipLevel = 0; MipLevel < ProcessingMipLevelCount; MipLevel++)
			{
				PassParameters->OutputMips[1 + MipLevel] = CreateUAVs(GraphBuilder, ReducedGatherInputTextures, ReducedMipLevelCount + MipLevel);
			}

			TShaderMapRef<FDiaphragmDOFReduceCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DOF Reduce(Mips=[%d;%d]%s) %dx%d",
					ReducedMipLevelCount,
					ReducedMipLevelCount + ProcessingMipLevelCount - 1,
					bRGBBufferSeparateCocBuffer ? TEXT(" R11G11B10") : TEXT(""),
					PassViewSize.X, PassViewSize.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(PassViewSize, kDefaultGroupSize));

			ReducedMipLevelCount += ProcessingMipLevelCount;
		}
		
		// Pack multiple scattering group on same primitive instance to increase wave occupancy in the scattering vertex shader.
		if (bForegroundHybridScattering || bBackgroundHybridScattering)
		{
			// TODO: could avoid multiple shader permutation by doing multiple passes with a no barrier UAV that isn't implemented yet.

			FDiaphragmDOFScatterGroupPackCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFScatterGroupPackCS::FParameters>();
			PassParameters->MaxScatteringGroupCount = MaxScatteringGroupCount;
			PassParameters->OutScatterDrawIndirectParameters = GraphBuilder.CreateUAV(DrawIndirectParametersBuffer);
			if (ForegroundScatterDrawListBuffer)
				PassParameters->OutForegroundScatterDrawList = GraphBuilder.CreateUAV(ForegroundScatterDrawListBuffer);
			if (BackgroundScatterDrawListBuffer)
				PassParameters->OutBackgroundScatterDrawList = GraphBuilder.CreateUAV(BackgroundScatterDrawListBuffer);

			FDiaphragmDOFScatterGroupPackCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FDiaphragmDOFReduceCS::FHybridScatterForeground>(bForegroundHybridScattering);
			PermutationVector.Set<FDiaphragmDOFReduceCS::FHybridScatterBackground>(bBackgroundHybridScattering);

			TShaderMapRef<FDiaphragmDOFScatterGroupPackCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DOF ScatterGroupPack"),
				ComputeShader,
				PassParameters,
				FIntVector(2, 1, 1));
		}
	}
	
	// Add a pass to build a bokeh LUTs.
	auto AddBuildBokehLUTPass = [&](EDiaphragmDOFBokehLUTFormat LUTFormat)
	{
		if (BokehModel.BokehShape == EBokehShape::Circle)
		{
			return FRDGTextureRef();
		}

		static const TCHAR* const DebugNames[] = {
			TEXT("DOF.ScatterBokehLUT"),
			TEXT("DOF.RecombineBokehLUT"),
			TEXT("DOF.GatherBokehLUT"),
		};

		FRDGTextureDesc BokehLUTDesc = FRDGTextureDesc::Create2D(
			FIntPoint(32, 32),
			LUTFormat == EDiaphragmDOFBokehLUTFormat::GatherSamplePos ? PF_G16R16F : PF_R16F,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV);

		// Retrieve cached BokehLUT from DOFViewState, if any
		if (HasCachedBokehLUT[int32(LUTFormat)])
		{
			return GraphBuilder.RegisterExternalTexture(DOFViewState->BokehLUTs[int32(LUTFormat)], DebugNames[int32(LUTFormat)]);
		}

		FRDGTextureRef BokehLUT = GraphBuilder.CreateTexture(BokehLUTDesc, DebugNames[int32(LUTFormat)]);

		FDiaphragmDOFBuildBokehLUTCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FDiaphragmDOFBuildBokehLUTCS::FBokehSimulationDim>(BokehModel.BokehShape == DiaphragmDOF::EBokehShape::RoundedBlades);
		PermutationVector.Set<FDiaphragmDOFBuildBokehLUTCS::FLUTFormatDim>(LUTFormat);
	
		FDiaphragmDOFBuildBokehLUTCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFBuildBokehLUTCS::FParameters>();
		PassParameters->BladeCount = BokehModel.DiaphragmBladeCount;
		PassParameters->DiaphragmRotation = BokehModel.DiaphragmRotation;
		PassParameters->CocRadiusToCircumscribedRadius = BokehModel.CocRadiusToCircumscribedRadius;
		PassParameters->CocRadiusToIncircleRadius = BokehModel.CocRadiusToIncircleRadius;
		PassParameters->DiaphragmBladeRadius = BokehModel.RoundedBlades.DiaphragmBladeRadius;
		PassParameters->DiaphragmBladeCenterOffset = BokehModel.RoundedBlades.DiaphragmBladeCenterOffset;
		PassParameters->CocSqueeze = BokehModel.Squeeze;
		PassParameters->CocInvSqueeze = 1.0f / BokehModel.Squeeze;
		PassParameters->BokehLUTOutput = GraphBuilder.CreateUAV(BokehLUT);
			
		TShaderMapRef<FDiaphragmDOFBuildBokehLUTCS> ComputeShader(View.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("DOF BuildBokehLUT(Blades=%d Shape=%s, LUT=%s) %dx%d",
				BokehModel.DiaphragmBladeCount,
				BokehModel.BokehShape == DiaphragmDOF::EBokehShape::RoundedBlades ? TEXT("Rounded") : TEXT("Straight"),
				GetEventName(LUTFormat),
				BokehLUTDesc.Extent.X, BokehLUTDesc.Extent.Y),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(BokehLUTDesc.Extent, kDefaultGroupSize));

		if (DOFViewState)
		{
			GraphBuilder.QueueTextureExtraction(BokehLUT, &DOFViewState->BokehLUTs[int32(LUTFormat)]);
		}

		return BokehLUT;
	};
	
	// Add all passes for convolutions.
	FDOFConvolutionTextures ForegroundConvolutionTextures;
	FDOFConvolutionTextures ForegroundHoleFillingConvolutionTextures;
	FDOFConvolutionTextures BackgroundConvolutionTextures;
	FDOFConvolutionTextures SlightOutOfFocusConvolutionTextures;
	{
		// High level configuration of a convolution
		struct FConvolutionSettings
		{
			/** Which layer to gather. */
			EDiaphragmDOFLayerProcessing LayerProcessing;

			/** Configuration of the pass. */
			EDiaphragmDOFGatherQuality QualityConfig;
		
			/** Postfilter method to apply on this gather pass. */
			EDiaphragmDOFPostfilterMethod PostfilterMethod;

			/** Bokeh simulation to do. */
			EDiaphragmDOFBokehSimulation BokehSimulation;
		
			/** Number of rings. */
			int32 RingCount;

			/** Whether there is a scattering pass. */
			bool bHasScatterPass = false;

			FConvolutionSettings()
				: LayerProcessing(EDiaphragmDOFLayerProcessing::ForegroundAndBackground)
				, QualityConfig(EDiaphragmDOFGatherQuality::HighQuality)
				, PostfilterMethod(EDiaphragmDOFPostfilterMethod::None)
				, BokehSimulation(EDiaphragmDOFBokehSimulation::Disabled)
			{ }
		};

		// Add a gather pass
		auto AddGatherPass = [&](
			const FConvolutionSettings& ConvolutionSettings,
			FRDGTextureRef BokehLUT,
			FDOFConvolutionTextures* ConvolutionOutputTextures,
			FRDGTextureRef* ScatterOcclusionTexture)
		{

			bool bUsesScatterOcclusionTexture =
				ConvolutionSettings.QualityConfig == EDiaphragmDOFGatherQuality::HighQualityWithHybridScatterOcclusion ||
				ConvolutionSettings.QualityConfig == EDiaphragmDOFGatherQuality::Cinematic;

			// Allocate output textures
			{
				FRDGTextureDesc Desc = ReducedGatherInputTextures.SceneColor->Desc;
				Desc.Extent = RefBufferSize;
				Desc.Format = PF_FloatRGBA;
				Desc.Flags |= TexCreate_UAV;
				Desc.NumMips = 1;

				{
					const TCHAR* DebugName = nullptr;
					if (ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::ForegroundOnly)
						DebugName = TEXT("DOF.GatherForeground");
					else if (ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::ForegroundHoleFilling)
						DebugName = TEXT("DOF.GatherForegroundFill");
					else if (ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::BackgroundOnly)
						DebugName = TEXT("DOF.GatherBackground");
					else if (ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::SlightOutOfFocus)
						DebugName = TEXT("DOF.GatherFocus");
					else
						check(0);

					FRDGTextureDesc SceneColorDesc = Desc;
					// Scattering pass will be drawing directly into the color of the gathered texture, so need to be render targetable.
					if (ConvolutionSettings.bHasScatterPass && ConvolutionSettings.PostfilterMethod == EDiaphragmDOFPostfilterMethod::None)
					{
						SceneColorDesc.Flags |= TexCreate_RenderTargetable;
					}

					ConvolutionOutputTextures->SceneColor = GraphBuilder.CreateTexture(SceneColorDesc, DebugName);
				
					if (bProcessSceneAlpha)
					{
						Desc.Format = PF_R16F;
						ConvolutionOutputTextures->SeparateAlpha = GraphBuilder.CreateTexture(Desc, DebugName);
					}
				}

				if (bUsesScatterOcclusionTexture)
				{
					Desc.Format = PF_G16R16F;
			
					const TCHAR* DebugName = nullptr;
					if (ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::BackgroundOnly)
						DebugName = TEXT("DOF.ScatterOcclusionBackground");
					else
						check(0);
			
					*ScatterOcclusionTexture = GraphBuilder.CreateTexture(Desc, DebugName);
				}
			}

			FIntPoint ReduceOutputRectMip0(
				kDefaultGroupSize * FMath::DivideAndRoundUp(PreprocessViewSize.X, kDefaultGroupSize),
				kDefaultGroupSize * FMath::DivideAndRoundUp(PreprocessViewSize.Y, kDefaultGroupSize));
		
			FIntPoint SrcSize = ReducedGatherInputTextures.SceneColor->Desc.Extent;

			FDiaphragmDOFGatherCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FDiaphragmDOFShader::FAlphaChannelDim>(bProcessSceneAlpha);
			PermutationVector.Set<FDDOFLayerProcessingDim>(ConvolutionSettings.LayerProcessing);
			PermutationVector.Set<FDDOFGatherRingCountDim>(ConvolutionSettings.RingCount);
			PermutationVector.Set<FDDOFGatherQualityDim>(ConvolutionSettings.QualityConfig);
			PermutationVector.Set<FDDOFBokehSimulationDim>(ConvolutionSettings.BokehSimulation);
			PermutationVector.Set<FDDOFRGBColorBufferDim>(bRGBBufferSeparateCocBuffer);
			PermutationVector = FDiaphragmDOFGatherCS::RemapPermutation(PermutationVector);

			// Affine transformtation to control whether a CocRadius is considered or not.
			FVector2f ConsiderCocRadiusAffineTransformation0 = kContantlyPassingAffineTransformation;
			FVector2f ConsiderCocRadiusAffineTransformation1 = kContantlyPassingAffineTransformation;
			FVector2f ConsiderAbsCocRadiusAffineTransformation = kContantlyPassingAffineTransformation;
			{
				// Coc radius considered.
				const float RecombineCocRadiusBorder = kMaxSlightOutOfFocusCocRadius - 1.0f;

				if (ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::ForegroundOnly)
				{
					ConsiderCocRadiusAffineTransformation0 = GenerateSaturatedAffineTransformation(
						-(RecombineCocRadiusBorder - 1.0f), -RecombineCocRadiusBorder);

					ConsiderAbsCocRadiusAffineTransformation = GenerateSaturatedAffineTransformation(
						RecombineCocRadiusBorder - 1.0f, RecombineCocRadiusBorder);
				}
				else if (ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::ForegroundHoleFilling)
				{
					ConsiderCocRadiusAffineTransformation0 = GenerateSaturatedAffineTransformation(
						RecombineCocRadiusBorder, RecombineCocRadiusBorder + 1.0f);
				}
				else if (ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::BackgroundOnly)
				{
					ConsiderCocRadiusAffineTransformation0 = GenerateSaturatedAffineTransformation(
						RecombineCocRadiusBorder - 1.0f, RecombineCocRadiusBorder);

					ConsiderAbsCocRadiusAffineTransformation = GenerateSaturatedAffineTransformation(
						RecombineCocRadiusBorder - 1.0f, RecombineCocRadiusBorder);
				}
				else if (ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::SlightOutOfFocus)
				{
					ConsiderAbsCocRadiusAffineTransformation = GenerateSaturatedAffineTransformation(
						RecombineCocRadiusBorder + 1.0f, RecombineCocRadiusBorder);
				}
				else
				{
					checkf(0, TEXT("What layer processing is that?"));
				}
			}

			FDiaphragmDOFGatherCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFGatherCS::FParameters>();
			PassParameters->ViewportSize = FVector4f(GatheringViewSize.X, GatheringViewSize.Y, 1.0f / GatheringViewSize.X, 1.0f / GatheringViewSize.Y);
			PassParameters->ViewportRect = FIntRect(0, 0, GatheringViewSize.X, GatheringViewSize.Y);
			PassParameters->TemporalJitterPixels = FVector2f(View.TemporalJitterPixels);	// LWC_TODO: Precision loss
			PassParameters->DispatchThreadIdToInputBufferUV = FVector2f(
				float(PreprocessViewSize.X) / float(GatheringViewSize.X * SrcSize.X),
				float(PreprocessViewSize.Y) / float(GatheringViewSize.Y * SrcSize.Y));;
			PassParameters->ConsiderCocRadiusAffineTransformation0 = ConsiderCocRadiusAffineTransformation0;
			PassParameters->ConsiderCocRadiusAffineTransformation1 = ConsiderCocRadiusAffineTransformation1;
			PassParameters->ConsiderAbsCocRadiusAffineTransformation = ConsiderAbsCocRadiusAffineTransformation;
			PassParameters->InputBufferUVToOutputPixel = FVector2f(
				float(SrcSize.X * GatheringViewSize.X) / float(PreprocessViewSize.X),
				float(SrcSize.Y * GatheringViewSize.Y) / float(PreprocessViewSize.Y));
			PassParameters->MipBias = FMath::Log2(float(PreprocessViewSize.X) / float(GatheringViewSize.X));
			PassParameters->MaxRecombineAbsCocRadius = float(kMaxSlightOutOfFocusCocRadius);
			PassParameters->CocSqueeze = CocModel.Squeeze;
			PassParameters->CocInvSqueeze = 1.0f / CocModel.Squeeze;
			
			PassParameters->Petzval = CocModel.Petzval;
			PassParameters->PetzvalFalloffPower = CocModel.PetzvalFalloffPower;
			PassParameters->PetzvalExclusionBoxExtents = CocModel.PetzvalExclusionBoxExtents;
			PassParameters->PetzvalExclusionBoxRadius = CocModel.PetzvalExclusionBoxRadius;

			PassParameters->TileDecisionParameters.MinGatherRadius = float(kMaxSlightOutOfFocusCocRadius) - 1;
			PassParameters->TileDecisionParameters.SlightOutOfFocusRadiusBoundary = float(kMaxSlightOutOfFocusCocRadius);
			PassParameters->CommonParameters = CommonParameters;
		
			PassParameters->GatherInputSize = FVector4f(SrcSize.X, SrcSize.Y, 1.0f / SrcSize.X, 1.0f / SrcSize.Y);
			PassParameters->GatherInputViewportSize = FVector2f(PreprocessViewSize.X, PreprocessViewSize.Y);
			PassParameters->GatherInput = ReducedGatherInputTextures;
		
			PassParameters->TileClassification = TileClassificationTextures;
			PassParameters->BokehLUT = BokehLUT;

			PassParameters->ConvolutionOutput = CreateUAVs(GraphBuilder, *ConvolutionOutputTextures);
			if (bUsesScatterOcclusionTexture)
			{
				PassParameters->ScatterOcclusionOutput = GraphBuilder.CreateUAV(*ScatterOcclusionTexture);
			}

			{
				FRDGTextureDesc DebugDesc = FRDGTextureDesc::Create2D(
					RefBufferSize,
					PF_FloatRGBA,
					FClearValueBinding::None,
					TexCreate_ShaderResource | TexCreate_UAV);
				PassParameters->DebugOutput = GraphBuilder.CreateUAV(GraphBuilder.CreateTexture(DebugDesc, TEXT("Debug.DOF.Gather")));
			}
			
			TShaderMapRef<FDiaphragmDOFGatherCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DOF Gather(%s %s Bokeh=%s Rings=%d%s) %dx%d",
					GetEventName(ConvolutionSettings.LayerProcessing),
					GetEventName(ConvolutionSettings.QualityConfig),
					GetEventName(ConvolutionSettings.BokehSimulation),
					int32(PermutationVector.Get<FDDOFGatherRingCountDim>()),
					PermutationVector.Get<FDDOFRGBColorBufferDim>() ? TEXT(" R11G11B10") : TEXT(""),
					GatheringViewSize.X, GatheringViewSize.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(GatheringViewSize, kDefaultGroupSize));
		}; // AddGatherPass()

		auto AddPostFilterPass = [&](
			const FConvolutionSettings& ConvolutionSettings,
			FDOFConvolutionTextures* ConvolutionTextures)
		{
			if (ConvolutionSettings.PostfilterMethod == EDiaphragmDOFPostfilterMethod::None)
			{
				return;
			}

			FDOFConvolutionTextures NewConvolutionTextures;
			{
				FRDGTextureDesc Desc = ConvolutionTextures->SceneColor->Desc;

				// Scattering pass will be drawing directly into the post filtered color texture, so need to be render targetable.
				if (ConvolutionSettings.bHasScatterPass)
				{
					Desc.Flags |= TexCreate_RenderTargetable;
				}

				NewConvolutionTextures.SceneColor = GraphBuilder.CreateTexture(Desc, ConvolutionTextures->SceneColor->Name);
			}

			if (ConvolutionTextures->SeparateAlpha)
			{
				NewConvolutionTextures.SeparateAlpha = GraphBuilder.CreateTexture(ConvolutionTextures->SeparateAlpha->Desc, ConvolutionTextures->SeparateAlpha->Name);
			}

			FDiaphragmDOFPostfilterCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FDiaphragmDOFShader::FAlphaChannelDim>(bProcessSceneAlpha);
			PermutationVector.Set<FDDOFLayerProcessingDim>(ConvolutionSettings.LayerProcessing);
			PermutationVector.Set<FDDOFPostfilterMethodDim>(ConvolutionSettings.PostfilterMethod);
			PermutationVector.Set<FDiaphragmDOFPostfilterCS::FTileOptimization>(true); // TODO
			PermutationVector = FDiaphragmDOFPostfilterCS::RemapPermutationVector(PermutationVector);
			
			float MaxRecombineAbsCocRadius = float(kMaxSlightOutOfFocusCocRadius);

			FDiaphragmDOFPostfilterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFPostfilterCS::FParameters>();
			PassParameters->ViewportRect = FIntRect(0, 0, GatheringViewSize.X, GatheringViewSize.Y);
			PassParameters->MaxInputBufferUV = FVector2f(
				(GatheringViewSize.X - 0.5f) / float(RefBufferSize.X),
				(GatheringViewSize.Y - 0.5f) / float(RefBufferSize.Y));
			PassParameters->TileDecisionParameters.MinGatherRadius = MaxRecombineAbsCocRadius - 1;
			PassParameters->CommonParameters = CommonParameters;

			PassParameters->ConvolutionInputSize = FVector4f(RefBufferSize.X, RefBufferSize.Y, 1.0f / RefBufferSize.X, 1.0f / RefBufferSize.Y);
			PassParameters->ConvolutionInput = *ConvolutionTextures;

			PassParameters->TileClassification = TileClassificationTextures;
			PassParameters->ConvolutionOutput = CreateUAVs(GraphBuilder, NewConvolutionTextures);
			
			{
				FRDGTextureDesc DebugDesc = FRDGTextureDesc::Create2D(
					RefBufferSize,
					PF_FloatRGBA,
					FClearValueBinding::None,
					TexCreate_ShaderResource | TexCreate_UAV);
				PassParameters->DebugOutput = GraphBuilder.CreateUAV(GraphBuilder.CreateTexture(DebugDesc, TEXT("Debug.DOF.PostFilter")));
			}

			TShaderMapRef<FDiaphragmDOFPostfilterCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("DOF Postfilter(%s %s%s) %dx%d",
					GetEventName(ConvolutionSettings.LayerProcessing), GetEventName(ConvolutionSettings.PostfilterMethod),
					PermutationVector.Get<FDiaphragmDOFPostfilterCS::FTileOptimization>() ? TEXT(" TileOptimisation") : TEXT(""),
					GatheringViewSize.X, GatheringViewSize.Y),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(GatheringViewSize, kDefaultGroupSize));
			
			*ConvolutionTextures = NewConvolutionTextures;
		}; // AddPostFilterPass()
		
		FRDGTextureRef GatheringBokehLUT = nullptr;
		if (bEnableGatherBokehSettings)
			 GatheringBokehLUT = AddBuildBokehLUTPass(EDiaphragmDOFBokehLUTFormat::GatherSamplePos);

		FRDGTextureRef ScatteringBokehLUT = nullptr;
		if (bEnableScatterBokehSettings || bEnableSlightOutOfFocusBokeh)
			ScatteringBokehLUT = AddBuildBokehLUTPass(EDiaphragmDOFBokehLUTFormat::CocRadiusToBokehEdgeFactor);

		auto AddHybridScatterPass = [&](
			const FConvolutionSettings& ConvolutionSettings,
			FDOFConvolutionTextures* ConvolutionTextures,
			FRDGTextureRef ScatterOcclusionTexture,
			FRDGBufferRef ScatterDrawList)
		{
			bool bIsForeground = ConvolutionSettings.LayerProcessing == EDiaphragmDOFLayerProcessing::ForegroundOnly;
			int32 DrawIndirectParametersOffset = bIsForeground ? 0 : 1;

			EDiaphragmDOFScatterQuality Quality {};
			if (ConvolutionSettings.QualityConfig >= EDiaphragmDOFGatherQuality::HighQuality)
			{
				Quality = EDiaphragmDOFScatterQuality::High;
			}
			else
			{
				Quality = EDiaphragmDOFScatterQuality::Medium;
			}

			FDiaphragmDOFHybridScatterVS::FPermutationDomain VSPermutationVector;
			VSPermutationVector.Set<FDDOFScatterQualityDim>(Quality);

			FDiaphragmDOFHybridScatterPS::FPermutationDomain PSPermutationVector;
			PSPermutationVector.Set<FDDOFLayerProcessingDim>(ConvolutionSettings.LayerProcessing);
			PSPermutationVector.Set<FDiaphragmDOFHybridScatterPS::FBokehSimulationDim>(ScatteringBokehLUT ? true : false);
			PSPermutationVector.Set<FDDOFScatterOcclusionDim>(ScatterOcclusionTexture ? true : false);
			PSPermutationVector.Set<FDDOFScatterQualityDim>(Quality);
			PSPermutationVector = FDiaphragmDOFHybridScatterPS::RemapPermutation(PSPermutationVector);

			TShaderMapRef<FDiaphragmDOFHybridScatterVS> VertexShader(View.ShaderMap, VSPermutationVector);
			TShaderMapRef<FDiaphragmDOFHybridScatterPS> PixelShader(View.ShaderMap, PSPermutationVector);

			FDOFHybridScatterParameters* PassParameters = GraphBuilder.AllocParameters<FDOFHybridScatterParameters>();
			PassParameters->ViewportSize = FVector4f(GatheringViewSize.X, GatheringViewSize.Y, 1.0f / GatheringViewSize.X, 1.0f / GatheringViewSize.Y);
			PassParameters->CocRadiusToCircumscribedRadius = BokehModel.CocRadiusToCircumscribedRadius;
			PassParameters->ScatteringScaling = float(GatheringViewSize.X) / float(PreprocessViewSize.X);
			PassParameters->Petzval = CocModel.Petzval;
			PassParameters->PetzvalFalloffPower = CocModel.PetzvalFalloffPower;
			PassParameters->PetzvalExclusionBoxExtents = CocModel.PetzvalExclusionBoxExtents;
			PassParameters->PetzvalExclusionBoxRadius = CocModel.PetzvalExclusionBoxRadius;
			PassParameters->CommonParameters = CommonParameters;
			SetCocModelParameters(GraphBuilder, &PassParameters->CocModel, CocModel, float(GatheringViewSize.X));
			PassParameters->BokehLUT = ScatteringBokehLUT;
			PassParameters->ScatterOcclusionSize = FVector4f(RefBufferSize.X, RefBufferSize.Y, 1.0f / RefBufferSize.X, 1.0f / RefBufferSize.Y);
			PassParameters->ScatterOcclusion = ScatterOcclusionTexture;
			PassParameters->IndirectDrawParameter = DrawIndirectParametersBuffer;
			PassParameters->ScatterDrawList = GraphBuilder.CreateSRV(ScatterDrawList);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(
				ConvolutionTextures->SceneColor, ERenderTargetLoadAction::ELoad);

			PassParameters->LensImageDistance = 1.0f / (1.0f/CocModel.VerticalFocalLength - 1.0f/CocModel.FocusDistance);
			PassParameters->SensorSize = FVector2f(CocModel.SensorWidth, CocModel.SensorHeight);
			PassParameters->Aperture = CocModel.GetLensRadius().Y * 2.0f;
				
			PassParameters->BarrelRadius = CocModel.BarrelRadius;
			PassParameters->BarrelLength = bVignetteActive ? CocModel.BarrelLength : -1.0f;
			PassParameters->MatteBoxPlanes = MatteBoxPlanes;

			ClearUnusedGraphResources(VertexShader, PixelShader, PassParameters);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("DOF IndirectScatter(%s Bokeh=%s Occlusion=%s) %dx%d",
					GetEventName(bIsForeground ? EDiaphragmDOFLayerProcessing::ForegroundOnly : EDiaphragmDOFLayerProcessing::BackgroundOnly),
					PSPermutationVector.Get<FDiaphragmDOFHybridScatterPS::FBokehSimulationDim>() ? TEXT("Generic") : TEXT("None"),
					PSPermutationVector.Get<FDDOFScatterOcclusionDim>() ? TEXT("Yes") : TEXT("No"),
					GatheringViewSize.X, GatheringViewSize.Y),
				PassParameters,
				ERDGPassFlags::Raster,
				[PassParameters, VertexShader, PixelShader, GatheringViewSize, DrawIndirectParametersOffset](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(0, 0, 0.0f, GatheringViewSize.X, GatheringViewSize.Y, 1.0f);

				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.PrimitiveType = GRHISupportsRectTopology ? PT_RectList : PT_TriangleList;
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

				RHICmdList.SetStreamSource(0, NULL, 0);

				// Marks the indirect draw parameter as used by the pass, given it's not used directly by any of the shaders.
				PassParameters->IndirectDrawParameter->MarkResourceAsUsed();

				if (GRHISupportsRectTopology)
				{
					RHICmdList.DrawPrimitiveIndirect(
						PassParameters->IndirectDrawParameter->GetIndirectRHICallBuffer(),
						sizeof(FRHIDrawIndirectParameters) * DrawIndirectParametersOffset);
				}
				else
				{
					RHICmdList.DrawIndexedPrimitiveIndirect(
						GDOFGlobalResource.ScatterIndexBuffer.IndexBufferRHI,
						PassParameters->IndirectDrawParameter->GetIndirectRHICallBuffer(),
						sizeof(FRHIDrawIndexedIndirectParameters) * DrawIndirectParametersOffset);
				}
			});
		}; // AddHybridScatterPass()

		struct FDOFPassDescriptor
		{
			FConvolutionSettings ConvolutionSettings;
			FRDGTextureRef BokehLUT;
			FDOFConvolutionTextures* ConvolutionTextures;
			FRDGBufferRef ScatterDrawList;
			FRDGTextureRef ScatterOcclusionTexture = nullptr;

			uint32 bDoGather : 1;
			uint32 bDoPostFilter : 1;
			uint32 bDoScatter : 1;
		};
		TArray<FDOFPassDescriptor, TInlineAllocator<4>> Passes;

		// Wire foreground gathering passes.
		if (bGatherForeground)
		{
			FConvolutionSettings ConvolutionSettings;
			ConvolutionSettings.LayerProcessing = EDiaphragmDOFLayerProcessing::ForegroundOnly;
			ConvolutionSettings.PostfilterMethod = PostfilterMethod;
			ConvolutionSettings.RingCount = HalfResRingCount;
			ConvolutionSettings.bHasScatterPass = bForegroundHybridScattering;

			if (bEnableGatherBokehSettings)
				ConvolutionSettings.BokehSimulation = BokehSimulation;

			if (bUseLowAccumulatorQuality)
			{
				ConvolutionSettings.QualityConfig = EDiaphragmDOFGatherQuality::LowQualityAccumulator;
			}

			Passes.Emplace(FDOFPassDescriptor {
				.ConvolutionSettings = ConvolutionSettings,
				.BokehLUT = GatheringBokehLUT,
				.ConvolutionTextures = &ForegroundConvolutionTextures,
				.ScatterDrawList = ForegroundScatterDrawListBuffer,
				
				.bDoGather = true,
				.bDoPostFilter = true,
				.bDoScatter = bForegroundHybridScattering
			});
		}

		// Wire hole filling gathering passes.
		if (bRecombineDoesSeparateForegroundHoleFilling)
		{
			FConvolutionSettings ConvolutionSettings;
			ConvolutionSettings.LayerProcessing = EDiaphragmDOFLayerProcessing::ForegroundHoleFilling;
			ConvolutionSettings.PostfilterMethod = PostfilterMethod;
			ConvolutionSettings.RingCount = HalfResRingCount;
			
			Passes.Emplace(FDOFPassDescriptor {
				.ConvolutionSettings = ConvolutionSettings,
				.BokehLUT = nullptr,
				.ConvolutionTextures = &ForegroundHoleFillingConvolutionTextures,
				
				.bDoGather = true,
				.bDoPostFilter = false,
				.bDoScatter = false
			});
		}
		
		// Gather slight out of focus.
		if (bRecombineDoesSlightOutOfFocus)
		{
			FConvolutionSettings ConvolutionSettings;
			ConvolutionSettings.LayerProcessing = EDiaphragmDOFLayerProcessing::SlightOutOfFocus;
			if (bEnableSlightOutOfFocusBokeh)
				ConvolutionSettings.BokehSimulation = BokehSimulation;

			// Number of rings is dynamic in shader.
			ConvolutionSettings.RingCount = kMinGatheringRingCount;

			Passes.Emplace(FDOFPassDescriptor {
				.ConvolutionSettings = ConvolutionSettings,
				.BokehLUT = bEnableSlightOutOfFocusBokeh ? ScatteringBokehLUT : nullptr,
				.ConvolutionTextures = &SlightOutOfFocusConvolutionTextures,
				
				.bDoGather = true,
				.bDoPostFilter = false,
				.bDoScatter = false
			});
		}

		// Wire background gathering passes.
		if (bGatherBackground)
		{
			FConvolutionSettings ConvolutionSettings;
			ConvolutionSettings.LayerProcessing = EDiaphragmDOFLayerProcessing::BackgroundOnly;
			ConvolutionSettings.PostfilterMethod = PostfilterMethod;
			ConvolutionSettings.RingCount = HalfResRingCount;
			ConvolutionSettings.bHasScatterPass = bBackgroundHybridScattering;

			if (bEnableGatherBokehSettings)
				ConvolutionSettings.BokehSimulation = BokehSimulation;

			ConvolutionSettings.QualityConfig = EDiaphragmDOFGatherQuality::LowQualityAccumulator;
			if (bBackgroundHybridScattering && BgdHybridScatteringMode == EHybridScatterMode::Occlusion)
			{
				if (bUseCinematicAccumulatorQuality)
				{
					ConvolutionSettings.QualityConfig = EDiaphragmDOFGatherQuality::Cinematic;
				}
				else
				{
					ConvolutionSettings.QualityConfig = EDiaphragmDOFGatherQuality::HighQualityWithHybridScatterOcclusion;
				}
			}
			
			Passes.Emplace(FDOFPassDescriptor {
				.ConvolutionSettings = ConvolutionSettings,
				.BokehLUT = GatheringBokehLUT,
				.ConvolutionTextures = &BackgroundConvolutionTextures,
				.ScatterDrawList = BackgroundScatterDrawListBuffer,
				
				.bDoGather = true,
				.bDoPostFilter = true,
				.bDoScatter = bBackgroundHybridScattering
			});
		}

		const bool bAvoidPSOSwitch = true;
		if (bAvoidPSOSwitch)
		{
			for (FDOFPassDescriptor& Pass : Passes)
			{
				if (Pass.bDoGather)
					AddGatherPass(Pass.ConvolutionSettings, Pass.BokehLUT, Pass.ConvolutionTextures, &Pass.ScatterOcclusionTexture);
			}
			for (FDOFPassDescriptor& Pass : Passes)
			{
				if (Pass.bDoPostFilter)
					AddPostFilterPass(Pass.ConvolutionSettings, Pass.ConvolutionTextures);
			}
			for (FDOFPassDescriptor& Pass : Passes)
			{
				if (Pass.bDoScatter)
					AddHybridScatterPass(Pass.ConvolutionSettings, Pass.ConvolutionTextures, Pass.ScatterOcclusionTexture, Pass.ScatterDrawList);
			}
		}
		else
		{
			for (FDOFPassDescriptor& Pass : Passes)
			{
				if (Pass.bDoGather)
					AddGatherPass(Pass.ConvolutionSettings, Pass.BokehLUT, Pass.ConvolutionTextures, &Pass.ScatterOcclusionTexture);

				if (Pass.bDoPostFilter)
					AddPostFilterPass(Pass.ConvolutionSettings, Pass.ConvolutionTextures);
			
				if (Pass.bDoScatter)
					AddHybridScatterPass(Pass.ConvolutionSettings, Pass.ConvolutionTextures, Pass.ScatterOcclusionTexture, Pass.ScatterDrawList);
			}
		}
	}
	
	// Recombine lower res out of focus with full res scene color.
	FRDGTextureRef NewSceneColor;
	{
		{
			FRDGTextureDesc Desc = InputSceneColor->Desc;
			Desc.Reset();
			Desc.Flags |= TexCreate_UAV;
			NewSceneColor = GraphBuilder.CreateTexture(Desc, TEXT("DOF.Recombine"));
		}

		bool bScaleSeparateTranslucency = false;
		FScreenPassTextureViewport SeparateTranslucencyViewport;
		if (TranslucencyPassResources.IsValid())
		{
			SeparateTranslucencyViewport = TranslucencyPassResources.GetTextureViewport();
			bScaleSeparateTranslucency = SeparateTranslucencyViewport.Rect.Size() != FullResViewSize;
		}
		else
		{
			SeparateTranslucencyViewport = FScreenPassTextureViewport(FIntPoint(0, 0), FIntRect(0, 0, 1, 1));
		}

		FIntRect PassViewRect = View.ViewRect;

		FDiaphragmDOFRecombineCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FDiaphragmDOFShader::FAlphaChannelDim>(bProcessSceneAlpha);
		if (bGatherForeground && bGatherBackground)
		{
			PermutationVector.Set<FDDOFLayerProcessingDim>(EDiaphragmDOFLayerProcessing::ForegroundAndBackground);
		}
		else if (bGatherForeground && !bGatherBackground)
		{
			PermutationVector.Set<FDDOFLayerProcessingDim>(EDiaphragmDOFLayerProcessing::ForegroundOnly);
		}
		else if (!bGatherForeground && bGatherBackground)
		{
			PermutationVector.Set<FDDOFLayerProcessingDim>(EDiaphragmDOFLayerProcessing::BackgroundOnly);
		}
		else
		{
			PermutationVector.Set<FDDOFLayerProcessingDim>(EDiaphragmDOFLayerProcessing::BackgroundOnly);
		}

		if (bEnableSlightOutOfFocusBokeh)
			PermutationVector.Set<FDDOFBokehSimulationDim>(BokehSimulation);
		PermutationVector.Set<FDiaphragmDOFRecombineCS::FQualityDim>(RecombineQuality);

		FRDGTextureRef SeparateTranslucency = TranslucencyPassResources.GetColorForRead(GraphBuilder);
		FRDGTextureRef SeparateTranslucencyDepth = TranslucencyPassResources.GetDepthForRead(GraphBuilder);
		FRDGTextureRef SeparateTranslucencyModulateColor = TranslucencyPassResources.GetColorModulateForRead(GraphBuilder);

		const FIntPoint FullResExtent = FullResGatherInputDescs.SceneColor.Extent;
		const FVector4f FullResExtentAndInvExtent
		{
			float(FullResExtent.X), float(FullResExtent.Y), 1.0f / FullResExtent.X, 1.0f / FullResExtent.Y
		};
		const FVector4f FullResViewRectUV
		{
			FVector2f(FullResGatherInputViewRect.Min) / FullResExtent, 
			(FVector2f(FullResGatherInputViewRect.Max) - 0.5f) / FullResExtent
		};

		FDiaphragmDOFRecombineCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDiaphragmDOFRecombineCS::FParameters>();
		PassParameters->CommonParameters = CommonParameters;
		SetCocModelParameters(GraphBuilder, &PassParameters->CocModel, CocModel, /* CocRadiusBasis = */ float(GatheringViewSize.X));

		PassParameters->ViewportRect = PassViewRect;
		PassParameters->ViewportSize = FVector4f(PassViewRect.Width(), PassViewRect.Height(), 1.0f / PassViewRect.Width(), 1.0f / PassViewRect.Height());
		PassParameters->DispatchThreadIdToDOFBufferUV = (
			FScreenTransform::DispatchThreadIdToViewportUV(PassViewRect) *
			FScreenTransform::ChangeTextureBasisFromTo(
				RefBufferSize,
				FIntRect(FIntPoint::ZeroValue, GatheringViewSize),
				FScreenTransform::ETextureBasis::ViewportUV,
				FScreenTransform::ETextureBasis::TextureUV)
			- FVector2f(View.TemporalJitterPixels) * FVector2f(1.0f / float(InputSceneColor->Desc.Extent.X), 1.0f / float(InputSceneColor->Desc.Extent.Y)));

		PassParameters->DOFBufferUVMax = FVector2f(
			(GatheringViewSize.X - 0.5f) / float(RefBufferSize.X),
			(GatheringViewSize.Y - 0.5f) / float(RefBufferSize.Y));
		PassParameters->EncodedCocRadiusToRecombineCocRadius = float(GatheringViewSize.X) / EncodedCocRadiusBasis; 
		PassParameters->MaxRecombineAbsCocRadius = float(kMaxSlightOutOfFocusCocRadius) * PassParameters->EncodedCocRadiusToRecombineCocRadius;
		PassParameters->SOFHistoryRejectionBoost = CVarSOFHistoryRejectionBoost.GetValueOnRenderThread();

		PassParameters->SeparateTranslucencyBilinearUVMinMax.X = (SeparateTranslucencyViewport.Rect.Min.X + 0.5f) / float(SeparateTranslucencyViewport.Extent.X);
		PassParameters->SeparateTranslucencyBilinearUVMinMax.Y = (SeparateTranslucencyViewport.Rect.Min.Y + 0.5f) / float(SeparateTranslucencyViewport.Extent.Y);
		PassParameters->SeparateTranslucencyBilinearUVMinMax.Z = (SeparateTranslucencyViewport.Rect.Max.X - 0.5f) / float(SeparateTranslucencyViewport.Extent.X);
		PassParameters->SeparateTranslucencyBilinearUVMinMax.W = (SeparateTranslucencyViewport.Rect.Max.Y - 0.5f) / float(SeparateTranslucencyViewport.Extent.Y);
		PassParameters->SeparateTranslucencyUpscaling = bScaleSeparateTranslucency ? 1 : 0;

		PassParameters->SceneColorExtentAndInvExtent = FullResExtentAndInvExtent;
		PassParameters->SceneColorViewRectUV = FullResViewRectUV;

		PassParameters->SceneColorInput = FullResGatherInputTextures.SceneColor;
		PassParameters->SceneDepthTexture = SceneTextures.SceneDepthTexture;
		PassParameters->SceneSeparateCoc = FullResGatherInputTextures.SeparateCoc; // TODO looks useless.
		PassParameters->SceneSeparateTranslucency = SeparateTranslucency;
		PassParameters->SceneSeparateTranslucencyModulateColor = SeparateTranslucencyModulateColor;
		
		PassParameters->ConvolutionInputSize = FVector4f(RefBufferSize.X, RefBufferSize.Y, 1.0f / RefBufferSize.X, 1.0f / RefBufferSize.Y);
		PassParameters->ForegroundConvolution = ForegroundConvolutionTextures;
		PassParameters->ForegroundHoleFillingConvolution = ForegroundHoleFillingConvolutionTextures;
		PassParameters->SlightOutOfFocusConvolution = SlightOutOfFocusConvolutionTextures;
		PassParameters->BackgroundConvolution = BackgroundConvolutionTextures;

		// Separate translucency upsampling
		PassParameters->FullResDepthTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMetaData(SceneTextures.SceneDepthTexture, ERDGTextureMetaDataAccess::Depth));
		PassParameters->LowResDepthTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMetaData(SeparateTranslucencyDepth, ERDGTextureMetaDataAccess::Depth));
		const FIntPoint LowResExtent = SeparateTranslucency->Desc.Extent;
		PassParameters->SeparateTranslucencyTextureLowResExtentInverse = FVector2f(1.0f / LowResExtent.X, 1.0f / LowResExtent.Y);

		if (!bGatherForeground && !bGatherBackground)
		{
			check(PassParameters->BackgroundConvolution.SceneColor == nullptr);
			check(PassParameters->BackgroundConvolution.SeparateAlpha == nullptr);
			PassParameters->BackgroundConvolution.SceneColor = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
			PassParameters->BackgroundConvolution.SeparateAlpha = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		}

		if (bEnableSlightOutOfFocusBokeh) // && ScatteringBokehLUTOutput.IsValid() && SlightOutOfFocusConvolutionOutput.IsValid())
		{
			PassParameters->BokehLUT = AddBuildBokehLUTPass(EDiaphragmDOFBokehLUTFormat::FullResOffsetToCocDistance);
		}

		PassParameters->SceneColorOutput = GraphBuilder.CreateUAV(NewSceneColor);

		{
			FRDGTextureDesc DebugDesc = FRDGTextureDesc::Create2D(
				InputSceneColor->Desc.Extent,
				PF_FloatRGBA,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_UAV);
			PassParameters->DebugOutput = GraphBuilder.CreateUAV(GraphBuilder.CreateTexture(DebugDesc, TEXT("Debug.DOF.Recombine")));
		}

		TShaderMapRef<FDiaphragmDOFRecombineCS> ComputeShader(View.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("DOF Recombine(%s Quality=%d Bokeh=%s%s) %dx%d",
				GetEventName(PermutationVector.Get<FDDOFLayerProcessingDim>()),
				RecombineQuality,
				GetEventName(PermutationVector.Get<FDDOFBokehSimulationDim>()),
				bScaleSeparateTranslucency ? TEXT(" RescaleSeparateTranslucency") : TEXT(""),
				PassViewRect.Width(), PassViewRect.Height()),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(PassViewRect.Size(), kDefaultGroupSize));
	}

	OutputColor = NewSceneColor;
	return true;
}
