// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessDOF.h: Post process Depth of Field implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "PostProcess/PostProcessing.h"
#include "Engine/Scene.h"

class FViewInfo;
class FSceneTextureParameters;
struct FTranslucencyPassResources;
struct FTemporalAAHistory;


namespace DiaphragmDOF
{

// Whether DOF is enabled for the requested view.
bool IsEnabled(const FViewInfo& View);

FVector4f CircleDofHalfCoc(const FViewInfo& View);

/** Physically based circle of confusion computation model. */
struct FPhysicalCocModel
{
	// Size of the sensor, in unreal unit.
	float SensorWidth;
	float SensorHeight;

	// Aspect ratio of the croped frame being rendered.
	float RenderingAspectRatio;

	// Focal length of the lens in unreal unit.
	float VerticalFocalLength;
	// float HorizontalFocalLength = VerticalFocalLength / Squeeze;

	// Apperture diameter in fstop
	float FStops; // = VerticalFocalLength / ApartureDiameter.

	// Unclamped background vertical coc radius, in horizontal ViewportUV unit.
	float InfinityBackgroundCocRadius;
	// HorizontalInfinityBackgroundCocRadius = InfinityBackgroundCocRadius / Squeeze
	// VerticalInfinityBackgroundCocRadius = InfinityBackgroundCocRadius

	/** Indicates whether a dynamic offset dependent on scene depth should be computed for every pixel */
	bool bEnableDynamicOffset;

	/** When dynamic offset is enabled, this is the coc radius at which objects will be perfectly sharp */
	float InFocusRadius;

	/** Radius offset lookup table */
	FRHITexture* DynamicRadiusOffsetLUT;

	// Resolution less minimal foreground coc radius < 0.
	float MinForegroundCocRadius;

	// Resolution less maximal background coc radius.
	float MaxBackgroundCocRadius;

	// Focus distance in unreal unit.
	float FocusDistance;

	// SqueezeFactor = VerticalFocalDistance / HorizontalFocalDistance
	float Squeeze;

	// Swirly bokeh Petzval lens effect
	float Petzval;
	float PetzvalFalloffPower;
	FVector2f PetzvalExclusionBoxExtents;
	float PetzvalExclusionBoxRadius;
	float BarrelRadius;
	float BarrelLength;

	static const uint32 MaxMatteBoxFlags = 3;
	TStaticArray<FMatteBoxFlag, MaxMatteBoxFlags> MatteBoxFlags;

	// The maximum radius of depth blur.
	float MaxDepthBlurRadius;
	float DepthBlurExponent;

	/** Compile the coc model from a view. */
	void Compile(const FViewInfo& View);
	
	/** Returns the lens radius in unreal unit from which the path traced ray should be traced from. */
	FVector2f GetLensRadius() const;

	/** Returns the CocRadius in half res pixels for given scene depth (in world unit).
	 *
	 * Notes: Matches Engine/Shaders/Private/DiaphragmDOF/Common.ush's SceneDepthToCocRadius().
	 */
	float DepthToResCocRadius(float SceneDepth, float HorizontalResolution) const;

	/** Returns limit(SceneDepthToCocRadius) for SceneDepth -> Infinity. */
	FORCEINLINE float ComputeViewMaxBackgroundCocRadius(float HorizontalResolution) const
	{
		// Dynamic CoC offset is designed to go to zero at infinite distance, so we don't need to factor it into the max background radius
		return FMath::Min(FMath::Max(InfinityBackgroundCocRadius, MaxDepthBlurRadius), MaxBackgroundCocRadius) * HorizontalResolution;
	}
	
	/** Returns limit(SceneDepthToCocRadius) for SceneDepth -> 0.
	 *
	 * Note: this return negative or null value since this is foreground.
	 */
	FORCEINLINE float ComputeViewMinForegroundCocRadius(float HorizontalResolution) const
	{
		return DepthToResCocRadius(GNearClippingPlane, HorizontalResolution);
	}

private:
	/** Gets the offset to the circle of confusion to apply for the specified radius */
	float GetCocOffset(float CocRadius) const;
};


enum class EBokehShape
{
	// No blade simulation.
	Circle,

	// Diaphragm's blades are straight.
	StraightBlades,

	// Diaphragm's blades are circle with a radius matching largest aperture of the lens system settings.
	RoundedBlades,
};


/** Model of bokeh to simulate a lens' diaphragm. */
struct FBokehModel
{
	// Shape of the bokeh.
	EBokehShape BokehShape;

	// Scale factor to transform a CocRadius to CircumscribedRadius or in circle radius.
	float CocRadiusToCircumscribedRadius;
	float CocRadiusToIncircleRadius;

	// Number of blades of the diaphragm.
	int32 DiaphragmBladeCount;
	
	// Rotation angle of the diaphragm in radians.
	float DiaphragmRotation;

	// BokehShape == RoundedBlades specific parameters.
	struct
	{
		// Radius of the blade for a boked area=PI.
		float DiaphragmBladeRadius;

		// Offset of the center of the blade's circle from the center of the bokeh.
		float DiaphragmBladeCenterOffset;
	} RoundedBlades;

	float Squeeze;

	/** Compile the model from a view. */
	void Compile(const FViewInfo& View, const FPhysicalCocModel& CocModel);

	bool operator==(const FBokehModel& Other) const
	{
		return BokehShape == Other.BokehShape
			&& CocRadiusToCircumscribedRadius == Other.CocRadiusToCircumscribedRadius
			&& CocRadiusToIncircleRadius == Other.CocRadiusToIncircleRadius
			&& DiaphragmBladeCount == Other.DiaphragmBladeCount
			&& DiaphragmRotation == Other.DiaphragmRotation
			&& RoundedBlades.DiaphragmBladeRadius == Other.RoundedBlades.DiaphragmBladeRadius
			&& RoundedBlades.DiaphragmBladeCenterOffset == Other.RoundedBlades.DiaphragmBladeCenterOffset
			&& Squeeze == Other.Squeeze;
	}

	bool operator!=(const FBokehModel& Other) const
	{
		return !(*this == Other);
	}
};

/** Returns whether DOF is supported. */
RENDERER_API bool IsSupported(const FStaticShaderPlatform ShaderPlatform);

/** Wire all DOF's passes according to view settings and cvars to convolve the scene color. */
RENDERER_API bool AddPasses(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfo& View,
	FRDGTextureRef InputSceneColor,
	const FTranslucencyPassResources& TranslucencyViewResources,
	FRDGTextureRef& OutputColor,
	FRDGTextureRef OriginalSceneDepthForStencil);


BEGIN_SHADER_PARAMETER_STRUCT(FDOFCocModelShaderParameters, )
	SHADER_PARAMETER(float, CocInfinityRadius)
	SHADER_PARAMETER(float, CocInFocusRadius)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DynamicRadiusOffsetLUT)
	SHADER_PARAMETER_SAMPLER(SamplerState, DynamicRadiusOffsetLUTSampler)
	SHADER_PARAMETER(uint32, bCocEnableDynamicRadiusOffset)
	SHADER_PARAMETER(float, CocMinRadius)
	SHADER_PARAMETER(float, CocMaxRadius)
	SHADER_PARAMETER(float, CocSqueeze)
	SHADER_PARAMETER(float, CocInvSqueeze)
	SHADER_PARAMETER(float, DepthBlurRadius)
	SHADER_PARAMETER(float, DepthBlurExponent)
END_SHADER_PARAMETER_STRUCT()

void SetCocModelParameters(
	FRDGBuilder& GraphBuilder,
	FDOFCocModelShaderParameters* OutParameters,
	const FPhysicalCocModel& CocModel,
	float CocRadiusBasis = 1.0f);

} // namespace DiaphragmDOF

