// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessDOF.cpp: Post process Depth of Field implementation.
=============================================================================*/

#include "PostProcess/DiaphragmDOF.h"
#include "SceneRendering.h"

namespace 
{

TAutoConsoleVariable<float> CVarMaxForegroundRadius(
	TEXT("r.DOF.Kernel.MaxForegroundRadius"),
	0.025f,
	TEXT("Maximum size of the foreground bluring radius in screen space (default=0.025)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarMaxBackgroundRadius(
	TEXT("r.DOF.Kernel.MaxBackgroundRadius"),
	0.025f,
	TEXT("Maximum size of the background bluring radius in screen space (default=0.025)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

} // namespace

// TODO: delete.
float ComputeFocalLengthFromFov(const FSceneView& View)
{
	// Convert FOV to focal length,
	// 
	// fov = 2 * atan(d/(2*f))
	// where,
	//   d = sensor dimension (APS-C 24.576 mm)
	//   f = focal length
	// 
	// f = 0.5 * d * (1/tan(fov/2))

	float const d = View.FinalPostProcessSettings.DepthOfFieldSensorWidth;
	float const HalfFOV = FMath::Atan(1.0f / View.ViewMatrices.GetViewToClip().M[0][0]);
	float const FocalLength = 0.5f * d * (1.0f/FMath::Tan(HalfFOV));

	return FocalLength;
}

// Convert f-stop and focal distance into projected size in half resolution pixels.
// Setup depth based blur.
// TODO: This logic does not account for the Squeeze factor in the same way as the logic below. See JIRA UE-203727
FVector4f DiaphragmDOF::CircleDofHalfCoc(const FViewInfo& View)
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DepthOfFieldQuality"));
	bool bDepthOfField = View.Family->EngineShowFlags.DepthOfField && CVar->GetValueOnRenderThread() > 0 
		&& View.FinalPostProcessSettings.DepthOfFieldEnabled && View.FinalPostProcessSettings.DepthOfFieldFocalDistance > 0;

	FVector4f Ret(0, 1, 0, 0);

	if(bDepthOfField)
	{
		float FocalLengthInMM = ComputeFocalLengthFromFov(View); // TODO for Material.
	 
		// Convert focal distance in world position to mm (from cm to mm)
		float FocalDistanceInMM = View.FinalPostProcessSettings.DepthOfFieldFocalDistance * 10.0f;

		// Convert f-stop, focal length, and focal distance to
		// projected circle of confusion size at infinity in mm.
		//
		// coc = f * f / (n * (d - f))
		// where,
		//   f = focal length
		//   d = focal distance
		//   n = fstop (where n is the "n" in "f/n")
		float Radius = FMath::Square(FocalLengthInMM) / (View.FinalPostProcessSettings.DepthOfFieldFstop * (FocalDistanceInMM - FocalLengthInMM));

		// Convert mm to pixels.
		float const Width = (float)View.ViewRect.Width();
		float const SensorWidth = View.FinalPostProcessSettings.DepthOfFieldSensorWidth;
		Radius = Radius * Width * (1.0f / SensorWidth);

		// Convert diameter to radius at half resolution (algorithm radius is at half resolution).
		Radius *= 0.25f;

		// Comment out for now, allowing settings which the algorithm cannot cleanly do.
		#if 0
			// Limit to algorithm max size.
			if(Radius > 6.0f) 
			{
				Radius = 6.0f; 
			}
		#endif

		// The DepthOfFieldDepthBlurAmount = km at which depth blur is 50%.
		// Need to convert to cm here.
		Ret = FVector4f(
			Radius, 
			1.0f / (View.FinalPostProcessSettings.DepthOfFieldDepthBlurAmount * 100000.0f),
			View.FinalPostProcessSettings.DepthOfFieldDepthBlurRadius * Width / 1920.0f,
			Width / 1920.0f);
	}

	return Ret;
}

void DiaphragmDOF::FPhysicalCocModel::Compile(const FViewInfo& View)
{
	// Fetches lens and filmback settings settings.
	{
		const float MMToUU = 0.1f;

		FocusDistance = View.FinalPostProcessSettings.DepthOfFieldFocalDistance;
		FStops = View.FinalPostProcessSettings.DepthOfFieldFstop;
		Squeeze = FMath::Clamp(View.FinalPostProcessSettings.DepthOfFieldSqueezeFactor, 0.1f, 10.0f);
		Petzval = View.FinalPostProcessSettings.DepthOfFieldPetzvalBokeh;
		PetzvalFalloffPower = FMath::Clamp(View.FinalPostProcessSettings.DepthOfFieldPetzvalBokehFalloff, 0, 100.0);
		PetzvalExclusionBoxExtents = View.FinalPostProcessSettings.DepthOfFieldPetzvalExclusionBoxExtents;
		PetzvalExclusionBoxRadius = View.FinalPostProcessSettings.DepthOfFieldPetzvalExclusionBoxRadius;
		
		static_assert(MaxMatteBoxFlags == UE_ARRAY_COUNT(View.FinalPostProcessSettings.DepthOfFieldMatteBoxFlags));
		for (int Index = 0; Index < MaxMatteBoxFlags; ++Index)
		{
			MatteBoxFlags[Index] = View.FinalPostProcessSettings.DepthOfFieldMatteBoxFlags[Index];
		}

		RenderingAspectRatio = float(View.UnscaledViewRect.Width()) / float(View.UnscaledViewRect.Height()) * View.FinalPostProcessSettings.DepthOfFieldAspectRatioScalar;
		const float HorizontalHalfFOV = FMath::Atan(1.0f / View.ViewMatrices.GetViewToClip().M[0][0]);
		const float VerticalHalfFOV = FMath::Atan(1.0f / View.ViewMatrices.GetViewToClip().M[1][1]);

		// If the focal length isn't set, compute based of the sensor height and vertical FOV.
		const float SensorAspectRatio = RenderingAspectRatio / Squeeze;
		SensorWidth  = View.FinalPostProcessSettings.DepthOfFieldSensorWidth * MMToUU;
		SensorHeight = SensorWidth / SensorAspectRatio;
		VerticalFocalLength = 0.5f * SensorHeight * (1.0f / FMath::Tan(VerticalHalfFOV));

		float Aperture = VerticalFocalLength / FStops;
		BarrelRadius = FMath::Max(View.FinalPostProcessSettings.DepthOfFieldBarrelRadius, (Aperture / 2) + 0.5f);
		BarrelLength = View.FinalPostProcessSettings.DepthOfFieldBarrelLength;
	}

	// Fetch the max bluring radius
	{
		// -because foreground Coc are negative.
		MinForegroundCocRadius = -CVarMaxForegroundRadius.GetValueOnRenderThread();
		MaxBackgroundCocRadius = CVarMaxBackgroundRadius.GetValueOnRenderThread();
	}

	// Fetch the depth blur.
	{
		MaxDepthBlurRadius = View.FinalPostProcessSettings.DepthOfFieldDepthBlurRadius / 1920.0f;

		// Circle DOF was actually computing in this depth blur radius in half res.
		MaxDepthBlurRadius *= 2.0f;

		DepthBlurExponent = 1.0f / (View.FinalPostProcessSettings.DepthOfFieldDepthBlurAmount * 100000.0f);
	}

	// Compile coc model equation.
	if (FStops > 0.f && FocusDistance > 0.f)
	{
		// Convert f-stop, focal length, and focal distance to
		// projected circle of confusion size of infinity on the sensor in unreal unit.
		//
		// coc = f * f / (n * (d - f))
		// where,
		//   f = focal length
		//   d = focal distance
		//   n = fstop (where n is the "n" in "f/n")
		float VerticalDiameter = FMath::Square(VerticalFocalLength) / (FStops * (FocusDistance - VerticalFocalLength));

		// Convert vertical diameter in unreal unit to radius on the filmback in vertical ViewportUV unit in uncropped viewport.
		float UncroppedVerticalInfinityBackgroundCocRadius = VerticalDiameter * 0.5f / SensorHeight;

		const float DesqueezedAspectRatio = SensorWidth / SensorHeight * Squeeze;
		float VerticalInfinityBackgroundCocRadius = UncroppedVerticalInfinityBackgroundCocRadius * FMath::Max(RenderingAspectRatio / DesqueezedAspectRatio, 1.0);

		// Convert diameter from vertical ViewportUV unit to horizontal ViewportUV unit.
		InfinityBackgroundCocRadius = VerticalInfinityBackgroundCocRadius / RenderingAspectRatio;

		if (View.InFocusDistance > 0)
		{
			// For now, the dynamic CoC offset only handles cases where the in-focus radius is positive (the in-focus distance is further than the focal point)
			// so clamp the in focus radius to always be positive
			InFocusRadius = FMath::Max(InfinityBackgroundCocRadius * (1 - FocusDistance / View.InFocusDistance), 0.0f);
			bEnableDynamicOffset = View.bEnableDynamicCocOffset;
			DynamicRadiusOffsetLUT = View.bEnableDynamicCocOffset ? View.DynamicCocOffsetLUT.GetReference() : nullptr;
		}
		else
		{
			InFocusRadius = 0.0;
			bEnableDynamicOffset = false;
			DynamicRadiusOffsetLUT = nullptr;
		}
	}
	else
	{
		InfinityBackgroundCocRadius = 0.0f;
		MinForegroundCocRadius = 0.0;
		InFocusRadius = 0.0;
		bEnableDynamicOffset = false;
		DynamicRadiusOffsetLUT = nullptr;
	}
}

FVector2f DiaphragmDOF::FPhysicalCocModel::GetLensRadius() const
{
	// Size of the vertical aperture in unreal unit.
	float ApertureDiameter = VerticalFocalLength / FStops;

	float VerticalLensRadius = 0.5f * ApertureDiameter;
	float HorizontalLensRadius = VerticalLensRadius / Squeeze;
	return FVector2f(HorizontalLensRadius, VerticalLensRadius);
}

float DiaphragmDOF::FPhysicalCocModel::DepthToResCocRadius(float SceneDepth, float HorizontalResolution) const
{
	float InitialCocRadius = ((SceneDepth - FocusDistance) / SceneDepth) * InfinityBackgroundCocRadius;
	float CocRadius = InitialCocRadius + GetCocOffset(InitialCocRadius);

	// Depth blur based.
	float DepthBlurAbsRadius = (1.0 - FMath::Exp2(-SceneDepth * DepthBlurExponent)) * MaxDepthBlurRadius;

	float ReturnCoc = FMath::Max(FMath::Abs(CocRadius), DepthBlurAbsRadius);
	if (CocRadius < 0.0)
	{
		// near CoC is using negative values
		ReturnCoc = -ReturnCoc;
	}
	return HorizontalResolution * FMath::Clamp(ReturnCoc, MinForegroundCocRadius, MaxBackgroundCocRadius);
}

float DiaphragmDOF::FPhysicalCocModel::GetCocOffset(float CocRadius) const
{
	float DynamicOffset = 0.0;
	if (bEnableDynamicOffset)
	{
		const float B = 0.467743 + 7.89656 * pow(InFocusRadius, -1.89051);
		const float V = -(2.57186 + 0.142159 * InFocusRadius);
		DynamicOffset = -InFocusRadius * (1 - pow(1 + exp(B * (InFocusRadius - CocRadius)), V));
	}

	return DynamicOffset;
}

void DiaphragmDOF::FBokehModel::Compile(const FViewInfo& View, const FPhysicalCocModel& CocModel)
{
	{
		DiaphragmBladeCount = FMath::Clamp(View.FinalPostProcessSettings.DepthOfFieldBladeCount, 4, 16);
	}

	float Fstop = View.FinalPostProcessSettings.DepthOfFieldFstop;
	float MinFstop = View.FinalPostProcessSettings.DepthOfFieldMinFstop > 0 ? View.FinalPostProcessSettings.DepthOfFieldMinFstop : 0;

	const float CircumscribedRadius = 1.0f;

	// Target a constant bokeh area to be eenergy preservative.
	const float TargetedBokehArea = PI * (CircumscribedRadius * CircumscribedRadius);

	Squeeze = CocModel.Squeeze;

	// Always uses circle if max aparture is smaller or equal to aperture. 
	if (Fstop <= MinFstop)
	{
		BokehShape = EBokehShape::Circle;

		CocRadiusToCircumscribedRadius = 1.0f;
		CocRadiusToIncircleRadius = 1.0f;
		DiaphragmBladeCount = 0;
		DiaphragmRotation = 0;
	}
	// Uses straight blades when max aperture is infinitely large. 
	else if (MinFstop == 0.0)
	{
		BokehShape = EBokehShape::StraightBlades;

		const float BladeCoverageAngle = PI / DiaphragmBladeCount;

		// Compute CocRadiusToCircumscribedRadius coc that the area of the boked remains identical,
		// to be energy conservative acorss the DiaphragmBladeCount.
		const float TriangleArea = ((CircumscribedRadius * CircumscribedRadius) *
			FMath::Cos(BladeCoverageAngle) *
			FMath::Sin(BladeCoverageAngle));
		const float CircleRadius = FMath::Sqrt(DiaphragmBladeCount * TriangleArea / TargetedBokehArea);

		CocRadiusToCircumscribedRadius = CircumscribedRadius / CircleRadius;
		CocRadiusToIncircleRadius = CocRadiusToCircumscribedRadius * FMath::Cos(PI / DiaphragmBladeCount);
		DiaphragmRotation = 0; // TODO.
	}
	else // if (BokehShape == EBokehShape::RoundedBlades)
	{
		BokehShape = EBokehShape::RoundedBlades;

		// Angle covered by a single blade in the bokeh.
		float BladeCoverageAngle = PI / DiaphragmBladeCount;

		// Blade radius for CircumscribedRadius == 1.0.
		// TODO: this computation is not very accurate.
		float BladeRadius = CircumscribedRadius * Fstop / MinFstop;

		// Visible angle of a single blade.
		float BladeVisibleAngle = FMath::Asin((CircumscribedRadius / BladeRadius) * FMath::Sin(BladeCoverageAngle));

		// Distance between the center of the blade's circle and center of the bokeh.
		float BladeCircleOffset = BladeRadius * FMath::Cos(BladeVisibleAngle) - CircumscribedRadius * FMath::Cos(BladeCoverageAngle);

		// Area of the triangle inscribed in the circle radius=CircumscribedRadius.
		float InscribedTriangleArea = ((CircumscribedRadius * CircumscribedRadius) *
			FMath::Cos(BladeCoverageAngle) *
			FMath::Sin(BladeCoverageAngle));

		// Area of the triangle inscribed in the circle radius=BladeRadius.
		float BladeInscribedTriangleArea = ((BladeRadius * BladeRadius) *
			FMath::Cos(BladeVisibleAngle) *
			FMath::Sin(BladeVisibleAngle));

		// Additional area added by the fact the blade has a circle shape and not a straight.
		float AdditonalCircleArea = PI * BladeRadius * BladeRadius * (BladeVisibleAngle / PI) - BladeInscribedTriangleArea;

		// Total area of the bokeh inscribed in circle radius=CircumscribedRadius.
		float InscribedBokedArea = DiaphragmBladeCount * (InscribedTriangleArea + AdditonalCircleArea);

		// Geometric upscale factor for to do target the desired bokeh area.
		float UpscaleFactor = FMath::Sqrt(TargetedBokehArea / InscribedBokedArea);

		// Compute the coordinate where the blade rotate.
		float BladePivotCenterX = 0.5 * (BladeRadius - CircumscribedRadius);
		float BladePivotCenterY = FMath::Sqrt(BladeRadius * BladeRadius - BladePivotCenterX * BladePivotCenterX);

		DiaphragmRotation = FMath::Atan2(BladePivotCenterX, BladePivotCenterY);

		RoundedBlades.DiaphragmBladeRadius = UpscaleFactor * BladeRadius;
		RoundedBlades.DiaphragmBladeCenterOffset = UpscaleFactor * BladeCircleOffset;

		CocRadiusToCircumscribedRadius = UpscaleFactor * CircumscribedRadius;
		CocRadiusToIncircleRadius = UpscaleFactor * (BladeRadius - BladeCircleOffset);
	}
}