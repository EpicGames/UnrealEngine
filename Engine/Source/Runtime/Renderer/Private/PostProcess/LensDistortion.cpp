// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/LensDistortion.h"

#include "PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"
#include "TemporalAA.h"

namespace
{
	
TAutoConsoleVariable<float> CVarLensDistortionLUTResolutionDivisor(
	TEXT("r.LensDistortion.LUTScreenPercentage"),
	100.0f * 256.0f / 3840.0f,
	TEXT("Screen percentage of the procedurally generated LUTs.\n"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarLensDistortionPaniniD(
	TEXT("r.LensDistortion.Panini.D"),
	0.0f,
	TEXT("Allow and configure to apply a panini distortion to the rendered image. Values between 0 and 1 allow to fade the effect (lerp).\n")
	TEXT("Implementation from research paper \"Pannini: A New Projection for Rendering Wide Angle Perspective Images\"\n")
	TEXT(" 0: off (default)\n")
	TEXT(">0: enabled (requires an extra post processing pass if upsampling wasn't used - see r.ScreenPercentage)\n")
	TEXT(" 1: Panini cylindrical stereographic projection\n")
	TEXT("Note: the projection is automatically disabled when the view FOV is too wide for Panini to\n")
	TEXT("represent (beyond ~140 degrees at D=1); the LUT cannot be built and the effect is skipped."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarLensDistortionPaniniS(
	TEXT("r.LensDistortion.Panini.S"),
	0.0f,
	TEXT("Panini projection's hard vertical compression factor.\n")
	TEXT(" 0: no vertical compression factor (default)\n")
	TEXT(" 1: Hard vertical compression"),
	ECVF_RenderThreadSafe);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST

TAutoConsoleVariable<float> CVarLensDistortionPaniniScreenFit(
	TEXT("r.LensDistortion.Panini.ScreenFit"),
	1.0f,
	TEXT("Panini projection screen fit effect factor (lerp) for debugging purposes.\n")
	TEXT(" 0: fit vertically\n")
	TEXT(" 1: fit horizontally (default)"),
	ECVF_RenderThreadSafe);

#endif

FVector2f PaniniProjection(FVector2f OM, float d, float s)
{
	float PaniniDirectionXZInvLength = 1.0f / FMath::Sqrt(1.0f + OM.X * OM.X);
	float SinPhi = OM.X * PaniniDirectionXZInvLength;
	float TanTheta = OM.Y * PaniniDirectionXZInvLength;
	float CosPhi = FMath::Sqrt(1.0f - SinPhi * SinPhi);
	float S = (d + 1.0f) / (d + CosPhi);

	return S * FVector2f(SinPhi, FMath::Lerp(TanTheta, TanTheta / CosPhi, s));
}

FVector2f PaniniInverseProjection(FVector2f ON, float d, float s)
{
	// line D equation through DN: A x + B z + C = 0
	float A = 1.0f + d;
	float B = -ON.X;
	float C = ON.X * d;

	// find intersection K(x,z) between line DN and unit circle with center O
	// solve: x^2 + z^2 = 1, z < 0, A x + B z + C = 0
	// ends up with polynom: a z^2 + b z + x = 0
	float a = 1.0f + (B * B) / (A * A);
	float b = 2.0f * (B * C) / (A * A);
	float c = (C * C) / (A * A) - 1.0f;

	float z = (-b - FMath::Sqrt(b * b - 4.0f * a * c)) / (2.0f * a);

	float CosPhi = -z;
	float SinPhi = FMath::Sqrt(1.0f - CosPhi * CosPhi);
	if (ON.X < 0.0f)
	{
		SinPhi = -SinPhi;
	}

	float S = (d + 1.0f) / (d + CosPhi);

	float OMx = SinPhi / CosPhi;
	float PaniniDirectionXZInvLength = 1.0f / FMath::Sqrt(1.0f + OMx * OMx);

	float TanTheta = ON.Y / (S * FMath::Lerp(1.0f, 1.0f / CosPhi, s));
	float OMy = TanTheta / PaniniDirectionXZInvLength;

	return FVector2f(OMx, OMy);
}


} //! namespace

// static
bool FPaniniProjectionConfig::IsEnabledByCVars()
{
	check(IsInRenderingThread());
	return CVarLensDistortionPaniniD.GetValueOnRenderThread() > 0.01f;
}

// static
FPaniniProjectionConfig FPaniniProjectionConfig::ReadCVars()
{
	check(FPaniniProjectionConfig::IsEnabledByCVars());

	FPaniniProjectionConfig Config;
	Config.D = CVarLensDistortionPaniniD.GetValueOnRenderThread();
	Config.S = CVarLensDistortionPaniniS.GetValueOnRenderThread();
	Config.Sanitize();
	return Config;
}

class FGeneratePaniniUVDisplacementCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGeneratePaniniUVDisplacementCS);
	SHADER_USE_PARAMETER_STRUCT(FGeneratePaniniUVDisplacementCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, ScreenSpaceToPaniniFactor)
		SHADER_PARAMETER(FVector2f, PaniniToScreenSpaceFactor)
		SHADER_PARAMETER(FScreenTransform, DispatchThreadIdToDestViewportUV)
		SHADER_PARAMETER(float, PaniniD)
		SHADER_PARAMETER(float, PaniniS)
		SHADER_PARAMETER(float, ScreenPosScale)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DistortingDisplacementOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, UndistortingDisplacementOutput)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FGeneratePaniniUVDisplacementCS, "/Engine/Private/PaniniProjection.usf", "MainCS", SF_Compute);

FLensDistortionLUT FPaniniProjectionConfig::GenerateLUTPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View) const
{
	check(IsEnabled());
	check(View.ViewMatrices.IsPerspectiveProjection());

	const float LUTResolutionFraction = FMath::Clamp(CVarLensDistortionLUTResolutionDivisor.GetValueOnRenderThread() / 100.0f, 0.25f, 100.0f);
	const FIntPoint SecondaryViewSize = View.GetSecondaryViewRectSize();

	const FIntPoint LUTResolution = FIntPoint(
		FMath::RoundToInt(LUTResolutionFraction * float(SecondaryViewSize.X)),
		FMath::RoundToInt(LUTResolutionFraction * float(SecondaryViewSize.Y)));

	const FVector2f FOVPerAxis = FVector2f(View.ViewMatrices.ComputeHalfFieldOfViewPerAxis());
	const FVector2f ScreenPosToPaniniFactor = FVector2f(FMath::Tan(FOVPerAxis.X), FMath::Tan(FOVPerAxis.Y));

	/** 
	* Panini is undefined at/near a 180 degree FOV.
	* Bail out with a disabled LUT so callers (which gate on FLensDistortionLUT::IsEnabled) simply skip the pass.
	*/
	if (!FMath::IsFinite(ScreenPosToPaniniFactor.X) || !FMath::IsFinite(ScreenPosToPaniniFactor.Y))
	{
		return FLensDistortionLUT();
	}

	// Compute the overscan adjustment.
	float ScreenPosScale;
	{
		FVector2f PaniniDirection = FVector2f(1.0f, 0.0f) * ScreenPosToPaniniFactor;
		FVector2f PaniniPosition = PaniniProjection(PaniniDirection, D, S);

		float WidthFit = ScreenPosToPaniniFactor.X / PaniniPosition.X;

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
		float ScreenFit = CVarLensDistortionPaniniScreenFit.GetValueOnRenderThread();
		ScreenPosScale = FMath::Lerp(1.0f, WidthFit, ScreenFit);
#else
		ScreenPosScale = WidthFit;
#endif
	}

	// Compute the resolution fraction happening at the center of distortion that end upscaling.
	float ResolutionFraction;
	{
		const float PrecisionMultiplier = 10.0f;
		FVector2f UndistortedScreenPos(PrecisionMultiplier / float(SecondaryViewSize.X), PrecisionMultiplier / float(SecondaryViewSize.Y));

		FVector2f PaniniPosition = UndistortedScreenPos * ScreenPosToPaniniFactor * (1.0f / ScreenPosScale);
		FVector2f PaniniDirection = PaniniInverseProjection(PaniniPosition, D, S);
		FVector2f DistortedScreenPos = PaniniDirection / ScreenPosToPaniniFactor;

		FVector2f ResolutionFractionVector = UndistortedScreenPos / DistortedScreenPos;
		ResolutionFraction = FMath::Max(ResolutionFractionVector.X, ResolutionFractionVector.Y);

		if (!FMath::IsFinite(ResolutionFraction) || ResolutionFraction <= 1.0f || ResolutionFraction >= 2.0f)
		{
			return FLensDistortionLUT();
		}
	}

	FLensDistortionLUT LensDistortionLUT;
	LensDistortionLUT.ResolutionFraction = ResolutionFraction;

	FGeneratePaniniUVDisplacementCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGeneratePaniniUVDisplacementCS::FParameters>();
	PassParameters->ScreenSpaceToPaniniFactor = ScreenPosToPaniniFactor;
	PassParameters->PaniniToScreenSpaceFactor = FVector2f(1.0f, 1.0f) / ScreenPosToPaniniFactor;
	PassParameters->DispatchThreadIdToDestViewportUV = FScreenTransform::DispatchThreadIdToViewportUV(FIntRect(FIntPoint::ZeroValue, LUTResolution));
	{
		PassParameters->PaniniD = D;
		PassParameters->PaniniS = S;
		PassParameters->ScreenPosScale = ScreenPosScale;
	}

	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			LUTResolution,
			PF_G32R32F,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV);

		LensDistortionLUT.DistortingDisplacementTexture   = GraphBuilder.CreateTexture(Desc, TEXT("Panini.DistortingDisplacement"));
		LensDistortionLUT.UndistortingDisplacementTexture = GraphBuilder.CreateTexture(Desc, TEXT("Panini.UndistortingDisplacement"));

		PassParameters->DistortingDisplacementOutput   = GraphBuilder.CreateUAV(LensDistortionLUT.DistortingDisplacementTexture);
		PassParameters->UndistortingDisplacementOutput = GraphBuilder.CreateUAV(LensDistortionLUT.UndistortingDisplacementTexture);
	}

	TShaderMapRef<FGeneratePaniniUVDisplacementCS> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("GeneratePaniniUVDisplacement %dx%d", LUTResolution.X, LUTResolution.Y),
		ComputeShader,
		PassParameters,
		FIntVector(FMath::DivideAndRoundUp(LUTResolution.X, 8), FMath::DivideAndRoundUp(LUTResolution.Y, 8), 2));

	return LensDistortionLUT;
}

FLensDistortionLUT FPaniniProjectionConfig::GenerateLUTPassesUnsafe(FRDGBuilder& GraphBuilder, const FSceneView& InView) const
{
	return GenerateLUTPasses(GraphBuilder, static_cast<const FViewInfo&>(InView));
}

LensDistortion::EPassLocation LensDistortion::GetPassLocation(const FViewInfo& InViewInfo)
{
	if (IsPostProcessingEnabled(InViewInfo)
		&& GetMainTAAPassConfig(InViewInfo) == EMainTAAPassConfig::TSR
		&& IsTSRLensDistortionEnabled(InViewInfo.GetShaderPlatform()))
	{
		return LensDistortion::EPassLocation::TSR;
	}
	else
	{
		return LensDistortion::EPassLocation::PrimaryUpscale;
	}
}

LensDistortion::EPassLocation LensDistortion::GetPassLocationUnsafe(const FSceneView& InView)
{
	check(InView.bIsViewInfo);

	return LensDistortion::GetPassLocation(static_cast<const FViewInfo&>(InView));
}

const FLensDistortionLUT& LensDistortion::GetLUTUnsafe(const FSceneView& InView)
{
	check(InView.bIsViewInfo);

	return static_cast<const FViewInfo&>(InView).LensDistortionLUT;
}

void LensDistortion::SetLUTUnsafe(FSceneView& InView, const FLensDistortionLUT& DistortionLUT)
{
	check(InView.bIsViewInfo);

	FViewInfo& ViewInfo = static_cast<FViewInfo&>(InView);

	ViewInfo.LensDistortionLUT = DistortionLUT;
}
