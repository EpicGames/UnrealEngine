// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralPostProcess.h"
#include "SystemTextures.h"
#include "MaterialShared.h"

TUniquePtr<INeuralPostProcessInterface> GNeuralPostProcess;

bool IsNeuralPostProcessEnabled()
{
	return GNeuralPostProcess.IsValid();
}

bool ShouldApplyNeuralPostProcessForMaterial(const FMaterial* Material)
{
	const bool bUseWithNeuralPostProcess = Material->IsUsedWithNeuralNetworks();
	const int32 NeuralProfileId = Material->GetNeuralProfileId();
	bool bApply = true;
	if (!bUseWithNeuralPostProcess ||
		!IsNeuralPostProcessEnabled() ||
		NeuralProfileId == INDEX_NONE)
	{
		bApply = false;
	}

	return bApply;
}

bool IsNeuralPostProcessShaderParameterUsed(FNeuralPostProcessShaderParameters& NeuralPostProcessShaderParameters)
{
	bool bInputNodeInUse = 
		NeuralPostProcessShaderParameters.InputNeuralBuffer != nullptr || 
		NeuralPostProcessShaderParameters.RWNeuralTexture != nullptr;

	return bInputNodeInUse;
}

FNeuralPostProcessShaderParameters GetDefaultNeuralPostProcessShaderParameters(
	FRDGBuilder& GraphBuilder
)
{
	FNeuralPostProcessShaderParameters NeuralPostProcessShaderParameters;
	NeuralPostProcessShaderParameters.NeuralTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy)));
	NeuralPostProcessShaderParameters.OutputNeuralBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 4), PF_R32_FLOAT);

	return NeuralPostProcessShaderParameters;
}

void SetupNeuralPostProcessShaderParametersForWrite(
	FNeuralPostProcessShaderParameters& NeuralPostProcessShaderParameters,
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef Texture,
	FRDGBufferRef Buffer,
	FVector4f Dimension,
	FRDGBufferRef SourceTypeBuffer
)
{
	NeuralPostProcessShaderParameters.RWNeuralTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Texture));
	NeuralPostProcessShaderParameters.InputNeuralBuffer = GraphBuilder.CreateUAV(Buffer, PF_R32_FLOAT);
	NeuralPostProcessShaderParameters.InputNeuralBufferDimension = Dimension;
	NeuralPostProcessShaderParameters.NeuralSourceType = GraphBuilder.CreateUAV(SourceTypeBuffer, PF_R32_UINT);
}

void SetupNeuralPostProcessShaderParametersForWrite(
	FNeuralPostProcessShaderParameters& NeuralPostProcessShaderParameters, 
	FRDGBuilder& GraphBuilder, 
	const FNeuralPostProcessResource& NeuralPostProcessResource
)
{
	SetupNeuralPostProcessShaderParametersForWrite(NeuralPostProcessShaderParameters, GraphBuilder,
		NeuralPostProcessResource.Texture,
		NeuralPostProcessResource.InputBuffer,
		NeuralPostProcessResource.InputBufferDimension,
		NeuralPostProcessResource.SourceTypeBuffer);
}

void SetupNeuralPostProcessShaderParametersForRead(FNeuralPostProcessShaderParameters& NeuralPostProcessShaderParameters, 
	FRDGBuilder& GraphBuilder, 
	FRDGTextureRef Texture, 
	FRDGBufferRef Buffer, 
	FVector4f Dimension
)
{
	if (Texture && Buffer)
	{
		NeuralPostProcessShaderParameters.NeuralTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(Texture));
		NeuralPostProcessShaderParameters.OutputNeuralBuffer = GraphBuilder.CreateSRV(Buffer, PF_R32_FLOAT);
		NeuralPostProcessShaderParameters.OutputNeuralBufferDimension = Dimension;
	}
}

void SetupNeuralPostProcessShaderParametersForRead(
	FNeuralPostProcessShaderParameters& NeuralPostProcessShaderParameters, 
	FRDGBuilder& GraphBuilder, 
	const FNeuralPostProcessResource& NeuralPostProcessResource
)
{
	SetupNeuralPostProcessShaderParametersForRead(
		NeuralPostProcessShaderParameters,
		GraphBuilder,
		NeuralPostProcessResource.Texture,
		NeuralPostProcessResource.OutputBuffer,
		NeuralPostProcessResource.OutputBufferDimension);
}

FNeuralPostProcessResource AllocateNeuralPostProcessingResourcesIfNeeded(
	FRDGBuilder& GraphBuilder,
	const FScreenPassTextureViewport& OutputViewport,
	int32 NeuralProfileId,
	bool bUsedWithNeuralPass)
{
	FNeuralPostProcessResource NeuralPostProcessResource;
	
	if (bUsedWithNeuralPass && GNeuralPostProcess)
	{
		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(
			OutputViewport.Extent,
			PF_FloatRGBA,
			FClearValueBinding(),
			TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV);

		// Allocate neural buffer and texture
		NeuralPostProcessResource.SourceTypeBuffer =
			GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1), TEXT("NeuralPostprocess.SourceType"));
		NeuralPostProcessResource.Texture = GraphBuilder.CreateTexture(TextureDesc, TEXT("NeuralPostprocess.Texture"));
		NeuralPostProcessResource.NeuralProfileId = NeuralProfileId;

		// Allocate and get the input buffer for the profile
		GNeuralPostProcess->AllocateBuffer(
			GraphBuilder,
			OutputViewport,
			NeuralProfileId,
			NeuralPostProcessResource.InputBuffer,
			NeuralPostProcessResource.InputBufferDimension);
	}

	return NeuralPostProcessResource;
}

void ApplyNeuralPostProcess(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View, 
	FIntRect Rect,
	FNeuralPostProcessResource& NeuralPostProcessResource
)
{
	if (IsNeuralPostProcessEnabled() && NeuralPostProcessResource.IsValid())
	{
		GNeuralPostProcess->Apply(
			GraphBuilder,
			NeuralPostProcessResource.NeuralProfileId,
			NeuralPostProcessResource.Texture,
			Rect,
			NeuralPostProcessResource.SourceTypeBuffer,
			NeuralPostProcessResource.OutputBuffer,
			NeuralPostProcessResource.OutputBufferDimension);
	}
}

