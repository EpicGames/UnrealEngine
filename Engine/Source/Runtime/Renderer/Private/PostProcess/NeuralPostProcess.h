// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "OverridePassSequence.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "PostProcess/NeuralPostProcessInterface.h"

BEGIN_SHADER_PARAMETER_STRUCT(FNeuralPostProcessShaderParameters, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWNeuralTexture)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, NeuralTexture)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float>, InputNeuralBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, NeuralSourceType)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, OutputNeuralBuffer)
	SHADER_PARAMETER(FVector4f, InputNeuralBufferDimension)
	SHADER_PARAMETER(FVector4f, OutputNeuralBufferDimension)
END_SHADER_PARAMETER_STRUCT()


FNeuralPostProcessShaderParameters GetDefaultNeuralPostProcessShaderParameters(
	FRDGBuilder& GraphBuilder
);

struct FNeuralPostProcessResource
{
	FRDGTextureRef Texture = nullptr;
	FRDGBufferRef InputBuffer = nullptr;
	FRDGBufferRef OutputBuffer = nullptr;
	FRDGBufferRef SourceTypeBuffer = nullptr;
	
	FVector4f InputBufferDimension;
	FVector4f OutputBufferDimension;

	int32 NeuralProfileId = INDEX_NONE;

	bool IsValid() const { return InputBuffer != nullptr && NeuralProfileId != INDEX_NONE;}
};

void SetupNeuralPostProcessShaderParametersForWrite(
	FNeuralPostProcessShaderParameters& NeuralPostProcessShaderParameters,
	FRDGBuilder& GraphBuilder,
	const FNeuralPostProcessResource& NeuralPostProcessResource);

void SetupNeuralPostProcessShaderParametersForRead(
	FNeuralPostProcessShaderParameters& NeuralPostProcessShaderParameters,
	FRDGBuilder& GraphBuilder,
	const FNeuralPostProcessResource& NeuralPostProcessResource);

FNeuralPostProcessResource AllocateNeuralPostProcessingResourcesIfNeeded(
	FRDGBuilder& GraphBuilder,
	const class FScreenPassTextureViewport& OutputViewport, 
	int32 NeuralProfileId,
	bool bUsedWithNeuralNetworks
);

// Apply neural post process to the resource
void ApplyNeuralPostProcess(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FIntRect Rect,
	FNeuralPostProcessResource& NeuralPostProcessResource
);

bool IsNeuralPostProcessEnabled();
bool ShouldApplyNeuralPostProcessForMaterial(const class FMaterial* Material);
bool IsNeuralPostProcessShaderParameterUsed(FNeuralPostProcessShaderParameters& NeuralPostProcessShaderParameters);
