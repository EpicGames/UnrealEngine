// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessAA.h"
#include "PostProcess/PostProcessing.h"
#include "PixelShaderUtils.h"
#include "SceneRendering.h"

DECLARE_GPU_STAT(FXAA);

TAutoConsoleVariable<int32> CVarFXAAQuality(
	TEXT("r.FXAA.Quality"), 4,
	TEXT("Selects the quality permutation of FXAA.\n")
	TEXT(" 0: Console\n")
	TEXT(" 1: PC medium-dither 3-sample\n")
	TEXT(" 2: PC medium-dither 5-sample\n")
	TEXT(" 3: PC medium-dither 8-sample\n")
	TEXT(" 4: PC low-dither 12-sample (Default)\n")
	TEXT(" 5: PC extrem quality 12-samples"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

class FFXAAPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FFXAAPS);
	SHADER_USE_PARAMETER_STRUCT(FFXAAPS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	class FAlphaChannelDim : SHADER_PERMUTATION_BOOL("DIM_ALPHA_CHANNEL");
	class FQualityDimension : SHADER_PERMUTATION_ENUM_CLASS("FXAA_PRESET", EFXAAQuality);
	using FPermutationDomain = TShaderPermutationDomain<FAlphaChannelDim, FQualityDimension>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER(FScreenTransform, SvPositionToInputTextureUV)
		SHADER_PARAMETER(FVector4f, fxaaConsoleRcpFrameOpt)
		SHADER_PARAMETER(FVector4f, fxaaConsoleRcpFrameOpt2)
		SHADER_PARAMETER(float, fxaaQualitySubpix)
		SHADER_PARAMETER(float, fxaaQualityEdgeThreshold)
		SHADER_PARAMETER(float, fxaaQualityEdgeThresholdMin)
		SHADER_PARAMETER(float, fxaaConsoleEdgeSharpness)
		SHADER_PARAMETER(float, fxaaConsoleEdgeThreshold)
		SHADER_PARAMETER(float, fxaaConsoleEdgeThresholdMin)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FFXAAPS, "/Engine/Private/FXAAShader.usf", "FxaaPS", SF_Pixel);

EFXAAQuality GetFXAAQuality()
{
	return EFXAAQuality(FMath::Clamp(CVarFXAAQuality.GetValueOnRenderThread(), 0, 5));
}

FScreenPassTexture AddFXAAPass(FRDGBuilder& GraphBuilder, const FSceneView& InSceneView, const FFXAAInputs& Inputs)
{
	
	check(Inputs.SceneColor.IsValid());
	check(Inputs.Quality != EFXAAQuality::MAX);

	checkSlow(InSceneView.bIsViewInfo);
	const FViewInfo& View = static_cast<const FViewInfo&>(InSceneView);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, FXAA, "FXAA");

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("FXAA"));
	}

	const FVector2f OutputExtentInverse = FVector2f(1.0f / (float)Output.Texture->Desc.Extent.X, 1.0f / (float)Output.Texture->Desc.Extent.Y);

	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FFXAAPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFXAAPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->Input = GetScreenPassTextureInput(Inputs.SceneColor, BilinearClampSampler);
	PassParameters->SvPositionToInputTextureUV = (
		FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
		FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Inputs.SceneColor), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));

	{
		float N = 0.5f;
		FVector4f Value(-N * OutputExtentInverse.X, -N * OutputExtentInverse.Y, N * OutputExtentInverse.X, N * OutputExtentInverse.Y);
		PassParameters->fxaaConsoleRcpFrameOpt = Value;
	}

	{
		float N = 2.0f;
		FVector4f Value(-N * OutputExtentInverse.X, -N * OutputExtentInverse.Y, N * OutputExtentInverse.X, N * OutputExtentInverse.Y);
		PassParameters->fxaaConsoleRcpFrameOpt2 = Value;
	}

	PassParameters->fxaaQualitySubpix = 0.75f;
	PassParameters->fxaaQualityEdgeThreshold = 0.166f;
	PassParameters->fxaaQualityEdgeThresholdMin = 0.0833f;
	PassParameters->fxaaConsoleEdgeSharpness = 8.0f;
	PassParameters->fxaaConsoleEdgeThreshold = 0.125f;
	PassParameters->fxaaConsoleEdgeThresholdMin = 0.05f;

	FFXAAPS::FPermutationDomain PixelPermutationVector;
	PixelPermutationVector.Set<FFXAAPS::FAlphaChannelDim>(IsPostProcessingWithAlphaChannelSupported());
	PixelPermutationVector.Set<FFXAAPS::FQualityDimension>(Inputs.Quality);

	TShaderMapRef<FFXAAPS> PixelShader(View.ShaderMap, PixelPermutationVector);

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME("FXAA(Quality=%d%s) %dx%d PS",
			Inputs.Quality,
			PixelPermutationVector.Get<FFXAAPS::FAlphaChannelDim>() ? TEXT(" Alpha") : TEXT(""),
			Output.ViewRect.Width(),
			Output.ViewRect.Height()),
		PixelShader,
		PassParameters,
		Output.ViewRect);

	return MoveTemp(Output);
}
