// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessAmbientOcclusionMobile.cpp
=============================================================================*/

#include "PostProcess/PostProcessAmbientOcclusionMobile.h"
#include "CompositionLighting/PostProcessAmbientOcclusion.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderParameterStruct.h"
#include "SceneRendering.h"
#include "RenderTargetPool.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/SceneRenderTargets.h"
#include "SystemTextures.h"
#include "ScreenPass.h"
#include "ScenePrivate.h"
#include "SceneTextureParameters.h"
#include "SceneRenderTargetParameters.h"
#include "ClearQuad.h"
#include "PixelShaderUtils.h"

extern FSSAOCommonParameters GetSSAOCommonParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	uint32 Levels,
	bool bAllowGBufferRead);

static TAutoConsoleVariable<int32> CVarMobileAmbientOcclusion(
	TEXT("r.Mobile.AmbientOcclusion"),
	0,
	TEXT("Caution: Affects only mobile Forward shading. An extra sampler will be occupied in mobile base pass pixel shader after enable the mobile ambient occlusion.\n")
	TEXT("0: Disable Ambient Occlusion on mobile platform. [default]\n")
	TEXT("1: Enable Ambient Occlusion on mobile platform.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarMobileSSAOHalfResolution(
	TEXT("r.Mobile.SSAOHalfResolution"),
	0,
	TEXT("Whether to calculate SSAO at half resolution.\n")
	TEXT("0: Disabled.\n")
	TEXT("1: Half Resolution with bilinear upsample\n")
	TEXT("2: Half Resolution with 4 tap bilateral upsample\n")
	TEXT("3: Half Resolution with 9 tap bilateral upsample\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileSSAOShadingModelExcludeMask(
	TEXT("r.Mobile.SSAOShadingModelsExcludeMask"),
	0,
	TEXT("ShadingModels mask that determines which pixels are excluded from SSAO computation.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

// --------------------------------------------------------------------------------------------------------------------
DECLARE_GPU_STAT_NAMED(MobileSSAO, TEXT("SSAO"));

class FMobileAmbientOcclusionUpsamplePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMobileAmbientOcclusionUpsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FMobileAmbientOcclusionUpsamplePS, FGlobalShader);

	class FUpsampleQualityDim : SHADER_PERMUTATION_INT("UPSAMPLE_QUALITY", 3);
	using FPermutationDomain = TShaderPermutationDomain <FUpsampleQualityDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AOTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, AOSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("UPSAMPLE_PASS"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FMobileAmbientOcclusionUpsamplePS, "/Engine/Private/PostProcessAmbientOcclusion.usf", "AmbientOcclusionUpsamplePS", SF_Pixel);

// --------------------------------------------------------------------------------------------------------------------
static void AddAmbientOcclusionUpsamplePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	int32 UpsampleQuality,
	FScreenPassTexture Input,
	FScreenPassRenderTarget Output)
{
	const bool bDepthBoundsTestEnabled = 
		FSSAOHelper::IsDepthBoundsTestEnabled()
		&& CommonParameters.SceneDepth.IsValid()
		&& CommonParameters.SceneDepth.Texture->Desc.NumSamples == 1;

	const FScreenPassTextureViewport OutputViewport(Output);

	FDepthStencilBinding DepthStencilBinding{}; 
	float DepthFar = 0.0f;
	if (bDepthBoundsTestEnabled)
	{
		DepthStencilBinding = FDepthStencilBinding(CommonParameters.SceneDepth.Texture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);
		
		const FFinalPostProcessSettings& Settings = View.FinalPostProcessSettings;
		const FMatrix& ProjectionMatrix = View.ViewMatrices.GetViewToClip();
		const FVector4f Far = (FVector4f)ProjectionMatrix.TransformFVector4(FVector4(0, 0, Settings.AmbientOcclusionFadeDistance));
		DepthFar = FMath::Min(1.0f, Far.Z / Far.W);

		FRenderTargetParameters* ClearParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
		ClearParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		ClearParameters->RenderTargets.DepthStencil = DepthStencilBinding;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("DepthBounds ClearQuad(%s)", Output.Texture->Name),
			ClearParameters,
			ERDGPassFlags::Raster,
			[OutputViewport, DepthFar](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				// We must clear all pixels that won't be touched by AO shader.
				FClearQuadCallbacks Callbacks;
				Callbacks.PSOModifier = [](FGraphicsPipelineStateInitializer& PSOInitializer)
				{
					PSOInitializer.bDepthBounds = true;
				};
				Callbacks.PreClear = [DepthFar](FRHICommandList& InRHICmdList)
				{
					// This is done by rendering a clear quad over a depth range from AmbientOcclusionFadeDistance to far plane.
					InRHICmdList.SetDepthBounds(0, DepthFar);	// NOTE: Inverted depth
				};
				Callbacks.PostClear = [DepthFar](FRHICommandList& InRHICmdList)
				{
					// Set depth bounds test to cover everything from near plane to AmbientOcclusionFadeDistance and run AO pixel shader.
					InRHICmdList.SetDepthBounds(DepthFar, 1.0f);
				};

				RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

				DrawClearQuad(RHICmdList, FLinearColor::White, Callbacks);
			});

		// Make sure the following pass doesn't clear or ignore the data
		Output.LoadAction = ERenderTargetLoadAction::ELoad;
	}
	
	FMobileAmbientOcclusionUpsamplePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileAmbientOcclusionUpsamplePS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	if (bDepthBoundsTestEnabled)
	{
		PassParameters->RenderTargets.DepthStencil = DepthStencilBinding;
	}

	PassParameters->View = GetShaderBinding(View.ViewUniformBuffer);
	PassParameters->SceneTextures = CommonParameters.SceneTextures;
	PassParameters->AOTexture = Input.Texture;
	PassParameters->AOSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	typename FMobileAmbientOcclusionUpsamplePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<typename FMobileAmbientOcclusionUpsamplePS::FUpsampleQualityDim>(UpsampleQuality);
	TShaderMapRef<FMobileAmbientOcclusionUpsamplePS> PixelShader(View.ShaderMap, PermutationVector);

	TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);

	ClearUnusedGraphResources(PixelShader, PassParameters);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("UpsamplePS Quality %d", UpsampleQuality),
		PassParameters,
		ERDGPassFlags::Raster,
		[PassParameters, &View, Output, OutputViewport, VertexShader, PixelShader, bDepthBoundsTestEnabled, DepthFar](FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			FPixelShaderUtils::InitFullscreenPipelineState(RHICmdList, View.ShaderMap, PixelShader, GraphicsPSOInit);
			GraphicsPSOInit.bDepthBounds = bDepthBoundsTestEnabled;
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			if (bDepthBoundsTestEnabled)
			{
				RHICmdList.SetDepthBounds(DepthFar, 1.0f);
			}

			FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList);
			
			if (bDepthBoundsTestEnabled)
			{
				RHICmdList.SetDepthBounds(0.0f, 1.0f);
			}
		});
}

static FScreenPassTexture AddPostProcessingAmbientOcclusionMobile(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSSAOCommonParameters& CommonParameters,
	FScreenPassRenderTarget FinalTarget)
{
	if (CommonParameters.MobileHalfResolutionSetting > 0)
	{
		const FScreenPassTextureViewport SceneViewport(CommonParameters.SceneTexturesViewport);
		const FScreenPassTextureViewport HalfResOutputViewport(GetDownscaledViewport(SceneViewport, 2));

		FScreenPassRenderTarget HalfResOutput;
		{
			bool bBilateralUpsample = (CommonParameters.MobileHalfResolutionSetting > 1);
			
			FRDGTextureDesc OutputDesc = FinalTarget.Texture->Desc;
			OutputDesc.Reset();
			// R:AmbientOcclusion, GBA:used for depth
			OutputDesc.Format = bBilateralUpsample ? PF_B8G8R8A8 : PF_R8;
			OutputDesc.ClearValue = FClearValueBinding::None;
			OutputDesc.Flags &= ~TexCreate_DepthStencilTargetable;
			OutputDesc.Flags |= TexCreate_RenderTargetable;
			if (FSSAOHelper::IsAmbientOcclusionCompute())
			{
				OutputDesc.Flags |= TexCreate_UAV;
			}
			OutputDesc.Extent = HalfResOutputViewport.Extent;

			HalfResOutput.Texture = GraphBuilder.CreateTexture(OutputDesc, TEXT("HalfResAmbientOcclusion"));
			HalfResOutput.ViewRect = HalfResOutputViewport.Rect;
			HalfResOutput.LoadAction = ERenderTargetLoadAction::ENoAction;
			HalfResOutput.UpdateVisualizeTextureExtent();
		}

		FScreenPassTexture UpsampleInput = AddAmbientOcclusionFinalPass(
			GraphBuilder,
			View,
			CommonParameters,
			FScreenPassTexture(),
			FScreenPassTexture(),
			FScreenPassTexture(),
			HalfResOutput);

		const int32 UpsampleQuality = FMath::Clamp(CommonParameters.MobileHalfResolutionSetting - 1, 0, 2);
		AddAmbientOcclusionUpsamplePass(
			GraphBuilder,
			View,
			CommonParameters,
			UpsampleQuality,
			UpsampleInput,
			FinalTarget);
		
		return FinalTarget;
	}
	else
	{
		FScreenPassTexture AmbientOcclusionInMip1;
		FScreenPassTexture AmbientOcclusionPassMip1;
		if (CommonParameters.Levels >= 2)
		{
			AmbientOcclusionInMip1 =
				AddAmbientOcclusionSetupPass(
					GraphBuilder,
					View,
					CommonParameters,
					CommonParameters.SceneDepth);

			FScreenPassTexture AmbientOcclusionPassMip2;
			if (CommonParameters.Levels >= 3)
			{
				FScreenPassTexture AmbientOcclusionInMip2 =
					AddAmbientOcclusionSetupPass(
						GraphBuilder,
						View,
						CommonParameters,
						AmbientOcclusionInMip1);

				AmbientOcclusionPassMip2 =
					AddAmbientOcclusionStepPass(
						GraphBuilder,
						View,
						CommonParameters,
						AmbientOcclusionInMip2,
						AmbientOcclusionInMip2,
						FScreenPassTexture());
			}

			AmbientOcclusionPassMip1 =
				AddAmbientOcclusionStepPass(
					GraphBuilder,
					View,
					CommonParameters,
					AmbientOcclusionInMip1,
					AmbientOcclusionInMip1,
					AmbientOcclusionPassMip2);
		}

		FScreenPassTexture SetupTexture = CommonParameters.GBufferA;
		if (Substrate::IsSubstrateEnabled())
		{
			// For Substrate, we invalidate the setup texture for the final pass:
			//	- We do not need GBufferA, the Substrate TopLayer texture will fill in for that.
			//	- Setting it to nullptr will make the AddAmbientOcclusionPass use a valid viewport from SceneTextures.
			SetupTexture.Texture = nullptr;
		}

		return AddAmbientOcclusionFinalPass(
			GraphBuilder,
			View,
			CommonParameters,
			SetupTexture,
			AmbientOcclusionInMip1,
			AmbientOcclusionPassMip1,
			FinalTarget);
	}
}


static FRDGTextureRef CreateMobileScreenSpaceAOTexture(FRDGBuilder& GraphBuilder, const FSceneTexturesConfig& Config)
{
	EPixelFormat Format = PF_R8;
	const bool bComputeShader = FSSAOHelper::IsAmbientOcclusionCompute();

	// G8 isn't supported as UAV on Android OpenGLES, fall back to RGBA8 for compute shader usage.
	if (bComputeShader && IsOpenGLPlatform(Config.ShaderPlatform))
	{
		Format = PF_R8G8B8A8;
	}
	ETextureCreateFlags TextureCreateFlags = TexCreate_ShaderResource | TexCreate_RenderTargetable;
	if (bComputeShader)
	{
		TextureCreateFlags |= TexCreate_UAV;
	}
	if (UE::PixelFormat::HasCapabilities(Format, EPixelFormatCapabilities::LossyCompressible))
	{
		TextureCreateFlags |= TexCreate_LossyCompressionLowBitrate;
	}
	return GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(Config.Extent, Format, FClearValueBinding::Black, TextureCreateFlags), TEXT("ScreenSpaceAO"));
}

bool RequiresMobileAmbientOcclusionPass(FScene* Scene, const FViewInfo& View)
{
	int32 CVarLevel = FSSAOHelper::GetNumAmbientOcclusionLevels();
	return CVarLevel > 0
		&& View.FinalPostProcessSettings.AmbientOcclusionIntensity > 0
		&& (View.FinalPostProcessSettings.AmbientOcclusionStaticFraction >= 1 / 100.0f || (Scene && Scene->SkyLight && View.Family->EngineShowFlags.SkyLighting))
		&& View.Family->EngineShowFlags.Lighting
		&& !View.bIsReflectionCapture
		&& !View.bIsPlanarReflection
		&& !View.Family->EngineShowFlags.HitProxies
		&& !View.Family->EngineShowFlags.VisualizeLightCulling
		&& !View.Family->UseDebugViewPS();
}

bool MobileSSAOUsesCompute()
{
	return FSSAOHelper::IsAmbientOcclusionCompute();
}

// --------------------------------------------------------------------------------------------------------------------
FRDGTextureRef FMobileSceneRenderer::RenderAmbientOcclusion(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures)
{
	if (!RequiresMobileAmbientOcclusionPass(Scene, Views[0]))
	{
		return nullptr;
	}
	RDG_EVENT_SCOPE_STAT(GraphBuilder, MobileSSAO, "MobileSSAO");
	const int32 HalfResolutionSetting = CVarMobileSSAOHalfResolution.GetValueOnRenderThread();
	FRDGTextureRef AmbientOcclusionTexture = CreateMobileScreenSpaceAOTexture(GraphBuilder, SceneTextures.Config);
	FScreenPassTexture Output;

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		FViewInfo& View = Views[ViewIndex];
		View.BeginRenderView();
		
		uint32 Levels = HalfResolutionSetting ? 1 : FSSAOHelper::ComputeAmbientOcclusionPassCount(View);
		FSSAOCommonParameters Parameters = GetSSAOCommonParameters(GraphBuilder, View, Levels, false);
		
		// In case TAA is not active and we do not do half-res upsampling, do a smoothing pass to hide a sampling pattern 
		Parameters.bNeedSmoothingPass = (Views.Num() == 1) && (!IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod) && HalfResolutionSetting == 0);
		Parameters.MobileHalfResolutionSetting = HalfResolutionSetting;
		if (Parameters.FullscreenType == ESSAOType::EPS 
			&& IsMobileDeferredShadingEnabled(View.GetShaderPlatform()) 
			&& MobileUsesGBufferCustomData(View.GetShaderPlatform()))
		{
			Parameters.MobileShadingModelsExcludeMask = CVarMobileSSAOShadingModelExcludeMask.GetValueOnRenderThread();
		}
		
		FScreenPassRenderTarget FinalTarget = FScreenPassRenderTarget(AmbientOcclusionTexture, View.ViewRect, (ViewIndex == 0 ? ERenderTargetLoadAction::ENoAction : ERenderTargetLoadAction::ELoad));

		Output = AddPostProcessingAmbientOcclusionMobile(
			GraphBuilder,
			View,
			Parameters,
			FinalTarget);
	}
	
	return Output.Texture;
}
