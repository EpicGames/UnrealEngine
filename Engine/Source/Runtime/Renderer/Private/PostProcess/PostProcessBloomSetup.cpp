// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessBloomSetup.h"
#include "PostProcess/PostProcessDownsample.h"
#include "PostProcess/PostProcessFFTBloom.h"
#include "PostProcess/PostProcessWeightedSampleSum.h"
#include "PostProcess/PostProcessEyeAdaptation.h"
#include "PostProcess/PostProcessLocalExposure.h"
#include "PixelShaderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"

namespace
{
const int32 GBloomSetupComputeTileSizeX = 8;
const int32 GBloomSetupComputeTileSizeY = 8;

TAutoConsoleVariable<float> CVarBloomCross(
	TEXT("r.GaussianBloom.Cross"),
	0.0f,
	TEXT("Experimental feature to give bloom kernel a more bright center sample (values between 1 and 3 work without causing aliasing)\n")
	TEXT("Existing bloom get lowered to match the same brightness\n")
	TEXT("<0 for a anisomorphic lens flare look (X only)\n")
	TEXT(" 0 off (default)\n")
	TEXT(">0 for a cross look (X and Y)"),
	ECVF_RenderThreadSafe);

BEGIN_SHADER_PARAMETER_STRUCT(FBloomSetupParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, LumBilateralGrid)
	SHADER_PARAMETER_SAMPLER(SamplerState, LumBilateralGridSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BlurredLogLum)
	SHADER_PARAMETER_SAMPLER(SamplerState, BlurredLogLumSampler)
	SHADER_PARAMETER_STRUCT(FLocalExposureParameters, LocalExposure)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
	SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
	SHADER_PARAMETER(float, BloomThreshold)
END_SHADER_PARAMETER_STRUCT()

FBloomSetupParameters GetBloomSetupParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTextureViewport& InputViewport,
	const FBloomSetupInputs& Inputs)
{
	FBloomSetupParameters Parameters;
	Parameters.View = View.ViewUniformBuffer;
	Parameters.Input = GetScreenPassTextureViewportParameters(InputViewport);
	Parameters.InputTexture = Inputs.SceneColor.TextureSRV;
	Parameters.InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters.LumBilateralGrid = Inputs.LocalExposureTexture;
	Parameters.LumBilateralGridSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters.BlurredLogLum = Inputs.BlurredLogLuminanceTexture;
	Parameters.BlurredLogLumSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters.LocalExposure = Inputs.LocalExposureParameters != nullptr ? *Inputs.LocalExposureParameters : FLocalExposureParameters();
	Parameters.EyeAdaptationBuffer = GraphBuilder.CreateSRV(Inputs.EyeAdaptationBuffer);
	Parameters.EyeAdaptation = *Inputs.EyeAdaptationParameters;
	Parameters.BloomThreshold = Inputs.Threshold;
	return Parameters;
}

class FBloomSetupPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomSetupPS);
	SHADER_USE_PARAMETER_STRUCT(FBloomSetupPS, FGlobalShader);

	class FLocalExposureDim : SHADER_PERMUTATION_BOOL("USE_LOCAL_EXPOSURE");
	class FThresholdDim : SHADER_PERMUTATION_BOOL("USE_THRESHOLD");
	using FPermutationDomain = TShaderPermutationDomain<FLocalExposureDim, FThresholdDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FBloomSetupParameters, BloomSetup)
		SHADER_PARAMETER(FScreenTransform, SvPositionToInputTextureUV)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FBloomSetupPS, "/Engine/Private/PostProcessBloom.usf", "BloomSetupPS", SF_Pixel);

class FBloomSetupCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomSetupCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomSetupCS, FGlobalShader);

	class FLocalExposureDim : SHADER_PERMUTATION_BOOL("USE_LOCAL_EXPOSURE");
	class FThresholdDim : SHADER_PERMUTATION_BOOL("USE_THRESHOLD");
	using FPermutationDomain = TShaderPermutationDomain<FLocalExposureDim, FThresholdDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FBloomSetupParameters, BloomSetup)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWOutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GBloomSetupComputeTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GBloomSetupComputeTileSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FBloomSetupCS, "/Engine/Private/PostProcessBloom.usf", "BloomSetupCS", SF_Compute);
} //!namespace

FScreenPassTexture AddBloomSetupPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FBloomSetupInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.EyeAdaptationBuffer);

	const bool bIsComputePass = View.bUseComputePasses;
	const bool bLocalExposureEnabled = Inputs.LocalExposureTexture != nullptr;
	const bool bThresholdEnabled = Inputs.Threshold > -1.0f;

	check(bLocalExposureEnabled || bThresholdEnabled);

	const FRDGTextureDesc& InputDesc = Inputs.SceneColor.TextureSRV->Desc.Texture->Desc;
	FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
		InputDesc.Extent,
		InputDesc.Format,
		FClearValueBinding::None,
		/* InFlags = */ TexCreate_ShaderResource | (bIsComputePass ? TexCreate_UAV : TexCreate_RenderTargetable) | (InputDesc.Flags & (TexCreate_FastVRAM | TexCreate_FastVRAMPartialAlloc)));

	const FScreenPassTextureViewport Viewport(Inputs.SceneColor);
	const FScreenPassRenderTarget Output(GraphBuilder.CreateTexture(OutputDesc, TEXT("BloomSetup")), Viewport.Rect, ERenderTargetLoadAction::ENoAction);

	if (bIsComputePass)
	{
		FBloomSetupCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomSetupCS::FParameters>();
		PassParameters->BloomSetup = GetBloomSetupParameters(GraphBuilder, View, Viewport, Inputs);
		PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(Output.Texture);

		FBloomSetupCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FBloomSetupCS::FLocalExposureDim>(bLocalExposureEnabled);
		PermutationVector.Set<FBloomSetupCS::FThresholdDim>(bThresholdEnabled);

		auto ComputeShader = View.ShaderMap->GetShader<FBloomSetupCS>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("BloomSetup %dx%d (CS)", Viewport.Rect.Width(), Viewport.Rect.Height()),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(Viewport.Rect.Size(), FIntPoint(GBloomSetupComputeTileSizeX, GBloomSetupComputeTileSizeY)));
	}
	else
	{
		FBloomSetupPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomSetupPS::FParameters>();
		PassParameters->BloomSetup = GetBloomSetupParameters(GraphBuilder, View, Viewport, Inputs);
		PassParameters->SvPositionToInputTextureUV = (
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Inputs.SceneColor), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		FBloomSetupPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FBloomSetupPS::FLocalExposureDim>(bLocalExposureEnabled);
		PermutationVector.Set<FBloomSetupPS::FThresholdDim>(bThresholdEnabled);

		auto PixelShader = View.ShaderMap->GetShader<FBloomSetupPS>(PermutationVector);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("BloomSetup %dx%d (PS)", Viewport.Rect.Width(), Viewport.Rect.Height()),
			PixelShader,
			PassParameters,
			Output.ViewRect);
	}

	return FScreenPassTexture(Output);
}

EBloomQuality GetBloomQuality()
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.BloomQuality"));

	return static_cast<EBloomQuality>(FMath::Clamp(
		CVar->GetValueOnRenderThread(),
		static_cast<int32>(EBloomQuality::Disabled),
		static_cast<int32>(EBloomQuality::MAX) - 1));
}

FScreenPassTexture AddGaussianBloomPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FTextureDownsampleChain* SceneDownsampleChain)
{
	checkf(SceneDownsampleChain && SceneDownsampleChain->GetStageCount() == static_cast<uint32>(EBloomQuality::MAX), TEXT("The total number of stages in the scene downsample chain and the number of bloom quality levels must match."));
	check(!IsFFTBloomEnabled(View));

	const FFinalPostProcessSettings& Settings = View.FinalPostProcessSettings;

	const EBloomQuality BloomQuality = GetBloomQuality();

	FScreenPassTexture PassOutputs;
	if (BloomQuality != EBloomQuality::Disabled)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "Bloom");

		const float CrossBloom = CVarBloomCross.GetValueOnRenderThread();

		const FVector2D CrossCenterWeight(FMath::Max(CrossBloom, 0.0f), FMath::Abs(CrossBloom));

		check(BloomQuality != EBloomQuality::Disabled);
		const uint32 BloomQualityIndex = static_cast<uint32>(BloomQuality);
		const uint32 BloomQualityCountMax = static_cast<uint32>(EBloomQuality::MAX);

		struct FBloomStage
		{
			const float Size;
			const FLinearColor& Tint;
		};

		FBloomStage BloomStages[] =
		{
			{ Settings.Bloom6Size, Settings.Bloom6Tint },
			{ Settings.Bloom5Size, Settings.Bloom5Tint },
			{ Settings.Bloom4Size, Settings.Bloom4Tint },
			{ Settings.Bloom3Size, Settings.Bloom3Tint },
			{ Settings.Bloom2Size, Settings.Bloom2Tint },
			{ Settings.Bloom1Size, Settings.Bloom1Tint }
		};

		const uint32 BloomQualityToSceneDownsampleStage[] =
		{
			static_cast<uint32>(-1), // Disabled (sentinel entry to preserve indices)
			3, // Q1
			3, // Q2
			4, // Q3
			5, // Q4
			6  // Q5
		};

		static_assert(UE_ARRAY_COUNT(BloomStages) == BloomQualityCountMax, "Array must be one less than the number of bloom quality entries.");
		static_assert(UE_ARRAY_COUNT(BloomQualityToSceneDownsampleStage) == BloomQualityCountMax, "Array must be one less than the number of bloom quality entries.");

		check(BloomQualityIndex < BloomQualityCountMax);

		// Use bloom quality to select the number of downsample stages to use for bloom.
		const uint32 BloomStageCount = BloomQualityToSceneDownsampleStage[BloomQualityIndex];

		const float BloomIntensity = Settings.BloomIntensity * Settings.BloomGaussianIntensity;
		const float TintScale = (1.0f / BloomQualityCountMax) * BloomIntensity;

		for (uint32 StageIndex = 0, SourceIndex = BloomQualityCountMax - 1; StageIndex < BloomStageCount; ++StageIndex, --SourceIndex)
		{
			const FBloomStage& BloomStage = BloomStages[StageIndex];

			if (BloomStage.Size > SMALL_NUMBER)
			{
				FGaussianBlurInputs PassInputs;
				PassInputs.NameX = TEXT("BloomX");
				PassInputs.NameY = TEXT("BloomY");
				PassInputs.Filter = SceneDownsampleChain->GetTexture(SourceIndex);
				PassInputs.Additive = PassOutputs;
				PassInputs.CrossCenterWeight = FVector2f(CrossCenterWeight);	// LWC_TODO: Precision loss
				PassInputs.KernelSizePercent = BloomStage.Size * Settings.BloomSizeScale;
				PassInputs.TintColor = BloomStage.Tint * TintScale;

				PassOutputs = AddGaussianBlurPass(GraphBuilder, View, PassInputs);
			}
		}
	}

	return PassOutputs;
}
