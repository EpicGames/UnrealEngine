// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

class FTextureDownsampleChain;
class FEyeAdaptationParameters;
class FLocalExposureParameters;

enum class EBloomQuality : uint32
{
	Disabled,
	Q1,
	Q2,
	Q3,
	Q4,
	Q5,
	MAX
};

EBloomQuality GetBloomQuality();

FScreenPassTexture AddGaussianBloomPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FTextureDownsampleChain* SceneDownsampleChain);

struct FBloomSetupInputs
{
	// [Required]: The intermediate scene color being processed.
	FScreenPassTextureSlice SceneColor;

	// [Required]: The scene eye adaptation buffer.
	FRDGBufferRef EyeAdaptationBuffer = nullptr;

	// [Required]: The bloom threshold to apply. Must be >0.
	float Threshold = 0.0f;

	// [Optional] Eye adaptation parameters.
	const FEyeAdaptationParameters* EyeAdaptationParameters = nullptr;

	// [Optional] Local exposure parameters.
	const FLocalExposureParameters* LocalExposureParameters = nullptr;

	// [Optional] Luminance bilateral grid. If this is null, local exposure is disabled.
	FRDGTextureRef LocalExposureTexture = nullptr;

	// [Optional] Blurred luminance texture used to calculate local exposure.
	FRDGTextureRef BlurredLogLuminanceTexture = nullptr;
};

FScreenPassTexture AddBloomSetupPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FBloomSetupInputs& Inputs);