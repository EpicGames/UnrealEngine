// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessBokehDOF.h"
#include "PostProcess/PostProcessDOF.h"
#include "CanvasTypes.h"
#include "PostProcess/DiaphragmDOF.h"
#include "SceneRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "UnrealEngine.h"

class FVisualizeDOFPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeDOFPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeDOFPS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)
		SHADER_PARAMETER(FScreenTransform, SvPositionToColorUV)
		SHADER_PARAMETER(FScreenTransform, SvPositionToDepthUV)
		SHADER_PARAMETER(FVector4f, DepthOfFieldParams)
		SHADER_PARAMETER(FVector4f, NearColor)
		SHADER_PARAMETER(FVector4f, FarColor)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeDOFPS, "/Engine/Private/PostProcessVisualizeDOF.usf", "VisualizeDOFPS", SF_Pixel);

FScreenPassTexture AddVisualizeDOFPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeDOFInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneDepth.IsValid());

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeDOF"));
	}

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);

	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FVisualizeDOFPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeDOFPS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
	PassParameters->ColorTexture = Inputs.SceneColor.Texture;
	PassParameters->DepthTexture = Inputs.SceneDepth.Texture;
	PassParameters->ColorSampler = PointClampSampler;
	PassParameters->DepthSampler = PointClampSampler;
	{
		FScreenTransform SvPositionToViewportUV = FScreenTransform::SvPositionToViewportUV(Output.ViewRect);
		PassParameters->SvPositionToColorUV = 
			SvPositionToViewportUV *
			FScreenTransform::ChangeTextureBasisFromTo(
				InputViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
		PassParameters->SvPositionToDepthUV =
			SvPositionToViewportUV *
			FScreenTransform::ChangeTextureBasisFromTo(
				FScreenPassTextureViewport(Inputs.SceneDepth), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
	}
	PassParameters->DepthOfFieldParams = GetDepthOfFieldParameters(View.FinalPostProcessSettings);
	PassParameters->NearColor = FLinearColor(0, 0.8f, 0, 0);
	PassParameters->FarColor = FLinearColor(0, 0, 0.8f, 0);
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	TShaderMapRef<FVisualizeDOFPS> PixelShader(View.ShaderMap);

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeDOF");

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Visualizer"), View, FScreenPassTextureViewport(Output), InputViewport, PixelShader, PassParameters);

	Output.LoadAction = ERenderTargetLoadAction::ELoad;

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("Overlay"), View, Output, [&View](FCanvas& Canvas)
	{
		const float DPIScale = Canvas.GetDPIScale();
		Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale) * Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

		auto DrawShadowedString = [&](float X, float Y, FStringView Text, const FLinearColor& Color = FLinearColor::White)
		{
			Canvas.DrawShadowedString(X / DPIScale, Y / DPIScale, Text, GetStatsFont(), Color);
		};

		float X = 30;
		float Y = 18;
		const float YStep = 14;
		const float ColumnWidth = 250;

		FString Line;

		Line = FString::Printf(TEXT("Visualize Depth of Field"));
		DrawShadowedString(20, Y += YStep, Line, FLinearColor(1, 1, 0));
		Y += YStep;

		Line = FString::Printf(TEXT("Cinematic DOF (blue is far, green is near, black is in focus, cross hair shows Depth and CoC radius in pixel)"));
		DrawShadowedString(X, Y += YStep, Line);
		Y += YStep;
		Line = FString::Printf(TEXT("FocalDistance: %.2f"), View.FinalPostProcessSettings.DepthOfFieldFocalDistance);
		DrawShadowedString(X, Y += YStep, Line);
		Line = FString::Printf(TEXT("Aperture F-stop: %.2f"), View.FinalPostProcessSettings.DepthOfFieldFstop);
		DrawShadowedString(X, Y += YStep, Line);
		Line = FString::Printf(TEXT("Aperture: f/%.2f"), View.FinalPostProcessSettings.DepthOfFieldFstop);
		DrawShadowedString(X, Y += YStep, Line);
		Y += YStep;
		Line = FString::Printf(TEXT("DepthBlur (not related to Depth of Field, due to light traveling long distances in atmosphere)"));
		DrawShadowedString(X, Y += YStep, Line);
		Line = FString::Printf(TEXT("  km for 50%%: %.2f"), View.FinalPostProcessSettings.DepthOfFieldDepthBlurAmount);
		DrawShadowedString(X, Y += YStep, Line);
		Line = FString::Printf(TEXT("  Radius (pixels in 1920x): %.2f"), View.FinalPostProcessSettings.DepthOfFieldDepthBlurRadius);
		DrawShadowedString(X, Y += YStep, Line);
		Y += YStep;

		const FVector2D Fov = View.ViewMatrices.ComputeHalfFieldOfViewPerAxis();

		DiaphragmDOF::FPhysicalCocModel CocModel;
		CocModel.Compile(View);

		const float UUToMM = 10.0f;

		Line = FString::Printf(TEXT("Field Of View in deg.: %.1f x %.1f"), FMath::RadiansToDegrees(Fov.X) * 2.0f, FMath::RadiansToDegrees(Fov.Y) * 2.0f);
		DrawShadowedString(X, Y += YStep, Line, FLinearColor(0.5, 0.5, 1));
		Line = FString::Printf(TEXT("Focal Length: %.1f mm"), UUToMM * CocModel.VerticalFocalLength);
		DrawShadowedString(X, Y += YStep, Line, FLinearColor(0.5, 0.5, 1));
		Line = FString::Printf(TEXT("Squeeze Factor: %.1f"), CocModel.Squeeze);
		DrawShadowedString(X, Y += YStep, Line, FLinearColor(0.5, 0.5, 1));
		Line = FString::Printf(TEXT("Sensor: %.1f x %.1f mm"), UUToMM * CocModel.SensorWidth, UUToMM * CocModel.SensorHeight);
		DrawShadowedString(X, Y += YStep, Line, FLinearColor(0.5, 0.5, 1));
	});

	return MoveTemp(Output);
}
