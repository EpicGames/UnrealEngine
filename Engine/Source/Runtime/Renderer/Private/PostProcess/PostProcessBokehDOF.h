// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "OverridePassSequence.h"

struct FVisualizeDOFInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The scene color to composite with DOF visualization.
	FScreenPassTexture SceneColor;

	// [Required] The scene depth to derive depth of field parameters.
	FScreenPassTexture SceneDepth;
};

FScreenPassTexture AddVisualizeDOFPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeDOFInputs& Inputs);