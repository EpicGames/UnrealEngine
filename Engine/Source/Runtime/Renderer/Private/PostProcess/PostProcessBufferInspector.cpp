// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PostProcess/PostProcessBufferInspector.h"
#include "SceneTextureParameters.h"
#include "ScenePrivate.h"
#include "UnrealClient.h"
#include "UnrealEngine.h"

BEGIN_SHADER_PARAMETER_STRUCT(FPixelInspectorParameters, )
	RDG_TEXTURE_ACCESS(GBufferA, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(GBufferB, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(GBufferC, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(GBufferD, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(GBufferE, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(GBufferF, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(SceneColor, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(SceneColorBeforeTonemap, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(SceneDepth, ERHIAccess::CopySrc)
	RDG_TEXTURE_ACCESS(OriginalSceneColor, ERHIAccess::CopySrc)
END_SHADER_PARAMETER_STRUCT()

void ProcessPixelInspectorRequests(
	FRHICommandListImmediate& RHICmdList,
	const FViewInfo& View,
	const FPixelInspectorParameters& Parameters,
	const FIntRect SceneColorViewRect)
{
	const FSceneViewFamily& ViewFamily = *(View.Family);
	const int32 ViewUniqueId = View.State->GetViewKey();

	FPixelInspectorData& PixelInspectorData = static_cast<FScene*>(ViewFamily.Scene)->PixelInspectorData;
	TArray<FVector2D> ProcessRequests;

	// Process all request for this view.
	for (auto KeyValue : PixelInspectorData.Requests)
	{
		FPixelInspectorRequest *PixelInspectorRequest = KeyValue.Value;
		if (PixelInspectorRequest->RequestComplete == true)
		{
			PixelInspectorRequest->RenderingCommandSend = true;
			ProcessRequests.Add(FVector2D(KeyValue.Key));
		}
		else if (PixelInspectorRequest->RenderingCommandSend == false && PixelInspectorRequest->ViewId == ViewUniqueId)
		{
			FVector2D SourceViewportUV = FVector2D(PixelInspectorRequest->SourceViewportUV);
			FVector2D ExtendSize(1.0f, 1.0f);

			//////////////////////////////////////////////////////////////////////////
			// Pixel Depth
			if (PixelInspectorData.RenderTargetBufferDepth[PixelInspectorRequest->BufferIndex] != nullptr)
			{
				const FIntVector SourcePoint(
					FMath::FloorToInt(SourceViewportUV.X * View.ViewRect.Width()),
					FMath::FloorToInt(SourceViewportUV.Y * View.ViewRect.Height()),
					0
				);

				const FTextureRHIRef &DestinationBufferDepth = PixelInspectorData.RenderTargetBufferDepth[PixelInspectorRequest->BufferIndex]->GetRenderTargetTexture();
				if (DestinationBufferDepth.IsValid())
				{
					FRHITexture* SourceBufferSceneDepth = Parameters.SceneDepth->GetRHI();
					if (DestinationBufferDepth->GetFormat() == SourceBufferSceneDepth->GetFormat())
					{
						FRHICopyTextureInfo CopyInfo;
						CopyInfo.SourcePosition = SourcePoint;
						CopyInfo.Size = FIntVector(1);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferDepth, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
						RHICmdList.CopyTexture(SourceBufferSceneDepth, DestinationBufferDepth, CopyInfo);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferDepth, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
					}
				}
			}

			//////////////////////////////////////////////////////////////////////////
			// FINAL COLOR
			const FTextureRHIRef &DestinationBufferFinalColor = PixelInspectorData.RenderTargetBufferFinalColor[PixelInspectorRequest->BufferIndex]->GetRenderTargetTexture();
			if (DestinationBufferFinalColor.IsValid())
			{
				const FIntVector SourcePoint(
					FMath::FloorToInt(SourceViewportUV.X * SceneColorViewRect.Width()),
					FMath::FloorToInt(SourceViewportUV.Y * SceneColorViewRect.Height()),
					0
				);

				FRHITexture* SourceBufferFinalColor = Parameters.SceneColor->GetRHI();
				if (DestinationBufferFinalColor->GetFormat() == SourceBufferFinalColor->GetFormat())
				{
					FRHICopyTextureInfo CopyInfo;
					CopyInfo.Size = DestinationBufferFinalColor->GetSizeXYZ();
					CopyInfo.SourcePosition = SourcePoint - CopyInfo.Size / 2;

					const FIntVector OutlineCornerMin(
						FMath::Min(CopyInfo.SourcePosition.X - SceneColorViewRect.Min.X, 0),
						FMath::Min(CopyInfo.SourcePosition.Y - SceneColorViewRect.Min.Y, 0),
						0
					);
					CopyInfo.SourcePosition -= OutlineCornerMin;
					CopyInfo.DestPosition -= OutlineCornerMin;
					CopyInfo.Size += OutlineCornerMin;

					const FIntVector OutlineCornerMax(
						FMath::Max(0, CopyInfo.SourcePosition.X + CopyInfo.Size.X - SceneColorViewRect.Max.X),
						FMath::Max(0, CopyInfo.SourcePosition.Y + CopyInfo.Size.Y - SceneColorViewRect.Max.Y),
						0
					);
					CopyInfo.Size -= OutlineCornerMax;

					if (CopyInfo.Size.X > 0 && CopyInfo.Size.Y > 0)
					{
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferFinalColor, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
						RHICmdList.CopyTexture(SourceBufferFinalColor, DestinationBufferFinalColor, CopyInfo);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferFinalColor, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
					}
				}
			}

			//////////////////////////////////////////////////////////////////////////
			// ORIGINAL SCENE COLOR
			const FTextureRHIRef& DestinationBufferSceneColor = PixelInspectorData.RenderTargetBufferSceneColor[PixelInspectorRequest->BufferIndex]->GetRenderTargetTexture();
			if (DestinationBufferSceneColor.IsValid())
			{
				const FIntVector SourcePoint(
					FMath::FloorToInt(SourceViewportUV.X * View.ViewRect.Width()),
					FMath::FloorToInt(SourceViewportUV.Y * View.ViewRect.Height()),
					0
				);

				FRHITexture* SourceBufferSceneColor = Parameters.OriginalSceneColor->GetRHI();
				if (DestinationBufferSceneColor->GetFormat() == SourceBufferSceneColor->GetFormat())
				{
					FRHICopyTextureInfo CopyInfo;
					CopyInfo.SourcePosition = SourcePoint;
					CopyInfo.Size = FIntVector(1, 1, 1);
					RHICmdList.Transition(FRHITransitionInfo(DestinationBufferSceneColor, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
					RHICmdList.CopyTexture(SourceBufferSceneColor, DestinationBufferSceneColor, CopyInfo);
					RHICmdList.Transition(FRHITransitionInfo(DestinationBufferSceneColor, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
				}
			}

			//////////////////////////////////////////////////////////////////////////
			// HDR
			const FTextureRHIRef &DestinationBufferHDR = PixelInspectorData.RenderTargetBufferHDR[PixelInspectorRequest->BufferIndex]->GetRenderTargetTexture();
			if (DestinationBufferHDR.IsValid())
			{
				const FIntVector SourcePoint(
					FMath::FloorToInt(SourceViewportUV.X * SceneColorViewRect.Width()),
					FMath::FloorToInt(SourceViewportUV.Y * SceneColorViewRect.Height()),
					0
				);

				if (Parameters.SceneColorBeforeTonemap)
				{
					FRHITexture* SourceBufferHDR = Parameters.SceneColorBeforeTonemap->GetRHI();
					if (DestinationBufferHDR->GetFormat() == SourceBufferHDR->GetFormat())
					{
						FRHICopyTextureInfo CopyInfo;
						CopyInfo.SourcePosition = SourcePoint;
						CopyInfo.Size = FIntVector(1, 1, 1);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferHDR, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
						RHICmdList.CopyTexture(SourceBufferHDR, DestinationBufferHDR, CopyInfo);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferHDR, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
					}
				}
			}

			//////////////////////////////////////////////////////////////////////////
			// GBuffer A
			if (PixelInspectorData.RenderTargetBufferA[PixelInspectorRequest->BufferIndex] != nullptr)
			{
				const FIntVector SourcePoint(
					FMath::FloorToInt(SourceViewportUV.X * View.ViewRect.Width()),
					FMath::FloorToInt(SourceViewportUV.Y * View.ViewRect.Height()),
					0
				);

				const FTextureRHIRef &DestinationBufferA = PixelInspectorData.RenderTargetBufferA[PixelInspectorRequest->BufferIndex]->GetRenderTargetTexture();
				if (DestinationBufferA.IsValid() && Parameters.GBufferA)
				{
					FRHITexture* SourceBufferA = Parameters.GBufferA->GetRHI();
					if (DestinationBufferA->GetFormat() == SourceBufferA->GetFormat())
					{
						FRHICopyTextureInfo CopyInfo;
						CopyInfo.SourcePosition = SourcePoint;
						CopyInfo.Size = FIntVector(1, 1, 1);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferA, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
						RHICmdList.CopyTexture(SourceBufferA, DestinationBufferA, CopyInfo);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferA, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
					}
				}
			}

			//////////////////////////////////////////////////////////////////////////
			// GBuffer BCDEF
			const FTextureRHIRef &DestinationBufferBCDEF = PixelInspectorData.RenderTargetBufferBCDEF[PixelInspectorRequest->BufferIndex]->GetRenderTargetTexture();
			if (DestinationBufferBCDEF.IsValid())
			{
				const FIntVector SourcePoint(
					FMath::FloorToInt(SourceViewportUV.X * View.ViewRect.Width()),
					FMath::FloorToInt(SourceViewportUV.Y * View.ViewRect.Height()),
					0
				);

				if (Parameters.GBufferB)
				{
					FRHITexture* SourceBufferB = Parameters.GBufferB->GetRHI();
					if (DestinationBufferBCDEF->GetFormat() == SourceBufferB->GetFormat())
					{
						FRHICopyTextureInfo CopyInfo;
						CopyInfo.SourcePosition = SourcePoint;
						CopyInfo.Size = FIntVector(1, 1, 1);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
						RHICmdList.CopyTexture(SourceBufferB, DestinationBufferBCDEF, CopyInfo);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
					}
				}

				if (Parameters.GBufferC)
				{
					FRHITexture* SourceBufferC = Parameters.GBufferC->GetRHI();
					if (DestinationBufferBCDEF->GetFormat() == SourceBufferC->GetFormat())
					{
						FRHICopyTextureInfo CopyInfo;
						CopyInfo.SourcePosition = SourcePoint;
						CopyInfo.DestPosition = FIntVector(1, 0, 0);
						CopyInfo.Size = FIntVector(1, 1, 1);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
						RHICmdList.CopyTexture(SourceBufferC, DestinationBufferBCDEF, CopyInfo);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
					}
				}

				if (Parameters.GBufferD)
				{
					FRHITexture* SourceBufferD = Parameters.GBufferD->GetRHI();
					if (DestinationBufferBCDEF->GetFormat() == SourceBufferD->GetFormat())
					{
						FRHICopyTextureInfo CopyInfo;
						CopyInfo.SourcePosition = SourcePoint;
						CopyInfo.DestPosition = FIntVector(2, 0, 0);
						CopyInfo.Size = FIntVector(1, 1, 1);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
						RHICmdList.CopyTexture(SourceBufferD, DestinationBufferBCDEF, CopyInfo);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
					}
				}

				if (Parameters.GBufferE)
				{
					FRHITexture* SourceBufferE = Parameters.GBufferE->GetRHI();
					if (DestinationBufferBCDEF->GetFormat() == SourceBufferE->GetFormat())
					{
						FRHICopyTextureInfo CopyInfo;
						CopyInfo.SourcePosition = SourcePoint;
						CopyInfo.DestPosition = FIntVector(3, 0, 0);
						CopyInfo.Size = FIntVector(1, 1, 1);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
						RHICmdList.CopyTexture(SourceBufferE, DestinationBufferBCDEF, CopyInfo);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
					}
				}

				if (Parameters.GBufferF)
				{
					FRHITexture* SourceBufferF = Parameters.GBufferF->GetRHI();
					if (DestinationBufferBCDEF->GetFormat() == SourceBufferF->GetFormat())
					{
						FRHICopyTextureInfo CopyInfo;
						CopyInfo.SourcePosition = SourcePoint;
						CopyInfo.Size = FIntVector(1, 1, 1);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::SRVMask, ERHIAccess::CopyDest));
						RHICmdList.CopyTexture(SourceBufferF, DestinationBufferBCDEF, CopyInfo);
						RHICmdList.Transition(FRHITransitionInfo(DestinationBufferBCDEF, ERHIAccess::CopyDest, ERHIAccess::SRVMask));
					}
				}
			}

			PixelInspectorRequest->RenderingCommandSend = true;
			ProcessRequests.Add(FVector2D(KeyValue.Key));
		}
	}

	// Remove request we just processed.
	for (FVector2D RequestKey : ProcessRequests)
	{
		PixelInspectorData.Requests.Remove(FVector2f(RequestKey));	// LWC_TODO: Precision loss
	}
}

FScreenPassTexture AddPixelInspectorPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FPixelInspectorInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneColor.ViewRect == Inputs.SceneColorBeforeTonemap.ViewRect);
	check(Inputs.OriginalSceneColor.IsValid());
	check(Inputs.OriginalSceneColor.ViewRect == View.ViewRect);
	check(View.bUsePixelInspector);

	RDG_EVENT_SCOPE(GraphBuilder, "PixelInspector");

	// Perform copies of scene textures data into staging resources for visualization.
	{
		FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

		// GBufferF is optional, so it may be a dummy texture. Revert it to null if so.
		if (SceneTextures.GBufferFTexture->Desc.Extent != Inputs.OriginalSceneColor.Texture->Desc.Extent)
		{
			SceneTextures.GBufferFTexture = nullptr;
		}

		FPixelInspectorParameters* PassParameters = GraphBuilder.AllocParameters<FPixelInspectorParameters>();
		PassParameters->GBufferA = SceneTextures.GBufferATexture;
		PassParameters->GBufferB = SceneTextures.GBufferBTexture;
		PassParameters->GBufferC = SceneTextures.GBufferCTexture;
		PassParameters->GBufferD = SceneTextures.GBufferDTexture;
		PassParameters->GBufferE = SceneTextures.GBufferETexture;
		PassParameters->GBufferF = SceneTextures.GBufferFTexture;
		PassParameters->SceneColor = Inputs.SceneColor.Texture;
		PassParameters->SceneColorBeforeTonemap = Inputs.SceneColorBeforeTonemap.Texture;
		PassParameters->SceneDepth = SceneTextures.SceneDepthTexture;
		PassParameters->OriginalSceneColor = Inputs.OriginalSceneColor.Texture;

		const FIntRect SceneColorViewRect(Inputs.SceneColor.ViewRect);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Copy"),
			PassParameters,
			ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
			[PassParameters, &View, SceneColorViewRect](FRHICommandListImmediate& RHICmdList)
		{
			ProcessPixelInspectorRequests(RHICmdList, View, *PassParameters, SceneColorViewRect);
		});
	}

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	// When an output is specified, copy scene color to output before compositing the debug text.
	if (Output.IsValid())
	{
		AddDrawTexturePass(GraphBuilder, View, Inputs.SceneColor, Output);
	}
	// Otherwise, re-use the scene color as the output.
	else
	{
		Output = FScreenPassRenderTarget(Inputs.SceneColor, ERenderTargetLoadAction::ELoad);
	}

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("Overlay"), View, FScreenPassRenderTarget(Inputs.SceneColor, ERenderTargetLoadAction::ELoad),
		[](FCanvas& Canvas)
	{
		FLinearColor LabelColor(1, 1, 1);
		Canvas.DrawShadowedString(100, 50, TEXT("Pixel Inspector On"), GetStatsFont(), LabelColor);
	});

	return MoveTemp(Output);
}

#endif