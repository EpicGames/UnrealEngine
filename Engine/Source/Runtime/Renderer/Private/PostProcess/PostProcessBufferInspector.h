// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ScreenPass.h"
#include "SceneRenderTargetParameters.h"

struct FPixelInspectorInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The tonemapped scene color.
	FScreenPassTexture SceneColor;

	// [Required] The scene color before tonemapping in HDR.
	FScreenPassTexture SceneColorBeforeTonemap;

	// [Required] The original scene color before processing.
	FScreenPassTexture OriginalSceneColor;
};

FScreenPassTexture AddPixelInspectorPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FPixelInspectorInputs& Inputs);

#endif