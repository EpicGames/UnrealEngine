// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessCombineLUTs.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Texture.h"
#include "PostProcess/PostProcessTonemap.h"
#include "PostProcess/SceneFilterRendering.h"
#include "VolumeRendering.h"
#include "HDRHelper.h"
#include "TextureResource.h"
#include "ACESUtils.h"
#include "SceneViewState.h"
#include "RenderTargetPool.h"
#include "LogRenderer.h"

namespace
{
TAutoConsoleVariable<float> CVarColorMin(
	TEXT("r.Color.Min"),
	0.0f,
	TEXT("Allows to define where the value 0 in the color channels is mapped to after color grading.\n")
	TEXT("The value should be around 0, positive: a gray scale is added to the darks, negative: more dark values become black, Default: 0"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarColorMid(
	TEXT("r.Color.Mid"),
	0.5f,
	TEXT("Allows to define where the value 0.5 in the color channels is mapped to after color grading (This is similar to a gamma correction).\n")
	TEXT("Value should be around 0.5, smaller values darken the mid tones, larger values brighten the mid tones, Default: 0.5"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarColorMax(
	TEXT("r.Color.Max"),
	1.0f,
	TEXT("Allows to define where the value 1.0 in the color channels is mapped to after color grading.\n")
	TEXT("Value should be around 1, smaller values darken the highlights, larger values move more colors towards white, Default: 1"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarLUTSize(
	TEXT("r.LUT.Size"),
	32,
	TEXT("Size of film LUT"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

TAutoConsoleVariable<int32> CVarLUTInnerSize(
	TEXT("r.LUT.Inner.Size"),
	64,
	TEXT("Size of inner LUT"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

TAutoConsoleVariable<bool> CVarLUTAsyncCompute(
	TEXT("r.LUT.AsyncCompute"),
	1,
	TEXT("Controls whether the tonemapping LUT pass is run on async compute."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarColorGrading(
	TEXT("r.Color.Grading"), 1,
	TEXT("Controls whether post process settings's color grading settings should be applied."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarLUTUpdateEveryFrame(
	TEXT("r.LUT.UpdateEveryFrame"), 0,
	TEXT("Controls whether the tonemapping LUT pass is executed every frame."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarLUTInnerUpdateEveryFrame(
	TEXT("r.LUT.Inner.UpdateEveryFrame"), 0,
	TEXT("Controls whether the inner tonemapping LUT pass is executed every frame."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarLUTTonemappingMethod(
	TEXT("r.LUT.TonemappingMethod"), -1,
	TEXT("Set the tonemapping method to transform colors for the output device.\n")
	TEXT("-1: Use PostProcessSettings (default)\n")
	TEXT("0: Filmic\n")
	TEXT("1: Standard ACES"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarLUTTonemappingSDRInvEOTF(
	TEXT("r.LUT.TonemappingSDRInvEOTF"), -1,
	TEXT("Set the SDR EOTF for tonemapping to target for the output device.\n")
	TEXT("The default for ACES is BT.1886 to prioritize compatibility between SDR and Rec. 2100 for HDR.\n")
	TEXT("The default for Filmic is sRGB to prioritize symmetry with GPU SRGB conversion although HDR should still align.\n")
	TEXT("-1: Use default\n")
	TEXT("0: sRGB\n")
	TEXT("1: BT.1886"),
	ECVF_RenderThreadSafe);

// Including the neutral one at index 0
const uint32 GMaxLUTBlendCount = 5;

struct FColorTransform
{
	FColorTransform()
	{
		Reset();
	}

	float MinValue;
	float MidValue;
	float MaxValue;

	void Reset()
	{
		MinValue = 0.0f;
		MidValue = 0.5f;
		MaxValue = 1.0f;
	}
};
} //! namespace

FVector3f GetMappingPolynomial()
{
	FColorTransform ColorTransform;
	ColorTransform.MinValue = FMath::Clamp(CVarColorMin.GetValueOnRenderThread(), -10.0f, 10.0f);
	ColorTransform.MidValue = FMath::Clamp(CVarColorMid.GetValueOnRenderThread(), -10.0f, 10.0f);
	ColorTransform.MaxValue = FMath::Clamp(CVarColorMax.GetValueOnRenderThread(), -10.0f, 10.0f);

	// x is the input value, y the output value
	// RGB = a, b, c where y = a * x*x + b * x + c
	float c = ColorTransform.MinValue;
	float b = 4 * ColorTransform.MidValue - 3 * ColorTransform.MinValue - ColorTransform.MaxValue;
	float a = ColorTransform.MaxValue - ColorTransform.MinValue - b;

	return FVector3f(a, b, c);
}

FColorRemapParameters GetColorRemapParameters()
{
	FColorRemapParameters Parameters;
	Parameters.MappingPolynomial = GetMappingPolynomial();
	return Parameters;
}

FPooledRenderTargetDesc FSceneViewState::CreateLUTRenderTarget(const int32 LUTSize, const bool bNeedFloatOutput, const TCHAR* const DebugName)
{
	// Create the texture needed for the tonemapping LUT in one place
	EPixelFormat LUTPixelFormat = PF_A2B10G10R10;
	if (!UE::PixelFormat::HasCapabilities(LUTPixelFormat, EPixelFormatCapabilities::UAV))
	{
		LUTPixelFormat = PF_R8G8B8A8;
	}
	if (bNeedFloatOutput)
	{
		LUTPixelFormat = PF_FloatRGBA;
	}
#if !WITH_EDITOR
	// Editor can enter this function frequently. Log for other build targets.
	UE_LOGF(LogRenderer, Log, "Pixel format for LUT is %ls", GetPixelFormatString(LUTPixelFormat));
#endif

	FPooledRenderTargetDesc Desc = FPooledRenderTargetDesc::CreateVolumeDesc(
		LUTSize,
		LUTSize,
		LUTSize,
		LUTPixelFormat,
		FClearValueBinding::Transparent,
		GFastVRamConfig.CombineLUTs,
		TexCreate_ShaderResource | TexCreate_UAV,
		false);
		Desc.DebugName = DebugName;

	return Desc;
}

IPooledRenderTarget* FSceneViewState::GetTonemappingLUT(FRHICommandList& RHICmdList, const int32 LUTSize, const bool bNeedFloatOutput)
{
	if (CombinedLUTRenderTarget.IsValid() == false || 
		CombinedLUTRenderTarget->GetDesc().Extent.Y != LUTSize ||
		(CombinedLUTRenderTarget->GetDesc().Format == PF_FloatRGBA) != bNeedFloatOutput)
	{
		// Create the texture needed for the tonemapping LUT
		FPooledRenderTargetDesc Desc = CreateLUTRenderTarget(LUTSize, bNeedFloatOutput, TEXT("CombineLUTs"));

		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, CombinedLUTRenderTarget, Desc.DebugName);
	}

	return CombinedLUTRenderTarget.GetReference();
}

IPooledRenderTarget* FSceneViewState::GetInnerTonemappingLUT(FRHICommandList& RHICmdList, const int32 LUTSize, const bool bNeedFloatOutput)
{
	if (InnerLUTRenderTarget.IsValid() == false ||
		InnerLUTRenderTarget->GetDesc().Extent.Y != LUTSize ||
		(InnerLUTRenderTarget->GetDesc().Format == PF_FloatRGBA) != bNeedFloatOutput)
	{
		// Create the texture needed for the tonemapping LUT
		FPooledRenderTargetDesc Desc = CreateLUTRenderTarget(LUTSize, bNeedFloatOutput, TEXT("CombineLUTs_Inner"));

		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, InnerLUTRenderTarget, Desc.DebugName);
	}

	return InnerLUTRenderTarget.GetReference();
}

BEGIN_SHADER_PARAMETER_STRUCT(FACESTonemapShaderParameters, )
	SHADER_PARAMETER(FVector4f, ACESMinMaxData) // xy = min ACES/luminance, zw = max ACES/luminance
	SHADER_PARAMETER(FVector4f, ACESMidData) // x = mid ACES, y = mid luminance, z = mid slope
	SHADER_PARAMETER(FVector4f, ACESCoefsLow_0) // coeflow 0-3
	SHADER_PARAMETER(FVector4f, ACESCoefsHigh_0) // coefhigh 0-3
	SHADER_PARAMETER(float, ACESCoefsLow_4)
	SHADER_PARAMETER(float, ACESCoefsHigh_4)
	SHADER_PARAMETER(float, ACESSceneColorMultiplier)
	SHADER_PARAMETER(float, ACESGamutCompression)
	SHADER_PARAMETER(uint32, bScaleWhite)
	SHADER_PARAMETER(uint32, bScaleWhiteSDR)
	SHADER_PARAMETER_SRV(Texture2D<float>, ACESReachTable)
	SHADER_PARAMETER_SRV(Texture2D<float>, ACESGamutTable)
	SHADER_PARAMETER_SRV(Texture2D<float>, ACESGammaTable)
	SHADER_PARAMETER_SRV(Texture2D<float>, ACESReachTableSDR)
	SHADER_PARAMETER_SRV(Texture2D<float>, ACESGamutTableSDR)
	SHADER_PARAMETER_SRV(Texture2D<float>, ACESGammaTableSDR)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FCombineLUTParameters, )
	SHADER_PARAMETER_TEXTURE_ARRAY(Texture2D, Textures, [GMaxLUTBlendCount])
	SHADER_PARAMETER_SAMPLER_ARRAY(SamplerState, Samplers, [GMaxLUTBlendCount])
	SHADER_PARAMETER_SCALAR_ARRAY(float, LUTWeights, [GMaxLUTBlendCount])
	SHADER_PARAMETER_STRUCT_REF(FWorkingColorSpaceShaderParameters, WorkingColorSpace)
	SHADER_PARAMETER(FVector3f, LimitingRgbToXYZ0)
	SHADER_PARAMETER(FVector3f, LimitingRgbToXYZ1)
	SHADER_PARAMETER(FVector3f, LimitingRgbToXYZ2)
	SHADER_PARAMETER(FVector3f, LimitingXYZToRgb0)
	SHADER_PARAMETER(FVector3f, LimitingXYZToRgb1)
	SHADER_PARAMETER(FVector3f, LimitingXYZToRgb2)
	SHADER_PARAMETER(FVector4f, OverlayColor)
	SHADER_PARAMETER(FVector3f, ColorScale)
	SHADER_PARAMETER(FVector4f, ColorSaturation)
	SHADER_PARAMETER(FVector4f, ColorContrast)
	SHADER_PARAMETER(FVector4f, ColorGamma)
	SHADER_PARAMETER(FVector4f, ColorGain)
	SHADER_PARAMETER(FVector4f, ColorOffset)
	SHADER_PARAMETER(FVector4f, ColorSaturationShadows)
	SHADER_PARAMETER(FVector4f, ColorContrastShadows)
	SHADER_PARAMETER(FVector4f, ColorGammaShadows)
	SHADER_PARAMETER(FVector4f, ColorGainShadows)
	SHADER_PARAMETER(FVector4f, ColorOffsetShadows)
	SHADER_PARAMETER(FVector4f, ColorSaturationMidtones)
	SHADER_PARAMETER(FVector4f, ColorContrastMidtones)
	SHADER_PARAMETER(FVector4f, ColorGammaMidtones)
	SHADER_PARAMETER(FVector4f, ColorGainMidtones)
	SHADER_PARAMETER(FVector4f, ColorOffsetMidtones)
	SHADER_PARAMETER(FVector4f, ColorSaturationHighlights)
	SHADER_PARAMETER(FVector4f, ColorContrastHighlights)
	SHADER_PARAMETER(FVector4f, ColorGammaHighlights)
	SHADER_PARAMETER(FVector4f, ColorGainHighlights)
	SHADER_PARAMETER(FVector4f, ColorOffsetHighlights)
	SHADER_PARAMETER(float, LUTInvMax)
	SHADER_PARAMETER(float, LUTSize)
	SHADER_PARAMETER(float, InnerLUTSize)
	SHADER_PARAMETER(float, WhiteTemp)
	SHADER_PARAMETER(float, WhiteTint)
	SHADER_PARAMETER(float, ColorCorrectionShadowsMax)
	SHADER_PARAMETER(float, ColorCorrectionHighlightsMin)
	SHADER_PARAMETER(float, ColorCorrectionHighlightsMax)
	SHADER_PARAMETER(float, BlueCorrection)
	SHADER_PARAMETER(float, ExpandGamut)
	SHADER_PARAMETER(float, ToneCurveAmount)
	SHADER_PARAMETER(uint32, TonemappingSDRInvEOTF)
	SHADER_PARAMETER(float, FilmSlope)
	SHADER_PARAMETER(float, FilmToe)
	SHADER_PARAMETER(float, FilmShoulder)
	SHADER_PARAMETER(float, FilmBlackClip)
	SHADER_PARAMETER(float, FilmWhiteClip)
	SHADER_PARAMETER(uint32, bIsTemperatureWhiteBalance)
	SHADER_PARAMETER_STRUCT_INCLUDE(FColorRemapParameters, ColorRemap)
	SHADER_PARAMETER_STRUCT_INCLUDE(FTonemapperOutputDeviceParameters, OutputDevice)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FInnerLUTParameters, )
	SHADER_PARAMETER_STRUCT_REF(FWorkingColorSpaceShaderParameters, WorkingColorSpace)
	SHADER_PARAMETER_STRUCT_INCLUDE(FACESTonemapShaderParameters, ACESTonemapParameters)
	SHADER_PARAMETER(FVector3f, LimitingRgbToXYZ0)
	SHADER_PARAMETER(FVector3f, LimitingRgbToXYZ1)
	SHADER_PARAMETER(FVector3f, LimitingRgbToXYZ2)
	SHADER_PARAMETER(FVector3f, LimitingXYZToRgb0)
	SHADER_PARAMETER(FVector3f, LimitingXYZToRgb1)
	SHADER_PARAMETER(FVector3f, LimitingXYZToRgb2)
	SHADER_PARAMETER(float, TotalSceneColorMultiplier)
	SHADER_PARAMETER(uint32, TonemappingMethod)
	SHADER_PARAMETER_STRUCT_INCLUDE(FTonemapperOutputDeviceParameters, OutputDevice)
END_SHADER_PARAMETER_STRUCT()

#define UPDATE_CACHE_SETTINGS(DestParameters, ParamValue, bOutHasChanged) \
if(DestParameters != (ParamValue)) \
{ \
	DestParameters = (ParamValue); \
	bOutHasChanged = true; \
}

uint32 DoesGamutWhitePointMatch(uint32 OutputGamut, const UE::Color::FColorSpace& ColorSpace)
{
	static_assert((uint32)EDisplayColorGamut::MAX == 5u, "Update logic if new values are added.");

	bool bMatch = false;

	// Seeing values from DXGI that look like (X / 1024), so tolerate 50% more than that.
	constexpr double Tolerance = 0x3p-11;

	switch (static_cast<EDisplayColorGamut>(OutputGamut))
	{
	case EDisplayColorGamut::sRGB_D65:
	case EDisplayColorGamut::DCIP3_D65:
	case EDisplayColorGamut::Rec2020_D65:
		bMatch = ColorSpace.GetWhiteChromaticity().Equals(FVector2d(.3127, .329), Tolerance);
		break;
	case EDisplayColorGamut::ACES_D60:
	case EDisplayColorGamut::ACEScg_D60:
		bMatch = ColorSpace.GetWhiteChromaticity().Equals(FVector2d(.32168, .33767), Tolerance);
		break;
	default:
		checkNoEntry();
	}

	return bMatch;
}

ETonemappingMethod GetTonemappingMethod(const FPostProcessSettings& Settings)
{
	const int32 TonemappingMethodOverride = CVarLUTTonemappingMethod.GetValueOnRenderThread();
	return (TonemappingMethodOverride >= 0) ?
		static_cast<ETonemappingMethod>(TonemappingMethodOverride) :
		Settings.TonemappingMethod;
}

float GetTonemappingMax(const FTonemapperOutputDeviceParameters& Parameters, ETonemappingMethod TonemappingMethod)
{
	float LUTMax = 1.f;
	if (TonemappingMethod == ETonemappingMethod::Filmic)
	{
		switch (static_cast<EDisplayOutputFormat>(Parameters.OutputDevice))
		{
		case EDisplayOutputFormat::SDR_sRGB:
		case EDisplayOutputFormat::SDR_Rec709:
		case EDisplayOutputFormat::HDR_LinearWithToneCurve:
				LUTMax =  1.05f;
		}
	}
	return LUTMax;
}

struct FCachedLUTSettings
{
	uint32 UniqueID = 0;
	EShaderPlatform ShaderPlatform = GMaxRHIShaderPlatform;
	ETonemappingMethod TonemappingMethod{};
	FCombineLUTParameters Parameters;
	FWorkingColorSpaceShaderParameters WorkingColorSpaceShaderParameters;
	int32 AcesVersion = 0;

	bool UpdateCachedValues(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FPostProcessSettings& Settings, const FTexture* const* Textures, const float* Weights, uint32 BlendCount)
	{
		bool bHasChanged = false;
		UPDATE_CACHE_SETTINGS(TonemappingMethod, GetTonemappingMethod(Settings), bHasChanged);
		GetCombineLUTParameters(GraphBuilder, View, Settings, Textures, Weights, BlendCount, bHasChanged);
		UPDATE_CACHE_SETTINGS(UniqueID, View.State ? View.State->GetViewKey() : 0, bHasChanged);
		UPDATE_CACHE_SETTINGS(ShaderPlatform, View.GetShaderPlatform(), bHasChanged);
		static const auto CVarAcesVersion = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.Aces.Version"));
		UPDATE_CACHE_SETTINGS(AcesVersion, CVarAcesVersion->GetValueOnRenderThread(), bHasChanged);

		const FWorkingColorSpaceShaderParameters* InWorkingColorSpaceShaderParameters = reinterpret_cast<const FWorkingColorSpaceShaderParameters*>(GDefaultWorkingColorSpaceUniformBuffer.GetContents());
		if (InWorkingColorSpaceShaderParameters)
		{
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.ToXYZ, InWorkingColorSpaceShaderParameters->ToXYZ, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.FromXYZ, InWorkingColorSpaceShaderParameters->FromXYZ, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.ToAP1, InWorkingColorSpaceShaderParameters->ToAP1, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.FromAP1, InWorkingColorSpaceShaderParameters->FromAP1, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.ToAP0, InWorkingColorSpaceShaderParameters->ToAP0, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.bIsSRGB, InWorkingColorSpaceShaderParameters->bIsSRGB, bHasChanged);
		}

		return bHasChanged;
	}
	
	void GetCombineLUTParameters(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FPostProcessSettings& Settings,
		const FTexture* const* Textures,
		const float* Weights,
		uint32 BlendCount,
		bool& bHasChanged)
	{
		check(Textures);
		check(Weights);

		for (uint32 BlendIndex = 0; BlendIndex < BlendCount; ++BlendIndex)
		{
			// Neutral texture occupies the first slot and doesn't actually need to be set.
			if (BlendIndex != 0)
			{
				check(Textures[BlendIndex]);

				// Don't use texture asset sampler as it might have anisotropic filtering enabled
				UPDATE_CACHE_SETTINGS(Parameters.Textures[BlendIndex], Textures[BlendIndex]->TextureRHI, bHasChanged);
				Parameters.Samplers[BlendIndex] = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp, 0, 1>::GetRHI();
			}

			UPDATE_CACHE_SETTINGS(GET_SCALAR_ARRAY_ELEMENT(Parameters.LUTWeights, BlendIndex), Weights[BlendIndex], bHasChanged);
		}

		Parameters.WorkingColorSpace = GDefaultWorkingColorSpaceUniformBuffer.GetUniformBufferRef();

		const FSceneViewFamily& ViewFamily = *(View.Family);
		FTonemapperOutputDeviceParameters TonemapperOutputDeviceParameters = GetTonemapperOutputDeviceParameters(ViewFamily);
		UPDATE_CACHE_SETTINGS(Parameters.OutputDevice.InverseGamma, TonemapperOutputDeviceParameters.InverseGamma, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.OutputDevice.OutputDevice, TonemapperOutputDeviceParameters.OutputDevice, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.OutputDevice.OutputGamut, TonemapperOutputDeviceParameters.OutputGamut, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.OutputDevice.OutputMaxLuminance, TonemapperOutputDeviceParameters.OutputMaxLuminance, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.OutputDevice.LUTShaper, TonemapperOutputDeviceParameters.LUTShaper, bHasChanged);

		UPDATE_CACHE_SETTINGS(Parameters.ColorScale, FVector3f(View.ColorScale), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.OverlayColor, FVector4f(View.OverlayColor), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorRemap.MappingPolynomial, GetMappingPolynomial(), bHasChanged);

		// White balance
		UPDATE_CACHE_SETTINGS(Parameters.bIsTemperatureWhiteBalance, uint32(Settings.TemperatureType == ETemperatureMethod::TEMP_WhiteBalance), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.LUTInvMax, 1.f / GetTonemappingMax(TonemapperOutputDeviceParameters, TonemappingMethod), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.LUTSize, (float)CVarLUTSize->GetInt(), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.InnerLUTSize, (float)CVarLUTInnerSize->GetInt(), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.WhiteTemp, Settings.WhiteTemp, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.WhiteTint, Settings.WhiteTint, bHasChanged);

		// Color grade
		UPDATE_CACHE_SETTINGS(Parameters.ColorSaturation, FVector4f(Settings.ColorSaturation), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorContrast, FVector4f(Settings.ColorContrast), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorGamma, FVector4f(Settings.ColorGamma), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorGain, FVector4f(Settings.ColorGain), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorOffset, FVector4f(Settings.ColorOffset), bHasChanged);

		UPDATE_CACHE_SETTINGS(Parameters.ColorSaturationShadows, FVector4f(Settings.ColorSaturationShadows), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorContrastShadows, FVector4f(Settings.ColorContrastShadows), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorGammaShadows, FVector4f(Settings.ColorGammaShadows), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorGainShadows, FVector4f(Settings.ColorGainShadows), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorOffsetShadows, FVector4f(Settings.ColorOffsetShadows), bHasChanged);

		UPDATE_CACHE_SETTINGS(Parameters.ColorSaturationMidtones, FVector4f(Settings.ColorSaturationMidtones), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorContrastMidtones, FVector4f(Settings.ColorContrastMidtones), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorGammaMidtones, FVector4f(Settings.ColorGammaMidtones), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorGainMidtones, FVector4f(Settings.ColorGainMidtones), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorOffsetMidtones, FVector4f(Settings.ColorOffsetMidtones), bHasChanged);

		UPDATE_CACHE_SETTINGS(Parameters.ColorSaturationHighlights, FVector4f(Settings.ColorSaturationHighlights), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorContrastHighlights, FVector4f(Settings.ColorContrastHighlights), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorGammaHighlights, FVector4f(Settings.ColorGammaHighlights), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorGainHighlights, FVector4f(Settings.ColorGainHighlights), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorOffsetHighlights, FVector4f(Settings.ColorOffsetHighlights), bHasChanged);

		UPDATE_CACHE_SETTINGS(Parameters.ColorCorrectionShadowsMax, Settings.ColorCorrectionShadowsMax, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorCorrectionHighlightsMin, Settings.ColorCorrectionHighlightsMin, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ColorCorrectionHighlightsMax, Settings.ColorCorrectionHighlightsMax, bHasChanged);

		UPDATE_CACHE_SETTINGS(Parameters.BlueCorrection, Settings.BlueCorrection, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ExpandGamut, Settings.ExpandGamut, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ToneCurveAmount, Settings.ToneCurveAmount, bHasChanged);

		const int32 TonemappingSDRInvEOTFOverride = CVarLUTTonemappingSDRInvEOTF.GetValueOnRenderThread();
		const uint32 TonemappingSDRInvEOTF = (TonemappingSDRInvEOTFOverride >= 0) ?
			static_cast<uint32>(TonemappingSDRInvEOTFOverride) :
			static_cast<uint32>(Settings.TonemappingMethod == ETonemappingMethod::StandardACES);
		UPDATE_CACHE_SETTINGS(Parameters.TonemappingSDRInvEOTF, TonemappingSDRInvEOTF, bHasChanged);

		UPDATE_CACHE_SETTINGS(Parameters.FilmSlope, Settings.FilmSlope, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.FilmToe, Settings.FilmToe, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.FilmShoulder, Settings.FilmShoulder, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.FilmBlackClip, Settings.FilmBlackClip, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.FilmWhiteClip, Settings.FilmWhiteClip, bHasChanged);
	}
};

class FLUTBlenderCS : public FGlobalShader
{
public:
	static constexpr int32 GroupSize = 4;

	class FBlendCount : SHADER_PERMUTATION_RANGE_INT("BLENDCOUNT", 1, 5);
	class FOutputDeviceSRGB : SHADER_PERMUTATION_BOOL("OUTPUT_DEVICE_SRGB");
	class FTonemappingMethod : SHADER_PERMUTATION_ENUM_CLASS("TONEMAPPING_METHOD", ETonemappingMethod);
	using FPermutationDomain = TShaderPermutationDomain<FBlendCount, FOutputDeviceSRGB, FTonemappingMethod>;

	DECLARE_GLOBAL_SHADER(FLUTBlenderCS);
	SHADER_USE_PARAMETER_STRUCT(FLUTBlenderCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FCombineLUTParameters, CombineLUT)
		SHADER_PARAMETER(float, InvLUTSizeMinusOne)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWOutputTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, InnerLUT)
		SHADER_PARAMETER_SAMPLER(SamplerState, InnerLUTSampler)
	END_SHADER_PARAMETER_STRUCT()

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FLUTBlenderCS, "/Engine/Private/PostProcessCombineLUTs.usf", "MainCS", SF_Compute);

class FLUTInnerCS : public FGlobalShader
{
public:
	static const int32 GroupSize = 4;

	class FOutputDeviceSRGB : SHADER_PERMUTATION_BOOL("OUTPUT_DEVICE_SRGB");
	class FUseACES2 : SHADER_PERMUTATION_BOOL("USE_ACES_2");
	using FPermutationDomain = TShaderPermutationDomain<FOutputDeviceSRGB, FUseACES2>;

	DECLARE_GLOBAL_SHADER(FLUTInnerCS);
	SHADER_USE_PARAMETER_STRUCT(FLUTInnerCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FInnerLUTParameters, InnerLUT)
		SHADER_PARAMETER(float, InvLUTSizeMinusOne)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, RWOutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), GroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FLUTInnerCS, "/Engine/Private/PostProcessCombineLUTsInner.usf", "MainCS", SF_Compute);

struct FCachedInnerLUTSettings
{
	uint32 UniqueID = 0;
	EShaderPlatform ShaderPlatform = GMaxRHIShaderPlatform;
	FInnerLUTParameters Parameters;
	FWorkingColorSpaceShaderParameters WorkingColorSpaceShaderParameters;
	bool bUseCompute = false;
	int32 AcesVersion = 0;
	int32 InnerLUTSize = 0;

	bool UpdateCachedValues(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 InInnerLUTSize, const FTexture* const* Textures, const float* Weights, uint32 BlendCount)
	{
		bool bHasChanged = false;
		GetInnerLUTParameters(GraphBuilder, View, bHasChanged);
		UPDATE_CACHE_SETTINGS(UniqueID, View.State ? View.State->GetViewKey() : 0, bHasChanged);
		UPDATE_CACHE_SETTINGS(ShaderPlatform, View.GetShaderPlatform(), bHasChanged);
		UPDATE_CACHE_SETTINGS(bUseCompute, View.bUseComputePasses, bHasChanged);
		static const auto CVarAcesVersion = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.Aces.Version"));
		UPDATE_CACHE_SETTINGS(AcesVersion, CVarAcesVersion->GetValueOnRenderThread(), bHasChanged);
		UPDATE_CACHE_SETTINGS(InnerLUTSize, InInnerLUTSize, bHasChanged);

		const FWorkingColorSpaceShaderParameters* InWorkingColorSpaceShaderParameters = reinterpret_cast<const FWorkingColorSpaceShaderParameters*>(GDefaultWorkingColorSpaceUniformBuffer.GetContents());
		if (InWorkingColorSpaceShaderParameters)
		{
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.ToXYZ, InWorkingColorSpaceShaderParameters->ToXYZ, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.FromXYZ, InWorkingColorSpaceShaderParameters->FromXYZ, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.ToAP1, InWorkingColorSpaceShaderParameters->ToAP1, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.FromAP1, InWorkingColorSpaceShaderParameters->FromAP1, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.ToAP0, InWorkingColorSpaceShaderParameters->ToAP0, bHasChanged);
			UPDATE_CACHE_SETTINGS(WorkingColorSpaceShaderParameters.bIsSRGB, InWorkingColorSpaceShaderParameters->bIsSRGB, bHasChanged);
		}

		return bHasChanged;
	}
	
	void GetInnerLUTParameters(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		bool& bHasChanged)
	{
		const FSceneViewFamily& ViewFamily = *(View.Family);
		const FPostProcessSettings& Settings = (
			ViewFamily.EngineShowFlags.ColorGrading && CVarColorGrading.GetValueOnRenderThread() != 0)
			? View.FinalPostProcessSettings
			: FPostProcessSettings::GetDefault();

		Parameters.WorkingColorSpace = GDefaultWorkingColorSpaceUniformBuffer.GetUniformBufferRef();

		FTonemapperOutputDeviceParameters TonemapperOutputDeviceParameters = GetTonemapperOutputDeviceParameters(ViewFamily);
		UPDATE_CACHE_SETTINGS(Parameters.OutputDevice.InverseGamma, TonemapperOutputDeviceParameters.InverseGamma, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.OutputDevice.OutputDevice, TonemapperOutputDeviceParameters.OutputDevice, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.OutputDevice.OutputGamut, TonemapperOutputDeviceParameters.OutputGamut, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.OutputDevice.OutputMaxLuminance, TonemapperOutputDeviceParameters.OutputMaxLuminance, bHasChanged);

		const UE::Color::FColorSpace& LimitingColorSpace = ViewFamily.RenderTarget->GetLimitingColorSpace();
		const FMatrix44d& LimitingRgbToXYZ = LimitingColorSpace.GetRgbToXYZ();
		UPDATE_CACHE_SETTINGS(Parameters.LimitingRgbToXYZ0, FVector3f(LimitingRgbToXYZ.GetColumn(0)), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.LimitingRgbToXYZ1, FVector3f(LimitingRgbToXYZ.GetColumn(1)), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.LimitingRgbToXYZ2, FVector3f(LimitingRgbToXYZ.GetColumn(2)), bHasChanged);
		const FMatrix44d& LimitingXYZToRgb = LimitingColorSpace.GetXYZToRgb();
		UPDATE_CACHE_SETTINGS(Parameters.LimitingXYZToRgb0, FVector3f(LimitingXYZToRgb.GetColumn(0)), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.LimitingXYZToRgb1, FVector3f(LimitingXYZToRgb.GetColumn(1)), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.LimitingXYZToRgb2, FVector3f(LimitingXYZToRgb.GetColumn(2)), bHasChanged);

		FACESTonemapParams TonemapperParams;
		GetACESTonemapParameters(TonemapperParams);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.ACESMinMaxData, TonemapperParams.ACESMinMaxData, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.ACESMidData, TonemapperParams.ACESMidData, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.ACESCoefsLow_0, TonemapperParams.ACESCoefsLow_0, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.ACESCoefsHigh_0, TonemapperParams.ACESCoefsHigh_0, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.ACESCoefsLow_4, TonemapperParams.ACESCoefsLow_4, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.ACESCoefsHigh_4, TonemapperParams.ACESCoefsHigh_4, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.ACESSceneColorMultiplier, TonemapperParams.ACESSceneColorMultiplier, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.ACESGamutCompression, TonemapperParams.ACESGamutCompression, bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.bScaleWhite, !DoesGamutWhitePointMatch(TonemapperOutputDeviceParameters.OutputGamut, LimitingColorSpace), bHasChanged);
		UPDATE_CACHE_SETTINGS(Parameters.ACESTonemapParameters.bScaleWhiteSDR, !DoesGamutWhitePointMatch(TonemapperOutputDeviceParameters.OutputGamut, UE::Color::FColorSpace::GetWorking()), bHasChanged);
		UE::Color::ACES::GetTransformResources(
			GraphBuilder,
			TonemapperOutputDeviceParameters.OutputMaxLuminance,
			LimitingColorSpace,
			Parameters.ACESTonemapParameters.ACESReachTable,
			Parameters.ACESTonemapParameters.ACESGamutTable,
			Parameters.ACESTonemapParameters.ACESGammaTable
		);
		UE::Color::ACES::GetTransformResourcesSDR(
			GraphBuilder,
			Parameters.ACESTonemapParameters.ACESReachTableSDR,
			Parameters.ACESTonemapParameters.ACESGamutTableSDR,
			Parameters.ACESTonemapParameters.ACESGammaTableSDR
		);

		const float HDRPaperWhiteRatio = ViewFamily.RenderTarget->GetHDRPaperWhiteInNits() / 203.f;
		UPDATE_CACHE_SETTINGS(Parameters.TotalSceneColorMultiplier, HDRPaperWhiteRatio * TonemapperParams.ACESSceneColorMultiplier, bHasChanged);

		UPDATE_CACHE_SETTINGS(Parameters.TonemappingMethod, static_cast<uint32>(GetTonemappingMethod(Settings)), bHasChanged);
	}
};

uint32 GenerateFinalTable(const FFinalPostProcessSettings& Settings, const FTexture* OutTextures[], float OutWeights[], uint32 MaxCount)
{
	// Find the n strongest contributors, drop small contributors.
	uint32 LocalCount = 1;

	// Add the neutral one (done in the shader) as it should be the first and always there.
	OutTextures[0] = nullptr;
	OutWeights[0] = 0.0f;

	// Neutral index is the entry with no LUT texture assigned.
	for (int32 Index = 0; Index < Settings.ContributingLUTs.Num(); ++Index)
	{
		if (Settings.ContributingLUTs[Index].LUTTexture == nullptr)
		{
			OutWeights[0] = Settings.ContributingLUTs[Index].Weight;
			break;
		}
	}

	float OutWeightsSum = OutWeights[0];
	for (; LocalCount < MaxCount; ++LocalCount)
	{
		int32 BestIndex = INDEX_NONE;

		// Find the one with the strongest weight, add until full.
		for (int32 InputIndex = 0; InputIndex < Settings.ContributingLUTs.Num(); ++InputIndex)
		{
			bool bAlreadyInArray = false;

			{
				UTexture* LUTTexture = Settings.ContributingLUTs[InputIndex].LUTTexture;
				FTexture* Internal = LUTTexture ? LUTTexture->GetResource() : nullptr;
				for (uint32 OutputIndex = 0; OutputIndex < LocalCount; ++OutputIndex)
				{
					if (Internal == OutTextures[OutputIndex])
					{
						bAlreadyInArray = true;
						break;
					}
				}
			}

			if (bAlreadyInArray)
			{
				// We already have this one.
				continue;
			}

			// Take the first or better entry.
			if (BestIndex == INDEX_NONE || Settings.ContributingLUTs[BestIndex].Weight <= Settings.ContributingLUTs[InputIndex].Weight)
			{
				BestIndex = InputIndex;
			}
		}

		if (BestIndex == INDEX_NONE)
		{
			// No more elements to process.
			break;
		}

		const float WeightThreshold = 1.0f / 512.0f;

		const float BestWeight = Settings.ContributingLUTs[BestIndex].Weight;

		if (BestWeight < WeightThreshold)
		{
			// Drop small contributor 
			break;
		}

		UTexture* BestLUTTexture = Settings.ContributingLUTs[BestIndex].LUTTexture;
		FTexture* BestInternal = BestLUTTexture ? BestLUTTexture->GetResource() : nullptr;

		OutTextures[LocalCount] = BestInternal;
		OutWeights[LocalCount] = BestWeight;
		OutWeightsSum += BestWeight;
	}

	// Normalize the weights.
	if (OutWeightsSum > 0.001f)
	{
		const float OutWeightsSumInverse = 1.0f / OutWeightsSum;

		for (uint32 Index = 0; Index < LocalCount; ++Index)
		{
			OutWeights[Index] *= OutWeightsSumInverse;
		}
	}
	else
	{
		// Just the neutral texture at full weight.
		OutWeights[0] = 1.0f;
		LocalCount = 1;
	}

	return LocalCount;
}

FRDGTextureRef AddCombineLUTPass(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	static FCachedLUTSettings CachedLUTSettings;
	static FCachedInnerLUTSettings CachedInnerLUTSettings;

	const FSceneViewFamily& ViewFamily = *(View.Family);
	const FTexture* LocalTextures[GMaxLUTBlendCount];
	float LocalWeights[GMaxLUTBlendCount];
	uint32 LocalCount = 1;

	// Default to no LUTs.
	LocalTextures[0] = nullptr;
	LocalWeights[0] = 1.0f;

	if (ViewFamily.EngineShowFlags.ColorGrading)
	{
		LocalCount = GenerateFinalTable(View.FinalPostProcessSettings, LocalTextures, LocalWeights, GMaxLUTBlendCount);
	}

	const EDisplayOutputFormat OutputDevice = ViewFamily.RenderTarget->GetDisplayOutputFormat();
	// Note: We also enforce floats with HDR ScRGB (editor only) to preserve negative values.
	const bool bUseFloatOutput = ViewFamily.SceneCaptureSource == SCS_FinalColorHDR ||
		ViewFamily.SceneCaptureSource == SCS_FinalToneCurveHDR ||
		OutputDevice == EDisplayOutputFormat::HDR_ACES_1000nit_ScRGB ||
		OutputDevice == EDisplayOutputFormat::HDR_ACES_2000nit_ScRGB;

	// Attempt to register the persistent view LUT texture.
	const int32 LUTSize = CVarLUTSize->GetInt();
	FRDGTextureRef OutputTexture = TryRegisterExternalTexture(GraphBuilder,
		View.GetTonemappingLUT(GraphBuilder.RHICmdList, LUTSize, bUseFloatOutput));

	View.SetValidTonemappingLUT();

	const FPostProcessSettings& Settings = (
		ViewFamily.EngineShowFlags.ColorGrading && CVarColorGrading.GetValueOnRenderThread() != 0)
		? View.FinalPostProcessSettings
		: FPostProcessSettings::GetDefault();

	const int32 InnerLUTSize = CVarLUTInnerSize->GetInt();
	const bool bHasInnerChanged = CachedInnerLUTSettings.UpdateCachedValues(GraphBuilder, View, InnerLUTSize, LocalTextures, LocalWeights, LocalCount);
	const bool bHasChanged = CachedLUTSettings.UpdateCachedValues(GraphBuilder, View, Settings, LocalTextures, LocalWeights, LocalCount);
	if (!bHasInnerChanged && !bHasChanged && OutputTexture && CVarLUTUpdateEveryFrame.GetValueOnRenderThread() == 0)
	{
		return OutputTexture;
	}

	// View doesn't support a persistent LUT, so create a temporary one.
	if (!OutputTexture)
	{
		OutputTexture = GraphBuilder.CreateTexture(
			Translate(FSceneViewState::CreateLUTRenderTarget(LUTSize, bUseFloatOutput, TEXT("CombineLUTs"))),
			TEXT("CombineLUTs"));
	}

	check(OutputTexture);

	const bool bOutputDeviceSRGB = (CachedLUTSettings.Parameters.OutputDevice.OutputDevice == (uint32)EDisplayOutputFormat::SDR_sRGB);
	FRDGTextureRef InnerTexture;

	const ETonemappingMethod TonemappingMethod = static_cast<ETonemappingMethod>(CachedInnerLUTSettings.Parameters.TonemappingMethod);
	const bool bNeedsInnerTonemappingLUT =
		(TonemappingMethod == ETonemappingMethod::StandardACES) ||
		((TonemappingMethod == ETonemappingMethod::Filmic) && CachedInnerLUTSettings.Parameters.OutputDevice.OutputMaxLuminance > 100.f);
	if (bNeedsInnerTonemappingLUT)
	{
		InnerTexture = TryRegisterExternalTexture(GraphBuilder,
			View.GetInnerTonemappingLUT(GraphBuilder.RHICmdList, InnerLUTSize, bUseFloatOutput));

		if (bHasInnerChanged || !InnerTexture || CVarLUTInnerUpdateEveryFrame.GetValueOnRenderThread())
		{
			// View doesn't support a persistent LUT, so create a temporary one.
			if (!InnerTexture)
			{
				InnerTexture = GraphBuilder.CreateTexture(
					Translate(FSceneViewState::CreateLUTRenderTarget(InnerLUTSize, bUseFloatOutput, TEXT("CombineLUTs_Inner"))),
					TEXT("CombineLUTs_Inner"));
			}

			check(InnerTexture);

			FLUTInnerCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FLUTInnerCS::FUseACES2>(CachedInnerLUTSettings.AcesVersion > 1);

			FLUTInnerCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FLUTInnerCS::FParameters>();
			PassParameters->InnerLUT = CachedInnerLUTSettings.Parameters;
			PassParameters->InvLUTSizeMinusOne = 1.f / (InnerLUTSize - 1);
			PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(InnerTexture);

			PermutationVector.Set<FLUTInnerCS::FOutputDeviceSRGB>(bOutputDeviceSRGB);
			TShaderMapRef<FLUTInnerCS> ComputeShader(View.ShaderMap, PermutationVector);

			const uint32 GroupSize = FMath::DivideAndRoundUp(InnerLUTSize, FLUTInnerCS::GroupSize);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("CombineLUTs_Inner %d (CS)", InnerLUTSize),
				CVarLUTAsyncCompute.GetValueOnRenderThread() ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FIntVector(GroupSize, GroupSize, GroupSize));
		}
	}
	else
	{
		View.ClearInnerTonemappingLUT();
		InnerTexture = RegisterExternalTexture(GraphBuilder, GBlackVolumeTexture->GetTextureRHI(), TEXT("CombineLUTs_Inner_Fallback"));
	}

	FLUTBlenderCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FLUTBlenderCS::FBlendCount>(LocalCount);
	PermutationVector.Set<FLUTBlenderCS::FTonemappingMethod>(CachedLUTSettings.TonemappingMethod);

	FLUTBlenderCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FLUTBlenderCS::FParameters>();
	PassParameters->CombineLUT = CachedLUTSettings.Parameters;
	PassParameters->InnerLUT = InnerTexture;
	PassParameters->InnerLUTSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->InvLUTSizeMinusOne = 1.f / (LUTSize - 1);
	PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(OutputTexture);

	PermutationVector.Set<FLUTBlenderCS::FOutputDeviceSRGB>(bOutputDeviceSRGB);
	TShaderMapRef<FLUTBlenderCS> ComputeShader(View.ShaderMap, PermutationVector);

	const uint32 GroupSize = FMath::DivideAndRoundUp(LUTSize, FLUTBlenderCS::GroupSize);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("CombineLUTs %d (CS)", LUTSize),
		CVarLUTAsyncCompute.GetValueOnRenderThread() ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
		ComputeShader,
		PassParameters,
		FIntVector(GroupSize, GroupSize, GroupSize));

	return OutputTexture;
}
