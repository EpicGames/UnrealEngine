// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

BEGIN_SHADER_PARAMETER_STRUCT(FColorRemapParameters, )
	SHADER_PARAMETER(FVector3f, MappingPolynomial)
END_SHADER_PARAMETER_STRUCT()

FColorRemapParameters GetColorRemapParameters();

FRDGTextureRef AddCombineLUTPass(FRDGBuilder& GraphBuilder, const FViewInfo& View);
