// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessCompositeDebugPrimitives.h"
#if UE_ENABLE_DEBUG_DRAWING
#include "ScenePrivate.h"
#include "PixelShaderUtils.h"
#include "PostProcess/PostProcessing.h"

#include "InstanceCulling/InstanceCullingOcclusionQuery.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"

BEGIN_SHADER_PARAMETER_STRUCT(FDebugPrimitivesPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

struct FDebugPrimitivesDrawingContext
{
	const FViewInfo* DebugView = nullptr;
	FRDGTextureRef DebugPrimitiveColor = nullptr;
	FRDGTextureRef DebugPrimitiveDepth = nullptr;
	FRDGTextureRef UpscaledSceneDepth = nullptr;
	FScreenPassTexture SceneColor {};
	FScreenPassTexture SceneDepth {};
	FScreenPassRenderTarget OutputRT {};
	FScreenPassRenderTarget OutputDepthRT {};
};

static FDebugPrimitivesDrawingContext InitializeDebugPrimitiveTextures(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View,
	const FCompositePrimitiveInputs& Inputs,
	const FScreenPassRenderTarget& Output)
{
	// Setup view
	const FIntRect ViewRect = Output.ViewRect;
	FIntPoint OutputExtent = Output.Texture->Desc.Extent;
	FIntPoint SceneColorExtent = Inputs.SceneColor.Texture->Desc.Extent;

	FViewFamilyInfo* ViewFamily = (FViewFamilyInfo*)View.Family;
	int NumMSAASamples = 1;
#if WITH_EDITOR
	NumMSAASamples = ViewFamily->SceneTexturesConfig.EditorPrimitiveNumSamples;
#endif

	const FViewInfo* DebugView = CreateCompositePrimitiveView(View, ViewRect, NumMSAASamples);	

	// Setup textures
	const FRDGTextureDesc ColorDesc(FRDGTextureDesc::Create2D(OutputExtent, PF_B8G8R8A8, FClearValueBinding::Transparent, TexCreate_ShaderResource | TexCreate_RenderTargetable, 1, NumMSAASamples));
	FRDGTextureRef DebugPrimitiveColor = GraphBuilder.CreateTexture(ColorDesc, TEXT("Debug.PrimitivesColor"));

	FRDGTextureDesc DepthDesc(FRDGTextureDesc::Create2D(OutputExtent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_ShaderResource | TexCreate_DepthStencilTargetable, 1, NumMSAASamples));
	DepthDesc.AliasableFormats.Add(PF_X24_G8);
	FRDGTextureRef DebugPrimitiveDepth = GraphBuilder.CreateTexture(DepthDesc, TEXT("Debug.PrimitivesDepth"));
	
	
	AddClearRenderTargetPass(GraphBuilder, DebugPrimitiveColor);
	AddClearDepthStencilPass(GraphBuilder, DebugPrimitiveDepth);

	FRDGTextureDesc UpscaledSceneDepthDesc(FRDGTextureDesc::Create2D(SceneColorExtent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_ShaderResource | TexCreate_DepthStencilTargetable, 1, Inputs.SceneColor.Texture->Desc.NumSamples));
	
	FRDGTextureRef UpscaledSceneDepth = GraphBuilder.CreateTexture(UpscaledSceneDepthDesc, TEXT("Debug.UpscaledWorldDepth"));

	// Debug primitives with SDPG_World are depth tested against the scene depth, and immediately composed on SceneColor.
	// SceneDepth is upscaled, to match the size of post-TSR SceneColor.
	// Other debug primitives are only depth tested amongst themselves, and drawn to PrimitivesColor/PrimitivesDepth.
	// Finally, PrimitivesColor is drawn to SceneColor, with occluded lines (tested against SceneDepth) drawn in a dithered style.
	{
		FScreenPassTexture UpsampledDepth = Inputs.SceneDepth;
		FVector2f SceneDepthJitter = FVector2f(View.TemporalJitterPixels);

		if (IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod))
		{
			TemporalUpscaleDepthPass(GraphBuilder,
									 *DebugView,
									 Inputs.SceneColor,
									 UpsampledDepth,
									 SceneDepthJitter);
		}

		//Simple element pixel shaders do not output background color for composite, 
		//so this allows the background to be drawn to the RT at the same time as depth without adding extra draw calls
		PopulateDepthPass(GraphBuilder,
						  *DebugView,
						  Inputs.SceneColor,
						  UpsampledDepth,
						  nullptr,
						  UpscaledSceneDepth,
						  Inputs.SceneColor.ViewRect,
						  SceneDepthJitter,
						  UpscaledSceneDepth->Desc.NumSamples,
						  false,
						  Inputs.bUseMetalMSAAHDRDecode);
	}

	FDebugPrimitivesDrawingContext Result {};
	Result.DebugView = DebugView;
	Result.DebugPrimitiveColor = DebugPrimitiveColor;
	Result.DebugPrimitiveDepth = DebugPrimitiveDepth;
	Result.UpscaledSceneDepth = UpscaledSceneDepth;
	Result.SceneColor = Inputs.SceneColor;
	Result.SceneDepth = Inputs.SceneDepth;
	Result.OutputRT = Output;
	Result.OutputDepthRT = Inputs.OverrideDepthOutput;

	return Result;
}

static bool DrawDebugPDE(
	FRDGBuilder& GraphBuilder,
	FDebugPrimitivesDrawingContext& Context,
	const FScreenPassTextureViewport& OutputViewport)
{
	if (!Context.DebugView->DebugSimpleElementCollector.HasAnyPrimitives())
	{
		return false;
	}

	// Draw SDPG_World elements, depth tested with SceneColor
	if (Context.DebugView->DebugSimpleElementCollector.HasPrimitives(SDPG_World))
	{
		FDebugPrimitivesPassParameters* PassParameters = GraphBuilder.AllocParameters<FDebugPrimitivesPassParameters>();
		PassParameters->View = Context.DebugView->GetShaderParameters();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(Context.SceneColor.Texture, ERenderTargetLoadAction::ELoad);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Context.UpscaledSceneDepth, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("DrawDebugPrimitives(SDPG_World)"),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, DebugView=Context.DebugView, ViewportRect=Context.SceneColor.ViewRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(ViewportRect.Min.X, ViewportRect.Min.Y, 0.0f, ViewportRect.Max.X, ViewportRect.Max.Y, 1.0f);

			FMeshPassProcessorRenderState DrawRenderState;
			DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite);
			DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI());

			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());		
			DebugView->DebugSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, *DebugView, EBlendModeFilter::OpaqueAndMasked, SDPG_World);	
		});
	}

	// Draw SDPG_Foreground elements, depth tested only with other debug primitives
	if (Context.DebugView->DebugSimpleElementCollector.HasPrimitives(SDPG_Foreground))
	{
		FDebugPrimitivesPassParameters* PassParameters = GraphBuilder.AllocParameters<FDebugPrimitivesPassParameters>();
		PassParameters->View = Context.DebugView->GetShaderParameters();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(Context.DebugPrimitiveColor, ERenderTargetLoadAction::EClear);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(Context.DebugPrimitiveDepth, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("DrawDebugPrimitives(SDPG_Foreground)"),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, DebugView=Context.DebugView, OutputViewport](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

			FMeshPassProcessorRenderState DrawRenderState;
			DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite);
			DrawRenderState.SetBlendState(TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI());

			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_Always>::GetRHI());
			DebugView->DebugSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, *DebugView, EBlendModeFilter::OpaqueAndMasked, SDPG_Foreground);		
		});
	}

	return true;
}

static void ComposeDebugPrimitives(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FCompositePrimitiveInputs& Inputs,
	FDebugPrimitivesDrawingContext& Context)
{
	const FScreenPassRenderTarget& DepthOutput = Context.OutputDepthRT;
	int NumMSAASamples = Context.DebugPrimitiveColor->Desc.NumSamples;
	
	const FScreenPassTexture& SceneDepth = Context.SceneDepth;
	FVector2f SceneDepthJitter = FVector2f(View.TemporalJitterPixels);

	const FScreenPassTextureViewport DebugPrimitivesViewport(Context.DebugPrimitiveColor, Context.DebugView->ViewRect);

	bool bCompositeAnyNonNullDepth = false;

	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const bool bOpaqueEditorGizmo = View.Family->EngineShowFlags.OpaqueCompositeEditorPrimitives || View.Family->EngineShowFlags.Wireframe;

	FCompositePostProcessPrimitivesPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositePostProcessPrimitivesPS::FParameters>();
	PassParameters->RenderTargets[0] = Context.OutputRT.GetRenderTargetBinding();

	bool bOutputIsMSAA = Context.OutputRT.Texture->Desc.NumSamples > 1;
	if (DepthOutput.IsValid())
	{
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthOutput.Texture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite);
		verify(Context.OutputRT.Texture->Desc.NumSamples == DepthOutput.Texture->Desc.NumSamples);
	}

	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->Color = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Context.SceneColor));
	PassParameters->Depth = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(SceneDepth));
	PassParameters->EditorPrimitives = GetScreenPassTextureViewportParameters(DebugPrimitivesViewport);

	PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
	PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	if (Inputs.LensDistortionLUT.IsEnabled())
	{
		PassParameters->UndistortingDisplacementTexture = Inputs.LensDistortionLUT.UndistortingDisplacementTexture;
	}

	PassParameters->ColorTexture = Context.SceneColor.Texture;
	PassParameters->ColorSampler = PointClampSampler;
	PassParameters->DepthTexture = SceneDepth.Texture;
	PassParameters->DepthSampler = PointClampSampler;
	PassParameters->EditorPrimitivesDepth = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(Context.DebugPrimitiveDepth, PF_DepthStencil));
	PassParameters->EditorPrimitivesStencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(Context.DebugPrimitiveDepth, PF_X24_G8));
	PassParameters->EditorPrimitivesColor = Context.DebugPrimitiveColor;
		
	PassParameters->PassSvPositionToViewportUV = FScreenTransform::SvPositionToViewportUV(Context.OutputRT.ViewRect);
	PassParameters->ViewportUVToColorUV = FScreenTransform::ChangeTextureBasisFromTo(
		FScreenPassTextureViewport(Context.SceneColor), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
	PassParameters->ViewportUVToDepthUV = FScreenTransform::ChangeTextureBasisFromTo(
		FScreenPassTextureViewport(SceneDepth), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
	PassParameters->ViewportUVToEditorPrimitivesUV = FScreenTransform::ChangeTextureBasisFromTo(
		DebugPrimitivesViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
		
	PassParameters->bOpaqueEditorGizmo = bOpaqueEditorGizmo;
	PassParameters->bCompositeAnyNonNullDepth = bCompositeAnyNonNullDepth;
	PassParameters->DepthTextureJitter = SceneDepthJitter;
	PassParameters->bProcessAlpha = IsPostProcessingWithAlphaChannelSupported();
	PassParameters->OccludedDitheringPatternSize = Inputs.OccludedDitheringPatternSize; // Initialize, but this won't have any visual effect (dithering is disabled below)
	PassParameters->OccludedDithering = 0.0f;
	PassParameters->OccludedBrightness = 1.0f;

	for (int32 i = 0; i < int32(NumMSAASamples); i++)
	{
		PassParameters->SampleOffsetArray[i].X = GetMSAASampleOffsets(NumMSAASamples, i).X;
		PassParameters->SampleOffsetArray[i].Y = GetMSAASampleOffsets(NumMSAASamples, i).Y;
	}

	FCompositePostProcessPrimitivesPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FCompositePostProcessPrimitivesPS::FSampleCountDimension>(NumMSAASamples);
	PermutationVector.Set<FCompositePostProcessPrimitivesPS::FMSAADontResolve>(bOutputIsMSAA);
	PermutationVector.Set<FCompositePostProcessPrimitivesPS::FWriteDepth>(DepthOutput.IsValid());

	TShaderMapRef<FCompositePostProcessPrimitivesPS> PixelShader(View.ShaderMap, PermutationVector);
		
	FRHIDepthStencilState* DepthStencilState = nullptr;
	if (DepthOutput.IsValid())
	{
		DepthStencilState = TStaticDepthStencilState<true, CF_Always>::GetRHI();
	}

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME("Composite %dx%d MSAA=%d", Context.OutputRT.ViewRect.Width(), Context.OutputRT.ViewRect.Height(), NumMSAASamples),
		PixelShader,
		PassParameters,
		Context.OutputRT.ViewRect,
		nullptr,
		nullptr,
		DepthStencilState);
}

FScreenPassTexture AddDebugPrimitivePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	int32 ViewIndex,
	FSceneUniformBuffer& SceneUniformBuffer,
	FVirtualShadowMapArray* VirtualShadowMapArray,
	const FCompositePrimitiveInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneDepth.IsValid());

	RDG_EVENT_SCOPE(GraphBuilder, "CompositeDebugPrimitives");

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("Debug.DrawPrimitivesColor"));
	}
	const FScreenPassTextureViewport OutputViewport(Output);

	FDebugPrimitivesDrawingContext Context = InitializeDebugPrimitiveTextures(GraphBuilder, View, Inputs, Output);

	bool bHasDrawn = DrawDebugPDE(GraphBuilder, Context, OutputViewport);

	{
		const EDebugViewShaderMode DebugViewMode = View.Family->GetDebugViewShaderMode();

		FScene* Scene = (FScene*)View.Family->Scene;
		if (View.Family->EngineShowFlags.VisualizeInstanceOcclusionQueries && Scene && Scene->InstanceCullingOcclusionQueryRenderer)
		{
			Scene->InstanceCullingOcclusionQueryRenderer->RenderDebug(GraphBuilder, Scene->GPUScene, *Context.DebugView, OutputViewport.Rect, Context.DebugPrimitiveColor, Context.DebugPrimitiveDepth);
			bHasDrawn = true;
		}

		if (VirtualShadowMapArray && DebugViewMode == DVSM_ShadowCasters)
		{
#if VSM_ENABLE_VISUALIZATION
			VirtualShadowMapArray->RenderShadowCasterBounds(GraphBuilder, *Context.DebugView, ViewIndex, SceneUniformBuffer, OutputViewport.Rect, Context.DebugPrimitiveColor, Context.DebugPrimitiveDepth, Context.UpscaledSceneDepth);
			bHasDrawn = true;
#endif
		}
	}

	if (bHasDrawn)
	{
		ComposeDebugPrimitives(GraphBuilder, View, Inputs, Context);
	}
	else
	{
		AddDrawTexturePass(GraphBuilder, View, Inputs.SceneColor, Output);
	}

	return MoveTemp(Output);
}

#endif // UE_ENABLE_DEBUG_DRAWING

bool IsDebugPrimitivePassEnabled(const FViewInfo& View)
{
#if UE_ENABLE_DEBUG_DRAWING
	static bool bIsForceDisabled = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ForceDebugViewModes"))->GetValueOnAnyThread() == 2;
	if (bIsForceDisabled || !View.Family->EngineShowFlags.CompositeDebugPrimitives)
	{
		return false;
	}

	bool bHasDebugPDEPrimitives = View.DebugSimpleElementCollector.HasAnyPrimitives();
	bool bVisualizeInstanceOcclusionCulling = View.Family->EngineShowFlags.VisualizeInstanceOcclusionQueries;
	bool bVisualizeShadowCasters = View.Family->EngineShowFlags.VisualizeShadowCasters;

	return bHasDebugPDEPrimitives || bVisualizeInstanceOcclusionCulling || bVisualizeShadowCasters;
#else
	return false;
#endif
}
