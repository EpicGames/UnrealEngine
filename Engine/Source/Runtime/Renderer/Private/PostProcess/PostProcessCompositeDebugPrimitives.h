// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PostProcess/PostProcessCompositePrimitivesCommon.h"

class FVirtualShadowMapArray;

#if UE_ENABLE_DEBUG_DRAWING
FScreenPassTexture AddDebugPrimitivePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 ViewIndex, FSceneUniformBuffer& SceneUniformBuffer, FVirtualShadowMapArray* VirtualShadowMapArray, const FCompositePrimitiveInputs& Inputs);
#endif

bool IsDebugPrimitivePassEnabled(const FViewInfo& View);
