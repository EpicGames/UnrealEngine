// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessCompositeEditorPrimitives.h"

#if UE_ENABLE_DEBUG_DRAWING

IMPLEMENT_GLOBAL_SHADER(FCompositePostProcessPrimitivesPS, "/Engine/Private/PostProcessCompositePrimitives.usf", "MainCompositePostProcessPrimitivesPS", SF_Pixel);

#endif //UE_ENABLE_DEBUG_DRAWING

#if WITH_EDITOR
#include "EditorPrimitivesRendering.h"
#include "MeshPassProcessor.inl"
#include "BasePassRendering.h"
#include "MobileBasePassRendering.h"
#include "PixelShaderUtils.h"
#include "Substrate/Substrate.h"
#include "MeshEdgesRendering.h"
#include "PixelShaderUtils.h"
#include "PostProcess/PostProcessing.h" // IsPostProcessingWithAlphaChannelSupported

namespace
{
struct FEditorPrimitivesDrawingContext
{
	// View at UnscaledViewRect size, with TAA jitter disabled, exposure set for post-tonemapper, ...
	const FViewInfo* EditorView = nullptr;

	// EditorPrimitives* textures, at UnscaledViewRect size
	FRDGTextureRef EditorPrimitiveColor = nullptr;
	FRDGTextureRef EditorPrimitiveDepth = nullptr;

	bool bCompositeAnyNonNullDepth;
};

static void RenderEditorPrimitives(FRHICommandList& RHICmdList, const FViewInfo& View, FMeshPassProcessorRenderState& DrawRenderState, FInstanceCullingManager& InstanceCullingManager)
{
	// Always depth test against other editor primitives
	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<
		true, CF_DepthNearOrEqual,
		true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		0xFF, GET_STENCIL_BIT_MASK(RECEIVE_DECAL, 1) | STENCIL_LIGHTING_CHANNELS_MASK(0x7)>::GetRHI());

	DrawDynamicMeshPass(View, RHICmdList,
		[&View, &DrawRenderState](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
	{
		FEditorPrimitivesBasePassMeshProcessor PassMeshProcessor(
			View.Family->Scene->GetRenderScene(),
			View.GetFeatureLevel(),
			&View,
			DrawRenderState,
			false,
			DynamicMeshPassContext);

		const uint64 DefaultBatchElementMask = ~0ull;
		const int32 NumDynamicEditorMeshBatches = View.DynamicEditorMeshElements.Num();

		for (int32 MeshIndex = 0; MeshIndex < NumDynamicEditorMeshBatches; MeshIndex++)
		{
			const FMeshBatchAndRelevance& MeshAndRelevance = View.DynamicEditorMeshElements[MeshIndex];

			if (MeshAndRelevance.GetHasOpaqueOrMaskedMaterial() || View.Family->EngineShowFlags.Wireframe)
			{
				PassMeshProcessor.AddMeshBatch(*MeshAndRelevance.Mesh, DefaultBatchElementMask, MeshAndRelevance.PrimitiveSceneProxy);
			}
		}

		for (int32 MeshIndex = 0; MeshIndex < View.ViewMeshElements.Num(); MeshIndex++)
		{
			const FMeshBatch& MeshBatch = View.ViewMeshElements[MeshIndex];
			PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
		}
	});

	View.EditorSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_World);

	const auto FeatureLevel = View.GetFeatureLevel();
	const auto ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

	// Draw the view's batched simple elements(lines, sprites, etc).
	View.BatchedViewElements.Draw(RHICmdList, DrawRenderState, FeatureLevel, View, false, 1.0f);
}

static void RenderForegroundTranslucentEditorPrimitives(FRHICommandList& RHICmdList, const FViewInfo& View, FMeshPassProcessorRenderState& DrawRenderState, FInstanceCullingManager& InstanceCullingManager)
{
	const auto FeatureLevel = View.GetFeatureLevel();
	const auto ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

	// Force all translucent editor primitives to standard translucent rendering
	DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

	View.EditorSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::Translucent, SDPG_Foreground);

	DrawDynamicMeshPass(View, RHICmdList,
		[&View, &DrawRenderState](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
		{
			FEditorPrimitivesBasePassMeshProcessor PassMeshProcessor(
				View.Family->Scene->GetRenderScene(),
				View.GetFeatureLevel(),
				&View,
				DrawRenderState,
				true,
				DynamicMeshPassContext);

			const uint64 DefaultBatchElementMask = ~0ull;

			for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
			{
				const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
				PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
			}
		});
	
	View.TopBatchedViewElements.Draw(RHICmdList, DrawRenderState, FeatureLevel, View, false, 1.0f, EBlendModeFilter::Translucent);
}

static void RenderForegroundEditorPrimitives(FRHICommandList& RHICmdList, const FViewInfo& View, FMeshPassProcessorRenderState& DrawRenderState, FInstanceCullingManager& InstanceCullingManager)
{
	const auto FeatureLevel = View.GetFeatureLevel();
	const auto ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

	// Draw a first time the foreground primitive without depth test to over right depth from non-foreground editor primitives.
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_Always>::GetRHI());

		View.EditorSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_Foreground);

		DrawDynamicMeshPass(View, RHICmdList,
			[&View, &DrawRenderState](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
		{
			FEditorPrimitivesBasePassMeshProcessor PassMeshProcessor(
				View.Family->Scene->GetRenderScene(),
				View.GetFeatureLevel(),
				&View,
				DrawRenderState,
				false,
				DynamicMeshPassContext);

			const uint64 DefaultBatchElementMask = ~0ull;

			for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
			{
				const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
				PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
			}
		});

		View.TopBatchedViewElements.Draw(RHICmdList, DrawRenderState, FeatureLevel, View, false);
	}

	// Draw a second time the foreground primitive with depth test to have proper depth test between foreground primitives.
	{
		DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Keep, SO_Replace, true, CF_Always, SO_Keep, SO_Keep, SO_Replace, 0, 0xFF>::GetRHI());

		DrawRenderState.SetStencilRef(FCompositePostProcessPrimitivesPS::EditorStencilFlagForegroundNoDithering);
		View.EditorSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_Foreground);

		DrawRenderState.SetStencilRef(0);
		DrawDynamicMeshPass(View, RHICmdList,
			[&View, &DrawRenderState](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
		{
			FEditorPrimitivesBasePassMeshProcessor PassMeshProcessor(
				View.Family->Scene->GetRenderScene(),
				View.GetFeatureLevel(),
				&View,
				DrawRenderState,
				false,
				DynamicMeshPassContext);

			const uint64 DefaultBatchElementMask = ~0ull;

			for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
			{
				const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
				PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
			}
		});

		View.TopBatchedViewElements.Draw(RHICmdList, DrawRenderState, FeatureLevel, View, false);
	}
}

} //! namespace

BEGIN_SHADER_PARAMETER_STRUCT(FEditorPrimitivesPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_STRUCT_REF(FReflectionCaptureShaderData, ReflectionCapture)
	SHADER_PARAMETER_STRUCT_REF(FMobileReflectionCaptureShaderData, MobileReflectionCaptureData)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FOpaqueBasePassUniformParameters, BasePass)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FTranslucentBasePassUniformParameters, TranslucentBasePass)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileBasePassUniformParameters, MobileBasePass)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FInstanceCullingGlobalUniforms, InstanceCulling)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static FEditorPrimitivesDrawingContext InitializeEditorPrimitivesTextures(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTexture& SceneColor,
	const FScreenPassTexture& SceneDepth)
{
	check(SceneColor.IsValid());
	check(SceneDepth.IsValid());

	const FSceneTextures& SceneTextures = View.GetSceneTextures();
	const uint32 NumMSAASamples = SceneTextures.Config.EditorPrimitiveNumSamples;
	const FViewInfo* EditorView = CreateCompositePrimitiveView(View, SceneColor.ViewRect, NumMSAASamples);	

	// In wireframe mode, the base pass has already rendered to EditorPrimitiveColor/Depth. We check the wireframe flag rather
	// than using HasBeenProduced (as originally done here), because in split-screen debug mode, HasBeenProduced will be true
	// for the second viewport, causing this initialization to be incorrectly skipped, compositing garbage onto the screen.
	bool bProducedByPriorPass = View.Family->EngineShowFlags.Wireframe;
	FIntPoint Extent = SceneColor.Texture->Desc.Extent;
	FRDGTextureRef EditorPrimitiveColor = SceneTextures.EditorPrimitiveColor;
	FRDGTextureRef EditorPrimitiveDepth = SceneTextures.EditorPrimitiveDepth;

	bool bViewIsScaled = View.ViewRect.Size() != EditorView->ViewRect.Size();
	bool bSceneDepthIsScaled = SceneColor.ViewRect != SceneDepth.ViewRect;

	if (bProducedByPriorPass)
	{
		// Upscaling EditorPrimitiveColor is not supported.
		ensure(!bViewIsScaled);
	}

	// If the view is scaled, use fresh textures with the correct size for the editor viewport
	if (bViewIsScaled || bSceneDepthIsScaled)
	{
		FRDGTextureDesc EditorPrimitiveColorDesc = EditorPrimitiveColor->Desc;
		FRDGTextureDesc EditorPrimitiveDepthDesc = EditorPrimitiveDepth->Desc;

		EditorPrimitiveColorDesc.Extent = Extent;
		EditorPrimitiveDepthDesc.Extent = Extent;
		EditorPrimitiveDepthDesc.AliasableFormats.Add(PF_X24_G8); // could this go into SceneTextures?

		if (bViewIsScaled)
		{
			EditorPrimitiveColor = GraphBuilder.CreateTexture(EditorPrimitiveColorDesc, TEXT("Editor.PrimitivesColor"));
		}
		EditorPrimitiveDepth = GraphBuilder.CreateTexture(EditorPrimitiveDepthDesc, TEXT("Editor.PrimitivesDepth"));

		bProducedByPriorPass = false;
	}

	const FScreenPassTextureViewport EditorPrimitivesViewport(EditorPrimitiveColor, SceneColor.ViewRect);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, EditorPrimitives, "InitializeEditorPrimitivesTextures %dx%d MSAA=%d",
						 EditorPrimitivesViewport.Rect.Width(),
						 EditorPrimitivesViewport.Rect.Height(),
						 NumMSAASamples);

	// The editor primitive composition pass is also used when rendering VMI_WIREFRAME in order to use MSAA.
	// So we need to check whether the editor primitives are enabled inside this function.
	// Similarly, MeshEdges is also wireframe, but as a separate pre-pass.
	if (View.Family->EngineShowFlags.CompositeEditorPrimitives || View.Family->EngineShowFlags.MeshEdges)
	{
		FVector2f SceneDepthJitter = FVector2f(View.TemporalJitterPixels);

		// Populate depth if a prior pass did not already do it.
		if (!bProducedByPriorPass)
		{
			FScreenPassTexture UpsampledDepth = SceneDepth;

			if (IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod))
			{
				TemporalUpscaleDepthPass(GraphBuilder,
										 *EditorView,
										 SceneColor,
										 UpsampledDepth,
										 SceneDepthJitter);
			}

			PopulateDepthPass(GraphBuilder,
							  *EditorView,
							  SceneColor,
							  UpsampledDepth,
							  EditorPrimitiveColor,
							  EditorPrimitiveDepth,
							  EditorView->ViewRect,
							  SceneDepthJitter,
							  NumMSAASamples);
		}	
		
		FScreenPassRenderTarget EditorPrimitiveColorRT(EditorPrimitiveColor, EditorPrimitivesViewport.Rect, ERenderTargetLoadAction::ELoad);
		FScreenPassRenderTarget EditorPrimitiveDepthRT(EditorPrimitiveDepth, EditorPrimitivesViewport.Rect, ERenderTargetLoadAction::ELoad);
		ComposeMeshEdges(GraphBuilder, View, EditorPrimitiveColorRT, EditorPrimitiveDepthRT);
	}

	FEditorPrimitivesDrawingContext Result {};
	Result.EditorView = EditorView;
	Result.bCompositeAnyNonNullDepth = bProducedByPriorPass && !View.Family->EngineShowFlags.MeshEdges;
	Result.EditorPrimitiveColor = EditorPrimitiveColor;
	Result.EditorPrimitiveDepth = EditorPrimitiveDepth;

	return Result;
}

FScreenPassTexture AddEditorPrimitivePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FCompositePrimitiveInputs& Inputs,
	FInstanceCullingManager& InstanceCullingManager)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneDepth.IsValid());
	check(Inputs.BasePassType != FCompositePrimitiveInputs::EBasePassType::MAX);
	
	FEditorPrimitivesDrawingContext Context = InitializeEditorPrimitivesTextures(GraphBuilder, View, Inputs.SceneColor, Inputs.SceneDepth);

	FRDGTextureRef EditorPrimitiveColor = Context.EditorPrimitiveColor;
	FRDGTextureRef EditorPrimitiveDepth = Context.EditorPrimitiveDepth;

	const FViewInfo* EditorView = Context.EditorView;
	const uint32 NumMSAASamples = EditorPrimitiveColor->Desc.NumSamples;
	
	// Substrate data might not be produced in certain case (e.g., path-tracer). In such a case we force generate 
	// them with a simple clear to please validation.
	if (Substrate::IsSubstrateEnabled() && Substrate::UsesSubstrateMaterialBuffer(View.GetShaderPlatform()) && View.SubstrateViewData.SceneData->TopLayerTexture && !HasBeenProduced(View.SubstrateViewData.SceneData->TopLayerTexture))
	{
		FRDGTextureClearInfo ClearInfo;
		AddClearRenderTargetPass(GraphBuilder, View.SubstrateViewData.SceneData->TopLayerTexture, ClearInfo);
	}

	const FScreenPassTextureViewport EditorPrimitivesViewport(EditorPrimitiveColor, Inputs.SceneColor.ViewRect);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, EditorPrimitives, "CompositeEditorPrimitives %dx%d MSAA=%d",
		EditorPrimitivesViewport.Rect.Width(),
		EditorPrimitivesViewport.Rect.Height(),
		NumMSAASamples);

	//Inputs is const so create a over-ridable texture reference
	FScreenPassTexture SceneDepth = Inputs.SceneDepth;
	FVector2f SceneDepthJitter = FVector2f(View.TemporalJitterPixels);

	if (View.Family->EngineShowFlags.CompositeEditorPrimitives)
	{
		// Draws the editors opaque primitives
		{
			FEditorPrimitivesPassParameters* PassParameters = GraphBuilder.AllocParameters<FEditorPrimitivesPassParameters>();
			PassParameters->View = EditorView->GetShaderParameters();
			PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
			PassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;
			PassParameters->MobileReflectionCaptureData = View.MobileReflectionCaptureUniformBuffer;
			PassParameters->InstanceCulling = InstanceCullingManager.GetDummyInstanceCullingUniformBuffer();
			PassParameters->RenderTargets[0] = FRenderTargetBinding(EditorPrimitiveColor, ERenderTargetLoadAction::ELoad);
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(EditorPrimitiveDepth, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);

			const FCompositePrimitiveInputs::EBasePassType BasePassType = Inputs.BasePassType;

			if (BasePassType == FCompositePrimitiveInputs::EBasePassType::Deferred)
			{
				PassParameters->BasePass = CreateOpaqueBasePassUniformBuffer(GraphBuilder, *EditorView, 0);
			}
			else
			{
				PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, *EditorView, EMobileBasePass::Translucent, EMobileSceneTextureSetupMode::None);
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("EditorPrimitives %dx%d MSAA=%d",
					EditorPrimitivesViewport.Rect.Width(),
					EditorPrimitivesViewport.Rect.Height(),
					NumMSAASamples),
				PassParameters,
				ERDGPassFlags::Raster,
				[&View, &InstanceCullingManager, PassParameters, EditorView, EditorPrimitivesViewport, BasePassType, NumMSAASamples](FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(EditorPrimitivesViewport.Rect.Min.X, EditorPrimitivesViewport.Rect.Min.Y, 0.0f, EditorPrimitivesViewport.Rect.Max.X, EditorPrimitivesViewport.Rect.Max.Y, 1.0f);

				FMeshPassProcessorRenderState DrawRenderState;
				DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilWrite);
				DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA>::GetRHI());

				// Draw editor primitives.
				{
					SCOPED_DRAW_EVENTF(RHICmdList, EditorPrimitives,
						TEXT("RenderViewEditorPrimitives %dx%d msaa=%d"),
						EditorPrimitivesViewport.Rect.Width(), EditorPrimitivesViewport.Rect.Height(), NumMSAASamples);

					RenderEditorPrimitives(RHICmdList, *EditorView, DrawRenderState, InstanceCullingManager);
				}

				// Draw foreground editor primitives.
				{
					SCOPED_DRAW_EVENTF(RHICmdList, EditorPrimitives,
						TEXT("RenderViewEditorForegroundPrimitives %dx%d msaa=%d"),
						EditorPrimitivesViewport.Rect.Width(), EditorPrimitivesViewport.Rect.Height(), NumMSAASamples);

					RenderForegroundEditorPrimitives(RHICmdList, *EditorView, DrawRenderState, InstanceCullingManager);
				}
			});
		}
	}

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	FScreenPassRenderTarget DepthOutput = Inputs.OverrideDepthOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("Editor.Primitives"));
	}

	{
		FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		const bool bOpaqueEditorGizmo = View.Family->EngineShowFlags.OpaqueCompositeEditorPrimitives || View.Family->EngineShowFlags.Wireframe;

		FCompositePostProcessPrimitivesPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FCompositePostProcessPrimitivesPS::FParameters>();
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		bool bOutputIsMSAA = Output.Texture->Desc.NumSamples > 1;
		if (DepthOutput.IsValid())
		{
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthOutput.Texture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite);
			verify(Output.Texture->Desc.NumSamples == DepthOutput.Texture->Desc.NumSamples);
		}

		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->Color = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Inputs.SceneColor));
		PassParameters->Depth = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(SceneDepth));
		PassParameters->EditorPrimitives = GetScreenPassTextureViewportParameters(EditorPrimitivesViewport);

		PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		if (Inputs.LensDistortionLUT.IsEnabled())
		{
			PassParameters->UndistortingDisplacementTexture = Inputs.LensDistortionLUT.UndistortingDisplacementTexture;
		}

		PassParameters->ColorTexture = Inputs.SceneColor.Texture;
		PassParameters->ColorSampler = PointClampSampler;
		if (View.Family->EngineShowFlags.SceneCaptureCopySceneDepth)
		{
			PassParameters->DepthTexture = SceneDepth.Texture;
		}
		else
		{
			PassParameters->DepthTexture = GSystemTextures.GetDepthDummy(GraphBuilder);
		}
		PassParameters->DepthSampler = PointClampSampler;
		PassParameters->EditorPrimitivesDepth = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(EditorPrimitiveDepth, PF_DepthStencil));
		PassParameters->EditorPrimitivesStencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(EditorPrimitiveDepth, PF_X24_G8));
		PassParameters->EditorPrimitivesColor = EditorPrimitiveColor;
		
		PassParameters->PassSvPositionToViewportUV = FScreenTransform::SvPositionToViewportUV(Output.ViewRect);
		PassParameters->ViewportUVToColorUV = FScreenTransform::ChangeTextureBasisFromTo(
			FScreenPassTextureViewport(Inputs.SceneColor), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
		PassParameters->ViewportUVToDepthUV = FScreenTransform::ChangeTextureBasisFromTo(
			FScreenPassTextureViewport(SceneDepth), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
		PassParameters->ViewportUVToEditorPrimitivesUV = FScreenTransform::ChangeTextureBasisFromTo(
			EditorPrimitivesViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);

		PassParameters->bOpaqueEditorGizmo = bOpaqueEditorGizmo;
		PassParameters->bCompositeAnyNonNullDepth = Context.bCompositeAnyNonNullDepth;
		PassParameters->DepthTextureJitter = SceneDepthJitter;
		PassParameters->bProcessAlpha = IsPostProcessingWithAlphaChannelSupported();
		PassParameters->OccludedDitheringPatternSize = Inputs.OccludedDitheringPatternSize;
		PassParameters->OccludedDithering = Inputs.OccludedDithering;
		PassParameters->OccludedBrightness = Inputs.OccludedBrightness;

		for (int32 i = 0; i < int32(NumMSAASamples); i++)
		{
			PassParameters->SampleOffsetArray[i].X = GetMSAASampleOffsets(NumMSAASamples, i).X;
			PassParameters->SampleOffsetArray[i].Y = GetMSAASampleOffsets(NumMSAASamples, i).Y;
		}

		FCompositePostProcessPrimitivesPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FCompositePostProcessPrimitivesPS::FSampleCountDimension>(NumMSAASamples);
		PermutationVector.Set<FCompositePostProcessPrimitivesPS::FMSAADontResolve>(bOutputIsMSAA);
		PermutationVector.Set<FCompositePostProcessPrimitivesPS::FWriteDepth>(DepthOutput.IsValid());

		TShaderMapRef<FCompositePostProcessPrimitivesPS> PixelShader(View.ShaderMap, PermutationVector);
		
		FRHIDepthStencilState* DepthStencilState = nullptr;
		if (DepthOutput.IsValid())
		{
			DepthStencilState = TStaticDepthStencilState<true, CF_Always>::GetRHI();
		}

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("Composite %dx%d MSAA=%d", Output.ViewRect.Width(), Output.ViewRect.Height(), NumMSAASamples),
			PixelShader,
			PassParameters,
			Output.ViewRect,
			nullptr,
			nullptr,
			DepthStencilState);
	}

	// Draws the editor translucent primitives on top of the opaque scene primitives
	if (View.Family->EngineShowFlags.CompositeEditorPrimitives && View.bHasTranslucentViewMeshElements)
	{
		// SetTranslucentRenderState can pick a stencil-write DSS (responsive-AA / distortion mask bits)
		// for translucent materials. Output is color-only. Bind a transient depth-stencil sink to absorb those writes.
		FRDGTextureDesc DepthStencilSinkDesc = FRDGTextureDesc::Create2D(
			Output.Texture->Desc.Extent,
			PF_DepthStencil,
			FClearValueBinding::DepthFar,
			TexCreate_DepthStencilTargetable);
		DepthStencilSinkDesc.NumSamples = Output.Texture->Desc.NumSamples;
		FRDGTextureRef DepthStencilSink = GraphBuilder.CreateTexture(
			DepthStencilSinkDesc, TEXT("Editor.TranslucentPrimitivesDSSink"));

		FEditorPrimitivesPassParameters* PassParameters = GraphBuilder.AllocParameters<FEditorPrimitivesPassParameters>();
		PassParameters->View = EditorView->GetShaderParameters();
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
		PassParameters->ReflectionCapture = View.ReflectionCaptureUniformBuffer;
		PassParameters->MobileReflectionCaptureData = View.MobileReflectionCaptureUniformBuffer;
		PassParameters->InstanceCulling = InstanceCullingManager.GetDummyInstanceCullingUniformBuffer();
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		PassParameters->RenderTargets[0].SetLoadAction(ERenderTargetLoadAction::ELoad);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
			DepthStencilSink,
			ERenderTargetLoadAction::ENoAction,
			ERenderTargetLoadAction::EClear,
			FExclusiveDepthStencil::DepthNop_StencilWrite);

		const FCompositePrimitiveInputs::EBasePassType BasePassType = Inputs.BasePassType;

		if (BasePassType == FCompositePrimitiveInputs::EBasePassType::Deferred)
		{
			PassParameters->TranslucentBasePass = CreateTranslucentBasePassUniformBuffer(GraphBuilder, nullptr, *EditorView, 0);
		}
		else
		{
			PassParameters->MobileBasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, *EditorView, EMobileBasePass::Translucent, EMobileSceneTextureSetupMode::None);
		}

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("EditorPrimitives Translucent %dx%d MSAA=%d",
				EditorPrimitivesViewport.Rect.Width(),
				EditorPrimitivesViewport.Rect.Height(),
				NumMSAASamples),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, &InstanceCullingManager, PassParameters, EditorView, EditorPrimitivesViewport, OutputViewportRect = Output.ViewRect, BasePassType, NumMSAASamples](FRHICommandList& RHICmdList)
			{
				RHICmdList.SetViewport(OutputViewportRect.Min.X, OutputViewportRect.Min.Y, 0.0f, OutputViewportRect.Max.X, OutputViewportRect.Max.Y, 1.0f);

				FMeshPassProcessorRenderState DrawRenderState;
				DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthNop_StencilWrite);
				DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA>::GetRHI());

				// Draw foreground editor primitives.
				{
					SCOPED_DRAW_EVENTF(RHICmdList, EditorPrimitives,
						TEXT("RenderViewEditorForegroundTranslucentPrimitives %dx%d msaa=%d"),
						EditorPrimitivesViewport.Rect.Width(), EditorPrimitivesViewport.Rect.Height(), NumMSAASamples);

					RenderForegroundTranslucentEditorPrimitives(RHICmdList, *EditorView, DrawRenderState, InstanceCullingManager);
				}
			});
	}

	return MoveTemp(Output);
}

#endif
