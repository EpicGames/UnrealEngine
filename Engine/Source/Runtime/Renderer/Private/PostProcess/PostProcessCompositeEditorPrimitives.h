// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PostProcess/PostProcessCompositePrimitivesCommon.h"

#if WITH_EDITOR
FScreenPassTexture AddEditorPrimitivePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FCompositePrimitiveInputs& Inputs, FInstanceCullingManager& InstanceCullingManager);
#endif
