// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessCompositePrimitivesCommon.h"

#if UE_ENABLE_DEBUG_DRAWING 
#include "BasePassRendering.h"
#include "PixelShaderUtils.h"
#include "SceneViewState.h"

#define MIN_DEPTHTEX_UPSCALE_FACTOR 1
#define MAX_DEPTHTEX_UPSCALE_FACTOR 4
TAutoConsoleVariable<int32> CVarCompositeTemporalUpsampleDepth(
	TEXT("r.Composite.TemporalUpsampleDepth"), 2,
	TEXT("Temporal upsample factor of the depth buffer for depth testing editor primitives against."),
	ECVF_RenderThreadSafe);

const FViewInfo* CreateCompositePrimitiveView(const FViewInfo& ParentView, FIntRect ViewRect, uint32 NumMSAASamples)
{
	FViewInfo* View = ParentView.CreateSnapshot();

	// Patch view rect.
	View->ViewRect = ViewRect;

	// Override pre exposure to 1.0f, because rendering after tonemapper. 
	View->PreExposure = 1.0f;

	// Kills material texture mipbias because after TAA.
	View->MaterialTextureMipBias = 0.0f;

	if (IsTemporalAccumulationBasedMethod(View->AntiAliasingMethod))
	{
		View->ViewMatrices.HackRemoveTemporalAAProjectionJitter();
	}

	View->InitRHIResources(NumMSAASamples);

	return View;
}

class FTemporalUpsampleDepthPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTemporalUpsampleDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FTemporalUpsampleDepthPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Depth)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, PrevHistory)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, History)
		SHADER_PARAMETER(FVector2f, DepthTextureJitter)
		SHADER_PARAMETER(int32, bCameraCut)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VelocityTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, VelocitySampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevHistoryTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, PrevHistorySampler)

		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		/** 
		* Compile for all platforms that support TAA minimum.
		* Runtime only executes if a temporal upscale method is enabled.
		*/
		return SupportsGen4TAA(Parameters.Platform);
	}

	static const EPixelFormat OutputFormat = PF_R32_FLOAT;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		/*Explicitly match PS render target output format to the desc of the texture.*/
		OutEnvironment.SetRenderTargetOutputFormat(0, OutputFormat);
	}
};

class FPopulateCompositeDepthPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FPopulateCompositeDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FPopulateCompositeDepthPS, FGlobalShader);

	class FUseMSAADimension : SHADER_PERMUTATION_BOOL("USE_MSAA");
	class FForceDebugDrawColorOutput : SHADER_PERMUTATION_BOOL("FORCE_DEBUG_DRAW_COLOR"); //Allow the option to draw out the scene color too to lower draw calls
	class FUseMetalMSAAHDRDecodeDim : SHADER_PERMUTATION_BOOL("METAL_MSAA_HDR_DECODE");
	using FPermutationDomain = TShaderPermutationDomain<FUseMSAADimension, FForceDebugDrawColorOutput, FUseMetalMSAAHDRDecodeDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Depth)
		SHADER_PARAMETER(FVector4f, OutputViewRectMinAndInvSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)
		SHADER_PARAMETER(FVector2f, DepthTextureJitter)
		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		FPermutationDomain PermutationVector(InParameters.PermutationId);
		if (PermutationVector.Get<FForceDebugDrawColorOutput>() != 0)
		{
			OutEnvironment.SetDefine(TEXT("MIN_DEPTHTEX_UPSCALE_FACTOR"), MIN_DEPTHTEX_UPSCALE_FACTOR);
			OutEnvironment.SetDefine(TEXT("MAX_DEPTHTEX_UPSCALE_FACTOR"), MAX_DEPTHTEX_UPSCALE_FACTOR);
		}
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const bool bUseMSAA = PermutationVector.Get<FUseMSAADimension>();
		const bool bUseMetalMSAAHDRDecodeDim = PermutationVector.Get<FUseMetalMSAAHDRDecodeDim>();

		if (bUseMSAA)
		{
			// Only SM5+ and Metal ES3.1 platforms support MSAA.
			if (IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
			{
				return true;
			}
			else if (IsMetalMobilePlatform(Parameters.Platform) && bUseMetalMSAAHDRDecodeDim)
			{
				//Only compile with decode dim if using metal platform on 3.1 + color output
				return PermutationVector.Get<FForceDebugDrawColorOutput>();
			}
		}
		else
		{
			//Never compile Metal decode dim permutations for non-MSAA passes
			return !bUseMetalMSAAHDRDecodeDim;
		}

		return false;
	}
};

IMPLEMENT_GLOBAL_SHADER(FTemporalUpsampleDepthPS, "/Engine/Private/PostProcessCompositePrimitives.usf", "MainTemporalUpsampleDepthPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FPopulateCompositeDepthPS, "/Engine/Private/PostProcessCompositePrimitives.usf", "MainPopulateSceneDepthPS", SF_Pixel);

void TemporalUpscaleDepthPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTexture& InSceneColor,
	FScreenPassTexture& InOutSceneDepth,
	FVector2f& SceneDepthJitter)
{
	if (InSceneColor.ViewRect != InOutSceneDepth.ViewRect)
	{
		FIntPoint Extent = InSceneColor.Texture->Desc.Extent;
		FIntRect ViewRect = InSceneColor.ViewRect;

		// Upscale factor of the depth buffer that might be needed.
		const float UpscaleFactor = float(View.ViewRect.Width()) / float(InOutSceneDepth.ViewRect.Width());

		// Upscale factor shouldn't be higher than there is TAA samples, or that means there will be unrendered pixels.
		const int32 ComputeMaxUpsampleFactorDueToTAA = FMath::FloorToInt(FMath::Sqrt(float(View.TemporalJitterSequenceLength) / (UpscaleFactor * UpscaleFactor)));

		const int32 MaxRHIUpsampleFactor = FMath::Clamp(FMath::FloorToInt(float(GetMax2DTextureDimension()) / float(Extent.GetMax())), MIN_DEPTHTEX_UPSCALE_FACTOR, MAX_DEPTHTEX_UPSCALE_FACTOR);

		const int32 ComputeMaxUpsampleFactor = FMath::Clamp(ComputeMaxUpsampleFactorDueToTAA, 0, MaxRHIUpsampleFactor);

		const int32 DepthUpsampleFactor = FMath::Clamp(CVarCompositeTemporalUpsampleDepth.GetValueOnRenderThread(), 0, ComputeMaxUpsampleFactor);

		// Upsample the depth at higher resolution to reduce depth intersection instability of editor primitives.
		if (DepthUpsampleFactor > 0 && View.ViewState && IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod))
		{
			FScreenPassTexture History;
			const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				Extent * DepthUpsampleFactor,
				FTemporalUpsampleDepthPS::OutputFormat /* PF_R32_FLOAT */,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_RenderTargetable);

			History.Texture = GraphBuilder.CreateTexture(Desc, TEXT("Composite.PrimitivesDepthHistory"));
			History.ViewRect = ViewRect * DepthUpsampleFactor;


			FTemporalUpsampleDepthPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FTemporalUpsampleDepthPS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->Depth = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(InOutSceneDepth));
			PassParameters->History = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(History));
			PassParameters->DepthTextureJitter = SceneDepthJitter;

			PassParameters->DepthTexture = InOutSceneDepth.Texture;
			PassParameters->DepthSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			// TODO
			PassParameters->VelocityTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
			PassParameters->VelocitySampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			if (View.PrevViewInfo.CompositePrimitiveDepthHistory.IsValid())
			{
				PassParameters->PrevHistory = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(
					View.PrevViewInfo.CompositePrimitiveDepthHistory.RT[0]->GetDesc().Extent, View.PrevViewInfo.CompositePrimitiveDepthHistory.ViewportRect));
				PassParameters->PrevHistoryTexture = GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.CompositePrimitiveDepthHistory.RT[0]);
				PassParameters->PrevHistorySampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->bCameraCut = false;
			}
			else
			{
				PassParameters->PrevHistory = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(
					FIntPoint(1, 1), FIntRect(FIntPoint(0, 0), FIntPoint(1, 1))));
				PassParameters->PrevHistoryTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
				PassParameters->PrevHistorySampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->bCameraCut = true;
			}

			PassParameters->RenderTargets[0] = FRenderTargetBinding(History.Texture, ERenderTargetLoadAction::ENoAction);

			TShaderMapRef<FTemporalUpsampleDepthPS> PixelShader(View.ShaderMap);
			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				View.ShaderMap,
				RDG_EVENT_NAME("TemporalUpsampleDepth %dx%d -> %dx%d",
					InOutSceneDepth.ViewRect.Width(),
					InOutSceneDepth.ViewRect.Height(),
					History.ViewRect.Width(),
					History.ViewRect.Height()),
				PixelShader,
				PassParameters,
				History.ViewRect);

			if (!View.bStatePrevViewInfoIsReadOnly)
			{
				FTemporalAAHistory* OutputHistory = &View.ViewState->PrevFrameViewInfo.CompositePrimitiveDepthHistory;
				OutputHistory->SafeRelease();

				GraphBuilder.QueueTextureExtraction(History.Texture, &OutputHistory->RT[0]);
				OutputHistory->ViewportRect = History.ViewRect;
				OutputHistory->ReferenceBufferSize = History.Texture->Desc.Extent;
			}

			InOutSceneDepth = History;
			SceneDepthJitter = FVector2f::ZeroVector;
		}
	}
}

// Populate the MSAA depth buffer from depth buffer or temporally upscaled depth buffer
void PopulateDepthPass(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTexture& InSceneColor,
	const FScreenPassTexture& InSceneDepth,
	FRDGTextureRef OutPopColor,
	FRDGTextureRef OutPopDepth,
	const FIntRect& OutputViewRect,
	const FVector2f& SceneDepthJitter,
	uint32 NumMSAASamples,
	bool bForceDrawColor,
	bool bUseMetalPlatformHDRDecode)
{
	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FPopulateCompositeDepthPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FPopulateCompositeDepthPS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;

	if (bForceDrawColor)
	{
		PassParameters->ColorTexture = InSceneColor.Texture;
		PassParameters->ColorSampler = PointClampSampler;
	}

	PassParameters->Color = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(InSceneColor));
	PassParameters->Depth = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(InSceneDepth));

	FIntPoint ViewRectSize = OutputViewRect.Size();
	PassParameters->OutputViewRectMinAndInvSize = FVector4f(OutputViewRect.Min.X, OutputViewRect.Min.Y, 1.0f / ViewRectSize.X, 1.0f / ViewRectSize.Y);

	PassParameters->DepthTexture = InSceneDepth.Texture;
	PassParameters->DepthSampler = PointClampSampler;
	PassParameters->DepthTextureJitter = SceneDepthJitter;
	if (OutPopColor)
	{
		const ERenderTargetLoadAction ColorLoadAction = View.IsPrimarySceneView() ? ERenderTargetLoadAction::ENoAction : ERenderTargetLoadAction::ELoad;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutPopColor, ColorLoadAction);
	}
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(OutPopDepth, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);

	FPopulateCompositeDepthPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FPopulateCompositeDepthPS::FUseMSAADimension>(NumMSAASamples > 1);
	PermutationVector.Set< FPopulateCompositeDepthPS::FForceDebugDrawColorOutput>(bForceDrawColor);
	PermutationVector.Set< FPopulateCompositeDepthPS::FUseMetalMSAAHDRDecodeDim>(NumMSAASamples > 1 && bForceDrawColor && bUseMetalPlatformHDRDecode);
	TShaderMapRef<FPopulateCompositeDepthPS> PixelShader(View.ShaderMap, PermutationVector);

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		View.ShaderMap,
		RDG_EVENT_NAME("PopulateDepth %dx%d%s",
			OutputViewRect.Width(),
			OutputViewRect.Height(),
			NumMSAASamples > 1 ? TEXT(" MSAA") : TEXT("")),
		PixelShader,
		PassParameters,
		OutputViewRect,
		/* BlendState = */ nullptr,
		/* RasterizerState = */ nullptr,
		TStaticDepthStencilState<true, CF_Always>::GetRHI());
}

#endif //UE_ENABLE_DEBUG_DRAWING
