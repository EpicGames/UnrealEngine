// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "OverridePassSequence.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PostProcess/LensDistortion.h"
#include "SceneView.h"

//UE_ENABLE_DEBUG_DRAWING i.e. !(UE_BUILD_SHIPPING || UE_BUILD_TEST) || WITH_EDITOR
//Only available in Debug/Development/Editor builds due to current use cases, but can be extended in future
#if UE_ENABLE_DEBUG_DRAWING

/** Base class for a global pixel shader which renders primitives (outlines, helpers, etc). */
class FCompositePrimitiveShaderBase : public FGlobalShader
{
public:
	static const uint32 kMSAASampleCountMax = 8;

	class FSampleCountDimension : SHADER_PERMUTATION_RANGE_INT("MSAA_SAMPLE_COUNT", 1, kMSAASampleCountMax + 1);
	class FMSAADontResolve : SHADER_PERMUTATION_BOOL("MSAA_DONT_RESOLVE");
	using FPermutationDomain = TShaderPermutationDomain<FSampleCountDimension, FMSAADontResolve>;

	static bool ShouldCompilePermutation(const FPermutationDomain& PermutationVector, const EShaderPlatform Platform)
	{
		const int32 SampleCount = PermutationVector.Get<FSampleCountDimension>();
		const bool bMSAADontResolve = PermutationVector.Get<FMSAADontResolve>();

		// Only use permutations with valid MSAA sample counts.
		if (!FMath::IsPowerOfTwo(SampleCount))
		{
			return false;
		}
		if (!RHISupportsMSAA(Platform) && (SampleCount > 1 || bMSAADontResolve))
		{
			return false;
		}

		return IsPCPlatform(Platform);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		return ShouldCompilePermutation(PermutationVector, Parameters.Platform);
	}

	FCompositePrimitiveShaderBase() = default;
	FCompositePrimitiveShaderBase(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

struct FCompositePrimitiveInputs
{
	enum class EBasePassType : uint32
	{
		Deferred,
		Mobile,
		MAX
	};
	// [Required] The type of base pass to use for rendering editor primitives.
	EBasePassType BasePassType = EBasePassType::MAX;

	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Optional] Render the depth to the specified output.
	FScreenPassRenderTarget OverrideDepthOutput;

	// [Required] The scene color to composite with editor primitives.
	FScreenPassTexture SceneColor;

	// [Required] The scene depth to composite with editor primitives.
	FScreenPassTexture SceneDepth;

	// [Optional] Lens distortion applied on the scene color.
	FLensDistortionLUT LensDistortionLUT;

	// [Optional] Size (in pixels) of the dithering pattern applied to occluded primitives.
	float OccludedDitheringPatternSize = 2.0f;

	// [Optional] The strength of the dithering applied to occluded primitives.
	float OccludedDithering = 0.8f;

	// [Optional] The brightness of occluded primitives.
	float OccludedBrightness = 0.7f;

	bool bUseMetalMSAAHDRDecode = false;
};

// Constructs a new view suitable for rendering debug primitives.
const FViewInfo* CreateCompositePrimitiveView(const FViewInfo& ParentView, FIntRect ViewRect, uint32 NumMSAASamples);

void TemporalUpscaleDepthPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTexture& InSceneColor,
	FScreenPassTexture& InOutSceneDepth,
	FVector2f& SceneDepthJitter);

void PopulateDepthPass(FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTexture& InSceneColor, 
	const FScreenPassTexture& InSceneDepth, 
	FRDGTextureRef OutPopColor, 
	FRDGTextureRef OutPopDepth, 
	const FIntRect& OutputViewRect,
	const FVector2f& SceneDepthJitter,
	uint32 NumMSAASamples,
	bool bForceDrawColor = false,
	bool bUseMetalPlatformHDRDecode = false);


class FCompositePostProcessPrimitivesPS : public FCompositePrimitiveShaderBase
{
public:
	class FWriteDepth : SHADER_PERMUTATION_BOOL("WRITE_DEPTH");
					
	using FPermutationDomain = TShaderPermutationDomain<FWriteDepth, FSampleCountDimension, FMSAADontResolve>;
			
	DECLARE_GLOBAL_SHADER(FCompositePostProcessPrimitivesPS);
	SHADER_USE_PARAMETER_STRUCT(FCompositePostProcessPrimitivesPS, FCompositePrimitiveShaderBase);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Depth)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, EditorPrimitives)
		SHADER_PARAMETER_ARRAY(FVector4f, SampleOffsetArray, [FCompositePrimitiveShaderBase::kMSAASampleCountMax])

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  UndistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, EditorPrimitivesDepth)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint2>, EditorPrimitivesStencil)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, EditorPrimitivesColor)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  ColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  DepthSampler)

		SHADER_PARAMETER(FScreenTransform, PassSvPositionToViewportUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToColorUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToDepthUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToEditorPrimitivesUV)
		SHADER_PARAMETER(uint32, bOpaqueEditorGizmo)
		SHADER_PARAMETER(uint32, bCompositeAnyNonNullDepth)
		SHADER_PARAMETER(FVector2f, DepthTextureJitter)
		SHADER_PARAMETER(uint32, bProcessAlpha)
		SHADER_PARAMETER(float, OccludedDitheringPatternSize)
		SHADER_PARAMETER(float, OccludedDithering)
		SHADER_PARAMETER(float, OccludedBrightness)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	// SDPG_World means "with occlusion", SDPG_Foreground means "without occlusion". But, SDPG_Foreground can be drawn fully opaque, or with dithering on 'occluded' areas (e.g. gizmos).
	// The stencil flag marks which areas should *not* receive dithering.
	static constexpr uint32 EditorStencilFlagForegroundNoDithering = 0x1;

	static bool ShouldCompilePermutation(const FPermutationDomain& PermutationVector, const EShaderPlatform Platform)
	{
		const int32 SampleCount = PermutationVector.Get<FSampleCountDimension>();
		const bool bMSAADontResolve = PermutationVector.Get<FMSAADontResolve>();

		// Only use permutations with valid MSAA sample counts.
		if (!FMath::IsPowerOfTwo(SampleCount))
		{
			return false;
		}
		if (!RHISupportsMSAA(Platform) && (SampleCount > 1 || bMSAADontResolve))
		{
			return false;
		}

		return true;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		return ShouldCompilePermutation(PermutationVector, Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FCompositePrimitiveShaderBase::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("EDITOR_STENCIL_FLAG_FOREGROUND_NO_DITHERING"), EditorStencilFlagForegroundNoDithering);
	}
};

#endif //#if UE_ENABLE_DEBUG_DRAWING