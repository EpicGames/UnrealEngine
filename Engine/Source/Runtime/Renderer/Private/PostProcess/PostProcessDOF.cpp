// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessDOF.cpp: Post process Depth of Field implementation.
=============================================================================*/

#include "PostProcess/PostProcessDOF.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "PostProcess/SceneRenderTargets.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessBokehDOF.h"
#include "SceneRenderTargetParameters.h"
#include "PostProcess/PostProcessing.h"
#include "ClearQuad.h"
#include "PipelineStateCache.h"
#include "SceneRendering.h"

FVector4f GetDepthOfFieldParameters(const FPostProcessSettings& PostProcessSettings)
{
	const float SkyFocusDistance = PostProcessSettings.DepthOfFieldSkyFocusDistance;

	// *2 to go to account for Radius/Diameter, 100 for percent
	const float DepthOfFieldVignetteSize = FMath::Max(0.0f, PostProcessSettings.DepthOfFieldVignetteSize / 100.0f * 2);

	// doesn't make much sense to expose this property as the effect is very non linear and it would cost some performance to fix that
	const float DepthOfFieldVignetteFeather = 10.0f / 100.0f;

	const float DepthOfFieldVignetteMul = 1.0f / DepthOfFieldVignetteFeather;
	const float DepthOfFieldVignetteAdd = (0.5f - DepthOfFieldVignetteSize) * DepthOfFieldVignetteMul;

	return FVector4f(
		(SkyFocusDistance > 0) ? SkyFocusDistance : 100000000.0f, // very large if <0 to not mask out skybox, can be optimized to disable feature completely
		DepthOfFieldVignetteMul,
		DepthOfFieldVignetteAdd,
		0.0f);
}

/** Encapsulates the DOF setup pixel shader. */
class FMobileDOFSetupPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileDOFSetupPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileDOFSetupPS, FGlobalShader);

	class FNearBlurCountDim : SHADER_PERMUTATION_INT("NEAR_BLUR_COUNT", 3);
	class FFarBlurCountDim : SHADER_PERMUTATION_INT("FAR_BLUR_COUNT", 2);
	using FPermutationDomain = TShaderPermutationDomain<FNearBlurCountDim, FFarBlurCountDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, DepthOfFieldParams)
		SHADER_PARAMETER(FVector4f, BufferSizeAndInvSize)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SunShaftAndDofTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SunShaftAndDofSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		auto NearBlurCountDim = PermutationVector.Get<FNearBlurCountDim>();

		auto FarBlurCountDim = PermutationVector.Get<FFarBlurCountDim>();

		return IsMobilePlatform(Parameters.Platform) && (NearBlurCountDim > 0 || FarBlurCountDim > 0);
	}
	
	static FPermutationDomain BuildPermutationVector(uint32 InNearBlur, uint32 InFarBlur)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FNearBlurCountDim>(InNearBlur);
		PermutationVector.Set<FFarBlurCountDim>(InFarBlur);
		return PermutationVector;
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileDOFSetupPS, "/Engine/Private/PostProcessDOF.usf", "SetupPS", SF_Pixel);

FMobileDofSetupOutputs AddMobileDofSetupPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofSetupInputs& Inputs)
{
	FRDGTextureDesc DofSetupDesc = Inputs.SceneColor.Texture->Desc;

	const FIntPoint& BufferSize = Inputs.SceneColor.Texture->Desc.Extent;

	DofSetupDesc.Extent = FIntPoint::DivideAndRoundUp(BufferSize, 4);

	DofSetupDesc.Format = PF_FloatRGBA;

	FIntRect OutputRect = FIntRect::DivideAndRoundUp(Inputs.SceneColor.ViewRect, 4);

	FScreenPassRenderTarget DestRenderTarget0 = FScreenPassRenderTarget(GraphBuilder.CreateTexture(DofSetupDesc, TEXT("DOFSetup0")), OutputRect, ERenderTargetLoadAction::EClear);
	FScreenPassRenderTarget DestRenderTarget1;

	uint32 NumRenderTargets = (Inputs.bNearBlur && Inputs.bFarBlur) ? 2 : 1;

	if (NumRenderTargets > 1)
	{
		DestRenderTarget1 = FScreenPassRenderTarget(GraphBuilder.CreateTexture(DofSetupDesc, TEXT("DOFSetup1")), OutputRect, ERenderTargetLoadAction::EClear);
	}

	const float DOFVignetteSize = FMath::Max(0.0f, View.FinalPostProcessSettings.DepthOfFieldVignetteSize);

	// todo: test is conservative, with bad content we would waste a bit of performance
	const bool bDOFVignette = Inputs.bNearBlur && DOFVignetteSize < 200.0f;

	// 0:off, 1:on, 2:on with Vignette
	uint32 NearBlur = Inputs.bNearBlur ? (bDOFVignette ? 2 : 1) : 0;
	uint32 FarBlur = Inputs.bFarBlur ? 1 : 0;

	auto ShaderPermutationVector = FMobileDOFSetupPS::BuildPermutationVector(NearBlur, FarBlur);

	TShaderMapRef<FMobileDOFSetupPS> PixelShader(View.ShaderMap, ShaderPermutationVector);

	FMobileDOFSetupPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileDOFSetupPS::FParameters>();
	PassParameters->RenderTargets[0] = DestRenderTarget0.GetRenderTargetBinding();
	PassParameters->RenderTargets[1] = DestRenderTarget1.GetRenderTargetBinding();
	PassParameters->DepthOfFieldParams = GetDepthOfFieldParameters(View.FinalPostProcessSettings);
	PassParameters->BufferSizeAndInvSize = FVector4f(BufferSize.X, BufferSize.Y, 1.0f / BufferSize.X, 1.0f / BufferSize.Y);
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters->SceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Clamp>::GetRHI();
	PassParameters->SunShaftAndDofTexture = Inputs.SunShaftAndDof.Texture;
	PassParameters->SunShaftAndDofSampler = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Clamp>::GetRHI();

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(DestRenderTarget0);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DOFSetup"), View, OutputViewport, InputViewport, PixelShader, PassParameters);

	FMobileDofSetupOutputs Outputs;
	Outputs.DofSetupFar = DestRenderTarget0;
	Outputs.DofSetupNear = Inputs.bFarBlur && Inputs.bNearBlur ? DestRenderTarget1 : DestRenderTarget0;
	return MoveTemp(Outputs);
}

/** Encapsulates the DOF setup pixel shader. */
// @param FarBlur 0:off, 1:on
// @param NearBlur 0:off, 1:on
class FMobileDOFRecombinePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileDOFRecombinePS);
	SHADER_USE_PARAMETER_STRUCT(FMobileDOFRecombinePS, FGlobalShader);

	class FNearBlurCountDim : SHADER_PERMUTATION_INT("NEAR_BLUR_COUNT", 3);
	class FFarBlurCountDim : SHADER_PERMUTATION_INT("FAR_BLUR_COUNT", 2);
	using FPermutationDomain = TShaderPermutationDomain<FNearBlurCountDim, FFarBlurCountDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, DepthOfFieldUVLimit)
		SHADER_PARAMETER(FVector4f, BufferSizeAndInvSize)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DofFarBlurTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DofFarBlurSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DofNearBlurTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DofNearBlurSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SunShaftAndDofTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SunShaftAndDofSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		auto NearBlurCountDim = PermutationVector.Get<FNearBlurCountDim>();

		auto FarBlurCountDim = PermutationVector.Get<FFarBlurCountDim>();

		return IsMobilePlatform(Parameters.Platform) && (NearBlurCountDim > 0 || FarBlurCountDim > 0);
	}

	static FPermutationDomain BuildPermutationVector(uint32 InNearBlur, uint32 InFarBlur)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FNearBlurCountDim>(InNearBlur);
		PermutationVector.Set<FFarBlurCountDim>(InFarBlur);
		return PermutationVector;
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileDOFRecombinePS, "/Engine/Private/PostProcessDOF.usf", "MainRecombinePS", SF_Pixel);

FScreenPassTexture AddMobileDofRecombinePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofRecombineInputs& Inputs)
{
	FRDGTextureDesc DofRecombineDesc = Inputs.SceneColor.Texture->Desc;
	DofRecombineDesc.Reset();

	const FIntPoint& BufferSize = Inputs.SceneColor.Texture->Desc.Extent;

	DofRecombineDesc.Format = PF_FloatR11G11B10;

	FScreenPassRenderTarget DofRecombinOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(DofRecombineDesc, TEXT("DofRecombine")), Inputs.SceneColor.ViewRect, ERenderTargetLoadAction::EClear);

	// 0:off, 1:on, 2:on with Vignette
	uint32 NearBlur = Inputs.bNearBlur ? 1 : 0;
	uint32 FarBlur = Inputs.bFarBlur ? 1 : 0;

	auto ShaderPermutationVector = FMobileDOFRecombinePS::BuildPermutationVector(NearBlur, FarBlur);

	TShaderMapRef<FMobileDOFRecombinePS> PixelShader(View.ShaderMap, ShaderPermutationVector);

	FVector4f Bounds;
	Bounds.X = (((float)((View.ViewRect.Min.X + 1) & (~1))) + 3.0f) / ((float)(BufferSize.X));
	Bounds.Y = (((float)((View.ViewRect.Min.Y + 1) & (~1))) + 3.0f) / ((float)(BufferSize.Y));
	Bounds.Z = (((float)(View.ViewRect.Max.X & (~1))) - 3.0f) / ((float)(BufferSize.X));
	Bounds.W = (((float)(View.ViewRect.Max.Y & (~1))) - 3.0f) / ((float)(BufferSize.Y));

	FMobileDOFRecombinePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileDOFRecombinePS::FParameters>();
	PassParameters->RenderTargets[0] = DofRecombinOutput.GetRenderTargetBinding();
	PassParameters->DepthOfFieldUVLimit = Bounds;
	PassParameters->BufferSizeAndInvSize = FVector4f(BufferSize.X, BufferSize.Y, 1.0f / BufferSize.X, 1.0f / BufferSize.Y);
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters->SceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->DofFarBlurTexture = Inputs.DofFarBlur.Texture;
	PassParameters->DofFarBlurSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->DofNearBlurTexture = Inputs.DofNearBlur.Texture;
	PassParameters->DofNearBlurSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->SunShaftAndDofTexture = Inputs.SunShaftAndDof.Texture;
	PassParameters->SunShaftAndDofSampler = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Clamp>::GetRHI();

	const FScreenPassTextureViewport InputViewport(Inputs.DofFarBlur.IsValid() ? Inputs.DofFarBlur : Inputs.DofNearBlur);
	const FScreenPassTextureViewport OutputViewport(Inputs.SceneColor);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("DOFRecombine"), View, OutputViewport, InputViewport, PixelShader, PassParameters);

	return MoveTemp(DofRecombinOutput);
}