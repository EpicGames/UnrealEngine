// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessDOF.h: Post process Depth of Field implementation.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "ScreenPass.h"
#include "Math/MathFwd.h"

struct FPostProcessSettings;

FVector4f GetDepthOfFieldParameters(const FPostProcessSettings& PostProcessSettings);

struct FMobileDofSetupInputs
{
	FScreenPassTexture SceneColor;
	FScreenPassTexture SunShaftAndDof;

	bool bFarBlur = false;
	bool bNearBlur = false;
};

struct FMobileDofSetupOutputs
{
	FScreenPassTexture DofSetupFar;
	FScreenPassTexture DofSetupNear;
};

// down sample and setup DOF input
FMobileDofSetupOutputs AddMobileDofSetupPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofSetupInputs& Inputs);

struct FMobileDofRecombineInputs
{
	FScreenPassTexture SceneColor;
	FScreenPassTexture DofFarBlur;
	FScreenPassTexture DofNearBlur;
	FScreenPassTexture SunShaftAndDof;

	bool bFarBlur = false;
	bool bNearBlur = false;
};

FScreenPassTexture AddMobileDofRecombinePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofRecombineInputs& Inputs);
