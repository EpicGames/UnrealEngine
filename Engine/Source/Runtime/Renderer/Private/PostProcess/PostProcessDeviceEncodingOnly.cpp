// Copyright Epic Games, Inc. All Rights Reserved.

/*=========================================================================================
	PostProcessDeviceEncodingOnly.cpp: Post processing device encoding only implementation.
===========================================================================================*/

#include "PostProcess/PostProcessDeviceEncodingOnly.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EngineGlobals.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessCombineLUTs.h"
#include "PostProcess/PostProcessMobile.h"
#include "ClearQuad.h"
#include "PipelineStateCache.h"
#include "HDRHelper.h"
#include "ACESUtils.h"
#include "SceneRendering.h"

namespace
{
	const int32 GDeviceEncodingOnlyComputeTileSizeX = 8;
	const int32 GDeviceEncodingOnlyComputeTileSizeY = 8;

	namespace DeviceEncodingOnlyPermutation
	{
		// Desktop renderer permutation dimensions.
		class FDeviceEncodingOnlyOutputDeviceSRGB : SHADER_PERMUTATION_BOOL("OUTPUT_DEVICE_SRGB");
		class FDeviceEncodingUseACES2 : SHADER_PERMUTATION_BOOL("USE_ACES_2");

		using FDesktopDomain = TShaderPermutationDomain<FDeviceEncodingOnlyOutputDeviceSRGB, FDeviceEncodingUseACES2>;
	
	} // namespace DeviceEncodingOnlyPermutation
} // namespace


FDeviceEncodingOnlyOutputDeviceParameters GetDeviceEncodingOnlyOutputDeviceParameters(const FSceneViewFamily& Family)
{
	static TConsoleVariableData<float>* CVarOutputGamma = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.TonemapperGamma"));

	EDisplayOutputFormat OutputDeviceValue;
	bool bIsHDR;

	if (Family.SceneCaptureSource == SCS_FinalColorHDR)
	{
		OutputDeviceValue = EDisplayOutputFormat::HDR_LinearNoToneCurve;
		bIsHDR = true;
	}
	else if (Family.SceneCaptureSource == SCS_FinalToneCurveHDR)
	{
		OutputDeviceValue = EDisplayOutputFormat::HDR_LinearWithToneCurve;
		bIsHDR = true;
	}
	else
	{
		OutputDeviceValue = Family.RenderTarget->GetDisplayOutputFormat();
		bIsHDR = Family.RenderTarget->GetSceneHDREnabled();
	}

	float Gamma = CVarOutputGamma->GetValueOnRenderThread();

	// In case gamma is unspecified, fall back to 2.2 which is the most common case
	if ((PLATFORM_APPLE || OutputDeviceValue == EDisplayOutputFormat::SDR_ExplicitGammaMapping) && Gamma == 0.0f)
	{
		Gamma = 2.2f;
	}

	// Enforce user-controlled ramp over sRGB or Rec709
	if (Gamma > 0.0f && (OutputDeviceValue == EDisplayOutputFormat::SDR_sRGB || OutputDeviceValue == EDisplayOutputFormat::SDR_Rec709))
	{
		OutputDeviceValue = EDisplayOutputFormat::SDR_ExplicitGammaMapping;
		bIsHDR = false;
	}

	FVector InvDisplayGammaValue;
	InvDisplayGammaValue.X = 1.0f / Family.RenderTarget->GetDisplayGamma();
	InvDisplayGammaValue.Y = 2.2f / Family.RenderTarget->GetDisplayGamma();
	InvDisplayGammaValue.Z = 1.0f / FMath::Max(Gamma, 1.0f);

	FDeviceEncodingOnlyOutputDeviceParameters Parameters;
	Parameters.InverseGamma = (FVector3f)InvDisplayGammaValue;
	Parameters.OutputDevice = static_cast<uint32>(OutputDeviceValue);
	Parameters.OutputGamut = static_cast<uint32>(Family.RenderTarget->GetDisplayColorGamut());
	static const IConsoleVariable* const CVarHDRDisplayOverrideOSMaxLuminance = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.Display.OverrideOSMaxLuminance"));
	Parameters.OutputMaxLuminance = CVarHDRDisplayOverrideOSMaxLuminance->GetBool() ?
		(bIsHDR ? HDRGetDisplayMaximumLuminance() : 100.f) :
		Family.RenderTarget->GetMaximumLuminanceInNits();
	return Parameters;
}

BEGIN_SHADER_PARAMETER_STRUCT(FDeviceEncodingACESTonemapShaderParameters, )
	SHADER_PARAMETER(FVector4f, ACESMinMaxData) // xy = min ACES/luminance, zw = max ACES/luminance
	SHADER_PARAMETER(FVector4f, ACESMidData) // x = mid ACES, y = mid luminance, z = mid slope
	SHADER_PARAMETER(FVector4f, ACESCoefsLow_0) // coeflow 0-3
	SHADER_PARAMETER(FVector4f, ACESCoefsHigh_0) // coefhigh 0-3
	SHADER_PARAMETER(float, ACESCoefsLow_4)
	SHADER_PARAMETER(float, ACESCoefsHigh_4)
	SHADER_PARAMETER(float, ACESSceneColorMultiplier)
	SHADER_PARAMETER(float, ACESGamutCompression)
	SHADER_PARAMETER(uint32, bScaleWhite)
	SHADER_PARAMETER_SRV(Texture2D<float>, ACESReachTable)
	SHADER_PARAMETER_SRV(Texture2D<float>, ACESGamutTable)
	SHADER_PARAMETER_SRV(Texture2D<float>, ACESGammaTable)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FDeviceEncodingOnlyParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_STRUCT_REF(FWorkingColorSpaceShaderParameters, WorkingColorSpace)
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeviceEncodingOnlyOutputDeviceParameters, OutputDevice)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
	SHADER_PARAMETER_STRUCT_INCLUDE(FDeviceEncodingACESTonemapShaderParameters, ACESTonemapParameters)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
	SHADER_PARAMETER(float, EditorNITLevel)
	SHADER_PARAMETER(float, TotalSceneColorMultiplier)
END_SHADER_PARAMETER_STRUCT()

class FDeviceEncodingOnlyPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDeviceEncodingOnlyPS);
	SHADER_USE_PARAMETER_STRUCT(FDeviceEncodingOnlyPS, FGlobalShader);

	using FPermutationDomain = DeviceEncodingOnlyPermutation::FDesktopDomain;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDeviceEncodingOnlyParameters, DeviceEncoding)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeviceEncodingOnlyPS, "/Engine/Private/PostProcessDeviceEncodingOnly.usf", "MainPS", SF_Pixel);

class FDeviceEncodingOnlyCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDeviceEncodingOnlyCS);
	SHADER_USE_PARAMETER_STRUCT(FDeviceEncodingOnlyCS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<DeviceEncodingOnlyPermutation::FDesktopDomain>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDeviceEncodingOnlyParameters, DeviceEncoding)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWOutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
		{
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GDeviceEncodingOnlyComputeTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GDeviceEncodingOnlyComputeTileSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDeviceEncodingOnlyCS, "/Engine/Private/PostProcessDeviceEncodingOnly.usf", "MainCS", SF_Compute);

extern uint32 DoesGamutWhitePointMatch(uint32 OutputGamut, const UE::Color::FColorSpace& ColorSpace);

FScreenPassTexture AddDeviceEncodingOnlyPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FDeviceEncodingOnlyInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	const FSceneViewFamily& ViewFamily = *(View.Family);
	const FPostProcessSettings& PostProcessSettings = View.FinalPostProcessSettings;

	const FScreenPassTextureViewport SceneColorViewport(Inputs.SceneColor);

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		FRDGTextureDesc OutputDesc = Inputs.SceneColor.Texture->Desc;
		OutputDesc.Reset();
		// RGB is the color in LDR, A is the luminance for PostprocessAA
		OutputDesc.Format = Inputs.bOutputInHDR ? GRHIHDRDisplayOutputFormat : PF_B8G8R8A8;
		OutputDesc.ClearValue = FClearValueBinding(FLinearColor(0, 0, 0, 0));
		OutputDesc.Flags |= View.bUseComputePasses ? TexCreate_UAV : TexCreate_RenderTargetable;
		OutputDesc.Flags |= GFastVRamConfig.Tonemap;

		const FDeviceEncodingOnlyOutputDeviceParameters OutputDeviceParameters = GetDeviceEncodingOnlyOutputDeviceParameters(*View.Family);
		const EDisplayOutputFormat OutputDevice = static_cast<EDisplayOutputFormat>(OutputDeviceParameters.OutputDevice);

		if (OutputDevice == EDisplayOutputFormat::HDR_LinearEXR)
		{
			OutputDesc.Format = PF_A32B32G32R32F;
		}
		if (OutputDevice == EDisplayOutputFormat::HDR_LinearNoToneCurve || OutputDevice == EDisplayOutputFormat::HDR_LinearWithToneCurve)
		{
			OutputDesc.Format = PF_FloatRGBA;
		}

		Output = FScreenPassRenderTarget(
			GraphBuilder.CreateTexture(OutputDesc, TEXT("DeviceEncodingOnly")),
			Inputs.SceneColor.ViewRect,
			View.GetOverwriteLoadAction());
	}

	const FScreenPassTextureViewport OutputViewport(Output);

	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	float EditorNITLevel = 160.0f;

#if WITH_EDITOR
	{
		static auto CVarHDRNITLevel = IConsoleManager::Get().FindConsoleVariable(TEXT("Editor.HDRNITLevel"));
		if (CVarHDRNITLevel)
		{
			EditorNITLevel = CVarHDRNITLevel->GetFloat();
		}
	}
#endif

	FDeviceEncodingOnlyParameters CommonParameters;
	CommonParameters.View = View.ViewUniformBuffer;
	CommonParameters.WorkingColorSpace = GDefaultWorkingColorSpaceUniformBuffer.GetUniformBufferRef();
	CommonParameters.OutputDevice = GetDeviceEncodingOnlyOutputDeviceParameters(ViewFamily);
	CommonParameters.Color = GetScreenPassTextureViewportParameters(SceneColorViewport);
	CommonParameters.Output = GetScreenPassTextureViewportParameters(OutputViewport);

	GetACESTonemapParameters(CommonParameters.ACESTonemapParameters);
	const UE::Color::FColorSpace& LimitingColorSpace = ViewFamily.RenderTarget->GetLimitingColorSpace();
	CommonParameters.ACESTonemapParameters.bScaleWhite = !DoesGamutWhitePointMatch(CommonParameters.OutputDevice.OutputGamut, LimitingColorSpace);
	UE::Color::ACES::GetTransformResources(
		GraphBuilder,
		CommonParameters.OutputDevice.OutputMaxLuminance,
		LimitingColorSpace,
		CommonParameters.ACESTonemapParameters.ACESReachTable,
		CommonParameters.ACESTonemapParameters.ACESGamutTable,
		CommonParameters.ACESTonemapParameters.ACESGammaTable
	);

	CommonParameters.ColorTexture = Inputs.SceneColor.Texture;
	CommonParameters.ColorSampler = BilinearClampSampler;
	CommonParameters.EditorNITLevel = EditorNITLevel;

	const float HDRPaperWhiteRatio = ViewFamily.RenderTarget->GetHDRPaperWhiteInNits() / 203.f;
	CommonParameters.TotalSceneColorMultiplier = HDRPaperWhiteRatio * CommonParameters.ACESTonemapParameters.ACESSceneColorMultiplier;

	// Generate permutation vector for the desktop tonemapper.
	DeviceEncodingOnlyPermutation::FDesktopDomain DesktopPermutationVector;

	const bool bOutputDeviceSRGB = (CommonParameters.OutputDevice.OutputDevice == (uint32)EDisplayOutputFormat::SDR_sRGB);
	DesktopPermutationVector.Set<DeviceEncodingOnlyPermutation::FDeviceEncodingOnlyOutputDeviceSRGB>(bOutputDeviceSRGB);

	static const auto CVarAcesVersion = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.Aces.Version"));
	DesktopPermutationVector.Set<DeviceEncodingOnlyPermutation::FDeviceEncodingUseACES2>(CVarAcesVersion->GetValueOnRenderThread() > 1);

	// Override output might not support UAVs.
	const bool bComputePass = (Output.Texture->Desc.Flags & TexCreate_UAV) == TexCreate_UAV ? View.bUseComputePasses : false;

	if (bComputePass)
	{
		FDeviceEncodingOnlyCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDeviceEncodingOnlyCS::FParameters>();
		PassParameters->DeviceEncoding = CommonParameters;
		PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(Output.Texture);

		FDeviceEncodingOnlyCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<DeviceEncodingOnlyPermutation::FDesktopDomain>(DesktopPermutationVector);

		TShaderMapRef<FDeviceEncodingOnlyCS> ComputeShader(View.ShaderMap, PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("DeviceEncodingOnly %dx%d (CS)", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), FIntPoint(GDeviceEncodingOnlyComputeTileSizeX, GDeviceEncodingOnlyComputeTileSizeY)));
	}
	else
	{
		FDeviceEncodingOnlyPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDeviceEncodingOnlyPS::FParameters>();
		PassParameters->DeviceEncoding = CommonParameters;
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FDeviceEncodingOnlyPS> PixelShader(View.ShaderMap, DesktopPermutationVector);

		// If this is a stereo view, there's a good chance we need alpha out of the tonemapper
		// @todo: Remove this once Oculus fix the bug in their runtime that requires alpha here.
		//const bool bIsStereo = IStereoRendering::IsStereoEyeView(View);
		//FRHIBlendState* BlendState = bIsStereo ? FScreenPassPipelineState::FDefaultBlendState::GetRHI() : TStaticBlendStateWriteMask<CW_RGB>::GetRHI();
		FRHIBlendState* BlendState = TStaticBlendStateWriteMask<CW_RGB>::GetRHI();
		FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

		EScreenPassDrawFlags DrawFlags = EScreenPassDrawFlags::AllowHMDHiddenAreaMask;

		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("DeviceEncodingOnly %dx%d (PS)", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
			View,
			OutputViewport,
			SceneColorViewport,
			FScreenPassPipelineState(VertexShader, PixelShader, BlendState, DepthStencilState),
			PassParameters,
			DrawFlags,
			[PixelShader, PassParameters](FRHICommandList& RHICmdList)
			{
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
			});
	}

	return MoveTemp(Output);
}

