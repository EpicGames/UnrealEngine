// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PostProcess/PostProcessEyeAdaptation.h"
#include "ScreenPass.h"
#include "OverridePassSequence.h"
#include "Math/Halton.h"

BEGIN_SHADER_PARAMETER_STRUCT(FDeviceEncodingOnlyOutputDeviceParameters, )
	SHADER_PARAMETER(FVector3f, InverseGamma)
	SHADER_PARAMETER(uint32, OutputDevice)
	SHADER_PARAMETER(uint32, OutputGamut)
	SHADER_PARAMETER(float, OutputMaxLuminance)
END_SHADER_PARAMETER_STRUCT()

FDeviceEncodingOnlyOutputDeviceParameters GetDeviceEncodingOnlyOutputDeviceParameters(const FSceneViewFamily& Family);

struct FDeviceEncodingOnlyInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] HDR scene color to tonemap.
	FScreenPassTexture SceneColor;

	// Whether to leave the final output in HDR.
	bool bOutputInHDR = false;
};

FScreenPassTexture AddDeviceEncodingOnlyPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FDeviceEncodingOnlyInputs& Inputs);
