// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessDownsample.h"
#include "PostProcess/PostProcessEyeAdaptation.h"
#include "PixelShaderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"

namespace
{
const int32 GDownsampleTileSizeX = 8;
const int32 GDownsampleTileSizeY = 8;

BEGIN_SHADER_PARAMETER_STRUCT(FDownsampleParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
END_SHADER_PARAMETER_STRUCT()

FDownsampleParameters GetDownsampleParameters(const FViewInfo& View, FScreenPassTexture Output, FScreenPassTextureSlice Input, EDownsampleQuality DownsampleMethod)
{
	check(Output.IsValid());
	check(Input.IsValid());

	const FScreenPassTextureViewportParameters InputParameters = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Input));
	const FScreenPassTextureViewportParameters OutputParameters = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Output));

	FDownsampleParameters Parameters;
	Parameters.ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters.Input = InputParameters;
	Parameters.Output = OutputParameters;
	Parameters.InputTexture = Input.TextureSRV;
	Parameters.InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	return Parameters;
}

class FDownsampleQualityDimension : SHADER_PERMUTATION_ENUM_CLASS("DOWNSAMPLE_QUALITY", EDownsampleQuality);
using FDownsamplePermutationDomain = TShaderPermutationDomain<FDownsampleQualityDimension>;

class FDownsamplePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDownsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FDownsamplePS, FGlobalShader);

	using FPermutationDomain = FDownsamplePermutationDomain;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDownsampleParameters, Common)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

IMPLEMENT_GLOBAL_SHADER(FDownsamplePS, "/Engine/Private/PostProcessDownsample.usf", "MainPS", SF_Pixel);

class FDownsampleCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDownsampleCS);
	SHADER_USE_PARAMETER_STRUCT(FDownsampleCS, FGlobalShader);

	using FPermutationDomain = FDownsamplePermutationDomain;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FDownsampleParameters, Common)
		SHADER_PARAMETER(FScreenTransform, DispatchThreadIdToInputUV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutComputeTexture)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters,OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GDownsampleTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GDownsampleTileSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FDownsampleCS, "/Engine/Private/PostProcessDownsample.usf", "MainCS", SF_Compute);
} //! namespace

FScreenPassTexture AddDownsamplePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FDownsamplePassInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	bool bIsComputePass = View.bUseComputePasses;

	if ((Inputs.Flags & EDownsampleFlags::ForceRaster) == EDownsampleFlags::ForceRaster)
	{
		bIsComputePass = false;
	}

	FScreenPassRenderTarget Output;

	// Construct the output texture to be half resolution (rounded up to even) with an optional format override.
	{
		FScreenPassTextureViewport SceneColorViewport(Inputs.SceneColor);
		FScreenPassTextureViewport DownscaledViewport = GetDownscaledViewport(SceneColorViewport, 2);

		const FRDGTextureDesc& InputDesc = Inputs.SceneColor.TextureSRV->Desc.Texture->Desc;
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			DownscaledViewport.Extent,
			Inputs.FormatOverride != PF_Unknown ? Inputs.FormatOverride  : InputDesc.Format,
			FClearValueBinding(FLinearColor(0, 0, 0, 0)),
			/* InFlags = */ TexCreate_ShaderResource | GFastVRamConfig.Downsample | (bIsComputePass ? TexCreate_UAV : (TexCreate_RenderTargetable | TexCreate_NoFastClear)));

		if (Inputs.UserSuppliedOutput && Translate(Inputs.UserSuppliedOutput->GetDesc()) == Desc)
		{
			Output.Texture = GraphBuilder.RegisterExternalTexture(Inputs.UserSuppliedOutput, Inputs.Name);
		}
		else
		{
			Output.Texture = GraphBuilder.CreateTexture(Desc, Inputs.Name);
		}
		Output.ViewRect = DownscaledViewport.Rect;
		Output.LoadAction = ERenderTargetLoadAction::ENoAction;
		Output.UpdateVisualizeTextureExtent();
	}

	if (bIsComputePass)
	{
		AddDownsampleComputePass(GraphBuilder, View, Inputs.SceneColor, Output, Inputs.Quality, bIsComputePass ? ERDGPassFlags::Compute : ERDGPassFlags::Raster);
	}
	else
	{
		FDownsamplePermutationDomain PermutationVector;
		PermutationVector.Set<FDownsampleQualityDimension>(Inputs.Quality);

		FDownsamplePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDownsamplePS::FParameters>();
		PassParameters->Common = GetDownsampleParameters(View, Output, Inputs.SceneColor, Inputs.Quality);
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		TShaderMapRef<FDownsamplePS> PixelShader(View.ShaderMap, PermutationVector);
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("Downsample(%s Quality=%s PS) %dx%d -> %dx%d",
				Output.Texture->Name,
				Inputs.Quality == EDownsampleQuality::High ? TEXT("High") : TEXT("Bilinear"),
				Inputs.SceneColor.ViewRect.Width(), Inputs.SceneColor.ViewRect.Height(),
				Output.ViewRect.Width(), Output.ViewRect.Height()),
			PixelShader,
			PassParameters,
			Output.ViewRect);
	}

	return MoveTemp(Output);
}

void AddDownsampleComputePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTextureSlice Input, FScreenPassTexture Output, EDownsampleQuality Quality, ERDGPassFlags PassFlags)
{
	check(PassFlags == ERDGPassFlags::Compute || PassFlags == ERDGPassFlags::AsyncCompute);

	FDownsampleCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDownsampleCS::FParameters>();
	PassParameters->Common = GetDownsampleParameters(View, Output, Input, Quality);
	PassParameters->DispatchThreadIdToInputUV = ((FScreenTransform::Identity + 0.5f) / Output.ViewRect.Size()) * FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Input), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
	PassParameters->OutComputeTexture = GraphBuilder.CreateUAV(Output.Texture);

	FDownsamplePermutationDomain PermutationVector;
	PermutationVector.Set<FDownsampleQualityDimension>(Quality);

	TShaderMapRef<FDownsampleCS> ComputeShader(View.ShaderMap, PermutationVector);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("Downsample(%s Quality=%s CS) %dx%d -> %dx%d",
			Output.Texture->Name,
			Quality == EDownsampleQuality::High ? TEXT("High") : TEXT("Bilinear"),
			Input.ViewRect.Width(), Input.ViewRect.Height(),
			Output.ViewRect.Width(), Output.ViewRect.Height()),
		PassFlags,
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(Output.ViewRect.Size(), FIntPoint(GDownsampleTileSizeX, GDownsampleTileSizeY)));
}

void AddDownsampleComputePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture Input, FScreenPassTexture Output, EDownsampleQuality Quality, ERDGPassFlags PassFlags)
{
	return AddDownsampleComputePass(GraphBuilder, View, FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, Input), Output, Quality, PassFlags);
}

void FTextureDownsampleChain::Init(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FScreenPassTextureSlice InputTexture,
	EDownsampleQuality DownsampleQuality,
	uint32 StageCount,
	bool bLogLumaInAlpha,
	const TCHAR* TextureName,
	uint32 InputResolutionExponent)
{
	check(InputTexture.IsValid());

	RDG_EVENT_SCOPE(GraphBuilder, "TextureDownsampleChain");

#if !UE_BUILD_SHIPPING
	checkf(IsInRenderingThread(), TEXT("Potential race condition accessing DebugNames?"));
	static TSet<FString> DebugNames;
#endif

	Textures.SetNum(StageCount);

	// The first stage is the input.
	Textures[0] = InputTexture;

	for (uint32 StageIndex = 1; StageIndex < StageCount; StageIndex++)
	{
		const uint32 PreviousStageIndex = StageIndex - 1;

		const TCHAR* DebugNamePtr = TEXT("DownsampledTexture");
#if !UE_BUILD_SHIPPING
		const FString TextureDebugName = FString::Printf(TEXT("%s(1/%u)"), TextureName ? TextureName : InputTexture.TextureSRV->Desc.Texture->Name, 1 << (StageIndex + InputResolutionExponent));
		DebugNamePtr = *DebugNames.FindOrAdd(TextureDebugName);
#endif

		FDownsamplePassInputs PassInputs;
		PassInputs.Name = DebugNamePtr;
		PassInputs.SceneColor = Textures[PreviousStageIndex];
		PassInputs.Quality = DownsampleQuality;

		Textures[StageIndex] = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, AddDownsamplePass(GraphBuilder, View, PassInputs));

		if (bLogLumaInAlpha)
		{
			bLogLumaInAlpha = false;

			Textures[StageIndex] = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, AddBasicEyeAdaptationSetupPass(GraphBuilder, View, EyeAdaptationParameters, FScreenPassTexture(Textures[StageIndex])));
		}
	}

	bInitialized = true;
}