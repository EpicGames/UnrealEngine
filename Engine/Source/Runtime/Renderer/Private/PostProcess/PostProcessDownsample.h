// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

class FEyeAdaptationParameters;

enum class EDownsampleFlags : uint8
{
	None = 0,

	// Forces the downsample pass to run on the raster pipeline, regardless of view settings.
	ForceRaster = 0x1
};
ENUM_CLASS_FLAGS(EDownsampleFlags);

enum class EDownsampleQuality : uint8
{
	// Single filtered sample (2x2 tap).
	Low,

	// Four filtered samples (4x4 tap).
	High,

	MAX
};

// The set of inputs needed to add a downsample pass to RDG.
struct FDownsamplePassInputs
{
	FDownsamplePassInputs() = default;

	// Friendly name of the pass. Used for logging and profiling.
	const TCHAR* Name = nullptr;

	// Optional user supplied output buffer.
	IPooledRenderTarget* UserSuppliedOutput = nullptr;

	// Input scene color RDG texture / view rect. Must not be null.
	FScreenPassTextureSlice SceneColor;

	// The downsample method to use.
	EDownsampleQuality Quality = EDownsampleQuality::Low;

	// Flags to control how the downsample pass is run.
	EDownsampleFlags Flags = EDownsampleFlags::None;

	// The format to use for the output texture (if unknown, the input format is used).
	EPixelFormat FormatOverride = PF_Unknown;
};

FScreenPassTexture AddDownsamplePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FDownsamplePassInputs& Inputs);

void AddDownsampleComputePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture Input, FScreenPassTexture Output, EDownsampleQuality Quality, ERDGPassFlags PassFlags);
void AddDownsampleComputePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTextureSlice Input, FScreenPassTexture Output, EDownsampleQuality Quality, ERDGPassFlags PassFlags);


class FTextureDownsampleChain
{
public:
	FTextureDownsampleChain() = default;

	void Init(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FEyeAdaptationParameters& EyeAdaptationParameters,
		FScreenPassTextureSlice InputTexture,
		EDownsampleQuality DownsampleQuality,
		uint32 StageCount,
		bool bLogLumaInAlpha,
		const TCHAR* TextureName = nullptr,
		uint32 InputResolutionExponent = 0);

	bool IsInitialized() const
	{
		return bInitialized;
	}

	FScreenPassTextureSlice GetTexture(uint32 StageIndex) const
	{
		return Textures[StageIndex];
	}

	FScreenPassTextureSlice GetFirstTexture() const
	{
		return Textures[0];
	}

	FScreenPassTextureSlice GetLastTexture() const
	{
		return Textures.Last();
	}

	uint32 GetStageCount() const
	{
		return Textures.Num();
	}

private:
	TArray<FScreenPassTextureSlice> Textures;
	bool bInitialized = false;
};