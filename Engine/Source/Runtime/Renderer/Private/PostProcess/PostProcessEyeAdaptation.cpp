// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessEyeAdaptation.h"
#include "CoreMinimal.h"
#include "BufferVisualizationData.h"
#include "CanvasTypes.h"
#include "ColorManagement/ColorSpace.h"
#include "Containers/ArrayView.h"
#include "Containers/ResourceArray.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveFloat.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/EngineTypes.h"
#include "Engine/Scene.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "GameTime.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"
#include "Lumen/LumenTranslucencyVolumeLighting.h"
#include "Math/Color.h"
#include "Misc/LargeWorldCoordinates.h"
#include "PathTracing.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessLocalExposure.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RayTracing/RaytracingOptions.h"
#include "RenderCommandTag.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIGPUReadback.h"
#include "RHIPipeline.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderParameterUtils.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "Tasks/Task.h"
#include "TextureResource.h"
#include "TranslucentLighting.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"
#include "SceneViewState.h"

bool IsMobileEyeAdaptationEnabled(const FViewInfo& View);

namespace
{
	TAutoConsoleVariable<float> CVarEyeAdaptationPreExposureOverride(
		TEXT("r.EyeAdaptation.PreExposureOverride"),
		0,
		TEXT("Overide the scene pre-exposure by a custom value. \n")
		TEXT("= 0 : No override\n")
		TEXT("> 0 : Override PreExposure\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarEyeAdaptationMethodOverride(
		TEXT("r.EyeAdaptation.MethodOverride"),
		-1,
		TEXT("Override the camera metering method set in post processing volumes\n")
		TEXT("-2: override with custom settings (for testing Basic Mode)\n")
		TEXT("-1: no override\n")
		TEXT(" 1: Auto Histogram-based\n")
		TEXT(" 2: Auto Basic\n")
		TEXT(" 3: Manual"),
		ECVF_Scalability | ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarEyeAdaptationExponentialTransitionDistance(
		TEXT("r.EyeAdaptation.ExponentialTransitionDistance"),
		1.5,
		TEXT("The auto exposure moves linearly, but when it gets ExponentialTransitionDistance F-stops away from the\n")
		TEXT("target exposure it switches to as slower exponential function.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int> CVarEyeAdaptationVisualizeDebugType(
		TEXT("r.EyeAdaptation.VisualizeDebugType"),
		0,
		TEXT("When enabling Show->Visualize->HDR (Eye Adaptation) is enabled, this flag controls the scene color.\n")
		TEXT("    0: Scene Color after tonemapping (default)\n")
		TEXT("    1: Histogram Debug\n")
		TEXT("    2: Luminance\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarEyeAdaptationLensAttenuation(
		TEXT("r.EyeAdaptation.LensAttenuation"),
		0.78f,
		TEXT("The camera lens attenuation (q). Set this number to 0.78 for lighting to be unitless (1.0cd/m^2 becomes 1.0 at EV100) or 0.65 to match previous versions (1.0cd/m^2 becomes 1.2 at EV100)."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarEyeAdaptationBlackHistogramBucketInfluence(
		TEXT("r.EyeAdaptation.BlackHistogramBucketInfluence"),
		0.0,
		TEXT("This parameter controls how much weight to apply to completely dark 0.0 values in the exposure histogram.\n")
		TEXT("When set to 1.0, fully dark pixels will accumulate normally, whereas when set to 0.0 fully dark pixels\n")
		TEXT("will have no influence.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarEyeAdaptationExposureCompensationCurveLUT(
		TEXT("r.EyeAdaptation.ExposureCompensationCurveLUT"),
		true,
		TEXT("Whether to bake the Exposure Compensation Curve to a LUT.\n")
		TEXT("This allows (approximate) evaluation of the curve on the GPU, avoiding delays due to GPU->CPU readback."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarEyeAdaptationExposureCompensationCurveLUTNumSamples(
		TEXT("r.EyeAdaptation.ExposureCompensationCurveLUT.NumSamples"),
		64,
		TEXT("Number of samples to use when baking Exposure Compensation Curve to a LUT."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarEyeAdaptationExposureCompensationCurveLUTMinX(
		TEXT("r.EyeAdaptation.ExposureCompensationCurveLUT.MinX"),
		-10.0f,
		TEXT("Minimum X to use when baking Exposure Compensation Curve to a LUT."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarEyeAdaptationExposureCompensationCurveLUTMaxX(
		TEXT("r.EyeAdaptation.ExposureCompensationCurveLUT.MaxX"),
		20.0f,
		TEXT("Maximum X to use when baking Exposure Compensation Curve to a LUT."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarAutoExposureIgnoreMaterials(
		TEXT("r.AutoExposure.IgnoreMaterials"),
		false,
		TEXT("(Experimental) Whether to calculate auto exposure assuming every surface uses a perfectly diffuse white material.\n")
		TEXT("(default: false)"),
		ECVF_Scalability | ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarAutoExposureLuminanceMethod(
		TEXT("r.AutoExposure.LuminanceMethod"),
		0,
		TEXT("0 - Uniform.\n")
		TEXT("1 - NSTC.\n")
		TEXT("2 - Working Color Space."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarAutoExposureIgnoreMaterialsReconstructFromSceneColor(
		TEXT("r.AutoExposure.IgnoreMaterials.ReconstructFromSceneColor"),
		true,
		TEXT(""),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarAutoExposureIgnoreMaterialsEvaluationPositionBias(
		TEXT("r.AutoExposure.IgnoreMaterials.EvaluationPositionBias"),
		10.0f,
		TEXT(""),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarAutoExposureIgnoreMaterialsLuminanceScale(
		TEXT("r.AutoExposure.IgnoreMaterials.LuminanceScale"),
		0.18f,
		TEXT(""),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarAutoExposureIgnoreMaterialsMinBaseColorLuminance(
		TEXT("r.AutoExposure.IgnoreMaterials.MinBaseColorLuminance"),
		0.01f,
		TEXT(""),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarAutoExposureIgnoreMaterialsUsePrecalculatedIlluminance(
		TEXT("r.AutoExposure.IgnoreMaterials.UsePrecalculatedIlluminance"),
		true,
		TEXT(""),
		ECVF_Scalability | ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarAutoExposureIgnoreMaterialsDownscaleFactor(
		TEXT("r.AutoExposure.IgnoreMaterials.DownscaleFactor"),
		2,
		TEXT(""),
		ECVF_Scalability | ECVF_RenderThreadSafe);

	TAutoConsoleVariable<bool> CVarAutoExposureIgnoreMaterialsDebug(
		TEXT("r.AutoExposure.IgnoreMaterials.Debug"),
		false,
		TEXT(""),
		ECVF_RenderThreadSafe);
}

static TAutoConsoleVariable<float> CVarEyeAdaptationCachedLightingPreExposure(
	TEXT("r.EyeAdaptation.CachedLightingPreExposure"),
	4.0f,
	TEXT("Fixed pre-exposure offset in EV units for cached lighting. Adjust it per project to cover used exposure range.\n")
	TEXT("Default 4 maps to ~[-8; 12] of usable exposure range, which should mostly cover physically based lighting range without any lighting response clipping.\n")
	TEXT("8 maps to ~[-4; 16] of usable exposure range, which fully covers physically based lighting range."),
	ECVF_RenderThreadSafe
);

namespace EyeAdaptation
{
	// Whether pre exposure should be used for this View
	bool IsPreExposureRelevant(const FViewInfo& View)
	{
		const FSceneViewFamily& ViewFamily = *View.Family;

		const bool bIsPreExposureRelevant = true
			&& ViewFamily.EngineShowFlags.EyeAdaptation // Controls whether scene luminance is computed at all.
			&& ViewFamily.EngineShowFlags.Lighting
			&& IsPostProcessingEnabled(View)
			&& ViewFamily.bResolveScene
			&& !ViewFamily.EngineShowFlags.LightMapDensity
			&& !ViewFamily.EngineShowFlags.StationaryLightOverlap
			&& !ViewFamily.EngineShowFlags.LightComplexity
			&& !ViewFamily.EngineShowFlags.LODColoration
			&& !ViewFamily.EngineShowFlags.HLODColoration
			&& !ViewFamily.EngineShowFlags.ActorColoration
			&& ((!ViewFamily.EngineShowFlags.VisualizeBuffer) || View.CurrentBufferVisualizationMode != NAME_None)   // disable pre-exposure for the buffer visualization modes
			&& (!ViewFamily.EngineShowFlags.PathTracing || PathTracing::NeedsTonemapping())                     // Either we aren't using path tracing at all, or we are using a path tracing mode that needs tone-mapping
			&& (!ViewFamily.EngineShowFlags.RayTracingDebug || RaytracingDebugViewModeNeedsTonemapping(View)); // Either we aren't using RT debug at all, or we are using a mode that needs tone-mapping

		return bIsPreExposureRelevant;
	}

	float GetCachedLightingPreExposureEV()
	{
		return FMath::Clamp(CVarEyeAdaptationCachedLightingPreExposure.GetValueOnRenderThread(), -16.0f, 16.0f);
	}

	float GetCachedLightingPreExposure()
	{
		return 1.0f / FMath::Exp2(GetCachedLightingPreExposureEV());
	}

	bool WriteWarnings(const FScene* Scene, const FViewInfo& View, FScreenMessageWriter* Writer)
	{
		// Float16 has [-14;+16] range in EV units
		// Additionally need at least 2 stops below and 8 stops above that in order to not clip any lighting response
		const float CachedExposureEV = GetCachedLightingPreExposureEV();
		const float CachedLightingPreExposureMin = CachedExposureEV - 14.0f + 2.0f;
		const float CachedLightingPreExposureMax = CachedExposureEV + 16.0f - 8.0f;

		const float ExposureEV = FMath::Log2(1.0f / View.PreExposure);

		bool bShowCachedLightingWarning = false;

		// This warning only makes scene if scene is lit, has Lumen and it's the main view
		if (Scene
			&& Scene->GetWorld()->WorldType != EWorldType::EditorPreview
			&& Scene->Lights.Num() > 0
			&& View.ViewState
			&& IsPreExposureRelevant(View)
			&& !View.bIsPlanarReflection
			&& !View.bIsSceneCapture
			&& !View.bIsSceneCaptureCube
			&& View.FinalPostProcessSettings.DynamicGlobalIlluminationMethod == EDynamicGlobalIlluminationMethod::Lumen)
		{
			bShowCachedLightingWarning = ExposureEV < CachedLightingPreExposureMin || ExposureEV > CachedLightingPreExposureMax;
		}

		if (bShowCachedLightingWarning && Writer)
		{
			const FString String = FString::Printf(
				TEXT("Cached lighting in Lumen and real-time sky capture lighting is going to be clipped. Please adjust r.EyeAdaptation.CachedLightingPreExposure to match expected exposure range in project. Exposure: %.1f. Safe exposure range: [%.1f, %.1f]."),
					ExposureEV,
					CachedLightingPreExposureMin,
					CachedLightingPreExposureMax);

			Writer->DrawLine(FText::FromString(String));
		}

		return bShowCachedLightingWarning;
	}
}

namespace AutoExposurePermutation
{
	bool ShouldCompileCommonPermutation(const FCommonDomain& PermutationVector)
	{
		if (PermutationVector.Get<FUseApproxIlluminanceDim>() && PermutationVector.Get<FUsePrecalculatedLuminanceDim>())
		{
			return false;
		}

		return true;
	}

	FCommonDomain BuildCommonPermutationDomain()
	{
		FCommonDomain PermutationVector;

		if (CVarAutoExposureIgnoreMaterials.GetValueOnRenderThread())
		{
			if (CVarAutoExposureIgnoreMaterialsUsePrecalculatedIlluminance.GetValueOnRenderThread())
			{
				PermutationVector.Set<FUsePrecalculatedLuminanceDim>(true);
			}
			else
			{
				PermutationVector.Set<FUseApproxIlluminanceDim>(true);
			}
		}

		PermutationVector.Set<FUseDebugOutputDim>(CVarAutoExposureIgnoreMaterialsDebug.GetValueOnRenderThread());

		return PermutationVector;
	}
} // namespace TonemapperPermutation

// Basic eye adaptation is supported everywhere except mobile when MobileHDR is disabled
static ERHIFeatureLevel::Type GetBasicEyeAdaptationMinFeatureLevel()
{
	return IsMobileHDR() ? ERHIFeatureLevel::ES3_1 : ERHIFeatureLevel::SM5;
}

bool IsAutoExposureMethodSupported(ERHIFeatureLevel::Type FeatureLevel, EAutoExposureMethod AutoExposureMethodId)
{
	switch (AutoExposureMethodId)
	{
	case EAutoExposureMethod::AEM_Histogram:
	case EAutoExposureMethod::AEM_Basic:
		return FeatureLevel > ERHIFeatureLevel::ES3_1 || IsMobileHDR();
	case EAutoExposureMethod::AEM_Manual:
		return true;
	}
	return false;
}

bool IsExtendLuminanceRangeEnabled()
{
	static const auto VarDefaultAutoExposureExtendDefaultLuminanceRange = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultFeature.AutoExposure.ExtendDefaultLuminanceRange"));

	return VarDefaultAutoExposureExtendDefaultLuminanceRange->GetValueOnRenderThread() == 1;
}

bool IsAutoExposureUsingIlluminanceEnabled(const FViewInfo& View)
{
	// NOTE: This method cannot be supported with PathTracing because it requires a GBuffer which is not available in the path tracing case
	return CVarAutoExposureIgnoreMaterials.GetValueOnRenderThread() && GetAutoExposureMethod(View) == AEM_Histogram && !View.Family->EngineShowFlags.PathTracing;
}

int32 GetAutoExposureIlluminanceDownscaleFactor()
{
	int32 DownscaleFactor = CVarAutoExposureIgnoreMaterialsDownscaleFactor.GetValueOnRenderThread();

	return FMath::Clamp(DownscaleFactor, 1, 16);
}

static EAutoExposureMethod ApplyEyeAdaptationQuality(EAutoExposureMethod AutoExposureMethod)
{
	static const auto CVarEyeAdaptationQuality = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.EyeAdaptationQuality"));
	const int32 EyeAdaptationQuality = CVarEyeAdaptationQuality->GetValueOnRenderThread();
	
	if (AutoExposureMethod != EAutoExposureMethod::AEM_Manual)
	{
		if (EyeAdaptationQuality == 1)
		{
			// Clamp current method to AEM_Basic
			AutoExposureMethod = EAutoExposureMethod::AEM_Basic;
		}
	}

	return AutoExposureMethod;
}

float LuminanceMaxFromLensAttenuation()
{
	const bool bExtendedLuminanceRange = IsExtendLuminanceRangeEnabled();

	float LensAttenuation = CVarEyeAdaptationLensAttenuation.GetValueOnRenderThread();

	// 78 is defined in the ISO 12232:2006 standard.
	const float kISOSaturationSpeedConstant = 0.78f;

	const float LuminanceMax = kISOSaturationSpeedConstant / FMath::Max<float>(LensAttenuation, .01f);

	// if we do not have luminance range extended, the math is hardcoded to 1.0 scale.
	return bExtendedLuminanceRange ? LuminanceMax : 1.0f;
}

// Query the view for the auto exposure method, and allow for CVar override.
EAutoExposureMethod GetAutoExposureMethod(const FViewInfo& View)
{
	EAutoExposureMethod AutoExposureMethod = View.FinalPostProcessSettings.AutoExposureMethod;

	// Fallback to basic (or manual) if the requested mode is not supported by the feature level.
	if (!IsAutoExposureMethodSupported(View.GetFeatureLevel(), AutoExposureMethod))
	{
		AutoExposureMethod = IsAutoExposureMethodSupported(View.GetFeatureLevel(), EAutoExposureMethod::AEM_Basic) ? EAutoExposureMethod::AEM_Basic : EAutoExposureMethod::AEM_Manual;
	}

	// Apply quality settings
	AutoExposureMethod = ApplyEyeAdaptationQuality(AutoExposureMethod);

	const int32 EyeOverride = CVarEyeAdaptationMethodOverride.GetValueOnRenderThread();

	EAutoExposureMethod OverrideAutoExposureMethod = AutoExposureMethod;

	if (EyeOverride >= 0)
	{
		// Additional branching for override.
		switch (EyeOverride)
		{
		case 1:
		{
			OverrideAutoExposureMethod = EAutoExposureMethod::AEM_Histogram;
			break;
		}
		case 2:
		{
			OverrideAutoExposureMethod = EAutoExposureMethod::AEM_Basic;
			break;
		}
		case 3:
		{
			OverrideAutoExposureMethod = EAutoExposureMethod::AEM_Manual;
			break;
		}
		}
	}

	if (IsAutoExposureMethodSupported(View.GetFeatureLevel(), OverrideAutoExposureMethod))
	{
		AutoExposureMethod = OverrideAutoExposureMethod;
	}

	// If auto exposure is disabled, revert to manual mode which will clamp to a reasonable default.
	if (!View.Family->EngineShowFlags.EyeAdaptation)
	{
		AutoExposureMethod = AEM_Manual;
	}

	return AutoExposureMethod;
}

float GetAutoExposureCompensationFromSettings(const FViewInfo& View)
{
	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

	// This scales the average luminance AFTER it gets clamped, affecting the exposure value directly.
	float AutoExposureBias = Settings.AutoExposureBias;

	// AutoExposureBias need to minus 1 if it is used for mobile LDR, because we don't go through the postprocess eye adaptation pass. 
	if (IsMobilePlatform(View.GetShaderPlatform()) && !IsMobileHDR())
	{
		AutoExposureBias = AutoExposureBias - 1.0f;
	}

	return FMath::Pow(2.0f, AutoExposureBias);
}


float GetAutoExposureCompensationFromCurve(const FViewInfo& View)
{
	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

	const float LuminanceMax = LuminanceMaxFromLensAttenuation();

	float AutoExposureBias = 0.0f;

	if (Settings.AutoExposureBiasCurve)
	{
		// Note that we are using View.GetLastAverageSceneLuminance() instead of the alternatives. GetLastAverageSceneLuminance()
		// immediately converges because it is calculated from the current frame's average luminance (without any history).
		// 
		// Note that when there is an abrupt change, there will be an immediate change in exposure compensation. But this is
		// fine because the shader will recalculate a new target exposure. The next result is that the smoothed exposure (purple
		// line in HDR visualization) will have sudden shifts, but the actual output exposure value (white line in HDR visualization)
		// will be smooth.
		const float AverageSceneLuminance = View.GetLastAverageSceneLuminance();

		if (AverageSceneLuminance > 0)
		{
			// We need the Log2(0.18) to convert from average luminance to saturation luminance
			const float LuminanceEV100 = LuminanceToEV100(LuminanceMax, AverageSceneLuminance) + FMath::Log2(1.0f / 0.18f);
			AutoExposureBias += Settings.AutoExposureBiasCurve->GetFloatValue(LuminanceEV100);
		}
	}

	return FMath::Pow(2.0f, AutoExposureBias);
}

bool IsAutoExposureDebugMode(const FViewInfo& View)
{
	const FEngineShowFlags& EngineShowFlags = View.Family->EngineShowFlags;

	return View.Family->UseDebugViewPS() ||
		!EngineShowFlags.Lighting ||
		(EngineShowFlags.VisualizeBuffer && View.CurrentBufferVisualizationMode != NAME_None && !GetBufferVisualizationData().GetMaterialApplyAutoExposure(View.CurrentBufferVisualizationMode)) ||
		
		// NOTE: Debug modes that are not tonemapped (such as normals, basecolor, etc ...) should not have exposure applied
		(EngineShowFlags.RayTracingDebug && !RaytracingDebugViewModeNeedsTonemapping(View)) ||
		(EngineShowFlags.PathTracing && !PathTracing::NeedsTonemapping()) ||

		EngineShowFlags.VisualizeDistanceFieldAO ||
		EngineShowFlags.VisualizeVolumetricCloudConservativeDensity ||
		EngineShowFlags.VisualizeVolumetricCloudEmptySpaceSkipping ||
		EngineShowFlags.CollisionVisibility ||
		EngineShowFlags.CollisionPawn ||
		!EngineShowFlags.PostProcessing;
}

float CalculateFixedAutoExposure(const FViewInfo& View)
{
	const float LuminanceMax = LuminanceMaxFromLensAttenuation();
	return EV100ToLuminance(LuminanceMax, View.Family->ExposureSettings.FixedEV100);
}

// on mobile, we are never using the Physical Camera, which is why we need the bForceDisablePhysicalCamera
float CalculateManualAutoExposure(const FViewInfo& View, bool bForceDisablePhysicalCamera)
{
	const float LuminanceMax = LuminanceMaxFromLensAttenuation();

	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

	const float BasePhysicalCameraEV100 = FMath::Log2(FMath::Square(Settings.DepthOfFieldFstop) * Settings.CameraShutterSpeed * 100 / FMath::Max(1.f, Settings.CameraISO));
	const float PhysicalCameraEV100 = (!bForceDisablePhysicalCamera && Settings.AutoExposureApplyPhysicalCameraExposure) ? BasePhysicalCameraEV100 : 0.0f;

	float FoundLuminance = EV100ToLuminance(LuminanceMax, PhysicalCameraEV100);
	return FoundLuminance;
}

FVector2f GetExposureCompensationCurveLUTScaleBias()
{
	const float MinX = CVarEyeAdaptationExposureCompensationCurveLUTMinX.GetValueOnRenderThread();
	const float MaxX = CVarEyeAdaptationExposureCompensationCurveLUTMaxX.GetValueOnRenderThread();
	const int32 NumSamples = FMath::Max(1, CVarEyeAdaptationExposureCompensationCurveLUTNumSamples.GetValueOnRenderThread());

	const float Scale = (NumSamples - 1) / (NumSamples * (MaxX - MinX));
	const float Bias = (0.5f / NumSamples) - (MinX * Scale);

	return FVector2f(Scale, Bias);
}

void BeginUpdateExposureCompensationCurveLUT(TArrayView<FViewInfo> Views, FUpdateExposureCompensationCurveLUTTaskData* TaskData)
{
	TaskData->Task = LaunchSceneRenderTask(TEXT("UpdateExposureCompensationCurveLUTs"),
		[TaskData, Views]
		{
			TaskData->CmdList = new FRHICommandList();
			TaskData->CmdList->SwitchPipeline(ERHIPipeline::Graphics);

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
			{
				const FViewInfo& View = Views[ViewIndex];
				if (View.ViewState != nullptr)
				{
					View.ViewState->UpdateExposureCompensationCurveLUT(*TaskData->CmdList, View.FinalPostProcessSettings.AutoExposureBiasCurve);
				}
			}

			TaskData->CmdList->FinishRecording();
		});
}

void FinishUpdateExposureCompensationCurveLUT(FRHICommandListImmediate& RHICmdList, FUpdateExposureCompensationCurveLUTTaskData* TaskData)
{
	if (TaskData->Task.IsValid())
	{
		TaskData->Task.Wait();
		TaskData->Task = {};

		RHICmdList.QueueAsyncCommandListSubmit(TaskData->CmdList);
		TaskData->CmdList = nullptr;
	}
}

static bool IsCurveLUTUsed(const FViewInfo& View)
{
	const bool IsDebugViewMode = IsAutoExposureDebugMode(View);

	const bool bUseCurveLUT =
		View.FinalPostProcessSettings.AutoExposureBiasCurve != nullptr
		&& View.ViewState != nullptr
		&& View.ViewState->GetExposureCompensationCurveLUT() != nullptr
		&& CVarEyeAdaptationExposureCompensationCurveLUT.GetValueOnRenderThread()
		&& !IsDebugViewMode
		&& !View.Family->ExposureSettings.bFixed;

	return bUseCurveLUT;
}

FEyeAdaptationScalarParameters GetEyeAdaptationScalarParameters(const FViewInfo& View)
{
	const bool bExtendedLuminanceRange = IsExtendLuminanceRangeEnabled();

	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

	const FEngineShowFlags& EngineShowFlags = View.Family->EngineShowFlags;

	const EAutoExposureMethod AutoExposureMethod = GetAutoExposureMethod(View);

	const float LuminanceMax = LuminanceMaxFromLensAttenuation();

	const float PercentToScale = 0.01f;

	const float ExposureHighPercent = FMath::Clamp(Settings.AutoExposureHighPercent, 1.0f, 99.0f) * PercentToScale;
	const float ExposureLowPercent = FMath::Min(FMath::Clamp(Settings.AutoExposureLowPercent, 1.0f, 99.0f) * PercentToScale, ExposureHighPercent);

	const float HistogramLogMax = bExtendedLuminanceRange ? EV100ToLog2(LuminanceMax, Settings.HistogramLogMax) : Settings.HistogramLogMax;
	const float HistogramLogMin = FMath::Min(bExtendedLuminanceRange ? EV100ToLog2(LuminanceMax, Settings.HistogramLogMin) : Settings.HistogramLogMin, HistogramLogMax - 1);

	// These clamp the average luminance computed from the scene color. We are going to calculate the white point first, and then
	// figure out the average grey point later. I.e. if the white point is 1.0, the middle grey point should be 0.18.
	float MinWhitePointLuminance = 1.0f;
	float MaxWhitePointLuminance = 1.0f;

	// Get the exposure compensation from the post process volume settings (everything except the curve)
	float ExposureCompensationSettings = GetAutoExposureCompensationFromSettings(View);

	const float BlackHistogramBucketInfluence = CVarEyeAdaptationBlackHistogramBucketInfluence.GetValueOnRenderThread();

	const float kMiddleGrey = 0.18f;

	// AEM_Histogram and AEM_Basic adjust their ExposureCompensation to middle grey (0.18). AEM_Manual ExposureCompensation is already calibrated to 1.0.
	const float GreyMult = (AutoExposureMethod == AEM_Manual) ? 1.0f : kMiddleGrey;

	const bool IsDebugViewMode = IsAutoExposureDebugMode(View);

	const bool bUseCurveLUT = IsCurveLUTUsed(View);

	// Get the exposure compensation from the curve
	float ExposureCompensationCurve = bUseCurveLUT ? 1.0f : GetAutoExposureCompensationFromCurve(View);

	if (IsDebugViewMode)
	{
		ExposureCompensationSettings = 1.0f;
		ExposureCompensationCurve = 1.0f;
	}
	// Fixed exposure override in effect.
	else if (View.Family->ExposureSettings.bFixed)
	{
		ExposureCompensationSettings = 1.0f;
		ExposureCompensationCurve = 1.0f;

		// ignores bExtendedLuminanceRange
		MinWhitePointLuminance = MaxWhitePointLuminance = CalculateFixedAutoExposure(View);
	}
	// The feature level check should always pass unless on mobile with MobileHDR is false
	else if (EngineShowFlags.EyeAdaptation && View.GetFeatureLevel() >= GetBasicEyeAdaptationMinFeatureLevel())
	{
		if (AutoExposureMethod == EAutoExposureMethod::AEM_Manual)
		{
			// ignores bExtendedLuminanceRange
			MinWhitePointLuminance = MaxWhitePointLuminance = CalculateManualAutoExposure(View, false);
		}
		else
		{
			if (bExtendedLuminanceRange)
			{
				MinWhitePointLuminance = EV100ToLuminance(LuminanceMax, Settings.AutoExposureMinBrightness);
				MaxWhitePointLuminance = EV100ToLuminance(LuminanceMax, Settings.AutoExposureMaxBrightness);
			}
			else
			{
				MinWhitePointLuminance = Settings.AutoExposureMinBrightness;
				MaxWhitePointLuminance = Settings.AutoExposureMaxBrightness;
			}
		}
	}
	else
	{
		// if eye adaptation is off, then set everything to 1.0
		ExposureCompensationSettings = 1.0f;
		ExposureCompensationCurve = 1.0f;

		// GetAutoExposureMethod() should return Manual in this case.
		check(AutoExposureMethod == AEM_Manual);

		// just lock to 1.0, it's not possible to guess a reasonable value using the min and max.
		MinWhitePointLuminance = MaxWhitePointLuminance = 1.0;
	}

	MinWhitePointLuminance = FMath::Min(MinWhitePointLuminance, MaxWhitePointLuminance);

	const float HistogramLogDelta = HistogramLogMax - HistogramLogMin;
	const float HistogramScale = 1.0f / HistogramLogDelta;
	const float HistogramBias = -HistogramLogMin * HistogramScale;

	// If we are in histogram mode, then we want to set the minimum to the bottom end of the histogram. But if we are in basic mode,
	// we want to simply use a small epsilon to keep true black values from returning a NaN and/or a very low value. Also, basic
	// mode does the calculation in pre-exposure space, which is why we need to multiply by View.PreExposure.
	const float LuminanceMin = (AutoExposureMethod == AEM_Basic) ? 0.0001f : FMath::Exp2(HistogramLogMin);

	// The distance at which we switch from linear to exponential. I.e. at StartDistance=1.5, when linear is 1.5 f-stops away from hitting the 
	// target, we switch to exponential.
	const float StartDistance = CVarEyeAdaptationExponentialTransitionDistance.GetValueOnRenderThread();
	const float StartTimeUp = StartDistance / FMath::Max(Settings.AutoExposureSpeedUp, 0.001f);
	const float StartTimeDown = StartDistance / FMath::Max(Settings.AutoExposureSpeedDown, 0.001f);

	// We want to ensure that at time=StartT, that the derivative of the exponential curve is the same as the derivative of the linear curve.
	// For the linear curve, the step will be AdaptationSpeed * FrameTime.
	// For the exponential curve, the step will be at t=StartT, M is slope modifier:
	//      slope(t) = M * (1.0f - exp2(-FrameTime * AdaptionSpeed)) * AdaptionSpeed * StartT
	//      AdaptionSpeed * FrameTime = M * (1.0f - exp2(-FrameTime * AdaptionSpeed)) * AdaptationSpeed * StartT
	//      M = FrameTime / (1.0f - exp2(-FrameTime * AdaptionSpeed)) * StartT
	//
	// To be technically correct, we should take the limit as FrameTime->0, but for simplicity we can make FrameTime a small number. So:
	const float kFrameTimeEps = 1.0f / 60.0f;
	const float ExponentialUpM = kFrameTimeEps / ((1.0f - exp2(-kFrameTimeEps * Settings.AutoExposureSpeedUp)) * StartTimeUp);
	const float ExponentialDownM = kFrameTimeEps / ((1.0f - exp2(-kFrameTimeEps * Settings.AutoExposureSpeedDown)) * StartTimeDown);

	// If the white point luminance is 1.0, then the middle grey luminance should be 0.18.
	const float MinAverageLuminance = MinWhitePointLuminance * kMiddleGrey;
	const float MaxAverageLuminance = MaxWhitePointLuminance * kMiddleGrey;

	const bool bValidRange = View.FinalPostProcessSettings.AutoExposureMinBrightness < View.FinalPostProcessSettings.AutoExposureMaxBrightness;
	const bool bValidSpeeds = View.FinalPostProcessSettings.AutoExposureSpeedDown >= 0.f && View.FinalPostProcessSettings.AutoExposureSpeedUp >= 0.f;

	// if it is a camera cut we force the exposure to go all the way to the target exposure without blending.
	// if it is manual mode, we also force the exposure to hit the target, which matters for HDR Visualization
	// if we don't have a valid range (AutoExposureMinBrightness == AutoExposureMaxBrightness) then force it like Manual as well.
	const float ForceTarget = (View.bCameraCut || View.bPrevTransformsReset || AutoExposureMethod == EAutoExposureMethod::AEM_Manual || !bValidRange || !bValidSpeeds) ? 1.0f : 0.0f;

	FVector3f LuminanceWeights;
	const int32 LuminanceMethod = CVarAutoExposureLuminanceMethod.GetValueOnRenderThread();
	if (LuminanceMethod == 1)
	{
		// NTSC (deprecated legacy weights in Common.ush)
		LuminanceWeights = FVector3f(0.3f, 0.59f, 0.11f);
	}
	else if (LuminanceMethod == 2)
	{
		// Working color space (sRGB/Rec709 default)
		LuminanceWeights = FVector3f(UE::Color::FColorSpace::GetWorking().GetLuminanceFactors());
	}
	else
	{
		// default (uniform weights)
		LuminanceWeights = FVector3f(1.0f / 3.0f);
	}

	FEyeAdaptationScalarParameters Parameters;
	Parameters.ExposureLowPercent = ExposureLowPercent;
	Parameters.ExposureHighPercent = ExposureHighPercent;
	Parameters.MinAverageLuminance = MinAverageLuminance;
	Parameters.MaxAverageLuminance = MaxAverageLuminance;
	Parameters.ExposureCompensationSettings = ExposureCompensationSettings;
	Parameters.ExposureCompensationCurve = ExposureCompensationCurve;
	Parameters.DeltaWorldTime = View.Family->Time.GetDeltaWorldTimeSeconds();
	Parameters.ExposureSpeedUp = Settings.AutoExposureSpeedUp;
	Parameters.ExposureSpeedDown = Settings.AutoExposureSpeedDown;
	Parameters.HistogramScale = HistogramScale;
	Parameters.HistogramBias = HistogramBias;
	Parameters.LuminanceMin = LuminanceMin;
	Parameters.BlackHistogramBucketInfluence = BlackHistogramBucketInfluence; // no calibration constant because it is now baked into ExposureCompensation
	Parameters.GreyMult = GreyMult;
	Parameters.ExponentialDownM = ExponentialDownM;
	Parameters.ExponentialUpM = ExponentialUpM;
	Parameters.StartDistance = StartDistance;
	Parameters.LuminanceMax = LuminanceMax;
	Parameters.IgnoreMaterialsEvaluationPositionBias = CVarAutoExposureIgnoreMaterialsEvaluationPositionBias.GetValueOnRenderThread();
	Parameters.IgnoreMaterialsLuminanceScale = CVarAutoExposureIgnoreMaterialsLuminanceScale.GetValueOnRenderThread();
	Parameters.IgnoreMaterialsMinBaseColorLuminance = CVarAutoExposureIgnoreMaterialsMinBaseColorLuminance.GetValueOnRenderThread();
	Parameters.IgnoreMaterialsReconstructFromSceneColor = CVarAutoExposureIgnoreMaterialsReconstructFromSceneColor.GetValueOnRenderThread();
	Parameters.ForceTarget = ForceTarget;
	Parameters.VisualizeDebugType = CVarEyeAdaptationVisualizeDebugType.GetValueOnRenderThread();
	Parameters.EV100ToExposureCompensationCurveLUTScaleBias = GetExposureCompensationCurveLUTScaleBias();
	Parameters.LuminanceWeights = LuminanceWeights;

	return Parameters;
}

FEyeAdaptationParameters GetEyeAdaptationParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	FEyeAdaptationScalarParameters ScalarParameters = GetEyeAdaptationScalarParameters(View);

	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

	const FEngineShowFlags& EngineShowFlags = View.Family->EngineShowFlags;

	const bool IsDebugViewMode = IsAutoExposureDebugMode(View);

	const bool bUseCurveLUT = IsCurveLUTUsed(View);

	FRDGTextureRef ExposureCompensationCurveLUT = GSystemTextures.GetWhiteDummy(GraphBuilder);
	if (bUseCurveLUT)
	{
		ExposureCompensationCurveLUT = RegisterExternalTexture(GraphBuilder, View.ViewState->GetExposureCompensationCurveLUT(), TEXT("ExposureCompensationCurveLUT"));
	}

	FRDGTextureRef MeterMask = GSystemTextures.GetWhiteDummy(GraphBuilder);
	if (Settings.AutoExposureMeterMask &&
		Settings.AutoExposureMeterMask->GetResource() &&
		Settings.AutoExposureMeterMask->GetResource()->TextureRHI)
	{
		MeterMask = RegisterExternalTexture(GraphBuilder, Settings.AutoExposureMeterMask->GetResource()->TextureRHI, TEXT("MeterMask"));
	}

	FEyeAdaptationParameters Parameters;
	Parameters.Scalars = ScalarParameters;
	Parameters.ExposureCompensationCurveLUT = ExposureCompensationCurveLUT;
	Parameters.ExposureCompensationCurveSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters.MeterMaskTexture = MeterMask;
	Parameters.MeterMaskSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	return Parameters;
}

float GetEyeAdaptationFixedExposure(const FViewInfo& View)
{
	const FEyeAdaptationScalarParameters Parameters = GetEyeAdaptationScalarParameters(View);

	const float Exposure = (Parameters.MinAverageLuminance + Parameters.MaxAverageLuminance) * 0.5f;

	const float kMiddleGrey = 0.18f;
	const float ExposureScale = kMiddleGrey / FMath::Max(0.0001f, Exposure);

	// We're ignoring any curve influence
	return ExposureScale * Parameters.ExposureCompensationSettings;
}

static FTextureRHIRef CreateCurveLUT(FRHICommandList& RHICmdList, UCurveFloat* Curve, float MinX, float MaxX, int32 NumSamples)
{
	TArray<FFloat16> Data;
	Data.SetNumUninitialized(NumSamples);

	const float Delta = (MaxX - MinX) / (NumSamples - 1);

	for (int32 Index = 0; Index < NumSamples; ++Index)
	{
		const float LuminanceEV100 = MinX + Delta * Index;

		const float Bias = Curve->GetFloatValue(LuminanceEV100);

		FFloat16& Value = Data[Index];
		Value = FMath::Pow(2.0f, Bias);
	}

	FResourceBulkDataArrayView BulkDataInterface(Data);

	const FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create2D(TEXT("ExposureCompensationCurveLUT"), NumSamples, 1, PF_R16F)
		.SetFlags(ETextureCreateFlags::ShaderResource)
		.SetInitActionBulkData(&BulkDataInterface);

	return RHICmdList.CreateTexture(Desc);
}

static uint32 GlobalExposureCompensationCurveLUTVersion = 0;

static void BumpGlobalExposureCompensationCurveLUTVersion()
{
	ENQUEUE_RENDER_COMMAND(BumpExposureCompensationCurveLUTVersionCommand)(
		[](FRHICommandListImmediate& RHICmdList)
		{
			++GlobalExposureCompensationCurveLUTVersion;
		});
}

void FSceneViewState::SetExposureCompensationCurve(UCurveFloat* NewCurve)
{
	check(IsInGameThread());

	// Use OnUpdateCurve delegate to get notified about changes to the curves.
	// At the moment the render thread is informed that curve LUTs need to be update by bumping a global version number.
	// TODO: could setup a more fine grained invalidation mechanism.

#if WITH_EDITORONLY_DATA
	if (!CVarEyeAdaptationExposureCompensationCurveLUT.GetValueOnAnyThread())
	{
		NewCurve = nullptr;
	}

	if (ExposureCompensationCurveLUT.Curve_GameThread != NewCurve)
	{
		if (ExposureCompensationCurveLUT.Curve_GameThread.IsValid()
			&& ExposureCompensationCurveLUT.CurveUpdateDelegateHandle.IsValid())
		{
			ExposureCompensationCurveLUT.Curve_GameThread->OnUpdateCurve.Remove(ExposureCompensationCurveLUT.CurveUpdateDelegateHandle);
			ExposureCompensationCurveLUT.CurveUpdateDelegateHandle.Reset();
		}

		ExposureCompensationCurveLUT.Curve_GameThread = NewCurve;

		if (ExposureCompensationCurveLUT.Curve_GameThread.IsValid())
		{
			ExposureCompensationCurveLUT.CurveUpdateDelegateHandle = ExposureCompensationCurveLUT.Curve_GameThread->OnUpdateCurve.AddLambda([](UCurveBase* /*Curve*/, EPropertyChangeType::Type /*ChangeType*/) { BumpGlobalExposureCompensationCurveLUTVersion(); });
		}
	}
#endif
}

void FSceneViewState::UpdateExposureCompensationCurveLUT(FRHICommandList& RHICmdList, UCurveFloat* Curve)
{
	if (!CVarEyeAdaptationExposureCompensationCurveLUT.GetValueOnRenderThread())
	{
		Curve = nullptr;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateExposureCompensationCurveLUT);

	const float MinX = CVarEyeAdaptationExposureCompensationCurveLUTMinX.GetValueOnRenderThread();
	const float MaxX = CVarEyeAdaptationExposureCompensationCurveLUTMaxX.GetValueOnRenderThread();
	const int32 NumSamples = FMath::Max(1, CVarEyeAdaptationExposureCompensationCurveLUTNumSamples.GetValueOnRenderThread());

	bool bNeedToUpdate = false;
	bNeedToUpdate |= (ExposureCompensationCurveLUT.Texture == nullptr || ExposureCompensationCurveLUT.Texture->GetSizeX() != NumSamples);
	bNeedToUpdate |= (ExposureCompensationCurveLUT.Curve != Curve);
#if WITH_EDITORONLY_DATA
	bNeedToUpdate |= (ExposureCompensationCurveLUT.Version != GlobalExposureCompensationCurveLUTVersion);
#endif
	
	if (Curve != nullptr && bNeedToUpdate)
	{
		ExposureCompensationCurveLUT.Texture = CreateCurveLUT(RHICmdList, Curve, MinX, MaxX, NumSamples);

#if WITH_EDITORONLY_DATA
		ExposureCompensationCurveLUT.Version = GlobalExposureCompensationCurveLUTVersion;
#endif
	}
	else if (Curve == nullptr)
	{
		ExposureCompensationCurveLUT.Texture = nullptr;
	}

	ExposureCompensationCurveLUT.Curve = Curve;
}

//////////////////////////////////////////////////////////////////////////
//! CopyEyeAdaptationToTexture
//////////////////////////////////////////////////////////////////////////

class FCopyEyeAdaptationToTextureCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyEyeAdaptationToTextureCS);
	SHADER_USE_PARAMETER_STRUCT(FCopyEyeAdaptationToTextureCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWEyeAdaptationTexture)
		END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCopyEyeAdaptationToTextureCS, "/Engine/Private/PostProcessEyeAdaptationUtils.usf", "CopyEyeAdaptationToTextureCS", SF_Compute);

void AddCopyEyeAdaptationDataToTexturePass(FRDGBuilder& GraphBuilder, const FGlobalShaderMap* ShaderMap, FRDGBufferRef EyeAdaptationBuffer, FRDGTextureRef OutputTexture)
{
	auto* PassParameters = GraphBuilder.AllocParameters<FCopyEyeAdaptationToTextureCS::FParameters>();
	PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(EyeAdaptationBuffer);
	PassParameters->RWEyeAdaptationTexture = GraphBuilder.CreateUAV(OutputTexture);

	auto ComputeShader = ShaderMap->GetShader<FCopyEyeAdaptationToTextureCS>();

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("CopyEyeAdaptationToTexture (CS)"),
		ComputeShader,
		PassParameters,
		FIntVector(1, 1, 1));
}

FRDGTextureRef AddCopyEyeAdaptationDataToTexturePass(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_A32B32G32R32F, FClearValueBinding::None, TexCreate_UAV | TexCreate_ShaderResource);
	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("EyeAdaptationTexture"));

	AddCopyEyeAdaptationDataToTexturePass(GraphBuilder, View.ShaderMap, GetEyeAdaptationBuffer(GraphBuilder, View), OutputTexture);

	return OutputTexture;
}

//////////////////////////////////////////////////////////////////////////
//! Setup
//////////////////////////////////////////////////////////////////////////

class FSetupExposureIlluminanceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSetupExposureIlluminanceCS);
	SHADER_USE_PARAMETER_STRUCT(FSetupExposureIlluminanceCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWIlluminanceTexture)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Illuminance)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER(FScreenTransform, IlluminanceRectToColorRect)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSetupExposureIlluminanceCS, "/Engine/Private/PostProcessEyeAdaptation.usf", "SetupExposureIlluminance", SF_Compute);

FRDGTextureRef AddSetupExposureIlluminancePass(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FSceneTextures& SceneTextures)
{
	const uint32 ViewCount = Views.Num();

	bool bAnyViewEnabled = false;

	for (uint32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
	{
		bAnyViewEnabled |= IsAutoExposureUsingIlluminanceEnabled(Views[ViewIndex]) && CVarAutoExposureIgnoreMaterialsUsePrecalculatedIlluminance.GetValueOnRenderThread();
	}

	if (!bAnyViewEnabled)
	{
		return nullptr;
	}

	const FIntPoint OutputExtent = GetDownscaledExtent(SceneTextures.Config.Extent, GetAutoExposureIlluminanceDownscaleFactor());

	const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(OutputExtent, PF_R16F, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource);
	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(Desc, TEXT("EyeAdaptation_Illuminance"));

	for (uint32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (IsAutoExposureUsingIlluminanceEnabled(View) && CVarAutoExposureIgnoreMaterialsUsePrecalculatedIlluminance.GetValueOnRenderThread())
		{
			const FScreenPassTextureViewport SceneViewport(SceneTextures.Config.Extent, View.ViewRect);
			const FScreenPassTextureViewport OutputViewport(GetDownscaledViewport(SceneViewport, GetAutoExposureIlluminanceDownscaleFactor()));

			auto* PassParameters = GraphBuilder.AllocParameters<FSetupExposureIlluminanceCS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
			PassParameters->ColorTexture = SceneTextures.Color.Resolve;
			PassParameters->RWIlluminanceTexture = GraphBuilder.CreateUAV(OutputTexture);
			PassParameters->Illuminance = GetScreenPassTextureViewportParameters(OutputViewport);
			PassParameters->EyeAdaptation = GetEyeAdaptationParameters(GraphBuilder, View);
			PassParameters->IlluminanceRectToColorRect = FScreenTransform::ChangeRectFromTo(OutputViewport.Rect, SceneViewport.Rect);

			auto ComputeShader = View.ShaderMap->GetShader<FSetupExposureIlluminanceCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SetupExposureIlluminance (ViewId=%d)", ViewIndex),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), FIntPoint(8, 8)));
		}
	}

	return OutputTexture;
}

class FCalculateExposureIlluminanceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCalculateExposureIlluminanceCS);
	SHADER_USE_PARAMETER_STRUCT(FCalculateExposureIlluminanceCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWIlluminanceTexture)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Illuminance)
		SHADER_PARAMETER(FScreenTransform, IlluminanceRectToColorRect)

		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)

		SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
		SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)

		SHADER_PARAMETER_STRUCT_INCLUDE(FTranslucencyLightingVolumeParameters, TranslucencyLightingVolume)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLumenTranslucencyLightingUniforms, LumenGIVolumeStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCalculateExposureIlluminanceCS, "/Engine/Private/PostProcessEyeAdaptation.usf", "CalculateExposureIlluminance", SF_Compute);

FRDGTextureRef AddCalculateExposureIlluminancePass(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FSceneTextures& SceneTextures,
	const FTranslucencyLightingVolumeTextures& TranslucencyLightingVolumeTextures,
	FRDGTextureRef ExposureIlluminanceSetup)
{
	if (!ExposureIlluminanceSetup)
	{
		return nullptr;
	}

	const uint32 ViewCount = Views.Num();

	for (uint32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (IsAutoExposureUsingIlluminanceEnabled(View))
		{
			check(CVarAutoExposureIgnoreMaterialsUsePrecalculatedIlluminance.GetValueOnRenderThread());

			const FScreenPassTextureViewport SceneViewport(SceneTextures.Config.Extent, View.ViewRect);
			const FScreenPassTextureViewport OutputViewport(GetDownscaledViewport(SceneViewport, GetAutoExposureIlluminanceDownscaleFactor()));

			auto* PassParameters = GraphBuilder.AllocParameters<FCalculateExposureIlluminanceCS::FParameters>();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->SceneTextures = GetSceneTextureParameters(GraphBuilder, SceneTextures);
			PassParameters->ColorTexture = SceneTextures.Color.Resolve;
			PassParameters->RWIlluminanceTexture = GraphBuilder.CreateUAV(ExposureIlluminanceSetup);
			PassParameters->Illuminance = GetScreenPassTextureViewportParameters(OutputViewport);
			PassParameters->IlluminanceRectToColorRect = FScreenTransform::ChangeRectFromTo(OutputViewport.Rect, SceneViewport.Rect);
			PassParameters->EyeAdaptation = GetEyeAdaptationParameters(GraphBuilder, View);

			PassParameters->PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
			PassParameters->PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			PassParameters->TranslucencyLightingVolume = GetTranslucencyLightingVolumeParameters(GraphBuilder, TranslucencyLightingVolumeTextures, View);

			auto* LumenUniforms = GraphBuilder.AllocParameters<FLumenTranslucencyLightingUniforms>();
			LumenUniforms->Parameters = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());
			PassParameters->LumenGIVolumeStruct = GraphBuilder.CreateUniformBuffer(LumenUniforms);
			PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

			auto ComputeShader = View.ShaderMap->GetShader<FCalculateExposureIlluminanceCS>();

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("CalculateExposureIlluminance (ViewId=%d)", ViewIndex),
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), FIntPoint(8, 8)));
		}
	}

	return ExposureIlluminanceSetup;
}

//////////////////////////////////////////////////////////////////////////
//! Histogram Eye Adaptation
//////////////////////////////////////////////////////////////////////////

class FEyeAdaptationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FEyeAdaptationCS);
	SHADER_USE_PARAMETER_STRUCT(FEyeAdaptationCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_STRUCT(FLocalExposureParameters, LocalExposure)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HistogramTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, RWEyeAdaptationBuffer)
	END_SHADER_PARAMETER_STRUCT()
		
	class FComputeAverageLocalExposure : SHADER_PERMUTATION_BOOL("COMPUTE_AVERAGE_LOCAL_EXPOSURE");
	using FPermutationDomain = TShaderPermutationDomain<FComputeAverageLocalExposure>;

	static const EPixelFormat OutputFormat = PF_A32B32G32R32F;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FEyeAdaptationCS, "/Engine/Private/PostProcessEyeAdaptation.usf", "EyeAdaptationCS", SF_Compute);

FRDGBufferRef AddHistogramEyeAdaptationPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	const FLocalExposureParameters& LocalExposureParameters,
	FRDGTextureRef HistogramTexture,
	bool bComputeAverageLocalExposure)
{
	bool bUpdateAdaptationBuffer = View.ShouldUpdateEyeAdaptationBuffer();

	FRDGBufferRef OutputBuffer;

	if (bUpdateAdaptationBuffer)
	{
		View.UpdateEyeAdaptationLastExposureFromBuffer();
		View.SwapEyeAdaptationBuffers();

		OutputBuffer = GraphBuilder.RegisterExternalBuffer(View.GetEyeAdaptationBuffer(GraphBuilder), ERDGBufferFlags::MultiFrame);

		FEyeAdaptationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FEyeAdaptationCS::FParameters>();
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->LocalExposure = LocalExposureParameters;
		PassParameters->HistogramTexture = HistogramTexture;
		PassParameters->RWEyeAdaptationBuffer = GraphBuilder.CreateUAV(OutputBuffer);

		FEyeAdaptationCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FEyeAdaptationCS::FComputeAverageLocalExposure>(bComputeAverageLocalExposure);

		auto ComputeShader = View.ShaderMap->GetShader<FEyeAdaptationCS>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HistogramEyeAdaptation (CS)"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1));

		View.EnqueueEyeAdaptationExposureBufferReadback(GraphBuilder);
	}
	else
	{
		// Return pre-existing eye adaptation buffer
		OutputBuffer = GraphBuilder.RegisterExternalBuffer(View.GetEyeAdaptationBuffer(), ERDGBufferFlags::MultiFrame);
	}

	return OutputBuffer;
}

//////////////////////////////////////////////////////////////////////////
//! Basic Eye Adaptation
//////////////////////////////////////////////////////////////////////////

/** Computes scaled and biased luma for the input scene color and puts it in the alpha channel. */
class FBasicEyeAdaptationSetupPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FBasicEyeAdaptationSetupPS);
	SHADER_USE_PARAMETER_STRUCT(FBasicEyeAdaptationSetupPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::Type(GetBasicEyeAdaptationMinFeatureLevel()));
	}
};

IMPLEMENT_GLOBAL_SHADER(FBasicEyeAdaptationSetupPS, "/Engine/Private/PostProcessEyeAdaptation.usf", "BasicEyeAdaptationSetupPS", SF_Pixel);

FScreenPassTexture AddBasicEyeAdaptationSetupPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FScreenPassTexture SceneColor)
{
	check(SceneColor.IsValid());

	FRDGTextureDesc OutputDesc = SceneColor.Texture->Desc;
	OutputDesc.Reset();
	// Require alpha channel for log2 information.
	OutputDesc.Format = PF_FloatRGBA;
	OutputDesc.Flags |= GFastVRamConfig.EyeAdaptation;

	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputDesc, TEXT("BasicEyeAdaptationSetup"));

	const FScreenPassTextureViewport Viewport(SceneColor);

	FBasicEyeAdaptationSetupPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBasicEyeAdaptationSetupPS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->EyeAdaptation = EyeAdaptationParameters;
	PassParameters->ColorTexture = SceneColor.Texture;
	PassParameters->ColorSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, View.GetOverwriteLoadAction());

	TShaderMapRef<FBasicEyeAdaptationSetupPS> PixelShader(View.ShaderMap);

	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("BasicEyeAdaptationSetup (PS) %dx%d", Viewport.Rect.Width(), Viewport.Rect.Height()),
		View,
		Viewport,
		Viewport,
		PixelShader,
		PassParameters,
		EScreenPassDrawFlags::AllowHMDHiddenAreaMask);

	return FScreenPassTexture(OutputTexture, SceneColor.ViewRect);
}

class FBasicEyeAdaptationCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FBasicEyeAdaptationCS);
	SHADER_USE_PARAMETER_STRUCT(FBasicEyeAdaptationCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_STRUCT(FLocalExposureParameters, LocalExposure)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, ColorTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, RWEyeAdaptationBuffer)
	END_SHADER_PARAMETER_STRUCT()
		
	class FComputeAverageLocalExposure : SHADER_PERMUTATION_BOOL("COMPUTE_AVERAGE_LOCAL_EXPOSURE");
	using FPermutationDomain = TShaderPermutationDomain<FComputeAverageLocalExposure>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FBasicEyeAdaptationCS, "/Engine/Private/PostProcessEyeAdaptation.usf", "BasicEyeAdaptationCS", SF_Compute);

FRDGBufferRef AddBasicEyeAdaptationPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	const FLocalExposureParameters& LocalExposureParameters,
	FScreenPassTextureSlice SceneColor,
	FRDGBufferRef EyeAdaptationBuffer,
	bool bComputeAverageLocalExposure)
{
	bool bUpdateAdaptationBuffer = View.ShouldUpdateEyeAdaptationBuffer();

	FRDGBufferRef OutputBuffer;

	// The owner of the adaptation buffer handles updates -- other views just read the results
	if (bUpdateAdaptationBuffer)
	{
		View.UpdateEyeAdaptationLastExposureFromBuffer();
		View.SwapEyeAdaptationBuffers();

		const FScreenPassTextureViewport SceneColorViewport(SceneColor);

		OutputBuffer = GraphBuilder.RegisterExternalBuffer(View.GetEyeAdaptationBuffer(GraphBuilder), ERDGBufferFlags::MultiFrame);

		FBasicEyeAdaptationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBasicEyeAdaptationCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->LocalExposure = LocalExposureParameters;
		PassParameters->Color = GetScreenPassTextureViewportParameters(SceneColorViewport);
		PassParameters->ColorTexture = SceneColor.TextureSRV;
		PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(EyeAdaptationBuffer);
		PassParameters->RWEyeAdaptationBuffer = GraphBuilder.CreateUAV(OutputBuffer);

		FBasicEyeAdaptationCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FBasicEyeAdaptationCS::FComputeAverageLocalExposure>(bComputeAverageLocalExposure);

		auto ComputeShader = View.ShaderMap->GetShader<FBasicEyeAdaptationCS>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("BasicEyeAdaptation (CS)"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1));

		View.EnqueueEyeAdaptationExposureBufferReadback(GraphBuilder);
	}
	else
	{
		// Return pre-existing eye adaptation buffer
		OutputBuffer = GraphBuilder.RegisterExternalBuffer(View.GetEyeAdaptationBuffer(), ERDGBufferFlags::MultiFrame);
	}

	return OutputBuffer;
}

FSceneViewState::FEyeAdaptationManager::FEyeAdaptationManager()
{
	ExposureReadbackBuffers.AddZeroed(MAX_READBACK_BUFFERS);
}

void FSceneViewState::FEyeAdaptationManager::SafeRelease()
{
	CurrentBufferIndex = 0;
	ReadbackBuffersWriteIndex = 0;
	ReadbackBuffersNumPending = 0;

	LastExposure = 0;
	LastAverageLocalExposure = 1.0f;
	LastAverageSceneLuminance = 0;

	for (int32 Index = 0; Index < NUM_BUFFERS; Index++)
	{
		ExposureBufferData[Index].SafeRelease();
	}

	for (int32 Index = 0; Index < ExposureReadbackBuffers.Num(); ++Index)
	{
		if (ExposureReadbackBuffers[Index])
		{
			delete ExposureReadbackBuffers[Index];
			ExposureReadbackBuffers[Index] = nullptr;
		}
	}
}

const TRefCountPtr<FRDGPooledBuffer>& FSceneViewState::FEyeAdaptationManager::GetBuffer(uint32 BufferIndex) const
{
	check(0 <= BufferIndex && BufferIndex < NUM_BUFFERS);

	return ExposureBufferData[BufferIndex];
}

const TRefCountPtr<FRDGPooledBuffer>& FSceneViewState::FEyeAdaptationManager::GetOrCreateBuffer(FRDGBuilder& GraphBuilder, uint32 BufferIndex)
{
	check(0 <= BufferIndex && BufferIndex < NUM_BUFFERS);

	// Create buffer if needed.
	if (!ExposureBufferData[BufferIndex].IsValid())
	{
		FRDGBufferDesc RDGBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FVector4f), EXPOSURE_BUFFER_SIZE_IN_VECTOR4);
		RDGBufferDesc.Usage |= BUF_SourceCopy;
		FRDGBufferRef RDGBuffer = GraphBuilder.CreateBuffer(RDGBufferDesc, TEXT("EyeAdaptationBuffer"), ERDGBufferFlags::MultiFrame);

		ExposureBufferData[BufferIndex] = GraphBuilder.ConvertToExternalBuffer(RDGBuffer);

		FVector4f* BufferData = (FVector4f*)GraphBuilder.RHICmdList.LockBuffer(ExposureBufferData[BufferIndex]->GetRHI(), 0, EXPOSURE_BUFFER_SIZE_IN_VECTOR4 * sizeof(FVector4f), RLM_WriteOnly);
		BufferData[0] = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
		BufferData[1] = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
		GraphBuilder.RHICmdList.UnlockBuffer(ExposureBufferData[BufferIndex]->GetRHI());
	}

	return ExposureBufferData[BufferIndex];
}

FRHIGPUBufferReadback* FSceneViewState::FEyeAdaptationManager::GetLatestReadbackBuffer()
{
	FRHIGPUBufferReadback* LatestReadbackBuffer = nullptr;

	// Find latest buffer that is ready
	while (ReadbackBuffersNumPending > 0)
	{
		uint32 Index = (ReadbackBuffersWriteIndex + MAX_READBACK_BUFFERS - ReadbackBuffersNumPending) % MAX_READBACK_BUFFERS;

		if (ExposureReadbackBuffers[Index]->IsReady())
		{
			--ReadbackBuffersNumPending;
			LatestReadbackBuffer = ExposureReadbackBuffers[Index];
		}
		else
		{
			break;
		}
	}

	return LatestReadbackBuffer;
}

void FSceneViewState::FEyeAdaptationManager::SwapBuffers()
{
	CurrentBufferIndex = (CurrentBufferIndex + 1) % NUM_BUFFERS;
}

void FSceneViewState::FEyeAdaptationManager::UpdateLastExposureFromBuffer()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FEyeAdaptationRTManager_ReadbackLastExposure);

	// Get the most up to date value
	FRHIGPUBufferReadback* ReadbackBuffer = GetLatestReadbackBuffer();

	if (ReadbackBuffer)
	{
		// Read the last request results.
		FVector4f* ReadbackData = (FVector4f*)ReadbackBuffer->Lock(EXPOSURE_BUFFER_SIZE_IN_VECTOR4 * sizeof(FVector4f));
		if (ReadbackData)
		{
			LastExposure = ReadbackData[0].X;
			LastAverageSceneLuminance = ReadbackData[0].Z;
			LastAverageLocalExposure = ReadbackData[1].X;

			ReadbackBuffer->Unlock();
		}
	}
}

void FSceneViewState::FEyeAdaptationManager::EnqueueExposureBufferReadback(FRDGBuilder& GraphBuilder)
{
	check(ExposureBufferData[CurrentBufferIndex].IsValid());

	if (ReadbackBuffersNumPending < MAX_READBACK_BUFFERS)
	{
		FRDGBufferRef CurrentRDGBuffer = GraphBuilder.RegisterExternalBuffer(ExposureBufferData[CurrentBufferIndex], ERDGBufferFlags::MultiFrame);

		FRHIGPUBufferReadback* ExposureReadbackBuffer = ExposureReadbackBuffers[ReadbackBuffersWriteIndex];

		if (ExposureReadbackBuffer == nullptr)
		{
			static const FName ExposureValueName("Scene view state exposure readback");
			ExposureReadbackBuffer = new FRHIGPUBufferReadback(ExposureValueName);
			ExposureReadbackBuffers[ReadbackBuffersWriteIndex] = ExposureReadbackBuffer;
		}

		AddEnqueueCopyPass(GraphBuilder, ExposureReadbackBuffer, CurrentRDGBuffer, 0);

		ReadbackBuffersWriteIndex = (ReadbackBuffersWriteIndex + 1) % MAX_READBACK_BUFFERS;
		ReadbackBuffersNumPending = FMath::Min(ReadbackBuffersNumPending + 1, MAX_READBACK_BUFFERS);
	}
}

void FViewInfo::UpdatePreExposure()
{
	const FSceneViewFamily& ViewFamily = *Family;

	const bool bMobilePlatform = IsMobilePlatform(GetShaderPlatform());
	const bool bEnableAutoExposure = !bMobilePlatform || IsMobileEyeAdaptationEnabled(*this);
	const float PreExposureOverride = CVarEyeAdaptationPreExposureOverride.GetValueOnRenderThread();
	const EAutoExposureMethod ExposureMethod = GetAutoExposureMethod(*this);
	const float FixedExposure = GetEyeAdaptationFixedExposure(*this);

	/** Whether PreExposure is supported at all for a output code path.
	 *  Mobile LDR renders directly into low bitdepth back buffer. So there the final exposure is directly applied in
	 *  BasePassPixelShader.usf, and ruse the View.PreExposure code path for code maintenance reason.
	 */
	const bool bSupportPreExposureDifferentThanGlobalExposure = !(bMobilePlatform && !IsMobileHDR());

	// One could use the IsRichView functionality to check if we need to update pre-exposure, 
	// but this is too limiting for certain view. For instance shader preview doesn't have 
	// volumetric lighting enabled, which makes the view be flagged as rich, and not updating 
	// the pre-exposition value.
	const bool bIsPreExposureRelevant = EyeAdaptation::IsPreExposureRelevant(*this);

	// Compute the PreExposure to use.
	bool bUpdateLastExposure = false;
	PreExposure = 1.f;
	if (!ViewState)
	{
		// NOP: UpdatePreExposure() was not being called if no view state.
		// TODO: should probably still do best effort on PreExposure when no view state is available, but might impact legacy behavior of USceneCaptureComponent that by default don't have a ViewState.
	}
	else if (!bSupportPreExposureDifferentThanGlobalExposure)
	{
		// Mobile LDR does not have post-processing and instead directly draws to back buffer. There want to apply the exposure as if it was PreExposure
		PreExposure = FixedExposure;
	}
	else if (!bIsPreExposureRelevant)
	{
		// NOP If the pre exposure isn't relevant for the final display image.
	}
	else if (PreExposureOverride > 0)
	{
		// If the pre-exposure is overriden by cvar, honor it.
		PreExposure = PreExposureOverride;
	}
	else
	{
		// How much the SceneColorTint changes the overall brightness of the image.
		// (prevent tint form being zero, because would trigger preExposure ensure)
		const float SceneColorTint = FMath::Max(FinalPostProcessSettings.SceneColorTint.GetLuminance(), 0.0001);

		// How much the vignette changes the overall brightness of the image. ComputeVignetteMask() always returns 1 at the center and dim
		// the edge of the screen.
		const float VignetteMask = 1.0;

		// How much the local exposure may change the overall image exposure when configured wrongly.
		const float LocalExposure = GetLastAverageLocalExposure();

		bool bUseFixedExposure = (ExposureMethod == AEM_Manual);
		if (!bUseFixedExposure)
		{
			// Mobile renderer skips EyeAdaptationBuffer update if there's no brightness range for the histogram
			bUseFixedExposure = bMobilePlatform &&
				(ExposureMethod == AEM_Histogram) &&
				(FinalPostProcessSettings.AutoExposureMinBrightness >= FinalPostProcessSettings.AutoExposureMaxBrightness);
		}

		// The global exposure of the scene regardless of the method used.
		float GlobalExposure = 1.0;
		if (bUseFixedExposure)
		{
			// Bypasses round trip CPU -> GPU -> CPU and instead directly use current frame's manual exposure.
			GlobalExposure = FixedExposure;
		}
		else
		{
			const float LastExposure = GetLastEyeAdaptationExposure();
			if (LastExposure > 0.0)
			{
				GlobalExposure = LastExposure;
			}
		}

		// This computation must match FinalLinearColor in PostProcessTonemap.usf.
		const float FinalPreExposure = SceneColorTint * GlobalExposure * VignetteMask * LocalExposure;
		ensure(FinalPreExposure > 0.0f);

		// Apply the computed PreExposure to view and view state.
		PreExposure = FinalPreExposure;
		bUpdateLastExposure = ViewState != nullptr; // TODO: technically not needed when ExposureMethod == AEM_Manual, unless there is transition from AEM_Manual to auto.
	}

	// Update the pre-exposure value on the view state
	if (ViewState)
	{
		// Update the PreExposure used on the view state
		ViewState->PreExposure = PreExposure;

		// Requests to read back last PreExposure back from GPU. 
		ViewState->bUpdateLastExposure = bUpdateLastExposure;

		// Update the pre exposure of all temporal histories.
		if (!bStatePrevViewInfoIsReadOnly)
		{
			ViewState->PrevFrameViewInfo.SceneColorPreExposure = PreExposure;
		}
	}
}
