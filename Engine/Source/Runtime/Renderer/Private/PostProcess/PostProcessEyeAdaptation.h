// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

class FLocalExposureParameters;
class FScene;
struct FPostProcessSettings;
enum EAutoExposureMethod : int;

namespace AutoExposurePermutation
{
	class FUsePrecalculatedLuminanceDim : SHADER_PERMUTATION_BOOL("USE_PRECALCULATED_LUMINANCE");
	class FUseApproxIlluminanceDim : SHADER_PERMUTATION_BOOL("USE_APPROX_ILLUMINANCE");
	class FUseDebugOutputDim : SHADER_PERMUTATION_BOOL("USE_DEBUG_OUTPUT");

	using FCommonDomain = TShaderPermutationDomain<FUsePrecalculatedLuminanceDim, FUseApproxIlluminanceDim, FUseDebugOutputDim>;

	bool ShouldCompileCommonPermutation(const FCommonDomain& PermutationVector);

	FCommonDomain BuildCommonPermutationDomain();
} // namespace AutoExposurePermutation

namespace EyeAdaptation
{
	float GetCachedLightingPreExposure();

	bool WriteWarnings(const FScene* Scene, const FViewInfo& View, FScreenMessageWriter* Writer);
}

// For converting the auto exposure to new values from 4.24 to 4.25
float CalculateEyeAdaptationParameterExposureConversion(const FPostProcessSettings& Settings, const bool bExtendedLuminanceRange);

// figure out the LuminanceMax (i.e. how much light in cd/m2 will saturate the camera sensor) from the CVar lens attenuation
float LuminanceMaxFromLensAttenuation();

// Returns whether the auto exposure method is supported by the feature level.
bool IsAutoExposureMethodSupported(ERHIFeatureLevel::Type FeatureLevel, EAutoExposureMethod AutoExposureMethodId);

// Returns true if the view is in a debug mode that disables all exposure
bool IsAutoExposureDebugMode(const FViewInfo& View);

// Returns the fixed exposure value
float CalculateFixedAutoExposure(const FViewInfo& View);

// Returns the manual exposure value
float CalculateManualAutoExposure(const FViewInfo& View, bool bForceDisablePhysicalCamera);

// Returns the exposure compensation from the View, 
float GetAutoExposureCompensationFromSettings(const FViewInfo& View);

bool IsExtendLuminanceRangeEnabled();

bool IsAutoExposureUsingIlluminanceEnabled(const FViewInfo& View);
int32 GetAutoExposureIlluminanceDownscaleFactor();

// Returns the auto exposure method enabled by the view (including CVar override).
EAutoExposureMethod GetAutoExposureMethod(const FViewInfo& View);

struct FUpdateExposureCompensationCurveLUTTaskData
{
	UE::Tasks::FTask Task;
	FRHICommandList* CmdList = nullptr;
};

void BeginUpdateExposureCompensationCurveLUT(TArrayView<FViewInfo> Views, FUpdateExposureCompensationCurveLUTTaskData* TaskData);
void FinishUpdateExposureCompensationCurveLUT(FRHICommandListImmediate& RHICmdList, FUpdateExposureCompensationCurveLUTTaskData* TaskData);

BEGIN_SHADER_PARAMETER_STRUCT(FEyeAdaptationScalarParameters, )
	SHADER_PARAMETER(float, ExposureLowPercent)
	SHADER_PARAMETER(float, ExposureHighPercent)
	SHADER_PARAMETER(float, MinAverageLuminance)
	SHADER_PARAMETER(float, MaxAverageLuminance)
	SHADER_PARAMETER(float, ExposureCompensationSettings)
	SHADER_PARAMETER(float, ExposureCompensationCurve)
	SHADER_PARAMETER(float, DeltaWorldTime)
	SHADER_PARAMETER(float, ExposureSpeedUp)
	SHADER_PARAMETER(float, ExposureSpeedDown)
	SHADER_PARAMETER(float, HistogramScale)
	SHADER_PARAMETER(float, HistogramBias)
	SHADER_PARAMETER(float, LuminanceMin)
	SHADER_PARAMETER(float, BlackHistogramBucketInfluence)
	SHADER_PARAMETER(float, GreyMult)
	SHADER_PARAMETER(FVector3f, LuminanceWeights)
	SHADER_PARAMETER(float, ExponentialUpM)
	SHADER_PARAMETER(float, ExponentialDownM)
	SHADER_PARAMETER(float, StartDistance)
	SHADER_PARAMETER(float, LuminanceMax)
	SHADER_PARAMETER(float, IgnoreMaterialsEvaluationPositionBias)
	SHADER_PARAMETER(float, IgnoreMaterialsLuminanceScale)
	SHADER_PARAMETER(float, IgnoreMaterialsMinBaseColorLuminance)
	SHADER_PARAMETER(uint32, IgnoreMaterialsReconstructFromSceneColor)
	SHADER_PARAMETER(float, ForceTarget)
	SHADER_PARAMETER(int, VisualizeDebugType)
	SHADER_PARAMETER(FVector2f, EV100ToExposureCompensationCurveLUTScaleBias)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FEyeAdaptationScalarParameters, Scalars)

	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ExposureCompensationCurveLUT)
	SHADER_PARAMETER_SAMPLER(SamplerState, ExposureCompensationCurveSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, MeterMaskTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, MeterMaskSampler)
END_SHADER_PARAMETER_STRUCT()

FEyeAdaptationScalarParameters GetEyeAdaptationScalarParameters(const FViewInfo& ViewInfo);
FEyeAdaptationParameters GetEyeAdaptationParameters(FRDGBuilder& GraphBuilder, const FViewInfo& ViewInfo);

// Computes the a fixed exposure to be used to replace the dynamic exposure when it's not supported (< SM5).
float GetEyeAdaptationFixedExposure(const FViewInfo& View);

FRDGTextureRef AddSetupExposureIlluminancePass(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FSceneTextures& SceneTextures);

FRDGTextureRef AddCalculateExposureIlluminancePass(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FSceneTextures& SceneTextures,
	const struct FTranslucencyLightingVolumeTextures& TranslucencyLightingVolumeTextures,
	FRDGTextureRef ExposureIlluminanceSetup);

// Returns the updated eye adaptation buffer.
FRDGBufferRef AddHistogramEyeAdaptationPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	const FLocalExposureParameters& LocalExposureParameters,
	FRDGTextureRef HistogramTexture,
	bool bComputeAverageLocalExposure);

// Computes luma of scene color stores in Alpha.
FScreenPassTexture AddBasicEyeAdaptationSetupPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FScreenPassTexture SceneColor);

// Returns the updated eye adaptation buffer.
FRDGBufferRef AddBasicEyeAdaptationPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	const FLocalExposureParameters& LocalExposureParameters,
	FScreenPassTextureSlice SceneColor,
	FRDGBufferRef EyeAdaptationBuffer,
	bool bComputeAverageLocalExposure);

/**
* Helper function to get current eye adaptation in a texture.
* Should only be used by external plugins that require eye adaptation data in texture format.
*/
RENDERER_API FRDGTextureRef AddCopyEyeAdaptationDataToTexturePass(FRDGBuilder& GraphBuilder, const FViewInfo& View);
