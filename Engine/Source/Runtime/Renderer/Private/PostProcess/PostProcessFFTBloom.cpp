// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessFFTBloom.h"
#include "PostProcess/PostProcessTonemap.h"
#include "PostProcess/PostProcessLocalExposure.h"
#include "PostProcess/PostProcessDownsample.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GPUFastFourierTransform.h"
#include "LogRenderer.h"
#include "Rendering/Texture2DResource.h"
#include "PostProcessing.h"
#include "SceneViewState.h"

namespace
{

TAutoConsoleVariable<float> CVarBloomScreenPercentage(
	TEXT("r.Bloom.ScreenPercentage"), 100.0f,
	TEXT("Controles the axis resolution of the FFT convolution for bloom.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarBloomCacheKernel(
	TEXT("r.Bloom.CacheKernel"), 1,
	TEXT("Whether to cache the kernel in spectral domain."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarAsynComputeFFTBloom(
	TEXT("r.Bloom.AsyncCompute"), 1,
	TEXT("Whether to run FFT bloom on async compute.\n"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarBloomWarnKernelResolution(
	TEXT("r.Bloom.WarnKernelResolution"), 1,
	TEXT("Whether to emit a warning when the resolution of the kernel is unecessary to high.\n")
	TEXT(" 0: Disabled;\n")
	TEXT(" 1: Emit the warning on consoles (default);\n")
	TEXT(" 2: Emit the warning on all platforms;\n"),
	ECVF_RenderThreadSafe);

static bool DoesPlatformSupportFFTBloom(EShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsFFTBloom(Platform);
}

class FFFTBloomShader : public FGlobalShader
{
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return DoesPlatformSupportFFTBloom(Parameters.Platform);
	}

	FFFTBloomShader() = default;
	FFFTBloomShader(const CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

class FBloomFindKernelCenterCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomFindKernelCenterCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomFindKernelCenterCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, KernelSpatialTextureSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, KernelSpatialTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, KernelCenterCoordOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FBloomSurveyMaxScatterDispersionCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomSurveyMaxScatterDispersionCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomSurveyMaxScatterDispersionCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, ViewTexelRadiusInKernelTexels)
		SHADER_PARAMETER(int32, SurveyGroupGridSize)
		SHADER_PARAMETER(FIntPoint, KernelSpatialTextureSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, KernelSpatialTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, KernelCenterCoordBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, SurveyOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FBloomSurveyKernelCenterEnergyCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomSurveyKernelCenterEnergyCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomSurveyKernelCenterEnergyCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, ViewTexelRadiusInKernelTexels)
		SHADER_PARAMETER(int32, SurveyGroupGridSize)
		SHADER_PARAMETER(FIntPoint, KernelSpatialTextureSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, KernelSpatialTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, KernelCenterCoordBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, MaxScatterDispersionBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, SurveyOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FBloomReduceKernelSurveyCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomReduceKernelSurveyCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomReduceKernelSurveyCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, SurveyReduceOp)
		SHADER_PARAMETER(int32, SurveyGroupCount)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, SurveyOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FBloomSumScatterDispersionEnergyCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomSumScatterDispersionEnergyCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomSumScatterDispersionEnergyCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, PassId)
		SHADER_PARAMETER(FIntPoint, ScatterDispersionTextureSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ScatterDispersionTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, MaxScatterDispersionBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ScatterDispersionOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FBloomPackKernelConstantsCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomPackKernelConstantsCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomPackKernelConstantsCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, KernelCenterCoordBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, KernelCenterEnergyBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, MaxScatterDispersionBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ScatterDispersionTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, KernelConstantsOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FBloomClampKernelCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomClampKernelCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomClampKernelCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, bProcessAlpha)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, KernelSpatialTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, KernelConstantsBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ClampedKernelSpatialOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FBloomDownsampleKernelCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomDownsampleKernelCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomDownsampleKernelCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, KernelSpatialTextureSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, KernelSpatialTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, KernelSpatialSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DownsampleKernelSpatialOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FBloomResizeKernelCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomResizeKernelCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomResizeKernelCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, DstExtent)
		SHADER_PARAMETER(FIntPoint, ImageExtent)
		SHADER_PARAMETER(FVector2f, KernelSpatialTextureInvSize)
		SHADER_PARAMETER(FIntPoint, DstBufferExtent)
		SHADER_PARAMETER(float, KernelSupportScale)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, KernelConstantsBuffer)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SrcTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SrcSampler)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DstTexture)
	END_SHADER_PARAMETER_STRUCT()
};

class FBloomFinalizeApplyConstantsCS : public FFFTBloomShader
{
public:
	DECLARE_GLOBAL_SHADER(FBloomFinalizeApplyConstantsCS);
	SHADER_USE_PARAMETER_STRUCT(FBloomFinalizeApplyConstantsCS, FFFTBloomShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, ScatterDispersionIntensity)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, KernelConstantsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, SceneColorApplyOutput)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, FFTMulitplyOutput)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FBloomFindKernelCenterCS,           "/Engine/Private/Bloom/BloomFindKernelCenter.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FBloomSurveyMaxScatterDispersionCS, "/Engine/Private/Bloom/BloomSurveyMaxScatterDispersion.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FBloomSurveyKernelCenterEnergyCS,   "/Engine/Private/Bloom/BloomSurveyKernelCenterEnergy.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FBloomReduceKernelSurveyCS,         "/Engine/Private/Bloom/BloomReduceKernelSurvey.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FBloomSumScatterDispersionEnergyCS, "/Engine/Private/Bloom/BloomSumScatterDispersionEnergy.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FBloomPackKernelConstantsCS,        "/Engine/Private/Bloom/BloomPackKernelConstants.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FBloomClampKernelCS,                "/Engine/Private/Bloom/BloomClampKernel.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FBloomDownsampleKernelCS,           "/Engine/Private/Bloom/BloomDownsampleKernel.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FBloomResizeKernelCS,               "/Engine/Private/Bloom/BloomResizeKernel.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FBloomFinalizeApplyConstantsCS,     "/Engine/Private/Bloom/BloomFinalizeApplyConstants.usf", "MainCS", SF_Compute);

} //! namespace

float GetFFTBloomResolutionFraction(const FIntPoint& ViewSize)
{
	float MaxResolutionFraction = FMath::Min(float(GMaxTextureDimensions) * 0.5f / float(FMath::Max(ViewSize.X, ViewSize.Y)), 1.0f);

	return FMath::Clamp(CVarBloomScreenPercentage.GetValueOnRenderThread() / 100.0f, 0.1f, MaxResolutionFraction);
}

bool IsFFTBloomEnabled(const FViewInfo& View)
{
	const bool bUseFFTBloom = View.FinalPostProcessSettings.BloomMethod == EBloomMethod::BM_FFT && View.ViewState != nullptr && DoesPlatformSupportFFTBloom(View.GetShaderPlatform());

	if (bUseFFTBloom)
	{
		return View.FFTBloomKernelTexture != nullptr;
	}
	else
	{
		return false;
	}
}

FRDGTextureRef TransformKernelFFT(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef ResizedKernel,
	bool bDoHorizontalFirst,
	FIntPoint FrequencySize)
{
	// Our frequency storage layout adds two elements to the first transform direction. 
	const FIntPoint FrequencyPadding = (bDoHorizontalFirst) ? FIntPoint(2, 0) : FIntPoint(0, 2);
	const FIntPoint PaddedFrequencySize = FrequencySize + FrequencyPadding;

	// Should read / write to PF_G16R16F or PF_G32R32F (float2 formats)
	// Need to set the render target description before we "request surface"
	FRDGTextureRef SpectralKernel;
	{

		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			PaddedFrequencySize,
			GPUFFT::PixelFormat(),
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV);

		SpectralKernel = GraphBuilder.CreateTexture(Desc, TEXT("Bloom.FFT.SpectralKernel"));
	}

	FIntRect SrcRect(FIntPoint(0, 0), FrequencySize);
	GPUFFT::FFTImage2D(
		GraphBuilder,
		View.ShaderMap,
		FrequencySize, bDoHorizontalFirst,
		GraphBuilder.CreateSRV(FRDGTextureSRVDesc(ResizedKernel)), SrcRect,
		SpectralKernel);

	return SpectralKernel;
}

struct FFFTBloomIntermediates
{
	// The sub-domain of the Input/Output buffers 
	// where the image lives, i.e. the region of interest
	FIntPoint ImageSize;

	// Image space, padded by black for kernel and  rounded up to powers of two
	// this defines the size of the FFT in each direction.
	FIntPoint FrequencySize;

	FVector3f PreFilter;

	float KernelSupportScale = 0.0f;
	float KernelSupportScaleClamp = 0.0f;

	// The order of the two-dimensional transform.  This implicitly defines
	// the data layout in transform space for both the kernel and image transform
	bool bDoHorizontalFirst = false;

	ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute;
};

void InitDomainAndGetKernel(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FFFTBloomIntermediates& Intermediates,
	FRDGTextureRef* OutSpectralKernelTexture,
	FRDGBufferRef* OutKernelConstantsBuffer)
{
	FSceneViewState* ViewState = View.ViewState;

	const auto& PPSettings = View.FinalPostProcessSettings;

	// This should exist if we called IsFFTBloomPhysicalKernelReady.
	check(View.FFTBloomKernelTexture && View.FFTBloomKernelTexture->TextureRHI);

	const FTexture2DResource* BloomConvolutionTextureResource = View.FFTBloomKernelTexture;
	const FTextureRHIRef& PhysicalSpaceKernelTextureRef = BloomConvolutionTextureResource->TextureRHI;

	const float BloomConvolutionSize = PPSettings.BloomConvolutionSize;
	const FVector2f CenterUV = FVector2f(PPSettings.BloomConvolutionCenterUV); // TODO: remove

	// Our frequency storage layout adds two elements to the first transform direction. 
	const FIntPoint FrequencyPadding = (Intermediates.bDoHorizontalFirst) ? FIntPoint(2, 0) : FIntPoint(0, 2);
	const FIntPoint PaddedFrequencySize = Intermediates.FrequencySize + FrequencyPadding;

	// Should read / write to PF_G16R16F or PF_G32R32F (float2 formats)
	// Need to set the render target description before we "request surface"
	const FRDGTextureDesc TransformDesc = FRDGTextureDesc::Create2D(
		PaddedFrequencySize,
		GPUFFT::PixelFormat(),
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV);

	if (ViewState && ViewState->BloomFFTKernel.Spectral && CVarBloomCacheKernel.GetValueOnRenderThread() != 0)
	{
		auto& FFTKernel = ViewState->BloomFFTKernel;

		FRDGTextureRef PrevCachedSpectralKernel = GraphBuilder.RegisterExternalTexture(FFTKernel.Spectral);

		const bool bSameTexture = (FFTKernel.PhysicalRHI == PhysicalSpaceKernelTextureRef);
		const bool bSameSpectralBuffer = TransformDesc.ClearValue == PrevCachedSpectralKernel->Desc.ClearValue
			&& TransformDesc.Flags == PrevCachedSpectralKernel->Desc.Flags
			&& TransformDesc.Format == PrevCachedSpectralKernel->Desc.Format
			&& TransformDesc.Extent == PrevCachedSpectralKernel->Desc.Extent;

		const bool bSameKernelSize = FMath::IsNearlyEqual(FFTKernel.Scale, BloomConvolutionSize, float(1.e-6) /*tol*/);
		const bool bSameImageSize = (Intermediates.ImageSize == FFTKernel.ImageSize);
		const bool bSameMipLevel = bSameTexture && FFTKernel.PhysicalMipLevel == BloomConvolutionTextureResource->GetCurrentMipCount();

		if (bSameTexture && bSameSpectralBuffer && bSameKernelSize && bSameImageSize && bSameMipLevel)
		{
			*OutSpectralKernelTexture = PrevCachedSpectralKernel;
			*OutKernelConstantsBuffer = GraphBuilder.RegisterExternalBuffer(FFTKernel.ConstantsBuffer);
			return;
		}
	}

	// Re-transform the kernel if needed -- needs to run on all GPUs in case view's GPU assignment changes later.
	RDG_EVENT_SCOPE(GraphBuilder, "InitBloomKernel");
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

	FRDGTextureRef SpatialKernelTexture = RegisterExternalTexture(GraphBuilder, PhysicalSpaceKernelTextureRef, TEXT("Bloom.FFT.OriginalKernel"));


	// Find the center of the kernel
	FRDGBufferRef KernelCenterCoordBuffer;
	{
		KernelCenterCoordBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(int32_t), /* NumElements = */ 4),
			TEXT("Bloom.FFT.KernelCenterCoord"));

		FBloomFindKernelCenterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomFindKernelCenterCS::FParameters>();
		PassParameters->KernelSpatialTextureSize = SpatialKernelTexture->Desc.Extent;
		PassParameters->KernelSpatialTexture = SpatialKernelTexture;
		PassParameters->KernelCenterCoordOutput = GraphBuilder.CreateUAV(KernelCenterCoordBuffer);

		AddClearUAVPass(GraphBuilder, PassParameters->KernelCenterCoordOutput, /* Value = */ 0);

		TShaderMapRef<FBloomFindKernelCenterCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FFTBloom FindKernelCenter %dx%d", SpatialKernelTexture->Desc.Extent.X, SpatialKernelTexture->Desc.Extent.Y),
			Intermediates.ComputePassFlags,
			ComputeShader,
			PassParameters, 
			FComputeShaderUtils::GetGroupCount(SpatialKernelTexture->Desc.Extent, 8));
	}

	// Find information about the kernel's energy distribution
	FRDGBufferRef MaxScatterDispersionBuffer;
	FRDGBufferRef KernelCenterEnergyBuffer;
	{
		const int32 SurveyTileSize = 8;

		float KernelSizeInDstPixels = FMath::Max(float(Intermediates.ImageSize.X) * Intermediates.KernelSupportScale, 1.0f);

		// Diameter of a view texel in the kernel.
		float ViewTexelDiameterInKernelTexels = FMath::Max(SpatialKernelTexture->Desc.Extent.X / KernelSizeInDstPixels, 1.0f);

		float ViewTexelRadiusInKernelTexels = ViewTexelDiameterInKernelTexels * 0.5f;

		int32 SurveyGroupGridSize = 2 * FMath::DivideAndRoundUp(FMath::CeilToInt(ViewTexelRadiusInKernelTexels) + 4, SurveyTileSize);
		int32 SurveyGroupCount = SurveyGroupGridSize * SurveyGroupGridSize;

		FRDGTextureUAVRef DebugTextureUAV;
		{
			FRDGTextureDesc DebugDesc = FRDGTextureDesc::Create2D(
				FIntPoint(SurveyGroupGridSize * 8, SurveyGroupGridSize * 8),
				PF_FloatRGBA,
				FClearValueBinding::None,
				/* InFlags = */ TexCreate_ShaderResource | TexCreate_UAV);

			FRDGTextureRef DebugTexture = GraphBuilder.CreateTexture(DebugDesc, TEXT("Debug.Bloom.Survey"));

			DebugTextureUAV = GraphBuilder.CreateUAV(DebugTexture);
		}

		RDG_EVENT_SCOPE(GraphBuilder, "FFTBloom SurveyKernel(TexelDiameter=%f)", ViewTexelDiameterInKernelTexels);

		// Reduce a survey buffer to a single value.
		auto ReduceSurveyBuffer = [&](FRDGBufferRef SurveyBuffer, int32 Op)
		{
			FBloomReduceKernelSurveyCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomReduceKernelSurveyCS::FParameters>();
			PassParameters->SurveyReduceOp = Op;
			PassParameters->SurveyGroupCount = SurveyGroupCount;
			PassParameters->SurveyOutput = GraphBuilder.CreateUAV(SurveyBuffer);

			TShaderMapRef<FBloomReduceKernelSurveyCS> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("FFTBloom ReduceKernelSurvey(Op=%d) %d", Op, SurveyGroupCount),
				Intermediates.ComputePassFlags,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(SurveyGroupCount, 64));
		};

		// Find the max scatter dispersion to use around the footprint of the view pixel in the kernel.
		{
			MaxScatterDispersionBuffer = GraphBuilder.CreateBuffer(
				FRDGBufferDesc::CreateStructuredDesc(sizeof(FLinearColor), /* NumElements = */ SurveyGroupCount),
				TEXT("Bloom.FFT.MaxScatterDispersion"));

			FBloomSurveyMaxScatterDispersionCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomSurveyMaxScatterDispersionCS::FParameters>();
			PassParameters->ViewTexelRadiusInKernelTexels = ViewTexelRadiusInKernelTexels;
			PassParameters->SurveyGroupGridSize = SurveyGroupGridSize;
			PassParameters->KernelSpatialTextureSize = SpatialKernelTexture->Desc.Extent;
			PassParameters->KernelSpatialTexture = SpatialKernelTexture;
			PassParameters->KernelCenterCoordBuffer = GraphBuilder.CreateSRV(KernelCenterCoordBuffer);
			PassParameters->SurveyOutput = GraphBuilder.CreateUAV(MaxScatterDispersionBuffer);
			PassParameters->DebugOutput = DebugTextureUAV;

			TShaderMapRef<FBloomSurveyMaxScatterDispersionCS> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("FFTBloom SurveyMaxScatterDispersion"),
				Intermediates.ComputePassFlags,
				ComputeShader,
				PassParameters,
				FIntVector(SurveyGroupGridSize, SurveyGroupGridSize, 1));

			ReduceSurveyBuffer(MaxScatterDispersionBuffer, /* Op = */ 0);
		}

		// Find the amount of energy at the center in the footprint of the view pixel in the kernel.
		{
			KernelCenterEnergyBuffer = GraphBuilder.CreateBuffer(
				FRDGBufferDesc::CreateStructuredDesc(sizeof(FLinearColor), /* NumElements = */ SurveyGroupCount),
				TEXT("Bloom.FFT.KernelCenterEnergy"));

			FBloomSurveyKernelCenterEnergyCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomSurveyKernelCenterEnergyCS::FParameters>();
			PassParameters->ViewTexelRadiusInKernelTexels = ViewTexelRadiusInKernelTexels;
			PassParameters->SurveyGroupGridSize = SurveyGroupGridSize;
			PassParameters->KernelSpatialTextureSize = SpatialKernelTexture->Desc.Extent;
			PassParameters->KernelSpatialTexture = SpatialKernelTexture;
			PassParameters->KernelCenterCoordBuffer = GraphBuilder.CreateSRV(KernelCenterCoordBuffer);
			PassParameters->MaxScatterDispersionBuffer = GraphBuilder.CreateSRV(MaxScatterDispersionBuffer);
			PassParameters->SurveyOutput = GraphBuilder.CreateUAV(KernelCenterEnergyBuffer);
			PassParameters->DebugOutput = DebugTextureUAV;

			TShaderMapRef<FBloomSurveyKernelCenterEnergyCS> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("FFTBloom SurveyKernelCenterEnergy"),
				Intermediates.ComputePassFlags,
				ComputeShader,
				PassParameters,
				FIntVector(SurveyGroupGridSize, SurveyGroupGridSize, 1));

			ReduceSurveyBuffer(KernelCenterEnergyBuffer, /* Op = */ 1);
		}
	}

	// Find out the total energy of the kernel - center.
	// Implemented in such away to prioritize numerical error rather than speed.
	FRDGTextureRef ScatterDispersionTexture;
	{
		RDG_EVENT_SCOPE(GraphBuilder, "FFTBloom SumScatterDispersionEnergy %dx%d",
			SpatialKernelTexture->Desc.Extent.X, SpatialKernelTexture->Desc.Extent.Y);

		ScatterDispersionTexture = SpatialKernelTexture;

		for (int32 PassId = 0; ScatterDispersionTexture->Desc.Extent.X > 1 && ScatterDispersionTexture->Desc.Extent.Y > 1; PassId++)
		{
			FRDGTextureRef NewScatterDispersionTexture;
			{
				FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
					FIntPoint::DivideAndRoundUp(ScatterDispersionTexture->Desc.Extent, 8),
					PF_A32B32G32R32F,
					FClearValueBinding::None,
					TexCreate_ShaderResource | TexCreate_UAV);

				NewScatterDispersionTexture = GraphBuilder.CreateTexture(Desc, TEXT("Bloom.FFT.KernelIntensity"));
			}

			FBloomSumScatterDispersionEnergyCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomSumScatterDispersionEnergyCS::FParameters>();
			PassParameters->PassId = PassId;
			PassParameters->ScatterDispersionTextureSize = ScatterDispersionTexture->Desc.Extent;
			PassParameters->ScatterDispersionTexture = ScatterDispersionTexture;
			PassParameters->MaxScatterDispersionBuffer = GraphBuilder.CreateSRV(MaxScatterDispersionBuffer);
			PassParameters->ScatterDispersionOutput = GraphBuilder.CreateUAV(NewScatterDispersionTexture);

			TShaderMapRef<FBloomSumScatterDispersionEnergyCS> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("FFTBloom SumScatterDispersionEnergy %dx%d -> %dx%d",
					ScatterDispersionTexture->Desc.Extent.X, ScatterDispersionTexture->Desc.Extent.Y,
					NewScatterDispersionTexture->Desc.Extent.X, NewScatterDispersionTexture->Desc.Extent.Y),
				Intermediates.ComputePassFlags,
				ComputeShader,
				PassParameters,
				FIntVector(NewScatterDispersionTexture->Desc.Extent.X, NewScatterDispersionTexture->Desc.Extent.Y, 1));

			ScatterDispersionTexture = NewScatterDispersionTexture;
		}
	}

	// Packs all the kernel information into the same buffer.
	FRDGBufferRef KernelConstantsBuffer = nullptr;
	{
		KernelConstantsBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(float) * 16, /* NumElements = */ 1),
			TEXT("Bloom.FFT.KernelConstants"));

		FBloomPackKernelConstantsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomPackKernelConstantsCS::FParameters>();
		PassParameters->KernelCenterCoordBuffer = GraphBuilder.CreateSRV(KernelCenterCoordBuffer);
		PassParameters->KernelCenterEnergyBuffer   = GraphBuilder.CreateSRV(KernelCenterEnergyBuffer);
		PassParameters->MaxScatterDispersionBuffer = GraphBuilder.CreateSRV(MaxScatterDispersionBuffer);
		PassParameters->ScatterDispersionTexture   = ScatterDispersionTexture;
		PassParameters->KernelConstantsOutput = GraphBuilder.CreateUAV(KernelConstantsBuffer);

		TShaderMapRef<FBloomPackKernelConstantsCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FFTBloom PackKernelConstants"),
			Intermediates.ComputePassFlags,
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1));
	}

	// Preprocess the original kernel for Fourier transformation
	FRDGTextureRef ResizedKernel;
	{
		RDG_EVENT_SCOPE(GraphBuilder, "FFTBloom PreprocessKernel");

		// Clamp the kernel to avoid highlight contamination in the resize.
		FRDGTextureRef ClampedKernelTexture;
		{
			{
				FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
					SpatialKernelTexture->Desc.Extent,
					PF_FloatRGBA,
					FClearValueBinding::None,
					TexCreate_ShaderResource | TexCreate_UAV);

				ClampedKernelTexture = GraphBuilder.CreateTexture(Desc, TEXT("Bloom.FFT.ClampedKernel"));
			}

			FBloomClampKernelCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomClampKernelCS::FParameters>();
			PassParameters->bProcessAlpha = IsPostProcessingWithAlphaChannelSupported();
			PassParameters->KernelSpatialTexture = SpatialKernelTexture;
			PassParameters->KernelConstantsBuffer = GraphBuilder.CreateSRV(KernelConstantsBuffer);
			PassParameters->ClampedKernelSpatialOutput = GraphBuilder.CreateUAV(ClampedKernelTexture);

			TShaderMapRef<FBloomClampKernelCS> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("FFTBloom ClampedKernel %dx%d",
					SpatialKernelTexture->Desc.Extent.X,
					SpatialKernelTexture->Desc.Extent.Y),
				Intermediates.ComputePassFlags,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(SpatialKernelTexture->Desc.Extent, 8));
		}

		// Downscale the clamped kernel succesively to have nice anti-aliased
		{
			int32 MajorAxisPixelCount = FMath::Max(Intermediates.ImageSize.X, Intermediates.ImageSize.Y);
			float KernelSizeInDstPixels = FMath::Max(float(MajorAxisPixelCount) * Intermediates.KernelSupportScale, 1.f);
			float DownscaleFactor = float(SpatialKernelTexture->Desc.Extent.X) / KernelSizeInDstPixels;
			int32 RecommendedKernelDownsscale = 1;

			for (float RemainingDownscaleFactor = DownscaleFactor; RemainingDownscaleFactor > 2.0f; RemainingDownscaleFactor *= 0.5f)
			{
				FRDGTextureRef NewClampedKernelTexture;
				{
					FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
						ClampedKernelTexture->Desc.Extent / 2,
						PF_FloatRGBA,
						FClearValueBinding::None,
						TexCreate_ShaderResource | TexCreate_UAV);

					NewClampedKernelTexture = GraphBuilder.CreateTexture(Desc, TEXT("Bloom.FFT.DownsampleKernel"));
				}

				FBloomDownsampleKernelCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomDownsampleKernelCS::FParameters>();
				PassParameters->KernelSpatialTextureSize = ClampedKernelTexture->Desc.Extent;
				PassParameters->KernelSpatialTexture = ClampedKernelTexture;
				PassParameters->KernelSpatialSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
				PassParameters->DownsampleKernelSpatialOutput = GraphBuilder.CreateUAV(NewClampedKernelTexture);

				TShaderMapRef<FBloomDownsampleKernelCS> ComputeShader(View.ShaderMap);
				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("FFTBloom DownsampleKernel %dx%d -> %dx%d",
						ClampedKernelTexture->Desc.Extent.X,
						ClampedKernelTexture->Desc.Extent.Y,
						NewClampedKernelTexture->Desc.Extent.X,
						NewClampedKernelTexture->Desc.Extent.Y),
					Intermediates.ComputePassFlags,
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(NewClampedKernelTexture->Desc.Extent, 8));

				ClampedKernelTexture = NewClampedKernelTexture;
				RecommendedKernelDownsscale *= 2;
			}

			if (RecommendedKernelDownsscale > 1 && (CVarBloomWarnKernelResolution.GetValueOnRenderThread() > 1 || (CVarBloomWarnKernelResolution.GetValueOnRenderThread() == 1 && FDataDrivenShaderPlatformInfo::GetIsConsole(View.GetShaderPlatform()))))
			{
				FString WarningTexturePathName;
				if (View.FinalPostProcessSettings.BloomConvolutionTexture)
				{
					WarningTexturePathName = "(" + View.FinalPostProcessSettings.BloomConvolutionTexture.GetPathName() + ")";
				}

				static bool bLogged = false;
				UE_CLOGF(!bLogged, LogRenderer, Warning, "The FPostProcessSettings::BloomConvolutionTexture %ls could have it's resolution lowered by a factor of %d to save memory.", *WarningTexturePathName, RecommendedKernelDownsscale);
				bLogged = true;
			}
		}

		// Final resize the kernel for Fourier transformation
		{
			ResizedKernel = GraphBuilder.CreateTexture(TransformDesc, TEXT("Bloom.FFT.ResizedKernel"));

			FBloomResizeKernelCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomResizeKernelCS::FParameters>();
			PassParameters->DstExtent = Intermediates.FrequencySize;
			PassParameters->ImageExtent = Intermediates.ImageSize;
			PassParameters->KernelSpatialTextureInvSize.X = 1.0f / float(SpatialKernelTexture->Desc.Extent.X);
			PassParameters->KernelSpatialTextureInvSize.Y = 1.0f / float(SpatialKernelTexture->Desc.Extent.Y);
			PassParameters->DstBufferExtent = Intermediates.FrequencySize;
			PassParameters->KernelSupportScale = Intermediates.KernelSupportScale;

			PassParameters->KernelConstantsBuffer = GraphBuilder.CreateSRV(KernelConstantsBuffer);
			PassParameters->SrcTexture = ClampedKernelTexture;
			PassParameters->SrcSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
		
			PassParameters->DstTexture = GraphBuilder.CreateUAV(ResizedKernel);

			TShaderMapRef<FBloomResizeKernelCS> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("FFTBloom PreProcessKernel %dx%d", Intermediates.FrequencySize.X, Intermediates.FrequencySize.Y),
				Intermediates.ComputePassFlags,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(PaddedFrequencySize, 8));
		}
	}

	// Two Dimensional FFT of the physical space kernel.  
	FRDGTextureRef SpectralKernelTexture = TransformKernelFFT(
		GraphBuilder, View,
		ResizedKernel,
		Intermediates.bDoHorizontalFirst,
		Intermediates.FrequencySize);

	// Update the data on the ViewState
	if (ViewState && CVarBloomCacheKernel.GetValueOnRenderThread() != 0)
	{
		ViewState->BloomFFTKernel.Scale = BloomConvolutionSize;
		ViewState->BloomFFTKernel.ImageSize = Intermediates.ImageSize;
		ViewState->BloomFFTKernel.PhysicalRHI = PhysicalSpaceKernelTextureRef;
		ViewState->BloomFFTKernel.PhysicalMipLevel = BloomConvolutionTextureResource->GetCurrentMipCount();
		{
			ViewState->BloomFFTKernel.Spectral.SafeRelease();
			GraphBuilder.QueueTextureExtraction(SpectralKernelTexture, &ViewState->BloomFFTKernel.Spectral);
		}
		{
			ViewState->BloomFFTKernel.ConstantsBuffer.SafeRelease();
			GraphBuilder.QueueBufferExtraction(KernelConstantsBuffer, &ViewState->BloomFFTKernel.ConstantsBuffer);
		}
	}

	*OutSpectralKernelTexture = SpectralKernelTexture;
	*OutKernelConstantsBuffer = KernelConstantsBuffer;
}

FFFTBloomOutput AddFFTBloomPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTextureSlice& InputSceneColor,
	float InputResolutionFraction,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FRDGBufferRef EyeAdaptationBuffer,
	const FLocalExposureParameters& LocalExposureParameters,
	FRDGTextureRef LocalExposureTexture,
	FRDGTextureRef BlurredLogLuminanceTexture)
{
	check(InputSceneColor.IsValid());

	const float ResolutionFraction = GetFFTBloomResolutionFraction(InputSceneColor.ViewRect.Size());
	const float DownscaleResolutionFraction = ResolutionFraction / InputResolutionFraction;
	check(DownscaleResolutionFraction <= 1.0f);

	FFFTBloomIntermediates Intermediates;
	{
		FSceneViewState* ViewState = (FSceneViewState*)View.State;
		check(ViewState);

		const auto& PPSettings = View.FinalPostProcessSettings;

		// The kernel parameters on the FinalPostProcess.

		const float BloomConvolutionSize = PPSettings.BloomConvolutionSize;
		const float KernelSupportScaleClamp = FMath::Clamp(PPSettings.BloomConvolutionBufferScale, 0.f, 1.f);

		// Clip the Kernel support (i.e. bloom size) to 100% the screen width 
		const float MaxBloomSize = 1.f;
		const float KernelSupportScale = FMath::Clamp(BloomConvolutionSize, 0.f, MaxBloomSize);

		// We padd by 1/2 the number of pixels the kernel needs in the x-direction
		// so if the kernel is being applied on the edge of the image it will see padding and not periodicity
		// NB:  If the kernel padding would force a transform buffer that is too big for group shared memory (> 4096)
		//      we clamp it.  This could result in a wrap-around in the bloom (from one side of the screen to the other),
		//      but since the amplitude of the bloom kernel tails is usually very small, this shouldn't be too bad.
		auto KernelRadiusSupportFunctor = [KernelSupportScale, KernelSupportScaleClamp](const FIntPoint& Size) ->int32
		{
			float ClampedKernelSupportScale = (KernelSupportScaleClamp > 0) ? FMath::Min(KernelSupportScale, KernelSupportScaleClamp) : KernelSupportScale;
			int32 FilterRadius = FMath::CeilToInt(0.5 * ClampedKernelSupportScale * Size.X);
			const int32 MaxFFTSize = GPUFFT::MaxScanLineLength();
			int32 MaxDim = FMath::Max(Size.X, Size.Y);
			if (MaxDim + FilterRadius > MaxFFTSize && MaxDim < MaxFFTSize) FilterRadius = MaxFFTSize - MaxDim;

			return FilterRadius;
		};

		Intermediates.ImageSize.X = FMath::CeilToInt(InputSceneColor.ViewRect.Width() * DownscaleResolutionFraction);
		Intermediates.ImageSize.Y = FMath::CeilToInt(InputSceneColor.ViewRect.Height() * DownscaleResolutionFraction);

		Intermediates.KernelSupportScale = KernelSupportScale;
		Intermediates.KernelSupportScaleClamp = KernelSupportScaleClamp;

		// The pre-filter boost parameters for bright pixels. Because the Convolution PP work in pre-exposure space, the min and max needs adjustment.
		Intermediates.PreFilter = FVector3f(PPSettings.BloomConvolutionPreFilterMin, PPSettings.BloomConvolutionPreFilterMax, PPSettings.BloomConvolutionPreFilterMult);

		// The length of the a side of the square kernel image in pixels
		const int32 KernelSize = FMath::CeilToInt(KernelSupportScale * FMath::Max(Intermediates.ImageSize.X, Intermediates.ImageSize.Y));

		const int32 SpectralPadding = KernelRadiusSupportFunctor(Intermediates.ImageSize);

		// The following are mathematically equivalent
		// 1) Horizontal FFT / Vertical FFT / Filter / Vertical InvFFT / Horizontal InvFFT
		// 2) Vertical FFT / Horizontal FFT / Filter / Horizontal InvFFT / Vertical InvFFT
		// but we choose the one with the smallest intermediate buffer size

		// The size of the input image plus padding that accounts for
		// the width of the kernel.  The ImageRect is virtually padded
		// with black to account for the gather action of the convolution.
		FIntPoint PaddedImageSize = Intermediates.ImageSize + FIntPoint(SpectralPadding, SpectralPadding);
		PaddedImageSize.X = FMath::Max(PaddedImageSize.X, KernelSize);
		PaddedImageSize.Y = FMath::Max(PaddedImageSize.Y, KernelSize);

		Intermediates.FrequencySize = FIntPoint(FMath::RoundUpToPowerOfTwo(PaddedImageSize.X), FMath::RoundUpToPowerOfTwo(PaddedImageSize.Y));

		// Choose to do to transform in the direction that results in writing the least amount of data to main memory.

		Intermediates.bDoHorizontalFirst = ((Intermediates.FrequencySize.Y * PaddedImageSize.X) > (Intermediates.FrequencySize.X * PaddedImageSize.Y));

		Intermediates.ComputePassFlags = (GSupportsEfficientAsyncCompute && CVarAsynComputeFFTBloom.GetValueOnRenderThread() != 0) ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "Bloom %dx%d", Intermediates.ImageSize.X, Intermediates.ImageSize.Y);

	// Downscale the input to the final resolution fraction
	FScreenPassTextureSlice FFTInputSceneColor;
	if (ResolutionFraction != InputResolutionFraction)
	{
		FIntPoint Extent;
		QuantizeSceneBufferSize(Intermediates.ImageSize, Extent);

		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			Extent,
			InputSceneColor.TextureSRV->Desc.Texture->Desc.Format,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV);

		FFTInputSceneColor.TextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(
			GraphBuilder.CreateTexture(Desc, TEXT("Bloom.FFT.Input"))));
		FFTInputSceneColor.ViewRect = FIntRect(FIntPoint::ZeroValue, Intermediates.ImageSize);

		AddDownsampleComputePass(GraphBuilder, View, InputSceneColor, FScreenPassTexture(FFTInputSceneColor.TextureSRV->Desc.Texture, FFTInputSceneColor.ViewRect), EDownsampleQuality::Low, Intermediates.ComputePassFlags);
	}
	else
	{
		FFTInputSceneColor = InputSceneColor;
	}

	if (LocalExposureTexture != nullptr)
	{
		FScreenPassTextureSlice Temp = FFTInputSceneColor;

		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			Temp.TextureSRV->Desc.Texture->Desc.Extent,
			Temp.TextureSRV->Desc.Texture->Desc.Format,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV);

		FFTInputSceneColor.TextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(
			GraphBuilder.CreateTexture(Desc, TEXT("Bloom.FFT.Input"))));

		AddApplyLocalExposurePass(
			GraphBuilder,
			View,
			EyeAdaptationParameters,
			EyeAdaptationBuffer,
			LocalExposureParameters,
			LocalExposureTexture,
			BlurredLogLuminanceTexture,
			Temp,
			FFTInputSceneColor,
			Intermediates.ComputePassFlags);
	}

	// Init the domain data update the cached kernel if needed.
	FRDGTextureRef SpectralKernelTexture;
	FRDGBufferRef KernelConstantsBuffer;
	InitDomainAndGetKernel(
		GraphBuilder, View, Intermediates,
		/* out */ &SpectralKernelTexture,
		/* out */ &KernelConstantsBuffer);

	FFFTBloomOutput BloomOutput;
	FRDGBufferRef FFTMulitplyParameters;
	// Generate the apply constant buffer for the tone-maping pass.
	{
		check(FTonemapInputs::SupportsSceneColorApplyParametersBuffer(View.GetShaderPlatform()));

		BloomOutput.SceneColorApplyParameters = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(FLinearColor), /* NumElements = */ 1),
			TEXT("Bloom.FFT.SceneColorApplyParameters"));

		FFTMulitplyParameters = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(FLinearColor), /* NumElements = */ 1),
			TEXT("Bloom.FFT.FFTMulitplyParameters"));

		const float BloomIntensity = View.FinalPostProcessSettings.BloomIntensity * View.FinalPostProcessSettings.BloomConvolutionIntensity;

		FBloomFinalizeApplyConstantsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBloomFinalizeApplyConstantsCS::FParameters>();
		PassParameters->ScatterDispersionIntensity = View.FinalPostProcessSettings.BloomConvolutionScatterDispersion * BloomIntensity;
		PassParameters->KernelConstantsBuffer = GraphBuilder.CreateSRV(KernelConstantsBuffer);
		PassParameters->SceneColorApplyOutput = GraphBuilder.CreateUAV(BloomOutput.SceneColorApplyParameters);
		PassParameters->FFTMulitplyOutput = GraphBuilder.CreateUAV(FFTMulitplyParameters);

		TShaderMapRef<FBloomFinalizeApplyConstantsCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FFTBloom FinalizeApplyConstants(ScatterDispersion=%f)", PassParameters->ScatterDispersionIntensity),
			Intermediates.ComputePassFlags,
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1));
	}

	FRDGTextureDesc OutputSceneColorDesc = FRDGTextureDesc::Create2D(
		FFTInputSceneColor.TextureSRV->Desc.Texture->Desc.Extent,
		FFTInputSceneColor.TextureSRV->Desc.Texture->Desc.Format,
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV | GFastVRamConfig.Bloom);
		
	BloomOutput.BloomTexture.Texture = GraphBuilder.CreateTexture(OutputSceneColorDesc, TEXT("Bloom.FFT.ScatterDispersion"));
	BloomOutput.BloomTexture.ViewRect = FFTInputSceneColor.ViewRect;

	GPUFFT::ConvolutionWithTextureImage2D(
		GraphBuilder,
		Intermediates.ComputePassFlags,
		View.ShaderMap,
		Intermediates.FrequencySize,
		Intermediates.bDoHorizontalFirst,
		SpectralKernelTexture,
		FFTInputSceneColor.TextureSRV, FFTInputSceneColor.ViewRect,
		BloomOutput.BloomTexture.Texture, BloomOutput.BloomTexture.ViewRect,
		Intermediates.PreFilter,
		FFTMulitplyParameters,
		GFastVRamConfig.Bloom);

	return BloomOutput;
}
