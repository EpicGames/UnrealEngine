// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "ScreenPass.h"

class FEyeAdaptationParameters;
class FLocalExposureParameters;
class FRDGBuffer;
class FRDGBuilder;
class FRDGTexture;
class FViewInfo;

using FRDGBufferRef = FRDGBuffer*;
using FRDGTextureRef = FRDGTexture*;

// Returns whether FFT bloom is enabled for the view.
bool IsFFTBloomEnabled(const FViewInfo& View);

float GetFFTBloomResolutionFraction(const FIntPoint& ViewSize);

struct FFFTBloomOutput
{
	FScreenPassTexture BloomTexture;
	FRDGBufferRef SceneColorApplyParameters = nullptr;
};

FFFTBloomOutput AddFFTBloomPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FScreenPassTextureSlice& InputSceneColor,
	float InputResolutionFraction,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FRDGBufferRef EyeAdaptationBuffer,
	const FLocalExposureParameters& LocalExposureParameters,
	FRDGTextureRef LocalExposureTexture,
	FRDGTextureRef BlurredLogLuminanceTexture);
