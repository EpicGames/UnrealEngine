// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessGBufferHints.h"
#include "CanvasTypes.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "UnrealEngine.h"

class FVisualizeGBufferHintsPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeGBufferHintsPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeGBufferHintsPS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, OriginalSceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, OriginalSceneColorSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeGBufferHintsPS, "/Engine/Private/PostProcessGBufferHints.usf", "MainPS", SF_Pixel);

FScreenPassTexture AddVisualizeGBufferHintsPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeGBufferHintsInputs& Inputs)
{
	check(Inputs.SceneTextures);
	check(Inputs.SceneColor.IsValid());
	check(Inputs.OriginalSceneColor.IsValid());

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeGBufferHints"));
	}

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(Output);

	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FVisualizeGBufferHintsPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeGBufferHintsPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
	PassParameters->SceneTextures = Inputs.SceneTextures;
	PassParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters->SceneColorSampler = PointClampSampler;
	PassParameters->OriginalSceneColorTexture = Inputs.OriginalSceneColor.Texture;
	PassParameters->OriginalSceneColorSampler = PointClampSampler;

	TShaderMapRef<FVisualizeGBufferHintsPS> PixelShader(View.ShaderMap);

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeGBufferHints");

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Visualizer"), View, OutputViewport, InputViewport, PixelShader, PassParameters);

	Output.LoadAction = ERenderTargetLoadAction::ELoad;

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("Overlay"), View, Output, [&View](FCanvas& Canvas)
	{
		float X = 30;
		float Y = 8;
		const float YStep = 14;
		const float ColumnWidth = 250;

		FString Line;

		Line = FString::Printf(TEXT("GBufferHints"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));

		Y += YStep;

		Line = FString::Printf(TEXT("Yellow: Unrealistic material (In nature even black materials reflect a small amount of light)"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(0.8f, 0.8f, 0));

		Line = FString::Printf(TEXT("Red: Impossible material (This material emits more light than it receives)"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 0, 0));
	});

	return MoveTemp(Output);
}