// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "OverridePassSequence.h"

class FSceneTextureParameters;

struct FVisualizeGBufferHintsInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The scene color to composite with the gbuffer hint visualization.
	FScreenPassTexture SceneColor;

	// [Required] The original, unprocessed scene color to composite with the gbuffer hint visualization.
	FScreenPassTexture OriginalSceneColor;

	// [Required] The scene textures used to visualize gbuffer hints.
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures = nullptr;
};

FScreenPassTexture AddVisualizeGBufferHintsPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeGBufferHintsInputs& Inputs);