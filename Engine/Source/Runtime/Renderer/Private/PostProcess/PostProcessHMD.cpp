// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessHMD.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "SceneRendering.h"

/** The filter vertex declaration resource type. */
class FDistortionVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	/** Destructor. */
	virtual ~FDistortionVertexDeclaration() {}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		uint16 Stride = sizeof(FDistortionVertex);
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, STRUCT_OFFSET(FDistortionVertex, Position),VET_Float2,0, Stride));
		Elements.Add(FVertexElement(0, STRUCT_OFFSET(FDistortionVertex, TexR), VET_Float2, 1, Stride));
		Elements.Add(FVertexElement(0, STRUCT_OFFSET(FDistortionVertex, TexG), VET_Float2, 2, Stride));
		Elements.Add(FVertexElement(0, STRUCT_OFFSET(FDistortionVertex, TexB), VET_Float2, 3, Stride));
		Elements.Add(FVertexElement(0, STRUCT_OFFSET(FDistortionVertex, VignetteFactor), VET_Float1, 4, Stride));
		Elements.Add(FVertexElement(0, STRUCT_OFFSET(FDistortionVertex, TimewarpFactor), VET_Float1, 5, Stride));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	virtual void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

/** The Distortion vertex declaration. */
TGlobalResource<FDistortionVertexDeclaration> GDistortionVertexDeclaration;

BEGIN_SHADER_PARAMETER_STRUCT(FHMDDistortionParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
	SHADER_PARAMETER(FVector2f, EyeToSrcUVScale)
	SHADER_PARAMETER(FVector2f, EyeToSrcUVOffset)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FHMDDistortionPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FHMDDistortionPS);
	SHADER_USE_PARAMETER_STRUCT(FHMDDistortionPS, FGlobalShader);
	using FParameters = FHMDDistortionParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

IMPLEMENT_GLOBAL_SHADER(FHMDDistortionPS, "/Engine/Private/PostProcessHMD.usf", "MainPS", SF_Pixel);

class FHMDDistortionVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FHMDDistortionVS);
	SHADER_USE_PARAMETER_STRUCT(FHMDDistortionVS, FGlobalShader);
	using FParameters = FHMDDistortionParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

IMPLEMENT_GLOBAL_SHADER(FHMDDistortionVS, "/Engine/Private/PostProcessHMD.usf", "MainVS", SF_Vertex);

FScreenPassTexture AddDefaultHMDDistortionPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FHMDDistortionInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, ERenderTargetLoadAction::EClear, TEXT("HMD Distortion"));
	}

	FHMDDistortionParameters* PassParameters = GraphBuilder.AllocParameters<FHMDDistortionParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->InputTexture = Inputs.SceneColor.Texture;
	PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Border>::GetRHI();

	check(GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice());
	IHeadMountedDisplay* HMDDevice = GEngine->XRSystem->GetHMDDevice();

	{
		FHeadMountedDisplayPassContext PassContext(GraphBuilder.RHICmdList, View);
		FVector2D EyeToSrcUVScaleValue;
		FVector2D EyeToSrcUVOffsetValue;
		HMDDevice->GetEyeRenderParams_RenderThread(PassContext, EyeToSrcUVScaleValue, EyeToSrcUVOffsetValue);
		PassParameters->EyeToSrcUVScale = FVector2f(EyeToSrcUVScaleValue);		// LWC_TODO: Precision loss
		PassParameters->EyeToSrcUVOffset = FVector2f(EyeToSrcUVOffsetValue);	// LWC_TODO: Precision loss
	}

	TShaderMapRef<FHMDDistortionVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FHMDDistortionPS> PixelShader(View.ShaderMap);

	const FIntRect OutputViewRect(Output.ViewRect);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("HMDDistortion"),
		PassParameters,
		ERDGPassFlags::Raster,
		[VertexShader, PixelShader, PassParameters, OutputViewRect, &View, HMDDevice](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewRect.Min.X, OutputViewRect.Min.Y, 0.0f, OutputViewRect.Max.X, OutputViewRect.Max.Y, 1.0f);

		FScreenPassPipelineState PipelineState(VertexShader,PixelShader);
		PipelineState.VertexDeclaration = GDistortionVertexDeclaration.VertexDeclarationRHI;
		SetScreenPassPipelineState(RHICmdList, PipelineState);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

		FHeadMountedDisplayPassContext PassContext(RHICmdList, View);
		HMDDevice->DrawDistortionMesh_RenderThread(PassContext, PassParameters->InputTexture->Desc.Extent);
	});

	return MoveTemp(Output);
}

FScreenPassTexture AddHMDDistortionPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FHMDDistortionInputs& Inputs)
{
	check(GEngine && GEngine->XRSystem.IsValid());
	checkf(GEngine->XRSystem->GetHMDDevice(), TEXT("EngineShowFlags.HMDDistortion can not be true when IXRTrackingSystem::GetHMDDevice returns null"));

	// First attempt to use a pass from the HMD system.
	FScreenPassTexture Output;
	GEngine->XRSystem->GetHMDDevice()->CreateHMDPostProcessPass_RenderThread(GraphBuilder, View, Inputs, Output);

	if (!Output.IsValid())
	{
		Output = AddDefaultHMDDistortionPass(GraphBuilder, View, Inputs);
	}

	return Output;
}