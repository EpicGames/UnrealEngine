// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessHistogram.cpp: Post processing histogram implementation.
=============================================================================*/

#include "PostProcess/PostProcessHistogram.h"
#include "PostProcess/PostProcessEyeAdaptation.h"
#include "ShaderCompilerCore.h"

#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "SystemTextures.h"
#include "DataDrivenShaderPlatformInfo.h"

TAutoConsoleVariable<int32> CVarUseAtomicHistogram(
	TEXT("r.Histogram.UseAtomic"), 1,
	TEXT("Uses atomic to speed up the generation of the histogram."),
	ECVF_RenderThreadSafe);

namespace
{
class FHistogramCS : public FGlobalShader
{
public:
	// Changing these numbers requires Histogram.usf to be recompiled.
	static const uint32 LoopCountX = 8;
	static const uint32 LoopCountY = 8;

	// we store 4 buckets in one ARGB texel.
	static const uint32 HistogramBucketsPerTexel = 4;

	DECLARE_GLOBAL_SHADER(FHistogramCS);
	SHADER_USE_PARAMETER_STRUCT(FHistogramCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, HistogramRWTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float2>, BilateralGridRWTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, DebugOutput)
		SHADER_PARAMETER(FIntPoint, ThreadGroupCount)
	END_SHADER_PARAMETER_STRUCT()

	class FBilateralGrid : SHADER_PERMUTATION_BOOL("BILATERAL_GRID");
	using FPermutationDomain = TShaderPermutationDomain<FBilateralGrid, AutoExposurePermutation::FCommonDomain>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const AutoExposurePermutation::FCommonDomain& AEPermutationCommon = PermutationVector.Get<AutoExposurePermutation::FCommonDomain>();

		if (!AutoExposurePermutation::ShouldCompileCommonPermutation(AEPermutationCommon))
		{
			return false;
		}

		if (PermutationVector.Get<FBilateralGrid>() && AEPermutationCommon != AutoExposurePermutation::FCommonDomain())
		{
			// bilateral grid permutation doesn't use AE permutations
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const bool bBilateralGrid = PermutationVector.Get<FBilateralGrid>();

		const FIntPoint ThreadGroupSize = GetThreadGroupSize(bBilateralGrid);
		const uint32 HistogramSize = GetHistogramSize(bBilateralGrid);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSize.X);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSize.Y);
		OutEnvironment.SetDefine(TEXT("LOOP_SIZEX"), LoopCountX);
		OutEnvironment.SetDefine(TEXT("LOOP_SIZEY"), LoopCountY);
		OutEnvironment.SetDefine(TEXT("HISTOGRAM_SIZE"), HistogramSize);

		OutEnvironment.CompilerFlags.Add( CFLAG_StandardOptimization );
	}

	static uint32 GetHistogramSize(bool bBilateralGrid)
	{
		return bBilateralGrid ? 32 : 64;
	}

	static FIntPoint GetThreadGroupSize(bool bBilateralGrid)
	{
		return bBilateralGrid ? FIntPoint(8, 8) : FIntPoint(8, 4);
	}

	static FIntPoint GetTexelsPerThreadGroup( bool bBilateralGrid)
	{
		// One ThreadGroup ThreadGroupSizeX*ThreadGroupSizeY processes blocks of size LoopCountX*LoopCountY
		return GetThreadGroupSize(bBilateralGrid) * FIntPoint(LoopCountX, LoopCountY);
	}

	static FIntPoint GetThreadGroupCount(FIntPoint InputExtent, bool bBilateralGrid)
	{
		const FIntPoint TexelsPerThreadGroup = GetTexelsPerThreadGroup(bBilateralGrid);

		return FIntPoint::DivideAndRoundUp(InputExtent, TexelsPerThreadGroup);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHistogramCS, "/Engine/Private/PostProcessHistogram.usf", "MainCS", SF_Compute);

class FHistogramReducePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FHistogramReducePS);
	SHADER_USE_PARAMETER_STRUCT(FHistogramReducePS, FGlobalShader);
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER(uint32, LoopSize)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	// Uses full float4 to get best quality for smooth eye adaptation transitions.
	static const EPixelFormat OutputFormat = PF_A32B32G32R32F;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetRenderTargetOutputFormat(0, OutputFormat);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHistogramReducePS, "/Engine/Private/PostProcessHistogramReduce.usf", "MainPS", SF_Pixel);

class FHistogramAtomicCS : public FGlobalShader
{
public:
	// Changing these numbers requires Histogram.usf to be recompiled.
	static const uint32 ThreadGroupSizeX = 64; //tested at 32, 64, 128 all fast.  but 2 or 256 were DISASTER.
	static const uint32 HistogramSize = 64;		//should be power of two

	DECLARE_GLOBAL_SHADER(FHistogramAtomicCS);
	SHADER_USE_PARAMETER_STRUCT(FHistogramAtomicCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER(FIntPoint, ThreadGroupCount)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, HistogramScatter64Output)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, HistogramScatter32Output)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, DebugOutput)

		SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
		SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<AutoExposurePermutation::FCommonDomain>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
			&& AutoExposurePermutation::ShouldCompileCommonPermutation(PermutationVector.Get<AutoExposurePermutation::FCommonDomain>());
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("HISTOGRAM_SIZE"), HistogramSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHistogramAtomicCS, "/Engine/Private/Histogram.usf", "MainAtomicCS", SF_Compute);

class FHistogramAtomicConvertCS : public FGlobalShader
{
public:
	// Changing these numbers requires Histogram.usf to be recompiled.
	static const uint32 ThreadGroupSizeX = FHistogramAtomicCS::ThreadGroupSizeX;
	static const uint32 HistogramSize = FHistogramAtomicCS::HistogramSize;

	// /4 as we store 4 buckets in one ARGB texel.
	static const uint32 HistogramTexelCount = HistogramSize;

	// The number of texels on each axis processed by a single thread group.
	//	static const FIntPoint TexelsPerThreadGroup;

	DECLARE_GLOBAL_SHADER(FHistogramAtomicConvertCS);
	SHADER_USE_PARAMETER_STRUCT(FHistogramAtomicConvertCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HistogramScatter64Texture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HistogramScatter32Texture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, HistogramOutput)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("HISTOGRAM_SIZE"), HistogramSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHistogramAtomicConvertCS, "/Engine/Private/Histogram.usf", "HistogramConvertCS", SF_Compute);

} //! namespace

static FRDGTextureRef AddHistogramLegacyPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	const FScreenPassTextureSlice& SceneColor,
	FRDGBufferRef EyeAdaptationBuffer)
{
	check(SceneColor.IsValid());
	check(EyeAdaptationBuffer);

	const FIntPoint HistogramThreadGroupCount = FHistogramCS::GetThreadGroupCount(SceneColor.ViewRect.Size(), false);
	const uint32 HistogramThreadGroupCountTotal = HistogramThreadGroupCount.X * HistogramThreadGroupCount.Y;
	const uint32 HistogramTexelCount = FHistogramCS::GetHistogramSize(false) / FHistogramCS::HistogramBucketsPerTexel;

	FRDGTextureRef HistogramTexture = nullptr;

	RDG_EVENT_SCOPE(GraphBuilder, "Histogram");

	// First pass outputs one flattened histogram per group.
	{
		const FIntPoint TextureExtent = FIntPoint(HistogramTexelCount, HistogramThreadGroupCountTotal);

		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(
			TextureExtent,
			PF_FloatRGBA,
			FClearValueBinding::None,
			GFastVRamConfig.Histogram | TexCreate_RenderTargetable | TexCreate_UAV | TexCreate_ShaderResource);

		HistogramTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("Histogram"));

		FHistogramCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHistogramCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(SceneColor));
		PassParameters->InputSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->InputTexture = SceneColor.TextureSRV;
		PassParameters->HistogramRWTexture = GraphBuilder.CreateUAV(HistogramTexture);
		PassParameters->ThreadGroupCount = HistogramThreadGroupCount;
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(SceneColor.ViewRect.Size(), PF_R16F, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource);
			FRDGTextureRef DebugOutputTexture = GraphBuilder.CreateTexture(Desc, TEXT("EyeAdaptation_DebugOutput"));
			PassParameters->DebugOutput = GraphBuilder.CreateUAV(DebugOutputTexture);
		}

		FHistogramCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<AutoExposurePermutation::FCommonDomain>(AutoExposurePermutation::BuildCommonPermutationDomain());

		auto ComputeShader = View.ShaderMap->GetShader<FHistogramCS>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Histogram %dx%d (CS)", SceneColor.ViewRect.Width(), SceneColor.ViewRect.Height()),
			ComputeShader,
			PassParameters,
			FIntVector(HistogramThreadGroupCount.X, HistogramThreadGroupCount.Y, 1));
	}

	FRDGTextureRef HistogramReduceTexture = nullptr;

	// Second pass further reduces the histogram to a single line. The second line contains the eye adaptation value (two line texture).
	{
		const FIntPoint TextureExtent = FIntPoint(HistogramTexelCount, 2);

		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(
			TextureExtent,
			FHistogramReducePS::OutputFormat,
			FClearValueBinding::None,
			GFastVRamConfig.HistogramReduce | TexCreate_RenderTargetable | TexCreate_ShaderResource);

		HistogramReduceTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("HistogramReduce"));

		const FScreenPassTextureViewport InputViewport(HistogramTexture);
		const FScreenPassTextureViewport OutputViewport(HistogramReduceTexture);

		FHistogramReducePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHistogramReducePS::FParameters>();
		PassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
		PassParameters->InputTexture = HistogramTexture;
		PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->LoopSize = HistogramThreadGroupCountTotal;
		PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(EyeAdaptationBuffer);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(HistogramReduceTexture, ERenderTargetLoadAction::ENoAction);

		TShaderMapRef<FHistogramReducePS> PixelShader(View.ShaderMap);

		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("HistogramReduce %dx%d (PS)", InputViewport.Extent.X, InputViewport.Extent.Y),
			View,
			OutputViewport,
			InputViewport,
			PixelShader,
			PassParameters);
	}

	return HistogramReduceTexture;
}

static FRDGTextureRef AddHistogramAtomicPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	const FScreenPassTextureSlice& SceneColor,
	const FSceneTextureParameters& SceneTextures,
	FRDGBufferRef EyeAdaptationBuffer)
{
	check(SceneColor.IsValid());
	check(EyeAdaptationBuffer);

	RDG_EVENT_SCOPE(GraphBuilder, "Histogram");

	//FRDGTextureRef HistogramScatter64Texture;
	FRDGTextureRef HistogramScatter32Texture;
	{
		// {
		// 	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
		// 		FIntPoint(FHistogramAtomicCS::HistogramSize, 1),
		// 		PF_R32G32_UINT,
		// 		FClearValueBinding::None,
		// 		TexCreate_UAV | TexCreate_AtomicCompatible);
		// 
		// 	HistogramScatter64Texture = GraphBuilder.CreateTexture(Desc, TEXT("Histogram.Scatter64"));
		// }

		{
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				FIntPoint(FHistogramAtomicCS::HistogramSize * 2, 1),
				PF_R32_UINT,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_UAV | TexCreate_AtomicCompatible);

			HistogramScatter32Texture = GraphBuilder.CreateTexture(Desc, TEXT("Histogram.Scatter32"));
		}

		FHistogramAtomicCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHistogramAtomicCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = SceneTextures;
		PassParameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(SceneColor));
		PassParameters->InputTexture = SceneColor.TextureSRV;
		PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(EyeAdaptationBuffer);
		PassParameters->ThreadGroupCount = FIntPoint(SceneColor.ViewRect.Size().Y, 1);
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		// PassParameters->HistogramScatter64Output = GraphBuilder->CreateUAV(HistogramScatter64Texture);
		PassParameters->HistogramScatter32Output = GraphBuilder.CreateUAV(HistogramScatter32Texture);
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(SceneColor.ViewRect.Size(), PF_R16F, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource);
			FRDGTextureRef DebugOutputTexture = GraphBuilder.CreateTexture(Desc, TEXT("EyeAdaptation_DebugOutput"));
			PassParameters->DebugOutput = GraphBuilder.CreateUAV(DebugOutputTexture);
		}

		PassParameters->PreIntegratedGF = GSystemTextures.PreintegratedGF->GetRHI();
		PassParameters->PreIntegratedGFSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		//clear the temp textures
		uint32 ClearValues[4] = { 0, 0, 0, 0 };
		// AddClearUAVPass(GraphBuilder, PassParameters->HistogramScatter64Output, ClearValues);
		AddClearUAVPass(GraphBuilder, PassParameters->HistogramScatter32Output, ClearValues);

		FHistogramAtomicCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<AutoExposurePermutation::FCommonDomain>(AutoExposurePermutation::BuildCommonPermutationDomain());

		auto ComputeShader = View.ShaderMap->GetShader<FHistogramAtomicCS>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Histogram Atomic %dx%d (CS)", SceneColor.ViewRect.Width(), SceneColor.ViewRect.Height()),
			ComputeShader,
			PassParameters,
			FIntVector(PassParameters->ThreadGroupCount.X, PassParameters->ThreadGroupCount.Y, 1));
	}

	FRDGTextureRef HistogramTexture;
	{
		{
			const FRDGTextureDesc TextureDescGather = FRDGTextureDesc::Create2D(
				FIntPoint(FHistogramAtomicCS::HistogramSize / FHistogramCS::HistogramBucketsPerTexel, 2),
				PF_A32B32G32R32F,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_UAV);

			HistogramTexture = GraphBuilder.CreateTexture(TextureDescGather, TEXT("Histogram"));
		}

		FHistogramAtomicConvertCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHistogramAtomicConvertCS::FParameters>();
		PassParameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(SceneColor));
		PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(EyeAdaptationBuffer);
		// PassParameters->HistogramScatter64Texture = HistogramScatter64Texture;
		PassParameters->HistogramScatter32Texture = HistogramScatter32Texture;
		PassParameters->HistogramOutput = GraphBuilder.CreateUAV(HistogramTexture);

		uint32 NumGroupsRequired = FMath::Max(1U, FHistogramAtomicConvertCS::HistogramSize / FHistogramAtomicConvertCS::ThreadGroupSizeX);

		const FIntPoint HistogramConvertThreadGroupCount = FIntPoint(NumGroupsRequired, 1);

		TShaderMapRef<FHistogramAtomicConvertCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Histogram Convert"),
			ComputeShader,
			PassParameters,
			FIntVector(HistogramConvertThreadGroupCount.X, HistogramConvertThreadGroupCount.Y, 1));
	}

	return HistogramTexture;
}


FRDGTextureRef AddHistogramPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FScreenPassTextureSlice SceneColor,
	const FSceneTextureParameters& SceneTextures,
	FRDGBufferRef EyeAdaptationBuffer)
{
	if (CVarUseAtomicHistogram.GetValueOnRenderThread() == 1)
	{
		return AddHistogramAtomicPass(
			GraphBuilder,
			View,
			EyeAdaptationParameters,
			SceneColor,
			SceneTextures,
			EyeAdaptationBuffer);
	}
	else
	{
		return AddHistogramLegacyPass(
			GraphBuilder,
			View,
			EyeAdaptationParameters,
			SceneColor,
			EyeAdaptationBuffer);
	}
}

FVector2f GetLocalExposureBilateralGridUVScale(const FIntPoint ViewRectSize)
{
	const FIntPoint TexelsPerThreadGroup = FHistogramCS::GetTexelsPerThreadGroup(true);

	const FIntPoint ThreadGroupCount = FHistogramCS::GetThreadGroupCount(ViewRectSize, true);

	return FVector2f(float(ViewRectSize.X) / TexelsPerThreadGroup.X / ThreadGroupCount.X, float(ViewRectSize.Y) / TexelsPerThreadGroup.Y / ThreadGroupCount.Y);
}

FRDGTextureRef AddLocalExposurePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FScreenPassTextureSlice SceneColor)
{
	check(SceneColor.IsValid());

	// Bilateral grid requires PF_G32R32F UAV. If the format doesn't support it, skip.
	if (!UE::PixelFormat::HasCapabilities(PF_G32R32F, EPixelFormatCapabilities::UAV))
	{
		return nullptr;
	}

	const FIntPoint ThreadGroupSize = FHistogramCS::GetThreadGroupSize(true);
	const FIntPoint ThreadGroupCount = FHistogramCS::GetThreadGroupCount(SceneColor.ViewRect.Size(), true);
	const FIntPoint NumTiles = ThreadGroupCount;

	FRDGTextureRef LocalExposureTexture = nullptr;

	RDG_EVENT_SCOPE(GraphBuilder, "LocalExposure");

	{
		const FIntVector TextureExtent = FIntVector(NumTiles.X, NumTiles.Y, FHistogramCS::GetHistogramSize(true));

		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create3D(
			TextureExtent,
			PF_G32R32F,
			FClearValueBinding::None,
			TexCreate_UAV | TexCreate_ShaderResource);

		LocalExposureTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("LocalExposure"));

		auto* PassParameters = GraphBuilder.AllocParameters<FHistogramCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(SceneColor));
		PassParameters->InputSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->InputTexture = SceneColor.TextureSRV;
		PassParameters->BilateralGridRWTexture = GraphBuilder.CreateUAV(LocalExposureTexture);
		PassParameters->ThreadGroupCount = ThreadGroupCount;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

		FHistogramCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FHistogramCS::FBilateralGrid>(true);

		auto ComputeShader = View.ShaderMap->GetShader<FHistogramCS>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FLocalExposure %dx%d (CS)", SceneColor.ViewRect.Width(), SceneColor.ViewRect.Height()),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FIntVector(ThreadGroupCount.X, ThreadGroupCount.Y, 1));
	}

	return LocalExposureTexture;
}
