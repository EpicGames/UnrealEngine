// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderGraphFwd.h"

class FEyeAdaptationParameters;
class FSceneTextureParameters;
class FViewInfo;
struct FScreenPassTextureSlice;

FRDGTextureRef AddHistogramPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FScreenPassTextureSlice SceneColor,
	const FSceneTextureParameters& SceneTextures,
	FRDGBufferRef EyeAdaptationBuffer);

FRDGTextureRef AddLocalExposurePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FScreenPassTextureSlice SceneColor);