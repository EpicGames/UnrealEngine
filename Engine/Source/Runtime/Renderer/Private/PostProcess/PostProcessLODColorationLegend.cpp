// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessLODColorationLegend.h"
#include "UnrealEngine.h"
#include "DebugViewModeRendering.h"
#include "SceneRendering.h"

namespace UE::Private
{
static void DrawLODBox(FCanvas& Canvas, int32 PosX, int32 PosY, int32 SizeX, int32 SizeY, const FLinearColor& Color, int32 LODValue)
{
	// the color coded tile
	Canvas.DrawTile(PosX, PosY, SizeX, SizeY, 0, 0, 1, 1, Color);

	// and the lil number label that goes on it
	const UFont* StatsFont = GetStatsFont();
	const FString LODString = FString::FromInt(LODValue);
	int32 TextSizeX = 0, TextSizeY = 0;
	StringSize(StatsFont, TextSizeX, TextSizeY, LODString);
	Canvas.DrawShadowedText(
		PosX + (SizeX / 2) - (TextSizeX / 2), 
		PosY + (SizeY / 2) - (TextSizeY / 2), 
		FText::FromString(LODString), 
		StatsFont, 
		FLinearColor(0.7f, 0.7f, 0.7f));
}
} //! namespace

FScreenPassTexture AddLODColorationLegendPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FLODColorationLegendInputs& Inputs)
{
	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (Output.IsValid())
	{
		AddDrawTexturePass(GraphBuilder, View, Inputs.SceneColor, Output);
	}
	else
	{
		Output = FScreenPassRenderTarget(Inputs.SceneColor, ERenderTargetLoadAction::ELoad);
	}

	const TArrayView<const FLinearColor> Colors = Inputs.Colors;
	const FString Title = Inputs.Title;
	const FString LeftLabel = Inputs.LeftLabel;
	const FString RightLabel = Inputs.RightLabel;

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("LODColorationLegend"), View, Output, [Colors, Title, LeftLabel, RightLabel](FCanvas& Canvas)
	{
		const FIntRect& OutputViewRect = Canvas.GetViewRect();
		// adjust canvas scale for DPI
		const float DPIScale = Canvas.GetDPIScale();
		Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale)* Canvas.CalcBaseTransform2D(OutputViewRect.Width(), OutputViewRect.Height())));

		const int32 NumColors = Colors.Num();
		const UFont* StatsFont = GetStatsFont();

		// some named variables make size calculations easy to follow
		const int32 BorderWidth = 2;
		const int32 OffsetFromViewBottom = 50;
		const int32 SectionWidth = 50;
		const int32 LegendHeight = 20;

		// figuring out size + placement
		const int32 LegendWidth = SectionWidth * NumColors;
		const int32 LegendHalfWidth = LegendWidth / 2;

		const int32 LegendPosX = OutputViewRect.Min.X + (OutputViewRect.Width() / 2) - LegendHalfWidth;
		const int32 LegendPosY = OutputViewRect.Max.Y - OffsetFromViewBottom;

		// draw title above the legend bar
		if (!Title.IsEmpty())
		{
			int32 TitleSizeX = 0, TitleSizeY = 0;
			StringSize(StatsFont, TitleSizeX, TitleSizeY, Title);
			Canvas.DrawShadowedText(
				LegendPosX + LegendHalfWidth - TitleSizeX * 0.5f,
				LegendPosY - TitleSizeY - 4.0f,
				FText::FromString(Title),
				StatsFont,
				FLinearColor::White);
		}

		// draw a thin black border around our legend
		Canvas.DrawTile(LegendPosX - BorderWidth, LegendPosY - BorderWidth, LegendWidth + BorderWidth*2, LegendHeight + BorderWidth*2, 0, 0, 1, 1, FLinearColor::Black);

		// and then the color coded boxes with their associated LOD number
		for (int32 i = 0; i < NumColors; ++i)
		{
			UE::Private::DrawLODBox(Canvas, LegendPosX + (SectionWidth * i), LegendPosY, SectionWidth, LegendHeight, Colors[i], i);
		}

		// draw endpoint labels below the legend bar
		const float LabelY = LegendPosY + LegendHeight + 4.0f;
		if (!LeftLabel.IsEmpty())
		{
			Canvas.DrawShadowedText(
				LegendPosX,
				LabelY,
				FText::FromString(LeftLabel),
				StatsFont,
				FLinearColor::White);
		}
		if (!RightLabel.IsEmpty())
		{
			int32 RightSizeX = 0, RightSizeY = 0;
			StringSize(StatsFont, RightSizeX, RightSizeY, RightLabel);
			Canvas.DrawShadowedText(
				LegendPosX + LegendWidth - RightSizeX,
				LabelY,
				FText::FromString(RightLabel),
				StatsFont,
				FLinearColor::White);
		}
	});

	return MoveTemp(Output);
}
