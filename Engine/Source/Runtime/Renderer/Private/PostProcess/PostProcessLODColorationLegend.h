// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

struct FLODColorationLegendInputs
{
	FScreenPassRenderTarget OverrideOutput;
	FScreenPassTexture SceneColor;
	TArrayView<const FLinearColor> Colors;

	/** Optional title text displayed above the legend bar. */
	FString Title;
	/** Optional label for the left end of the legend (deficit 0). */
	FString LeftLabel;
	/** Optional label for the right end of the legend (max deficit). */
	FString RightLabel;
};

FScreenPassTexture AddLODColorationLegendPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FLODColorationLegendInputs& Inputs);