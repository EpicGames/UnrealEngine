// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessLensFlares.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "GlobalRenderResources.h"
#include "PostProcess/PostProcessDownsample.h"
#include "PixelShaderUtils.h"
#include "TextureResource.h"
#include "PostProcess/SceneFilterRendering.h"
#include "SceneRendering.h"

DECLARE_GPU_STAT(LensFlare);

namespace
{
const int32 GLensFlareQuadsPerInstance = 4;

TAutoConsoleVariable<int32> CVarLensFlareQuality(
	TEXT("r.LensFlareQuality"),
	2,
	TEXT(" 0: off but best for performance\n")
	TEXT(" 1: low quality with good performance\n")
	TEXT(" 2: good quality (default)\n")
	TEXT(" 3: very good quality but bad performance"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

// The RDG inputs shared by all lens flare passes.
BEGIN_SHADER_PARAMETER_STRUCT(FLensFlarePassParameters, )
	SHADER_PARAMETER_RDG_TEXTURE_NON_PIXEL_SRV(Texture2D, InputTexture)
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FLensFlareShader : public FGlobalShader
{
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("QUADS_PER_INSTANCE"), GLensFlareQuadsPerInstance);
	}

	FLensFlareShader() = default;
	FLensFlareShader(const CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

class FLensFlareBlurVS : public FLensFlareShader
{
public:
	DECLARE_GLOBAL_SHADER(FLensFlareBlurVS);
	SHADER_USE_PARAMETER_STRUCT(FLensFlareBlurVS, FLensFlareShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_NON_PIXEL_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER(FIntPoint, TileCount)
		SHADER_PARAMETER(uint32, TileSize)
		SHADER_PARAMETER(float, KernelSize)
		SHADER_PARAMETER(float, KernelAreaInverse)
		SHADER_PARAMETER(float, Threshold)
		SHADER_PARAMETER(float, GuardBandScaleInverse)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FLensFlareBlurVS, "/Engine/Private/PostProcessLensFlares.usf", "LensFlareBlurVS", SF_Vertex);

class FLensFlareBlurPS : public FLensFlareShader
{
public:
	DECLARE_GLOBAL_SHADER(FLensFlareBlurPS);
	SHADER_USE_PARAMETER_STRUCT(FLensFlareBlurPS, FLensFlareShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, BokehTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BokehSampler)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FLensFlareBlurPS, "/Engine/Private/PostProcessLensFlares.usf", "LensFlareBlurPS", SF_Pixel);

class FLensFlareCompositePS : public FLensFlareShader
{
public:
	DECLARE_GLOBAL_SHADER(FLensFlareCompositePS);
	SHADER_USE_PARAMETER_STRUCT(FLensFlareCompositePS, FLensFlareShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FLensFlarePassParameters, Pass)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER(FLinearColor, FlareColor)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}
};

IMPLEMENT_GLOBAL_SHADER(FLensFlareCompositePS, "/Engine/Private/PostProcessLensFlares.usf", "LensFlareCompositePS", SF_Pixel);

class FLensFlareCopyBloomPS : public FLensFlareShader
{
	DECLARE_GLOBAL_SHADER(FLensFlareCopyBloomPS);
	SHADER_USE_PARAMETER_STRUCT(FLensFlareCopyBloomPS, FLensFlareShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FLensFlarePassParameters, Pass)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FLensFlareCopyBloomPS, "/Engine/Private/PostProcessLensFlares.usf", "LensFlareCopyBloomPS", SF_Pixel);
} //!namespace

ELensFlareQuality GetLensFlareQuality()
{
	return static_cast<ELensFlareQuality>(FMath::Clamp(
		CVarLensFlareQuality.GetValueOnRenderThread(),
		static_cast<int32>(ELensFlareQuality::Disabled),
		static_cast<int32>(ELensFlareQuality::MAX) - 1));
}

class FLensFlareTransitionBuffer : public FVertexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateStructured(TEXT("LensFlareTransitionBuffer"), sizeof(uint32) * 4, sizeof(uint32))
			.AddUsage(BUF_Static | BUF_ShaderResource | BUF_UnorderedAccess)
			.DetermineInitialState();

		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}
};

FLensFlareTransitionBuffer* GLensFlareTransitionBuffer = new TGlobalResource<FLensFlareTransitionBuffer, FRenderResource::EInitPhase::Pre>;

FScreenPassTexture AddLensFlaresPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FLensFlareInputs& Inputs)
{
	check(Inputs.Flare.IsValid());
	check(Inputs.Bloom.IsValid());
	check(Inputs.BokehShapeTexture);
	check(Inputs.LensFlareCount <= FLensFlareInputs::LensFlareCountMax);
	check(Inputs.TintColorsPerFlare.Num() == Inputs.LensFlareCount);
	check(Inputs.BokehSizePercent > 0.0f);
	check(Inputs.Intensity > 0.0f);
	check(Inputs.LensFlareCount > 0);

	RDG_EVENT_SCOPE(GraphBuilder, "LensFlares");

	const float PercentToScale = 0.01f;

	// This constant scales the lens flare blur viewport so that the bokeh shape doesn't clip. However,
	// changing this constant will not preserve energy properly. The kernel size should also change based
	// on this constant. This is not implemented in order to provide preserve the look of content. The masking
	// behavior is also affected by this constant.
	const float GuardBandScale = 2.0f;

	const FScreenPassTextureViewport BloomViewport(Inputs.Bloom);
	const FIntPoint BloomViewportSize(BloomViewport.Rect.Size());
	const FScreenPassTextureViewport FlareViewport(Inputs.Flare);
	const FIntPoint FlareViewSize = FlareViewport.Rect.Size();

	FRHIBlendState* AdditiveBlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FRDGTextureRef BlurOutputTexture = nullptr;

	// Initialize the blur output texture.
	{
		const FRDGTextureDesc& InputDesc = Inputs.Flare.TextureSRV->Desc.Texture->Desc;
		FRDGTextureDesc BlurOutputDesc = FRDGTextureDesc::Create2D(
			InputDesc.Extent,
			PF_FloatRGBA,
			FClearValueBinding(FLinearColor::Transparent),
			/* InFlags = */ TexCreate_ShaderResource | TexCreate_RenderTargetable);

		BlurOutputTexture = GraphBuilder.CreateTexture(BlurOutputDesc, TEXT("LensFlareBlur"));
	}

	// Lens flare blur pass. Rasterizes a bokeh quad for each pixel on the screen based on the intensity threshold.
	{
		const uint32 TileSizeInPixels = 1;

		const FIntPoint TileCount = FlareViewSize / TileSizeInPixels;

		const float KernelSize = (Inputs.BokehSizePercent * static_cast<float>(FlareViewSize.X)) * PercentToScale;

		FLensFlarePassParameters* PassParameters = GraphBuilder.AllocParameters<FLensFlarePassParameters>();
		PassParameters->InputTexture = Inputs.Flare.TextureSRV;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(BlurOutputTexture, ERenderTargetLoadAction::EClear);

		// Setup vertex shader parameters.
		FLensFlareBlurVS::FParameters VertexParameters;
		VertexParameters.InputTexture = Inputs.Flare.TextureSRV;
		VertexParameters.Input = GetScreenPassTextureViewportParameters(FlareViewport);
		VertexParameters.InputSampler = BilinearClampSampler;
		VertexParameters.TileCount = TileCount;
		VertexParameters.TileSize = TileSizeInPixels;
		VertexParameters.KernelSize = KernelSize;
		VertexParameters.Threshold = Inputs.Threshold;
		VertexParameters.KernelAreaInverse = 1.0f / FMath::Max(1.0f, KernelSize * KernelSize);
		VertexParameters.GuardBandScaleInverse = 1.0f / GuardBandScale;

		// Setup pixel shader parameters.
		FLensFlareBlurPS::FParameters PixelParameters;
		PixelParameters.BokehTexture = Inputs.BokehShapeTexture;
		PixelParameters.BokehSampler = BilinearClampSampler;

		TShaderMapRef<FLensFlareBlurVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FLensFlareBlurPS> PixelShader(View.ShaderMap);

		ERDGPassFlags LensFlareBlurFlags = ERDGPassFlags::Raster;
		LensFlareBlurFlags |= GRHINeedsSRVGraphicsNonPixelWorkaround ? ERDGPassFlags::SkipRenderPass : ERDGPassFlags::None;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("LensFlareBlur %dx%d", FlareViewSize.X, FlareViewSize.Y),
			PassParameters,
			LensFlareBlurFlags,
			[VertexShader, PixelShader, VertexParameters, PixelParameters, PassParameters, AdditiveBlendState, FlareViewport, TileCount](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				// Viewport is the same as the input.
				RHICmdList.SetViewport(
					FlareViewport.Rect.Min.X,
					FlareViewport.Rect.Min.Y,
					0.0f,
					FlareViewport.Rect.Max.X,
					FlareViewport.Rect.Max.Y,
					1.0f);

				// Force a transition to SRVGraphicsNonPixel on an Empty Structured Buffer to account for VS accessing the texture (using NON_PIXEL_SRV in struct gets merged)
				if (GRHINeedsSRVGraphicsNonPixelWorkaround)
				{
					RHICmdList.Transition(FRHITransitionInfo(GLensFlareTransitionBuffer->GetRHI(), ERHIAccess::UAVMask, ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVGraphicsNonPixel));

					FRHIRenderPassInfo RPInfo(PassParameters->RenderTargets[0].GetTexture()->GetRHI(), ERenderTargetActions::Clear_Store);
					RHICmdList.BeginRenderPass(RPInfo, TEXT("LensFlareBlur"));
				}
			
				// Apply additive blending pipeline state.
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = AdditiveBlendState;
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VertexParameters);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PixelParameters);
				// Emit an instanced quad draw call on the order of the number of pixels on the screen.
				RHICmdList.SetStreamSource(0, nullptr, 0);
				RHICmdList.DrawPrimitive(0, 2, FMath::DivideAndRoundUp(TileCount.X * TileCount.Y, GLensFlareQuadsPerInstance));

				if (GRHINeedsSRVGraphicsNonPixelWorkaround)
				{
					RHICmdList.EndRenderPass();
					RHICmdList.Transition(FRHITransitionInfo(GLensFlareTransitionBuffer->GetRHI(), ERHIAccess::VertexOrIndexBuffer | ERHIAccess::SRVGraphicsNonPixel, ERHIAccess::UAVMask));
				}
			});
	}

	FRDGTextureRef LensFlareTexture = nullptr;

	// Initialize the lens flare output texture.
	{
		const FRDGTextureDesc& InputDesc = Inputs.Bloom.TextureSRV->Desc.Texture->Desc;
		FRDGTextureDesc LensFlareTextureDesc = FRDGTextureDesc::Create2D(
			InputDesc.Extent,
			InputDesc.Format,
			FClearValueBinding(FLinearColor::Transparent),
			/* InFlags = */ TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_DisableDCC);

		LensFlareTexture = GraphBuilder.CreateTexture(LensFlareTextureDesc, TEXT("LensFlareTexture"));
	}

	ERenderTargetLoadAction LensFlareLoadAction = ERenderTargetLoadAction::ELoad;

	if (Inputs.bCompositeWithBloom)
	{
		FLensFlareCopyBloomPS::FParameters* CopyPassParameters = GraphBuilder.AllocParameters<FLensFlareCopyBloomPS::FParameters>();
		CopyPassParameters->Pass.InputTexture = Inputs.Bloom.TextureSRV;
		CopyPassParameters->Pass.RenderTargets[0] = FRenderTargetBinding(LensFlareTexture, ERenderTargetLoadAction::ENoAction);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("LensFlareCopyBloom %dx%d", Inputs.Bloom.ViewRect.Size().X, Inputs.Bloom.ViewRect.Size().Y),
			View.ShaderMap->GetShader<FLensFlareCopyBloomPS>(),
			CopyPassParameters,
			Inputs.Bloom.ViewRect);
	}
	else
	{
		// Clear to transparent black on the first render pass.
		LensFlareLoadAction = ERenderTargetLoadAction::EClear;
	}

	const FIntRect OutputViewRect = BloomViewport.Rect;

	const FVector2f OutputCenter = FVector2f(OutputViewRect.Min + OutputViewRect.Max) / 2;

	// Scales normalized flare tint alpha to a viewport scale factor.
	const float AlphaScale = static_cast<float>(Inputs.LensFlareCount - 1);
	const float AlphaBias = -AlphaScale * 0.5f;

	// Term to normalize the color based on the scale of the guard band.
	const float GuardBandAreaInverse = 1.0f / (GuardBandScale * GuardBandScale);

	const FLinearColor LensFlareHDRColor = Inputs.TintColor * Inputs.Intensity * GuardBandAreaInverse;

	// Render a scaled quad for each lens flare, additively blended onto the target.
	for (uint32 LensFlareIndex = 0; LensFlareIndex < Inputs.LensFlareCount; ++LensFlareIndex)
	{
		const FLinearColor LensFlareTint = Inputs.TintColorsPerFlare[LensFlareIndex];
		
		FLensFlareCompositePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FLensFlareCompositePS::FParameters>();
		PassParameters->Pass.InputTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(BlurOutputTexture));
		PassParameters->Pass.View = View.GetShaderParameters();
		PassParameters->Pass.RenderTargets[0] = FRenderTargetBinding(LensFlareTexture, LensFlareLoadAction);
		PassParameters->InputSampler = BilinearClampSampler;
		PassParameters->FlareColor = LensFlareHDRColor * LensFlareTint;

		// Alpha of the tint color is used to derive a scale of the flare quad.
		const float FinalOutputScale = (LensFlareTint.A * AlphaScale + AlphaBias) * GuardBandScale;

		const FVector2f QuadSize = FVector2f(OutputViewRect.Size()) * FinalOutputScale;

		const FVector2f QuadOffset = OutputCenter - 0.5f * QuadSize;

		TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FLensFlareCompositePS> PixelShader(View.ShaderMap);

		const FScreenPassPipelineState PipelineState(VertexShader, PixelShader, AdditiveBlendState);

		// This pass rasterizes the lens flare quad scaled and centered within the viewport.
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("LensFlare%d", LensFlareIndex),
			PassParameters,
			ERDGPassFlags::Raster,
			[PixelShader, PassParameters, OutputViewRect, FlareViewport, QuadSize, QuadOffset, PipelineState] (FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(OutputViewRect.Min.X, OutputViewRect.Min.Y, 0.0f, OutputViewRect.Max.X, OutputViewRect.Max.Y, 1.0f);

			SetScreenPassPipelineState(RHICmdList, PipelineState);

			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			DrawRectangle(
				RHICmdList,
				QuadOffset.X,
				QuadOffset.Y,
				QuadSize.X,
				QuadSize.Y,
				FlareViewport.Rect.Min.X,
				FlareViewport.Rect.Min.Y,
				FlareViewport.Rect.Width(),
				FlareViewport.Rect.Height(),
				OutputViewRect.Size(),
				FlareViewport.Extent,
				PipelineState.VertexShader,
				EDRF_Default);
		});

		// All subsequent passes must load.
		LensFlareLoadAction = ERenderTargetLoadAction::ELoad;
	}

	return FScreenPassTexture(LensFlareTexture, OutputViewRect);
}

bool IsLensFlaresEnabled(const FViewInfo& View)
{
	const ELensFlareQuality LensFlareQuality = GetLensFlareQuality();

	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

	return (LensFlareQuality != ELensFlareQuality::Disabled &&
		!Settings.LensFlareTint.IsAlmostBlack() &&
		Settings.LensFlareBokehSize > SMALL_NUMBER &&
		Settings.LensFlareIntensity > SMALL_NUMBER);
}

FScreenPassTexture AddLensFlaresPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture Bloom,
	FScreenPassTextureSlice QualitySceneDownsample,
	FScreenPassTextureSlice DefaultSceneDownsample)
{
	ensure(IsLensFlaresEnabled(View));

	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

	RDG_EVENT_SCOPE_STAT(GraphBuilder, LensFlare, "LensFlare");

	FRHITexture* BokehTextureRHI = GWhiteTexture->TextureRHI;

	if (GEngine->DefaultBokehTexture)
	{
		FTextureResource* BokehTextureResource = GEngine->DefaultBokehTexture->GetResource();

		if (BokehTextureResource && BokehTextureResource->TextureRHI)
		{
			BokehTextureRHI = BokehTextureResource->TextureRHI;
		}
	}

	if (Settings.LensFlareBokehShape)
	{
		FTextureResource* BokehTextureResource = Settings.LensFlareBokehShape->GetResource();

		if (BokehTextureResource && BokehTextureResource->TextureRHI)
		{
			BokehTextureRHI = BokehTextureResource->TextureRHI;
		}
	}

	FLensFlareInputs LensFlareInputs;
	LensFlareInputs.Bloom = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, Bloom);
	LensFlareInputs.Flare = QualitySceneDownsample;
	LensFlareInputs.BokehShapeTexture = BokehTextureRHI;
	LensFlareInputs.TintColorsPerFlare = Settings.LensFlareTints;
	LensFlareInputs.TintColor = Settings.LensFlareTint;
	LensFlareInputs.BokehSizePercent = Settings.LensFlareBokehSize;
	LensFlareInputs.Intensity = Settings.LensFlareIntensity * Settings.BloomIntensity;
	LensFlareInputs.Threshold = Settings.LensFlareThreshold;

	// If a bloom output texture isn't available, substitute the half resolution scene color instead, but disable bloom
	// composition. The pass needs a primary input in order to access the image descriptor and viewport for output.
	if (!Bloom.IsValid())
	{
		LensFlareInputs.Bloom = DefaultSceneDownsample;
		LensFlareInputs.bCompositeWithBloom = false;
	}

	return AddLensFlaresPass(GraphBuilder, View, LensFlareInputs);
}