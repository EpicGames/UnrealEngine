// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

enum class ELensFlareQuality : uint32
{
	Disabled,
	Low,
	High,
	VeryHigh,
	MAX
};

ELensFlareQuality GetLensFlareQuality();

struct FLensFlareInputs
{
	static const uint32 LensFlareCountMax = 8;

	// [Required] The bloom convolution texture. If enabled, this will be composited with lens flares. Otherwise,
	// a transparent black texture is used instead. Either way, the final output texture will use the this texture
	// descriptor and viewport.
	FScreenPassTextureSlice Bloom;

	// [Required] The scene color input, before bloom, which is used as the source of lens flares.
	// This can be a downsampled input based on the desired quality level.
	FScreenPassTextureSlice Flare;

	// [Required] The bokeh shape texture to use to blur the lens flares.
	FRHITexture* BokehShapeTexture;

	// The number of lens flares to render.
	uint32 LensFlareCount = LensFlareCountMax;

	// The array of per-flare tint colors. Length must be equal to LensFlareCount.
	TArrayView<const FLinearColor> TintColorsPerFlare;

	// The lens flare tint color to apply to all lens flares.
	FLinearColor TintColor;

	// The size of the bokeh shape in screen percentage.
	float BokehSizePercent = 0.0f;

	// Brightness scale of the lens flares.
	float Intensity = 1.0f;

	// Brightness threshold at which lens flares begin having an effect.
	float Threshold = 1.0f;

	// Whether to composite lens flares with the scene color input. If false, the result is composited on transparent black.
	bool bCompositeWithBloom = true;
};

using FLensFlareOutputs = FScreenPassTexture;


bool IsLensFlaresEnabled(const FViewInfo& View);

// Helper function which pulls inputs from the post process settings of the view.
FScreenPassTexture AddLensFlaresPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture Bloom,
	FScreenPassTextureSlice QualitySceneDownsample,
	FScreenPassTextureSlice DefaultSceneDownsample);