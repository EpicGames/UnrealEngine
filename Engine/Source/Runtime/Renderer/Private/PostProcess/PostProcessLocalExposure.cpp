// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessLocalExposure.cpp: Post processing local exposure implementation.
=============================================================================*/

#include "PostProcess/PostProcessLocalExposure.h"
#include "PostProcess/PostProcessEyeAdaptation.h"
#include "PostProcess/PostProcessWeightedSampleSum.h"
#include "PostProcess/PostProcessDownsample.h"
#include "Curves/CurveFloat.h"
#include "SceneRendering.h"
#include "ShaderCompilerCore.h"
#include "DataDrivenShaderPlatformInfo.h"

static TAutoConsoleVariable<float> CVarExposureFusionTargetLuminance(
	TEXT("r.LocalExposure.ExposureFusion.TargetLuminance"),
	0.5f,
	TEXT("Target Luminance used to determine the weight of each exposure."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarExposureFusionNumLevels(
	TEXT("r.LocalExposure.ExposureFusion.NumLevels"),
	16,
	TEXT("Number of levels in the Laplacian pyramid used to blend the different exposures."),
	ECVF_RenderThreadSafe);

namespace
{

class FSetupLogLuminanceCS : public FGlobalShader
{
public:
	// Changing these numbers requires LocalExposure.usf to be recompiled
	static const uint32 ThreadGroupSizeX = 8;
	static const uint32 ThreadGroupSizeY = 8;

	DECLARE_GLOBAL_SHADER(FSetupLogLuminanceCS);
	SHADER_USE_PARAMETER_STRUCT(FSetupLogLuminanceCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, OutputFloat)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSetupLogLuminanceCS, "/Engine/Private/PostProcessLocalExposure.usf", "SetupLogLuminanceCS", SF_Compute);

class FApplyLocalExposureCS : public FGlobalShader
{
public:
	// Changing these numbers requires LocalExposure.usf to be recompiled
	static const uint32 ThreadGroupSizeX = 8;
	static const uint32 ThreadGroupSizeY = 8;

	DECLARE_GLOBAL_SHADER(FApplyLocalExposureCS);
	SHADER_USE_PARAMETER_STRUCT(FApplyLocalExposureCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)

		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutputFloat4)

		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)

		SHADER_PARAMETER_STRUCT(FLocalExposureParameters, LocalExposure)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, LumBilateralGrid)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BlurredLogLum)

		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FApplyLocalExposureCS, "/Engine/Private/PostProcessLocalExposure.usf", "ApplyLocalExposureCS", SF_Compute);

class FFusionSetupCS : public FGlobalShader
{
public:
	// Changing these numbers requires LocalExposure.usf to be recompiled
	static const uint32 ThreadGroupSizeX = 8;
	static const uint32 ThreadGroupSizeY = 8;

	DECLARE_GLOBAL_SHADER(FFusionSetupCS);
	SHADER_USE_PARAMETER_STRUCT(FFusionSetupCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)

		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)

		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)

		SHADER_PARAMETER_STRUCT(FLocalExposureParameters, LocalExposure)

		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutputFloat4)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutputFloat4_1)

		SHADER_PARAMETER(float, TargetLuminance)

		SHADER_PARAMETER(float, FilmSlope)
		SHADER_PARAMETER(float, FilmToe)
		SHADER_PARAMETER(float, FilmShoulder)
		SHADER_PARAMETER(float, FilmBlackClip)
		SHADER_PARAMETER(float, FilmWhiteClip)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFusionSetupCS, "/Engine/Private/PostProcessLocalExposure.usf", "FusionSetupCS", SF_Compute);

class FFusionBlendCS : public FGlobalShader
{
public:
	// Changing these numbers requires LocalExposure.usf to be recompiled
	static const uint32 ThreadGroupSizeX = 8;
	static const uint32 ThreadGroupSizeY = 8;

	DECLARE_GLOBAL_SHADER(FFusionBlendCS);
	SHADER_USE_PARAMETER_STRUCT(FFusionBlendCS, FGlobalShader);

	class FLaplacianDim : SHADER_PERMUTATION_BOOL("LAPLACIAN");
	using FPermutationDomain = TShaderPermutationDomain<FLaplacianDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)

		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, WeightTexture)

		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, CoarserMip)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, CoarserMipTexture)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, PrevResultTexture)

		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, OutputFloat)

		SHADER_PARAMETER_SAMPLER(SamplerState, TextureSampler)

		SHADER_PARAMETER(FScreenTransform, DispatchThreadToCoarseMipUV)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFusionBlendCS, "/Engine/Private/PostProcessLocalExposure.usf", "FusionBlendCS", SF_Compute);

} //! namespace

FVector2f GetLocalExposureBilateralGridUVScale(const FIntPoint ViewRectSize);

FLocalExposureParameters GetLocalExposureParameters(const FViewInfo& View, FIntPoint ViewRectSize, const FEyeAdaptationParameters& EyeAdaptationParameters)
{
	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

	const EAutoExposureMethod AutoExposureMethod = GetAutoExposureMethod(View);

	float LocalExposureMiddleGreyExposureCompensation = FMath::Pow(2.0f, View.FinalPostProcessSettings.LocalExposureMiddleGreyBias);

	if (AutoExposureMethod == EAutoExposureMethod::AEM_Manual)
	{
		// when using manual exposure cancel exposure compensation setting and curve from middle grey used by local exposure.
		LocalExposureMiddleGreyExposureCompensation /= (EyeAdaptationParameters.Scalars.ExposureCompensationSettings * EyeAdaptationParameters.Scalars.ExposureCompensationCurve);
	}

	const FVector2f LocalExposureBilateralGridUVScale = GetLocalExposureBilateralGridUVScale(ViewRectSize);

	float HighlightContrast = Settings.LocalExposureHighlightContrastScale;
	float ShadowContrast = Settings.LocalExposureShadowContrastScale;

	const float AverageSceneLuminance = View.GetLastAverageSceneLuminance();
	if (AverageSceneLuminance > 0)
	{
		const float LuminanceMax = LuminanceMaxFromLensAttenuation();
		// We need the Log2(0.18) to convert from average luminance to saturation luminance
		const float LuminanceEV100 = LuminanceToEV100(LuminanceMax, AverageSceneLuminance) + FMath::Log2(1.0f / 0.18f);

		if (Settings.LocalExposureHighlightContrastCurve)
		{
			HighlightContrast *= Settings.LocalExposureHighlightContrastCurve->GetFloatValue(LuminanceEV100);
		}

		if (Settings.LocalExposureShadowContrastCurve)
		{
			ShadowContrast *= Settings.LocalExposureShadowContrastCurve->GetFloatValue(LuminanceEV100);
		}
	}

	if (View.FinalPostProcessSettings.LocalExposureMethod == ELocalExposureMethod::Fusion)
	{
		const float HighlightEV = FMath::Lerp<float>(6, 0, HighlightContrast);
		HighlightContrast = FMath::Pow(2, -HighlightEV);

		const float ShadowEV = FMath::Lerp<float>(6, 0, ShadowContrast);
		ShadowContrast = FMath::Pow(2, ShadowEV);
	}

	FLocalExposureParameters Parameters;
	Parameters.HighlightContrastScale = HighlightContrast;
	Parameters.ShadowContrastScale = ShadowContrast;
	Parameters.DetailStrength = Settings.LocalExposureDetailStrength;
	Parameters.BlurredLuminanceBlend = Settings.LocalExposureBlurredLuminanceBlend;
	Parameters.MiddleGreyExposureCompensation = LocalExposureMiddleGreyExposureCompensation;
	Parameters.BilateralGridUVScale = LocalExposureBilateralGridUVScale;
	Parameters.HighlightThreshold = Settings.LocalExposureHighlightThreshold;
	Parameters.ShadowThreshold = Settings.LocalExposureShadowThreshold;
	Parameters.HighlightThresholdStrength = Settings.LocalExposureHighlightThresholdStrength;
	Parameters.ShadowThresholdStrength = Settings.LocalExposureShadowThresholdStrength;
	return Parameters;
}

FRDGTextureRef AddLocalExposureBlurredLogLuminancePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FScreenPassTextureSlice InputTexture)
{
	check(InputTexture.IsValid());

	RDG_EVENT_SCOPE(GraphBuilder, "LocalExposure - Blurred Luminance");

	FRDGTextureRef GaussianLumSetupTexture;

	// Copy log luminance to temporary texture
	{
		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(
			InputTexture.ViewRect.Size(),
			PF_R16F,
			FClearValueBinding::None,
			TexCreate_UAV | TexCreate_ShaderResource);

		GaussianLumSetupTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("GaussianLumSetupTexture"));

		auto* PassParameters = GraphBuilder.AllocParameters<FSetupLogLuminanceCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(InputTexture));
		PassParameters->InputTexture = InputTexture.TextureSRV;
		PassParameters->OutputFloat = GraphBuilder.CreateUAV(GaussianLumSetupTexture);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("SetupLogLuminance %dx%d", GaussianLumSetupTexture->Desc.Extent.X, GaussianLumSetupTexture->Desc.Extent.Y),
			ERDGPassFlags::Compute,
			View.ShaderMap->GetShader<FSetupLogLuminanceCS>(),
			PassParameters,
			FComputeShaderUtils::GetGroupCount(GaussianLumSetupTexture->Desc.Extent, FIntPoint(FSetupLogLuminanceCS::ThreadGroupSizeX, FSetupLogLuminanceCS::ThreadGroupSizeY)));
	}

	FRDGTextureRef GaussianTexture;

	{
		FGaussianBlurInputs GaussianBlurInputs;
		GaussianBlurInputs.NameX = TEXT("LocalExposureGaussianX");
		GaussianBlurInputs.NameY = TEXT("LocalExposureGaussianY");
		GaussianBlurInputs.Filter = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, FScreenPassTexture(GaussianLumSetupTexture));
		GaussianBlurInputs.TintColor = FLinearColor::White;
		GaussianBlurInputs.CrossCenterWeight = FVector2f::ZeroVector;
		GaussianBlurInputs.KernelSizePercent = View.FinalPostProcessSettings.LocalExposureBlurredLuminanceKernelSizePercent;
		GaussianBlurInputs.UseMirrorAddressMode = true;

		GaussianTexture = AddGaussianBlurPass(GraphBuilder, View, GaussianBlurInputs).Texture;
	}

	return GaussianTexture;
}

void AddApplyLocalExposurePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FRDGBufferRef EyeAdaptationBuffer,
	const FLocalExposureParameters& LocalExposureParamaters,
	FRDGTextureRef LocalExposureTexture,
	FRDGTextureRef BlurredLogLuminanceTexture,
	FScreenPassTextureSlice Input,
	FScreenPassTextureSlice Output,
	ERDGPassFlags PassFlags)
{
	check(Input.IsValid() && Output.IsValid());
	check(PassFlags == ERDGPassFlags::Compute || PassFlags == ERDGPassFlags::AsyncCompute);

	RDG_EVENT_SCOPE(GraphBuilder, "LocalExposure - Apply");

	auto* PassParameters = GraphBuilder.AllocParameters<FApplyLocalExposureCS::FParameters>();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Input));
	PassParameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Output));

	PassParameters->InputTexture = Input.TextureSRV;
	{
		FRDGTextureUAVDesc OutputDesc(Output.TextureSRV->Desc.Texture);
		if (Output.TextureSRV->Desc.Texture->Desc.IsTextureArray())
		{
			OutputDesc.DimensionOverride = ETextureDimension::Texture2D;
			OutputDesc.FirstArraySlice = Output.TextureSRV->Desc.FirstArraySlice;
			OutputDesc.NumArraySlices = 1;
		}

		PassParameters->OutputFloat4 = GraphBuilder.CreateUAV(OutputDesc);
	}

	PassParameters->EyeAdaptation = EyeAdaptationParameters;
	PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(EyeAdaptationBuffer);

	PassParameters->LocalExposure = LocalExposureParamaters;
	PassParameters->LumBilateralGrid = LocalExposureTexture;
	PassParameters->BlurredLogLum = BlurredLogLuminanceTexture;

	PassParameters->TextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("ApplyLocalExposure %dx%d", Output.ViewRect.Width(), Output.ViewRect.Height()),
		PassFlags,
		View.ShaderMap->GetShader<FApplyLocalExposureCS>(),
		PassParameters,
		FComputeShaderUtils::GetGroupCount(Output.ViewRect.Size(), FIntPoint(FApplyLocalExposureCS::ThreadGroupSizeX, FApplyLocalExposureCS::ThreadGroupSizeY)));
}

FExposureFusionData AddLocalExposureFusionPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FRDGBufferRef EyeAdaptationBuffer,
	const FLocalExposureParameters& LocalExposureParamaters,
	FScreenPassTextureSlice Input)
{
	check(Input.IsValid());

	RDG_EVENT_SCOPE(GraphBuilder, "LocalExposure - Fusion");

	FScreenPassTexture LumTexture;
	FScreenPassTexture WeightTexture;

	{
		const FRDGTextureDesc& InputDesc = Input.TextureSRV->GetParent()->Desc;

		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(
			InputDesc.Extent,
			PF_FloatRGB,
			FClearValueBinding::None,
			TexCreate_UAV | TexCreate_ShaderResource);

		// output uses same viewport as input
		LumTexture = FScreenPassTexture(GraphBuilder.CreateTexture(TextureDesc, TEXT("LocalExposureLumTexture")), Input.ViewRect);
		WeightTexture = FScreenPassTexture(GraphBuilder.CreateTexture(TextureDesc, TEXT("LocalExposureWeightTexture")), Input.ViewRect);

		auto* PassParameters = GraphBuilder.AllocParameters<FFusionSetupCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(EyeAdaptationBuffer);
		PassParameters->LocalExposure = LocalExposureParamaters;
		PassParameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Input));
		PassParameters->InputTexture = Input.TextureSRV;
		PassParameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(LumTexture));
		PassParameters->OutputFloat4 = GraphBuilder.CreateUAV(LumTexture.Texture);
		PassParameters->OutputFloat4_1 = GraphBuilder.CreateUAV(WeightTexture.Texture);
		PassParameters->TargetLuminance = CVarExposureFusionTargetLuminance.GetValueOnRenderThread();

		const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

		PassParameters->FilmSlope = Settings.FilmSlope;
		PassParameters->FilmToe = Settings.FilmToe;
		PassParameters->FilmShoulder = Settings.FilmShoulder;
		PassParameters->FilmBlackClip = Settings.FilmBlackClip;
		PassParameters->FilmWhiteClip = Settings.FilmWhiteClip;

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FusionSetup %dx%d", Input.ViewRect.Width(), Input.ViewRect.Height()),
			ERDGPassFlags::Compute,
			View.ShaderMap->GetShader<FFusionSetupCS>(),
			PassParameters,
			FComputeShaderUtils::GetGroupCount(Input.ViewRect.Size(), FIntPoint(FFusionSetupCS::ThreadGroupSizeX, FFusionSetupCS::ThreadGroupSizeY)));
	}

	const bool bLogLumaInAlpha = false;

	const uint32 MaxMips = FMath::Log2(float(FMath::Min(LumTexture.Texture->Desc.Extent.X, LumTexture.Texture->Desc.Extent.Y))) + 1;
	const uint32 NumMips = FMath::Clamp((uint32)CVarExposureFusionNumLevels.GetValueOnRenderThread(), 1, MaxMips);

	FTextureDownsampleChain LumChain;
	LumChain.Init(
		GraphBuilder, View,
		EyeAdaptationParameters,
		FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, LumTexture),
		EDownsampleQuality::High,
		NumMips,
		bLogLumaInAlpha);

	FTextureDownsampleChain WeightChain;
	WeightChain.Init(
		GraphBuilder, View,
		EyeAdaptationParameters,
		FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, WeightTexture),
		EDownsampleQuality::High,
		NumMips,
		bLogLumaInAlpha);

	FScreenPassTexture Output;

	FScreenPassTextureSlice CoarserMip;

	for(int32 Index = NumMips - 1; Index >= 0; --Index)
	{
		FScreenPassTextureSlice CurrentLum = LumChain.GetTexture(Index);
		FScreenPassTextureSlice CurrentWeight = WeightChain.GetTexture(Index);

		FRDGTextureSRVRef PrevResult = Output.IsValid() ? GraphBuilder.CreateSRV(Output.Texture) : nullptr;

		{
			FRDGTextureDesc OutputDesc = CurrentLum.TextureSRV->GetParent()->Desc;
			OutputDesc.Reset();
			OutputDesc.Flags |= TexCreate_UAV;

			// output uses same viewport as mip
			Output = FScreenPassTexture(GraphBuilder.CreateTexture(OutputDesc, TEXT("LocalExposureResult")), CurrentLum.ViewRect);
		}

		auto* PassParameters = GraphBuilder.AllocParameters<FFusionBlendCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->InputTexture = CurrentLum.TextureSRV;
		PassParameters->WeightTexture = CurrentWeight.TextureSRV;
		if (CoarserMip.IsValid())
		{
			PassParameters->DispatchThreadToCoarseMipUV =
				FScreenTransform::DispatchThreadIdToViewportUV(Output.ViewRect) *
				FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(CoarserMip), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
			PassParameters->CoarserMip = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(CoarserMip));
			PassParameters->CoarserMipTexture = CoarserMip.TextureSRV;
		}
		PassParameters->PrevResultTexture = PrevResult;
		PassParameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Output));
		PassParameters->OutputFloat = GraphBuilder.CreateUAV(Output.Texture);
		PassParameters->TextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		FFusionBlendCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FFusionBlendCS::FLaplacianDim>(PrevResult != nullptr);

		auto ComputeShader = View.ShaderMap->GetShader<FFusionBlendCS>(PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FusionBlend %dx%d", Output.ViewRect.Width(), Output.ViewRect.Height()),
			ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(Output.ViewRect.Size(), FIntPoint(FFusionBlendCS::ThreadGroupSizeX, FFusionBlendCS::ThreadGroupSizeY)));

		CoarserMip = CurrentLum;
	}

	FExposureFusionData OutputData;
	OutputData.Result = Output;
	OutputData.Exposures = LumTexture;
	OutputData.Weights = WeightTexture;

	return MoveTemp(OutputData);
}