// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

class FEyeAdaptationParameters;

BEGIN_SHADER_PARAMETER_STRUCT(FLocalExposureParameters, )
	SHADER_PARAMETER(float, HighlightContrastScale)
	SHADER_PARAMETER(float, ShadowContrastScale)
	SHADER_PARAMETER(float, DetailStrength)
	SHADER_PARAMETER(float, BlurredLuminanceBlend)
	SHADER_PARAMETER(float, MiddleGreyExposureCompensation)
	SHADER_PARAMETER(FVector2f, BilateralGridUVScale)
	SHADER_PARAMETER(float, HighlightThreshold)
	SHADER_PARAMETER(float, ShadowThreshold)
	SHADER_PARAMETER(float, HighlightThresholdStrength)
	SHADER_PARAMETER(float, ShadowThresholdStrength)
END_SHADER_PARAMETER_STRUCT()

FLocalExposureParameters GetLocalExposureParameters(const FViewInfo& View, FIntPoint ViewRectSize, const FEyeAdaptationParameters& EyeAdaptationParameters);

FRDGTextureRef AddLocalExposureBlurredLogLuminancePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FScreenPassTextureSlice InputTexture);

void AddApplyLocalExposurePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FRDGBufferRef EyeAdaptationBuffer,
	const FLocalExposureParameters& LocalExposureParamaters,
	FRDGTextureRef LocalExposureTexture,
	FRDGTextureRef BlurredLogLuminanceTexture,
	FScreenPassTextureSlice Input,
	FScreenPassTextureSlice Output,
	ERDGPassFlags PassFlags);

struct FExposureFusionData
{
	FScreenPassTexture Result;
	FScreenPassTexture Exposures;
	FScreenPassTexture Weights;
};

FExposureFusionData AddLocalExposureFusionPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FEyeAdaptationParameters& EyeAdaptationParameters,
	FRDGBufferRef EyeAdaptationBuffer,
	const FLocalExposureParameters& LocalExposureParamaters,
	FScreenPassTextureSlice Input);