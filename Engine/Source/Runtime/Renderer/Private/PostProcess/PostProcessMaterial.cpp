// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessMaterial.cpp: Post processing Material implementation.
=============================================================================*/

#include "PostProcess/PostProcessMaterial.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "LogRenderer.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialDomain.h"
#include "MaterialShaderType.h"
#include "MaterialShader.h"
#include "RenderUtils.h"
#include "SceneUtils.h"
#include "PostProcess/SceneRenderTargets.h"
#include "PostProcess/SceneFilterRendering.h"
#include "SceneRendering.h"
#include "ClearQuad.h"
#include "Materials/MaterialExpressionSceneTexture.h"
#include "Materials/MaterialExpressionUserSceneTexture.h"
#include "PipelineStateCache.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessMobile.h"
#include "BufferVisualizationData.h"
#include "SceneTextureParameters.h"
#include "SystemTextures.h"
#include "Substrate/Substrate.h"
#include "SingleLayerWaterRendering.h"
#include "Engine/NeuralProfile.h"
#include "PathTracing.h"
#include "PSOPrecacheSettings.h"

namespace
{

TAutoConsoleVariable<int32> CVarPostProcessAllowStencilTest(
	TEXT("r.PostProcessAllowStencilTest"),
	1,
	TEXT("Enables stencil testing in post process materials.\n")
	TEXT("0: disable stencil testing\n")
	TEXT("1: allow stencil testing\n")
	);

TAutoConsoleVariable<int32> CVarPostProcessAllowBlendModes(
	TEXT("r.PostProcessAllowBlendModes"),
	1,
	TEXT("Enables blend modes in post process materials.\n")
	TEXT("0: disable blend modes. Uses replace\n")
	TEXT("1: allow blend modes\n")
	);

TAutoConsoleVariable<int32> CVarPostProcessingDisableMaterials(
	TEXT("r.PostProcessing.DisableMaterials"),
	0,
	TEXT(" Allows to disable post process materials. \n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static FName NAME_SceneColor("SceneColor");

static bool IsPostProcessStencilTestAllowed()
{
	return CVarPostProcessAllowStencilTest.GetValueOnRenderThread() != 0;
}

enum class EMaterialCustomDepthPolicy : uint32
{
	// Custom depth is disabled.
	Disabled,

	// Custom Depth-Stencil is enabled; potentially simultaneous SRV / DSV usage.
	Enabled
};

static EMaterialCustomDepthPolicy GetMaterialCustomDepthPolicy(const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial* Material)
{
	check(Material);

	// Material requesting stencil test and post processing CVar allows it.
	if (Material->IsStencilTestEnabled() && IsPostProcessStencilTestAllowed())
	{
		// Custom stencil texture allocated and available.
		if (GetCustomDepthMode() != ECustomDepthMode::EnabledWithStencil)
		{
			FString AssetNameString = Material->GetAssetName();
			UE_LOGF(LogRenderer, Warning, "PostProcessMaterial %ls uses stencil test, but stencil not allocated. Set r.CustomDepth to 3 to allocate custom stencil.", *AssetNameString);
		}
		else if (MaterialRenderProxy->GetBlendableLocation(Material) == BL_SceneColorAfterTonemapping)
		{
			// We can't support custom stencil after tonemapping due to target size differences
			FString AssetNameString = Material->GetAssetName();
			UE_LOGF(LogRenderer, Warning, "PostProcessMaterial %ls uses stencil test, but is set to blend After Tonemapping. This is not supported.", *AssetNameString);
		}
		else
		{
			return EMaterialCustomDepthPolicy::Enabled;
		}
	}

	return EMaterialCustomDepthPolicy::Disabled;
}

static FRHIDepthStencilState* GetMaterialStencilState(const FMaterial* Material)
{
	static FRHIDepthStencilState* StencilStates[] =
	{
		TStaticDepthStencilState<false, CF_Always, true, CF_Less>::GetRHI(),
		TStaticDepthStencilState<false, CF_Always, true, CF_LessEqual>::GetRHI(),
		TStaticDepthStencilState<false, CF_Always, true, CF_Greater>::GetRHI(),
		TStaticDepthStencilState<false, CF_Always, true, CF_GreaterEqual>::GetRHI(),
		TStaticDepthStencilState<false, CF_Always, true, CF_Equal>::GetRHI(),
		TStaticDepthStencilState<false, CF_Always, true, CF_NotEqual>::GetRHI(),
		TStaticDepthStencilState<false, CF_Always, true, CF_Never>::GetRHI(),
		TStaticDepthStencilState<false, CF_Always, true, CF_Always>::GetRHI(),
	};
	static_assert(EMaterialStencilCompare::MSC_Count == UE_ARRAY_COUNT(StencilStates), "Ensure that all EMaterialStencilCompare values are accounted for.");

	check(Material);

	return StencilStates[Material->GetStencilCompare()];
}

static bool IsMaterialBlendEnabled(const FMaterial* Material)
{
	check(Material);

	return Material->GetBlendableOutputAlpha() && CVarPostProcessAllowBlendModes.GetValueOnAnyThread() != 0;
}

static FRHIBlendState* GetMaterialBlendState(const FMaterial* Material)
{
	static FRHIBlendState* BlendStates[] =
	{
		TStaticBlendState<>::GetRHI(),
		TStaticBlendState<>::GetRHI(),
		TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI(),
		TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI(),
		TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_Zero>::GetRHI(),
		TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI(),
		TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI(),
		TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI(),
	};
	static_assert(EBlendMode::BLEND_MAX == UE_ARRAY_COUNT(BlendStates), "Ensure that all EBlendMode values are accounted for.");

	check(Material);

	if (Material->IsSubstrateMaterial())
	{
		switch (Material->GetBlendMode())
		{
		case EBlendMode::BLEND_Opaque:
		case EBlendMode::BLEND_Masked:
			return TStaticBlendState<>::GetRHI();
		case EBlendMode::BLEND_Additive:
			return TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
		case EBlendMode::BLEND_AlphaComposite:
		case EBlendMode::BLEND_TranslucentColoredTransmittance: // A platform may not support dual source blending so we always only use grey scale transmittance
		case EBlendMode::BLEND_TranslucentGreyTransmittance:
			return TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_One, BF_InverseSourceAlpha>::GetRHI();
		case EBlendMode::BLEND_ColoredTransmittanceOnly:
			return TStaticBlendState<CW_RGB, BO_Add, BF_DestColor, BF_Zero>::GetRHI();
		case EBlendMode::BLEND_AlphaHoldout:
			return TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_InverseSourceAlpha>::GetRHI();
		default:
			check(false);
			return TStaticBlendState<>::GetRHI();
		}
	}

	return BlendStates[Material->GetBlendMode()];
}

static bool PostProcessStencilTest(const uint32 StencilValue, const uint32 StencilComp, const uint32 StencilRef)
{
	bool bStencilTestPassed = true;

	switch (StencilComp)
	{
	case EMaterialStencilCompare::MSC_Less:
		bStencilTestPassed = (StencilRef < StencilValue);
		break;
	case EMaterialStencilCompare::MSC_LessEqual:
		bStencilTestPassed = (StencilRef <= StencilValue);
		break;
	case EMaterialStencilCompare::MSC_GreaterEqual:
		bStencilTestPassed = (StencilRef >= StencilValue);
		break;
	case EMaterialStencilCompare::MSC_Equal:
		bStencilTestPassed = (StencilRef == StencilValue);
		break;
	case EMaterialStencilCompare::MSC_Greater:
		bStencilTestPassed = (StencilRef > StencilValue);
		break;
	case EMaterialStencilCompare::MSC_NotEqual:
		bStencilTestPassed = (StencilRef != StencilValue);
		break;
	case EMaterialStencilCompare::MSC_Never:
		bStencilTestPassed = false;
		break;
	default:
		break;
	}

	return !bStencilTestPassed;
}

static uint32 GetManualStencilTestMask(uint32 StencilComp)
{
	// These enum values must match their #define counterparts in PostProcessMaterialShaders.ush
	enum StencilTestMask
	{
		Equal	= (1 << 0),
		Less	= (1 << 1),
		Greater	= (1 << 2)
	};
	uint32 Mask = 0;

	switch (StencilComp)
	{
	case EMaterialStencilCompare::MSC_Less:
		return Less;
	case EMaterialStencilCompare::MSC_LessEqual:
		return Less | Equal;
	case EMaterialStencilCompare::MSC_GreaterEqual:
		return Greater | Equal;
	case EMaterialStencilCompare::MSC_Equal:
		return Equal;
	case EMaterialStencilCompare::MSC_Greater:
		return Greater;
	case EMaterialStencilCompare::MSC_NotEqual:
		return Less | Greater;
	case EMaterialStencilCompare::MSC_Never:
		return 0;
	case EMaterialStencilCompare::MSC_Always:
	default:
		return Less | Equal | Greater;
	}
}

class FPostProcessMaterialShader : public FMaterialShader
{
public:
	using FParameters = FPostProcessMaterialParameters;
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FPostProcessMaterialShader, FMaterialShader);

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		if (Parameters.MaterialParameters.MaterialDomain == MD_PostProcess)
		{
			return !IsMobilePlatform(Parameters.Platform) || IsMobileHDR();
		}
		return false;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_MATERIAL"), 1);

		EBlendableLocation Location = EBlendableLocation(Parameters.MaterialParameters.BlendableLocation);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_MATERIAL_BEFORE_TONEMAP"), (Location == BL_SceneColorAfterTonemapping || Location == BL_ReplacingTonemapper) ? 0 : 1);
		OutEnvironment.SetDefine(TEXT("POST_PROCESS_DISABLE_PRE_EXPOSURE_SCALE"), Parameters.MaterialParameters.bDisablePreExposureScale ? 1 : 0);

		if (IsMobilePlatform(Parameters.Platform))
		{
			OutEnvironment.SetDefine(TEXT("POST_PROCESS_MATERIAL_BEFORE_TONEMAP"), (Parameters.MaterialParameters.BlendableLocation != BL_SceneColorAfterTonemapping) ? 1 : 0);
		}

		// PostProcessMaterial can both read & write Substrate data
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_INLINE_SHADING"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSTRATE_DEFERRED_SHADING"), 1);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FViewInfo& View, const FMaterialRenderProxy* Proxy, const FMaterial& Material)
	{
		FMaterialShader::SetParameters(BatchedParameters, Proxy, Material, View);
	}
};

class FPostProcessMaterialVS : public FPostProcessMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FPostProcessMaterialVS, Material);

	FPostProcessMaterialVS() = default;
	FPostProcessMaterialVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FPostProcessMaterialShader(Initializer)
	{}
};

class FPostProcessMaterialPS : public FPostProcessMaterialShader
{
public:
	DECLARE_SHADER_TYPE(FPostProcessMaterialPS, Material);

	class FManualStencilTestDim : SHADER_PERMUTATION_BOOL("MANUAL_STENCIL_TEST");
	class FNeuralPostProcessPrePass : SHADER_PERMUTATION_BOOL("NEURAL_POSTPROCESS_PREPASS");
	class FPathTracingDim : SHADER_PERMUTATION_BOOL("PATH_TRACING_POST_PROCESS_MATERIAL");
	using FPermutationDomain = TShaderPermutationDomain<FManualStencilTestDim,FNeuralPostProcessPrePass,FPathTracingDim>;

	FPostProcessMaterialPS() = default;
	FPostProcessMaterialPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FPostProcessMaterialShader(Initializer)
	{}

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		if (!FPostProcessMaterialShader::ShouldCompilePermutation(Parameters))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// Currently, we only need the manual stencil test permutations if stencil test is enabled and Nanite is supported.
		// See comments in CustomDepthRendering.h for more details.
		if (PermutationVector.Get<FManualStencilTestDim>())
		{
			if (!(Parameters.MaterialParameters.bIsStencilTestEnabled && DoesPlatformSupportNanite(Parameters.Platform)))
			{
				return false;
			}
		}

		// Only enable the path tracing specialization when path tracing is enabled on the project
		if (PermutationVector.Get<FPathTracingDim>())
		{
			if (!PathTracing::ShouldCompilePathTracingShadersForProject(Parameters.Platform))
			{
				return false;
			}
		}

		return true;
	}
};

IMPLEMENT_SHADER_TYPE(,FPostProcessMaterialVS, TEXT("/Engine/Private/PostProcessMaterialShaders.usf"), TEXT("MainVS"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(,FPostProcessMaterialPS, TEXT("/Engine/Private/PostProcessMaterialShaders.usf"), TEXT("MainPS"), SF_Pixel);

class FPostProcessMaterialVertexDeclaration : public FRenderResource
{
public:
	FVertexDeclarationRHIRef VertexDeclarationRHI;

	void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		FVertexDeclarationElementList Elements;
		uint32 Stride = sizeof(FFilterVertex);
		Elements.Add(FVertexElement(0, STRUCT_OFFSET(FFilterVertex, Position), VET_Float4, 0, Stride));
		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}

	void ReleaseRHI() override
	{
		VertexDeclarationRHI.SafeRelease();
	}
};

static bool GetMaterialShaders(
	const FMaterial& Material, 
	const EShaderPlatform ShaderPlatform,
	bool bManualStencilTest, 
	bool bNeuralPostProcessPrepass,
	bool bPathTracingEnabled,
	TShaderRef<FPostProcessMaterialVS>& OutVertexShader,
	TShaderRef<FPostProcessMaterialPS>& OutPixelShader)
{
	FMaterialShaderTypes ShaderTypes;

	FPostProcessMaterialPS::FPermutationDomain PermutationVectorPS;
	PermutationVectorPS.Set<FPostProcessMaterialPS::FManualStencilTestDim>(bManualStencilTest);
	PermutationVectorPS.Set<FPostProcessMaterialPS::FNeuralPostProcessPrePass>(bNeuralPostProcessPrepass);
	PermutationVectorPS.Set<FPostProcessMaterialPS::FPathTracingDim>(bPathTracingEnabled && PathTracing::ShouldCompilePathTracingShadersForProject(ShaderPlatform));

	ShaderTypes.AddShaderType<FPostProcessMaterialVS>();
	ShaderTypes.AddShaderType<FPostProcessMaterialPS>(PermutationVectorPS.ToDimensionValueId());

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, nullptr, Shaders))
	{
		return false;
	}
	
	Shaders.TryGetVertexShader(OutVertexShader);
	Shaders.TryGetPixelShader(OutPixelShader);

	return true;
}

static void GetMaterialInfo(
	const UMaterialInterface* InMaterialInterface,
	const ERHIFeatureLevel::Type InFeatureLevel,
	const EShaderPlatform ShaderPlatform,
	const bool bPathTracingEnabled,
	const FPostProcessMaterialInputs& Inputs,
	const FMaterial*& OutMaterial,
	const FMaterialRenderProxy*& OutMaterialProxy,
	const FMaterialShaderMap*& OutMaterialShaderMap,
	TShaderRef<FPostProcessMaterialVS>& OutVertexShader,
	TShaderRef<FPostProcessMaterialPS>& OutPixelShader,
	bool bNeuralPostProcessPrepass = false)
{
	const FMaterialRenderProxy* MaterialProxy = InMaterialInterface->GetRenderProxy();
	check(MaterialProxy);

	const FMaterial* Material = nullptr;
	FMaterialShaders Shaders;
	while (MaterialProxy)
	{
		Material = MaterialProxy->GetMaterialNoFallback(InFeatureLevel);
		if (Material && Material->GetMaterialDomain() == MD_PostProcess)
		{
			const bool bManualStencilTest = Inputs.bManualStencilTest && Material->IsStencilTestEnabled();
			if (GetMaterialShaders(*Material, ShaderPlatform, bManualStencilTest, bNeuralPostProcessPrepass, bPathTracingEnabled, OutVertexShader, OutPixelShader))
			{
				break;
			}
		}
		MaterialProxy = MaterialProxy->GetFallback(InFeatureLevel);
	}

	// NOTE: we expect to reach this point only via the break statement. If we exit the while loop via a null pointer, something has gone wrong
	check(Material);
	check(MaterialProxy);

	const FMaterialShaderMap* MaterialShaderMap = Material->GetRenderingThreadShaderMap();
	check(MaterialShaderMap);

	OutMaterial = Material;
	OutMaterialProxy = MaterialProxy;
	OutMaterialShaderMap = MaterialShaderMap;
}

TGlobalResource<FPostProcessMaterialVertexDeclaration> GPostProcessMaterialVertexDeclaration;

} //! namespace

void AddMobileMSAADecodeAndDrawTexturePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture Input,
	FScreenPassRenderTarget Output)
{
	const FScreenPassTextureViewport InputViewport(Input);
	const FScreenPassTextureViewport OutputViewport(Output);

	TShaderMapRef<FMSAADecodeAndCopyRectPS_Mobile> PixelShader(View.ShaderMap);

	FMSAADecodeAndCopyRectPS_Mobile::FParameters* Parameters = GraphBuilder.AllocParameters<FMSAADecodeAndCopyRectPS_Mobile::FParameters>();
	Parameters->InputTexture = Input.Texture;
	Parameters->InputSampler = TStaticSamplerState<>::GetRHI();
	Parameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("MobileMSAADecodeAndDrawTexture"), View, OutputViewport, InputViewport, PixelShader, Parameters);
}

FPostProcessMaterialParameters* GetPostProcessMaterialParameters(
	FRDGBuilder& GraphBuilder, 
	const FPostProcessMaterialInputs& Inputs, 
	const FViewInfo& View,
	const FScreenPassTextureViewport& OutputViewport,
	FScreenPassRenderTarget& Output, 
	FRDGTextureRef DepthStencilTexture, 
	const uint32 MaterialStencilRef, 
	const FMaterial* Material, 
	const FMaterialShaderMap* MaterialShaderMap)
{
	FPostProcessMaterialParameters* PostProcessMaterialParameters = GraphBuilder.AllocParameters<FPostProcessMaterialParameters>();
	PostProcessMaterialParameters->SceneTextures = Inputs.SceneTextures;
	PostProcessMaterialParameters->View = View.ViewUniformBuffer;
	PostProcessMaterialParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));
	PostProcessMaterialParameters->PostProcessOutput = GetScreenPassTextureViewportParameters(OutputViewport);
	PostProcessMaterialParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

	// The target color will be decoded if bForceIntermediateTarget is true in any case, but we might still need to decode the input color
	PostProcessMaterialParameters->bMetalMSAAHDRDecode = Inputs.bMetalMSAAHDRDecode ? 1 : 0;

	if (DepthStencilTexture)
	{
		PostProcessMaterialParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
			DepthStencilTexture,
			ERenderTargetLoadAction::ELoad,
			ERenderTargetLoadAction::ELoad,
			FExclusiveDepthStencil::DepthRead_StencilRead);
	}
	PostProcessMaterialParameters->ManualStencilReferenceValue = MaterialStencilRef;
	PostProcessMaterialParameters->ManualStencilTestMask = GetManualStencilTestMask(Material->GetStencilCompare());

	PostProcessMaterialParameters->PostProcessInput_BilinearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();;

	const FScreenPassTexture BlackDummy(GSystemTextures.GetBlackDummy(GraphBuilder));

	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	int32 NumUserSceneTextures = MaterialShaderMap->GetUserSceneTextureInputs().Num();
	for (uint32 InputIndex = 0; InputIndex < kPostProcessMaterialInputCountMax; ++InputIndex)
	{
		FScreenPassTextureSlice Input = Inputs.GetInput((EPostProcessMaterialInput)InputIndex);

		bool bIsUsed = MaterialShaderMap->UsesSceneTexture(PPI_PostProcessInput0 + InputIndex);

		// User scene textures consume any consecutive slots not used by PPI_PostProcessInput0-6
		if (!bIsUsed)
		{
			if (NumUserSceneTextures > 0)
			{
				NumUserSceneTextures--;
				bIsUsed = true;
			}
		}

		// Need to provide valid textures for when shader compilation doesn't cull unused parameters.
		if (!Input.IsValid() || !bIsUsed)
		{
			Input = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, BlackDummy);
		}

		PostProcessMaterialParameters->PostProcessInput[InputIndex] = GetScreenPassTextureInput(Input, PointClampSampler);
	}

	// Path tracing buffer textures
	for (uint32 InputIndex = 0; InputIndex < kPathTracingPostProcessMaterialInputCountMax; ++InputIndex)
	{
		FScreenPassTexture Input = Inputs.GetPathTracingInput((EPathTracingPostProcessMaterialInput)InputIndex);

		if (!Input.Texture || !MaterialShaderMap->UsesPathTracingBufferTexture(InputIndex))
		{
			Input = BlackDummy;
		}

		PostProcessMaterialParameters->PathTracingPostProcessInput[InputIndex] = GetScreenPassTextureInput(Input, PointClampSampler);
	}

	// Substrate requires the full rendering pipeline to be initialized.
	// Skip for standalone passes running outside the normal post-process pipeline.
	if (!Inputs.bStandalonePass)
	{
		PostProcessMaterialParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	}

	// SceneDepthWithoutWater
	const bool bHasValidSceneDepthWithoutWater = Inputs.SceneWithoutWaterTextures && Inputs.SceneWithoutWaterTextures->DepthTexture;
	const bool bShouldUseBilinearSamplerForDepth = bHasValidSceneDepthWithoutWater && ShouldUseBilinearSamplerForDepthWithoutSingleLayerWater(Inputs.SceneWithoutWaterTextures->DepthTexture->Desc.Format);
	PostProcessMaterialParameters->bSceneDepthWithoutWaterTextureAvailable = bHasValidSceneDepthWithoutWater;
	PostProcessMaterialParameters->SceneDepthWithoutSingleLayerWaterSampler = bShouldUseBilinearSamplerForDepth ? TStaticSamplerState<SF_Bilinear>::GetRHI() : TStaticSamplerState<SF_Point>::GetRHI();
	PostProcessMaterialParameters->SceneDepthWithoutSingleLayerWaterTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
	PostProcessMaterialParameters->SceneWithoutSingleLayerWaterMinMaxUV = FVector4f(0.0f, 0.0f, 1.0f, 1.0f);
	PostProcessMaterialParameters->SceneWithoutSingleLayerWaterTextureSize = FVector2f(0.0f, 0.0f);
	PostProcessMaterialParameters->SceneWithoutSingleLayerWaterInvTextureSize = FVector2f(0.0f, 0.0f);
	if (bHasValidSceneDepthWithoutWater)
	{
		const bool bIsInstancedStereo = View.IsInstancedStereo();
		int32 WaterViewIndex = INDEX_NONE;
		if (bIsInstancedStereo)
		{
			WaterViewIndex = View.PrimaryViewIndex; // The instanced view does not have MinMaxUV initialized, instead the primary view MinMaxUV covers both eyes
		}
		else
		{
			verify(View.Family->Views.Find(&View, WaterViewIndex));
		}

		PostProcessMaterialParameters->SceneDepthWithoutSingleLayerWaterTexture = Inputs.SceneWithoutWaterTextures->DepthTexture;
		PostProcessMaterialParameters->SceneWithoutSingleLayerWaterMinMaxUV = Inputs.SceneWithoutWaterTextures->Views[WaterViewIndex].MinMaxUV;

		const FIntVector DepthTextureSize = Inputs.SceneWithoutWaterTextures->DepthTexture->Desc.GetSize();
		PostProcessMaterialParameters->SceneWithoutSingleLayerWaterTextureSize = FVector2f(DepthTextureSize.X, DepthTextureSize.Y);
		PostProcessMaterialParameters->SceneWithoutSingleLayerWaterInvTextureSize = FVector2f(1.0f / DepthTextureSize.X, 1.0f / DepthTextureSize.Y);
	}

	PostProcessMaterialParameters->NeuralPostProcessParameters = GetDefaultNeuralPostProcessShaderParameters(GraphBuilder);

	// UserSceneTextureSceneColorInput is used for automatic scene color alpha propagation.  Alpha propagation only occurs if the output is scene color
	// (meaning not a user scene texture), so set this to INDEX_NONE if writing to a UserSceneTexture output instead.
	PostProcessMaterialParameters->UserSceneTextureSceneColorInput = Inputs.bUserSceneTextureOutput ? INDEX_NONE : Inputs.UserSceneTextureSceneColorInput;

	return PostProcessMaterialParameters;
}

void AddNeuralPostProcessPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FPostProcessMaterialInputs& Inputs,
	const UMaterialInterface* MaterialInterface,
	FNeuralPostProcessResource& NeuralPostProcessResource)
{
	Inputs.Validate();

	const FScreenPassTexture SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, Inputs.GetInput(EPostProcessMaterialInput::SceneColor));

	const ERHIFeatureLevel::Type FeatureLevel = View.GetFeatureLevel();
	const bool bPathTracingEnabled = View.Family->EngineShowFlags.PathTracing;

	const FMaterial* Material = nullptr;
	const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
	const FMaterialShaderMap* MaterialShaderMap = nullptr;
	TShaderRef<FPostProcessMaterialVS> NeuralPostProcessPassVertexShader;
	TShaderRef<FPostProcessMaterialPS> NeuralPostProcessPassPixelShader;
	GetMaterialInfo(MaterialInterface, FeatureLevel, View.GetShaderPlatform(), bPathTracingEnabled, Inputs, Material, MaterialRenderProxy, MaterialShaderMap, NeuralPostProcessPassVertexShader, NeuralPostProcessPassPixelShader, true);
	
	check(NeuralPostProcessPassVertexShader.IsValid());
	check(NeuralPostProcessPassPixelShader.IsValid());

	int32 NeuralProfileId = Material->GetNeuralProfileId();

	FRHIDepthStencilState* DefaultDepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();
	FRHIDepthStencilState* DepthStencilState = DefaultDepthStencilState;

	FRDGTextureRef DepthStencilTexture = nullptr;

	// Allocate custom depth stencil texture(s) and depth stencil state.
	const EMaterialCustomDepthPolicy CustomStencilPolicy = GetMaterialCustomDepthPolicy(MaterialRenderProxy, Material);

	if (CustomStencilPolicy == EMaterialCustomDepthPolicy::Enabled &&
		!Inputs.bManualStencilTest &&
		HasBeenProduced(Inputs.CustomDepthTexture))
	{
		check(Inputs.CustomDepthTexture);
		DepthStencilTexture = Inputs.CustomDepthTexture;
		DepthStencilState = GetMaterialStencilState(Material);
	}

	FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
	FRHIBlendState* BlendState = DefaultBlendState;

	if (IsMaterialBlendEnabled(Material))
	{
		BlendState = GetMaterialBlendState(Material);
	}

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	// Create a new texture instead of reusing the scene color output in the pre pass. Should not pollute the scene color texture.
	{
		// Allocate new transient output texture.
		{
			FRDGTextureDesc OutputDesc = SceneColor.Texture->Desc;
			OutputDesc.Reset();
			if (Inputs.OutputFormat != PF_Unknown)
			{
				OutputDesc.Format = Inputs.OutputFormat;
			}
			OutputDesc.ClearValue = FClearValueBinding(FLinearColor::Black);
			OutputDesc.Flags &= (~ETextureCreateFlags::FastVRAM);
			OutputDesc.Flags |= GFastVRamConfig.PostProcessMaterial;

			Output = FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, TEXT("PostProcessTempOutput")), SceneColor.ViewRect, View.GetOverwriteLoadAction());
		}
	}

	const FScreenPassTextureViewport SceneColorViewport(SceneColor);
	const FScreenPassTextureViewport OutputViewport(Output);

	RDG_EVENT_SCOPE(GraphBuilder, "PostProcessMaterial::NeuralPass");

	const uint32 MaterialStencilRef = Material->GetStencilRefValue();

	EScreenPassDrawFlags ScreenPassFlags = EScreenPassDrawFlags::AllowHMDHiddenAreaMask;

	// check if we can skip that draw call in case if all pixels will fail the stencil test of the material
	bool bSkipPostProcess = false;

	if (Material->IsStencilTestEnabled() && IsPostProcessStencilTestAllowed())
	{
		bool bFailStencil = true;

		const uint32 StencilComp = Material->GetStencilCompare();

		// Always check against clear value, since a material might want to perform operations against that value
		const uint32 StencilClearValue = Inputs.CustomDepthTexture ? Inputs.CustomDepthTexture->Desc.ClearValue.Value.DSValue.Stencil : 0;
		bFailStencil &= PostProcessStencilTest(StencilClearValue, StencilComp, MaterialStencilRef);


		for (const uint32& Value : View.CustomDepthStencilValues)
		{
			bFailStencil &= PostProcessStencilTest(Value, StencilComp, MaterialStencilRef);

			if (!bFailStencil)
			{
				break;
			}
		}

		bSkipPostProcess = bFailStencil;
	}

	if (!bSkipPostProcess)
	{
		NeuralPostProcessResource = AllocateNeuralPostProcessingResourcesIfNeeded(
			GraphBuilder, OutputViewport, NeuralProfileId, Material->IsUsedWithNeuralNetworks());

		if (NeuralPostProcessResource.IsValid())
		{ 
			// Prepass to extract the input to the NNE Engine
			FPostProcessMaterialParameters* PostProcessMaterialParameters =
				GetPostProcessMaterialParameters(GraphBuilder, Inputs, View, OutputViewport, Output, DepthStencilTexture, MaterialStencilRef, Material, MaterialShaderMap);

			SetupNeuralPostProcessShaderParametersForWrite(PostProcessMaterialParameters->NeuralPostProcessParameters, GraphBuilder, NeuralPostProcessResource);

			ClearUnusedGraphResources(NeuralPostProcessPassVertexShader, NeuralPostProcessPassPixelShader, PostProcessMaterialParameters);

			//Only call the neural network when the shader resource is actually used.
			if (IsNeuralPostProcessShaderParameterUsed(PostProcessMaterialParameters->NeuralPostProcessParameters))
			{
				AddDrawScreenPass(
					GraphBuilder,
#if RDG_EVENTS != RDG_EVENTS_STRING_COPY
					RDG_EVENT_NAME("PostProcessMaterial(Neural Prepass)"),
#else
					FRDGEventName(*Material->GetAssetName()),
#endif
					View,
					OutputViewport,
					SceneColorViewport,
					// Uses default depth stencil on mobile since the stencil test is done in pixel shader.
					FScreenPassPipelineState(NeuralPostProcessPassVertexShader, NeuralPostProcessPassPixelShader, BlendState, DepthStencilState, MaterialStencilRef),
					PostProcessMaterialParameters,
					ScreenPassFlags,
					[&View, NeuralPostProcessPassVertexShader, NeuralPostProcessPassPixelShader, MaterialRenderProxy, Material, PostProcessMaterialParameters](FRHICommandList& RHICmdList)
					{
						SetShaderParametersMixedVS(RHICmdList, NeuralPostProcessPassVertexShader, *PostProcessMaterialParameters, View, MaterialRenderProxy, *Material);
						SetShaderParametersMixedPS(RHICmdList, NeuralPostProcessPassPixelShader, *PostProcessMaterialParameters, View, MaterialRenderProxy, *Material);
					});

				ApplyNeuralPostProcess(GraphBuilder, View, Output.ViewRect, NeuralPostProcessResource);
			}
		}
	}
}

FScreenPassTexture AddPostProcessMaterialPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FPostProcessMaterialInputs& Inputs,
	const UMaterialInterface* MaterialInterface)
{
	Inputs.Validate();

	const ERHIFeatureLevel::Type FeatureLevel = View.GetFeatureLevel();
	const bool bPathTracingEnabled = View.Family->EngineShowFlags.PathTracing;

	const FMaterial* Material = nullptr;
	const FMaterialRenderProxy* MaterialRenderProxy = nullptr;
	const FMaterialShaderMap* MaterialShaderMap = nullptr;
	TShaderRef<FPostProcessMaterialVS> VertexShader;
	TShaderRef<FPostProcessMaterialPS> PixelShader;
	GetMaterialInfo(MaterialInterface, FeatureLevel, View.GetShaderPlatform(), bPathTracingEnabled, Inputs, Material, MaterialRenderProxy, MaterialShaderMap, VertexShader, PixelShader);

	EBlendableLocation BlendableLocation = MaterialRenderProxy->GetBlendableLocation(Material);
	const FScreenPassTextureSlice SceneColorOutput = Inputs.GetSceneColorOutput(BlendableLocation);

	check(VertexShader.IsValid());
	check(PixelShader.IsValid());

	FRHIDepthStencilState* DefaultDepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();
	FRHIDepthStencilState* DepthStencilState = DefaultDepthStencilState;

	FRDGTextureRef DepthStencilTexture = nullptr;

	// Allocate custom depth stencil texture(s) and depth stencil state.
	const EMaterialCustomDepthPolicy CustomStencilPolicy = GetMaterialCustomDepthPolicy(MaterialRenderProxy, Material);

	if (CustomStencilPolicy == EMaterialCustomDepthPolicy::Enabled &&
		!Inputs.bManualStencilTest &&
		HasBeenProduced(Inputs.CustomDepthTexture))
	{
		check(Inputs.CustomDepthTexture);
		DepthStencilTexture = Inputs.CustomDepthTexture;
		DepthStencilState = GetMaterialStencilState(Material);
	}

	FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
	FRHIBlendState* BlendState = DefaultBlendState;

	if (IsMaterialBlendEnabled(Material))
	{
		BlendState = GetMaterialBlendState(Material);
	}

	// Determine if the pixel shader may discard, requiring us to initialize the output texture
	const bool bMayDiscard = CustomStencilPolicy == EMaterialCustomDepthPolicy::Enabled && Inputs.bManualStencilTest;

	// Blend / Depth Stencil usage requires that the render target have primed color data.
	const bool bCompositeWithInput = DepthStencilState != DefaultDepthStencilState ||
									 BlendState != DefaultBlendState ||
									 bMayDiscard;

	// We only prime color on the output texture if we are using fixed function Blend / Depth-Stencil, or we need to
	// retain previously rendered views.  UserSceneTexture does its own output priming logic where required.
	const bool bPrimeOutputColor = (bCompositeWithInput || !View.IsFirstInFamily()) && !Inputs.bUserSceneTextureOutput;

	// Inputs.OverrideOutput is used to force drawing directly to the backbuffer. OpenGL doesn't support using the backbuffer color target with a custom depth/stencil
	// buffer, so in that case we must draw to an intermediate target and copy to the backbuffer at the end. Ideally, we would test if Inputs.OverrideOutput.Texture
	// is actually the backbuffer, but it's not worth doing all the plumbing and increasing the RHI surface area just for this hack.  UserSceneTexture is never a backbuffer.
	const bool bBackbufferWithDepthStencil = (DepthStencilTexture != nullptr && !GRHISupportsBackBufferWithCustomDepthStencil && Inputs.OverrideOutput.IsValid() && !Inputs.bUserSceneTextureOutput);

	// We need to decode the target color for blending material, force it rendering to an intermediate render target and decode the color.
	const bool bCompositeWithInputAndDecode = Inputs.bMetalMSAAHDRDecode && bCompositeWithInput;

	const bool bForceIntermediateTarget = bBackbufferWithDepthStencil || bCompositeWithInputAndDecode;

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	// We can re-use the scene color texture as the render target if we're not simultaneously reading from it.
	bool bInputReadsFromOutput = false;
	if (!Output.IsValid())
	{
		for (int32 InputIndex = 0; InputIndex < kPostProcessMaterialInputCountMax; InputIndex++)
		{
			if (MaterialShaderMap->UsesSceneTexture((ESceneTextureId)(PPI_PostProcessInput0 + InputIndex)) && Inputs.Textures[InputIndex].TextureSRV &&
				SceneColorOutput.TextureSRV->GetParent() == Inputs.Textures[InputIndex].TextureSRV->GetParent())
			{
				bInputReadsFromOutput = true;
				break;
			}
		}
	}

	const bool bValidShaderPlatform = (GMaxRHIShaderPlatform != SP_PCD3D_ES3_1);
	if (!Output.IsValid() && !bInputReadsFromOutput && !bForceIntermediateTarget && Inputs.bAllowSceneColorInputAsOutput && bValidShaderPlatform && !Inputs.bUserSceneTextureOutput)
	{
		FScreenPassTexture SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorOutput);

		Output = FScreenPassRenderTarget(SceneColor, ERenderTargetLoadAction::ELoad);

		// If material doesn't output alpha, and we are writing to an existing scene color, preserve its alpha by masking out writes
		if (!Material->GetBlendableOutputAlpha())
		{
			BlendState = TStaticBlendState<CW_RGB>::GetRHI();
		}
	}
	else
	{
		// Allocate new transient output texture if none exists.
		if (!Output.IsValid() || bForceIntermediateTarget)
		{
			FRDGTextureDesc OutputDesc = SceneColorOutput.TextureSRV->Desc.Texture->Desc;
			OutputDesc.Dimension = ETextureDimension::Texture2D;
			OutputDesc.ArraySize = 1;

			OutputDesc.Reset();
			if (Inputs.OutputFormat != PF_Unknown)
			{
				OutputDesc.Format = Inputs.OutputFormat;
			}
			OutputDesc.ClearValue = FClearValueBinding(FLinearColor::Black);
			OutputDesc.Flags &= (~ETextureCreateFlags::FastVRAM);
			OutputDesc.Flags |= GFastVRamConfig.PostProcessMaterial;

			Output = FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, TEXT("PostProcessMaterial")), SceneColorOutput.ViewRect, View.GetOverwriteLoadAction());
		}

		if (bPrimeOutputColor || bForceIntermediateTarget)
		{
			FScreenPassTexture SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorOutput);

			// Copy existing contents to new output and use load-action to preserve untouched pixels.
			if (Inputs.bMetalMSAAHDRDecode)
			{
				AddMobileMSAADecodeAndDrawTexturePass(GraphBuilder, View, SceneColor, Output);
			}
			else
			{
				AddDrawTexturePass(GraphBuilder, View, SceneColor, Output);
			}
			Output.LoadAction = ERenderTargetLoadAction::ELoad;
		}

		// If this is the first render to a UserSceneTexture which requires compositing, we copy the previous output as a starting point.
		if (bCompositeWithInput && Inputs.bUserSceneTextureOutput && Inputs.bUserSceneTextureFirstRender)
		{
			AddDrawTexturePass(GraphBuilder, View, SceneColorOutput, Output);
		}
	}

	const FScreenPassTextureViewport SceneColorViewport(SceneColorOutput);
	const FScreenPassTextureViewport OutputViewport(Output);

	RDG_EVENT_SCOPE(GraphBuilder, "PostProcessMaterial %dx%d Material=%s", SceneColorViewport.Rect.Width(), SceneColorViewport.Rect.Height(), *Material->GetAssetName());

	const uint32 MaterialStencilRef = Material->GetStencilRefValue();

	EScreenPassDrawFlags ScreenPassFlags = EScreenPassDrawFlags::AllowHMDHiddenAreaMask;

	// check if we can skip that draw call in case if all pixels will fail the stencil test of the material
	bool bSkipPostProcess = false;

	if (Material->IsStencilTestEnabled() && IsPostProcessStencilTestAllowed())
	{
		bool bFailStencil = true;

		const uint32 StencilComp = Material->GetStencilCompare();

		// Always check against clear value, since a material might want to perform operations against that value
		const uint32 StencilClearValue = Inputs.CustomDepthTexture ? Inputs.CustomDepthTexture->Desc.ClearValue.Value.DSValue.Stencil : 0;
		bFailStencil &= PostProcessStencilTest(StencilClearValue, StencilComp, MaterialStencilRef);

		for (const uint32& Value : View.CustomDepthStencilValues)
		{
			bFailStencil &= PostProcessStencilTest(Value, StencilComp, MaterialStencilRef);

			if (!bFailStencil)
			{
				break;
			}
		}

		bSkipPostProcess = bFailStencil;
	}

	if (!bSkipPostProcess)
	{
		FNeuralPostProcessResource NeuralPostProcessResource;
		const bool bShouldApplyNeuralPostProcessing = ShouldApplyNeuralPostProcessForMaterial(Material);

		if (bShouldApplyNeuralPostProcessing)
		{
			AddNeuralPostProcessPass(GraphBuilder, View, Inputs, MaterialInterface, NeuralPostProcessResource);
		}

		{
			FPostProcessMaterialParameters* PostProcessMaterialParameters =
				GetPostProcessMaterialParameters(GraphBuilder, Inputs, View, OutputViewport, Output, DepthStencilTexture, MaterialStencilRef, Material, MaterialShaderMap);

			if (bShouldApplyNeuralPostProcessing)
			{
				SetupNeuralPostProcessShaderParametersForRead(PostProcessMaterialParameters->NeuralPostProcessParameters, GraphBuilder, NeuralPostProcessResource);
			}

			ClearUnusedGraphResources(VertexShader, PixelShader, PostProcessMaterialParameters);

			AddDrawScreenPass(
				GraphBuilder,
#if RDG_EVENTS != RDG_EVENTS_STRING_COPY
				RDG_EVENT_NAME("PostProcessMaterial"),
#else
				FRDGEventName(*Material->GetAssetName()),
#endif
				View,
				OutputViewport,
				SceneColorViewport,
				// Uses default depth stencil on mobile since the stencil test is done in pixel shader.
				FScreenPassPipelineState(VertexShader, PixelShader, BlendState, DepthStencilState, MaterialStencilRef),
				PostProcessMaterialParameters,
				ScreenPassFlags,
				[&View, VertexShader, PixelShader, MaterialRenderProxy, Material, PostProcessMaterialParameters](FRHICommandList& RHICmdList)
				{
					SetShaderParametersMixedVS(RHICmdList, VertexShader, *PostProcessMaterialParameters, View, MaterialRenderProxy, *Material);
					SetShaderParametersMixedPS(RHICmdList, PixelShader, *PostProcessMaterialParameters, View, MaterialRenderProxy, *Material);
				});
		}

		if (bForceIntermediateTarget && !bCompositeWithInputAndDecode)
		{
			// We shouldn't get here unless we had an override target.
			check(Inputs.OverrideOutput.IsValid());
			AddDrawTexturePass(GraphBuilder, View, Output.Texture, Inputs.OverrideOutput.Texture);
			Output = Inputs.OverrideOutput;
		}
	}
	else
	{
		// When skipping the pass, we still need to output a valid FScreenPassRenderTarget
		if (Inputs.OverrideOutput.IsValid())
		{
			// If there is an override output, we can copy directly to that from the scene color slice.
			AddDrawTexturePass(GraphBuilder, View, SceneColorOutput, Inputs.OverrideOutput);
			Output = Inputs.OverrideOutput;
		}
		else
		{
			// Otherwise, we need to convert output to a single slice before returning
			FScreenPassTexture SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorOutput);
			Output = FScreenPassRenderTarget(SceneColor, ERenderTargetLoadAction::ENoAction);
		}
	}

	return MoveTemp(Output);
}

FScreenPassTexture AddPostProcessMaterialPass(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	const FPostProcessMaterialInputs& Inputs,
	const UMaterialInterface* MaterialInterface)
{
	if (!ensureMsgf(View.bIsViewInfo, TEXT("AddPostProcessMaterialPass requires that its View parameter is an FViewInfo.")))
	{
		return FScreenPassTexture::CopyFromSlice(GraphBuilder, Inputs.GetInput(EPostProcessMaterialInput::SceneColor));
	}

	return AddPostProcessMaterialPass(GraphBuilder, static_cast<const FViewInfo&>(View), Inputs, MaterialInterface);
}

static bool IsPostProcessMaterialsEnabledForView(const FViewInfo& View)
{
	// Runtime reflection captures have a special case to support a singleton post processing material, even though
	// post processing isn't enabled in the show flags.
	if (View.IsRuntimeReflectionCapture())
	{
		return CVarPostProcessingDisableMaterials.GetValueOnRenderThread() == 0;
	}

	if (!View.Family->EngineShowFlags.PostProcessing ||
		!View.Family->EngineShowFlags.PostProcessMaterial ||
		View.Family->EngineShowFlags.VisualizeShadingModels ||
		CVarPostProcessingDisableMaterials.GetValueOnRenderThread() != 0)
	{
		return false;
	}

	return true;
}

static FPostProcessMaterialNode* IteratePostProcessMaterialNodes(const FFinalPostProcessSettings& Dest, EBlendableLocation Location, FBlendableEntry*& Iterator)
{
	for (;;)
	{
		FPostProcessMaterialNode* DataPtr = Dest.BlendableManager.IterateBlendables<FPostProcessMaterialNode>(Iterator);

		if (!DataPtr || DataPtr->GetLocation() == Location || Location == EBlendableLocation::BL_MAX)
		{
			return DataPtr;
		}
	}
}

FPostProcessMaterialChain GetPostProcessMaterialChain(const FViewInfo& View, EBlendableLocation Location)
{
	if (!IsPostProcessMaterialsEnabledForView(View))
	{
		return {};
	}

	const FSceneViewFamily& ViewFamily = *View.Family;

	TArray<FPostProcessMaterialNode, TInlineAllocator<10>> Nodes;
	FBlendableEntry* Iterator = nullptr;

	while (FPostProcessMaterialNode* Data = IteratePostProcessMaterialNodes(View.FinalPostProcessSettings, Location, Iterator))
	{
		check(Data->GetMaterialInterface());
		Nodes.Add(*Data);
	}

	if (!Nodes.Num())
	{
		return {};
	}

	// Use stable sort, so if nodes with the same priority are explicitly ordered in the post process volume, they stay in that order
	Algo::StableSort(Nodes, FPostProcessMaterialNode::FCompare());

	FPostProcessMaterialChain OutputChain;
	OutputChain.Reserve(Nodes.Num());

	for (const FPostProcessMaterialNode& Node : Nodes)
	{
		OutputChain.Add(Node.GetMaterialInterface());
	}

	return OutputChain;
}

static void RemoveCollidingUserSceneTextureInputs(FPostProcessMaterialInputs& PassInputs, const FSceneTextures& SceneTextures)
{
	for (int32 InputIndex = 0; InputIndex < kPostProcessMaterialInputCountMax; InputIndex++)
	{
		if (PassInputs.bUserSceneTexturesSet[InputIndex])
		{
			if (PassInputs.UserSceneTextures[InputIndex].TextureSRV && PassInputs.UserSceneTextures[InputIndex].TextureSRV->GetParent() == PassInputs.OverrideOutput.Texture)
			{
				// Zero out the input, and label it as an error in the event stream if necessary

#if !(UE_BUILD_SHIPPING)
				// Get the name from the resource, and strip off the prefix
				const TCHAR* InputNameStr = PassInputs.UserSceneTextures[InputIndex].TextureSRV->Desc.Texture->Name;
				if (FCString::Strstr(InputNameStr, TEXT("UST.")) == InputNameStr)
				{
					InputNameStr += 4;
				}

				// Iterate over the events, looking for that name
				for (int32 EventIndex = SceneTextures.UserSceneTextureEvents.Num() - 1; EventIndex >= 0; EventIndex--)
				{
					// Stop if we reach a pass event marker
					if (SceneTextures.UserSceneTextureEvents[EventIndex].Event == EUserSceneTextureEvent::Pass)
					{
						break;
					}

					if (SceneTextures.UserSceneTextureEvents[EventIndex].Event == EUserSceneTextureEvent::FoundInput)
					{
						FString EventName = SceneTextures.UserSceneTextureEvents[EventIndex].Name.ToString();
						const TCHAR* EventNameStr = *EventName;

						// Resource may have a numeric allocation order suffix as well, like [1] -- check if the front of the string matches
						if (FCString::Strstr(InputNameStr, EventNameStr) == InputNameStr)
						{
							// Then check if that's the end of the string or an open bracket suffix
							int32 EventNameLen = FCString::Strlen(EventNameStr);
							if (InputNameStr[EventNameLen] == 0 || InputNameStr[EventNameLen] == TEXT('['))
							{
								SceneTextures.UserSceneTextureEvents[EventIndex].Event = EUserSceneTextureEvent::CollidingInput;
							}
						}
					}
				}
#endif

				PassInputs.UserSceneTextures[InputIndex] = FScreenPassTextureSlice();
			}
		}
	}
}

FScreenPassTexture AddPostProcessMaterialChain(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	int32 ViewIndex,
	const FPostProcessMaterialInputs& InputsTemplate,
	const FPostProcessMaterialChain& Materials,
	EPostProcessMaterialInput MaterialInput)
{
	FScreenPassTextureSlice CurrentInput = InputsTemplate.GetInput(MaterialInput);
	FScreenPassTexture Outputs {};

	// Get last material that writes to the output (ignoring materials that write to UserSceneTextures)
	const UMaterialInterface* LastOutputWrite = nullptr;
	for (int32 MaterialIndex = Materials.Num() - 1; MaterialIndex >= 0; MaterialIndex--)
	{
		const FMaterialRenderProxy* MaterialRenderProxy = Materials[MaterialIndex]->GetRenderProxy();
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(View.FeatureLevel);

		FName UserSceneTextureOutput = NAME_None;
		if (Material)
		{
			UserSceneTextureOutput = MaterialRenderProxy->GetUserSceneTextureOutput(Material);
		}

		if (UserSceneTextureOutput.IsNone() || UserSceneTextureOutput == NAME_SceneColor)
		{
			// If it doesn't write to a UserSceneTexture, it writes to the default SceneColor output (or it could be a UserSceneTexture set to write to SceneColor)
			LastOutputWrite = Materials[MaterialIndex];
			break;
		}
	}

	bool bFirstMaterialInChain = true;
	for (const UMaterialInterface* MaterialInterface : Materials)
	{
		FPostProcessMaterialInputs Inputs = InputsTemplate;
		Inputs.SetInput(MaterialInput, CurrentInput);

		// Get UserSceneTexture inputs and output from material if present
		const FMaterial* Material = MaterialInterface->GetRenderProxy()->GetMaterialNoFallback(View.FeatureLevel);
		FName UserSceneTextureOutput(NAME_None);
		FIntPoint UserTextureDivisor(0, 0);
		int32 UserSceneTextureInputNum = 0;

		if (Material)
		{
			const FMaterialShaderMap* MaterialShaderMap = Material->GetRenderingThreadShaderMap();
			const FSceneTextures& SceneTextures = View.GetSceneTextures();

			bool bFoundResolutionRelativeToInput = false;
			FName ResolutionRelativeToInput = NAME_None;

			TConstArrayView<FScriptName> UserSceneTextureInputs = MaterialShaderMap->GetUserSceneTextureInputs();
			if (!UserSceneTextureInputs.IsEmpty())
			{
				UserSceneTextureInputNum = UserSceneTextureInputs.Num();

				// We need to apply material instance input overrides to ResolutionRelativeToInput as well, so get the name here to
				// handle that in the input loop.
				ResolutionRelativeToInput = FName(MaterialShaderMap->GetResolutionRelativeToInput());

				int32 PostProcessIndex = 0;
				for (int32 UserIndex = 0; UserIndex < UserSceneTextureInputs.Num();)
				{
					check(PostProcessIndex < kPostProcessMaterialInputCountMax);
				
					// Skip over this slot if it's used by a SceneTexture node
					if (!MaterialShaderMap->UsesSceneTexture(PPI_PostProcessInput0 + PostProcessIndex))
					{
						FName UserSceneTextureInput(UserSceneTextureInputs[UserIndex]);
						bool bIsResolutionSource = ResolutionRelativeToInput == UserSceneTextureInputs[UserIndex];

						MaterialInterface->GetRenderProxy()->GetUserSceneTextureOverride(UserSceneTextureInput);

						if (bIsResolutionSource)
						{
							// Copy the overridden input to ResolutionRelativeToInput, and track that we found it
							ResolutionRelativeToInput = UserSceneTextureInput;
							bFoundResolutionRelativeToInput = true;
						}

						// Not used as a SceneTexture, so it's used by the next UserSceneTexture.  The special name "SceneColor" indicates use of
						// "SceneColor" as input.
						if (UserSceneTextureInput == NAME_SceneColor)
						{
							Inputs.SetUserSceneTextureInput((EPostProcessMaterialInput)PostProcessIndex, CurrentInput);

							// Need to disable optimization that attempts to reuse SceneColor as the output, when SceneColor isn't used as an input.  Normally
							// the use of SceneColor as an input is detected by the flags on the original FMaterialShaderMap (accessed via the UsesSceneTexture
							// function), but those flags won't be set if a UserSceneTexture input is overridden to point at SceneColor.
							Inputs.bAllowSceneColorInputAsOutput = false;

							// Handle automatic propagation of scene color alpha from a UserSceneTexture input
							Inputs.UserSceneTextureSceneColorInput = PostProcessIndex + (uint32)PPI_PostProcessInput0;
						}
						else
						{
							Inputs.SetUserSceneTextureInput((EPostProcessMaterialInput)PostProcessIndex, SceneTextures.GetUserSceneTexture(GraphBuilder, View, ViewIndex, UserSceneTextureInput, MaterialInterface));
						}
						UserIndex++;
					}
					PostProcessIndex++;
				}
			}

#if WITH_EDITOR
			// If this blendable is being previewed, don't write to the UserSceneTexture -- instead it will write to SceneColor
			bool bIsPreviewBlendable = false;
			if (View.FinalPostProcessSettings.PreviewBlendable)
			{
				if (Material->GetMaterialInterface() == View.FinalPostProcessSettings.PreviewBlendable)
				{
					// Material matches
					bIsPreviewBlendable = true;
				}
				else
				{
					FMaterialInheritanceChain MaterialInheritance;
					MaterialInterface->GetMaterialInheritanceChain(MaterialInheritance);
					if (MaterialInheritance.MaterialInstances.Contains(View.FinalPostProcessSettings.PreviewBlendable))
					{
						// Material instance matches
						bIsPreviewBlendable = true;
					}
				}
			}

			if (!bIsPreviewBlendable)
#endif
			{
				UserSceneTextureOutput = MaterialInterface->GetRenderProxy()->GetUserSceneTextureOutput(Material);

				// If output is set to the name "SceneColor", that means actually write to "SceneColor" as opposed to a transient UserSceneTexture.
				// The purpose of this is to give a general purpose Material asset operating on UserSceneTexture inputs and outputs the option to
				// read or write SceneColor as well, say if they are the first or last building block in a chain of materials.
				if (UserSceneTextureOutput == NAME_SceneColor)
				{
					// Clear to none, so it writes to SceneColor downstream
					UserSceneTextureOutput = NAME_None;
				}
				else
				{
					UserTextureDivisor = MaterialShaderMap->GetUserTextureDivisor();

					if (bFoundResolutionRelativeToInput)
					{
						// UserTextureDivisor is a relative divisor to the input, with positive values representing downscale, and negative upscale
						FIntPoint InputDivisor = SceneTextures.GetUserSceneTextureDivisor(ResolutionRelativeToInput);

						if (UserTextureDivisor.X >= 0)
						{
							UserTextureDivisor.X = InputDivisor.X * FMath::Max(UserTextureDivisor.X, 1);
						}
						else
						{
							UserTextureDivisor.X = FMath::Max(InputDivisor.X / FMath::Abs(UserTextureDivisor.X), 1);
						}

						if (UserTextureDivisor.Y >= 0)
						{
							UserTextureDivisor.Y = InputDivisor.Y * FMath::Max(UserTextureDivisor.Y, 1);
						}
						else
						{
							UserTextureDivisor.Y = FMath::Max(InputDivisor.Y / FMath::Abs(UserTextureDivisor.Y), 1);
						}
					}
					else
					{
						UserTextureDivisor.X = FMath::Max(UserTextureDivisor.X, 1);
						UserTextureDivisor.Y = FMath::Max(UserTextureDivisor.Y, 1);
					}
				}
			}
		}
		
		// Only the first material in the chain needs to decode the input color
		Inputs.bMetalMSAAHDRDecode = Inputs.bMetalMSAAHDRDecode && bFirstMaterialInChain;
		bFirstMaterialInChain = false;

		if (!UserSceneTextureOutput.IsNone())
		{
			// Writing to UserSceneTexture, don't set Outputs or CurrentInput, as this is writing to a disjoint texture that's not part of the chain
			FIntRect OutputRect = GetDownscaledViewRect(View.UnconstrainedViewRect, View.GetFamilyViewRect().Max, UserTextureDivisor);
			FRDGTextureRef UserOutput = ((FViewFamilyInfo*)View.Family)->GetSceneTextures().FindOrAddUserSceneTexture(GraphBuilder, ViewIndex, UserSceneTextureOutput, UserTextureDivisor, Inputs.bUserSceneTextureFirstRender, MaterialInterface, OutputRect);
			Inputs.OverrideOutput = FScreenPassRenderTarget(UserOutput, OutputRect, ERenderTargetLoadAction::ELoad);
			Inputs.bUserSceneTextureOutput = true;

			RemoveCollidingUserSceneTextureInputs(Inputs, View.GetSceneTextures());

			AddPostProcessMaterialPass(GraphBuilder, View, Inputs, MaterialInterface);
		}
		else
		{
			// Certain inputs are only respected by the final post process material in the chain, that writes to the Output
			if (MaterialInterface != LastOutputWrite)
			{
				Inputs.OverrideOutput = FScreenPassRenderTarget();
			}

			Outputs = AddPostProcessMaterialPass(GraphBuilder, View, Inputs, MaterialInterface);

			// Don't create the CurrentInput out of Outputs of the last material as this could possibly be the back buffer for AfterTonemap post process material
			if (MaterialInterface != LastOutputWrite)
			{
				CurrentInput = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, Outputs);
			}
		}

#if !UE_BUILD_SHIPPING
		if (UserSceneTextureInputNum || !UserSceneTextureOutput.IsNone())
		{
			View.GetSceneTextures().UserSceneTextureEvents.Add({ EUserSceneTextureEvent::Pass, NAME_None, 0, (uint16)ViewIndex, MaterialInterface });
		}
#endif
	}

	if (!Outputs.IsValid())
	{
		// If no passes wrote to OverrideOutput, we need to copy to OverrideOutput now.  This can happen if all the passes wrote
		// to UserSceneTextures instead of the default output.
		if (InputsTemplate.OverrideOutput.IsValid())
		{
			AddDrawTexturePass(GraphBuilder, View, CurrentInput, InputsTemplate.OverrideOutput);
			Outputs = InputsTemplate.OverrideOutput;
		}
		else
		{
			Outputs = FScreenPassTexture::CopyFromSlice(GraphBuilder, CurrentInput);
		}
	}

	return Outputs;
}

extern void AddDumpToColorArrayPass(FRDGBuilder& GraphBuilder, FScreenPassTexture Input, TArray<FColor>* OutputColorArray, FIntPoint* OutputExtents);

bool IsHighResolutionScreenshotMaskEnabled(const FViewInfo& View)
{
	return View.Family->EngineShowFlags.HighResScreenshotMask || View.FinalPostProcessSettings.HighResScreenshotCaptureRegionMaterial;
}

bool IsPathTracingVarianceTextureRequiredInPostProcessMaterial(const FViewInfo& View)
{
	// query the post process material to check if any variance texture has been used
	bool bIsPathTracingVarianceTextureRequired = false;

	auto CheckIfPathTracingVarianceTextureIsRequried = [&](const UMaterialInterface* MaterialInterface) {
		
		if (MaterialInterface)
		{
			// Get the RenderProxy of the material.
			const FMaterialRenderProxy* MaterialProxy = MaterialInterface->GetRenderProxy();

			if (MaterialProxy)
			{

				// Get the Shadermap for the view's feature level
				const FMaterial* Material = MaterialProxy->GetMaterialNoFallback(View.FeatureLevel);
				if (Material && Material->GetMaterialDomain() == MD_PostProcess)
				{
					const FMaterialShaderMap* MaterialShaderMap = Material->GetRenderingThreadShaderMap();

					if (MaterialShaderMap &&
						MaterialShaderMap->UsesPathTracingBufferTexture(static_cast<uint32>(EPathTracingPostProcessMaterialInput::Variance)))
					{
						return true;
					}
				}
			}
		}

		return false;
	};

	FPostProcessMaterialChain PostProcessMaterialChain = GetPostProcessMaterialChain(View, EBlendableLocation::BL_MAX);
	for (const UMaterialInterface* MaterialInterface : PostProcessMaterialChain)
	{
		if (CheckIfPathTracingVarianceTextureIsRequried(MaterialInterface))
		{
			bIsPathTracingVarianceTextureRequired = true;
			break;
		}
	}

	// Check buffer visualization pipes
	const FFinalPostProcessSettings& PostProcessSettings = View.FinalPostProcessSettings;
	for (const UMaterialInterface* MaterialInterface : PostProcessSettings.BufferVisualizationOverviewMaterials)
	{
		if (CheckIfPathTracingVarianceTextureIsRequried(MaterialInterface))
		{
			bIsPathTracingVarianceTextureRequired = true;
			break;
		}
	}

	return bIsPathTracingVarianceTextureRequired;
}

FScreenPassTexture AddHighResolutionScreenshotMaskPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHighResolutionScreenshotMaskInputs& Inputs)
{
	check(Inputs.Material || Inputs.MaskMaterial || Inputs.CaptureRegionMaterial);

	enum class EPass
	{
		Material,
		MaskMaterial,
		CaptureRegionMaterial,
		MAX
	};

	const TCHAR* PassNames[]
	{
		TEXT("Material"),
		TEXT("MaskMaterial"),
		TEXT("CaptureRegionMaterial")
	};

	static_assert(UE_ARRAY_COUNT(PassNames) == static_cast<uint32>(EPass::MAX), "Pass names array doesn't match pass enum");

	const bool bHighResScreenshotMask = View.Family->EngineShowFlags.HighResScreenshotMask != 0;

	TOverridePassSequence<EPass> PassSequence(Inputs.OverrideOutput);
	PassSequence.SetEnabled(EPass::Material, bHighResScreenshotMask && Inputs.Material != nullptr);
	PassSequence.SetEnabled(EPass::MaskMaterial, bHighResScreenshotMask && Inputs.MaskMaterial != nullptr && GIsHighResScreenshot);
	PassSequence.SetEnabled(EPass::CaptureRegionMaterial, Inputs.CaptureRegionMaterial != nullptr);
	PassSequence.Finalize();

	FScreenPassTexture Output = Inputs.SceneColor;

	if (PassSequence.IsEnabled(EPass::Material))
	{
		FPostProcessMaterialInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::Material, PassInputs.OverrideOutput);
		PassInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, Output);
		PassInputs.SceneTextures = Inputs.SceneTextures;

		Output = AddPostProcessMaterialPass(GraphBuilder, View, PassInputs, Inputs.Material);
	}

	if (PassSequence.IsEnabled(EPass::MaskMaterial))
	{
		PassSequence.AcceptPass(EPass::MaskMaterial);

		FPostProcessMaterialInputs PassInputs;
		PassInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, Output);
		PassInputs.SceneTextures = Inputs.SceneTextures;

		// Explicitly allocate the render target to match the FSceneView extents and rect, so the output pixel arrangement matches
		FRDGTextureDesc MaskOutputDesc = Output.Texture->Desc;
		MaskOutputDesc.Reset();
		MaskOutputDesc.ClearValue = FClearValueBinding(FLinearColor::Black);
		MaskOutputDesc.Flags |= GFastVRamConfig.PostProcessMaterial;
		MaskOutputDesc.Extent = View.UnconstrainedViewRect.Size();

		PassInputs.OverrideOutput = FScreenPassRenderTarget(
			GraphBuilder.CreateTexture(MaskOutputDesc, TEXT("PostProcessMaterial")), View.UnscaledViewRect, View.GetOverwriteLoadAction());

		// Disallow the scene color input as output optimization since we need to not pollute the scene texture.
		PassInputs.bAllowSceneColorInputAsOutput = false;

		FScreenPassTexture MaskOutput = AddPostProcessMaterialPass(GraphBuilder, View, PassInputs, Inputs.MaskMaterial);
		AddDumpToColorArrayPass(GraphBuilder, MaskOutput, FScreenshotRequest::GetHighresScreenshotMaskColorArray(), &FScreenshotRequest::GetHighresScreenshotMaskExtents());

		// The mask material pass is actually outputting to system memory. If we're the last pass in the chain
		// and the override output is valid, we need to perform a copy of the input to the output. Since we can't
		// sample from the override output (since it might be the backbuffer), we still need to participate in
		// the pass sequence.
		if (PassSequence.IsLastPass(EPass::MaskMaterial) && Inputs.OverrideOutput.IsValid())
		{
			AddDrawTexturePass(GraphBuilder, View, Output, Inputs.OverrideOutput);
			Output = Inputs.OverrideOutput;
		}
	}

	if (PassSequence.IsEnabled(EPass::CaptureRegionMaterial))
	{
		FPostProcessMaterialInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::CaptureRegionMaterial, PassInputs.OverrideOutput);
		PassInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, Output);
		PassInputs.SceneTextures = Inputs.SceneTextures;

		Output = AddPostProcessMaterialPass(GraphBuilder, View, PassInputs, Inputs.CaptureRegionMaterial);
	}

	return Output;
}

FScreenPassTexture FPostProcessMaterialInputs::ReturnUntouchedSceneColorForPostProcessing(FRDGBuilder& GraphBuilder) const
{
	FScreenPassTextureSlice& SceneColorSlice = const_cast<FScreenPassTextureSlice&>(Textures[(uint32)EPostProcessMaterialInput::SceneColor]);

	// Support format conversions here, to handle the case where the output is the final render target, and happens to
	// be a different format than the intermediate render targets.
	if (OverrideOutput.IsValid() && OverrideOutput.Texture->Desc.Format != SceneColorSlice.TextureSRV->GetParent()->Desc.Format)
	{
		AddDrawTexturePass(GraphBuilder, FScreenPassViewInfo(), SceneColorSlice, OverrideOutput);
		return OverrideOutput;
	}
	else
	{
		return FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorSlice, OverrideOutput);
	}
}

static const TCHAR* PostProcessMaterialPSOCollectorName = TEXT("PostProcessMaterialPSOCollector");

class FPostProcessMaterialPSOCollector : public IPSOCollector
{
public:
	FPostProcessMaterialPSOCollector(ERHIFeatureLevel::Type InFeatureLevel) : 
	IPSOCollector(FPSOCollectorCreateManager::GetIndex(GetFeatureLevelShadingPath(InFeatureLevel), PostProcessMaterialPSOCollectorName))
	{
	}

	virtual void CollectPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		TArray<FPSOPrecacheData>& PSOInitializers
	) override final;
};

void FPostProcessMaterialPSOCollector::CollectPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams,
	TArray<FPSOPrecacheData>& PSOInitializers
)
{
	if (!Material.IsPostProcessMaterial() || !UPSOPrecacheSettingsManager::GetSettings().bPostProcessingMaterial)
	{
		return;
	}

	const FMaterialShaderMap* MaterialShaderMap = Material.GetGameThreadShaderMap();	
	auto AddPSOInitializer = [&](bool bManualStencilTest)
	{
		TShaderRef<FPostProcessMaterialVS> VertexShader;
		TShaderRef<FPostProcessMaterialPS> PixelShader;

		// PSO Precaching will not take into account the following features for now as they are less frequently used
		bool bNeuralPostProcessPrepass = false;
		bool bPathTracingEnabled = false;

		if (!GetMaterialShaders(Material, SceneTexturesConfig.ShaderPlatform, bManualStencilTest, bNeuralPostProcessPrepass, bPathTracingEnabled, VertexShader, PixelShader))
		{
			return;
		}
		      			
		FRHIBlendState* DefaultBlendState = FScreenPassPipelineState::FDefaultBlendState::GetRHI();
		FRHIBlendState* BlendState = IsMaterialBlendEnabled(&Material) ? GetMaterialBlendState(&Material) : DefaultBlendState;
	
		FRHIDepthStencilState* DefaultDepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();
		FRHIDepthStencilState* DepthStencilState = bManualStencilTest ? GetMaterialStencilState(&Material) : DefaultDepthStencilState;
		
		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		GraphicsPSOInit.BlendState = BlendState;
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
		GraphicsPSOInit.DepthStencilState = DepthStencilState;
		FRHIVertexShader* VertexShaderRHI = VertexShader.GetVertexShader(false);
		FRHIPixelShader* PixelShaderRHI = PixelShader.GetPixelShader(false);
		if (!VertexShaderRHI || !PixelShaderRHI)
		{
			return;
		}
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShaderRHI;
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShaderRHI;
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			
		// What render target formats to support?
		FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
		RenderTargetsInfo.NumSamples = 1;
		AddRenderTargetInfo(SceneTexturesConfig.ColorFormat, SceneTexturesConfig.ColorCreateFlags, RenderTargetsInfo);
		
		GraphicsPSOInit.StatePrecachePSOHash = RHIComputeStatePrecachePSOHash(GraphicsPSOInit);
		ApplyTargetsInfo(GraphicsPSOInit, RenderTargetsInfo);

		FPSOPrecacheData PSOPrecacheData;
		PSOPrecacheData.bRequired = true;
		PSOPrecacheData.Type = FPSOPrecacheData::EType::Graphics;
		PSOPrecacheData.GraphicsPSOInitializer = GraphicsPSOInit;
#if PSO_PRECACHING_VALIDATE
		PSOPrecacheData.ResourceName = Material.GetAssetName();
		PSOPrecacheData.PSOCollectorIndex = PSOCollectorIndex;
		PSOPrecacheData.VertexFactoryType = nullptr;
#endif // PSO_PRECACHING_VALIDATE

		PSOInitializers.Add(MoveTemp(PSOPrecacheData));
	};
	
	AddPSOInitializer(false /*bManualStencilTest*/);
	if (Material.IsStencilTestEnabled())
	{
		AddPSOInitializer(true /*bManualStencilTest*/);
	}
}

IPSOCollector* CreatePostProcessMaterialPSOCollector(ERHIFeatureLevel::Type FeatureLevel)
{
	return new FPostProcessMaterialPSOCollector(FeatureLevel);
}
FRegisterPSOCollectorCreateFunction RegisterPostProcessMaterialPSOCollector(&CreatePostProcessMaterialPSOCollector, EShadingPath::Deferred, PostProcessMaterialPSOCollectorName);
FRegisterPSOCollectorCreateFunction RegisterMobilePostProcessMaterialPSOCollector(&CreatePostProcessMaterialPSOCollector, EShadingPath::Mobile, PostProcessMaterialPSOCollectorName);
