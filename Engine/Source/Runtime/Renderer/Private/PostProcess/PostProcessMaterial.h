// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "OverridePassSequence.h"
#include "Substrate/Substrate.h"
#include "NeuralPostProcess.h"

class UMaterialInterface;
struct FPostProcessMaterialInputs;
class FViewInfo;

using FPostProcessMaterialChain = TArray<const UMaterialInterface*, TInlineAllocator<10>>;

FPostProcessMaterialChain GetPostProcessMaterialChain(const FViewInfo& View, EBlendableLocation Location);

BEGIN_SHADER_PARAMETER_STRUCT(FPostProcessMaterialParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
	SHADER_PARAMETER_STRUCT_INCLUDE(FNeuralPostProcessShaderParameters, NeuralPostProcessParameters)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, PostProcessOutput)
	SHADER_PARAMETER_STRUCT_ARRAY(FScreenPassTextureSliceInput, PostProcessInput, [kPostProcessMaterialInputCountMax])
	SHADER_PARAMETER_STRUCT_ARRAY(FScreenPassTextureInput, PathTracingPostProcessInput, [kPathTracingPostProcessMaterialInputCountMax])
	SHADER_PARAMETER_SAMPLER(SamplerState, PostProcessInput_BilinearSampler)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
	SHADER_PARAMETER(uint32, bMetalMSAAHDRDecode)
	SHADER_PARAMETER(uint32, bSceneDepthWithoutWaterTextureAvailable)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthWithoutSingleLayerWaterTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthWithoutSingleLayerWaterSampler)
	SHADER_PARAMETER(FVector4f, SceneWithoutSingleLayerWaterMinMaxUV)
	SHADER_PARAMETER(FVector2f, SceneWithoutSingleLayerWaterTextureSize)
	SHADER_PARAMETER(FVector2f, SceneWithoutSingleLayerWaterInvTextureSize)
	SHADER_PARAMETER(uint32, ManualStencilReferenceValue)
	SHADER_PARAMETER(uint32, ManualStencilTestMask)
	SHADER_PARAMETER(uint32, UserSceneTextureSceneColorInput)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()



FScreenPassTexture AddPostProcessMaterialPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FPostProcessMaterialInputs& Inputs,
	const UMaterialInterface* MaterialInterface);

FScreenPassTexture AddPostProcessMaterialChain(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	int32 ViewIndex,
	const FPostProcessMaterialInputs& Inputs,
	const FPostProcessMaterialChain& MaterialChain,
	EPostProcessMaterialInput MaterialInput = EPostProcessMaterialInput::SceneColor);

struct FHighResolutionScreenshotMaskInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	FScreenPassTexture SceneColor;

	FSceneTextureShaderParameters SceneTextures;

	UMaterialInterface* Material = nullptr;
	UMaterialInterface* MaskMaterial = nullptr;
	UMaterialInterface* CaptureRegionMaterial = nullptr;
};

bool IsHighResolutionScreenshotMaskEnabled(const FViewInfo& View);
bool IsPathTracingVarianceTextureRequiredInPostProcessMaterial(const FViewInfo& View);

FScreenPassTexture AddHighResolutionScreenshotMaskPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FHighResolutionScreenshotMaskInputs& Inputs);