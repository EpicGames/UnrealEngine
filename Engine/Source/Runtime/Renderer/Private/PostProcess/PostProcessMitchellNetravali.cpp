// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessMitchellNetravali.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneUtils.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "PostProcessing.h"

class FMitchellNetravaliDownsampleCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMitchellNetravaliDownsampleCS);
	SHADER_USE_PARAMETER_STRUCT(FMitchellNetravaliDownsampleCS, FGlobalShader);

	class FAlphaChannelDim : SHADER_PERMUTATION_BOOL("DIM_ALPHA_CHANNEL");
	using FPermutationDomain = TShaderPermutationDomain<FAlphaChannelDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) && !IsOpenGLPlatform(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutputTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER(FVector2f, DispatchThreadToInputUVScale)
		SHADER_PARAMETER(FVector2f, DispatchThreadToInputUVBias)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FMitchellNetravaliDownsampleCS, "/Engine/Private/PostProcessMitchellNetravali.usf", "DownsampleMainCS", SF_Compute);

FRDGTextureRef ComputeMitchellNetravaliDownsample(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture Input,
	const FScreenPassTextureViewport OutputViewport)
{
	const FRDGTextureDesc OutputTextureDesc = FRDGTextureDesc::Create2D(
		OutputViewport.Extent,
		PF_FloatRGBA,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV);

	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputTextureDesc, TEXT("MitchelNetravaliDownsampleOutput"));

	FMitchellNetravaliDownsampleCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMitchellNetravaliDownsampleCS::FParameters>();
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Input));
	PassParameters->InputTexture = Input.Texture;
	PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->OutputTexture = GraphBuilder.CreateUAV(OutputTexture);
	PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));

	// Scale / Bias factor to map the dispatch thread id to the input texture UV.
	PassParameters->DispatchThreadToInputUVScale.X = Input.ViewRect.Width()  / float(OutputViewport.Rect.Width()  * Input.Texture->Desc.Extent.X);
	PassParameters->DispatchThreadToInputUVScale.Y = Input.ViewRect.Height() / float(OutputViewport.Rect.Height() * Input.Texture->Desc.Extent.Y);
	PassParameters->DispatchThreadToInputUVBias.X = PassParameters->DispatchThreadToInputUVScale.X * (0.5f + Input.ViewRect.Min.X);
	PassParameters->DispatchThreadToInputUVBias.Y = PassParameters->DispatchThreadToInputUVScale.Y * (0.5f + Input.ViewRect.Min.Y);

	FMitchellNetravaliDownsampleCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FMitchellNetravaliDownsampleCS::FAlphaChannelDim>(IsPostProcessingWithAlphaChannelSupported());

	TShaderMapRef<FMitchellNetravaliDownsampleCS> ComputeShader(View.ShaderMap, PermutationVector);

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("MitchellNetravaliDownsample(%s) %dx%d -> %dx%d",
			PermutationVector.Get<FMitchellNetravaliDownsampleCS::FAlphaChannelDim>() ? TEXT("Alpha") : TEXT(""),
			Input.ViewRect.Width(),
			Input.ViewRect.Height(),
			OutputViewport.Rect.Width(),
			OutputViewport.Rect.Height()),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), FComputeShaderUtils::kGolden2DGroupSize));

	return OutputTexture;
}