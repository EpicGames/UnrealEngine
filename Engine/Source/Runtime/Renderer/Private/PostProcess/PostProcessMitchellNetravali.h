// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

FRDGTextureRef ComputeMitchellNetravaliDownsample(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FScreenPassTexture Input,
	const FScreenPassTextureViewport OutputViewport);