// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessMobile.cpp: Uber post for mobile implementation.
=============================================================================*/

#include "PostProcess/PostProcessMobile.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Texture.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "SceneRenderTargetParameters.h"
#include "SceneRendering.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessEyeAdaptation.h"
#include "PostProcess/PostProcessBloomSetup.h"
#include "PipelineStateCache.h"
#include "ClearQuad.h"
#include "PostProcess/PostProcessing.h"
#include "TextureResource.h"

static TAutoConsoleVariable<int32> CVarMobileSupportBloomSetupRareCases(
	TEXT("r.Mobile.MobileSupportBloomSetupRareCases"),
	0,
	TEXT("0: Don't generate permutations for BloomSetup rare cases. (default, like Sun+MetalMSAAHDRDecode, Dof+MetalMSAAHDRDecode, EyeAdaptaion+MetalMSAAHDRDecode, and any of their combinations)\n")
	TEXT("1: Generate permutations for BloomSetup rare cases. "),
	ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarMobileEyeAdaptation(
	TEXT("r.Mobile.EyeAdaptation"),
	1,
	TEXT("EyeAdaptation for mobile platform.\n")
	TEXT(" 0: Disable\n")
	TEXT(" 1: Enabled (Default)"),
	ECVF_RenderThreadSafe);

bool FMSAADecodeAndCopyRectPS_Mobile::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return IsMetalMobilePlatform(Parameters.Platform);
}

IMPLEMENT_GLOBAL_SHADER(FMSAADecodeAndCopyRectPS_Mobile, "/Engine/Private/PostProcessMobile.usf", "MSAADecodeAndCopyRectPS", SF_Pixel);

static EPixelFormat GetHDRPixelFormat()
{
	return PF_FloatRGBA;
}

// return Depth of Field Scale if Gaussian DoF mode is active. 0.0f otherwise.
float GetMobileDepthOfFieldScale(const FViewInfo& View)
{
	return View.FinalPostProcessSettings.DepthOfFieldScale;
}

bool IsMobileEyeAdaptationEnabled(const FViewInfo& View)
{
	return View.ViewState != nullptr && View.HasEyeAdaptationViewState() && View.Family->EngineShowFlags.EyeAdaptation && CVarMobileEyeAdaptation.GetValueOnRenderThread() == 1 && IsMobileHDR();
}

//Following variations are always generated
// 1 = Bloom
// 3 = Bloom + SunShaft
// 5 = Bloom + Dof
// 7 = Bloom + Dof + SunShaft
// 9 = Bloom + EyeAdaptation
// 11 = Bloom + SunShaft + EyeAdaptation
// 13 = Bloom + Dof + EyeAdaptation
// 15 = Bloom + SunShaft + Dof + EyeAdaptation
// 8 = EyeAdaptation

//Following variations should only be generated on IOS, only IOS has to do PreTonemapMSAA if MSAA is enabled.
// 17 = Bloom + MetalMSAAHDRDecode
// 21 = Bloom + Dof + MetalMSAAHDRDecode
// 25 = Bloom + EyeAdaptation + MetalMSAAHDRDecode
// 29 = Bloom + Dof + EyeAdaptation + MetalMSAAHDRDecode

//Following variations are rare cases, depends on CVarMobileSupportBloomSetupRareCases
// 2 = SunShaft
// 4 = Dof
// 6 = SunShaft + Dof

// 10 = SunShaft + EyeAdaptation
// 12 = Dof + EyeAdaptation
// 14 = SunShaft + Dof + EyeAdaptation

// 20 = Dof + MetalMSAAHDRDecode
// 24 = EyeAdaptation + MetalMSAAHDRDecode
// 28 = Dof + EyeAdaptation + MetalMSAAHDRDecode

//Any variations with SunShaft + MetalMSAAHDRDecode should be not generated, because SceneColor has been decoded at SunMask pass
// 19 = Bloom + SunShaft + MetalMSAAHDRDecode
// 23 = Bloom + Dof + SunShaft + MetalMSAAHDRDecode
// 27 = Bloom + SunShaft + EyeAdaptation + MetalMSAAHDRDecode
// 31 = Bloom + SunShaft + Dof + EyeAdaptation + MetalMSAAHDRDecode

// 18 = SunShaft + MetalMSAAHDRDecode
// 22 = Dof + SunShaft + MetalMSAAHDRDecode
// 26 = SunShaft + EyeAdaptation + MetalMSAAHDRDecode
// 30 = SunShaft + Dof + EyeAdaptation + MetalMSAAHDRDecode


// Remove the variation from this list if it should not be a rare case or enable the CVarMobileSupportBloomSetupRareCases for full cases.
bool IsValidBloomSetupVariation(uint32 Variation)
{
	bool bIsRareCases =
		Variation == 2 ||
		Variation == 4 ||
		Variation == 6 ||

		Variation == 10 ||
		Variation == 12 ||
		Variation == 14 ||
		
		Variation == 20 || 
		Variation == 24 || 
		Variation == 28;

	return !bIsRareCases || CVarMobileSupportBloomSetupRareCases.GetValueOnAnyThread() != 0;
}

bool IsValidBloomSetupVariation(bool bUseBloom, bool bUseSun, bool bUseDof, bool bUseEyeAdaptation)
{
	uint32 Variation = bUseBloom	? 1 << 0 : 0;
	Variation |= bUseSun			? 1 << 1 : 0;
	Variation |= bUseDof			? 1 << 2 : 0;
	Variation |= bUseEyeAdaptation	? 1 << 3 : 0;
	return IsValidBloomSetupVariation(Variation);
}

enum class EBloomSetupOutputType : uint32
{
	Bloom = 1 << 0,
	SunShaftAndDof = 1 << 1,
	EyeAdaptation = 1 << 2,
};

const TCHAR* GetBloomSetupOutputTypeName(EBloomSetupOutputType BloomSetupOutputType)
{
	switch (BloomSetupOutputType)
	{
	case EBloomSetupOutputType::Bloom : return TEXT("BloomSetup_Bloom");
	case EBloomSetupOutputType::SunShaftAndDof: return TEXT("BloomSetup_SunShaftAndDof");
	case EBloomSetupOutputType::EyeAdaptation: return TEXT("BloomSetup_EyeAdaptation");
	default: return TEXT("Unknown");
	}
}

TArray<EBloomSetupOutputType> GetBloomSetupOutputType(bool bUseBloom, bool bUseSun, bool bUseDof, bool bUseEyeAdaptation)
{
	bool bValidVariation = IsValidBloomSetupVariation(bUseBloom, bUseSun, bUseDof, bUseEyeAdaptation);

	TArray<EBloomSetupOutputType> BloomSetupOutputType;

	//if the variation is invalid, always use bloom permutation
	if (!bValidVariation || bUseBloom)
	{
		BloomSetupOutputType.Add(EBloomSetupOutputType::Bloom);
	}

	if (bUseSun || bUseDof)
	{
		BloomSetupOutputType.Add(EBloomSetupOutputType::SunShaftAndDof);
	}

	if (bUseEyeAdaptation)
	{
		BloomSetupOutputType.Add(EBloomSetupOutputType::EyeAdaptation);
	}

	checkSlow(BloomSetupOutputType.Num() != 0);

	return MoveTemp(BloomSetupOutputType);
}

//
// BLOOM SETUP
//

class FMobileBloomSetupVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileBloomSetupVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileBloomSetupVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector4f, BufferSizeAndInvSize)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileBloomSetupVS, "/Engine/Private/PostProcessMobile.usf", "BloomVS_Mobile", SF_Vertex);


class FMobileBloomSetupPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileBloomSetupPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileBloomSetupPS, FGlobalShader);

	class FUseBloomDim :				SHADER_PERMUTATION_BOOL("MOBILE_USEBLOOM");
	class FUseSunDim :					SHADER_PERMUTATION_BOOL("MOBILE_USESUN");
	class FUseDofDim :					SHADER_PERMUTATION_BOOL("MOBILE_USEDOF");
	class FUseEyeAdaptationDim :		SHADER_PERMUTATION_BOOL("MOBILE_USEEYEADAPTATION");
	class FUseMetalMSAAHDRDecodeDim :	SHADER_PERMUTATION_BOOL("METAL_MSAA_HDR_DECODE");

	using FPermutationDomain = TShaderPermutationDomain<
		FUseBloomDim,
		FUseSunDim,
		FUseDofDim,
		FUseEyeAdaptationDim,
		FUseMetalMSAAHDRDecodeDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(float, BloomThreshold)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMultiView, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SunShaftAndDofTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SunShaftAndDofSampler)
		SHADER_PARAMETER(FVector4f, BufferViewRect)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		auto bUseBloomDim = PermutationVector.Get<FUseBloomDim>();

		auto bUseSunDim = PermutationVector.Get<FUseSunDim>();

		auto bUseDofDim = PermutationVector.Get<FUseDofDim>();

		auto bUseEyeAdaptationDim = PermutationVector.Get<FUseEyeAdaptationDim>();

		auto bUseMetalMSAAHDRDecodeDim = PermutationVector.Get<FUseMetalMSAAHDRDecodeDim>();

		bool bValidVariation = IsValidBloomSetupVariation(bUseBloomDim, bUseSunDim, bUseDofDim, bUseEyeAdaptationDim);

		return IsMobilePlatform(Parameters.Platform) && 
			// Exclude rare cases if CVarMobileSupportBloomSetupRareCases is 0
			(bValidVariation) && 
			// IOS should generate all valid variations except SunShaft + MetalMSAAHDRDecode, other mobile platform should exclude MetalMSAAHDRDecode permutation
			(!bUseMetalMSAAHDRDecodeDim || (IsMetalMobilePlatform(Parameters.Platform) && !bUseSunDim));
	}

	static FPermutationDomain RemapPermutationVector(FPermutationDomain PermutationVector, bool bValidVariation)
	{
		if (!bValidVariation)
		{
			//Use the permutation with Bloom
			PermutationVector.Set<FUseBloomDim>(true);
		}
		return PermutationVector;
	}

	static FPermutationDomain BuildPermutationVector(bool bInUseBloom, bool bInUseSun, bool bInUseDof, bool bInUseEyeAdaptation, bool bInUseMetalMSAAHDRDecode)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FUseBloomDim>(bInUseBloom);
		PermutationVector.Set<FUseSunDim>(bInUseSun);
		PermutationVector.Set<FUseDofDim>(bInUseDof);
		PermutationVector.Set<FUseEyeAdaptationDim>(bInUseEyeAdaptation);
		PermutationVector.Set<FUseMetalMSAAHDRDecodeDim>(bInUseMetalMSAAHDRDecode);
		return RemapPermutationVector(PermutationVector, IsValidBloomSetupVariation(bInUseBloom, bInUseSun, bInUseDof, bInUseEyeAdaptation));
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileBloomSetupPS, "/Engine/Private/PostProcessMobile.usf", "BloomPS_Mobile", SF_Pixel);

FMobileBloomSetupOutputs AddMobileBloomSetupPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FEyeAdaptationParameters& EyeAdaptationParameters, const FMobileBloomSetupInputs& Inputs)
{
	const FIntPoint OutputSize = FIntPoint::DivideAndRoundUp(Inputs.SceneColor.ViewRect.Size(), (GetBloomQuality() <= EBloomQuality::Q2) ? 4 : 2);
	const FIntPoint OutputFixedTextureSize = Inputs.OutputFixedTextureSize;

	const FIntPoint& BufferSize = Inputs.SceneColor.Texture->Desc.Extent;

	bool bIsValidVariation = IsValidBloomSetupVariation(Inputs.bUseBloom, Inputs.bUseSun, Inputs.bUseDof, Inputs.bUseEyeAdaptation);

	TArray<EBloomSetupOutputType> BloomSetupOutputType = GetBloomSetupOutputType(Inputs.bUseBloom, Inputs.bUseSun, Inputs.bUseDof, Inputs.bUseEyeAdaptation);

	const auto GetBloomSetupTarget = [&GraphBuilder, bIsValidVariation, &Inputs, &OutputFixedTextureSize, &BloomSetupOutputType, &View](int32 OutputIndex)
	{
		checkSlow(OutputIndex < BloomSetupOutputType.Num());

		ETextureCreateFlags TargetableFlags = TexCreate_RenderTargetable | TexCreate_ShaderResource;

		EPixelFormat Format = PF_R16F;

		if (BloomSetupOutputType[OutputIndex] == EBloomSetupOutputType::Bloom)
		{
			checkSlow(OutputIndex == 0);

			if (!bIsValidVariation)
			{
				TargetableFlags |= TexCreate_Memoryless;
			}

			Format = PF_FloatR11G11B10;
		}

		FRDGTextureDesc BloomSetupDesc = View.CreateStereoTextureDesc(OutputFixedTextureSize, Format, FClearValueBinding::Black, TargetableFlags);

		return FScreenPassRenderTarget(GraphBuilder.CreateTexture(BloomSetupDesc, GetBloomSetupOutputTypeName(BloomSetupOutputType[OutputIndex])), ERenderTargetLoadAction::ENoAction);
	};
	
	TArray<FScreenPassRenderTarget> DestRenderTargets;

	for (int32 i = 0; i < BloomSetupOutputType.Num(); ++i)
	{
		DestRenderTargets.Add(GetBloomSetupTarget(i));
	}

	TShaderMapRef<FMobileBloomSetupVS> VertexShader(View.ShaderMap);

	FMobileBloomSetupVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();
	VSShaderParameters.BufferSizeAndInvSize = FVector4f(BufferSize.X, BufferSize.Y, 1.0f / BufferSize.X, 1.0f / BufferSize.Y);

	auto ShaderPermutationVector = FMobileBloomSetupPS::BuildPermutationVector(Inputs.bUseBloom, Inputs.bUseSun, Inputs.bUseDof, Inputs.bUseEyeAdaptation, Inputs.bUseMetalMSAAHDRDecode);

	TShaderMapRef<FMobileBloomSetupPS> PixelShader(View.ShaderMap, ShaderPermutationVector);

	FMobileBloomSetupPS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileBloomSetupPS::FParameters>();
	
	for (int32 i = 0; i < BloomSetupOutputType.Num(); ++i)
	{
		PSShaderParameters->RenderTargets[i] = DestRenderTargets[i].GetRenderTargetBinding();
	}
	// If the scenecolor isn't multiview but the app is, need to render as a single-view multiview due to shaders
	PSShaderParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();
	
	PSShaderParameters->EyeAdaptation = EyeAdaptationParameters;
	PSShaderParameters->BloomThreshold = View.FinalPostProcessSettings.BloomThreshold;
	PSShaderParameters->View = View.GetShaderParameters();

	PSShaderParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PSShaderParameters->SceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PSShaderParameters->SunShaftAndDofTexture = Inputs.SunShaftAndDof.Texture;
	PSShaderParameters->SunShaftAndDofSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PSShaderParameters->BufferViewRect = FVector4f(
		(Inputs.SceneColor.ViewRect.Min.X + 0.5f) / BufferSize.X,
		(Inputs.SceneColor.ViewRect.Min.Y + 0.5f) / BufferSize.Y,
		(Inputs.SceneColor.ViewRect.Max.X - 0.5f) / BufferSize.X,
		(Inputs.SceneColor.ViewRect.Max.Y - 0.5f) / BufferSize.Y);

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(OutputSize);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("BloomSetup %dx%d in %dx%d (PS) ", OutputViewport.Extent.X, OutputViewport.Extent.Y, OutputFixedTextureSize.X, OutputFixedTextureSize.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, InputViewport, OutputViewport, BloomSetupOutputType, &View](FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			InputViewport.Rect.Min.X, InputViewport.Rect.Min.Y,
			InputViewport.Rect.Width(), InputViewport.Rect.Height(),
			OutputViewport.Extent,
			InputViewport.Extent,
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	FMobileBloomSetupOutputs Outputs;

	for (int32 i = 0; i < BloomSetupOutputType.Num(); ++i)
	{
		if (BloomSetupOutputType[i] == EBloomSetupOutputType::Bloom)
		{
			Outputs.Bloom = FScreenPassTexture(DestRenderTargets[i].Texture, OutputViewport.Rect);
		}
		else if(BloomSetupOutputType[i] == EBloomSetupOutputType::SunShaftAndDof)
		{
			Outputs.SunShaftAndDof = FScreenPassTexture(DestRenderTargets[i].Texture, OutputViewport.Rect);
		}
		else if (BloomSetupOutputType[i] == EBloomSetupOutputType::EyeAdaptation)
		{
			Outputs.EyeAdaptation = FScreenPassTexture(DestRenderTargets[i].Texture, OutputViewport.Rect);
		}
	}

	return MoveTemp(Outputs);
}


//
// BLOOM DOWNSAMPLE
//

class FMobileBloomDownPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileBloomDownPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileBloomDownPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMultiView, BloomDownSourceTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BloomDownSourceSampler)
		SHADER_PARAMETER(FVector4f, InputViewportRect)
		SHADER_PARAMETER(FVector4f, TextureSizeAndInvSize)
		SHADER_PARAMETER(float, BloomDownScale)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileBloomDownPS, "/Engine/Private/PostProcessMobile.usf", "BloomDownPS_Mobile", SF_Pixel);

class FMobileBloomDownVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileBloomDownVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileBloomDownVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector4f, BufferSizeAndInvSize)
		SHADER_PARAMETER(float, BloomDownScale)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileBloomDownVS, "/Engine/Private/PostProcessMobile.usf", "BloomDownVS_Mobile", SF_Vertex);

FScreenPassTexture AddMobileBloomDownPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileBloomDownInputs& Inputs)
{
	const FIntPoint& BufferSize = FIntPoint(Inputs.BloomDownSource.ViewRect.Max.X, Inputs.BloomDownSource.ViewRect.Max.Y);

	FRDGTextureDesc BloomDownDesc = View.CreateStereoTextureDesc(Inputs.BloomDownOutputFixedTextureExtent, PF_FloatR11G11B10, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);

	FScreenPassRenderTarget BloomDownOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(BloomDownDesc, TEXT("BloomDown")), FIntRect(0,0,Inputs.BloomDownOutputViewExtent.X,Inputs.BloomDownOutputViewExtent.Y), ERenderTargetLoadAction::EClear);

	TShaderMapRef<FMobileBloomDownVS> VertexShader(View.ShaderMap);

	FMobileBloomDownVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();
	VSShaderParameters.BufferSizeAndInvSize = FVector4f(BufferSize.X, BufferSize.Y, 1.0f / BufferSize.X, 1.0f / BufferSize.Y);
	VSShaderParameters.BloomDownScale = Inputs.BloomDownScale;

	TShaderMapRef<FMobileBloomDownPS> PixelShader(View.ShaderMap);

	FMobileBloomDownPS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileBloomDownPS::FParameters>();
	PSShaderParameters->RenderTargets[0] = BloomDownOutput.GetRenderTargetBinding();
	PSShaderParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();
	PSShaderParameters->View = View.GetShaderParameters();

	PSShaderParameters->BloomDownSourceTexture = Inputs.BloomDownSource.Texture;
	PSShaderParameters->BloomDownSourceSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PSShaderParameters->InputViewportRect = FVector4f(
		Inputs.BloomDownSource.ViewRect.Min.X,
		Inputs.BloomDownSource.ViewRect.Min.Y,
		Inputs.BloomDownSource.ViewRect.Width(),
		Inputs.BloomDownSource.ViewRect.Height()
	);
	PSShaderParameters->TextureSizeAndInvSize = FVector4f(
		Inputs.BloomDownSource.Texture->Desc.Extent.X,
		Inputs.BloomDownSource.Texture->Desc.Extent.Y,
		1.0f / Inputs.BloomDownSource.Texture->Desc.Extent.X,
		1.0f / Inputs.BloomDownSource.Texture->Desc.Extent.Y
	);
	PSShaderParameters->BloomDownScale = Inputs.BloomDownScale;

	const FScreenPassTextureViewport InputViewport(Inputs.BloomDownSource.ViewRect);
	const FIntPoint InputTextureSize = Inputs.BloomDownSource.Texture->Desc.Extent;
	const FScreenPassTextureViewport OutputViewport(Inputs.BloomDownOutputViewExtent);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("BloomDown %dx%d in %dx%d RT (PS)", OutputViewport.Extent.X, OutputViewport.Extent.Y, Inputs.BloomDownOutputFixedTextureExtent.X, Inputs.BloomDownOutputFixedTextureExtent.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, InputViewport, InputTextureSize, OutputViewport, &View](FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			0, 0,
			InputViewport.Rect.Width(), InputViewport.Rect.Height(),
			OutputViewport.Extent,
			InputTextureSize,
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	return MoveTemp(BloomDownOutput);
}

//
// BLOOM UPSAMPLE
//

class FMobileBloomUpPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileBloomUpPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileBloomUpPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, BloomTintA)
		SHADER_PARAMETER(FVector4f, BloomTintB)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMultiView, BloomUpSourceATexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BloomUpSourceASampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMultiView, BloomUpSourceBTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BloomUpSourceBSampler)
		SHADER_PARAMETER(FVector2f, BloomUpScales)
		SHADER_PARAMETER(FVector4f, ViewportARect)
		SHADER_PARAMETER(FVector4f, ViewportBRect)
		SHADER_PARAMETER(FVector4f, TextureASizeAndInvSize)
		SHADER_PARAMETER(FVector4f, TextureBSizeAndInvSize)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileBloomUpPS, "/Engine/Private/PostProcessMobile.usf", "BloomUpPS_Mobile", SF_Pixel);

class FMobileBloomUpVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileBloomUpVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileBloomUpVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector4f, BufferASizeAndInvSize)  // Texture A
		SHADER_PARAMETER(FVector4f, BufferBSizeAndInvSize)  // Texture B
		SHADER_PARAMETER(FVector2f, BloomUpScales)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileBloomUpVS, "/Engine/Private/PostProcessMobile.usf", "BloomUpVS_Mobile", SF_Vertex);

FScreenPassTexture AddMobileBloomUpPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileBloomUpInputs& Inputs)
{
	// Use actual texture sizes for UV calculations
	const FIntPoint& TextureSizeA = Inputs.BloomUpSourceA.Texture->Desc.Extent;
	const FIntPoint& TextureSizeB = Inputs.BloomUpSourceB.Texture->Desc.Extent;

	// Get viewport sizes
	const FIntRect& ViewportRectA = Inputs.BloomUpSourceA.ViewRect;
	const FIntRect& ViewportRectB = Inputs.BloomUpSourceB.ViewRect;

	FRDGTextureDesc BloomUpDesc = View.CreateStereoTextureDesc(Inputs.BloomUpOutputFixedTextureExtent, PF_FloatR11G11B10, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);

	FScreenPassRenderTarget BloomUpOutput = FScreenPassRenderTarget(
		GraphBuilder.CreateTexture(BloomUpDesc, TEXT("BloomUp")),
		FIntRect(0, 0, Inputs.BloomUpOutputViewExtent.X, Inputs.BloomUpOutputViewExtent.Y),
		ERenderTargetLoadAction::EClear
	);

	TShaderMapRef<FMobileBloomUpVS> VertexShader(View.ShaderMap);

	FMobileBloomUpVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();

	// Pass TEXTURE sizes for correct inverse calculations
	VSShaderParameters.BufferASizeAndInvSize = FVector4f(
		TextureSizeA.X, TextureSizeA.Y,
		1.0f / TextureSizeA.X, 1.0f / TextureSizeA.Y
	);
	VSShaderParameters.BufferBSizeAndInvSize = FVector4f(
		TextureSizeB.X, TextureSizeB.Y,
		1.0f / TextureSizeB.X, 1.0f / TextureSizeB.Y
	);

	VSShaderParameters.BloomUpScales = FVector2f(Inputs.ScaleAB);

	TShaderMapRef<FMobileBloomUpPS> PixelShader(View.ShaderMap);

	FMobileBloomUpPS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileBloomUpPS::FParameters>();
	PSShaderParameters->RenderTargets[0] = BloomUpOutput.GetRenderTargetBinding();
	PSShaderParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();
	PSShaderParameters->View = View.GetShaderParameters();

	PSShaderParameters->BloomUpSourceATexture = Inputs.BloomUpSourceA.Texture;
	PSShaderParameters->BloomUpSourceASampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSShaderParameters->BloomUpSourceBTexture = Inputs.BloomUpSourceB.Texture;
	PSShaderParameters->BloomUpSourceBSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PSShaderParameters->BloomTintA = Inputs.TintA * (1.0f / 8.0f);
	PSShaderParameters->BloomTintB = Inputs.TintB * (1.0f / 8.0f);

	PSShaderParameters->ViewportARect = FVector4f(
		ViewportRectA.Min.X,
		ViewportRectA.Min.Y,
		ViewportRectA.Width(),
		ViewportRectA.Height()
	);
	PSShaderParameters->ViewportBRect = FVector4f(
		ViewportRectB.Min.X,
		ViewportRectB.Min.Y,
		ViewportRectB.Width(),
		ViewportRectB.Height()
	);
	PSShaderParameters->TextureASizeAndInvSize = FVector4f(
		TextureSizeA.X,
		TextureSizeA.Y,
		1.0f / TextureSizeA.X,
		1.0f / TextureSizeA.Y
	);
	PSShaderParameters->TextureBSizeAndInvSize = FVector4f(
		TextureSizeB.X,
		TextureSizeB.Y,
		1.0f / TextureSizeB.X,
		1.0f / TextureSizeB.Y
	);

	PSShaderParameters->BloomUpScales = FVector2f(Inputs.ScaleAB);

	const FScreenPassTextureViewport OutputViewport(BloomUpOutput);
	const FIntPoint OutputViewportSize = Inputs.BloomUpOutputViewExtent;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("BloomUp %dx%d in %dx%d RT (PS)", OutputViewportSize.X, OutputViewportSize.Y, Inputs.BloomUpOutputFixedTextureExtent.X, Inputs.BloomUpOutputFixedTextureExtent.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, OutputViewport, OutputViewportSize, ViewportRectA, &View](FRHICommandList& RHICmdList)
	{
			RHICmdList.SetViewport(0, 0, 0.0f, OutputViewportSize.X, OutputViewportSize.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
				OutputViewportSize.X, OutputViewportSize.Y,
				ViewportRectA.Min.X, ViewportRectA.Min.Y,
				ViewportRectA.Width(), ViewportRectA.Height(),
				OutputViewportSize,
				FIntPoint(ViewportRectA.Width(), ViewportRectA.Height()),
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	return MoveTemp(BloomUpOutput);
}

//
// SUN MASK
//

class FMobileSunMaskPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileSunMaskPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileSunMaskPS, FGlobalShader);

	class FUseSunDim :					SHADER_PERMUTATION_BOOL("MOBILE_USESUN");
	class FUseDofDim :					SHADER_PERMUTATION_BOOL("MOBILE_USEDOF");
	class FUseDepthTextureDim :			SHADER_PERMUTATION_BOOL("MOBILE_USEDEPTHTEXTURE");
	class FUseMetalMSAAHDRDecodeDim :	SHADER_PERMUTATION_BOOL("METAL_MSAA_HDR_DECODE");

	using FPermutationDomain = TShaderPermutationDomain<
		FUseSunDim,
		FUseDofDim, 
		FUseDepthTextureDim,
		FUseMetalMSAAHDRDecodeDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, RENDERER_API)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER(FVector4f, SunColorApertureDiv2)
		SHADER_PARAMETER(float, BloomMaxBrightness)
		SHADER_PARAMETER(float, BloomThreshold)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		auto bUseSunDim = PermutationVector.Get<FUseSunDim>();

		auto bUseDofDim = PermutationVector.Get<FUseDofDim>();

		auto bUseMetalMSAAHDRDecodeDim = PermutationVector.Get<FUseMetalMSAAHDRDecodeDim>();

		return IsMobilePlatform(Parameters.Platform) && 
				// Only generate shaders with SunShaft and/or Dof
				(bUseSunDim || bUseDofDim) && 
				// Only generated MetalMSAAHDRDecode shaders for SunShaft
				(!bUseMetalMSAAHDRDecodeDim || (bUseSunDim && IsMetalMobilePlatform(Parameters.Platform)));
	}	

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_SUN_MASK"), 1);
	}

	static FPermutationDomain RemapPermutationVector(FPermutationDomain PermutationVector)
	{
		auto UseSunDim = PermutationVector.Get<FUseSunDim>();

		if (!UseSunDim)
		{
			// Don't use MetalMSAAHDRDecode permutation without SunShaft
			PermutationVector.Set<FUseMetalMSAAHDRDecodeDim>(false);
		}

		return PermutationVector;
	}

	static FPermutationDomain BuildPermutationVector(bool bInUseSun, bool bInUseDof, bool bInUseDepthTexture, bool bInUseMetalMSAAHDRDecode)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FUseSunDim>(bInUseSun);
		PermutationVector.Set<FUseDofDim>(bInUseDof);
		PermutationVector.Set<FUseDepthTextureDim>(bInUseDepthTexture);
		PermutationVector.Set<FUseMetalMSAAHDRDecodeDim>(bInUseMetalMSAAHDRDecode);
		return RemapPermutationVector(PermutationVector);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileSunMaskPS, "/Engine/Private/PostProcessMobile.usf", "SunMaskPS_Mobile", SF_Pixel);

FMobileSunMaskOutputs AddMobileSunMaskPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileSunMaskInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	FScreenPassRenderTarget SunMaskOutput;
	FScreenPassRenderTarget SceneColorOutput;

	{
		FRDGTextureDesc OutputDesc = Inputs.SceneColor.Texture->Desc;
		OutputDesc.Reset();

		if (Inputs.bUseSun && Inputs.bUseMetalMSAAHDRDecode)
		{
			if (IsMobilePropagateAlphaEnabled(View.GetShaderPlatform()))
			{
				OutputDesc.Format = PF_FloatRGBA;
			}
			else
			{
				OutputDesc.Format = PF_FloatR11G11B10;
			}
			SceneColorOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, TEXT("SceneColor")), Inputs.SceneColor.ViewRect, ERenderTargetLoadAction::ENoAction);
		}

		OutputDesc.Format = PF_R16F;
		SunMaskOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, TEXT("SunMask")), Inputs.SceneColor.ViewRect, ERenderTargetLoadAction::ENoAction);
	}

	FMobileLightShaftInfo MobileLightShaft;
	if (View.MobileLightShaft)
	{
		MobileLightShaft = *View.MobileLightShaft;
	}

	FMobileSunMaskPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileSunMaskPS::FParameters>();
	PassParameters->RenderTargets[0] = SunMaskOutput.GetRenderTargetBinding();
	PassParameters->RenderTargets[1] = SceneColorOutput.GetRenderTargetBinding();

	PassParameters->View = View.GetShaderParameters();
	PassParameters->SceneTextures = Inputs.SceneTextures;
	PassParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters->SceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PassParameters->SunColorApertureDiv2.X = MobileLightShaft.ColorMask.R;
	PassParameters->SunColorApertureDiv2.Y = MobileLightShaft.ColorMask.G;
	PassParameters->SunColorApertureDiv2.Z = MobileLightShaft.ColorMask.B;
	PassParameters->SunColorApertureDiv2.W = GetMobileDepthOfFieldScale(View) * 0.5f;

	PassParameters->BloomMaxBrightness = MobileLightShaft.BloomMaxBrightness;
	PassParameters->BloomThreshold = View.FinalPostProcessSettings.BloomThreshold;

	FMobileSunMaskPS::FPermutationDomain PermutationVector = FMobileSunMaskPS::BuildPermutationVector(Inputs.bUseSun, Inputs.bUseDof, Inputs.bUseDepthTexture, Inputs.bUseMetalMSAAHDRDecode);
	TShaderMapRef<FMobileSunMaskPS> PixelShader(View.ShaderMap, PermutationVector);

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(SunMaskOutput);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("MobileSunMask"), View, OutputViewport, InputViewport, PixelShader, PassParameters);

	FMobileSunMaskOutputs Outputs;
	Outputs.SunMask = SunMaskOutput;
	Outputs.SceneColor = SceneColorOutput;
	return MoveTemp(Outputs);
}

//
// SUN ALPHA
//
class FMobileSunAlphaPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileSunAlphaPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileSunAlphaPS, FGlobalShader);

	class FUseDofDim :					SHADER_PERMUTATION_BOOL("MOBILE_USEDOF");

	using FPermutationDomain = TShaderPermutationDomain<FUseDofDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SunShaftAndDofTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SunShaftAndDofSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static FPermutationDomain BuildPermutationVector(bool bInUseDof)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FUseDofDim>(bInUseDof);
		return PermutationVector;
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileSunAlphaPS, "/Engine/Private/PostProcessMobile.usf", "SunAlphaPS_Mobile", SF_Pixel);

class FMobileSunAlphaVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileSunAlphaVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileSunAlphaVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector2f, LightShaftCenter)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileSunAlphaVS, "/Engine/Private/PostProcessMobile.usf", "SunAlphaVS_Mobile", SF_Vertex);

FScreenPassTexture AddMobileSunAlphaPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileSunAlphaInputs& Inputs)
{
	const FIntPoint& BufferSize = Inputs.BloomSetup_SunShaftAndDof.Texture->Desc.Extent;

	FRDGTextureDesc SunAlphaDesc = FRDGTextureDesc::Create2D(BufferSize, PF_G8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);

	FScreenPassRenderTarget SunAlphaOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(SunAlphaDesc, TEXT("SunAlpha")), ERenderTargetLoadAction::EClear);

	TShaderMapRef<FMobileSunAlphaVS> VertexShader(View.ShaderMap);

	FMobileSunAlphaVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();
	VSShaderParameters.LightShaftCenter = FVector2f(View.MobileLightShaft->Center);	// LWC_TODO: Precision loss

	auto ShaderPermutationVector = FMobileSunAlphaPS::BuildPermutationVector(Inputs.bUseMobileDof);

	TShaderMapRef<FMobileSunAlphaPS> PixelShader(View.ShaderMap, ShaderPermutationVector);

	FMobileSunAlphaPS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileSunAlphaPS::FParameters>();
	PSShaderParameters->RenderTargets[0] = SunAlphaOutput.GetRenderTargetBinding();
	PSShaderParameters->View = View.GetShaderParameters();
	PSShaderParameters->SunShaftAndDofTexture = Inputs.BloomSetup_SunShaftAndDof.Texture;
	PSShaderParameters->SunShaftAndDofSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const FScreenPassTextureViewport InputViewport(Inputs.BloomSetup_SunShaftAndDof);
	const FScreenPassTextureViewport OutputViewport(SunAlphaOutput);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("SunAlpha %dx%d (PS)", OutputViewport.Extent.X, OutputViewport.Extent.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, InputViewport, OutputViewport, &View](FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			InputViewport.Rect.Min.X, InputViewport.Rect.Min.Y,      
			InputViewport.Rect.Width(), InputViewport.Rect.Height(), 
			OutputViewport.Extent,
			InputViewport.Extent,                                    
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	return MoveTemp(SunAlphaOutput);
}

//
// SUN BLUR
//

class FMobileSunBlurPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileSunBlurPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileSunBlurPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SunAlphaTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SunAlphaSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileSunBlurPS, "/Engine/Private/PostProcessMobile.usf", "SunBlurPS_Mobile", SF_Pixel);

class FMobileSunBlurVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileSunBlurVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileSunBlurVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector2f, LightShaftCenter)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileSunBlurVS, "/Engine/Private/PostProcessMobile.usf", "SunBlurVS_Mobile", SF_Vertex);

FScreenPassTexture AddMobileSunBlurPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileSunBlurInputs& Inputs)
{
	const FIntPoint& BufferSize = Inputs.SunAlpha.Texture->Desc.Extent;

	FRDGTextureDesc SunBlurDesc = FRDGTextureDesc::Create2D(BufferSize, PF_G8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);

	FScreenPassRenderTarget SunBlurOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(SunBlurDesc, TEXT("SunBlur")), ERenderTargetLoadAction::EClear);

	TShaderMapRef<FMobileSunBlurVS> VertexShader(View.ShaderMap);

	FMobileSunBlurVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();
	VSShaderParameters.LightShaftCenter = FVector2f(View.MobileLightShaft->Center);	// LWC_TODO: Precision loss

	TShaderMapRef<FMobileSunBlurPS> PixelShader(View.ShaderMap);

	FMobileSunBlurPS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileSunBlurPS::FParameters>();
	PSShaderParameters->RenderTargets[0] = SunBlurOutput.GetRenderTargetBinding();
	PSShaderParameters->View = View.GetShaderParameters();
	PSShaderParameters->SunAlphaTexture = Inputs.SunAlpha.Texture;
	PSShaderParameters->SunAlphaSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const FScreenPassTextureViewport OutputViewport(SunBlurOutput);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("SunBlur %dx%d (PS)", OutputViewport.Extent.X, OutputViewport.Extent.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, OutputViewport, &View](FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);


		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			OutputViewport.Extent,
			OutputViewport.Extent,
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	return MoveTemp(SunBlurOutput);
}

//
// SUN MERGE
//
class FMobileSunMergePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileSunMergePS);
	SHADER_USE_PARAMETER_STRUCT(FMobileSunMergePS, FGlobalShader);

	class FUseBloomDim :				SHADER_PERMUTATION_BOOL("MOBILE_USEBLOOM");
	class FUseSunDim :					SHADER_PERMUTATION_BOOL("MOBILE_USESUN");

	using FPermutationDomain = TShaderPermutationDomain<
		FUseBloomDim,
		FUseSunDim>;
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, BloomDirtMaskTint)
		SHADER_PARAMETER(FVector4f, SunColorVignetteIntensity)
		SHADER_PARAMETER(FVector3f, BloomColor)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SunBlurTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SunBlurSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMultiView, BloomSetup_BloomTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BloomSetup_BloomSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2DMultiView, BloomUpTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BloomUpSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BloomDirtMaskTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BloomDirtMaskSampler)

		SHADER_PARAMETER(FVector4f, SunMergeBloomSetupViewportRect)
		SHADER_PARAMETER(FVector4f, SunMergeBloomSetupTextureSizeAndInvSize)
		SHADER_PARAMETER(FVector4f, SunMergeBloomUpViewportRect)
		SHADER_PARAMETER(FVector4f, SunMergeBloomUpTextureSizeAndInvSize)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static FPermutationDomain BuildPermutationVector(bool bInUseBloom, bool bInUseSun)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FUseBloomDim>(bInUseBloom);
		PermutationVector.Set<FUseSunDim>(bInUseSun);
		return PermutationVector;
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileSunMergePS, "/Engine/Private/PostProcessMobile.usf", "SunMergePS_Mobile", SF_Pixel);

class FMobileSunMergeVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileSunMergeVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileSunMergeVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector2f, LightShaftCenter)
 		SHADER_PARAMETER(FVector4f, BloomUpTextureSizeAndInvSize)     // xy=TexSize, zw=1/TexSize
		SHADER_PARAMETER(FVector4f, ViewportSize)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileSunMergeVS, "/Engine/Private/PostProcessMobile.usf", "SunMergeVS_Mobile", SF_Vertex);

FScreenPassTexture AddMobileSunMergePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileSunMergeInputs& Inputs)
{
	FIntPoint OutputSize = Inputs.BloomSetup_Bloom.Texture->Desc.Extent;
	
	FRDGTextureDesc SunMergeDesc = View.CreateStereoTextureDesc(OutputSize, PF_FloatR11G11B10, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
	
	FScreenPassRenderTarget SunMergeOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(SunMergeDesc, TEXT("SunMerge")), ERenderTargetLoadAction::EClear);

	TShaderMapRef<FMobileSunMergeVS> VertexShader(View.ShaderMap);

	FMobileSunMergeVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();
	FMobileLightShaftInfo MobileLightShaft;
	if (View.MobileLightShaft)
	{
		MobileLightShaft = *View.MobileLightShaft;
	}
	VSShaderParameters.LightShaftCenter = FVector2f(MobileLightShaft.Center);	// LWC_TODO: Precision loss

	// Setup BloomSetup_Bloom parameters
	const FIntRect& BloomSetupViewportRect = Inputs.BloomSetup_Bloom.ViewRect;
	const FIntPoint BloomSetupTextureSize = Inputs.BloomSetup_Bloom.Texture->Desc.Extent;

	// Setup BloomUp parameters
	const FIntRect& BloomUpViewportRect = Inputs.BloomUp.ViewRect;
	const FIntPoint BloomUpTextureSize = Inputs.BloomUp.Texture ? Inputs.BloomUp.Texture->Desc.Extent : FIntPoint(1, 1);

	if (Inputs.BloomUp.IsValid())
	{
 		VSShaderParameters.BloomUpTextureSizeAndInvSize = FVector4f(
 			BloomUpTextureSize.X,
 			BloomUpTextureSize.Y,
 			1.0f / BloomUpTextureSize.X,
 			1.0f / BloomUpTextureSize.Y
 		);
	}
	else
	{
 		VSShaderParameters.BloomUpTextureSizeAndInvSize = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
	}

	VSShaderParameters.ViewportSize = FVector4f(OutputSize.X, OutputSize.Y, 1.0f / OutputSize.X, 1.0f / OutputSize.Y);
	
	auto ShaderPermutationVector = FMobileSunMergePS::BuildPermutationVector(Inputs.bUseBloom, Inputs.bUseSun);

	TShaderMapRef<FMobileSunMergePS> PixelShader(View.ShaderMap, ShaderPermutationVector);

	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;
	FVector4f SunColorVignetteIntensityParam(0.0f);
	SunColorVignetteIntensityParam.X = MobileLightShaft.ColorApply.R;
	SunColorVignetteIntensityParam.Y = MobileLightShaft.ColorApply.G;
	SunColorVignetteIntensityParam.Z = MobileLightShaft.ColorApply.B;
	SunColorVignetteIntensityParam.W = Settings.VignetteIntensity;

	const float BloomGaussianIntensity = Settings.BloomIntensity * Settings.BloomGaussianIntensity;
	FLinearColor BloomColor = Settings.Bloom1Tint * BloomGaussianIntensity * 0.5;

	FRDGTextureRef BloomDirtMaskTexture = GSystemTextures.GetBlackDummy(GraphBuilder);

	if (Settings.BloomDirtMask && Settings.BloomDirtMask->GetResource() && Settings.BloomDirtMask->GetResource()->TextureRHI)
	{
		BloomDirtMaskTexture = RegisterExternalTexture(GraphBuilder, Settings.BloomDirtMask->GetResource()->TextureRHI, TEXT("BloomDirtMask"));
	}

	FMobileSunMergePS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileSunMergePS::FParameters>();
	PSShaderParameters->RenderTargets[0] = SunMergeOutput.GetRenderTargetBinding();
	PSShaderParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();
	PSShaderParameters->View = View.GetShaderParameters();
	PSShaderParameters->BloomDirtMaskTint = Settings.BloomDirtMaskTint * Settings.BloomDirtMaskIntensity;
	PSShaderParameters->SunColorVignetteIntensity = SunColorVignetteIntensityParam;
	PSShaderParameters->BloomColor = FVector3f(BloomColor.R, BloomColor.G, BloomColor.B);
	PSShaderParameters->SunBlurTexture = Inputs.SunBlur.Texture;
	PSShaderParameters->SunBlurSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSShaderParameters->BloomSetup_BloomTexture = Inputs.BloomSetup_Bloom.Texture;
	PSShaderParameters->BloomSetup_BloomSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSShaderParameters->BloomUpTexture = Inputs.BloomUp.Texture;
	PSShaderParameters->BloomUpSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSShaderParameters->BloomDirtMaskTexture = BloomDirtMaskTexture;
	PSShaderParameters->BloomDirtMaskSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	PSShaderParameters->SunMergeBloomSetupViewportRect = FVector4f(
		BloomSetupViewportRect.Min.X,
		BloomSetupViewportRect.Min.Y,
		BloomSetupViewportRect.Width(),
		BloomSetupViewportRect.Height()
	);
	PSShaderParameters->SunMergeBloomSetupTextureSizeAndInvSize = FVector4f(
		BloomSetupTextureSize.X,
		BloomSetupTextureSize.Y,
		1.0f / BloomSetupTextureSize.X,
		1.0f / BloomSetupTextureSize.Y
	);
	PSShaderParameters->SunMergeBloomUpViewportRect = FVector4f(
		BloomUpViewportRect.Min.X,
		BloomUpViewportRect.Min.Y,
		BloomUpViewportRect.Width(),
		BloomUpViewportRect.Height()
	);
	PSShaderParameters->SunMergeBloomUpTextureSizeAndInvSize = FVector4f(
		BloomUpTextureSize.X,
		BloomUpTextureSize.Y,
		1.0f / BloomUpTextureSize.X,
		1.0f / BloomUpTextureSize.Y
	);

	const FScreenPassTextureViewport OutputViewport(SunMergeOutput);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("SunMerge %dx%d (PS)", OutputViewport.Extent.X, OutputViewport.Extent.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, OutputViewport, &View](FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			OutputViewport.Extent,
			OutputViewport.Extent,
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	return MoveTemp(SunMergeOutput);
}

//
// DOF DOWNSAMPLE
//

class FMobileDofDownVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileDofDownVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileDofDownVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector4f, BufferSizeAndInvSize)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileDofDownVS, "/Engine/Private/PostProcessMobile.usf", "DofDownVS_Mobile", SF_Vertex);

class FMobileDofDownPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileDofDownPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileDofDownPS, FGlobalShader);

	class FUseSunDim : SHADER_PERMUTATION_BOOL("MOBILE_USESUN");
	using FPermutationDomain = TShaderPermutationDomain<FUseSunDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SunShaftAndDofTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SunShaftAndDofSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DofNearTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DofNearSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static FPermutationDomain BuildPermutationVector(bool bInUseSun)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FUseSunDim>(bInUseSun);
		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.FullPrecisionInPS = 1;
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileDofDownPS, "/Engine/Private/PostProcessMobile.usf", "DofDownPS_Mobile", SF_Pixel);

FMobileDofDownOutputs AddMobileDofDownPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofDownInputs& Inputs)
{
	FIntPoint OutputSize = FIntPoint::DivideAndRoundUp(Inputs.SceneColor.ViewRect.Size(), 2);

	const FIntPoint& BufferSize = Inputs.SceneColor.Texture->Desc.Extent;

	FRDGTextureDesc DofDownDesc = FRDGTextureDesc::Create2D(OutputSize, GetHDRPixelFormat(), FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);

	FScreenPassRenderTarget DofDownOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(DofDownDesc, TEXT("DofDown")), ERenderTargetLoadAction::EClear);

	TShaderMapRef<FMobileDofDownVS> VertexShader(View.ShaderMap);

	FMobileDofDownVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();
	VSShaderParameters.BufferSizeAndInvSize = FVector4f(BufferSize.X, BufferSize.Y, 1.0f / BufferSize.X, 1.0f / BufferSize.Y);

	auto ShaderPermutationVector = FMobileDofDownPS::BuildPermutationVector(Inputs.bUseSun);

	TShaderMapRef<FMobileDofDownPS> PixelShader(View.ShaderMap, ShaderPermutationVector);

	FMobileDofDownPS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileDofDownPS::FParameters>();
	PSShaderParameters->RenderTargets[0] = DofDownOutput.GetRenderTargetBinding();
	PSShaderParameters->View = View.GetShaderParameters();

	PSShaderParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PSShaderParameters->SceneColorSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSShaderParameters->SunShaftAndDofTexture = Inputs.SunShaftAndDof.Texture;
	PSShaderParameters->SunShaftAndDofSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSShaderParameters->DofNearTexture = Inputs.DofNear.Texture;
	PSShaderParameters->DofNearSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(DofDownOutput);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("DofDown %dx%d (PS)", OutputViewport.Extent.X, OutputViewport.Extent.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, InputViewport, OutputViewport, &View](FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			InputViewport.Rect.Min.X, InputViewport.Rect.Min.Y,
			InputViewport.Rect.Width(), InputViewport.Rect.Height(),
			OutputViewport.Extent,
			InputViewport.Extent,
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	FMobileDofDownOutputs Outputs;

	Outputs.DofDown = DofDownOutput;

	return MoveTemp(Outputs);
}

//
// DOF NEAR
//

class FMobileDofNearVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileDofNearVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileDofNearVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector4f, BufferSizeAndInvSize)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileDofNearVS, "/Engine/Private/PostProcessMobile.usf", "DofNearVS_Mobile", SF_Vertex);

class FMobileDofNearPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileDofNearPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileDofNearPS, FGlobalShader);

	class FUseSunDim : SHADER_PERMUTATION_BOOL("MOBILE_USESUN");
	using FPermutationDomain = TShaderPermutationDomain<FUseSunDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SunShaftAndDofTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SunShaftAndDofSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static FPermutationDomain BuildPermutationVector(bool bInUseSun)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FUseSunDim>(bInUseSun);
		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.FullPrecisionInPS = 1;
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileDofNearPS, "/Engine/Private/PostProcessMobile.usf", "DofNearPS_Mobile", SF_Pixel);

FMobileDofNearOutputs AddMobileDofNearPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofNearInputs& Inputs)
{
	const FIntPoint& BufferSize = Inputs.BloomSetup_SunShaftAndDof.Texture->Desc.Extent;

	FRDGTextureDesc DofNearDesc = FRDGTextureDesc::Create2D(BufferSize, PF_G8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);

	FScreenPassRenderTarget DofNearOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(DofNearDesc, TEXT("DofNear")), ERenderTargetLoadAction::EClear);

	TShaderMapRef<FMobileDofNearVS> VertexShader(View.ShaderMap);

	FMobileDofNearVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();
	VSShaderParameters.BufferSizeAndInvSize = FVector4f(BufferSize.X, BufferSize.Y, 1.0f / BufferSize.X, 1.0f / BufferSize.Y);

	auto ShaderPermutationVector = FMobileDofNearPS::BuildPermutationVector(Inputs.bUseSun);

	TShaderMapRef<FMobileDofNearPS> PixelShader(View.ShaderMap, ShaderPermutationVector);

	FMobileDofNearPS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileDofNearPS::FParameters>();
	PSShaderParameters->RenderTargets[0] = DofNearOutput.GetRenderTargetBinding();
	PSShaderParameters->View = View.GetShaderParameters();

	PSShaderParameters->SunShaftAndDofTexture = Inputs.BloomSetup_SunShaftAndDof.Texture;
	PSShaderParameters->SunShaftAndDofSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const FScreenPassTextureViewport InputViewport(Inputs.BloomSetup_SunShaftAndDof);
	const FScreenPassTextureViewport OutputViewport(DofNearOutput);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("DofNear %dx%d (PS)", OutputViewport.Extent.X, OutputViewport.Extent.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, InputViewport, OutputViewport, &View](FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			InputViewport.Rect.Min.X, InputViewport.Rect.Min.Y,      
			InputViewport.Rect.Width(), InputViewport.Rect.Height(), 
			OutputViewport.Extent,
			InputViewport.Extent,                                    
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	FMobileDofNearOutputs Outputs;

	Outputs.DofNear = DofNearOutput;

	return MoveTemp(Outputs);
}

//
// DOF BLUR
//

class FMobileDofBlurPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileDofBlurPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileDofBlurPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DofNearTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DofNearSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DofDownTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DofDownSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.FullPrecisionInPS = 1;
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileDofBlurPS, "/Engine/Private/PostProcessMobile.usf", "DofBlurPS_Mobile", SF_Pixel);

class FMobileDofBlurVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileDofBlurVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileDofBlurVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector4f, BufferSizeAndInvSize)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileDofBlurVS, "/Engine/Private/PostProcessMobile.usf", "DofBlurVS_Mobile", SF_Vertex);

FMobileDofBlurOutputs AddMobileDofBlurPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofBlurInputs& Inputs)
{
	const FIntPoint& BufferSize = Inputs.DofDown.Texture->Desc.Extent;

	FRDGTextureDesc DofBlurDesc = FRDGTextureDesc::Create2D(BufferSize, GetHDRPixelFormat(), FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);

	FScreenPassRenderTarget DofBlurOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(DofBlurDesc, TEXT("DofBlur")), ERenderTargetLoadAction::EClear);

	TShaderMapRef<FMobileDofBlurVS> VertexShader(View.ShaderMap);

	FMobileDofBlurVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();
	VSShaderParameters.BufferSizeAndInvSize = FVector4f(BufferSize.X, BufferSize.Y, 1.0f / BufferSize.X, 1.0f / BufferSize.Y);

	TShaderMapRef<FMobileDofBlurPS> PixelShader(View.ShaderMap);

	FMobileDofBlurPS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileDofBlurPS::FParameters>();
	PSShaderParameters->RenderTargets[0] = DofBlurOutput.GetRenderTargetBinding();
	PSShaderParameters->View = View.GetShaderParameters();

	PSShaderParameters->DofDownTexture = Inputs.DofDown.Texture;
	PSShaderParameters->DofDownSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSShaderParameters->DofNearTexture = Inputs.DofNear.Texture;
	PSShaderParameters->DofNearSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const FScreenPassTextureViewport OutputViewport(DofBlurOutput);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("DofBlur %dx%d (PS)", OutputViewport.Extent.X, OutputViewport.Extent.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, OutputViewport, &View](FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			OutputViewport.Extent,
			OutputViewport.Extent,
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	FMobileDofBlurOutputs Outputs;

	Outputs.DofBlur = DofBlurOutput;

	return MoveTemp(Outputs);
}

class FMobileIntegrateDofPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileIntegrateDofPS);
	SHADER_USE_PARAMETER_STRUCT(FMobileIntegrateDofPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DofBlurTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DofBlurSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SunShaftAndDofTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SunShaftAndDofSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.FullPrecisionInPS = 1;
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileIntegrateDofPS, "/Engine/Private/PostProcessMobile.usf", "IntegrateDOFPS_Mobile", SF_Pixel);

class FMobileIntegrateDofVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileIntegrateDofVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FMobileIntegrateDofVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector4f, BufferSizeAndInvSize)
		SHADER_PARAMETER(FVector4f, DofBlurSizeAndInvSize)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileIntegrateDofVS, "/Engine/Private/PostProcessMobile.usf", "IntegrateDOFVS_Mobile", SF_Vertex);

FScreenPassTexture AddMobileIntegrateDofPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileIntegrateDofInputs& Inputs)
{
	const FIntPoint& BufferSize = Inputs.SceneColor.Texture->Desc.Extent;
	const FIntPoint& DofBlurSize = Inputs.DofBlur.Texture->Desc.Extent;

	FScreenPassRenderTarget IntegrateDofOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(Inputs.SceneColor.Texture->Desc, TEXT("IntegrateDof")), Inputs.SceneColor.ViewRect, ERenderTargetLoadAction::EClear);

	TShaderMapRef<FMobileIntegrateDofVS> VertexShader(View.ShaderMap);

	FMobileIntegrateDofVS::FParameters VSShaderParameters;

	VSShaderParameters.View = View.GetShaderParameters();
	VSShaderParameters.BufferSizeAndInvSize = FVector4f(BufferSize.X, BufferSize.Y, 1.0f / BufferSize.X, 1.0f / BufferSize.Y);
	VSShaderParameters.DofBlurSizeAndInvSize = FVector4f(DofBlurSize.X, DofBlurSize.Y, 1.0f / DofBlurSize.X, 1.0f / DofBlurSize.Y);

	TShaderMapRef<FMobileIntegrateDofPS> PixelShader(View.ShaderMap);

	FMobileIntegrateDofPS::FParameters* PSShaderParameters = GraphBuilder.AllocParameters<FMobileIntegrateDofPS::FParameters>();
	PSShaderParameters->RenderTargets[0] = IntegrateDofOutput.GetRenderTargetBinding();
	PSShaderParameters->View = View.GetShaderParameters();

	PSShaderParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PSShaderParameters->SceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSShaderParameters->DofBlurTexture = Inputs.DofBlur.Texture;
	PSShaderParameters->DofBlurSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PSShaderParameters->SunShaftAndDofTexture = Inputs.SunShaftAndDof.Texture;
	PSShaderParameters->SunShaftAndDofSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	const FScreenPassTextureViewport OutputViewport(IntegrateDofOutput);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("IntegrateDof %dx%d (PS)", OutputViewport.Extent.X, OutputViewport.Extent.Y),
		PSShaderParameters,
		ERDGPassFlags::Raster,
		[VertexShader, VSShaderParameters, PixelShader, PSShaderParameters, OutputViewport, &View](FRHICommandList& RHICmdList)
	{
		RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSShaderParameters);
		SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PSShaderParameters);

		DrawRectangle(
			RHICmdList,
			0, 0,
			OutputViewport.Extent.X, OutputViewport.Extent.Y,
			0, 0,
			OutputViewport.Rect.Width(), OutputViewport.Rect.Height(),
			OutputViewport.Extent,
			OutputViewport.Extent,
			VertexShader,
			EDRF_UseTriangleOptimization,
			View.GetStereoPassInstanceFactor());
	});

	return MoveTemp(IntegrateDofOutput);
}

/** Encapsulates the average luminance compute shader. */
class FMobileAverageLuminanceCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileAverageLuminanceCS);
	SHADER_USE_PARAMETER_STRUCT(FMobileAverageLuminanceCS, FGlobalShader);

	// Changing these numbers requires PostProcessMobile.usf to be recompiled.
	static const uint32 ThreadGroupSizeX = 16;
	static const uint32 ThreadGroupSizeY = 8;
	static const uint32 LoopCountX = 2;
	static const uint32 LoopCountY = 2;

	// The number of texels on each axis processed by a single thread group.
	static const FIntPoint TexelsPerThreadGroup;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, SourceSizeAndInvSize)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<half>, InputTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutputUIntBuffer)
		SHADER_PARAMETER(FVector2f, InputTextureSizeInv)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSizeY);
		OutEnvironment.SetDefine(TEXT("LOOP_SIZEX"), LoopCountX);
		OutEnvironment.SetDefine(TEXT("LOOP_SIZEY"), LoopCountY);
		OutEnvironment.SetDefine(TEXT("AVERAGE_LUMINANCE_COMPUTE_SHADER"), 1u);
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
	}
};

const FIntPoint FMobileAverageLuminanceCS::TexelsPerThreadGroup(ThreadGroupSizeX * LoopCountX * 2, ThreadGroupSizeY * LoopCountY * 2); // Multiply 2 because we use bilinear filter, to reduce the sample count

IMPLEMENT_GLOBAL_SHADER(FMobileAverageLuminanceCS, "/Engine/Private/PostProcessMobile.usf", "AverageLuminance_MainCS", SF_Compute);

/** Encapsulates the post processing histogram compute shader. */
class FMobileHistogram : public FGlobalShader
{
public:
	// Changing these numbers requires PostProcessMobile.usf to be recompiled.
	// the maximum total threadgroup memory allocation on A7 and A8 GPU is 16KB-32B, so it has to limit the thread group size on IOS/TVOS platform.
	// this is also needed by some Android devices
	static const uint32 LowThreadGroupSizeX = 8;
	static const uint32 LowThreadGroupSizeY = 4; 

	static const uint32 LowLoopCountX = 2;
	static const uint32 LowLoopCountY = 4;

	static const uint32 ThreadGroupSizeX = 16;
	static const uint32 ThreadGroupSizeY = 8;

	static const uint32 LoopCountX = 2;
	static const uint32 LoopCountY = 2;

	static const uint32 HistogramSize = 64; // HistogramSize must be 64 and ThreadGroupSizeX * ThreadGroupSizeY must be larger than 32

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(FVector4f, SourceSizeAndInvSize)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<half>, InputTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWHistogramBuffer)
		SHADER_PARAMETER(FVector2f, InputTextureSizeInv)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	FMobileHistogram()
	{}

	FMobileHistogram(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		Bindings.BindForLegacyShaderParameters(this, Initializer.PermutationId, Initializer.ParameterMap, *FParameters::FTypeInfo::GetStructMetadata());
	}
};



template< bool LowSharedComputeMemory >
class TMobileHistogramCS : public FMobileHistogram
{
public:
	typedef TMobileHistogramCS< LowSharedComputeMemory > ClassName; // typedef is only so that we can use in DECLARE_SHADER_TYPE macro
	DECLARE_GLOBAL_SHADER(ClassName);

	// The number of texels on each axis processed by a single thread group.
	static const FIntPoint TexelsPerThreadGroup;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), LowSharedComputeMemory ? LowThreadGroupSizeX : ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), LowSharedComputeMemory ? LowThreadGroupSizeY : ThreadGroupSizeY);
		OutEnvironment.SetDefine(TEXT("LOOP_SIZEX"), LowSharedComputeMemory ? LowLoopCountX : LoopCountX);
		OutEnvironment.SetDefine(TEXT("LOOP_SIZEY"), LowSharedComputeMemory ? LowLoopCountY : LoopCountY);
		OutEnvironment.SetDefine(TEXT("HISTOGRAM_COMPUTE_SHADER"), 1u);
		OutEnvironment.SetDefine(TEXT("LOW_SHARED_COMPUTE_MEMORY"), LowSharedComputeMemory);
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
	}

	TMobileHistogramCS()
	{}

	TMobileHistogramCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMobileHistogram(Initializer)
	{}
};

template<> const FIntPoint TMobileHistogramCS<true>::TexelsPerThreadGroup(LowThreadGroupSizeX* LowLoopCountX * 2, LowThreadGroupSizeY* LowLoopCountY * 2); // Multiply 2 because we use bilinear filter, to reduce the sample count
template<> const FIntPoint TMobileHistogramCS<false>::TexelsPerThreadGroup(ThreadGroupSizeX* LoopCountX * 2, ThreadGroupSizeY* LoopCountY * 2); // Multiply 2 because we use bilinear filter, to reduce the sample count

IMPLEMENT_SHADER_TYPE(template<>, TMobileHistogramCS< true >, TEXT("/Engine/Private/PostProcessMobile.usf"), TEXT("Histogram_MainCS"), SF_Compute);
IMPLEMENT_SHADER_TYPE(template<>, TMobileHistogramCS< false >, TEXT("/Engine/Private/PostProcessMobile.usf"), TEXT("Histogram_MainCS"), SF_Compute);

FMobileEyeAdaptationSetupOutputs AddMobileEyeAdaptationSetupPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FEyeAdaptationParameters& EyeAdaptationParameters, const FMobileEyeAdaptationSetupInputs& Inputs)
{
	// clear EyeAdaptationSetupBuffer History
	FRDGBufferRef EyeAdaptationSetupBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), Inputs.bUseBasicEyeAdaptation ? 2 : FMobileHistogram::HistogramSize), TEXT("EyeAdaptationSetupBuffer"));
	FRDGBufferSRVRef EyeAdaptationSetupBufferSRV = GraphBuilder.CreateSRV(EyeAdaptationSetupBuffer, PF_R32_UINT);
	FRDGBufferUAVRef EyeAdaptationSetupBufferUAV = GraphBuilder.CreateUAV(EyeAdaptationSetupBuffer, PF_R32_UINT);
	AddClearUAVPass(GraphBuilder, EyeAdaptationSetupBufferUAV, 0);

	const FIntPoint& ViewSize = Inputs.BloomSetup_EyeAdaptation.ViewRect.Size();
	const FVector2f InputTextureSizeInv(1.0f / Inputs.BloomSetup_EyeAdaptation.Texture->Desc.Extent.X, 1.0f / Inputs.BloomSetup_EyeAdaptation.Texture->Desc.Extent.Y);


	auto DispatchHistogramCS = [&](FRDGBufferUAVRef HistogramUAV, const TCHAR* PassName)
	{
		FMobileHistogram::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileHistogram::FParameters>();
		PassParameters->View = View.GetShaderParameters();
		PassParameters->SourceSizeAndInvSize = FVector4f(ViewSize.X, ViewSize.Y, 1.0f / ViewSize.X, 1.0f / ViewSize.Y);
		PassParameters->InputTexture = Inputs.BloomSetup_EyeAdaptation.Texture;
		PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->RWHistogramBuffer = HistogramUAV;
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->InputTextureSizeInv = InputTextureSizeInv;

		const bool bLowSharedComputeMemory = (GetMaxComputeSharedMemory() < (1 << 15));
		if (bLowSharedComputeMemory)
		{
			TShaderMapRef<TMobileHistogramCS<true>> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder, 
				RDG_EVENT_NAME("%s", PassName), 
				ComputeShader, 
				PassParameters,
				FComputeShaderUtils::GetGroupCount(ViewSize, TMobileHistogramCS<true>::TexelsPerThreadGroup));
		}
		else
		{
			TShaderMapRef<TMobileHistogramCS<false>> ComputeShader(View.ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder, 
				RDG_EVENT_NAME("%s", PassName), 
				ComputeShader, 
				PassParameters,
				FComputeShaderUtils::GetGroupCount(ViewSize, TMobileHistogramCS<false>::TexelsPerThreadGroup));
		}
	};

	FRDGBufferSRVRef VisualizeHDRHistogramSRV = nullptr;

	if (Inputs.bUseBasicEyeAdaptation)
	{
		FMobileAverageLuminanceCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileAverageLuminanceCS::FParameters>();

		PassParameters->SourceSizeAndInvSize = FVector4f(ViewSize.X, ViewSize.Y, 1.0f / ViewSize.X, 1.0f / ViewSize.Y);
		PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->InputTexture = Inputs.BloomSetup_EyeAdaptation.Texture;
		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->OutputUIntBuffer = EyeAdaptationSetupBufferUAV;
		PassParameters->InputTextureSizeInv = InputTextureSizeInv;

		TShaderMapRef<FMobileAverageLuminanceCS> ComputeShader(View.ShaderMap);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("EyeAdaptation_AverageLuminance (CS)"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(ViewSize, FMobileAverageLuminanceCS::TexelsPerThreadGroup));

		if (Inputs.bGenerateHistogramForVisualize)
		{
			FRDGBufferRef VisualizeHistogramBuffer = GraphBuilder.CreateBuffer(
				FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FMobileHistogram::HistogramSize),
				TEXT("VisualizeHDRHistogramBuffer"));
			FRDGBufferUAVRef VisualizeHistogramUAV = GraphBuilder.CreateUAV(VisualizeHistogramBuffer, PF_R32_UINT);
			AddClearUAVPass(GraphBuilder, VisualizeHistogramUAV, 0);

			DispatchHistogramCS(VisualizeHistogramUAV, TEXT("EyeAdaptation_VisualizeHDRHistogram (CS)"));

			VisualizeHDRHistogramSRV = GraphBuilder.CreateSRV(VisualizeHistogramBuffer, PF_R32_UINT);
		}
	}
	else if (Inputs.bUseHistogramEyeAdaptation)
	{
		DispatchHistogramCS(EyeAdaptationSetupBufferUAV, TEXT("EyeAdaptation_Histogram (CS)"));
		VisualizeHDRHistogramSRV = EyeAdaptationSetupBufferSRV;
	}

	FMobileEyeAdaptationSetupOutputs EyeAdaptationSetupOutputs;
	EyeAdaptationSetupOutputs.EyeAdaptationSetupSRV = EyeAdaptationSetupBufferSRV;
	EyeAdaptationSetupOutputs.VisualizeHDRHistogramSRV = VisualizeHDRHistogramSRV;

	return EyeAdaptationSetupOutputs;
}

class FMobileBasicEyeAdaptationCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FMobileBasicEyeAdaptationCS);
	SHADER_USE_PARAMETER_STRUCT(FMobileBasicEyeAdaptationCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, LogLuminanceWeightBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, OutputBuffer)
	END_SHADER_PARAMETER_STRUCT()

	/** Static Shader boilerplate */
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("BASIC_EYEADAPTATION_COMPUTE_SHADER"), 1u);
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileBasicEyeAdaptationCS, "/Engine/Private/PostProcessMobile.usf", "BasicEyeAdaptationCS_Mobile", SF_Compute);

//////////////////////////////////////////////////////////////////////////
//! Histogram Eye Adaptation
//////////////////////////////////////////////////////////////////////////

class FMobileHistogramEyeAdaptationCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMobileHistogramEyeAdaptationCS);
	SHADER_USE_PARAMETER_STRUCT(FMobileHistogramEyeAdaptationCS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HistogramBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, OutputBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("HISTOGRAM_EYEADAPTATION_COMPUTE_SHADER"), 1u);
		OutEnvironment.CompilerFlags.Add(CFLAG_StandardOptimization);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileHistogramEyeAdaptationCS, "/Engine/Private/PostProcessMobile.usf", "HistogramEyeAdaptationCS", SF_Compute);

void AddMobileEyeAdaptationPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FEyeAdaptationParameters& EyeAdaptationParameters, const FMobileEyeAdaptationInputs& Inputs)
{
	// Get the custom 1x1 target used to store exposure value and Toggle the two render targets used to store new and old.
	View.UpdateEyeAdaptationLastExposureFromBuffer();
	View.SwapEyeAdaptationBuffers();

	FRDGBufferRef EyeAdaptationBuffer = Inputs.EyeAdaptationBuffer;
	FRDGBufferSRVRef EyeAdaptationBufferSRV = GraphBuilder.CreateSRV(EyeAdaptationBuffer);

	FRDGBufferRef OutputBuffer = GraphBuilder.RegisterExternalBuffer(View.GetEyeAdaptationBuffer(GraphBuilder), ERDGBufferFlags::MultiFrame);
	FRDGBufferUAVRef OutputBufferUAV = GraphBuilder.CreateUAV(OutputBuffer);

	if (Inputs.bUseBasicEyeAdaptation)
	{
		FMobileBasicEyeAdaptationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileBasicEyeAdaptationCS::FParameters>();

		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->View = View.GetShaderParameters();
		PassParameters->EyeAdaptationBuffer = EyeAdaptationBufferSRV;
		PassParameters->LogLuminanceWeightBuffer = Inputs.EyeAdaptationSetupSRV;
		PassParameters->OutputBuffer = OutputBufferUAV;

		TShaderMapRef<FMobileBasicEyeAdaptationCS> ComputeShader(View.ShaderMap);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("BasicEyeAdaptation (CS)"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(1, 1));
	}
	else
	{
		FMobileHistogramEyeAdaptationCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMobileHistogramEyeAdaptationCS::FParameters>();

		PassParameters->EyeAdaptation = EyeAdaptationParameters;
		PassParameters->EyeAdaptationBuffer = EyeAdaptationBufferSRV;
		PassParameters->HistogramBuffer = Inputs.EyeAdaptationSetupSRV;
		PassParameters->OutputBuffer = OutputBufferUAV;

		TShaderMapRef<FMobileHistogramEyeAdaptationCS> ComputeShader(View.ShaderMap);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("HistogramEyeAdaptation (CS)"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(1, 1));
	}

	View.EnqueueEyeAdaptationExposureBufferReadback(GraphBuilder);
}
