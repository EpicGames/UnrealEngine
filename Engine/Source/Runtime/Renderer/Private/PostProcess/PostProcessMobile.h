// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessMobile.h: Mobile uber post processing.
=============================================================================*/

#pragma once

#include "ScreenPass.h"
#include "PostProcessEyeAdaptation.h"
#include "PostProcess/PostProcessTonemap.h"
#include "PostProcess/PostProcessUpscale.h"

class FViewInfo;

// return Depth of Field Scale if Gaussian DoF mode is active. 0.0f otherwise.
float GetMobileDepthOfFieldScale(const FViewInfo& View);

struct FMobileBloomSetupInputs
{
	FScreenPassTexture SceneColor;
	FScreenPassTexture SunShaftAndDof;
	FIntPoint OutputFixedTextureSize;

	bool bUseBloom = false;
	bool bUseSun = false;
	bool bUseDof = false;
	bool bUseEyeAdaptation = false;
	bool bUseMetalMSAAHDRDecode = false;
};

struct FMobileBloomSetupOutputs
{
	FScreenPassTexture Bloom;
	FScreenPassTexture SunShaftAndDof;
	FScreenPassTexture EyeAdaptation;
};

FMobileBloomSetupOutputs AddMobileBloomSetupPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FEyeAdaptationParameters& EyeAdaptationParameters, const FMobileBloomSetupInputs& Inputs);

struct FMobileDofNearInputs
{
	FScreenPassTexture BloomSetup_SunShaftAndDof;

	bool bUseSun = false;
};

struct FMobileDofNearOutputs
{
	FScreenPassTexture DofNear;
};

FMobileDofNearOutputs AddMobileDofNearPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofNearInputs& Inputs);

struct FMobileDofDownInputs
{
	FScreenPassTexture SceneColor;
	FScreenPassTexture SunShaftAndDof;
	FScreenPassTexture DofNear;

	bool bUseSun = false;
};

struct FMobileDofDownOutputs
{
	FScreenPassTexture DofDown;
};

FMobileDofDownOutputs AddMobileDofDownPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofDownInputs& Inputs);

struct FMobileDofBlurInputs
{
	FScreenPassTexture DofDown;
	FScreenPassTexture DofNear;
};

struct FMobileDofBlurOutputs
{
	FScreenPassTexture DofBlur;
};

FMobileDofBlurOutputs AddMobileDofBlurPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileDofBlurInputs& Inputs);

struct FMobileIntegrateDofInputs
{
	FScreenPassTexture SceneColor;
	FScreenPassTexture DofBlur;
	FScreenPassTexture SunShaftAndDof;
};

FScreenPassTexture AddMobileIntegrateDofPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileIntegrateDofInputs& Inputs);

struct FMobileBloomDownInputs
{
	FIntPoint			BloomDownOutputFixedTextureExtent;
	FIntPoint			BloomDownOutputViewExtent;
	FScreenPassTexture	BloomDownSource;

	float BloomDownScale = 0.66f * 4.0f;
};

FScreenPassTexture AddMobileBloomDownPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileBloomDownInputs& Inputs);

struct FMobileBloomUpInputs
{
	FScreenPassTexture BloomUpSourceA;
	FScreenPassTexture BloomUpSourceB;

	FIntPoint BloomUpOutputFixedTextureExtent;
	FIntPoint BloomUpOutputViewExtent;
	FVector2D ScaleAB;
	FVector4f TintA;
	FVector4f TintB;
};

FScreenPassTexture AddMobileBloomUpPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileBloomUpInputs& Inputs);

struct FMobileSunMaskInputs
{
	FScreenPassTexture SceneColor;
	TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> SceneTextures = nullptr;

	bool bUseSun = false;
	bool bUseDof = false;
	bool bUseDepthTexture = false;
	bool bUseMetalMSAAHDRDecode = false;
};

struct FMobileSunMaskOutputs
{
	FScreenPassTexture SunMask;
	FScreenPassTexture SceneColor;
};

FMobileSunMaskOutputs AddMobileSunMaskPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileSunMaskInputs& Inputs);

struct FMobileSunAlphaInputs
{
	FScreenPassTexture BloomSetup_SunShaftAndDof;
	bool bUseMobileDof;
};
FScreenPassTexture AddMobileSunAlphaPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileSunAlphaInputs& Inputs);

struct FMobileSunBlurInputs
{
	FScreenPassTexture SunAlpha;
};
FScreenPassTexture AddMobileSunBlurPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileSunBlurInputs& Inputs);

struct FMobileSunMergeInputs
{
	FScreenPassTexture SunBlur;
	FScreenPassTexture BloomSetup_Bloom;
	FScreenPassTexture BloomUp;
	bool bUseBloom;
	bool bUseSun;
};

FScreenPassTexture AddMobileSunMergePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMobileSunMergeInputs& Inputs);

struct FMobileEyeAdaptationSetupInputs
{
	FScreenPassTexture BloomSetup_EyeAdaptation;
	bool bUseBasicEyeAdaptation;
	bool bUseHistogramEyeAdaptation;
	// When AEM_Basic is active, run the histogram CS so VisualizeHDR has bucket data.
	bool bGenerateHistogramForVisualize = false;
};

struct FMobileEyeAdaptationSetupOutputs
{
	FRDGBufferSRVRef EyeAdaptationSetupSRV;
	// Histogram buffer for the VisualizeHDR overlay.
	FRDGBufferSRVRef VisualizeHDRHistogramSRV = nullptr;
};

FMobileEyeAdaptationSetupOutputs AddMobileEyeAdaptationSetupPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FEyeAdaptationParameters& EyeAdaptationParameters, const FMobileEyeAdaptationSetupInputs& Inputs);

struct FMobileEyeAdaptationInputs
{
	FRDGBufferSRVRef EyeAdaptationSetupSRV;
	FRDGBufferRef EyeAdaptationBuffer;
	bool bUseBasicEyeAdaptation;
	bool bUseHistogramEyeAdaptation;
};

void AddMobileEyeAdaptationPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FEyeAdaptationParameters& EyeAdaptationParameters, const FMobileEyeAdaptationInputs& Inputs);

/** Pixel shader to decode the input color and  copy pixels from src to dst only for mobile metal platform. */
class FMSAADecodeAndCopyRectPS_Mobile : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMSAADecodeAndCopyRectPS_Mobile);
	SHADER_USE_PARAMETER_STRUCT(FMSAADecodeAndCopyRectPS_Mobile, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

FScreenPassTexture AddEASUPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const ISpatialUpscaler::FInputs& PassInputs);
FScreenPassTexture AddCASPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const ISpatialUpscaler::FInputs& PassInputs);