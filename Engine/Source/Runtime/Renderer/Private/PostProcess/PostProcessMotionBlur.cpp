// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessMotionBlur.h"
#include "StaticBoundShaderState.h"
#include "CanvasTypes.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Engine.h"
#include "SpriteIndexBuffer.h"
#include "PostProcess/PostProcessing.h"
#include "VelocityRendering.h"
#include "UnrealEngine.h"
#include "PixelShaderUtils.h"
#include "SceneViewState.h"

namespace
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	TAutoConsoleVariable<int32> CVarMotionBlurFiltering(
		TEXT("r.MotionBlurFiltering"),
		0,
		TEXT("Useful developer variable\n")
		TEXT("0: off (default, expected by the shader for better quality)\n")
		TEXT("1: on"),
		ECVF_Cheat | ECVF_RenderThreadSafe);
#endif

	TAutoConsoleVariable<float> CVarMotionBlur2ndScale(
		TEXT("r.MotionBlur2ndScale"),
		1.0f,
		TEXT(""),
		ECVF_Cheat | ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarMotionBlurScatter(
		TEXT("r.MotionBlurScatter"),
		0,
		TEXT("Forces scatter based max velocity method (slower)."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarMotionBlurSeparable(
		TEXT("r.MotionBlurSeparable"),
		0,
		TEXT("Adds a second motion blur pass that smooths noise for a higher quality blur."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarMotionBlurDirections(
		TEXT("r.MotionBlur.Directions"),
		1,
		TEXT("Number of bluring direction (default = 1)."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarMotionBlurHalfResInput(
		TEXT("r.MotionBlur.HalfResInput"), 1,
		TEXT("Whether motion blur also blur with a half resolution input."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarMotionBlurHalfResGather(
		TEXT("r.MotionBlur.HalfResGather"), 1,
		TEXT("Whether to do motion blur filter dynamically at half res under heavy motion."),
		ECVF_Scalability | ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarMotionBlurAllowExternalVelocityFlatten(
		TEXT("r.MotionBlur.AllowExternalVelocityFlatten"), 1,
		TEXT("Whether to allow motion blur's velocity flatten into other pass."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarAllowMotionBlurInVR(
		TEXT("vr.AllowMotionBlurInVR"),
		0,
		TEXT("For projects with motion blur enabled, this allows motion blur to be enabled even while in VR."));

	TAutoConsoleVariable<bool> CVarVisualizeMotionBlurEnableDebugInformation(
		TEXT("r.MotionBlur.VisualizeDebugInformation"),
		true,
		TEXT("Enables checkerboard and debug text visualization when Visualize Motion Blur show flag is enabled.\n"),
		ECVF_RenderThreadSafe
	);

	TAutoConsoleVariable<int32> CVarMotionBlurVisualize(
		TEXT("r.MotionBlur.Visualize"),
		0,
		TEXT("0: Default view.\n")
		TEXT("1: HSV view. Encode direction in H, and magnitude in V.\n")
		TEXT("2: Plasma view. Encode motion vector magnitude only.")
	);

	TAutoConsoleVariable<int32> CVarOrthoUsePreviousMotionVelocityFlattenPass(
		TEXT("r.Ortho.UsePreviousMotionVelocityFlattenPass"),
		0,
		TEXT("Enable/Disable the option for Ortho motion blur pass to use the previous flatten textures (e.g. from TSR). ")
		TEXT("Currently causes shimmering on distance planes if disabled"),
		ECVF_RenderThreadSafe);

	FMatrix GetPreviousWorldToClipMatrix(const FViewInfo& View)
	{
		if (View.Family->EngineShowFlags.CameraInterpolation)
		{
			// Instead of finding the world space position of the current pixel, calculate the world space position offset by the camera position, 
			// then translate by the difference between last frame's camera position and this frame's camera position,
			// then apply the rest of the transforms.  This effectively avoids precision issues near the extents of large levels whose world space position is very large.
			FVector ViewOriginDelta = View.ViewMatrices.GetViewOrigin() - View.PrevViewInfo.ViewMatrices.GetViewOrigin();
			return FTranslationMatrix(ViewOriginDelta) * View.PrevViewInfo.ViewMatrices.ComputeViewRotationProjectionMatrix();
		}
		else
		{
			return View.ViewMatrices.ComputeViewRotationProjectionMatrix();
		}
	}

	int32 GetMotionBlurQualityFromCVar()
	{
		int32 MotionBlurQuality;

		static const auto ICVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MotionBlurQuality"));
		MotionBlurQuality = FMath::Clamp(ICVar->GetValueOnRenderThread(), 0, 4);

		return MotionBlurQuality;
	}

	DECLARE_GPU_STAT(MotionBlur);

}

const int32 kMotionBlurFlattenTileSize = FVelocityFlattenTextures::kTileSize;
const int32 kMotionBlurFilterTileSize = 16;
const int32 kMotionBlurComputeTileSizeX = 8;
const int32 kMotionBlurComputeTileSizeY = 8;

bool IsMotionBlurEnabled(const FViewInfo& View)
{
	if (View.GetFeatureLevel() < ERHIFeatureLevel::SM5)
	{
		return false;
	}

	const int32 MotionBlurQuality = GetMotionBlurQualityFromCVar();

	const FSceneViewFamily& ViewFamily = *View.Family;

	return ViewFamily.EngineShowFlags.PostProcessing
		&& ViewFamily.EngineShowFlags.MotionBlur
		&& View.FinalPostProcessSettings.MotionBlurAmount > 0.001f
		&& View.FinalPostProcessSettings.MotionBlurMax > 0.001f
		&& ViewFamily.bRealtimeUpdate
		&& MotionBlurQuality > 0
		&& (CVarAllowMotionBlurInVR->GetInt() != 0 || !GEngine->StereoRenderingDevice.IsValid() || !GEngine->StereoRenderingDevice->IsStereoEnabled());
}

bool IsVisualizeMotionBlurEnabled(const FViewInfo& View)
{
	return View.Family->EngineShowFlags.VisualizeMotionBlur && View.GetFeatureLevel() >= ERHIFeatureLevel::SM5;
}

bool IsMotionBlurScatterRequired(const FViewInfo& View, const FScreenPassTextureViewport& SceneViewport)
{
	const FSceneViewState* ViewState = View.ViewState;
	const float ViewportWidth = SceneViewport.Rect.Width();

	// Normalize percentage value.
	const float VelocityMax = View.FinalPostProcessSettings.MotionBlurMax / 100.0f;

	// Scale by 0.5 due to blur samples going both ways and convert to tiles.
	const float VelocityMaxInTiles = VelocityMax * ViewportWidth * (0.5f / 16.0f);

	// Compute path only supports the immediate neighborhood of tiles.
	const float TileDistanceMaxGathered = 3.0f;

	// Scatter is used when maximum velocity exceeds the distance supported by the gather approach.
	const bool bIsScatterRequiredByVelocityLength = VelocityMaxInTiles > TileDistanceMaxGathered;

	// Cinematic is paused.
	const bool bInPausedCinematic = (ViewState && ViewState->SequencerState == ESS_Paused);

	// Use the scatter approach if requested by cvar or we're in a paused cinematic (higher quality).
	const bool bIsScatterRequiredByUser = CVarMotionBlurScatter.GetValueOnRenderThread() == 1 || bInPausedCinematic;

	return bIsScatterRequiredByUser || bIsScatterRequiredByVelocityLength;
}

FIntPoint GetMotionBlurTileCount(FIntPoint SizeInPixels)
{
	const uint32 TilesX = FMath::DivideAndRoundUp(SizeInPixels.X, kMotionBlurFlattenTileSize);
	const uint32 TilesY = FMath::DivideAndRoundUp(SizeInPixels.Y, kMotionBlurFlattenTileSize);
	return FIntPoint(TilesX, TilesY);
}

bool DoesMotionBlurNeedsHalfResInput()
{
	return CVarMotionBlurHalfResInput.GetValueOnRenderThread() != 0;
}

EMotionBlurQuality GetMotionBlurQuality()
{
	// Quality levels begin at 1. 0 is reserved for 'off'.
	const int32 Quality = FMath::Clamp(GetMotionBlurQualityFromCVar(), 1, static_cast<int32>(EMotionBlurQuality::MAX));

	return static_cast<EMotionBlurQuality>(Quality - 1);
}

EMotionBlurFilter GetMotionBlurFilter()
{
	return CVarMotionBlurSeparable.GetValueOnRenderThread() != 0 ? EMotionBlurFilter::Separable : EMotionBlurFilter::Unified;
}

int32 GetMotionBlurDirections()
{
	return FMath::Clamp(CVarMotionBlurDirections.GetValueOnRenderThread(), 1, 2);
}

// static
bool FVelocityFlattenTextures::AllowExternal(const FViewInfo& View)
{
	const bool bEnableCameraMotionBlur = View.bCameraMotionBlur.Get(true);
	const bool bOverrideCameraMotionBlur = View.ClipToPrevClipOverride.IsSet();

	// Do not use external velocity flatten passes if the camera motion blur needs to be modified.
	if (!bEnableCameraMotionBlur || bOverrideCameraMotionBlur)
	{
		return false;
	}

	return CVarMotionBlurAllowExternalVelocityFlatten.GetValueOnRenderThread() != 0;
}

FRHISamplerState* GetMotionBlurColorSampler()
{
	bool bFiltered = false;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bFiltered = CVarMotionBlurFiltering.GetValueOnRenderThread() != 0;
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	if (bFiltered)
	{
		return TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
	else
	{
		return TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
}

FRHISamplerState* GetMotionBlurVelocitySampler()
{
	return TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
}

FRHISamplerState* GetPostMotionBlurTranslucencySampler(bool bUpscale)
{
	if (bUpscale)
	{
		return TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
	else 
	{
		return TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}
}

FRDGTextureUAVRef CreateDebugUAV(FRDGBuilder& GraphBuilder, const FIntPoint& Extent, const TCHAR* DebugName)
{
#if !UE_BUILD_SHIPPING
	FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
		Extent,
		PF_FloatRGBA,
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV);

	FRDGTextureRef Texture = GraphBuilder.CreateTexture(Desc, DebugName);

	return GraphBuilder.CreateUAV(Texture);
#else
	return nullptr;
#endif
}

BEGIN_SHADER_PARAMETER_STRUCT(FVecocityTileTextureSRVs, )
	// The NON_PIXEL shader parameter will still generate proper SRVCompute value in compute passes
	SHADER_PARAMETER_RDG_TEXTURE_NON_PIXEL_SRV_ARRAY(Texture2D, Textures, [FVelocityFlattenTextures::kMaxVelocityTileTextureCount])
END_SHADER_PARAMETER_STRUCT()

FRDGTextureRef CreateVelocityTileTexture(FRDGBuilder& GraphBuilder, FIntPoint VelocityTileCount, int32 BlurDirections, const TCHAR* DebugName, bool ScatterDilatation = false)
{
	FRDGTextureDesc Desc = FRDGTextureDesc::Create2DArray(
		VelocityTileCount,
		PF_FloatRGBA,
		FClearValueBinding::None,
		GFastVRamConfig.MotionBlur | TexCreate_ShaderResource | (ScatterDilatation ? TexCreate_RenderTargetable : TexCreate_UAV),
		/* ArraySize = */ BlurDirections);

	return GraphBuilder.CreateTexture(Desc, DebugName);
}

FVecocityTileTextureSRVs CreateVelocityTileTextureSRVs(FRDGBuilder& GraphBuilder, FRDGTextureRef VecocityTileArray)
{
	FVecocityTileTextureSRVs SRVs;
	for (int32 i = 0; i < VecocityTileArray->Desc.ArraySize; i++)
	{
		SRVs.Textures[i] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForSlice(VecocityTileArray, i));
	}
	return SRVs;
}

// Base class for a motion blur / velocity shader.
class FMotionBlurShader : public FGlobalShader
{
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	FMotionBlurShader() = default;
	FMotionBlurShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

class FMotionBlurDirections : SHADER_PERMUTATION_SPARSE_INT("DIM_BLUR_DIRECTIONS", 1, 2);

class FMotionBlurVelocityFlattenCS : public FMotionBlurShader
{
public:
	static const uint32 ThreadGroupSize = 16;

	using FPermutationDomain = TShaderPermutationDomain<FMotionBlurDirections>;

	DECLARE_GLOBAL_SHADER(FMotionBlurVelocityFlattenCS);
	SHADER_USE_PARAMETER_STRUCT(FMotionBlurVelocityFlattenCS, FMotionBlurShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Velocity)
		SHADER_PARAMETER_STRUCT_INCLUDE(FVelocityFlattenParameters, VelocityFlattenParameters)
		SHADER_PARAMETER(FMatrix44f, ClipToPrevClipOverride)
		SHADER_PARAMETER(FScreenTransform, ThreadIdToViewportUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToPixelPos)
		SHADER_PARAMETER(int32, bCancelCameraMotion)
		SHADER_PARAMETER(int32, bAddCustomCameraMotion)
		SHADER_PARAMETER(int32, bLensDistortion)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, UndistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VelocityTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutVelocityFlatTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, OutVelocityTileArray)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
};

BEGIN_SHADER_PARAMETER_STRUCT(FMotionBlurVelocityDilateParameters, )
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, VelocityTile)
	SHADER_PARAMETER_STRUCT(FVecocityTileTextureSRVs, VelocityTileTextures)
	SHADER_PARAMETER(float, VelocityScaleForFlattenTiles)
END_SHADER_PARAMETER_STRUCT()

class FMotionBlurVelocityDilateGatherCS : public FMotionBlurShader
{
public:
	static const uint32 ThreadGroupSize = 16;

	DECLARE_GLOBAL_SHADER(FMotionBlurVelocityDilateGatherCS);
	SHADER_USE_PARAMETER_STRUCT(FMotionBlurVelocityDilateGatherCS, FMotionBlurShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FMotionBlurVelocityDilateParameters, Dilate)
		SHADER_PARAMETER_STRUCT(FVecocityTileTextureSRVs, CenterVelocityTileTextures)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, OutVelocityTileArray)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<FMotionBlurDirections>;
};

enum class EMotionBlurVelocityScatterPass : uint32
{
	DrawMin,
	DrawMax,
	MAX
};

class FMotionBlurVelocityDilateScatterVS : public FMotionBlurShader
{
	DECLARE_GLOBAL_SHADER(FMotionBlurVelocityDilateScatterVS);
	SHADER_USE_PARAMETER_STRUCT(FMotionBlurVelocityDilateScatterVS, FMotionBlurShader);
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, ScatterPass)
		SHADER_PARAMETER_STRUCT_INCLUDE(FMotionBlurVelocityDilateParameters, Dilate)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

class FMotionBlurVelocityDilateScatterPS : public FMotionBlurShader
{
	DECLARE_GLOBAL_SHADER(FMotionBlurVelocityDilateScatterPS);
	SHADER_USE_PARAMETER_STRUCT(FMotionBlurVelocityDilateScatterPS, FMotionBlurShader);

	using FParameters = FEmptyShaderParameters;
};

class FMotionBlurFilterTileClassifyCS : public FMotionBlurShader
{
	DECLARE_GLOBAL_SHADER(FMotionBlurFilterTileClassifyCS);
	SHADER_USE_PARAMETER_STRUCT(FMotionBlurFilterTileClassifyCS, FMotionBlurShader);

	using FPermutationDomain = TShaderPermutationDomain<FMotionBlurDirections>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FScreenTransform, FilterTileIdToFlattenTileId)
		SHADER_PARAMETER(FIntPoint, FlattenTileMaxId)
		SHADER_PARAMETER(FIntPoint, FilterTileCount)
		SHADER_PARAMETER(int32, bAllowHalfResGather)
		SHADER_PARAMETER(int32, TileListMaxSize)
		SHADER_PARAMETER(float, HalfResPixelVelocityThreshold)
		SHADER_PARAMETER_STRUCT(FVecocityTileTextureSRVs, VelocityTileTextures)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, TileListsOutput)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, TileListsSizeOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()
};

class FSetupMotionBlurFilterCS : public FMotionBlurShader
{
	DECLARE_GLOBAL_SHADER(FSetupMotionBlurFilterCS);
	SHADER_USE_PARAMETER_STRUCT(FSetupMotionBlurFilterCS, FMotionBlurShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileListsSizeBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, DispatchParametersOutput)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("DIM_BLUR_DIRECTIONS"), 1);
		OutEnvironment.SetDefine(TEXT("SETUP_PASS"), 1);
	}
};

class FMotionBlurFilterCS : public FMotionBlurShader
{
	DECLARE_GLOBAL_SHADER(FMotionBlurFilterCS);
	SHADER_USE_PARAMETER_STRUCT(FMotionBlurFilterCS, FMotionBlurShader);

	enum class ETileClassification
	{
		GatherHalfRes,
		GatherFullRes,
		ScatterAsGatherOneVelocityHalfRes,
		ScatterAsGatherOneVelocityFullRes,
		ScatterAsGatherTwoVelocityFullRes,
		MAX
	};

	static bool IsHalfResTileClassification(ETileClassification TileClassification)
	{
		return TileClassification == ETileClassification::GatherHalfRes || TileClassification == ETileClassification::ScatterAsGatherOneVelocityHalfRes;
	}

	class FAlphaChannelDim : SHADER_PERMUTATION_BOOL("DIM_ALPHA_CHANNEL");
	class FTileClassificationDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_TILE_CLASSIFICATION", ETileClassification);

	using FPermutationDomain = TShaderPermutationDomain<FMotionBlurDirections, FTileClassificationDim, FAlphaChannelDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Velocity)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, VelocityTile)

		SHADER_PARAMETER(FScreenTransform, ColorToVelocity)
		SHADER_PARAMETER(FScreenTransform, SeparateTranslucencyUVToViewportUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToSeparateTranslucencyUV)
		SHADER_PARAMETER(int32, MaxSampleCount)
		SHADER_PARAMETER(int32, OutputMip1)
		SHADER_PARAMETER(int32, OutputMip2)
		SHADER_PARAMETER(int32, bLensDistortion)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, ColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VelocityFlatTexture)
		SHADER_PARAMETER_STRUCT(FVecocityTileTextureSRVs, VelocityTileTextures)

		SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VelocitySampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VelocityTileSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VelocityFlatSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, UndistortingDisplacementSampler)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, TranslucencyTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, TranslucencySampler)
		SHADER_PARAMETER(FScreenTransform, ColorToTranslucency)
		SHADER_PARAMETER(FVector2f, TranslucencyUVMin)
		SHADER_PARAMETER(FVector2f, TranslucencyUVMax)
		SHADER_PARAMETER(FVector2f, TranslucencyExtentInverse)

		SHADER_PARAMETER(int32, TileListOffset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileListsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, TileListsSizeBuffer)
		RDG_BUFFER_ACCESS(DispatchParameters, ERHIAccess::IndirectArgs)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SceneColorOutputMip0)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SceneColorOutputMip1)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SceneColorOutputMip2)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FTileClassificationDim>() == ETileClassification::ScatterAsGatherTwoVelocityFullRes)
		{
			PermutationVector.Set<FMotionBlurDirections>(2);
		}
		else
		{
			PermutationVector.Set<FMotionBlurDirections>(1);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (RemapPermutation(PermutationVector) != PermutationVector)
		{
			return false;
		}
		return FMotionBlurShader::ShouldCompilePermutation(Parameters);
	}

};

class FMotionBlurVisualizePS : public FMotionBlurShader
{
	DECLARE_GLOBAL_SHADER(FMotionBlurVisualizePS);
	SHADER_USE_PARAMETER_STRUCT(FMotionBlurVisualizePS, FMotionBlurShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FMatrix44f, WorldToClipPrev)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  UndistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VelocityTexture)

		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Velocity)

		SHADER_PARAMETER_SAMPLER(SamplerState, VelocitySampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)

		SHADER_PARAMETER(FScreenTransform, SvPositionToScreenPos)
		SHADER_PARAMETER(FScreenTransform, ScreenPosToVelocityUV)
		SHADER_PARAMETER(int, CheckerboardEnabled)
		SHADER_PARAMETER(int, VisualizeMode)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FMotionBlurVelocityFlattenCS,       "/Engine/Private/MotionBlur/MotionBlurVelocityFlatten.usf",    "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FMotionBlurVelocityDilateGatherCS,  "/Engine/Private/MotionBlur/MotionBlurTileGather.usf",         "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FMotionBlurVelocityDilateScatterVS, "/Engine/Private/MotionBlur/MotionBlurTileScatter.usf",        "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FMotionBlurVelocityDilateScatterPS, "/Engine/Private/MotionBlur/MotionBlurTileScatter.usf",        "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FMotionBlurFilterTileClassifyCS,    "/Engine/Private/MotionBlur/MotionBlurFilterTileClassify.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FSetupMotionBlurFilterCS,           "/Engine/Private/MotionBlur/MotionBlurFilterTileClassify.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FMotionBlurFilterCS,                "/Engine/Private/MotionBlur/MotionBlurApply.usf",              "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FMotionBlurVisualizePS,             "/Engine/Private/MotionBlur/MotionBlurVisualize.usf",          "MainPS", SF_Pixel);

TGlobalResource<FSpriteIndexBuffer<8>> GScatterQuadIndexBuffer;

enum class EMotionBlurFilterPass : uint32
{
	Separable0,
	Separable1,
	Unified,
	MAX
};

struct FMotionBlurViewports
{
	FMotionBlurViewports(
		FScreenPassTextureViewport InColorViewport,
		FScreenPassTextureViewport InVelocityViewport)
	{
		Color = InColorViewport;
		Velocity = InVelocityViewport;
		VelocityTile = FScreenPassTextureViewport(
			FIntRect(
				FIntPoint::ZeroValue,
				GetMotionBlurTileCount(Velocity.Rect.Size())));

		ColorParameters = GetScreenPassTextureViewportParameters(Color);
		VelocityParameters = GetScreenPassTextureViewportParameters(Velocity);
		VelocityTileParameters = GetScreenPassTextureViewportParameters(VelocityTile);

		ColorToVelocityTransform = FScreenTransform::ChangeTextureUVCoordinateFromTo(Color, Velocity);
	}

	FScreenPassTextureViewport Color;
	FScreenPassTextureViewport Velocity;
	FScreenPassTextureViewport VelocityTile;

	FScreenPassTextureViewportParameters ColorParameters;
	FScreenPassTextureViewportParameters VelocityParameters;
	FScreenPassTextureViewportParameters VelocityTileParameters;

	FScreenTransform ColorToVelocityTransform;
};

FVelocityFlattenParameters GetVelocityFlattenParameters(const FViewInfo& View)
{
	const FSceneViewState* ViewState = View.ViewState;

	const float SceneViewportSizeX = View.GetSecondaryViewRectSize().X;
	const float SceneViewportSizeY = View.GetSecondaryViewRectSize().Y;
	const float MotionBlurTimeScale = ViewState ? ViewState->MotionBlurTimeScale : 1.0f;

	// Scale by 0.5 due to blur samples going both ways.
	const float VelocityScale = MotionBlurTimeScale * View.FinalPostProcessSettings.MotionBlurAmount * 0.5f;

	// 0:no 1:full screen width, percent conversion
	const float UVVelocityMax = View.FinalPostProcessSettings.MotionBlurMax / 100.0f;

	FVelocityFlattenParameters VelocityFlattenParameters;
	VelocityFlattenParameters.VelocityScale.X = SceneViewportSizeX * 0.5f * VelocityScale;
	VelocityFlattenParameters.VelocityScale.Y = -SceneViewportSizeY * 0.5f * VelocityScale;
	VelocityFlattenParameters.VelocityMax = SceneViewportSizeX * 0.5f * UVVelocityMax;
	return VelocityFlattenParameters;
}

void AddMotionBlurVelocityPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FMotionBlurViewports& Viewports,
	const FMotionBlurInputs& Inputs,
	FRDGTextureRef* VelocityFlatTextureOutput,
	FVecocityTileTextureSRVs* VelocityTileTexturesOutput)
{
	check(VelocityFlatTextureOutput);
	check(VelocityTileTexturesOutput);

	const int32 BlurDirections = GetMotionBlurDirections();

	const FIntPoint VelocityTileCount = Viewports.VelocityTile.Extent;

	// Velocity flatten pass: combines depth / velocity into a single target for sampling efficiency.
	FRDGTextureRef VelocityFlatTexture = nullptr;
	FRDGTextureRef VelocityTileTextureSetup = nullptr;

	bool bViewUsesPreviousFlattenTexture = View.IsPerspectiveProjection() || CVarOrthoUsePreviousMotionVelocityFlattenPass.GetValueOnRenderThread() != 0;
	if (bViewUsesPreviousFlattenTexture && Inputs.VelocityFlattenTextures.IsValid())
	{
		ensure(Inputs.VelocityFlattenTextures.VelocityFlatten.ViewRect == View.ViewRect);
		ensure(Inputs.VelocityFlattenTextures.VelocityTileArray.ViewRect == FIntRect(FIntPoint::ZeroValue, VelocityTileCount));

		VelocityFlatTexture = Inputs.VelocityFlattenTextures.VelocityFlatten.Texture;
		VelocityTileTextureSetup = Inputs.VelocityFlattenTextures.VelocityTileArray.Texture;
	}
	else
	{
		{
			// NOTE: Use scene depth's dimensions because velocity can actually be a 1x1 black texture when there are no moving objects in sight.
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				Inputs.SceneDepth.Texture->Desc.Extent,
				View.IsPerspectiveProjection() ? PF_FloatR11G11B10 : PF_A32B32G32R32F,
				FClearValueBinding::None,
				GFastVRamConfig.VelocityFlat | TexCreate_ShaderResource | TexCreate_UAV);

			VelocityFlatTexture = GraphBuilder.CreateTexture(Desc, TEXT("MotionBlur.VelocityFlatten"));
		}

		VelocityTileTextureSetup = CreateVelocityTileTexture(
			GraphBuilder, VelocityTileCount, BlurDirections, TEXT("MotionBlur.VelocityTile"));

		const bool bEnableCameraMotionBlur = View.bCameraMotionBlur.Get(true);
		const bool bOverrideCameraMotionBlur = View.ClipToPrevClipOverride.IsSet();

		FMotionBlurVelocityFlattenCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMotionBlurVelocityFlattenCS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->VelocityFlattenParameters = GetVelocityFlattenParameters(View);
		PassParameters->bCancelCameraMotion = !bEnableCameraMotionBlur || bOverrideCameraMotionBlur;
		PassParameters->bAddCustomCameraMotion = bOverrideCameraMotionBlur;
		if (PassParameters->bAddCustomCameraMotion)
		{
			PassParameters->ClipToPrevClipOverride = FMatrix44f(View.ClipToPrevClipOverride.GetValue());
		}

		PassParameters->DistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		PassParameters->DistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->bLensDistortion = Inputs.LensDistortionLUT.IsEnabled();
		if (Inputs.LensDistortionLUT.IsEnabled())
		{
			PassParameters->DistortingDisplacementTexture = Inputs.LensDistortionLUT.DistortingDisplacementTexture;
			PassParameters->UndistortingDisplacementTexture = Inputs.LensDistortionLUT.UndistortingDisplacementTexture;
		}

		PassParameters->ThreadIdToViewportUV = FScreenTransform::DispatchThreadIdToViewportUV(Viewports.Velocity.Rect);
		PassParameters->ViewportUVToPixelPos = FScreenTransform::ChangeTextureBasisFromTo(
			FScreenPassTextureViewport(Viewports.Velocity), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TexelPosition);

		PassParameters->Velocity = Viewports.VelocityParameters;
		PassParameters->DepthTexture = Inputs.SceneDepth.Texture;
		PassParameters->VelocityTexture = Inputs.SceneVelocity.Texture;

		PassParameters->OutVelocityFlatTexture = GraphBuilder.CreateUAV(VelocityFlatTexture);
		PassParameters->OutVelocityTileArray = GraphBuilder.CreateUAV(VelocityTileTextureSetup);
		PassParameters->DebugOutput = CreateDebugUAV(GraphBuilder, VelocityFlatTexture->Desc.Extent, TEXT("Debug.MotionBlur.Flatten"));

		FMotionBlurVelocityFlattenCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FMotionBlurDirections>(BlurDirections);

		TShaderMapRef<FMotionBlurVelocityFlattenCS> ComputeShader(View.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Velocity Flatten(CameraMotionBlur%s) %dx%d",
				bEnableCameraMotionBlur ? (bOverrideCameraMotionBlur ? TEXT("Override") : TEXT("On")) : TEXT("Off"),
				Viewports.Velocity.Rect.Width(), Viewports.Velocity.Rect.Height()),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(Viewports.Velocity.Rect.Size(), FMotionBlurVelocityFlattenCS::ThreadGroupSize));
	}

	bool ScatterDilatation = IsMotionBlurScatterRequired(View, Viewports.Color);

	FMotionBlurVelocityDilateParameters VelocityDilateParameters;
	VelocityDilateParameters.VelocityTile = Viewports.VelocityTileParameters;
	VelocityDilateParameters.VelocityTileTextures = CreateVelocityTileTextureSRVs(GraphBuilder, VelocityTileTextureSetup);
	VelocityDilateParameters.VelocityScaleForFlattenTiles = (1.0f / float(kMotionBlurFlattenTileSize)) * (float(Viewports.Velocity.Rect.Width()) / float(Viewports.Color.Rect.Width()));

	FVecocityTileTextureSRVs VelocityTileTextures;
	if (ScatterDilatation)
	{
		FRDGTextureRef VelocityTileArrayTexture = CreateVelocityTileTexture(
			GraphBuilder, VelocityTileCount, /* BlurDirections = */ 1, TEXT("MotionBlur.ScatteredVelocityTile"), /* ScatterDilatation = */ true);

		VelocityTileTextures.Textures[0] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForSlice(VelocityTileArrayTexture, 0));

		FRDGTextureRef VelocityTileDepthTexture =
			GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(
					VelocityTileCount,
					PF_ShadowDepth,
					FClearValueBinding::DepthOne,
					TexCreate_DepthStencilTargetable),
				TEXT("MotionBlur.DilatedVelocityDepth"));

		FMotionBlurVelocityDilateScatterVS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMotionBlurVelocityDilateScatterVS::FParameters>();
		PassParameters->Dilate = VelocityDilateParameters;

		PassParameters->RenderTargets.DepthStencil =
			FDepthStencilBinding(
				VelocityTileDepthTexture,
				ERenderTargetLoadAction::EClear,
				ERenderTargetLoadAction::ENoAction,
				FExclusiveDepthStencil::DepthWrite_StencilNop);

		PassParameters->RenderTargets[0] =
			FRenderTargetBinding(
				VelocityTileArrayTexture,
				ERenderTargetLoadAction::ENoAction,
				/* MipIndex = */ 0,
				/* ArraySlice = */ 0);

		TShaderMapRef<FMotionBlurVelocityDilateScatterVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FMotionBlurVelocityDilateScatterPS> PixelShader(View.ShaderMap);
		
		ClearUnusedGraphResources(VertexShader, PassParameters);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("VelocityTileScatter %dx%d", VelocityTileCount.X, VelocityTileCount.Y),
			PassParameters,
			ERDGPassFlags::Raster,
			[VertexShader, PixelShader, VelocityTileCount, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FRHIVertexShader* RHIVertexShader = VertexShader.GetVertexShader();

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = RHIVertexShader;
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			// Max >= Min so no need to clear on second pass
			RHICmdList.SetViewport(0, 0, 0.0f, VelocityTileCount.X, VelocityTileCount.Y, 1.0f);

			// Min, Max
			for (uint32 ScatterPassIndex = 0; ScatterPassIndex < static_cast<uint32>(EMotionBlurVelocityScatterPass::MAX); ScatterPassIndex++)
			{
				const EMotionBlurVelocityScatterPass ScatterPass = static_cast<EMotionBlurVelocityScatterPass>(ScatterPassIndex);

				if (ScatterPass == EMotionBlurVelocityScatterPass::DrawMin)
				{
					GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_Less>::GetRHI();
				}
				else
				{
					GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_BA>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_Greater>::GetRHI();
				}

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				PassParameters->ScatterPass = ScatterPassIndex;

				SetShaderParameters(RHICmdList, VertexShader, RHIVertexShader, *PassParameters);

				// Needs to be the same on shader side (faster on NVIDIA and AMD)
				const int32 QuadsPerInstance = 8;

				RHICmdList.SetStreamSource(0, nullptr, 0);
				RHICmdList.DrawIndexedPrimitive(GScatterQuadIndexBuffer.IndexBufferRHI, 0, 0, 32, 0, 2 * QuadsPerInstance, FMath::DivideAndRoundUp(VelocityTileCount.X * VelocityTileCount.Y, QuadsPerInstance));
			}
		});
	}

	// ScatterGather the dilatation
	if (!ScatterDilatation || BlurDirections > 1)
	{
		FRDGTextureRef DilatedTileTexture = CreateVelocityTileTexture(
			GraphBuilder, VelocityTileCount, BlurDirections, TEXT("MotionBlur.GatheredVelocityTile"));

		FMotionBlurVelocityDilateGatherCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMotionBlurVelocityDilateGatherCS::FParameters>();
		PassParameters->Dilate = VelocityDilateParameters;
		if (ScatterDilatation)
		{
			check(BlurDirections > 1);
			// Feed the scattered min max to make sure ReducePolarVelocityRange() is aware the highest velocity might be coming from tiles further away
			PassParameters->CenterVelocityTileTextures.Textures[0] = VelocityTileTextures.Textures[0];
			PassParameters->CenterVelocityTileTextures.Textures[1] = VelocityDilateParameters.VelocityTileTextures.Textures[1];
		}
		else
		{
			PassParameters->CenterVelocityTileTextures = VelocityDilateParameters.VelocityTileTextures;
		}
		PassParameters->OutVelocityTileArray = GraphBuilder.CreateUAV(DilatedTileTexture);

		FMotionBlurVelocityDilateGatherCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FMotionBlurDirections>(BlurDirections);

		TShaderMapRef<FMotionBlurVelocityDilateGatherCS> ComputeShader(View.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("VelocityTileGatherCS %dx%d", VelocityTileCount.X, VelocityTileCount.Y),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(VelocityTileCount, FMotionBlurVelocityDilateGatherCS::ThreadGroupSize));

		if (!ScatterDilatation)
		{
			VelocityTileTextures = CreateVelocityTileTextureSRVs(GraphBuilder, DilatedTileTexture);
		}
		else if (BlurDirections > 1)
		{
			VelocityTileTextures.Textures[1] = CreateVelocityTileTextureSRVs(GraphBuilder, DilatedTileTexture).Textures[1];
		}
	}

	*VelocityFlatTextureOutput = VelocityFlatTexture;
	*VelocityTileTexturesOutput = VelocityTileTextures;
}

FMotionBlurOutputs AddMotionBlurFilterPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FMotionBlurInputs& Inputs,
	const FMotionBlurViewports& Viewports,
	FRDGTextureSRVRef ColorTexture,
	FRDGTextureRef VelocityFlatTexture,
	FVecocityTileTextureSRVs VelocityTileTextures,
	FRDGTextureRef PostMotionBlurTranslucency,
	const FIntPoint& PostMotionBlurTranslucencySize,
	EMotionBlurFilterPass MotionBlurFilterPass,
	EMotionBlurQuality MotionBlurQuality)
{
	check(ColorTexture);
	check(VelocityFlatTexture);
	check(MotionBlurFilterPass != EMotionBlurFilterPass::MAX);
	check(MotionBlurQuality != EMotionBlurQuality::MAX);

	const int32 BlurDirections = GetMotionBlurDirections();

	const float MotionBlur2ndScale = CVarMotionBlur2ndScale.GetValueOnRenderThread();

	const bool bUseCompute = View.bUseComputePasses;

	const float BlurScaleLUT[static_cast<uint32>(EMotionBlurFilterPass::MAX)][static_cast<uint32>(EMotionBlurQuality::MAX)] =
	{
		// Separable0
		{
			1.0f - 0.5f / 4.0f,
			1.0f - 0.5f / 6.0f,
			1.0f - 0.5f / 8.0f,
			1.0f - 0.5f / 16.0f
		},

		// Separable1
		{
			1.0f / 4.0f  * MotionBlur2ndScale,
			1.0f / 6.0f  * MotionBlur2ndScale,
			1.0f / 8.0f  * MotionBlur2ndScale,
			1.0f / 16.0f * MotionBlur2ndScale
		},

		// Unified
		{
			1.0f,
			1.0f,
			1.0f,
			1.0f
		}
	};

	static int32 kMaxSampleCountPerQuality[] = { 4, 8, 12, 16 };
	static_assert(UE_ARRAY_COUNT(kMaxSampleCountPerQuality) == int32(EMotionBlurQuality::MAX), "Fix me!");

	const int32 MaxSampleCount = kMaxSampleCountPerQuality[int32(MotionBlurQuality)];

	const float BlurScale = BlurScaleLUT[static_cast<uint32>(MotionBlurFilterPass)][static_cast<uint32>(MotionBlurQuality)];

	const FIntVector FilterTileCount = FComputeShaderUtils::GetGroupCount(Viewports.Color.Rect.Size(), kMotionBlurFilterTileSize);

	const bool bAlphaChannel = IsPostProcessingWithAlphaChannelSupported();
	
	int32 TileListMaxSize = FilterTileCount.X * FilterTileCount.Y;

	// Tile classify the filtering
	FRDGBufferRef TileListsBuffer;
	FRDGBufferRef TileListsSizeBuffer;
	{

		TileListsBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), TileListMaxSize * int32(FMotionBlurFilterCS::ETileClassification::MAX)),
			TEXT("MotionBlur.TileOffsets"));

		TileListsSizeBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), int32(FMotionBlurFilterCS::ETileClassification::MAX)),
			TEXT("MotionBlur.TileCounters"));

		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(TileListsSizeBuffer), 0);

		FMotionBlurFilterTileClassifyCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMotionBlurFilterTileClassifyCS::FParameters>();
		PassParameters->FilterTileIdToFlattenTileId = (FScreenTransform::Identity + 0.5f) * (float(kMotionBlurFilterTileSize * Viewports.Velocity.Rect.Width()) / float(kMotionBlurFlattenTileSize * Viewports.Color.Rect.Width()));
		PassParameters->FlattenTileMaxId = Viewports.VelocityTile.Rect.Size() - 1;
		PassParameters->FilterTileCount = FIntPoint(FilterTileCount.X, FilterTileCount.Y);
		PassParameters->TileListMaxSize = TileListMaxSize;
		PassParameters->HalfResPixelVelocityThreshold = FMath::Square(float(MaxSampleCount));
		PassParameters->bAllowHalfResGather = CVarMotionBlurHalfResGather.GetValueOnRenderThread() ? 1 : 0;
		PassParameters->VelocityTileTextures = VelocityTileTextures;
		PassParameters->TileListsOutput = GraphBuilder.CreateUAV(TileListsBuffer);
		PassParameters->TileListsSizeOutput = GraphBuilder.CreateUAV(TileListsSizeBuffer);
		PassParameters->DebugOutput = CreateDebugUAV(GraphBuilder, PassParameters->FilterTileCount, TEXT("Debug.MotionBlur.FilterTileClassify"));

		FMotionBlurFilterTileClassifyCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FMotionBlurDirections>(BlurDirections);

		TShaderMapRef<FMotionBlurFilterTileClassifyCS> ComputeShader(View.ShaderMap, PermutationVector);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("MotionBlur FilterTileClassify %dx%d", FilterTileCount.X, FilterTileCount.Y),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(FilterTileCount, kMotionBlurFilterTileSize));
	}

	// Setup the filter's dispatch parameters.
	FRDGBufferRef DispatchParameters;
	{
		DispatchParameters = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(int32(FMotionBlurFilterCS::ETileClassification::MAX)),
			TEXT("MotionBlur.FilterDispatchParameters"));

		FSetupMotionBlurFilterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSetupMotionBlurFilterCS::FParameters>();
		PassParameters->TileListsSizeBuffer = GraphBuilder.CreateSRV(TileListsSizeBuffer);
		PassParameters->DispatchParametersOutput = GraphBuilder.CreateUAV(DispatchParameters);

		TShaderMapRef<FSetupMotionBlurFilterCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("MotionBlur SetupFilter"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1));
	}


	// Apply the filter.
	FMotionBlurOutputs Output;
	{
		static const TCHAR* kTileClassificationNames[] = {
			TEXT("GatherHalfRes"),
			TEXT("GatherFullRes"),
			TEXT("ScatterAsGatherOneVelocityHalfRes"),
			TEXT("ScatterAsGatherOneVelocityFullRes"),
			TEXT("ScatterAsGatherTwoVelocityFullRes"),
		};
		static_assert(UE_ARRAY_COUNT(kTileClassificationNames) == int32(FMotionBlurFilterCS::ETileClassification::MAX), "Fix me!");

		FMotionBlurFilterCS::FParameters OriginalPassParameters;
		OriginalPassParameters.Color = Viewports.ColorParameters;
		OriginalPassParameters.Velocity = Viewports.VelocityParameters;
		OriginalPassParameters.VelocityTile = Viewports.VelocityTileParameters;
		OriginalPassParameters.ColorToVelocity = Viewports.ColorToVelocityTransform;
		OriginalPassParameters.MaxSampleCount = MaxSampleCount;
		OriginalPassParameters.OutputMip1 = Inputs.bOutputHalfRes ? 1 : 0;
		OriginalPassParameters.OutputMip2 = Inputs.bOutputQuarterRes ? 1 : 0;

		OriginalPassParameters.ColorTexture = ColorTexture;
		OriginalPassParameters.VelocityFlatTexture = VelocityFlatTexture;
		OriginalPassParameters.VelocityTileTextures = VelocityTileTextures;
		OriginalPassParameters.ColorSampler = GetMotionBlurColorSampler();
		OriginalPassParameters.VelocitySampler = GetMotionBlurVelocitySampler();
		OriginalPassParameters.VelocityTileSampler = GetMotionBlurVelocitySampler();
		OriginalPassParameters.VelocityFlatSampler = GetMotionBlurVelocitySampler();

		OriginalPassParameters.TileListsBuffer = GraphBuilder.CreateSRV(TileListsBuffer);
		OriginalPassParameters.TileListsSizeBuffer = GraphBuilder.CreateSRV(TileListsSizeBuffer);
		OriginalPassParameters.DispatchParameters = DispatchParameters;

		OriginalPassParameters.UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		OriginalPassParameters.UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		OriginalPassParameters.bLensDistortion = 0;

		if (PostMotionBlurTranslucency != nullptr)
		{
			// TODO: broken with split screen
			const bool bScaleTranslucency = Viewports.Color.Rect.Size() != PostMotionBlurTranslucencySize;
			const FVector2f OutputSize(Viewports.Color.Rect.Size());
			const FVector2f OutputSizeInv = FVector2f(1.0f, 1.0f) / OutputSize;
			const FVector2f PostMotionBlurTranslucencyExtent(PostMotionBlurTranslucency->Desc.Extent);
			const FVector2f PostMotionBlurTranslucencyExtentInv = FVector2f(1.0f, 1.0f) / PostMotionBlurTranslucencyExtent;
		
			OriginalPassParameters.TranslucencyTexture = PostMotionBlurTranslucency;
			OriginalPassParameters.TranslucencySampler = GetPostMotionBlurTranslucencySampler(bScaleTranslucency);
			FScreenPassTextureViewport TranslucencyViewport = FScreenPassTextureViewport(PostMotionBlurTranslucency->Desc.Extent, FIntRect(FIntPoint::ZeroValue, PostMotionBlurTranslucencySize));
			OriginalPassParameters.ColorToTranslucency = FScreenTransform::ChangeTextureUVCoordinateFromTo(Viewports.Color, TranslucencyViewport);
			OriginalPassParameters.SeparateTranslucencyUVToViewportUV = FScreenTransform::ChangeTextureBasisFromTo(
				TranslucencyViewport, FScreenTransform::ETextureBasis::TextureUV, FScreenTransform::ETextureBasis::ViewportUV);
			OriginalPassParameters.ViewportUVToSeparateTranslucencyUV = FScreenTransform::ChangeTextureBasisFromTo(
				TranslucencyViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
			OriginalPassParameters.TranslucencyUVMin = FVector2f(0.0f, 0.0f);
			OriginalPassParameters.TranslucencyUVMax = (FVector2f(PostMotionBlurTranslucencySize) - FVector2f(0.5f, 0.5f)) * PostMotionBlurTranslucencyExtentInv;
			OriginalPassParameters.TranslucencyExtentInverse = PostMotionBlurTranslucencyExtentInv;

			OriginalPassParameters.bLensDistortion = Inputs.LensDistortionLUT.IsEnabled();
			if (Inputs.LensDistortionLUT.IsEnabled())
			{
				OriginalPassParameters.UndistortingDisplacementTexture = Inputs.LensDistortionLUT.UndistortingDisplacementTexture;
			}
		}
		else
		{
			OriginalPassParameters.TranslucencyTexture = GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder);
			OriginalPassParameters.TranslucencySampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			OriginalPassParameters.ColorToTranslucency = FScreenTransform::Identity;
			OriginalPassParameters.SeparateTranslucencyUVToViewportUV = FScreenTransform::Identity;
			OriginalPassParameters.ViewportUVToSeparateTranslucencyUV = FScreenTransform::Identity;
			OriginalPassParameters.TranslucencyUVMin = FVector2f(0.0f, 0.0f);
			OriginalPassParameters.TranslucencyUVMax = FVector2f(0.0f, 0.0f);
			OriginalPassParameters.TranslucencyExtentInverse = FVector2f(0.0f, 0.0f);
		}

		{
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				ColorTexture->Desc.Texture->Desc.Extent,
				bAlphaChannel ? PF_FloatRGBA : PF_FloatRGB,
				FClearValueBinding::None,
				TexCreate_UAV | TexCreate_ShaderResource | TexCreate_RenderTargetable | GFastVRamConfig.MotionBlur);

			FRDGTextureRef FullResTexture = GraphBuilder.CreateTexture(Desc, TEXT("MotionBlur.SceneColor"));

			Output.FullRes.TextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(FullResTexture));
			Output.FullRes.ViewRect = Viewports.Color.Rect;
			OriginalPassParameters.SceneColorOutputMip0 = GraphBuilder.CreateUAV(FullResTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		}

		if (OriginalPassParameters.OutputMip1)
		{
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				Output.FullRes.TextureSRV->Desc.Texture->Desc.Extent / 2,
				Output.FullRes.TextureSRV->Desc.Texture->Desc.Format,
				FClearValueBinding::None,
				TexCreate_UAV | TexCreate_ShaderResource);

			Output.HalfRes.Texture = GraphBuilder.CreateTexture(Desc, TEXT("MotionBlur.SceneColor.HalfRes"));
			Output.HalfRes.ViewRect.Min = Viewports.Color.Rect.Min / 2;
			Output.HalfRes.ViewRect.Max = Output.HalfRes.ViewRect.Min + FIntPoint::DivideAndRoundUp(Viewports.Color.Rect.Size(), 2);
			OriginalPassParameters.SceneColorOutputMip1 = GraphBuilder.CreateUAV(Output.HalfRes.Texture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		}
		else
		{
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				FIntPoint(1, 1),
				PF_FloatR11G11B10,
				FClearValueBinding::None,
				/* InFlags = */ TexCreate_ShaderResource | TexCreate_UAV);

			FRDGTextureRef DummyTexture = GraphBuilder.CreateTexture(Desc, TEXT("MotionBlur.DummyOutput"));
			OriginalPassParameters.SceneColorOutputMip1 = GraphBuilder.CreateUAV(DummyTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		}

		if (OriginalPassParameters.OutputMip2)
		{
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				Output.FullRes.TextureSRV->Desc.Texture->Desc.Extent / 4,
				Output.FullRes.TextureSRV->Desc.Texture->Desc.Format,
				FClearValueBinding::None,
				TexCreate_UAV | TexCreate_ShaderResource);

			Output.QuarterRes.Texture = GraphBuilder.CreateTexture(Desc, TEXT("MotionBlur.SceneColor.QuarterRes"));
			Output.QuarterRes.ViewRect.Min = Viewports.Color.Rect.Min / 4;
			Output.QuarterRes.ViewRect.Max = Output.QuarterRes.ViewRect.Min + FIntPoint::DivideAndRoundUp(Viewports.Color.Rect.Size(), 4);
			OriginalPassParameters.SceneColorOutputMip2 = GraphBuilder.CreateUAV(Output.QuarterRes.Texture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		}
		else
		{
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				FIntPoint(1, 1),
				PF_FloatR11G11B10,
				FClearValueBinding::None,
				/* InFlags = */ TexCreate_ShaderResource | TexCreate_UAV);

			FRDGTextureRef DummyTexture = GraphBuilder.CreateTexture(Desc, TEXT("MotionBlur.DummyOutput"));
			OriginalPassParameters.SceneColorOutputMip2 = GraphBuilder.CreateUAV(DummyTexture, ERDGUnorderedAccessViewFlags::SkipBarrier);
		}

		OriginalPassParameters.DebugOutput = CreateDebugUAV(GraphBuilder, Output.FullRes.TextureSRV->Desc.Texture->Desc.Extent, TEXT("Debug.MotionBlur.Filter"));

		RDG_EVENT_SCOPE(GraphBuilder, "MotionBlur FullResFilter(BlurDirections=%d MaxSamples=%d%s%s%s%s) %dx%d",
			BlurDirections, OriginalPassParameters.MaxSampleCount,
			bAlphaChannel ? TEXT(" AlphaChannel") : TEXT(""),
			PostMotionBlurTranslucency ? TEXT(" ComposeTranslucency") : TEXT(""),
			OriginalPassParameters.OutputMip1 ? TEXT(" OutputMip1") : TEXT(""),
			OriginalPassParameters.OutputMip2 ? TEXT(" OutputMip2") : TEXT(""),
			Viewports.Color.Rect.Width(), Viewports.Color.Rect.Height());

		for (int32 TileClassifcation = 0; TileClassifcation < int32(FMotionBlurFilterCS::ETileClassification::MAX); TileClassifcation++)
		{
			FMotionBlurFilterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMotionBlurFilterCS::FParameters>();
			*PassParameters = OriginalPassParameters;
			PassParameters->TileListOffset = TileListMaxSize * TileClassifcation;

			FMotionBlurFilterCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FMotionBlurFilterCS::FAlphaChannelDim>(bAlphaChannel);
			PermutationVector.Set<FMotionBlurFilterCS::FTileClassificationDim>(FMotionBlurFilterCS::ETileClassification(TileClassifcation));
			PermutationVector.Set<FMotionBlurDirections>(BlurDirections);
			PermutationVector = FMotionBlurFilterCS::RemapPermutation(PermutationVector);

			if (PermutationVector.Get<FMotionBlurDirections>() > BlurDirections)
			{
				continue;
			}

			TShaderMapRef<FMotionBlurFilterCS> ComputeShader(View.ShaderMap, PermutationVector);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("MotionBlur Filter(%s)", kTileClassificationNames[int32(PermutationVector.Get<FMotionBlurFilterCS::FTileClassificationDim>())]),
				ComputeShader,
				PassParameters,
				DispatchParameters,
				/* IndirectArgsOffset = */ sizeof(FRHIDispatchIndirectParameters) * TileClassifcation);
		}
	}

	return Output;
}

FScreenPassTextureSlice AddVisualizeMotionBlurPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMotionBlurInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneDepth.IsValid());
	check(Inputs.SceneVelocity.IsValid());
	checkf(Inputs.SceneDepth.ViewRect == Inputs.SceneVelocity.ViewRect, TEXT("The implementation requires that depth and velocity have the same viewport."));

	FScreenPassRenderTarget Output;
	bool bOverrideMissingSRFlag = false;
	if (Inputs.OverrideOutput.IsValid())
	{
		/**
		 * If the override texture is used, check whether the output texture supports SRVs before assigning it, as this causes a crash in
		 * FScreenPassTextureSlice::CreateFromScreenPassTexture at the end of this function if not.
		 */
		if(static_cast<uint64>(Inputs.OverrideOutput.Texture->Desc.Flags) & static_cast<uint64>(ETextureCreateFlags::ShaderResource))
		{
			Output = Inputs.OverrideOutput;
		}
		else
		{
			bOverrideMissingSRFlag = true;
		}
	}

	if (!Output.IsValid())
	{
		FIntPoint OutputExtent;
		EPixelFormat OutputFormat;
		FIntRect OutputRect;
		if (bOverrideMissingSRFlag)
		{
			//If the Override texture does not support SRVs, we need to create one that does so we can call/return via FScreenPassTextureSlice::CreateFromScreenPassTexture
			OutputExtent = Inputs.OverrideOutput.Texture->Desc.Extent;
			OutputFormat = Inputs.OverrideOutput.Texture->Desc.Format;
			OutputRect = Inputs.OverrideOutput.ViewRect;
		}
		else
		{
			OutputExtent = Inputs.SceneColor.TextureSRV->Desc.Texture->Desc.Extent;
			OutputFormat = Inputs.SceneColor.TextureSRV->Desc.Texture->Desc.Format;
			OutputRect = Inputs.SceneColor.ViewRect;
		}

		FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
			OutputExtent,
			OutputFormat,
			FClearValueBinding::None,
			ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable);
			
		Output = FScreenPassRenderTarget(GraphBuilder.CreateTexture(OutputDesc, TEXT("MotionBlur.Visualize")), OutputRect, View.GetOverwriteLoadAction());
	}

	// NOTE: Scene depth is used as the velocity viewport because velocity can actually be a 1x1 black texture.
	const FMotionBlurViewports Viewports(FScreenPassTextureViewport(Inputs.SceneColor), FScreenPassTextureViewport(Inputs.SceneDepth));

	const bool bVisualizeDebugInfo = CVarVisualizeMotionBlurEnableDebugInformation.GetValueOnRenderThread();
	const int32 VisualizeMode = CVarMotionBlurVisualize.GetValueOnRenderThread();

	{
		FMotionBlurVisualizePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FMotionBlurVisualizePS::FParameters>();
		PassParameters->WorldToClipPrev = FMatrix44f(GetPreviousWorldToClipMatrix(View));		// LWC_TODO: Precision loss
		PassParameters->View = View.ViewUniformBuffer;

		PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		if (Inputs.LensDistortionLUT.IsEnabled())
		{
			PassParameters->UndistortingDisplacementTexture = Inputs.LensDistortionLUT.UndistortingDisplacementTexture;
		}

		PassParameters->DepthTexture = Inputs.SceneDepth.Texture;
		PassParameters->VelocityTexture = Inputs.SceneVelocity.Texture;
		PassParameters->Velocity = Viewports.VelocityParameters;
		PassParameters->VelocitySampler = GetMotionBlurVelocitySampler();
		PassParameters->DepthSampler = GetMotionBlurVelocitySampler();
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		PassParameters->SvPositionToScreenPos = (
			FScreenTransform::SvPositionToViewportUV(Output.ViewRect) *
			FScreenTransform::ViewportUVToScreenPos);
		PassParameters->ScreenPosToVelocityUV = FScreenTransform::ChangeTextureBasisFromTo(
			Inputs.SceneDepth, FScreenTransform::ETextureBasis::ScreenPosition, FScreenTransform::ETextureBasis::TextureUV);

		PassParameters->CheckerboardEnabled = bVisualizeDebugInfo;
		PassParameters->VisualizeMode = VisualizeMode;

		TShaderMapRef<FMotionBlurVisualizePS> PixelShader(View.ShaderMap);
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("VisualizeMotionVectors %dx%d", Output.ViewRect.Width(), Output.ViewRect.Height()),
			PixelShader,
			PassParameters,
			Output.ViewRect);
	}

	Output.LoadAction = ERenderTargetLoadAction::ELoad;

	if (bVisualizeDebugInfo)
	{
		AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("VisualizeMotionBlurOverlay"), View, Output,
			[&View](FCanvas& Canvas)
			{
				float X = 20;
				float Y = 38;
				const float YStep = 14;
				const float ColumnWidth = 200;

				FString Line;

				Line = FString::Printf(TEXT("Visualize MotionBlur"));
				Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 0));

				static const auto MotionBlurDebugVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MotionBlurDebug"));
				const int32 MotionBlurDebug = MotionBlurDebugVar ? MotionBlurDebugVar->GetValueOnRenderThread() : 0;

				Line = FString::Printf(TEXT("%d, %d"), View.Family->FrameNumber, MotionBlurDebug);
				Canvas.DrawShadowedString(X, Y += YStep, TEXT("FrameNo, r.MotionBlurDebug:"), GetStatsFont(), FLinearColor(1, 1, 0));
				Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 1, 0));

				static const auto VelocityTestVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VelocityTest"));
				const int32 VelocityTest = VelocityTestVar ? VelocityTestVar->GetValueOnRenderThread() : 0;

				Line = FString::Printf(TEXT("%d, %d, %d"), View.Family->bWorldIsPaused, VelocityTest, FVelocityRendering::IsParallelVelocity(View.GetShaderPlatform()));
				Canvas.DrawShadowedString(X, Y += YStep, TEXT("Paused, r.VelocityTest, Parallel:"), GetStatsFont(), FLinearColor(1, 1, 0));
				Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 1, 0));

				const FSceneViewState* SceneViewState = (const FSceneViewState*)View.State;

				Line = FString::Printf(TEXT("View=%.4x PrevView=%.4x"),
					View.ViewMatrices.GetWorldToView().ComputeHash() & 0xffff,
					View.PrevViewInfo.ViewMatrices.GetWorldToView().ComputeHash() & 0xffff);
				Canvas.DrawShadowedString(X, Y += YStep, TEXT("ViewMatrix:"), GetStatsFont(), FLinearColor(1, 1, 0));
				Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 1, 0));
			});
	}

	if (bOverrideMissingSRFlag)
	{
		const FIntPoint ViewSize = Output.ViewRect.Size();
		const FIntPoint ViewMin = Output.ViewRect.Min;

		/**
		 * The OverrideOutput texture did not support SRVs, so we created one that did, but we still need to resolve the output image back
		 * to the OverrideOutput texture too.
		 */
		FRHICopyTextureInfo CopyInfo = {
			.Size = FIntVector(ViewSize.X, ViewSize.Y, 1),
			.SourcePosition = FIntVector(ViewMin.X, ViewMin.Y, 0),
			.DestPosition = FIntVector(ViewMin.X, ViewMin.Y, 0)
		};
		AddCopyTexturePass(GraphBuilder, Output.Texture, Inputs.OverrideOutput.Texture, CopyInfo);
	}

	return FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, Output);
}

FMotionBlurOutputs AddMotionBlurPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMotionBlurInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneDepth.IsValid());
	check(Inputs.SceneVelocity.IsValid());
	checkf(Inputs.SceneDepth.ViewRect == Inputs.SceneVelocity.ViewRect, TEXT("The motion blur depth and velocity must have the same viewport."));
	checkf(!Inputs.OverrideOutput.IsValid(), TEXT("The motion blur override output support is unimplemented."));

	// NOTE: Use SceneDepth as the velocity viewport because SceneVelocity can actually be a 1x1 black texture when there are no moving objects in sight.
	const FMotionBlurViewports Viewports(FScreenPassTextureViewport(Inputs.SceneColor), FScreenPassTextureViewport(Inputs.SceneDepth));

	RDG_EVENT_SCOPE_STAT(GraphBuilder, MotionBlur, "MotionBlur");

	FRDGTextureRef VelocityFlatTexture = nullptr;
	FVecocityTileTextureSRVs VelocityTileTextures;
	AddMotionBlurVelocityPass(
		GraphBuilder,
		View,
		Viewports,
		Inputs,
		&VelocityFlatTexture,
		&VelocityTileTextures);

	FMotionBlurOutputs Output;
	if (Inputs.Filter == EMotionBlurFilter::Separable)
	{
		FRDGTextureSRVRef MotionBlurFilterTextureSRV = AddMotionBlurFilterPass(
			GraphBuilder,
			View,
			Inputs,
			Viewports,
			Inputs.SceneColor.TextureSRV,
			VelocityFlatTexture,
			VelocityTileTextures,
			nullptr,
			FIntPoint(0, 0),
			EMotionBlurFilterPass::Separable0,
			Inputs.Quality).FullRes.TextureSRV;

		Output = AddMotionBlurFilterPass(
			GraphBuilder,
			View,
			Inputs,
			Viewports,
			MotionBlurFilterTextureSRV,
			VelocityFlatTexture,
			VelocityTileTextures,
			Inputs.PostMotionBlurTranslucency.ColorTexture.Resolve,
			Inputs.PostMotionBlurTranslucency.ViewRect.Size(),
			EMotionBlurFilterPass::Separable1,
			Inputs.Quality);
	}
	else
	{
		Output = AddMotionBlurFilterPass(
			GraphBuilder,
			View,
			Inputs,
			Viewports,
			Inputs.SceneColor.TextureSRV,
			VelocityFlatTexture,
			VelocityTileTextures,
			Inputs.PostMotionBlurTranslucency.ColorTexture.Resolve,
			Inputs.PostMotionBlurTranslucency.ViewRect.Size(),
			EMotionBlurFilterPass::Unified,
			Inputs.Quality);
	}

	return Output;
}

