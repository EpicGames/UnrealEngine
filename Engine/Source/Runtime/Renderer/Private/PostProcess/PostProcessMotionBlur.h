// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessMotionBlur.h: Post process MotionBlur implementation.
=============================================================================*/

#pragma once

#include "ScreenPass.h"
#include "TranslucentRendering.h"
#include "PostProcess/LensDistortion.h"
#include "TranslucentPassResource.h"

// Returns whether motion blur is enabled for the requested view.
bool IsMotionBlurEnabled(const FViewInfo& View);

// Returns whether visualization of motion blur is enabled for the requested view.
bool IsVisualizeMotionBlurEnabled(const FViewInfo& View);

// The quality level of the motion blur pass.
enum class EMotionBlurQuality : uint32
{
	Low,
	Medium,
	High,
	VeryHigh,
	MAX
};

enum class EMotionBlurFilter : uint32
{
	Unified,
	Separable
};

// Returns whether motion blur also want half res input the scene color's MIP1.
bool DoesMotionBlurNeedsHalfResInput();

// Returns the global setting for motion blur quality.
EMotionBlurQuality GetMotionBlurQuality();

// Returns the global setting for motion blur filter.
EMotionBlurFilter GetMotionBlurFilter();

// Number of simultaneous direction motion blur can blur per tiles.
int32 GetMotionBlurDirections();

// Shader parameters for MotionBlurVelocityFlatten.ush when another pass can also generate the FVelocityFlattenTextures
BEGIN_SHADER_PARAMETER_STRUCT(FVelocityFlattenParameters, )
	SHADER_PARAMETER(FVector2f, VelocityScale)
	SHADER_PARAMETER(float, VelocityMax)
END_SHADER_PARAMETER_STRUCT()

FVelocityFlattenParameters GetVelocityFlattenParameters(const FViewInfo& View);

// Textures generated by the velocity flattening pass.
struct FVelocityFlattenTextures
{
	static constexpr int32 kMaxVelocityTileTextureCount = 2;
	static constexpr int32 kTileSize = 16;

	FScreenPassTexture VelocityFlatten;
	FScreenPassTexture VelocityTileArray;

	bool IsValid() const
	{
		return VelocityFlatten.IsValid() && VelocityTileArray.IsValid();
	}

	// returns whether FVelocityFlattenTextures can be generated in external system.
	static bool AllowExternal(const FViewInfo& View);
};

struct FMotionBlurInputs
{
	bool bOutputHalfRes = false;
	bool bOutputQuarterRes = false;

	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The input scene color and view rect.
	FScreenPassTextureSlice SceneColor;

	// [Required] The input scene depth and view rect.
	FScreenPassTexture SceneDepth;

	// [Required] The input scene velocity and view rect.
	FScreenPassTexture SceneVelocity;

	// [Optional] The separate translucency buffer to be composited after motion blur
	FTranslucencyPassResources PostMotionBlurTranslucency;

	// [Required] Quality to use when processing motion blur.
	EMotionBlurQuality Quality = EMotionBlurQuality::VeryHigh;

	// [Required] Filter to use when processing motion blur.
	EMotionBlurFilter Filter = EMotionBlurFilter::Separable;

	// [Optional] the velocity flatten is can be generated by a earlier pass.
	FVelocityFlattenTextures VelocityFlattenTextures;

	// [Optional] Lens distortion applied on the scene color.
	FLensDistortionLUT LensDistortionLUT;
};

struct FMotionBlurOutputs
{
	FScreenPassTextureSlice FullRes;
	FScreenPassTexture HalfRes;
	FScreenPassTexture QuarterRes;
};

FMotionBlurOutputs AddMotionBlurPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMotionBlurInputs& Inputs);
FScreenPassTextureSlice AddVisualizeMotionBlurPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FMotionBlurInputs& Inputs);