// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PostProcess/PostProcessSelectionOutline.h"
#include "CoreMinimal.h"
#include "ClearQuad.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EditorPrimitivesRendering.h"
#include "Engine/Engine.h"
#include "GlobalShader.h"
#include "HairStrands/HairStrandsData.h"
#include "HAL/IConsoleManager.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "Logging/LogMacros.h"
#include "Math/Color.h"
#include "MeshDrawCommands.h"
#include "MeshPassProcessor.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Nanite/NaniteCullRaster.h"
#include "Nanite/NaniteEditor.h"
#include "PixelShaderUtils.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/PostProcessCompositePrimitivesCommon.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTexturesConfig.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "Stats/Stats.h"
#include "StereoRendering.h"
#include "SystemTextures.h"
#include "UnrealClient.h"

namespace
{
static int32 GEnableSelectionOutlineColors = 0;
static FAutoConsoleVariableRef CVarEnableSelectionOutlineColors(
	TEXT("r.Viewport.EnableSelectionOutlineColors"),
	GEnableSelectionOutlineColors,
	TEXT("Enable Selection Outline Colors")
	);
	
class FSelectionOutlinePS : public FCompositePrimitiveShaderBase
{
public:
	DECLARE_GLOBAL_SHADER(FSelectionOutlinePS);
	SHADER_USE_PARAMETER_STRUCT(FSelectionOutlinePS, FCompositePrimitiveShaderBase);
	class FSelectionOutlineHDRDim : SHADER_PERMUTATION_BOOL("SELECTION_OUTLINE_HDR");
	using FPermutationDomain = TShaderPermutationDomain< FCompositePrimitiveShaderBase::FPermutationDomain, FSelectionOutlineHDRDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Depth)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  UndistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  ColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  DepthSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, EditorPrimitivesDepth)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, EditorPrimitivesStencil)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, OverlayLookupTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  OverlayLookupSampler)

		SHADER_PARAMETER(FScreenTransform, PassSvPositionToViewportUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToColorUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToDepthUV)
		SHADER_PARAMETER_ARRAY(FVector4f, OutlineColors, [8])
		SHADER_PARAMETER(int, OutlineColorIndexBits)
		SHADER_PARAMETER(float, SelectionHighlightIntensity)
		SHADER_PARAMETER(float, BSPSelectionIntensity)
		SHADER_PARAMETER(float, UILuminanceAndIsSCRGB)
		SHADER_PARAMETER(float, SecondaryViewportOffset)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool SupportHDR(const EShaderPlatform Platform)
	{
		return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5);
	}

	static bool ShouldCompilePermutation(const FPermutationDomain& PermutationVector, const EShaderPlatform Platform)
	{
		FCompositePrimitiveShaderBase::FPermutationDomain EditorPrimitiveShaderDomain = PermutationVector.Get<FCompositePrimitiveShaderBase::FPermutationDomain>();
		bool bIsHDR = PermutationVector.Get<FSelectionOutlineHDRDim>();
		if (bIsHDR && !SupportHDR(Platform))
		{
			return false;
		}

		if (!IsPCPlatform(Platform))
		{
			return false;
		}

		return FCompositePrimitiveShaderBase::ShouldCompilePermutation(EditorPrimitiveShaderDomain, Platform);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		return ShouldCompilePermutation(PermutationVector, Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSelectionOutlinePS, "/Engine/Private/PostProcessSelectionOutline.usf", "MainPS", SF_Pixel);
} //! namespace

BEGIN_SHADER_PARAMETER_STRUCT(FSelectionOutlinePassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSelectionOutlineClearBorderParameters, )
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

FScreenPassTexture AddSelectionOutlinePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSceneUniformBuffer &SceneUniformBuffer,
	const FSelectionOutlineInputs& Inputs,
	const Nanite::FRasterResults* NaniteRasterResults,
	FRDGTextureRef& DepthStencilTexture)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneDepth.IsValid());

	const bool bNaniteEnabled = NaniteRasterResults != nullptr;

	RDG_EVENT_SCOPE_STAT(GraphBuilder, EditorPrimitives, "EditorSelectionOutlines");

	const uint32 NumSamples = View.GetSceneTexturesConfig().NumSamples;

	// Patch uniform buffers with updated state for rendering the outline mesh draw commands.
	const bool bIsInstancedStereoView = View.bIsInstancedStereoEnabled && IStereoRendering::IsStereoEyePass(View.StereoPass);
	const FViewInfo* EditorView = CreateCompositePrimitiveView(View, bIsInstancedStereoView ? View.ViewRect : Inputs.SceneColor.ViewRect, NumSamples);
	FRDGTextureMSAA OverlayColorTexture;

	// Generate custom depth / stencil for outline shapes.
	{
		FScreenPassTextureViewport DepthStencilViewport(Inputs.SceneColor);
		RDG_EVENT_SCOPE(GraphBuilder, "OutlineDepth %dx%d", DepthStencilViewport.Rect.Width(), DepthStencilViewport.Rect.Height());

		FScene* Scene = View.Family->Scene->GetRenderScene();

		{
			FRDGTextureDesc OverlayColorDesc = Inputs.SceneColor.Texture->Desc;
			OverlayColorDesc.Reset();
			OverlayColorDesc.Format = PF_R8G8B8A8;
			OverlayColorDesc.ClearValue = FClearValueBinding::Transparent;
			OverlayColorDesc.Flags = TexCreate_RenderTargetable | TexCreate_ShaderResource;
			OverlayColorDesc.NumSamples = NumSamples;

			OverlayColorTexture = CreateTextureMSAA(GraphBuilder, OverlayColorDesc, TEXT("Editor.ColorOverlayMSAA"), TEXT("Editor.ColorOverlay"));
		}
		
		if (View.ShouldRenderView() || !DepthStencilTexture)
		{

			// If instanced stereo is enabled, we can only do traditional draws from the primary view, so we need to generate the secondary view's depth data at the same time.
			// To hold that secondary view depth data, we patch the depth stencil texture and its viewport to be double-wide.
			if (bIsInstancedStereoView /* Only primary ISR views have View.ShouldRenderView() == true */)
			{
				DepthStencilViewport = FScreenPassTextureViewport(View.ViewRectWithSecondaryViews);
			}

			{
				FRDGTextureDesc DepthStencilDesc = Inputs.SceneColor.Texture->Desc;
				DepthStencilDesc.Reset();
				DepthStencilDesc.Format = PF_DepthStencil;
				// This is a reversed Z depth surface, so 0.0f is the far plane.
				DepthStencilDesc.ClearValue = FClearValueBinding(0.f, 0);
				DepthStencilDesc.Flags = TexCreate_DepthStencilTargetable | TexCreate_ShaderResource;
				DepthStencilDesc.NumSamples = NumSamples;
				DepthStencilDesc.Extent = DepthStencilViewport.Extent;
				DepthStencilDesc.AliasableFormats.Add(PF_X24_G8);

				DepthStencilTexture = GraphBuilder.CreateTexture(DepthStencilDesc, TEXT("Editor.SelectionOutline"));
			}

			{
				auto* PassParameters = GraphBuilder.AllocParameters<FSelectionOutlinePassParameters>();

				PassParameters->RenderTargets[0] = FRenderTargetBinding(OverlayColorTexture.Target, OverlayColorTexture.Resolve, ERenderTargetLoadAction::EClear);
				PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
					DepthStencilTexture,
					ERenderTargetLoadAction::EClear,
					ERenderTargetLoadAction::EClear,
					FExclusiveDepthStencil::DepthWrite_StencilWrite);

				const float ViewportScale = (float)Inputs.SceneColor.ViewRect.Width() / (float)View.ViewRect.Width(); // Outline selection can be before or after the primary upscale

				auto* Pass = const_cast<FViewInfo&>(View).ParallelMeshDrawCommandPasses[EMeshPass::EditorSelection];

				if (Pass)
				{
					PassParameters->View = EditorView->GetShaderParameters();
					PassParameters->SceneTextures = Inputs.SceneTextures;

					Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);
				}

				// Pass always added due to Clear load action.
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("EditorSelectionDepth"),
					PassParameters,
					ERDGPassFlags::Raster,
					[&View, Pass, DepthStencilViewport, PassParameters, ViewportScale](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					if (Pass)
					{
						if (View.bIsInstancedStereoEnabled && View.StereoPass == EStereoscopicPass::eSSP_PRIMARY)
						{
							FSceneRenderer::SetStereoViewport(RHICmdList, View, ViewportScale);
						}
						else
						{
							RHICmdList.SetViewport(DepthStencilViewport.Rect.Min.X, DepthStencilViewport.Rect.Min.Y, 0.0f, DepthStencilViewport.Rect.Max.X, DepthStencilViewport.Rect.Max.Y, 1.0f);
						}

						// Run selection pass on static elements
						Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
					}
				});
			}
		}

		// If the depth stencil is double-wide, retarget the view rect to the appropriate side for remaining steps.
		if (bIsInstancedStereoView)
		{
			// Use View.ViewRect rather than SceneColor.ViewRect, since SceneColor may be single-wide (thus always 0,0 locked).
			DepthStencilViewport = FScreenPassTextureViewport(DepthStencilTexture->Desc.Extent, View.ViewRect);
		}

		// Render Nanite mesh outlines after regular mesh outline, but before borders
		if (bNaniteEnabled)
		{
			// Update editor view to true target view rect
			Nanite::DrawEditorSelection(GraphBuilder, DepthStencilTexture, OverlayColorTexture.Target, *Scene, View, *EditorView, SceneUniformBuffer, NaniteRasterResults);
		}

		// Render HairStrands outlines
		if (HairStrands::HasViewHairStrandsData(View))
		{
			HairStrands::DrawEditorSelection(GraphBuilder, View, DepthStencilViewport.Rect, DepthStencilTexture);
		}

		// Clear the borders to get an outline around the objects if it's partly outside of the screen
		// Unnecessary for stereo (VR editor) views since the edge of the screen is not visible
		if (!IStereoRendering::IsStereoEyePass(View.StereoPass))
		{
			auto* PassParameters = GraphBuilder.AllocParameters<FSelectionOutlineClearBorderParameters>();

			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
				DepthStencilTexture,
				ERenderTargetLoadAction::ELoad,
				ERenderTargetLoadAction::ELoad,
				FExclusiveDepthStencil::DepthWrite_StencilWrite);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("DrawOutlineBorder"),
				PassParameters,
				ERDGPassFlags::Raster,
				[DepthStencilViewport](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					RHICmdList.SetViewport(DepthStencilViewport.Rect.Min.X, DepthStencilViewport.Rect.Min.Y, 0.0f, DepthStencilViewport.Rect.Max.X, DepthStencilViewport.Rect.Max.Y, 1.0f);

					FIntRect InnerRect = DepthStencilViewport.Rect;

					// 1 as we have an outline that is that thick
					InnerRect.InflateRect(-1);

					// top
					RHICmdList.SetScissorRect(true, DepthStencilViewport.Rect.Min.X, DepthStencilViewport.Rect.Min.Y, DepthStencilViewport.Rect.Max.X, InnerRect.Min.Y);
					DrawClearQuad(RHICmdList, false, FLinearColor(), true, 0.f, true, 0, DepthStencilViewport.Extent, FIntRect());
					// bottom
					RHICmdList.SetScissorRect(true, DepthStencilViewport.Rect.Min.X, InnerRect.Max.Y, DepthStencilViewport.Rect.Max.X, DepthStencilViewport.Rect.Max.Y);
					DrawClearQuad(RHICmdList, false, FLinearColor(), true, 0.f, true, 0, DepthStencilViewport.Extent, FIntRect());
					// left
					RHICmdList.SetScissorRect(true, DepthStencilViewport.Rect.Min.X, DepthStencilViewport.Rect.Min.Y, InnerRect.Min.X, DepthStencilViewport.Rect.Max.Y);
					DrawClearQuad(RHICmdList, false, FLinearColor(), true, 0.f, true, 0, DepthStencilViewport.Extent, FIntRect());
					// right
					RHICmdList.SetScissorRect(true, InnerRect.Max.X, DepthStencilViewport.Rect.Min.Y, DepthStencilViewport.Rect.Max.X, DepthStencilViewport.Rect.Max.Y);
					DrawClearQuad(RHICmdList, false, FLinearColor(), true, 0.f, true, 0, DepthStencilViewport.Extent, FIntRect());

					RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
				}
			);
		}
	}

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("SelectionOutlineColor"));
	}

	// Render selection outlines.
	{
		const FScreenPassTextureViewport OutputViewport(Output);
		const FScreenPassTextureViewport ColorViewport(Inputs.SceneColor);
		const FScreenPassTextureViewport DepthViewport(Inputs.SceneDepth);

		FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		// In side-by-side stereo, the SceneColor texture can come in as single- or double-wide depending on what post-processing passes are enabled, and it can also be pre- or post-primary upscale
		// If it's double-wide, the left eye will always be on the left side of the texture, but the right eye may be on the left (0,0 aligned) or the right depending on preceding passes
		// If an instanced secondary view is on the left, it's depth stencil data will still be on the right, so we need to apply an offset to the outline shader when using instanced stencil data
		const bool bIsInstancedSecondaryView = bIsInstancedStereoView && View.StereoPass == EStereoscopicPass::eSSP_SECONDARY;
		const float SecondaryViewOffset = (bIsInstancedSecondaryView && Inputs.SceneColor.ViewRect.Min.X == 0) ? View.ViewRect.Min.X : 0.0f;

		FSelectionOutlinePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSelectionOutlinePS::FParameters>();
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->Color = GetScreenPassTextureViewportParameters(ColorViewport);
		PassParameters->Depth = GetScreenPassTextureViewportParameters(DepthViewport);

		PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		if (Inputs.LensDistortionLUT.IsEnabled())
		{
			PassParameters->UndistortingDisplacementTexture = Inputs.LensDistortionLUT.UndistortingDisplacementTexture;
		}

		PassParameters->ColorTexture = Inputs.SceneColor.Texture;
		PassParameters->ColorSampler = PointClampSampler;

		PassParameters->DepthTexture = Inputs.SceneDepth.Texture;
		PassParameters->DepthSampler = PointClampSampler;

		PassParameters->EditorPrimitivesDepth = DepthStencilTexture;
		PassParameters->EditorPrimitivesStencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(DepthStencilTexture, PF_X24_G8));
		
		PassParameters->OverlayLookupTexture = OverlayColorTexture.Resolve;
		PassParameters->OverlayLookupSampler = PointClampSampler;
		
		PassParameters->OutlineColors[0] = View.SelectionOutlineColor;
		PassParameters->OutlineColors[1] = View.SubduedSelectionOutlineColor;
		
		PassParameters->PassSvPositionToViewportUV = FScreenTransform::SvPositionToViewportUV(OutputViewport.Rect);
		PassParameters->ViewportUVToColorUV = FScreenTransform::ChangeTextureBasisFromTo(
			ColorViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);
		PassParameters->ViewportUVToDepthUV = FScreenTransform::ChangeTextureBasisFromTo(
			DepthViewport, FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV);

		if (GEnableSelectionOutlineColors)
		{
			for (int OutlineColorIndex = 2; OutlineColorIndex < PassParameters->OutlineColors.Num(); ++OutlineColorIndex)
			{
				PassParameters->OutlineColors[OutlineColorIndex] = View.AdditionalSelectionOutlineColors[OutlineColorIndex - 2];
			}
		}
		else
		{
			for (int OutlineColorIndex = 2; OutlineColorIndex < PassParameters->OutlineColors.Num(); ++OutlineColorIndex)
			{
				PassParameters->OutlineColors[OutlineColorIndex] = View.SelectionOutlineColor;
			}
		}
		PassParameters->OutlineColorIndexBits = 3;
		PassParameters->SelectionHighlightIntensity = GEngine->SelectionHighlightIntensity;
		PassParameters->BSPSelectionIntensity = GEngine->BSPSelectionHighlightIntensity;
		PassParameters->SecondaryViewportOffset = SecondaryViewOffset;

		EDisplayOutputFormat DisplayOutputFormat = View.Family->RenderTarget->GetDisplayOutputFormat();
		bool bIsHDR = false;
		bool bIsSCRGB = false;
		if (FSelectionOutlinePS::SupportHDR(GetFeatureLevelShaderPlatform(GMaxRHIFeatureLevel)))
		{
			switch (DisplayOutputFormat)
			{
			case EDisplayOutputFormat::HDR_ACES_1000nit_ScRGB:
			case EDisplayOutputFormat::HDR_ACES_2000nit_ScRGB:
				bIsHDR = true;
				bIsSCRGB = true;
				break;
			case EDisplayOutputFormat::HDR_ACES_1000nit_ST2084:
			case EDisplayOutputFormat::HDR_ACES_2000nit_ST2084:
				bIsHDR = true;
				break;
			case EDisplayOutputFormat::SDR_sRGB:
			case EDisplayOutputFormat::SDR_Rec709:
			case EDisplayOutputFormat::SDR_ExplicitGammaMapping:
			case EDisplayOutputFormat::HDR_LinearEXR:
			case EDisplayOutputFormat::HDR_LinearNoToneCurve:
			case EDisplayOutputFormat::HDR_LinearWithToneCurve:
				break;
			default:
				checkNoEntry();
				break;
			}
		}

		float UILuminance = 203.f;
		static const auto* const CVarHDRUILuminanceMode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.UI.Luminance.Mode"));
		if (CVarHDRUILuminanceMode && (CVarHDRUILuminanceMode->GetValueOnRenderThread() == 0))
		{
			UILuminance = View.Family->RenderTarget->GetHDRPaperWhiteInNits();
		}
		else
		{
			static const auto* const CVarHDRUILuminance = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.HDR.UI.Luminance"));
			if (CVarHDRUILuminance)
			{
				UILuminance = CVarHDRUILuminance->GetValueOnRenderThread();
			}
		}
		PassParameters->UILuminanceAndIsSCRGB = bIsSCRGB ? UILuminance : -UILuminance;

		FSelectionOutlinePS::FPermutationDomain PermutationVector;
		FCompositePrimitiveShaderBase::FPermutationDomain PermutationVectorBase;
		PermutationVectorBase.Set<FSelectionOutlinePS::FSampleCountDimension>(NumSamples);
		PermutationVector.Set<FCompositePrimitiveShaderBase::FPermutationDomain>(PermutationVectorBase);
		PermutationVector.Set<FSelectionOutlinePS::FSelectionOutlineHDRDim>(bIsHDR);

		TShaderMapRef<FSelectionOutlinePS> PixelShader(View.ShaderMap, PermutationVector);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("OutlineColor %dx%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
			PixelShader,
			PassParameters,
			Output.ViewRect);
	}

	return MoveTemp(Output);
}

#endif
