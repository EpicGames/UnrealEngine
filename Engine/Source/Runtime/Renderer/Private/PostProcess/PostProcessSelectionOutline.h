// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ScreenPass.h"
#include "OverridePassSequence.h"
#include "PostProcess/LensDistortion.h"

class FSceneUniformBuffer;
namespace Nanite
{
	struct FRasterResults;
}

struct FSelectionOutlineInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The scene color to composite with selection outlines.
	FScreenPassTexture SceneColor;

	// [Required] The scene depth to composite with selection outlines.
	FScreenPassTexture SceneDepth;

	// [Required] Used when scene textures are required by the material.
	FSceneTextureShaderParameters SceneTextures;

	// [Optional] Lens distortion applied on the scene color.
	FLensDistortionLUT LensDistortionLUT;
};

FScreenPassTexture AddSelectionOutlinePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSceneUniformBuffer &SceneUniformBuffer,
	const FSelectionOutlineInputs& Inputs,
	const Nanite::FRasterResults* NaniteRasterResults,
	FRDGTextureRef& DepthStencilTexture);

#endif
