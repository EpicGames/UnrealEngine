// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessStreamingAccuracyLegend.h"
#include "UnrealEngine.h"
#include "DebugViewModeRendering.h"
#include "SceneRendering.h"

namespace
{
void DrawDesc(FCanvas& Canvas, float PosX, float PosY, const FText& Text)
{
	Canvas.DrawShadowedText(PosX + 18, PosY, Text, GetStatsFont(), FLinearColor(0.7f, 0.7f, 0.7f), FLinearColor::Black);
}

void DrawBox(FCanvas& Canvas, float PosX, float PosY, const FLinearColor& Color, const FText& Text)
{
	Canvas.DrawTile(PosX, PosY, 16, 16, 0, 0, 1, 1, FLinearColor::Black);
	Canvas.DrawTile(PosX + 1, PosY + 1, 14, 14, 0, 0, 1, 1, Color);
	Canvas.DrawShadowedText(PosX + 18, PosY, Text, GetStatsFont(), FLinearColor(0.7f, 0.7f, 0.7f), FLinearColor::Black);
}

void DrawCheckerBoard(FCanvas& Canvas, float PosX, float PosY, const FLinearColor& Color0, const FLinearColor& Color1, const FText& Text)
{
	Canvas.DrawTile(PosX, PosY, 16, 16, 0, 0, 1, 1, FLinearColor::Black);
	Canvas.DrawTile(PosX + 1, PosY + 1, 14, 14, 0, 0, 1, 1, Color0);
	Canvas.DrawTile(PosX + 1, PosY + 1, 7, 7, 0, 0, 1, 1, Color1);
	Canvas.DrawTile(PosX + 8, PosY + 8, 7, 7, 0, 0, 1, 1, Color1);
	Canvas.DrawShadowedText(PosX + 18, PosY, Text, GetStatsFont(), FLinearColor(0.7f, 0.7f, 0.7f), FLinearColor::Black);
}
} //! namespace

#define LOCTEXT_NAMESPACE "TextureStreamingBuild"

FScreenPassTexture AddStreamingAccuracyLegendPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FStreamingAccuracyLegendInputs& Inputs)
{
	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (Output.IsValid())
	{
		AddDrawTexturePass(GraphBuilder, View, Inputs.SceneColor, Output);
	}
	else
	{
		Output = FScreenPassRenderTarget(Inputs.SceneColor, ERenderTargetLoadAction::ELoad);
	}

	const TArrayView<const FLinearColor> Colors = Inputs.Colors;
	const FIntRect OutputViewRect = Output.ViewRect;

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("StreamingAccuracyLegend"), View, Output, [&View, Colors, OutputViewRect](FCanvas& Canvas)
	{
		const FSceneViewFamily& ViewFamily = *View.Family;

		if (ViewFamily.GetDebugViewShaderMode() == DVSM_RequiredTextureResolution)
		{
			DrawDesc(Canvas, OutputViewRect.Min.X + 115, OutputViewRect.Max.Y - 75, LOCTEXT("DescRequiredTextureResolution", "Shows the ratio between the currently streamed texture resolution and the resolution wanted by the GPU."));
		}
		else if (ViewFamily.GetDebugViewShaderMode() == DVSM_MaterialTextureScaleAccuracy)
		{
			DrawDesc(Canvas, OutputViewRect.Min.X + 115, OutputViewRect.Max.Y - 75, LOCTEXT("DescMaterialTextureScaleAccuracy", "Shows under/over texture streaming caused by the material texture scales applied when sampling."));
		}
		else if (ViewFamily.GetDebugViewShaderMode() == DVSM_MeshUVDensityAccuracy)
		{
			DrawDesc(Canvas, OutputViewRect.Min.X + 115, OutputViewRect.Max.Y - 75, LOCTEXT("DescUVDensityAccuracy", "Shows under/over texture streaming caused by the mesh UV densities."));
		}
		else if (ViewFamily.GetDebugViewShaderMode() == DVSM_PrimitiveDistanceAccuracy)
		{
			DrawDesc(Canvas, OutputViewRect.Min.X + 115, OutputViewRect.Max.Y - 100, LOCTEXT("DescPrimitiveDistanceAccuracy", "Shows under/over texture streaming caused by the difference between the streamer calculated"));
			DrawDesc(Canvas, OutputViewRect.Min.X + 165, OutputViewRect.Max.Y - 75, LOCTEXT("DescPrimitiveDistanceAccuracy2", "distance-to-mesh via bounding box versus the actual per-pixel depth value."));
		}

		DrawBox(Canvas, OutputViewRect.Min.X + 115, OutputViewRect.Max.Y - 25, Colors[0], LOCTEXT("2XUnder", "2X+ Under"));
		DrawBox(Canvas, OutputViewRect.Min.X + 215, OutputViewRect.Max.Y - 25, Colors[1], LOCTEXT("1XUnder", "1X Under"));
		DrawBox(Canvas, OutputViewRect.Min.X + 315, OutputViewRect.Max.Y - 25, Colors[2], LOCTEXT("Good", "Good"));

		DrawBox(Canvas, OutputViewRect.Min.X + 415, OutputViewRect.Max.Y - 25, Colors[3], LOCTEXT("1xOver", "1X Over"));
		DrawBox(Canvas, OutputViewRect.Min.X + 515, OutputViewRect.Max.Y - 25, Colors[4], LOCTEXT("2XOver", "2X+ Over"));

		const FLinearColor UndefColor(UndefinedStreamingAccuracyIntensity, UndefinedStreamingAccuracyIntensity, UndefinedStreamingAccuracyIntensity, 1.f);
		DrawBox(Canvas, OutputViewRect.Min.X + 615, OutputViewRect.Max.Y - 25, UndefColor, LOCTEXT("Undefined", "Undefined"));

		if (ViewFamily.GetDebugViewShaderMode() == DVSM_MaterialTextureScaleAccuracy || ViewFamily.GetDebugViewShaderMode() == DVSM_MeshUVDensityAccuracy)
		{
			DrawCheckerBoard(Canvas, OutputViewRect.Min.X + 715, OutputViewRect.Max.Y - 25, Colors[0], Colors[4], LOCTEXT("WorstUnderAndOver", "Worst Under / Worst Over"));
		}
	});

	return MoveTemp(Output);
}

#undef LOCTEXT_NAMESPACE
