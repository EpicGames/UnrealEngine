// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

struct FStreamingAccuracyLegendInputs
{
	FScreenPassRenderTarget OverrideOutput;
	FScreenPassTexture SceneColor;
	TArrayView<const FLinearColor> Colors;
};

FScreenPassTexture AddStreamingAccuracyLegendPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FStreamingAccuracyLegendInputs& Inputs);