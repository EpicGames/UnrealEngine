// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessceSetupce.cpp: Screenspace subsurface scattering implementation.
		Indirect dispatch implementation high level description
	   1. Initialize counters
	   2. Setup pass: record the tiles that need to draw Burley and Separable in two different buffer.
	   3. Indirect dispatch Burley.
	   4. Indirect dispatch Separable.
	   5. Recombine.
=============================================================================*/

#include "PostProcess/PostProcessSubsurface.h"
#include "PostProcess/SceneRenderTargets.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/SubsurfaceProfile.h"
#include "CanvasTypes.h"
#include "GenerateMips.h"
#include "ClearQuad.h"
#include "Substrate/Substrate.h"
#include "PostProcess/TemporalAA.h"
#include "SubsurfaceTiles.h"
#include "UnrealEngine.h"
#include "SceneViewState.h"

namespace
{
	// Subsurface common parameters
	TAutoConsoleVariable<int32> CVarSubsurfaceScattering(
		TEXT("r.SubsurfaceScattering"),
		1,
		TEXT(" 0: disabled\n")
		TEXT(" 1: enabled (default)"),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	TAutoConsoleVariable<float> CVarSSSScale(
		TEXT("r.SSS.Scale"),
		1.0f,
		TEXT("Affects the Screen space Separable subsurface scattering pass ")
		TEXT("(use shadingmodel SubsurfaceProfile, get near to the object as the default)\n")
		TEXT("is human skin which only scatters about 1.2cm)\n")
		TEXT(" 0: off (if there is no object on the screen using this pass it should automatically disable the post process pass)\n")
		TEXT("<1: scale scatter radius down (for testing)\n")
		TEXT(" 1: use given radius form the Subsurface scattering asset (default)\n")
		TEXT(">1: scale scatter radius up (for testing)"),
		ECVF_Scalability | ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarSSSHalfRes(
		TEXT("r.SSS.HalfRes"),
		1,
		TEXT(" 0: full quality (Combined Burley and Separable pass. Separable is not optimized, as reference)\n")
		TEXT(" 1: parts of the algorithm runs in half resolution which is lower quality but faster (default, Separable only)"),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	TAutoConsoleVariable<int32> CVarSSSHalfResForceSeparable(
		TEXT("r.SSS.HalfRes.ForceSeparable"),
		1,
		TEXT(" 0: Use the implementation hint from the subsurface profile.\n")
		TEXT(" 1: Force to use separable filter in HalfRes mode (default)."),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	TAutoConsoleVariable<int32> CVarSSSQuality(
		TEXT("r.SSS.Quality"),
		0,
		TEXT("Defines the quality of the recombine pass when using the SubsurfaceScatteringProfile shading model\n")
		TEXT(" 0: low (faster, default)\n")
		TEXT(" 1: high (sharper details but slower)\n")
		TEXT("-1: auto, 1 if TemporalAA is disabled (without TemporalAA the quality is more noticable)"),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	TAutoConsoleVariable<int32> CVarSSSFilter(
		TEXT("r.SSS.Filter"),
		1,
		TEXT("Defines the filter method for Screenspace Subsurface Scattering feature.\n")
		TEXT(" 0: point filter (useful for testing, could be cleaner)\n")
		TEXT(" 1: bilinear filter"),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	TAutoConsoleVariable<int32> CVarSSSSampleSet(
		TEXT("r.SSS.SampleSet"),
		2,
		TEXT("Defines how many samples we use for Separable Screenspace Subsurface Scattering feature.\n")
		TEXT(" 0: lowest quality (6*2+1)\n")
		TEXT(" 1: medium quality (9*2+1)\n")
		TEXT(" 2: high quality (13*2+1) (default)"),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	TAutoConsoleVariable<int32> CVarSSSBurleyUpdateParameter(
		TEXT("r.SSS.Burley.AlwaysUpdateParametersFromSeparable"),
		0,
		TEXT("0: Will not update parameters when the program loads. (default)")
		TEXT("1: Always update from the separable when the program loads. (Correct only when Subsurface color is 1)."),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	TAutoConsoleVariable<int32> CVarSSSCheckerboard(
		TEXT("r.SSS.Checkerboard"),
		2,
		TEXT("Enables or disables checkerboard rendering for subsurface profile rendering.\n")
		TEXT("This is necessary if SceneColor does not include a floating point alpha channel (e.g 32-bit formats)\n")
		TEXT(" 0: Disabled (high quality). If the rendertarget format does not have an alpha channel (e.g., PF_FloatR11G11B10), it leads to over-washed SSS. \n")
		TEXT(" 1: Enabled (low quality). Surface lighting will be at reduced resolution.\n")
		TEXT(" 2: Automatic. Non-checkerboard lighting will be applied if we have a suitable rendertarget format\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarSSSCheckerboardNeighborSSSValidation(
		TEXT("r.SSS.Checkerboard.NeighborSSSValidation"),
		0,
		TEXT("Enable or disable checkerboard neighbor subsurface scattering validation.\n")
		TEXT("This validation can remove border light leakage into subsurface scattering, creating a sharpe border with correct color")
		TEXT(" 0: Disabled (default)")
		TEXT(" 1: Enabled. Add 1 subsurface profile id query/pixel (low quality), 4 id query/pixel (high quality) at recombine pass"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarSSSBurleyQuality(
		TEXT("r.SSS.Burley.Quality"),
		1,
		TEXT("0: Fallback mode. Burley falls back to run scattering in Separable with transmission in Burley for better performance. Separable parameters are automatically fitted.")
		TEXT("1: Automatic. The subsurface will only switch to separable in half resolution. (default)"),
		ECVF_RenderThreadSafe | ECVF_Scalability
	);

	TAutoConsoleVariable<int32> CVarSSSBurleyNumSamplesOverride(
		TEXT("r.SSS.Burley.NumSamplesOverride"),
		0,
		TEXT("When zero, Burley SSS adaptively determines the number of samples. When non-zero, this value overrides the sample count.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarSSSBurleyEnableProfileIdCache(
		TEXT("r.SSS.Burley.EnableProfileIdCache"),
		1,
		TEXT("0: Disable profile id cache using in the sampling pass.\n")
		TEXT("1: Consumes 1 byte per pixel more memory to make Burley pass much faster. (default)\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarSSSBurleyBilateralFilterKernelFunctionType(
		TEXT("r.SSS.Burley.BilateralFilterKernelFunctionType"),
		1,
		TEXT("0: Depth Only. It is more performant (x2 faster for close view).")
		TEXT("1: Depth and normal. It leads to better quality in regions like eyelids. (default)"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarSSSMipmapsMinTileCount(
		TEXT("r.SSS.Burley.MinGenerateMipsTileCount"),
		4000,
		TEXT("4000. (default) The minimal number of tiles to trigger subsurface radiance mip generation. Set to zero to always generate mips (Experimental value)"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarSSSSubpixelThreshold(
		TEXT("r.SSS.Subpixel.Threshold"),
		1.0f,
		TEXT("If a scattering is always within 1 pixel footage, there is no need to scatter. This value determines at which percentage the scattering is considered all inside\n")
		TEXT("the texel itself. Default is below 1. Once it reaches this value, we stop doing subsurface scattering. The logic is per tile."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<float> CVarSubSurfaceColorAsTannsmittanceAtDistance(
		TEXT("r.SSS.SubSurfaceColorAsTansmittanceAtDistance"),
		0.15f,
		TEXT("Normalized distance (0..1) at which the surface color is interpreted as transmittance color to compute extinction coefficients."),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarSSSSeparatedDiffuseEnable(
		TEXT("r.SSS.SeparatedDiffuse.Enable"),
		1,
		TEXT("Whether store diffuse luminance in a separated texture\n")
		TEXT(" 0: stores in A channel of SceneColor if RGBA format, checkerboard of SceneColor if RGB format:\n")
		TEXT(" 1: Force the luminance to store in the separated texture.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarSSSSeparatedDiffuseSupported(
		TEXT("r.SSS.SeparatedDiffuse.Supported"),
		0,
		TEXT("Whether separated diffuse texture is supported for subsurface scattering in GBuffer format path.\n")
		TEXT("When it's on, a separated texture will be created and bound inevitably. Should use with SceneColor format RGB for quality and performance\n")
		TEXT("It is not compatible to Substrate experimental IsOpaqueRoughRefraction feature.\n"),
		ECVF_ReadOnly);

	TAutoConsoleVariable<int32> CVarSSSSeparatedDiffuseDebug(
		TEXT("r.SSS.SeparatedDiffuse.Debug"),
		0,
		TEXT("Whether the separated diffuse texture is meaningful when non-RGB format is enabled.\n"),
		ECVF_RenderThreadSafe);

	TAutoConsoleVariable<int32> CVarSSSDebugPixel(
		TEXT("r.SSS.DebugPixel"),
		0,
		TEXT("Debug the pixel value and recombine process in full resolution. Need to enable shaderprint with r.ShaderPrint 1.\n"),
		ECVF_RenderThreadSafe);

	DECLARE_GPU_STAT(SubsurfaceScattering);
}

// Define to use a custom ps to clear UAV.
#define USE_CUSTOM_CLEAR_UAV

enum class ESubsurfaceMode : uint32
{
	// Performs a full resolution scattering filter.
	FullRes,

	// Performs a half resolution scattering filter.
	HalfRes,

	// Reconstructs lighting, but does not perform scattering.
	Bypass,

	MAX
};


const TCHAR* GetEventName(ESubsurfaceMode SubsurfaceMode)
{
	static const TCHAR* const kEventNames[] = {
		TEXT("FullRes"),
		TEXT("HalfRes"),
		TEXT("Bypass"),
	};
	static_assert(UE_ARRAY_COUNT(kEventNames) == int32(ESubsurfaceMode::MAX), "Fix me");
	return kEventNames[int32(SubsurfaceMode)];
}


// Returns the [0, N] clamped value of the 'r.SSS.Scale' CVar.
float GetSubsurfaceRadiusScale()
{
	static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.SSS.Scale"));
	check(CVar);

	return FMath::Max(0.0f, CVar->GetValueOnRenderThread());
}

int32 GetSSSFilter()
{
	return CVarSSSFilter.GetValueOnRenderThread();
}

int32 GetSSSSampleSet()
{
	return CVarSSSSampleSet.GetValueOnAnyThread();
}

int32 GetSSSQuality()
{
	return CVarSSSQuality.GetValueOnAnyThread();
}

int32 GetSSSBurleyBilateralFilterKernelFunctionType()
{
	return CVarSSSBurleyBilateralFilterKernelFunctionType.GetValueOnAnyThread();
}

// Returns the current subsurface mode required by the current view.
ESubsurfaceMode GetSubsurfaceModeForView(const FViewInfo& View)
{
	const float Radius = GetSubsurfaceRadiusScale();
	const bool bShowSubsurfaceScattering = Radius > 0 && View.Family->EngineShowFlags.SubsurfaceScattering;

	if (bShowSubsurfaceScattering)
	{
		const bool bHalfRes = CVarSSSHalfRes.GetValueOnRenderThread() != 0;
		if (bHalfRes)
		{
			return ESubsurfaceMode::HalfRes;
		}
		else
		{
			return ESubsurfaceMode::FullRes;
		}
	}
	else
	{
		return ESubsurfaceMode::Bypass;
	}
}

// A shader parameter struct for a single subsurface input texture.
BEGIN_SHADER_PARAMETER_STRUCT(FSubsurfaceInput, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FScreenPassTextureViewportParameters, Viewport)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Texture)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSubsurfaceSRVInput, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FScreenPassTextureViewportParameters, Viewport)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, Texture)
END_SHADER_PARAMETER_STRUCT();

// Set of common shader parameters shared by all subsurface shaders.
BEGIN_SHADER_PARAMETER_STRUCT(FSubsurfaceParameters, )
	SHADER_PARAMETER(FVector4f, SubsurfaceParams)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
END_SHADER_PARAMETER_STRUCT()

uint32 GetSSSMipmapsMinTileCount(const FViewInfo& View)
{
	return FMath::Max(0, CVarSSSMipmapsMinTileCount.GetValueOnRenderThread());
}

FSubsurfaceParameters GetSubsurfaceCommonParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures)
{
	const float DistanceToProjectionWindow = View.ViewMatrices.GetViewToClip().M[0][0];
	const float SSSScaleZ = DistanceToProjectionWindow * GetSubsurfaceRadiusScale();
	const float SSSScaleX = SSSScaleZ / SUBSURFACE_KERNEL_SIZE * 0.5f;

	const float SSSOverrideNumSamples = float(CVarSSSBurleyNumSamplesOverride.GetValueOnRenderThread());
	const float MinGenerateMipsTileCount = GetSSSMipmapsMinTileCount(View);

	FSubsurfaceParameters Parameters;
	Parameters.SubsurfaceParams = FVector4f(SSSScaleX, SSSScaleZ, SSSOverrideNumSamples, MinGenerateMipsTileCount);
	Parameters.ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters.SceneTextures = SceneTextures;
	return Parameters;
}

FSubsurfaceInput GetSubsurfaceInput(FRDGTextureRef Texture, const FScreenPassTextureViewportParameters& ViewportParameters)
{
	FSubsurfaceInput Input;
	Input.Texture = Texture;
	Input.Viewport = ViewportParameters;
	return Input;
}

FSubsurfaceSRVInput GetSubsurfaceSRVInput(FRDGTextureSRVRef Texture, const FScreenPassTextureViewportParameters& ViewportParameters)
{
	FSubsurfaceSRVInput Input;
	Input.Texture = Texture;
	Input.Viewport = ViewportParameters;
	return Input;
}

bool IsSubsurfaceEnabled(const FEngineShowFlags& EngineShowFlags)
{
	const bool bEnabled = CVarSubsurfaceScattering.GetValueOnAnyThread() != 0 && EngineShowFlags.Lighting;
	const bool bHasScale = CVarSSSScale.GetValueOnAnyThread() > 0.0f;
	return (bEnabled && bHasScale);
}

bool IsSubsurfaceRequiredForView(const FViewInfo& View)
{
	const bool bSimpleDynamicLighting = IsForwardShadingEnabled(View.GetShaderPlatform());
	const bool bSubsurfaceEnabled = IsSubsurfaceEnabled(View.Family->EngineShowFlags);
	const bool bViewHasSubsurfaceMaterials = ((View.ShadingModelMaskInView & GetUseSubsurfaceProfileShadingModelMask()) != 0);
	return (bSubsurfaceEnabled && bViewHasSubsurfaceMaterials && !bSimpleDynamicLighting);
}

bool IsProfileIdCacheEnabled()
{
	return UE::PixelFormat::HasCapabilities(PF_R8_UINT, EPixelFormatCapabilities::TypedUAVLoad)
		&& CVarSSSBurleyEnableProfileIdCache.GetValueOnRenderThread() != 0;
}

bool UseSeparatedSubsurfaceDiffuse(EShaderPlatform ShaderPlatform)
{
	return PlatformSupportsSeparatedSubsurfaceDiffuse(ShaderPlatform)
		&& IsUsingGBuffers(ShaderPlatform)
		&& !Substrate::IsOpaqueRoughRefractionEnabled(ShaderPlatform);
}

uint32 GetSubsurfaceRequiredViewMask(TArrayView<const FViewInfo> Views)
{
	const uint32 ViewCount = Views.Num();
	uint32 ViewMask = 0;

	// Traverse the views to make sure we only process subsurface if requested by any view.
	for (uint32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
	{
		const FViewInfo& View = Views[ViewIndex];

		if (IsSubsurfaceRequiredForView(View))
		{
			const uint32 ViewBit = 1 << ViewIndex;

			ViewMask |= ViewBit;
		}
	}

	return ViewMask;
}

bool IsSubsurfaceCheckerboardFormat(EPixelFormat SceneColorFormat, EShaderPlatform ShaderPlatform)
{
	if (Substrate::IsOpaqueRoughRefractionEnabled(ShaderPlatform))
	{
		// With this mode, specular and subsurface colors are correctly separated so checkerboard is not required.
		return false;
	}
	const int SeparatedDiffuseEnable = CVarSSSSeparatedDiffuseEnable.GetValueOnAnyThread();
	const bool bSeparatedDiffuseSupported = UseSeparatedSubsurfaceDiffuse(ShaderPlatform);
	if (SeparatedDiffuseEnable > 0 && bSeparatedDiffuseSupported)
	{
		// Force the luminance to store in the separated texture
		return false;
	}
	int CVarValue = CVarSSSCheckerboard.GetValueOnRenderThread();
	if (CVarValue == 0)
	{
		return false;
	}
	else if (CVarValue == 1)
	{
		return true;
	}
	else if (CVarValue == 2)
	{
		switch (SceneColorFormat)
		{
		case PF_A32B32G32R32F:
		case PF_FloatRGBA:
			return false;
		default:
			return true;
		}
	}
	return true;
}

bool IsSubsurfaceCheckerboardFormat(EPixelFormat SceneColorFormat, const FViewInfo& View)
{
	return IsSubsurfaceCheckerboardFormat(SceneColorFormat, View.GetShaderPlatform());
}

bool IsSeparatedDiffuseLuminanceEnabled(const FEngineShowFlags& EngineShowFlags, EPixelFormat SceneColorFormat, EShaderPlatform ShaderPlatform)
{
	const int bSeparatedDiffuseEnable = UseSeparatedSubsurfaceDiffuse(ShaderPlatform);
	const bool bEnabled = IsSubsurfaceEnabled(EngineShowFlags);

	const bool bIsCheckerboardEnabled = IsSubsurfaceCheckerboardFormat(SceneColorFormat, ShaderPlatform);
	const bool bColorFormatDoesNotHaveAlpha = SceneColorFormat != PF_A32B32G32R32F && SceneColorFormat!= PF_FloatRGBA;
	const bool bDebugSeparatedDiffuse = CVarSSSSeparatedDiffuseDebug.GetValueOnAnyThread() != 0;
	return bEnabled && bSeparatedDiffuseEnable && !bIsCheckerboardEnabled && (bColorFormatDoesNotHaveAlpha || bDebugSeparatedDiffuse);
}

// Base class for a subsurface shader.
class FSubsurfaceShader : public FGlobalShader
{
public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_RADIUS_SCALE"), SUBSURFACE_RADIUS_SCALE);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_KERNEL_SIZE"), SUBSURFACE_KERNEL_SIZE);
	}

	FSubsurfaceShader() = default;
	FSubsurfaceShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

// Encapsulates the post processing subsurface scattering common pixel shader.
class FSubsurfaceVisualizePS : public FSubsurfaceShader
{
public:
	DECLARE_GLOBAL_SHADER(FSubsurfaceVisualizePS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceVisualizePS, FSubsurfaceShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceParameters, Subsurface)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput0)
		SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler0)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceVisualizePS, "/Engine/Private/PostProcessSubsurface.usf", "VisualizePS", SF_Pixel);

// Encapsulates a simple copy pixel shader.
class FSubsurfaceViewportCopyPS : public FSubsurfaceShader
{
	DECLARE_GLOBAL_SHADER(FSubsurfaceViewportCopyPS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceViewportCopyPS, FSubsurfaceShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SubsurfaceInput0_Texture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler0)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceViewportCopyPS, "/Engine/Private/PostProcessSubsurface.usf", "SubsurfaceViewportCopyPS", SF_Pixel);


//-------------------------------------------------------------------------------------------
// Indirect dispatch class and functions
//-------------------------------------------------------------------------------------------

// Subsurface uniform buffer layout
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSubsurfaceUniformParameters, )
	SHADER_PARAMETER(uint32, MaxGroupCount)
	SHADER_PARAMETER(uint32, bRectPrimitive)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSubsurfaceUniformParameters, "SubsurfaceUniformParameters");
typedef TUniformBufferRef<FSubsurfaceUniformParameters> FSubsurfaceUniformRef;

// Return a uniform buffer with values filled and with single frame lifetime
FSubsurfaceUniformRef CreateUniformBuffer(FViewInfo const& View, int32 MaxGroupCount)
{
	FSubsurfaceUniformParameters Parameters;
	Parameters.MaxGroupCount = MaxGroupCount;
	Parameters.bRectPrimitive = GRHISupportsRectTopology;
	return FSubsurfaceUniformRef::CreateUniformBufferImmediate(Parameters, UniformBuffer_SingleFrame);
}

class FSubsurfaceInitValueBufferCS : public FSubsurfaceShader
{
public:
	DECLARE_GLOBAL_SHADER(FSubsurfaceInitValueBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceInitValueBufferCS, FSubsurfaceShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSubsurfaceShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_BURLEY_COMPUTE"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWTileTypeCountBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceInitValueBufferCS, "/Engine/Private/PostProcessSubsurface.usf", "InitValueBufferCS", SF_Compute);

class FFillMipsConditionBufferCS : public FSubsurfaceShader
{
public:
	DECLARE_GLOBAL_SHADER(FFillMipsConditionBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FFillMipsConditionBufferCS, FSubsurfaceShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSubsurfaceShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_BURLEY_COMPUTE"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MinGenerateMipsTileCount)
		SHADER_PARAMETER(uint32, Offset)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileTypeCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWMipsConditionBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFillMipsConditionBufferCS, "/Engine/Private/PostProcessSubsurface.usf", "FillMipsConditionBufferCS", SF_Compute);

class FSubsurfaceBuildIndirectDispatchArgsCS : public FSubsurfaceShader
{
public:
	DECLARE_GLOBAL_SHADER(FSubsurfaceBuildIndirectDispatchArgsCS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceBuildIndirectDispatchArgsCS, FSubsurfaceShader)

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSubsurfaceShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_BURLEY_COMPUTE"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FSubsurfaceUniformParameters, SubsurfaceUniformParameters)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDispatchArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDrawArgsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileTypeCountBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceBuildIndirectDispatchArgsCS, "/Engine/Private/PostProcessSubsurface.usf", "BuildIndirectDispatchArgsCS", SF_Compute);

class FSubsurfaceIndirectDispatchSetupCS : public FSubsurfaceShader
{
public:
	DECLARE_GLOBAL_SHADER(FSubsurfaceIndirectDispatchSetupCS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceIndirectDispatchSetupCS, FSubsurfaceShader)

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSubsurfaceShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_BURLEY_COMPUTE"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_DIFFUSE_OPERATION"),/*DIFFUSE_LUMINANCE_READ*/ 0);
		OutEnvironment.SetDefine(TEXT("SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE"), PlatformSupportsSeparatedSubsurfaceDiffuse(Parameters.Platform) ? 1 : 0);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceDiffuseReadParameters, SubsurfaceDiffuseParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceParameters, Subsurface)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput0)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler0)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, SetupTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, GroupBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWSeparableGroupBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWBurleyGroupBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWAllGroupBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWTileTypeCountBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, ProfileIdTexture)
		SHADER_PARAMETER_STRUCT_REF(FSubsurfaceUniformParameters, SubsurfaceUniformParameters)
		RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer, ERHIAccess::IndirectArgs)
	END_SHADER_PARAMETER_STRUCT()
	
	// Controls the quality of lighting reconstruction.
	enum class EQuality : uint32
	{
		Low,
		High,
		MAX
	};

	enum class ESubsurfaceSetupPass : uint32
	{
		PassOneMain,    // Subsurface Setup. Write to Setup texture
		PassTwoPassThrough,	// PassThrough to final texture before recombine (due to Subpixel).
		MAX
	};

	static const TCHAR* GetEventName(EQuality Quality)
	{
		static const TCHAR* const kEventNames[] = {
			TEXT("Low"),
			TEXT("High"),
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == int32(EQuality::MAX), "Fix me");
		return kEventNames[int32(Quality)];
	}

	static const TCHAR* GetPassName(ESubsurfaceSetupPass Pass)
	{
		static const TCHAR* const kEventNames[] = {
			TEXT("Main"),
			TEXT("Passthrough"),
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == int32(ESubsurfaceSetupPass::MAX), "Fix me");
		return kEventNames[int32(Pass)];
	}

	class FDimensionHalfRes : SHADER_PERMUTATION_BOOL("SUBSURFACE_HALF_RES");
	class FDimensionCheckerboard : SHADER_PERMUTATION_BOOL("SUBSURFACE_PROFILE_CHECKERBOARD");
	class FRunningInSeparable : SHADER_PERMUTATION_BOOL("SUBSURFACE_FORCE_SEPARABLE");
	class FDimensionEnableProfileIDCache : SHADER_PERMUTATION_BOOL("ENABLE_PROFILE_ID_CACHE");
	class FDimensionQuality : SHADER_PERMUTATION_ENUM_CLASS("SUBSURFACE_RECOMBINE_QUALITY", EQuality);
	class FDimensionSetupPass :SHADER_PERMUTATION_ENUM_CLASS("SUBSURFACE_SETUP_PASS", ESubsurfaceSetupPass);
	using FPermutationDomain = TShaderPermutationDomain<FDimensionHalfRes, FDimensionCheckerboard, FRunningInSeparable, FDimensionEnableProfileIDCache, FDimensionQuality, FDimensionSetupPass>;
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceIndirectDispatchSetupCS, "/Engine/Private/PostProcessSubsurface.usf", "SetupIndirectCS", SF_Compute);

class FSubsurfaceIndirectDispatchCS : public FSubsurfaceShader
{
public:
	DECLARE_GLOBAL_SHADER(FSubsurfaceIndirectDispatchCS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceIndirectDispatchCS, FSubsurfaceShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSubsurfaceShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_BURLEY_COMPUTE"), 1);
		OutEnvironment.SetDefine(TEXT("ENABLE_VELOCITY"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_GROUP_SIZE"), FSubsurfaceTiles::TileSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceParameters, Subsurface)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, SSSColorUAV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, HistoryUAV)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, GroupBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, TileTypeCountBuffer)
		RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput0)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler0)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput1)	// History
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler1)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput2)   // Profile mask | Velocity
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler2)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput3)   // Control Variates
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler3)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, ProfileIdTexture)
	END_SHADER_PARAMETER_STRUCT()

	// Direction of the 1D separable filter.
	enum class EDirection : uint32
	{
		Horizontal,
		Vertical,
		MAX
	};

	enum class ESubsurfacePass : uint32
	{
		PassOne,    // Burley sampling   (or Horizontal) pass   pass one
		PassTwo,	// Variance updating (or   Vertical) pass pass two
		MAX
	};

	// Controls the quality (number of samples) of the blur kernel.
	enum class EQuality : uint32
	{
		Low,
		Medium,
		High,
		MAX
	};
	
	enum class EBilateralFilterKernelFunctionType : uint32
	{
		Depth,
		DepthAndNormal,
		MAX
	};

	enum class ESubsurfaceType : uint32
	{
		BURLEY,
		SEPARABLE,
		MAX
	};

	static const TCHAR* GetEventName(EDirection Direction)
	{
		static const TCHAR* const kEventNames[] = {
			TEXT("Horizontal"),
			TEXT("Vertical"),
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == int32(EDirection::MAX), "Fix me");
		return kEventNames[int32(Direction)];
	}

	static const TCHAR* GetEventName(ESubsurfacePass SubsurfacePass)
	{
		static const TCHAR* const kEventNames[] = {
			TEXT("PassOne"),
			TEXT("PassTwo"),
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == int32(ESubsurfacePass::MAX), "Fix me");
		return kEventNames[int32(SubsurfacePass)];
	}

	static const TCHAR* GetEventName(EQuality Quality)
	{
		static const TCHAR* const kEventNames[] = {
			TEXT("Low"),
			TEXT("Medium"),
			TEXT("High"),
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == int32(EQuality::MAX), "Fix me");
		return kEventNames[int32(Quality)];
	}

	static const TCHAR* GetEventName(ESubsurfaceType SubsurfaceType)
	{
		static const TCHAR* const kEventNames[] = {
			TEXT("Burley"),
			TEXT("Separable"),
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == int32(ESubsurfaceType::MAX), "Fix me");
		return kEventNames[int32(SubsurfaceType)];
	}

	class FSubsurfacePassFunction : SHADER_PERMUTATION_ENUM_CLASS("SUBSURFACE_PASS", ESubsurfacePass);
	class FDimensionQuality : SHADER_PERMUTATION_ENUM_CLASS("SUBSURFACE_QUALITY", EQuality);
	class FBilateralFilterKernelFunctionType : SHADER_PERMUTATION_ENUM_CLASS("BILATERAL_FILTER_KERNEL_FUNCTION_TYPE", EBilateralFilterKernelFunctionType);
	class FSubsurfaceType : SHADER_PERMUTATION_ENUM_CLASS("SUBSURFACE_TYPE", ESubsurfaceType);
	class FDimensionHalfRes : SHADER_PERMUTATION_BOOL("SUBSURFACE_HALF_RES");
	class FRunningInSeparable : SHADER_PERMUTATION_BOOL("SUBSURFACE_FORCE_SEPARABLE");
	class FDimensionEnableProfileIDCache : SHADER_PERMUTATION_BOOL("ENABLE_PROFILE_ID_CACHE");
	using FPermutationDomain = TShaderPermutationDomain<FSubsurfacePassFunction, FDimensionQuality, 
		FBilateralFilterKernelFunctionType, FSubsurfaceType, FDimensionHalfRes, FRunningInSeparable, FDimensionEnableProfileIDCache>;

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FDimensionQuality>() != GetQuality())
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}
		if (PermutationVector.Get<FBilateralFilterKernelFunctionType>() != GetBilateralFilterKernelFunctionType())
		{
			return EShaderPermutationPrecacheRequest::NotUsed;
		}

		return EShaderPermutationPrecacheRequest::Precached;
	}

	// Returns the sampler state based on the requested SSS filter CVar setting and half resolution setting.
	static FRHISamplerState* GetSamplerState(bool bHalfRes)
	{
		if (GetSSSFilter())
		{	// Trilinear is used for mipmap sampling in full resolution
			if (bHalfRes)
			{
				return TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Border>::GetRHI();//SF_Bilinear
			}
			else
			{
				return TStaticSamplerState<SF_Trilinear, AM_Border, AM_Border, AM_Border>::GetRHI();//SF_Bilinear
			}
		}
		else
		{
			return TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border>::GetRHI();
		}
	}

	// Returns the SSS quality level requested by the SSS SampleSet CVar setting.
	static EQuality GetQuality()
	{
		return static_cast<FSubsurfaceIndirectDispatchCS::EQuality>(
			FMath::Clamp(
				GetSSSSampleSet(),
				static_cast<int32>(FSubsurfaceIndirectDispatchCS::EQuality::Low),
				static_cast<int32>(FSubsurfaceIndirectDispatchCS::EQuality::High)));
	}

	static EBilateralFilterKernelFunctionType GetBilateralFilterKernelFunctionType()
	{
		return static_cast<FSubsurfaceIndirectDispatchCS::EBilateralFilterKernelFunctionType>(
			FMath::Clamp(
				GetSSSBurleyBilateralFilterKernelFunctionType(),
				static_cast<int32>(FSubsurfaceIndirectDispatchCS::EBilateralFilterKernelFunctionType::Depth),
				static_cast<int32>(FSubsurfaceIndirectDispatchCS::EBilateralFilterKernelFunctionType::DepthAndNormal)));
	}

	static FSubsurfaceTiles::ETileType ToSubsurfaceTileType(ESubsurfaceType Type, bool bForceSeparable)
	{
		if (bForceSeparable)
		{
			return FSubsurfaceTiles::ETileType::AllNonPassthrough;
		}

		FSubsurfaceTiles::ETileType TileType = FSubsurfaceTiles::ETileType::AFIS;

		switch (Type)
		{
			case ESubsurfaceType::BURLEY: TileType = FSubsurfaceTiles::ETileType::AFIS; break;
			case ESubsurfaceType::SEPARABLE: TileType = FSubsurfaceTiles::ETileType::SEPARABLE; break;
		}

		return TileType;
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceIndirectDispatchCS, "/Engine/Private/PostProcessSubsurface.usf", "MainIndirectDispatchCS", SF_Compute);

// resolve textures that is not SRV
// Encapsulates a simple copy pixel shader.
class FSubsurfaceSRVResolvePS : public FSubsurfaceShader
{
	DECLARE_GLOBAL_SHADER(FSubsurfaceSRVResolvePS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceSRVResolvePS, FSubsurfaceShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SubsurfaceInput0_Texture)
		RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer, ERHIAccess::IndirectArgs)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler0)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceSRVResolvePS, "/Engine/Private/PostProcessSubsurface.usf", "SubsurfaceViewportCopyPS", SF_Pixel);

FRDGTextureRef CreateBlackUAVTexture(FRDGBuilder& GraphBuilder, FRDGTextureDesc SRVDesc, const TCHAR* Name, const FViewInfo& View,
	const FScreenPassTextureViewport& SceneViewport)
{
#ifdef USE_CUSTOM_CLEAR_UAV
	SRVDesc.Flags |= TexCreate_ShaderResource | TexCreate_UAV;
	FRDGTextureRef SRVTextureOutput = GraphBuilder.CreateTexture(SRVDesc, Name);

	FSubsurfaceSRVResolvePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubsurfaceSRVResolvePS::FParameters>();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SRVTextureOutput, ERenderTargetLoadAction::ENoAction);
	PassParameters->SubsurfaceInput0_Texture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
	PassParameters->SubsurfaceSampler0 = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	TShaderMapRef<FSubsurfaceSRVResolvePS> PixelShader(View.ShaderMap);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("SSS::ClearUAV"), View, SceneViewport, SceneViewport, PixelShader, PassParameters);
#else
	FRDGTextureRef SRVTextureOutput = GraphBuilder.CreateTexture(SRVDesc, Name);
	FRDGTextureUAVDesc UAVClearDesc(SRVTextureOutput, 0);

	ClearUAV(GraphBuilder, FRDGEventName(TEXT("SSS::ClearUAV")), GraphBuilder.CreateUAV(UAVClearDesc), FLinearColor::Black);
#endif

	return SRVTextureOutput;
}

// create a black UAV only when CondtionBuffer[Offset] > 0.
FRDGTextureRef CreateBlackUAVTexture(FRDGBuilder& GraphBuilder, FRDGTextureDesc SRVDesc, const TCHAR* TextureName, FRDGEventName&& PassName,
	const FScreenPassTextureViewport& SceneViewport, FRDGBufferRef ConditionBuffer, uint32 Offset)
{
	SRVDesc.Flags |= TexCreate_ShaderResource | TexCreate_UAV;
	FRDGTextureRef TextureOutput = GraphBuilder.CreateTexture(SRVDesc, TextureName);
	
	AddConditionalClearBlackUAVPass(
		GraphBuilder,
		Forward<FRDGEventName>(PassName),
		TextureOutput,
		SceneViewport,
		ConditionBuffer, 
		Offset);

	return TextureOutput;
}

// Helper function to use external textures for the current GraphBuilder.
// When the texture is null, we use BlackDummy.
FRDGTextureRef RegisterExternalRenderTarget(FRDGBuilder& GraphBuilder, TRefCountPtr<IPooledRenderTarget>* PtrExternalTexture, FIntPoint CurentViewExtent, const TCHAR* Name)
{
	FRDGTextureRef RegisteredTexture = NULL;

	if (!PtrExternalTexture || !(*PtrExternalTexture))
	{
		RegisteredTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy, Name);
	}
	else
	{
		if (CurentViewExtent != (*PtrExternalTexture)->GetDesc().Extent)
		{
			RegisteredTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy, Name);
		}
		else
		{
			RegisteredTexture = GraphBuilder.RegisterExternalTexture(*PtrExternalTexture, Name);
		}
	}

	return RegisteredTexture;
}

// Encapsulates the post processing subsurface recombine pixel shader.
class FSubsurfaceRecombinePS : public FSubsurfaceShader
{
	DECLARE_GLOBAL_SHADER(FSubsurfaceRecombinePS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceRecombinePS, FSubsurfaceShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceParameters, Subsurface)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceDiffuseReadParameters, SubsurfaceDiffuseParameters)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceTilePassVS::FParameters, TileParameters)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput0)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput1)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler0)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler1)
		SHADER_PARAMETER(uint32, CheckerboardNeighborSSSValidation)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();

	// Controls the quality of lighting reconstruction.
	enum class EQuality : uint32
	{
		Low,
		High,
		MAX
	};

	static const TCHAR* GetEventName(EQuality Quality)
	{
		static const TCHAR* const kEventNames[] = {
			TEXT("Low"),
			TEXT("High"),
		};
		static_assert(UE_ARRAY_COUNT(kEventNames) == int32(EQuality::MAX), "Fix me");
		return kEventNames[int32(Quality)];
	}

	class FDimensionMode : SHADER_PERMUTATION_ENUM_CLASS("SUBSURFACE_RECOMBINE_MODE", ESubsurfaceMode);
	class FDimensionQuality : SHADER_PERMUTATION_ENUM_CLASS("SUBSURFACE_RECOMBINE_QUALITY", EQuality);
	class FDimensionCheckerboard : SHADER_PERMUTATION_BOOL("SUBSURFACE_PROFILE_CHECKERBOARD");
	class FDimensionHalfRes : SHADER_PERMUTATION_BOOL("SUBSURFACE_HALF_RES");
	class FRunningInSeparable : SHADER_PERMUTATION_BOOL("SUBSURFACE_FORCE_SEPARABLE");
	using FPermutationDomain = TShaderPermutationDomain<FDimensionMode, FDimensionQuality, FDimensionCheckerboard, FDimensionHalfRes, FRunningInSeparable>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::SM5;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSubsurfaceShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_RECOMBINE"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_DIFFUSE_OPERATION"),/*DIFFUSE_LUMINANCE_READ*/ 0);
		OutEnvironment.SetDefine(TEXT("SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE"), PlatformSupportsSeparatedSubsurfaceDiffuse(Parameters.Platform) ? 1 : 0);
	}

	// Returns the Recombine quality level requested by the SSS Quality CVar setting.
	static EQuality GetQuality(const FViewInfo& View)
	{
		const uint32 QualityCVar = GetSSSQuality();

		EMainTAAPassConfig MainTAAConfig = GetMainTAAPassConfig(View);

		// Low quality is really bad with modern temporal upscalers.
		bool bAllowLowQuality = MainTAAConfig == EMainTAAPassConfig::Disabled || MainTAAConfig == EMainTAAPassConfig::TAA;
		if (!bAllowLowQuality)
		{
			return EQuality::High;
		}

		// Quality is forced to high when the CVar is set to 'auto' and TAA is NOT enabled.
		// TAA improves quality through temporal filtering and clamping box, making it less necessary to use
		// high quality mode.
		if (QualityCVar == -1)
		{
			if (MainTAAConfig == EMainTAAPassConfig::TAA)
			{
				return EQuality::Low;
			}
			else
			{
				return EQuality::High;
			}
		}

		if (QualityCVar == 1)
		{
			return EQuality::High;
		}
		else
		{
			return EQuality::Low;
		}
	}

	static uint32 GetCheckerBoardNeighborSSSValidation(bool bCheckerBoard)
	{
		bool bValidation = CVarSSSCheckerboardNeighborSSSValidation.GetValueOnRenderThread() > 0 ? true : false;
		return (bCheckerBoard && bValidation) ? 1u : 0u;
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceRecombinePS, "/Engine/Private/PostProcessSubsurface.usf", "SubsurfaceRecombinePS", SF_Pixel);

// copy write back additively to the scene color texture
class FSubsurfaceRecombineCopyPS : public FSubsurfaceShader
{
	DECLARE_GLOBAL_SHADER(FSubsurfaceRecombineCopyPS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceRecombineCopyPS, FSubsurfaceShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceParameters, Subsurface)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceTilePassVS::FParameters, TileParameters)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput0)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler0)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceRecombineCopyPS, "/Engine/Private/PostProcessSubsurface.usf", "SubsurfaceRecombineCopyPS", SF_Pixel);

// Calculate the Luminance after the base pass to get the Luminance of the scene color
class FSubsurfaceDiffuseInitPassPS : public FSubsurfaceShader
{
	DECLARE_GLOBAL_SHADER(FSubsurfaceDiffuseInitPassPS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceDiffuseInitPassPS, FSubsurfaceShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceTilePassVS::FParameters, TileParameters)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT();

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceDiffuseInitPassPS, "/Engine/Private/PostProcessSubsurfaceDiffuse.usf", "SubsurfaceDiffuseInitPassPS", SF_Pixel);

// Debug pass: prints SSS reconstruction + recombine data at cursor via ShaderPrint
class FSubsurfaceDebugCS : public FSubsurfaceShader
{
	DECLARE_GLOBAL_SHADER(FSubsurfaceDebugCS);
	SHADER_USE_PARAMETER_STRUCT(FSubsurfaceDebugCS, FSubsurfaceShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceParameters, Subsurface)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput0)
		SHADER_PARAMETER_STRUCT(FSubsurfaceInput, SubsurfaceInput1)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler0)
		SHADER_PARAMETER_SAMPLER(SamplerState, SubsurfaceSampler1)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSubsurfaceDiffuseReadParameters, SubsurfaceDiffuseParameters)
		SHADER_PARAMETER(float, ViewResolutionFraction)
		SHADER_PARAMETER(int, SceneColorHasAlpha)
	END_SHADER_PARAMETER_STRUCT()

	class FDimensionCheckerboard : SHADER_PERMUTATION_BOOL("SUBSURFACE_PROFILE_CHECKERBOARD");
	class FDimensionQuality : SHADER_PERMUTATION_ENUM_CLASS("SUBSURFACE_RECOMBINE_QUALITY", FSubsurfaceRecombinePS::EQuality);
	using FPermutationDomain = TShaderPermutationDomain<FDimensionCheckerboard, FDimensionQuality>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return FSubsurfaceShader::ShouldCompilePermutation(Parameters) && ShaderPrint::IsSupported(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSubsurfaceShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_DEBUG_PASS"), 1);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_DIFFUSE_OPERATION"), /*DIFFUSE_LUMINANCE_READ*/ 0);
		OutEnvironment.SetDefine(TEXT("SUBSURFACE_RECOMBINE_MODE"), /*SUBSURFACE_RECOMBINE_MODE_FULLRES*/ 0);
		OutEnvironment.SetDefine(TEXT("SSS_SUPPORT_SEPARATED_SUBSURFACE_DIFFUSE"), PlatformSupportsSeparatedSubsurfaceDiffuse(Parameters.Platform) ? 1 : 0);
	}
};

IMPLEMENT_GLOBAL_SHADER(FSubsurfaceDebugCS, "/Engine/Private/PostProcessSubsurface.usf", "SubsurfaceDebugCS", SF_Compute);


void SetupSubsurfaceDiffuseWriteParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSubsurfaceDiffuseWriteParameters& OutParameters)
{
	const FSceneTextures& SceneTextures = View.GetSceneTextures();
	FRDGTextureRef SeparatedSubsurfaceDiffuse = View.GetSceneTextures().SeparatedSubsurfaceDiffuse;
	bool bSeparatedSubsurfaceDiffuseEnabled = 
		IsSeparatedDiffuseLuminanceEnabled(
			View.Family->EngineShowFlags,
			SceneTextures.Color.Target->Desc.Format,
			View.GetShaderPlatform());

	if (SeparatedSubsurfaceDiffuse)
	{
		if (bSeparatedSubsurfaceDiffuseEnabled)
		{
			OutParameters.RWSeparatedSubsurfaceDiffuseLuminance = GraphBuilder.CreateUAV(SeparatedSubsurfaceDiffuse);
			OutParameters.bWriteSeparatedSubsurfaceDiffuse = 1;
		}
		else
		{
			FRDGTextureRef Dummy = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_R16F, FClearValueBinding::None,
					TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV),
				TEXT("DummyScatterableLuminance"));

			OutParameters.RWSeparatedSubsurfaceDiffuseLuminance = GraphBuilder.CreateUAV(Dummy);
			OutParameters.bWriteSeparatedSubsurfaceDiffuse = 0;
		}
	}
	else
	{
		OutParameters.RWSeparatedSubsurfaceDiffuseLuminance = nullptr;
		OutParameters.bWriteSeparatedSubsurfaceDiffuse = 0;
	}
}

void SetupSubsurfaceDiffuseReadParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSubsurfaceDiffuseReadParameters& OutParameters)
{
	const FSceneTextures& SceneTextures = View.GetSceneTextures();
	FRDGTextureRef SeparatedSubsurfaceDiffuse = View.GetSceneTextures().SeparatedSubsurfaceDiffuse;
	bool bSeparatedSubsurfaceDiffuseEnabled = 
		IsSeparatedDiffuseLuminanceEnabled(
			View.Family->EngineShowFlags,
			SceneTextures.Color.Target->Desc.Format,
			View.GetShaderPlatform());

	if (SeparatedSubsurfaceDiffuse)
	{
		if (bSeparatedSubsurfaceDiffuseEnabled)
		{
			OutParameters.SeparatedSubsurfaceDiffuseLuminance = GraphBuilder.CreateSRV(SeparatedSubsurfaceDiffuse);
			OutParameters.bUseSeparatedSubsurfaceDiffuse = 1;
		}
		else
		{
			FRDGTextureRef Dummy = GraphBuilder.CreateTexture(
				FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_R16F, FClearValueBinding::None,
					TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV),
				TEXT("DummySeparatedSubsurfaceDiffuseLuminance"));

			AddClearUAVPass(GraphBuilder,GraphBuilder.CreateUAV(Dummy),FLinearColor::White);
			OutParameters.SeparatedSubsurfaceDiffuseLuminance = GraphBuilder.CreateSRV(Dummy);
			OutParameters.bUseSeparatedSubsurfaceDiffuse = 0;
		}
	}
	else
	{
		OutParameters.SeparatedSubsurfaceDiffuseLuminance = nullptr;
		OutParameters.bUseSeparatedSubsurfaceDiffuse = 0;
	}
}

void RenderSubsurfaceDiffuseInitPass(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FSceneTextures& SceneTextures,
	TArray<FSubsurfaceTiles>& OutPerViewTiles)
{
	FRDGTextureRef SceneColorTexture = SceneTextures.Color.Target;
	FRDGTextureRef SeparatedSubsurfaceDiffuse = SceneTextures.SeparatedSubsurfaceDiffuse;
	OutPerViewTiles.SetNum(Views.Num());

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SubsurfaceScattering, "SSS::InitSeparatedDiffuse");

	auto ComputeSeparatedScatteringLuminance = [&](const FViewInfo& View, FSubsurfaceTiles& Tiles, 
		ERenderTargetLoadAction SeparatedDiffuseLoadAction){

		const ESubsurfaceMode SubsurfaceMode = GetSubsurfaceModeForView(View);
		const bool bByPassSubsurface = SubsurfaceMode == ESubsurfaceMode::Bypass;
		const FScreenPassTextureViewport SceneViewport(SceneColorTexture, View.ViewRect);
		const bool bHalfRes = (SubsurfaceMode == ESubsurfaceMode::HalfRes);
		const uint32 ScaleFactor = bHalfRes ? 2 : 1;
		const FScreenPassTextureViewport SubsurfaceViewport = GetDownscaledViewport(SceneViewport, ScaleFactor);

		const FScreenPassTextureViewportParameters SceneViewportParameters = GetScreenPassTextureViewportParameters(SceneViewport);
		const FScreenPassTextureViewportParameters SubsurfaceViewportParameters = GetScreenPassTextureViewportParameters(SubsurfaceViewport);

		// 1. Build SSS tiles in Init pass
		// Should not build tiles if bypassing as a whole pass will be initiated.
		if (!bByPassSubsurface)
		{
			Tiles = ClassifySSSTiles(GraphBuilder, View, SceneTextures, SceneViewportParameters, SubsurfaceViewportParameters, bHalfRes);
		}
		// 2. Tile-driven PS copy: SceneColor luminance -> SeparatedSubsurfaceDiffuse
		{
			FSubsurfaceDiffuseInitPassPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubsurfaceDiffuseInitPassPS::FParameters>();
			PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
			PassParameters->SceneColorTexture = SceneColorTexture;
			if (!bByPassSubsurface)
			{
				PassParameters->TileParameters = GetSubsurfaceTileParameters(SubsurfaceViewport, Tiles, FSubsurfaceTiles::ETileType::All);
			}
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SeparatedSubsurfaceDiffuse, SeparatedDiffuseLoadAction);

			TShaderMapRef<FSubsurfaceTilePassVS> VertexShader(View.ShaderMap);
			TShaderMapRef<FSubsurfaceDiffuseInitPassPS> PixelShader(View.ShaderMap);

			FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero>::GetRHI();
			FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			// ByPassing Subsurface still requires the whole diffuse in recombine pass.
			AddSubsurfaceTiledScreenPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSS::InitSeparatedDiffuse(%s) %dx%d",
					bByPassSubsurface ? TEXT("Full") : TEXT("Tiled"),
					View.ViewRect.Width(), 
					View.ViewRect.Height()),
				View,
				PassParameters,
				VertexShader,
				PixelShader,
				BlendState,
				DepthStencilState,
				SceneViewport,
				FSubsurfaceTiles::ETileType::All,
				bByPassSubsurface /*bShouldFallbackToFullScreenPass*/);
		}
	};

	// Initialize the separated scattering luminance from the base pass, indirect lighting contribution.
	// We build tiles here if possible.
	if (SeparatedSubsurfaceDiffuse)
	{
		const uint32 ViewMask = GetSubsurfaceRequiredViewMask(Views);

		if (!ViewMask)
		{
			return;
		}

		ERenderTargetLoadAction SeparatedDiffuseLoadAction = ERenderTargetLoadAction::EClear;
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const uint32 ViewBit = 1 << ViewIndex;

			const bool bIsSubsurfaceView = (ViewMask & ViewBit) != 0;

			const FViewInfo& View = Views[ViewIndex];

			bool bSeparatedSubsurfaceDiffuseEnabled = IsSeparatedDiffuseLuminanceEnabled( 
				View.Family->EngineShowFlags,
				SceneColorTexture->Desc.Format,
				View.GetShaderPlatform());

			if (bIsSubsurfaceView && bSeparatedSubsurfaceDiffuseEnabled)
			{
				ComputeSeparatedScatteringLuminance(View, OutPerViewTiles[ViewIndex], SeparatedDiffuseLoadAction);
				SeparatedDiffuseLoadAction = ERenderTargetLoadAction::ELoad;
			}
		}
	}
}

void AddSubsurfaceViewPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSubsurfaceTiles* PrebuiltTiles,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef SceneColorTextureOutput,
	ERenderTargetLoadAction SceneColorTextureLoadAction)
{
	FRDGTextureRef SceneColorTexture = SceneTextures.Color.Target;
	const FScreenPassTextureViewport SceneViewport(SceneColorTexture, View.ViewRect);

	check(SceneColorTextureOutput);

	const FSceneViewFamily* ViewFamily = View.Family;

	const FRDGTextureDesc& SceneColorTextureDesc = SceneColorTexture->Desc;

	const ESubsurfaceMode SubsurfaceMode = GetSubsurfaceModeForView(View);

	const bool bHalfRes = (SubsurfaceMode == ESubsurfaceMode::HalfRes);

	const bool bCheckerboard = IsSubsurfaceCheckerboardFormat(SceneColorTextureDesc.Format, View);

	const uint32 ScaleFactor = bHalfRes ? 2 : 1;
	
	//We run in separable mode under two conditions: 1) Run Burley fallback mode. 2) when the screen is in half resolution.
	const bool bForceRunningInSeparable = CVarSSSBurleyQuality.GetValueOnRenderThread() == 0 || 
		 (bHalfRes && CVarSSSHalfResForceSeparable.GetValueOnRenderThread() != 0 );

	const bool bUseProfileIdCache = IsProfileIdCacheEnabled();

	const uint32 MinGenerateMipsTileCount = GetSSSMipmapsMinTileCount(View);

	/**
	 * All subsurface passes within the screen-space subsurface effect can operate at half or full resolution,
	 * depending on the subsurface mode. The values are precomputed and shared among all Subsurface textures.
	 */
	const FScreenPassTextureViewport SubsurfaceViewport = GetDownscaledViewport(SceneViewport, ScaleFactor);

	const FRDGTextureDesc SceneColorTextureDescriptor = FRDGTextureDesc::Create2D(
		SceneViewport.Extent,
		PF_FloatRGBA,
		FClearValueBinding(),
		TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV);

	const FRDGTextureDesc SubsurfaceTextureDescriptor = FRDGTextureDesc::Create2D(
		SubsurfaceViewport.Extent,
		PF_FloatRGBA,
		FClearValueBinding(),
		TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV);

	const FRDGTextureDesc ProfileIdTextureDescriptor = FRDGTextureDesc::Create2D(
		SubsurfaceViewport.Extent,
		PF_R8_UINT,
		FClearValueBinding(),
		TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV);

	// Create texture desc with 6 mips if possible, otherwise clamp number of mips to match the viewport resolution
	const FRDGTextureDesc SubsurfaceTextureWith6MipsDescriptor = FRDGTextureDesc::Create2D(
		SubsurfaceViewport.Extent,
		PF_FloatRGBA,
		FClearValueBinding(),
		TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV,
		FMath::Min(6u, 1 + FMath::FloorLog2((uint32)SubsurfaceViewport.Extent.GetMin())));

	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	const FSubsurfaceParameters SubsurfaceCommonParameters = GetSubsurfaceCommonParameters(GraphBuilder, View, SceneTextures.UniformBuffer);
	const FScreenPassTextureViewportParameters SubsurfaceViewportParameters = GetScreenPassTextureViewportParameters(SubsurfaceViewport);
	const FScreenPassTextureViewportParameters SceneViewportParameters = GetScreenPassTextureViewportParameters(SceneViewport);

	const bool bReadSeparatedSubSurfaceSceneColor = Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform());
	const bool bWriteSeparatedOpaqueRoughRefractionSceneColor = Substrate::IsOpaqueRoughRefractionEnabled(View.GetShaderPlatform());
	FRDGTextureRef SeparatedSubSurfaceSceneColor = View.SubstrateViewData.SceneData->SeparatedSubSurfaceSceneColor;
	FRDGTextureRef SeparatedOpaqueRoughRefractionSceneColor = View.SubstrateViewData.SceneData->SeparatedOpaqueRoughRefractionSceneColor;

	FRDGTextureRef SetupTexture = SceneColorTexture;
	FRDGTextureRef SubsurfaceSubpassOneTex = nullptr;
	FRDGTextureRef SubsurfaceSubpassTwoTex = nullptr;
	FRDGTextureRef ProfileIdTexture = nullptr;

	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	FRHISamplerState* BilinearBorderSampler = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Border>::GetRHI();

	//History texture
	FSceneViewState* ViewState = (FSceneViewState*)View.State;
	TRefCountPtr<IPooledRenderTarget>* QualityHistoryState = ViewState ? &ViewState->SubsurfaceScatteringQualityHistoryRT : NULL;

	//allocate/reallocate the quality history texture. 
	FRDGTextureRef QualityHistoryTexture = RegisterExternalRenderTarget(GraphBuilder, QualityHistoryState, SubsurfaceTextureDescriptor.Extent, TEXT("Subsurface.QualityHistoryTexture"));
	FRDGTextureRef NewQualityHistoryTexture = nullptr;

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SubsurfaceScattering, "SubsurfaceScattering");

	FSubsurfaceTiles Tiles;

	/**
	 * When in bypass mode, the setup and convolution passes are skipped, but lighting
	 * reconstruction is still performed in the recombine pass.
	 */
	if (SubsurfaceMode != ESubsurfaceMode::Bypass)
	{
		// Support mipmaps in full resolution only. Need to clear the setup texture due to tiling logic.
		SetupTexture = CreateBlackUAVTexture(GraphBuilder, 
			bForceRunningInSeparable ? SubsurfaceTextureDescriptor:SubsurfaceTextureWith6MipsDescriptor, 
			TEXT("Subsurface.SetupTexture"), 
			View,
			SubsurfaceViewport);

		// profile cache to accelerate sampling
		if (bUseProfileIdCache)
		{
			// This path was designed to get used when r.SSS.Burley.EnableProfileIdCache is true. 
			ProfileIdTexture = GraphBuilder.CreateTexture(ProfileIdTextureDescriptor, TEXT("Subsurface.ProfileIdTexture"));
		}
		else
		{
			ProfileIdTexture = SystemTextures.Black;
		}

		FRDGTextureRef VelocityTexture = GetIfProduced(SceneTextures.Velocity, SystemTextures.Black);
		
		if (PrebuiltTiles && PrebuiltTiles->IsValid())
		{
			Tiles = *PrebuiltTiles;
		}
		else
		{
			Tiles = ClassifySSSTiles(GraphBuilder, View, SceneTextures, SceneViewportParameters, SubsurfaceViewportParameters, bHalfRes);
		}
		// Pre-allocate black UAV for the second buffer if all tiles are rendered in half resolution.
		// Need to access this texture to directly setup passthrough pixel.
		if (bHalfRes)
		{
			SubsurfaceSubpassTwoTex = CreateBlackUAVTexture(
				GraphBuilder,
				SubsurfaceTextureDescriptor,
				TEXT("Subsurface.SubpassTwoTex"),
				RDG_EVENT_NAME("SSS::ClearUAV(%s, %s.TileCount > 0)", TEXT("Subsurface.SubpassTwoTex"), ToString(FSubsurfaceTiles::ETileType::All)),
				SubsurfaceViewport,
				Tiles.TileTypeCountBuffer,
				ToIndex(FSubsurfaceTiles::ETileType::All));
		}
		else
		{
			SubsurfaceSubpassTwoTex = GraphBuilder.CreateTexture(SubsurfaceTextureDescriptor, TEXT("Subsurface.SubpassTwoTex"));
		}

		FSubsurfaceUniformRef UniformBuffer = CreateUniformBuffer(View, Tiles.TileCount);

		// Call the indirect setup (with tile classification)
		{
			typedef FSubsurfaceIndirectDispatchSetupCS SHADER;

			struct FSubsurfaceSetupPassInfo
			{
				FSubsurfaceSetupPassInfo(FSubsurfaceTiles::ETileType InTileType, FRDGTextureRef InOutput, SHADER::ESubsurfaceSetupPass InPass)
					: TileType(InTileType), Output(InOutput), Pass(InPass)
				{}

				FSubsurfaceTiles::ETileType TileType;
				FRDGTextureRef Output;
				SHADER::ESubsurfaceSetupPass Pass;
			};

			const int NumOfSubsurfaceSetupPass = 2;

			const FSubsurfaceSetupPassInfo SubsurfaceSetupPassInfos[NumOfSubsurfaceSetupPass] =
			{
				{FSubsurfaceTiles::ETileType::AllNonPassthrough, SetupTexture           , SHADER::ESubsurfaceSetupPass::PassOneMain},      // Main setup
				{FSubsurfaceTiles::ETileType::PASSTHROUGH,       SubsurfaceSubpassTwoTex, SHADER::ESubsurfaceSetupPass::PassTwoPassThrough} // Passthrough SSS for subpixel SSS
			};

			for (int PassIndex = 0; PassIndex < NumOfSubsurfaceSetupPass; ++PassIndex)
			{
				FSubsurfaceSetupPassInfo PassInfo = SubsurfaceSetupPassInfos[PassIndex];

				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				PassParameters->Subsurface = SubsurfaceCommonParameters;
				PassParameters->Output = SubsurfaceViewportParameters;
				PassParameters->SubsurfaceInput0 = GetSubsurfaceInput(bReadSeparatedSubSurfaceSceneColor ? SeparatedSubSurfaceSceneColor : SceneColorTexture, SceneViewportParameters);
				PassParameters->SubsurfaceSampler0 = PointClampSampler;
				PassParameters->SetupTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(PassInfo.Output, 0));
				if (bUseProfileIdCache)
				{
					PassParameters->ProfileIdTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(ProfileIdTexture));
				}
				PassParameters->RWBurleyGroupBuffer = GraphBuilder.CreateUAV(Tiles.GetTileBuffer(FSubsurfaceTiles::ETileType::AFIS), EPixelFormat::PF_R32_UINT);
				PassParameters->RWSeparableGroupBuffer = GraphBuilder.CreateUAV(Tiles.GetTileBuffer(FSubsurfaceTiles::ETileType::SEPARABLE), EPixelFormat::PF_R32_UINT);
				PassParameters->RWTileTypeCountBuffer = GraphBuilder.CreateUAV(Tiles.TileTypeCountBuffer, EPixelFormat::PF_R32_UINT);
				PassParameters->IndirectDispatchArgsBuffer = Tiles.TileIndirectDispatchBuffer;
				PassParameters->GroupBuffer = Tiles.GetTileBufferSRV(PassInfo.TileType);

				PassParameters->SubsurfaceUniformParameters = UniformBuffer;
				PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
				SetupSubsurfaceDiffuseReadParameters(GraphBuilder, View, PassParameters->SubsurfaceDiffuseParameters);

				SHADER::EQuality Quality = static_cast<SHADER::EQuality>(FSubsurfaceRecombinePS::GetQuality(View));

				SHADER::FPermutationDomain ComputeShaderPermutationVector;
				ComputeShaderPermutationVector.Set<SHADER::FDimensionHalfRes>(bHalfRes);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionCheckerboard>(bCheckerboard);
				ComputeShaderPermutationVector.Set<SHADER::FRunningInSeparable>(bForceRunningInSeparable);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionEnableProfileIDCache>(bUseProfileIdCache);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionQuality>(Quality);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionSetupPass>(PassInfo.Pass);
				TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, ComputeShaderPermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SSS::Setup(%s%s%s %s %s) %dx%d",
						ComputeShaderPermutationVector.Get<SHADER::FDimensionHalfRes>() ? TEXT(" HalfRes") : TEXT(""),
						ComputeShaderPermutationVector.Get<SHADER::FDimensionCheckerboard>() ? TEXT(" Checkerboard") : TEXT(""),
						ComputeShaderPermutationVector.Get<SHADER::FRunningInSeparable>() ? TEXT(" RunningInSeparable") : TEXT(""),
						SHADER::GetPassName(ComputeShaderPermutationVector.Get<SHADER::FDimensionSetupPass>()),
						SHADER::GetEventName(ComputeShaderPermutationVector.Get<SHADER::FDimensionQuality>()),
						SubsurfaceViewport.Extent.X,
						SubsurfaceViewport.Extent.Y),
					ComputeShader,
					PassParameters,
					PassParameters->IndirectDispatchArgsBuffer,
					Tiles.GetIndirectDispatchArgOffset(PassInfo.TileType));
			}
		}

	
		if (!bForceRunningInSeparable)
		{
			// Setup the indirect dispatch & draw arguments if not running in separable mode.
			{
				typedef FSubsurfaceBuildIndirectDispatchArgsCS ARGSETUPSHADER;
				ARGSETUPSHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<ARGSETUPSHADER::FParameters>();
				PassParameters->SubsurfaceUniformParameters = UniformBuffer;
				PassParameters->RWIndirectDispatchArgsBuffer = GraphBuilder.CreateUAV(Tiles.TileIndirectDispatchBuffer, EPixelFormat::PF_R32_UINT);
				PassParameters->RWIndirectDrawArgsBuffer = GraphBuilder.CreateUAV(Tiles.TileIndirectDrawBuffer, EPixelFormat::PF_R32_UINT);
				PassParameters->TileTypeCountBuffer = GraphBuilder.CreateSRV(Tiles.TileTypeCountBuffer, EPixelFormat::PF_R32_UINT);
				TShaderMapRef<ARGSETUPSHADER> ComputeShader(View.ShaderMap);
				FComputeShaderUtils::AddPass(GraphBuilder, FRDGEventName(TEXT("SSS::BuildIndirectArgs(Dispatch & Draw)")), ComputeShader, PassParameters, FIntVector(1, 1, 1));
			}

			// Separable does not need this mipmap.
			{
				FRDGBufferRef MipsConditionBuffer = GraphBuilder.CreateBuffer(
					FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), FSubsurfaceTiles::TileTypeCount), TEXT("Subsurface.MipsConditionBuffer"));
				const int32 Offset = ToIndex(FSubsurfaceTiles::ETileType::AFIS);

				{
					typedef FFillMipsConditionBufferCS SHADER;
					TShaderMapRef<SHADER> ComputeShader(View.ShaderMap);
					SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
					PassParameters->MinGenerateMipsTileCount = MinGenerateMipsTileCount;
					PassParameters->Offset = Offset;
					PassParameters->TileTypeCountBuffer = GraphBuilder.CreateSRV(Tiles.TileTypeCountBuffer, EPixelFormat::PF_R32_UINT);
					PassParameters->RWMipsConditionBuffer = GraphBuilder.CreateUAV(MipsConditionBuffer, EPixelFormat::PF_R32_UINT);

					FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("SSS::FillMipsConditionBufferCS"), ComputeShader, PassParameters, FIntVector(1, 1, 1));
				}

				// Generate mipmap for the diffuse scene color and depth, use bilinear filter conditionally
				FGenerateMips::ExecuteCompute(GraphBuilder, View.FeatureLevel, SetupTexture, BilinearBorderSampler, MipsConditionBuffer, Offset);
			}
		}

		// Allocate additional auxiliary buffers
		{
			const int32 Offset = ToIndex(bForceRunningInSeparable ? FSubsurfaceTiles::ETileType::AllNonPassthrough : FSubsurfaceTiles::ETileType::SEPARABLE);
			// Pre-allocate black UAV if we have separable tiles.
			SubsurfaceSubpassOneTex = CreateBlackUAVTexture(
				GraphBuilder, 
				SubsurfaceTextureDescriptor,
				TEXT("Subsurface.SubpassOneTex"),
				RDG_EVENT_NAME("SSS::ClearUAV(%s, %s.TileCount > 0)",TEXT("Subsurface.SubpassOneTex"), ToString(FSubsurfaceTiles::ETileType::SEPARABLE)),
				SubsurfaceViewport,
				Tiles.TileTypeCountBuffer,
				Offset);

			if (!bForceRunningInSeparable)
			{
				NewQualityHistoryTexture = GraphBuilder.CreateTexture(SubsurfaceTextureDescriptor, TEXT("Subsurface.QualityHistoryState"));
			}
		}

		// Major pass to combine Burley and Separable
		{
			typedef FSubsurfaceIndirectDispatchCS SHADER;

			FRHISamplerState* SubsurfaceSamplerState = SHADER::GetSamplerState(bHalfRes);

			struct FSubsurfacePassInfo
			{
				FSubsurfacePassInfo(const TCHAR* InName, FRDGTextureRef InInput, FRDGTextureRef InOutput,
					SHADER::ESubsurfaceType InSurfaceType, SHADER::ESubsurfacePass InSurfacePass)
					: Name(InName), Input(InInput), Output(InOutput), SurfaceType(InSurfaceType), SubsurfacePass(InSurfacePass)
				{}

				const TCHAR* Name;
				FRDGTextureRef Input;
				FRDGTextureRef Output;
				SHADER::ESubsurfaceType SurfaceType;
				SHADER::ESubsurfacePass SubsurfacePass;
			};

			const int NumOfSubsurfacePass = 4;

			const FSubsurfacePassInfo SubsurfacePassInfos[NumOfSubsurfacePass] =
			{
				{	TEXT("PassOne_Burley"),	SetupTexture, SubsurfaceSubpassOneTex, SHADER::ESubsurfaceType::BURLEY	 , SHADER::ESubsurfacePass::PassOne}, //Burley main pass
				{	TEXT("PassTwo_SepHon"),	SetupTexture, SubsurfaceSubpassOneTex, SHADER::ESubsurfaceType::SEPARABLE, SHADER::ESubsurfacePass::PassOne}, //Separable horizontal
				{ TEXT("PassThree_SepVer"),	SubsurfaceSubpassOneTex, SubsurfaceSubpassTwoTex, SHADER::ESubsurfaceType::SEPARABLE, SHADER::ESubsurfacePass::PassTwo}, //Separable Vertical
				{	 TEXT("PassFour_BVar"),	SubsurfaceSubpassOneTex, SubsurfaceSubpassTwoTex, SHADER::ESubsurfaceType::BURLEY	 , SHADER::ESubsurfacePass::PassTwo}  //Burley Variance
			};

			const FRDGBufferSRVRef SubsurfaceBufferUsage[] = { 
				Tiles.GetTileBufferSRV(FSubsurfaceTiles::ETileType::AFIS), 
				Tiles.GetTileBufferSRV(bForceRunningInSeparable ? FSubsurfaceTiles::ETileType::AllNonPassthrough : FSubsurfaceTiles::ETileType::SEPARABLE) };

			//Dispatch the two phase for both SSS
			for (int PassIndex = 0; PassIndex < NumOfSubsurfacePass; ++PassIndex)
			{
				const FSubsurfacePassInfo& PassInfo = SubsurfacePassInfos[PassIndex];

				const SHADER::ESubsurfaceType SubsurfaceType = PassInfo.SurfaceType;
				const auto SubsurfacePassFunction = PassInfo.SubsurfacePass;
				const int SubsurfaceTypeIndex = static_cast<int>(SubsurfaceType);
				FRDGTextureRef TextureInput = PassInfo.Input;
				FRDGTextureRef TextureOutput = PassInfo.Output;

				FRDGTextureUAVDesc SSSColorUAVDesc(TextureOutput, 0);
				FRDGTextureSRVDesc InputSRVDesc = FRDGTextureSRVDesc::Create(TextureInput);

				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				PassParameters->Subsurface = SubsurfaceCommonParameters;
				PassParameters->Output = SubsurfaceViewportParameters;
				PassParameters->SSSColorUAV = GraphBuilder.CreateUAV(SSSColorUAVDesc);
				PassParameters->SubsurfaceInput0 = GetSubsurfaceInput(TextureInput, SubsurfaceViewportParameters);
				PassParameters->SubsurfaceSampler0 = SubsurfaceSamplerState;
				PassParameters->GroupBuffer = SubsurfaceBufferUsage[SubsurfaceTypeIndex];
				PassParameters->TileTypeCountBuffer = GraphBuilder.CreateSRV(Tiles.TileTypeCountBuffer,EPixelFormat::PF_R32_UINT);
				PassParameters->IndirectDispatchArgsBuffer = Tiles.TileIndirectDispatchBuffer;
				PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

				if (SubsurfacePassFunction == SHADER::ESubsurfacePass::PassOne && SubsurfaceType == SHADER::ESubsurfaceType::BURLEY)
				{
					PassParameters->SubsurfaceInput1 = GetSubsurfaceInput(QualityHistoryTexture, SubsurfaceViewportParameters);
					PassParameters->SubsurfaceSampler1 = PointClampSampler;
					PassParameters->SubsurfaceInput2 = GetSubsurfaceInput(VelocityTexture, SubsurfaceViewportParameters);
					PassParameters->SubsurfaceSampler2 = PointClampSampler;
				}

				if (SubsurfacePassFunction == SHADER::ESubsurfacePass::PassTwo && SubsurfaceType == SHADER::ESubsurfaceType::BURLEY)
				{
					// we do not write to history in separable mode.
					if (!bForceRunningInSeparable)
					{
						PassParameters->HistoryUAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(NewQualityHistoryTexture, 0));
						PassParameters->SubsurfaceInput3 = GetSubsurfaceInput(SetupTexture, SubsurfaceViewportParameters);
						PassParameters->SubsurfaceSampler3 = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Border>::GetRHI();
					}

					PassParameters->SubsurfaceInput1 = GetSubsurfaceInput(QualityHistoryTexture, SubsurfaceViewportParameters);
					PassParameters->SubsurfaceSampler1 = PointClampSampler;
					PassParameters->SubsurfaceInput2 = GetSubsurfaceInput(VelocityTexture, SubsurfaceViewportParameters);
					PassParameters->SubsurfaceSampler2 = PointClampSampler;
				}

				if (bUseProfileIdCache)
				{
					PassParameters->ProfileIdTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(ProfileIdTexture));
				}

				SHADER::FPermutationDomain ComputeShaderPermutationVector;
				ComputeShaderPermutationVector.Set<SHADER::FSubsurfacePassFunction>(SubsurfacePassFunction);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionQuality>(SHADER::GetQuality());
                ComputeShaderPermutationVector.Set<SHADER::FBilateralFilterKernelFunctionType>(SHADER::GetBilateralFilterKernelFunctionType());
				ComputeShaderPermutationVector.Set<SHADER::FSubsurfaceType>(SubsurfaceType);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionHalfRes>(bHalfRes);
				ComputeShaderPermutationVector.Set<SHADER::FRunningInSeparable>(bForceRunningInSeparable);
				ComputeShaderPermutationVector.Set<SHADER::FDimensionEnableProfileIDCache>(bUseProfileIdCache);
				TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, ComputeShaderPermutationVector);

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SSS::%s(%s %s %s %s%s)",
						PassInfo.Name,
						SHADER::GetEventName(ComputeShaderPermutationVector.Get<SHADER::FSubsurfacePassFunction>()),
						SHADER::GetEventName(ComputeShaderPermutationVector.Get<SHADER::FDimensionQuality>()),
						SHADER::GetEventName(ComputeShaderPermutationVector.Get<SHADER::FSubsurfaceType>()),
						ComputeShaderPermutationVector.Get<SHADER::FDimensionHalfRes>() ? TEXT(" HalfRes") : TEXT(""),
						ComputeShaderPermutationVector.Get<SHADER::FRunningInSeparable>() ? TEXT(" RunningInSeparable") : TEXT("")),
					ComputeShader,
					PassParameters,
					PassParameters->IndirectDispatchArgsBuffer, 
					Tiles.GetIndirectDispatchArgOffset(SHADER::ToSubsurfaceTileType(SubsurfaceType,bForceRunningInSeparable)));
			}
		}
	}

	// Debug: print SSS reconstruction + recombine at cursor when in full res.
	const bool bDebugPixel = CVarSSSDebugPixel.GetValueOnRenderThread() != 0;
	if (ShaderPrint::IsEnabled() && ShaderPrint::IsValid(View.ShaderPrintData) 
		&& !bHalfRes && SubsurfaceMode != ESubsurfaceMode::Bypass && bDebugPixel)
	{
		ShaderPrint::RequestSpaceForCharacters(2048);
		
		typedef FSubsurfaceDebugCS SHADER;

		SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
		PassParameters->Subsurface = SubsurfaceCommonParameters;
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintUniformBuffer);
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->SubsurfaceInput0 = GetSubsurfaceInput(bReadSeparatedSubSurfaceSceneColor ? SeparatedSubSurfaceSceneColor : SceneColorTexture, SceneViewportParameters);
		PassParameters->SubsurfaceSampler0 = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		if (SubsurfaceSubpassTwoTex)
		{
			PassParameters->SubsurfaceInput1 = GetSubsurfaceInput(SubsurfaceSubpassTwoTex, SubsurfaceViewportParameters);
		}
		else
		{
			PassParameters->SubsurfaceInput1 = GetSubsurfaceInput(SystemTextures.Black, SubsurfaceViewportParameters);
		}

		PassParameters->SubsurfaceSampler1 = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Border>::GetRHI();

		SetupSubsurfaceDiffuseReadParameters(GraphBuilder,View,PassParameters->SubsurfaceDiffuseParameters);
		PassParameters->ViewResolutionFraction = View.ViewRect.Width() / (float)View.UnscaledViewRect.Width();

		const bool bColorFormatHasAlpha = bReadSeparatedSubSurfaceSceneColor ? false : 
			(SceneColorTexture->Desc.Format == PF_A32B32G32R32F || SceneColorTexture->Desc.Format == PF_FloatRGBA);
		PassParameters->SceneColorHasAlpha = bColorFormatHasAlpha ? 1 : 0;

		SHADER::FPermutationDomain PermutationVectors;
		PermutationVectors.Set<SHADER::FDimensionCheckerboard>(bCheckerboard);
		PermutationVectors.Set<SHADER::FDimensionQuality>(FSubsurfaceRecombinePS::GetQuality(View));
		TShaderMapRef<SHADER> ComputeShader(View.ShaderMap, PermutationVectors);

		FIntPoint ViewSize = View.ViewRect.Size();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("SSS::Debug"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(ViewSize, FIntPoint(8, 8)));
	}

	// Recombines scattering result with scene color, and updates only SSS region using specialized vertex shader.
	{
		FRDGTextureRef SubsurfaceIntermediateTexture = GraphBuilder.CreateTexture(SceneColorTextureDesc, TEXT("Subsurface.Recombines"));

		// When bypassing subsurface scattering, we use the directly full screen pass instead of building the tile
		// and apply the tile based copy
		const bool bShouldFallbackToFullScreenPass = SubsurfaceMode == ESubsurfaceMode::Bypass;
		FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		// Recombine into intermediate textures before copying back.
		{
			FSubsurfaceTiles::ETileType TileType = FSubsurfaceTiles::ETileType::All;

			FSubsurfaceRecombinePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubsurfaceRecombinePS::FParameters>();
			PassParameters->Subsurface = SubsurfaceCommonParameters;
			// Add dynamic branch parameters for recombine pass only.
			PassParameters->CheckerboardNeighborSSSValidation = FSubsurfaceRecombinePS::GetCheckerBoardNeighborSSSValidation(bCheckerboard);
			
			if (SubsurfaceMode != ESubsurfaceMode::Bypass)
			{
				PassParameters->TileParameters = GetSubsurfaceTileParameters(SubsurfaceViewport, Tiles, TileType);
			}
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SubsurfaceIntermediateTexture, SceneColorTextureLoadAction);
			PassParameters->SubsurfaceInput0 = GetSubsurfaceInput(bReadSeparatedSubSurfaceSceneColor ? SeparatedSubSurfaceSceneColor : SceneColorTexture, SceneViewportParameters);
			PassParameters->SubsurfaceSampler0 = BilinearBorderSampler;
			PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

			// Scattering output target is only used when scattering is enabled.
			if (SubsurfaceMode != ESubsurfaceMode::Bypass)
			{
				PassParameters->SubsurfaceInput1 = GetSubsurfaceInput(SubsurfaceSubpassTwoTex, SubsurfaceViewportParameters);
				PassParameters->SubsurfaceSampler1 = BilinearBorderSampler;
			}

			SetupSubsurfaceDiffuseReadParameters(GraphBuilder, View, PassParameters->SubsurfaceDiffuseParameters);

			const FSubsurfaceRecombinePS::EQuality RecombineQuality = FSubsurfaceRecombinePS::GetQuality(View);

			FSubsurfaceRecombinePS::FPermutationDomain PixelShaderPermutationVector;
			PixelShaderPermutationVector.Set<FSubsurfaceRecombinePS::FDimensionMode>(SubsurfaceMode);
			PixelShaderPermutationVector.Set<FSubsurfaceRecombinePS::FDimensionQuality>(RecombineQuality);
			PixelShaderPermutationVector.Set<FSubsurfaceRecombinePS::FDimensionCheckerboard>(bCheckerboard);
			PixelShaderPermutationVector.Set<FSubsurfaceRecombinePS::FDimensionHalfRes>(bHalfRes);
			PixelShaderPermutationVector.Set<FSubsurfaceRecombinePS::FRunningInSeparable>(bForceRunningInSeparable);

			TShaderMapRef<FSubsurfaceRecombinePS> PixelShader(View.ShaderMap, PixelShaderPermutationVector);
			TShaderMapRef<FSubsurfaceTilePassVS> VertexShader(View.ShaderMap);
			
			FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
			/**
				* See the related comment above in the prepare pass. The scene viewport is used as both the target and
				* texture viewport in order to ensure that the correct pixel is sampled for checkerboard rendering.
				*/
			AddSubsurfaceTiledScreenPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSS::Recombine(%s %s%s%s%s%s%s) %dx%d",
					GetEventName(PixelShaderPermutationVector.Get<FSubsurfaceRecombinePS::FDimensionMode>()),
					FSubsurfaceRecombinePS::GetEventName(PixelShaderPermutationVector.Get<FSubsurfaceRecombinePS::FDimensionQuality>()),
					PixelShaderPermutationVector.Get<FSubsurfaceRecombinePS::FDimensionCheckerboard>() ? TEXT(" Checkerboard") : TEXT(""),
					FSubsurfaceRecombinePS::GetCheckerBoardNeighborSSSValidation(bCheckerboard) ? TEXT("-Validation") : TEXT(""),
					PixelShaderPermutationVector.Get<FSubsurfaceRecombinePS::FDimensionHalfRes>() ? TEXT(" HalfRes") : TEXT(""),
					PixelShaderPermutationVector.Get<FSubsurfaceRecombinePS::FRunningInSeparable>() ? TEXT(" RunningInSeparable") : TEXT(""),
					!bShouldFallbackToFullScreenPass ? TEXT(" Tiled") : TEXT(""),
					View.ViewRect.Width(),
					View.ViewRect.Height()),
				View,
				PassParameters,
				VertexShader,
				PixelShader,
				BlendState,
				DepthStencilState,
				SceneViewport,
				TileType,
				bShouldFallbackToFullScreenPass);
		}

		//Write back to the SceneColor texture 
		{
			FSubsurfaceTiles::ETileType TileType = FSubsurfaceTiles::ETileType::All;

			FSubsurfaceRecombineCopyPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubsurfaceRecombineCopyPS::FParameters>();
			PassParameters->Subsurface = SubsurfaceCommonParameters;
			if (SubsurfaceMode != ESubsurfaceMode::Bypass)
			{
				PassParameters->TileParameters = GetSubsurfaceTileParameters(SubsurfaceViewport, Tiles, TileType);
			}
			PassParameters->RenderTargets[0] = bWriteSeparatedOpaqueRoughRefractionSceneColor ? 
				FRenderTargetBinding(SeparatedOpaqueRoughRefractionSceneColor, ERenderTargetLoadAction::ELoad) :
				FRenderTargetBinding(SceneColorTexture, SceneColorTextureLoadAction);
			PassParameters->SubsurfaceInput0 = GetSubsurfaceInput(SubsurfaceIntermediateTexture, SceneViewportParameters);
			PassParameters->SubsurfaceSampler0 = PointClampSampler;
			PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

			TShaderMapRef<FSubsurfaceRecombineCopyPS> PixelShader(View.ShaderMap);			
			TShaderMapRef<FSubsurfaceTilePassVS> VertexShader(View.ShaderMap);

			FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
			if (bWriteSeparatedOpaqueRoughRefractionSceneColor)
			{
				BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_One, BF_One>::GetRHI();
			}
			
			AddSubsurfaceTiledScreenPass(
				GraphBuilder,
				RDG_EVENT_NAME("SSS::CopyToSceneColor%s %dx%d",
					!bShouldFallbackToFullScreenPass ? TEXT("( Tiled)") : TEXT(""),
					View.ViewRect.Width(),
					View.ViewRect.Height()),
				View,
				PassParameters,
				VertexShader,
				PixelShader,
				BlendState,
				DepthStencilState,
				SceneViewport,
				TileType,
				bShouldFallbackToFullScreenPass);
		}
	}

	if (SubsurfaceMode != ESubsurfaceMode::Bypass && QualityHistoryState && !bForceRunningInSeparable)
	{
		GraphBuilder.QueueTextureExtraction(NewQualityHistoryTexture, QualityHistoryState);
	}
}

FRDGTextureRef AddSubsurfacePass(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const TArray<FSubsurfaceTiles>& PrebuiltTiles,
	const uint32 ViewMask,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef SceneColorTextureOutput)
{
	const uint32 ViewCount = Views.Num();
	const uint32 ViewMaskAll = (1 << ViewCount) - 1;
	check(ViewMask);

	FRDGTextureRef SceneColorTexture = SceneTextures.Color.Target;
	ERenderTargetLoadAction SceneColorTextureLoadAction = ERenderTargetLoadAction::ELoad;

	for (uint32 ViewIndex = 0; ViewIndex < ViewCount; ++ViewIndex)
	{
		const uint32 ViewBit = 1 << ViewIndex;

		const bool bIsSubsurfaceView = (ViewMask & ViewBit) != 0;

		if (bIsSubsurfaceView)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "SubsurfaceScattering(ViewId=%d)", ViewIndex);

			const FViewInfo& View = Views[ViewIndex];
			const bool bHasValidTile = (ViewIndex < (uint32)PrebuiltTiles.Num()) && PrebuiltTiles[ViewIndex].IsValid();
			const FSubsurfaceTiles* ViewTiles = bHasValidTile ? &PrebuiltTiles[ViewIndex] : nullptr;

			AddSubsurfaceViewPass(GraphBuilder, View, ViewTiles, SceneTextures, SceneColorTextureOutput, SceneColorTextureLoadAction);
			SceneColorTextureLoadAction = ERenderTargetLoadAction::ELoad;
		}
	}

	return SceneColorTextureOutput;
}

FScreenPassTexture AddVisualizeSubsurfacePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeSubsurfaceInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeSubsurface"));
	}

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);

	FSubsurfaceVisualizePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSubsurfaceVisualizePS::FParameters>();
	PassParameters->Subsurface = GetSubsurfaceCommonParameters(GraphBuilder, View, Inputs.SceneTextures);
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->SubsurfaceInput0.Texture = Inputs.SceneColor.Texture;
	PassParameters->SubsurfaceInput0.Viewport = GetScreenPassTextureViewportParameters(InputViewport);
	PassParameters->SubsurfaceSampler0 = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->MiniFontTexture = GetMiniFontTexture();
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

	TShaderMapRef<FSubsurfaceVisualizePS> PixelShader(View.ShaderMap);

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeSubsurface");

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("SSS::Visualizer"), View, FScreenPassTextureViewport(Output), InputViewport, PixelShader, PassParameters);

	Output.LoadAction = ERenderTargetLoadAction::ELoad;

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("SSS::Text"), View, Output, [](FCanvas& Canvas)
	{
		float X = 30;
		float Y = 28;
		const float YStep = 14;

		FString Line = FString::Printf(TEXT("Visualize Screen Space Subsurface Scattering"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));

		Y += YStep;

		uint32 Index = 0;
		while (GSubsurfaceProfileTextureObject.GetEntryString(Index++, Line))
		{
			Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		}
	});

	return MoveTemp(Output);
}

void AddSubsurfacePass(
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	TArrayView<const FViewInfo> Views,
	const TArray<FSubsurfaceTiles>& PrebuiltTiles)
{
	const uint32 ViewMask = GetSubsurfaceRequiredViewMask(Views);

	if (!ViewMask)
	{
		return;
	}

	checkf(!SceneTextures.Color.IsSeparate(), TEXT("Subsurface rendering requires the deferred renderer."));

	AddSubsurfacePass(GraphBuilder, Views, PrebuiltTiles, ViewMask, SceneTextures, SceneTextures.Color.Target);
	SceneTextures.Color.Resolve = SceneTextures.Color.Target;
}
