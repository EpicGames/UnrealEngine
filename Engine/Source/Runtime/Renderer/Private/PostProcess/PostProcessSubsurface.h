// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "OverridePassSequence.h"
#include "SubsurfaceTiles.h"

// Returns whether subsurface scattering is globally enabled.
bool IsSubsurfaceEnabled(const FEngineShowFlags& EngineShowFlags);

// Returns whether subsurface scattering is required for the provided view.
bool IsSubsurfaceRequiredForView(const FViewInfo& View);

// Returns whether checkerboard rendering is enabled for the provided format.
bool IsSubsurfaceCheckerboardFormat(EPixelFormat SceneColorFormat, const FViewInfo& View);

bool UseSeparatedSubsurfaceDiffuse(EShaderPlatform ShaderPlatform);

// Common headers to allow separated separated subsurface diffuse
BEGIN_SHADER_PARAMETER_STRUCT(FSubsurfaceDiffuseWriteParameters, )
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWSeparatedSubsurfaceDiffuseLuminance)
	SHADER_PARAMETER(uint32, bWriteSeparatedSubsurfaceDiffuse)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSubsurfaceDiffuseReadParameters, )
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, SeparatedSubsurfaceDiffuseLuminance)
	SHADER_PARAMETER(uint32, bUseSeparatedSubsurfaceDiffuse)
END_SHADER_PARAMETER_STRUCT()

void SetupSubsurfaceDiffuseWriteParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSubsurfaceDiffuseWriteParameters& OutParameters);

void SetupSubsurfaceDiffuseReadParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSubsurfaceDiffuseReadParameters& OutParameters);

void RenderSubsurfaceDiffuseInitPass(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FSceneTextures& SceneTextures,
	TArray<FSubsurfaceTiles>& OutPerViewTiles);

class FSceneTextureParameters;

struct FVisualizeSubsurfaceInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The scene color to composite with the visualization.
	FScreenPassTexture SceneColor;

	// [Required] The scene textures used to visualize shading models.
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures = nullptr;
};

FScreenPassTexture AddVisualizeSubsurfacePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeSubsurfaceInputs& Inputs);

void AddSubsurfacePass(
	FRDGBuilder& GraphBuilder,
	FSceneTextures& SceneTextures,
	TArrayView<const FViewInfo> Views,
	const TArray<FSubsurfaceTiles>& PrebuiltTiles);