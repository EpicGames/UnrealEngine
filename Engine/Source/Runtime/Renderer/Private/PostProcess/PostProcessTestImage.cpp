// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessTestImage.h"
#include "PostProcess/PostProcessCombineLUTs.h"
#include "CanvasTypes.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"
#include "UnrealEngine.h"

class FTestImagePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTestImagePS);
	SHADER_USE_PARAMETER_STRUCT(FTestImagePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_STRUCT_INCLUDE(FColorRemapParameters, ColorRemap)
		SHADER_PARAMETER(uint32, FrameNumber)
		SHADER_PARAMETER(float, FrameTime)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FTestImagePS, "/Engine/Private/PostProcessTestImage.usf", "MainPS", SF_Pixel);

void AddTestImagePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture Output)
{
	check(Output.IsValid());

	RDG_EVENT_SCOPE(GraphBuilder, "TestImage");

	FRDGTextureRef OutputTexture = Output.Texture;
	const FScreenPassTextureViewport Viewport(Output);
	const FSceneViewFamily& ViewFamily = *(View.Family);

	FTestImagePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FTestImagePS::FParameters>();
	PassParameters->Output = GetScreenPassTextureViewportParameters(Viewport);
	PassParameters->ColorRemap = GetColorRemapParameters();
	PassParameters->FrameNumber = ViewFamily.FrameNumber;
	PassParameters->FrameTime = ViewFamily.Time.GetRealTimeSeconds();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::EClear);

	TShaderMapRef<FTestImagePS> PixelShader(View.ShaderMap);

	AddDrawScreenPass(GraphBuilder, {}, View, Viewport, Viewport, PixelShader, PassParameters);

	FScreenPassRenderTarget CanvasOutput(Output, ERenderTargetLoadAction::ELoad);
	AddDrawCanvasPass(GraphBuilder, {}, View, FScreenPassRenderTarget(Output, ERenderTargetLoadAction::ELoad),
		[](FCanvas& Canvas)
	{
		float X = 30;
		float Y = 8;
		const float YStep = 14;
		const float ColumnWidth = 250;

		FString Line;

		Line = FString::Printf(TEXT("Top bars:"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   Moving bars using FrameTime"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   Black and white raster, Pixel sized, Watch for Moire pattern"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   Black and white raster, 2x2 block sized"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("Bottom bars:"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   8 bars near white, 4 right bars should appear as one (HDTV)"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   8 bars near black, 4 left bars should appear as one (HDTV)"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   Linear Greyscale in sRGB from 0 to 255"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("Color bars:"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   Red, Green, Blue"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("Outside:"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   Moving bars using FrameNumber, Tearing without VSync"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("Circles:"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   Should be round and centered"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("Border:"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
		Line = FString::Printf(TEXT("   4 white pixel sized lines (only visible without overscan)"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));
	});
}