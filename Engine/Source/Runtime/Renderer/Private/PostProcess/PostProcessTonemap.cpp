// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PostProcessTonemap.cpp: Post processing tone mapping implementation.
=============================================================================*/

#include "PostProcess/PostProcessTonemap.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "EngineGlobals.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessCombineLUTs.h"
#include "PostProcess/PostProcessLocalExposure.h"
#include "PostProcess/PostProcessMobile.h"
#include "PostProcess/PostProcessing.h"
#include "ClearQuad.h"
#include "PipelineStateCache.h"
#include "Rendering/Texture2DResource.h"
#include "Math/Halton.h"
#include "SystemTextures.h"
#include "HDRHelper.h"
#include "VariableRateShadingImageManager.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "CommonRenderResources.h"
#include "SceneViewState.h"

bool SupportsFilmGrain(EShaderPlatform Platform)
{
	return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5);
}

namespace
{
TAutoConsoleVariable<float> CVarTonemapperSharpen(
	TEXT("r.Tonemapper.Sharpen"),
	-1,
	TEXT("Sharpening in the tonemapper (not for mobile), actual implementation is work in progress, clamped at 10\n")
	TEXT("  <0: inherit from PostProcessVolume settings (default)\n")
	TEXT("   0: off\n")
	TEXT(" 0.5: half strength\n")
	TEXT("   1: full strength"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarTonemapperGamma(
	TEXT("r.TonemapperGamma"),
	0.0f,
	TEXT("0: Default behavior\n")
	TEXT("#: Use fixed gamma # instead of sRGB or Rec709 transform"),
	ECVF_Scalability | ECVF_RenderThreadSafe);	

TAutoConsoleVariable<float> CVarGamma(
	TEXT("r.Gamma"),
	1.0f,
	TEXT("Gamma on output"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarFilmGrainSequenceLength(
	TEXT("r.FilmGrain.SequenceLength"), 97,
	TEXT("Length of the random sequence for film grain (preferably a prime number, default=97)."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarFilmGrainCacheTextureConstants(
	TEXT("r.FilmGrain.CacheTextureConstants"), 1,
	TEXT("Wether the constants related to the film grain should be cached."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarBackbufferQuantizationDitheringOverride(
	TEXT("r.BackbufferQuantizationDitheringOverride"), 0,
	TEXT("Override the bitdepth in bits of each channel of the backbuffer targeted by the quantization dithering. ")
	TEXT("Disabled by default. Instead is automatically found out by FSceneViewFamily::RenderTarget's pixel format of the backbuffer."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarLUTShaper(
	TEXT("r.LUT.Shaper"), 0,
	TEXT("Which shaper to use for addressing the LUT\n")
	TEXT(" 0: Log2 (fast)\n")
	TEXT(" 1: PQ"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

const int32 GTonemapComputeTileSizeX = 8;
const int32 GTonemapComputeTileSizeY = 8;

struct FOutputLuminance
{
	FRDGTextureRef Texture = nullptr;
};

namespace TonemapperPermutation
{
// Shared permutation dimensions between deferred and mobile renderer.
class FTonemapperGammaOnlyDim      		: SHADER_PERMUTATION_BOOL("USE_GAMMA_ONLY");
class FTonemapperLocalExposureDim  		: SHADER_PERMUTATION_INT("LOCAL_EXPOSURE_MODE", 3);
class FTonemapperSharpenDim        		: SHADER_PERMUTATION_BOOL("USE_SHARPEN");
class FTonemapperFilmGrainDim      		: SHADER_PERMUTATION_BOOL("USE_FILM_GRAIN");
class FTonemapperAdvancedVignetteDim	: SHADER_PERMUTATION_BOOL("USE_ADVANCED_VIGNETTE");
class FTonemapperVignetteTextureDim		: SHADER_PERMUTATION_BOOL("USE_VIGNETTE_TEXTURE");
class FTonemapperMsaaDim           		: SHADER_PERMUTATION_BOOL("METAL_MSAA_HDR_DECODE");

using FCommonDomain = TShaderPermutationDomain<
	FTonemapperGammaOnlyDim,
	FTonemapperLocalExposureDim,
	FTonemapperSharpenDim,
	FTonemapperFilmGrainDim,
	FTonemapperAdvancedVignetteDim,
	FTonemapperVignetteTextureDim,
	FTonemapperMsaaDim>;

bool ShouldCompileCommonPermutation(const FGlobalShaderPermutationParameters& Parameters, const FCommonDomain& PermutationVector)
{
	// MSAA pre-resolve step only used on iOS atm
	if (PermutationVector.Get<FTonemapperMsaaDim>() && !IsMetalMobilePlatform(Parameters.Platform))
	{
		return false;
	}

	// If GammaOnly, don't compile any other dimmension == true.
	if (PermutationVector.Get<FTonemapperGammaOnlyDim>())
	{
		return !PermutationVector.Get<FTonemapperLocalExposureDim>() &&
			!PermutationVector.Get<FTonemapperSharpenDim>() &&
			!PermutationVector.Get<FTonemapperFilmGrainDim>() &&
			!PermutationVector.Get<FTonemapperAdvancedVignetteDim>() &&
			!PermutationVector.Get<FTonemapperVignetteTextureDim>() &&
			!PermutationVector.Get<FTonemapperMsaaDim>();
	}

	return true;
}

static float GetSharpenSetting(const FPostProcessSettings& Settings)
{
	float CVarSharpen = CVarTonemapperSharpen.GetValueOnRenderThread();
	float Sharpen = CVarSharpen >= 0.0 ? CVarSharpen : Settings.Sharpen;
	return FMath::Clamp(Sharpen, 0.0f, 10.0f);
}

// Common conversion of engine settings into.
FCommonDomain BuildCommonPermutationDomain(const FViewInfo& View, bool bGammaOnly, bool bLocalExposure, bool bMetalMSAAHDRDecode)
{
	const FSceneViewFamily* Family = View.Family;

	FCommonDomain PermutationVector;

	// Gamma
	if (bGammaOnly ||
		(Family->EngineShowFlags.Tonemapper == 0) ||
		(Family->EngineShowFlags.PostProcessing == 0))
	{
		PermutationVector.Set<FTonemapperGammaOnlyDim>(true);
		return PermutationVector;
	}

	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;
	PermutationVector.Set<FTonemapperFilmGrainDim>(View.FinalPostProcessSettings.FilmGrainIntensity > 0.0f && View.FilmGrainTexture != nullptr);
	PermutationVector.Set<FTonemapperSharpenDim>(GetSharpenSetting(Settings) > 0.0f);
	PermutationVector.Set<FTonemapperMsaaDim>(bMetalMSAAHDRDecode);
	PermutationVector.Set<FTonemapperAdvancedVignetteDim>(View.FinalPostProcessSettings.VignetteType != EVignetteType::CosineFourthLaw);
	PermutationVector.Set<FTonemapperVignetteTextureDim>(View.FinalPostProcessSettings.VignetteType == EVignetteType::Texture);

	if (bLocalExposure)
	{
		PermutationVector.Set<FTonemapperLocalExposureDim>(View.FinalPostProcessSettings.LocalExposureMethod == ELocalExposureMethod::Bilateral ? 1 : 2);
	}
	else
	{
		PermutationVector.Set<FTonemapperLocalExposureDim>(0);
	}

	return PermutationVector;
}

// Desktop renderer permutation dimensions.
class FTonemapperColorFringeDim       : SHADER_PERMUTATION_BOOL("USE_COLOR_FRINGE");
class FTonemapperOutputDeviceSRGB     : SHADER_PERMUTATION_BOOL("OUTPUT_DEVICE_SRGB");
class FTonemapperOutputLuminance	  : SHADER_PERMUTATION_BOOL("OUTPUT_LUMINANCE");

using FDesktopDomain = TShaderPermutationDomain<
	FCommonDomain,
	FTonemapperColorFringeDim,
	FTonemapperOutputLuminance,
	FTonemapperOutputDeviceSRGB>;

FDesktopDomain RemapPermutation(FDesktopDomain PermutationVector, ERHIFeatureLevel::Type FeatureLevel)
{
	FCommonDomain CommonPermutationVector = PermutationVector.Get<FCommonDomain>();

	// No remapping if gamma only.
	if (CommonPermutationVector.Get<FTonemapperGammaOnlyDim>())
	{
		return PermutationVector;
	}

	bool bFallbackToSlowest = CommonPermutationVector.Get<FTonemapperFilmGrainDim>() || CommonPermutationVector.Get<FTonemapperVignetteTextureDim>();
	if (bFallbackToSlowest)
	{
		CommonPermutationVector.Set<FTonemapperFilmGrainDim>(true);
		CommonPermutationVector.Set<FTonemapperVignetteTextureDim>(true);
		CommonPermutationVector.Set<FTonemapperAdvancedVignetteDim>(true);
		CommonPermutationVector.Set<FTonemapperSharpenDim>(true);

		PermutationVector.Set<FTonemapperColorFringeDim>(true);
	}

	if (FeatureLevel <= ERHIFeatureLevel::SM5)
	{
		CommonPermutationVector.Set<FTonemapperVignetteTextureDim>(false);
	}

	if (FeatureLevel < ERHIFeatureLevel::SM5)
	{
		// Mobile doesn't support film grain.
		CommonPermutationVector.Set<FTonemapperFilmGrainDim>(false);
	}

	PermutationVector.Set<FCommonDomain>(CommonPermutationVector);

	return PermutationVector;
}

bool ShouldCompileDesktopPermutation(const FGlobalShaderPermutationParameters& Parameters, FDesktopDomain PermutationVector)
{
	auto CommonPermutationVector = PermutationVector.Get<FCommonDomain>();

	if (RemapPermutation(PermutationVector, GetMaxSupportedFeatureLevel(Parameters.Platform)) != PermutationVector)
	{
		return false;
	}

	if (!ShouldCompileCommonPermutation(Parameters, CommonPermutationVector))
	{
		return false;
	}

	if (CommonPermutationVector.Get<FTonemapperGammaOnlyDim>())
	{
		return !PermutationVector.Get<FTonemapperColorFringeDim>();
	}
	return true;
}

} // namespace TonemapperPermutation
} // namespace

RDG_REGISTER_BLACKBOARD_STRUCT(FOutputLuminance);

FTonemapperOutputDeviceParameters GetTonemapperOutputDeviceParameters(const FSceneViewFamily& Family)
{
	EDisplayOutputFormat OutputDeviceValue;
	bool bIsHDR = false;

	if (Family.SceneCaptureSource == SCS_FinalColorHDR)
	{
		OutputDeviceValue = EDisplayOutputFormat::HDR_LinearNoToneCurve;
	}
	else if (Family.SceneCaptureSource == SCS_FinalToneCurveHDR)
	{
		OutputDeviceValue = EDisplayOutputFormat::HDR_LinearWithToneCurve;
	}
	else
	{
		OutputDeviceValue = Family.RenderTarget->GetDisplayOutputFormat();
		bIsHDR = Family.RenderTarget->GetSceneHDREnabled();
	}

	float Gamma = CVarTonemapperGamma.GetValueOnRenderThread();

	// In case gamma is unspecified, fall back to 2.2 which is the most common case
	if ((PLATFORM_APPLE || OutputDeviceValue == EDisplayOutputFormat::SDR_ExplicitGammaMapping) && Gamma == 0.0f)
	{
		Gamma = 2.2f;
	}

	// Enforce user-controlled ramp over sRGB or Rec709
	if (Gamma > 0.0f && (OutputDeviceValue == EDisplayOutputFormat::SDR_sRGB || OutputDeviceValue == EDisplayOutputFormat::SDR_Rec709))
	{
		OutputDeviceValue = EDisplayOutputFormat::SDR_ExplicitGammaMapping;
	}

	FVector3f InvDisplayGammaValue;
	InvDisplayGammaValue.X = 1.0f / Family.RenderTarget->GetDisplayGamma();
	InvDisplayGammaValue.Y = (OutputDeviceValue == EDisplayOutputFormat::HDR_LinearWithToneCurve) ? 1.f : (2.2f / Family.RenderTarget->GetDisplayGamma());
	InvDisplayGammaValue.Z = 1.0f / FMath::Max(Gamma, 1.0f);

	FTonemapperOutputDeviceParameters Parameters;
	Parameters.InverseGamma = InvDisplayGammaValue;
	Parameters.OutputDevice = static_cast<uint32>(OutputDeviceValue);
	Parameters.OutputGamut = static_cast<uint32>(Family.RenderTarget->GetDisplayColorGamut());
	static const IConsoleVariable* const CVarHDRDisplayOverrideOSMaxLuminance = IConsoleManager::Get().FindConsoleVariable(TEXT("r.HDR.Display.OverrideOSMaxLuminance"));
	Parameters.OutputMaxLuminance = CVarHDRDisplayOverrideOSMaxLuminance->GetBool() ?
		(bIsHDR ? HDRGetDisplayMaximumLuminance() : 100.f) :
		Family.RenderTarget->GetMaximumLuminanceInNits();
	Parameters.LUTShaper = CVarLUTShaper.GetValueOnRenderThread();
	return Parameters;
}

BEGIN_SHADER_PARAMETER_STRUCT(FFilmGrainParameters, )
	SHADER_PARAMETER(FVector3f, GrainRandomFull)
	SHADER_PARAMETER(float, FilmGrainIntensityShadows)
	SHADER_PARAMETER(float, FilmGrainIntensityMidtones)
	SHADER_PARAMETER(float, FilmGrainIntensityHighlights)
	SHADER_PARAMETER(float, FilmGrainShadowsMax)
	SHADER_PARAMETER(float, FilmGrainHighlightsMin)
	SHADER_PARAMETER(float, FilmGrainHighlightsMax)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FilmGrainTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, FilmGrainSampler)
	SHADER_PARAMETER(FScreenTransform, ScreenPosToFilmGrainTextureUV)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, FilmGrainTextureConstants)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FTonemapParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FFilmGrainParameters, FilmGrain)
	SHADER_PARAMETER_STRUCT_INCLUDE(FTonemapperOutputDeviceParameters, OutputDevice)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
	SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
	SHADER_PARAMETER_STRUCT(FLocalExposureParameters, LocalExposure)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, ColorTexture)

	// Parameters to apply to the scene color.
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, SceneColorApplyParamaters)

	// Bloom texture
	SHADER_PARAMETER(FScreenTransform, ColorToBloom)
	SHADER_PARAMETER(FVector2f, BloomUVViewportBilinearMin)
	SHADER_PARAMETER(FVector2f, BloomUVViewportBilinearMax)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BloomTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, BloomSampler)

	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, LumBilateralGrid)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BlurredLogLum)
	SHADER_PARAMETER_SAMPLER(SamplerState, LumBilateralGridSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, BlurredLogLumSampler)

	SHADER_PARAMETER(FScreenTransform, ColorToExposureFusion)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, ExposureFusion)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, ExposureFusionTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ExposureFusionSampler)

	SHADER_PARAMETER(float, FilmSlope)
	SHADER_PARAMETER(float, FilmToe)
	SHADER_PARAMETER(float, FilmShoulder)
	SHADER_PARAMETER(float, FilmBlackClip)
	SHADER_PARAMETER(float, FilmWhiteClip)

	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
	SHADER_PARAMETER_RDG_TEXTURE(Texture3D, ColorGradingLUT)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BloomDirtMaskTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, ColorGradingLUTSampler)
	SHADER_PARAMETER_SAMPLER(SamplerState, BloomDirtMaskSampler)
	SHADER_PARAMETER(FVector4f, ColorScale0)
	SHADER_PARAMETER(FVector4f, BloomDirtMaskTint)
	SHADER_PARAMETER(FVector4f, ChromaticAberrationParams)
	SHADER_PARAMETER(FVector4f, LensPrincipalPointOffsetScale)
	SHADER_PARAMETER(FVector4f, LensPrincipalPointOffsetScaleInverse)
	SHADER_PARAMETER(uint32, Flags)
	SHADER_PARAMETER(float, SharpenDiv6)
	SHADER_PARAMETER(float, LUTMax)
	SHADER_PARAMETER(float, LUTSize)
	SHADER_PARAMETER(float, InvLUTSize)
	SHADER_PARAMETER(float, LUTScale)
	SHADER_PARAMETER(float, LUTOffset)
	SHADER_PARAMETER(float, EditorNITLevel)
	SHADER_PARAMETER(float, BackbufferQuantizationDithering)

	SHADER_PARAMETER(FVector4f, VignetteParams0)
	SHADER_PARAMETER(FVector4f, VignetteParams1)
	SHADER_PARAMETER(float, VignetteParams2)
	SHADER_PARAMETER(uint32, VignetteType)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VignetteTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, VignetteTextureSampler)
END_SHADER_PARAMETER_STRUCT()

class FFilmGrainReduceCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FFilmGrainReduceCS);
	SHADER_USE_PARAMETER_STRUCT(FFilmGrainReduceCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, FilmGrainTextureSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, FilmGrainTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Output)
	END_SHADER_PARAMETER_STRUCT()
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return SupportsFilmGrain(Parameters.Platform);
	}
};

class FFilmGrainPackConstantsCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FFilmGrainPackConstantsCS);
	SHADER_USE_PARAMETER_STRUCT(FFilmGrainPackConstantsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, OriginalFilmGrainTextureSize)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ReducedFilmGrainTexture)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<float4>, FilmGrainConstantsOutput)
	END_SHADER_PARAMETER_STRUCT()
	
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return SupportsFilmGrain(Parameters.Platform);
	}
};

enum class ETonemapperFlags : uint32
{
	AlphaChannel = 1 << 0
};

static void ModifyTonemapCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	OutEnvironment.SetDefine(TEXT("SUPPORTS_SCENE_COLOR_APPLY_PARAMETERS"), FTonemapInputs::SupportsSceneColorApplyParametersBuffer(Parameters.Platform) ? 1 : 0);
	OutEnvironment.SetDefine(TEXT("SUPPORTS_FILM_GRAIN"), SupportsFilmGrain(Parameters.Platform) ? 1 : 0);
	OutEnvironment.SetDefine(TEXT("FLAG_ALPHA_CHANNEL"), static_cast<uint32>(ETonemapperFlags::AlphaChannel));
}

class FTonemapVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTonemapVS);

	// FDrawRectangleParameters is filled by DrawScreenPass.
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FTonemapVS, FGlobalShader);

	using FParameters = FTonemapParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		ModifyTonemapCompilationEnvironment(Parameters, OutEnvironment);
	}
};

class FTonemapPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTonemapPS);
	SHADER_USE_PARAMETER_STRUCT(FTonemapPS, FGlobalShader);

	using FPermutationDomain = TonemapperPermutation::FDesktopDomain;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FTonemapParameters, Tonemap)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::ES3_1))
		{
			return false;
		}
		return TonemapperPermutation::ShouldCompileDesktopPermutation(Parameters, FPermutationDomain(Parameters.PermutationId));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		ModifyTonemapCompilationEnvironment(Parameters, OutEnvironment);
	}
};

class FTonemapCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FTonemapCS);
	SHADER_USE_PARAMETER_STRUCT(FTonemapCS, FGlobalShader);

	using FPermutationDomain = TonemapperPermutation::FDesktopDomain;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FTonemapParameters, Tonemap)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWOutputTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWOutputLuminance)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		if (!IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
		{
			return false;
		}

		FPermutationDomain PermutationVector(Parameters.PermutationId);

		return TonemapperPermutation::ShouldCompileDesktopPermutation(Parameters, PermutationVector);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GTonemapComputeTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GTonemapComputeTileSizeY);

		ModifyTonemapCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFilmGrainReduceCS,        "/Engine/Private/PostProcessing/FilmGrainReduce.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FFilmGrainPackConstantsCS, "/Engine/Private/PostProcessing/FilmGrainPackConstants.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FTonemapVS, "/Engine/Private/PostProcessTonemap.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FTonemapPS, "/Engine/Private/PostProcessTonemap.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FTonemapCS, "/Engine/Private/PostProcessTonemap.usf", "MainCS", SF_Compute);

bool FTonemapInputs::SupportsSceneColorApplyParametersBuffer(EShaderPlatform Platform)
{
	return FDataDrivenShaderPlatformInfo::GetSupportsFFTBloom(Platform);
}

static
FRDGBufferRef BuildFilmGrainConstants(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef FilmGrainTexture)
{
	RDG_EVENT_SCOPE(GraphBuilder, "FilmGrain BuildTextureConstants");

	FRDGTextureRef ReducedFilmGrainTexture = FilmGrainTexture;

	for (int32 PassId = 0; ReducedFilmGrainTexture->Desc.Extent.X > 1 && ReducedFilmGrainTexture->Desc.Extent.Y > 1; PassId++)
	{
		FRDGTextureRef NewReducedFilmGrainTexture;
		{
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				FIntPoint::DivideAndRoundUp(ReducedFilmGrainTexture->Desc.Extent, 8),
				PF_A32B32G32R32F,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_UAV);

			NewReducedFilmGrainTexture = GraphBuilder.CreateTexture(Desc, TEXT("FilmGrain.Reduce"));
		}

		FFilmGrainReduceCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFilmGrainReduceCS::FParameters>();
		PassParameters->FilmGrainTextureSize = ReducedFilmGrainTexture->Desc.Extent;
		PassParameters->FilmGrainTexture = ReducedFilmGrainTexture;
		PassParameters->Output = GraphBuilder.CreateUAV(NewReducedFilmGrainTexture);

		TShaderMapRef<FFilmGrainReduceCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FilmGrain Reduce %dx%d -> %dx%d",
				ReducedFilmGrainTexture->Desc.Extent.X, ReducedFilmGrainTexture->Desc.Extent.Y,
				NewReducedFilmGrainTexture->Desc.Extent.X, NewReducedFilmGrainTexture->Desc.Extent.Y),
			ComputeShader,
			PassParameters,
			FIntVector(NewReducedFilmGrainTexture->Desc.Extent.X, NewReducedFilmGrainTexture->Desc.Extent.Y, 1));

		ReducedFilmGrainTexture = NewReducedFilmGrainTexture;
	}

	FRDGBufferRef FilmGrainConstantsBuffer;
	{
		FilmGrainConstantsBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(sizeof(FLinearColor), /* NumElements = */ 1),
			TEXT("FilmGrain.TextureConstants"));

		FFilmGrainPackConstantsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFilmGrainPackConstantsCS::FParameters>();
		PassParameters->OriginalFilmGrainTextureSize = FilmGrainTexture->Desc.Extent;
		PassParameters->ReducedFilmGrainTexture = ReducedFilmGrainTexture;
		PassParameters->FilmGrainConstantsOutput = GraphBuilder.CreateUAV(FilmGrainConstantsBuffer);

		TShaderMapRef<FFilmGrainPackConstantsCS> ComputeShader(View.ShaderMap);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FilmGrain PackConstants"),
			ComputeShader,
			PassParameters,
			FIntVector(1, 1, 1));
	}

	return FilmGrainConstantsBuffer;
}

bool ShouldWriteAlphaChannel(const FViewInfo& View, const FTonemapInputs& Inputs, EPixelFormat Format)
{
	// If this is a stereo view, there's a good chance we need alpha out of the tonemapper
	// @todo: Remove this once Oculus fix the bug in their runtime that requires alpha here.
	const bool bIsStereo = IStereoRendering::IsStereoEyeView(View);
	const bool bFormatNeedsAlphaWrite = Format == PF_R9G9B9EXP5;
	const bool bAlphaIsOnlyTwoBitsSoAvoidClear = Format == PF_A2B10G10R10;
	return (Inputs.bWriteAlphaChannel || bIsStereo || bFormatNeedsAlphaWrite || bAlphaIsOnlyTwoBitsSoAvoidClear);
}

bool ShouldOverrideOutputLoadActionToClear(const FScreenPassRenderTarget& Output, bool bShouldWriteAlphaChannel)
{
	bool bShouldOverride = false;

	// If the load action is already load, there should be content in the texture. Disable clear overriding.
	if (Output.LoadAction == ERenderTargetLoadAction::ENoAction)
	{
		EPixelFormatChannelFlags OutputFlags = GetPixelFormatValidChannels(Output.Texture->Desc.Format);
		// If we do not write through alpha channel but the output texture has alpha channel
		// need to override to fast clear load action ERenderTargetLoadAction::EClear, otherwise,
		// the alpha channel can be garbage data in terms of different driver implementation.
		if ((!bShouldWriteAlphaChannel) && EnumHasAnyFlags(OutputFlags, EPixelFormatChannelFlags::A))
		{
			bShouldOverride = true;
		}
	}

	return bShouldOverride;
}

extern ETonemappingMethod GetTonemappingMethod(const FPostProcessSettings& Settings);
extern float GetTonemappingMax(const FTonemapperOutputDeviceParameters& Parameters, ETonemappingMethod TonemappingMethod);
FScreenPassTexture AddTonemapPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FTonemapInputs& Inputs)
{
	if (!Inputs.bGammaOnly)
	{
		check(Inputs.ColorGradingTexture);
	}
	check(Inputs.SceneColor.IsValid());

	const FSceneViewFamily& ViewFamily = *(View.Family);
	const FPostProcessSettings& PostProcessSettings = View.FinalPostProcessSettings;

	const bool bIsEyeAdaptationResource = Inputs.EyeAdaptationBuffer != nullptr;
	const bool bEyeAdaptation = ViewFamily.EngineShowFlags.EyeAdaptation && bIsEyeAdaptationResource;
	FRDGBufferRef EyeAdaptationBuffer = Inputs.EyeAdaptationBuffer;

	if (!bEyeAdaptation)
	{
		const float DefaultEyeAdaptation = GetEyeAdaptationFixedExposure(View);
		const FVector4f DefaultEyeAdaptationData(DefaultEyeAdaptation, 0, 0, DefaultEyeAdaptation);
		EyeAdaptationBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("DefaultEyeAdaptationBuffer"), sizeof(DefaultEyeAdaptationData), 1, &DefaultEyeAdaptationData, sizeof(DefaultEyeAdaptationData), ERDGInitialDataFlags::None);
	}

	const FScreenPassTextureViewport SceneColorViewport(Inputs.SceneColor);

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		const bool bUseTextureArrays = View.UsesMobileMultiViewShaders() && !View.bUseComputePasses; // MMV not currently supported for compute passes
		const EPixelFormat OutputFormat = Inputs.SceneColor.TextureSRV->Desc.Texture->Desc.Format;
		const ETextureCreateFlags OutputFlags = GFastVRamConfig.Tonemap | TexCreate_ShaderResource | TexCreate_RenderTargetable | (View.bUseComputePasses ? TexCreate_UAV : TexCreate_None);

		FRDGTextureDesc OutputDesc = bUseTextureArrays ?
			FRDGTextureDesc::Create2DArray(SceneColorViewport.Extent, OutputFormat, FClearValueBinding(FLinearColor(0, 0, 0, 0)), OutputFlags, 2) :
			FRDGTextureDesc::Create2D(SceneColorViewport.Extent, OutputFormat, FClearValueBinding(FLinearColor(0, 0, 0, 0)), OutputFlags);

		const FTonemapperOutputDeviceParameters OutputDeviceParameters = GetTonemapperOutputDeviceParameters(*View.Family);
		const EDisplayOutputFormat OutputDevice = static_cast<EDisplayOutputFormat>(OutputDeviceParameters.OutputDevice);
		const bool bPostProcessingAlpha = IsPostProcessingWithAlphaChannelSupported();
		// If scene color is high-precision and alpha is supported, we make sure to preserve at least half precision in the alpha channel until the end of post-processing.
		const bool bPreserveHalfPrecisionAlpha = bPostProcessingAlpha && IsHDR(OutputDesc.Format);

		if (OutputDevice == EDisplayOutputFormat::HDR_LinearEXR)
		{
			OutputDesc.Format = PF_A32B32G32R32F;
		}
		else if (OutputDevice == EDisplayOutputFormat::HDR_LinearNoToneCurve || OutputDevice == EDisplayOutputFormat::HDR_LinearWithToneCurve || bPreserveHalfPrecisionAlpha)
		{
			OutputDesc.Format = PF_FloatRGBA;
		}
		else if (View.Family->RenderTarget && View.Family->RenderTarget->GetRenderTargetTexture())
		{
			// Render into a pixel format that do not loose bit depth precision for the view family.
			OutputDesc.Format = View.Family->RenderTarget->GetRenderTargetTexture()->GetFormat();
		}
		else if (bPostProcessingAlpha)
		{
			// Make sure there is no loss for a 10bit bit-depth using the 10bit of mantissa of halfs
			OutputDesc.Format = PF_FloatRGBA;
		}
		else 
		{
			// Make sure there is no loss for a 10bit bit-depth
			OutputDesc.Format = PF_A2B10G10R10;
		}

		if (
			(OutputDesc.Format == PF_R8G8B8A8 || OutputDesc.Format == PF_B8G8R8A8) && 
			!ShouldWriteAlphaChannel(View, Inputs, OutputDesc.Format) && 
			GPixelFormats[PF_R8G8B8].Supported &&
			!!(GPixelFormats[PF_R8G8B8].Capabilities & EPixelFormatCapabilities::RenderTarget)
			)
		{
			OutputDesc.Format = PF_R8G8B8;
		}

		Output = FScreenPassRenderTarget(
			GraphBuilder.CreateTexture(OutputDesc, TEXT("Tonemap")),
			Inputs.SceneColor.ViewRect,
			View.GetOverwriteLoadAction());
	}

	const bool bShouldWriteAlphaChannel = ShouldWriteAlphaChannel(View, Inputs, Output.Texture->Desc.Format);
	if (ShouldOverrideOutputLoadActionToClear(Output, bShouldWriteAlphaChannel))
	{
		Output.LoadAction = ERenderTargetLoadAction::EClear;
	}

	const FScreenPassTextureViewport OutputViewport(Output);

	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	uint32 Flags = 0;
	Flags |= IsPostProcessingWithAlphaChannelSupported() ? static_cast<uint32>(ETonemapperFlags::AlphaChannel) : 0;

	const float SharpenDiv6 = TonemapperPermutation::GetSharpenSetting(View.FinalPostProcessSettings) / 6.0f;

	FVector4f ChromaticAberrationParams;

	{
		// for scene color fringe
		// from percent to fraction
		float Offset = 0.0f;
		float StartOffset = 0.0f;
		float Multiplier = 1.0f;

		if (PostProcessSettings.ChromaticAberrationStartOffset < 1.0f - KINDA_SMALL_NUMBER)
		{
			Offset = PostProcessSettings.SceneFringeIntensity * 0.01f;
			StartOffset = PostProcessSettings.ChromaticAberrationStartOffset;
			Multiplier = 1.0f / (1.0f - StartOffset);
		}

		// Wavelength of primaries in nm
		const float PrimaryR = 611.3f;
		const float PrimaryG = 549.1f;
		const float PrimaryB = 464.3f;

		// Simple lens chromatic aberration is roughly linear in wavelength
		float ScaleR = 0.007f * (PrimaryR - PrimaryB);
		float ScaleG = 0.007f * (PrimaryG - PrimaryB);
		ChromaticAberrationParams = FVector4f(Offset * ScaleR * Multiplier, Offset * ScaleG * Multiplier, StartOffset, 0.f);
	}

	float EditorNITLevel = 160.0f;

	#if WITH_EDITOR
	{
		static auto CVarHDRNITLevel = IConsoleManager::Get().FindConsoleVariable(TEXT("Editor.HDRNITLevel"));
		if (CVarHDRNITLevel)
		{
			EditorNITLevel = CVarHDRNITLevel->GetFloat();
		}
	}
	#endif

	FTonemapParameters CommonParameters;
	CommonParameters.View = View.GetShaderParameters();

	{
		uint8 OutputFrameIndexMod8 = 0;
		if (View.ViewState)
		{
			// Grain should be temporally stable when accumulating multiple samples per output frame, so we use OutputFrameIndex instead of FrameIndex.
			// Without this, the effect of grain is softened which goes against artistic intent after they tune it to the real-time viewport.
			OutputFrameIndexMod8 = View.ViewState->GetOutputFrameIndex(8);
		}
		GrainRandomFromFrame(&CommonParameters.FilmGrain.GrainRandomFull, OutputFrameIndexMod8);
	}

	if (View.FilmGrainTexture)
	{
		check(SupportsFilmGrain(View.GetShaderPlatform()));

		FRHITexture* FilmGrainTextureRHI = View.FilmGrainTexture->GetTexture2DRHI();
		FRDGTextureRef FilmGrainTexture = RegisterExternalTexture(GraphBuilder, FilmGrainTextureRHI, TEXT("FilmGrain.OriginalTexture"));

		FRDGBufferRef FilmGrainConstantsBuffer;
		if (View.ViewState && View.ViewState->FilmGrainCache.TextureRHI == FilmGrainTextureRHI && CVarFilmGrainCacheTextureConstants.GetValueOnRenderThread() != 0)
		{
			FilmGrainConstantsBuffer = GraphBuilder.RegisterExternalBuffer(View.ViewState->FilmGrainCache.ConstantsBuffer, TEXT("FilmGrain.TextureConstants"));
		}
		else
		{
			FilmGrainConstantsBuffer = BuildFilmGrainConstants(GraphBuilder, View, FilmGrainTexture);

			if (View.ViewState)
			{
				View.ViewState->FilmGrainCache.TextureRHI = FilmGrainTextureRHI;
				GraphBuilder.QueueBufferExtraction(FilmGrainConstantsBuffer, &View.ViewState->FilmGrainCache.ConstantsBuffer);
			}
		}

		// (FilmGrainTexture * FilmGrainDecodeMultiply + FilmGrainDecodeAdd)
		CommonParameters.FilmGrain.FilmGrainIntensityShadows    = View.FinalPostProcessSettings.FilmGrainIntensity * View.FinalPostProcessSettings.FilmGrainIntensityShadows;
		CommonParameters.FilmGrain.FilmGrainIntensityMidtones   = View.FinalPostProcessSettings.FilmGrainIntensity * View.FinalPostProcessSettings.FilmGrainIntensityMidtones;
		CommonParameters.FilmGrain.FilmGrainIntensityHighlights = View.FinalPostProcessSettings.FilmGrainIntensity * View.FinalPostProcessSettings.FilmGrainIntensityHighlights;
		CommonParameters.FilmGrain.FilmGrainShadowsMax = View.FinalPostProcessSettings.FilmGrainShadowsMax;
		CommonParameters.FilmGrain.FilmGrainHighlightsMin = View.FinalPostProcessSettings.FilmGrainHighlightsMin;
		CommonParameters.FilmGrain.FilmGrainHighlightsMax = View.FinalPostProcessSettings.FilmGrainHighlightsMax;

		CommonParameters.FilmGrain.FilmGrainTexture = FilmGrainTexture;
		CommonParameters.FilmGrain.FilmGrainSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap>::GetRHI();

		// Grain should be temporally stable when accumulating multiple samples per output frame, so we use OutputFrameIndex instead of FrameIndex.
		const int32 RandomSequenceLength = FMath::Max(1, CVarFilmGrainSequenceLength.GetValueOnRenderThread());
		const int32 RandomSequenceIndex = (View.ViewState ? View.ViewState->GetOutputFrameIndex() : 0) % RandomSequenceLength;

		FVector2f RandomGrainTextureUVOffset;
		RandomGrainTextureUVOffset.X = Halton(RandomSequenceIndex + 1, 2);
		RandomGrainTextureUVOffset.Y = Halton(RandomSequenceIndex + 1, 3);

		FVector2f TextureSize(View.FilmGrainTexture->GetSizeX(), View.FilmGrainTexture->GetSizeY());
		FVector2f OutputSizeF(1920.0f, 1920.0f * float(OutputViewport.Rect.Height()) / float(OutputViewport.Rect.Width()));

		FVector2f FilmGrainScale
		{
			FMath::Max(View.FinalPostProcessSettings.FilmGrainScale.X, 0.001f),
			FMath::Max(View.FinalPostProcessSettings.FilmGrainScale.Y, 0.001f)
		};
		CommonParameters.FilmGrain.ScreenPosToFilmGrainTextureUV = 
			FScreenTransform::ScreenPosToViewportUV *
			((OutputSizeF / TextureSize) * (FVector2f(1.0f, 1.0f) / FilmGrainScale)) + RandomGrainTextureUVOffset;

		CommonParameters.FilmGrain.FilmGrainTextureConstants = GraphBuilder.CreateSRV(FilmGrainConstantsBuffer);
	}
	else
	{
		// Release the film grain cache
		if (View.ViewState)
		{
			View.ViewState->FilmGrainCache.SafeRelease();
		}

		CommonParameters.FilmGrain.FilmGrainIntensityShadows = 0.0f;
		CommonParameters.FilmGrain.FilmGrainIntensityMidtones = 0.0f;
		CommonParameters.FilmGrain.FilmGrainIntensityHighlights = 0.0f;
		CommonParameters.FilmGrain.FilmGrainShadowsMax = 0.09f;
		CommonParameters.FilmGrain.FilmGrainHighlightsMin = 0.5f;
		CommonParameters.FilmGrain.FilmGrainHighlightsMax = 1.0f;
		CommonParameters.FilmGrain.FilmGrainTexture = GSystemTextures.GetWhiteDummy(GraphBuilder);
		CommonParameters.FilmGrain.FilmGrainSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap>::GetRHI();
		CommonParameters.FilmGrain.ScreenPosToFilmGrainTextureUV = FScreenTransform::ScreenPosToViewportUV;

		// Set a dummy texture constants
		if (SupportsFilmGrain(View.GetShaderPlatform()))
		{
			FRDGBufferRef DummyFilmGrainTextureConstants = GSystemTextures.GetDefaultStructuredBuffer(
				GraphBuilder, sizeof(FVector4f), FVector4f(FLinearColor::White));

			CommonParameters.FilmGrain.FilmGrainTextureConstants = GraphBuilder.CreateSRV(DummyFilmGrainTextureConstants);
		}
		else
		{
			CommonParameters.FilmGrain.FilmGrainTextureConstants = nullptr;
		}
	}
	
	const float LUTSize = Inputs.ColorGradingTexture ? (float)Inputs.ColorGradingTexture->Desc.Extent.Y : /* unused (default): */ 32.0f;

	const bool bBilateralLocalExposureEnabled = Inputs.LocalExposureBilateralGridTexture != nullptr;
	const bool bExposureFusionEnabled = Inputs.ExposureFusion.IsValid();
	const bool bLocalExposureEnabled = bBilateralLocalExposureEnabled || bExposureFusionEnabled;

	if (bLocalExposureEnabled)
	{
		checkf(Inputs.LocalExposureParameters != nullptr, TEXT("When Local Exposure is enabled, corresponding parameters must be provided"));
		CommonParameters.LocalExposure = *Inputs.LocalExposureParameters;
	}

	CommonParameters.OutputDevice = GetTonemapperOutputDeviceParameters(ViewFamily);
	CommonParameters.Color = GetScreenPassTextureViewportParameters(SceneColorViewport);
	CommonParameters.Output = GetScreenPassTextureViewportParameters(OutputViewport);
	CommonParameters.ColorTexture = Inputs.SceneColor.TextureSRV;
	CommonParameters.LumBilateralGrid = Inputs.LocalExposureBilateralGridTexture;
	CommonParameters.BlurredLogLum = Inputs.BlurredLogLuminanceTexture;
	CommonParameters.LumBilateralGridSampler = BilinearClampSampler;
	CommonParameters.BlurredLogLumSampler = BilinearClampSampler;
	if(Inputs.ExposureFusion.IsValid())
	{
		const FScreenPassTextureViewport ExposureFusionViewport(Inputs.ExposureFusion);
		CommonParameters.ColorToExposureFusion = FScreenTransform::ChangeTextureUVCoordinateFromTo(SceneColorViewport, ExposureFusionViewport);
		CommonParameters.ExposureFusion = GetScreenPassTextureViewportParameters(ExposureFusionViewport);
		CommonParameters.ExposureFusionTexture = Inputs.ExposureFusion.TextureSRV;
		CommonParameters.ExposureFusionSampler = BilinearClampSampler;
	}
	CommonParameters.EyeAdaptationBuffer = GraphBuilder.CreateSRV(EyeAdaptationBuffer);
	CommonParameters.EyeAdaptation = *Inputs.EyeAdaptationParameters;
	CommonParameters.ColorGradingLUT = Inputs.ColorGradingTexture;
	CommonParameters.ColorSampler = BilinearClampSampler;
	CommonParameters.ColorGradingLUTSampler = BilinearClampSampler;
	CommonParameters.ColorScale0 = PostProcessSettings.SceneColorTint;
	CommonParameters.ChromaticAberrationParams = ChromaticAberrationParams;
	CommonParameters.Flags = Flags;
	CommonParameters.SharpenDiv6 = SharpenDiv6;
	CommonParameters.EditorNITLevel = EditorNITLevel;

	CommonParameters.VignetteType = (uint32)PostProcessSettings.VignetteType;
	CommonParameters.VignetteParams0 = FVector4f(PostProcessSettings.VignetteIntensity, PostProcessSettings.VignetteColor.R, PostProcessSettings.VignetteColor.G, PostProcessSettings.VignetteColor.B);
	CommonParameters.VignetteParams1 = FVector4f(
		PostProcessSettings.VignetteCenter.X, 
		PostProcessSettings.VignetteCenter.Y, 
		1.0f / FMath::Max(0.001, PostProcessSettings.VignetteSize.X), 
		1.0f / FMath::Max(0.001, PostProcessSettings.VignetteSize.Y));
	CommonParameters.VignetteParams2 = PostProcessSettings.VignetteSoftness;
	CommonParameters.VignetteTexture = GSystemTextures.GetWhiteDummy(GraphBuilder);
	CommonParameters.VignetteTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	if (PostProcessSettings.VignetteType == EVignetteType::Texture && PostProcessSettings.VignetteTexture)
	{
		FTextureResource* VignetteTextureResource = PostProcessSettings.VignetteTexture->GetResource();

		if (VignetteTextureResource && VignetteTextureResource->TextureRHI)
		{
			FRDGTextureRef VignetteTextureRDG = RegisterExternalTexture(GraphBuilder, VignetteTextureResource->TextureRHI, TEXT("VignetteTexture"));
			CommonParameters.VignetteTexture = VignetteTextureRDG;
		}
	}

	CommonParameters.FilmSlope = PostProcessSettings.FilmSlope;
	CommonParameters.FilmToe = PostProcessSettings.FilmToe;
	CommonParameters.FilmShoulder = PostProcessSettings.FilmShoulder;
	CommonParameters.FilmBlackClip = PostProcessSettings.FilmBlackClip;
	CommonParameters.FilmWhiteClip = PostProcessSettings.FilmWhiteClip;

	{
		const bool bSupportsBackbufferQuantizationDithering = EDisplayOutputFormat(CommonParameters.OutputDevice.OutputDevice) != EDisplayOutputFormat::HDR_LinearNoToneCurve && EDisplayOutputFormat(CommonParameters.OutputDevice.OutputDevice) != EDisplayOutputFormat::HDR_LinearWithToneCurve;
		const int32 BitdepthOverride = CVarBackbufferQuantizationDitheringOverride.GetValueOnRenderThread();

		if (!bSupportsBackbufferQuantizationDithering)
		{
			CommonParameters.BackbufferQuantizationDithering = 0.0f;
		}
		else if (BitdepthOverride > 0)
		{
			CommonParameters.BackbufferQuantizationDithering = 1.0f / (FMath::Pow(2.0f, float(BitdepthOverride)) - 1.0f);
		}
		else
		{
			CommonParameters.BackbufferQuantizationDithering = 1.0f / 1023.0f;

			if (View.Family->RenderTarget && View.Family->RenderTarget->GetRenderTargetTexture())
			{
				EPixelFormat BackbufferPixelFormat = View.Family->RenderTarget->GetRenderTargetTexture()->GetFormat();
				if (BackbufferPixelFormat == PF_B8G8R8A8 || BackbufferPixelFormat == PF_R8G8B8A8)
				{
					CommonParameters.BackbufferQuantizationDithering = 1.0f / 255.0f;
				}
			}
		}
	}
	CommonParameters.LUTMax = GetTonemappingMax(CommonParameters.OutputDevice, GetTonemappingMethod(PostProcessSettings));
	CommonParameters.LUTSize = LUTSize;
	CommonParameters.InvLUTSize = 1.0f / LUTSize;
	CommonParameters.LUTScale = (LUTSize - 1.0f) / LUTSize;
	CommonParameters.LUTOffset = 0.5f / LUTSize;
	CommonParameters.LensPrincipalPointOffsetScale = View.LensPrincipalPointOffsetScale;

	// Bloom parameters
	{
		const bool bUseBloom = Inputs.Bloom.Texture != nullptr;

		if (bUseBloom)
		{
			const FScreenPassTextureViewport BloomViewport(Inputs.Bloom);
			CommonParameters.ColorToBloom = FScreenTransform::ChangeTextureUVCoordinateFromTo(SceneColorViewport, BloomViewport);
			CommonParameters.BloomUVViewportBilinearMin = GetScreenPassTextureViewportParameters(BloomViewport).UVViewportBilinearMin;
			CommonParameters.BloomUVViewportBilinearMax = GetScreenPassTextureViewportParameters(BloomViewport).UVViewportBilinearMax;
			CommonParameters.BloomTexture = Inputs.Bloom.Texture;
			CommonParameters.BloomSampler = BilinearClampSampler;
		}
		else
		{
			CommonParameters.ColorToBloom = FScreenTransform::Identity;
			CommonParameters.BloomUVViewportBilinearMin = FVector2f::ZeroVector;
			CommonParameters.BloomUVViewportBilinearMax = FVector2f::UnitVector;
			CommonParameters.BloomTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
			CommonParameters.BloomSampler = BilinearClampSampler;
		}

		if (!FTonemapInputs::SupportsSceneColorApplyParametersBuffer(View.GetShaderPlatform()))
		{
			check(Inputs.SceneColorApplyParamaters == nullptr);
		}
		else if (Inputs.SceneColorApplyParamaters)
		{
			CommonParameters.SceneColorApplyParamaters = GraphBuilder.CreateSRV(Inputs.SceneColorApplyParamaters);
		}
		else
		{
			FRDGBufferRef ApplyParametersBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f), FVector4f(1.0f, 1.0f, 1.0f, 1.0f));
			CommonParameters.SceneColorApplyParamaters = GraphBuilder.CreateSRV(ApplyParametersBuffer);
		}

		// TODO: PostProcessSettings.BloomDirtMask->GetResource() is not thread safe
		const bool bUseBloomDirtMask = 
			bUseBloom 
			&& PostProcessSettings.BloomDirtMask 
			&& PostProcessSettings.BloomDirtMask->GetResource() 
			&& PostProcessSettings.BloomDirtMask->GetResource()->TextureRHI
			&& PostProcessSettings.BloomDirtMaskIntensity > 0 
			&& PostProcessSettings.BloomIntensity > 0.0;
		if (bUseBloomDirtMask)
		{
			CommonParameters.BloomDirtMaskTint = PostProcessSettings.BloomDirtMaskTint * PostProcessSettings.BloomDirtMaskIntensity / PostProcessSettings.BloomIntensity;
			FRDGTextureRef BloomDirtMaskRDG = RegisterExternalTexture(GraphBuilder, PostProcessSettings.BloomDirtMask->GetResource()->TextureRHI, TEXT("BloomDirtMask"));
			CommonParameters.BloomDirtMaskTexture = BloomDirtMaskRDG;
			CommonParameters.BloomDirtMaskSampler = BilinearClampSampler;
		}
		else
		{
			CommonParameters.BloomDirtMaskTint = FLinearColor::Black;
			CommonParameters.BloomDirtMaskTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
			CommonParameters.BloomDirtMaskSampler = BilinearClampSampler;
		}
	}

	// forward transformation from shader:
	//return LensPrincipalPointOffsetScale.xy + UV * LensPrincipalPointOffsetScale.zw;

	// reverse transformation from shader:
	//return UV*(1.0f/LensPrincipalPointOffsetScale.zw) - LensPrincipalPointOffsetScale.xy/LensPrincipalPointOffsetScale.zw;

	CommonParameters.LensPrincipalPointOffsetScaleInverse.X = -View.LensPrincipalPointOffsetScale.X / View.LensPrincipalPointOffsetScale.Z;
	CommonParameters.LensPrincipalPointOffsetScaleInverse.Y = -View.LensPrincipalPointOffsetScale.Y / View.LensPrincipalPointOffsetScale.W;
	CommonParameters.LensPrincipalPointOffsetScaleInverse.Z = 1.0f / View.LensPrincipalPointOffsetScale.Z;
	CommonParameters.LensPrincipalPointOffsetScaleInverse.W = 1.0f / View.LensPrincipalPointOffsetScale.W;

	// Generate permutation vector for the desktop tonemapper.
	TonemapperPermutation::FDesktopDomain DesktopPermutationVector;

	{
		TonemapperPermutation::FCommonDomain CommonDomain = TonemapperPermutation::BuildCommonPermutationDomain(View, Inputs.bGammaOnly, bLocalExposureEnabled, Inputs.bMetalMSAAHDRDecode);
		DesktopPermutationVector.Set<TonemapperPermutation::FCommonDomain>(CommonDomain);

		if (!CommonDomain.Get<TonemapperPermutation::FTonemapperGammaOnlyDim>())
		{
			DesktopPermutationVector.Set<TonemapperPermutation::FTonemapperColorFringeDim>(PostProcessSettings.SceneFringeIntensity > 0.01f);
		}

		DesktopPermutationVector.Set<TonemapperPermutation::FTonemapperOutputLuminance>(!View.UsesMobileMultiViewShaders() && GVRSImageManager.IsVRSEnabledForFrame() && FVariableRateShadingImageManager::IsVRSCompatibleWithView(View));

		const bool bOutputDeviceSRGB = (CommonParameters.OutputDevice.OutputDevice == (uint32)EDisplayOutputFormat::SDR_sRGB);
		DesktopPermutationVector.Set<TonemapperPermutation::FTonemapperOutputDeviceSRGB>(bOutputDeviceSRGB);

		DesktopPermutationVector = TonemapperPermutation::RemapPermutation(DesktopPermutationVector, View.GetFeatureLevel());
	}

	// Override output might not support UAVs.
	const bool bComputePass = (Output.Texture->Desc.Flags & TexCreate_UAV) == TexCreate_UAV ? View.bUseComputePasses : false;

	FRDGTextureRef OutputLuminance = {};
	const bool bOutputLuminance = DesktopPermutationVector.Get<TonemapperPermutation::FTonemapperOutputLuminance>();
	if (bOutputLuminance)
	{
		// Due to the way split-screen shares the same render target for all views, make sure
		// that we use the same texture for saving out the luminance as well to keep the memory
		// footprint small - but only if we're outputting directly to the final render target for both views
		// (that is, neither view has any post-processing steps after tonemapping)
		const FOutputLuminance* CachedOutputLuminance = GraphBuilder.Blackboard.Get<FOutputLuminance>();
		if (CachedOutputLuminance && Inputs.OverrideOutput.IsValid() &&
			CachedOutputLuminance->Texture->Desc.Extent == Output.Texture->Desc.Extent)
		{
			OutputLuminance = CachedOutputLuminance->Texture;
		}
		else
		{
			FIntPoint OutputSize = Output.Texture->Desc.Extent;
			const FIntPoint SDROutputSize = FIntPoint(OutputSize.X, OutputSize.Y);
			FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
				SDROutputSize,
				PF_R8,
				FClearValueBinding::Black,
				bComputePass ? TexCreate_UAV : TexCreate_RenderTargetable);
			OutputLuminance = GraphBuilder.CreateTexture(Desc, TEXT("Final Luminance"));

			if (CachedOutputLuminance == nullptr)
			{
				FOutputLuminance* NewOutputLuminance = &GraphBuilder.Blackboard.Create<FOutputLuminance>();
				NewOutputLuminance->Texture = OutputLuminance;
			}
		}
	}

	if (bComputePass)
	{
		FTonemapCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FTonemapCS::FParameters>();
		PassParameters->Tonemap = CommonParameters;
		PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(Output.Texture);

		if (OutputLuminance)
		{
			PassParameters->RWOutputLuminance = GraphBuilder.CreateUAV(OutputLuminance);
		}
		else
		{
			check(DesktopPermutationVector.Get<TonemapperPermutation::FTonemapperOutputLuminance>() == false);
		}

		TShaderMapRef<FTonemapCS> ComputeShader(View.ShaderMap, DesktopPermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Tonemap %dx%d (CS GammaOnly=%d)", OutputViewport.Rect.Width(), OutputViewport.Rect.Height(), Inputs.bGammaOnly),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), FIntPoint(GTonemapComputeTileSizeX, GTonemapComputeTileSizeY)));
	}
	else
	{
		FTonemapPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FTonemapPS::FParameters>();
		PassParameters->Tonemap = CommonParameters;
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		PassParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();

		if (OutputLuminance)
		{
			PassParameters->RenderTargets[1] = FRenderTargetBinding(OutputLuminance, ERenderTargetLoadAction::ELoad);
		}

		TShaderMapRef<FTonemapVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FTonemapPS> PixelShader(View.ShaderMap, DesktopPermutationVector);

		FRHIBlendState* BlendState = bShouldWriteAlphaChannel ? FScreenPassPipelineState::FDefaultBlendState::GetRHI() : TStaticBlendStateWriteMask<CW_RGB>::GetRHI();

		FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

		EScreenPassDrawFlags DrawFlags = EScreenPassDrawFlags::AllowHMDHiddenAreaMask;

		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("Tonemap %dx%d (PS GammaOnly=%d)", OutputViewport.Rect.Width(), OutputViewport.Rect.Height(), Inputs.bGammaOnly),
			View,
			OutputViewport,
			SceneColorViewport,
			FScreenPassPipelineState(VertexShader, PixelShader, BlendState, DepthStencilState),
			PassParameters,
			DrawFlags,
			[VertexShader, PixelShader, PassParameters](FRHICommandList& RHICmdList)
		{
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->Tonemap);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
		});
	}

	if (OutputLuminance && View.ViewState)
	{
		View.ViewState->PrevFrameViewInfo.LuminanceViewRectHistory = OutputViewport.Rect;
		GraphBuilder.QueueTextureExtraction(OutputLuminance, &View.ViewState->PrevFrameViewInfo.LuminanceHistory);
	}

	return MoveTemp(Output);
}


// MSAA custom resolve shader that does tonemapping 
class FMobileCustomResolvePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FMobileCustomResolvePS);

	SHADER_USE_PARAMETER_STRUCT(FMobileCustomResolvePS, FGlobalShader);
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
		SHADER_PARAMETER(FVector4f, ColorScale0)
		SHADER_PARAMETER_TEXTURE(Texture3D, ColorGradingLUT)
		SHADER_PARAMETER_SAMPLER(SamplerState, ColorGradingLUTSampler)
		SHADER_PARAMETER(float, LUTMax)
		SHADER_PARAMETER(float, LUTSize)
		SHADER_PARAMETER(float, InvLUTSize)
		SHADER_PARAMETER(float, LUTScale)
		SHADER_PARAMETER(float, LUTOffset)
		SHADER_PARAMETER(uint32, LUTShaper)
	END_SHADER_PARAMETER_STRUCT()

	class FTonemapperSubpassMsaaDim : SHADER_PERMUTATION_SPARSE_INT("SUBPASS_MSAA_SAMPLES", 0, 1, 2, 4, 8);
	using FPermutationDomain = TShaderPermutationDomain<FTonemapperSubpassMsaaDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		ModifyTonemapCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("OUTPUT_DEVICE_SRGB"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FMobileCustomResolvePS, "/Engine/Private/PostProcessTonemap.usf", "MobileCustomResolve_MainPS", SF_Pixel);

void RenderMobileCustomResolve(FRHICommandList& RHICmdList, const FViewInfo& View, const int32 SubpassMSAASamples, FSceneTextures& SceneTextures)
{
	// Part of scene rendering pass
	check(RHICmdList.IsInsideRenderPass());
	SCOPED_DRAW_EVENT(RHICmdList, MobileTonemapSubpass);

	IPooledRenderTarget* ColorGradingLUT = View.GetTonemappingLUT();
	const FIntPoint TargetSize = SceneTextures.Color.Target->Desc.Extent;
	const float LUTSize = ColorGradingLUT ? (float)ColorGradingLUT->GetDesc().GetSize().Y : /* unused (default): */ 32.0f;
	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;
	
	TShaderMapRef<FMobileMultiViewVertexShaderVS> VertexShader(View.ShaderMap);
	
	FMobileCustomResolvePS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FMobileCustomResolvePS::FTonemapperSubpassMsaaDim>(SubpassMSAASamples);
	TShaderMapRef<FMobileCustomResolvePS> PixelShader(View.ShaderMap, PermutationVector);

	FMobileCustomResolvePS::FParameters PSShaderParameters;
	PSShaderParameters.View = View.GetShaderParameters();
	PSShaderParameters.ColorScale0 = FVector4f(Settings.SceneColorTint.R, Settings.SceneColorTint.G, Settings.SceneColorTint.B, 0);
	PSShaderParameters.ColorGradingLUT = ColorGradingLUT ? ColorGradingLUT->GetRHI() : GBlackVolumeTexture->TextureRHI.GetReference();
	PSShaderParameters.ColorGradingLUTSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	const FTonemapperOutputDeviceParameters OutputDeviceParameters = GetTonemapperOutputDeviceParameters(*View.Family);
	PSShaderParameters.LUTMax = GetTonemappingMax(OutputDeviceParameters, GetTonemappingMethod(Settings));
	PSShaderParameters.LUTSize = LUTSize;
	PSShaderParameters.InvLUTSize = 1.0f / LUTSize;
	PSShaderParameters.LUTScale = (LUTSize - 1.0f) / LUTSize;
	PSShaderParameters.LUTOffset = 0.5f / LUTSize;
	PSShaderParameters.LUTShaper = CVarLUTShaper.GetValueOnRenderThread();
	if (SubpassMSAASamples == 0u)
	{
		PSShaderParameters.ColorTexture = SceneTextures.Color.Resolve;
		PSShaderParameters.ColorSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
	SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PSShaderParameters);
	RHICmdList.SetViewport(0, 0, 0.0f, TargetSize.X, TargetSize.Y, 1.0f);

	DrawRectangle(
		RHICmdList,
		0, 0,
		TargetSize.X, TargetSize.Y,
		0, 0,
		TargetSize.X, TargetSize.Y,
		TargetSize,
		TargetSize,
		VertexShader,
		EDRF_UseTriangleOptimization,
		View.GetStereoPassInstanceFactor());
}

BEGIN_SHADER_PARAMETER_STRUCT(FMobileCustomResolveParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
	RDG_TEXTURE_ACCESS(ColorGradingLUT, ERHIAccess::SRVGraphics)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void AddMobileCustomResolvePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSceneTextures& SceneTextures, FRDGTextureRef ViewFamilyTexture)
{
	FRDGTextureRef ColorGradingLUT = AddCombineLUTPass(GraphBuilder, View);

	FMobileCustomResolveParameters* PassParameters = GraphBuilder.AllocParameters<FMobileCustomResolveParameters>();
	PassParameters->View = View.GetShaderParameters();
	PassParameters->ColorTexture = SceneTextures.Color.Resolve;
	PassParameters->ColorGradingLUT = ColorGradingLUT;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(ViewFamilyTexture, ERenderTargetLoadAction::EClear);
	// Need to specify multi view count for when the custom resolve pass is a separate pass and is not within the main pass.
	PassParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("MobileCustomResolvePass"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, &SceneTextures](FRHICommandListImmediate& RHICmdList)
		{
			const uint32 SubpassMSAASamples = 0u; // not using subpass resolve
			RenderMobileCustomResolve(RHICmdList, View, SubpassMSAASamples, SceneTextures);
		});
}

