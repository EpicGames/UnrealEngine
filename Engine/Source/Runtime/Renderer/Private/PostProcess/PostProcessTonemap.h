// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PostProcess/PostProcessEyeAdaptation.h"
#include "PostProcess/PostProcessBloomSetup.h"
#include "ScreenPass.h"
#include "OverridePassSequence.h"
#include "Math/Halton.h"

class FLocalExposureParameters;

bool SupportsFilmGrain(EShaderPlatform Platform);

BEGIN_SHADER_PARAMETER_STRUCT(FTonemapperOutputDeviceParameters, )
	SHADER_PARAMETER(FVector3f, InverseGamma)
	SHADER_PARAMETER(uint32, OutputDevice)
	SHADER_PARAMETER(uint32, OutputGamut)
	SHADER_PARAMETER(float, OutputMaxLuminance)
	SHADER_PARAMETER(uint32, LUTShaper)
END_SHADER_PARAMETER_STRUCT()

RENDERER_API FTonemapperOutputDeviceParameters GetTonemapperOutputDeviceParameters(const FSceneViewFamily& Family);

static void GrainRandomFromFrame(FVector3f* RESTRICT const Constant, uint32 FrameNumber)
{
	Constant->X = Halton(FrameNumber & 1023, 2);
	Constant->Y = Halton(FrameNumber & 1023, 3);
}

struct FTonemapInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] HDR scene color to tonemap.
	FScreenPassTextureSlice SceneColor;

	// [Required] Filtered bloom texture to composite with tonemapped scene color. This should be transparent black for no bloom.
	FScreenPassTexture Bloom;

	// [Optional] structured buffer of multiply parameters to apply to the scene color.
	FRDGBufferRef SceneColorApplyParamaters = nullptr;

	// [Optional] Luminance bilateral grid. If this is null, local exposure is disabled.
	FRDGTextureRef LocalExposureBilateralGridTexture = nullptr;

	// [Optional] Blurred luminance texture used to calculate local exposure.
	FRDGTextureRef BlurredLogLuminanceTexture = nullptr;

	// [Optional] Exposure Fusion. If this is null, local exposure is disabled.
	FScreenPassTextureSlice ExposureFusion;

	// [Optional] Local exposure parameters.
	const FLocalExposureParameters* LocalExposureParameters = nullptr;

	// [Required] Eye adaptation parameters.
	const FEyeAdaptationParameters* EyeAdaptationParameters = nullptr;

	// [Required] Color grading texture used to remap colors.
	FRDGTextureRef ColorGradingTexture = nullptr;

	// Eye adaptation buffer used to compute exposure. 
	FRDGBufferRef EyeAdaptationBuffer = nullptr;

	// [Raster Only] Controls whether the alpha channel of the scene texture should be written to the output texture.
	bool bWriteAlphaChannel = false;

	// Configures the tonemapper to only perform gamma correction.
	bool bGammaOnly = false;

	// Whether to leave the final output in HDR.
	bool bOutputInHDR = false;

	bool bMetalMSAAHDRDecode = false;

	// Returns whether ApplyParameters is supported by the tonemapper.
	static bool SupportsSceneColorApplyParametersBuffer(EShaderPlatform Platform);
};

FScreenPassTexture AddTonemapPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FTonemapInputs& Inputs);
void RenderMobileCustomResolve(FRHICommandList& RHICmdList, const FViewInfo& View, const int32 SubpassMSAASamples, FSceneTextures& SceneTextures);
void AddMobileCustomResolvePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSceneTextures& SceneTextures, FRDGTextureRef ViewFamilyTexture);
