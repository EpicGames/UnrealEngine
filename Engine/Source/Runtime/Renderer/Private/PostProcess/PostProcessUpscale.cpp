// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessUpscale.h"
#include "PostProcess/SceneFilterRendering.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"
#include "PostProcessing.h"
#include "RHIResourceUtils.h"
#include "PostProcess/DrawRectangle.h"

namespace
{
TAutoConsoleVariable<float> CVarUpscaleSoftness(
	TEXT("r.Upscale.Softness"),
	1.0f,
	TEXT("Amount of sharpening for Gaussian Unsharp filter (r.UpscaleQuality=5). Reduce if ringing is visible\n")
	TEXT("  1: Normal sharpening (default)\n")
	TEXT("  0: No sharpening (pure Gaussian)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarUpscaleQuality(
	TEXT("r.Upscale.Quality"),
	3,
	TEXT("Defines the quality in which ScreenPercentage and WindowedFullscreen scales the 3d rendering.\n")
	TEXT(" 0: Nearest filtering\n")
	TEXT(" 1: Simple Bilinear\n")
	TEXT(" 2: Directional blur with unsharp mask upsample.\n")
	TEXT(" 3: 5-tap Catmull-Rom bicubic, approximating Lanczos 2. (default)\n")
	TEXT(" 4: 13-tap Lanczos 3.\n")
	TEXT(" 5: 36-tap Gaussian-filtered unsharp mask (very expensive, but good for extreme upsampling).\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarMobileUpscaleQuality(
	TEXT("r.Mobile.Upscale.Quality"),
	1,
	TEXT("Defines the quality in which ScreenPercentage and WindowedFullscreen scales the 3d rendering on mobile platforms.\n")
	TEXT(" 0: Nearest filtering\n")
	TEXT(" 1: Simple Bilinear (default)\n")
	TEXT(" 8: Area\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarUpscaleComputeEnabled(
	TEXT("r.Upscale.ComputeEnabled"),
	1,
	TEXT("Allow running the upscaler as a compute pass. \n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarUpscaleSharpeningQuality(
	TEXT("r.Upscale.SharpeningQuality"),
	1,
	TEXT("0: off\n")
	TEXT("1: cheaper\n")
	TEXT("2: higher quality"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarUpscaleSharpening(
	TEXT("r.Upscale.Sharpening"),
	0.0f,
	TEXT("Increase to get more sharpening on the final upscale. Requires ComputeEnabled.\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

} //! namespace

BEGIN_SHADER_PARAMETER_STRUCT(FUpscaleParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DistortingDisplacementTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, DistortingDisplacementSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, UndistortingDisplacementSampler)
	SHADER_PARAMETER(FIntPoint, GridDimensions)
	SHADER_PARAMETER(uint32, bInvertAlpha)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PointSceneColorTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray, PointSceneColorTextureArray)
	SHADER_PARAMETER_SAMPLER(SamplerState, PointSceneColorSampler)
	SHADER_PARAMETER(float, UpscaleSoftness)
	SHADER_PARAMETER(float, Sharpening)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FUpscaleRasterParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FUpscaleParameters, UpscaleParameters)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FUpscalePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FUpscalePS);
	SHADER_USE_PARAMETER_STRUCT(FUpscalePS, FGlobalShader);
	using FParameters = FUpscaleRasterParameters;

	class FAlphaChannelDim : SHADER_PERMUTATION_BOOL("DIM_ALPHA_CHANNEL");
	class FMethodDimension : SHADER_PERMUTATION_ENUM_CLASS("DIM_METHOD", EUpscaleMethod);
	using FPermutationDomain = TShaderPermutationDomain<FAlphaChannelDim, FMethodDimension>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const EUpscaleMethod UpscaleMethod = PermutationVector.Get<FMethodDimension>();

		if (UpscaleMethod == EUpscaleMethod::None)
		{
			return false;
		}

		// Always allow point and bilinear and area upscale. (Provides upscaling for mobile emulation)
		if (UpscaleMethod == EUpscaleMethod::Nearest ||
			UpscaleMethod == EUpscaleMethod::Bilinear ||
			UpscaleMethod == EUpscaleMethod::Area)
		{
			return true;
		}

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FUpscalePS, "/Engine/Private/PostProcessUpscale.usf", "MainPS", SF_Pixel);

class FUpscaleVS : public FScreenPassVS
{
public:
	DECLARE_GLOBAL_SHADER(FUpscaleVS);
	// FDrawRectangleParameters is filled by DrawScreenPass.
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FUpscaleVS, FScreenPassVS);
	using FParameters = FUpscaleRasterParameters;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FScreenPassVS::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
};

IMPLEMENT_GLOBAL_SHADER(FUpscaleVS, "/Engine/Private/PostProcessUpscale.usf", "MainVS", SF_Vertex);

const int32 GUpscaleComputeTileSizeX = 16;
const int32 GUpscaleComputeTileSizeY = 16;

enum class ESharpeningQuality : uint8
{
	Off,
	Low,
	High,
	MAX
};

enum class EMethodSet : uint8
{
	None,
	Simple,
	Complex,
	MAX
};

EMethodSet GetMethodSet(EUpscaleMethod Method)
{
	switch (Method)
	{
		case EUpscaleMethod::Nearest:
		case EUpscaleMethod::Bilinear:
		case EUpscaleMethod::SmoothStep:
		case EUpscaleMethod::Area:
			return EMethodSet::Simple;
		case EUpscaleMethod::Directional:
		case EUpscaleMethod::CatmullRom:
		case EUpscaleMethod::Lanczos:
		case EUpscaleMethod::Gaussian:
			return EMethodSet::Complex;
		default:
			return EMethodSet::MAX;
	};
}

class FUpscaleCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FUpscaleCS);
	SHADER_USE_PARAMETER_STRUCT(FUpscaleCS, FGlobalShader);
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FUpscaleParameters, UpscaleParameters)
		SHADER_PARAMETER(uint32, UpscaleMethod)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWOutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	class FMethodDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_METHOD", EUpscaleMethod);
	class FMethodSetDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_METHODSET", EMethodSet);
	class FAlphaChannelDim : SHADER_PERMUTATION_BOOL("DIM_ALPHA_CHANNEL");
	class FLensDistortionDim : SHADER_PERMUTATION_BOOL("DIM_LENS_DISTORTION");
	class FSharpeningQualityDim : SHADER_PERMUTATION_ENUM_CLASS("DIM_SHARPENING_QUALITY", ESharpeningQuality);
	using FPermutationDomain = TShaderPermutationDomain<FMethodDim, FMethodSetDim, FAlphaChannelDim, FLensDistortionDim, FSharpeningQualityDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		const EUpscaleMethod UpscaleMethod = PermutationVector.Get<FMethodDim>();
		const EMethodSet UpscaleMethodSet = PermutationVector.Get<FMethodSetDim>();

		if (UpscaleMethodSet != EMethodSet::None && UpscaleMethod != EUpscaleMethod::None)
		{
			return false;
		}
		else if (UpscaleMethodSet == EMethodSet::None && UpscaleMethod == EUpscaleMethod::None)
		{
			return false;
		}

		if (!IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5))
		{
			return false;
		}

		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		const EUpscaleMethod UpscaleMethod = PermutationVector.Get<FMethodDim>();

		// Special path for SmoothStep, as that is used in secondary upscaler
		if (UpscaleMethod != EUpscaleMethod::SmoothStep)
		{
			PermutationVector.Set<FMethodDim>(EUpscaleMethod::None);
			PermutationVector.Set<FMethodSetDim>(GetMethodSet(UpscaleMethod));
		}

		return PermutationVector;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GUpscaleComputeTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GUpscaleComputeTileSizeY);
	}
};

IMPLEMENT_GLOBAL_SHADER(FUpscaleCS, "/Engine/Private/PostProcessUpscale.usf", "MainCS", SF_Compute);

EUpscaleMethod GetUpscaleMethod()
{
	const int32 StartIndex = 1; // 0 is EUpscaleMethod::None, methods start at 1
	const int32 Value = CVarUpscaleQuality.GetValueOnRenderThread() + StartIndex;

	return static_cast<EUpscaleMethod>(FMath::Clamp(Value, StartIndex, static_cast<int32>(EUpscaleMethod::Gaussian)));
}

EUpscaleMethod GetMobileUpscaleMethod()
{
	const int32 StartIndex = 1; // 0 is EUpscaleMethod::None, methods start at 1
	const int32 Value = CVarMobileUpscaleQuality.GetValueOnRenderThread() + StartIndex;

	return Value <= static_cast<int32>(EUpscaleMethod::Bilinear) ? static_cast<EUpscaleMethod>(FMath::Max(Value, StartIndex)) : EUpscaleMethod::Area;
}

// static
FScreenPassTexture ISpatialUpscaler::AddDefaultUpscalePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FInputs& Inputs,
	EUpscaleMethod Method,
	FLensDistortionLUT LensDistortionLUT)
{
	check(Inputs.SceneColor.IsValid());
	check(Method != EUpscaleMethod::None);
	check(Method != EUpscaleMethod::MAX);
	check(Inputs.Stage != EUpscaleStage::MAX);

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
			Inputs.SceneColor.Texture->Desc.Extent,
			Inputs.SceneColor.Texture->Desc.Format,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_RenderTargetable | GFastVRamConfig.Upscale);

		if (Inputs.Stage == EUpscaleStage::PrimaryToSecondary)
		{
			const FIntPoint SecondaryViewRectSize = View.GetSecondaryViewRectSize();
			QuantizeSceneBufferSize(SecondaryViewRectSize, OutputDesc.Extent);
			Output.ViewRect.Min = FIntPoint::ZeroValue;
			Output.ViewRect.Max = SecondaryViewRectSize;
		}
		else
		{
			OutputDesc.Extent = View.UnscaledViewRect.Max;
			Output.ViewRect = View.UnscaledViewRect;
		}

		Output.Texture = GraphBuilder.CreateTexture(OutputDesc, TEXT("Upscale"));
		Output.LoadAction = ERenderTargetLoadAction::EClear;
		Output.UpdateVisualizeTextureExtent();
	}

	const FIntRect InputRect = Inputs.Stage == EUpscaleStage::SecondaryToOutput ? View.GetSecondaryViewCropRect() : Inputs.SceneColor.ViewRect;
	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor.Texture, InputRect);
	const FScreenPassTextureViewport OutputViewport(Output);

	const bool bApplyLensDistortion = LensDistortionLUT.IsEnabled();

	bool bIsUpscaleToOutput = Inputs.Stage == EUpscaleStage::PrimaryToOutput || Inputs.Stage == EUpscaleStage::SecondaryToOutput;
	float Sharpening = bIsUpscaleToOutput ? CVarUpscaleSharpening.GetValueOnRenderThread() : 0.0f;
	
	FUpscaleParameters PassParameters;
	PassParameters.Input = GetScreenPassTextureViewportParameters(InputViewport);
	PassParameters.Output = GetScreenPassTextureViewportParameters(OutputViewport);
	PassParameters.DistortingDisplacementTexture = LensDistortionLUT.DistortingDisplacementTexture;
	PassParameters.DistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters.UndistortingDisplacementTexture = LensDistortionLUT.UndistortingDisplacementTexture;
	PassParameters.UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters.GridDimensions = LensDistortionLUT.DistortionGridDimensions;
	PassParameters.bInvertAlpha = View.Family->EngineShowFlags.AlphaInvert;
	PassParameters.SceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters.SceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Border>::GetRHI();
	PassParameters.PointSceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters.PointSceneColorTextureArray = Inputs.SceneColor.Texture;
	PassParameters.PointSceneColorSampler = TStaticSamplerState<SF_Point, AM_Border, AM_Border, AM_Border>::GetRHI();
	PassParameters.UpscaleSoftness = FMath::Clamp(CVarUpscaleSoftness.GetValueOnRenderThread(), 0.0f, 1.0f);
	PassParameters.Sharpening = Sharpening;
	PassParameters.View = View.GetShaderParameters();

	const TCHAR* const StageNames[] = { TEXT("PrimaryToSecondary"), TEXT("PrimaryToOutput"), TEXT("SecondaryToOutput") };
	static_assert(UE_ARRAY_COUNT(StageNames) == static_cast<uint32>(EUpscaleStage::MAX), "StageNames does not match EUpscaleStage");
	const TCHAR* StageName = StageNames[static_cast<uint32>(Inputs.Stage)];

	bool bUseCompute = bIsUpscaleToOutput && CVarUpscaleComputeEnabled.GetValueOnRenderThread() > 0 && View.bUseComputePasses;
	if (bUseCompute)
	{
		FRDGTextureRef ComputeRenderTarget{};
		bool bOutputSupportsUAV = (Output.Texture->Desc.Flags & TexCreate_UAV) == TexCreate_UAV;
		if (bOutputSupportsUAV)
		{
			ComputeRenderTarget = Output.Texture;
		}
		else
		{
			const FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(Output.Texture->Desc.Extent, Output.Texture->Desc.Format, FClearValueBinding::None, ETextureCreateFlags::UAV | ETextureCreateFlags::ShaderResource));
			ComputeRenderTarget = GraphBuilder.CreateTexture(Desc, TEXT("SecondaryUpscalerOutput"));
		}

		FUpscaleCS::FParameters* ComputePassParameters = GraphBuilder.AllocParameters<FUpscaleCS::FParameters>();

		ComputePassParameters->UpscaleParameters = PassParameters;
		ComputePassParameters->RWOutputTexture = GraphBuilder.CreateUAV(ComputeRenderTarget);
		ComputePassParameters->UpscaleMethod = (uint32)Method;

		ESharpeningQuality SharpeningQuality = (ESharpeningQuality)FMath::Clamp(CVarUpscaleSharpeningQuality.GetValueOnRenderThread(), 0, static_cast<int32>(ESharpeningQuality::MAX) - 1); 
		SharpeningQuality = Sharpening != 0.0f ? SharpeningQuality : ESharpeningQuality::Off;

		FUpscaleCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FUpscaleCS::FAlphaChannelDim>(IsPostProcessingWithAlphaChannelSupported());
		PermutationVector.Set<FUpscaleCS::FLensDistortionDim>(bApplyLensDistortion);
		PermutationVector.Set<FUpscaleCS::FMethodDim>(Method);
		PermutationVector.Set<FUpscaleCS::FMethodSetDim>(EMethodSet::None); // RemapPermutation sets this to the correct one.
		PermutationVector.Set<FUpscaleCS::FSharpeningQualityDim>(SharpeningQuality);
		TShaderMapRef<FUpscaleCS> ComputeShader(View.ShaderMap, PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Upscale(CS %s Method=%d%s%s) %dx%d -> %dx%d",
				StageName,
				int32(Method),
				PermutationVector.Get<FUpscaleCS::FAlphaChannelDim>() ? TEXT(" Alpha") : TEXT(""),
				bApplyLensDistortion ? TEXT(" LensDistortion") : TEXT(""),
				Inputs.SceneColor.ViewRect.Width(), Inputs.SceneColor.ViewRect.Height(),
				Output.ViewRect.Width(), Output.ViewRect.Height()),
			ComputeShader,
			ComputePassParameters,
			FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), FIntPoint(GUpscaleComputeTileSizeX, GUpscaleComputeTileSizeY)));
		
		FRHICopyTextureInfo CopyInfo {};
		CopyInfo.SourcePosition = FIntVector{ OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0 };
		CopyInfo.DestPosition = CopyInfo.SourcePosition;
		CopyInfo.Size = FIntVector{ OutputViewport.Rect.Size().X, OutputViewport.Rect.Size().Y, 1 };
		AddCopyTexturePass(GraphBuilder, ComputeRenderTarget, Output.Texture, CopyInfo);
	}
	else
	{
		FUpscaleRasterParameters* RasterPassParameters = GraphBuilder.AllocParameters<FUpscaleRasterParameters>();
		RasterPassParameters->UpscaleParameters = PassParameters;
		RasterPassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		FUpscalePS::FPermutationDomain PixelPermutationVector;
		PixelPermutationVector.Set<FUpscalePS::FAlphaChannelDim>(IsPostProcessingWithAlphaChannelSupported());
		PixelPermutationVector.Set<FUpscalePS::FMethodDimension>(Method);
		TShaderMapRef<FUpscalePS> PixelShader(View.ShaderMap, PixelPermutationVector);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("Upscale(%s Method=%d%s%s) %dx%d -> %dx%d",
				StageName,
				int32(Method),
				PixelPermutationVector.Get<FUpscalePS::FAlphaChannelDim>() ? TEXT(" Alpha") : TEXT(""),
				bApplyLensDistortion ? TEXT(" LensDistortion") : TEXT(""),
				Inputs.SceneColor.ViewRect.Width(), Inputs.SceneColor.ViewRect.Height(),
				Output.ViewRect.Width(), Output.ViewRect.Height()),
			RasterPassParameters,
			ERDGPassFlags::Raster,
			[&View, bApplyLensDistortion, PixelShader, RasterPassParameters, InputViewport, OutputViewport](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetViewport(OutputViewport.Rect.Min.X, OutputViewport.Rect.Min.Y, 0.0f, OutputViewport.Rect.Max.X, OutputViewport.Rect.Max.Y, 1.0f);

			TShaderRef<FShader> VertexShader;
			if (bApplyLensDistortion)
			{
				TShaderMapRef<FUpscaleVS> TypedVertexShader(View.ShaderMap);
				SetScreenPassPipelineState(RHICmdList, FScreenPassPipelineState(TypedVertexShader, PixelShader));
				SetShaderParameters(RHICmdList, TypedVertexShader, TypedVertexShader.GetVertexShader(), *RasterPassParameters);
				VertexShader = TypedVertexShader;
			}
			else
			{
				TShaderMapRef<FScreenPassVS> TypedVertexShader(View.ShaderMap);
				SetScreenPassPipelineState(RHICmdList, FScreenPassPipelineState(TypedVertexShader, PixelShader));
				VertexShader = TypedVertexShader;
			}
			check(VertexShader.IsValid());

			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *RasterPassParameters);

			if (bApplyLensDistortion)
			{
				TArray<uint32> IndexBuffer;

				const uint32 Width = RasterPassParameters->UpscaleParameters.GridDimensions.X;
				const uint32 Height = RasterPassParameters->UpscaleParameters.GridDimensions.Y;
				const uint32 NumVertices = (Width + 1) * (Height + 1);
				const uint32 NumTriangles = Width * Height * 2;
				const uint32 NumIndices = NumTriangles * 3;
			
				IndexBuffer.AddUninitialized(NumIndices);

				uint32* Out = (uint32*)IndexBuffer.GetData();

				for(uint32 y = 0; y < Height; ++y)
				{
					for(uint32 x = 0; x < Width; ++x)
					{
						// left top to bottom right in reading order
						uint32 Index00 = x  + y * (Width + 1);
						uint32 Index10 = Index00 + 1;
						uint32 Index01 = Index00 + (Width + 1);
						uint32 Index11 = Index01 + 1;
					
						// triangle A
						*Out++ = Index00; *Out++ = Index01; *Out++ = Index10;

						// triangle B
						*Out++ = Index11; *Out++ = Index10; *Out++ = Index01;
					}
				}

				// Create index buffer. Fill buffer with initial data upon creation
				FBufferRHIRef IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("LensDistortionIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(IndexBuffer));

			
				FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
				UE::Renderer::PostProcess::SetDrawRectangleParameters(BatchedParameters, VertexShader.GetShader(),
					0, 0, OutputViewport.Rect.Width(), OutputViewport.Rect.Height(),
					InputViewport.Rect.Min.X, InputViewport.Rect.Min.Y, InputViewport.Rect.Width(), InputViewport.Rect.Height(),
					OutputViewport.Rect.Size(),
					InputViewport.Extent);
				RHICmdList.SetBatchedShaderParameters(VertexShader.GetVertexShader(), BatchedParameters);

				// no vertex buffer needed as we compute it in VS
				RHICmdList.SetStreamSource(0, nullptr, 0);

				RHICmdList.DrawIndexedPrimitive(
					IndexBufferRHI,
					/*BaseVertexIndex=*/ 0,
					/*MinIndex=*/ 0,
					/*NumVertices=*/ NumVertices,
					/*StartIndex=*/ 0,
					/*NumPrimitives=*/ NumTriangles,
					/*NumInstances=*/ 1
					);
			}
			else
			{
				DrawRectangle(
					RHICmdList,
					// Output Rect (RHI viewport relative).
					0, 0, OutputViewport.Rect.Width(), OutputViewport.Rect.Height(),
					// Input Rect
					InputViewport.Rect.Min.X, InputViewport.Rect.Min.Y, InputViewport.Rect.Width(), InputViewport.Rect.Height(),
					OutputViewport.Rect.Size(),
					InputViewport.Extent,
					VertexShader,
					EDRF_UseTriangleOptimization);
			}
		});
	}

	return MoveTemp(Output);
}
