// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OverridePassSequence.h"
#include "PostProcess/LensDistortion.h"
#include "SceneView.h"

enum class EUpscaleMethod : uint8
{
	None,
	Nearest,
	Bilinear,
	Directional,
	CatmullRom,
	Lanczos,
	Gaussian,
	SmoothStep,
	Area,
	MAX
};

EUpscaleMethod GetUpscaleMethod();
EUpscaleMethod GetMobileUpscaleMethod();

enum class EUpscaleStage
{
	// Upscaling from the primary to the secondary view rect. The override output cannot be valid when using this stage.
	PrimaryToSecondary,

	// Upscaling in one pass to the final target size.
	PrimaryToOutput,

	// Upscaling from the secondary view rect to the final view size.
	SecondaryToOutput,

	MAX
};

/** Interface for custom spatial upscaling algorithm meant to be set on the FSceneViewFamily by ISceneViewExtension::BeginRenderViewFamily(). */
class ISpatialUpscaler : public ISceneViewFamilyExtention
{
public:
	struct FInputs
	{
		// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
		FScreenPassRenderTarget OverrideOutput;

		// [Required] The input scene color and view rect.
		FScreenPassTexture SceneColor;

		// Whether this is a secondary upscale to the final view family target.
		EUpscaleStage Stage = EUpscaleStage::MAX;
	};

	virtual const TCHAR* GetDebugName() const = 0;

	/** Create a new ISpatialUpscaler interface for a new view family. */
	virtual ISpatialUpscaler* Fork_GameThread(const class FSceneViewFamily& ViewFamily) const = 0;

	virtual FScreenPassTexture AddPasses(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FInputs& PassInputs) const = 0;

	static RENDERER_API FScreenPassTexture AddDefaultUpscalePass(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FInputs& PassInputs,
		EUpscaleMethod Method,
		FLensDistortionLUT LensDistortionLUT = FLensDistortionLUT());
};
