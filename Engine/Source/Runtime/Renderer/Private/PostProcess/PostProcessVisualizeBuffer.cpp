// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessVisualizeBuffer.h"
#include "HighResScreenshot.h"
#include "PostProcess/PostProcessMaterial.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "PostProcess/PostProcessDownsample.h"
#include "ImagePixelData.h"
#include "ImageWriteStream.h"
#include "ImageWriteTask.h"
#include "ImageWriteQueue.h"
#include "HighResScreenshot.h"
#include "BufferVisualizationData.h"
#include "SceneRendering.h"
#include "UnrealEngine.h"
#include "PathTracing.h"
#include "Materials/Material.h"
#include "Internationalization/Internationalization.h"

class FVisualizeBufferPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeBufferPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeBufferPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER(FLinearColor, SelectionColor)
		SHADER_PARAMETER(float, InvGamma)
		SHADER_PARAMETER(float, bShowTileBorder)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeBufferPS, "/Engine/Private/PostProcessVisualizeBuffer.usf", "MainPS", SF_Pixel);

FScreenPassTexture AddVisualizeBufferPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeBufferInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeBuffer");

	// Re-use the scene color as the output if no override was provided.
	if (Output.IsValid())
	{
		AddDrawTexturePass(GraphBuilder, View, Inputs.SceneColor, Output);

		// All remaining passes are load.
		Output.LoadAction = ERenderTargetLoadAction::ELoad;
	}
	// Otherwise, reuse the scene color as the output.
	else
	{
		Output = FScreenPassRenderTarget(Inputs.SceneColor, ERenderTargetLoadAction::ELoad);
	}

	struct FTileLabel
	{
		FString Label;
		FIntPoint Location;
	};

	TArray<FTileLabel> TileLabels;
	TileLabels.Reserve(Inputs.Tiles.Num());

	const int32 MaxTilesX = 4;
	const int32 MaxTilesY = 4;
	const int32 TileWidth = Output.ViewRect.Width() / MaxTilesX;
	const int32 TileHeight = Output.ViewRect.Height() / MaxTilesY;

	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	for (int32 TileIndex = 0; TileIndex < Inputs.Tiles.Num(); ++TileIndex)
	{
		const FVisualizeBufferTile& Tile = Inputs.Tiles[TileIndex];

		// The list can contain invalid entries to keep the indices static.
		if (!Tile.Input.IsValid())
		{
			continue;
		}

		const int32 TileX = TileIndex % MaxTilesX;
		const int32 TileY = TileIndex / MaxTilesX;

		FScreenPassTextureViewport OutputViewport(Output);
		OutputViewport.Rect.Min = OutputViewport.Rect.Min + FIntPoint(TileX * TileWidth, TileY * TileHeight);
		OutputViewport.Rect.Max = OutputViewport.Rect.Min + FIntPoint(TileWidth, TileHeight);

		const FLinearColor SelectionColor = Tile.bSelected ? FLinearColor::Yellow : FLinearColor::Transparent;

		FVisualizeBufferPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeBufferPS::FParameters>();
		PassParameters->Output = GetScreenPassTextureViewportParameters(OutputViewport);
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		PassParameters->InputTexture = Tile.Input.Texture;
		PassParameters->InputSampler = BilinearClampSampler;
		PassParameters->SelectionColor = SelectionColor;
		PassParameters->InvGamma = Tile.bNeedsGammaCorrection ? (1.0f / GEngine->GetDisplayGamma()) : 1.0f;
		PassParameters->bShowTileBorder = 1.0f;

		const FScreenPassTextureViewport InputViewport(Tile.Input);

		TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FVisualizeBufferPS> PixelShader(View.ShaderMap);
		FRHIBlendState* BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();

		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Tile: %s", *Tile.Label), View, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, PassParameters);

		FTileLabel TileLabel;
		TileLabel.Label = Tile.Label;
		TileLabel.Location.X = 8 + TileX * TileWidth;
		TileLabel.Location.Y = (TileY + 1) * TileHeight - 19;
		TileLabels.Add(TileLabel);
	}

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("Labels"), View, Output, [LocalTileLabels = MoveTemp(TileLabels)](FCanvas& Canvas)
	{
		Canvas.SetBaseTransform(FMatrix(FScaleMatrix(Canvas.GetDPIScale()) * Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

		const FLinearColor LabelColor(1, 1, 0);
		for (const FTileLabel& TileLabel : LocalTileLabels)
		{
			const float DPIScale = Canvas.GetDPIScale();
			Canvas.DrawShadowedString(TileLabel.Location.X / DPIScale, TileLabel.Location.Y / DPIScale, *TileLabel.Label, GetStatsFont(), LabelColor);
		}
	});

	return MoveTemp(Output);
}

bool IsVisualizeGBufferEnabled(const FViewInfo& View)
{
	return GetActiveBufferVisualizationMaterial(View) != nullptr
		|| IsVisualizeGBufferOverviewEnabled(View)
		|| IsVisualizeGBufferDumpToFileEnabled(View)
		|| IsVisualizeGBufferDumpToPipeEnabled(View);
}

UMaterial* GetActiveBufferVisualizationMaterial(const FViewInfo& View)
{
	if (View.Family->EngineShowFlags.VisualizeBuffer && GetBufferVisualizationData().IsInitialized())
	{
		UMaterialInterface* VisMaterial = GetBufferVisualizationData().GetMaterial(View.CurrentBufferVisualizationMode);
		UMaterial* Material = VisMaterial ? VisMaterial->GetMaterial() : nullptr;

		if (Material)
		{
			return Material;
		}
	}

	return nullptr;
}

bool IsVisualizeGBufferOverviewEnabled(const FViewInfo& View)
{
	return View.Family->EngineShowFlags.VisualizeBuffer && View.CurrentBufferVisualizationMode == NAME_None;
}

bool IsVisualizeGBufferDumpToFileEnabled(const FViewInfo& View)
{
	static const auto CVarDumpFrames = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.BufferVisualizationDumpFrames"));

	const bool bDumpHighResolutionScreenshot = GIsHighResScreenshot && GetHighResScreenshotConfig().bDumpBufferVisualizationTargets;

	const bool bFrameDumpAllowed = CVarDumpFrames->GetValueOnRenderThread() != 0 || bDumpHighResolutionScreenshot;

	const bool bFrameDumpRequested = View.FinalPostProcessSettings.bBufferVisualizationDumpRequired;

	return (bFrameDumpRequested && bFrameDumpAllowed);
}

bool IsVisualizeGBufferDumpToPipeEnabled(const FViewInfo& View)
{
	return View.FinalPostProcessSettings.BufferVisualizationPipes.Num() > 0;
}

TUniquePtr<FImagePixelData> ReadbackPixelData(FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, FIntRect SourceRect)
{
	check(Texture);
	check(Texture->GetTexture2D());

	const int32 MSAAXSamples = Texture->GetNumSamples();
	SourceRect.Min.X *= MSAAXSamples;
	SourceRect.Max.X *= MSAAXSamples;

	// todo: SourceRect is not clipped to Texture bounds

	switch (Texture->GetFormat())
	{
	case PF_FloatRGBA:
	{
		TArray<FFloat16Color> RawPixels;
		RawPixels.SetNum(SourceRect.Width() * SourceRect.Height());
		RHICmdList.ReadSurfaceFloatData(Texture, SourceRect, RawPixels, (ECubeFace)0, 0, 0);
		TUniquePtr<TImagePixelData<FFloat16Color>> PixelData = MakeUnique<TImagePixelData<FFloat16Color>>(SourceRect.Size(), TArray64<FFloat16Color>(MoveTemp(RawPixels)));

		check(PixelData->IsDataWellFormed());
		return PixelData;
	}

	case PF_A32B32G32R32F:
	case PF_A2B10G10R10:
	{
		FReadSurfaceDataFlags ReadDataFlags(RCM_MinMax);
		ReadDataFlags.SetLinearToGamma(false);

		TArray<FLinearColor> RawPixels;
		RawPixels.SetNum(SourceRect.Width() * SourceRect.Height());
		RHICmdList.ReadSurfaceData(Texture, SourceRect, RawPixels, ReadDataFlags);
		TUniquePtr<TImagePixelData<FLinearColor>> PixelData = MakeUnique<TImagePixelData<FLinearColor>>(SourceRect.Size(), TArray64<FLinearColor>(MoveTemp(RawPixels)));

		check(PixelData->IsDataWellFormed());
		return PixelData;
	}

	case PF_R8G8B8A8:
	case PF_B8G8R8A8:
	{
		FReadSurfaceDataFlags ReadDataFlags;
		ReadDataFlags.SetLinearToGamma(false);

		TArray<FColor> RawPixels;
		RawPixels.SetNum(SourceRect.Width() * SourceRect.Height());
		RHICmdList.ReadSurfaceData(Texture, SourceRect, RawPixels, ReadDataFlags);
		TUniquePtr<TImagePixelData<FColor>> PixelData = MakeUnique<TImagePixelData<FColor>>(SourceRect.Size(), TArray64<FColor>(MoveTemp(RawPixels)));

		check(PixelData->IsDataWellFormed());
		return PixelData;
	}
	}

	return nullptr;
}

void AddDumpToPipePass(FRDGBuilder& GraphBuilder, FScreenPassTexture Input, FImagePixelPipe* OutputPipe)
{
	check(Input.IsValid());
	check(OutputPipe);
	AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("DumpToPipe(%s)", Input.Texture->Name), Input.Texture,
		[Input, OutputPipe](FRHICommandListImmediate& RHICmdList)
	{
		OutputPipe->Push(ReadbackPixelData(RHICmdList, Input.Texture->GetRHI(), Input.ViewRect));
	});
}

void AddDumpToFilePass(FRDGBuilder& GraphBuilder, FScreenPassTexture Input, const FString& Filename)
{
	check(Input.IsValid());

	FHighResScreenshotConfig& HighResScreenshotConfig = GetHighResScreenshotConfig();

	if (!ensureMsgf(HighResScreenshotConfig.ImageWriteQueue, TEXT("Unable to write images unless FHighResScreenshotConfig::Init has been called.")))
	{
		return;
	}

	if (GIsHighResScreenshot && HighResScreenshotConfig.CaptureRegion.Area())
	{
		// todo: CaptureRegion is not clipped to Texture bounds
		Input.ViewRect = HighResScreenshotConfig.CaptureRegion;
	}

	AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("DumpToFile(%s)", Input.Texture->Name), Input.Texture,
		[&HighResScreenshotConfig, Input, Filename](FRHICommandListImmediate& RHICmdList)
	{
		// this is where HighResShot bDumpBufferVisualizationTargets are written to EXRs

		// @todo Oodle alternative : use the exact same pixelformat that this buffer would have in the renderer
		//	 use the DDS writer which can output arbitrary pixel formats
		//	 do no format conversions so we dump the exact same bits the game renderer would see

		TUniquePtr<FImagePixelData> PixelData = ReadbackPixelData(RHICmdList, Input.Texture->GetRHI(), Input.ViewRect);

		if (!PixelData.IsValid())
		{
			return;
		}

		TUniquePtr<FImageWriteTask> ImageTask = MakeUnique<FImageWriteTask>();
		ImageTask->PixelData = MoveTemp(PixelData);

		HighResScreenshotConfig.PopulateImageTaskParams(*ImageTask);
		ImageTask->Filename = Filename;

		if (ImageTask->PixelData->GetType() == EImagePixelType::Color)
		{
			// Always write opaque alpha
			ImageTask->AddPreProcessorToSetAlphaOpaque();

			// ImageTask->PixelData should be sRGB
			//  it will gamma correct automatically if written to EXR
		}
		else if(ImageTask->Format == EImageFormat::PNG)
		{
			// PNGs can't have 0 alpha or RGB data is destroyed.
			ImageTask->AddPreProcessorToSetAlphaOpaque();
		}

		HighResScreenshotConfig.ImageWriteQueue->Enqueue(MoveTemp(ImageTask));
	});
}

void AddDumpToColorArrayPass(FRDGBuilder& GraphBuilder, FScreenPassTexture Input, TArray<FColor>* OutputColorArray, FIntPoint* OutputExtents)
{
	check(Input.IsValid());
	check(OutputColorArray);
	AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("DumpToPipe(%s)", Input.Texture->Name), Input.Texture,
		[Input, OutputColorArray, OutputExtents](FRHICommandListImmediate& RHICmdList)
	{
		// By design, we want the whole surface, not the view rectangle, as this code is used for generating a screenshot
		// mask surface that needs to match the corresponding screenshot color surface.  The scene may render as a viewport
		// inside a larger surface, but the screenshot logic emits the entire surface, not just the viewport, and we must
		// do the same for correct results (also to prevent an assert in FHighResScreenshotConfig::MergeMaskIntoAlpha).
		// See FSceneView, UnscaledViewRect versus UnconstrainedViewRect.
		FIntRect WholeSurfaceRect;
		WholeSurfaceRect.Min = FIntPoint(0, 0);
		WholeSurfaceRect.Max = Input.Texture->Desc.Extent;

		RHICmdList.ReadSurfaceData(Input.Texture->GetRHI(), WholeSurfaceRect, *OutputColorArray, FReadSurfaceDataFlags());
		*OutputExtents = Input.Texture->Desc.Extent;
	});
}

EPixelFormat OverridePostProcessingColorFormat(const EPixelFormat InFormat)
{
	EPixelFormat OutputFormat = InFormat;
	
	static const auto CVarPostProcessingColorFormat = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PostProcessingColorFormat"));

	if (CVarPostProcessingColorFormat && CVarPostProcessingColorFormat->GetValueOnRenderThread() == 1)
	{
		if (OutputFormat == PF_FloatRGBA)
		{
			OutputFormat = PF_A32B32G32R32F;
		}
	}
	
	return OutputFormat;
}

void AddVisualizeBufferMaterialPass(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View, 
	const EBlendableLocation Location, 
	const FPostProcessMaterialInputs& PostProcessMaterialInputs, 
	const bool bOutputInHDR,
	FVisualizeBufferContext& Context)
{
	FPostProcessMaterialInputs VisualizeBufferPPMInputs = PostProcessMaterialInputs;

	if (UMaterial* VisMaterial = GetActiveBufferVisualizationMaterial(View))
	{
		if (VisMaterial->BlendableLocation == Location || Location == EBlendableLocation::BL_MAX)
		{
			VisualizeBufferPPMInputs.bAllowSceneColorInputAsOutput = false;
			Context.Buffer = AddPostProcessMaterialPass(GraphBuilder, View, VisualizeBufferPPMInputs, VisMaterial);
		}
	}

	if (IsVisualizeGBufferOverviewEnabled(View)
		|| IsVisualizeGBufferDumpToFileEnabled(View)
		|| IsVisualizeGBufferDumpToPipeEnabled(View))
	{
		const FFinalPostProcessSettings& PostProcessSettings = View.FinalPostProcessSettings;
		
		int NumMaterials = PostProcessSettings.BufferVisualizationOverviewMaterials.Num();
		for (int Index = 0; Index < NumMaterials; ++Index)
		{
			UMaterialInterface* VisMaterialInterface = PostProcessSettings.BufferVisualizationOverviewMaterials[Index];
			UMaterial* VisMaterial = VisMaterialInterface ? VisMaterialInterface->GetMaterial() : nullptr;

			if (VisMaterial && (VisMaterial->BlendableLocation == Location || Location == EBlendableLocation::BL_MAX))
			{
				const TSharedPtr<FImagePixelPipe, ESPMode::ThreadSafe>* OutputPipe = PostProcessSettings.BufferVisualizationPipes.Find(VisMaterialInterface->GetFName());
				const bool bIsValidOutputPipe = OutputPipe && OutputPipe->IsValid();

				if (bIsValidOutputPipe && OutputPipe->Get()->bIsExpecting32BitPixelData)
				{
					VisualizeBufferPPMInputs.OutputFormat = PF_A32B32G32R32F;
				}
				else
				{
					// Respect the r.PostProcessingColorFormat cvar just like the main rendering path
					EPixelFormat OutputFormat = OverridePostProcessingColorFormat(bOutputInHDR ? PF_FloatRGBA : PF_Unknown);
					VisualizeBufferPPMInputs.OutputFormat = OutputFormat;
				}
				VisualizeBufferPPMInputs.bAllowSceneColorInputAsOutput = false;

				Context.OverviewBuffers.SetNum(NumMaterials);
				Context.OverviewBuffers[Index] = AddPostProcessMaterialPass(GraphBuilder, View, VisualizeBufferPPMInputs, VisMaterialInterface);
			}
		}
	}
}

static bool RunsBeforeTonemapping(UMaterialInterface* MaterialInterface)
{
	UMaterial* BaseMaterial = MaterialInterface->GetMaterial();
	EBlendableLocation BlendableLocation = MaterialInterface->GetBlendableLocation(BaseMaterial);

	bool bMaterialRunsBeforeTonemapping = false;
	switch (BlendableLocation)
	{
		case BL_SceneColorBeforeDOF:
		case BL_SceneColorAfterDOF:
		case BL_TranslucencyAfterDOF:
		case BL_SSRInput:
		case BL_SceneColorBeforeBloom:
			bMaterialRunsBeforeTonemapping = true;
			break;
		case BL_ReplacingTonemapper:
		case BL_SceneColorAfterTonemapping:
			bMaterialRunsBeforeTonemapping = false;
			break;
		default: ensure(0);
	}

	return bMaterialRunsBeforeTonemapping;
}

FScreenPassTexture AddVisualizeGBufferPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FVisualizeGBufferInputs& Inputs,
	FVisualizeBufferContext& Context)
{
	const FFinalPostProcessSettings& PostProcessSettings = View.FinalPostProcessSettings;

	const bool bOverview = IsVisualizeGBufferOverviewEnabled(View);
	const bool bDumpToFile = IsVisualizeGBufferDumpToFileEnabled(View);
	const bool bDumpToPipe = IsVisualizeGBufferDumpToPipeEnabled(View);
	const bool bActiveBufferVisualizationMaterial = GetActiveBufferVisualizationMaterial(View) != nullptr;

	check(Inputs.SceneColor.IsValid());
	check(bActiveBufferVisualizationMaterial || bDumpToFile || bOverview || PostProcessSettings.BufferVisualizationPipes.Num() > 0);

	if (bActiveBufferVisualizationMaterial)
	{
		if (Context.Buffer.IsValid())
		{
			FScreenPassRenderTarget Output = Inputs.OverrideOutput;

			if (!Output.IsValid())
			{
				Output = FScreenPassRenderTarget(Inputs.SceneColor, ERenderTargetLoadAction::ELoad);
			}

			FScreenPassTextureViewport OutputViewport(Output);

			bool bMaterialRunsBeforeTonemapping = RunsBeforeTonemapping(GetActiveBufferVisualizationMaterial(View));

			FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

			FVisualizeBufferPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeBufferPS::FParameters>();
			PassParameters->Output = GetScreenPassTextureViewportParameters(OutputViewport);
			PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
			PassParameters->InputTexture = Context.Buffer.Texture;
			PassParameters->InputSampler = BilinearClampSampler;
			PassParameters->SelectionColor = FLinearColor::Transparent;
			PassParameters->InvGamma = bMaterialRunsBeforeTonemapping ? (1.0f / GEngine->GetDisplayGamma()) : 1.0f;
			PassParameters->bShowTileBorder = 0.0f;

			const FScreenPassTextureViewport InputViewport(Context.Buffer);

			TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
			TShaderMapRef<FVisualizeBufferPS> PixelShader(View.ShaderMap);
			FRHIBlendState* BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI();

			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("VisualizeBuffer"), View, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, PassParameters);

			return Output;
		}
	}
	else if(!Context.OverviewBuffers.IsEmpty())
	{
		TArray<FVisualizeBufferTile> Tiles;

		RDG_EVENT_SCOPE(GraphBuilder, "VisualizeGBufferOverview");

		const FString& BaseFilename = PostProcessSettings.BufferVisualizationDumpBaseFilename;

		int NumMaterials = PostProcessSettings.BufferVisualizationOverviewMaterials.Num();
		for (int Index = 0; Index < NumMaterials; ++Index)
		{
			UMaterialInterface* MaterialInterface = PostProcessSettings.BufferVisualizationOverviewMaterials[Index];
			FScreenPassTexture Output = Context.OverviewBuffers[Index];

			if (!MaterialInterface || !Output.IsValid())
			{
				// Add an empty tile to keep the location of each static on the grid.
				Tiles.Emplace();
				continue;
			}

			const FString MaterialName = MaterialInterface->GetName();

			RDG_EVENT_SCOPE(GraphBuilder, "%s", *MaterialName);

			const TSharedPtr<FImagePixelPipe, ESPMode::ThreadSafe>* OutputPipe = PostProcessSettings.BufferVisualizationPipes.Find(MaterialInterface->GetFName());
			const bool bIsValidOutputPipe = OutputPipe && OutputPipe->IsValid();

			if (bIsValidOutputPipe)
			{
				AddDumpToPipePass(GraphBuilder, Output, OutputPipe->Get());
			}

			if (bDumpToFile)
			{
				// First off, allow the user to specify the pass as a format arg (using {material})
				TMap<FString, FStringFormatArg> FormatMappings;
				FormatMappings.Add(TEXT("material"), MaterialName);

				FString MaterialFilename = FString::Format(*BaseFilename, FormatMappings);

				// If the format made no change to the string, we add the name of the material to ensure uniqueness
				if (MaterialFilename == BaseFilename)
				{
					MaterialFilename = BaseFilename + TEXT("_") + MaterialName;
				}

				// always makes a .png filename even when bCaptureHDR was set, which will actually save an EXR
				MaterialFilename.Append(TEXT(".png"));

				AddDumpToFilePass(GraphBuilder, Output, MaterialFilename);
			}

			if (bOverview)
			{
				FDownsamplePassInputs DownsampleInputs;
				DownsampleInputs.Name = TEXT("MaterialHalfSize");
				DownsampleInputs.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, Output);
				DownsampleInputs.Flags = EDownsampleFlags::ForceRaster;
				DownsampleInputs.Quality = EDownsampleQuality::Low;

				FScreenPassTexture HalfSize = AddDownsamplePass(GraphBuilder, View, DownsampleInputs);

				DownsampleInputs.Name = TEXT("MaterialQuarterSize");
				DownsampleInputs.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, HalfSize);

				bool bMaterialRunsBeforeTonemapping = RunsBeforeTonemapping(MaterialInterface);
				const bool bIsMobilePlatform = IsMobilePlatform(View.GetShaderPlatform());

				FVisualizeBufferTile Tile;
				Tile.Input = AddDownsamplePass(GraphBuilder, View, DownsampleInputs);

				const FName MaterialFName = MaterialInterface->GetFName();
				if (!bIsMobilePlatform || GetBufferVisualizationData().GetMaterialEnableOnMobile(MaterialFName))
				{
					Tile.Label = GetBufferVisualizationData().GetMaterialDisplayName(MaterialFName).ToString();
				}
				else
				{
					const FText NotImplementedOnMobile = NSLOCTEXT("PostProcessVisualizeBuffer", "NotImplementedOnMobile", "(Not implemented on Mobile)");
					Tile.Label = FText::Format(
						NSLOCTEXT("PostProcessVisualizeBuffer", "TileNotImplementedOnMobile", "{0} {1}"),
						GetBufferVisualizationData().GetMaterialDisplayName(MaterialFName),
						NotImplementedOnMobile).ToString();
				}

				// VisualizeGBufferOverviewPass runs after tonemapping, so any materials that were expecting to run before
				// need to be tonemapped manually.
				Tile.bNeedsGammaCorrection = bMaterialRunsBeforeTonemapping;
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				Tile.bSelected = 
					PostProcessSettings.bBufferVisualizationOverviewTargetIsSelected &&
					PostProcessSettings.BufferVisualizationOverviewSelectedTargetMaterialName == MaterialName;
	#endif
				Tiles.Add(Tile);
			}
		}

		if (bOverview)
		{
			FVisualizeBufferInputs PassInputs;
			PassInputs.OverrideOutput = Inputs.OverrideOutput;
			PassInputs.SceneColor = Inputs.SceneColor;
			PassInputs.Tiles = Tiles;

			return AddVisualizeBufferPass(GraphBuilder, View, PassInputs);
		}
	}

	if (Inputs.OverrideOutput.IsValid())
	{
		AddDrawTexturePass(GraphBuilder, View, Inputs.SceneColor, Inputs.OverrideOutput);
	}
	return Inputs.SceneColor;
}
