// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OverridePassSequence.h"
#include "Engine/BlendableInterface.h"

class UMaterial;

// True if any of the buffer visualization systems are enabled
bool IsVisualizeGBufferEnabled(const FViewInfo& View);

// Returns the buffer visualization material used to render in this view, if any is enabled.
UMaterial* GetActiveBufferVisualizationMaterial(const FViewInfo& View);

// Returns whether the gbuffer overview visualization pass needs to render on screen.
bool IsVisualizeGBufferOverviewEnabled(const FViewInfo& View);

// Returns whether the gubffer visualization pass needs to dump targets to files.
bool IsVisualizeGBufferDumpToFileEnabled(const FViewInfo& View);

// Returns whether the gbuffer visualization pass needs to dump to a pipe.
bool IsVisualizeGBufferDumpToPipeEnabled(const FViewInfo& View);

// Returns whether the gbuffer visualization pass should output in floating point format.
bool IsVisualizeGBufferInFloatFormat();

struct FVisualizeBufferContext
{
	FScreenPassTexture Buffer {};
	TArray<FScreenPassTexture, TInlineAllocator<16>> OverviewBuffers {};
};

void AddVisualizeBufferMaterialPass(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View, 
	const EBlendableLocation Location, 
	const FPostProcessMaterialInputs& PostProcessMaterialInputs,
	const bool bOutputInHDR,
	FVisualizeBufferContext& Context);

struct FVisualizeGBufferInputs
{
	FScreenPassRenderTarget OverrideOutput;

	// The current scene color being processed.
	FScreenPassTexture SceneColor;

	// Whether to emit outputs in HDR.
	bool bOutputInHDR = false;
};

FScreenPassTexture AddVisualizeGBufferPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeGBufferInputs& Inputs, FVisualizeBufferContext& Context);

struct FVisualizeBufferTile
{
	// The input texture to visualize.
	FScreenPassTexture Input;

	// The label of the tile shown on the visualizer.
	FString Label;

	// Whether the tile is shown as selected.
	bool bSelected = false;

	// Whether to apply gamma correction to this tile, or to display unmodified.
	bool bNeedsGammaCorrection = false;
};

struct FVisualizeBufferInputs
{
	FScreenPassRenderTarget OverrideOutput;

	// The scene color input to propagate.
	FScreenPassTexture SceneColor;

	// The array of tiles to render onto the scene color texture.
	TArrayView<const FVisualizeBufferTile> Tiles;
};

FScreenPassTexture AddVisualizeBufferPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeBufferInputs& Inputs);
