// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessVisualizeCalibrationMaterial.h"
#include "EngineGlobals.h"
#include "SceneView.h"

static bool IsPostProcessVisualizeCalibrationColorMaterialEnabled(const FSceneView& View)
{
	return View.Family->EngineShowFlags.VisualizeCalibrationColor && View.CurrentVisualizeCalibrationColorMaterialName != NAME_None;
}

static bool IsPostProcessVisualizeCalibrationGrayscaleMaterialEnabled(const FSceneView& View) 
{
	return View.Family->EngineShowFlags.VisualizeCalibrationGrayscale && View.CurrentVisualizeCalibrationGrayscaleMaterialName != NAME_None;
}

static bool IsPostProcessVisualizeCalibrationCustomMaterialEnabled(const FSceneView& View)
{
	return View.Family->EngineShowFlags.VisualizeCalibrationCustom && View.CurrentVisualizeCalibrationCustomMaterialName != NAME_None;
}

bool IsPostProcessVisualizeCalibrationMaterialEnabled(const FSceneView& View)
{
	return (IsPostProcessVisualizeCalibrationColorMaterialEnabled(View) || IsPostProcessVisualizeCalibrationGrayscaleMaterialEnabled(View) || IsPostProcessVisualizeCalibrationCustomMaterialEnabled(View));
}

// Returns whether the Calibration custom material pass needs to render on screen.
const UMaterialInterface* GetPostProcessVisualizeCalibrationMaterialInterface(const FSceneView& View)
{
	if (IsPostProcessVisualizeCalibrationColorMaterialEnabled(View))
	{
		return View.FinalPostProcessSettings.VisualizeCalibrationColorMaterial;
	}
	else if (IsPostProcessVisualizeCalibrationGrayscaleMaterialEnabled(View))
	{
		return View.FinalPostProcessSettings.VisualizeCalibrationGrayscaleMaterial;
	}
	else if (IsPostProcessVisualizeCalibrationCustomMaterialEnabled(View))
	{
		return View.FinalPostProcessSettings.VisualizeCalibrationCustomMaterial;
	}

	return NULL;
}
