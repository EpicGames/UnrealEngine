// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

// Returns whether any calibration material pass is enabled.
bool IsPostProcessVisualizeCalibrationMaterialEnabled(const FSceneView& View);

// Returns whether any calibration material pass is enabled.
const UMaterialInterface* GetPostProcessVisualizeCalibrationMaterialInterface(const FSceneView& View);
