// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessVisualizeComplexity.h"
#include "CanvasTypes.h"
#include "UnrealEngine.h"
#include "SceneRendering.h"
#include "SystemTextures.h"
#include "DataDrivenShaderPlatformInfo.h"

class FVisualizeComplexityApplyPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeComplexityApplyPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeComplexityApplyPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, QuadOverdrawTexture)
		SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
		SHADER_PARAMETER_ARRAY(FLinearColor, ShaderComplexityColors, [MaxNumShaderComplexityColors])
		SHADER_PARAMETER(FIntPoint, UsedQuadTextureSize)
		SHADER_PARAMETER(uint32, bLegend)
		SHADER_PARAMETER(uint32, bShowError)
		SHADER_PARAMETER(uint32, DebugViewShaderMode)
		SHADER_PARAMETER(uint32, ColorSamplingMethod)
		SHADER_PARAMETER(float, ShaderComplexityColorCount)
		SHADER_PARAMETER(float, ComplexityScale)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	enum class EQuadOverdraw
	{
		Disable,
		Enable,
		MAX
	};

	class FQuadOverdraw : SHADER_PERMUTATION_ENUM_CLASS("READ_QUAD_OVERDRAW", EQuadOverdraw);
	using FPermutationDomain = TShaderPermutationDomain<FQuadOverdraw>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FVisualizeComplexityApplyPS::FQuadOverdraw>() == FVisualizeComplexityApplyPS::EQuadOverdraw::Enable)
		{
			return SupportDebugViewShaderMode(DVSM_QuadComplexity, Parameters.Platform);
		}
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MAX_NUM_COMPLEXITY_COLORS"), MaxNumShaderComplexityColors);
		// EVisualizeComplexityColorSamplingMethod values
		OutEnvironment.SetDefine(TEXT("CS_RAMP"), (uint32)FVisualizeComplexityInputs::EColorSamplingMethod::Ramp);
		OutEnvironment.SetDefine(TEXT("CS_LINEAR"), (uint32)FVisualizeComplexityInputs::EColorSamplingMethod::Linear);
		OutEnvironment.SetDefine(TEXT("CS_STAIR"), (uint32)FVisualizeComplexityInputs::EColorSamplingMethod::Stair);
		// EDebugViewShaderMode values
		OutEnvironment.SetDefine(TEXT("DVSM_None"), (uint32)DVSM_None);
		OutEnvironment.SetDefine(TEXT("DVSM_ShaderComplexity"), (uint32)DVSM_ShaderComplexity);
		OutEnvironment.SetDefine(TEXT("DVSM_ShaderComplexityContainedQuadOverhead"), (uint32)DVSM_ShaderComplexityContainedQuadOverhead);
		OutEnvironment.SetDefine(TEXT("DVSM_ShaderComplexityBleedingQuadOverhead"), (uint32)DVSM_ShaderComplexityBleedingQuadOverhead);
		OutEnvironment.SetDefine(TEXT("DVSM_QuadComplexity"), (uint32)DVSM_QuadComplexity);
		OutEnvironment.SetDefine(TEXT("DVSM_LWCComplexity"), (uint32)DVSM_LWCComplexity);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeComplexityApplyPS, "/Engine/Private/ShaderComplexityApplyPixelShader.usf", "Main", SF_Pixel);

float GetMaxShaderComplexityCount(ERHIFeatureLevel::Type FeatureLevel)
{
	switch (FeatureLevel)
	{
	case ERHIFeatureLevel::ES3_1:
		return GEngine->MaxES3PixelShaderAdditiveComplexityCount;
	default:
		return GEngine->MaxPixelShaderAdditiveComplexityCount;
	}
}

FScreenPassTexture AddVisualizeComplexityPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeComplexityInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeComplexity"));
	}

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(Output);

	FVisualizeComplexityApplyPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeComplexityApplyPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
	PassParameters->Output = GetScreenPassTextureViewportParameters(OutputViewport);
	PassParameters->InputTexture = Inputs.SceneColor.Texture;
	PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	{
		const int32 ClampedColorCount = FMath::Min<int32>(Inputs.Colors.Num(), MaxNumShaderComplexityColors);
		if (ClampedColorCount > 0)
		{
			for (int32 ColorIndex = 0; ColorIndex < ClampedColorCount; ColorIndex++)
			{
				PassParameters->ShaderComplexityColors[ColorIndex] = Inputs.Colors[ColorIndex];
			}
			PassParameters->ShaderComplexityColorCount = ClampedColorCount;
		}
		else // Otherwise fallback to a safe value.
		{
			PassParameters->ShaderComplexityColors[0] = FLinearColor::Gray;
			PassParameters->ShaderComplexityColorCount = 1;
		}
	}

	const uint32 ShaderComplexityColorCount = PassParameters->ShaderComplexityColorCount;

	PassParameters->MiniFontTexture = GetMiniFontTexture();

	const FSceneTextures& SceneTextures = View.GetSceneTextures();
	const EDebugViewShaderMode DebugViewShaderMode = View.Family->GetDebugViewShaderMode();

	PassParameters->DebugViewShaderMode = DVSM_ShaderComplexity;
	FVisualizeComplexityApplyPS::EQuadOverdraw QuadOverdrawEnum = FVisualizeComplexityApplyPS::EQuadOverdraw::Disable;

	if ((DebugViewShaderMode == DVSM_QuadComplexity || DebugViewShaderMode == DVSM_ShaderComplexityContainedQuadOverhead) && SceneTextures.DebugAux)
	{
		const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);
		PassParameters->QuadOverdrawTexture = HasBeenProduced(SceneTextures.DebugAux) ? SceneTextures.DebugAux : GSystemTextures.GetZeroUIntDummy(GraphBuilder);
		PassParameters->DebugViewShaderMode = DebugViewShaderMode;
		QuadOverdrawEnum = FVisualizeComplexityApplyPS::EQuadOverdraw::Enable;
	}
	else if (DebugViewShaderMode == DVSM_LWCComplexity)
	{
		PassParameters->DebugViewShaderMode = DVSM_LWCComplexity;
	}

	PassParameters->bLegend = Inputs.bDrawLegend;
	PassParameters->bShowError = PassParameters->DebugViewShaderMode != DVSM_QuadComplexity;
	PassParameters->ColorSamplingMethod = (uint32)Inputs.ColorSamplingMethod;
	PassParameters->ComplexityScale = Inputs.ComplexityScale;
	PassParameters->UsedQuadTextureSize = FIntPoint(View.ViewRect.Size() + FIntPoint(1, 1)) / 2;

	FVisualizeComplexityApplyPS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FVisualizeComplexityApplyPS::FQuadOverdraw>(QuadOverdrawEnum);
	TShaderMapRef<FVisualizeComplexityApplyPS> PixelShader(View.ShaderMap, PermutationVector);

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeComplexity");

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Visualizer"), View, FScreenPassTextureViewport(Output), InputViewport, PixelShader, PassParameters);

	Output.LoadAction = ERenderTargetLoadAction::ELoad;

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("Overlay"), View, Output,
		[Output, &View, ShaderComplexityColorCount](FCanvas& Canvas)
	{
		const float DPIScale = Canvas.GetDPIScale();
		Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale)* Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

		auto DrawString = [&](int X, int Y, const TCHAR* Text, const FLinearColor& Color = FLinearColor(0.5f, 0.5f, 0.5f)){
			Canvas.DrawShadowedString(X / DPIScale, Y / DPIScale, Text, GetStatsFont(), Color);
		};

		const FIntPoint CanvasMin(0, 0);
		const FIntPoint CanvasMax = Output.ViewRect.Max - Output.ViewRect.Min;

		if (View.Family->GetDebugViewShaderMode() == DVSM_QuadComplexity)
		{
			int32 StartX = CanvasMin.X + 62;
			int32 EndX = CanvasMax.X - 66;
			int32 NumOffset = (EndX - StartX) / (ShaderComplexityColorCount - 1);
			for (int32 PosX = StartX, Number = 0; PosX <= EndX; PosX += NumOffset, ++Number)
			{
				FString Line;
				Line = FString::Printf(TEXT("%d"), Number);
				DrawString(PosX, CanvasMax.Y - 87, *Line);
			}
		}
		else
		{
			DrawString(CanvasMin.X + 63, CanvasMax.Y - 51, TEXT("Good"));
			DrawString(CanvasMin.X + 63 + (int32)(Output.ViewRect.Width() * 107.0f / 397.0f), CanvasMax.Y - 51, TEXT("Bad"));
			DrawString(CanvasMax.X - 170, CanvasMax.Y - 51, TEXT("Extremely bad"));

			DrawString(CanvasMin.X + 62, CanvasMax.Y - 87, TEXT("0"));

			if (View.Family->GetDebugViewShaderMode() == DVSM_LWCComplexity)
			{
#if WITH_DEBUG_VIEW_MODES
				extern float GMaxLWCComplexity;
				FString Line = FString::Printf(TEXT("r.ShaderComplexity.MaxLWCComplexity=%d"), (int32)GMaxLWCComplexity);
				DrawString(CanvasMax.X - 430, CanvasMax.Y - 88, *Line);
#endif
			}
			else
			{
				FString Line = FString::Printf(TEXT("MaxShaderComplexityCount=%d"), (int32)GetMaxShaderComplexityCount(View.GetFeatureLevel()));
				DrawString(CanvasMax.X - 330, CanvasMax.Y - 88, *Line);
			}
		}
	});

	return MoveTemp(Output);
}