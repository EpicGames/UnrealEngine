// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "DebugViewModeHelpers.h"
#include "OverridePassSequence.h"

/**
* The number of shader complexity colors from the engine ini that will be passed to the shader. 
* Changing this requires a recompile of the FShaderComplexityApplyPS.
*/
const uint32 MaxNumShaderComplexityColors = 11;
const float NormalizedQuadComplexityValue = 1.f / 16.f;

// Gets the maximum shader complexity count from the ini settings.
float GetMaxShaderComplexityCount(ERHIFeatureLevel::Type InFeatureType);

struct FVisualizeComplexityInputs
{
	enum class EColorSamplingMethod : uint32
	{
		Ramp,
		Linear,
		Stair
	};

	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The input scene color and view rect.
	FScreenPassTexture SceneColor;

	// [Required] The table of colors used for visualization, ordered by least to most complex.
	TArrayView<const FLinearColor> Colors;

	// The method used to derive a color from the sampled complexity value.
	EColorSamplingMethod ColorSamplingMethod = EColorSamplingMethod::Ramp;

	// A scale applied to the sampled scene complexity value prior to colorizing.
	float ComplexityScale = 1.0f;

	// Renders the complexity legend overlay.
	bool bDrawLegend = false;
};

FScreenPassTexture AddVisualizeComplexityPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeComplexityInputs& Inputs);