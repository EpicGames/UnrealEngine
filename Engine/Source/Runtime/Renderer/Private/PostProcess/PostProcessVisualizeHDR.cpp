// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessVisualizeHDR.h"
#include "PostProcess/PostProcessTonemap.h"
#include "Curves/CurveFloat.h"
#include "UnrealEngine.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"
#include "ShaderCompilerCore.h"

TAutoConsoleVariable<float> CVarEyeAdaptationVisualizeLuminanceScale(
	TEXT("r.EyeAdaptation.VisualizeLuminanceScale"),
	1.0f,
	TEXT("Scale applied to output when visualizing luminance.\n"),
	ECVF_RenderThreadSafe);

extern bool IsExtendLuminanceRangeEnabled();

class FVisualizeHDRPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeHDRPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeHDRPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FTonemapperOutputDeviceParameters, OutputDevice)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HDRSceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HistogramTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, HistogramBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, LuminanceTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_SAMPLER(SamplerState, HDRSceneColorSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, LuminanceSampler)
		SHADER_PARAMETER_TEXTURE(Texture2D, MiniFontTexture)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER(float, FilmSlope)
		SHADER_PARAMETER(float, FilmToe)
		SHADER_PARAMETER(float, FilmShoulder)
		SHADER_PARAMETER(float, FilmBlackClip)
		SHADER_PARAMETER(float, FilmWhiteClip)
		SHADER_PARAMETER(FScreenTransform, ColorUVToLuminanceUV)
		SHADER_PARAMETER(float, IlluminanceMeterEnabled)
		SHADER_PARAMETER(float, UsingIlluminance)
		SHADER_PARAMETER(float, LuminanceScale)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// This shader takes a very long time to compile with FXC, so we pre-compile it with DXC first and then forward the optimized HLSL to FXC.
		OutEnvironment.CompilerFlags.Add(CFLAG_PrecompileWithDXC);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeHDRPS, "/Engine/Private/PostProcessVisualizeHDR.usf", "MainPS", SF_Pixel);

bool IsIlluminanceMeterSupportedByView(const FViewInfo& View)
{
	return !IsForwardShadingEnabled(View.GetShaderPlatform()) && !View.Family->EngineShowFlags.PathTracing;
}

FScreenPassTexture AddVisualizeHDRPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeHDRInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneColorBeforeTonemap.IsValid());
	// HistogramTexture (desktop) and HistogramBuffer (mobile) may both be null when auto exposure is disabled
	// we substitute dummy resources in that case, so the overlay still renders, just with an empty histogram.
	checkf(!(Inputs.HistogramTexture && Inputs.HistogramBuffer), TEXT("VisualizeHDR cannot have both HistogramTexture and HistogramBuffer set."));
	check(Inputs.EyeAdaptationBuffer);
	check(Inputs.EyeAdaptationParameters);

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeHDR"));
	}

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(Output);
	const FScreenPassTextureViewport LuminanceViewport = Inputs.Luminance.IsValid() ? FScreenPassTextureViewport(Inputs.Luminance) : FScreenPassTextureViewport();

	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	FRHISamplerState* PointSampler = TStaticSamplerState<SF_Point>::GetRHI();

	const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

	FVisualizeHDRPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeHDRPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
	PassParameters->Output = GetScreenPassTextureViewportParameters(OutputViewport);
	PassParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters->SceneColorSampler = BilinearClampSampler;
	PassParameters->HDRSceneColorTexture = Inputs.SceneColorBeforeTonemap.Texture;
	PassParameters->HDRSceneColorSampler = BilinearClampSampler;
	PassParameters->LuminanceTexture = Inputs.Luminance.IsValid() ? Inputs.Luminance.Texture : GSystemTextures.GetBlackDummy(GraphBuilder);
	PassParameters->LuminanceSampler = PointSampler;
	PassParameters->HistogramTexture = Inputs.HistogramTexture ? Inputs.HistogramTexture : GSystemTextures.GetBlackDummy(GraphBuilder);
	PassParameters->HistogramBuffer = Inputs.HistogramBuffer ? Inputs.HistogramBuffer : GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 4), PF_R32_UINT);
	PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(Inputs.EyeAdaptationBuffer);
	PassParameters->EyeAdaptation = *Inputs.EyeAdaptationParameters;
	PassParameters->OutputDevice = GetTonemapperOutputDeviceParameters(*View.Family);
	PassParameters->MiniFontTexture = GetMiniFontTexture();
	PassParameters->FilmSlope = Settings.FilmSlope;
	PassParameters->FilmToe = Settings.FilmToe;
	PassParameters->FilmShoulder = Settings.FilmShoulder;
	PassParameters->FilmBlackClip = Settings.FilmBlackClip;
	PassParameters->FilmWhiteClip = Settings.FilmWhiteClip;
	PassParameters->ColorUVToLuminanceUV = FScreenTransform::ChangeTextureUVCoordinateFromTo(InputViewport, LuminanceViewport);
	PassParameters->IlluminanceMeterEnabled = IsIlluminanceMeterSupportedByView(View) ? 1.0f : 0.0f;
	PassParameters->UsingIlluminance = IsAutoExposureUsingIlluminanceEnabled(View) ? 1.0f : 0.0f;
	PassParameters->LuminanceScale = CVarEyeAdaptationVisualizeLuminanceScale.GetValueOnRenderThread();

	TShaderMapRef<FVisualizeHDRPS> PixelShader(View.ShaderMap);

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeComplexity");

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Visualizer"), View, OutputViewport, InputViewport, PixelShader, PassParameters);

	Output.LoadAction = ERenderTargetLoadAction::ELoad;

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("Overlay"), View, Output,
		[Output, &View](FCanvas& Canvas)
	{
		const float DPIScale = Canvas.GetDPIScale();

		const EAutoExposureMethod AutoExposureMethod = GetAutoExposureMethod(View);
		const bool bExtendedLuminanceRange = IsExtendLuminanceRangeEnabled();

		const float LuminanceMax = LuminanceMaxFromLensAttenuation();

		float X = Output.ViewRect.Min.X + 30;
		float Y = Output.ViewRect.Min.Y + 28;
		const float YStep = 14;
		const float ColumnWidth = 250;

		FString Line;

		Line = FString::Printf(TEXT("HDR Histogram (EV100, max of RGB)"));
		Canvas.DrawShadowedString(X, Y += YStep, *Line, GetStatsFont(), FLinearColor(1, 1, 1));

		Y += 160;

		switch (AutoExposureMethod)
		{
		case EAutoExposureMethod::AEM_Basic:
			Line = FString::Printf(TEXT("Basic"));
			break;
		case EAutoExposureMethod::AEM_Histogram:
			Line = FString::Printf(TEXT("Histogram"));
			break;
		case EAutoExposureMethod::AEM_Manual:
			Line = FString::Printf(TEXT("Manual"));
			break;
		default:
			Line = FString::Printf(TEXT("Unknown"));
			break;
		}
		Canvas.DrawShadowedString(X, Y += YStep, TEXT("Auto Exposure Method:"), GetStatsFont(), FLinearColor(1, 1, 1));
		Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 1, 1));

		Line = FString::Printf(TEXT("%g%% .. %g%%"), View.FinalPostProcessSettings.AutoExposureLowPercent, View.FinalPostProcessSettings.AutoExposureHighPercent);
		Canvas.DrawShadowedString(X, Y += YStep, TEXT("Percent Low/High:"), GetStatsFont(), FLinearColor(1, 1, 1));
		Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 1, 1));

		if (bExtendedLuminanceRange)
		{
			Line = FString::Printf(TEXT("%.1f .. %.1f"), View.FinalPostProcessSettings.AutoExposureMinBrightness, View.FinalPostProcessSettings.AutoExposureMaxBrightness);
		}
		else
		{
			Line = FString::Printf(TEXT("%.1f .. %.1f"), LuminanceToEV100(LuminanceMax,View.FinalPostProcessSettings.AutoExposureMinBrightness), LuminanceToEV100(LuminanceMax,View.FinalPostProcessSettings.AutoExposureMaxBrightness));
		}
		Canvas.DrawShadowedString(X, Y += YStep, TEXT("EV100 Min/Max"), GetStatsFont(), FLinearColor(1, 1, 1));
		Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(0.3f, 0.3f, 1));

		Line = FString::Printf(TEXT("%g / %g"), View.FinalPostProcessSettings.AutoExposureSpeedUp, View.FinalPostProcessSettings.AutoExposureSpeedDown);
		Canvas.DrawShadowedString(X, Y += YStep, TEXT("Speed Up/Down:"), GetStatsFont(), FLinearColor(1, 1, 1));
		Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 1, 1));

		float AutoExposureBias = View.FinalPostProcessSettings.AutoExposureBias;
		{
			float AverageSceneLuminance = View.GetLastAverageSceneLuminance();

			float CurveExposureBias = 0.0f;
			float AverageSceneLuminanceEV100 = 0.0f;
			if (AverageSceneLuminance > 0)
			{
				// We need the Log2(0.18) to convert from average luminance to saturation luminance
				AverageSceneLuminanceEV100 = LuminanceToEV100(LuminanceMax, AverageSceneLuminance) + FMath::Log2(1.0f / 0.18f);
				if (View.FinalPostProcessSettings.AutoExposureBiasCurve)
				{
					CurveExposureBias = View.FinalPostProcessSettings.AutoExposureBiasCurve->GetFloatValue(AverageSceneLuminanceEV100);
				}
			}

			Line = FString::Printf(TEXT("%.3g"), AverageSceneLuminanceEV100);
			Canvas.DrawShadowedString(X, Y += YStep, TEXT("Average Scene EV100:"), GetStatsFont(), FLinearColor(1, 1, 1));
			Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 1, 1));

			Line = FString::Printf(TEXT("%.3g"), AutoExposureBias);
			Canvas.DrawShadowedString(X, Y += YStep, TEXT("Exposure Compensation (Settings):"), GetStatsFont(), FLinearColor(1, 1, 1));
			Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 1, 1));

			Line = FString::Printf(TEXT("%.3g"), CurveExposureBias);
			Canvas.DrawShadowedString(X, Y += YStep, TEXT("Exposure Compensation (Curve):"), GetStatsFont(), FLinearColor(1, 1, 1));
			Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 1, 1));

			const float ScreenCenterX = Output.ViewRect.Min.X + Output.ViewRect.Width() * 0.5f;
			const float ScreenCenterY = Output.ViewRect.Min.Y + Output.ViewRect.Height() * 0.5f;

			const float IlluminanceMeterTextX = (ScreenCenterX - 10.0f) / DPIScale;
			const float IlluminanceMeterTextY = (ScreenCenterY - 140.0f) / DPIScale;
			if(IsIlluminanceMeterSupportedByView(View))
			{
				Canvas.DrawShadowedString(IlluminanceMeterTextX, IlluminanceMeterTextY   , TEXT("Illuminance meter - over the hemisphere of the surface patch"), GetStatsFont(), FLinearColor(1, 1, 1));
				Canvas.DrawShadowedString(IlluminanceMeterTextX, IlluminanceMeterTextY+15, TEXT("                    (Forced to be a perfect white diffuse only surface)"), GetStatsFont(), FLinearColor(1, 1, 1));
			}
			else
			{
				Canvas.DrawShadowedString(IlluminanceMeterTextX, IlluminanceMeterTextY, TEXT("Illuminance meter - not available with Forward Shading"), GetStatsFont(), FLinearColor(1, 1, 1));
			}

			const float LuminanceMeterTextX = (ScreenCenterX - 10.0f) / DPIScale;
			const float LuminanceMeterTextY = (ScreenCenterY - 40.0f) / DPIScale;
			Canvas.DrawShadowedString(LuminanceMeterTextX, LuminanceMeterTextY, TEXT("Luminance meter"), GetStatsFont(), FLinearColor(1, 1, 1));

			AutoExposureBias += CurveExposureBias;
		}

		// Draw EV100 values at the bottom of the histogram
		{
			const float MinX = Output.ViewRect.Min.X + 64 + 8;
			const float MaxY = Output.ViewRect.Max.Y - 68;
			const float SizeX = Output.ViewRect.Width() - 64 * 2 - 20;

			for (uint32 i = 0; i <= 4; ++i)
			{
				int XAdd = (int)(i * SizeX / 4);
				float HistogramPosition = i / 4.0f;
				float EV100Value = FMath::Lerp(View.FinalPostProcessSettings.HistogramLogMin, View.FinalPostProcessSettings.HistogramLogMax, HistogramPosition);
				if (!bExtendedLuminanceRange)
				{
					// In this case the post process settings are actually Log2 values.
					EV100Value = Log2ToEV100(LuminanceMax, EV100Value);
				}

				Line = FString::Printf(TEXT("%.2g"), EV100Value);

				const float PosX = (MinX + XAdd - 5) / DPIScale;
				const float PosY = (MaxY + YStep) / DPIScale;

				Canvas.DrawShadowedString(PosX, PosY, *Line, GetStatsFont(), FLinearColor(1, 0.3f, 0.3f));
			}
		}

		Line = FString::Printf(TEXT("%.3g"), AutoExposureBias);
		Canvas.DrawShadowedString(X, Y += YStep, TEXT("Exposure Compensation (All): "), GetStatsFont(), FLinearColor(1, 1, 1));
		Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(1, 0.3f, 0.3f));

		if (bExtendedLuminanceRange)
		{
			Line = FString::Printf(TEXT("%.1f .. %.1f"), View.FinalPostProcessSettings.HistogramLogMin, View.FinalPostProcessSettings.HistogramLogMax);
		}
		else
		{
			Line = FString::Printf(TEXT("%.1f .. %.1f"), Log2ToEV100(LuminanceMax,View.FinalPostProcessSettings.HistogramLogMin), Log2ToEV100(LuminanceMax,View.FinalPostProcessSettings.HistogramLogMax));
		}

		Canvas.DrawShadowedString(X, Y += YStep, TEXT("Histogram EV100 Min/Max:"), GetStatsFont(), FLinearColor(1, 1, 1));
		Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(0.3f, 0.3f, 1));

		Line = FString::Printf(TEXT("%.5g"), View.PreExposure);
		Canvas.DrawShadowedString(X, Y += YStep, TEXT("PreExposure Value:"), GetStatsFont(), FLinearColor(1, 1, 1));
		Canvas.DrawShadowedString(X + ColumnWidth, Y, *Line, GetStatsFont(), FLinearColor(0.3f, 0.3f, 1));


	});

	return MoveTemp(Output);
}