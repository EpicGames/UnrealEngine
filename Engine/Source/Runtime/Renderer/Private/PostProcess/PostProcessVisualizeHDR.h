// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OverridePassSequence.h"

class FEyeAdaptationParameters;

struct FVisualizeHDRInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	FScreenPassTexture SceneColor; 
	FScreenPassTexture SceneColorBeforeTonemap;
	FScreenPassTexture Luminance;

	FRDGTextureRef HistogramTexture = nullptr;
	// Mobile histogram source.
	FRDGBufferSRVRef HistogramBuffer = nullptr;
	FRDGBufferRef EyeAdaptationBuffer = nullptr;

	const FEyeAdaptationParameters* EyeAdaptationParameters = nullptr;
};

FScreenPassTexture AddVisualizeHDRPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeHDRInputs& Inputs);