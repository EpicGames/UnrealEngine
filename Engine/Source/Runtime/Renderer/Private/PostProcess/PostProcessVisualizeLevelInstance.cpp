// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "PostProcess/PostProcessVisualizeLevelInstance.h"
#include "CoreMinimal.h"
#include "Containers/StaticArray.h"
#include "GlobalShader.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "Logging/LogMacros.h"
#include "MeshDrawCommands.h"
#include "MeshPassProcessor.h"
#include "Nanite/NaniteCullRaster.h"
#include "Nanite/NaniteEditor.h"
#include "PostProcess/PostProcessCompositePrimitivesCommon.h"
#include "RenderGraphBuilder.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTexturesConfig.h"
#include "SceneUniformBuffer.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "Stats/Stats.h"

namespace
{
class FVisualizeLevelInstancePS : public FCompositePrimitiveShaderBase
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeLevelInstancePS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeLevelInstancePS, FCompositePrimitiveShaderBase);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Depth)
		SHADER_PARAMETER(FScreenTransform, ColorToDepth)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, EditorPrimitivesDepth)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, EditorPrimitivesStencil)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeLevelInstancePS, "/Engine/Private/PostProcessVisualizeLevelInstance.usf", "MainPS", SF_Pixel);
} //! namespace


BEGIN_SHADER_PARAMETER_STRUCT(FVisualizeLevelInstancePassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

FScreenPassTexture AddVisualizeLevelInstancePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSceneUniformBuffer &SceneUniformBuffer,
	const FVisualizeLevelInstanceInputs& Inputs,
	const Nanite::FRasterResults* NaniteRasterResults
)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneDepth.IsValid());

	const bool bNaniteEnabled = DoesPlatformSupportNanite(GMaxRHIShaderPlatform); // TODO: Respect r.Nanite

	RDG_EVENT_SCOPE(GraphBuilder, "EditorVisualizeLevelInstance");

	const uint32 NumSamples = View.GetSceneTexturesConfig().NumSamples;

	// Patch uniform buffers with updated state for rendering the outline mesh draw commands.
	const FViewInfo* EditorView = CreateCompositePrimitiveView(View, Inputs.SceneColor.ViewRect, NumSamples);

	FRDGTextureRef DepthStencilTexture = nullptr;

	// Generate custom depth / stencil for outline shapes.
	{
		const FScreenPassTextureViewport SceneColorViewport(Inputs.SceneColor);
		RDG_EVENT_SCOPE(GraphBuilder, "LevelInstanceDepth %dx%d", SceneColorViewport.Rect.Width(), SceneColorViewport.Rect.Height());

		FScene* Scene = View.Family->Scene->GetRenderScene();

		{
			FRDGTextureDesc DepthStencilDesc = Inputs.SceneColor.Texture->Desc;
			DepthStencilDesc.Reset();
			DepthStencilDesc.Format = PF_DepthStencil;
			// This is a reversed Z depth surface, so 0.0f is the far plane.
			DepthStencilDesc.ClearValue = FClearValueBinding(0.f, 0);
			DepthStencilDesc.Flags = TexCreate_DepthStencilTargetable | TexCreate_ShaderResource;
			DepthStencilDesc.NumSamples = NumSamples;
			DepthStencilDesc.AliasableFormats.Add(PF_X24_G8);

			DepthStencilTexture = GraphBuilder.CreateTexture(DepthStencilDesc, TEXT("LevelInstanceDepth"));
		}

		{
			auto* PassParameters = GraphBuilder.AllocParameters<FVisualizeLevelInstancePassParameters>();

			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
				DepthStencilTexture,
				ERenderTargetLoadAction::EClear,
				ERenderTargetLoadAction::EClear,
				FExclusiveDepthStencil::DepthWrite_StencilWrite);

			auto* Pass = const_cast<FViewInfo&>(View).ParallelMeshDrawCommandPasses[EMeshPass::EditorLevelInstance];

			if (Pass)
			{
				PassParameters->View = EditorView->GetShaderParameters();
				PassParameters->SceneTextures = Inputs.SceneTextures;

				Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);
			}

			// Pass always added due to clear action.
			GraphBuilder.AddPass(
				RDG_EVENT_NAME("EditorLevelInstance"),
				PassParameters,
				ERDGPassFlags::Raster,
				[Pass, SceneColorViewport, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				if (Pass)
				{
					RHICmdList.SetViewport(SceneColorViewport.Rect.Min.X, SceneColorViewport.Rect.Min.Y, 0.0f, SceneColorViewport.Rect.Max.X, SceneColorViewport.Rect.Max.Y, 1.0f);

					// Run LevelInstance pass on static elements
					Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
				}
			});
		}

		if (bNaniteEnabled)
		{
			Nanite::DrawEditorVisualizeLevelInstance(GraphBuilder, DepthStencilTexture, *Scene, View, *EditorView, SceneUniformBuffer, NaniteRasterResults);
		}
	}

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("LevelInstanceColor"));
	}

	// Render grey-post process effect.
	{
		const FScreenPassTextureViewport OutputViewport(Output);
		const FScreenPassTextureViewport ColorViewport(Inputs.SceneColor);
		const FScreenPassTextureViewport DepthViewport(Inputs.SceneDepth);

		FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		FVisualizeLevelInstancePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeLevelInstancePS::FParameters>();
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->Color = GetScreenPassTextureViewportParameters(ColorViewport);
		PassParameters->Depth = GetScreenPassTextureViewportParameters(DepthViewport);
		PassParameters->ColorToDepth = FScreenTransform::ChangeTextureUVCoordinateFromTo(ColorViewport, DepthViewport);
		PassParameters->ColorTexture = Inputs.SceneColor.Texture;
		PassParameters->ColorSampler = PointClampSampler;
		PassParameters->DepthTexture = Inputs.SceneDepth.Texture;
		PassParameters->DepthSampler = BilinearClampSampler;
		PassParameters->EditorPrimitivesDepth = DepthStencilTexture;
		PassParameters->EditorPrimitivesStencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(DepthStencilTexture, PF_X24_G8));

		FVisualizeLevelInstancePS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FVisualizeLevelInstancePS::FSampleCountDimension>(NumSamples);

		TShaderMapRef<FVisualizeLevelInstancePS> PixelShader(View.ShaderMap, PermutationVector);

		AddDrawScreenPass(
			GraphBuilder,
			RDG_EVENT_NAME("LevelInstanceColor %dx%d", OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
			View,
			OutputViewport,
			ColorViewport,
			PixelShader,
			PassParameters);
	}

	return MoveTemp(Output);
}

#endif
