// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "ScreenPass.h"
#include "OverridePassSequence.h"

namespace Nanite
{
	struct FRasterResults;
}

struct FVisualizeLevelInstanceInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The scene color to composite with selection outlines.
	FScreenPassTexture SceneColor;

	// [Required] The scene depth to composite with selection outlines.
	FScreenPassTexture SceneDepth;

	// [Required] Used when scene textures are required by the material.
	FSceneTextureShaderParameters SceneTextures;
};

FScreenPassTexture AddVisualizeLevelInstancePass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSceneUniformBuffer &SceneUniformBuffer,
	const FVisualizeLevelInstanceInputs& Inputs,
	const Nanite::FRasterResults* NaniteRasterResults
);

#endif
