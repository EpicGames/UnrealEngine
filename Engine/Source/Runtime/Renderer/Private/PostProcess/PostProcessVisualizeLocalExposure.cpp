// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessVisualizeLocalExposure.h"
#include "PostProcess/PostProcessTonemap.h"
#include "PostProcess/PostProcessLocalExposure.h"
#include "PostProcess/PostProcessVisualizeBuffer.h"
#include "UnrealEngine.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"

TAutoConsoleVariable<int> CVarLocalExposureVisualizationMode(
	TEXT("r.LocalExposure.VisualizationMode"),
	0,
	TEXT("When enabling Show->Visualize->Local Exposure is enabled, this cvar controls which mode to use.\n")
	TEXT("    0: Overview\n")
	TEXT("    1: Local Exposure\n")
	TEXT("    2: Thresholds\n")
	TEXT("    3: Base Luminance\n")
	TEXT("    4: Detail Luminance\n")
	TEXT("    5: Valid Bilateral Grid Lookup\n")
	TEXT("    6: Fusion - Base Exposure\n")
	TEXT("    7: Fusion - Shadows Exposure\n")
	TEXT("    8: Fusion - Highlights Exposure\n")
	TEXT("    9: Fusion - Weights\n"),
	ECVF_RenderThreadSafe);

class FVisualizeLocalExposurePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeLocalExposurePS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeLocalExposurePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT(FEyeAdaptationParameters, EyeAdaptation)
		SHADER_PARAMETER_STRUCT(FLocalExposureParameters, LocalExposure)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HDRSceneColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture3D, LumBilateralGrid)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BlurredLogLum)

		SHADER_PARAMETER(FScreenTransform, ColorToExposureFusion)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, ExposureFusion)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ExposureFusionTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ExposuresTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, WeightsTexture)

		SHADER_PARAMETER(float, FilmSlope)
		SHADER_PARAMETER(float, FilmToe)
		SHADER_PARAMETER(float, FilmShoulder)
		SHADER_PARAMETER(float, FilmBlackClip)
		SHADER_PARAMETER(float, FilmWhiteClip)

		SHADER_PARAMETER_SAMPLER(SamplerState, BilinearClampSampler)

		SHADER_PARAMETER(uint32, DebugMode)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	class FExposureFusion : SHADER_PERMUTATION_BOOL("EXPOSURE_FUSION");

	using FPermutationDomain = TShaderPermutationDomain<FExposureFusion>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeLocalExposurePS, "/Engine/Private/PostProcessVisualizeLocalExposure.usf", "MainPS", SF_Pixel);

FScreenPassTexture AddVisualizeLocalExposurePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeLocalExposureInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.HDRSceneColor.IsValid());
	check(Inputs.EyeAdaptationBuffer);
	check(Inputs.EyeAdaptationParameters);

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeLocalExposure");

	enum class EVisualizeId : int32
	{
		Overview = 0,
		LocalExposure = 1,
		Thresholds = 2,
		BaseLuminance = 3,
		DetailLuminance = 4,
		ValidLookup = 5,
		FusionBase = 6,
		FusionShadows = 7,
		FusionHighlights = 8,
		FusionWeights = 9,
		MAX,
	};

	static const TCHAR* kVisualizationName[] = {
		TEXT(""),
		TEXT("LocalExposure"),
		TEXT("Thresholds"),
		TEXT("BaseLuminance"),
		TEXT("DetailLuminance"),
		TEXT("ValidLookup"),
		TEXT("FusionBase"),
		TEXT("FusionShadows"),
		TEXT("FusionHighlights"),
		TEXT("FusionWeights"),
	};
	static_assert(UE_ARRAY_COUNT(kVisualizationName) == int32(EVisualizeId::MAX), "kVisualizationName doesn't match EVisualizeId");

	const EVisualizeId Visualization = EVisualizeId(FMath::Clamp(CVarLocalExposureVisualizationMode.GetValueOnRenderThread(), 0, int32(EVisualizeId::MAX) - 1));

	const bool bIsOverviewVisualize = Visualization == EVisualizeId::Overview;

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeLocalExposure"));
	}

	auto Visualize = [&](EVisualizeId VisualizeId, FString Label, FScreenPassRenderTarget OverrideOutput = FScreenPassRenderTarget())
		{
			FScreenPassRenderTarget TmpOutput = OverrideOutput;

			if (!TmpOutput.IsValid())
			{
				FIntPoint TmpOutputExtent = FIntPoint::DivideAndRoundUp(Inputs.SceneColor.ViewRect.Size(), 4);
				FIntRect TmpOutputViewRect = FIntRect(FIntPoint(0, 0), TmpOutputExtent);

				FRDGTextureDesc TmpOutputDesc = Inputs.SceneColor.Texture->Desc;
				TmpOutputDesc.Extent = TmpOutputExtent;
				TmpOutputDesc.Flags |= TexCreate_UAV | TexCreate_RenderTargetable;
				TmpOutput = FScreenPassRenderTarget(GraphBuilder.CreateTexture(TmpOutputDesc, TEXT("VisualizeLumenScene")), TmpOutputViewRect, ERenderTargetLoadAction::ENoAction);
			}

			const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
			const FScreenPassTextureViewport OutputViewport(TmpOutput);

			const FPostProcessSettings& Settings = View.FinalPostProcessSettings;

			auto PassParameters = GraphBuilder.AllocParameters<FVisualizeLocalExposurePS::FParameters>();
			PassParameters->RenderTargets[0] = TmpOutput.GetRenderTargetBinding();
			PassParameters->View = View.ViewUniformBuffer;
			PassParameters->EyeAdaptation = *Inputs.EyeAdaptationParameters;
			PassParameters->LocalExposure = *Inputs.LocalExposureParameters;
			PassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
			PassParameters->Output = GetScreenPassTextureViewportParameters(OutputViewport);
			PassParameters->HDRSceneColorTexture = Inputs.HDRSceneColor.Texture;
			PassParameters->SceneColorTexture = Inputs.SceneColor.Texture;
			PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(Inputs.EyeAdaptationBuffer);
			PassParameters->LumBilateralGrid = Inputs.LumBilateralGridTexture;
			PassParameters->BlurredLogLum = Inputs.BlurredLumTexture;

			if (Inputs.ExposureFusionData != nullptr)
			{
				const FScreenPassTextureViewport ExposureFusionViewport(Inputs.ExposureFusionData->Result);
				PassParameters->ColorToExposureFusion = FScreenTransform::ChangeTextureUVCoordinateFromTo(InputViewport, ExposureFusionViewport);
				PassParameters->ExposureFusion = GetScreenPassTextureViewportParameters(ExposureFusionViewport);
				PassParameters->ExposureFusionTexture = Inputs.ExposureFusionData->Result.Texture;
				PassParameters->ExposuresTexture = Inputs.ExposureFusionData->Exposures.Texture;
				PassParameters->WeightsTexture = Inputs.ExposureFusionData->Weights.Texture;
			}

			PassParameters->FilmSlope = Settings.FilmSlope;
			PassParameters->FilmToe = Settings.FilmToe;
			PassParameters->FilmShoulder = Settings.FilmShoulder;
			PassParameters->FilmBlackClip = Settings.FilmBlackClip;
			PassParameters->FilmWhiteClip = Settings.FilmWhiteClip;

			PassParameters->BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->DebugMode = uint32(VisualizeId);

			FVisualizeLocalExposurePS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FVisualizeLocalExposurePS::FExposureFusion>(Inputs.ExposureFusionData != nullptr);
			auto PixelShader = View.ShaderMap->GetShader<FVisualizeLocalExposurePS>(PermutationVector);

			FRDGEventName PassName = RDG_EVENT_NAME("LocalExposure Visualize(%s) %dx%d", kVisualizationName[int32(VisualizeId)], OutputViewport.Rect.Width(), OutputViewport.Rect.Height());
			AddDrawScreenPass(GraphBuilder, MoveTemp(PassName), View, OutputViewport, InputViewport, PixelShader, PassParameters);

			FVisualizeBufferTile Tile;
			Tile.Input = TmpOutput;
			Tile.Label = Label;
			return Tile;
		};

	if (bIsOverviewVisualize)
	{
		TArray<FVisualizeBufferTile> Tiles;
		Tiles.SetNum(16);
		if (Visualization == EVisualizeId::Overview)
		{
			Tiles[4 * 0 + 0] = Visualize(EVisualizeId::LocalExposure, FString::Printf(TEXT("Local Exposure (H=%.2f / S=%.2f)"), Inputs.LocalExposureParameters->HighlightContrastScale, Inputs.LocalExposureParameters->ShadowContrastScale));
			Tiles[4 * 0 + 3] = Visualize(EVisualizeId::Thresholds, FString::Printf(TEXT("Thresholds (H=%.2f / S=%.2f)"), Inputs.LocalExposureParameters->HighlightThreshold, Inputs.LocalExposureParameters->ShadowThreshold));

			if (Inputs.ExposureFusionData != nullptr)
			{
				Tiles[4 * 3 + 0] = Visualize(EVisualizeId::FusionShadows, TEXT("Shadows"));
				Tiles[4 * 3 + 1] = Visualize(EVisualizeId::FusionBase, TEXT("Base"));
				Tiles[4 * 3 + 2] = Visualize(EVisualizeId::FusionHighlights, TEXT("Highlights"));
				Tiles[4 * 3 + 3] = Visualize(EVisualizeId::FusionWeights, TEXT("Weights"));
			}
			else
			{
				Tiles[4 * 3 + 0] = Visualize(EVisualizeId::BaseLuminance, TEXT("Base Luminance"));
				Tiles[4 * 3 + 1] = Visualize(EVisualizeId::DetailLuminance, TEXT("Detail Luminance"));
			}
		}
		else
		{
			unimplemented();
		}

		{
			FVisualizeBufferInputs VisualizeBufferInputs;
			VisualizeBufferInputs.OverrideOutput = Output;
			VisualizeBufferInputs.SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, Inputs.SceneColor));
			VisualizeBufferInputs.Tiles = Tiles;
			AddVisualizeBufferPass(GraphBuilder, View, VisualizeBufferInputs);
		}
	}
	else
	{
		Visualize(Visualization, TEXT(""), Output);
	}

	return MoveTemp(Output);
}