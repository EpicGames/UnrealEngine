// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OverridePassSequence.h"

class FEyeAdaptationParameters;
class FLocalExposureParameters;

struct FExposureFusionData;

struct FVisualizeLocalExposureInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	FScreenPassTexture SceneColor;
	FScreenPassTexture HDRSceneColor;

	FRDGBufferRef EyeAdaptationBuffer = nullptr;

	const FEyeAdaptationParameters* EyeAdaptationParameters = nullptr;
	const FLocalExposureParameters* LocalExposureParameters = nullptr;

	FRDGTextureRef LumBilateralGridTexture = nullptr;
	FRDGTextureRef BlurredLumTexture = nullptr;

	FExposureFusionData* ExposureFusionData = nullptr;
};

FScreenPassTexture AddVisualizeLocalExposurePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeLocalExposureInputs& Inputs);