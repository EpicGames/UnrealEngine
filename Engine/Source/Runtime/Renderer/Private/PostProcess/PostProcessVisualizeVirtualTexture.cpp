// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessVisualizeVirtualTexture.h"

#include "CanvasTypes.h"
#include "SceneRendering.h"
#include "UnrealEngine.h"
#include "VT/VirtualTextureVisualizationData.h"

#define LOCTEXT_NAMESPACE "VisualizeVirtualTexture"

class FVisualizeVirtualTextureApplyPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FVisualizeVirtualTextureApplyPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeVirtualTextureApplyPS, FGlobalShader);

	static constexpr int32 MaxNumColors = 11;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, DebugBuffer)
		SHADER_PARAMETER(FIntPoint, ViewSize)
		SHADER_PARAMETER(uint32, ViewMode)
		SHADER_PARAMETER_ARRAY(FLinearColor, Colors, [MaxNumColors])
		SHADER_PARAMETER(uint32, ColorCount)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("MAX_NUM_COLORS"), MaxNumColors);
	}
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeVirtualTextureApplyPS, "/Engine/Private/VisualizeVirtualTexture.usf", "Main", SF_Pixel);

FScreenPassTexture AddVisualizeVirtualTexturePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeVirtualTextureInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeVirtualTexture"));
	}

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(Output);

	FVisualizeVirtualTextureApplyPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeVirtualTextureApplyPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->Input = GetScreenPassTextureViewportParameters(InputViewport);
	PassParameters->Output = GetScreenPassTextureViewportParameters(OutputViewport);
	PassParameters->InputTexture = Inputs.SceneColor.Texture;
	PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->DebugBuffer = GraphBuilder.CreateSRV(Inputs.DebugBuffer);
	PassParameters->ViewSize = View.ViewRect.Size();

	const int32 ClampedColorCount = FMath::Min<int32>(Inputs.Colors.Num(), FVisualizeVirtualTextureApplyPS::MaxNumColors);
	if (ClampedColorCount > 0)
	{
		for (int32 ColorIndex = 0; ColorIndex < ClampedColorCount; ColorIndex++)
		{
			PassParameters->Colors[ColorIndex] = Inputs.Colors[ColorIndex];
		}
		PassParameters->ColorCount = ClampedColorCount;
	}
	else // Otherwise fallback to a safe value.
	{
		PassParameters->Colors[0] = FLinearColor::Gray;
		PassParameters->ColorCount = 1;
	}

	const EVirtualTextureVisualizationMode Mode = GetVirtualTextureVisualizationData().GetModeID(Inputs.ModeName);
	const FText ModeDesc = GetVirtualTextureVisualizationData().GetModeDisplayDesc(Inputs.ModeName);
	
	PassParameters->ViewMode = (uint32)Mode;

	TShaderMapRef<FVisualizeVirtualTextureApplyPS> PixelShader(View.ShaderMap);

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("VisualizeVirtualTexture"), View, FScreenPassTextureViewport(Output), InputViewport, PixelShader, PassParameters);

	Output.LoadAction = ERenderTargetLoadAction::ELoad;

	const TArrayView<const FLinearColor> Colors = Inputs.Colors;
	const FIntRect OutputViewRect = Output.ViewRect;

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("VisualizeVirtualTextureLegend"), View, Output, [&View, ModeDesc, Colors, ClampedColorCount, OutputViewRect](FCanvas& Canvas)
	{
		auto DrawDesc = [&](FCanvas& Canvas, float PosX, float PosY, const FText& Text)
		{
			Canvas.DrawShadowedText(PosX + 18, PosY, Text, GetStatsFont(), FLinearColor(0.7f, 0.7f, 0.7f), FLinearColor::Black);
		};

		auto DrawBox = [&](FCanvas& Canvas, float PosX, float PosY, const FLinearColor& Color, const FText& Text)
		{
			Canvas.DrawTile(PosX, PosY, 16, 16, 0, 0, 1, 1, FLinearColor::Black);
			Canvas.DrawTile(PosX + 1, PosY + 1, 14, 14, 0, 0, 1, 1, Color);
			Canvas.DrawShadowedText(PosX + 18, PosY, Text, GetStatsFont(), FLinearColor(0.7f, 0.7f, 0.7f), FLinearColor::Black);
		};

		int32 CoordX = 115;
		DrawDesc(Canvas, OutputViewRect.Min.X + CoordX, OutputViewRect.Max.Y - 75, ModeDesc);

		for (int32 Index = 0; Index < ClampedColorCount; ++Index)
		{
			DrawBox(Canvas, OutputViewRect.Min.X + CoordX, OutputViewRect.Max.Y - 25, Colors[Index], FText::AsNumber(Index + 1));
			CoordX += 50;
		}
	});

	return MoveTemp(Output);
}

#undef LOCTEXT_NAMESPACE