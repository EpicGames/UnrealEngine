// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "DebugViewModeHelpers.h"
#include "OverridePassSequence.h"

struct FVisualizeVirtualTextureInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;
	// [Required] The input scene color and view rect.
	FScreenPassTexture SceneColor;
	// [Required] The buffer generated by main pass rendering which contains debug data.
	FRDGBufferRef DebugBuffer;
	// [Required] The visualization mode.
	FName ModeName;
	// [Required] The table of colors used for visualization.
	TArrayView<const FLinearColor> Colors;
};

FScreenPassTexture AddVisualizeVirtualTexturePass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeVirtualTextureInputs& Inputs);