// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessWeightedSampleSum.h"
#include "PixelShaderUtils.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneRendering.h"

// maximum number of sample using the shader that has the dynamic loop
#define MAX_FILTER_SAMPLES	128
#define MAX_PACKED_SAMPLES_OFFSET ((MAX_FILTER_SAMPLES + 1) / 2)

// maximum number of sample available using unrolled loop shaders
#define MAX_FILTER_COMPILE_TIME_SAMPLES 32
#define MAX_FILTER_COMPILE_TIME_SAMPLES_IOS 15
#define MAX_FILTER_COMPILE_TIME_SAMPLES_ES3_1 7

namespace
{
const int32 GFilterComputeTileSizeX = 8;
const int32 GFilterComputeTileSizeY = 8;

TAutoConsoleVariable<int32> CVarLoopMode(
	TEXT("r.Filter.LoopMode"),
	0,
	TEXT("Controls when to use either dynamic or unrolled loops to iterates over the Gaussian filtering.\n")
	TEXT("This passes is used for Gaussian Blur, Bloom and Depth of Field. The dynamic loop allows\n")
	TEXT("up to 128 samples versus the 32 samples of unrolled loops, but add an additional cost for\n")
	TEXT("the loop's stop test at every iterations.\n")
	TEXT(" 0: Unrolled loop only (default; limited to 32 samples).\n")
	TEXT(" 1: Fall back to dynamic loop if needs more than 32 samples.\n")
	TEXT(" 2: Dynamic loop only."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarFilterSizeScale(
	TEXT("r.Filter.SizeScale"),
	1.0f,
	TEXT("Allows to scale down or up the sample count used for bloom and Gaussian depth of field (scale is clamped to give reasonable results).\n")
	TEXT("Values down to 0.6 are hard to notice\n")
	TEXT(" 1 full quality (default)\n")
	TEXT(" >1 more samples (slower)\n")
	TEXT(" <1 less samples (faster, artifacts with HDR content or boxy results with GaussianDOF)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarFastBlurThreshold(
	TEXT("r.FastBlurThreshold"),
	7.0f,
	TEXT("Defines at what radius the Gaussian blur optimization kicks in (estimated 25% - 40% faster).\n")
	TEXT("The optimization uses slightly less memory and has a quality loss on smallblur radius.\n")
	TEXT("  0: use the optimization always (fastest, lowest quality)\n")
	TEXT("  3: use the optimization starting at a 3 pixel radius (quite fast)\n")
	TEXT("  7: use the optimization starting at a 7 pixel radius (default)\n")
	TEXT(">15: barely ever use the optimization (high quality)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

BEGIN_SHADER_PARAMETER_STRUCT(FFilterParameters, )
	SHADER_PARAMETER_STRUCT(FScreenPassTextureSliceInput, Filter)
	SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Additive)
	SHADER_PARAMETER_ARRAY(FVector4f, SampleOffsets, [MAX_PACKED_SAMPLES_OFFSET])
	SHADER_PARAMETER_ARRAY(FLinearColor, SampleWeights, [MAX_FILTER_SAMPLES])
	SHADER_PARAMETER(int32, SampleCount)
END_SHADER_PARAMETER_STRUCT()

void GetFilterParameters(
	FFilterParameters& OutParameters,
	const FScreenPassTextureSliceInput& Filter,
	const FScreenPassTextureInput& Additive,
	TArrayView<const FVector2f> SampleOffsets,
	TArrayView<const FLinearColor> SampleWeights)
{
	OutParameters.Filter = Filter;
	OutParameters.Additive = Additive;

	for (int32 SampleIndex = 0; SampleIndex < SampleOffsets.Num(); SampleIndex += 2)
	{
		OutParameters.SampleOffsets[SampleIndex / 2].X = SampleOffsets[SampleIndex + 0].X;
		OutParameters.SampleOffsets[SampleIndex / 2].Y = SampleOffsets[SampleIndex + 0].Y;

		if (SampleIndex + 1 < SampleOffsets.Num())
		{
			OutParameters.SampleOffsets[SampleIndex / 2].Z = SampleOffsets[SampleIndex + 1].X;
			OutParameters.SampleOffsets[SampleIndex / 2].W = SampleOffsets[SampleIndex + 1].Y;
		}
	}

	for (int32 SampleIndex = 0; SampleIndex < SampleWeights.Num(); ++SampleIndex)
	{
		OutParameters.SampleWeights[SampleIndex] = SampleWeights[SampleIndex];
	}

	OutParameters.SampleCount = SampleOffsets.Num();
}

// Evaluates an unnormalized normal distribution PDF around 0 at given X with Variance.
float NormalDistributionUnscaled(float X, float Sigma, float CrossCenterWeight)
{
	const float DX = FMath::Abs(X);

	const float ClampedOneMinusDX = FMath::Max(0.0f, 1.0f - DX);

	// Tweak the gaussian shape e.g. "r.Bloom.Cross 3.5"
	if (CrossCenterWeight > 1.0f)
	{
		return FMath::Pow(ClampedOneMinusDX, CrossCenterWeight);
	}
	else
	{
		// Constant is tweaked give a similar look to UE before we fixed the scale bug (Some content tweaking might be needed).
		// The value defines how much of the Gaussian clipped by the sample window.
		// r.Filter.SizeScale allows to tweak that for performance/quality.
		const float LegacyCompatibilityConstant = -16.7f;

		const float Gaussian = FMath::Exp(LegacyCompatibilityConstant * FMath::Square(DX / Sigma));

		return FMath::Lerp(Gaussian, ClampedOneMinusDX, CrossCenterWeight);
	}
}

uint32 GetSampleCountMax(ERHIFeatureLevel::Type InFeatureLevel, EShaderPlatform InPlatform)
{
	if (CVarLoopMode.GetValueOnRenderThread() != 0)
	{
		return MAX_FILTER_SAMPLES;
	}
	else if (IsMetalMobileSM5Platform(InPlatform) || IsMetalMobileSM6Platform(InPlatform))
	{
		return MAX_FILTER_COMPILE_TIME_SAMPLES_IOS;
	}
	else if (InFeatureLevel < ERHIFeatureLevel::SM5)
	{
		return MAX_FILTER_COMPILE_TIME_SAMPLES_ES3_1;
	}
	else
	{
		return MAX_FILTER_COMPILE_TIME_SAMPLES;
	}
}

float GetClampedKernelRadius(uint32 SampleCountMax, float KernelRadius)
{
	return FMath::Clamp<float>(KernelRadius, DELTA, SampleCountMax - 1);
}

int GetIntegerKernelRadius(uint32 SampleCountMax, float KernelRadius)
{
	// Smallest radius will be 1.
	return FMath::Min<int32>(FMath::CeilToInt(GetClampedKernelRadius(SampleCountMax, KernelRadius)), SampleCountMax - 1);
}

uint32 Compute1DGaussianFilterKernel(FVector2f OutOffsetAndWeight[MAX_FILTER_SAMPLES], uint32 SampleCountMax,	float KernelRadius, float CrossCenterWeight)
{
	const float FilterSizeScale = FMath::Clamp(CVarFilterSizeScale.GetValueOnRenderThread(), 0.1f, 10.0f);

	const float ClampedKernelRadius = GetClampedKernelRadius(SampleCountMax, KernelRadius);

	const int32 IntegerKernelRadius = GetIntegerKernelRadius(SampleCountMax, KernelRadius * FilterSizeScale);

	uint32 SampleCount = 0;
	float WeightSum = 0.0f;

	for (int32 SampleIndex = -IntegerKernelRadius; SampleIndex <= IntegerKernelRadius; SampleIndex += 2)
	{
		float Weight0 = NormalDistributionUnscaled(SampleIndex, ClampedKernelRadius, CrossCenterWeight);
		float Weight1 = 0.0f;

		// We use the bilinear filter optimization for gaussian blur. However, we don't want to bias the
		// last sample off the edge of the filter kernel, so the very last tap just is on the pixel center.
		if(SampleIndex != IntegerKernelRadius)
		{
			Weight1 = NormalDistributionUnscaled(SampleIndex + 1, ClampedKernelRadius, CrossCenterWeight);
		}

		const float TotalWeight = Weight0 + Weight1;
		OutOffsetAndWeight[SampleCount].X = SampleIndex + (Weight1 / TotalWeight);
		OutOffsetAndWeight[SampleCount].Y = TotalWeight;
		WeightSum += TotalWeight;
		SampleCount++;
	}

	// Normalize blur weights.
	const float WeightSumInverse = 1.0f / WeightSum;
	for (uint32 SampleIndex = 0; SampleIndex < SampleCount; ++SampleIndex)
	{
		OutOffsetAndWeight[SampleIndex].Y *= WeightSumInverse;
	}

	return SampleCount;
}

float GetBlurRadius(uint32 ViewSize, float KernelSizePercent)
{
	const float PercentToScale = 0.01f;

	const float DiameterToRadius = 0.5f;

	return static_cast<float>(ViewSize) * KernelSizePercent * PercentToScale * DiameterToRadius;
}

bool IsFastBlurEnabled(float BlurRadius)
{
	const float FastBlurRadiusThreshold = CVarFastBlurThreshold.GetValueOnRenderThread();

	return BlurRadius >= FastBlurRadiusThreshold;
}

uint32 GetStaticSampleCount(uint32 RequiredSampleCount)
{
	const int32 LoopMode = CVarLoopMode.GetValueOnRenderThread();

	const bool bForceStaticLoop = LoopMode == 0;

	const bool bForceDynamicLoop = LoopMode == 2;

	if (!bForceDynamicLoop && (bForceStaticLoop || RequiredSampleCount <= MAX_FILTER_COMPILE_TIME_SAMPLES))
	{
		return RequiredSampleCount;
	}
	else
	{
		return 0;
	}
}

class FFilterShader : public FGlobalShader
{
public:
	class FStaticSampleCount : SHADER_PERMUTATION_INT("STATIC_SAMPLE_COUNT", MAX_FILTER_COMPILE_TIME_SAMPLES + 1);
	class FCombineAdditive : SHADER_PERMUTATION_BOOL("USE_COMBINE_ADDITIVE");
	class FManualUVBorder : SHADER_PERMUTATION_BOOL("USE_MANUAL_UV_BORDER");

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SAMPLE_COUNT_MAX"), MAX_FILTER_SAMPLES);
	}

	static bool ShouldCompilePermutation(EShaderPlatform Platform, uint32 StaticSampleCount)
	{
		if (IsMetalMobileSM5Platform(Platform) || IsMetalMobileSM6Platform(Platform))
		{
			return StaticSampleCount <= MAX_FILTER_COMPILE_TIME_SAMPLES_IOS;
		}
		else if (IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5))
		{
			return true;
		}
		else
		{
			return StaticSampleCount <= MAX_FILTER_COMPILE_TIME_SAMPLES_ES3_1;
		}
	}

	FFilterShader() = default;
	FFilterShader(const CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

class FFilterPS : public FFilterShader
{
public:
	DECLARE_GLOBAL_SHADER(FFilterPS);
	SHADER_USE_PARAMETER_STRUCT(FFilterPS, FFilterShader);

	using FPermutationDomain = TShaderPermutationDomain<FStaticSampleCount, FCombineAdditive, FManualUVBorder>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FFilterParameters, Filter)
		SHADER_PARAMETER(FScreenTransform, SvPositionToTextureUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToTextureUV)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		return FFilterShader::ShouldCompilePermutation(Parameters.Platform, PermutationVector.Get<FStaticSampleCount>());
	}
};

class FFilterVS : public FFilterShader
{
public:
	DECLARE_GLOBAL_SHADER(FFilterVS);
	SHADER_USE_PARAMETER_STRUCT(FFilterVS, FFilterShader);

	using FPermutationDomain = TShaderPermutationDomain<FStaticSampleCount>;
	using FParameters = FFilterPS::FParameters;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		const uint32 StaticSampleCount = PermutationVector.Get<FStaticSampleCount>();

		if (StaticSampleCount == 0)
		{
			return false;
		}
		else
		{
			return FFilterShader::ShouldCompilePermutation(Parameters.Platform, StaticSampleCount);
		}
	}
};

IMPLEMENT_GLOBAL_SHADER(FFilterVS, "/Engine/Private/FilterVertexShader.usf", "MainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FFilterPS, "/Engine/Private/FilterPixelShader.usf", "MainPS", SF_Pixel);

class FFilterCS : public FFilterShader
{
public:
	DECLARE_GLOBAL_SHADER(FFilterCS);
	SHADER_USE_PARAMETER_STRUCT(FFilterCS, FFilterShader);

	using FPermutationDomain = TShaderPermutationDomain<FStaticSampleCount, FCombineAdditive, FManualUVBorder>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FFilterParameters, Filter)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, RWOutputTexture)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FFilterShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GFilterComputeTileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GFilterComputeTileSizeY);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FFilterCS, "/Engine/Private/FilterPixelShader.usf", "MainCS", SF_Compute);
} //! namespace

FScreenPassTexture AddGaussianBlurPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const TCHAR* Name,
	FScreenPassTextureViewport OutputViewport,
	FScreenPassTextureSlice Filter,
	FScreenPassTexture Additive,
	TArrayView<const FVector2f> SampleOffsets,
	TArrayView<const FLinearColor> SampleWeights,
	bool UseMirrorAddressMode)
{
	check(!OutputViewport.IsEmpty());
	check(Filter.IsValid());
	check(SampleOffsets.Num() == SampleWeights.Num());
	check(SampleOffsets.Num() != 0);

	const bool bIsComputePass = View.bUseComputePasses;

	const bool bCombineAdditive = Additive.IsValid();

	const bool bManualUVBorder = Filter.ViewRect.Min != FIntPoint::ZeroValue || Filter.ViewRect.Max != Filter.TextureSRV->Desc.Texture->Desc.Extent;

	const uint32 SampleCount = SampleOffsets.Num();

	const uint32 StaticSampleCount = GetStaticSampleCount(SampleCount);

	FRHISamplerState* SamplerState;

	if (UseMirrorAddressMode)
	{
		SamplerState = TStaticSamplerState<SF_Bilinear, AM_Mirror, AM_Mirror, AM_Clamp>::GetRHI();
	}
	else
	{
		SamplerState = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Clamp>::GetRHI();
	}

	const FScreenPassTextureSliceInput FilterInput = GetScreenPassTextureInput(Filter, SamplerState);

	FScreenPassTextureInput AdditiveInput;

	if (bCombineAdditive)
	{
		AdditiveInput = GetScreenPassTextureInput(Additive, SamplerState);
	}

	const FRDGTextureDesc& InputDesc = Filter.TextureSRV->Desc.Texture->Desc;
	FRDGTextureDesc OutputDesc = FRDGTextureDesc::Create2D(
		OutputViewport.Extent,
		InputDesc.Format,
		FClearValueBinding(FLinearColor::Transparent),
		/* InFlags = */ TexCreate_ShaderResource | (bIsComputePass ? TexCreate_UAV : (TexCreate_RenderTargetable | TexCreate_NoFastClear)) | (InputDesc.Flags & (TexCreate_FastVRAM | TexCreate_FastVRAMPartialAlloc)));

	const FScreenPassRenderTarget Output(GraphBuilder.CreateTexture(OutputDesc, Name), OutputViewport.Rect, ERenderTargetLoadAction::ENoAction);

	if (bIsComputePass)
	{
		FFilterCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFilterCS::FParameters>();
		GetFilterParameters(PassParameters->Filter, FilterInput, AdditiveInput, SampleOffsets, SampleWeights);
		PassParameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Output));
		PassParameters->RWOutputTexture = GraphBuilder.CreateUAV(Output.Texture);

		FFilterCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FFilterCS::FCombineAdditive>(bCombineAdditive);
		PermutationVector.Set<FFilterCS::FStaticSampleCount>(StaticSampleCount);
		PermutationVector.Set<FFilterPS::FManualUVBorder>(bManualUVBorder);
		TShaderMapRef<FFilterCS> ComputeShader(View.ShaderMap, PermutationVector);

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("GaussianBlur.%s %dx%d (CS)", Name, OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(OutputViewport.Rect.Size(), FIntPoint(GFilterComputeTileSizeX, GFilterComputeTileSizeY)));
	}
	else
	{
		FFilterPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FFilterPS::FParameters>();
		GetFilterParameters(PassParameters->Filter, FilterInput, AdditiveInput, SampleOffsets, SampleWeights);
		PassParameters->ViewportUVToTextureUV = (
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Filter), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TextureUV));
		PassParameters->SvPositionToTextureUV = (
			FScreenTransform::ChangeTextureBasisFromTo(FScreenPassTextureViewport(Output), FScreenTransform::ETextureBasis::TexelPosition, FScreenTransform::ETextureBasis::ViewportUV) *
			PassParameters->ViewportUVToTextureUV);
		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		FFilterPS::FPermutationDomain PixelPermutationVector;
		PixelPermutationVector.Set<FFilterPS::FCombineAdditive>(bCombineAdditive);
		PixelPermutationVector.Set<FFilterPS::FStaticSampleCount>(StaticSampleCount);
		PixelPermutationVector.Set<FFilterPS::FManualUVBorder>(bManualUVBorder);
		TShaderMapRef<FFilterPS> PixelShader(View.ShaderMap, PixelPermutationVector);

		if (StaticSampleCount != 0)
		{
			FFilterVS::FPermutationDomain VertexPermutationVector;
			VertexPermutationVector.Set<FFilterVS::FStaticSampleCount>(StaticSampleCount);
			TShaderMapRef<FFilterVS> VertexShader(View.ShaderMap, VertexPermutationVector);

			AddDrawScreenPass(
				GraphBuilder,
				RDG_EVENT_NAME("GaussianBlur.%s %dx%d (PS, Static)", Name, OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
				View,
				OutputViewport,
				FScreenPassTextureViewport(Filter),
				FScreenPassPipelineState(VertexShader, PixelShader),
				PassParameters,
				[VertexShader, PixelShader, PassParameters] (FRHICommandList& RHICmdList)
			{
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
			});
		}
		else
		{
			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				View.ShaderMap,
				RDG_EVENT_NAME("GaussianBlur.%s %dx%d (PS, Dynamic)", Name, OutputViewport.Rect.Width(), OutputViewport.Rect.Height()),
				PixelShader,
				PassParameters,
				Output.ViewRect);
		}
	}

	return FScreenPassTexture(Output);
}

FScreenPassTexture AddGaussianBlurPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FGaussianBlurInputs& Inputs)
{
	check(Inputs.Filter.IsValid());

	const FScreenPassTextureViewport FilterViewport(Inputs.Filter);

	const float BlurRadius = GetBlurRadius(FilterViewport.Rect.Width(), Inputs.KernelSizePercent);

	const uint32 SampleCountMax = GetSampleCountMax(View.GetFeatureLevel(), View.GetShaderPlatform());

	const FVector2f InverseFilterTextureExtent(
		1.0f / static_cast<float>(FilterViewport.Extent.X),
		1.0f / static_cast<float>(FilterViewport.Extent.Y));

	const bool bFastBlurEnabled = IsFastBlurEnabled(BlurRadius);

	// Downscale output width by half when using fast blur optimization.
	const FScreenPassTextureViewport HorizontalOutputViewport = bFastBlurEnabled
		? GetDownscaledViewport(FilterViewport, FIntPoint(2, 1))
		: FilterViewport;

	FScreenPassTexture HorizontalOutput;

	// Horizontal Pass
	{
		FVector2f OffsetAndWeight[MAX_FILTER_SAMPLES];
		FLinearColor SampleWeights[MAX_FILTER_SAMPLES];
		FVector2f SampleOffsets[MAX_FILTER_SAMPLES];

		const uint32 SampleCount = Compute1DGaussianFilterKernel(OffsetAndWeight, SampleCountMax, BlurRadius, Inputs.CrossCenterWeight.X);

		// Weights multiplied by a white tint.
		for (uint32 i = 0; i < SampleCount; ++i)
		{
			const float Weight = OffsetAndWeight[i].Y;

			SampleWeights[i] = FLinearColor(Weight, Weight, Weight, Weight);
		}

		for (uint32 i = 0; i < SampleCount; ++i)
		{
			const float Offset = OffsetAndWeight[i].X;

			SampleOffsets[i] = FVector2f(InverseFilterTextureExtent.X * Offset, 0.0f);
		}

		// Horizontal pass doesn't use additive combine.
		FScreenPassTexture Additive;

		HorizontalOutput = AddGaussianBlurPass(
			GraphBuilder,
			View,
			Inputs.NameX,
			HorizontalOutputViewport,
			Inputs.Filter,
			Additive,
			TArrayView<const FVector2f>(SampleOffsets, SampleCount),
			TArrayView<const FLinearColor>(SampleWeights, SampleCount),
			Inputs.UseMirrorAddressMode);
	}

	// Vertical Pass
	{
		FVector2f OffsetAndWeight[MAX_FILTER_SAMPLES];
		FLinearColor SampleWeights[MAX_FILTER_SAMPLES];
		FVector2f SampleOffsets[MAX_FILTER_SAMPLES];

		const uint32 SampleCount = Compute1DGaussianFilterKernel(OffsetAndWeight, SampleCountMax, BlurRadius, Inputs.CrossCenterWeight.Y);

		// Weights multiplied by a input tint color.
		for (uint32 i = 0; i < SampleCount; ++i)
		{
			const float Weight = OffsetAndWeight[i].Y;

			SampleWeights[i] = Inputs.TintColor * Weight;
		}

		for (uint32 i = 0; i < SampleCount; ++i)
		{
			const float Offset = OffsetAndWeight[i].X;

			SampleOffsets[i] = FVector2f(0, InverseFilterTextureExtent.Y * Offset);
		}

		return AddGaussianBlurPass(
			GraphBuilder,
			View,
			Inputs.NameY,
			FilterViewport,
			FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, HorizontalOutput),
			Inputs.Additive,
			TArrayView<const FVector2f>(SampleOffsets, SampleCount),
			TArrayView<const FLinearColor>(SampleWeights, SampleCount),
			Inputs.UseMirrorAddressMode);
	}
}
