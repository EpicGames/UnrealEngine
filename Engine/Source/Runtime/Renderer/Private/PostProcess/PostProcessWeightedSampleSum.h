// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"

struct FGaussianBlurInputs
{
	// Friendly names of the blur passes along the X and Y axis. Used for logging and profiling.
	const TCHAR* NameX = nullptr;
	const TCHAR* NameY = nullptr;

	// The input texture to be filtered.
	FScreenPassTextureSlice Filter;

	// The input texture to be added after filtering.
	FScreenPassTexture Additive;

	// The color to tint when filtering.
	FLinearColor TintColor;

	// Controls the cross shape of the blur, in both X / Y directions. See r.Bloom.Cross.
	FVector2f CrossCenterWeight = FVector2f::ZeroVector;

	// The filter kernel size in percentage of the screen.
	float KernelSizePercent = 0.0f;

	bool UseMirrorAddressMode = false;
};

using FGaussianBlurOutputs = FScreenPassTexture;

extern RENDERER_API FGaussianBlurOutputs AddGaussianBlurPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FGaussianBlurInputs& Inputs);
