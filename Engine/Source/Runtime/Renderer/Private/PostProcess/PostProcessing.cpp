// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/PostProcessing.h"
#include "CoreMinimal.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "Containers/StringView.h"
#include "CustomRenderPassSceneCapture.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DebugViewModeHelpers.h"
#include "DeferredShadingRendererTypes.h"
#include "DistanceFieldLightingShared.h"
#include "Engine/BlendableInterface.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/Font.h"
#include "Engine/FontImportOptions.h"
#include "Engine/Scene.h"
#include "FinalPostProcessSettings.h"
#include "FXSystem.h"
#include "GlobalShader.h"
#include "GPUSkinCache.h"
#include "HairStrands/HairStrandsComposition.h"
#include "HairStrands/HairStrandsUtils.h"
#include "HAL/IConsoleManager.h"
#include "HighResScreenshot.h"
#include "IESTextureManager.h"
#include "IlluminanceMeter.h"
#include "InstanceCulling/InstanceCullingManager.h"
#if WITH_EDITOR
	#include "LightFunctionAtlas.h"
#endif
#include "LocalFogVolumeRendering.h"
#include "Logging/LogMacros.h"
#include "Lumen/LumenSceneData.h"
#include "Lumen/LumenVisualize.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "MegaLights/MegaLights.h"
#include "MegaLights/MegaLightsVisualize.h"
#include "Misc/LargeWorldCoordinates.h"
#include "MobileDistortionPass.h"
#include "MobileSeparateTranslucencyPass.h"
#include "MobileSSR.h"
#include "Nanite/NaniteCullRaster.h"
#include "OverridePassSequence.h"
#include "PathTracingResources.h"
#include "PostProcess/AlphaInvert.h"
#include "PostProcess/ChannelMask.h"
#include "PostProcess/DiaphragmDOF.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/PostProcessAA.h"
#include "PostProcess/PostProcessBloomSetup.h"
#include "PostProcess/PostProcessBokehDOF.h"
#if WITH_EDITOR
	#include "PostProcess/PostProcessBufferInspector.h"
#endif
#include "PostProcess/PostProcessCombineLUTs.h"
#include "PostProcess/PostProcessCompositeDebugPrimitives.h"
#if WITH_EDITOR
	#include "PostProcess/PostProcessCompositeEditorPrimitives.h"
#endif
#include "PostProcess/PostProcessCompositePrimitivesCommon.h"
#include "PostProcess/PostProcessDeviceEncodingOnly.h"
#include "PostProcess/PostProcessDOF.h"
#include "PostProcess/PostProcessDownsample.h"
#include "PostProcess/PostProcessEyeAdaptation.h"
#include "PostProcess/PostProcessFFTBloom.h"
#include "PostProcess/PostProcessGBufferHints.h"
#include "PostProcess/PostProcessHistogram.h"
#include "PostProcess/PostProcessHMD.h"
#include "PostProcess/PostProcessInputs.h"
#include "PostProcess/PostProcessLensFlares.h"
#include "PostProcess/PostProcessLocalExposure.h"
#include "PostProcess/PostProcessLODColorationLegend.h"
#include "PostProcess/PostProcessMaterial.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "PostProcess/PostProcessMobile.h"
#include "PostProcess/PostProcessMotionBlur.h"
#include "PostProcess/PostProcessSelectionOutline.h"
#include "PostProcess/PostProcessStreamingAccuracyLegend.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "PostProcess/PostProcessTestImage.h"
#include "PostProcess/PostProcessTonemap.h"
#include "PostProcess/PostProcessUpscale.h"
#include "PostProcess/PostProcessVisualizeBuffer.h"
#include "PostProcess/PostProcessVisualizeComplexity.h"
#include "PostProcess/PostProcessVisualizeHDR.h"
#include "PostProcess/PostProcessVisualizeLevelInstance.h"
#include "PostProcess/PostProcessVisualizeLocalExposure.h"
#include "PostProcess/PostProcessVisualizeNanite.h"
#include "PostProcess/PostProcessVisualizeVirtualTexture.h"
#include "PostProcess/PostProcessWeightedSampleSum.h"
#include "PostProcess/SubpixelMorphologicalAA.h"
#include "PostProcess/TemporalAA.h"
#include "PostProcess/VisualizeMotionVectors.h"
#include "PostProcess/VisualizeShadingModels.h"
#include "RectLightTextureManager.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/CustomRenderPass.h"
#include "Rendering/MotionVectorSimulation.h"
#include "RenderUtils.h"
#include "RenderViewportFeedback.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHIResources.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneTypes.h"
#include "SceneUniformBuffer.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderPrint.h"
#include "ShaderPrintParameters.h"
#include "ShowFlags.h"
#include "SkyAtmosphereRendering.h"
#include "SparseVolumeTexture/SparseVolumeTextureStreamingVisualize.h"
#include "Stats/Stats.h"
#include "StereoRendering.h"
#include "StereoRenderUtils.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "TemporalUpscaler.h"
#include "TranslucentPassResource.h"
#include "TranslucentRendering.h"
#include "UnrealClient.h"
#include "UnrealEngine.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VisualizeTexture.h"
#include "VT/VirtualTextureFeedbackResource.h"
#include "VT/VirtualTextureVisualizationData.h"
#include "SceneViewState.h"

bool IsMobileEyeAdaptationEnabled(const FViewInfo& View);

bool IsValidBloomSetupVariation(bool bUseBloom, bool bUseSun, bool bUseDof, bool bUseEyeAdaptation);

extern FScreenPassTexture AddVisualizeLightGridPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture ScreenPassSceneColor, FScreenPassTexture SceneDepthTexture);
extern bool ShouldVisualizeLightGrid(EShaderPlatform ShaderPlatform);

namespace
{
TAutoConsoleVariable<float> CVarDepthOfFieldNearBlurSizeThreshold(
	TEXT("r.DepthOfField.NearBlurSizeThreshold"),
	0.01f,
	TEXT("Sets the minimum near blur size before the effect is forcably disabled. Currently only affects Gaussian DOF.\n")
	TEXT(" (default: 0.01)"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarDepthOfFieldMaxSize(
	TEXT("r.DepthOfField.MaxSize"),
	100.0f,
	TEXT("Allows to clamp the gaussian depth of field radius (for better performance), default: 100"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<bool> CVarBloomApplyLocalExposure(
	TEXT("r.Bloom.ApplyLocalExposure"),
	true,
	TEXT("Whether to apply local exposure when calculating bloom, default: true"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static bool GPostProcessingPropagateAlpha = false;
/**
* NOTE (5.5):
* r.PostProcessing.PropagateAlpha has been converted back to a boolean. In order to prevent silent failures
* with IConsoleManager::Get().FindTConsoleVariableDataInt returning 0 with a boolean cvar set to True, we now use
* a FAutoConsoleVariableRef which will warn licensees with typed-access at runtime, see IConsoleObject::AsVariableBool()
* or IConsoleObject::AsVariableInt(). However both CVar->GetBool() & CVar->GetInt() will continue to work, assuming > 0
* comparison was used.
*/
FAutoConsoleVariableRef CVarPostProcessingPropagateAlpha(
	TEXT("r.PostProcessing.PropagateAlpha"),
	GPostProcessingPropagateAlpha,
	TEXT("Enforce alpha in scene color (overriding r.SceneColorFormat if necessary) and propagate it through the renderer's post-processing chain, default: false"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarPostProcessingPreferCompute(
	TEXT("r.PostProcessing.PreferCompute"),
	0,
	TEXT("Will use compute shaders for post processing where implementations available."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarPostProcessingQuarterResolutionDownsample(
	TEXT("r.PostProcessing.QuarterResolutionDownsample"),
	0,
	TEXT("Uses quarter resolution downsample instead of half resolution to feed into exposure / bloom."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarDownsampleQuality(
	TEXT("r.PostProcessing.DownsampleQuality"), 0,
	TEXT("Defines the quality used for downsampling to half or quarter res the scene color in post processing chain.\n")
	TEXT(" 0: low quality (default)\n")
	TEXT(" 1: high quality\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarDownsampleChainQuality(
	TEXT("r.PostProcessing.DownsampleChainQuality"), 1,
	TEXT("Defines the quality used for downsampling to the scene color in scene color chains.\n")
	TEXT(" 0: low quality\n")
	TEXT(" 1: high quality (default)\n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

#if !(UE_BUILD_SHIPPING)
TAutoConsoleVariable<int32> CVarPostProcessingForceAsyncDispatch(
	TEXT("r.PostProcessing.ForceAsyncDispatch"),
	0,
	TEXT("Will force asynchronous dispatch for post processing compute shaders where implementations available.\n")
	TEXT("Only available for testing in non-shipping builds."),
	ECVF_RenderThreadSafe);
#endif

#if WITH_EDITOR
TAutoConsoleVariable<int32> CVarGBufferPicking(
	TEXT("r.PostProcessing.GBufferPicking"), 0,
	TEXT("Evaluate GBuffer value for debugging purpose."),
	ECVF_RenderThreadSafe);
#endif

#if !(UE_BUILD_SHIPPING)
TAutoConsoleVariable<int32> CVarUserSceneTextureDebug(
	TEXT("r.PostProcessing.UserSceneTextureDebug"),
	2,
	TEXT("Enable debug display of post process UserSceneTexture inputs and outputs.\n")
	TEXT(" 0: disabled\n")
	TEXT(" 1: enabled\n")
	TEXT(" 2: enable on error -- missing input or unused output (default).  Suppressed by DisableAllScreenMessages.\n")
	TEXT(" 3: enable only for view with texture visualized through Vis / VisualizeTexture command, to avoid debug clutter in other views.\n"),
	ECVF_RenderThreadSafe);
#endif
}

#if WITH_EDITOR
static void AddGBufferPicking(FRDGBuilder& GraphBuilder, const FViewInfo& View, const TRDGUniformBufferRef<FSceneTextureUniformParameters>& SceneTextures);
#endif 

EDownsampleQuality GetDownsampleQuality(const TAutoConsoleVariable<int32>& CVar)
{
	const int32 DownsampleQuality = FMath::Clamp(CVar.GetValueOnRenderThread(), 0, 1);
	return static_cast<EDownsampleQuality>(DownsampleQuality);
}

bool IsPostProcessingWithComputeEnabled(ERHIFeatureLevel::Type FeatureLevel)
{
	// Any thread is used due to FViewInfo initialization.
	return CVarPostProcessingPreferCompute.GetValueOnAnyThread() && FeatureLevel >= ERHIFeatureLevel::SM5;
}

bool IsPostProcessingOutputInHDR()
{
	static const auto CVarDumpFramesAsHDR = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.BufferVisualizationDumpFramesAsHDR"));

	return CVarDumpFramesAsHDR->GetValueOnRenderThread() != 0 || GetHighResScreenshotConfig().bCaptureHDR;
}

bool IsPostProcessingEnabled(const FViewInfo& View)
{
	if (View.GetFeatureLevel() >= ERHIFeatureLevel::SM5)
	{
		return
			 View.Family->EngineShowFlags.PostProcessing &&
			!View.Family->EngineShowFlags.VisualizeDistanceFieldAO &&
			!View.Family->EngineShowFlags.VisualizeShadingModels &&
			!View.Family->EngineShowFlags.VisualizeVolumetricCloudConservativeDensity &&
			!View.Family->EngineShowFlags.VisualizeVolumetricCloudEmptySpaceSkipping &&
			!View.Family->EngineShowFlags.ShaderComplexity;
	}
	else
	{
		return View.Family->EngineShowFlags.PostProcessing && !View.Family->EngineShowFlags.ShaderComplexity && IsMobileHDR();
	}
}

bool IsPostProcessingWithAlphaChannelSupported()
{
	return CVarPostProcessingPropagateAlpha->GetBool();
}

#if DEBUG_POST_PROCESS_VOLUME_ENABLE
FScreenPassTexture AddFinalPostProcessDebugInfoPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor, FScreenPassRenderTarget& OverrideOutput);
#endif

#if !UE_BUILD_SHIPPING
static void AddUserSceneTextureDebugPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 ViewIndex, FScreenPassTexture Output);
#endif

FDefaultTemporalUpscaler::FOutputs AddThirdPartyTemporalUpscalerPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FDefaultTemporalUpscaler::FInputs& Inputs)
{
	using UE::Renderer::Private::ITemporalUpscaler;

	const ITemporalUpscaler* UpscalerToUse = View.Family->GetTemporalUpscalerInterface();
	check(UpscalerToUse);

	const TCHAR* UpscalerName = UpscalerToUse->GetDebugName();

	// Translate the inputs to the third party temporal upscaler.
	ITemporalUpscaler::FInputs ThirdPartyInputs;
	ThirdPartyInputs.OutputViewRect.Min = FIntPoint::ZeroValue;
	ThirdPartyInputs.OutputViewRect.Max = View.GetSecondaryViewRectSize();
	ThirdPartyInputs.TemporalJitterPixels = FVector2f(View.TemporalJitterPixels);
	ThirdPartyInputs.PreExposure = View.PreExposure;
	ThirdPartyInputs.SceneColor = Inputs.SceneColor;
	ThirdPartyInputs.SceneDepth = Inputs.SceneDepth;
	ThirdPartyInputs.SceneVelocity = Inputs.SceneVelocity;
	ThirdPartyInputs.EyeAdaptationTexture = AddCopyEyeAdaptationDataToTexturePass(GraphBuilder, View);
	
	if (View.PrevViewInfo.ThirdPartyTemporalUpscalerHistory && View.PrevViewInfo.ThirdPartyTemporalUpscalerHistory->GetDebugName() == UpscalerName)
	{
		ThirdPartyInputs.PrevHistory = View.PrevViewInfo.ThirdPartyTemporalUpscalerHistory;
	}

	// Standard event scope for temporal upscaler to have all profiling information not matter what,
	// and with explicit detection of third party.
	RDG_EVENT_SCOPE(
		GraphBuilder,
		"ThirdParty %s %dx%d -> %dx%d",
		UpscalerToUse->GetDebugName(),
		View.ViewRect.Width(), View.ViewRect.Height(),
		ThirdPartyInputs.OutputViewRect.Width(), ThirdPartyInputs.OutputViewRect.Height());

	ITemporalUpscaler::FOutputs ThirdPartyOutputs = UpscalerToUse->AddPasses(
		GraphBuilder,
		View,
		ThirdPartyInputs);

	check(ThirdPartyOutputs.FullRes.ViewRect == ThirdPartyInputs.OutputViewRect);
	check(ThirdPartyOutputs.FullRes.ViewRect.Max.X <= ThirdPartyOutputs.FullRes.Texture->Desc.Extent.X);
	check(ThirdPartyOutputs.FullRes.ViewRect.Max.Y <= ThirdPartyOutputs.FullRes.Texture->Desc.Extent.Y);

	check(ThirdPartyOutputs.NewHistory);
	check(ThirdPartyOutputs.NewHistory->GetDebugName() == UpscalerToUse->GetDebugName());

	// Translate the output.
	FDefaultTemporalUpscaler::FOutputs Outputs;
	Outputs.FullRes = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, ThirdPartyOutputs.FullRes);

	// Saves history for next frame.
	if (!View.bStatePrevViewInfoIsReadOnly)
	{
		View.ViewState->PrevFrameViewInfo.ThirdPartyTemporalUpscalerHistory = ThirdPartyOutputs.NewHistory;
	}

	// Save output for next frame's SSR
	if (!View.bStatePrevViewInfoIsReadOnly)
	{
		FTemporalAAHistory& OutputHistory = View.ViewState->PrevFrameViewInfo.TemporalAAHistory;

		GraphBuilder.QueueTextureExtraction(ThirdPartyOutputs.FullRes.Texture, &OutputHistory.RT[0]);

		OutputHistory.ViewportRect = ThirdPartyOutputs.FullRes.ViewRect;
		OutputHistory.ReferenceBufferSize = ThirdPartyOutputs.FullRes.Texture->Desc.Extent;
	}

	return Outputs;
}

namespace UE::Renderer::PostProcess
{
	/*
	* Issue scene view extension pass callbacks, mimicking "AddPostProcessMaterialChain".
	* The "AddAfterPass" lambdas are used instead for later extension points in the override pass sequence.
	*/
	FScreenPassTexture AddSceneViewExtensionPassChain(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FPostProcessMaterialInputs& InputsTemplate,
		const FPostProcessingPassDelegateArray& Delegates,
		EPostProcessMaterialInput MaterialInput = EPostProcessMaterialInput::SceneColor)
	{
		FScreenPassTextureSlice CurrentInput = InputsTemplate.GetInput(MaterialInput);
		FScreenPassTexture Outputs;

		for (int32 DelegateIndex = 0; DelegateIndex < Delegates.Num(); ++DelegateIndex)
		{
			FPostProcessMaterialInputs Inputs = InputsTemplate;
			Inputs.SetInput(MaterialInput, CurrentInput);
			
			Outputs = Delegates[DelegateIndex].Execute(GraphBuilder, View, Inputs);

			CurrentInput = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, Outputs);
		}

		if (!Outputs.IsValid())
		{
			Outputs = FScreenPassTexture::CopyFromSlice(GraphBuilder, CurrentInput);
		}

		return Outputs;
	};
}

void AddPostProcessingPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, 
	const int32 ViewIndex,
	FSceneUniformBuffer& SceneUniformBuffer,
	EDiffuseIndirectMethod DiffuseIndirectMethod,
	EReflectionsMethod ReflectionsMethod,
	const FPostProcessingInputs& Inputs,
	const Nanite::FRasterResults* NaniteRasterResults,
	FInstanceCullingManager& InstanceCullingManager,
	FVirtualShadowMapArray* VirtualShadowMapArray, 
	const FLumenSceneFrameTemporaries& LumenFrameTemporaries,
	const TSharedPtr<FMegaLightsFrameTemporaries>& MegaLightsFrameTemporaries,
	const FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
	FScreenPassTexture TSRFlickeringInput,
	FRDGTextureRef& InstancedEditorDepthTexture)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderPostProcessing);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PostProcessing_Process);
	using namespace UE::Renderer::PostProcess;

	check(IsInRenderingThread());
#if DO_CHECK || USING_CODE_ANALYSIS
	check(View.VerifyMembersChecks());
#endif
	Inputs.Validate();

	check(View.Family);

	FScene* Scene = View.Family->Scene->GetRenderScene();

	const FIntRect PrimaryViewRect = View.ViewRect;

	const FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, Inputs.SceneTextures);

	const FScreenPassRenderTarget ViewFamilyOutput = FScreenPassRenderTarget::CreateViewFamilyOutput(Inputs.ViewFamilyTexture, View);
	const FScreenPassRenderTarget ViewFamilyDepthOutput = FScreenPassRenderTarget::CreateViewFamilyOutput(Inputs.ViewFamilyDepthTexture, View);
	const FScreenPassTexture SceneDepth(SceneTextureParameters.SceneDepthTexture, PrimaryViewRect);
	const FScreenPassTexture CustomDepth(Inputs.CustomDepthTexture, PrimaryViewRect);
	const FScreenPassTexture Velocity(SceneTextureParameters.GBufferVelocityTexture, PrimaryViewRect);
	const FScreenPassTexture BlackDummy(GSystemTextures.GetBlackDummy(GraphBuilder));
	
	FTranslucencyPassResources PostDOFTranslucencyResources = Inputs.TranslucencyViewResourcesMap.Get(ETranslucencyPass::TPT_TranslucencyAfterDOF);
	const FTranslucencyPassResources& PostMotionBlurTranslucencyResources = Inputs.TranslucencyViewResourcesMap.Get(ETranslucencyPass::TPT_TranslucencyAfterMotionBlur);

	// Whether should process the alpha channel of the scene color.
	const bool bProcessSceneColorAlpha = IsPostProcessingWithAlphaChannelSupported();
	const EPixelFormat SceneColorFormat = bProcessSceneColorAlpha ? PF_FloatRGBA : PF_FloatR11G11B10;

	// Scene color is updated incrementally through the post process pipeline.
	FScreenPassTexture SceneColor((*Inputs.SceneTextures)->SceneColorTexture, PrimaryViewRect);

	// Assigned before and after the tonemapper.
	FScreenPassTextureSlice SceneColorBeforeTonemapSlice;
	FScreenPassTexture SceneColorAfterTonemap;

	// Unprocessed scene color stores the original input.
	const FScreenPassTexture OriginalSceneColor = SceneColor;

	// Default the new eye adaptation to the last one in case it's not generated this frame.
	const FEyeAdaptationParameters EyeAdaptationParameters = GetEyeAdaptationParameters(GraphBuilder, View);
	FRDGBufferRef LastEyeAdaptationBuffer = GetEyeAdaptationBuffer(GraphBuilder, View);
	FRDGBufferRef EyeAdaptationBuffer = LastEyeAdaptationBuffer;

	const FIntRect ExposureIlluminanceRect = GetDownscaledRect(PrimaryViewRect, GetAutoExposureIlluminanceDownscaleFactor());
	FScreenPassTexture ExposureIlluminance = FScreenPassTexture(Inputs.ExposureIlluminance, ExposureIlluminanceRect);

	FLocalExposureParameters LocalExposureParameters;

	// Histogram defaults to black because the histogram eye adaptation pass is used for the manual metering mode.
	FRDGTextureRef HistogramTexture = BlackDummy.Texture;

	FRDGTextureRef LocalExposureBilateralGridTexture = nullptr;
	FRDGTextureRef LocalExposureBlurredLogLumTexture = BlackDummy.Texture;
	FExposureFusionData ExposureFusionData;

	FVisualizeTemporalUpscalerInputs VisualizeTemporalUpscalerInputs;

	const bool bViewDebugMaterialsEnabled = View.RequiresDebugMaterials();
	const FEngineShowFlags& EngineShowFlags = View.Family->EngineShowFlags;
	const bool bVisualizeHDR = EngineShowFlags.VisualizeHDR && !EngineShowFlags.Wireframe;
	const bool bViewFamilyOutputInHDR = View.Family->RenderTarget->GetSceneHDREnabled();
	const bool bVisualizeGBuffer = IsVisualizeGBufferEnabled(View);
	const bool bOutputInHDR = IsPostProcessingOutputInHDR();
	const int32 LumenVisualizeMode = GetLumenVisualizeMode(View);
	const bool bPostProcessingEnabled = IsPostProcessingEnabled(View);

	// Temporal Anti-aliasing. Also may perform a temporal upsample from primary to secondary view rect.
	const EMainTAAPassConfig TAAConfig = GetMainTAAPassConfig(View);

	const bool bApplyLensDistortion = View.LensDistortionLUT.IsEnabled();
	const bool bApplyLensDistortionInTSR = (LensDistortion::GetPassLocation(View) == LensDistortion::EPassLocation::TSR);

	FVisualizeBufferContext VisualizeBufferContext{};

	enum class EPass : uint32
	{
		MotionBlur,
		PostProcessMaterialBeforeBloom,
		Tonemap,
		FXAA,
		SMAA,
		PostProcessMaterialAfterTonemapping,
		VisualizeLumenScene,
		VisualizeMegaLights,
		VisualizeMegaLightsComplexity,
		VisualizeDepthOfField,
		VisualizeStationaryLightOverlap,
		VisualizeLightCulling,
		VisualizePostProcessStack,
		VisualizeSubstrate,
		VisualizeLightGrid,
		VisualizeSkyAtmosphere,
		VisualizeSkyLightIlluminanceMeter,
		VisualizeLightFunctionAtlas,
		VisualizeLocalFogVolumes,
		VisualizeLevelInstance,
		VisualizeVirtualShadowMaps_PreEditorPrimitives,
		VisualizeGBuffer,
		ChannelMask,
		SelectionOutline,
		EditorPrimitive,
		DebugPrimitive,
		VisualizeVirtualShadowMaps_PostEditorPrimitives,
		VisualizeVirtualTexture,
		VisualizeShadingModels,
		VisualizeGBufferHints,
		VisualizeSubsurface,
		VisualizeLumenSceneOverview,
		VisualizeMegaLightsOverview,
		VisualizeHDR,
		VisualizeLocalExposure,
		VisualizeMotionVectors,
		VisualizeTemporalUpscaler,
		PixelInspector,
		HMDDistortion,
		HighResolutionScreenshotMask,

		PrimaryUpscale,
		SecondaryUpscale,
		AlphaInvert,
		MAX
	};

	const auto TranslatePass = [](ISceneViewExtension::EPostProcessingPass Pass) -> EPass
	{
		switch (Pass)
		{
			case ISceneViewExtension::EPostProcessingPass::MotionBlur            : return EPass::MotionBlur;
			case ISceneViewExtension::EPostProcessingPass::Tonemap               : return EPass::Tonemap;
			case ISceneViewExtension::EPostProcessingPass::FXAA                  : return EPass::FXAA;
			case ISceneViewExtension::EPostProcessingPass::SMAA                  : return EPass::SMAA;
			case ISceneViewExtension::EPostProcessingPass::VisualizeDepthOfField : return EPass::VisualizeDepthOfField;

			default:
				check(false);
				return EPass::MAX;
		};
	};

	const TCHAR* PassNames[] =
	{
		TEXT("MotionBlur"),
		TEXT("PostProcessMaterial (SceneColorBeforeBloom)"),
		TEXT("Tonemap"),
		TEXT("FXAA"),
		TEXT("SMAA"),
		TEXT("PostProcessMaterial (SceneColorAfterTonemapping)"),
		TEXT("VisualizeLumenScene"),
		TEXT("VisualizeMegaLights"),
		TEXT("VisualizeMegaLightsComplexity"),
		TEXT("VisualizeDepthOfField"),
		TEXT("VisualizeStationaryLightOverlap"),
		TEXT("VisualizeLightCulling"),
		TEXT("VisualizePostProcessStack"),
		TEXT("VisualizeSubstrate"),
		TEXT("VisualizeLightGrid"),
		TEXT("VisualizeSkyAtmosphere"),
		TEXT("VisualizeSkyLightIlluminanceMeter"),
		TEXT("VisualizeLightFunctionAtlas"),
		TEXT("VisualizeLocalFogVolumes"),
		TEXT("VisualizeLevelInstance"),
		TEXT("VisualizeVirtualShadowMaps_PreEditorPrimitives"),
		TEXT("VisualizeGBuffer"),
		TEXT("ChannelMask"),
		TEXT("SelectionOutline"),
		TEXT("EditorPrimitive"),
		TEXT("DebugPrimitive"),
		TEXT("VisualizeVirtualShadowMaps_PostEditorPrimitives"),
		TEXT("VisualizeVirtualTexture"),
		TEXT("VisualizeShadingModels"),
		TEXT("VisualizeGBufferHints"),
		TEXT("VisualizeSubsurface"),
		TEXT("VisualizeLumenSceneOverview"),
		TEXT("VisualizeMegaLightsOverview"),
		TEXT("VisualizeHDR"),
		TEXT("VisualizeLocalExposure"),
		TEXT("VisualizeMotionVectors"),
		TEXT("VisualizeTemporalUpscaler"),
		TEXT("PixelInspector"),
		TEXT("HMDDistortion"),
		TEXT("HighResolutionScreenshotMask"),

		TEXT("PrimaryUpscale"),
		TEXT("SecondaryUpscale"),
		TEXT("AlphaInvert")
	};

	static_assert(static_cast<uint32>(EPass::MAX) == UE_ARRAY_COUNT(PassNames), "EPass does not match PassNames."); 

	TOverridePassSequence<EPass> PassSequence(ViewFamilyOutput);
	PassSequence.SetNames(PassNames, UE_ARRAY_COUNT(PassNames));
	PassSequence.SetEnabled(EPass::VisualizeStationaryLightOverlap, EngineShowFlags.StationaryLightOverlap);
	PassSequence.SetEnabled(EPass::VisualizeLightCulling, EngineShowFlags.VisualizeLightCulling);
#if DEBUG_POST_PROCESS_VOLUME_ENABLE
	PassSequence.SetEnabled(EPass::VisualizePostProcessStack, EngineShowFlags.VisualizePostProcessStack);
#else
	PassSequence.SetEnabled(EPass::VisualizePostProcessStack, false);
#endif
	PassSequence.SetEnabled(EPass::VisualizeLumenScene, LumenVisualizeMode >= 0 && LumenVisualizeMode != VISUALIZE_MODE_OVERVIEW && LumenVisualizeMode != VISUALIZE_MODE_PERFORMANCE_OVERVIEW && bPostProcessingEnabled && !EngineShowFlags.PathTracing);
	PassSequence.SetEnabled(EPass::VisualizeMegaLights, MegaLights::ShouldAddVisualizePostProcessingPass(View, ViewIndex, MegaLightsFrameTemporaries, /*bOverview*/ false));
	PassSequence.SetEnabled(EPass::VisualizeMegaLightsComplexity, MegaLights::ShouldAddVisualizeLightComplexityPostProcessingPass(View, ViewIndex, MegaLightsFrameTemporaries));
	PassSequence.SetEnabled(EPass::VisualizeSubstrate, Substrate::ShouldRenderSubstrateDebugPasses(View));
	PassSequence.SetEnabled(EPass::VisualizeLightGrid, View.Family && ShouldVisualizeLightGrid(View.Family->GetShaderPlatform()));

#if WITH_EDITOR
	PassSequence.SetEnabled(EPass::VisualizeSkyAtmosphere, GIsEditor && Scene && View.Family && View.Family->EngineShowFlags.VisualizeSkyAtmosphere && ShouldRenderSkyAtmosphereDebugPasses(Scene, View.Family->EngineShowFlags));
	PassSequence.SetEnabled(EPass::VisualizeSkyLightIlluminanceMeter, GIsEditor && Scene && Scene->SkyLight && View.Family && View.Family->EngineShowFlags.VisualizeSkyLightIlluminance);
	PassSequence.SetEnabled(EPass::VisualizeLightFunctionAtlas, GIsEditor && Scene && Scene->LightFunctionAtlasSceneData.GetLightFunctionAtlasEnabled() && View.Family && View.Family->EngineShowFlags.VisualizeLightFunctionAtlas);
	PassSequence.SetEnabled(EPass::VisualizeLocalFogVolumes, GIsEditor && Scene && View.Family && ShouldRenderLocalFogVolume(Scene, *View.Family) && View.Family->EngineShowFlags.VisualizeLocalFogVolumes);
	PassSequence.SetEnabled(EPass::VisualizeLevelInstance, GIsEditor && EngineShowFlags.EditingLevelInstance && EngineShowFlags.VisualizeLevelInstanceEditing && !bVisualizeHDR);
	PassSequence.SetEnabled(EPass::ChannelMask, View.Family->ChannelMaskParams.ColorChannelMask != EColorChannelMask::All || View.Family->ChannelMaskParams.bDrawAlphaBlendedCheckerboard);
	PassSequence.SetEnabled(EPass::SelectionOutline, GIsEditor && EngineShowFlags.Selection && EngineShowFlags.SelectionOutline && !EngineShowFlags.Wireframe && !bVisualizeHDR);
	PassSequence.SetEnabled(EPass::EditorPrimitive, FSceneRenderer::ShouldCompositeEditorPrimitives(View));
#else
	PassSequence.SetEnabled(EPass::VisualizeSkyAtmosphere, false);
	PassSequence.SetEnabled(EPass::VisualizeSkyLightIlluminanceMeter, false);
	PassSequence.SetEnabled(EPass::VisualizeLightFunctionAtlas, false);
	PassSequence.SetEnabled(EPass::VisualizeLocalFogVolumes, false);
	PassSequence.SetEnabled(EPass::VisualizeLevelInstance, false);
	PassSequence.SetEnabled(EPass::ChannelMask, false);
	PassSequence.SetEnabled(EPass::SelectionOutline, false);
	PassSequence.SetEnabled(EPass::EditorPrimitive, false);
#endif

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	PassSequence.SetEnabled(EPass::VisualizeVirtualShadowMaps_PreEditorPrimitives, VirtualShadowMapArray != nullptr && VirtualShadowMapArray->IsVisualizePassEnabled(View, ViewIndex, EVSMVisualizationPostPass::PreEditorPrimitives));
	PassSequence.SetEnabled(EPass::VisualizeVirtualShadowMaps_PostEditorPrimitives, VirtualShadowMapArray != nullptr && VirtualShadowMapArray->IsVisualizePassEnabled(View, ViewIndex, EVSMVisualizationPostPass::PostEditorPrimitives));
#else
	PassSequence.SetEnabled(EPass::VisualizeVirtualShadowMaps_PreEditorPrimitives, false);
	PassSequence.SetEnabled(EPass::VisualizeVirtualShadowMaps_PostEditorPrimitives, false);
#endif

	PassSequence.SetEnabled(EPass::DebugPrimitive, IsDebugPrimitivePassEnabled(View));
	PassSequence.SetEnabled(EPass::VisualizeVirtualTexture, EngineShowFlags.VisualizeVirtualTexture && bViewDebugMaterialsEnabled);
	PassSequence.SetEnabled(EPass::VisualizeShadingModels, EngineShowFlags.VisualizeShadingModels);
	PassSequence.SetEnabled(EPass::VisualizeGBufferHints, EngineShowFlags.GBufferHints);
	PassSequence.SetEnabled(EPass::VisualizeSubsurface, EngineShowFlags.VisualizeSSS);
	PassSequence.SetEnabled(EPass::VisualizeGBuffer, bVisualizeGBuffer);
	PassSequence.SetEnabled(EPass::VisualizeLumenSceneOverview, (LumenVisualizeMode == VISUALIZE_MODE_OVERVIEW || LumenVisualizeMode == VISUALIZE_MODE_PERFORMANCE_OVERVIEW) && bPostProcessingEnabled && !EngineShowFlags.PathTracing);
	PassSequence.SetEnabled(EPass::VisualizeMegaLightsOverview, MegaLights::ShouldAddVisualizePostProcessingPass(View, ViewIndex, MegaLightsFrameTemporaries, /*bOverview*/ true));
	PassSequence.SetEnabled(EPass::VisualizeHDR, bVisualizeHDR);
	PassSequence.SetEnabled(EPass::VisualizeMotionVectors, EngineShowFlags.VisualizeMotionVectors || EngineShowFlags.VisualizeReprojection);
	PassSequence.SetEnabled(EPass::VisualizeTemporalUpscaler, EngineShowFlags.VisualizeTemporalUpscaler);
#if WITH_EDITOR
	PassSequence.SetEnabled(EPass::PixelInspector, View.bUsePixelInspector);
#else
	PassSequence.SetEnabled(EPass::PixelInspector, false);
#endif
	PassSequence.SetEnabled(EPass::HMDDistortion, EngineShowFlags.StereoRendering && EngineShowFlags.HMDDistortion);
	PassSequence.SetEnabled(EPass::HighResolutionScreenshotMask, IsHighResolutionScreenshotMaskEnabled(View));
	PassSequence.SetEnabled(EPass::PrimaryUpscale, (bApplyLensDistortion && !bApplyLensDistortionInTSR) || (View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::SpatialUpscale && PrimaryViewRect.Size() != View.GetSecondaryViewRectSize()));
	PassSequence.SetEnabled(EPass::SecondaryUpscale, View.RequiresSecondaryUpscale() || View.Family->GetSecondarySpatialUpscalerInterface() != nullptr);
	PassSequence.SetEnabled(EPass::AlphaInvert, EngineShowFlags.AlphaInvert && !PassSequence.IsEnabled(EPass::PrimaryUpscale)); // The primary upscale does an alpha invert, so if that is active we do not run the AlphaInvert pass (which would undo the invert).

	const auto GetPostProcessMaterialInputs = [&](FScreenPassTextureSlice InSceneColorSlice)
	{ 
		FPostProcessMaterialInputs PostProcessMaterialInputs;

		PostProcessMaterialInputs.SetInput(EPostProcessMaterialInput::SceneColor, InSceneColorSlice);

		FIntRect ViewRect{ 0, 0, 1, 1 };

		if (Inputs.PathTracingResources.bPostProcessEnabled)
		{
			const FPathTracingResources& PathTracingResources = Inputs.PathTracingResources;

			ViewRect = InSceneColorSlice.ViewRect;
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::Radiance, FScreenPassTexture(PathTracingResources.Radiance, ViewRect));
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::DenoisedRadiance, FScreenPassTexture(PathTracingResources.DenoisedRadiance, ViewRect));
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::Albedo, FScreenPassTexture(PathTracingResources.Albedo, ViewRect));
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::Normal, FScreenPassTexture(PathTracingResources.Normal, ViewRect));
			PostProcessMaterialInputs.SetPathTracingInput(EPathTracingPostProcessMaterialInput::Variance, FScreenPassTexture(PathTracingResources.Variance, ViewRect));
		}
		
		if (PostDOFTranslucencyResources.IsValid())
		{
			ViewRect = PostDOFTranslucencyResources.ViewRect;
		}

		PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SeparateTranslucency, FScreenPassTexture(PostDOFTranslucencyResources.GetColorForRead(GraphBuilder), ViewRect));
		PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::Velocity, Velocity);
		PostProcessMaterialInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);
		PostProcessMaterialInputs.CustomDepthTexture = CustomDepth.Texture;
		PostProcessMaterialInputs.bManualStencilTest = Inputs.bSeparateCustomStencil;
		PostProcessMaterialInputs.SceneWithoutWaterTextures = &SceneWithoutWaterTextures;

		return PostProcessMaterialInputs;
	};

	const auto AddAfterPass = [&](EPass InPass, FScreenPassTexture InSceneColor) -> FScreenPassTexture
	{
		// In some cases (e.g. OCIO color conversion) we want View Extensions to be able to add extra custom post processing after the pass.

		FPostProcessingPassDelegateArray& PassCallbacks = PassSequence.GetAfterPassCallbacks(InPass);

		if (PassCallbacks.Num())
		{
			FScreenPassTextureSlice InSceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, InSceneColor);
			FPostProcessMaterialInputs InOutPostProcessAfterPassInputs = GetPostProcessMaterialInputs(InSceneColorSlice);

			for (int32 AfterPassCallbackIndex = 0; AfterPassCallbackIndex < PassCallbacks.Num(); AfterPassCallbackIndex++)
			{
				InOutPostProcessAfterPassInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, InSceneColor);

				FAfterPassCallbackDelegate& AfterPassCallback = PassCallbacks[AfterPassCallbackIndex];
				PassSequence.AcceptOverrideIfLastPass(InPass, InOutPostProcessAfterPassInputs.OverrideOutput, AfterPassCallbackIndex);
				InSceneColor = AfterPassCallback.Execute(GraphBuilder, View, InOutPostProcessAfterPassInputs);
			}
		}

		return MoveTemp(InSceneColor);
	};

	const auto AddAfterPassForSceneColorSlice = [&](EPass InPass, const FScreenPassTextureSlice& InSceneColor) -> FScreenPassTextureSlice
	{
		FPostProcessingPassDelegateArray& PassCallbacks = PassSequence.GetAfterPassCallbacks(InPass);

		if (PassCallbacks.Num())
		{
			FScreenPassTexture SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, InSceneColor);

			return FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, AddAfterPass(InPass, SceneColor));
		}

		return InSceneColor;
	};

	if (bPostProcessingEnabled)
	{
		const bool bPrimaryView = IStereoRendering::IsAPrimaryView(View);
		const bool bHasViewState = View.ViewState != nullptr;
		const bool bDepthOfFieldEnabled = DiaphragmDOF::IsEnabled(View);
		const bool bVisualizeDepthOfField = bDepthOfFieldEnabled && EngineShowFlags.VisualizeDOF;
		const bool bVisualizeMotionBlur = IsVisualizeMotionBlurEnabled(View);
		const bool bVisualizeTSR = IsVisualizeTSREnabled(View);

		const EAutoExposureMethod AutoExposureMethod = GetAutoExposureMethod(View);
		const EAntiAliasingMethod AntiAliasingMethod = !bVisualizeDepthOfField ? View.AntiAliasingMethod : AAM_None;
		const EDownsampleQuality DownsampleQuality = GetDownsampleQuality(CVarDownsampleQuality);
		const EDownsampleQuality DownsampleChainQuality = GetDownsampleQuality(CVarDownsampleChainQuality);
		const EPixelFormat DownsampleOverrideFormat = PF_FloatRGB;

		// Previous transforms are nonsensical on camera cuts, unless motion vector simulation is enabled (providing FrameN+1 transforms to FrameN+0)
		const bool bMotionBlurValid = FMotionVectorSimulation::IsEnabled() || (!View.bCameraCut && !View.bPrevTransformsReset);

		// Motion blur gets replaced by the visualization pass.
		const bool bMotionBlurEnabled = !bVisualizeMotionBlur && IsMotionBlurEnabled(View) && bMotionBlurValid && !bVisualizeTSR;

		// Skip tonemapping for visualizers which overwrite the HDR scene color.
		const bool bTonemapEnabled = !bVisualizeMotionBlur;
		const bool bTonemapOutputInHDR = View.Family->SceneCaptureSource == SCS_FinalColorHDR || View.Family->SceneCaptureSource == SCS_FinalToneCurveHDR || bOutputInHDR || bViewFamilyOutputInHDR;

		// We don't test for the EyeAdaptation engine show flag here. If disabled, the auto exposure pass is still executes but performs a clamp.
		const bool bEyeAdaptationEnabled =
			// Skip for transient views.
			bHasViewState &&
			View.HasEyeAdaptationViewState() &&
			// Skip for secondary views in a stereo setup.
			bPrimaryView;

		const bool bHistogramEnabled =
			// Force the histogram on when we are visualizing HDR.
			bVisualizeHDR ||
			// Skip if not using histogram eye adaptation.
			(bEyeAdaptationEnabled && AutoExposureMethod == EAutoExposureMethod::AEM_Histogram);

		const bool bLocalExposureEnabled =
			EngineShowFlags.VisualizeLocalExposure ||
			!FMath::IsNearlyEqual(View.FinalPostProcessSettings.LocalExposureHighlightContrastScale, 1.0f) ||
			!FMath::IsNearlyEqual(View.FinalPostProcessSettings.LocalExposureShadowContrastScale, 1.0f) ||
			View.FinalPostProcessSettings.LocalExposureHighlightContrastCurve ||
			View.FinalPostProcessSettings.LocalExposureShadowContrastCurve ||
			!FMath::IsNearlyEqual(View.FinalPostProcessSettings.LocalExposureDetailStrength, 1.0f);

		// Whether separate translucency is composed in TSR.
		bool bComposeSeparateTranslucencyInTSR = PostDOFTranslucencyResources.IsValid() && TAAConfig == EMainTAAPassConfig::TSR;

		const FIntPoint PostTAAViewSize = (View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale && TAAConfig != EMainTAAPassConfig::Disabled) ? View.GetSecondaryViewRectSize() : View.ViewRect.Size();

		const FPostProcessMaterialChain PostProcessMaterialBeforeBloomChain = GetPostProcessMaterialChain(View, BL_SceneColorBeforeBloom);
		const FPostProcessMaterialChain PostProcessMaterialAfterTonemappingChain = GetPostProcessMaterialChain(View, BL_SceneColorAfterTonemapping);

		PassSequence.SetEnabled(EPass::MotionBlur, bVisualizeMotionBlur || bMotionBlurEnabled);
		PassSequence.SetEnabled(EPass::PostProcessMaterialBeforeBloom, PostProcessMaterialBeforeBloomChain.Num() != 0 || bVisualizeGBuffer);
		PassSequence.SetEnabled(EPass::Tonemap, bTonemapEnabled);
		PassSequence.SetEnabled(EPass::FXAA, AntiAliasingMethod == AAM_FXAA);
		PassSequence.SetEnabled(EPass::SMAA, AntiAliasingMethod == AAM_SMAA);
		PassSequence.SetEnabled(EPass::PostProcessMaterialAfterTonemapping, PostProcessMaterialAfterTonemappingChain.Num() != 0 || bVisualizeGBuffer);
		PassSequence.SetEnabled(EPass::VisualizeDepthOfField, bVisualizeDepthOfField);
		PassSequence.SetEnabled(EPass::VisualizeLocalExposure, EngineShowFlags.VisualizeLocalExposure);

		static_assert(static_cast<int32>(EPass::MotionBlur) == 0);
		constexpr int32 FirstAfterPass = static_cast<int32>(ISceneViewExtension::EPostProcessingPass::MotionBlur);
		
		// Scene view extension delegates that precede the override pass sequence are to be called directly.
		TStaticArray<FPostProcessingPassDelegateArray, static_cast<uint32>(FirstAfterPass)> SceneViewExtensionDelegates;

		for (const TSharedRef<ISceneViewExtension>& ViewExtension : View.Family->ViewExtensions)
		{
			for (int32 SceneViewPassId = 0; SceneViewPassId < FirstAfterPass; SceneViewPassId++)
			{
				const ISceneViewExtension::EPostProcessingPass SceneViewPass = static_cast<ISceneViewExtension::EPostProcessingPass>(SceneViewPassId);
				const bool bIsEnabled = (SceneViewPass == ISceneViewExtension::EPostProcessingPass::ReplacingTonemapper) ? PassSequence.IsEnabled(EPass::Tonemap) : true;

				ViewExtension->SubscribeToPostProcessingPass(SceneViewPass, View, SceneViewExtensionDelegates[SceneViewPassId], bIsEnabled);
			}

			for (int32 SceneViewPassId = FirstAfterPass; SceneViewPassId < static_cast<int32>(ISceneViewExtension::EPostProcessingPass::MAX); SceneViewPassId++)
			{
				const ISceneViewExtension::EPostProcessingPass SceneViewPass = static_cast<ISceneViewExtension::EPostProcessingPass>(SceneViewPassId);
				const EPass PostProcessingPass = TranslatePass(SceneViewPass);

				ViewExtension->SubscribeToPostProcessingPass(
					SceneViewPass,
					View,
					PassSequence.GetAfterPassCallbacks(PostProcessingPass),
					PassSequence.IsEnabled(PostProcessingPass));
			}
		}

		PassSequence.Finalize();

		bool bFFTBloomEnabled = IsFFTBloomEnabled(View);
#if WITH_EDITOR
		if (View.ViewportFeedback)
		{
			View.ViewportFeedback->Data().BloomMethod = bFFTBloomEnabled ? EBloomMethod::BM_FFT : EBloomMethod::BM_SOG;
		}
#endif
		const bool bBloomEnabled = !bVisualizeTSR
			&& View.FinalPostProcessSettings.BloomIntensity > 0.0f
			&& (bFFTBloomEnabled ? View.FinalPostProcessSettings.BloomConvolutionIntensity : View.FinalPostProcessSettings.BloomGaussianIntensity) > 0.0f;
		bFFTBloomEnabled = bFFTBloomEnabled && bBloomEnabled;

		const bool bLensFlareEnabled = bBloomEnabled && IsLensFlaresEnabled(View);

		const bool bBasicEyeAdaptationEnabled = bEyeAdaptationEnabled && (AutoExposureMethod == EAutoExposureMethod::AEM_Basic);
		const bool bLocalExposureBlurredLum = bLocalExposureEnabled && View.FinalPostProcessSettings.LocalExposureMethod == ELocalExposureMethod::Bilateral && View.FinalPostProcessSettings.LocalExposureBlurredLuminanceBlend > 0.0f;

		const bool bProcessQuarterResolution = CVarPostProcessingQuarterResolutionDownsample.GetValueOnRenderThread() == 1;
		const bool bProcessEighthResolution = CVarPostProcessingQuarterResolutionDownsample.GetValueOnRenderThread() == 2;
		const bool bMotionBlurNeedsHalfResInput = IsMotionBlurEnabled(View) && DoesMotionBlurNeedsHalfResInput() && !bVisualizeMotionBlur && !bVisualizeTSR;

		const float FFTBloomResolutionFraction = GetFFTBloomResolutionFraction(PostTAAViewSize);

		const bool bProduceSceneColorChain = (
			bBasicEyeAdaptationEnabled ||
			(bBloomEnabled && !bFFTBloomEnabled) ||
			(bLensFlareEnabled && bFFTBloomEnabled) ||
			bLocalExposureBlurredLum);
		extern int32 GSSRHalfResSceneColor;
		const bool bNeedBeforeBloomHalfRes    = (!bProcessQuarterResolution && !bProcessEighthResolution) || (bFFTBloomEnabled && FFTBloomResolutionFraction > 0.25f && FFTBloomResolutionFraction <= 0.5f) || (ReflectionsMethod == EReflectionsMethod::SSR && !View.bStatePrevViewInfoIsReadOnly && GSSRHalfResSceneColor);
		const bool bNeedBeforeBloomQuarterRes = bProcessQuarterResolution || (bFFTBloomEnabled && FFTBloomResolutionFraction > 0.125f && FFTBloomResolutionFraction <= 0.25f);
		const bool bNeedBeforeBloomEighthRes  = bProcessEighthResolution || (bFFTBloomEnabled && FFTBloomResolutionFraction <= 0.125f);


		const FPostProcessMaterialChain MaterialChainSceneColorBeforeDOF = GetPostProcessMaterialChain(View, BL_SceneColorBeforeDOF);
		const FPostProcessMaterialChain MaterialChainSceneColorAfterDOF = GetPostProcessMaterialChain(View, BL_SceneColorAfterDOF);
		const FPostProcessMaterialChain MaterialChainTranslucencyAfterDOF = GetPostProcessMaterialChain(View, BL_TranslucencyAfterDOF);

		// Scene view extension delegates - BeforeDOF
		if (SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::BeforeDOF)].Num())
		{
			FScreenPassTextureSlice SceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
			SceneColor = AddSceneViewExtensionPassChain(GraphBuilder, View, GetPostProcessMaterialInputs(SceneColorSlice),
				SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::BeforeDOF)]);
		}

		auto ApplyPostProcessMaterials = [&GraphBuilder, &View, ViewIndex, bOutputInHDR, &VisualizeBufferContext, GetPostProcessMaterialInputs](
			EBlendableLocation Location,
			FPostProcessMaterialInputs Inputs,
			const FPostProcessMaterialChain& MaterialChain,
			const FScreenPassTexture& InputTexture,
			EPostProcessMaterialInput MaterialInput = EPostProcessMaterialInput::SceneColor
		) -> FScreenPassTexture
		{
			AddVisualizeBufferMaterialPass(GraphBuilder, View, Location, Inputs, bOutputInHDR, VisualizeBufferContext);

			if (MaterialChain.Num())
			{
				return AddPostProcessMaterialChain(GraphBuilder, View, ViewIndex, Inputs, MaterialChain, MaterialInput);
			}
			else
			{
				FScreenPassTextureSlice CurrentInputSlice = Inputs.GetInput(MaterialInput);
				if (Inputs.OverrideOutput.IsValid())
				{
					AddDrawTexturePass(GraphBuilder, View, CurrentInputSlice, Inputs.OverrideOutput);
					return Inputs.OverrideOutput;
				}
				else
				{
					return InputTexture;
				}
			}
		};

		auto ApplyPostProcessMaterialsToSlice = [&GraphBuilder, &View, ViewIndex, bOutputInHDR, &VisualizeBufferContext, GetPostProcessMaterialInputs](
			EBlendableLocation Location,
			FPostProcessMaterialInputs Inputs,
			const FPostProcessMaterialChain& MaterialChain,
			EPostProcessMaterialInput MaterialInput = EPostProcessMaterialInput::SceneColor
		) -> FScreenPassTextureSlice
		{
			AddVisualizeBufferMaterialPass(GraphBuilder, View, Location, Inputs, bOutputInHDR, VisualizeBufferContext);

			if (MaterialChain.Num())
			{
				FScreenPassTexture PassOutput = AddPostProcessMaterialChain(GraphBuilder, View, ViewIndex, Inputs, MaterialChain, MaterialInput);
				return FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, PassOutput);
			}
			else
			{
				FScreenPassTextureSlice CurrentInput = Inputs.GetInput(MaterialInput);
				if (Inputs.OverrideOutput.IsValid())
				{
					AddDrawTexturePass(GraphBuilder, View, CurrentInput, Inputs.OverrideOutput);
					return FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, Inputs.OverrideOutput);
				}
				else
				{
					return CurrentInput;
				}
			}
		};

		// Post Process Material Chain - BL_SceneColorBeforeDOF
		{
			FScreenPassTextureSlice SceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
			SceneColor = ApplyPostProcessMaterials(BL_SceneColorBeforeDOF, GetPostProcessMaterialInputs(SceneColorSlice), MaterialChainSceneColorBeforeDOF, SceneColor);
		}

		// Diaphragm Depth of Field
		bool bSceneColorHasPostDOFTranslucency = false;
		{
			FRDGTextureRef InputSceneColorTexture = SceneColor.Texture;

			if (bDepthOfFieldEnabled)
			{
				FTranslucencyPassResources DummyTranslucency;

				bool bComposeTranslucency = PostDOFTranslucencyResources.IsValid() && !bComposeSeparateTranslucencyInTSR && MaterialChainTranslucencyAfterDOF.Num() == 0;

				if (DiaphragmDOF::AddPasses(
					GraphBuilder,
					SceneTextureParameters,
					View,
					InputSceneColorTexture,
					bComposeTranslucency ? PostDOFTranslucencyResources : DummyTranslucency,
					SceneColor.Texture,
					Inputs.OriginalSceneDepthForStencil))
				{
					bSceneColorHasPostDOFTranslucency = bComposeTranslucency;
				}
			}

			if (GetHairStrandsComposition() == EHairStrandsCompositionType::AfterSeparateTranslucent)
			{
				RenderHairComposition(GraphBuilder, View, SceneColor.Texture, SceneDepth.Texture, Velocity.Texture);
			}

			if (GetTranslucentHoldoutComposition() == ETranslucentHoldoutCompositionType::AfterDOF)
			{
				FTranslucencyPassResources HoldoutTranslucencyResources = Inputs.TranslucencyViewResourcesMap.Get(ETranslucencyPass::TPT_TranslucencyHoldout);
				if (HoldoutTranslucencyResources.IsValid())
				{
					AddTranslucentHoldoutWithCoverageToSceneColorAlphaPass(GraphBuilder, View, SceneColor.Texture, SceneDepth.Texture, HoldoutTranslucencyResources);
				}
			}
		}

		// Allows for the scene color to be the slice of an array between temporal upscaler and tonemaper.
		FScreenPassTextureSlice SceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
		SceneColor = FScreenPassTexture();

		// Scene view extension delegates - AfterDOF
		if (SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::AfterDOF)].Num())
		{
			FScreenPassTexture PassOutput = AddSceneViewExtensionPassChain(GraphBuilder, View, GetPostProcessMaterialInputs(SceneColorSlice),
				SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::AfterDOF)]);
			SceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, PassOutput);
		}

		// Post Process Material Chain - BL_SceneColorAfterDOF
		{
			SceneColorSlice = ApplyPostProcessMaterialsToSlice(BL_SceneColorAfterDOF, GetPostProcessMaterialInputs(SceneColorSlice), MaterialChainSceneColorAfterDOF);
		}

		// Post Process Material Chain - BL_TranslucencyAfterDOF
		if (bSceneColorHasPostDOFTranslucency)
		{
			ensure(MaterialChainTranslucencyAfterDOF.Num() == 0);
			ensure(!bComposeSeparateTranslucencyInTSR);
		}
		else if (PostDOFTranslucencyResources.IsValid())
		{
			if (SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::TranslucencyAfterDOF)].Num())
			{
				FScreenPassTexture PostDOFTranslucency = AddSceneViewExtensionPassChain(
					GraphBuilder, View,
					GetPostProcessMaterialInputs(SceneColorSlice),
					SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::TranslucencyAfterDOF)],
					EPostProcessMaterialInput::SeparateTranslucency);

				PostDOFTranslucencyResources.ColorTexture = PostDOFTranslucency.Texture;
				ensure(PostDOFTranslucencyResources.ViewRect == PostDOFTranslucency.ViewRect);
			}

			if (MaterialChainTranslucencyAfterDOF.Num())
			{
				FScreenPassTexture PostDOFTranslucency {};
				PostDOFTranslucency = ApplyPostProcessMaterials(
					BL_TranslucencyAfterDOF,
					GetPostProcessMaterialInputs(SceneColorSlice), 
					MaterialChainTranslucencyAfterDOF, 
					PostDOFTranslucency,
					EPostProcessMaterialInput::SeparateTranslucency);
				ensure(PostDOFTranslucency.IsValid());

				PostDOFTranslucencyResources.ColorTexture = PostDOFTranslucency.Texture;
				ensure(PostDOFTranslucencyResources.ViewRect == PostDOFTranslucency.ViewRect);
			}

			// DOF passes were not added, therefore need to compose Separate translucency manually.
			if (!bSceneColorHasPostDOFTranslucency)
			{
				FTranslucencyComposition TranslucencyComposition;
				TranslucencyComposition.Operation = FTranslucencyComposition::EOperation::ComposeToNewSceneColor;
				TranslucencyComposition.bApplyModulateOnly = bComposeSeparateTranslucencyInTSR;
				TranslucencyComposition.SceneColor = SceneColorSlice;
				TranslucencyComposition.SceneDepth = SceneDepth;
				TranslucencyComposition.OutputViewport = FScreenPassTextureViewport(SceneColorSlice);
				TranslucencyComposition.OutputPixelFormat = SceneColorFormat;

				SceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(
					GraphBuilder, 
					TranslucencyComposition.AddPass(GraphBuilder, View, PostDOFTranslucencyResources));

				bSceneColorHasPostDOFTranslucency = !TranslucencyComposition.bApplyModulateOnly;
			}
		}
		else
		{
			bSceneColorHasPostDOFTranslucency = true;
		}

		ensure(bSceneColorHasPostDOFTranslucency != bComposeSeparateTranslucencyInTSR);

		// For async compute, schedule AddCombineLUTPass before TSR, which might be waiting on Single Layer Water.
		const FPostProcessingPassDelegateArray& ReplacingTonemapperDelegates = SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::ReplacingTonemapper)];
		const FPostProcessMaterialChain ReplacingTonemapperMaterialChain = GetPostProcessMaterialChain(View, BL_ReplacingTonemapper);
		FRDGTextureRef ColorGradingTexture = nullptr;
		if (PassSequence.IsEnabled(EPass::Tonemap) && ReplacingTonemapperDelegates.IsEmpty() && ReplacingTonemapperMaterialChain.IsEmpty())
		{
			if (bPrimaryView)
			{
				ColorGradingTexture = AddCombineLUTPass(GraphBuilder, View);
			}
			// We can re-use the color grading texture from the primary view.
			else if (View.GetTonemappingLUT())
			{
				ColorGradingTexture = TryRegisterExternalTexture(GraphBuilder, View.GetTonemappingLUT());
			}
			else
			{
				const FViewInfo* PrimaryView = static_cast<const FViewInfo*>(View.Family->Views[0]);
				ColorGradingTexture = TryRegisterExternalTexture(GraphBuilder, PrimaryView->GetTonemappingLUT());
			}
		}

		FScreenPassTextureSlice HalfResSceneColor;
		FScreenPassTextureSlice QuarterResSceneColor;
		FScreenPassTextureSlice EighthResSceneColor;
		FVelocityFlattenTextures VelocityFlattenTextures;
		if (TAAConfig != EMainTAAPassConfig::Disabled)
		{
			FDefaultTemporalUpscaler::FInputs UpscalerPassInputs;
			UpscalerPassInputs.SceneColor = FScreenPassTexture(SceneColorSlice);
			UpscalerPassInputs.SceneDepth = FScreenPassTexture(SceneDepth.Texture, View.ViewRect);
			UpscalerPassInputs.SceneVelocity = FScreenPassTexture(Velocity.Texture, View.ViewRect);
			UpscalerPassInputs.bGenerateOutputMip1 = bMotionBlurNeedsHalfResInput;
			if (PassSequence.IsEnabled(EPass::MotionBlur))
			{
				if (bVisualizeMotionBlur)
				{
					// NOP
				}
				else
				{
					UpscalerPassInputs.bGenerateVelocityFlattenTextures = FVelocityFlattenTextures::AllowExternal(View) && !bVisualizeMotionBlur && !bApplyLensDistortionInTSR;
				}
			}
			else if (PostProcessMaterialBeforeBloomChain.Num() > 0)
			{
				// NOP
			}
			else
			{
				UpscalerPassInputs.bGenerateSceneColorHalfRes =
					bNeedBeforeBloomHalfRes &&
					DownsampleQuality == EDownsampleQuality::Low;

				UpscalerPassInputs.bGenerateSceneColorQuarterRes =
					bNeedBeforeBloomQuarterRes &&
					DownsampleQuality == EDownsampleQuality::Low;

				UpscalerPassInputs.bGenerateSceneColorEighthRes =
					bNeedBeforeBloomEighthRes &&
					DownsampleQuality == EDownsampleQuality::Low;
			}
			UpscalerPassInputs.bAllowFullResSlice = PassSequence.IsEnabled(EPass::MotionBlur) || PassSequence.IsEnabled(EPass::Tonemap);
			UpscalerPassInputs.DownsampleOverrideFormat = DownsampleOverrideFormat;
			UpscalerPassInputs.PostDOFTranslucencyResources = PostDOFTranslucencyResources;
			UpscalerPassInputs.FlickeringInputTexture = TSRFlickeringInput;
			if (bApplyLensDistortionInTSR)
			{
				UpscalerPassInputs.LensDistortionLUT = View.LensDistortionLUT;
			}
			check(UpscalerPassInputs.SceneColor.ViewRect == View.ViewRect);

			FDefaultTemporalUpscaler::FOutputs Outputs;
			if (TAAConfig == EMainTAAPassConfig::TSR)
			{
				Outputs = AddMainTemporalSuperResolutionPasses(
					GraphBuilder,
					View,
					UpscalerPassInputs);
			}
			else if (TAAConfig == EMainTAAPassConfig::TAA)
			{
				Outputs = AddGen4MainTemporalAAPasses(
					GraphBuilder,
					View,
					UpscalerPassInputs);
			}
			else if (TAAConfig == EMainTAAPassConfig::ThirdParty)
			{
				Outputs = AddThirdPartyTemporalUpscalerPasses(
					GraphBuilder,
					View,
					UpscalerPassInputs);
			}
			else
			{
				unimplemented();
			}

			SceneColorSlice = Outputs.FullRes;
			HalfResSceneColor = Outputs.HalfRes;
			QuarterResSceneColor = Outputs.QuarterRes;
			EighthResSceneColor = Outputs.EighthRes;
			VelocityFlattenTextures = Outputs.VelocityFlattenTextures;

			if (PassSequence.IsEnabled(EPass::VisualizeTemporalUpscaler))
			{
				VisualizeTemporalUpscalerInputs.TAAConfig = TAAConfig;
				VisualizeTemporalUpscalerInputs.UpscalerUsed = View.Family->GetTemporalUpscalerInterface();
				VisualizeTemporalUpscalerInputs.Inputs = UpscalerPassInputs;
				VisualizeTemporalUpscalerInputs.Outputs = Outputs;
			}
		}
		else if (ReflectionsMethod == EReflectionsMethod::SSR)
		{
			// If we need SSR, and TAA is enabled, then AddTemporalAAPass() has already handled the scene history.
			// If we need SSR, and TAA is not enable, then we just need to extract the history.
			if (!View.bStatePrevViewInfoIsReadOnly)
			{
				check(View.ViewState);
				FTemporalAAHistory& OutputHistory = View.ViewState->PrevFrameViewInfo.TemporalAAHistory;
				GraphBuilder.QueueTextureExtraction(SceneColorSlice.TextureSRV->Desc.Texture, &OutputHistory.RT[0]);

				// For SSR, we still fill up the rest of the OutputHistory data using shared math from FTAAPassParameters.
				FTAAPassParameters TAAInputs(View);
				TAAInputs.SceneColorInput = SceneColorSlice.TextureSRV->Desc.Texture;
				TAAInputs.SetupViewRect(View);
				OutputHistory.ViewportRect = TAAInputs.OutputViewRect;
				OutputHistory.ReferenceBufferSize = TAAInputs.GetOutputExtent() * TAAInputs.ResolutionDivisor;
			}
		}

		ensure(SceneColorSlice.ViewRect.Size() == PostTAAViewSize);

		// SVE/Post Process Material Chain - SSR Input
		if (View.ViewState && !View.bStatePrevViewInfoIsReadOnly)
		{
			FScreenPassTexture PassOutput;
			FPostProcessMaterialInputs PassInputs;
			const FPostProcessMaterialChain MaterialChain = GetPostProcessMaterialChain(View, BL_SSRInput);

			if (SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::SSRInput)].Num())
			{
				PassInputs = GetPostProcessMaterialInputs(SceneColorSlice);
				PassOutput = AddSceneViewExtensionPassChain(GraphBuilder, View, PassInputs,
					SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::SSRInput)]);
			}

			{
				FScreenPassTextureSlice InputSceneColorSlice = PassOutput.IsValid() ? FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, PassOutput) : SceneColorSlice;
				PassOutput = ApplyPostProcessMaterials(BL_SSRInput, GetPostProcessMaterialInputs(InputSceneColorSlice), MaterialChain, PassOutput);
			}

			if (PassOutput.IsValid())
			{
				// Save off SSR post process output for the next frame.
				GraphBuilder.QueueTextureExtraction(PassOutput.Texture, &View.ViewState->PrevFrameViewInfo.CustomSSRInput.RT[0]);

				View.ViewState->PrevFrameViewInfo.CustomSSRInput.ViewportRect = PassOutput.ViewRect;
				View.ViewState->PrevFrameViewInfo.CustomSSRInput.ReferenceBufferSize = PassOutput.Texture->Desc.Extent;
			}
		}

		if (PassSequence.IsEnabled(EPass::MotionBlur))
		{
			FMotionBlurInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::MotionBlur, PassInputs.OverrideOutput);
			PassInputs.bOutputHalfRes = PostProcessMaterialBeforeBloomChain.Num() == 0 && bNeedBeforeBloomHalfRes && DownsampleQuality == EDownsampleQuality::Low;
			PassInputs.bOutputQuarterRes = (bNeedBeforeBloomQuarterRes || bNeedBeforeBloomEighthRes) && DownsampleQuality == EDownsampleQuality::Low;
			PassInputs.SceneColor = SceneColorSlice;
			PassInputs.SceneDepth = SceneDepth;
			PassInputs.SceneVelocity = Velocity;
			PassInputs.PostMotionBlurTranslucency = PostMotionBlurTranslucencyResources;
			PassInputs.Quality = GetMotionBlurQuality();
			PassInputs.Filter = GetMotionBlurFilter();
			PassInputs.VelocityFlattenTextures = VelocityFlattenTextures;
			if (bApplyLensDistortionInTSR)
			{
				PassInputs.LensDistortionLUT = View.LensDistortionLUT;
			}

			// Motion blur visualization replaces motion blur when enabled.
			if (bVisualizeMotionBlur)
			{
				SceneColorSlice = AddVisualizeMotionBlurPass(GraphBuilder, View, PassInputs);
			}
			else
			{
				FMotionBlurOutputs PassOutputs = AddMotionBlurPass(GraphBuilder, View, PassInputs);
				SceneColorSlice = PassOutputs.FullRes;
				HalfResSceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, PassOutputs.HalfRes);
				QuarterResSceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, PassOutputs.QuarterRes);
			}
		}
		else if (PostMotionBlurTranslucencyResources.IsValid())
		{
			// Compose Post-MotionBlur translucency in a new scene color to ensure it's not writing out in TAA's output that is also the history.
			FTranslucencyComposition TranslucencyComposition;
			TranslucencyComposition.Operation = FTranslucencyComposition::EOperation::ComposeToNewSceneColor;
			TranslucencyComposition.SceneColor = SceneColorSlice;
			TranslucencyComposition.OutputViewport = FScreenPassTextureViewport(SceneColorSlice);
			TranslucencyComposition.OutputPixelFormat = SceneColorFormat;
			if (bApplyLensDistortionInTSR)
			{
				TranslucencyComposition.LensDistortionLUT = &View.LensDistortionLUT;
			}

			SceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, TranslucencyComposition.AddPass(
				GraphBuilder, View, PostMotionBlurTranslucencyResources));
		}

		{
			FScreenPassTextureSlice NewSceneColorSlice = AddAfterPassForSceneColorSlice(EPass::MotionBlur, SceneColorSlice);

			// Force invalidation if MotionBlur pass is not enabled and Post motion blur translucency overrides the scene color 
			// so that the translucency affects bloom.
			const bool InvalidationRequired = !PassSequence.IsEnabled(EPass::MotionBlur) && PostMotionBlurTranslucencyResources.IsValid();

			// Invalidate half and quarter res.
			if (NewSceneColorSlice != SceneColorSlice || InvalidationRequired)
			{
				HalfResSceneColor = FScreenPassTextureSlice();
				QuarterResSceneColor = FScreenPassTextureSlice();
				EighthResSceneColor = FScreenPassTextureSlice();
			}

			SceneColorSlice = NewSceneColorSlice;
		}

		// Post Process Material Chain - Before Bloom
		if (PassSequence.IsEnabled(EPass::PostProcessMaterialBeforeBloom))
		{
			FPostProcessMaterialInputs PostProcessMaterialInputs = GetPostProcessMaterialInputs(SceneColorSlice);
			PassSequence.AcceptOverrideIfLastPass(EPass::PostProcessMaterialBeforeBloom, PostProcessMaterialInputs.OverrideOutput);
			SceneColorSlice = ApplyPostProcessMaterialsToSlice(BL_SceneColorBeforeBloom, PostProcessMaterialInputs, PostProcessMaterialBeforeBloomChain);
		}

		// Generate before bloom lower res scene color if they have not been generated.
		{
			if ((bNeedBeforeBloomHalfRes && !HalfResSceneColor.IsValid()) ||
				(bNeedBeforeBloomQuarterRes && !QuarterResSceneColor.IsValid() && !HalfResSceneColor.IsValid()) ||
				(bNeedBeforeBloomEighthRes && !EighthResSceneColor.IsValid() && !QuarterResSceneColor.IsValid() && !HalfResSceneColor.IsValid()))
			{
				FDownsamplePassInputs PassInputs;
				PassInputs.Name = TEXT("PostProcessing.SceneColor.HalfRes");
				PassInputs.SceneColor = SceneColorSlice;
				PassInputs.Quality = DownsampleQuality;
				PassInputs.FormatOverride = DownsampleOverrideFormat;

				HalfResSceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, AddDownsamplePass(GraphBuilder, View, PassInputs));
			}

			if ((bNeedBeforeBloomQuarterRes && !QuarterResSceneColor.IsValid()) ||
				(bNeedBeforeBloomEighthRes && !EighthResSceneColor.IsValid() && !QuarterResSceneColor.IsValid()))
			{
				FDownsamplePassInputs PassInputs;
				PassInputs.Name = TEXT("PostProcessing.SceneColor.QuarterRes");
				PassInputs.SceneColor = HalfResSceneColor;
				PassInputs.Quality = DownsampleQuality;

				QuarterResSceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, AddDownsamplePass(GraphBuilder, View, PassInputs));
			}

			if (bNeedBeforeBloomEighthRes && !EighthResSceneColor.IsValid())
			{
				FDownsamplePassInputs PassInputs;
				PassInputs.Name = TEXT("PostProcessing.SceneColor.EighthRes");
				PassInputs.SceneColor = QuarterResSceneColor;
				PassInputs.Quality = DownsampleQuality;

				EighthResSceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, AddDownsamplePass(GraphBuilder, View, PassInputs));
			}
		}



		// Store half res scene color in the history
		if (ReflectionsMethod == EReflectionsMethod::SSR && !View.bStatePrevViewInfoIsReadOnly && GSSRHalfResSceneColor && HalfResSceneColor.IsValid())
		{
			check(View.ViewState);
			GraphBuilder.QueueTextureExtraction(HalfResSceneColor.TextureSRV->Desc.Texture, &View.ViewState->PrevFrameViewInfo.HalfResTemporalAAHistory);
		}

		{
			FScreenPassTextureSlice LocalExposureSceneColor = bProcessEighthResolution ? EighthResSceneColor : (bProcessQuarterResolution ? QuarterResSceneColor : HalfResSceneColor);

			if (bLocalExposureEnabled && View.FinalPostProcessSettings.LocalExposureMethod == ELocalExposureMethod::Bilateral)
			{
				LocalExposureBilateralGridTexture = AddLocalExposurePass(
					GraphBuilder, View,
					EyeAdaptationParameters,
					LocalExposureSceneColor);
			}

			LocalExposureParameters = GetLocalExposureParameters(View, LocalExposureSceneColor.ViewRect.Size(), EyeAdaptationParameters);
		}

		if (bHistogramEnabled)
		{
			FScreenPassTextureSlice HistogramSceneColor = bProcessEighthResolution ? EighthResSceneColor : (bProcessQuarterResolution ? QuarterResSceneColor : HalfResSceneColor);

			if (IsAutoExposureUsingIlluminanceEnabled(View))
			{
				if (ExposureIlluminance.IsValid())
				{
					HistogramSceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, ExposureIlluminance);
				}
				else
				{
					HistogramSceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, OriginalSceneColor);
				}
			}

			// Optionally generate eye adaptation from the entire set of view rects.  Rects must combine to form a contiguous rect!
			if (View.bEyeAdaptationAllViewPixels && View.Family->Views.Num() > 1)
			{
				FIntRect EyeAdaptationRect = View.Family->Views[0]->UnconstrainedViewRect;
				for (int32 OtherViewIndex = 1; OtherViewIndex < View.Family->Views.Num(); OtherViewIndex++)
				{
					EyeAdaptationRect.Union(View.Family->Views[OtherViewIndex]->UnconstrainedViewRect);
				}

				HistogramSceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, FScreenPassTexture(OriginalSceneColor.Texture, EyeAdaptationRect));
			}

			HistogramTexture = AddHistogramPass(
				GraphBuilder, View,
				EyeAdaptationParameters,
				HistogramSceneColor,
				SceneTextureParameters,
				LastEyeAdaptationBuffer);
		}

		FTextureDownsampleChain SceneDownsampleChain;
		if (bProduceSceneColorChain)
		{
			const bool bLogLumaInAlpha = bBasicEyeAdaptationEnabled;
			SceneDownsampleChain.Init(
				GraphBuilder, View,
				EyeAdaptationParameters,
				bProcessEighthResolution ? EighthResSceneColor : (bProcessQuarterResolution ? QuarterResSceneColor : HalfResSceneColor),
				DownsampleChainQuality,
				6,
				bLogLumaInAlpha,
				TEXT("Scene"),
				bProcessEighthResolution ? 3 : (bProcessQuarterResolution ? 2 : 1));
		}

		if (bLocalExposureBlurredLum)
		{
			const uint32 BlurredLumMip = bProcessEighthResolution ? 2 : (bProcessQuarterResolution ? 3 : 4);
			LocalExposureBlurredLogLumTexture = AddLocalExposureBlurredLogLuminancePass(
				GraphBuilder, View,
				EyeAdaptationParameters, SceneDownsampleChain.GetTexture(BlurredLumMip));
		}

		if (bBasicEyeAdaptationEnabled)
		{
			// Use the alpha channel in the last downsample (smallest) to compute eye adaptations values.
			EyeAdaptationBuffer = AddBasicEyeAdaptationPass(
				GraphBuilder, View,
				EyeAdaptationParameters,
				LocalExposureParameters,
				SceneDownsampleChain.GetLastTexture(),
				LastEyeAdaptationBuffer,
				bLocalExposureEnabled && View.FinalPostProcessSettings.LocalExposureMethod == ELocalExposureMethod::Bilateral);
		}
		// Add histogram eye adaptation pass even if no histogram exists to support the manual clamping mode.
		else if (bEyeAdaptationEnabled)
		{
			EyeAdaptationBuffer = AddHistogramEyeAdaptationPass(
				GraphBuilder, View,
				EyeAdaptationParameters,
				LocalExposureParameters,
				HistogramTexture,
				bLocalExposureEnabled && View.FinalPostProcessSettings.LocalExposureMethod == ELocalExposureMethod::Bilateral);
		}

		if (bLocalExposureEnabled && View.FinalPostProcessSettings.LocalExposureMethod == ELocalExposureMethod::Fusion)
		{
			ExposureFusionData = AddLocalExposureFusionPass(
				GraphBuilder, View,
				EyeAdaptationParameters,
				EyeAdaptationBuffer,
				LocalExposureParameters,
				//bProcessQuarterResolution ? QuarterResSceneColor : HalfResSceneColor);
				SceneColorSlice);
		}

		FScreenPassTexture Bloom;
		FRDGBufferRef SceneColorApplyParameters = nullptr;
		if (bBloomEnabled)
		{
			const FTextureDownsampleChain* LensFlareSceneDownsampleChain;

			FTextureDownsampleChain BloomDownsampleChain;

			if (bFFTBloomEnabled)
			{
				LensFlareSceneDownsampleChain = &SceneDownsampleChain;

				float InputResolutionFraction;
				FScreenPassTextureSlice InputSceneColor;

				if (FFTBloomResolutionFraction <= 0.125f)
				{
					InputSceneColor = EighthResSceneColor;
					InputResolutionFraction = 0.125f;
				}
				else if (FFTBloomResolutionFraction <= 0.25f)
				{
					InputSceneColor = QuarterResSceneColor;
					InputResolutionFraction = 0.25f;
				}
				else if (FFTBloomResolutionFraction <= 0.5f)
				{
					InputSceneColor = HalfResSceneColor;
					InputResolutionFraction = 0.5f;
				}
				else
				{
					InputSceneColor = SceneColorSlice;
					InputResolutionFraction = 1.0f;
				}

				FFFTBloomOutput Outputs = AddFFTBloomPass(
					GraphBuilder, 
					View,
					InputSceneColor,
					InputResolutionFraction,
					EyeAdaptationParameters,
					EyeAdaptationBuffer,
					LocalExposureParameters,
					CVarBloomApplyLocalExposure.GetValueOnRenderThread() ? LocalExposureBilateralGridTexture : nullptr,
					LocalExposureBlurredLogLumTexture);

				Bloom = Outputs.BloomTexture;
				SceneColorApplyParameters = Outputs.SceneColorApplyParameters;
			}
			else
			{
				const bool bApplyLocalExposureToBloom = 
					CVarBloomApplyLocalExposure.GetValueOnRenderThread()
					&& View.FinalPostProcessSettings.LocalExposureMethod == ELocalExposureMethod::Bilateral
					&& LocalExposureBilateralGridTexture != nullptr;

				const bool bBloomSetupRequiredEnabled = View.FinalPostProcessSettings.BloomThreshold > -1.0f || bApplyLocalExposureToBloom;

				// Reuse the main scene downsample chain if setup isn't required for gaussian bloom.
				if (SceneDownsampleChain.IsInitialized() && !bBloomSetupRequiredEnabled)
				{
					LensFlareSceneDownsampleChain = &SceneDownsampleChain;
				}
				else
				{
					FScreenPassTextureSlice DownsampleInput = bProcessEighthResolution ? EighthResSceneColor : (bProcessQuarterResolution ? QuarterResSceneColor : HalfResSceneColor);

					if (bBloomSetupRequiredEnabled)
					{
						const float BloomThreshold = View.FinalPostProcessSettings.BloomThreshold;

						FBloomSetupInputs SetupPassInputs;
						SetupPassInputs.SceneColor = DownsampleInput;
						SetupPassInputs.EyeAdaptationBuffer = EyeAdaptationBuffer;
						SetupPassInputs.EyeAdaptationParameters = &EyeAdaptationParameters;
						SetupPassInputs.Threshold = BloomThreshold;

						if (bApplyLocalExposureToBloom)
						{
							SetupPassInputs.LocalExposureParameters = &LocalExposureParameters;
							SetupPassInputs.LocalExposureTexture = LocalExposureBilateralGridTexture;
							SetupPassInputs.BlurredLogLuminanceTexture = LocalExposureBlurredLogLumTexture;
						}

						DownsampleInput = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, AddBloomSetupPass(GraphBuilder, View, SetupPassInputs));
					}

					const bool bLogLumaInAlpha = false;
					BloomDownsampleChain.Init(GraphBuilder, View, EyeAdaptationParameters, DownsampleInput, DownsampleChainQuality, (uint32)EBloomQuality::MAX, bLogLumaInAlpha, nullptr, bProcessEighthResolution ? 3 : (bProcessQuarterResolution ? 2 : 1));

					LensFlareSceneDownsampleChain = &BloomDownsampleChain;
				}

				Bloom = AddGaussianBloomPasses(GraphBuilder, View, LensFlareSceneDownsampleChain);
			}

			if (bLensFlareEnabled)
			{
				const ELensFlareQuality LensFlareQuality = GetLensFlareQuality();
				const uint32 LensFlareDownsampleStageIndex = static_cast<uint32>(ELensFlareQuality::MAX) - static_cast<uint32>(LensFlareQuality) - 1;
				Bloom = AddLensFlaresPass(GraphBuilder, View, Bloom,
					LensFlareSceneDownsampleChain->GetTexture(LensFlareDownsampleStageIndex),
					LensFlareSceneDownsampleChain->GetFirstTexture());
			}
		}

		SceneColorBeforeTonemapSlice = SceneColorSlice;

		if (PassSequence.IsEnabled(EPass::Tonemap))
		{
			// GPU Skin Cache for next frame can overlap with the tone-mapping pass.
			if (FGPUSkinCache* GPUSkinCache = Scene->GetGPUSkinCache(); GPUSkinCache && View.IsLastInFamily())
			{
				GPUSkinCache->AddAsyncComputeSignal(GraphBuilder);
			}

			auto GetReplaceTonemapperInputs = [&]() -> FPostProcessMaterialInputs
				{
					FPostProcessMaterialInputs PassInputs;
					PassSequence.AcceptOverrideIfLastPass(EPass::Tonemap, PassInputs.OverrideOutput);
					PassInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorSlice));
					PassInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::CombinedBloom, Bloom);
					PassInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);
					PassInputs.CustomDepthTexture = CustomDepth.Texture;
					PassInputs.bManualStencilTest = Inputs.bSeparateCustomStencil;

					return PassInputs;
				};

			if(ReplacingTonemapperDelegates.Num())
			{
				const FAfterPassCallbackDelegate& HighestPriorityDelegate = ReplacingTonemapperDelegates[0];

				SceneColor = HighestPriorityDelegate.Execute(GraphBuilder, View, GetReplaceTonemapperInputs());
			}
			else if (ReplacingTonemapperMaterialChain.Num())
			{
				const UMaterialInterface* HighestPriorityMaterial = ReplacingTonemapperMaterialChain[0];

				SceneColor = AddPostProcessMaterialPass(GraphBuilder, View, GetReplaceTonemapperInputs(), HighestPriorityMaterial);
			}
			else
			{
				FTonemapInputs PassInputs;
				PassSequence.AcceptOverrideIfLastPass(EPass::Tonemap, PassInputs.OverrideOutput);
				PassInputs.SceneColor = SceneColorSlice;
				PassInputs.Bloom = Bloom;
				PassInputs.SceneColorApplyParamaters = SceneColorApplyParameters;
				PassInputs.LocalExposureBilateralGridTexture = LocalExposureBilateralGridTexture;
				PassInputs.BlurredLogLuminanceTexture = LocalExposureBlurredLogLumTexture;
				PassInputs.ExposureFusion = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, ExposureFusionData.Result);
				PassInputs.LocalExposureParameters = &LocalExposureParameters;
				PassInputs.EyeAdaptationParameters = &EyeAdaptationParameters;
				PassInputs.EyeAdaptationBuffer = EyeAdaptationBuffer;
				PassInputs.ColorGradingTexture = ColorGradingTexture;
				PassInputs.bWriteAlphaChannel = AntiAliasingMethod == AAM_FXAA || AntiAliasingMethod == AAM_SMAA || bProcessSceneColorAlpha;
				PassInputs.bOutputInHDR = bTonemapOutputInHDR;

				SceneColor = AddTonemapPass(GraphBuilder, View, PassInputs);
			}
		}
		else
		{
			SceneColor = FScreenPassTexture(SceneColorSlice);
		}
		
		SceneColor = AddAfterPass(EPass::Tonemap, SceneColor);

		SceneColorAfterTonemap = SceneColor;

		if (PassSequence.IsEnabled(EPass::FXAA))
		{
			FFXAAInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::FXAA, PassInputs.OverrideOutput);
			PassInputs.SceneColor = SceneColor;
			PassInputs.Quality = GetFXAAQuality();
			
			SceneColor = AddFXAAPass(GraphBuilder, View, PassInputs);
		}

		SceneColor = AddAfterPass(EPass::FXAA, SceneColor);

		if (PassSequence.IsEnabled(EPass::SMAA))
		{
			FSMAAInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::SMAA, PassInputs.OverrideOutput);
			PassInputs.SceneColor = SceneColor;
			PassInputs.Quality = GetSMAAQuality();
			PassInputs.SceneColorBeforeTonemap = FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorBeforeTonemapSlice);

			SceneColor = AddSMAAPasses(GraphBuilder, View, PassInputs);
		}

		SceneColor = AddAfterPass(EPass::SMAA, SceneColor);

		// Post Process Material Chain - After Tonemapping
		if (PassSequence.IsEnabled(EPass::PostProcessMaterialAfterTonemapping))
		{
			FScreenPassTextureSlice InSceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
			FPostProcessMaterialInputs PassInputs = GetPostProcessMaterialInputs(InSceneColorSlice);
			PassSequence.AcceptOverrideIfLastPass(EPass::PostProcessMaterialAfterTonemapping, PassInputs.OverrideOutput);
			PassInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::PreTonemapHDRColor, FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorBeforeTonemapSlice));
			PassInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::PostTonemapHDRColor, SceneColorAfterTonemap);
			PassInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);

			SceneColor = ApplyPostProcessMaterials(BL_SceneColorAfterTonemapping, PassInputs, PostProcessMaterialAfterTonemappingChain, SceneColor);
		}

		if (PassSequence.IsEnabled(EPass::VisualizeLumenScene))
		{
			FVisualizeLumenSceneInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeLumenScene, PassInputs.OverrideOutput);
			PassInputs.SceneColor = SceneColor;
			PassInputs.SceneDepth = SceneDepth;
			PassInputs.SceneTextures.SceneTextures = Inputs.SceneTextures;

			SceneColor = AddVisualizeLumenScenePass(GraphBuilder, View, DiffuseIndirectMethod, ReflectionsMethod, PassInputs, LumenFrameTemporaries);
		}

		if (PassSequence.IsEnabled(EPass::VisualizeMegaLights))
		{
			FScreenPassRenderTarget OverrideOutput;
			PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeMegaLights, OverrideOutput);
			SceneColor = MegaLights::AddVisualizePass(GraphBuilder, View, ViewIndex, SceneColor, SceneDepth, OverrideOutput, MegaLightsFrameTemporaries);
		}

		if (PassSequence.IsEnabled(EPass::VisualizeDepthOfField))
		{
			FVisualizeDOFInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeDepthOfField, PassInputs.OverrideOutput);
			PassInputs.SceneColor = SceneColor;
			PassInputs.SceneDepth = SceneDepth;

			SceneColor = AddVisualizeDOFPass(GraphBuilder, View, PassInputs);
		}

		SceneColor = AddAfterPass(EPass::VisualizeDepthOfField, SceneColor);
	}
	// Minimal PostProcessing - Separate translucency composition and gamma-correction only.
	else
	{
		PassSequence.SetEnabled(EPass::MotionBlur, false);
		PassSequence.SetEnabled(EPass::PostProcessMaterialBeforeBloom, false);
		PassSequence.SetEnabled(EPass::Tonemap, true);
		PassSequence.SetEnabled(EPass::FXAA, false);
		PassSequence.SetEnabled(EPass::SMAA, false);
		PassSequence.SetEnabled(EPass::PostProcessMaterialAfterTonemapping, false);
		PassSequence.SetEnabled(EPass::VisualizeDepthOfField, false);
		PassSequence.SetEnabled(EPass::VisualizeLocalExposure, false);
		PassSequence.Finalize();

		{
			FPostProcessMaterialInputs PostProcessMaterialInputs = GetPostProcessMaterialInputs(FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor));
			AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_SceneColorBeforeDOF, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
			AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_SceneColorAfterDOF, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
		}

		// Compose separate translucency passes
		{
			FTranslucencyComposition TranslucencyComposition;
			TranslucencyComposition.Operation = FTranslucencyComposition::EOperation::ComposeToNewSceneColor;
			TranslucencyComposition.OutputViewport = FScreenPassTextureViewport(SceneColor);
			TranslucencyComposition.OutputPixelFormat = SceneColorFormat;

			if (PostDOFTranslucencyResources.IsValid())
			{
				TranslucencyComposition.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
				SceneColor = TranslucencyComposition.AddPass(
					GraphBuilder, View, PostDOFTranslucencyResources);
			}

			if (PostMotionBlurTranslucencyResources.IsValid())
			{
				TranslucencyComposition.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
				SceneColor = TranslucencyComposition.AddPass(
					GraphBuilder, View, PostMotionBlurTranslucencyResources);
			}
		}

		{
			FPostProcessMaterialInputs PostProcessMaterialInputs = GetPostProcessMaterialInputs(FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor));
			AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_TranslucencyAfterDOF, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
			AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_SSRInput, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
			AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_SceneColorBeforeBloom, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
		}

		SceneColorBeforeTonemapSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);

		if (PassSequence.IsEnabled(EPass::Tonemap))
		{
			FTonemapInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::Tonemap, PassInputs.OverrideOutput);
			PassInputs.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
			PassInputs.EyeAdaptationParameters = &EyeAdaptationParameters;
			PassInputs.EyeAdaptationBuffer = EyeAdaptationBuffer;
			PassInputs.bOutputInHDR = bViewFamilyOutputInHDR;
			PassInputs.bGammaOnly = true;

			SceneColor = AddTonemapPass(GraphBuilder, View, PassInputs);
		}

		SceneColor = AddAfterPass(EPass::Tonemap, SceneColor);

		{
			FPostProcessMaterialInputs PostProcessMaterialInputs = GetPostProcessMaterialInputs(FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor));
			AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_SceneColorAfterTonemapping, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
		}

		SceneColorAfterTonemap = SceneColor;
	}

	if (PassSequence.IsEnabled(EPass::VisualizeStationaryLightOverlap))
	{
		ensureMsgf(View.PrimaryScreenPercentageMethod != EPrimaryScreenPercentageMethod::TemporalUpscale, TEXT("TAAU should be disabled when visualizing stationary light overlap."));

		FVisualizeComplexityInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeStationaryLightOverlap, PassInputs.OverrideOutput);
		PassInputs.SceneColor = OriginalSceneColor;
		PassInputs.Colors = GEngine->StationaryLightOverlapColors;
		PassInputs.ColorSamplingMethod = FVisualizeComplexityInputs::EColorSamplingMethod::Ramp;
		PassInputs.bDrawLegend = true;

		SceneColor = AddVisualizeComplexityPass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeLightCulling))
	{
		ensureMsgf(View.PrimaryScreenPercentageMethod != EPrimaryScreenPercentageMethod::TemporalUpscale, TEXT("TAAU should be disabled when visualizing light culling."));

		// 0.1f comes from the values used in LightAccumulator_GetResult
		const float ComplexityScale = 1.0f / (float)(GEngine->LightComplexityColors.Num() - 1) / 0.1f;

		FVisualizeComplexityInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeLightCulling, PassInputs.OverrideOutput);
		PassInputs.SceneColor = OriginalSceneColor;
		PassInputs.Colors = GEngine->LightComplexityColors;
		PassInputs.ColorSamplingMethod = FVisualizeComplexityInputs::EColorSamplingMethod::Linear;
		PassInputs.ComplexityScale = ComplexityScale;
		PassInputs.bDrawLegend = true;

		SceneColor = AddVisualizeComplexityPass(GraphBuilder, View, PassInputs);
	}

#if DEBUG_POST_PROCESS_VOLUME_ENABLE
	if (PassSequence.IsEnabled(EPass::VisualizePostProcessStack))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizePostProcessStack, OverrideOutput);
		SceneColor = AddFinalPostProcessDebugInfoPasses(GraphBuilder, View, SceneColor, OverrideOutput);
	}
#endif

	if (PassSequence.IsEnabled(EPass::VisualizeSubstrate))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeSubstrate, OverrideOutput);

		FScreenPassTexture DebugColorOutput = Substrate::AddSubstrateDebugPasses(GraphBuilder, View, SceneColor);
		if (OverrideOutput.IsValid())
		{
			AddDrawTexturePass(GraphBuilder, View, DebugColorOutput, OverrideOutput);
			SceneColor = OverrideOutput;
		}
		else
		{
			SceneColor = DebugColorOutput;
		}
	}

	if (PassSequence.IsEnabled(EPass::VisualizeLightGrid))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeLightGrid, OverrideOutput);
		SceneColor = AddVisualizeLightGridPass(GraphBuilder, View, OverrideOutput.IsValid() ? OverrideOutput : SceneColor, SceneDepth);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeMegaLightsComplexity))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeMegaLightsComplexity, OverrideOutput);
		SceneColor = MegaLights::AddVisualizeLightComplexityPostProcessingPass(GraphBuilder, View, ViewIndex, MegaLightsFrameTemporaries, SceneColor, OverrideOutput);
	}

#if WITH_EDITOR
	if (PassSequence.IsEnabled(EPass::VisualizeSkyAtmosphere))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeSkyAtmosphere, OverrideOutput);
		SceneColor = AddSkyAtmosphereDebugPasses(GraphBuilder, Scene, *View.Family, View, SceneColor);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeSkyLightIlluminanceMeter))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeSkyLightIlluminanceMeter, OverrideOutput);
		SceneColor = ProcessAndRenderIlluminanceMeter(GraphBuilder, View, SceneColor);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeLightFunctionAtlas))
	{
		if (Scene->LightFunctionAtlasSceneData.GetLightFunctionAtlas())
		{
			FScreenPassRenderTarget OverrideOutput;
			PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeLightFunctionAtlas, OverrideOutput);
			SceneColor = Scene->LightFunctionAtlasSceneData.GetLightFunctionAtlas()->AddDebugVisualizationPasses(GraphBuilder, View, SceneColor, OverrideOutput);
		}
	}

	if (PassSequence.IsEnabled(EPass::VisualizeLocalFogVolumes))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeLocalFogVolumes, OverrideOutput);
		SceneColor = RenderLocalFogVolumeVisualizationPostProcess(GraphBuilder, View, SceneColor, SceneDepth, OverrideOutput);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeLevelInstance))
	{
		FVisualizeLevelInstanceInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeLevelInstance, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;
		PassInputs.SceneTextures.SceneTextures = Inputs.SceneTextures;

		SceneColor = AddVisualizeLevelInstancePass(GraphBuilder, View, SceneUniformBuffer, PassInputs, NaniteRasterResults);
	}
#endif //WITH_EDITOR
	
	if (PassSequence.IsEnabled(EPass::VisualizeVirtualShadowMaps_PreEditorPrimitives))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeVirtualShadowMaps_PreEditorPrimitives, OverrideOutput);
		SceneColor = VirtualShadowMapArray->AddVisualizePass(GraphBuilder, View, ViewIndex, EVSMVisualizationPostPass::PreEditorPrimitives, SceneUniformBuffer, SceneColor, SceneDepth, OverrideOutput);
	}
	
#if WITH_EDITOR || !UE_BUILD_SHIPPING
	if (EngineShowFlags.VisualizeNanite && NaniteRasterResults != nullptr)
	{
		AddVisualizeNanitePass(GraphBuilder, View, SceneColor, *NaniteRasterResults);
	}
#endif

	if (PassSequence.IsEnabled(EPass::VisualizeGBuffer))
	{
		FVisualizeGBufferInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeGBuffer, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.bOutputInHDR = bOutputInHDR;

		SceneColor = AddVisualizeGBufferPass(GraphBuilder, View, PassInputs, VisualizeBufferContext);
	}

#if WITH_EDITOR
	if (PassSequence.IsEnabled(EPass::ChannelMask))
	{
		ChannelMask::FChannelMaskInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::ChannelMask, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		SceneColor = ChannelMask::AddChannelMaskPass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::SelectionOutline))
	{
		FSelectionOutlineInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::SelectionOutline, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;
		PassInputs.SceneTextures.SceneTextures = Inputs.SceneTextures;
		if (bApplyLensDistortionInTSR)
		{
			PassInputs.LensDistortionLUT = View.LensDistortionLUT;
		}
		SceneColor = AddSelectionOutlinePass(GraphBuilder, View, SceneUniformBuffer, PassInputs, NaniteRasterResults, InstancedEditorDepthTexture);
	}

	if (PassSequence.IsEnabled(EPass::EditorPrimitive))
	{
		FCompositePrimitiveInputs PassInputs;
		if (PassSequence.AcceptOverrideIfLastPass(EPass::EditorPrimitive, PassInputs.OverrideOutput))
		{
			PassInputs.OverrideDepthOutput = ViewFamilyDepthOutput;
		}
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;
		PassInputs.BasePassType = FCompositePrimitiveInputs::EBasePassType::Deferred;
		if (bApplyLensDistortionInTSR)
		{
			PassInputs.LensDistortionLUT = View.LensDistortionLUT;
		}
		PassInputs.OccludedDitheringPatternSize = View.CompositePrimitiveParams.OccludedDitheringPatternSize;
		PassInputs.OccludedDithering = View.CompositePrimitiveParams.OccludedDithering;
		PassInputs.OccludedBrightness = View.CompositePrimitiveParams.OccludedBrightness;
		SceneColor = AddEditorPrimitivePass(GraphBuilder, View, PassInputs, InstanceCullingManager);
	}
#endif //WITH_EDITOR

#if UE_ENABLE_DEBUG_DRAWING
	if (PassSequence.IsEnabled(EPass::DebugPrimitive))
	{
		FCompositePrimitiveInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::DebugPrimitive, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;

		SceneColor = AddDebugPrimitivePass(GraphBuilder, View, ViewIndex, SceneUniformBuffer, VirtualShadowMapArray, PassInputs);
	}
#endif
	
	if (PassSequence.IsEnabled(EPass::VisualizeVirtualShadowMaps_PostEditorPrimitives))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeVirtualShadowMaps_PostEditorPrimitives, OverrideOutput);
		SceneColor = VirtualShadowMapArray->AddVisualizePass(GraphBuilder, View, ViewIndex, EVSMVisualizationPostPass::PostEditorPrimitives, SceneUniformBuffer, SceneColor, SceneDepth, OverrideOutput);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeVirtualTexture))
	{
		if (FRDGBuffer* DebugBuffer = VirtualTexture::ResolveExtendedDebugBuffer(GraphBuilder))
		{
			FVisualizeVirtualTextureInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeVirtualTexture, PassInputs.OverrideOutput);
			PassInputs.SceneColor = SceneColor;
			PassInputs.DebugBuffer = DebugBuffer;
			PassInputs.ModeName = GetVirtualTextureVisualizationData().GetActiveMode(View);
			PassInputs.Colors = GEngine->ShaderComplexityColors;

			SceneColor = AddVisualizeVirtualTexturePass(GraphBuilder, View, PassInputs);
		}
	}

	if (PassSequence.IsEnabled(EPass::VisualizeShadingModels))
	{
		ensureMsgf(View.PrimaryScreenPercentageMethod != EPrimaryScreenPercentageMethod::TemporalUpscale, TEXT("TAAU should be disabled when visualizing shading models."));

		FVisualizeShadingModelInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeShadingModels, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneTextures = Inputs.SceneTextures;

		SceneColor = AddVisualizeShadingModelPass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeGBufferHints))
	{
		ensureMsgf(View.PrimaryScreenPercentageMethod != EPrimaryScreenPercentageMethod::TemporalUpscale, TEXT("TAAU should be disabled when visualizing gbuffer hints."));

		FVisualizeGBufferHintsInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeGBufferHints, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.OriginalSceneColor = OriginalSceneColor;
		PassInputs.SceneTextures = Inputs.SceneTextures;

		SceneColor = AddVisualizeGBufferHintsPass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeSubsurface))
	{
		ensureMsgf(View.PrimaryScreenPercentageMethod != EPrimaryScreenPercentageMethod::TemporalUpscale, TEXT("TAAU should be disabled when visualizing subsurface."));

		FVisualizeSubsurfaceInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeSubsurface, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneTextures = Inputs.SceneTextures;

		SceneColor = AddVisualizeSubsurfacePass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeLumenSceneOverview))
	{
		FVisualizeLumenSceneInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeLumenSceneOverview, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;
		PassInputs.SceneTextures.SceneTextures = Inputs.SceneTextures;

		SceneColor = AddVisualizeLumenScenePass(GraphBuilder, View, DiffuseIndirectMethod, ReflectionsMethod, PassInputs, LumenFrameTemporaries);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeMegaLightsOverview))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeMegaLightsOverview, OverrideOutput);
		SceneColor = MegaLights::AddVisualizePass(GraphBuilder, View, ViewIndex, SceneColor, SceneDepth, OverrideOutput, MegaLightsFrameTemporaries);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeHDR))
	{
		FVisualizeHDRInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeHDR, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneColorBeforeTonemap = FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorBeforeTonemapSlice);
		PassInputs.Luminance = ExposureIlluminance;
		PassInputs.HistogramTexture = HistogramTexture;
		PassInputs.EyeAdaptationBuffer = GetEyeAdaptationBuffer(GraphBuilder, View);
		PassInputs.EyeAdaptationParameters = &EyeAdaptationParameters;

		SceneColor = AddVisualizeHDRPass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeLocalExposure))
	{
		FVisualizeLocalExposureInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeLocalExposure, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.HDRSceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorBeforeTonemapSlice);
		PassInputs.LumBilateralGridTexture = LocalExposureBilateralGridTexture;
		PassInputs.BlurredLumTexture = LocalExposureBlurredLogLumTexture;
		PassInputs.ExposureFusionData = View.FinalPostProcessSettings.LocalExposureMethod == ELocalExposureMethod::Fusion ? &ExposureFusionData : nullptr;
		PassInputs.LocalExposureParameters = &LocalExposureParameters;
		PassInputs.EyeAdaptationBuffer = GetEyeAdaptationBuffer(GraphBuilder, View);
		PassInputs.EyeAdaptationParameters = &EyeAdaptationParameters;

		SceneColor = AddVisualizeLocalExposurePass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeMotionVectors))
	{
		FVisualizeMotionVectorsInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeMotionVectors, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;
		PassInputs.SceneVelocity = Velocity;
		if (bApplyLensDistortionInTSR)
		{
			PassInputs.LensDistortionLUT = View.LensDistortionLUT;
		}

		SceneColor = AddVisualizeMotionVectorsPass(GraphBuilder, View, PassInputs, EVisualizeMotionVectors::ReprojectionAlignment);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeTemporalUpscaler))
	{
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeTemporalUpscaler, VisualizeTemporalUpscalerInputs.OverrideOutput);
		VisualizeTemporalUpscalerInputs.SceneColor = SceneColor;

		SceneColor = AddVisualizeTemporalUpscalerPass(GraphBuilder, View, VisualizeTemporalUpscalerInputs);
	}

#if WITH_EDITOR
	if (PassSequence.IsEnabled(EPass::PixelInspector))
	{
		FPixelInspectorInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::PixelInspector, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneColorBeforeTonemap = FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorBeforeTonemapSlice);
		PassInputs.OriginalSceneColor = OriginalSceneColor;

		SceneColor = AddPixelInspectorPass(GraphBuilder, View, PassInputs);
	}
#endif

	if (PassSequence.IsEnabled(EPass::HMDDistortion))
	{
		FHMDDistortionInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::HMDDistortion, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;

		SceneColor = AddHMDDistortionPass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::HighResolutionScreenshotMask))
	{
		FHighResolutionScreenshotMaskInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::HighResolutionScreenshotMask, PassInputs.OverrideOutput);
		PassInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);
		PassInputs.SceneColor = SceneColor;
		PassInputs.Material = View.FinalPostProcessSettings.HighResScreenshotMaterial;
		PassInputs.MaskMaterial = View.FinalPostProcessSettings.HighResScreenshotMaskMaterial;
		PassInputs.CaptureRegionMaterial = View.FinalPostProcessSettings.HighResScreenshotCaptureRegionMaterial;

		SceneColor = AddHighResolutionScreenshotMaskPass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::PrimaryUpscale))
	{
		ISpatialUpscaler::FInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::PrimaryUpscale, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.Stage = PassSequence.IsEnabled(EPass::SecondaryUpscale) ? EUpscaleStage::PrimaryToSecondary : EUpscaleStage::PrimaryToOutput;

		const ISpatialUpscaler* CustomUpscaler = View.Family ? View.Family->GetPrimarySpatialUpscalerInterface() : nullptr;
		if (CustomUpscaler)
		{
			RDG_EVENT_SCOPE(
				GraphBuilder,
				"ThirdParty PrimaryUpscale %s %dx%d -> %dx%d",
				CustomUpscaler->GetDebugName(),
				SceneColor.ViewRect.Width(), SceneColor.ViewRect.Height(),
				View.GetSecondaryViewRectSize().X, View.GetSecondaryViewRectSize().Y);

			SceneColor = CustomUpscaler->AddPasses(GraphBuilder, View, PassInputs);

			if (PassSequence.IsLastPass(EPass::PrimaryUpscale))
			{
				check(SceneColor == ViewFamilyOutput);
			}
			else
			{
				check(SceneColor.ViewRect.Size() == View.GetSecondaryViewRectSize());
			}
		}
		else
		{
			EUpscaleMethod Method = GetUpscaleMethod();

			SceneColor = ISpatialUpscaler::AddDefaultUpscalePass(GraphBuilder, View, PassInputs, Method, View.LensDistortionLUT);
		}
	}

	if (PassSequence.IsEnabled(EPass::SecondaryUpscale))
	{
		ISpatialUpscaler::FInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::SecondaryUpscale, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.Stage = EUpscaleStage::SecondaryToOutput;

		const ISpatialUpscaler* CustomUpscaler = View.Family ? View.Family->GetSecondarySpatialUpscalerInterface() : nullptr;
		if (CustomUpscaler)
		{
			RDG_EVENT_SCOPE(
				GraphBuilder,
				"ThirdParty SecondaryUpscale %s %dx%d -> %dx%d",
				CustomUpscaler->GetDebugName(),
				SceneColor.ViewRect.Width(), SceneColor.ViewRect.Height(),
				View.UnscaledViewRect.Width(), View.UnscaledViewRect.Height());

			SceneColor = CustomUpscaler->AddPasses(GraphBuilder, View, PassInputs);
			check(SceneColor == ViewFamilyOutput);
		}
		else
		{
			EUpscaleMethod Method = View.Family && View.Family->SecondaryScreenPercentageMethod == ESecondaryScreenPercentageMethod::LowerPixelDensitySimulation
				? EUpscaleMethod::SmoothStep
				: EUpscaleMethod::Nearest;

			SceneColor = ISpatialUpscaler::AddDefaultUpscalePass(GraphBuilder, View, PassInputs, Method);
		}
	}

	#if WITH_EDITOR || !UE_BUILD_SHIPPING
	{
		// Draw debug stuff directly onto the back buffer

		RDG_EVENT_SCOPE(GraphBuilder, "Debug Drawing");

		if (EngineShowFlags.TestImage)
		{
			AddTestImagePass(GraphBuilder, View, SceneColor);
		}

		#if WITH_EDITOR
		if (CVarGBufferPicking.GetValueOnRenderThread())
		{
			AddGBufferPicking(GraphBuilder, View, Inputs.SceneTextures);
		}
		#endif

		{
			RectLightAtlas::AddDebugPass(GraphBuilder, View, SceneColor.Texture);
			IESAtlas::AddDebugPass(GraphBuilder, View, SceneColor.Texture);
		}

		// Piggy back off of OnScreenDebug to avoid having to create a new show flag just for this simple debug visualization. Otherwise it might render into certain thumbnails.
		// In the future it might be worth it to introduce a show flag?
		if (EngineShowFlags.OnScreenDebug)
		{
			UE::SVT::AddStreamingDebugPass(GraphBuilder, View, SceneColor);
		}

		if (ShaderPrint::IsEnabled(View.ShaderPrintData))
		{
			ShaderPrint::DrawView(GraphBuilder, View, SceneColor, SceneDepth);
		}

		if (View.Family && View.Family->Scene)
		{
			if (FFXSystemInterface* FXSystem = View.Family->Scene->GetFXSystem())
			{
				FXSystem->DrawSceneDebug_RenderThread(GraphBuilder, (const FSceneView&)View, SceneColor.Texture, SceneDepth.Texture);
			}
		}
	}
	#endif

#if !UE_BUILD_SHIPPING
	AddUserSceneTextureDebugPass(GraphBuilder, View, ViewIndex, SceneColor);
#endif

	if (PassSequence.IsEnabled(EPass::AlphaInvert))
	{
		AlphaInvert::FAlphaInvertInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::AlphaInvert, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		SceneColor = AlphaInvert::AddAlphaInvertPass(GraphBuilder, View, PassInputs);
	}
}

void AddDebugViewPostProcessingPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	int32 ViewIndex,
	FSceneUniformBuffer& SceneUniformBuffer,
	const FPostProcessingInputs& Inputs,
	const Nanite::FRasterResults* NaniteRasterResults,
	FVirtualShadowMapArray* VirtualShadowMapArray)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderPostProcessing);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PostProcessing_Process);

	check(IsInRenderingThread());
#if DO_CHECK || USING_CODE_ANALYSIS
	check(View.VerifyMembersChecks());
#endif
	Inputs.Validate();

	const FIntRect PrimaryViewRect = View.ViewRect;

	const FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, Inputs.SceneTextures);

	const FScreenPassRenderTarget ViewFamilyOutput = FScreenPassRenderTarget::CreateViewFamilyOutput(Inputs.ViewFamilyTexture, View);
	const FScreenPassRenderTarget ViewFamilyDepthOutput = FScreenPassRenderTarget::CreateViewFamilyOutput(Inputs.ViewFamilyDepthTexture, View);
	const FScreenPassTexture SceneDepth(SceneTextureParameters.SceneDepthTexture, PrimaryViewRect);
	FScreenPassTexture SceneColor((*Inputs.SceneTextures)->SceneColorTexture, PrimaryViewRect);

	// Some view modes do not actually output a color so they should not be tonemapped.
	const bool bTonemapAfter = View.Family->EngineShowFlags.RayTracingDebug || View.Family->EngineShowFlags.VisualizeGPUSkinCache;
	const bool bTonemapBefore = !bTonemapAfter && !View.Family->EngineShowFlags.ShaderComplexity;
	const bool bViewFamilyOutputInHDR = View.Family->RenderTarget->GetSceneHDREnabled();

	enum class EPass : uint32
	{
		Visualize,
		TonemapAfter,
		VisualizeVirtualShadowMaps_PreEditorPrimitives,
		SelectionOutline,
		DebugPrimitive,
		VisualizeVirtualShadowMaps_PostEditorPrimitives,
		PrimaryUpscale,
		SecondaryUpscale,
		MAX
	};

	const TCHAR* PassNames[] =
	{
		TEXT("Visualize"),
		TEXT("TonemapAfter"),
		TEXT("VisualizeVirtualShadowMaps_PreEditorPrimitives"),
		TEXT("SelectionOutline"),
		TEXT("DebugPrimitive"),
		TEXT("VisualizeVirtualShadowMaps_PostEditorPrimitives"),
		TEXT("PrimaryUpscale"),
		TEXT("SecondaryUpscale")
	};

	static_assert(static_cast<uint32>(EPass::MAX) == UE_ARRAY_COUNT(PassNames), "EPass does not match PassNames.");

	TOverridePassSequence<EPass> PassSequence(ViewFamilyOutput);
	PassSequence.SetNames(PassNames, UE_ARRAY_COUNT(PassNames));
	PassSequence.SetEnabled(EPass::Visualize, true);
	PassSequence.SetEnabled(EPass::TonemapAfter, bTonemapAfter);
	PassSequence.SetEnabled(EPass::SelectionOutline, GIsEditor);

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	PassSequence.SetEnabled(EPass::VisualizeVirtualShadowMaps_PreEditorPrimitives, VirtualShadowMapArray != nullptr && VirtualShadowMapArray->IsVisualizePassEnabled(View, ViewIndex, EVSMVisualizationPostPass::PreEditorPrimitives));
	PassSequence.SetEnabled(EPass::VisualizeVirtualShadowMaps_PostEditorPrimitives, VirtualShadowMapArray != nullptr && VirtualShadowMapArray->IsVisualizePassEnabled(View, ViewIndex, EVSMVisualizationPostPass::PostEditorPrimitives));
#else
	PassSequence.SetEnabled(EPass::VisualizeVirtualShadowMaps_PreEditorPrimitives, false);
	PassSequence.SetEnabled(EPass::VisualizeVirtualShadowMaps_PostEditorPrimitives, false);
#endif
	PassSequence.SetEnabled(EPass::DebugPrimitive, IsDebugPrimitivePassEnabled(View));
	PassSequence.SetEnabled(EPass::PrimaryUpscale, View.ViewRect.Size() != View.GetSecondaryViewRectSize() && View.PrimaryScreenPercentageMethod != EPrimaryScreenPercentageMethod::TemporalUpscale);
	PassSequence.SetEnabled(EPass::SecondaryUpscale, View.RequiresSecondaryUpscale() || View.Family->GetSecondarySpatialUpscalerInterface() != nullptr);
	PassSequence.Finalize();

	const FEyeAdaptationParameters EyeAdaptationParameters = GetEyeAdaptationParameters(GraphBuilder, View);

	if (bTonemapBefore)
	{
		FTonemapInputs PassInputs;
		PassInputs.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
		PassInputs.bOutputInHDR = bViewFamilyOutputInHDR;
		PassInputs.bGammaOnly = true;
		PassInputs.EyeAdaptationParameters = &EyeAdaptationParameters;

		SceneColor = AddTonemapPass(GraphBuilder, View, PassInputs);
	}

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	if (View.Family->EngineShowFlags.VisualizeNanite && NaniteRasterResults != nullptr)
	{
		AddVisualizeNanitePass(GraphBuilder, View, SceneColor, *NaniteRasterResults);
	}
#endif

	check(PassSequence.IsEnabled(EPass::Visualize));
	{

		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::Visualize, OverrideOutput);

		switch (View.Family->GetDebugViewShaderMode())
		{
		case DVSM_QuadComplexity:
		{
			float ComplexityScale = 1.f / (float)(GEngine->QuadComplexityColors.Num() - 1) / NormalizedQuadComplexityValue; // .1f comes from the values used in LightAccumulator_GetResult

			FVisualizeComplexityInputs PassInputs;
			PassInputs.OverrideOutput = OverrideOutput;
			PassInputs.SceneColor = SceneColor;
			PassInputs.Colors = GEngine->QuadComplexityColors;
			PassInputs.ColorSamplingMethod = FVisualizeComplexityInputs::EColorSamplingMethod::Stair;
			PassInputs.ComplexityScale = ComplexityScale;
			PassInputs.bDrawLegend = true;

			SceneColor = AddVisualizeComplexityPass(GraphBuilder, View, PassInputs);
			break;
		}
		case DVSM_ShaderComplexity:
		case DVSM_ShaderComplexityContainedQuadOverhead:
		case DVSM_ShaderComplexityBleedingQuadOverhead:
		case DVSM_LWCComplexity:
		{
			FVisualizeComplexityInputs PassInputs;
			PassInputs.OverrideOutput = OverrideOutput;
			PassInputs.SceneColor = SceneColor;
			PassInputs.Colors = GEngine->ShaderComplexityColors;
			PassInputs.ColorSamplingMethod = FVisualizeComplexityInputs::EColorSamplingMethod::Ramp;
			PassInputs.ComplexityScale = 1.0f;
			PassInputs.bDrawLegend = true;

			SceneColor = AddVisualizeComplexityPass(GraphBuilder, View, PassInputs);
			break;
		}
		case DVSM_PrimitiveDistanceAccuracy:
		case DVSM_MeshUVDensityAccuracy:
		case DVSM_MaterialTextureScaleAccuracy:
		case DVSM_RequiredTextureResolution:
		{
			FStreamingAccuracyLegendInputs PassInputs;
			PassInputs.OverrideOutput = OverrideOutput;
			PassInputs.SceneColor = SceneColor;
			PassInputs.Colors = GEngine->StreamingAccuracyColors;

			SceneColor = AddStreamingAccuracyLegendPass(GraphBuilder, View, PassInputs);
			break;
		}
		case DVSM_VisualizeGPUSkinCache:
		{
			FTAAPassParameters Parameters(View);
			Parameters.SceneDepthTexture = SceneTextureParameters.SceneDepthTexture;
			Parameters.SceneVelocityTexture = SceneTextureParameters.GBufferVelocityTexture;
			Parameters.SceneColorInput = SceneColor.Texture;
			Parameters.Pass = View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale
				? ETAAPassConfig::MainUpsampling
				: ETAAPassConfig::Main;
			Parameters.SetupViewRect(View);

			const FTemporalAAHistory& InputHistory = View.PrevViewInfo.TemporalAAHistory;
			FTemporalAAHistory* OutputHistory = &View.ViewState->PrevFrameViewInfo.TemporalAAHistory;

			FTAAOutputs Outputs = AddTemporalAAPass(GraphBuilder, View, Parameters, InputHistory, OutputHistory);
			SceneColor.Texture = Outputs.SceneColor;
			SceneColor.ViewRect = Parameters.OutputViewRect;

			break;
		}
		case DVSM_LODColoration:
		{
			FLODColorationLegendInputs PassInputs{};
			PassInputs.OverrideOutput = OverrideOutput;
			PassInputs.SceneColor = SceneColor;
			const FEngineShowFlags& Flags = View.Family->EngineShowFlags;
			if (Flags.HLODColoration != 0)
			{
				PassInputs.Colors = GEngine->HLODColorationColors;
			}
			else if (Flags.LODColoration != 0)
			{
				PassInputs.Colors = GEngine->LODColorationColors;
			}

			SceneColor = AddLODColorationLegendPass(GraphBuilder, View, PassInputs);

			break;
		}
		case DVSM_ShadowCasters:
			break;
		case DVSM_StreamingDeficit:
		{
			FLODColorationLegendInputs PassInputs{};
			PassInputs.OverrideOutput = OverrideOutput;
			PassInputs.SceneColor = SceneColor;
			PassInputs.Colors = GEngine->StreamingDeficitColors;

			const FEngineShowFlags& Flags = View.Family->EngineShowFlags;
			if (Flags.StreamingTextureDeficit)
			{
				PassInputs.Title = TEXT("Streaming Texture Deficit (Wanted - Resident)");
				PassInputs.LeftLabel = TEXT("Fully Streamed");
				PassInputs.RightLabel = TEXT("Starved");
			}
			else if (Flags.StreamingTextureResidency)
			{
				PassInputs.Title = TEXT("Streaming Texture Residency (Max - Resident)");
				PassInputs.LeftLabel = TEXT("All Mips Loaded");
				PassInputs.RightLabel = TEXT("Most Mips Missing");
			}
			else if (Flags.StreamingMeshLODDeficit)
			{
				PassInputs.Title = TEXT("Streaming Mesh LOD Deficit (Wanted - Resident)");
				PassInputs.LeftLabel = TEXT("Fully Streamed");
				PassInputs.RightLabel = TEXT("Starved");
			}
			else if (Flags.StreamingMeshLODResidency)
			{
				PassInputs.Title = TEXT("Streaming Mesh LOD Residency (Max - Resident)");
				PassInputs.LeftLabel = TEXT("All LODs Loaded");
				PassInputs.RightLabel = TEXT("Most LODs Missing");
			}

			SceneColor = AddLODColorationLegendPass(GraphBuilder, View, PassInputs);
			break;
		}
		default:
			ensure(false);
			break;
		}
	}

	if (PassSequence.IsEnabled(EPass::TonemapAfter))
	{
		FTonemapInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::TonemapAfter, PassInputs.OverrideOutput);
		PassInputs.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
		PassInputs.bOutputInHDR = bViewFamilyOutputInHDR;
		PassInputs.bGammaOnly = true;
		// Do eye adaptation in ray tracing debug modes to match raster buffer visualization modes
		PassInputs.EyeAdaptationParameters = &EyeAdaptationParameters;
		PassInputs.EyeAdaptationBuffer = GetEyeAdaptationBuffer(GraphBuilder, View);

		SceneColor = AddTonemapPass(GraphBuilder, View, PassInputs);
	}

	if (PassSequence.IsEnabled(EPass::VisualizeVirtualShadowMaps_PreEditorPrimitives))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeVirtualShadowMaps_PreEditorPrimitives, OverrideOutput);
		SceneColor = VirtualShadowMapArray->AddVisualizePass(GraphBuilder, View, ViewIndex, EVSMVisualizationPostPass::PreEditorPrimitives, SceneUniformBuffer, SceneColor, SceneDepth, OverrideOutput);
	}

#if WITH_EDITOR
	if (PassSequence.IsEnabled(EPass::SelectionOutline))
	{
		FSelectionOutlineInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::SelectionOutline, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;
		PassInputs.SceneTextures.SceneTextures = Inputs.SceneTextures;

		FRDGTextureRef DummyStencilTexture = nullptr;
		SceneColor = AddSelectionOutlinePass(GraphBuilder, View, SceneUniformBuffer, PassInputs, NaniteRasterResults, DummyStencilTexture);
	}
#endif

#if UE_ENABLE_DEBUG_DRAWING
	if (PassSequence.IsEnabled(EPass::DebugPrimitive))
	{
		FCompositePrimitiveInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::DebugPrimitive, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;

		SceneColor = AddDebugPrimitivePass(GraphBuilder, View, ViewIndex, SceneUniformBuffer, VirtualShadowMapArray, PassInputs);
	}
#endif

	if (PassSequence.IsEnabled(EPass::VisualizeVirtualShadowMaps_PostEditorPrimitives))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeVirtualShadowMaps_PostEditorPrimitives, OverrideOutput);
		SceneColor = VirtualShadowMapArray->AddVisualizePass(GraphBuilder, View, ViewIndex, EVSMVisualizationPostPass::PostEditorPrimitives, SceneUniformBuffer, SceneColor, SceneDepth, OverrideOutput);
	}

	if (PassSequence.IsEnabled(EPass::PrimaryUpscale))
	{
		ISpatialUpscaler::FInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::PrimaryUpscale, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.Stage = PassSequence.IsEnabled(EPass::SecondaryUpscale) ? EUpscaleStage::PrimaryToSecondary : EUpscaleStage::PrimaryToOutput;

		if (const ISpatialUpscaler* CustomUpscaler = View.Family->GetPrimarySpatialUpscalerInterface())
		{
			RDG_EVENT_SCOPE(
				GraphBuilder,
				"ThirdParty PrimaryUpscale %s %dx%d -> %dx%d",
				CustomUpscaler->GetDebugName(),
				SceneColor.ViewRect.Width(), SceneColor.ViewRect.Height(),
				View.GetSecondaryViewRectSize().X, View.GetSecondaryViewRectSize().Y);

			SceneColor = CustomUpscaler->AddPasses(GraphBuilder, View, PassInputs);

			if (PassSequence.IsLastPass(EPass::PrimaryUpscale))
			{
				check(SceneColor == ViewFamilyOutput);
			}
			else
			{
				check(SceneColor.ViewRect.Size() == View.GetSecondaryViewRectSize());
			}
		}
		else
		{
			EUpscaleMethod Method = GetUpscaleMethod();

			SceneColor = ISpatialUpscaler::AddDefaultUpscalePass(GraphBuilder, View, PassInputs, Method);
		}
	}

	if (PassSequence.IsEnabled(EPass::SecondaryUpscale))
	{
		ISpatialUpscaler::FInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::SecondaryUpscale, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.Stage = EUpscaleStage::SecondaryToOutput;

		if (const ISpatialUpscaler* CustomUpscaler = View.Family->GetSecondarySpatialUpscalerInterface())
		{
			RDG_EVENT_SCOPE(
				GraphBuilder,
				"ThirdParty SecondaryUpscale %s %dx%d -> %dx%d",
				CustomUpscaler->GetDebugName(),
				SceneColor.ViewRect.Width(), SceneColor.ViewRect.Height(),
				View.UnscaledViewRect.Width(), View.UnscaledViewRect.Height());

			SceneColor = CustomUpscaler->AddPasses(GraphBuilder, View, PassInputs);
			check(SceneColor == ViewFamilyOutput);
		}
		else
		{
			EUpscaleMethod Method = View.Family->SecondaryScreenPercentageMethod == ESecondaryScreenPercentageMethod::LowerPixelDensitySimulation
				? EUpscaleMethod::SmoothStep
				: EUpscaleMethod::Nearest;

			SceneColor = ISpatialUpscaler::AddDefaultUpscalePass(GraphBuilder, View, PassInputs, Method);
		}
	}

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	if (ShaderPrint::IsEnabled(View.ShaderPrintData))
	{
		ShaderPrint::DrawView(GraphBuilder, View, SceneColor, SceneDepth);
	}
#endif
}

void AddVisualizeCalibrationMaterialPostProcessingPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FPostProcessingInputs& Inputs, const UMaterialInterface* InMaterialInterface)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderPostProcessing);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PostProcessing_Process);

	check(IsInRenderingThread());
#if DO_CHECK || USING_CODE_ANALYSIS
	check(View.VerifyMembersChecks());
#endif
	check(InMaterialInterface);
	Inputs.Validate();

	const FIntRect PrimaryViewRect = View.ViewRect;

	const FSceneTextureParameters& SceneTextures = GetSceneTextureParameters(GraphBuilder, Inputs.SceneTextures);
	const FScreenPassRenderTarget ViewFamilyOutput = FScreenPassRenderTarget::CreateViewFamilyOutput(Inputs.ViewFamilyTexture, View);

	// Scene color is updated incrementally through the post process pipeline.
	FScreenPassTexture SceneColor((*Inputs.SceneTextures)->SceneColorTexture, PrimaryViewRect);

	const FEngineShowFlags& EngineShowFlags = View.Family->EngineShowFlags;
	const bool bViewFamilyOutputInHDR = View.Family->RenderTarget->GetSceneHDREnabled();
	const bool bOutputInHDR = IsPostProcessingOutputInHDR();

	// Post Process Material - Before Color Correction
	FPostProcessMaterialInputs PostProcessMaterialInputs;
	PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, SceneColor);
	PostProcessMaterialInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);

	SceneColor = AddPostProcessMaterialPass(GraphBuilder, View, PostProcessMaterialInputs, InMaterialInterface);

	// Replace tonemapper with device encoding only pass, which converts the scene color to device-specific color.
	FDeviceEncodingOnlyInputs PassInputs;
	PassInputs.OverrideOutput = ViewFamilyOutput;
	PassInputs.SceneColor = SceneColor;
	PassInputs.bOutputInHDR = bViewFamilyOutputInHDR;

	SceneColor = AddDeviceEncodingOnlyPass(GraphBuilder, View, PassInputs);
}

///////////////////////////////////////////////////////////////////////////
// Mobile Post Processing
//////////////////////////////////////////////////////////////////////////

static bool IsGaussianActive(const FViewInfo& View)
{
	float FarSize = View.FinalPostProcessSettings.DepthOfFieldFarBlurSize;
	float NearSize = View.FinalPostProcessSettings.DepthOfFieldNearBlurSize;

	float MaxSize = CVarDepthOfFieldMaxSize.GetValueOnRenderThread();

	FarSize = FMath::Min(FarSize, MaxSize);
	NearSize = FMath::Min(NearSize, MaxSize);
	const float CVarThreshold = CVarDepthOfFieldNearBlurSizeThreshold.GetValueOnRenderThread();

	if ((FarSize < 0.01f) && (NearSize < CVarThreshold))
	{
		return false;
	}
	return true;
}

void AddMobilePostProcessingPasses(FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View, int32 ViewIndex, FSceneUniformBuffer &SceneUniformBuffer, const FMobilePostProcessingInputs& Inputs, FInstanceCullingManager& InstanceCullingManager)
{
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RenderPostProcessing);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PostProcessing_Process);
	using namespace UE::Renderer::PostProcess;

	check(View.Family);

	check(IsInRenderingThread());
	Inputs.Validate();

	const FIntRect FinalOutputViewRect = View.ViewRect;

	const FScreenPassRenderTarget ViewFamilyOutput = FScreenPassRenderTarget::CreateViewFamilyOutput(Inputs.ViewFamilyTexture, View);
	const FScreenPassRenderTarget ViewFamilyDepthOutput = FScreenPassRenderTarget::CreateViewFamilyOutput(Inputs.ViewFamilyDepthTexture, View);
	const FScreenPassTexture SceneDepth((*Inputs.SceneTextures)->SceneDepthTexture, FinalOutputViewRect);
	const FScreenPassTexture CustomDepth((*Inputs.SceneTextures)->CustomDepthTexture, FinalOutputViewRect);
	const FScreenPassTexture Velocity((*Inputs.SceneTextures)->SceneVelocityTexture, FinalOutputViewRect);
	const FScreenPassTexture BlackAlphaOneDummy(GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder));

	// Scene color is updated incrementally through the post process pipeline.
	FScreenPassTexture SceneColor((*Inputs.SceneTextures)->SceneColorTexture, FinalOutputViewRect);
	FScreenPassTexture SceneDepthAux((*Inputs.SceneTextures)->SceneDepthAuxTexture, FinalOutputViewRect);

	// Default the new eye adaptation to the last one in case it's not generated this frame.
	const FEyeAdaptationParameters EyeAdaptationParameters = GetEyeAdaptationParameters(GraphBuilder, View);
	FRDGBufferRef LastEyeAdaptationBuffer = GetEyeAdaptationBuffer(GraphBuilder, View);

	// Histogram SRV for the VisualizeHDR overlay; populated by the eye-adaptation setup pass.
	FRDGBufferSRVRef VisualizeHDRHistogramSRV = nullptr;

	FVisualizeBufferContext VisualizeBufferContext{};

	// Local exposure state
	FLocalExposureParameters LocalExposureParameters;
	FRDGTextureRef LocalExposureBilateralGridTexture = nullptr;
	FRDGTextureRef LocalExposureBlurredLogLumTexture = nullptr;
	FExposureFusionData ExposureFusionData;

	enum class EPass : uint32
	{
		Distortion,
		SunMask,
		LocalExposure,
		BloomSetup,
		DepthOfField,
		Bloom,
		EyeAdaptation,
		SunMerge,
		SeparateTranslucency,
		TAA,
		Tonemap,
		FXAA,
		SMAA,
		PostProcessMaterialAfterTonemapping,
		VisualizeHDR,
		HighResolutionScreenshotMask,
		VisualizeSkyLightIlluminanceMeter,
		VisualizeGBuffer,
		ChannelMask,
		SelectionOutline,
		EditorPrimitive,
		DebugPrimitive,
		PrimaryUpscale,
		SecondaryUpscale,
		Visualize,
		VisualizeLightGrid,
		HMDDistortion,
		MAX
	};

	// Mobile unsupported passes return EPass::MAX
	const auto TranslatePass = [](ISceneViewExtension::EPostProcessingPass Pass) -> EPass
		{
			switch (Pass)
			{
			case ISceneViewExtension::EPostProcessingPass::MotionBlur: return EPass::MAX;
			case ISceneViewExtension::EPostProcessingPass::Tonemap: return EPass::Tonemap;
			case ISceneViewExtension::EPostProcessingPass::FXAA: return EPass::FXAA;
			case ISceneViewExtension::EPostProcessingPass::SMAA: return EPass::SMAA;
			case ISceneViewExtension::EPostProcessingPass::VisualizeDepthOfField: return EPass::MAX;

			default:
				check(false);
				return EPass::MAX;
			};
		};

	static const TCHAR* PassNames[] =
	{
		TEXT("Distortion"),
		TEXT("SunMask"),
		TEXT("LocalExposure"),
		TEXT("BloomSetup"),
		TEXT("DepthOfField"),
		TEXT("Bloom"),
		TEXT("EyeAdaptation"),
		TEXT("SunMerge"),
		TEXT("SeparateTranslucency"),
		TEXT("TAA"),
		TEXT("SMAA"),
		TEXT("Tonemap"),
		TEXT("PostProcessMaterial (AfterTonemapping)"),
		TEXT("FXAA"),
		TEXT("VisualizeHDR"),
		TEXT("HighResolutionScreenshotMask"),
		TEXT("VisualizeSkyLightIlluminanceMeter"),
		TEXT("VisualizeGBuffer"),
		TEXT("ChannelMask"),
		TEXT("SelectionOutline"),
		TEXT("EditorPrimitive"),
		TEXT("DebugPrimitive"),
		TEXT("PrimaryUpscale"),
		TEXT("SecondaryUpscale"),
		TEXT("Visualize"),
		TEXT("VisualizeLightGrid"),
		TEXT("HMDDistortion")
	};

	static_assert(static_cast<uint32>(EPass::MAX) == UE_ARRAY_COUNT(PassNames), "EPass does not match PassNames.");

	TOverridePassSequence<EPass> PassSequence(ViewFamilyOutput);
	PassSequence.SetNames(PassNames, UE_ARRAY_COUNT(PassNames));

	// This page: https://udn.epicgames.com/Three/RenderingOverview#Rendering%20state%20defaults 
	// describes what state a pass can expect and to what state it need to be set back.

	// All post processing is happening on the render thread side. All passes can access FinalPostProcessSettings and all
	// view settings. Those are copies for the RT then never get access by the main thread again.
	// Pointers to other structures might be unsafe to touch.

	const EDebugViewShaderMode DebugViewShaderMode = View.Family->GetDebugViewShaderMode();

	FScreenPassTexture BloomOutput;
	FScreenPassTexture DofOutput;
	FScreenPassTexture PostProcessSunShaftAndDof;
	
	const EAutoExposureMethod AutoExposureMethod = GetAutoExposureMethod(View);
	const bool bUseEyeAdaptation = IsMobileEyeAdaptationEnabled(View);
	const bool bIsPostProcessingEnabled = IsPostProcessingEnabled(View);
	
	//The input scene color has been encoded to non-linear space and needs to decode somewhere if MSAA enabled on Metal platform
	bool bMetalMSAAHDRDecode = GSupportsShaderFramebufferFetch && IsMetalMobilePlatform(View.GetShaderPlatform()) && GetDefaultMSAACount(ERHIFeatureLevel::ES3_1) > 1;

	// add the passes we want to add to the graph (commenting a line means the pass is not inserted into the graph) ---------

	// HQ gaussian 
	bool bUseDof = GetMobileDepthOfFieldScale(View) > 0.0f && View.Family->EngineShowFlags.DepthOfField && !View.Family->EngineShowFlags.VisualizeDOF && View.FinalPostProcessSettings.DepthOfFieldEnabled;
	bool bUseMobileDof = bUseDof && !View.FinalPostProcessSettings.bMobileHQGaussian;

	// Do not use the tonemapper if output texture is sRGB cause the conversion will be performed by HW.
	const bool bIsOutputTexsRGB = EnumHasAnyFlags(Inputs.ViewFamilyTexture->Desc.Flags, TexCreate_SRGB);
	bool bUseToneMapper = !View.Family->EngineShowFlags.ShaderComplexity && (IsMobileHDR() || (IsMobileColorsRGB() && !bIsOutputTexsRGB));

	bool bUseHighResolutionScreenshotMask = IsHighResolutionScreenshotMaskEnabled(View);

	//Primary does not require the interface to be available to run as it has a fallback logic already.
	bool bShouldPrimaryUpscale = (View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::SpatialUpscale && View.GetSecondaryViewRectSize() != View.ViewRect.Size()) || View.LensDistortionLUT.IsEnabled();

	const bool bVisualizeGBuffer = IsVisualizeGBufferEnabled(View);
	const bool bVisualizeHDR = View.Family->EngineShowFlags.VisualizeHDR && !View.Family->EngineShowFlags.Wireframe;
	const bool bOutputInHDR = IsPostProcessingOutputInHDR();

	PassSequence.SetEnabled(EPass::Tonemap, bUseToneMapper);
	PassSequence.SetEnabled(EPass::HighResolutionScreenshotMask, bUseHighResolutionScreenshotMask);
#if WITH_EDITOR
	PassSequence.SetEnabled(EPass::VisualizeSkyLightIlluminanceMeter, Scene && Scene->SkyLight && View.Family->EngineShowFlags.VisualizeSkyLightIlluminance);
	PassSequence.SetEnabled(EPass::ChannelMask, View.Family->ChannelMaskParams.ColorChannelMask != EColorChannelMask::All || View.Family->ChannelMaskParams.bDrawAlphaBlendedCheckerboard);
	PassSequence.SetEnabled(EPass::SelectionOutline, GIsEditor && View.Family->EngineShowFlags.Selection && View.Family->EngineShowFlags.SelectionOutline && !View.Family->EngineShowFlags.Wireframe);
	PassSequence.SetEnabled(EPass::EditorPrimitive, FSceneRenderer::ShouldCompositeEditorPrimitives(View));
#else
	PassSequence.SetEnabled(EPass::VisualizeSkyLightIlluminanceMeter, false);
	PassSequence.SetEnabled(EPass::ChannelMask, false);
	PassSequence.SetEnabled(EPass::SelectionOutline, false);
	PassSequence.SetEnabled(EPass::EditorPrimitive, false);
#endif

	// Debug primitive compositing does not currently support mobile multi-view
	PassSequence.SetEnabled(EPass::DebugPrimitive, IsDebugPrimitivePassEnabled(View) && !View.UsesMobileMultiViewShaders());
	PassSequence.SetEnabled(EPass::PrimaryUpscale, bShouldPrimaryUpscale);
	// Mobile is different from the deferred that the secondary upscale logic only runs when the interface is available
	// While the deferred itself has a fallback when secondary upscaler interface is not present.
	PassSequence.SetEnabled(EPass::SecondaryUpscale, View.Family->GetSecondarySpatialUpscalerInterface() != nullptr);

	PassSequence.SetEnabled(EPass::Visualize, View.Family->EngineShowFlags.ShaderComplexity);
	PassSequence.SetEnabled(EPass::VisualizeLightGrid, ShouldVisualizeLightGrid(View.Family->GetShaderPlatform()));
	PassSequence.SetEnabled(EPass::VisualizeGBuffer, bVisualizeGBuffer);
	PassSequence.SetEnabled(EPass::HMDDistortion, View.Family->EngineShowFlags.StereoRendering && View.Family->EngineShowFlags.HMDDistortion);

	// TODO: Add support for EPass::AlphaInvert if the AlphaInvert show flag is enabled, but EPass::PrimaryUpscale is disabled

	const auto GetPostProcessMaterialInputs = [&](FScreenPassTextureSlice InSceneColor)
		{
			FPostProcessMaterialInputs PostProcessMaterialInputs;

			PostProcessMaterialInputs.SetInput(EPostProcessMaterialInput::SceneColor, InSceneColor);
			PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::Velocity, Velocity);
			PostProcessMaterialInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);
			PostProcessMaterialInputs.CustomDepthTexture = CustomDepth.Texture;

			return PostProcessMaterialInputs;
		};

	const auto AddAfterPass = [&](EPass InPass, FScreenPassTexture InSceneColor) -> FScreenPassTexture
	{
		// In some cases (e.g. OCIO color conversion) we want View Extensions to be able to add extra custom post processing after the pass.

		FPostProcessingPassDelegateArray& PassCallbacks = PassSequence.GetAfterPassCallbacks(InPass);

		if (PassCallbacks.Num())
		{
			FScreenPassTextureSlice InSceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, InSceneColor);
			FPostProcessMaterialInputs InOutPostProcessAfterPassInputs = GetPostProcessMaterialInputs(InSceneColorSlice);

			for (int32 AfterPassCallbackIndex = 0; AfterPassCallbackIndex < PassCallbacks.Num(); AfterPassCallbackIndex++)
			{
				InOutPostProcessAfterPassInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, InSceneColor);

				FAfterPassCallbackDelegate& AfterPassCallback = PassCallbacks[AfterPassCallbackIndex];
				PassSequence.AcceptOverrideIfLastPass(InPass, InOutPostProcessAfterPassInputs.OverrideOutput, AfterPassCallbackIndex);
				InSceneColor = AfterPassCallback.Execute(GraphBuilder, View, InOutPostProcessAfterPassInputs);
			}
		}

		return MoveTemp(InSceneColor);
	};

	// Always evaluate custom post processes
	// The scene color will be decoded at the first post-process material and output linear color space for the following passes
	// bMetalMSAAHDRDecode will be set to false if there is any post-process material exist

	auto AddPostProcessMaterialPass = [&GraphBuilder, &View, ViewIndex, &Inputs, &SceneColor, &CustomDepth, &bMetalMSAAHDRDecode, &PassSequence, &BloomOutput, &BlackAlphaOneDummy](EBlendableLocation BlendableLocation)
	{
		FPostProcessMaterialInputs PostProcessMaterialInputs;

		if (BlendableLocation == BL_SceneColorAfterTonemapping && PassSequence.IsEnabled(EPass::PostProcessMaterialAfterTonemapping))
		{
			PassSequence.AcceptOverrideIfLastPass(EPass::PostProcessMaterialAfterTonemapping, PostProcessMaterialInputs.OverrideOutput);
		}

		if(BlendableLocation == BL_ReplacingTonemapper && PassSequence.IsEnabled(EPass::Tonemap))
		{
			PassSequence.AcceptOverrideIfLastPass(EPass::Tonemap, PostProcessMaterialInputs.OverrideOutput);
		}

		PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, SceneColor);

		if (BlendableLocation == BL_ReplacingTonemapper && PassSequence.IsEnabled(EPass::Tonemap))
		{
			if (!BloomOutput.IsValid())
			{
				BloomOutput = BlackAlphaOneDummy;
			}
			PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::CombinedBloom, BloomOutput);
		}

		PostProcessMaterialInputs.CustomDepthTexture = CustomDepth.Texture;

		PostProcessMaterialInputs.bMetalMSAAHDRDecode = bMetalMSAAHDRDecode;

		PostProcessMaterialInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);

		const FPostProcessMaterialChain MaterialChain = GetPostProcessMaterialChain(View, BlendableLocation);

		if (MaterialChain.Num())
		{
			SceneColor = AddPostProcessMaterialChain(GraphBuilder, View, ViewIndex, PostProcessMaterialInputs, MaterialChain);

			// For solid material, we decode the input color and output the linear color
			// For blend material, we force it rendering to an intermediate render target and decode there
			bMetalMSAAHDRDecode = false;
		}
	};
	

	constexpr int32 FirstAfterPass = static_cast<int32>(ISceneViewExtension::EPostProcessingPass::MotionBlur);
	// Scene view extension delegates that precede the override pass sequence are to be called directly.
	TStaticArray<FPostProcessingPassDelegateArray, static_cast<uint32>(FirstAfterPass)> SceneViewExtensionDelegates;

	if (bIsPostProcessingEnabled)
	{
		bool bUseSun = View.MobileLightShaft.IsSet();
			
		bool bUseBloom = View.FinalPostProcessSettings.BloomIntensity > 0.0f;

		bool bUseBasicEyeAdaptation = bUseEyeAdaptation && (AutoExposureMethod == EAutoExposureMethod::AEM_Basic);
		bool bUseHistogramEyeAdaptation = bUseEyeAdaptation && (AutoExposureMethod == EAutoExposureMethod::AEM_Histogram) &&
			// Skip if we don't have any exposure range to generate (eye adaptation will clamp).
			View.FinalPostProcessSettings.AutoExposureMinBrightness < View.FinalPostProcessSettings.AutoExposureMaxBrightness;

		bool bUseTAA = View.AntiAliasingMethod == AAM_TemporalAA;
		ensure(View.AntiAliasingMethod != AAM_TSR);

		bool bUseDistortion = IsMobileDistortionActive(View) && !IsMobileDistortionBeforeTranslucency();

		bool bUseSeparateTranslucency = IsMobileSeparateTranslucencyActive(View);

		const bool bLocalExposureEnabled =
			(!FMath::IsNearlyEqual(View.FinalPostProcessSettings.LocalExposureHighlightContrastScale, 1.0f) ||
			!FMath::IsNearlyEqual(View.FinalPostProcessSettings.LocalExposureShadowContrastScale, 1.0f) ||
			View.FinalPostProcessSettings.LocalExposureHighlightContrastCurve ||
			View.FinalPostProcessSettings.LocalExposureShadowContrastCurve ||
			!FMath::IsNearlyEqual(View.FinalPostProcessSettings.LocalExposureDetailStrength, 1.0f));

		const FPostProcessMaterialChain PostProcessMaterialAfterTonemappingChain = GetPostProcessMaterialChain(View, BL_SceneColorAfterTonemapping);

		PassSequence.SetEnabled(EPass::Distortion, bUseDistortion);
		PassSequence.SetEnabled(EPass::SunMask, bUseSun || bUseDof);
		PassSequence.SetEnabled(EPass::LocalExposure, bLocalExposureEnabled);
		PassSequence.SetEnabled(EPass::EyeAdaptation, bUseBasicEyeAdaptation || bUseHistogramEyeAdaptation);
		PassSequence.SetEnabled(EPass::BloomSetup, bUseSun || bUseMobileDof || bUseBloom || PassSequence.IsEnabled(EPass::EyeAdaptation));
		PassSequence.SetEnabled(EPass::DepthOfField, bUseDof);
		PassSequence.SetEnabled(EPass::Bloom, bUseBloom);
		PassSequence.SetEnabled(EPass::SunMerge, bUseBloom || bUseSun);
		PassSequence.SetEnabled(EPass::SeparateTranslucency, bUseSeparateTranslucency);
		PassSequence.SetEnabled(EPass::TAA, bUseTAA);
		PassSequence.SetEnabled(EPass::FXAA, View.AntiAliasingMethod == AAM_FXAA);
		PassSequence.SetEnabled(EPass::SMAA, View.AntiAliasingMethod == AAM_SMAA);
		PassSequence.SetEnabled(EPass::PostProcessMaterialAfterTonemapping, PostProcessMaterialAfterTonemappingChain.Num() != 0);
		PassSequence.SetEnabled(EPass::VisualizeHDR, bVisualizeHDR);

		for (const TSharedRef<ISceneViewExtension>& ViewExtension : View.Family->ViewExtensions)
		{
			for (int32 SceneViewPassId = 0; SceneViewPassId < FirstAfterPass; SceneViewPassId++)
			{
				const ISceneViewExtension::EPostProcessingPass SceneViewPass = static_cast<ISceneViewExtension::EPostProcessingPass>(SceneViewPassId);
				const bool bIsEnabled = (SceneViewPass == ISceneViewExtension::EPostProcessingPass::ReplacingTonemapper) ? PassSequence.IsEnabled(EPass::Tonemap) : true;

				ViewExtension->SubscribeToPostProcessingPass(SceneViewPass, View, SceneViewExtensionDelegates[SceneViewPassId], bIsEnabled);
			}

			for (int32 SceneViewPassId = FirstAfterPass; SceneViewPassId < static_cast<int32>(ISceneViewExtension::EPostProcessingPass::MAX); SceneViewPassId++)
			{
				const ISceneViewExtension::EPostProcessingPass SceneViewPass = static_cast<ISceneViewExtension::EPostProcessingPass>(SceneViewPassId);
				const EPass PostProcessingPass = TranslatePass(SceneViewPass);

				if (PostProcessingPass != EPass::MAX)
				{
					ViewExtension->SubscribeToPostProcessingPass(
						SceneViewPass,
						View,
						PassSequence.GetAfterPassCallbacks(PostProcessingPass),
						PassSequence.IsEnabled(PostProcessingPass));
				}
			}
		}

		PassSequence.Finalize();

		if (PassSequence.IsEnabled(EPass::Distortion))
		{
			PassSequence.AcceptPass(EPass::Distortion);
			FMobileDistortionAccumulateInputs DistortionAccumulateInputs;
			DistortionAccumulateInputs.SceneColor = SceneColor;

			FMobileDistortionAccumulateOutputs DistortionAccumulateOutputs = AddMobileDistortionAccumulatePass(GraphBuilder, Scene, View, DistortionAccumulateInputs);

			FMobileDistortionMergeInputs DistortionMergeInputs;
			DistortionMergeInputs.SceneColor = SceneColor;
			DistortionMergeInputs.DistortionAccumulate = DistortionAccumulateOutputs.DistortionAccumulate;

			SceneColor = AddMobileDistortionMergePass(GraphBuilder, View, DistortionMergeInputs);
		}

		if (SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::BeforeDOF)].Num())
		{
			FScreenPassTextureSlice SceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
			SceneColor = AddSceneViewExtensionPassChain(GraphBuilder, View, GetPostProcessMaterialInputs(SceneColorSlice),
				SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::BeforeDOF)]);
		}

		AddPostProcessMaterialPass(BL_SceneColorBeforeDOF);

		{
			FPostProcessMaterialInputs PostProcessMaterialInputs = GetPostProcessMaterialInputs(FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor));
			AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_SceneColorBeforeDOF, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
		}

		// Optional fixed pass processes
		if (PassSequence.IsEnabled(EPass::SunMask))
		{
			PassSequence.AcceptPass(EPass::SunMask);
			bool bUseDepthTexture = !MobileRequiresSceneDepthAux(View.GetShaderPlatform()) || IsMobileDeferredShadingEnabled(View.GetShaderPlatform());

			FMobileSunMaskInputs SunMaskInputs;
			SunMaskInputs.bUseDepthTexture = bUseDepthTexture;
			SunMaskInputs.bUseDof = bUseDof;
			SunMaskInputs.bUseMetalMSAAHDRDecode = bMetalMSAAHDRDecode;
			SunMaskInputs.bUseSun = bUseSun;
			SunMaskInputs.SceneColor = SceneColor;
			SunMaskInputs.SceneTextures = Inputs.SceneTextures;

			// Convert depth to {circle of confusion, sun shaft intensity}
			FMobileSunMaskOutputs SunMaskOutputs = AddMobileSunMaskPass(GraphBuilder, View, SunMaskInputs);

			PostProcessSunShaftAndDof = SunMaskOutputs.SunMask;

			// The scene color will be decoded after sun mask pass and output to linear color space for following passes if sun shaft enabled
			// set bMetalMSAAHDRDecode to false if sun shaft enabled
			if (bMetalMSAAHDRDecode && bUseSun)
			{
				SceneColor = SunMaskOutputs.SceneColor;
				bMetalMSAAHDRDecode = false;
			}
			//@todo Ronin sunmask pass isnt clipping to image only.
		}

		// Mobile local exposure
		if (PassSequence.IsEnabled(EPass::LocalExposure))
		{
			PassSequence.AcceptPass(EPass::LocalExposure);
			const ELocalExposureMethod LocalExposureMethod = View.FinalPostProcessSettings.LocalExposureMethod;

			FScreenPassTextureSlice LocalExposureSceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);

			if (LocalExposureMethod == ELocalExposureMethod::Bilateral)
			{
				LocalExposureBilateralGridTexture = AddLocalExposurePass(GraphBuilder, View, EyeAdaptationParameters, LocalExposureSceneColor);
			}

			LocalExposureParameters = GetLocalExposureParameters(View, LocalExposureSceneColor.ViewRect.Size(), EyeAdaptationParameters);

			if (LocalExposureMethod == ELocalExposureMethod::Bilateral && LocalExposureBilateralGridTexture != nullptr && View.FinalPostProcessSettings.LocalExposureBlurredLuminanceBlend > 0.0f)
			{
				LocalExposureBlurredLogLumTexture = AddLocalExposureBlurredLogLuminancePass(
					GraphBuilder, View, EyeAdaptationParameters, LocalExposureSceneColor);
			}

			if (LocalExposureMethod == ELocalExposureMethod::Fusion && LastEyeAdaptationBuffer)
			{
				ExposureFusionData = AddLocalExposureFusionPass(
					GraphBuilder, View, EyeAdaptationParameters,
					LastEyeAdaptationBuffer, LocalExposureParameters,
					LocalExposureSceneColor);
			}
		}

		FMobileBloomSetupOutputs BloomSetupOutputs;

		const int32 BloomQualityDivider = GetBloomQuality() <= EBloomQuality::Q2 ? 4 : 2;
		const FIntPoint BloomOutputTextureSize = FIntPoint::DivideAndRoundUp(SceneColor.ViewRect.Size(), BloomQualityDivider);
		const FIntPoint BloomOutputFixedTextureSize = FIntPoint::DivideAndRoundUp(View.GetSceneTexturesConfig().Extent, BloomQualityDivider);
		
		
		if (PassSequence.IsEnabled(EPass::BloomSetup))
		{
			PassSequence.AcceptPass(EPass::BloomSetup);
			bool bHasEyeAdaptationPass = (bUseBasicEyeAdaptation || bUseHistogramEyeAdaptation);

			FMobileBloomSetupInputs BloomSetupInputs;
			BloomSetupInputs.bUseBloom = bUseBloom;
			BloomSetupInputs.bUseDof = bUseMobileDof;
			BloomSetupInputs.bUseEyeAdaptation = bHasEyeAdaptationPass;
			BloomSetupInputs.bUseMetalMSAAHDRDecode = bMetalMSAAHDRDecode;
			BloomSetupInputs.bUseSun = bUseSun;
			BloomSetupInputs.SceneColor = SceneColor;
			BloomSetupInputs.SunShaftAndDof = PostProcessSunShaftAndDof;
			BloomSetupInputs.OutputFixedTextureSize = BloomOutputFixedTextureSize;

			BloomSetupOutputs = AddMobileBloomSetupPass(GraphBuilder, View, EyeAdaptationParameters, BloomSetupInputs);
		}

		if (PassSequence.IsEnabled(EPass::DepthOfField))
		{
			PassSequence.AcceptPass(EPass::DepthOfField);
			if (bUseMobileDof)
			{
				// Near dilation circle of confusion size.
				// Samples at 1/16 area, writes to 1/16 area.
				FMobileDofNearInputs DofNearInputs;
				DofNearInputs.BloomSetup_SunShaftAndDof = BloomSetupOutputs.SunShaftAndDof;
				DofNearInputs.bUseSun = bUseSun;

				FMobileDofNearOutputs DofNearOutputs = AddMobileDofNearPass(GraphBuilder, View, DofNearInputs);

				// DOF downsample pass.
				// Samples at full resolution, writes to 1/4 area.
				FMobileDofDownInputs DofDownInputs;
				DofDownInputs.bUseSun = bUseSun;
				DofDownInputs.DofNear = DofNearOutputs.DofNear;
				DofDownInputs.SceneColor = SceneColor;
				DofDownInputs.SunShaftAndDof = PostProcessSunShaftAndDof;

				FMobileDofDownOutputs DofDownOutputs = AddMobileDofDownPass(GraphBuilder, View, DofDownInputs);

				// DOF blur pass.
				// Samples at 1/4 area, writes to 1/4 area.
				FMobileDofBlurInputs DofBlurInputs;
				DofBlurInputs.DofDown = DofDownOutputs.DofDown;
				DofBlurInputs.DofNear = DofNearOutputs.DofNear;

				FMobileDofBlurOutputs DofBlurOutputs = AddMobileDofBlurPass(GraphBuilder, View, DofBlurInputs);

				DofOutput = DofBlurOutputs.DofBlur;

				FMobileIntegrateDofInputs IntegrateDofInputs;
				IntegrateDofInputs.DofBlur = DofBlurOutputs.DofBlur;
				IntegrateDofInputs.SceneColor = SceneColor;
				IntegrateDofInputs.SunShaftAndDof = PostProcessSunShaftAndDof;

				SceneColor = AddMobileIntegrateDofPass(GraphBuilder, View, IntegrateDofInputs);
			}
			else
			{
				bool bDepthOfField = IsGaussianActive(View);

				if (bDepthOfField)
				{
					float FarSize = View.FinalPostProcessSettings.DepthOfFieldFarBlurSize;
					float NearSize = View.FinalPostProcessSettings.DepthOfFieldNearBlurSize;
					const float MaxSize = CVarDepthOfFieldMaxSize.GetValueOnRenderThread();
					FarSize = FMath::Min(FarSize, MaxSize);
					NearSize = FMath::Min(NearSize, MaxSize);
					const bool bFar = FarSize >= 0.01f;
					const bool bNear = NearSize >= CVarDepthOfFieldNearBlurSizeThreshold.GetValueOnRenderThread();
					const bool bCombinedNearFarPass = bFar && bNear;

					if (bFar || bNear)
					{
						// AddGaussianDofBlurPass produces a blurred image from setup or potentially from taa result.
						auto AddGaussianDofBlurPass = [&GraphBuilder, &View](FScreenPassTexture& DOFSetup, bool bFarPass, float KernelSizePercent)
						{
							const TCHAR* BlurDebugX = bFarPass ? TEXT("FarDOFBlurX") : TEXT("NearDOFBlurX");
							const TCHAR* BlurDebugY = bFarPass ? TEXT("FarDOFBlurY") : TEXT("NearDOFBlurY");

							FGaussianBlurInputs GaussianBlurInputs;
							GaussianBlurInputs.NameX = BlurDebugX;
							GaussianBlurInputs.NameY = BlurDebugY;
							GaussianBlurInputs.Filter = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, DOFSetup);
							GaussianBlurInputs.TintColor = FLinearColor::White;
							GaussianBlurInputs.CrossCenterWeight = FVector2f::ZeroVector;
							GaussianBlurInputs.KernelSizePercent = KernelSizePercent;

							return AddGaussianBlurPass(GraphBuilder, View, GaussianBlurInputs);
						};

						FMobileDofSetupInputs DofSetupInputs;
						DofSetupInputs.bFarBlur = bFar;
						DofSetupInputs.bNearBlur = bNear;
						DofSetupInputs.SceneColor = SceneColor;
						DofSetupInputs.SunShaftAndDof = PostProcessSunShaftAndDof;
						FMobileDofSetupOutputs DofSetupOutputs = AddMobileDofSetupPass(GraphBuilder, View, DofSetupInputs);

						FScreenPassTexture DofFarBlur, DofNearBlur;
						if (bFar)
						{
							DofFarBlur = AddGaussianDofBlurPass(DofSetupOutputs.DofSetupFar, true, FarSize);
						}

						if (bNear)
						{
							DofNearBlur = AddGaussianDofBlurPass(DofSetupOutputs.DofSetupNear, false, NearSize);
						}

						FMobileDofRecombineInputs DofRecombineInputs;
						DofRecombineInputs.bFarBlur = bFar;
						DofRecombineInputs.bNearBlur = bNear;
						DofRecombineInputs.DofFarBlur = DofFarBlur;
						DofRecombineInputs.DofNearBlur = DofNearBlur;
						DofRecombineInputs.SceneColor = SceneColor;
						DofRecombineInputs.SunShaftAndDof = PostProcessSunShaftAndDof;

						SceneColor = AddMobileDofRecombinePass(GraphBuilder, View, DofRecombineInputs);
					}
				}
			}
		}

		// Bloom.
		FScreenPassTexture BloomUpOutputs;

		if (PassSequence.IsEnabled(EPass::Bloom))
		{
			PassSequence.AcceptPass(EPass::Bloom);
			auto AddBloomDownPass = [&GraphBuilder, &View](const FScreenPassTexture& BloomDownSource, float BloomDownScale, const FIntPoint& OutputTextureFixedExtent, const FIntPoint& OutputViewExtent)
			{
				FMobileBloomDownInputs BloomDownInputs;
				BloomDownInputs.BloomDownScale = BloomDownScale;
				BloomDownInputs.BloomDownSource = BloomDownSource;
				BloomDownInputs.BloomDownOutputFixedTextureExtent = OutputTextureFixedExtent;
				BloomDownInputs.BloomDownOutputViewExtent = OutputViewExtent;

				return AddMobileBloomDownPass(GraphBuilder, View, BloomDownInputs);
			};

			const float BloomDownScale = 0.66f * 4.0f;
			const int MaxPasses = 6;
			const EBloomQuality BloomQuality = GetBloomQuality();
			const int NumDownsamplePasses = BloomQuality == EBloomQuality::Q1 ? 4 : BloomQuality == EBloomQuality::Q2 ? 5 : 6;
			FScreenPassTexture PostProcessDownsample_Bloom[MaxPasses];

			// The bloom pyramid always steps by 2
			const int32 BloomPyramidStep = 2;
			int32 Divider = BloomPyramidStep;

			for (int32 i = 0; i < NumDownsamplePasses; ++i)
			{
				FIntPoint OutputTextureFixedExtent = FIntPoint::DivideAndRoundUp(BloomOutputFixedTextureSize, Divider);
				FIntPoint OutputViewExtent = FIntPoint::DivideAndRoundUp(BloomOutputTextureSize, Divider);
				Divider *= BloomPyramidStep;

				PostProcessDownsample_Bloom[i] = AddBloomDownPass(i == 0 ? BloomSetupOutputs.Bloom : PostProcessDownsample_Bloom[i - 1], BloomDownScale, OutputTextureFixedExtent, OutputViewExtent );
			}

			const FFinalPostProcessSettings& Settings = View.FinalPostProcessSettings;

			auto AddBloomUpPass = [&GraphBuilder, &View](FScreenPassTexture& BloomUpSourceA, FScreenPassTexture& BloomUpSourceB, float BloomSourceScale, const FVector4f& TintA, const FVector4f& TintB, const FIntPoint& OutputTextureFixedExtent)
			{
				FMobileBloomUpInputs BloomUpInputs;
				BloomUpInputs.BloomUpSourceA = BloomUpSourceA;
				BloomUpInputs.BloomUpSourceB = BloomUpSourceB;
				BloomUpInputs.ScaleAB = FVector2D(BloomSourceScale, BloomSourceScale);
				BloomUpInputs.TintA = TintA;
				BloomUpInputs.TintB = TintB;

				BloomUpInputs.BloomUpOutputFixedTextureExtent = OutputTextureFixedExtent;
				BloomUpInputs.BloomUpOutputViewExtent = BloomUpSourceA.ViewRect.Size();

				return AddMobileBloomUpPass(GraphBuilder, View, BloomUpInputs);
			};

			Divider /= BloomPyramidStep;
			const float BloomUpScale = 0.66f * 2.0f;
			const float BloomGaussianIntensity = Settings.BloomIntensity * Settings.BloomGaussianIntensity;

			const FLinearColor Tint[]{ Settings.Bloom1Tint, Settings.Bloom2Tint, Settings.Bloom3Tint, Settings.Bloom4Tint, Settings.Bloom5Tint, Settings.Bloom6Tint };

			// Upsample by 2
			{
				const int IndexA = NumDownsamplePasses - 2;
				const int IndexB = NumDownsamplePasses - 1;
				FVector4f TintA{ Tint[IndexA], 0.0f };
				FVector4f TintB{ Tint[IndexB], 0.0f };
				TintA *= BloomGaussianIntensity;
				TintB *= BloomGaussianIntensity;

				Divider /= BloomPyramidStep;
				BloomUpOutputs = AddBloomUpPass(PostProcessDownsample_Bloom[IndexA], PostProcessDownsample_Bloom[IndexB], BloomUpScale, TintA, TintB, FIntPoint::DivideAndRoundUp(BloomOutputFixedTextureSize, Divider));
			}

			for (int Index = NumDownsamplePasses - 3; Index > 0; --Index)
			{
				// Upsample by 2
				FVector4f TintA{ Tint[Index], 0.0f };
				TintA *= BloomGaussianIntensity;
				FVector4f TintB = FVector4f(1.0f, 1.0f, 1.0f, 0.0f);

				Divider /= BloomPyramidStep;
				BloomUpOutputs = AddBloomUpPass(PostProcessDownsample_Bloom[Index], BloomUpOutputs, BloomUpScale, TintA, TintB, FIntPoint::DivideAndRoundUp(BloomOutputFixedTextureSize, Divider));
			}

			// Upsample by 2
			{
				FVector4f TintA = FVector4f(Settings.Bloom2Tint.R, Settings.Bloom2Tint.G, Settings.Bloom2Tint.B, 0.0f);
				TintA *= BloomGaussianIntensity;
				// Scaling Bloom2 by extra factor to match filter area difference between PC default and mobile.
				TintA *= 0.5;
				FVector4f TintB = FVector4f(1.0f, 1.0f, 1.0f, 0.0f);

				Divider /= BloomPyramidStep;
				BloomUpOutputs = AddBloomUpPass(PostProcessDownsample_Bloom[0], BloomUpOutputs, BloomUpScale, TintA, TintB, FIntPoint::DivideAndRoundUp(BloomOutputFixedTextureSize, Divider));
			}

			if (IsLensFlaresEnabled(View))
			{
				const ELensFlareQuality LensFlareQuality = GetLensFlareQuality();
				const uint32 LensFlareDownsampleStageIndex = static_cast<uint32>(ELensFlareQuality::MAX) - static_cast<uint32>(LensFlareQuality) - 1;
				BloomUpOutputs = AddLensFlaresPass(GraphBuilder, View, BloomUpOutputs,
					FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, PostProcessDownsample_Bloom[LensFlareDownsampleStageIndex]),
					FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, PostProcessDownsample_Bloom[0]));
			}
		}

		if (PassSequence.IsEnabled(EPass::EyeAdaptation))
		{
			PassSequence.AcceptPass(EPass::EyeAdaptation);
			FMobileEyeAdaptationSetupInputs EyeAdaptationSetupInputs;
			
			EyeAdaptationSetupInputs.bUseBasicEyeAdaptation = bUseBasicEyeAdaptation;
			EyeAdaptationSetupInputs.bUseHistogramEyeAdaptation = bUseHistogramEyeAdaptation;
			// AEM_Basic doesn't produce a histogram; ask for one when the VisualizeHDR overlay is on.
			EyeAdaptationSetupInputs.bGenerateHistogramForVisualize = bVisualizeHDR && bUseBasicEyeAdaptation;
			EyeAdaptationSetupInputs.BloomSetup_EyeAdaptation = FScreenPassTexture(TryRegisterExternalTexture(GraphBuilder, View.PrevViewInfo.MobileBloomSetup_EyeAdaptation), View.PrevViewInfo.MobileBloomSetup_EyeAdaptationViewSize);
			if (!EyeAdaptationSetupInputs.BloomSetup_EyeAdaptation.IsValid())
			{
				EyeAdaptationSetupInputs.BloomSetup_EyeAdaptation = BloomSetupOutputs.EyeAdaptation;
			}

			FMobileEyeAdaptationSetupOutputs EyeAdaptationSetupOutputs = AddMobileEyeAdaptationSetupPass(GraphBuilder, View, EyeAdaptationParameters, EyeAdaptationSetupInputs);

			// Buffer for VisualizeHDR.
			VisualizeHDRHistogramSRV = EyeAdaptationSetupOutputs.VisualizeHDRHistogramSRV;

			FMobileEyeAdaptationInputs EyeAdaptationInputs;
			EyeAdaptationInputs.bUseBasicEyeAdaptation = bUseBasicEyeAdaptation;
			EyeAdaptationInputs.bUseHistogramEyeAdaptation = bUseHistogramEyeAdaptation;
			EyeAdaptationInputs.EyeAdaptationSetupSRV = EyeAdaptationSetupOutputs.EyeAdaptationSetupSRV;
			EyeAdaptationInputs.EyeAdaptationBuffer = LastEyeAdaptationBuffer;

			AddMobileEyeAdaptationPass(GraphBuilder, View, EyeAdaptationParameters, EyeAdaptationInputs);

			if ((bUseBasicEyeAdaptation || bUseHistogramEyeAdaptation) && View.ViewState && !View.bStatePrevViewInfoIsReadOnly)
			{
				GraphBuilder.QueueTextureExtraction(BloomSetupOutputs.EyeAdaptation.Texture, &View.ViewState->PrevFrameViewInfo.MobileBloomSetup_EyeAdaptation);
				View.ViewState->PrevFrameViewInfo.MobileBloomSetup_EyeAdaptationViewSize = BloomSetupOutputs.EyeAdaptation.ViewRect;
			}
		}

		if (PassSequence.IsEnabled(EPass::SunMerge))
		{
			PassSequence.AcceptPass(EPass::SunMerge);
			FScreenPassTexture SunBlurOutputs;
			
			if (bUseSun)
			{
				FMobileSunAlphaInputs SunAlphaInputs;
				SunAlphaInputs.BloomSetup_SunShaftAndDof = BloomSetupOutputs.SunShaftAndDof;
				SunAlphaInputs.bUseMobileDof = bUseMobileDof;

				FScreenPassTexture SunAlphaOutputs = AddMobileSunAlphaPass(GraphBuilder, View, SunAlphaInputs);

				FMobileSunBlurInputs SunBlurInputs;
				SunBlurInputs.SunAlpha = SunAlphaOutputs;

				SunBlurOutputs = AddMobileSunBlurPass(GraphBuilder, View, SunBlurInputs);
			}

			FMobileSunMergeInputs SunMergeInputs;
			SunMergeInputs.BloomSetup_Bloom = BloomSetupOutputs.Bloom;
			SunMergeInputs.BloomUp = BloomUpOutputs;
			SunMergeInputs.SunBlur = SunBlurOutputs;
			SunMergeInputs.bUseBloom = bUseBloom;
			SunMergeInputs.bUseSun = bUseSun;

			BloomOutput = AddMobileSunMergePass(GraphBuilder, View, SunMergeInputs);
		}

		// mobile separate translucency 
		if (PassSequence.IsEnabled(EPass::SeparateTranslucency))
		{
			PassSequence.AcceptPass(EPass::SeparateTranslucency);
			FMobileSeparateTranslucencyInputs SeparateTranslucencyInputs;
			SeparateTranslucencyInputs.SceneColor = SceneColor;
			SeparateTranslucencyInputs.SceneDepthAux = SceneDepthAux;
			SeparateTranslucencyInputs.SceneDepth = SceneDepth;
			
			AddMobileSeparateTranslucencyPass(GraphBuilder, Scene, View, SeparateTranslucencyInputs);
		}

		if (SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::AfterDOF)].Num())
		{
			FScreenPassTextureSlice SceneColorSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
			SceneColor = AddSceneViewExtensionPassChain(GraphBuilder, View, GetPostProcessMaterialInputs(SceneColorSlice),
				SceneViewExtensionDelegates[static_cast<uint32>(ISceneViewExtension::EPostProcessingPass::AfterDOF)]);
		}

		AddPostProcessMaterialPass(BL_SceneColorAfterDOF);

		{
			FPostProcessMaterialInputs PostProcessMaterialInputs = GetPostProcessMaterialInputs(FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor));
			AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_SceneColorAfterDOF, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
		}

		// Temporal Anti-aliasing. Also may perform a temporal upsample from primary to secondary view rect.
		if (PassSequence.IsEnabled(EPass::TAA))
		{
			PassSequence.AcceptPass(EPass::TAA);

			EMainTAAPassConfig TAAConfig = GetMainTAAPassConfig(View);
			checkSlow(TAAConfig != EMainTAAPassConfig::Disabled);

			FDefaultTemporalUpscaler::FInputs UpscalerPassInputs{};
			UpscalerPassInputs.SceneColor = FScreenPassTexture(SceneColor.Texture, View.ViewRect);
			UpscalerPassInputs.SceneDepth = FScreenPassTexture(SceneDepth.Texture, View.ViewRect);
			UpscalerPassInputs.SceneVelocity = FScreenPassTexture(Velocity.Texture, View.ViewRect);

			FDefaultTemporalUpscaler::FOutputs Outputs;
			if (TAAConfig == EMainTAAPassConfig::TAA)
			{
				Outputs = AddGen4MainTemporalAAPasses(
					GraphBuilder,
					View,
					UpscalerPassInputs);
			}
			else if (TAAConfig == EMainTAAPassConfig::ThirdParty)
			{
				Outputs = AddThirdPartyTemporalUpscalerPasses(
					GraphBuilder,
					View,
					UpscalerPassInputs);
			}
			else
			{
				unimplemented();
			}
			SceneColor = FScreenPassTexture(Outputs.FullRes);
		}
		else if (IsMobileSSREnabled(View))
		{
			// If we need SSR, and TAA is enabled, then AddTemporalAAPass() has already handled the scene history.
			// If we need SSR, and TAA is not enabled, then we just need to extract the history.
			if (!View.bStatePrevViewInfoIsReadOnly)
			{
				check(View.ViewState);
				FTemporalAAHistory& OutputHistory = View.ViewState->PrevFrameViewInfo.TemporalAAHistory;
				GraphBuilder.QueueTextureExtraction(SceneColor.Texture, &OutputHistory.RT[0]);

				// For SSR, we still fill up the rest of the OutputHistory data using shared math from FTAAPassParameters.
				FTAAPassParameters TAAInputs(View);
				TAAInputs.SceneColorInput = SceneColor.Texture;
				TAAInputs.SetupViewRect(View);
				OutputHistory.ViewportRect = TAAInputs.OutputViewRect;
				OutputHistory.ReferenceBufferSize = TAAInputs.GetOutputExtent() * TAAInputs.ResolutionDivisor;
			}
		}
	}
	else
	{
		PassSequence.SetEnabled(EPass::Distortion, false);
		PassSequence.SetEnabled(EPass::SunMask, false);
		PassSequence.SetEnabled(EPass::LocalExposure, false);
		PassSequence.SetEnabled(EPass::BloomSetup, false);
		PassSequence.SetEnabled(EPass::DepthOfField, false);
		PassSequence.SetEnabled(EPass::Bloom, false);
		PassSequence.SetEnabled(EPass::EyeAdaptation, false);
		PassSequence.SetEnabled(EPass::SunMerge, false);
		PassSequence.SetEnabled(EPass::SeparateTranslucency, false);
		PassSequence.SetEnabled(EPass::TAA, false);
		PassSequence.SetEnabled(EPass::FXAA, false);
		PassSequence.SetEnabled(EPass::SMAA, false);
		PassSequence.SetEnabled(EPass::PostProcessMaterialAfterTonemapping, false);
		PassSequence.SetEnabled(EPass::VisualizeHDR, false);
		PassSequence.Finalize();
	}

	AddPostProcessMaterialPass(BL_SceneColorBeforeBloom);

	{
		FPostProcessMaterialInputs PostProcessMaterialInputs = GetPostProcessMaterialInputs(FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor));
		AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_SceneColorBeforeBloom, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
	}

	FScreenPassTextureSlice SceneColorBeforeTonemapSlice = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
	
	if (PassSequence.IsEnabled(EPass::Tonemap))
	{
		const FPostProcessMaterialChain MaterialChain = GetPostProcessMaterialChain(View, BL_ReplacingTonemapper);

		if (MaterialChain.Num())
		{
			AddPostProcessMaterialPass(BL_ReplacingTonemapper);
		}
		else
		{
			bool bHDRTonemapperOutput = false;

			if (!BloomOutput.IsValid())
			{
				BloomOutput = BlackAlphaOneDummy;
			}

			bool bDoGammaOnly = !IsMobileHDR();

			FRDGTextureRef ColorGradingTexture = nullptr;

			if (IStereoRendering::IsAPrimaryView(View) && !bDoGammaOnly)
			{
				ColorGradingTexture = AddCombineLUTPass(GraphBuilder, View);
			}
			// We can re-use the color grading texture from the primary view.
			else if (View.GetTonemappingLUT())
			{
				ColorGradingTexture = TryRegisterExternalTexture(GraphBuilder, View.GetTonemappingLUT());
			}
			else
			{
				const FViewInfo* PrimaryView = static_cast<const FViewInfo*>(View.Family->Views[0]);
				ColorGradingTexture = TryRegisterExternalTexture(GraphBuilder, PrimaryView->GetTonemappingLUT());
			}

			FTonemapInputs TonemapperInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::Tonemap, TonemapperInputs.OverrideOutput);

			// This is the view family render target.
			if (TonemapperInputs.OverrideOutput.Texture)
			{
				FIntRect OutputViewRect;
				if (View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::RawOutput)
				{
					OutputViewRect = View.ViewRect;
				}
				else
				{
					OutputViewRect = View.UnscaledViewRect;
				}

				TonemapperInputs.OverrideOutput.ViewRect = OutputViewRect;
				TonemapperInputs.OverrideOutput.UpdateVisualizeTextureExtent();
			}

			TonemapperInputs.SceneColor = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor);
			TonemapperInputs.Bloom = BloomOutput;
			TonemapperInputs.EyeAdaptationParameters = &EyeAdaptationParameters;
			TonemapperInputs.ColorGradingTexture = ColorGradingTexture;
			TonemapperInputs.bWriteAlphaChannel = View.AntiAliasingMethod == AAM_FXAA || View.AntiAliasingMethod == AAM_SMAA || IsPostProcessingWithAlphaChannelSupported() || bUseMobileDof || IsMobilePropagateAlphaEnabled(View.GetShaderPlatform());
			TonemapperInputs.bOutputInHDR = bHDRTonemapperOutput;
			TonemapperInputs.bGammaOnly = bDoGammaOnly;
			TonemapperInputs.bMetalMSAAHDRDecode = bMetalMSAAHDRDecode;
			TonemapperInputs.EyeAdaptationBuffer = PassSequence.IsEnabled(EPass::EyeAdaptation) ? LastEyeAdaptationBuffer : nullptr;

			// Local exposure
			TonemapperInputs.LocalExposureBilateralGridTexture = LocalExposureBilateralGridTexture;
			TonemapperInputs.BlurredLogLuminanceTexture = LocalExposureBlurredLogLumTexture;
			TonemapperInputs.ExposureFusion = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, ExposureFusionData.Result);
			TonemapperInputs.LocalExposureParameters = &LocalExposureParameters;

			SceneColor = AddTonemapPass(GraphBuilder, View, TonemapperInputs);
		}

		//The output color should been decoded to linear space after tone mapper apparently
		bMetalMSAAHDRDecode = false;
	}

	SceneColor = AddAfterPass(EPass::Tonemap, SceneColor);

	if (IsPostProcessingEnabled(View))
	{
		if (PassSequence.IsEnabled(EPass::FXAA))
		{
			FFXAAInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::FXAA, PassInputs.OverrideOutput);
			PassInputs.SceneColor = SceneColor;
			PassInputs.Quality = GetFXAAQuality();
			
			SceneColor = AddFXAAPass(GraphBuilder, View, PassInputs);
		}

		SceneColor = AddAfterPass(EPass::FXAA, SceneColor);

		if (PassSequence.IsEnabled(EPass::SMAA))
		{
			FSMAAInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::SMAA, PassInputs.OverrideOutput);
			PassInputs.SceneColor = SceneColor;
			PassInputs.Quality = GetSMAAQuality();

			const uint32 MultiViewCount = View.GetMultiViewCount(); // SMAA can support MMV
			PassInputs.SceneColorBeforeTonemap = FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorBeforeTonemapSlice, FScreenPassTexture(), MultiViewCount);

			SceneColor = AddSMAAPasses(GraphBuilder, View, PassInputs);
		}

		SceneColor = AddAfterPass(EPass::SMAA, SceneColor);

		if (PassSequence.IsEnabled(EPass::PostProcessMaterialAfterTonemapping))
		{
			AddPostProcessMaterialPass(BL_SceneColorAfterTonemapping);
		}

		{
			FPostProcessMaterialInputs PostProcessMaterialInputs = GetPostProcessMaterialInputs(FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, SceneColor));
			AddVisualizeBufferMaterialPass(GraphBuilder, View, BL_SceneColorAfterTonemapping, PostProcessMaterialInputs, bOutputInHDR, VisualizeBufferContext);
		}

		if (PassSequence.IsEnabled(EPass::VisualizeHDR))
		{
			FVisualizeHDRInputs PassInputs;
			PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeHDR, PassInputs.OverrideOutput);
			PassInputs.SceneColor = SceneColor;
			PassInputs.SceneColorBeforeTonemap = FScreenPassTexture::CopyFromSlice(GraphBuilder, SceneColorBeforeTonemapSlice);
			PassInputs.HistogramBuffer = VisualizeHDRHistogramSRV;
			PassInputs.EyeAdaptationBuffer = LastEyeAdaptationBuffer;
			PassInputs.EyeAdaptationParameters = &EyeAdaptationParameters;
			SceneColor = AddVisualizeHDRPass(GraphBuilder, View, PassInputs);
		}

	}

	if (PassSequence.IsEnabled(EPass::HighResolutionScreenshotMask))
	{
		FHighResolutionScreenshotMaskInputs HighResolutionScreenshotMaskInputs;
		HighResolutionScreenshotMaskInputs.SceneColor = SceneColor;
		HighResolutionScreenshotMaskInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);
		HighResolutionScreenshotMaskInputs.Material = View.FinalPostProcessSettings.HighResScreenshotMaterial;
		HighResolutionScreenshotMaskInputs.MaskMaterial = View.FinalPostProcessSettings.HighResScreenshotMaskMaterial;
		HighResolutionScreenshotMaskInputs.CaptureRegionMaterial = View.FinalPostProcessSettings.HighResScreenshotCaptureRegionMaterial;
		PassSequence.AcceptOverrideIfLastPass(EPass::HighResolutionScreenshotMask, HighResolutionScreenshotMaskInputs.OverrideOutput);

		SceneColor = AddHighResolutionScreenshotMaskPass(GraphBuilder, View, HighResolutionScreenshotMaskInputs);
	}

#if WITH_EDITOR

	if (PassSequence.IsEnabled(EPass::VisualizeSkyLightIlluminanceMeter))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeSkyLightIlluminanceMeter, OverrideOutput);
		SceneColor = ProcessAndRenderIlluminanceMeter(GraphBuilder, View, SceneColor);
	}
#endif

	if (PassSequence.IsEnabled(EPass::VisualizeGBuffer))
	{
		FVisualizeGBufferInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeGBuffer, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.bOutputInHDR = bOutputInHDR;

		SceneColor = AddVisualizeGBufferPass(GraphBuilder, View, PassInputs, VisualizeBufferContext);
	}

#if WITH_EDITOR
	if (PassSequence.IsEnabled(EPass::ChannelMask))
	{
		ChannelMask::FChannelMaskInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::ChannelMask, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		SceneColor = ChannelMask::AddChannelMaskPass(GraphBuilder, View, PassInputs);
	}
	
	// Show the selection outline if it is in the editor and we aren't in wireframe 
	// If the engine is in demo mode and game view is on we also do not show the selection outline
	if (PassSequence.IsEnabled(EPass::SelectionOutline))
	{
		FSelectionOutlineInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::SelectionOutline, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;
		PassInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);

		// TODO: Nanite - pipe through results
		FRDGTextureRef DummyStencilTexture = nullptr;
		SceneColor = AddSelectionOutlinePass(GraphBuilder, View, SceneUniformBuffer, PassInputs, nullptr, DummyStencilTexture);
	}

	if (PassSequence.IsEnabled(EPass::EditorPrimitive))
	{
		FCompositePrimitiveInputs PassInputs;
		if (PassSequence.AcceptOverrideIfLastPass(EPass::EditorPrimitive, PassInputs.OverrideOutput))
		{
			PassInputs.OverrideDepthOutput = ViewFamilyDepthOutput;
		}
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;
		PassInputs.BasePassType = FCompositePrimitiveInputs::EBasePassType::Mobile;

		SceneColor = AddEditorPrimitivePass(GraphBuilder, View, PassInputs, InstanceCullingManager);
	}
#endif


#if UE_ENABLE_DEBUG_DRAWING
	if (PassSequence.IsEnabled(EPass::DebugPrimitive)) //Create new debug pass sequence
	{
		FCompositePrimitiveInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::DebugPrimitive, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.SceneDepth = SceneDepth;
		PassInputs.bUseMetalMSAAHDRDecode = bMetalMSAAHDRDecode;

		SceneColor = AddDebugPrimitivePass(GraphBuilder, View, ViewIndex, SceneUniformBuffer, nullptr, PassInputs);
	}
#endif

	// Apply ScreenPercentage
	if (PassSequence.IsEnabled(EPass::PrimaryUpscale))
	{
		ISpatialUpscaler::FInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::PrimaryUpscale, PassInputs.OverrideOutput);
		PassInputs.Stage = PassSequence.IsEnabled(EPass::SecondaryUpscale) ? EUpscaleStage::PrimaryToSecondary : EUpscaleStage::PrimaryToOutput;
		PassInputs.SceneColor = SceneColor;

		if (const ISpatialUpscaler* CustomUpscaler = View.Family->GetPrimarySpatialUpscalerInterface())
		{
			RDG_EVENT_SCOPE(
				GraphBuilder,
				"ThirdParty PrimaryUpscale %s %dx%d -> %dx%d",
				CustomUpscaler->GetDebugName(),
				SceneColor.ViewRect.Width(), SceneColor.ViewRect.Height(),
				View.UnscaledViewRect.Width(), View.UnscaledViewRect.Height());

			SceneColor = CustomUpscaler->AddPasses(GraphBuilder, View, PassInputs);

			if (PassSequence.IsLastPass(EPass::PrimaryUpscale))
			{
				check(SceneColor == ViewFamilyOutput);
			}
			else
			{
				check(SceneColor.ViewRect.Size() == View.GetSecondaryViewRectSize());
			}
		}
		else
		{
			EUpscaleMethod Method = GetMobileUpscaleMethod();

			SceneColor = ISpatialUpscaler::AddDefaultUpscalePass(GraphBuilder, View, PassInputs, Method, View.LensDistortionLUT);
		}
	}

	if (PassSequence.IsEnabled(EPass::SecondaryUpscale))
	{
		ISpatialUpscaler::FInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::SecondaryUpscale, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;
		PassInputs.Stage = EUpscaleStage::SecondaryToOutput;

		const ISpatialUpscaler* CustomUpscaler = View.Family->GetSecondarySpatialUpscalerInterface();
		if (CustomUpscaler)
		{
			RDG_EVENT_SCOPE(
				GraphBuilder,
				"ThirdParty SecondaryUpscale %s %dx%d -> %dx%d",
				CustomUpscaler->GetDebugName(),
				SceneColor.ViewRect.Width(), SceneColor.ViewRect.Height(),
				View.UnscaledViewRect.Width(), View.UnscaledViewRect.Height());

			SceneColor = CustomUpscaler->AddPasses(GraphBuilder, View, PassInputs);

			if (PassSequence.IsLastPass(EPass::SecondaryUpscale))
			{
				check(SceneColor == ViewFamilyOutput);
			}
			else
			{
				check(SceneColor.ViewRect.Size() == View.UnscaledViewRect.Size());
			}
		}
	}

	if (PassSequence.IsEnabled(EPass::Visualize))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::Visualize, OverrideOutput);

		switch (View.Family->GetDebugViewShaderMode())
		{
		case DVSM_QuadComplexity:
		{
			float ComplexityScale = 1.f / (float)(GEngine->QuadComplexityColors.Num() - 1) / NormalizedQuadComplexityValue; // .1f comes from the values used in LightAccumulator_GetResult

			FVisualizeComplexityInputs PassInputs;
			PassInputs.OverrideOutput = OverrideOutput;
			PassInputs.SceneColor = SceneColor;
			PassInputs.Colors = GEngine->QuadComplexityColors;
			PassInputs.ColorSamplingMethod = FVisualizeComplexityInputs::EColorSamplingMethod::Stair;
			PassInputs.ComplexityScale = ComplexityScale;
			PassInputs.bDrawLegend = true;

			SceneColor = AddVisualizeComplexityPass(GraphBuilder, View, PassInputs);
			break;
		}
		case DVSM_ShaderComplexity:
		case DVSM_ShaderComplexityContainedQuadOverhead:
		case DVSM_ShaderComplexityBleedingQuadOverhead:
		{
			FVisualizeComplexityInputs PassInputs;
			PassInputs.OverrideOutput = OverrideOutput;
			PassInputs.SceneColor = SceneColor;
			PassInputs.Colors = GEngine->ShaderComplexityColors;
			PassInputs.ColorSamplingMethod = FVisualizeComplexityInputs::EColorSamplingMethod::Ramp;
			PassInputs.ComplexityScale = 1.0f;
			PassInputs.bDrawLegend = true;

			SceneColor = AddVisualizeComplexityPass(GraphBuilder, View, PassInputs);
			break;
		}
		default:
			ensure(false);
			break;
		}
	}

	if (PassSequence.IsEnabled(EPass::VisualizeLightGrid))
	{
		FScreenPassRenderTarget OverrideOutput;
		PassSequence.AcceptOverrideIfLastPass(EPass::VisualizeLightGrid, OverrideOutput);
		SceneColor = AddVisualizeLightGridPass(GraphBuilder, View, OverrideOutput.IsValid() ? OverrideOutput : SceneColor, SceneDepth);
	}

	if (ShaderPrint::IsEnabled(View.ShaderPrintData))
	{
		ShaderPrint::DrawView(GraphBuilder, View, SceneColor, SceneDepth);
	}

	if (PassSequence.IsEnabled(EPass::HMDDistortion))
	{
		FHMDDistortionInputs PassInputs;
		PassSequence.AcceptOverrideIfLastPass(EPass::HMDDistortion, PassInputs.OverrideOutput);
		PassInputs.SceneColor = SceneColor;

		SceneColor = AddHMDDistortionPass(GraphBuilder, View, PassInputs);
	}

#if !UE_BUILD_SHIPPING
	AddUserSceneTextureDebugPass(GraphBuilder, View, ViewIndex, SceneColor);
#endif

	// Copy the scene color to back buffer in case there is no post process, such as LDR MSAA.
	if (SceneColor.Texture != ViewFamilyOutput.Texture)
	{
		const uint32 RTMultiViewCount = View.GetMultiViewCount();
		AddDrawTexturePass(GraphBuilder, View, SceneColor, ViewFamilyOutput, RTMultiViewCount);
	}
}

void AddRuntimeReflectionCapturePostPass(FRDGBuilder& GraphBuilder, FViewFamilyInfo& ViewFamily, FRDGTextureRef ViewFamilyTexture, FRDGTextureRef ViewFamilyDepthTexture, TConstArrayView<FViewInfo> Views)
{
	// We are assuming this logic replaces regular post processing -- make sure that won't be running later.
	check(ViewFamily.bResolveScene == false);

	FMobilePostProcessingInputs Inputs;
	Inputs.ViewFamilyTexture = ViewFamilyTexture;
	Inputs.ViewFamilyDepthTexture = ViewFamilyDepthTexture;
	Inputs.SceneTextures = CreateMobileSceneTextureUniformBuffer(GraphBuilder, &ViewFamily.GetSceneTextures(), EMobileSceneTextureSetupMode::All);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		// Runtime reflection captures apply only BL_SceneColorBeforeBloom post process materials.
		const FPostProcessMaterialChain MaterialChain = GetPostProcessMaterialChain(View, BL_SceneColorBeforeBloom);

		if (MaterialChain.Num())
		{
			const FScreenPassRenderTarget ViewFamilyOutput = FScreenPassRenderTarget::CreateViewFamilyOutput(Inputs.ViewFamilyTexture, View);
			const FScreenPassTexture CustomDepth((*Inputs.SceneTextures)->CustomDepthTexture, View.ViewRect);
			FScreenPassTexture SceneColor((*Inputs.SceneTextures)->SceneColorTexture, View.ViewRect);

			FPostProcessMaterialInputs PostProcessMaterialInputs;
			PostProcessMaterialInputs.SetInput(GraphBuilder, EPostProcessMaterialInput::SceneColor, SceneColor);
			PostProcessMaterialInputs.CustomDepthTexture = CustomDepth.Texture;
			PostProcessMaterialInputs.SceneTextures = GetSceneTextureShaderParameters(Inputs.SceneTextures);
			SceneColor = AddPostProcessMaterialChain(GraphBuilder, View, ViewIndex, PostProcessMaterialInputs, MaterialChain);

			// Copy the resulting post processed texture to the output.  Note that it doesn't work to directly write to ViewFamilyOutput via
			// PostProcessMaterialInputs.OverrideOutput, because the output rect generally won't be the same as the scene texture rect where
			// rendering occurred, as those are compacted to the upper left corner.  Various SvPosition related math gets messed up.  You
			// can't fix up the view uniforms to point to the final output rect either, because it messes up the math for certain inputs.
			// A bunch of stuff would need to be cleaned up to make the math consistent, but this works fine, and the texture in question is
			// low resolution, and cheap to copy.
			AddDrawTexturePass(GraphBuilder, View, SceneColor, ViewFamilyOutput);

			// Override the SceneCaptureSource to communicate that final color has been written by a post pass.
			// This prevents CopySceneCaptureComponentToTarget from overwriting it with SceneColor later.
			ViewFamily.SceneCaptureSource = SCS_FinalColorHDR;
		}
	}
}

FRDGTextureRef AddProcessPlanarReflectionPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture)
{
	FSceneViewState* ViewState = View.ViewState;
	const EAntiAliasingMethod AntiAliasingMethod = View.AntiAliasingMethod;

	if (IsTemporalAccumulationBasedMethod(AntiAliasingMethod))
	{
		check(ViewState);

		FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, View);

		const FTemporalAAHistory& InputHistory = View.PrevViewInfo.TemporalAAHistory;
		FTemporalAAHistory* OutputHistory = &ViewState->PrevFrameViewInfo.TemporalAAHistory;

		FTAAPassParameters Parameters(View);
		Parameters.SceneDepthTexture = SceneTextures.SceneDepthTexture;

		// Planar reflections don't support velocity.
		Parameters.SceneVelocityTexture = nullptr;

		Parameters.SceneColorInput = SceneColorTexture;

		FTAAOutputs PassOutputs = AddTemporalAAPass(
			GraphBuilder,
			View,
			Parameters,
			InputHistory,
			OutputHistory);

		return PassOutputs.SceneColor;
	}
	else
	{
		return SceneColorTexture;
	}
}

#if DEBUG_POST_PROCESS_VOLUME_ENABLE
FScreenPassTexture AddFinalPostProcessDebugInfoPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassTexture& ScreenPassSceneColor, FScreenPassRenderTarget& OverrideOutput)
{
	RDG_EVENT_SCOPE(GraphBuilder, "FinalPostProcessDebugInfo");

	FScreenPassRenderTarget Output;
	if (OverrideOutput.IsValid())
	{
		Output = OverrideOutput;
		const FIntPoint MaxSize = Output.Texture->Desc.Extent - OverrideOutput.ViewRect.Min;
		const FIntPoint Size{ FMath::Min(ScreenPassSceneColor.ViewRect.Size().X, MaxSize.X), FMath::Min(ScreenPassSceneColor.ViewRect.Size().Y, MaxSize.Y) };
		AddCopyTexturePass(GraphBuilder, ScreenPassSceneColor.Texture, Output.Texture, ScreenPassSceneColor.ViewRect.Min, OverrideOutput.ViewRect.Min, Size);
	}
	else
	{
		Output = FScreenPassRenderTarget(ScreenPassSceneColor, ERenderTargetLoadAction::ELoad);
	}

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("PostProcessDebug"), View, Output,
		[&View](FCanvas& Canvas)
		{
			const float DPIScale = Canvas.GetDPIScale();
			Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale) * Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

			FLinearColor TextColor(FLinearColor::White);
			FLinearColor GrayTextColor(FLinearColor::Gray);
			FLinearColor GreenTextColor(FLinearColor::Green);
			FString Text;

			const float ViewPortWidth = float(Canvas.GetViewRect().Width());
			const float ViewPortHeight = float(Canvas.GetViewRect().Height());

			const float CRHeight = 20.0f;
			const float PrintX_CR = ViewPortWidth * 0.1f;

			float PrintX = PrintX_CR;
			float PrintY = ViewPortHeight * 0.2f;

			Text = FString::Printf(TEXT("Post-processing volume debug (count = %i)"), View.FinalPostProcessDebugInfo.Num());
			Canvas.DrawShadowedString(PrintX, PrintY, *Text, GetStatsFont(), GreenTextColor);					PrintX = PrintX_CR; PrintY += CRHeight * 1.5;

			Canvas.DrawShadowedString(PrintX, PrintY, *FString("Name"), GetStatsFont(), GrayTextColor);				PrintX += 256.0f;
			Canvas.DrawShadowedString(PrintX, PrintY, *FString("IsEnabled"), GetStatsFont(), GrayTextColor);		PrintX += 96.0f;
			Canvas.DrawShadowedString(PrintX, PrintY, *FString("Priority"), GetStatsFont(), GrayTextColor);			PrintX += 96.0f;
			Canvas.DrawShadowedString(PrintX, PrintY, *FString("CurrentWeight"), GetStatsFont(), GrayTextColor);	PrintX += 96.0f;
			Canvas.DrawShadowedString(PrintX, PrintY, *FString("bIsUnbound"), GetStatsFont(), GrayTextColor);		PrintX += 96.0f;

			if (View.PostProcessDebugPropertyName != NAME_None)
			{
				FString Name = View.PostProcessDebugPropertyName.ToString();
				Canvas.DrawShadowedString(PrintX, PrintY, *Name, GetStatsFont(), GrayTextColor);		PrintX += 96.0f;
			}
			
			PrintY += CRHeight;
			PrintX = PrintX_CR;

			const int32 PPDebugInfoCount = View.FinalPostProcessDebugInfo.Num() - 1;
			for (int32 i = PPDebugInfoCount; i >= 0 ; --i)
			{
				const FPostProcessSettingsDebugInfo& PPDebugInfo = View.FinalPostProcessDebugInfo[i];

				Text = FString::Printf(TEXT("%s"), *PPDebugInfo.Name.Left(40)); // Clamp the name to a reasonable length
				Canvas.DrawShadowedString(PrintX, PrintY, *Text, GetStatsFont(), TextColor); PrintX += 256.0f;

				Text = FString::Printf(TEXT("%d"), PPDebugInfo.bIsEnabled ? 1 : 0);
				Canvas.DrawShadowedString(PrintX+32.0f, PrintY, *Text, GetStatsFont(), TextColor); PrintX += 96.0f;

				Text = FString::Printf(TEXT("%.3f"), PPDebugInfo.Priority);
				Canvas.DrawShadowedString(PrintX, PrintY, *Text, GetStatsFont(), TextColor); PrintX += 96.0f;

				Text = FString::Printf(TEXT("%3.3f"), PPDebugInfo.CurrentBlendWeight);
				Canvas.DrawShadowedString(PrintX+32.0f, PrintY, *Text, GetStatsFont(), TextColor); PrintX += 96.0f;

				Text = FString::Printf(TEXT("%d"), PPDebugInfo.bIsUnbound ? 1 : 0);
				Canvas.DrawShadowedString(PrintX+32.0f, PrintY, *Text, GetStatsFont(), TextColor); PrintX += 96.0f;

				if (View.PostProcessDebugPropertyName != NAME_None)
				{
					Text = FString::Printf(TEXT("%s"), *PPDebugInfo.DebugPropertyValue.Left(256));
					Canvas.DrawShadowedString(PrintX, PrintY, *Text, GetStatsFont(), TextColor); PrintX += 256.0f;
				}

				Canvas.DrawShadowedString(PrintX_CR, PrintY+3.0f, *FString("______________________________________________________________________________________________________________"), GetStatsFont(), TextColor);

				PrintX = PrintX_CR;
				PrintY += CRHeight;
			}

			PrintY += CRHeight;
			if (View.PostProcessDebugPropertyName == NAME_None)
			{
				Canvas.DrawShadowedString(PrintX, PrintY, TEXT("Use r.PostProcessing.Debug.Property <SomePropertyName> to visualize property values"), GetStatsFont(), GrayTextColor);
			}
		});

	return MoveTemp(Output);
}
#endif

#if !UE_BUILD_SHIPPING

// Canvas.DrawShadowedString returns height -- this variation returns width
static float CanvasDrawShadowedStringReturnWidth(FCanvas& Canvas, float PrintX, float PrintY, const FString& Text, const UFont* Font, FLinearColor TextColor)
{
	FCanvasTextStringViewItem TextItem(FVector2D(PrintX, PrintY), Text, Font, TextColor);
	if (Font && Font->ImportOptions.bUseDistanceFieldAlpha)
	{
		TextItem.BlendMode = SE_BLEND_MaskedDistanceFieldShadowed;
	}
	else
	{
		TextItem.EnableShadow(FLinearColor::Black);
	}
	Canvas.DrawItem(TextItem);
	return TextItem.DrawnSize.X / Canvas.GetDPIScale();
}

static void AddUserSceneTextureDebugPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 ViewIndex, FScreenPassTexture Output)
{
	int32 UserSceneTextureDebug = CVarUserSceneTextureDebug.GetValueOnRenderThread();
	bool bEnableUserSceneTextureDebug = false;

	if (UserSceneTextureDebug == 1)
	{
		// Enable always
		bEnableUserSceneTextureDebug = true;
	}
	else if (UserSceneTextureDebug == 2 && GAreScreenMessagesEnabled)
	{
		// Enable conditionally if there are errors
		const FSceneTextures& SceneTextures = View.GetSceneTextures();
		for (const FUserSceneTextureEventData& EventData : SceneTextures.UserSceneTextureEvents)
		{
			if (EventData.Event == EUserSceneTextureEvent::MissingInput || EventData.Event == EUserSceneTextureEvent::CollidingInput)
			{
				bEnableUserSceneTextureDebug = true;
				break;
			}
		}

		for (auto& UserSceneTextureElement : SceneTextures.UserSceneTextures)
		{
			if (bEnableUserSceneTextureDebug)
			{
				break;
			}
			for (FTransientUserSceneTexture& UserSceneTexture : UserSceneTextureElement.Value)
			{
				if (!UserSceneTexture.bUsed)
				{
					bEnableUserSceneTextureDebug = true;
					break;
				}
			}
		}
	}
	else if (UserSceneTextureDebug == 3)
	{
		// Enable conditionally for view with texture being visualized
		if (GVisualizeTexture.IsRequestedView())
		{
			bEnableUserSceneTextureDebug = true;
		}
	}

	if (!bEnableUserSceneTextureDebug)
	{
		return;
	}

	const FSceneTextures& SceneTextures = View.GetSceneTextures();
	if (!SceneTextures.UserSceneTextureEvents.IsEmpty())
	{
		FScreenPassRenderTarget OutputTarget = FScreenPassRenderTarget(Output, ERenderTargetLoadAction::ELoad);
		AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("UserSceneTextureDebug"), View, OutputTarget, [&View, ViewIndex, &SceneTextures](FCanvas& Canvas)
		{
			FLinearColor TextColor(FLinearColor::White);
			FLinearColor GrayTextColor(FLinearColor::Gray);
			FLinearColor GreenTextColor(FLinearColor::Green);
			FLinearColor RedTextColor(FLinearColor::Red);
			FLinearColor YellowTextColor(FLinearColor::Yellow);
			FLinearColor MagentaTextColor(1.f, 0.f, 1.f);
			FString Text;

			const UFont* Font = GetStatsFont();

			const float ViewPortWidth = float(View.ViewRect.Width());
			const float ViewPortHeight = float(View.ViewRect.Height());

			const float CRHeight = 20.0f;
			const float OffsetFromLeft = 0.05f;
			const float OffsetFromTop = 0.2f;
			const float OffsetFromHeader = CRHeight * 1.5f;

			float PrintX_CR = ViewPortWidth * OffsetFromLeft;

			float PrintX = PrintX_CR;
			float PrintY = ViewPortHeight * OffsetFromTop;

			int32 NumPasses = 0;
			for (const FUserSceneTextureEventData& EventData : SceneTextures.UserSceneTextureEvents)
			{
				if (EventData.ViewIndex == ViewIndex && EventData.Event == EUserSceneTextureEvent::Pass)
				{
					NumPasses++;
				}
			}

			// Draw header
			Text = FString::Printf(TEXT("User Scene Texture Passes (count = %i)"), NumPasses);
			PrintX += CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, *Text, Font, GreenTextColor);
			if (CVarUserSceneTextureDebug.GetValueOnRenderThread() == 2)
			{
				CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, TEXT(" - enabled on error via \"r.PostProcessing.UserSceneTextureDebug 2\""), Font, GreenTextColor);
			}
			PrintX = PrintX_CR;
			PrintY += CRHeight;

			// Draw column description
			CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, TEXT("Location [Priority]  Material:   Inputs   -->  Output"), Font, GrayTextColor);
			PrintY += OffsetFromHeader;

			// Draw blendable locations and priorities
			static_assert(BL_MAX == 7);
			static const TCHAR* GBlendableLocationShortNames[BL_MAX + 1] = {};
			if (!GBlendableLocationShortNames[0])
			{
				// One time init -- enum in header isn't in numerical order, so it's simpler to initialize this way
				GBlendableLocationShortNames[BL_SceneColorBeforeDOF] = TEXT("BeforeDOF");
				GBlendableLocationShortNames[BL_SceneColorAfterDOF] = TEXT("AfterDOF");
				GBlendableLocationShortNames[BL_TranslucencyAfterDOF] = TEXT("Translucent");
				GBlendableLocationShortNames[BL_SSRInput] = TEXT("SSRInput");
				GBlendableLocationShortNames[BL_SceneColorBeforeBloom] = TEXT("BeforeBloom");
				GBlendableLocationShortNames[BL_ReplacingTonemapper] = TEXT("ReplaceTonemap");
				GBlendableLocationShortNames[BL_SceneColorAfterTonemapping] = TEXT("AfterTonemap");
				GBlendableLocationShortNames[BL_MAX] = TEXT("MAX");
			}

			static_assert((int32)FCustomRenderPassBase::ERenderOutput::MAX == 7);
			static const TCHAR* GCustomRenderPassOutputShortNames[(int32)FCustomRenderPassBase::ERenderOutput::MAX] =
			{
				TEXT("Depth"),			// SceneDepth
				TEXT("Devdepth"),		// DeviceDepth
				TEXT("ColDepth"),		// SceneColorAndDepth
				TEXT("Color"),			// SceneColorAndAlpha
				TEXT("ColorNoA"),		// SceneColorNoAlpha
				TEXT("Base"),			// BaseColor
				TEXT("Norm"),			// Normal
			};

			float MaxBlendableInfoWidth = 0.0f;
			for (const FUserSceneTextureEventData& EventData : SceneTextures.UserSceneTextureEvents)
			{
				if (EventData.ViewIndex == ViewIndex && EventData.Event == EUserSceneTextureEvent::Pass)
				{
					const FMaterialRenderProxy* RenderProxy = EventData.MaterialInterface->GetRenderProxy();
					const FMaterial* Material = RenderProxy->GetMaterialNoFallback(View.FeatureLevel);

					Text = FString::Printf(TEXT("%s [%d]"), GBlendableLocationShortNames[FMath::Min((uint32)RenderProxy->GetBlendableLocation(Material), (uint32)BL_MAX)], RenderProxy->GetBlendablePriority(Material));

					float BlendableInfoWidth = CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, Text, Font, TextColor);
					MaxBlendableInfoWidth = FMath::Max(MaxBlendableInfoWidth, BlendableInfoWidth);
					PrintY += CRHeight;
				}
				else if (EventData.ViewIndex == ViewIndex && EventData.Event == EUserSceneTextureEvent::CustomRenderPass)
				{
					const FCustomRenderPassBase* RenderPass = (const FCustomRenderPassBase*)EventData.MaterialInterface;
					const FSceneCaptureCustomRenderPassUserData& UserData = FSceneCaptureCustomRenderPassUserData::Get(RenderPass);

					// EventData.AllocationOrder stores the ERenderOutput enum
					Text = FString::Printf(TEXT("%s (CRP:%s)"), *UserData.CaptureActorName, GCustomRenderPassOutputShortNames[EventData.AllocationOrder]);

					float BlendableInfoWidth = CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, Text, Font, TextColor);
					MaxBlendableInfoWidth = FMath::Max(MaxBlendableInfoWidth, BlendableInfoWidth);
					PrintY += CRHeight;
				}
			}

			PrintX_CR = PrintX_CR + MaxBlendableInfoWidth + 10.0f;
			PrintX = PrintX_CR;
			PrintY = ViewPortHeight * OffsetFromTop + CRHeight + OffsetFromHeader;

			// Draw material names
			float MaxNameWidth = 0.0f;
			for (const FUserSceneTextureEventData& EventData : SceneTextures.UserSceneTextureEvents)
			{
				if (EventData.ViewIndex == ViewIndex && EventData.Event == EUserSceneTextureEvent::Pass)
				{
					const UMaterialInterface* MaterialInterface = EventData.MaterialInterface;

					// Skip over runtime generated dynamic instance when producing name
					while (MaterialInterface->IsA<UMaterialInstanceDynamic>())
					{
						MaterialInterface = ((const UMaterialInstanceDynamic*)MaterialInterface)->Parent;
					}

					Text = FString::Printf(TEXT("%s:"), *MaterialInterface->GetName());

					float NameWidth = CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, Text, Font, TextColor);
					MaxNameWidth = FMath::Max(MaxNameWidth, NameWidth);
					PrintY += CRHeight;
				}
			}

			PrintX_CR = PrintX_CR + MaxNameWidth + 10.0f;
			PrintX = PrintX_CR;
			PrintY = ViewPortHeight * OffsetFromTop + CRHeight + OffsetFromHeader;

			// Draw everything else (inputs and outputs)
			bool bAnyMissing = false;
			bool bAnyUnused = false;
			bool bAnyColliding = false;

			for (const FUserSceneTextureEventData& EventData : SceneTextures.UserSceneTextureEvents)
			{
				if (EventData.ViewIndex == ViewIndex)
				{
					switch (EventData.Event)
					{
					case EUserSceneTextureEvent::MissingInput:
						Text = FString::Printf(TEXT("  %s"), *EventData.Name.ToString());
						PrintX += CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, Text, Font, RedTextColor);
						bAnyMissing = true;
						break;
					case EUserSceneTextureEvent::CollidingInput:
						Text = FString::Printf(TEXT("  %s"), *EventData.Name.ToString());
						PrintX += CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, Text, Font, MagentaTextColor);
						bAnyColliding = true;
						break;
					case EUserSceneTextureEvent::FoundInput:
						Text = FString::Printf(TEXT("  %s"), *EventData.Name.ToString());
						PrintX += CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, Text, Font, GrayTextColor);
						break;
					case EUserSceneTextureEvent::Output:
						{
							const FTransientUserSceneTexture* UserTexture = SceneTextures.FindUserSceneTextureByEvent(EventData);
							check(UserTexture);

							Text = FString::Printf(TEXT("  --> %s [%dx%d]"), *EventData.Name.ToString(), EventData.RectSize.X, EventData.RectSize.Y);
							PrintX += CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, *Text, Font, UserTexture->bUsed ? GrayTextColor : YellowTextColor);
							bAnyUnused = bAnyUnused || !UserTexture->bUsed;

							// MaterialInterface can be null if this output was generated by a CustomRenderPass
							if (EventData.MaterialInterface && EventData.MaterialInterface->GetBlendMode() != BLEND_Opaque)
							{
								PrintX += CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, TEXT("  Blend"), Font, GrayTextColor);
							}
						}
						break;
					case EUserSceneTextureEvent::Pass:
					case EUserSceneTextureEvent::CustomRenderPass:
						// End of line
						PrintY += CRHeight;
						PrintX = PrintX_CR;
						break;
					}
				}
			}

			// Print color codings for warnings if present
			PrintX = ViewPortWidth * OffsetFromLeft;
			PrintY += CRHeight * 0.5f;
			if (bAnyUnused)
			{
				CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, TEXT("Yellow:  Unused Output"), Font, YellowTextColor);
				PrintY += CRHeight;
			}
			if (bAnyMissing)
			{
				CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, TEXT("Red:  Missing Input"), Font, RedTextColor);
				PrintY += CRHeight;
			}
			if (bAnyColliding)
			{
				CanvasDrawShadowedStringReturnWidth(Canvas, PrintX, PrintY, TEXT("Magenta:  Input collides with Output"), Font, MagentaTextColor);
				PrintY += CRHeight;
			}
		});
	}
}
#endif  // !UE_BUILD_SHIPPING

// Shader for visualizing GBuffer values
class FGBufferPickingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGBufferPickingCS);
	SHADER_USE_PARAMETER_STRUCT(FGBufferPickingCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTexturesStruct)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool IsSupported(EShaderPlatform Platform)
	{
		return
			IsPCPlatform(Platform) && // Only on PC because some platforms have trouble to compile this shader. See JIRA UE-370697.
			IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5) &&
			ShaderPrint::IsSupported(Platform) &&
			(!Substrate::IsSubstrateEnabled() || Substrate::IsSubstrateBlendableGBufferEnabled(Platform));
	}
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return IsSupported(Parameters.Platform) && EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SHADER_GBUFFER_PICKING"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGBufferPickingCS, "/Engine/Private/PostProcessGBufferHints.usf", "MainCS", SF_Compute);

#if WITH_EDITOR
static void AddGBufferPicking(FRDGBuilder& GraphBuilder, const FViewInfo& View, const TRDGUniformBufferRef<FSceneTextureUniformParameters>& SceneTextures)
{
	if (CVarGBufferPicking.GetValueOnRenderThread() <= 0 || !FGBufferPickingCS::IsSupported(View.Family->GetShaderPlatform()))
	{
		return;
	}

	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);

	FGBufferPickingCS::FParameters* Parameters = GraphBuilder.AllocParameters<FGBufferPickingCS::FParameters>();
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->SceneTexturesStruct = SceneTextures;
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);

	TShaderMapRef<FGBufferPickingCS> ComputeShader(View.ShaderMap);
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("Debug::GBufferPicking"), ComputeShader, Parameters, FIntVector(1,1,1));
}
#endif
