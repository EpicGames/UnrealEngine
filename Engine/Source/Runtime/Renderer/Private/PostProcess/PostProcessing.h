// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "RenderGraphFwd.h"
#include "RHIFeatureLevel.h"
#include "ScreenPass.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "TranslucentRendering.h"
#include "PathTracing.h"
#include "PostProcess/PostProcessInputs.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

enum class EReflectionsMethod;
enum class EDiffuseIndirectMethod;

class FScreenPassVS;
class FViewFamilyInfo;
class FViewInfo;
class FVirtualShadowMapArray;
struct FLumenSceneFrameTemporaries;
struct FMegaLightsFrameTemporaries;
class FSceneUniformBuffer;
struct FPostProcessingInputs;
class FInstanceCullingManager;
struct FSceneWithoutWaterTextures;
class UMaterialInterface;
class FMobileSceneTextureUniformParameters;
class FScene;

namespace Nanite
{
	struct FRasterResults;
}

// Returns whether the full post process pipeline is enabled. Otherwise, the minimal set of operations are performed.
bool IsPostProcessingEnabled(const FViewInfo& View);

// Returns whether the post process pipeline supports using compute passes.
bool IsPostProcessingWithComputeEnabled(ERHIFeatureLevel::Type FeatureLevel);

// Returns whether the post process pipeline supports propagating the alpha channel.
bool IsPostProcessingWithAlphaChannelSupported();

using FPostProcessVS = FScreenPassVS;

void AddPostProcessingPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View, int32 ViewIndex,
	FSceneUniformBuffer& SceneUniformBuffer,
	EDiffuseIndirectMethod DiffuseIndirectMethod,
	EReflectionsMethod ReflectionsMethod,
	const FPostProcessingInputs& Inputs,
	const Nanite::FRasterResults* NaniteRasterResults,
	FInstanceCullingManager& InstanceCullingManager,
	FVirtualShadowMapArray* VirtualShadowMapArray,
	const FLumenSceneFrameTemporaries& LumenFrameTemporaries,
	const TSharedPtr<FMegaLightsFrameTemporaries>& MegaLightsFrameTemporaries,
	const FSceneWithoutWaterTextures& SceneWithoutWaterTextures,
	FScreenPassTexture TSRFlickeringInput,
	FRDGTextureRef& InstancedEditorDepthTexture);

void AddDebugViewPostProcessingPasses(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View, 
	int32 ViewIndex,
	FSceneUniformBuffer& SceneUniformBuffer, 
	const FPostProcessingInputs& Inputs, 
	const Nanite::FRasterResults* NaniteRasterResults, 
	FVirtualShadowMapArray* VirtualShadowMapArray);

void AddVisualizeCalibrationMaterialPostProcessingPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FPostProcessingInputs& Inputs, const UMaterialInterface* InMaterialInterface);

struct FMobilePostProcessingInputs
{
	TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> SceneTextures = nullptr;
	FRDGTextureRef ViewFamilyTexture = nullptr;
	FRDGTextureRef ViewFamilyDepthTexture = nullptr;

	void Validate() const
	{
		check(ViewFamilyTexture);
		check(SceneTextures);
	}
};

void AddMobilePostProcessingPasses(FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View, int32 ViewIndex, FSceneUniformBuffer &SceneUniformBuffer, const FMobilePostProcessingInputs& Inputs, FInstanceCullingManager& InstanceCullingManager);

void AddBasicPostProcessPasses(FRDGBuilder& GraphBuilder, const FViewInfo& View);

FRDGTextureRef AddProcessPlanarReflectionPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneColorTexture);

void AddRuntimeReflectionCapturePostPass(FRDGBuilder& GraphBuilder, FViewFamilyInfo& ViewFamily, FRDGTextureRef ViewFamilyTexture, FRDGTextureRef ViewFamilyDepthTexture, TConstArrayView<FViewInfo> Views);
