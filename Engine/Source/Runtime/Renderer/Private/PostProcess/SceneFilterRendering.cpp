// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneFilterRendering.cpp: Filter rendering implementation.
=============================================================================*/

#include "PostProcess/SceneFilterRendering.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "IHeadMountedDisplay.h"
#include "IXRTrackingSystem.h"
#include "PixelShaderUtils.h"
#include "CommonRenderResources.h"
#include "PostProcess/DrawRectangle.h"
#include "RHIResourceUtils.h"
#include "SceneRendering.h"

void FTesselatedScreenRectangleIndexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	TArray<uint16> IndexBuffer;

	uint32 NumIndices = NumPrimitives() * 3;
	IndexBuffer.AddUninitialized(NumIndices);

	uint16* Out = (uint16*)IndexBuffer.GetData();

	for(uint32 y = 0; y < Height; ++y)
	{
		for(uint32 x = 0; x < Width; ++x)
		{
			// left top to bottom right in reading order
			uint16 Index00 = x  + y * (Width + 1);
			uint16 Index10 = Index00 + 1;
			uint16 Index01 = Index00 + (Width + 1);
			uint16 Index11 = Index01 + 1;

			// todo: diagonal can be flipped on parts of the screen

			// triangle A
			*Out++ = Index00; *Out++ = Index01; *Out++ = Index10;

			// triangle B
			*Out++ = Index11; *Out++ = Index10; *Out++ = Index01;
		}
	}

	// Create index buffer. Fill buffer with initial data upon creation
	IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FTesselatedScreenRectangleIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(IndexBuffer));
}

uint32 FTesselatedScreenRectangleIndexBuffer::NumVertices() const
{
	// 4 vertices per quad but shared
	return (Width + 1) * (Height + 1);
}

uint32 FTesselatedScreenRectangleIndexBuffer::NumPrimitives() const
{
	// triangle has 3 corners, 2 triangle per quad
	return 2 * Width * Height;
}

/** We don't need a vertex buffer as we can compute the vertex attributes in the VS */
static TGlobalResource<FTesselatedScreenRectangleIndexBuffer> GTesselatedScreenRectangleIndexBuffer;


IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FDrawRectangleParameters, "DrawRectangleParameters");

static TAutoConsoleVariable<int32> CVarDrawRectangleOptimization(
	TEXT("r.DrawRectangleOptimization"),
	1,
	TEXT("Controls an optimization for DrawRectangle(). When enabled a triangle can be used to draw a quad in certain situations (viewport sized quad).\n")
	TEXT("Using a triangle allows for slightly faster post processing in lower resolutions but can not always be used.\n")
	TEXT(" 0: Optimization is disabled, DrawDenormalizedQuad always render with quad\n")
	TEXT(" 1: Optimization is enabled, a triangle can be rendered where specified (default)"),
	ECVF_RenderThreadSafe);

static void DoDrawRectangleFlagOverride(EDrawRectangleFlags& Flags)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Determine triangle draw mode
	int Value = CVarDrawRectangleOptimization.GetValueOnRenderThread();

	if(!Value)
	{
		// don't use triangle optimization
		Flags = EDRF_Default;
	}
#endif
}

namespace UE::Renderer::PostProcess
{
	void SetDrawRectangleParameters(
		FRHIBatchedShaderParameters& BatchedParameters,
		const FShader* VertexShader,
		float X,
		float Y,
		float SizeX,
		float SizeY,
		float U,
		float V,
		float SizeU,
		float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize)
	{
		// Set up vertex uniform parameters for scaling and biasing the rectangle.
		// Note: Use DrawRectangle in the vertex shader to calculate the correct vertex position and uv.

		FDrawRectangleParameters Parameters;
		Parameters.PosScaleBias = FVector4f(SizeX, SizeY, X, Y);
		Parameters.UVScaleBias = FVector4f(SizeU, SizeV, U, V);

		Parameters.InvTargetSizeAndTextureSize = FVector4f(
			1.0f / TargetSize.X, 1.0f / TargetSize.Y,
			1.0f / TextureSize.X, 1.0f / TextureSize.Y);

		SetUniformBufferParameterImmediate(BatchedParameters, VertexShader->GetUniformBufferParameter<FDrawRectangleParameters>(), Parameters);
	}

	void SetDrawRectangleParameters(FRHIBatchedShaderParameters& BatchedParameters, const FShader* VertexShader, const FIntPoint& ViewSize)
	{
		SetDrawRectangleParameters(
			BatchedParameters,
			VertexShader,
			0.0f, 0.0f,
			ViewSize.X, ViewSize.Y,
			0.0f, 0.0f,
			1.0f, 1.0f,
			ViewSize,
			FIntPoint(1.0f, 1.0f)
		);
	}

	void SetDrawRectangleParameters(FRHIBatchedShaderParameters& BatchedParameters, const FShader* VertexShader, const FSceneView& View)
	{
		SetDrawRectangleParameters(BatchedParameters, VertexShader, View.UnconstrainedViewRect.Size());
	}
}

inline void InternalDrawRectangle(
	FRHICommandList& RHICmdList,
	float X,
	float Y,
	float SizeX,
	float SizeY,
	float U,
	float V,
	float SizeU,
	float SizeV,
	FIntPoint TargetSize,
	FIntPoint TextureSize,
	const TShaderRef<FShader>& VertexShader,
	EDrawRectangleFlags Flags,
	uint32 InstanceCount
	)
{
	DoDrawRectangleFlagOverride(Flags);

	// triangle if extending to left and top of the given rectangle, if it's not left top of the viewport it can cause artifacts
	if(X > 0.0f || Y > 0.0f)
	{
		// don't use triangle optimization
		Flags = EDRF_Default;
	}

	{
		FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
		UE::Renderer::PostProcess::SetDrawRectangleParameters(BatchedParameters, VertexShader.GetShader(), X, Y, SizeX, SizeY, U, V, SizeU, SizeV, TargetSize, TextureSize);
		RHICmdList.SetBatchedShaderParameters(VertexShader.GetVertexShader(), BatchedParameters);
	}

	if(Flags == EDRF_UseTesselatedIndexBuffer)
	{
		// no vertex buffer needed as we compute it in VS
		RHICmdList.SetStreamSource(0, NULL, 0);

		RHICmdList.DrawIndexedPrimitive(
			GTesselatedScreenRectangleIndexBuffer.IndexBufferRHI,
			/*BaseVertexIndex=*/ 0,
			/*MinIndex=*/ 0,
			/*NumVertices=*/ GTesselatedScreenRectangleIndexBuffer.NumVertices(),
			/*StartIndex=*/ 0,
			/*NumPrimitives=*/ GTesselatedScreenRectangleIndexBuffer.NumPrimitives(),
			/*NumInstances=*/ InstanceCount
			);
	}
	else if (Flags == EDRF_UseTriangleOptimization)
	{
		FPixelShaderUtils::DrawFullscreenTriangle(RHICmdList, InstanceCount);
	}
	else
	{
		FPixelShaderUtils::DrawFullscreenQuad(RHICmdList, InstanceCount);
	}
}

void DrawRectangle(
	FRHICommandList& RHICmdList,
	float X,
	float Y,
	float SizeX,
	float SizeY,
	float U,
	float V,
	float SizeU,
	float SizeV,
	FIntPoint TargetSize,
	FIntPoint TextureSize,
	const TShaderRef<FShader>& VertexShader,
	EDrawRectangleFlags Flags,
	uint32 InstanceCount
)
{
	InternalDrawRectangle(RHICmdList, X, Y, SizeX, SizeY, U, V, SizeU, SizeV, TargetSize, TextureSize, VertexShader, Flags, InstanceCount);
}

void DrawTransformedRectangle(
	FRHICommandList& RHICmdList,
	float X,
	float Y,
	float SizeX,
	float SizeY,
	const FMatrix& PosTransform,
	float U,
	float V,
	float SizeU,
	float SizeV,
	const FMatrix& TexTransform,
	FIntPoint TargetSize,
	FIntPoint TextureSize
	)
{
	float ClipSpaceQuadZ = 0.0f;

	// we don't do the triangle optimization as this case is rare for the DrawTransformedRectangle case

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FFilterVertex>(TEXT("DrawTransformedRectangle"), 4)
		.AddUsage(EBufferUsageFlags::Volatile)
		.SetInitActionInitializer()
		.DetermineInitialState();

	TRHIBufferInitializer<FFilterVertex> Vertices = RHICmdList.CreateBufferInitializer(CreateDesc);

	// LWC_TODO: precision loss
	Vertices[0].Position = (FVector4f)PosTransform.TransformFVector4(FVector4(X,			Y,			ClipSpaceQuadZ,	1));
	Vertices[1].Position = (FVector4f)PosTransform.TransformFVector4(FVector4(X + SizeX,	Y,			ClipSpaceQuadZ,	1));
	Vertices[2].Position = (FVector4f)PosTransform.TransformFVector4(FVector4(X,			Y + SizeY,	ClipSpaceQuadZ,	1));
	Vertices[3].Position = (FVector4f)PosTransform.TransformFVector4(FVector4(X + SizeX,	Y + SizeY,	ClipSpaceQuadZ,	1));

	Vertices[0].UV = (FVector2f)FVector2D(TexTransform.TransformFVector4(FVector(U,			V,         0)));	// LWC_TODO: Precision loss
	Vertices[1].UV = (FVector2f)FVector2D(TexTransform.TransformFVector4(FVector(U + SizeU,	V,         0)));	// LWC_TODO: Precision loss
	Vertices[2].UV = (FVector2f)FVector2D(TexTransform.TransformFVector4(FVector(U,			V + SizeV, 0)));	// LWC_TODO: Precision loss
	Vertices[3].UV = (FVector2f)FVector2D(TexTransform.TransformFVector4(FVector(U + SizeU,	V + SizeV, 0)));	// LWC_TODO: Precision loss

	for (int32 VertexIndex = 0; VertexIndex < 4; VertexIndex++)
	{
		Vertices[VertexIndex].Position.X = -1.0f + 2.0f * (Vertices[VertexIndex].Position.X) / (float)TargetSize.X;
		Vertices[VertexIndex].Position.Y = (+1.0f - 2.0f * (Vertices[VertexIndex].Position.Y) / (float)TargetSize.Y) * GProjectionSignY;

		Vertices[VertexIndex].UV.X = Vertices[VertexIndex].UV.X / (float)TextureSize.X;
		Vertices[VertexIndex].UV.Y = Vertices[VertexIndex].UV.Y / (float)TextureSize.Y;
	}

	FBufferRHIRef VertexBufferRHI = Vertices.Finalize();

	RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
	RHICmdList.DrawIndexedPrimitive(GTwoTrianglesIndexBuffer.IndexBufferRHI, 0, 0, 4, 0, 2, 1);
	VertexBufferRHI.SafeRelease();
}

void DrawHmdMesh(
	FRHICommandList& RHICmdList,
	float X,
	float Y,
	float SizeX,
	float SizeY,
	float U,
	float V,
	float SizeU,
	float SizeV,
	FIntPoint TargetSize,
	FIntPoint TextureSize,
	int32 StereoView,
	const TShaderRef<FShader>& VertexShader,
	int32 InstanceCount
	)
{
	{
		FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
		UE::Renderer::PostProcess::SetDrawRectangleParameters(
			BatchedParameters, VertexShader.GetShader(),
			X, Y, SizeX, SizeY, U, V, SizeU, SizeV,
			TargetSize, TextureSize
		);
		RHICmdList.SetBatchedShaderParameters(VertexShader.GetVertexShader(), BatchedParameters);
	}

	if (GEngine->XRSystem->GetHMDDevice())
	{
		GEngine->XRSystem->GetHMDDevice()->DrawVisibleAreaMesh(RHICmdList, StereoView, InstanceCount);
	}
}

void DrawPostProcessPass(
	FRHICommandList& RHICmdList,
	float X,
	float Y,
	float SizeX,
	float SizeY,
	float U,
	float V,
	float SizeU,
	float SizeV,
	FIntPoint TargetSize,
	FIntPoint TextureSize,
	const TShaderRef<FShader>& VertexShader,
	int32 StereoViewIndex,
	bool bHasCustomMesh,
	EDrawRectangleFlags Flags,
	int32 InstanceCount)
{
	if (bHasCustomMesh && StereoViewIndex != INDEX_NONE)
	{
		DrawHmdMesh(RHICmdList, X, Y, SizeX, SizeY, U, V, SizeU, SizeV, TargetSize, TextureSize, StereoViewIndex, VertexShader, InstanceCount);
	}
	else
	{
		DrawRectangle(RHICmdList, X, Y, SizeX, SizeY, U, V, SizeU, SizeV, TargetSize, TextureSize, VertexShader, Flags, InstanceCount);
	}
}

namespace UE::Renderer::PostProcess
{
	void DrawRectangle(
		FRHICommandList& RHICmdList,
		const TShaderRef<FShader>& VertexShader,
		float X,
		float Y,
		float SizeX,
		float SizeY,
		float U,
		float V,
		float SizeU,
		float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize,
		EDrawRectangleFlags Flags,
		uint32 InstanceCount
	)
	{
		::DrawRectangle(RHICmdList, X, Y, SizeX, SizeY, U, V, SizeU, SizeV, TargetSize, TextureSize, VertexShader, Flags, InstanceCount);
	}

	void DrawRectangle(
		FRHICommandList& RHICmdList,
		const TShaderRef<FShader>& VertexShader,
		const FSceneView& InView,
		EDrawRectangleFlags Flags,
		uint32 InstanceCount
	)
	{
		if (ensure(InView.bIsViewInfo))
		{
			const FViewInfo& View = static_cast<const FViewInfo&>(InView);
			DrawRectangle(
				RHICmdList,
				VertexShader,
				View.ViewRect.Min.X, View.ViewRect.Min.Y,
				View.ViewRect.Width(), View.ViewRect.Height(),
				View.ViewRect.Min.X, View.ViewRect.Min.Y,
				View.ViewRect.Width(), View.ViewRect.Height(),
				View.UnconstrainedViewRect.Size(),
				View.UnconstrainedViewRect.Size(),
				Flags,
				InstanceCount);
		}
	}

	void DrawPostProcessPass(
		FRHICommandList& RHICmdList,
		const TShaderRef<FShader>& VertexShader,
		float X,
		float Y,
		float SizeX,
		float SizeY,
		float U,
		float V,
		float SizeU,
		float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize,
		int32 StereoViewIndex,
		bool bHasCustomMesh,
		EDrawRectangleFlags Flags
	)
	{
		::DrawPostProcessPass(RHICmdList, X, Y, SizeX, SizeY, U, V, SizeU, SizeV, TargetSize, TextureSize, VertexShader, StereoViewIndex, bHasCustomMesh, Flags);
	}
}
