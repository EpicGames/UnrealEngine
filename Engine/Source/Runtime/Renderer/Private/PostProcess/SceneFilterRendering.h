// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneFilterRendering.h: Filter rendering definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RenderResource.h"
#include "RendererInterface.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "PostProcess/SceneRenderTargets.h"
#include "ShaderParameterUtils.h"
#include "SceneRenderTargetParameters.h"

enum class EStereoscopicPass;

/** Uniform buffer for computing the vertex positional and UV adjustments in the vertex shader. */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT( FDrawRectangleParameters, )
	SHADER_PARAMETER( FVector4f, PosScaleBias )
	SHADER_PARAMETER( FVector4f, UVScaleBias )
	SHADER_PARAMETER( FVector4f, InvTargetSizeAndTextureSize )
END_GLOBAL_SHADER_PARAMETER_STRUCT()

/**
 * Draws a quad with the given vertex positions and UVs in denormalized pixel/texel coordinates.
 * The platform-dependent mapping from pixels to texels is done automatically.
 * Note that the positions are affected by the current viewport.
 * NOTE: DrawRectangle should be used in the vertex shader to calculate the correct position and uv for vertices.
 * NOTE2: Assumes previously set PSO has PrimitiveType = PT_TriangleList
 *
 * X, Y							Position in screen pixels of the top left corner of the quad
 * SizeX, SizeY					Size in screen pixels of the quad
 * U, V							Position in texels of the top left corner of the quad's UV's
 * SizeU, SizeV					Size in texels of the quad's UV's
 * TargetSizeX, TargetSizeY		Size in screen pixels of the target surface
 * TextureSize                  Size in texels of the source texture
 * VertexShader					The vertex shader used for rendering
 * Flags						see EDrawRectangleFlags
 * InstanceCount				Number of instances of rectangle
 */
extern RENDERER_API void DrawRectangle(
	FRHICommandList& RHICmdList,
	float X,
	float Y,
	float SizeX,
	float SizeY,
	float U,
	float V,
	float SizeU,
	float SizeV,
	FIntPoint TargetSize,
	FIntPoint TextureSize,
	const TShaderRef<FShader>& VertexShader,
	EDrawRectangleFlags Flags = EDRF_Default,
	uint32 InstanceCount = 1
	);

// NOTE: Assumes previously set PSO has PrimitiveType = PT_TriangleList
extern RENDERER_API void DrawTransformedRectangle(
	FRHICommandList& RHICmdList,
	float X,
	float Y,
	float SizeX,
	float SizeY,
	const FMatrix& PosTransform,
	float U,
	float V,
	float SizeU,
	float SizeV,
	const FMatrix& TexTransform,
	FIntPoint TargetSize,
	FIntPoint TextureSize
	);

// NOTE: Assumes previously set PSO has PrimitiveType = PT_TriangleList
extern RENDERER_API void DrawHmdMesh(
	FRHICommandList& RHICmdList,
	float X,
	float Y,
	float SizeX,
	float SizeY,
	float U,
	float V,
	float SizeU,
	float SizeV,
	FIntPoint TargetSize,
	FIntPoint TextureSize,
	EStereoscopicPass StereoView,
	const TShaderRef<FShader>& VertexShader,
	int32 InstanceCount = 1
	);

// NOTE: Assumes previously set PSO has PrimitiveType = PT_TriangleList
extern RENDERER_API void DrawPostProcessPass(
	FRHICommandList& RHICmdList,
	float X,
	float Y,
	float SizeX,
	float SizeY,
	float U,
	float V,
	float SizeU,
	float SizeV,
	FIntPoint TargetSize,
	FIntPoint TextureSize,
	const TShaderRef<FShader>& VertexShader,
	int32 StereoViewIndex,
	bool bHasCustomMesh,
	EDrawRectangleFlags Flags = EDRF_Default,
	int32 InstanceCount = 1
	);

class FTesselatedScreenRectangleIndexBuffer : public FIndexBuffer
{
public:

	// if one of those constants change, UpscaleVS needs to be recompiled

	// number of quads in x
	static const uint32 Width = 32;		// used for CylindricalProjection (smaller FOV could do less tessellation)
	// number of quads in y
	static const uint32 Height = 20;	// to minimize distortion we also tessellate in Y but a perspective distortion could do that with fewer triangles.

	/** Initialize the RHI for this rendering resource */
	void InitRHI(FRHICommandListBase& RHICmdList) override;

	uint32 NumVertices() const;

	uint32 NumPrimitives() const;
};

