// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*
* Stencil layout during basepass / deferred decals:
*		BIT ID    | USE
*		[0]       | sandbox bit (bit to be use by any rendering passes, but must be properly reset to 0 after using)
*		[1]       | Cast Shadow
*		[2]       | Ray Tracing Representation
*		[3]       | Temporal AA mask for translucent object.
*		[4]       | Lighting channels
*		[5]       | Lighting channels
*		[6]       | Lighting channels
*		[7]       | primitive receive decal bit
*
* After deferred decals, stencil is cleared to 0 and no longer packed in this way, to ensure use of fast hardware clears and HiStencil.
* Stencil [0] is used by RT shadow for shadow LOD disthering (see: DitheredLODFadingOutMaskPass)
*/
#define STENCIL_SANDBOX_BIT_ID							0
// Must match usf
#define STENCIL_CAST_SHADOW_BIT_ID						1
#define STENCIL_RAY_TRACING_REPRESENTATION_BIT_ID		2
#define STENCIL_TEMPORAL_RESPONSIVE_AA_BIT_ID			3
#define STENCIL_LIGHTING_CHANNELS_BIT_ID				4
#define STENCIL_RECEIVE_DECAL_BIT_ID					7
// Used only during the lighting pass - alias/reuse light channels (which copied from stencil to a texture prior to lighting pass)
#define STENCIL_SUBSTRATE_FASTPATH							4
#define STENCIL_SUBSTRATE_SINGLEPATH						5
#define STENCIL_SUBSTRATE_COMPLEX							6
#define STENCIL_SUBSTRATE_COMPLEX_SPECIAL					7
// Used only by Substrate during the base pass when bUseDBufferPass is enabled (to mark material SUBSTRATE_DBUFFER_RESPONSE_xxx Normal/BaseColor/Roughness)
#define STENCIL_SUBSTRATE_RECEIVE_DBUFFER_NORMAL_BIT_ID		1
#define STENCIL_SUBSTRATE_RECEIVE_DBUFFER_DIFFUSE_BIT_ID	3
#define STENCIL_SUBSTRATE_RECEIVE_DBUFFER_ROUGHNESS_BIT_ID	7

// Outputs a compile-time constant stencil's bit mask ready to be used
// in TStaticDepthStencilState<> template parameter. It also takes care
// of masking the Value macro parameter to only keep the low significant
// bit to ensure to not overflow on other bits.
#define GET_STENCIL_BIT_MASK(BIT_NAME,Value) uint8((uint8(Value) & uint8(0x01)) << (STENCIL_##BIT_NAME##_BIT_ID))

#define STENCIL_SANDBOX_MASK GET_STENCIL_BIT_MASK(SANDBOX,1)

#define STENCIL_TEMPORAL_RESPONSIVE_AA_MASK GET_STENCIL_BIT_MASK(TEMPORAL_RESPONSIVE_AA,1)

#define STENCIL_LIGHTING_CHANNELS_MASK(Value) uint8(((Value) & 0x7) << STENCIL_LIGHTING_CHANNELS_BIT_ID)

// [Mobile specific]

// stencil [0-2] bits are used to render per-object shadows (see ShadowStencilMask in ShadowRendering.cpp)

// Sky material mask - bit 3
#define STENCIL_MOBILE_SKY_MASK						uint8(1 << 3)

// [Mobile Deferred only]
// Store shading model into stencil [1-2] bits
#define GET_STENCIL_MOBILE_SM_MASK(Value)			uint8(((Value) & 0x3) << 1)
#define STENCIL_MOBILE_SHADINGMODELS_BIT_ID			1
#define STENCIL_MOBILE_DEFAULTLIT_BIT_ID			2

// [Mobile Forward only]
// Cast contact shadow mask - bit 4 / Must match shader (ScreenSpaceShadows.usf)
#define STENCIL_MOBILE_CAST_CONTACT_SHADOW_BIT_ID	4
// Forward local light buffer mask for light function - bit 5
#define STENCIL_MOBILE_LIGHTFUNCTION_MASK			uint8(1 << 5)
