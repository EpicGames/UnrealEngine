// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/SubpixelMorphologicalAA.h"

#include "Engine/Texture2D.h"
#include "PostProcess/PostProcessing.h"
#include "PixelShaderUtils.h"
#include "TextureResource.h"
#include "SceneRendering.h"

DECLARE_GPU_STAT(SMAA);

TAutoConsoleVariable<int32> CVarSMAAQuality(
	TEXT("r.SMAA.Quality"), 2,
	TEXT("Selects the quality permutation of SMAA.\n")
	TEXT(" 0: Low (%60 of the quality) \n")
	TEXT(" 1: Medium (%80 of the quality) \n")
	TEXT(" 2: High (%95 of the quality - Default) \n")
	TEXT(" 3: Ultra (%99 of the quality) \n"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarSMAADebugVisualization(
	TEXT("r.SMAA.DebugVisualization"), 0,
	TEXT("Selects the SMAA debug visualization mode.\n")
	TEXT(" 0: Disabled \n")
	TEXT(" 1: Edge Texture \n")
	TEXT(" 2: Blend Texture \n")
	TEXT(" 3: Scene Color + Smoothened edges highlights \n"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarSMAAEdgeDetectionMode(
	TEXT("r.SMAA.EdgeMode"), 0,
	TEXT("Edge detection mode used.\n")
	TEXT(" 0: Color \n")
	TEXT(" 1: Luminance \n"),
	ECVF_RenderThreadSafe);

enum class ESMAAEdgeMode : uint8
{
	Color,
	Luminance,

	Count
};

#define DECLARE_SMAA_QUALITY_PERMUTATIONS               \
class FQuality : SHADER_PERMUTATION_INT("SMAA_QUALITY", 4);

template<typename T>
void SetupSMAAQualityPermutations(typename T::FPermutationDomain& Domain, ESMAAQuality Quality)
{
	Domain.template Set<typename T::FQuality>((int) Quality);
}

class FSMAAEdgeDetectionPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSMAAEdgeDetectionPS);
	SHADER_USE_PARAMETER_STRUCT(FSMAAEdgeDetectionPS, FGlobalShader);

	DECLARE_SMAA_QUALITY_PERMUTATIONS
	using FPermutationDomain = TShaderPermutationDomain<FQuality>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER_SAMPLER(SamplerState, PointSampler)
		SHADER_PARAMETER(FVector4f, RTMetrics)
		SHADER_PARAMETER(int32, EdgeMode)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

class FSMAAEdgeDetectionVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSMAAEdgeDetectionVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FSMAAEdgeDetectionVS, FGlobalShader);

	using FParameters = FSMAAEdgeDetectionPS::FParameters;
};

IMPLEMENT_GLOBAL_SHADER(FSMAAEdgeDetectionVS, "/Engine/Private/SMAA/SMAAEdgeDetectionShader.usf", "MainVS", SF_Vertex);

IMPLEMENT_GLOBAL_SHADER(FSMAAEdgeDetectionPS, "/Engine/Private/SMAA/SMAAEdgeDetectionShader.usf", "MainPS", SF_Pixel);

class FSMAABlendingWeightCalculationPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSMAABlendingWeightCalculationPS);
	SHADER_USE_PARAMETER_STRUCT(FSMAABlendingWeightCalculationPS, FGlobalShader);

	DECLARE_SMAA_QUALITY_PERMUTATIONS
	using FPermutationDomain = TShaderPermutationDomain<FQuality>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER(FVector4f, RTMetrics)
		SHADER_PARAMETER_TEXTURE(Texture2D, AreaTex)
		SHADER_PARAMETER_TEXTURE(Texture2D, SearchTex)
		SHADER_PARAMETER_SAMPLER(SamplerState, PointSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

class FSMAABlendingWeightCalculationVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSMAABlendingWeightCalculationVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FSMAABlendingWeightCalculationVS, FGlobalShader);

	using FParameters = FSMAABlendingWeightCalculationPS::FParameters;

	DECLARE_SMAA_QUALITY_PERMUTATIONS
	using FPermutationDomain = TShaderPermutationDomain<FQuality>;
};

IMPLEMENT_GLOBAL_SHADER(FSMAABlendingWeightCalculationVS, "/Engine/Private/SMAA/SMAABlendingWeightCalculationShader.usf", "MainVS", SF_Vertex);

IMPLEMENT_GLOBAL_SHADER(FSMAABlendingWeightCalculationPS, "/Engine/Private/SMAA/SMAABlendingWeightCalculationShader.usf", "MainPS", SF_Pixel);

class FSMAANeighborhoodBlendingPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSMAANeighborhoodBlendingPS);
	SHADER_USE_PARAMETER_STRUCT(FSMAANeighborhoodBlendingPS, FGlobalShader);

	class FAlphaChannelDim : SHADER_PERMUTATION_BOOL("DIM_ALPHA_CHANNEL");
	using FPermutationDomain = TShaderPermutationDomain<FAlphaChannelDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER(FVector4f, RTMetrics)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BlendTex)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

class FSMAANeighborhoodBlendingVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSMAANeighborhoodBlendingVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FSMAANeighborhoodBlendingVS, FGlobalShader);

	using FParameters = FSMAANeighborhoodBlendingPS::FParameters;
};

IMPLEMENT_GLOBAL_SHADER(FSMAANeighborhoodBlendingVS, "/Engine/Private/SMAA/SMAANeighborhoodBlendingShader.usf", "MainVS", SF_Vertex);

IMPLEMENT_GLOBAL_SHADER(FSMAANeighborhoodBlendingPS, "/Engine/Private/SMAA/SMAANeighborhoodBlendingShader.usf", "MainPS", SF_Pixel);

class FSMAADebugVisualizationPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSMAADebugVisualizationPS);
	SHADER_USE_PARAMETER_STRUCT(FSMAADebugVisualizationPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT(FScreenPassTextureInput, Input)
		SHADER_PARAMETER(FVector4f, RTMetrics)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, EdgeTex)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BlendTex)
		SHADER_PARAMETER_SAMPLER(SamplerState, LinearSampler)
		SHADER_PARAMETER(int32, DebugMode)
		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()
};

class FSMAADebugVisualizationVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSMAADebugVisualizationVS);
	SHADER_USE_PARAMETER_STRUCT_WITH_LEGACY_BASE(FSMAADebugVisualizationVS, FGlobalShader);

	using FParameters = FSMAADebugVisualizationPS::FParameters;
};

IMPLEMENT_GLOBAL_SHADER(FSMAADebugVisualizationVS, "/Engine/Private/SMAA/SMAADebugVisualization.usf", "MainVS", SF_Vertex);

IMPLEMENT_GLOBAL_SHADER(FSMAADebugVisualizationPS, "/Engine/Private/SMAA/SMAADebugVisualization.usf", "MainPS", SF_Pixel);

ESMAAQuality GetSMAAQuality()
{
	return ESMAAQuality(FMath::Clamp(CVarSMAAQuality.GetValueOnRenderThread(), 0, 3));
}

void AddSMAAEdgeDetectionPass(FRDGBuilder& GraphBuilder, const FSceneView& InSceneView, const FScreenPassTexture& SceneColorTexture, const FRDGTextureRef& EdgeTexture, const FRDGTextureRef& DepthStencilTexture, ESMAAQuality Quality, const FVector4f& RTMetrics)
{
	checkSlow(InSceneView.bIsViewInfo);
	const FViewInfo& View = static_cast<const FViewInfo&>(InSceneView);

	FRHISamplerState* PointClampSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FSMAAEdgeDetectionPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSMAAEdgeDetectionPS::FParameters>();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(EdgeTexture, ERenderTargetLoadAction::EClear);
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
		DepthStencilTexture,
		ERenderTargetLoadAction::EClear,
		ERenderTargetLoadAction::EClear,
		FExclusiveDepthStencil::DepthWrite_StencilWrite);
	PassParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();
	PassParameters->Input = GetScreenPassTextureInput(SceneColorTexture, PointClampSampler);
	PassParameters->RTMetrics = RTMetrics;
	PassParameters->PointSampler = PointClampSampler;
	PassParameters->EdgeMode = FMath::Clamp(CVarSMAAEdgeDetectionMode->GetInt(), 0, (int)ESMAAEdgeMode::Count - 1);

	FSMAAEdgeDetectionPS::FPermutationDomain PixelPermutationVector;
	SetupSMAAQualityPermutations<FSMAAEdgeDetectionPS>(PixelPermutationVector, Quality);
	TShaderMapRef<FSMAAEdgeDetectionPS> PixelShader(View.ShaderMap, PixelPermutationVector);

	TShaderMapRef<FSMAAEdgeDetectionVS> VertexShader(View.ShaderMap);

	// Write to stencil
	FRHIDepthStencilState* DSState = TStaticDepthStencilState<
		false, CF_Always, 
		true, CF_Always, SO_Keep, SO_Keep, SO_Replace,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		255, 255
		>::GetRHI();
	
	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("SMAAedgeDetection"),
		View,
		FScreenPassTextureViewport(EdgeTexture),
		FScreenPassTextureViewport(View.ViewRect.Size()),
		FScreenPassPipelineState(VertexShader, PixelShader, FScreenPassPipelineState::FDefaultBlendState::GetRHI(), DSState, 1),
		PassParameters,
		[VertexShader, PixelShader, PassParameters] (FRHICommandList& RHICmdList)
		{
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
		});
}

void AddSMAABlendingWeightCalculationPass(FRDGBuilder& GraphBuilder, const FSceneView& InSceneView, const FRDGTextureRef& EdgeTexture, const FRDGTextureRef& BlendTexture, const FRDGTextureRef& DepthStencilTexture, ESMAAQuality Quality, const FVector4f& RTMetrics)
{
	checkSlow(InSceneView.bIsViewInfo);
	const FViewInfo& View = static_cast<const FViewInfo&>(InSceneView);

	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FSMAABlendingWeightCalculationPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSMAABlendingWeightCalculationPS::FParameters>();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(BlendTexture, ERenderTargetLoadAction::EClear);
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(
		DepthStencilTexture,
		ERenderTargetLoadAction::EClear,
		ERenderTargetLoadAction::ELoad,
		FExclusiveDepthStencil::DepthWrite_StencilRead);
	PassParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();
	PassParameters->Input = GetScreenPassTextureInput(FScreenPassTexture(EdgeTexture), BilinearClampSampler);
	PassParameters->RTMetrics = RTMetrics;
	PassParameters->PointSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->LinearSampler = BilinearClampSampler;

	check(GEngine);
	check(GEngine->SMAAAreaTexture && GEngine->SMAASearchTexture);

	PassParameters->AreaTex = GEngine->SMAAAreaTexture->GetResource()->TextureRHI;
	PassParameters->SearchTex = GEngine->SMAASearchTexture->GetResource()->TextureRHI;
	
	FSMAABlendingWeightCalculationPS::FPermutationDomain PixelPermutationVector;
	SetupSMAAQualityPermutations<FSMAABlendingWeightCalculationPS>(PixelPermutationVector, Quality);
	TShaderMapRef<FSMAABlendingWeightCalculationPS> PixelShader(View.ShaderMap, PixelPermutationVector);

	FSMAABlendingWeightCalculationVS::FPermutationDomain VertexPermutationVector;
	SetupSMAAQualityPermutations<FSMAABlendingWeightCalculationVS>(VertexPermutationVector, Quality);
	TShaderMapRef<FSMAABlendingWeightCalculationVS> VertexShader(View.ShaderMap, VertexPermutationVector);

	// Only draw when stencil value is set by the edge detection pass
	FRHIDepthStencilState* DSState = TStaticDepthStencilState<
		false, CF_Always, 
		true, CF_Equal, SO_Keep, SO_Keep, SO_Keep,
		false, CF_Always, SO_Keep, SO_Keep, SO_Keep,
		255, 255
		>::GetRHI();
	
	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("SMAABlendingWeightCalculation"),
		View,
		FScreenPassTextureViewport(BlendTexture),
		FScreenPassTextureViewport(View.ViewRect.Size()),
		FScreenPassPipelineState(VertexShader, PixelShader, FScreenPassPipelineState::FDefaultBlendState::GetRHI(), DSState, 1),
		PassParameters,
		[VertexShader, PixelShader, PassParameters] (FRHICommandList& RHICmdList)
		{
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
		});
}

FScreenPassTexture AddSMAANeighborhoodBlendingPass(FRDGBuilder& GraphBuilder, const FSceneView& InSceneView, const FScreenPassTexture& InputSceneTexture, const FRDGTextureRef& InputBlendTex, FScreenPassRenderTarget& Output, const FVector4f& RTMetrics)
{
	checkSlow(InSceneView.bIsViewInfo);
	const FViewInfo& View = static_cast<const FViewInfo&>(InSceneView);

	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FSMAANeighborhoodBlendingPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSMAANeighborhoodBlendingPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();
	PassParameters->Input = GetScreenPassTextureInput(InputSceneTexture, BilinearClampSampler);
	PassParameters->RTMetrics = RTMetrics;
	PassParameters->BlendTex = InputBlendTex;
	PassParameters->LinearSampler = BilinearClampSampler;

	FSMAANeighborhoodBlendingPS::FPermutationDomain PixelPermutationVector;
	PixelPermutationVector.Set<FSMAANeighborhoodBlendingPS::FAlphaChannelDim>(IsPostProcessingWithAlphaChannelSupported());
	TShaderMapRef<FSMAANeighborhoodBlendingPS> PixelShader(View.ShaderMap, PixelPermutationVector);

	TShaderMapRef<FSMAANeighborhoodBlendingVS> VertexShader(View.ShaderMap);
	
	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("SMAANeighborhoodBlending"),
		View,
		FScreenPassTextureViewport(Output),
		FScreenPassTextureViewport(InputSceneTexture),
		FScreenPassPipelineState(VertexShader, PixelShader),
		PassParameters,
		[VertexShader, PixelShader, PassParameters] (FRHICommandList& RHICmdList)
		{
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
		});

	return MoveTemp(Output);
}

FScreenPassTexture AddSMAADebugVisualizationPass(FRDGBuilder& GraphBuilder, const FSceneView& InSceneView, const FScreenPassTexture& InputSceneTexture, const FRDGTextureRef& InputEdgeTex, const FRDGTextureRef& InputBlendTex, FScreenPassRenderTarget& Output, const FVector4f& RTMetrics)
{
	checkSlow(InSceneView.bIsViewInfo);
	const FViewInfo& View = static_cast<const FViewInfo&>(InSceneView);

	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	FSMAADebugVisualizationPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSMAADebugVisualizationPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->RenderTargets.MultiViewCount = View.GetMultiViewCount();
	PassParameters->Input = GetScreenPassTextureInput(InputSceneTexture, BilinearClampSampler);
	PassParameters->RTMetrics = RTMetrics;
	PassParameters->EdgeTex = InputEdgeTex;
	PassParameters->BlendTex = InputBlendTex;
	PassParameters->LinearSampler = BilinearClampSampler;
	PassParameters->DebugMode = (int) CVarSMAADebugVisualization.GetValueOnAnyThread();

	TShaderMapRef<FSMAADebugVisualizationVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FSMAADebugVisualizationPS> PixelShader(View.ShaderMap);
	
	AddDrawScreenPass(
		GraphBuilder,
		RDG_EVENT_NAME("SMAADebugVisualization"),
		View,
		FScreenPassTextureViewport(Output),
		FScreenPassTextureViewport(InputSceneTexture),
		FScreenPassPipelineState(VertexShader, PixelShader),
		PassParameters,
		[VertexShader, PixelShader, PassParameters] (FRHICommandList& RHICmdList)
		{
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), *PassParameters);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);
		});

	return MoveTemp(Output);
}

FScreenPassTexture AddSMAAPasses(FRDGBuilder& GraphBuilder, const FSceneView& View, const FSMAAInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	checkSlow(View.bIsViewInfo);
	const FViewInfo& ViewInfo = static_cast<const FViewInfo&>(View);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, SMAA, "SMAA");

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("SMAA"));
	}

	FIntPoint InputExtents = Inputs.SceneColor.Texture->Desc.Extent;
	FIntPoint TexSize = InputExtents;
	QuantizeSceneBufferSize(InputExtents, TexSize);

	FVector4f RTMetrics = FVector4f(1.0f / TexSize.X, 1.0f / TexSize.Y, TexSize.X, TexSize.Y);

	FRDGTextureDesc EdgeTexDesc = View.CreateStereoTextureDesc(TexSize, PF_R8G8, FClearValueBinding(FLinearColor::Transparent), TexCreate_RenderTargetable | TexCreate_ShaderResource);
	FRDGTextureRef EdgeTexture = GraphBuilder.CreateTexture(EdgeTexDesc, TEXT("SMAAEdgeTexture"));

	FRDGTextureDesc BlendTexDesc = View.CreateStereoTextureDesc(TexSize, PF_R8G8B8A8, FClearValueBinding(FLinearColor::Transparent), TexCreate_RenderTargetable | TexCreate_ShaderResource);
	FRDGTextureRef BlendTexture = GraphBuilder.CreateTexture(BlendTexDesc, TEXT("SMAABlendTexture"));

	FRDGTextureDesc DepthStencilDesc = View.CreateStereoTextureDesc(TexSize, PF_DepthStencil, FClearValueBinding(0.0f, 0), TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
	FRDGTextureRef DepthStencilTexture = GraphBuilder.CreateTexture(DepthStencilDesc, TEXT("SMAAStencilTexture"));

	AddSMAAEdgeDetectionPass(GraphBuilder, View, Inputs.SceneColorBeforeTonemap, EdgeTexture, DepthStencilTexture, Inputs.Quality, RTMetrics);

	AddSMAABlendingWeightCalculationPass(GraphBuilder, View, EdgeTexture, BlendTexture, DepthStencilTexture, Inputs.Quality, RTMetrics);

#if !UE_BUILD_SHIPPING
	int32 DebugVisMode = CVarSMAADebugVisualization.GetValueOnAnyThread();
	if (DebugVisMode > 0 && DebugVisMode <= 3)
	{
		return AddSMAADebugVisualizationPass(GraphBuilder, View, Inputs.SceneColor, EdgeTexture, BlendTexture, Output, RTMetrics);
	}
	else
#endif // !UE_BUILD_SHIPPING
	{
		return AddSMAANeighborhoodBlendingPass(GraphBuilder, View, Inputs.SceneColor, BlendTexture, Output, RTMetrics);
	}	
}
