// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/TemporalAA.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "PostProcess/PostProcessTonemap.h"
#include "PostProcess/PostProcessMitchellNetravali.h"
#include "PostProcess/PostProcessing.h"
#include "ClearQuad.h"
#include "PostProcess/PostProcessing.h"
#include "SceneTextureParameters.h"
#include "PixelShaderUtils.h"
#include "ShaderPlatformCachedIniValue.h"
#include "Quantization.h"
#include "SceneViewState.h"

namespace
{

const int32 GTemporalAATileSizeX = 8;
const int32 GTemporalAATileSizeY = 8;

TAutoConsoleVariable<float> CVarTemporalAAFilterSize(
	TEXT("r.TemporalAAFilterSize"),
	1.0f,
	TEXT("Size of the filter kernel. (1.0 = smoother, 0.0 = sharper but aliased)."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarTemporalAACatmullRom(
	TEXT("r.TemporalAACatmullRom"),
	0,
	TEXT("Whether to use a Catmull-Rom filter kernel. Should be a bit sharper than Gaussian."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarTemporalAAPauseCorrect(
	TEXT("r.TemporalAAPauseCorrect"),
	1,
	TEXT("Correct temporal AA in pause. This holds onto render targets longer preventing reuse and consumes more memory."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarTemporalAACurrentFrameWeight(
	TEXT("r.TemporalAACurrentFrameWeight"),
	.04f,
	TEXT("Weight of current frame's contribution to the history.  Low values cause blurriness and ghosting, high values fail to hide jittering."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarTemporalAAQuality(
	TEXT("r.TemporalAA.Quality"), 2,
	TEXT("Quality of the main Temporal AA pass.\n")
	TEXT(" 0: Disable input filtering;\n")
	TEXT(" 1: Enable input filtering;\n")
	TEXT(" 2: Enable more input filtering, enable mobility based anti-ghosting (Default)\n")
	TEXT(" 3: Quality 1 input filtering, enable anti-ghosting"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

TAutoConsoleVariable<float> CVarTemporalAAHistorySP(
	TEXT("r.TemporalAA.HistoryScreenPercentage"),
	100.0f,
	TEXT("Size of temporal AA's history."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarUseTemporalAAUpscaler(
	TEXT("r.TemporalAA.Upscaler"),
	1,
	TEXT("Choose the upscaling algorithm.\n")
	TEXT(" 0: Forces the default temporal upscaler of the renderer;\n")
	TEXT(" 1: GTemporalUpscaler which may be overridden by a third party plugin (default)."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarTAAR11G11B10History(
	TEXT("r.TemporalAA.R11G11B10History"), 1,
	TEXT("Select the bitdepth of the history."),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarTAAUseMobileConfig(
	TEXT("r.TemporalAA.UseMobileConfig"),
	0,
	TEXT("1 to use mobile TAA config. This will disable groupshared caching of the color and depth buffers.\n")
	TEXT(" 0: disabled (default);\n")
	TEXT(" 1: enabled;\n"),
	ECVF_ReadOnly);

TAutoConsoleVariable<int32> CVarTemporalAAMobileUseCompute(
	TEXT("r.TemporalAA.Mobile.UseCompute"),
	1,
	TEXT(" 0: Uses pixel shader to save bandwidth with FBC on tiled gpu;\n")
	TEXT(" 1: Uses compute shader (default);\n"),
	ECVF_RenderThreadSafe);

#if WITH_MGPU
const FName TAAEffectName("TAA");
#endif

inline bool DoesPlatformSupportTemporalHistoryUpscale(EShaderPlatform Platform)
{
	return (IsPCPlatform(Platform) || FDataDrivenShaderPlatformInfo::GetSupportsTemporalHistoryUpscale(Platform))
		&& IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM5);
}

class FTemporalAA : public FGlobalShader
{
public:
	class FAlphaChannelDim : SHADER_PERMUTATION_BOOL("TAA_ALPHA_CHANNEL");
	class FTAAPassConfigDim : SHADER_PERMUTATION_ENUM_CLASS("TAA_PASS_CONFIG", ETAAPassConfig);
	class FTAAQualityDim : SHADER_PERMUTATION_ENUM_CLASS("TAA_QUALITY", ETAAQuality);
	class FTAAScreenPercentageDim : SHADER_PERMUTATION_INT("TAA_SCREEN_PERCENTAGE_RANGE", 4);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, ViewportUVToInputBufferUV)
		SHADER_PARAMETER(FVector4f, MaxViewportUVAndSvPositionToViewportUV)
		SHADER_PARAMETER(FVector2f, ScreenPosAbsMax)
		SHADER_PARAMETER(float, HistoryPreExposureCorrection)
		SHADER_PARAMETER(float, CurrentFrameWeight)
		SHADER_PARAMETER(int32, bCameraCut)

		SHADER_PARAMETER_SCALAR_ARRAY(float, SampleWeights, [9])
		SHADER_PARAMETER_SCALAR_ARRAY(float, PlusWeights, [5])

		SHADER_PARAMETER(FVector4f, InputSceneColorSize)
		SHADER_PARAMETER(FIntPoint, InputMinPixelCoord)
		SHADER_PARAMETER(FIntPoint, InputMaxPixelCoord)
		SHADER_PARAMETER(FVector4f, OutputViewportSize)
		SHADER_PARAMETER(FVector4f, OutputViewportRect)
		SHADER_PARAMETER(FVector3f, OutputQuantizationError)

		// History parameters
		SHADER_PARAMETER(FVector4f, HistoryBufferSize)
		SHADER_PARAMETER(FVector4f, HistoryBufferUVMinMax)
		SHADER_PARAMETER(FVector4f, ScreenPosToHistoryBufferUV)

		SHADER_PARAMETER(float, CoCBilateralFilterStrength)

		// Temporal upsample specific parameters.
		SHADER_PARAMETER(FVector4f, InputViewSize)
		SHADER_PARAMETER(FVector2f, InputViewMin)
		SHADER_PARAMETER(FVector2f, TemporalJitterPixels)
		SHADER_PARAMETER(float, ScreenPercentage)
		SHADER_PARAMETER(float, UpscaleFactor)

		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, EyeAdaptationBuffer)

		// Inputs
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputSceneColor)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSceneColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputSceneMetadata)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSceneMetadataSampler)

		// History resources
		SHADER_PARAMETER_RDG_TEXTURE_ARRAY(Texture2D, HistoryBuffer, [FTemporalAAHistory::kRenderTargetCount])
		SHADER_PARAMETER_SAMPLER_ARRAY(SamplerState, HistoryBufferSampler, [FTemporalAAHistory::kRenderTargetCount])

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferVelocityTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, GBufferVelocityTextureSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, GBufferVelocityTextureSRV)

		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, StencilTexture)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		static FShaderPlatformCachedIniValue<bool> UseMobileConfig(TEXT("r.TemporalAA.UseMobileConfig"));
		bool bUseMobileConfig = (UseMobileConfig.Get((EShaderPlatform)Parameters.Platform) != 0);

		bool bIsMobileTiledGPU = RHIHasTiledGPU(Parameters.Platform) || IsSimulatedPlatform(Parameters.Platform);

		// There are some mobile specific shader optimizations need to be set in the shader, such as disable shared memory usage, disable stencil texture sampling.
		OutEnvironment.SetDefine(TEXT("AA_MOBILE_CONFIG"), (bIsMobileTiledGPU || bUseMobileConfig) ? 1 : 0);
	}

	FTemporalAA() = default;
	FTemporalAA(const CompiledShaderInitializerType & Initializer)
	:	FGlobalShader(Initializer)
	{}
}; // class FTemporalAA

class FTemporalAAPS : public FTemporalAA
{
	DECLARE_GLOBAL_SHADER(FTemporalAAPS);
	SHADER_USE_PARAMETER_STRUCT(FTemporalAAPS, FTemporalAA);

	using FPermutationDomain = TShaderPermutationDomain<
		FTemporalAA::FAlphaChannelDim,
		FTemporalAA::FTAAPassConfigDim,
		FTemporalAA::FTAAQualityDim,
		FTemporalAA::FTAAScreenPercentageDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FTemporalAA::FParameters, Common)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
		
	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::Main ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::MainUpsampling ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::MainSuperSampling ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::DiaphragmDOF ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::DiaphragmDOFUpsampling)
		{
			// NOP
		}
		else
		{
			// Alpha channel is not supported on these permutations
			PermutationVector.Set<FAlphaChannelDim>(false);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// Don't compile the shader permutation if gets remaped at runtime.
		if (PermutationVector != RemapPermutation(PermutationVector))
		{
			return false;
		}

		// Pixel shader is only used on mobile to utilize the hardware frame buffer compression to save bandwidth
		if (!IsMobilePlatform(Parameters.Platform))
		{
			return false;
		}

		// Screen percentage dimension is only for upsampling permutation.
		if (!IsTAAUpsamplingConfig(PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>()) &&
			PermutationVector.Get<FTemporalAA::FTAAScreenPercentageDim>() != 0)
		{
			return false;
		}

		if (PermutationVector.Get<FTAAPassConfigDim>() == ETAAPassConfig::MainSuperSampling)
		{
			// Super sampling is only available in certain configurations.
			if (!DoesPlatformSupportTemporalHistoryUpscale(Parameters.Platform))
			{
				return false;
			}
		}

		// Screen percentage range 3 is only for super sampling.
		if (PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() != ETAAPassConfig::MainSuperSampling &&
			PermutationVector.Get<FTemporalAA::FTAAScreenPercentageDim>() == 3)
		{
			return false;
		}

		// Only Main and MainUpsampling config are supported on pixel shader.
		if (PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() != ETAAPassConfig::Main &&
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() != ETAAPassConfig::MainUpsampling)
		{
			return false;
		}

		return SupportsGen4TAA(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FTemporalAA::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}
}; // class FTemporalAAPS

class FTemporalAACS : public FTemporalAA
{
	DECLARE_GLOBAL_SHADER(FTemporalAACS);
	SHADER_USE_PARAMETER_STRUCT(FTemporalAACS, FTemporalAA);

	class FTAADownsampleDim : SHADER_PERMUTATION_BOOL("TAA_DOWNSAMPLE");

	using FPermutationDomain = TShaderPermutationDomain<
		FTemporalAA::FAlphaChannelDim,
		FTemporalAA::FTAAPassConfigDim,
		FTemporalAA::FTAAQualityDim,
		FTemporalAA::FTAAScreenPercentageDim,
		FTAADownsampleDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FTemporalAA::FParameters, Common)

		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D, OutComputeTex, [FTemporalAAHistory::kRenderTargetCount])
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutComputeTexDownsampled)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, DebugOutput)
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		if (PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::Main ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::MainUpsampling)
		{
			// No point downsampling if not using the faster quality permutation already.
			if (PermutationVector.Get<FTemporalAA::FTAAQualityDim>() == ETAAQuality::High)
			{
				PermutationVector.Set<FTAADownsampleDim>(false);
			}
		}
		else if (
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::DiaphragmDOF ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::DiaphragmDOFUpsampling)
		{
			// DOF pass allow only quality 1 and 2
			PermutationVector.Set<FTemporalAA::FTAAQualityDim>(ETAAQuality(FMath::Max(int(PermutationVector.Get<FTemporalAA::FTAAQualityDim>()), int(ETAAQuality::Medium))));

			// Only the Main and Main Upsampling can downsample the output.
			PermutationVector.Set<FTAADownsampleDim>(false);
		}
		else
		{
			// Only the Main and Main Upsampling have quality options 0, 1, 2, 3
			PermutationVector.Set<FTemporalAA::FTAAQualityDim>(ETAAQuality::High);

			// Only the Main and Main Upsampling can downsample the output.
			PermutationVector.Set<FTAADownsampleDim>(false);
		}

		if (PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::Main ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::MainUpsampling ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::MainSuperSampling ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::DiaphragmDOF ||
			PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() == ETAAPassConfig::DiaphragmDOFUpsampling)
		{
			// NOP
		}
		else
		{
			// Alpha channel is not supported on these permutations
			PermutationVector.Set<FAlphaChannelDim>(false);
		}

		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// Don't compile the shader permutation if gets remaped at runtime.
		if (PermutationVector != RemapPermutation(PermutationVector))
		{
			return false;
		}

		// Screen percentage dimension is only for upsampling permutation.
		if (!IsTAAUpsamplingConfig(PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>()) &&
			PermutationVector.Get<FTemporalAA::FTAAScreenPercentageDim>() != 0)
		{
			return false;
		}

		if (PermutationVector.Get<FTAAPassConfigDim>() == ETAAPassConfig::MainSuperSampling)
		{
			// Super sampling is only available in certain configurations.
			if (!DoesPlatformSupportTemporalHistoryUpscale(Parameters.Platform))
			{
				return false;
			}
		}

		// Screen percentage range 3 is only for super sampling.
		if (PermutationVector.Get<FTemporalAA::FTAAPassConfigDim>() != ETAAPassConfig::MainSuperSampling &&
			PermutationVector.Get<FTemporalAA::FTAAScreenPercentageDim>() == 3)
		{
			return false;
		}
		
		if (IsMobilePlatform(Parameters.Platform))
		{
			// Only Main and MainUpsampling config are supported on mobile platform.
			if ((PermutationVector.Get<FTAAPassConfigDim>() != ETAAPassConfig::Main && PermutationVector.Get<FTAAPassConfigDim>() != ETAAPassConfig::MainUpsampling)
				// DownSample is not supported on mobile platform
				|| PermutationVector.Get<FTAADownsampleDim>())
			{
				return false;
			}
		}

		return SupportsGen4TAA(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FTemporalAA::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), GTemporalAATileSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), GTemporalAATileSizeY);
	}
}; // class FTemporalAACS

IMPLEMENT_GLOBAL_SHADER(FTemporalAAPS, "/Engine/Private/TemporalAA.usf", "MainPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FTemporalAACS, "/Engine/Private/TemporalAA.usf", "MainCS", SF_Compute);

float CatmullRom(float x)
{
	float ax = FMath::Abs(x);
	if (ax > 1.0f)
		return ((-0.5f * ax + 2.5f) * ax - 4.0f) *ax + 2.0f;
	else
		return (1.5f * ax - 2.5f) * ax*ax + 1.0f;
}

void SetupSampleWeightParameters(FTemporalAA::FParameters* OutTAAParameters, const FTAAPassParameters& PassParameters, FVector2f TemporalJitterPixels)
{
	float JitterX = TemporalJitterPixels.X;
	float JitterY = TemporalJitterPixels.Y;
	float ResDivisorInv = 1.0f / float(PassParameters.ResolutionDivisor);

	static const float SampleOffsets[9][2] =
	{
		{ -1.0f, -1.0f },
		{  0.0f, -1.0f },
		{  1.0f, -1.0f },
		{ -1.0f,  0.0f },
		{  0.0f,  0.0f },
		{  1.0f,  0.0f },
		{ -1.0f,  1.0f },
		{  0.0f,  1.0f },
		{  1.0f,  1.0f },
	};

	float FilterSize = CVarTemporalAAFilterSize.GetValueOnRenderThread();
	int32 bCatmullRom = CVarTemporalAACatmullRom.GetValueOnRenderThread();

	// Compute 3x3 weights
	{
		float TotalWeight = 0.0f;
		for (int32 i = 0; i < 9; i++)
		{
			float PixelOffsetX = SampleOffsets[i][0] - JitterX * ResDivisorInv;
			float PixelOffsetY = SampleOffsets[i][1] - JitterY * ResDivisorInv;

			PixelOffsetX /= FilterSize;
			PixelOffsetY /= FilterSize;

			if (bCatmullRom)
			{
				const float CurrWeight = CatmullRom(PixelOffsetX) * CatmullRom(PixelOffsetY);
				GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, i) = CurrWeight;
				TotalWeight += CurrWeight;
			}
			else
			{
				// Normal distribution, Sigma = 0.47
				const float CurrWeight = FMath::Exp(-2.29f * (PixelOffsetX * PixelOffsetX + PixelOffsetY * PixelOffsetY));
				GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, i) = CurrWeight;
				TotalWeight += CurrWeight;
			}
		}
	
		for (int32 i = 0; i < 9; i++)
		{
			GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, i) /= TotalWeight;
		}
	}

	// Compute 3x3 + weights.
	{
		GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->PlusWeights, 0) = GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 1);
		GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->PlusWeights, 1) = GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 3);
		GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->PlusWeights, 2) = GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 4);
		GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->PlusWeights, 3) = GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 5);
		GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->PlusWeights, 4) = GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 7);
		float TotalWeightPlus = (
			GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 1) +
			GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 3) +
			GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 4) +
			GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 5) +
			GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->SampleWeights, 7));
	
		for (int32 i = 0; i < 5; i++)
		{
			GET_SCALAR_ARRAY_ELEMENT(OutTAAParameters->PlusWeights, i) /= TotalWeightPlus;
		}
	}
}

DECLARE_GPU_STAT(TAA);

const TCHAR* const kTAAOutputNames[] = {
	TEXT("TAA.History"),
	TEXT("TAA.History"),
	TEXT("TAA.History"),
	TEXT("SSR.TemporalAA"),
	TEXT("LightShaft.TemporalAA"),
	TEXT("DOF.TemporalAA"),
	TEXT("DOF.TemporalAA"),
	TEXT("Hair.TemporalAA"),
};

const TCHAR* const kTAAPassNames[] = {
	TEXT("Main"),
	TEXT("MainUpsampling"),
	TEXT("MainSuperSampling"),
	TEXT("ScreenSpaceReflections"),
	TEXT("LightShaft"),
	TEXT("DOF"),
	TEXT("DOFUpsampling"),
	TEXT("Hair"),
};

const TCHAR* const kTAAQualityNames[] = {
	TEXT("Low"),
	TEXT("Medium"),
	TEXT("High"),
	TEXT("MediumHigh"),
};

static_assert(UE_ARRAY_COUNT(kTAAOutputNames) == int32(ETAAPassConfig::MAX), "Missing TAA output name.");
static_assert(UE_ARRAY_COUNT(kTAAPassNames) == int32(ETAAPassConfig::MAX), "Missing TAA pass name.");
static_assert(UE_ARRAY_COUNT(kTAAQualityNames) == int32(ETAAQuality::MAX), "Missing TAA quality name.");
} //! namespace

float GetTemporalAAHistoryUpscaleFactor(const FViewInfo& View)
{
	float UpscaleFactor = 1.0f;

	// We only support history upscale in certain configurations.
	if (DoesPlatformSupportTemporalHistoryUpscale(View.GetShaderPlatform()))
	{
		UpscaleFactor = FMath::Clamp(CVarTemporalAAHistorySP.GetValueOnRenderThread() / 100.0f, 1.0f, 2.0f);
	}

	return UpscaleFactor;
}

bool DoesTemporalAAUseComputeShader(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> TemporalAAMobileUseCompute(TEXT("r.TemporalAA.Mobile.UseCompute"));
	return !IsMobilePlatform(Platform) || (TemporalAAMobileUseCompute.Get(Platform) > 0);
}

FIntPoint FTAAPassParameters::GetOutputExtent() const
{
	check(Validate());
	check(SceneColorInput);

	FIntPoint InputExtent = SceneColorInput->Desc.Extent;

	if (!IsTAAUpsamplingConfig(Pass))
		return InputExtent;

	check(OutputViewRect.Min == FIntPoint::ZeroValue);
	FIntPoint PrimaryUpscaleViewSize = FIntPoint::DivideAndRoundUp(OutputViewRect.Size(), ResolutionDivisor);
	FIntPoint QuantizedPrimaryUpscaleViewSize;
	QuantizeSceneBufferSize(PrimaryUpscaleViewSize, QuantizedPrimaryUpscaleViewSize);

	return FIntPoint(
		FMath::Max(InputExtent.X, QuantizedPrimaryUpscaleViewSize.X),
		FMath::Max(InputExtent.Y, QuantizedPrimaryUpscaleViewSize.Y));
}

bool FTAAPassParameters::Validate() const
{
	if (IsTAAUpsamplingConfig(Pass))
	{
		check(OutputViewRect.Min == FIntPoint::ZeroValue);
	}
	else
	{
		check(InputViewRect == OutputViewRect);
	}
	return true;
}

FTAAOutputs AddTemporalAAPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FTAAPassParameters& Inputs,
	const FTemporalAAHistory& InputHistory,
	FTemporalAAHistory* OutputHistory)
{
	check(Inputs.Validate());

	// Whether alpha channel is supported.
	const bool bSupportsAlpha = IsPostProcessingWithAlphaChannelSupported();

	// Number of render target in TAA history.
	const int32 IntputTextureCount = (IsDOFTAAConfig(Inputs.Pass) && bSupportsAlpha) ? 2 : 1;

	// Whether this is main TAA pass;
	const bool bIsMainPass = IsMainTAAConfig(Inputs.Pass);

	// Whether to use camera cut shader permutation or not.
	const bool bCameraCut = !InputHistory.IsValid() || View.bCameraCut || InputHistory.RT[0]->GetDesc().IsArray();

	const FIntPoint OutputExtent = Inputs.GetOutputExtent();

	// Src rectangle.
	const FIntRect SrcRect = Inputs.InputViewRect;
	const FIntRect DestRect = Inputs.OutputViewRect;
	const FIntRect PracticableSrcRect = FIntRect::DivideAndRoundUp(SrcRect, Inputs.ResolutionDivisor);
	const FIntRect PracticableDestRect = FIntRect::DivideAndRoundUp(DestRect, Inputs.ResolutionDivisor);

	const uint32 PassIndex = static_cast<uint32>(Inputs.Pass);

	// Name of the pass.
	const TCHAR* PassName = kTAAPassNames[PassIndex];

	// Create outputs
	FTAAOutputs Outputs;

	TStaticArray<FRDGTextureRef, FTemporalAAHistory::kRenderTargetCount> NewHistoryTexture;

	const bool bIsComputePass = DoesTemporalAAUseComputeShader(View.GetShaderPlatform());

	{
		EPixelFormat HistoryPixelFormat = PF_FloatRGBA;
		if (bIsMainPass && 
			(Inputs.Quality != ETAAQuality::High) && (Inputs.Quality != ETAAQuality::MediumHigh) && 
			!bSupportsAlpha && CVarTAAR11G11B10History.GetValueOnRenderThread())
		{
			HistoryPixelFormat = PF_FloatR11G11B10;
		}

		FRDGTextureDesc SceneColorDesc = FRDGTextureDesc::Create2D(
			OutputExtent,
			HistoryPixelFormat,
			FClearValueBinding::Black,
			TexCreate_ShaderResource);

		if (bIsComputePass)
		{
			SceneColorDesc.Flags |= TexCreate_UAV;
		}

		if (Inputs.bOutputRenderTargetable || !bIsComputePass)
		{
			SceneColorDesc.Flags |= TexCreate_RenderTargetable;
		}

		const TCHAR* OutputName = kTAAOutputNames[PassIndex];

		for (int32 i = 0; i < FTemporalAAHistory::kRenderTargetCount; i++)
		{
			NewHistoryTexture[i] = GraphBuilder.CreateTexture(
				SceneColorDesc,
				OutputName,
				ERDGTextureFlags::MultiFrame);
		}

		NewHistoryTexture[0] = Outputs.SceneColor = NewHistoryTexture[0];

		if (IntputTextureCount == 2)
		{
			Outputs.SceneMetadata = NewHistoryTexture[1];
		}

		if (Inputs.bDownsample)
		{
			check(bIsComputePass);
			if (Inputs.DownsampleRenderTarget)
			{
				Outputs.DownsampledSceneColor = Inputs.DownsampleRenderTarget;
			}
			else
			{
				const FRDGTextureDesc HalfResSceneColorDesc = FRDGTextureDesc::Create2D(
					SceneColorDesc.Extent / 2,
					Inputs.DownsampleOverrideFormat != PF_Unknown ? Inputs.DownsampleOverrideFormat : Inputs.SceneColorInput->Desc.Format,
					FClearValueBinding::Black,
					TexCreate_ShaderResource | TexCreate_UAV | GFastVRamConfig.Downsample);

				Outputs.DownsampledSceneColor = GraphBuilder.CreateTexture(HalfResSceneColorDesc, TEXT("SceneColorHalfRes"));
			}
		}
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, TAA, "TAA");

	TStaticArray<bool, FTemporalAAHistory::kRenderTargetCount> bUseHistoryTexture;

	// Setups common shader parameters
	const FIntPoint InputExtent = Inputs.SceneColorInput->Desc.Extent;
	const FIntRect InputViewRect = Inputs.InputViewRect;
	const FIntRect OutputViewRect = Inputs.OutputViewRect;

	auto SetupTemporalAACommonPassParameters = [&](FTemporalAA::FParameters* PassParameters)
	{
		if (!IsTAAUpsamplingConfig(Inputs.Pass))
		{
			SetupSampleWeightParameters(PassParameters, Inputs, FVector2f(View.TemporalJitterPixels));
		}

		const float ResDivisor = Inputs.ResolutionDivisor;
		const float ResDivisorInv = 1.0f / ResDivisor;

		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->CurrentFrameWeight = CVarTemporalAACurrentFrameWeight.GetValueOnRenderThread();
		PassParameters->bCameraCut = bCameraCut;

		PassParameters->SceneDepthTexture = Inputs.SceneDepthTexture;
		PassParameters->GBufferVelocityTexture = Inputs.SceneVelocityTexture;

		PassParameters->SceneDepthTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		PassParameters->GBufferVelocityTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();

		FRDGTextureRef StencilSource = Inputs.StencilDepthTexture ? Inputs.StencilDepthTexture : Inputs.SceneDepthTexture;
		PassParameters->StencilTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(StencilSource, PF_X24_G8));

		// We need a valid velocity buffer texture. Use black (no velocity) if none exists.
		if (!PassParameters->GBufferVelocityTexture)
		{
			PassParameters->GBufferVelocityTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);;
		}

		// Input buffer shader parameters
		{
			PassParameters->InputSceneColorSize = FVector4f(
				InputExtent.X,
				InputExtent.Y,
				1.0f / float(InputExtent.X),
				1.0f / float(InputExtent.Y));
			PassParameters->InputMinPixelCoord = PracticableSrcRect.Min;
			PassParameters->InputMaxPixelCoord = PracticableSrcRect.Max - FIntPoint(1, 1);
			PassParameters->InputSceneColor = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(Inputs.SceneColorInput, 0));
			PassParameters->InputSceneColorSampler = TStaticSamplerState<SF_Point>::GetRHI();
			PassParameters->InputSceneMetadata = Inputs.SceneMetadataInput;
			PassParameters->InputSceneMetadataSampler = TStaticSamplerState<SF_Point>::GetRHI();
		}

		// Temporal upsample specific shader parameters.
		{
			// Temporal AA upscale specific params.
			float InputViewSizeInvScale = Inputs.ResolutionDivisor;
			float InputViewSizeScale = 1.0f / InputViewSizeInvScale;

			PassParameters->TemporalJitterPixels = InputViewSizeScale * FVector2f(View.TemporalJitterPixels);
			PassParameters->ScreenPercentage = float(InputViewRect.Width()) / float(OutputViewRect.Width());
			PassParameters->UpscaleFactor = float(OutputViewRect.Width()) / float(InputViewRect.Width());
			PassParameters->InputViewMin = InputViewSizeScale * FVector2f(InputViewRect.Min.X, InputViewRect.Min.Y);
			PassParameters->InputViewSize = FVector4f(
				InputViewSizeScale * InputViewRect.Width(), InputViewSizeScale * InputViewRect.Height(),
				InputViewSizeInvScale / InputViewRect.Width(), InputViewSizeInvScale / InputViewRect.Height());
		}

		PassParameters->CoCBilateralFilterStrength = Inputs.CoCBilateralFilterStrength;

		PassParameters->OutputViewportSize = FVector4f(
			PracticableDestRect.Width(), PracticableDestRect.Height(), 1.0f / float(PracticableDestRect.Width()), 1.0f / float(PracticableDestRect.Height()));
		PassParameters->OutputViewportRect = FVector4f(PracticableDestRect.Min.X, PracticableDestRect.Min.Y, PracticableDestRect.Max.X, PracticableDestRect.Max.Y);
		PassParameters->OutputQuantizationError = (FVector3f)ComputePixelFormatQuantizationError(NewHistoryTexture[0]->Desc.Format);

		// Set history shader parameters.
		{
			FRDGTextureRef BlackDummy = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);

			if (bCameraCut)
			{
				PassParameters->ScreenPosToHistoryBufferUV = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
				PassParameters->ScreenPosAbsMax = FVector2f(0.0f, 0.0f);
				PassParameters->HistoryBufferUVMinMax = FVector4f(0.0f, 0.0f, 0.0f, 0.0f);
				PassParameters->HistoryBufferSize = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);

				for (int32 i = 0; i < FTemporalAAHistory::kRenderTargetCount; i++)
				{
					PassParameters->HistoryBuffer[i] = BlackDummy;
				}

				// Remove dependency of the velocity buffer on camera cut, given it's going to be ignored by the shader.
				PassParameters->GBufferVelocityTexture = BlackDummy;
			}
			else
			{
				FIntPoint ReferenceViewportOffset = InputHistory.ViewportRect.Min;
				FIntPoint ReferenceViewportExtent = InputHistory.ViewportRect.Size();
				FIntPoint ReferenceBufferSize = InputHistory.ReferenceBufferSize;

				float InvReferenceBufferSizeX = 1.f / float(InputHistory.ReferenceBufferSize.X);
				float InvReferenceBufferSizeY = 1.f / float(InputHistory.ReferenceBufferSize.Y);

				PassParameters->ScreenPosToHistoryBufferUV = FVector4f(
					ReferenceViewportExtent.X * 0.5f * InvReferenceBufferSizeX,
					-ReferenceViewportExtent.Y * 0.5f * InvReferenceBufferSizeY,
					(ReferenceViewportExtent.X * 0.5f + ReferenceViewportOffset.X) * InvReferenceBufferSizeX,
					(ReferenceViewportExtent.Y * 0.5f + ReferenceViewportOffset.Y) * InvReferenceBufferSizeY);

				FIntPoint ViewportOffset = ReferenceViewportOffset / Inputs.ResolutionDivisor;
				FIntPoint ViewportExtent = FIntPoint::DivideAndRoundUp(ReferenceViewportExtent, Inputs.ResolutionDivisor);
				FIntPoint BufferSize = ReferenceBufferSize / Inputs.ResolutionDivisor;

				PassParameters->ScreenPosAbsMax = FVector2f(1.0f - 1.0f / float(ViewportExtent.X), 1.0f - 1.0f / float(ViewportExtent.Y));

				float InvBufferSizeX = 1.f / float(BufferSize.X);
				float InvBufferSizeY = 1.f / float(BufferSize.Y);

				PassParameters->HistoryBufferUVMinMax = FVector4f(
					(ViewportOffset.X + 0.5f) * InvBufferSizeX,
					(ViewportOffset.Y + 0.5f) * InvBufferSizeY,
					(ViewportOffset.X + ViewportExtent.X - 0.5f) * InvBufferSizeX,
					(ViewportOffset.Y + ViewportExtent.Y - 0.5f) * InvBufferSizeY);

				PassParameters->HistoryBufferSize = FVector4f(BufferSize.X, BufferSize.Y, InvBufferSizeX, InvBufferSizeY);

				for (int32 i = 0; i < FTemporalAAHistory::kRenderTargetCount; i++)
				{
					if (InputHistory.RT[i].IsValid())
					{
						PassParameters->HistoryBuffer[i] = GraphBuilder.RegisterExternalTexture(InputHistory.RT[i]);
					}
					else
					{
						PassParameters->HistoryBuffer[i] = BlackDummy;
					}
				}
			}

			for (int32 i = 0; i < FTemporalAAHistory::kRenderTargetCount; i++)
			{
				PassParameters->HistoryBufferSampler[i] = TStaticSamplerState<SF_Bilinear>::GetRHI();
			}
		}

		PassParameters->MaxViewportUVAndSvPositionToViewportUV = FVector4f(
			(PracticableDestRect.Width() - 0.5f * ResDivisor) / float(PracticableDestRect.Width()),
			(PracticableDestRect.Height() - 0.5f * ResDivisor) / float(PracticableDestRect.Height()),
			ResDivisor / float(DestRect.Width()),
			ResDivisor / float(DestRect.Height()));

		PassParameters->HistoryPreExposureCorrection = View.PreExposure / View.PrevViewInfo.SceneColorPreExposure;

		{
			float InvSizeX = 1.0f / float(InputExtent.X);
			float InvSizeY = 1.0f / float(InputExtent.Y);
			PassParameters->ViewportUVToInputBufferUV = FVector4f(
				ResDivisorInv * InputViewRect.Width() * InvSizeX,
				ResDivisorInv * InputViewRect.Height() * InvSizeY,
				ResDivisorInv * InputViewRect.Min.X * InvSizeX,
				ResDivisorInv * InputViewRect.Min.Y * InvSizeY);
		}

		PassParameters->GBufferVelocityTextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(PassParameters->GBufferVelocityTexture));

		PassParameters->EyeAdaptationBuffer = GraphBuilder.CreateSRV(GetEyeAdaptationBuffer(GraphBuilder, View));
	};

	if (bIsComputePass)
	{
		FTemporalAACS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FTemporalAA::FAlphaChannelDim>(bSupportsAlpha);
		PermutationVector.Set<FTemporalAA::FTAAPassConfigDim>(Inputs.Pass);
		PermutationVector.Set<FTemporalAA::FTAAQualityDim>(Inputs.Quality);
		PermutationVector.Set<FTemporalAACS::FTAADownsampleDim>(Inputs.bDownsample);

		if (IsTAAUpsamplingConfig(Inputs.Pass))
		{
			// If screen percentage > 100% on X or Y axes, then use screen percentage range = 2 shader permutation to disable LDS caching.
			if (SrcRect.Width() > DestRect.Width() ||
				SrcRect.Height() > DestRect.Height())
			{
				PermutationVector.Set<FTemporalAA::FTAAScreenPercentageDim>(2);
			}
			// If screen percentage < 50% on X and Y axes, then use screen percentage range = 3 shader permutation.
			else if (SrcRect.Width() * 100 < 50 * DestRect.Width() &&
				SrcRect.Height() * 100 < 50 * DestRect.Height() &&
				Inputs.Pass == ETAAPassConfig::MainSuperSampling)
			{
				PermutationVector.Set<FTemporalAA::FTAAScreenPercentageDim>(3);
			}
			// If screen percentage < 71% on X and Y axes, then use screen percentage range = 1 shader permutation to have smaller LDS caching.
			else if (SrcRect.Width() * 100 < 71 * DestRect.Width() &&
				SrcRect.Height() * 100 < 71 * DestRect.Height())
			{
				PermutationVector.Set<FTemporalAA::FTAAScreenPercentageDim>(1);
			}
		}

		PermutationVector = FTemporalAACS::RemapPermutation(PermutationVector);

		FTemporalAACS::FParameters* PassParameters = GraphBuilder.AllocParameters<FTemporalAACS::FParameters>();

		SetupTemporalAACommonPassParameters(&PassParameters->Common);

		// UAVs
		{
			for (int32 i = 0; i < FTemporalAAHistory::kRenderTargetCount; i++)
			{
				PassParameters->OutComputeTex[i] = GraphBuilder.CreateUAV(NewHistoryTexture[i]);
			}

			if (Outputs.DownsampledSceneColor)
			{
				if (Inputs.DownsampleRenderTarget)
				{
					// Use the specified render target and mip level
					check(Inputs.DownsampleRenderTarget->Desc.NumMips > Inputs.DownsampleRenderTargetMipLevel);
					PassParameters->OutComputeTexDownsampled = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(Inputs.DownsampleRenderTarget, Inputs.DownsampleRenderTargetMipLevel));
				}
				else
				{
					PassParameters->OutComputeTexDownsampled = GraphBuilder.CreateUAV(Outputs.DownsampledSceneColor);
				}
			}
		}

		// Debug UAVs
		{
			FRDGTextureDesc DebugDesc = FRDGTextureDesc::Create2D(
				OutputExtent,
				PF_FloatRGBA,
				FClearValueBinding::None,
				/* InFlags = */ TexCreate_ShaderResource | TexCreate_UAV);

			FRDGTextureRef DebugTexture = GraphBuilder.CreateTexture(DebugDesc, TEXT("Debug.TAA"));
			PassParameters->DebugOutput = GraphBuilder.CreateUAV(DebugTexture);
		}

		TShaderMapRef<FTemporalAACS> ComputeShader(View.ShaderMap, PermutationVector);

		ClearUnusedGraphResources(ComputeShader, PassParameters);
		for (int32 i = 0; i < FTemporalAAHistory::kRenderTargetCount; i++)
		{
			bUseHistoryTexture[i] = PassParameters->Common.HistoryBuffer[i] != nullptr;
		}

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("TAA(%s Quality=%s%s) %dx%d -> %dx%d",
				PassName,
				kTAAQualityNames[int32(PermutationVector.Get<FTemporalAA::FTAAQualityDim>())],
				PermutationVector.Get<FTemporalAA::FAlphaChannelDim>() ? TEXT(" Alpha") : TEXT(""),
				PracticableSrcRect.Width(), PracticableSrcRect.Height(),
				PracticableDestRect.Width(), PracticableDestRect.Height()),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(PracticableDestRect.Size(), GTemporalAATileSizeX));
	}
	else
	{
		check(IsMobilePlatform(View.GetShaderPlatform()));

		FTemporalAAPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FTemporalAA::FAlphaChannelDim>(bSupportsAlpha);
		PermutationVector.Set<FTemporalAA::FTAAPassConfigDim>(Inputs.Pass);
		PermutationVector.Set<FTemporalAA::FTAAQualityDim>(Inputs.Quality);

		if (IsTAAUpsamplingConfig(Inputs.Pass))
		{
			// If screen percentage > 100% on X or Y axes, then use screen percentage range = 2 shader permutation to disable LDS caching.
			if (SrcRect.Width() > DestRect.Width() ||
				SrcRect.Height() > DestRect.Height())
			{
				PermutationVector.Set<FTemporalAA::FTAAScreenPercentageDim>(2);
			}
			// If screen percentage < 50% on X and Y axes, then use screen percentage range = 3 shader permutation.
			else if (SrcRect.Width() * 100 < 50 * DestRect.Width() &&
				SrcRect.Height() * 100 < 50 * DestRect.Height() &&
				Inputs.Pass == ETAAPassConfig::MainSuperSampling)
			{
				PermutationVector.Set<FTemporalAA::FTAAScreenPercentageDim>(3);
			}
			// If screen percentage < 71% on X and Y axes, then use screen percentage range = 1 shader permutation to have smaller LDS caching.
			else if (SrcRect.Width() * 100 < 71 * DestRect.Width() &&
				SrcRect.Height() * 100 < 71 * DestRect.Height())
			{
				PermutationVector.Set<FTemporalAA::FTAAScreenPercentageDim>(1);
			}
		}

		PermutationVector = FTemporalAAPS::RemapPermutation(PermutationVector);

		FTemporalAAPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FTemporalAAPS::FParameters>();

		SetupTemporalAACommonPassParameters(&PassParameters->Common);

		PassParameters->RenderTargets[0] = FRenderTargetBinding(
			Outputs.SceneColor,
			ERenderTargetLoadAction::EClear);

		TShaderMapRef<FTemporalAAPS> PixelShader(View.ShaderMap, PermutationVector);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("TAA(%s Quality=%s%s) %dx%d -> %dx%d",
				PassName,
				kTAAQualityNames[int32(PermutationVector.Get<FTemporalAA::FTAAQualityDim>())],
				PermutationVector.Get<FTemporalAA::FAlphaChannelDim>() ? TEXT(" Alpha") : TEXT(""),
				PracticableSrcRect.Width(), PracticableSrcRect.Height(),
				PracticableDestRect.Width(), PracticableDestRect.Height()),
			PixelShader,
			PassParameters,
			PracticableDestRect);

		for (int32 i = 0; i < FTemporalAAHistory::kRenderTargetCount; i++)
		{
			bUseHistoryTexture[i] = PassParameters->Common.HistoryBuffer[i] != nullptr;
		}
	}

	if (!View.bStatePrevViewInfoIsReadOnly)
	{
		OutputHistory->SafeRelease();

		for (int32 i = 0; i < FTemporalAAHistory::kRenderTargetCount; i++)
		{
			if (bUseHistoryTexture[i])
			{
				GraphBuilder.QueueTextureExtraction(NewHistoryTexture[i], &OutputHistory->RT[i]);
			}
		}

		OutputHistory->ViewportRect = DestRect;
		OutputHistory->ReferenceBufferSize = OutputExtent * Inputs.ResolutionDivisor;
	}

	return Outputs;
} // AddTemporalAAPass()

FDefaultTemporalUpscaler::FOutputs AddGen4MainTemporalAAPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FDefaultTemporalUpscaler::FInputs& PassInputs)
{
	check(View.ViewState);

	FTAAPassParameters TAAParameters(View);

	TAAParameters.Pass = View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale
		? ETAAPassConfig::MainUpsampling
		: ETAAPassConfig::Main;

	TAAParameters.SetupViewRect(View);

	TAAParameters.Quality = ETAAQuality(FMath::Clamp(CVarTemporalAAQuality.GetValueOnRenderThread(), 0, int32(ETAAQuality::MAX) - 1));

	const FIntRect SecondaryViewRect = TAAParameters.OutputViewRect;

	const float HistoryUpscaleFactor = GetTemporalAAHistoryUpscaleFactor(View);

	// Configures TAA to upscale the history buffer; this is in addition to the secondary screen percentage upscale.
	// We end up with a scene color that is larger than the secondary screen percentage. We immediately downscale
	// afterwards using a Mitchel-Netravali filter.
	if (HistoryUpscaleFactor > 1.0f)
	{
		const FIntPoint HistoryViewSize(
			TAAParameters.OutputViewRect.Width() * HistoryUpscaleFactor,
			TAAParameters.OutputViewRect.Height() * HistoryUpscaleFactor);

		TAAParameters.Pass = ETAAPassConfig::MainSuperSampling;
		TAAParameters.Quality = ETAAQuality::High;

		TAAParameters.OutputViewRect.Min.X = 0;
		TAAParameters.OutputViewRect.Min.Y = 0;
		TAAParameters.OutputViewRect.Max = HistoryViewSize;
	}

	// Post process material access the alpha after TAA pass requires the buffer to be rendertargetable in the main pass.
	TAAParameters.bOutputRenderTargetable = TAAParameters.Pass == ETAAPassConfig::Main || 
		                                    TAAParameters.Pass == ETAAPassConfig::MainUpsampling || 
		                                    TAAParameters.Pass == ETAAPassConfig::MainSuperSampling;

	TAAParameters.DownsampleOverrideFormat = PassInputs.DownsampleOverrideFormat;

	TAAParameters.bDownsample = (PassInputs.bGenerateSceneColorHalfRes || PassInputs.bGenerateSceneColorQuarterRes) && TAAParameters.Quality != ETAAQuality::High;

	TAAParameters.SceneDepthTexture = PassInputs.SceneDepth.Texture;
	TAAParameters.SceneVelocityTexture = PassInputs.SceneVelocity.Texture;
	TAAParameters.SceneColorInput = PassInputs.SceneColor.Texture;

	const FTemporalAAHistory& InputHistory = View.PrevViewInfo.TemporalAAHistory;

	FTemporalAAHistory& OutputHistory = View.ViewState->PrevFrameViewInfo.TemporalAAHistory;


	const FTAAOutputs TAAOutputs = ::AddTemporalAAPass(
		GraphBuilder,
		View,
		TAAParameters,
		InputHistory,
		&OutputHistory);

	FRDGTextureRef SceneColorTexture = TAAOutputs.SceneColor;

	// If we upscaled the history buffer, downsize back to the secondary screen percentage size.
	if (HistoryUpscaleFactor > 1.0f)
	{
		const FIntRect InputViewport = TAAParameters.OutputViewRect;

		FIntPoint QuantizedOutputSize;
		QuantizeSceneBufferSize(SecondaryViewRect.Size(), QuantizedOutputSize);

		FScreenPassTextureViewport OutputViewport;
		OutputViewport.Rect = SecondaryViewRect;
		OutputViewport.Extent.X = FMath::Max(PassInputs.SceneColor.Texture->Desc.Extent.X, QuantizedOutputSize.X);
		OutputViewport.Extent.Y = FMath::Max(PassInputs.SceneColor.Texture->Desc.Extent.Y, QuantizedOutputSize.Y);

		SceneColorTexture = ComputeMitchellNetravaliDownsample(GraphBuilder, View, FScreenPassTexture(SceneColorTexture, InputViewport), OutputViewport);
	}

	FDefaultTemporalUpscaler::FOutputs Outputs;
	Outputs.FullRes = FScreenPassTextureSlice::CreateFromScreenPassTexture(GraphBuilder, FScreenPassTexture(SceneColorTexture, SecondaryViewRect));
	if (TAAOutputs.DownsampledSceneColor)
	{
		Outputs.HalfRes.TextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(TAAOutputs.DownsampledSceneColor));
		Outputs.HalfRes.ViewRect = FIntRect::DivideAndRoundUp(SecondaryViewRect, 2);
	}
	return Outputs;
} // AddGen4MainTemporalAAPasses()

UE::Renderer::Private::IMegaFilters::FOutputs AddMegaFiltersPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const TCHAR* Name,const UE::Renderer::Private::IMegaFilters::FInputs& Inputs)
{
	using UE::Renderer::Private::IMegaFilters;

	const IMegaFilters* MegaFiltersToUse = View.Family->GetMegaFiltersInterface();
	IMegaFilters::FOutputs MegaFilterOutputs;

	if (MegaFiltersToUse)
	{
		const TCHAR* FilterName = MegaFiltersToUse->GetDebugName();

		IMegaFilters::FInputs MegaFilterInputs = Inputs;
		checkf(MegaFilterInputs.PrevHistory == nullptr,TEXT("History is set internally"));
		{
			// History can be accessed/updated in the same frame at different location for the same MegaFiltersPassType.
			TRefCountPtr<IMegaFilters::IHistory> PrevHistory = 
				(View.ViewState && View.ViewState->PrevFrameViewInfo.MegaFiltersHistory) ? 
				View.ViewState->PrevFrameViewInfo.MegaFiltersHistory : View.PrevViewInfo.MegaFiltersHistory;

			if (PrevHistory && PrevHistory->GetDebugName() == FilterName)
			{
				MegaFilterInputs.PrevHistory = PrevHistory;
			}
		}

		checkf(MegaFilterInputs.OutputViewRects.Num() > 0, TEXT("At least define one output view rect in MegaFilterInputs.OutputViewRects"));

		RDG_EVENT_SCOPE(
			GraphBuilder,
			"MegaFilters %s %dx%d -> %dx%d",
			Name,
			View.ViewRect.Width(), View.ViewRect.Height(),
			MegaFilterInputs.OutputViewRects[0].Width(), MegaFilterInputs.OutputViewRects[0].Height());

		MegaFilterOutputs = MegaFiltersToUse->AddPasses(
			GraphBuilder,
			View,
			MegaFilterInputs);

		if (View.ViewState && !View.bStatePrevViewInfoIsReadOnly)
		{
			View.ViewState->PrevFrameViewInfo.MegaFiltersHistory = MegaFilterOutputs.NewHistory;
		}
	}

	return MegaFilterOutputs;
}

FRDGTextureRef AddMegaFiltersTextureCopy(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SourceTexture, UE::Renderer::Private::EMegaFiltersPassType PassType, const TCHAR* TextureName)
{
    const UE::Renderer::Private::IMegaFilters* MegaFiltersToUse = View.Family->GetMegaFiltersInterface();

    if (MegaFiltersToUse && SourceTexture)
    {
        FRDGTextureRef Copy = GraphBuilder.CreateTexture(SourceTexture->Desc, TextureName);
        AddCopyTexturePass(GraphBuilder, SourceTexture, Copy);
        return Copy;
    }

    return nullptr;
}

EMainTAAPassConfig GetMainTAAPassConfig(const FViewInfo& View)
{
	if (!IsPostProcessingEnabled(View) && !View.bSceneCaptureMainViewJitter)
	{
		return EMainTAAPassConfig::Disabled;
	}
	else if (!IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod))
	{
		return EMainTAAPassConfig::Disabled;
	}

	int32 CustomUpscalerMode = CVarUseTemporalAAUpscaler.GetValueOnRenderThread();

	if (View.Family->GetTemporalUpscalerInterface() && CustomUpscalerMode != 0)
	{
		return EMainTAAPassConfig::ThirdParty;
	}
	else if (View.AntiAliasingMethod == AAM_TSR)
	{
		check(SupportsTSR(View.GetShaderPlatform()));
		return EMainTAAPassConfig::TSR;
	}
	else
	{
		return EMainTAAPassConfig::TAA;
	}
}
