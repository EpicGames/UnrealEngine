// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "PostProcess/PostProcessMotionBlur.h"
#include "PostProcess/LensDistortion.h"
#include "TemporalUpscaler.h"
#include "MegaFiltersInterface.h"
#include "TranslucentPassResource.h"
#include "SceneRendering.h"

struct FTemporalAAHistory;
struct FTranslucencyPassResources;
struct FTSRHistory;

/** Configuration of the main temporal AA pass. */
enum class EMainTAAPassConfig : uint8
{
	// TAA is disabled.
	Disabled,

	// Uses old UE4's Temporal AA maintained for Gen4 consoles
	TAA,

	// Uses Temporal Super Resolution
	TSR,

	// Uses third party View.Family->GetTemporalUpscalerInterface()
	ThirdParty,
};


/** List of TAA configurations. */
enum class ETAAPassConfig
{
	// Permutations for main scene color TAA.
	Main,
	MainUpsampling,
	MainSuperSampling,

	// Permutation for SSR noise accumulation.
	ScreenSpaceReflections,
	
	// Permutation for light shaft noise accumulation.
	LightShaft,

	// Permutation for DOF that handle Coc.
	DiaphragmDOF,
	DiaphragmDOFUpsampling,
	
	// Permutation for hair.
	Hair,

	MAX
};

static FORCEINLINE bool IsTAAUpsamplingConfig(ETAAPassConfig Pass)
{
	return Pass == ETAAPassConfig::MainUpsampling || Pass == ETAAPassConfig::DiaphragmDOFUpsampling || Pass == ETAAPassConfig::MainSuperSampling;
}

static FORCEINLINE bool IsMainTAAConfig(ETAAPassConfig Pass)
{
	return Pass == ETAAPassConfig::Main || Pass == ETAAPassConfig::MainUpsampling || Pass == ETAAPassConfig::MainSuperSampling;
}

static FORCEINLINE bool IsDOFTAAConfig(ETAAPassConfig Pass)
{
	return Pass == ETAAPassConfig::DiaphragmDOF || Pass == ETAAPassConfig::DiaphragmDOFUpsampling;
}

/** GPU Output of the TAA pass. */
struct FTAAOutputs
{
	// Anti aliased scene color.
	// Can have alpha channel, or CoC for DOF.
	FRDGTexture* SceneColor = nullptr;

	// Optional information that get anti aliased, such as separate CoC for DOF.
	FRDGTexture* SceneMetadata = nullptr;

	// Optional scene color output at half the resolution.
	FRDGTexture* DownsampledSceneColor = nullptr;
};

/** Quality of TAA. */
enum class ETAAQuality : uint8
{
	Low,
	Medium,
	High,
	MediumHigh,
	MAX
};

/** Configuration of TAA. */
struct FTAAPassParameters
{
	// TAA pass to run.
	ETAAPassConfig Pass = ETAAPassConfig::Main;

	// Whether to use the faster shader permutation.
	ETAAQuality Quality = ETAAQuality::High;

	// Whether output texture should be render targetable.
	bool bOutputRenderTargetable = false;

	// Whether downsampled (box filtered, half resolution) frame should be written out.
	bool bDownsample = false;
	EPixelFormat DownsampleOverrideFormat = PF_Unknown;

	// Not null if this texture should be used as the half resolution render target when bDownsample=true.
	FRDGTexture* DownsampleRenderTarget = nullptr;
	// Mip level to write to when writing to DownsampleRenderTarget.
	uint8 DownsampleRenderTargetMipLevel = 0;

	// Viewport rectangle of the input and output of TAA at ResolutionDivisor == 1.
	FIntRect InputViewRect;
	FIntRect OutputViewRect;

	// Resolution divisor.
	int32 ResolutionDivisor = 1;

	// Full resolution depth and velocity textures to reproject the history.
	FRDGTexture* SceneDepthTexture = nullptr;
	FRDGTexture* SceneVelocityTexture = nullptr;

	// Anti aliased scene color.
	// Can have alpha channel, or CoC for DOF.
	FRDGTexture* SceneColorInput = nullptr;

	// Optional information that get anti aliased, such as separate CoC for DOF.
	FRDGTexture* SceneMetadataInput = nullptr;

	// Optional override for stencil source (when depth has been swapped but stencil lives elsewhere, e.g., translucency early velocity pass)
	FRDGTexture* StencilDepthTexture = nullptr;

	// If 1, a bilateral filter based on the circle-of-confusion for depth-of-field is used to reject history.
	// If 0, this is disabled.
	float CoCBilateralFilterStrength = 1.0;


	FTAAPassParameters(const FViewInfo& View)
		: InputViewRect(View.ViewRect)
		, OutputViewRect(View.ViewRect)
	{ }


	// Customizes the view rectangles for input and output.
	FORCEINLINE void SetupViewRect(const FViewInfo& View, int32 InResolutionDivisor = 1)
	{
		ResolutionDivisor = InResolutionDivisor;

		InputViewRect = View.ViewRect;

		// When upsampling, always upsampling to top left corner to reuse same RT as before upsampling.
		if (IsTAAUpsamplingConfig(Pass))
		{
			OutputViewRect.Min = FIntPoint(0, 0);
			OutputViewRect.Max =  View.GetSecondaryViewRectSize();
		}
		else
		{
			OutputViewRect = InputViewRect;
		}
	}
	
	/** Returns the texture resolution that will be output. */
	FIntPoint GetOutputExtent() const;

	/** Validate the settings of TAA, to make sure there is no issue. */
	bool Validate() const;
};

/** Temporal AA pass which emits a filtered scene color and new history. */
extern RENDERER_API FTAAOutputs AddTemporalAAPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FTAAPassParameters& Inputs,
	const FTemporalAAHistory& InputHistory,
	FTemporalAAHistory* OutputHistory);

/** Returns whether TSR support lens distortion in its shader. */
bool IsTSRLensDistortionSupported(EShaderPlatform ShaderPlatform);

/** Returns whether TSR lens distortion is enabled (for runtime toggle). */
bool IsTSRLensDistortionEnabled(EShaderPlatform ShaderPlatform);

/** Returns whether a given view need to measure the scene for moire anti-flickering. */
bool NeedTSRAntiFlickeringPass(const FViewInfo& View);

bool NeedTSRThinGeometryDetectionPass(const FViewInfo& View);

/** Returns whether TSR needs the bits set by translucent clipped depth pass*/
bool NeedTSRTranslucentClippedDepthPass(const FViewInfo& View);

/** Returns whether TSR internal visualization is enabled on the view. */
bool IsVisualizeTSREnabled(const FViewInfo& View);

/** Measure different metrics of the scene for anti-flickering */
FScreenPassTexture AddTSRMainAntiFlickeringPass(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FScreenPassTexture SceneColor, const FSceneTextures& SceneTextures);

FScreenPassTexture AddTSRMeasureFlickeringLuma(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FScreenPassTexture SceneColor);

void AddTSRMeasureThinGeometryCoverage(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, const FSceneTextures& SceneTextures, const FScreenPassTexture& ThinGeometryCoverage); 

EMainTAAPassConfig GetMainTAAPassConfig(const FViewInfo& View);

/** Interface for the default temporal upscaling algorithm. */
struct FDefaultTemporalUpscaler
{
	struct FInputs
	{
		bool bAllowFullResSlice = false;
		bool bGenerateSceneColorHalfRes = false;
		bool bGenerateSceneColorQuarterRes = false;
		bool bGenerateSceneColorEighthRes = false;
		bool bGenerateOutputMip1 = false;
		bool bGenerateVelocityFlattenTextures = false;
		EPixelFormat DownsampleOverrideFormat;
		FScreenPassTexture SceneColor;
		FScreenPassTexture SceneDepth;
		FScreenPassTexture SceneVelocity;
		FTranslucencyPassResources PostDOFTranslucencyResources;
		FScreenPassTexture FlickeringInputTexture;
		FLensDistortionLUT LensDistortionLUT;
	};

	struct FOutputs
	{
		FScreenPassTextureSlice FullRes;
		FScreenPassTextureSlice HalfRes;
		FScreenPassTextureSlice QuarterRes;
		FScreenPassTextureSlice EighthRes;
		FVelocityFlattenTextures VelocityFlattenTextures;
	};
};

FDefaultTemporalUpscaler::FOutputs AddGen4MainTemporalAAPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FDefaultTemporalUpscaler::FInputs& PassInputs);

/** List of TSR configurations. */
enum class ETSRPassConfig
{
	// Permutations for main scene color TSR.
	Main,				// Temporal accumulation without upscaling
	MainUpsampling,		// Temporal accumulation with upsampling

	MAX
};

struct FTSRPassConfig
{
	bool  ResurrectionEnable = true;
	int32 ResurrectionPersistentFrameCount = 2;
	int32 ResurrectionPersistentFrameInterval = 31;

	int32 AlphaChannel = -1;

	bool  ShadingRejectionFlickering = true;
	int32 ShadingRejectionFlickeringAdjustToFrameRate = 1;
	float ShadingRejectionFlickeringFrameRateCap = 60;
	float ShadingRejectionFlickeringPeriod = 2.0f;
	float ShadingRejectionFlickeringMaxParallaxVelocity = 10.0f;
	float ShadingRejectionExposureOffsetFactor = 0.0f;

	bool  ThinGeometryDetectionEnable = false;
	float ThinGeometryErrorMultiplier = 200.0f;

	// Spatial antialiasing
	int32 RejectionAntiAliasingQuality = 3;

	float HistoryRejectionSampleCount = 2.0f;
	float HistoryScreenPercentage = 100.0f;
	float HistorySampleCount = 16.0f;
	int32 HistoryUpdateQuality = 3;
	int32 HistoryR11G11B10 = 1;

	int32 ReprojectionField = 0;
	float ReprojectionFieldAntiAliasPixelSpeed = 0.125f;

	float VelocityWeightClampingSampleCount = 4.0f;
	float VelocityWeightClampingPixelSpeed = 1.0f;
	
	bool  MotionBlurValidInCurrentFrame = true;

	int32  Visualize = -1;

	ETSRPassConfig Pass = ETSRPassConfig::MainUpsampling;
};

FTSRPassConfig GetTSRMainPassConfig(const FViewInfo& View);

FDefaultTemporalUpscaler::FOutputs  AddTemporalSuperResolutionPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FDefaultTemporalUpscaler::FInputs& PassInputs,
	const FTSRPassConfig& PassConfig,
	const FTSRHistory& InputHistory,
	FTSRHistory& OutputHistory);

FDefaultTemporalUpscaler::FOutputs AddMainTemporalSuperResolutionPasses(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FDefaultTemporalUpscaler::FInputs& PassInputs);

/** Interface for the VisualizeTSR showflag. */
struct FVisualizeTemporalUpscalerInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// Scene color.
	FScreenPassTexture SceneColor;

	// Temporal upscaler used and its inputs and outputs.
	EMainTAAPassConfig TAAConfig = EMainTAAPassConfig::Disabled;
	const UE::Renderer::Private::ITemporalUpscaler* UpscalerUsed = nullptr;
	FDefaultTemporalUpscaler::FInputs Inputs;
	FDefaultTemporalUpscaler::FOutputs Outputs;
};

FScreenPassTexture AddVisualizeTemporalUpscalerPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeTemporalUpscalerInputs& Inputs);

UE::Renderer::Private::IMegaFilters::FOutputs AddMegaFiltersPass(
	FRDGBuilder& GraphBuilder, 
	const FViewInfo& View,
	const TCHAR* Name,
	const UE::Renderer::Private::IMegaFilters::FInputs& Inputs);

FRDGTextureRef AddMegaFiltersTextureCopy(
    FRDGBuilder& GraphBuilder,
    const FViewInfo& View,
    FRDGTextureRef SourceTexture,
    UE::Renderer::Private::EMegaFiltersPassType PassType,
    const TCHAR* TextureName);