// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/VisualizeColorGrading.h"
#include "PostProcess/ColorGradingScopes.h"
#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "ScreenPass.h"
#include "RenderGraphUtils.h"
#include "SceneRendering.h"
#include "GenerateMips.h"
#include "CanvasItem.h"
#include "SceneViewExtension.h"
#include "PostProcess/PostProcessMaterialInputs.h"

static bool UseWaveOps(EShaderPlatform ShaderPlatform)
{
	const ERHIFeatureSupport UseWaveOpsSupport = FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(ShaderPlatform);
	return (GRHISupportsWaveOperations && UseWaveOpsSupport == ERHIFeatureSupport::RuntimeDependent) || (UseWaveOpsSupport == ERHIFeatureSupport::RuntimeGuaranteed);
}

namespace UE::ColorParade
{
	constexpr uint32 FlagDrawSeparateColumnPerChannel = 1<<0;
	constexpr uint32 FlagColorize 					  = 1<<1;
	constexpr uint32 FlagHighlightNaNsAndInfs		  = 1<<2;

	bool IsPlatformSupported(EShaderPlatform Platform)
	{
		return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM6);
	}

	class FBuildColorParadeHistogramCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FBuildColorParadeHistogramCS);
		SHADER_USE_PARAMETER_STRUCT(FBuildColorParadeHistogramCS, FGlobalShader);

		class FDimWaveOps : SHADER_PERMUTATION_BOOL("DIM_WAVE_OPS");
		using FPermutationDomain = TShaderPermutationDomain<FDimWaveOps>;

		inline static const FIntPoint GroupSize{ 1, 256 };
	
		// Number of histogram bins per column
		static constexpr uint32 NumBins = 1024;
	
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputColorTexture)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutputTexture)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutputColumnFlagsBuffer)
			SHADER_PARAMETER(FVector4f, InputColorViewRect)
			SHADER_PARAMETER(FIntPoint, OutputHistogramSize)

			SHADER_PARAMETER(FVector2f, MinMaxValue)
			SHADER_PARAMETER(uint32, Flags)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);

			OutEnvironment.SetDefine(TEXT("NUM_BINS"), NumBins);

			OutEnvironment.SetDefine(TEXT("FLAG_DRAW_SEPARATE_COLUMN_PER_CHANNEL"), FlagDrawSeparateColumnPerChannel);
			OutEnvironment.SetDefine(TEXT("FLAG_COLORIZE"), FlagColorize);
			OutEnvironment.SetDefine(TEXT("FLAG_HIGHLIGHT_NANS_AND_INFS"), FlagHighlightNaNsAndInfs);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			FPermutationDomain PermutationVector(Parameters.PermutationId);

			ERHIFeatureSupport WaveOpsSupport = FDataDrivenShaderPlatformInfo::GetSupportsWaveOperations(Parameters.Platform);
			if (PermutationVector.Get<FDimWaveOps>())
			{
				if (WaveOpsSupport == ERHIFeatureSupport::Unsupported)
				{
					return false;
				}
			}
			else
			{
				if (WaveOpsSupport == ERHIFeatureSupport::RuntimeGuaranteed)
				{
					return false;
				}
			}

			return IsPlatformSupported(Parameters.Platform);
		}
	};
	IMPLEMENT_GLOBAL_SHADER(FBuildColorParadeHistogramCS, "/Engine/Private/Tools/ColorGradingParade.usf", "BuildColorParadeHistogramCS", SF_Compute);

	class FDrawColorParadeCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FDrawColorParadeCS);
		SHADER_USE_PARAMETER_STRUCT(FDrawColorParadeCS, FGlobalShader);

		inline static const FIntPoint GroupSize{ 8, 8 };

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, HistogramTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, HistogramTextureSampler)
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, ColumnFlagsBuffer)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, OutputTexture)
			SHADER_PARAMETER(FIntPoint, InputViewSize)
			SHADER_PARAMETER(FIntPoint, HistogramSize)
			SHADER_PARAMETER(FIntVector4, OutputViewRect)
			SHADER_PARAMETER(float, Brightness)
			SHADER_PARAMETER(float, Gamma)
			SHADER_PARAMETER(float, DesaturationPower)

			SHADER_PARAMETER(FVector2f, MinMaxValue)
			SHADER_PARAMETER(uint32, Flags)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);

			OutEnvironment.SetDefine(TEXT("NUM_BINS"), FBuildColorParadeHistogramCS::NumBins);

			OutEnvironment.SetDefine(TEXT("FLAG_DRAW_SEPARATE_COLUMN_PER_CHANNEL"), FlagDrawSeparateColumnPerChannel);
			OutEnvironment.SetDefine(TEXT("FLAG_COLORIZE"), FlagColorize);
			OutEnvironment.SetDefine(TEXT("FLAG_HIGHLIGHT_NANS_AND_INFS"), FlagHighlightNaNsAndInfs);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsPlatformSupported(Parameters.Platform);
		}
	};
	IMPLEMENT_GLOBAL_SHADER(FDrawColorParadeCS, "/Engine/Private/Tools/ColorGradingParade.usf", "DrawColorParadeCS", SF_Compute);

	void Draw(
		FRDGBuilder& GraphBuilder,
		FRDGTextureSRVRef InputSRV, FIntRect InputViewRect,
		FRDGTextureRef Output, FIntRect OutputViewRect,
		const FDrawParameters& Parameters)
	{
		constexpr int NumComponents = 3;

		check(IsPlatformSupported(GMaxRHIShaderPlatform));
		check(InputSRV);
		check(Output);
		// The histogram is stored as a uint32 texture, with one value per channel, to support atomics.
		// This defines a limit on the output rect.
		constexpr int MaxWidth = (1U << 16U) / NumComponents;
		constexpr int MaxHeight = (1U << 16U);
		check(OutputViewRect.Size().X < MaxWidth);
		check(OutputViewRect.Size().Y < MaxHeight);
		check(InputViewRect.Size().X > 0);
		check(InputViewRect.Size().Y > 0);
		check(OutputViewRect.Size().X > 0);
		check(OutputViewRect.Size().Y > 0);

		RDG_EVENT_SCOPE(GraphBuilder, "ColorParade");

		FIntPoint OutputSize = OutputViewRect.Size();
		FIntPoint HistogramSize(OutputSize.X, FBuildColorParadeHistogramCS::NumBins);

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		uint32 Flags = 0;
		Flags |= Parameters.bDrawSeparateColumnPerChannel ? FlagDrawSeparateColumnPerChannel : 0U;
		Flags |= Parameters.bColorize ? FlagColorize : 0U;
		Flags |= Parameters.bHighlightNaNsAndInfs ? FlagHighlightNaNsAndInfs : 0U;

		FVector2f MinMaxValue{ Parameters.MinValue, Parameters.MaxValue };

		// Build a per-column histogram of pixel values.
		FRDGTextureRef Histogram = nullptr;
		FRDGBufferRef ColumnFlags = nullptr;
		{
			FIntPoint HistogramTextureSize(HistogramSize.X * NumComponents, HistogramSize.Y);

			FRDGTextureDesc HistogramDesc = FRDGTextureDesc::Create2D(
				HistogramTextureSize,
				PF_R32_UINT, // used for support of atomics
				FClearValueBinding::Black,
				TexCreate_ShaderResource | TexCreate_UAV);

			Histogram = GraphBuilder.CreateTexture(HistogramDesc, TEXT("ColorParadeHistogram"));
			FRDGTextureUAVRef HistogramUAV = GraphBuilder.CreateUAV(Histogram);

			FRDGBufferDesc ColumnFlagsDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), HistogramTextureSize.X);
			ColumnFlags = GraphBuilder.CreateBuffer(ColumnFlagsDesc, TEXT("ColumnFlags"));
			FRDGBufferUAVRef ColumnFlagsUAV = GraphBuilder.CreateUAV(ColumnFlags);

			AddClearUAVPass(GraphBuilder, HistogramUAV, 0U);
			AddClearUAVPass(GraphBuilder, ColumnFlagsUAV, 0U);

			FBuildColorParadeHistogramCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildColorParadeHistogramCS::FParameters>();
			PassParameters->InputColorTexture = InputSRV;
			PassParameters->OutputTexture = HistogramUAV;
			PassParameters->OutputColumnFlagsBuffer = ColumnFlagsUAV;
			PassParameters->InputColorViewRect = FVector4f(InputViewRect.Min, InputViewRect.Max);
			PassParameters->OutputHistogramSize = HistogramSize;
			PassParameters->MinMaxValue = MinMaxValue;
			PassParameters->Flags = Flags;

			FBuildColorParadeHistogramCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FBuildColorParadeHistogramCS::FDimWaveOps>(UseWaveOps(GMaxRHIShaderPlatform));
			auto ComputeShader = ShaderMap->GetShader<FBuildColorParadeHistogramCS>(PermutationVector);

			FComputeShaderUtils::AddPass(GraphBuilder,
										 RDG_EVENT_NAME("BuildColorParadeHistogramCS"),
										 ComputeShader,
										 PassParameters,
										 // One thread handles N input values, where N = InputWidth/OutputWidth.
										 FComputeShaderUtils::GetGroupCount(FIntPoint(OutputSize.X, InputViewRect.Size().Y), FBuildColorParadeHistogramCS::GroupSize));
		}

		// Draw this histogram data into a nice graph
		{
			FRDGTextureUAVRef RGBParadeUAV = GraphBuilder.CreateUAV(Output);

			FDrawColorParadeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDrawColorParadeCS::FParameters>();
			PassParameters->HistogramTexture = Histogram;
			PassParameters->HistogramTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->ColumnFlagsBuffer = GraphBuilder.CreateSRV(ColumnFlags);
			PassParameters->OutputTexture = RGBParadeUAV;
			PassParameters->InputViewSize = InputViewRect.Size();
			PassParameters->HistogramSize = HistogramSize;
			PassParameters->OutputViewRect = FIntVector4(OutputViewRect.Min.X, OutputViewRect.Min.Y, OutputViewRect.Max.X, OutputViewRect.Max.Y);
			PassParameters->Brightness = Parameters.Brightness;
			PassParameters->Gamma = Parameters.Gamma;
			PassParameters->DesaturationPower = Parameters.DesaturationPower;
			PassParameters->MinMaxValue = MinMaxValue;
			PassParameters->Flags = Flags;

			auto ComputeShader = ShaderMap->GetShader<FDrawColorParadeCS>();

			FComputeShaderUtils::AddPass(GraphBuilder,
										 RDG_EVENT_NAME("DrawColorParadeCS"),
										 ComputeShader,
										 PassParameters,
										 FComputeShaderUtils::GetGroupCount(OutputSize, FDrawColorParadeCS::GroupSize));
		}
	}
} // namespace UE::ColorParade

namespace UE::VectorScope
{
	constexpr uint32 FlagColorize = 1<<0;
	constexpr uint32 HueResolution = 1024;
	constexpr uint32 SaturationResolution = 1024;

	bool IsPlatformSupported(EShaderPlatform Platform)
	{
		return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM6);
	}

	class FBuildVectorScopeHistogramCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FBuildVectorScopeHistogramCS);
		SHADER_USE_PARAMETER_STRUCT(FBuildVectorScopeHistogramCS, FGlobalShader);

		inline static const FIntPoint GroupSize{ 256, 1 };
		
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputColorTexture)
			SHADER_PARAMETER(FIntVector4, InputColorViewRect)
			SHADER_PARAMETER(FVector2f, LuminanceFilterMinMax)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutputHistogramTexture)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);

			OutEnvironment.SetDefine(TEXT("HUE_RESOLUTION"), HueResolution);
			OutEnvironment.SetDefine(TEXT("SATURATION_RESOLUTION"), SaturationResolution);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsPlatformSupported(Parameters.Platform);
		}
	};
	IMPLEMENT_GLOBAL_SHADER(FBuildVectorScopeHistogramCS, "/Engine/Private/Tools/ColorGradingVectorScope.usf", "BuildVectorScopeHistogramCS", SF_Compute);

	class FDrawVectorScopeCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FDrawVectorScopeCS);
		SHADER_USE_PARAMETER_STRUCT(FDrawVectorScopeCS, FGlobalShader);

		inline static const FIntPoint GroupSize{ 8, 8 };
	
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, HistogramTexture)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, OutputTexture)
			SHADER_PARAMETER(FIntVector4, OutputViewRect)
			SHADER_PARAMETER(FIntVector4, InputColorViewRect)
			SHADER_PARAMETER(uint32, Flags)
			SHADER_PARAMETER(float, Rotation)
			SHADER_PARAMETER(float, Brightness)
			SHADER_PARAMETER(float, Gamma)
			SHADER_PARAMETER(float, DesaturationPower)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);

			OutEnvironment.SetDefine(TEXT("HUE_RESOLUTION"), HueResolution);
			OutEnvironment.SetDefine(TEXT("SATURATION_RESOLUTION"), SaturationResolution);

			OutEnvironment.SetDefine(TEXT("FLAG_COLORIZE"), FlagColorize);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsPlatformSupported(Parameters.Platform);
		}
	};
	IMPLEMENT_GLOBAL_SHADER(FDrawVectorScopeCS, "/Engine/Private/Tools/ColorGradingVectorScope.usf", "DrawVectorScopeCS", SF_Compute);

	void Draw(
		FRDGBuilder& GraphBuilder,
		FRDGTextureSRVRef InputSRV, FIntRect InputViewRect,
		FRDGTextureRef Output, FIntRect OutputViewRect,
		const FDrawParameters& Parameters)
	{
		check(IsPlatformSupported(GMaxRHIShaderPlatform));
		check(InputSRV);
		check(Output);
		check(InputViewRect.Size().X > 0);
		check(InputViewRect.Size().Y > 0);
		check(OutputViewRect.Size().X > 0);
		check(OutputViewRect.Size().Y > 0);
		
		RDG_EVENT_SCOPE(GraphBuilder, "VectorScope");

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		FIntPoint InputColorViewSize = InputViewRect.Size();

		uint32 Flags = 0;
		Flags |= Parameters.bColorize ? FlagColorize : 0;

		FRDGTextureRef HistogramTexture = nullptr;
		{
			FRDGTextureDesc HistogramTextureDesc = FRDGTextureDesc::Create2D(
				FIntPoint(SaturationResolution, HueResolution),
				PF_R32_UINT,
				FClearValueBinding::Black,
				TexCreate_ShaderResource | TexCreate_UAV);

			HistogramTexture = GraphBuilder.CreateTexture(HistogramTextureDesc, TEXT("VectorScopeHistogram"));

			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HistogramTexture), 0U);

			FBuildVectorScopeHistogramCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildVectorScopeHistogramCS::FParameters>();
			PassParameters->InputColorTexture = InputSRV;
			PassParameters->InputColorViewRect = FIntVector4(InputViewRect.Min.X, InputViewRect.Min.Y, InputViewRect.Max.X, InputViewRect.Max.Y);
			PassParameters->LuminanceFilterMinMax = FVector2f(Parameters.LuminanceFilterMin, Parameters.LuminanceFilterMax);
			PassParameters->OutputHistogramTexture = GraphBuilder.CreateUAV(HistogramTexture);

			auto ComputeShader = ShaderMap->GetShader<FBuildVectorScopeHistogramCS>();

			FComputeShaderUtils::AddPass(GraphBuilder,
										 RDG_EVENT_NAME("BuildVectorScopeHistogramCS"),
										 ComputeShader,
										 PassParameters,
										 FComputeShaderUtils::GetGroupCount(InputColorViewSize.X * InputColorViewSize.Y, FBuildVectorScopeHistogramCS::GroupSize.X));
		}

		{
			FDrawVectorScopeCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDrawVectorScopeCS::FParameters>();
			PassParameters->HistogramTexture = HistogramTexture;
			PassParameters->OutputTexture = GraphBuilder.CreateUAV(Output);
			PassParameters->OutputViewRect = FIntVector4(OutputViewRect.Min.X, OutputViewRect.Min.Y, OutputViewRect.Max.X, OutputViewRect.Max.Y);
			PassParameters->InputColorViewRect = FIntVector4(InputViewRect.Min.X, InputViewRect.Min.Y, InputViewRect.Max.X, InputViewRect.Max.Y);
			PassParameters->Rotation = Parameters.Rotation;
			PassParameters->Brightness = Parameters.Brightness;
			PassParameters->Gamma = Parameters.Gamma;
			PassParameters->DesaturationPower = Parameters.DesaturationPower;
			PassParameters->Flags = Flags;

			auto ComputeShader = ShaderMap->GetShader<FDrawVectorScopeCS>();

			FComputeShaderUtils::AddPass(GraphBuilder,
										 RDG_EVENT_NAME("DrawVectorScopeCS"),
										 ComputeShader,
										 PassParameters,
										 FComputeShaderUtils::GetGroupCount(OutputViewRect.Size(), FDrawVectorScopeCS::GroupSize));
		}
	}
} // namespace UE::VectorScope

namespace UE::ColorHistogram
{
	constexpr uint32 FlagDrawSeparateRowPerChannel = 1<<0;
	constexpr uint32 FlagColorize 				   = 1<<1;

	constexpr uint32 PerChannelResolution = 1024;

	bool IsPlatformSupported(EShaderPlatform Platform)
	{
		return IsFeatureLevelSupported(Platform, ERHIFeatureLevel::SM6);
	}

	class FBuildColorHistogramCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FBuildColorHistogramCS);
		SHADER_USE_PARAMETER_STRUCT(FBuildColorHistogramCS, FGlobalShader);

		inline static const FIntPoint GroupSize{ 16, 16 };
		
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputColorTexture)
			SHADER_PARAMETER(FIntVector4, InputColorViewRect)
			SHADER_PARAMETER(FVector2f, MinMaxValue)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutputHistogramTexture)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint>, OutputMetaDataBuffer)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);

			OutEnvironment.SetDefine(TEXT("PER_CHANNEL_RESOLUTION"), PerChannelResolution);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsPlatformSupported(Parameters.Platform);
		}
	};
	IMPLEMENT_GLOBAL_SHADER(FBuildColorHistogramCS, "/Engine/Private/Tools/ColorGradingHistogram.usf", "BuildColorHistogramCS", SF_Compute);

	class FDrawColorHistogramCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FDrawColorHistogramCS);
		SHADER_USE_PARAMETER_STRUCT(FDrawColorHistogramCS, FGlobalShader);

		inline static const FIntPoint GroupSize{ 8, 8 };
	
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, HistogramTexture)
			SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, MetaDataBuffer)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float3>, OutputTexture)
			SHADER_PARAMETER(FIntVector4, OutputViewRect)
			SHADER_PARAMETER(FIntVector4, InputColorViewRect)
			SHADER_PARAMETER(uint32, Flags)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
		{
			FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), GroupSize.X);
			OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), GroupSize.Y);

			OutEnvironment.SetDefine(TEXT("PER_CHANNEL_RESOLUTION"), PerChannelResolution);

			OutEnvironment.SetDefine(TEXT("FLAG_COLORIZE"), FlagColorize);
			OutEnvironment.SetDefine(TEXT("FLAG_DRAW_SEPARATE_ROW_PER_CHANNEL"), FlagDrawSeparateRowPerChannel);
		}

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return IsPlatformSupported(Parameters.Platform);
		}
	};
	IMPLEMENT_GLOBAL_SHADER(FDrawColorHistogramCS, "/Engine/Private/Tools/ColorGradingHistogram.usf", "DrawColorHistogramCS", SF_Compute);

	void Draw(
		FRDGBuilder& GraphBuilder,
		FRDGTextureSRVRef InputSRV, FIntRect InputViewRect,
		FRDGTextureRef Output, FIntRect OutputViewRect,
		const FDrawParameters& Parameters)
	{
		check(IsPlatformSupported(GMaxRHIShaderPlatform));
		check(InputSRV);
		check(Output);
		check(InputViewRect.Size().X > 0);
		check(InputViewRect.Size().Y > 0);
		check(OutputViewRect.Size().X > 0);
		check(OutputViewRect.Size().Y > 0);
		
		RDG_EVENT_SCOPE(GraphBuilder, "ColorHistogram");

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		FIntPoint InputColorViewSize = InputViewRect.Size();

		uint32 Flags = 0;
		Flags |= Parameters.bDrawSeparateRowPerChannel ? FlagDrawSeparateRowPerChannel : 0;
		Flags |= Parameters.bColorize ? FlagColorize : 0;

		FVector2f MinMaxValue{ Parameters.MinValue, Parameters.MaxValue };

		FRDGTextureRef HistogramTexture = nullptr;
		FRDGBufferRef MetaDataBuffer = nullptr;
		{
			FRDGTextureDesc HistogramTextureDesc = FRDGTextureDesc::Create2D(
				FIntPoint(PerChannelResolution, 3),
				PF_R32_UINT,
				FClearValueBinding::Black,
				TexCreate_ShaderResource | TexCreate_UAV);

			HistogramTexture = GraphBuilder.CreateTexture(HistogramTextureDesc, TEXT("ColorHistogram"));
			FRDGTextureUAVRef HistogramTextureUAV = GraphBuilder.CreateUAV(HistogramTexture);
			
			FRDGBufferDesc MetaDataBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), 3);
			MetaDataBuffer = GraphBuilder.CreateBuffer(MetaDataBufferDesc, TEXT("MetaData"));
			FRDGBufferUAVRef MetaDataBufferUAV =  GraphBuilder.CreateUAV(MetaDataBuffer);

			AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(HistogramTexture), 0U);
			AddClearUAVPass(GraphBuilder, MetaDataBufferUAV, 0U);

			FBuildColorHistogramCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FBuildColorHistogramCS::FParameters>();
			PassParameters->InputColorTexture = InputSRV;
			PassParameters->InputColorViewRect = FIntVector4(InputViewRect.Min.X, InputViewRect.Min.Y, InputViewRect.Max.X, InputViewRect.Max.Y);
			PassParameters->MinMaxValue = MinMaxValue;
			PassParameters->OutputHistogramTexture = HistogramTextureUAV;
			PassParameters->OutputMetaDataBuffer = MetaDataBufferUAV;

			auto ComputeShader = ShaderMap->GetShader<FBuildColorHistogramCS>();

			FComputeShaderUtils::AddPass(GraphBuilder,
										 RDG_EVENT_NAME("BuildColorHistogramCS"),
										 ComputeShader,
										 PassParameters,
										 FComputeShaderUtils::GetGroupCount(InputColorViewSize, FBuildColorHistogramCS::GroupSize));
		}

		{
			FDrawColorHistogramCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDrawColorHistogramCS::FParameters>();
			PassParameters->HistogramTexture = HistogramTexture;
			PassParameters->MetaDataBuffer = GraphBuilder.CreateSRV(MetaDataBuffer);
			PassParameters->OutputTexture = GraphBuilder.CreateUAV(Output);
			PassParameters->OutputViewRect = FIntVector4(OutputViewRect.Min.X, OutputViewRect.Min.Y, OutputViewRect.Max.X, OutputViewRect.Max.Y);
			PassParameters->InputColorViewRect = FIntVector4(InputViewRect.Min.X, InputViewRect.Min.Y, InputViewRect.Max.X, InputViewRect.Max.Y);
			PassParameters->Flags = Flags;

			auto ComputeShader = ShaderMap->GetShader<FDrawColorHistogramCS>();

			FComputeShaderUtils::AddPass(GraphBuilder,
										 RDG_EVENT_NAME("DrawColorHistogramCS"),
										 ComputeShader,
										 PassParameters,
										 FComputeShaderUtils::GetGroupCount(OutputViewRect.Size(), FDrawColorHistogramCS::GroupSize));
		}
	}
} // namespace UE::ColorHistogram

static FScreenPassTexture VisualizeColorGrading(FRDGBuilder& GraphBuilder, const FSceneView& View, FScreenPassTexture& SceneColor, FScreenPassRenderTarget& OverrideOutput)
{
	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeColorGrading");

	auto FinalizeOutput = [](FRDGBuilder& GraphBuilder, const FSceneView& View, FScreenPassTexture& Output, FScreenPassRenderTarget& OverrideOutput) -> FScreenPassTexture
	{
		if (OverrideOutput.IsValid())
		{
			AddDrawTexturePass(GraphBuilder, View, Output, OverrideOutput);
			return OverrideOutput;
		}

		return Output;
	};

	FRDGTextureSRVRef InputSRV = GraphBuilder.CreateSRV(SceneColor.Texture);
	FScreenPassTexture Output = MoveTemp(SceneColor);

	FRHISamplerState* BilinearClampSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Color Parade
	UE::ColorParade::FDrawParameters RGBParadeDrawParameters
	{
		.MinValue = -0.1f,
		.MaxValue =  1.1f,
		.Gamma 	  = 1.0f / GEngine->DisplayGamma
	};
	FRDGTextureRef RGBParade = nullptr;
	FIntRect RGBParadeRect{};
	if (UE::ColorParade::IsPlatformSupported(GMaxRHIShaderPlatform))
	{
		int DownsampleFactor = 4;

		int Width = FMath::Min(Output.ViewRect.Size().X, Output.ViewRect.Size().Y) / 2;
		FIntPoint Size {Width, (Output.ViewRect.Size().Y * Width) / Output.ViewRect.Size().X};
		FIntPoint RenderSize = Size * DownsampleFactor;
		RenderSize = FIntPoint { FMath::Max(RenderSize.X, 1), FMath::Max(RenderSize.Y, 1) };
		uint8 NumMips = DownsampleFactor;

		FIntPoint OutputPosition{ Output.ViewRect.Max.X - Size.X, Output.ViewRect.Max.Y - Size.Y };
		RGBParadeRect = FIntRect(OutputPosition, OutputPosition + Size);

		FRDGTextureDesc RGBParadeDesc = FRDGTextureDesc::Create2D(
			RenderSize,
			PF_FloatRGB,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_UAV,
			NumMips);

		RGBParade = GraphBuilder.CreateTexture(RGBParadeDesc, TEXT("ColorParade"));

		UE::ColorParade::Draw(GraphBuilder, InputSRV, Output.ViewRect, RGBParade, FIntRect(0, 0, RenderSize.X, RenderSize.Y), RGBParadeDrawParameters);

		FGenerateMips::Execute(GraphBuilder, GMaxRHIFeatureLevel, RGBParade, BilinearClampSampler);
	}

	// VectorScope
	UE::VectorScope::FDrawParameters VectorScopeDrawParameters
	{
		.Gamma = 1.0f / GEngine->DisplayGamma
	};
	FRDGTextureRef VectorScope = nullptr;
	FIntRect VectorScopeRect {};
	if (UE::VectorScope::IsPlatformSupported(GMaxRHIShaderPlatform))
	{
		int DownsampleFactor = 4;

		int Width = FMath::Min(Output.ViewRect.Size().X, Output.ViewRect.Size().Y) / 3;
		FIntPoint Size{ Width, Width };
		FIntPoint RenderSize = Size * DownsampleFactor;
		RenderSize = FIntPoint { FMath::Max(RenderSize.X, 1), FMath::Max(RenderSize.Y, 1) };
		uint8 NumMips = DownsampleFactor;

		FIntPoint OutputPosition{ 0, Output.ViewRect.Max.Y - Size.Y };
		VectorScopeRect = FIntRect(OutputPosition, OutputPosition + Size);

		FRDGTextureDesc VectorScopeDesc = FRDGTextureDesc::Create2D(
			RenderSize,
			PF_FloatRGB,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_UAV,
			NumMips);

		VectorScope = GraphBuilder.CreateTexture(VectorScopeDesc, TEXT("VectorScope"));

		UE::VectorScope::Draw(GraphBuilder, InputSRV, Output.ViewRect, VectorScope, FIntRect(0, 0, RenderSize.X, RenderSize.Y), VectorScopeDrawParameters);

		FGenerateMips::Execute(GraphBuilder, GMaxRHIFeatureLevel, VectorScope, BilinearClampSampler);
	}

	// ColorHistogram
	UE::ColorHistogram::FDrawParameters ColorHistogramDrawParameters{};
	FRDGTextureRef ColorHistogram = nullptr;
	FIntRect ColorHistogramRect {};
	if (UE::ColorHistogram::IsPlatformSupported(GMaxRHIShaderPlatform))
	{
		int DownsampleFactor = 4;

		int Width = FMath::Min(Output.ViewRect.Size().X, Output.ViewRect.Size().Y) / 3;
		FIntPoint Size{ Width, Width / 2 };
		FIntPoint RenderSize = Size * DownsampleFactor;
		RenderSize = FIntPoint { FMath::Max(RenderSize.X, 1), FMath::Max(RenderSize.Y, 1) };
		uint8 NumMips = DownsampleFactor;

		FIntPoint OutputPosition{ Output.ViewRect.Max.X - Size.X, 0 };
		ColorHistogramRect = FIntRect(OutputPosition, OutputPosition + Size);

		FRDGTextureDesc ColorHistogramDesc = FRDGTextureDesc::Create2D(
			RenderSize,
			PF_FloatRGB,
			FClearValueBinding::Black,
			TexCreate_ShaderResource | TexCreate_UAV,
			NumMips);

		ColorHistogram = GraphBuilder.CreateTexture(ColorHistogramDesc, TEXT("ColorHistogramTexture"));

		UE::ColorHistogram::Draw(GraphBuilder, InputSRV, Output.ViewRect, ColorHistogram, FIntRect(0, 0, RenderSize.X, RenderSize.Y), ColorHistogramDrawParameters);

		FGenerateMips::Execute(GraphBuilder, GMaxRHIFeatureLevel, ColorHistogram, BilinearClampSampler);
	}

	if (RGBParade)
	{
		FIntPoint InputPosition = FIntPoint::ZeroValue;
		AddDrawTexturePass(GraphBuilder, View, RGBParade, Output.Texture, InputPosition, RGBParade->Desc.Extent, RGBParadeRect.Min, RGBParadeRect.Size(), BilinearClampSampler);

		AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("ColorParade UI"), View, FScreenPassRenderTarget(Output, ERenderTargetLoadAction::ELoad),
		[&](FCanvas& Canvas)
		{
			UFont* Font = GEngine->GetMonospaceFont();

			const float DPIScale = Canvas.GetDPIScale();
			Canvas.SetBaseTransform(FMatrix(
				FScaleMatrix(DPIScale)
				* FTranslationMatrix(FVector(RGBParadeRect.Min.X, RGBParadeRect.Min.Y, 0))
				* Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

			auto DrawShadowedString = [&](float X, float Y, FStringView Text, const FLinearColor& Color = FLinearColor::White)
			{
				Canvas.DrawShadowedString(X / DPIScale, Y / DPIScale, Text, Font, Color);
			};
		
			auto GetStringSize = [&](FStringView StringView) -> FVector2f
			{
				int32 XL = 0;
				int32 YL = 0;
				StringSize(Font, XL, YL, StringView);
				return FVector2f(XL, YL);
			};

			FVector2f TickLabelSize = GetStringSize(TEXT("-0.00 ")) * FVector2f(1.2f, 1.0f); // Monospace, only character count matters
			Canvas.DrawTile(-TickLabelSize.X, 0, TickLabelSize.X, RGBParadeRect.Size().Y, 0, 0, 0, 0, FLinearColor::Black, nullptr, false);

			int NumTicks = 13;
			for (int TickIndex = 0; TickIndex < NumTicks; ++TickIndex)
			{
				float Progress = float(TickIndex) / (NumTicks - 1);
				float TickValue = FMath::Lerp(RGBParadeDrawParameters.MinValue, RGBParadeDrawParameters.MaxValue, Progress);
				float Y = (1.0f - Progress) * RGBParadeRect.Size().Y;
				FCanvasLineItem Line { FVector2D(0, Y) / DPIScale, FVector2D(5, Y) / DPIScale };
				Line.SetColor(FLinearColor(1.0f, 1.0f, 1.0f, 0.2f));
				Canvas.DrawItem(Line);
				FString Label = FString::Printf(TEXT("% 0.2f"), TickValue);
				if (TickIndex == 0)
				{
					DrawShadowedString(-TickLabelSize.X, Y - TickLabelSize.Y, Label);
				}
				else if (TickIndex == NumTicks - 1)
				{
					DrawShadowedString(-TickLabelSize.X, Y, Label);
				}
				else
				{
					DrawShadowedString(-TickLabelSize.X, Y - (TickLabelSize.Y / 2), Label);
				}
			}
		});
	}

	if (VectorScope)
	{
		FIntPoint InputPosition = FIntPoint::ZeroValue;
		AddDrawTexturePass(GraphBuilder, View, VectorScope, Output.Texture, InputPosition, VectorScope->Desc.Extent, VectorScopeRect.Min, VectorScopeRect.Size(), BilinearClampSampler);

		AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("VectorScope UI"), View, FScreenPassRenderTarget(Output, ERenderTargetLoadAction::ELoad),
		[&](FCanvas& Canvas)
		{
			UFont* Font = GEngine->GetMonospaceFont();

			const float DPIScale = Canvas.GetDPIScale();
			Canvas.SetBaseTransform(FMatrix(
				FScaleMatrix(DPIScale)
				* FTranslationMatrix(FVector(VectorScopeRect.Min.X, VectorScopeRect.Min.Y, 0))
				* Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

			auto DrawShadowedString = [&](float X, float Y, FStringView Text, const FLinearColor& Color = FLinearColor::White)
			{
				Canvas.DrawShadowedString(X / DPIScale, Y / DPIScale, Text, Font, Color);
			};
		
			auto GetStringSize = [&](FStringView StringView) -> FVector2f
			{
				int32 XL = 0;
				int32 YL = 0;
				StringSize(Font, XL, YL, StringView);
				return FVector2f(XL, YL);
			};

			int NumTicks = 72;
			for (int TickIndex = 0; TickIndex < NumTicks; ++TickIndex)
			{
				float Progress = float(TickIndex) / (NumTicks - 1);

				FVector2D Start(FMath::Sin(Progress * 2.0f * PI), FMath::Cos(Progress * 2.0f * PI));
				bool bIsEven = (TickIndex & 1) == 0;
				FVector2D End = Start * (bIsEven ? 0.98f : 0.95f);

				FVector2D ChartRadius = VectorScopeRect.Size() / 2;
				FVector2D StartPos = ChartRadius + Start * ChartRadius;
				FVector2D EndPos = ChartRadius + End * ChartRadius;
				FCanvasLineItem Line { StartPos / DPIScale, EndPos / DPIScale };
				Line.SetColor(FLinearColor(0.3f, 0.3f, 0.3f));
				Canvas.DrawItem(Line);
			}
		});
	}

	if (ColorHistogram)
	{
		FIntPoint InputPosition = FIntPoint::ZeroValue;
		AddDrawTexturePass(GraphBuilder, View, ColorHistogram, Output.Texture, InputPosition, ColorHistogram->Desc.Extent, ColorHistogramRect.Min, ColorHistogramRect.Size(), BilinearClampSampler);

		AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("ColorHistogram UI"), View, FScreenPassRenderTarget(Output, ERenderTargetLoadAction::ELoad),
		[&](FCanvas& Canvas)
		{
			UFont* Font = GEngine->GetMonospaceFont();

			const float DPIScale = Canvas.GetDPIScale();
			Canvas.SetBaseTransform(FMatrix(
				FScaleMatrix(DPIScale)
				* FTranslationMatrix(FVector(ColorHistogramRect.Min.X, ColorHistogramRect.Min.Y, 0))
				* Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

			for (float YNorm : { 1.0f/3, 2.0f/3 })
			{
				int Y = YNorm * ColorHistogramRect.Size().Y;
				FCanvasLineItem Line { FVector2D(0, Y) / DPIScale, FVector2D(ColorHistogramRect.Size().X, Y) / DPIScale };
				Line.SetColor(FLinearColor(1.0f, 1.0f, 1.0f, 0.2f));
				Canvas.DrawItem(Line);
			}
		});
	}

	return FinalizeOutput(GraphBuilder, View, Output, OverrideOutput);
}


class FVisualizeColorGradingExtension : public FSceneViewExtensionBase
{
public:
	FVisualizeColorGradingExtension( const FAutoRegister& AutoRegister)
	: FSceneViewExtensionBase( AutoRegister )
	{
		FCoreDelegates::OnEnginePreExit.AddLambda([]()
			{ 
				Instance = nullptr;
			});
	}

	virtual void SubscribeToPostProcessingPass(EPostProcessingPass Pass, const FSceneView& InView, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) override
	{
		if (!InView.Family->EngineShowFlags.VisualizeColorGrading)
		{
			return;
		}

		if (Pass == EPostProcessingPass::VisualizeDepthOfField)
		{
			InOutPassCallbacks.AddDefaulted_GetRef().BindLambda(
				[this](FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& Inputs)
				{
					FScreenPassTexture InputSceneColor = FScreenPassTexture{ Inputs.GetInput(EPostProcessMaterialInput::SceneColor) };
					FScreenPassRenderTarget Output = Inputs.OverrideOutput;
					return VisualizeColorGrading(GraphBuilder, View, InputSceneColor, Output);
				});
		}
	}

	inline static TSharedPtr<FVisualizeColorGradingExtension,ESPMode::ThreadSafe> Instance = nullptr;
};

void InitVisualizeColorGradingExtension()
{
	FVisualizeColorGradingExtension::Instance = FSceneViewExtensions::NewExtension<FVisualizeColorGradingExtension>(); 
}
