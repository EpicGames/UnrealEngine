// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/VisualizeMotionVectors.h"
#include "PostProcess/PostProcessing.h"
#include "ScreenPass.h"
#include "PixelShaderUtils.h"
#include "SceneViewState.h"

class FVisualizeMotionVectorsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FVisualizeMotionVectorsPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeMotionVectorsPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevDistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  PrevDistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  UndistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ColorTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, VelocityTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, PrevColorTexture)

		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Color)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Velocity)
		SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, PrevColor)

		SHADER_PARAMETER_SAMPLER(SamplerState, ColorSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, VelocitySampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, DepthSampler)
		SHADER_PARAMETER_SAMPLER(SamplerState, PrevColorSampler)

		SHADER_PARAMETER(FScreenTransform, SvPositionToScreenPos)
		SHADER_PARAMETER(FScreenTransform, ScreenPosToVelocity)
		SHADER_PARAMETER(FScreenTransform, ScreenPosToColor)
		SHADER_PARAMETER(FScreenTransform, PrevScreenPosToPrevColor)
		SHADER_PARAMETER(int32, VisualizeType)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FVisualizeMotionVectorsPS, "/Engine/Private/MotionBlur/VisualizeMotionVectors.usf", "MainPS", SF_Pixel);

FScreenPassTexture AddVisualizeMotionVectorsPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeMotionVectorsInputs& Inputs, EVisualizeMotionVectors Visualize)
{
	check(Inputs.SceneColor.IsValid());
	check(Inputs.SceneDepth.IsValid());
	check(Inputs.SceneVelocity.IsValid());
	checkf(Inputs.SceneDepth.ViewRect == Inputs.SceneVelocity.ViewRect, TEXT("The implementation requires that depth and velocity have the same viewport."));

	FScreenPassTexture PrevColor;
	if (View.PrevViewInfo.VisualizeMotionVectors.IsValid())
	{
		PrevColor.Texture = GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.VisualizeMotionVectors);
		PrevColor.ViewRect = View.PrevViewInfo.VisualizeMotionVectorsRect;
	}
	else
	{
		PrevColor.Texture = GSystemTextures.GetBlackDummy(GraphBuilder);
		PrevColor.ViewRect = FIntRect(0, 0, 1, 1);
	}

	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	{
		if (!Output.IsValid())
		{
			Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("MotionVectors.Visualize"));
		}

		FVisualizeMotionVectorsPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeMotionVectorsPS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;

		PassParameters->PrevDistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		PassParameters->PrevDistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		if (View.PrevViewInfo.bIsVisualizeMotionVectorsDistorted)
		{
			PassParameters->PrevDistortingDisplacementTexture = GraphBuilder.RegisterExternalTexture(View.PrevViewInfo.DistortingDisplacementTexture);
		}
		if (Inputs.LensDistortionLUT.IsEnabled())
		{
			PassParameters->UndistortingDisplacementTexture = Inputs.LensDistortionLUT.UndistortingDisplacementTexture;
		}

		PassParameters->ColorTexture = Inputs.SceneColor.Texture;
		PassParameters->DepthTexture = Inputs.SceneDepth.Texture;
		PassParameters->VelocityTexture = Inputs.SceneVelocity.Texture;
		PassParameters->PrevColorTexture = PrevColor.Texture;

		PassParameters->Color = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Inputs.SceneColor));
		PassParameters->Velocity = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(Inputs.SceneDepth));
		PassParameters->PrevColor = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(PrevColor));

		PassParameters->ColorSampler     = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->VelocitySampler  = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->DepthSampler     = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->PrevColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		PassParameters->SvPositionToScreenPos = (
			FScreenTransform::SvPositionToViewportUV(Output.ViewRect) *
			FScreenTransform::ViewportUVToScreenPos);
		PassParameters->ScreenPosToColor = FScreenTransform::ChangeTextureBasisFromTo(
			Inputs.SceneColor, FScreenTransform::ETextureBasis::ScreenPosition, FScreenTransform::ETextureBasis::TextureUV);
		PassParameters->ScreenPosToVelocity = FScreenTransform::ChangeTextureBasisFromTo(
			Inputs.SceneDepth, FScreenTransform::ETextureBasis::ScreenPosition, FScreenTransform::ETextureBasis::TextureUV);
		PassParameters->PrevScreenPosToPrevColor = FScreenTransform::ChangeTextureBasisFromTo(
			PrevColor, FScreenTransform::ETextureBasis::ScreenPosition, FScreenTransform::ETextureBasis::TextureUV);
		PassParameters->VisualizeType = int32(Visualize);

		PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();

		TShaderMapRef<FVisualizeMotionVectorsPS> PixelShader(View.ShaderMap);
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("VisualizeMotionVectors %dx%d", Output.ViewRect.Width(), Output.ViewRect.Height()),
			PixelShader,
			PassParameters,
			Output.ViewRect);
	}

	if (View.ViewState && !View.bStatePrevViewInfoIsReadOnly)
	{
		GraphBuilder.QueueTextureExtraction(Inputs.SceneColor.Texture, &View.ViewState->PrevFrameViewInfo.VisualizeMotionVectors);
		View.ViewState->PrevFrameViewInfo.VisualizeMotionVectorsRect = Inputs.SceneColor.ViewRect;
		View.ViewState->PrevFrameViewInfo.bIsVisualizeMotionVectorsDistorted = Inputs.LensDistortionLUT.IsEnabled();
		
		if (Inputs.LensDistortionLUT.IsEnabled())
		{
			GraphBuilder.QueueTextureExtraction(Inputs.LensDistortionLUT.DistortingDisplacementTexture, &View.ViewState->PrevFrameViewInfo.DistortingDisplacementTexture);
		}
	}

	return MoveTemp(Output);
}
