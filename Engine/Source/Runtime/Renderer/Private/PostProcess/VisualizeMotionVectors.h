// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "PostProcess/LensDistortion.h"


enum class EVisualizeMotionVectors : uint8
{
	ReprojectionAlignment,
	HasPixelAnimationFlag,
};

struct FVisualizeMotionVectorsInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	FScreenPassTexture SceneColor;
	FScreenPassTexture SceneDepth;
	FScreenPassTexture SceneVelocity;

	// [Optional] Lens distortion applied on the scene color.
	FLensDistortionLUT LensDistortionLUT;
};

FScreenPassTexture AddVisualizeMotionVectorsPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeMotionVectorsInputs& Inputs, EVisualizeMotionVectors Visualize);