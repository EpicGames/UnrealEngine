// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/VisualizeShadingModels.h"
#include "CanvasTypes.h"
#include "SceneRendering.h"
#include "SceneTextureParameters.h"
#include "UnrealEngine.h"
#include "DataDrivenShaderPlatformInfo.h"

class FVisualizeShadingModelPS : public FGlobalShader
{
public:
	static const uint32 ShadingModelCount = 16;

	DECLARE_GLOBAL_SHADER(FVisualizeShadingModelPS);
	SHADER_USE_PARAMETER_STRUCT(FVisualizeShadingModelPS, FGlobalShader);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneTextureUniformParameters, SceneTextures)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_SCALAR_ARRAY(uint32, ShadingModelMaskInView, [ShadingModelCount])
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

static_assert(
	sizeof(FViewInfo::ShadingModelMaskInView) * 8 == FVisualizeShadingModelPS::ShadingModelCount,
	"Number of shading model bits doesn't match the shader.");

IMPLEMENT_GLOBAL_SHADER(FVisualizeShadingModelPS, "/Engine/Private/VisualizeShadingModels.usf", "MainPS", SF_Pixel);

FScreenPassTexture AddVisualizeShadingModelPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeShadingModelInputs& Inputs)
{
	check(Inputs.SceneTextures);
	check(Inputs.SceneColor.IsValid());
	
	FScreenPassRenderTarget Output = Inputs.OverrideOutput;

	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("VisualizeShadingModel"));
	}

	const FScreenPassTextureViewport InputViewport(Inputs.SceneColor);
	const FScreenPassTextureViewport OutputViewport(Output);

	FVisualizeShadingModelPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FVisualizeShadingModelPS::FParameters>();
	PassParameters->RenderTargets[0] = Output.GetRenderTargetBinding();
	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneTextures = Inputs.SceneTextures;
	PassParameters->SceneColorTexture = Inputs.SceneColor.Texture;
	PassParameters->SceneColorSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	for (uint32 BitIndex = 0; BitIndex < FVisualizeShadingModelPS::ShadingModelCount; ++BitIndex)
	{
		const uint32 BitMask = (1 << BitIndex);

		GET_SCALAR_ARRAY_ELEMENT(PassParameters->ShadingModelMaskInView, BitIndex) = (View.ShadingModelMaskInView & BitMask) != 0;
	}

	TShaderMapRef<FVisualizeShadingModelPS> PixelShader(View.ShaderMap);

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeShadingModels");

	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("Visualizer"), View, OutputViewport, InputViewport, PixelShader, PassParameters);

	Output.LoadAction = ERenderTargetLoadAction::ELoad;

	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("Overlay"), View, Output, [&View](FCanvas& Canvas)
	{
		float X = 30;
		float Y = 28;
		const float YStep = 14;
		const float ColumnWidth = 250;
	
		FString Line;

		Canvas.DrawShadowedString(X, Y += YStep, TEXT("Visualize ShadingModels"), GetStatsFont(), FLinearColor(1, 1, 1));

		Y = 160 - YStep - 4;

		const uint32 Value = View.ShadingModelMaskInView;

		Line = FString::Printf(TEXT("View.ShadingModelMaskInView = 0x%x"), Value);
		Canvas.DrawShadowedString(X, Y, *Line, GetStatsFont(), FLinearColor(0.5f, 0.5f, 0.5f));
		Y += YStep;

		UEnum* Enum = StaticEnum<EMaterialShadingModel>();
		check(Enum);

		Y += 5;

		for (uint32 i = 0; i < MSM_NUM; ++i)
		{
			FString Name = Enum->GetNameStringByValue(i);
			Line = FString::Printf(TEXT("%d.  %s"), i, *Name);

			bool bThere = (Value & (1 << i)) != 0;

			Canvas.DrawShadowedString(X + 30, Y, *Line, GetStatsFont(), bThere ? FLinearColor(1, 1, 1) : FLinearColor(0, 0, 0));
			Y += 20;
		}

		Line = FString::Printf(TEXT("(On CPU, based on what gets rendered)"));
		Canvas.DrawShadowedString(X, Y, *Line, GetStatsFont(), FLinearColor(0.5f, 0.5f, 0.5f)); Y += YStep;
	});

	return MoveTemp(Output);
}
