// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ScreenPass.h"
#include "OverridePassSequence.h"

class FSceneTextureParameters;

struct FVisualizeShadingModelInputs
{
	// [Optional] Render to the specified output. If invalid, a new texture is created and returned.
	FScreenPassRenderTarget OverrideOutput;

	// [Required] The scene color to composite with the shading model visualization.
	FScreenPassTexture SceneColor;

	// [Required] The scene textures used to visualize shading models.
	TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextures = nullptr;
};

FScreenPassTexture AddVisualizeShadingModelPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeShadingModelInputs& Inputs);