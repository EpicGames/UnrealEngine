// Copyright Epic Games, Inc. All Rights Reserved.

#include "PostProcess/TemporalAA.h"
#include "PostProcess/PostProcessing.h"
#include "ScreenPass.h"
#include "PixelShaderUtils.h"
#include "UnrealEngine.h"
#include "PostProcess/PostProcessMotionBlur.h"
#include "PostProcess/PostProcessVisualizeBuffer.h"
#include "PostProcess/VisualizeMotionVectors.h"
#include "VisualizeTexture.h"


FScreenPassTexture AddVisualizeTemporalUpscalerPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FVisualizeTemporalUpscalerInputs& Inputs)
{
	check(Inputs.SceneColor.IsValid());

	RDG_EVENT_SCOPE(GraphBuilder, "VisualizeTemporalUpscaler %dx%d", Inputs.SceneColor.ViewRect.Width(), Inputs.SceneColor.ViewRect.Height());

	const bool bSupportsAlpha = IsPostProcessingWithAlphaChannelSupported();

	// Allocate output and copy
	FScreenPassRenderTarget Output = Inputs.OverrideOutput;
	if (!Output.IsValid())
	{
		Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, Inputs.SceneColor, View.GetOverwriteLoadAction(), TEXT("MotionVectors.Visualize"));
	}

	// Populate all the tiles
	TArray<FVisualizeBufferTile> Tiles;
	Tiles.SetNum(16);

	if (Inputs.TAAConfig != EMainTAAPassConfig::Disabled)
	{
		FScreenPassTexture OutputTexture;
		if (Inputs.Outputs.FullRes.TextureSRV->Desc.Texture->Desc.IsTextureArray())
		{
			FRDGTextureDesc Desc = Inputs.Outputs.FullRes.TextureSRV->Desc.Texture->Desc;
			Desc.Dimension = ETextureDimension::Texture2D;
			Desc.ArraySize = 1;
			Desc.NumMips = 1;

			FRDGTextureRef NewTexture = GraphBuilder.CreateTexture(Desc, Inputs.Outputs.FullRes.TextureSRV->Desc.Texture->Name);

			FRHICopyTextureInfo CopyInfo;
			CopyInfo.SourceSliceIndex = Inputs.Outputs.FullRes.TextureSRV->Desc.FirstArraySlice;

			AddCopyTexturePass(
				GraphBuilder,
				Inputs.Outputs.FullRes.TextureSRV->Desc.Texture,
				NewTexture,
				CopyInfo);

			OutputTexture = FScreenPassTexture(NewTexture, Inputs.Outputs.FullRes.ViewRect);
		}
		else
		{
			OutputTexture = FScreenPassTexture(Inputs.Outputs.FullRes.TextureSRV->Desc.Texture, Inputs.Outputs.FullRes.ViewRect);
		}

		auto VisualizeTextureLabel = [](FRDGTextureRef Texture, const TCHAR* Suffix = TEXT(""))
		{
			return FString::Printf(TEXT("vis %s%s"), Texture->Name, Suffix);
		};

		auto CropViewRectToCenter = [](FIntRect ViewRect)
		{
			return FIntRect(ViewRect.Min + ViewRect.Size() / 4, ViewRect.Min + (ViewRect.Size() * 3) / 4);
		};

		// Depth buffer
		{
			FVisualizeBufferTile& Tile = Tiles[4 * 0 + 0];
			Tile.Input.Texture = FVisualizeTexture::AddVisualizeTexturePass(GraphBuilder, View.ShaderMap, Inputs.Inputs.SceneDepth.Texture);
			Tile.Input.ViewRect = CropViewRectToCenter(View.ViewRect);
			Tile.Label = VisualizeTextureLabel(Inputs.Inputs.SceneDepth.Texture);
		}

		// Scene Color
		{
			FVisualizeBufferTile& Tile = Tiles[4 * 0 + 1];
			Tile.Input = FScreenPassTexture(Inputs.Inputs.SceneColor.Texture, CropViewRectToCenter(View.ViewRect));
			Tile.Label = VisualizeTextureLabel(Inputs.Inputs.SceneColor.Texture);
		}

		// Scene Color alpha
		if (bSupportsAlpha)
		{
			FVisualizeBufferTile& Tile = Tiles[4 * 3 + 1];
			//Tile.Input = FScreenPassTexture(Inputs.Inputs.SceneColorTexture, CropViewRectToCenter(View.ViewRect));
			Tile.Input.Texture = FVisualizeTexture::AddVisualizeTextureAlphaPass(GraphBuilder, View.ShaderMap, Inputs.Inputs.SceneColor.Texture);
			Tile.Input.ViewRect = CropViewRectToCenter(View.ViewRect);
			Tile.Label = VisualizeTextureLabel(Inputs.Inputs.SceneColor.Texture, TEXT(" A"));
		}

		// Translucency
		if (Inputs.TAAConfig == EMainTAAPassConfig::TSR)
		{
			FVisualizeBufferTile& Tile = Tiles[4 * 0 + 2];
			if (Inputs.Inputs.PostDOFTranslucencyResources.IsValid())
			{
				Tile.Input.Texture = Inputs.Inputs.PostDOFTranslucencyResources.ColorTexture.Resolve;
				Tile.Input.ViewRect = CropViewRectToCenter(Inputs.Inputs.PostDOFTranslucencyResources.ViewRect);
				Tile.Label = VisualizeTextureLabel(Inputs.Inputs.PostDOFTranslucencyResources.ColorTexture.Resolve);
			}
			else
			{
				Tile.Input = FScreenPassTexture(GSystemTextures.GetBlackDummy(GraphBuilder));
				Tile.Label = TEXT("No PostDOF Translucency!");
			}
		}

		// Translucency alpha
		if (Inputs.TAAConfig == EMainTAAPassConfig::TSR)
		{
			FVisualizeBufferTile& Tile = Tiles[4 * 0 + 3];
			if (Inputs.Inputs.PostDOFTranslucencyResources.IsValid())
			{
				Tile.Input.Texture = FVisualizeTexture::AddVisualizeTextureAlphaPass(GraphBuilder, View.ShaderMap, Inputs.Inputs.PostDOFTranslucencyResources.ColorTexture.Resolve);
				Tile.Input.ViewRect = CropViewRectToCenter(Inputs.Inputs.PostDOFTranslucencyResources.ViewRect);
				Tile.Label = VisualizeTextureLabel(Inputs.Inputs.PostDOFTranslucencyResources.ColorTexture.Resolve, TEXT(" A"));
			}
			else
			{
				Tile.Input = FScreenPassTexture(GSystemTextures.GetBlackDummy(GraphBuilder));
				Tile.Label = TEXT("No PostDOF Translucency!");
			}
		}

		// Motion blur
		{
			FMotionBlurInputs PassInputs;
			PassInputs.SceneColor = Inputs.Outputs.FullRes;
			PassInputs.SceneDepth = Inputs.Inputs.SceneDepth;
			PassInputs.SceneVelocity = Inputs.Inputs.SceneVelocity;
			PassInputs.LensDistortionLUT = Inputs.Inputs.LensDistortionLUT;

			FVisualizeBufferTile& Tile = Tiles[4 * 1 + 0];
			Tile.Input = FScreenPassTexture(AddVisualizeMotionBlurPass(GraphBuilder, View, PassInputs));
			Tile.Input.ViewRect = CropViewRectToCenter(Tile.Input.ViewRect);
			Tile.Label = TEXT("show VisualizeMotionBlur");
		}

		// Reprojection
		{
			FVisualizeMotionVectorsInputs PassInputs;
			PassInputs.SceneColor = Inputs.SceneColor;
			PassInputs.SceneDepth = Inputs.Inputs.SceneDepth;
			PassInputs.SceneVelocity = Inputs.Inputs.SceneVelocity;
			PassInputs.LensDistortionLUT = Inputs.Inputs.LensDistortionLUT;

			FVisualizeBufferTile& Tile = Tiles[4 * 2 + 0];
			Tile.Input = AddVisualizeMotionVectorsPass(GraphBuilder, View, PassInputs, EVisualizeMotionVectors::ReprojectionAlignment);
			Tile.Input.ViewRect = CropViewRectToCenter(Tile.Input.ViewRect);
			Tile.Label = TEXT("show VisualizeReprojection");
		}

		// Display TSR's luminance used for stability.
		if (Inputs.TAAConfig == EMainTAAPassConfig::TSR)
		{
			FVisualizeBufferTile& Tile = Tiles[4 * 1 + 3];
			if (Inputs.Inputs.FlickeringInputTexture.IsValid())
			{
				Tile.Input = Inputs.Inputs.FlickeringInputTexture;
				Tile.Input.ViewRect = CropViewRectToCenter(View.ViewRect);
				Tile.Label = VisualizeTextureLabel(Inputs.Inputs.FlickeringInputTexture.Texture);
			}
			else
			{
				Tile.Input = FScreenPassTexture(GSystemTextures.GetBlackDummy(GraphBuilder));
				Tile.Label = TEXT("No Moire Luma!");
			}
		}

		// Display UMaterial::bHasPixelAnimation used to disable TSR's anti-flickering heuristic (r.TSR.ShadingRejection.Flickering) on per pixel basis
		if (Inputs.TAAConfig == EMainTAAPassConfig::TSR)
		{
			FVisualizeMotionVectorsInputs PassInputs;
			PassInputs.SceneColor = Inputs.SceneColor;
			PassInputs.SceneDepth = Inputs.Inputs.SceneDepth;
			PassInputs.SceneVelocity = Inputs.Inputs.SceneVelocity;

			FVisualizeBufferTile& Tile = Tiles[4 * 2 + 3];
			Tile.Input = AddVisualizeMotionVectorsPass(GraphBuilder, View, PassInputs, EVisualizeMotionVectors::HasPixelAnimationFlag);
			Tile.Input.ViewRect = CropViewRectToCenter(Tile.Input.ViewRect);
			Tile.Label = TEXT("UMaterial::bHasPixelAnimation");
		}

		// Output
		{
			FVisualizeBufferTile& Tile = Tiles[4 * 3 + 3];
			Tile.Input.Texture = OutputTexture.Texture;
			Tile.Input.ViewRect = CropViewRectToCenter(OutputTexture.ViewRect);
			Tile.Label = VisualizeTextureLabel(OutputTexture.Texture);
		}

		// Output alpha
		if (bSupportsAlpha)
		{
			FVisualizeBufferTile& Tile = Tiles[4 * 3 + 2];
			Tile.Input.Texture = FVisualizeTexture::AddVisualizeTextureAlphaPass(GraphBuilder, View.ShaderMap, OutputTexture.Texture);
			Tile.Input.ViewRect = CropViewRectToCenter(OutputTexture.ViewRect);
			Tile.Label = VisualizeTextureLabel(OutputTexture.Texture, TEXT(" A"));
		}

		// Black bottom left corner
		{
			FVisualizeBufferTile& Tile = Tiles[4 * 3 + 0];
			Tile.Input = FScreenPassTexture(GSystemTextures.GetBlackDummy(GraphBuilder));
			Tile.Label = TEXT("Summary");
		}
	}

	// Draws all overview tiles
	{
		FVisualizeBufferInputs PassInputs;
		PassInputs.OverrideOutput = Output;
		PassInputs.SceneColor = Inputs.SceneColor;
		PassInputs.Tiles = Tiles;

		AddVisualizeBufferPass(GraphBuilder, View, PassInputs);
	}

	// Draw additional text
	// Early return if not using a temporal upscaler.
	if (Inputs.TAAConfig != EMainTAAPassConfig::Disabled)
	{
		AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("VisualizeTemporalUpscaler Text"), View, FScreenPassRenderTarget(Output, ERenderTargetLoadAction::ELoad),
			[&View, &ViewRect = Output.ViewRect, &Inputs, bSupportsAlpha](FCanvas& Canvas)
		{
			const float DPIScale = Canvas.GetDPIScale();
			Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale) * Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

			auto QuickDrawSummary = [&](int32 Location, const FString& Text)
			{
				FIntPoint LabelLocation(20, 20 + 22 * Location + (ViewRect.Height() * 3) / 4);
				Canvas.DrawShadowedString(LabelLocation.X / DPIScale, LabelLocation.Y / DPIScale, *Text, GetStatsFont(), FLinearColor::White);
			};

			// Display which temporal upscaler is being used.
			{
				static auto CVarAntiAliasingQuality = IConsoleManager::Get().FindConsoleVariable(TEXT("sg.AntiAliasingQuality"));
				check(CVarAntiAliasingQuality);

				FString Text;
				if (Inputs.TAAConfig == EMainTAAPassConfig::TAA)
				{
					Text = FString::Printf(TEXT("TemporalUpscaler: TAA (sg.AntiAliasingQuality=%d)"), CVarAntiAliasingQuality->GetInt());
				}
				else if (Inputs.TAAConfig == EMainTAAPassConfig::TSR)
				{
					Text = FString::Printf(TEXT("TemporalUpscaler: TSR (sg.AntiAliasingQuality=%d)"), CVarAntiAliasingQuality->GetInt());
				}
				else
				{
					Text = FString::Printf(TEXT("ThirdParty TemporalUpscaler: %s"), Inputs.UpscalerUsed->GetDebugName());
				}
				QuickDrawSummary(/* Location = */ 0, Text);
			}

			// Display the input/output resolutions
			{
				QuickDrawSummary(/* Location = */ 1, FString::Printf(TEXT("Input: %dx%d %s"), View.ViewRect.Width(), View.ViewRect.Height(), GPixelFormats[Inputs.Inputs.SceneColor.Texture->Desc.Format].Name));
				QuickDrawSummary(/* Location = */ 2, FString::Printf(TEXT("Output: %dx%d %s"), Inputs.Outputs.FullRes.ViewRect.Width(), Inputs.Outputs.FullRes.ViewRect.Height(), GPixelFormats[Inputs.Outputs.FullRes.TextureSRV->Desc.Texture->Desc.Format].Name));
			}

			// Display the pre-exposure being used
			{
				static auto CVarPreExposureOverride = IConsoleManager::Get().FindConsoleVariable(TEXT("r.EyeAdaptation.PreExposureOverride"));
				check(CVarPreExposureOverride);

				FString Text = FString::Printf(TEXT("PreExposure: %f"), View.PreExposure);
				if (CVarPreExposureOverride->GetFloat() > 0.0f)
				{
					Text += TEXT(" (r.EyeAdaptation.PreExposureOverride)");
				}

				QuickDrawSummary(/* Location = */ 3, Text);
			}

			// Display alpha support
			{
				FString Text = bSupportsAlpha ? TEXT("r.PostProcessing.PropagateAlpha=true") : TEXT("r.PostProcessing.PropagateAlpha=false");
				if (Inputs.TAAConfig == EMainTAAPassConfig::ThirdParty)
				{
					Text = TEXT("Unknown");
				}
				QuickDrawSummary(/* Location = */ 4, TEXT("Support Alpha: ") + Text);
			}

			// Display memory size of the history.
			{
				static auto CVarAntiAliasingQuality = IConsoleManager::Get().FindConsoleVariable(TEXT("sg.AntiAliasingQuality"));
				check(CVarAntiAliasingQuality);

				const uint64 PingPongHistorySizeMultiplier = 2;
				uint64 HistorySize = 0;
				if (Inputs.TAAConfig == EMainTAAPassConfig::TAA)
				{
					HistorySize = View.PrevViewInfo.TemporalAAHistory.GetGPUSizeBytes(/* bLogSizes = */ false) * PingPongHistorySizeMultiplier;
				}
				else if (Inputs.TAAConfig == EMainTAAPassConfig::TSR && View.PrevViewInfo.TSRHistory.IsValid())
				{
					HistorySize = View.PrevViewInfo.TSRHistory.GetGPUSizeBytes(/* bLogSizes = */ false);

					bool bHasTSRHistoryResurrection = View.PrevViewInfo.TSRHistory.MetadataArray->GetDesc().ArraySize > 1;
					bool bHasTSRPingPongHistory = !bHasTSRHistoryResurrection;
					if (bHasTSRPingPongHistory)
					{
						HistorySize *= PingPongHistorySizeMultiplier;
					}
				}
				else if (Inputs.TAAConfig == EMainTAAPassConfig::ThirdParty && View.PrevViewInfo.ThirdPartyTemporalUpscalerHistory.IsValid())
				{
					HistorySize = View.PrevViewInfo.ThirdPartyTemporalUpscalerHistory->GetGPUSizeBytes();
				}
				QuickDrawSummary(/* Location = */ 5, FString::Printf(TEXT("History VRAM footprint: %.1f MB"), float(HistorySize) / float(1024 * 1024)));
			}

			// Display if any additional sharpening is happening
			{
				static auto CVarSharpen = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Tonemapper.Sharpen"));
				check(CVarSharpen);
				float Sharpen = CVarSharpen->GetFloat();
				Sharpen = (Sharpen < 0) ? View.FinalPostProcessSettings.Sharpen : Sharpen;
				QuickDrawSummary(/* Location = */ 6, Sharpen > 0 ? FString::Printf(TEXT("Tonemapper Sharpen: %f"), Sharpen) : TEXT("Tonemapper Sharpen: Off"));
			}

			// Display if doing any lens distortion
			{
				QuickDrawSummary(/* Location = */ 7, View.LensDistortionLUT.IsEnabled() ? FString::Printf(TEXT("Lens Distortion: Enabled (%3.1f%% Additional Upscale)"), View.LensDistortionLUT.ResolutionFraction * 100.0f) : TEXT("Lens Distortion: Disabled"));
			}
		});

	}
	else
	{
		check(Inputs.TAAConfig == EMainTAAPassConfig::Disabled);
		AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("VisualizeTemporalUpscaler Text"), View, FScreenPassRenderTarget(Output, ERenderTargetLoadAction::ELoad),
			[&ViewRect = Output.ViewRect](FCanvas& Canvas)
			{
				const float DPIScale = Canvas.GetDPIScale();
				Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale) * Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

				FIntPoint LabelLocation(60, 60);
				Canvas.DrawShadowedString(LabelLocation.X / DPIScale, LabelLocation.Y / DPIScale, TEXT("No temporal upscaler used"), GetStatsFont(), FLinearColor::Red);
			});
	}

	return MoveTemp(Output);
}
