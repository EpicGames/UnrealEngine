// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PrimitiveSceneInfo.cpp: Primitive scene info implementation.
=============================================================================*/

#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "Components/PrimitiveComponent.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneCore.h"
#include "VelocityRendering.h"
#include "ScenePrivate.h"
#include "RayTracingGeometry.h"
#include "Components/ComponentInterfaces.h"

#include "LogRenderer.h"
#include "HAL/LowLevelMemTracker.h"
#include "RayTracing/RayTracingMaterialHitShaders.h"
#include "RayTracing/RayTracingInstanceMask.h"
#include "VT/RuntimeVirtualTextureSceneProxy.h"
#include "VT/VirtualTextureSystem.h"
#include "GPUScene.h"
#include "Async/ParallelFor.h"
#include "ProfilingDebugging/ExternalProfiler.h"
#include "Nanite/Nanite.h"
#include "Nanite/NaniteRayTracing.h"
#include "Nanite/NaniteShading.h"
#include "Rendering/NaniteResources.h"
#include "NaniteSceneProxy.h"
#include "Lumen/LumenSceneData.h"
#include "Lumen/LumenSceneCardCapture.h"
#include "RayTracingDefinitions.h"
#include "RenderCore.h"
#include "Materials/MaterialRenderProxy.h"
#include "StaticMeshBatch.h"
#include "PrimitiveSceneDesc.h"
#include "BasePassRendering.h" // TODO: Remove with later refactor (moving Nanite shading into its own files)
#include "InstanceDataSceneProxy.h"
#include "DecalRenderingCommon.h"
#include "RendererPrivateUtils.h"
#include "ODSC/ODSCManager.h"
#include "VirtualTextureEnum.h"
#include "MaterialCache/MaterialCacheMeshProcessor.h"
#include "MaterialCache/MaterialCacheVirtualTextureRenderProxy.h"
#include "Streaming/SimpleStreamableAssetManager.h"
#include "Nanite/NaniteMaterialsSceneExtension.h"

extern int32 GGPUSceneInstanceClearList;

static int32 GMeshDrawCommandsCacheMultithreaded = 1;
static FAutoConsoleVariableRef CVarDrawCommandsCacheMultithreaded(
	TEXT("r.MeshDrawCommands.CacheMultithreaded"),
	GMeshDrawCommandsCacheMultithreaded,
	TEXT("Enable multithreading of draw command caching for static meshes. 0=disabled, 1=enabled (default)"),
	ECVF_RenderThreadSafe);

static int32 GMeshDrawCommandsBatchSize = 12;
static FAutoConsoleVariableRef CVarDrawCommandsCacheMultithreadedBatchSize(
	TEXT("r.MeshDrawCommands.BatchSize"),
	GMeshDrawCommandsBatchSize,
	TEXT("Batch size of cache mesh draw commands when multithreading of draw command caching is enabled"),
	ECVF_RenderThreadSafe);

static int32 GNaniteMaterialBinCacheParallel = 1;
static FAutoConsoleVariableRef CVarNaniteCacheMaterialBinsParallel(
	TEXT("r.Nanite.CacheMaterialBinsParallel"),
	GNaniteMaterialBinCacheParallel,
	TEXT("Enable parallel caching of raster and shading bins for Nanite materials. 0=disabled, 1=enabled (default)"),
	ECVF_RenderThreadSafe);

static int32 GRayTracingPrimitiveCacheMultithreaded = 1;
static FAutoConsoleVariableRef CVarRayTracingPrimitiveCacheMultithreaded(
	TEXT("r.RayTracing.MeshDrawCommands.CacheMultithreaded"),
	GRayTracingPrimitiveCacheMultithreaded,
	TEXT("Enable multithreading of raytracing primitive mesh command caching. 0=disabled, 1=enabled (default)"),
	ECVF_RenderThreadSafe);

extern TAutoConsoleVariable<int32> CVarVirtualShadowMapNaniteAllowTessellationDirectional;
extern TAutoConsoleVariable<int32> CVarVirtualShadowMapNaniteAllowTessellationLocal;
static bool ShouldForceNaniteDisplacementFallback()
{
	return CVarVirtualShadowMapNaniteAllowTessellationDirectional.GetValueOnAnyThread() == 0 ||
		CVarVirtualShadowMapNaniteAllowTessellationLocal.GetValueOnAnyThread() == 0;
}

/** An implementation of FStaticPrimitiveDrawInterface that stores the drawn elements for the rendering thread to use. */
class FBatchingSPDI : public FStaticPrimitiveDrawInterface
{
public:

	// Constructor.
	FBatchingSPDI(FPrimitiveSceneInfo* InPrimitiveSceneInfo):
		PrimitiveSceneInfo(InPrimitiveSceneInfo)
	{}

	// FStaticPrimitiveDrawInterface.
	virtual void SetHitProxy(HHitProxy* HitProxy) final override
	{
		CurrentHitProxy = HitProxy;

		if(HitProxy)
		{
			// Only use static scene primitive hit proxies in the editor.
			if(GIsEditor)
			{
				// Keep a reference to the hit proxy from the FPrimitiveSceneInfo, to ensure it isn't deleted while the static mesh still
				// uses its id.
				PrimitiveSceneInfo->HitProxies.Add(HitProxy);
			}
		}
	}

	virtual void ReserveMemoryForMeshes(int32 MeshNum)
	{
		PrimitiveSceneInfo->StaticMeshRelevances.Reserve(PrimitiveSceneInfo->StaticMeshRelevances.Num() + MeshNum);
		PrimitiveSceneInfo->StaticMeshes.Reserve(PrimitiveSceneInfo->StaticMeshes.Num() + MeshNum);
	}

	virtual void DrawMesh(const FMeshBatch& Mesh, float ScreenSize) final override
	{
		if (Mesh.HasAnyDrawCalls())
		{
			checkSlow(IsInParallelRenderingThread());

			FPrimitiveSceneProxy* PrimitiveSceneProxy = PrimitiveSceneInfo->Proxy;
			const ERHIFeatureLevel::Type FeatureLevel = PrimitiveSceneInfo->Scene->GetFeatureLevel();

			if (!Mesh.Validate(PrimitiveSceneProxy, FeatureLevel))
			{
				return;
			}

			FStaticMeshBatch* StaticMesh = new(PrimitiveSceneInfo->StaticMeshes) FStaticMeshBatch(
				PrimitiveSceneInfo,
				Mesh,
				CurrentHitProxy ? CurrentHitProxy->Id : FHitProxyId()
			);

			StaticMesh->PreparePrimitiveUniformBuffer(PrimitiveSceneProxy, FeatureLevel);
			// Volumetric self shadow mesh commands need to be generated every frame, as they depend on single frame uniform buffers with self shadow data.
			const bool bSupportsCachingMeshDrawCommands = SupportsCachingMeshDrawCommands(*StaticMesh, FeatureLevel) && !PrimitiveSceneProxy->CastsVolumetricTranslucentShadow();

			const FMaterial& Material = Mesh.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
			bool bUseSkyMaterial = Material.IsSky();
			bool bUseSingleLayerWaterMaterial = Material.GetShadingModels().HasShadingModel(MSM_SingleLayerWater);
			bool bUseAnisotropy = Material.GetShadingModels().HasAnyShadingModel({MSM_DefaultLit, MSM_ClearCoat}) && Material.MaterialUsesAnisotropy_RenderThread();
			bool bSupportsNaniteRendering = SupportsNaniteRendering(StaticMesh->VertexFactory, PrimitiveSceneProxy, Mesh.MaterialRenderProxy, FeatureLevel);
			bool bSupportsGPUScene = StaticMesh->VertexFactory->SupportsGPUScene(FeatureLevel);
			bool bUseForWaterInfoTextureDepth = Mesh.bUseForWaterInfoTextureDepth;
			bool bUseForLumenSceneCapture = Mesh.bUseForLumenSurfaceCacheCapture;

			uint8 DecalRenderTargetModeMask = 0;
			if (Mesh.IsDecal(FeatureLevel))
			{
				DecalRenderTargetModeMask = DecalRendering::GetDecalRenderTargetModeMask(Material, FeatureLevel);
			}

			FStaticMeshBatchRelevance* StaticMeshRelevance = new(PrimitiveSceneInfo->StaticMeshRelevances) FStaticMeshBatchRelevance(
				*StaticMesh, 
				ScreenSize, 
				bSupportsCachingMeshDrawCommands,
				bUseSkyMaterial,
				bUseSingleLayerWaterMaterial,
				bUseAnisotropy,
				bSupportsNaniteRendering,
				bSupportsGPUScene,
				bUseForWaterInfoTextureDepth,
				bUseForLumenSceneCapture,
				DecalRenderTargetModeMask,
				FeatureLevel
				);
		}
	}

private:
	FPrimitiveSceneInfo* PrimitiveSceneInfo;
	TRefCountPtr<HHitProxy> CurrentHitProxy;
};

FPrimitiveSceneInfo::FPrimitiveSceneInfoEvent FPrimitiveSceneInfo::OnGPUSceneInstancesAllocated;
FPrimitiveSceneInfo::FPrimitiveSceneInfoEvent FPrimitiveSceneInfo::OnGPUSceneInstancesFreed;

FPrimitiveFlagsCompact::FPrimitiveFlagsCompact(const FPrimitiveSceneProxy* Proxy)
	: bCastDynamicShadow(Proxy->CastsDynamicShadow())
	, bStaticLighting(Proxy->HasStaticLighting())
	, bCastStaticShadow(Proxy->CastsStaticShadow())
	, bIsNaniteMesh(Proxy->IsNaniteMesh())
	, bIsAlwaysVisible(Proxy->IsAlwaysVisible())
	, bSupportsGPUScene(Proxy->SupportsGPUScene())
	, bIsForceHidden(Proxy->IsForceHidden())
{}

FPrimitiveSceneInfoCompact::FPrimitiveSceneInfoCompact(FPrimitiveSceneInfo* InPrimitiveSceneInfo) :
	PrimitiveFlagsCompact(InPrimitiveSceneInfo->Proxy)
{
	PrimitiveSceneInfo = InPrimitiveSceneInfo;
	Proxy = PrimitiveSceneInfo->Proxy;
	Bounds = FCompactBoxSphereBounds(PrimitiveSceneInfo->Proxy->GetBounds());
	MinDrawDistance = PrimitiveSceneInfo->Proxy->GetMinDrawDistance();
	MaxDrawDistance = PrimitiveSceneInfo->Proxy->GetMaxDrawDistance();

	VisibilityId = PrimitiveSceneInfo->Proxy->GetVisibilityId();
}

struct FPrimitiveSceneInfoAdapter
{
	void CreateHitProxies()
	{
		if (PrimitiveComponentInterface)
		{
			// Support for legacy path for proxy creation, if not handled it'll internally invoke the IPrimitiveComponentInterface path
			if (UPrimitiveComponent* PrimitiveComponent =  PrimitiveComponentInterface->GetUObject<UPrimitiveComponent>())
			{
				DefaultHitProxy = SceneProxy->CreateHitProxies(PrimitiveComponent, HitProxies);
			}
			else 
			{
				// For all other implementers
				DefaultHitProxy = SceneProxy->CreateHitProxies(PrimitiveComponentInterface, HitProxies);
			}
		}
	}

	FPrimitiveSceneInfoAdapter(UPrimitiveComponent* InComponent)
	{
		SceneProxy = InComponent->SceneProxy;
		SceneData = &InComponent->SceneData;
		ComponentId = SceneData->PrimitiveSceneId;
		Component = InComponent;
		PrimitiveComponentInterface = InComponent->GetPrimitiveComponentInterface();
		PrimitiveDesc = nullptr;
		
		// This validates the UPrimitiveComponent has properly initialized its OwnerLastRenderTimePtr
		check(InComponent->SceneData.OwnerLastRenderTimePtr == FActorLastRenderTime::GetPtr(InComponent->GetOwner()));
		Mobility = InComponent->Mobility;

		const UPrimitiveComponent* SearchParentComponent = InComponent->GetLightingAttachmentRoot();

		if (SearchParentComponent && SearchParentComponent != InComponent)
		{
			LightingAttachmentComponentId = SearchParentComponent->GetPrimitiveSceneId();
		}

		// set LOD parent info if exists
		UPrimitiveComponent* LODParent = InComponent->GetLODParentPrimitive();
		if (LODParent)
		{
			LODParentComponentId = LODParent->GetPrimitiveSceneId();
		}

		if (GIsEditor)
		{
			CreateHitProxies();
		}
	}
	
	FPrimitiveSceneInfoAdapter(FPrimitiveSceneDesc* InPrimitiveSceneDesc)
	{
		check(InPrimitiveSceneDesc);

		Component = nullptr;
		PrimitiveComponentInterface = InPrimitiveSceneDesc->GetPrimitiveComponentInterface();
		SceneData = &InPrimitiveSceneDesc->GetSceneData();
		PrimitiveDesc = InPrimitiveSceneDesc;
		SceneProxy = InPrimitiveSceneDesc->GetSceneProxy();
		check(SceneProxy);
		ComponentId = InPrimitiveSceneDesc->GetPrimitiveSceneId();
		LODParentComponentId = InPrimitiveSceneDesc->GetLODParentId();
		LightingAttachmentComponentId = InPrimitiveSceneDesc->GetLightingAttachmentId();
		Mobility = InPrimitiveSceneDesc->GetMobility();
		
		if (GIsEditor && PrimitiveComponentInterface)
		{
			CreateHitProxies();
		}
	}
	
	FPrimitiveSceneProxy* SceneProxy;
	FPrimitiveComponentId ComponentId;
	FPrimitiveComponentId LODParentComponentId;
	FPrimitiveComponentId LightingAttachmentComponentId;
	EComponentMobility::Type Mobility;

	// mutable so that hit proxies can be moved to final destination
	mutable TArray<TRefCountPtr<HHitProxy>> HitProxies;
	HHitProxy* DefaultHitProxy = nullptr;

	FPrimitiveSceneInfoData* SceneData;
	UPrimitiveComponent* Component;
	IPrimitiveComponent* PrimitiveComponentInterface;
	FPrimitiveSceneDesc* PrimitiveDesc;
};

FPrimitiveSceneInfo::FPrimitiveSceneInfo(const FPrimitiveSceneInfoAdapter& InAdapter, FScene* InScene):
	Proxy(InAdapter.SceneProxy),
	PrimitiveComponentId(InAdapter.ComponentId),
	IndirectLightingCacheAllocation(NULL),
	CachedPlanarReflectionProxy(NULL),
	CachedReflectionCaptureProxy(NULL),
	DefaultDynamicHitProxy(NULL),
	LastRenderTime(-FLT_MAX),
	LightList(NULL),
	Scene(InScene),
	NumMobileDynamicLocalLights(0),
	GpuLodInstanceRadius(0),
	PackedIndex(INDEX_NONE),
	PersistentIndex(FPersistentPrimitiveIndex{ INDEX_NONE }),
	PrimitiveComponentInterfaceForDebuggingOnly(InAdapter.PrimitiveComponentInterface),
	SceneData(InAdapter.SceneData),	
	bNeedsUniformBufferUpdate(false),
	bIndirectLightingCacheBufferDirty(false),
	bRegisteredLightmapVirtualTextureProducerCallback(false),
	bRegisteredWithVelocityData(false),
	bNaniteRasterBinsRenderCustomDepth(false),
	bPendingAddToScene(false),
	bPendingAddStaticMeshes(false),
	bPendingFlushRuntimeVirtualTexture(false),
	bShouldRenderInMainPass(InAdapter.SceneProxy->ShouldRenderInMainPass()),
	bVisibleInRealTimeSkyCapture(InAdapter.SceneProxy->IsVisibleInRealTimeSkyCaptures()),
	bWritesRuntimeVirtualTexture(InAdapter.SceneProxy->WritesVirtualTexture()),
#if RHI_RAYTRACING
	bIsCachedRayTracingInstanceValid(false),
	bCachedRayTracingInstanceMaskAndFlagsDirty(true),
	bCachedRayTracingInstanceAnySegmentsDecal(false),
	bCachedRayTracingInstanceAllSegmentsDecal(false),
	bCachedRayTracingInstanceAllSegmentsTranslucent(false),
#endif
	// We want the unsynchronized access here, as the responsibility passes to the primitive scene info.
	InstanceSceneDataBuffersInternal(InAdapter.SceneProxy->GetInstanceSceneDataBuffers(FPrimitiveSceneProxy::EInstanceBufferAccessFlags::UnsynchronizedAndUnsafe)),
	InstanceDataUpdateTaskInfo(InAdapter.SceneProxy->GetInstanceDataUpdateTaskInfo()),
	LevelUpdateNotificationIndex(INDEX_NONE),
	InstanceSceneDataOffset(INDEX_NONE),
	NumInstanceSceneDataEntries(0),
	InstancePayloadDataOffset(INDEX_NONE),
	InstancePayloadDataStride(0),
	LightmapDataOffset(INDEX_NONE),
	NumLightmapDataEntries(0)
{
	check(PrimitiveComponentId.IsValid());
	check(Proxy);
	check(SceneData);

	LightingAttachmentRoot  = InAdapter.LightingAttachmentComponentId;	

	// Only create hit proxies in the Editor as that's where they are used.
	if (GIsEditor)
	{
		// Create a dynamic hit proxy for the primitive. 
		DefaultDynamicHitProxy = InAdapter.DefaultHitProxy;		
		HitProxies = MoveTemp(InAdapter.HitProxies);

		if( DefaultDynamicHitProxy )
		{
			check(HitProxies.Contains(DefaultDynamicHitProxy));
			DefaultDynamicHitProxyId = DefaultDynamicHitProxy->Id;
		}
	}
	
	LODParentComponentId = InAdapter.LODParentComponentId;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMemory::Memzero(CachedReflectionCaptureProxies);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (FInstanceCullingContext::IsGPUCullingEnabled())
	{
		GpuLodInstanceRadius = InAdapter.SceneProxy->GetGpuLodInstanceRadius();
	}
}

FPrimitiveSceneInfo::FPrimitiveSceneInfo(UPrimitiveComponent* InPrimitive,FScene* InScene)
	: FPrimitiveSceneInfo(FPrimitiveSceneInfoAdapter(InPrimitive), InScene)
{
}	

FPrimitiveSceneInfo::FPrimitiveSceneInfo(FPrimitiveSceneDesc* InPrimitiveSceneDesc,FScene* InScene)
	: FPrimitiveSceneInfo(FPrimitiveSceneInfoAdapter(InPrimitiveSceneDesc), InScene)
{
}

FPrimitiveSceneInfo::~FPrimitiveSceneInfo()
{
	check(!OctreeId.IsValidId());
	for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; PassIndex++)
	{
		check(StaticMeshCommandInfos.Num() == 0);
	}
}

#if RHI_RAYTRACING
bool FPrimitiveSceneInfo::IsCachedRayTracingGeometryValid(int8 LODIndex, int32 InstanceIndex) const
{
	check(LODIndex >= 0 && LODIndex < RayTracingLODData.Num());

	const FRayTracingInstanceData& InstanceData = RayTracingLODData[LODIndex].Instances[InstanceIndex];
	const FRayTracingGeometry* CachedRayTracingGeometry = InstanceData.RayTracingGeometry;

	if (CachedRayTracingGeometry)
	{
		// TODO: Doesn't take Nanite Ray Tracing into account

		check(!CachedRayTracingGeometry->GetRequiresBuild() && !CachedRayTracingGeometry->HasPendingBuildRequest());

		return CachedRayTracingGeometry->IsValid() && !CachedRayTracingGeometry->IsEvicted();
	}

	return false;
}

void FPrimitiveSceneInfo::AllocateRayTracingSBT(FRayTracingInstanceData& InstanceData, const FRHIRayTracingGeometry* CachedRayTracingInstanceGeometryRHI, uint32 CachedRayTracingInstanceSegmentCount)
{
	check(InstanceData.SBTAllocation == nullptr);

	const FRHIRayTracingGeometry* RayTracingGeometry = nullptr;
	uint32 SegmentCount = 0;

	if (InstanceData.RayTracingGeometry != nullptr)
	{
		// If there is a valid FRayTracingGeometry, retrieve the RHI object and segment count from this object (RenderThread timeline valid)
		RayTracingGeometry = InstanceData.RayTracingGeometry->GetRHI();
		SegmentCount = InstanceData.RayTracingGeometry->GetInitializer().Segments.Num();
	}
	else
	{
		// If we have a valid cached raytracing instance geometry then use this one
		RayTracingGeometry = CachedRayTracingInstanceGeometryRHI;
		SegmentCount = CachedRayTracingInstanceSegmentCount;

		// TODO: Move this check out of the branch so it also applies when CachedRayTracingInstanceGeometryRHI == nullptr
		// Need to fix UE-309710 first
		checkf(InstanceData.NumCachedMeshCommands <= (int32)SegmentCount, TEXT("Number of cached ray tracing mesh commands (%d), should be lower or equal to number of segments (%d)."), InstanceData.NumCachedMeshCommands, (int32)SegmentCount);
	}

	if (SegmentCount > 0)
	{
		InstanceData.SBTAllocation = Scene->RayTracingSBT.AllocateStaticRange(SegmentCount, RayTracingGeometry, InstanceData.CachedMeshCommandFlags);

		if (InstanceData.SBTAllocation != nullptr)
		{
			InstanceData.SBTAllocationUniqueId = InstanceData.SBTAllocation->GetUniqueId();

			if (InstanceData.SBTAllocation->HasLayer(ERayTracingShaderBindingLayer::Base))
			{
				InstanceData.InstanceContributionToHitGroupIndexBase = InstanceData.SBTAllocation->GetInstanceContributionToHitGroupIndex(ERayTracingShaderBindingLayer::Base);
			}
			if (InstanceData.SBTAllocation->HasLayer(ERayTracingShaderBindingLayer::Decals))
			{
				InstanceData.InstanceContributionToHitGroupIndexDecal = InstanceData.SBTAllocation->GetInstanceContributionToHitGroupIndex(ERayTracingShaderBindingLayer::Decals);
			}
		}
	}
}

void FPrimitiveSceneInfo::CacheRayTracingShaderBindingData(FRayTracingInstanceData& InstanceData, const FRHIRayTracingGeometry* CachedRayTracingInstanceGeometryRHI)
{
	if (!ensure(InstanceData.CachedShaderBindingDataBase.IsEmpty()))
	{
		InstanceData.CachedShaderBindingDataBase.Empty();
	}

	if (!ensure(InstanceData.CachedShaderBindingDataDecal.IsEmpty()))
	{
		InstanceData.CachedShaderBindingDataDecal.Empty();
	}

	const FRHIRayTracingGeometry* RayTracingGeometry = nullptr;
	bool bDynamicRayTracingGeometry = false;

	if (InstanceData.RayTracingGeometry != nullptr)
	{
		RayTracingGeometry = InstanceData.RayTracingGeometry->GetRHI();
		bDynamicRayTracingGeometry = InstanceData.RayTracingGeometry->GetInitializer().bAllowUpdate;
	}
	else
	{
		// this path is only used by Nanite Ray Tracing so bDynamicRayTracingGeometry = false
		RayTracingGeometry = CachedRayTracingInstanceGeometryRHI;
	}

	if (InstanceData.SBTAllocation != nullptr)
	{
		for (int32 CachedMeshCommandIndex = 0; CachedMeshCommandIndex < InstanceData.NumCachedMeshCommands; ++CachedMeshCommandIndex)
		{
			const int32 CommandIndex = InstanceData.BaseCachedMeshCommandIndex + CachedMeshCommandIndex;
			const FRayTracingMeshCommand& MeshCommand = Scene->CachedRayTracingMeshCommands[CommandIndex];
			const ERayTracingLocalShaderBindingType BindingType = MeshCommand.bCanBeCached ? ERayTracingLocalShaderBindingType::Persistent : ERayTracingLocalShaderBindingType::Transient;

			if (InstanceData.SBTAllocation->HasLayer(ERayTracingShaderBindingLayer::Base))
			{
				const bool bHidden = MeshCommand.bDecal;
				const uint32 RecordIndex = InstanceData.SBTAllocation->GetRecordIndex(ERayTracingShaderBindingLayer::Base, MeshCommand.GeometrySegmentIndex);
				InstanceData.CachedShaderBindingDataBase.Add(FRayTracingShaderBindingData(CommandIndex, RayTracingGeometry, bDynamicRayTracingGeometry, RecordIndex, BindingType, bHidden));
			}

			if (InstanceData.SBTAllocation->HasLayer(ERayTracingShaderBindingLayer::Decals))
			{
				const bool bHidden = !MeshCommand.bDecal;
				const uint32 RecordIndex = InstanceData.SBTAllocation->GetRecordIndex(ERayTracingShaderBindingLayer::Decals, MeshCommand.GeometrySegmentIndex);
				InstanceData.CachedShaderBindingDataDecal.Add(FRayTracingShaderBindingData(CommandIndex, RayTracingGeometry, bDynamicRayTracingGeometry, RecordIndex, BindingType, bHidden));
			}
		}
	}
}

void FPrimitiveSceneInfo::CacheRayTracingInstance(FRayTracingInstanceData& InstanceData, FRayTracingGeometryInstance Instance)
{
	check(InstanceData.RayTracingInstanceIndexMain == UINT32_MAX);
	check(InstanceData.RayTracingInstanceIndexDecal == UINT32_MAX);

	const RayTracing::FGeometryHandle GeometryHandle = InstanceData.RayTracingGeometry ? InstanceData.RayTracingGeometry->GetGeometryHandle() : RayTracing::FGeometryHandle();

	const bool bIsFarFieldInstance = Proxy->IsRayTracingFarField();

	const bool bNeedMainInstance = !InstanceData.CachedMeshCommandFlags.bAllSegmentsDecal;

	// if primitive has mixed decal and non-decal segments we need to have two ray tracing instances
	// one containing non-decal segments and the other with decal segments
	// masking of segments is done using "hidden" hitgroups
	// TODO: Debug Visualization to highlight primitives using this?
	const bool bNeedDecalInstance = !bIsFarFieldInstance && InstanceData.CachedMeshCommandFlags.bAnySegmentsDecal;

	if (bNeedMainInstance)
	{
		Instance.InstanceContributionToHitGroupIndex = InstanceData.InstanceContributionToHitGroupIndexBase;
		InstanceData.RayTracingInstanceIndexMain = Scene->RayTracingScene.AddCachedInstance(Instance, bIsFarFieldInstance ? ERayTracingSceneLayer::FarField : ERayTracingSceneLayer::Base, Proxy, false, GeometryHandle).AsUint32();
	}
	else
	{
		InstanceData.RayTracingInstanceIndexMain = FRayTracingScene::INVALID_INSTANCE_HANDLE.AsUint32();
	}

	if (bNeedDecalInstance)
	{
		Instance.InstanceContributionToHitGroupIndex = InstanceData.InstanceContributionToHitGroupIndexDecal;
		InstanceData.RayTracingInstanceIndexDecal = Scene->RayTracingScene.AddCachedInstance(Instance, ERayTracingSceneLayer::Decals, Proxy, false, GeometryHandle).AsUint32();
	}
	else
	{
		InstanceData.RayTracingInstanceIndexDecal = FRayTracingScene::INVALID_INSTANCE_HANDLE.AsUint32();
	}

	bIsCachedRayTracingInstanceValid = Instance.GeometryRHI != nullptr || EnumHasAllFlags(Instance.Flags, ERayTracingInstanceFlags::NaniteRayTracing);
}

const FRayTracingGeometry* FPrimitiveSceneInfo::GetStaticRayTracingGeometry(int8 LODIndex) const
{
	check(LODIndex >= 0);

	if (LODIndex < RayTracingLODData.Num())
	{
		return RayTracingLODData[LODIndex].Instances[0].RayTracingGeometry;
	}
	else
	{
		return nullptr;
	}
}

const FRayTracingGeometry* FPrimitiveSceneInfo::GetValidStaticRayTracingGeometry(int8& InOutLODIndex) const
{
	// TODO: Move HasPendingBuildRequest() / BoostBuildPriority() out of this function

	for (; InOutLODIndex < RayTracingLODData.Num(); ++InOutLODIndex)
	{
		const FRayTracingGeometry* RayTracingGeometry = RayTracingLODData[InOutLODIndex].Instances[0].RayTracingGeometry;

		if (RayTracingGeometry->HasPendingBuildRequest())
		{
			RayTracingGeometry->BoostBuildPriority();
		}
		else if (RayTracingGeometry->IsValid() && !RayTracingGeometry->IsEvicted())
		{
			return RayTracingGeometry;
		}
	}

	return nullptr;
}
#endif


void FPrimitiveSceneInfo::CacheMeshDrawCommands(FScene* Scene, TArrayView<FPrimitiveSceneInfo*> SceneInfos)
{
	SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_CacheMeshDrawCommands, FColor::Emerald);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(FPrimitiveSceneInfo_CacheMeshDrawCommands);

	QUICK_SCOPE_CYCLE_COUNTER(STAT_CacheMeshDrawCommands);

	// This reduce stuttering in editor by improving balancing of all the 
	// shadermap processing. Keep it as it is for runtime as the requirements are different.
	const int BATCH_SIZE = WITH_EDITOR ? 1 : GMeshDrawCommandsBatchSize;
	const int NumBatches = (SceneInfos.Num() + BATCH_SIZE - 1) / BATCH_SIZE;

	auto DoWorkLambda = [Scene, SceneInfos, BATCH_SIZE](FCachedPassMeshDrawListContext& DrawListContext, int32 Index)
	{
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_CacheMeshDrawCommand, FColor::Green);

		struct FMeshInfoAndIndex
		{
			int32 InfoIndex;
			int32 MeshIndex;
		};

		TArray<FMeshInfoAndIndex, SceneRenderingAllocator> MeshBatches;
		MeshBatches.Reserve(3 * BATCH_SIZE);

		int LocalNum = FMath::Min((Index * BATCH_SIZE) + BATCH_SIZE, SceneInfos.Num());
		for (int LocalIndex = (Index * BATCH_SIZE); LocalIndex < LocalNum; LocalIndex++)
		{
			FPrimitiveSceneInfo* SceneInfo = SceneInfos[LocalIndex];
			check(SceneInfo->StaticMeshCommandInfos.Num() == 0);
			SceneInfo->StaticMeshCommandInfos.AddDefaulted(EMeshPass::Num * SceneInfo->StaticMeshes.Num());
			FPrimitiveSceneProxy* SceneProxy = SceneInfo->Proxy;

			// Volumetric self shadow mesh commands need to be generated every frame, as they depend on single frame uniform buffers with self shadow data.
			if (!SceneProxy->CastsVolumetricTranslucentShadow())
			{
				for (int32 MeshIndex = 0; MeshIndex < SceneInfo->StaticMeshes.Num(); MeshIndex++)
				{
					FStaticMeshBatch& Mesh = SceneInfo->StaticMeshes[MeshIndex];
					if (SupportsCachingMeshDrawCommands(Mesh))
					{
						MeshBatches.Add(FMeshInfoAndIndex{ LocalIndex, MeshIndex });
					}
				}
			}
		}

		const int32 NumCommonPassesExpected = 4;	// to avoid reserving too much, account only for heuristically likely passes like depth / velocity / base / something
		DrawListContext.ReserveMemoryForCommands(NumCommonPassesExpected * MeshBatches.Num());

		for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; PassIndex++)
		{
			const EShadingPath ShadingPath = GetFeatureLevelShadingPath(Scene->GetFeatureLevel());
			EMeshPass::Type PassType = (EMeshPass::Type)PassIndex;

			if ((FPassProcessorManager::GetPassFlags(ShadingPath, PassType) & EMeshPassFlags::CachedMeshCommands) != EMeshPassFlags::None)
			{
				FCachedPassMeshDrawListContext::FMeshPassScope MeshPassScope(DrawListContext, PassType);

				FMeshPassProcessor* PassMeshProcessor = FPassProcessorManager::CreateMeshPassProcessor(ShadingPath, PassType, Scene->GetFeatureLevel(), Scene, nullptr, &DrawListContext);

				if (PassMeshProcessor != nullptr)
				{
					for (const FMeshInfoAndIndex& MeshAndInfo : MeshBatches)
					{
						FPrimitiveSceneInfo* SceneInfo = SceneInfos[MeshAndInfo.InfoIndex];
						FStaticMeshBatch& Mesh = SceneInfo->StaticMeshes[MeshAndInfo.MeshIndex];

						FStaticMeshBatchRelevance& MeshRelevance = SceneInfo->StaticMeshRelevances[MeshAndInfo.MeshIndex];

						check(!MeshRelevance.CommandInfosMask.Get(PassType));

#if WITH_ODSC
						FODSCPrimitiveSceneInfoScope ODSCPrimitiveSceneInfoScope(SceneInfo);
#endif

						uint64 BatchElementMask = ~0ull;
						// NOTE: AddMeshBatch calls FCachedPassMeshDrawListContext::FinalizeCommand
						PassMeshProcessor->AddMeshBatch(Mesh, BatchElementMask, SceneInfo->Proxy);

						FCachedMeshDrawCommandInfo CommandInfo = DrawListContext.GetCommandInfoAndReset();
						if (CommandInfo.CommandIndex != -1 || CommandInfo.StateBucketId != -1)
						{
							static_assert(sizeof(MeshRelevance.CommandInfosMask) * 8 >= EMeshPass::Num, "CommandInfosMask is too small to contain all mesh passes.");
							MeshRelevance.CommandInfosMask.Set(PassType);
							MeshRelevance.CommandInfosBase++;

							int CommandInfoIndex = MeshAndInfo.MeshIndex * EMeshPass::Num + PassType;
							FCachedMeshDrawCommandInfo& CurrentCommandInfo = SceneInfo->StaticMeshCommandInfos[CommandInfoIndex];
							checkf(CurrentCommandInfo.MeshPass == EMeshPass::Num,
								TEXT("SceneInfo->StaticMeshCommandInfos[%d] is not expected to be initialized yet. MeshPass is %d, but expected EMeshPass::Num (%d)."),
								CommandInfoIndex, (int32)EMeshPass::Num, CurrentCommandInfo.MeshPass);
							CurrentCommandInfo = CommandInfo;
						}
					}

					delete PassMeshProcessor;
				}
			}
		}

		for (int LocalIndex = (Index * BATCH_SIZE); LocalIndex < LocalNum; LocalIndex++)
		{
			FPrimitiveSceneInfo* SceneInfo = SceneInfos[LocalIndex];
			int PrefixSum = 0;
			for (int32 MeshIndex = 0; MeshIndex < SceneInfo->StaticMeshes.Num(); MeshIndex++)
			{
				FStaticMeshBatchRelevance& MeshRelevance = SceneInfo->StaticMeshRelevances[MeshIndex];
				if (MeshRelevance.CommandInfosBase > 0)
				{
					EMeshPass::Type PassType = EMeshPass::DepthPass;
					int NewPrefixSum = PrefixSum;
					for (;;)
					{
						PassType = MeshRelevance.CommandInfosMask.SkipEmpty(PassType);
						if (PassType == EMeshPass::Num)
						{
							break;
						}

						int CommandInfoIndex = MeshIndex * EMeshPass::Num + PassType;
						checkSlow(CommandInfoIndex >= NewPrefixSum);
						SceneInfo->StaticMeshCommandInfos[NewPrefixSum] = SceneInfo->StaticMeshCommandInfos[CommandInfoIndex];
						NewPrefixSum++;
						PassType = EMeshPass::Type(PassType + 1);
					}

#if DO_GUARD_SLOW
					int NumBits = MeshRelevance.CommandInfosMask.GetNum();
					check(PrefixSum + NumBits == NewPrefixSum);
					int LastPass = -1;
					for (int32 TestIndex = PrefixSum; TestIndex < NewPrefixSum; TestIndex++)
					{
						int MeshPass = SceneInfo->StaticMeshCommandInfos[TestIndex].MeshPass;
						check(MeshPass > LastPass);
						LastPass = MeshPass;
					}
#endif
					MeshRelevance.CommandInfosBase = PrefixSum;
					PrefixSum = NewPrefixSum;
				}
			}

			SceneInfo->StaticMeshCommandInfos.SetNum(PrefixSum, EAllowShrinking::No);
			SceneInfo->StaticMeshCommandInfos.Shrink();
		}
	};

	bool bAnyLooseParameterBuffers = false;
	if (GMeshDrawCommandsCacheMultithreaded && FApp::ShouldUseThreadingForPerformance())
	{
		TArray<FCachedPassMeshDrawListContextDeferred> DrawListContexts;
		DrawListContexts.Reserve(NumBatches);
		for(int32 ContextIndex = 0; ContextIndex < NumBatches; ++ContextIndex)
		{
			DrawListContexts.Emplace(*Scene);
		}

		ParallelForTemplate(
			NumBatches, 
			[&DrawListContexts, &DoWorkLambda](int32 Index)
			{
				FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
				DoWorkLambda(DrawListContexts[Index], Index);
			},
			EParallelForFlags::Unbalanced
		);

		if (NumBatches > 0)
		{
			SCOPED_NAMED_EVENT(DeferredFinalizeMeshDrawCommands, FColor::Emerald);

			for (int32 Index = 0; Index < NumBatches; ++Index)
			{
				FCachedPassMeshDrawListContextDeferred& DrawListContext = DrawListContexts[Index];
				const int32 Start = Index * BATCH_SIZE;
				const int32 End = FMath::Min((Index * BATCH_SIZE) + BATCH_SIZE, SceneInfos.Num());
				DrawListContext.DeferredFinalizeMeshDrawCommands(SceneInfos, Start, End);
				bAnyLooseParameterBuffers |= DrawListContext.HasAnyLooseParameterBuffers();
			}
		}
	}
	else
	{
		FCachedPassMeshDrawListContextImmediate DrawListContext(*Scene);
		for (int32 Idx = 0; Idx < NumBatches; Idx++)
		{
			DoWorkLambda(DrawListContext, Idx);
		}
		bAnyLooseParameterBuffers = DrawListContext.HasAnyLooseParameterBuffers();
	}

#if DO_GUARD_SLOW
	{
		static int32 LogCount = 0;
		if (bAnyLooseParameterBuffers && (LogCount++ % 1000) == 0)
		{
			UE_LOGF(LogRenderer, Warning, "One or more Cached Mesh Draw commands use loose parameters. This causes overhead and will break dynamic instancing, potentially reducing performance further. Use Uniform Buffers instead.");
		}
	}
#endif

	if (!FParallelMeshDrawCommandPass::IsOnDemandShaderCreationEnabled())
	{
		FGraphicsMinimalPipelineStateId::InitializePersistentIds();
	}
}

void FPrimitiveSceneInfo::RemoveCachedMeshDrawCommands()
{
	checkSlow(IsInRenderingThread());

	for (int32 CommandIndex = 0; CommandIndex < StaticMeshCommandInfos.Num(); ++CommandIndex)
	{
		const FCachedMeshDrawCommandInfo& CachedCommand = StaticMeshCommandInfos[CommandIndex];

		if (CachedCommand.StateBucketId != INDEX_NONE)
		{
			EMeshPass::Type PassIndex = CachedCommand.MeshPass;
			FGraphicsMinimalPipelineStateId CachedPipelineId;

			{
				auto& ElementKVP = Scene->CachedMeshDrawCommandStateBuckets[PassIndex].GetByElementId(CachedCommand.StateBucketId);
				CachedPipelineId = ElementKVP.Key.CachedPipelineId;

				FMeshDrawCommandCount& StateBucketCount = ElementKVP.Value;
				check(StateBucketCount.Num > 0);
				StateBucketCount.Num--;
				if (StateBucketCount.Num == 0)
				{
					Scene->CachedMeshDrawCommandStateBuckets[PassIndex].RemoveByElementId(CachedCommand.StateBucketId);
				}
			}

			FGraphicsMinimalPipelineStateId::RemovePersistentId(CachedPipelineId);
		}
		else if (CachedCommand.CommandIndex >= 0)
		{
			FCachedPassMeshDrawList& PassDrawList = Scene->CachedDrawLists[CachedCommand.MeshPass];
			FGraphicsMinimalPipelineStateId CachedPipelineId = PassDrawList.MeshDrawCommands[CachedCommand.CommandIndex].CachedPipelineId;

			PassDrawList.MeshDrawCommands.RemoveAt(CachedCommand.CommandIndex);
			FGraphicsMinimalPipelineStateId::RemovePersistentId(CachedPipelineId);

			// Track the lowest index that might be free for faster AddAtLowestFreeIndex
			PassDrawList.LowestFreeIndexSearchStart = FMath::Min(PassDrawList.LowestFreeIndexSearchStart, CachedCommand.CommandIndex);
		}

	}

	for (int32 MeshIndex = 0; MeshIndex < StaticMeshRelevances.Num(); ++MeshIndex)
	{
		FStaticMeshBatchRelevance& MeshRelevance = StaticMeshRelevances[MeshIndex];

		MeshRelevance.CommandInfosMask.Reset();
	}

	StaticMeshCommandInfos.Empty();
}

static void BuildNaniteMaterialBins(FScene* Scene, FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bLumenEnabled, FNaniteMaterialListContext& MaterialListContext);

void FPrimitiveSceneInfo::CacheNaniteMaterialBins(FScene* Scene, const TArrayView<FPrimitiveSceneInfo*>& SceneInfos)
{
	SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_CacheNaniteMaterialBins, FColor::Emerald);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(FPrimitiveSceneInfo_CacheNaniteMaterialBins);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_CacheNaniteMaterialBins);

	const bool bNaniteEnabled = DoesPlatformSupportNanite(Scene->GetShaderPlatform());
	if (bNaniteEnabled)
	{
		const bool bLumenEnabled = DoesPlatformSupportLumenGI(GetFeatureLevelShaderPlatform(Scene->GetFeatureLevel()));

		TArray<FNaniteMaterialListContext, TInlineAllocator<8>> MaterialListContexts;

		if (GNaniteMaterialBinCacheParallel && FApp::ShouldUseThreadingForPerformance())
		{
			ParallelForWithTaskContext(
				MaterialListContexts,
				SceneInfos.Num(),
				[Scene, &SceneInfos, bLumenEnabled](FNaniteMaterialListContext& Context, int32 Index)
				{
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
					BuildNaniteMaterialBins(Scene, SceneInfos[Index], bLumenEnabled, Context);
				}
			);
		}
		else
		{
			FNaniteMaterialListContext& MaterialListContext = MaterialListContexts.AddDefaulted_GetRef();
			for (FPrimitiveSceneInfo* PrimitiveSceneInfo : SceneInfos)
			{
				BuildNaniteMaterialBins(Scene, PrimitiveSceneInfo, bLumenEnabled, MaterialListContext);
			}
		}

		if (MaterialListContexts.Num() > 0)
		{
			SCOPED_NAMED_EVENT(NaniteMaterialListApply, FColor::Emerald);
			for (FNaniteMaterialListContext& Context : MaterialListContexts)
			{
				Context.Apply(*Scene);
			}
		}

		// Primitive and material relevance
		{
			SCOPED_NAMED_EVENT(NaniteComputeRelevance, FColor::Orange);
			Scene->NaniteShadingPipelines[ENaniteMeshPass::BasePass].ComputeRelevance(Scene->GetFeatureLevel());
		}

		Scene->NaniteShadingPipelines[ENaniteMeshPass::BasePass].bBuildCommands = true;
		Scene->NaniteShadingPipelines[ENaniteMeshPass::LumenCardCapture].bBuildCommands = true;
		Scene->NaniteShadingPipelines[ENaniteMeshPass::MaterialCache].bBuildCommands = true;
	}
}

void BuildNaniteMaterialBins(FScene* Scene, FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bLumenEnabled, FNaniteMaterialListContext& MaterialListContext)
{
	FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;
	if (Proxy->IsNaniteMesh())
	{
		Nanite::FSceneProxyBase* NaniteProxy = static_cast<Nanite::FSceneProxyBase*>(Proxy);
		
		// Pre-allocate the max possible material slots for the slot array here, before contexts are applied serially.
		const int32 NumMaterialSections = NaniteProxy->GetMaterialSections().Num();

		float WPODistance;
		const bool bHasWPODistance = NaniteProxy->GetInstanceWorldPositionOffsetDisableDistance(WPODistance);
		const float PixelProgrammableDistance = NaniteProxy->GetPixelProgrammableDistance();
		const float DisplacementFadeOutSize = NaniteProxy->GetMaterialDisplacementFadeOutSize();
		const bool bAnyPixelProgrammableMaterial = NaniteProxy->HasPixelProgrammableRaster();
		const bool bForceNaniteDisplacementFallback = ShouldForceNaniteDisplacementFallback();

		TArray<Nanite::FSceneProxyBase::FMaterialSection>& NaniteMaterialSections = NaniteProxy->GetMaterialSections();
		if (NaniteMaterialSections.Num() > 0)
		{
			for (int32 MeshPassIndex = 0; MeshPassIndex < ENaniteMeshPass::Num; ++MeshPassIndex)
			{
				switch (MeshPassIndex)
				{
					case ENaniteMeshPass::LumenCardCapture:
					{
						if (!LumenScene::HasPrimitiveNaniteMeshBatches(Proxy) || !bLumenEnabled)
						{
							continue;
						}
						break;
					}
					case ENaniteMeshPass::MaterialCache:
					{
						if (!Proxy->SupportsMaterialCache())
						{
							continue;
						}
						break;
					}
				}

				PrimitiveSceneInfo->NaniteMaterialSlots[MeshPassIndex].Reset(NumMaterialSections);

				FNaniteMaterialListContext::FDeferredPipelines& PipelinesCommand = MaterialListContext.DeferredPipelines[MeshPassIndex].Emplace_GetRef();
				PipelinesCommand.PrimitiveSceneInfo = PrimitiveSceneInfo;

				for (int32 MaterialSectionIndex = 0; MaterialSectionIndex < NaniteMaterialSections.Num(); ++MaterialSectionIndex)
				{
					Nanite::FSceneProxyBase::FMaterialSection& MaterialSection = NaniteMaterialSections[MaterialSectionIndex];
					check(MaterialSection.RasterMaterialProxy != nullptr);
					check(MaterialSection.ShadingMaterialProxy != nullptr);

					FNaniteMaterialListContext::FMaterialSectionData& MaterialSectionData = PipelinesCommand.MaterialSectionData.Emplace_GetRef();

					FNaniteRasterPipeline& RasterPipeline = MaterialSectionData.RasterPipeline;
					RasterPipeline.RasterMaterial = MaterialSection.RasterMaterialProxy;

					RasterPipeline.bTranslucent |= MaterialSection.bTranslucent;
					RasterPipeline.bIsTwoSided = !!MaterialSection.MaterialRelevance.bTwoSided;
					RasterPipeline.bCastShadow = MaterialSection.bCastShadow;
					
					// Spline and Skinned mesh are mutually exclusive
					RasterPipeline.bSkinnedMesh = NaniteProxy->IsSkinnedMesh();
					if (RasterPipeline.bSkinnedMesh)
					{
						RasterPipeline.bSplineMesh = false;
					}
					else
					{
						RasterPipeline.bSplineMesh = NaniteProxy->IsSplineMesh();
					}

					RasterPipeline.bWPOEnabled = MaterialSection.MaterialRelevance.bUsesWorldPositionOffset;
					RasterPipeline.bDisplacementEnabled = MaterialSection.MaterialRelevance.bUsesDisplacement;
					RasterPipeline.bPerPixelEval = MaterialSection.MaterialRelevance.bMasked || MaterialSection.MaterialRelevance.bUsesPixelDepthOffset;
					RasterPipeline.bVertexUVs = MaterialSection.MaterialRelevance.bUsesVertexInterpolator || MaterialSection.MaterialRelevance.bUsesCustomizedUVs;
					RasterPipeline.bFirstPersonLerp = MaterialSection.MaterialRelevance.bUsesFirstPersonInterpolation;

					RasterPipeline.DisplacementScaling = MaterialSection.DisplacementScaling;
					RasterPipeline.DisplacementFadeRange = MaterialSection.DisplacementFadeRange;

					RasterPipeline.bHasWPODistance = RasterPipeline.bWPOEnabled && !MaterialSection.bAlwaysEvaluateWPO && bHasWPODistance;
					RasterPipeline.bHasPixelDistance = RasterPipeline.bPerPixelEval && PixelProgrammableDistance > 0.0f;
					RasterPipeline.bHasDisplacementFadeOut = RasterPipeline.bDisplacementEnabled && DisplacementFadeOutSize > 0.0f;
					RasterPipeline.bRequiresDisplacementFallback = RasterPipeline.bDisplacementEnabled && bForceNaniteDisplacementFallback;

					// If the proxy has both pixel programmable distance and WPO disable distance, and the pixel distance is shorter,
					// then treat the pipeline as if it has no WPO distance. Fallback raster is chosen for the whole cluster, which
					// can have multiple associated raster bins. It's easier and more performant to branch over WPO than to push out
					// pixel programmable distances to fix visual issues, so we opt for the former.
					if (RasterPipeline.bHasWPODistance &&
						PixelProgrammableDistance > 0.0f &&
						PixelProgrammableDistance < WPODistance &&
						bAnyPixelProgrammableMaterial)
					{
						RasterPipeline.bHasWPODistance = false;
					}

					// Check if the raster pipeline is still compiling it's PSOs
					RasterPipeline.IsPSOPrecaching(Scene->GetFeatureLevel(), MaterialSectionData.RasterAsyncPSOCompilationEvents);

					FNaniteShadingPipeline& TriangleShadingPipeline = MaterialSectionData.TriangleShadingPipeline;
					if (RasterPipeline.bTranslucent)
					{
						check(!MaterialSection.bHasVoxels); // TODO: Not implemented
						bool bLoaded = LoadTranslucencyPassPipeline(*Scene, NaniteProxy, MaterialSection, RasterPipeline);
					}
					else
					{
						switch (MeshPassIndex)
						{
							case ENaniteMeshPass::BasePass:
							{
								bool bLoaded = LoadBasePassPipeline(*Scene, NaniteProxy, MaterialSection, false, false, TriangleShadingPipeline);
								check(bLoaded);

								FNaniteShadingPipeline& VoxelShadingPipeline = MaterialSectionData.VoxelShadingPipeline;
								if (MaterialSection.bHasVoxels)
								{
									LoadBasePassPipeline(*Scene, NaniteProxy, MaterialSection, true, false, VoxelShadingPipeline);
								}

								FNaniteShadingPipeline& CurveShadingPipeline = MaterialSectionData.CurveShadingPipeline;
								if (MaterialSection.bHasCurves)
								{
									LoadBasePassPipeline(*Scene, NaniteProxy, MaterialSection, false, true, CurveShadingPipeline);
								}
								break;
							}
							case ENaniteMeshPass::LumenCardCapture:
							{
								LoadLumenCardPipeline(*Scene, NaniteProxy, MaterialSection, TriangleShadingPipeline, MaterialSectionData.TriangleShadingAsyncPSOCompilationEvents);
								break;
							}
						}
					} // Translucent
				}
			}
		}
	}
}

void FPrimitiveSceneInfo::RemoveCachedNaniteMaterialBins()
{
	checkSlow(IsInRenderingThread());

	if (!Proxy->IsNaniteMesh())
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_RemoveCachedNaniteMaterialBins);

	for (int32 MeshPassIndex = 0; MeshPassIndex < ENaniteMeshPass::Num; ++MeshPassIndex)
	{
		FNaniteRasterPipelines& RasterPipelines = Scene->NaniteRasterPipelines[MeshPassIndex];
		FNaniteShadingPipelines& ShadingPipelines = Scene->NaniteShadingPipelines[MeshPassIndex];
		FNaniteVisibility& Visibility = Scene->NaniteVisibility[MeshPassIndex];

		TArray<FNaniteRasterBin>& NanitePassRasterBins = NaniteRasterBins[MeshPassIndex];
		for (int32 RasterBinIndex = 0; RasterBinIndex < NanitePassRasterBins.Num(); ++RasterBinIndex)
		{
			const FNaniteRasterBin& RasterBin = NanitePassRasterBins[RasterBinIndex];
			if (MeshPassIndex == ENaniteMeshPass::BasePass && bNaniteRasterBinsRenderCustomDepth)
			{
				// need to unregister these bins for custom pass first
				RasterPipelines.UnregisterBinForCustomPass(RasterBin.BinIndex);
			}
			RasterPipelines.Unregister(RasterBin);
		}

		TArray<FNaniteShadingBin>& NanitePassShadingBins = NaniteShadingBins[MeshPassIndex];
		for (int32 ShadingBinIndex = 0; ShadingBinIndex < NanitePassShadingBins.Num(); ++ShadingBinIndex)
		{
			const FNaniteShadingBin& ShadingBin = NanitePassShadingBins[ShadingBinIndex];
			ShadingPipelines.Unregister(ShadingBin);
		}

		// Need to rebuild the shading commands list
		ShadingPipelines.bBuildCommands = true;

		Visibility.RemoveReferences(this);

		NanitePassRasterBins.Reset();
		NanitePassShadingBins.Reset();
		NaniteMaterialSlots[MeshPassIndex].Reset();
	} 
	
	// Primitive doesn't care about pending static mesh updates anymore
	if (auto MaterialsExtension = Scene->GetExtensionPtr<Nanite::FMaterialsSceneExtension>())
	{
		MaterialsExtension->ClearPendingRequestStaticMeshUpdate(this);
	}

	bNaniteRasterBinsRenderCustomDepth = false;
}

#if RHI_RAYTRACING
void FScene::RefreshCachedRayTracingData()
{
	// Get rid of all existing cached commands
	for (FPrimitiveSceneInfo* SceneInfo : Primitives)
	{
		SceneInfo->RemoveCachedRayTracingPrimitives();
	}

	check(CachedRayTracingMeshCommands.IsEmpty());

	// Re-cache all current primitives
	FPrimitiveSceneInfo::CacheRayTracingPrimitives(this, Primitives);
}

static void CheckPrimitiveRayTracingData(const FScene::FPrimitiveRayTracingData& PrimitiveRayTracingData, const FPrimitiveSceneProxy* SceneProxy)
{
	check(PrimitiveRayTracingData.RayTracingGeometryGroupHandle == SceneProxy->GetRayTracingGeometryGroupHandle());
	check(PrimitiveRayTracingData.bDrawInGame == SceneProxy->IsDrawnInGame());
	check(PrimitiveRayTracingData.bRayTracingFarField == SceneProxy->IsRayTracingFarField());
	check(PrimitiveRayTracingData.bRetainWhileHidden == (SceneProxy->CastsHiddenShadow() || SceneProxy->AffectsIndirectLightingWhileHidden()));
	check(PrimitiveRayTracingData.bIsVisibleInSceneCaptures == !SceneProxy->IsHiddenInSceneCapture());
	check(PrimitiveRayTracingData.bIsVisibleInSceneCapturesOnly == SceneProxy->IsVisibleInSceneCaptureOnly());
	check(PrimitiveRayTracingData.LightingChannelMask == SceneProxy->GetLightingChannelMask());
	check(PrimitiveRayTracingData.ProxyGeometryType == SceneProxy->GetRayTracingProxyType());
}

struct FDeferredPrimitiveRayTracingInstanceData
{
	TArray<int32, TInlineAllocator<2>> CommandIndices;
	FRayTracingGeometryInstance Instance;
	uint32 SegmentCount = 0;
	bool bCacheInstance = false;
};

struct FDeferredPrimitiveRayTracingData
{
	FPrimitiveSceneInfo* SceneInfo = nullptr;
	TArray<TArray<FDeferredPrimitiveRayTracingInstanceData, TInlineAllocator<1>>, TInlineAllocator<1>> InstancesPerLOD;
};

template<class T>
class FCacheRayTracingPrimitivesContext
{
public:
	FCacheRayTracingPrimitivesContext(FScene* Scene)
		: CommandContext(Commands)
		, RayTracingMeshProcessor(&CommandContext, Scene, nullptr, Scene->CachedRayTracingMeshCommandsType)
	{ }

	FTempRayTracingMeshCommandStorage Commands;
	FCachedRayTracingMeshCommandContext<T> CommandContext;
	FRayTracingMeshProcessor RayTracingMeshProcessor;
	TArray<FDeferredPrimitiveRayTracingData> DeferredDatas;
};

template<class T>
int32 CacheRayTracingMeshBatch(
	const FMeshBatch& MeshBatch,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	FPrimitiveSceneInfo::FRayTracingInstanceData& InstanceData,
	T& Commands,
	FCachedRayTracingMeshCommandContext<T>& CommandContext,
	FRayTracingMeshProcessor& RayTracingMeshProcessor)
{
	// Why do we pass a full mask here when the dynamic case only uses a mask of 1?
	// Also note that the code below assumes only a single command was generated per batch (see SupportsCachingMeshDrawCommands(...))
	const uint64 BatchElementMask = ~0ull;
	RayTracingMeshProcessor.AddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy);

	int32 CommandIndex = CommandContext.CommandIndex;

	if (CommandContext.CommandIndex >= 0)
	{
		FRayTracingMeshCommand& RTMeshCommand = Commands[CommandContext.CommandIndex];

		RTMeshCommand.UpdateFlags(InstanceData.CachedMeshCommandFlags);

		// Update the hash
		uint64& Hash = InstanceData.CachedMeshCommandFlags.CachedMeshCommandHash;
	
		// We want the hash to change if either the shader or the binding contents change. This is used by the autoinstance feature.
		const FRHIShader* Shader = RTMeshCommand.MaterialShader;

		// TODO: It would be better to use 64 bits for both of these to reduce the chance of hash collisions
		//       but GetDynamicInstancingHash is currently a public function, so changing the return type would be an API change
		uint32 ShaderHash = Shader != nullptr ? GetTypeHash(Shader->GetHash()) : 0;
		uint32 ShaderBindingsHash = RTMeshCommand.ShaderBindings.GetDynamicInstancingHash();

		// Add MaterialShaderIndex to the hash because it's used to deduplicate SBT allocations
		// and the material shader index is stored in the user data of the SBT binding data (same shader hash can be moved to another material shader index value)

		// Add GeometrySegmentIndex to the hash as a positional encoding of the section that the material is assigned to (otherwise it's possible to get collisions in situations such as:
		// - Mesh A - 2 segments, segment 0 has 0 triangles (no mesh command generated), segment 1 uses material X
		// - Mesh B - 2 segments, segment 0 uses material X, segment 1 has 0 triangles (no mesh command generated)

		Hash = Murmur64({ Hash, (uint64(ShaderBindingsHash) << 32) | uint64(ShaderHash), (uint64(RTMeshCommand.MaterialShaderIndex) << 32) | uint64(RTMeshCommand.GeometrySegmentIndex) });

		CommandContext.CommandIndex = -1;
	}

	return CommandIndex;
}

template<class T>
void CacheRayTracingMeshCommands(
	TConstArrayView<FMeshBatch> MeshBatches,
	FPrimitiveSceneProxy* PrimitiveSceneProxy,
	int32 LODIndex,
	FPrimitiveSceneInfo::FRayTracingInstanceData& RayTracingInstanceData,
	FDeferredPrimitiveRayTracingInstanceData& DeferredData,
	T& Commands,
	FCachedRayTracingMeshCommandContext<T>& CommandContext,
	FRayTracingMeshProcessor& RayTracingMeshProcessor)
{
	ensureMsgf(MeshBatches.Num() > 0, TEXT("Caching ray tracing mesh commands but instance doesn't have any."));
		
	for (const FMeshBatch& Mesh : MeshBatches)
	{
		check(Mesh.LODIndex == LODIndex);
		const int32 CommandIndex = CacheRayTracingMeshBatch(Mesh, PrimitiveSceneProxy, RayTracingInstanceData, Commands, CommandContext, RayTracingMeshProcessor);

		if (CommandIndex >= 0)
		{
			DeferredData.CommandIndices.Add(CommandIndex);
		}
	}
}

void FPrimitiveSceneInfo::CacheRayTracingPrimitives(FScene* Scene, const TArrayView<FPrimitiveSceneInfo*>& SceneInfos)
{
	if (IsRayTracingEnabled(Scene->GetShaderPlatform()))
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(FPrimitiveSceneInfo_CacheRayTracingPrimitives)
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_CacheRayTracingPrimitives, FColor::Emerald);

		checkf(GRHISupportsMultithreadedShaderCreation, TEXT("Raytracing code needs the ability to create shaders from task threads."));
		
		FRayTracingMeshCommandStorage& CachedRayTracingMeshCommands = Scene->CachedRayTracingMeshCommands;

		{
			TArray<FCacheRayTracingPrimitivesContext<FTempRayTracingMeshCommandStorage>> Contexts;
			ParallelForWithTaskContext(
				Contexts,
				SceneInfos.Num(),
				[Scene](int32 ContextIndex, int32 NumContexts) { return Scene; },
				[Scene, &SceneInfos](FCacheRayTracingPrimitivesContext<FTempRayTracingMeshCommandStorage>& Context, int32 Index)
				{
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

					FPrimitiveSceneInfo* SceneInfo = SceneInfos[Index];
					FPrimitiveSceneProxy* SceneProxy = SceneInfo->Proxy;

					const int32 PrimitiveIndex = SceneInfo->GetIndex();

					FScene::FPrimitiveRayTracingData& PrimitiveRayTracingData = Scene->PrimitiveRayTracingDatas[PrimitiveIndex];

#if DO_CHECK
					{
						check(SceneInfo->GetRayTracingLODDataNum() == 0);

						Experimental::FHashElementId SceneRayTracingGroupId;
						const int32 RayTracingGroupId = SceneProxy->GetRayTracingGroupId();
						if (RayTracingGroupId != -1)
						{
							SceneRayTracingGroupId = Scene->PrimitiveRayTracingGroups.FindId(RayTracingGroupId);
						}

						check(Scene->PrimitiveRayTracingGroupIds[PrimitiveIndex] == SceneRayTracingGroupId);

						CheckPrimitiveRayTracingData(PrimitiveRayTracingData, SceneProxy);
					}
#endif

					ERayTracingPrimitiveFlags& Flags = Scene->PrimitiveRayTracingFlags[PrimitiveIndex];
					Flags = SceneProxy->GetRayTracingPrimitiveFlags();

					// the following flags cause ray tracing mesh command / instance caching to be disabled
					static const ERayTracingPrimitiveFlags DisableCacheFlags =
						ERayTracingPrimitiveFlags::Dynamic
						| ERayTracingPrimitiveFlags::Exclude
						| ERayTracingPrimitiveFlags::Skip
						| ERayTracingPrimitiveFlags::UnsupportedProxyType;

					if (!EnumHasAnyFlags(Flags, DisableCacheFlags))
					{
						check(SceneInfo->GetRayTracingLODDataNum() == 0);

						FStaticRayTracingInstances StaticRayTracingInstances;
						SceneProxy->GetStaticRayTracingInstances(StaticRayTracingInstances);

						const int32 NumLODs = StaticRayTracingInstances.LODs.Num();

						TArray<FRayTracingLODData> RayTracingLODDatas;
						RayTracingLODDatas.Reserve(NumLODs);

						FDeferredPrimitiveRayTracingData& DeferredData = Context.DeferredDatas.AddDefaulted_GetRef();
						DeferredData.SceneInfo = SceneInfo;
						DeferredData.InstancesPerLOD.SetNum(NumLODs);

						for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
						{
							const FStaticRayTracingInstancesLOD& StaticRayTracingInstancesLOD = StaticRayTracingInstances.LODs[LODIndex];
							const int32 NumInstances = StaticRayTracingInstancesLOD.Instances.Num();

							FRayTracingLODData& RayTracingLODData = RayTracingLODDatas.AddDefaulted_GetRef();
							RayTracingLODData.Instances.Reserve(NumInstances);

							DeferredData.InstancesPerLOD[LODIndex].Reserve(NumInstances);

							for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
							{
								const FRayTracingInstance& RayTracingInstance = StaticRayTracingInstancesLOD.Instances[InstanceIndex];

								FRayTracingInstanceData& RayTracingInstanceData = RayTracingLODData.Instances.AddDefaulted_GetRef();
								FDeferredPrimitiveRayTracingInstanceData& DeferredInstanceData = DeferredData.InstancesPerLOD[LODIndex].AddDefaulted_GetRef();

								RayTracingInstanceData.RayTracingGeometry = RayTracingInstance.Geometry;

								CacheRayTracingMeshCommands(RayTracingInstance.GetMaterials(), SceneInfo->Proxy, LODIndex, RayTracingInstanceData, DeferredInstanceData, Context.Commands, Context.CommandContext, Context.RayTracingMeshProcessor);

								if (EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::CacheInstances))
								{
									FPrimitiveSceneInfo::SetupCachedRayTracingInstance(SceneInfo, RayTracingInstance, DeferredInstanceData.Instance, DeferredInstanceData.SegmentCount);
									DeferredInstanceData.bCacheInstance = true;
								}

								if (RayTracingInstanceData.RayTracingGeometry != nullptr)
								{
									// Checks to make sure that the primitive gets the necessary update requests when the geometry is evicted or streamed out
									checkf(!RayTracingInstanceData.RayTracingGeometry->GroupHandle.IsNull(),
										TEXT("Static ray tracing instances must use a geometry assigned to the same group as returned by GetRayTracingGeometryGroupHandle()."));
									checkf(RayTracingInstanceData.RayTracingGeometry->GroupHandle == PrimitiveRayTracingData.RayTracingGeometryGroupHandle,
										TEXT("Static ray tracing instances must use a geometry assigned to the same group as returned by GetRayTracingGeometryGroupHandle()."));
								}
							}
						}

						// Store in the Scene info
						SceneInfo->SetRayTracingLODData(MoveTemp(RayTracingLODDatas));
					}
					else
					{
						checkf(!EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::CacheInstances), TEXT("Can't cache ray tracing instance when caching is disabled due to other flags."));
					}

					PrimitiveRayTracingData.bCachedRaytracingDataDirty = false;
				},
				GRayTracingPrimitiveCacheMultithreaded&& FApp::ShouldUseThreadingForPerformance() ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None
			);

			if (Contexts.Num() > 0)
			{
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(FPrimitiveSceneInfo_CacheRayTracingPrimitives_Merge)
				SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_CacheRayTracingPrimitives_Merge, FColor::Emerald);

				// copy commands generated by multiple threads to the sparse array in FScene
				// and set each mesh LOD command index
				// Also allocate the actual SBT data for each LOD
				for (const auto& Context : Contexts)
				{
					for (const FDeferredPrimitiveRayTracingData& Entry : Context.DeferredDatas)
					{
						for (int32 LODIndex = 0; LODIndex < Entry.InstancesPerLOD.Num(); ++LODIndex)
						{
							FRayTracingLODData& RayTracingLODData = Entry.SceneInfo->RayTracingLODData[LODIndex];

							for (int32 InstanceIndex = 0; InstanceIndex < Entry.InstancesPerLOD[LODIndex].Num(); ++InstanceIndex)
							{
								FRayTracingInstanceData& RayTracingInstanceData = RayTracingLODData.Instances[InstanceIndex];

								const FDeferredPrimitiveRayTracingInstanceData& DeferredInstanceData = Entry.InstancesPerLOD[LODIndex][InstanceIndex];

								const int32 NumCommands = DeferredInstanceData.CommandIndices.Num();

								const int32 StartOffset = CachedRayTracingMeshCommands.Allocate(NumCommands);

								RayTracingInstanceData.BaseCachedMeshCommandIndex = StartOffset;
								RayTracingInstanceData.NumCachedMeshCommands = NumCommands;

								// Setup the final cache mesh command indices on shared Scene CachedRayTracingMeshCommands
								for (int32 Index = 0; Index < DeferredInstanceData.CommandIndices.Num(); ++Index)
								{
									CachedRayTracingMeshCommands[StartOffset + Index] = Context.Commands[DeferredInstanceData.CommandIndices[Index]];
								}

								// Allocate SBT data now that the LOD data is fully setup
								Entry.SceneInfo->AllocateRayTracingSBT(RayTracingInstanceData, DeferredInstanceData.Instance.GeometryRHI, DeferredInstanceData.SegmentCount);

								Entry.SceneInfo->CacheRayTracingShaderBindingData(RayTracingInstanceData, DeferredInstanceData.Instance.GeometryRHI);

								if (DeferredInstanceData.bCacheInstance)
								{
									Entry.SceneInfo->CacheRayTracingInstance(RayTracingInstanceData, DeferredInstanceData.Instance);
								}
							}
						}
					}
				}
			}
		}
	}
}

void FPrimitiveSceneInfo::SetupCachedRayTracingInstance(FPrimitiveSceneInfo* SceneInfo, const FRayTracingInstance& RayTracingInstance, FRayTracingGeometryInstance& OutCachedRayTracingInstance, uint32& OutSegmentCount)
{
	checkf(RayTracingInstance.InstanceTransforms.IsEmpty() && RayTracingInstance.InstanceTransformsView.IsEmpty(),
		TEXT("Primitives with ERayTracingPrimitiveFlags::CacheInstances get instances transforms from GPUScene"));

	FPrimitiveSceneProxy* SceneProxy = SceneInfo->Proxy;

	OutCachedRayTracingInstance.NumTransforms = RayTracingInstance.NumTransforms;
	OutCachedRayTracingInstance.BaseInstanceSceneDataOffset = SceneInfo->GetInstanceSceneDataOffset();

	{
		TConstArrayView<uint32> PrimitiveInstanceIndices = RayTracingInstance.GetPrimitiveInstanceIndices();
		if (!PrimitiveInstanceIndices.IsEmpty())
		{
			checkf(RayTracingInstance.NumTransforms <= 1 && PrimitiveInstanceIndices.Num() == 1,
				TEXT("Cached static ray tracing instances currently only supports PrimitiveInstanceIndices for instances with a single transform."));
			OutCachedRayTracingInstance.BaseInstanceSceneDataOffset += PrimitiveInstanceIndices[0];
		}
	}

	if (Nanite::GetRayTracingMode() != Nanite::ERayTracingMode::Fallback && SceneProxy->IsNaniteMesh())
	{
		// nanite ray tracing geometry might not be ready yet
		// if not ready, this pointer will be patched as soon as it is

		OutCachedRayTracingInstance.GeometryRHI = Nanite::GRayTracingManager.GetRayTracingGeometry(SceneInfo);
		OutSegmentCount = Nanite::GRayTracingManager.GetRayTracingGeometrySegmentCount(SceneInfo);
	}
	else
	{
		checkf(RayTracingInstance.Geometry, TEXT("Cached ray tracing instances must have valid geometries.")); // unless using nanite ray tracing

		OutCachedRayTracingInstance.GeometryRHI = RayTracingInstance.Geometry->GetRHI();
		OutSegmentCount = RayTracingInstance.Geometry->GetInitializer().Segments.Num();
	}

	// At this point (in AddToScene()) PrimitiveIndex has been set
	check(SceneInfo->GetPersistentIndex().IsValid());
	OutCachedRayTracingInstance.DefaultUserData = SceneInfo->GetInstanceSceneDataOffset();
	OutCachedRayTracingInstance.bIncrementUserDataPerInstance = true;

	OutCachedRayTracingInstance.bApplyLocalBoundsTransform = RayTracingInstance.bApplyLocalBoundsTransform;
	OutCachedRayTracingInstance.LightingChannelMask = SceneProxy->GetLightingChannelMask();

	OutCachedRayTracingInstance.Flags = ERayTracingInstanceFlags::None;

	FRayTracingMaskAndFlags InstanceMaskAndFlags;

	// TODO: Check CachedRayTracingInstance.bInstanceMaskAndFlagsDirty?

	if (RayTracingInstance.GetMaterials().IsEmpty())
	{
		// If the material list is empty, explicitly set the mask to 0 so it will not be added in the raytracing scene
		InstanceMaskAndFlags.Mask = 0;
	}
	else
	{
		InstanceMaskAndFlags = BuildRayTracingInstanceMaskAndFlags(RayTracingInstance, *SceneProxy);
	}

	OutCachedRayTracingInstance.Mask = InstanceMaskAndFlags.Mask;

	if (InstanceMaskAndFlags.bForceOpaque)
	{
		OutCachedRayTracingInstance.Flags |= ERayTracingInstanceFlags::ForceOpaque;
	}

	if (InstanceMaskAndFlags.bDoubleSided)
	{
		OutCachedRayTracingInstance.Flags |= ERayTracingInstanceFlags::TriangleCullDisable;
	}

	if (InstanceMaskAndFlags.bReverseCulling)
	{
		OutCachedRayTracingInstance.Flags |= ERayTracingInstanceFlags::TriangleCullReverse;
	}

	if (Nanite::GetRayTracingMode() == Nanite::ERayTracingMode::CLAS && SceneProxy->IsNaniteMesh())
	{
		OutCachedRayTracingInstance.Flags |= ERayTracingInstanceFlags::NaniteRayTracing;
	}
}

void FPrimitiveSceneInfo::SetCachedRayTracingInstanceGeometryRHI(FRHIRayTracingGeometry* GeometryRHI, uint32 SegmentCount)
{
	if (RayTracingLODData.IsEmpty())
	{
		return;
	}
	checkf(RayTracingLODData.Num() == 1, TEXT("We expect to have a single LOD when using SetCachedRayTracingInstanceGeometryRHI."));

	FRayTracingLODData& LODData = RayTracingLODData[0];

	if (LODData.Instances.IsEmpty())
	{
		return;
	}

	checkf(LODData.Instances.Num() == 1, TEXT("We expect primitive to have a single ray tracing instance when using SetCachedRayTracingInstanceGeometryRHI."));

	FRayTracingInstanceData& InstanceData = LODData.Instances[0];

	// TODO: check Nanite Ray Tracing is enabled

	if (InstanceData.SBTAllocation)
	{
		Scene->RayTracingSBT.FreeStaticRange(InstanceData.SBTAllocation);
		InstanceData.SBTAllocation = nullptr;

		InstanceData.CachedShaderBindingDataBase.Empty();
		InstanceData.CachedShaderBindingDataDecal.Empty();
	}
	
	// TODO: Could update existing SBT entries with new GeometryRHI instead of free/allocate (similar to UpdateCachedInstanceGeometry(...) below)
	AllocateRayTracingSBT(InstanceData, GeometryRHI, SegmentCount);
	CacheRayTracingShaderBindingData(InstanceData, GeometryRHI);

	if (InstanceData.RayTracingInstanceIndexMain != UINT32_MAX)
	{
		Scene->RayTracingScene.UpdateCachedInstanceGeometry(InstanceData.RayTracingInstanceIndexMain, GeometryRHI, InstanceData.InstanceContributionToHitGroupIndexBase);
	}

	if (InstanceData.RayTracingInstanceIndexDecal != UINT32_MAX)
	{
		Scene->RayTracingScene.UpdateCachedInstanceGeometry(InstanceData.RayTracingInstanceIndexDecal, GeometryRHI, InstanceData.InstanceContributionToHitGroupIndexBase);
	}

	bIsCachedRayTracingInstanceValid = GeometryRHI != nullptr || (Nanite::GetRayTracingMode() == Nanite::ERayTracingMode::CLAS && Proxy->IsNaniteMesh());
}

void FPrimitiveSceneInfo::RemoveCachedRayTracingPrimitives()
{
	if (IsRayTracingAllowed())
	{
		for (auto& LODData : RayTracingLODData)
		{
			for (auto& InstanceData : LODData.Instances)
			{
				Scene->RayTracingScene.FreeCachedInstance(InstanceData.RayTracingInstanceIndexMain);
				Scene->RayTracingScene.FreeCachedInstance(InstanceData.RayTracingInstanceIndexDecal);
				InstanceData.RayTracingInstanceIndexMain = UINT32_MAX;
				InstanceData.RayTracingInstanceIndexDecal = UINT32_MAX;

				InstanceData.CachedShaderBindingDataBase.Empty();
				InstanceData.CachedShaderBindingDataDecal.Empty();

				Scene->CachedRayTracingMeshCommands.Free(InstanceData.BaseCachedMeshCommandIndex, InstanceData.NumCachedMeshCommands);
				Scene->RayTracingSBT.FreeStaticRange(InstanceData.SBTAllocation);
			}

			LODData.Instances.Empty();
		}

		RayTracingLODData.Empty();
	}
	else
	{
		check(RayTracingLODData.IsEmpty());
	}
}
#endif

static bool GetRuntimeVirtualTextureLODRange(TArray<class FStaticMeshBatchRelevance> const& MeshRelevances, int8& OutMinLOD, int8& OutMaxLOD)
{
	OutMinLOD = MAX_int8;
	OutMaxLOD = 0;

	for (int32 MeshIndex = 0; MeshIndex < MeshRelevances.Num(); ++MeshIndex)
	{
		const FStaticMeshBatchRelevance& MeshRelevance = MeshRelevances[MeshIndex];
		if (MeshRelevance.bRenderToVirtualTexture)
		{
			OutMinLOD = FMath::Min(OutMinLOD, MeshRelevance.GetLODIndex());
			OutMaxLOD = FMath::Max(OutMaxLOD, MeshRelevance.GetLODIndex());
		}
	}

	return OutMinLOD <= OutMaxLOD;
}

static FPrimitiveRuntimeVirtualTextureLodInfo BuildRuntimeVirtualTextureLodInfo(FPrimitiveSceneInfo const& InPrimitveSceneInfo)
{
	FPrimitiveRuntimeVirtualTextureLodInfo LodInfo;

	if (InPrimitveSceneInfo.bWritesRuntimeVirtualTexture)
	{
		int8 MinLod, MaxLod;
		if (GetRuntimeVirtualTextureLODRange(InPrimitveSceneInfo.StaticMeshRelevances, MinLod, MaxLod))
		{
			FPrimitiveSceneProxy* Proxy = InPrimitveSceneInfo.Proxy;

			LodInfo.MinLod = FMath::Clamp((int32)MinLod, 0, 15);
			LodInfo.MaxLod = FMath::Clamp((int32)MaxLod, 0, 15);
			LodInfo.LodBias = FMath::Clamp(Proxy->GetVirtualTextureLodBias() + FPrimitiveRuntimeVirtualTextureLodInfo::LodBiasOffset, 0, 15);
			LodInfo.CullMethod = Proxy->GetVirtualTextureMinCoverage() == 0 ? 0 : 1;
			LodInfo.CullValue = LodInfo.CullMethod == 0 ? Proxy->GetVirtualTextureCullMips() : Proxy->GetVirtualTextureMinCoverage();
		}
	}

	return LodInfo;
}

void FPrimitiveSceneInfo::AddStaticMeshes(FRHICommandListBase& RHICmdList, FScene* Scene, TArrayView<FPrimitiveSceneInfo*> SceneInfos, bool bCacheMeshDrawCommands)
{
	LLM_SCOPE(ELLMTag::StaticMesh);

	{
		ParallelForTemplate(SceneInfos.Num(), [Scene, &SceneInfos](int32 Index)
		{
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
			SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddStaticMeshes_DrawStaticElements, FColor::Magenta);
			FPrimitiveSceneInfo* SceneInfo = SceneInfos[Index];
			// Cache the primitive's static mesh elements.
			FBatchingSPDI BatchingSPDI(SceneInfo);
			BatchingSPDI.SetHitProxy(SceneInfo->DefaultDynamicHitProxy);
			SceneInfo->Proxy->DrawStaticElements(&BatchingSPDI);
			SceneInfo->StaticMeshes.Shrink();
			SceneInfo->StaticMeshRelevances.Shrink();
			SceneInfo->RuntimeVirtualTextureLodInfo = BuildRuntimeVirtualTextureLodInfo(*SceneInfo);
			SceneInfo->bPendingAddStaticMeshes = false;

			check(SceneInfo->StaticMeshRelevances.Num() == SceneInfo->StaticMeshes.Num());
		});
	}

	{
		const ERHIFeatureLevel::Type FeatureLevel = Scene->GetFeatureLevel();

		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddStaticMeshes_UpdateSceneArrays, FColor::Blue);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			// Allocate OIT index buffer where needed
			const bool bAllocateSortedTriangles = OIT::IsSortedTrianglesEnabled(GMaxRHIShaderPlatform) && SceneInfo->Proxy->SupportsSortedTriangles();

			for (int32 MeshIndex = 0; MeshIndex < SceneInfo->StaticMeshes.Num(); MeshIndex++)
			{
				FStaticMeshBatchRelevance& MeshRelevance = SceneInfo->StaticMeshRelevances[MeshIndex];
				FStaticMeshBatch& Mesh = SceneInfo->StaticMeshes[MeshIndex];

				// Add the static mesh to the scene's static mesh list.
				FSparseArrayAllocationInfo SceneArrayAllocation = Scene->StaticMeshes.AddUninitialized();
				Scene->StaticMeshes[SceneArrayAllocation.Index] = &Mesh;
				Mesh.Id = SceneArrayAllocation.Index;
				MeshRelevance.Id = SceneArrayAllocation.Index;

				if (bAllocateSortedTriangles && OIT::IsCompatible(Mesh, FeatureLevel))
				{
					Scene->OITSceneData.Allocate(RHICmdList, EPrimitiveType(Mesh.Type), Mesh.Elements[0], Mesh.Elements[0].DynamicIndexBuffer);
				}
			}
		}
	}

	if (bCacheMeshDrawCommands)
	{
		CacheMeshDrawCommands(Scene, SceneInfos);
		CacheNaniteMaterialBins(Scene, SceneInfos);
	#if RHI_RAYTRACING
		CacheRayTracingPrimitives(Scene, SceneInfos);
	#endif
	}
}

static void OnLightmapVirtualTextureDestroyed(const FVirtualTextureProducerHandle& InHandle, void* Baton)
{
	FPrimitiveSceneInfo* PrimitiveSceneInfo = static_cast<FPrimitiveSceneInfo*>(Baton);

	// Update the main uniform buffer
	PrimitiveSceneInfo->UpdateStaticLightingBuffer();

	// Also need to update lightmap data inside GPUScene, if that's enabled
	PrimitiveSceneInfo->Scene->GPUScene.AddPrimitiveToUpdate(PrimitiveSceneInfo->GetPersistentIndex(), EPrimitiveDirtyState::ChangedStaticLighting);
}

int32 FPrimitiveSceneInfo::UpdateStaticLightingBuffer()
{
	checkSlow(IsInRenderingThread());

	if (bRegisteredLightmapVirtualTextureProducerCallback)
	{
		// Remove any previous VT callbacks
		FVirtualTextureSystem::Get().RemoveAllProducerDestroyedCallbacks(this);
		bRegisteredLightmapVirtualTextureProducerCallback = false;
	}

	FPrimitiveSceneProxy::FLCIArray LCIs;
	Proxy->GetLCIs(LCIs);
	for (int32 i = 0; i < LCIs.Num(); ++i)
	{
		FLightCacheInterface* LCI = LCIs[i];

		if (LCI)
		{
			LCI->CreatePrecomputedLightingUniformBuffer_RenderingThread(Scene->GetFeatureLevel());

			// If lightmap is using virtual texture, need to set a callback to update our uniform buffers if VT is destroyed,
			// since we cache VT parameters inside these uniform buffers
			FVirtualTextureProducerHandle VTProducerHandle;
			if (LCI->GetVirtualTextureLightmapProducer(Scene->GetFeatureLevel(), VTProducerHandle))
			{
				FVirtualTextureSystem::Get().AddProducerDestroyedCallback(VTProducerHandle, &OnLightmapVirtualTextureDestroyed, this);
				bRegisteredLightmapVirtualTextureProducerCallback = true;
			}
		}
	}

	return LCIs.Num();
}

void FPrimitiveSceneInfo::AllocateGPUSceneInstances(FScene* Scene, const TArrayView<FPrimitiveSceneInfo*>& SceneInfos)
{
	if (Scene->GPUScene.IsEnabled())
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateGPUSceneTime);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			check
			(
				SceneInfo->InstanceSceneDataOffset == INDEX_NONE &&
				SceneInfo->NumInstanceSceneDataEntries == 0 &&
				SceneInfo->InstancePayloadDataOffset == INDEX_NONE &&
				SceneInfo->InstancePayloadDataStride == 0
			);

			// Note: this will return 1 instance for primitives without the instance data buffer.
			FInstanceDataBufferHeader InstanceDataHeader = SceneInfo->GetInstanceDataHeader();
			SceneInfo->NumInstanceSceneDataEntries = InstanceDataHeader.NumInstances;
				if (SceneInfo->NumInstanceSceneDataEntries > 0)
				{
					SceneInfo->InstanceSceneDataOffset = Scene->GPUScene.AllocateInstanceSceneDataSlots(SceneInfo->GetPersistentIndex(), SceneInfo->NumInstanceSceneDataEntries);
					SceneInfo->InstancePayloadDataStride = InstanceDataHeader.PayloadDataStride;
					if (SceneInfo->InstancePayloadDataStride > 0)
					{
						const uint32 TotalFloat4Count = SceneInfo->NumInstanceSceneDataEntries * SceneInfo->InstancePayloadDataStride;
						SceneInfo->InstancePayloadDataOffset = Scene->GPUScene.AllocateInstancePayloadDataSlots(TotalFloat4Count);
					}
				}
				
			// Force a primitive update in the GPU scene, 
			// NOTE: does not set Added as this is handled elsewhere.
			Scene->GPUScene.AddPrimitiveToUpdate(SceneInfo->GetPersistentIndex(), EPrimitiveDirtyState::ChangedAll);

			// Force a primitive update in the Lumen scene(s)
			for (FLumenSceneDataIterator LumenSceneData = Scene->GetLumenSceneDataIterator(); LumenSceneData; ++LumenSceneData)
			{
				LumenSceneData->UpdatePrimitiveInstanceOffset(SceneInfo->PackedIndex);
			}
		}

		OnGPUSceneInstancesAllocated.Broadcast();
	}
}

void FPrimitiveSceneInfo::FreeGPUSceneInstances()
{
	if (!Scene->GPUScene.IsEnabled())
	{
		return;
	}

	// Release all instance data slots associated with this primitive.
	if (InstanceSceneDataOffset != INDEX_NONE)
	{
		SCOPE_CYCLE_COUNTER(STAT_UpdateGPUSceneTime);

		check(Proxy->SupportsInstanceDataBuffer() || NumInstanceSceneDataEntries == 1);

		// Release all instance payload data slots associated with this primitive.
		if (InstancePayloadDataOffset != INDEX_NONE)
		{
			check(InstancePayloadDataStride > 0);

			const uint32 TotalFloat4Count = NumInstanceSceneDataEntries * InstancePayloadDataStride;
			Scene->GPUScene.FreeInstancePayloadDataSlots(InstancePayloadDataOffset, TotalFloat4Count);
			InstancePayloadDataOffset = INDEX_NONE;
			InstancePayloadDataStride = 0;
		}

		Scene->GPUScene.FreeInstanceSceneDataSlots(InstanceSceneDataOffset, NumInstanceSceneDataEntries);
		InstanceSceneDataOffset = INDEX_NONE;
		NumInstanceSceneDataEntries = 0;

		OnGPUSceneInstancesFreed.Broadcast();
	}
}

void FPrimitiveSceneInfo::UpdateOcclusionFlags()
{
	if (IsIndexValid())
	{
		uint8 OcclusionFlags = EOcclusionFlags::None;
		// First person primitives potentially deform the geometry outside of its bounds in a view dependent way. They are very unlikely to be occluded anyways,
		// so to avoid falsely culling them, it is better to simply don't occlusion cull them at all.
		if (Proxy->CanBeOccluded() && !Proxy->IsFirstPerson())
		{
			OcclusionFlags |= EOcclusionFlags::CanBeOccluded;
		}
		if (Proxy->HasSubprimitiveOcclusionQueries())
		{
			OcclusionFlags |= EOcclusionFlags::HasSubprimitiveQueries;
		}
		if (Proxy->AllowApproximateOcclusion()
			// Allow approximate occlusion if attached, even if the parent does not have bLightAttachmentsAsGroup enabled
			|| LightingAttachmentRoot.IsValid())
		{
			OcclusionFlags |= EOcclusionFlags::AllowApproximateOcclusion;
		}
		if (Proxy->GetVisibilityId() >= 0)
		{
			OcclusionFlags |= EOcclusionFlags::HasPrecomputedVisibility;
		}
		if (Proxy->IsForceHidden())
		{
			OcclusionFlags |= EOcclusionFlags::IsForceHidden;
		}

		Scene->PrimitiveOcclusionFlags[PackedIndex] = OcclusionFlags;
	}	
}

void FPrimitiveSceneInfo::AddToScene(FScene* Scene, TArrayView<FPrimitiveSceneInfo*> SceneInfos)
{
	check(IsInRenderingThread());
	SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddToScene, FColor::Turquoise);

	{
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddToScene_IndirectLightingCacheUniformBuffer, FColor::Turquoise);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			FPrimitiveSceneProxy* Proxy = SceneInfo->Proxy;
			// Create an indirect lighting cache uniform buffer if we attaching a primitive that may require it, as it may be stored inside a cached mesh command.
			if (IsIndirectLightingCacheAllowed(Scene->GetFeatureLevel())
				&& Proxy->WillEverBeLit()
				&& ((Proxy->HasStaticLighting() && Proxy->NeedsUnbuiltPreviewLighting()) || (Proxy->IsMovable() && Proxy->GetIndirectLightingCacheQuality() != ILCQ_Off) || Proxy->GetLightmapType() == ELightmapType::ForceVolumetric))
			{
				if (!SceneInfo->IndirectLightingCacheUniformBuffer)
				{
					FIndirectLightingCacheUniformParameters Parameters;

					GetIndirectLightingCacheParameters(
						Scene->GetFeatureLevel(),
						Parameters,
						nullptr,
						nullptr,
						FVector(0.0f, 0.0f, 0.0f),
						0,
						nullptr);

					SceneInfo->IndirectLightingCacheUniformBuffer = TUniformBufferRef<FIndirectLightingCacheUniformParameters>::CreateUniformBufferImmediate(Parameters, UniformBuffer_MultiFrame, EUniformBufferValidation::None);
				}
			}

			SceneInfo->bPendingAddToScene = false;
		}
	}

	{
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddToScene_IndirectLightingCacheAllocation, FColor::Orange);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			FPrimitiveSceneProxy* Proxy = SceneInfo->Proxy;
			// If we are attaching a primitive that should be statically lit but has unbuilt lighting,
			// Allocate space in the indirect lighting cache so that it can be used for previewing indirect lighting
			if (Proxy->HasStaticLighting()
				&& Proxy->NeedsUnbuiltPreviewLighting()
				&& IsIndirectLightingCacheAllowed(Scene->GetFeatureLevel()))
			{
				FIndirectLightingCacheAllocation* PrimitiveAllocation = Scene->IndirectLightingCache.FindPrimitiveAllocation(SceneInfo->PrimitiveComponentId);

				if (PrimitiveAllocation)
				{
					SceneInfo->IndirectLightingCacheAllocation = PrimitiveAllocation;
					PrimitiveAllocation->SetDirty();
				}
				else
				{
					PrimitiveAllocation = Scene->IndirectLightingCache.AllocatePrimitive(SceneInfo, true);
					PrimitiveAllocation->SetDirty();
					SceneInfo->IndirectLightingCacheAllocation = PrimitiveAllocation;
				}
			}
			SceneInfo->MarkIndirectLightingCacheBufferDirty();
		}
	}

	{
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddToScene_LightmapDataOffset, FColor::Green);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			const bool bAllowStaticLighting = IsStaticLightingAllowed();
			if (bAllowStaticLighting)
			{
				SceneInfo->NumLightmapDataEntries = SceneInfo->UpdateStaticLightingBuffer();
				if (SceneInfo->NumLightmapDataEntries > 0 && UseGPUScene(Scene->GetShaderPlatform()))
				{
					SceneInfo->LightmapDataOffset = Scene->GPUScene.LightmapDataAllocator.Allocate(SceneInfo->NumLightmapDataEntries);
				}
			}
		}
	}


	if (Scene->ShouldCacheReflectionCaptures())
	{
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddToScene_ReflectionCaptures, FColor::Yellow);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			// Cache the nearest reflection proxy if needed
			if (SceneInfo->NeedsReflectionCaptureUpdate())
			{
				SceneInfo->CacheReflectionCaptures();
			}
		}
	}

	{
		const bool bSkipNaniteInOctree = ShouldSkipNaniteLPIs(Scene->GetShaderPlatform());
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddToScene_AddToPrimitiveOctree, FColor::Red);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			// doing this check after updating PrimitiveFlagsCompact (next loop) would be more efficient.
			if (!bSkipNaniteInOctree || !SceneInfo->Proxy->IsNaniteMesh())
			{
				// create potential storage for our compact info
				FPrimitiveSceneInfoCompact CompactPrimitiveSceneInfo(SceneInfo);

				// Add the primitive to the octree.
				check(!SceneInfo->OctreeId.IsValidId());
				Scene->PrimitiveOctree.AddElement(CompactPrimitiveSceneInfo);
				check(SceneInfo->OctreeId.IsValidId());
			}
		}
	}

	{
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddToScene_UpdateBounds, FColor::Cyan);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			FPrimitiveSceneProxy* Proxy = SceneInfo->Proxy;
			int32 PackedIndex = SceneInfo->PackedIndex;

			if (Proxy->CastsDynamicIndirectShadow())
			{
				Scene->DynamicIndirectCasterPrimitives.Add(SceneInfo);
			}

			Scene->PrimitiveSceneProxies[PackedIndex] = Proxy;
			Scene->PrimitiveTransforms[PackedIndex] = Proxy->GetLocalToWorld();

			// Set bounds.
			FPrimitiveBounds& PrimitiveBounds = Scene->PrimitiveBounds[PackedIndex];
			FBoxSphereBounds BoxSphereBounds = Proxy->GetBounds();
			PrimitiveBounds.BoxSphereBounds = BoxSphereBounds;
			PrimitiveBounds.MinDrawDistance = Proxy->GetMinDrawDistance();
			PrimitiveBounds.MaxDrawDistance = Proxy->GetMaxDrawDistance();
			PrimitiveBounds.MaxCullDistance = PrimitiveBounds.MaxDrawDistance;

			Scene->PrimitiveFlagsCompact[PackedIndex] = FPrimitiveFlagsCompact(Proxy);

			// Store precomputed visibility ID.
			int32 VisibilityBitIndex = Proxy->GetVisibilityId();
			FPrimitiveVisibilityId& VisibilityId = Scene->PrimitiveVisibilityIds[PackedIndex];
			VisibilityId.ByteIndex = VisibilityBitIndex / 8;
			VisibilityId.BitMask = (1 << (VisibilityBitIndex & 0x7));

			// Store occlusion flags.
			SceneInfo->UpdateOcclusionFlags();

			// Store occlusion bounds.
			FBoxSphereBounds OcclusionBounds = BoxSphereBounds;
			if (Proxy->HasCustomOcclusionBounds())
			{
				OcclusionBounds = Proxy->GetCustomOcclusionBounds();
			}
			OcclusionBounds.BoxExtent.X = OcclusionBounds.BoxExtent.X + OCCLUSION_SLOP;
			OcclusionBounds.BoxExtent.Y = OcclusionBounds.BoxExtent.Y + OCCLUSION_SLOP;
			OcclusionBounds.BoxExtent.Z = OcclusionBounds.BoxExtent.Z + OCCLUSION_SLOP;
			OcclusionBounds.SphereRadius = OcclusionBounds.SphereRadius + OCCLUSION_SLOP;
			Scene->PrimitiveOcclusionBounds[PackedIndex] = OcclusionBounds;

			// Store the component.
			Scene->PrimitiveComponentIds[PackedIndex] = SceneInfo->PrimitiveComponentId;
			Scene->PrimitiveComponentIdToInfoMap.Add(SceneInfo->PrimitiveComponentId, SceneInfo);

#if RHI_RAYTRACING
			// Set group id
			const int32 RayTracingGroupId = SceneInfo->Proxy->GetRayTracingGroupId();
			if (RayTracingGroupId != -1)
			{
				Scene->PrimitiveRayTracingGroupIds[PackedIndex] = Scene->PrimitiveRayTracingGroups.FindId(RayTracingGroupId);
			}
#endif

			INC_MEMORY_STAT_BY(STAT_PrimitiveInfoMemory, sizeof(*SceneInfo) + SceneInfo->StaticMeshes.GetAllocatedSize() + SceneInfo->StaticMeshRelevances.GetAllocatedSize() + Proxy->GetMemoryFootprint());
		}
	}

	{
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddToScene_LevelNotifyPrimitives, FColor::Blue);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			if (SceneInfo->Proxy->ShouldNotifyOnWorldAddRemove())
			{
				TArray<FPrimitiveSceneInfo*>& LevelNotifyPrimitives = Scene->PrimitivesNeedingLevelUpdateNotification.FindOrAdd(SceneInfo->Proxy->GetLevelName());
				SceneInfo->LevelUpdateNotificationIndex = LevelNotifyPrimitives.Num();
				LevelNotifyPrimitives.Add(SceneInfo);
			}
		}
	}

#if UE_WITH_PSO_PRECACHING
	{
		SCOPED_NAMED_EVENT(FPrimitiveSceneInfo_AddToScene_BoostAlwaysVisibleProxyPSOs, FColor::Silver);
		for (FPrimitiveSceneInfo* SceneInfo : SceneInfos)
		{
			// If proxy is always visible then boost right now, because won't go over relevance path and won't be boosted otherwise
			if (SceneInfo->Proxy->IsAlwaysVisible())
			{
				SceneInfo->Proxy->BoostPrecachedPSORequestsOnDraw();
			}
		}
	}
#endif // UE_WITH_PSO_PRECACHING
}

void FPrimitiveSceneInfo::RemoveStaticMeshes()
{
	// Deallocate potential OIT dynamic index buffer
	if (OIT::IsSortedTrianglesEnabled(GMaxRHIShaderPlatform))
	{
		for (int32 MeshIndex = 0; MeshIndex < StaticMeshes.Num(); MeshIndex++)
		{
			FStaticMeshBatch& Mesh = StaticMeshes[MeshIndex];
			if (Mesh.Elements.Num() > 0)
			{
				Scene->OITSceneData.Deallocate(Mesh.Elements[0]);
			}
		}
	}

	// Remove static meshes from the scene.
	StaticMeshes.Empty();
	StaticMeshRelevances.Empty();
	RemoveCachedMeshDrawCommands();
	RemoveCachedNaniteMaterialBins();
#if RHI_RAYTRACING
	RemoveCachedRayTracingPrimitives();
#endif
}

void FPrimitiveSceneInfo::RemoveFromScene(bool bUpdateStaticDrawLists)
{
	check(IsInRenderingThread());

	// implicit linked list. The destruction will update this "head" pointer to the next item in the list.
	while (LightList)
	{
		FLightPrimitiveInteraction::Destroy(LightList);
	}

	// Remove the primitive from the octree.
	if (OctreeId.IsValidId())
	{
		check(Scene->PrimitiveOctree.GetElementById(OctreeId).PrimitiveSceneInfo == this);
		Scene->PrimitiveOctree.RemoveElement(OctreeId);
	}

	OctreeId = FOctreeElementId2();

	if (LightmapDataOffset != INDEX_NONE && UseGPUScene(Scene->GetShaderPlatform()))
	{
		Scene->GPUScene.LightmapDataAllocator.Free(LightmapDataOffset, NumLightmapDataEntries);
	}

	if (Proxy->CastsDynamicIndirectShadow())
	{
		Scene->DynamicIndirectCasterPrimitives.RemoveSingleSwap(this);
	}

	IndirectLightingCacheAllocation = NULL;

	if (Proxy->IsOftenMoving())
	{
		MarkIndirectLightingCacheBufferDirty();
	}

	DEC_MEMORY_STAT_BY(STAT_PrimitiveInfoMemory, sizeof(*this) + StaticMeshes.GetAllocatedSize() + StaticMeshRelevances.GetAllocatedSize() + Proxy->GetMemoryFootprint());

	if (bUpdateStaticDrawLists)
	{
		if (IsIndexValid()) // PackedIndex
		{
			Scene->PrimitivesNeedingStaticMeshUpdate[PackedIndex] = false;
		}

		// IndirectLightingCacheUniformBuffer may be cached inside cached mesh draw commands, so we 
		// can't delete it unless we also update cached mesh command.
		IndirectLightingCacheUniformBuffer.SafeRelease();

		RemoveStaticMeshes();
	}

	if (bRegisteredLightmapVirtualTextureProducerCallback)
	{
		FVirtualTextureSystem::Get().RemoveAllProducerDestroyedCallbacks(this);
		bRegisteredLightmapVirtualTextureProducerCallback = false;
	}

	if (Proxy->ShouldNotifyOnWorldAddRemove())
	{
		TArray<FPrimitiveSceneInfo*>* LevelNotifyPrimitives = Scene->PrimitivesNeedingLevelUpdateNotification.Find(Proxy->GetLevelName());
		if (LevelNotifyPrimitives != nullptr)
		{
			checkSlow(LevelUpdateNotificationIndex != INDEX_NONE);
			LevelNotifyPrimitives->RemoveAtSwap(LevelUpdateNotificationIndex, EAllowShrinking::No);
			if (LevelNotifyPrimitives->Num() == 0)
			{
				Scene->PrimitivesNeedingLevelUpdateNotification.Remove(Proxy->GetLevelName());
			}
			else if (LevelUpdateNotificationIndex < LevelNotifyPrimitives->Num())
			{
				// Update swapped element's LevelUpdateNotificationIndex
				((*LevelNotifyPrimitives)[LevelUpdateNotificationIndex])->LevelUpdateNotificationIndex = LevelUpdateNotificationIndex;
			}
		}
	}
}

void FPrimitiveSceneInfo::UpdateStaticMeshes(FScene* Scene, TArrayView<FPrimitiveSceneInfo*> SceneInfos, EUpdateStaticMeshFlags UpdateFlags, bool bReAddToDrawLists)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FPrimitiveSceneInfo_UpdateStaticMeshes);
	TRACE_CPUPROFILER_EVENT_SCOPE(FPrimitiveSceneInfo_UpdateStaticMeshes);

	const bool bUpdateRayTracingCommands = EnumHasAnyFlags(UpdateFlags, EUpdateStaticMeshFlags::RayTracingCommands) || !IsRayTracingEnabled();
	const bool bUpdateAllCommands = EnumHasAnyFlags(UpdateFlags, EUpdateStaticMeshFlags::RasterCommands) && bUpdateRayTracingCommands;

	const bool bNeedsStaticMeshUpdate = !(bReAddToDrawLists && bUpdateAllCommands);

	for (int32 Index = 0; Index < SceneInfos.Num(); Index++)
	{
		FPrimitiveSceneInfo* SceneInfo = SceneInfos[Index];
		Scene->PrimitivesNeedingStaticMeshUpdate[SceneInfo->PackedIndex] = bNeedsStaticMeshUpdate;

		if (EnumHasAnyFlags(UpdateFlags, EUpdateStaticMeshFlags::RasterCommands))
		{
			SceneInfo->RemoveCachedMeshDrawCommands();
			SceneInfo->RemoveCachedNaniteMaterialBins();
		}

	#if RHI_RAYTRACING
		if (EnumHasAnyFlags(UpdateFlags, EUpdateStaticMeshFlags::RayTracingCommands))
		{
			SceneInfo->RemoveCachedRayTracingPrimitives();
		}
	#endif

		if (SceneInfo->Proxy && SceneInfo->Proxy->IsNaniteMesh())
		{
			// Make sure material table indirections are kept in sync with GPU Scene and cached Nanite MDCs
			SceneInfo->RequestGPUSceneUpdate(EPrimitiveDirtyState::ChangedOther);
		}
	}

	if (bReAddToDrawLists)
	{
		if (EnumHasAnyFlags(UpdateFlags, EUpdateStaticMeshFlags::RasterCommands))
		{
			CacheMeshDrawCommands(Scene, SceneInfos);
			CacheNaniteMaterialBins(Scene, SceneInfos);
		}

	#if RHI_RAYTRACING
		if (EnumHasAnyFlags(UpdateFlags, EUpdateStaticMeshFlags::RayTracingCommands))
		{
			CacheRayTracingPrimitives(Scene, SceneInfos);
		}
	#endif
	}
}

#if RHI_RAYTRACING
void FPrimitiveSceneInfo::UpdateCachedRaytracingData(FScene* Scene, const TArrayView<FPrimitiveSceneInfo*>& SceneInfos)
{
	if (SceneInfos.Num() > 0)
	{
		for (int32 Index = 0; Index < SceneInfos.Num(); Index++)
		{
			FPrimitiveSceneInfo* SceneInfo = SceneInfos[Index]; 
			// should have been marked dirty by calling UpdateCachedRayTracingState on the scene before
			// scene info is being updated here
			check(Scene->PrimitiveRayTracingDatas[SceneInfo->GetIndex()].bCachedRaytracingDataDirty);
		}

		CacheRayTracingPrimitives(Scene, SceneInfos);
	}
}
#endif //RHI_RAYTRACING

void FPrimitiveSceneInfo::RequestStaticMeshUpdate()
{
	// Set a flag which causes InitViews to update the static meshes the next time the primitive is visible.
	if (IsIndexValid()) // PackedIndex
	{
		Scene->PrimitivesNeedingStaticMeshUpdate[PackedIndex] = true;
	}
}

bool FPrimitiveSceneInfo::RequestUniformBufferUpdate()
{
	if (IsIndexValid()) // PackedIndex
	{
		Scene->PrimitivesNeedingUniformBufferUpdate[PackedIndex] = true;
		return true;
	}
	return false;
}

const FInstanceSceneDataBuffers* FPrimitiveSceneInfo::GetInstanceSceneDataBuffers() const
{ 
	if (!HasInstanceDataBuffers())
	{
		return nullptr;
	}

	if (InstanceDataUpdateTaskInfo)
	{
		InstanceDataUpdateTaskInfo->WaitForUpdateCompletion();
	}

	return InstanceSceneDataBuffersInternal;
}

FInstanceDataBufferHeader FPrimitiveSceneInfo::GetInstanceDataHeader() const
{
	if (!HasInstanceDataBuffers())
	{
		return FInstanceDataBufferHeader::SinglePrimitiveHeader;
	}

	if (InstanceDataUpdateTaskInfo)
	{
		return InstanceDataUpdateTaskInfo->GetHeader();
	}

	return InstanceSceneDataBuffersInternal->GetHeader();
}

void FPrimitiveSceneInfo::FlushRuntimeVirtualTexture()
{
	if (!Proxy)
	{
		return;
	}
	
	if (bWritesRuntimeVirtualTexture)
	{
		for (TSparseArray<FRuntimeVirtualTextureSceneProxy*>::TIterator It(Scene->RuntimeVirtualTextures); It; ++It)
		{
			if (Proxy->GetRuntimeVirtualTextureIds().Contains((*It)->RuntimeVirtualTextureId))
			{
				(*It)->Dirty(Proxy->GetBounds(), EVTInvalidatePriority::Normal);
			}
		}
	}

	if (Proxy->SupportsMaterialCache())
	{
		for (FMaterialCacheVirtualTextureRenderProxy* MaterialCacheProxy : Proxy->MaterialCacheRenderProxies)
		{
			MaterialCacheProxy->Flush(Scene);
		}
	}
}

void FPrimitiveSceneInfo::LinkLODParentComponent()
{
	if (LODParentComponentId.IsValid())
	{
		Scene->SceneLODHierarchy.AddChildNode(LODParentComponentId, this);
	}
}

void FPrimitiveSceneInfo::UnlinkLODParentComponent()
{
	if(LODParentComponentId.IsValid())
	{
		Scene->SceneLODHierarchy.RemoveChildNode(LODParentComponentId, this);
	}
}

void FPrimitiveSceneInfo::LinkAttachmentGroup()
{
	// Add the primitive to its attachment group.
	if (LightingAttachmentRoot.IsValid())
	{
		FAttachmentGroupSceneInfo* AttachmentGroup = Scene->AttachmentGroups.Find(LightingAttachmentRoot);

		if (!AttachmentGroup)
		{
			// If this is the first primitive attached that uses this attachment parent, create a new attachment group.
			AttachmentGroup = &Scene->AttachmentGroups.Add(LightingAttachmentRoot, FAttachmentGroupSceneInfo());
		}

		AttachmentGroup->Primitives.Add(this);
	}
	else if (Proxy->LightAttachmentsAsGroup())
	{
		FAttachmentGroupSceneInfo* AttachmentGroup = Scene->AttachmentGroups.Find(PrimitiveComponentId);

		if (!AttachmentGroup)
		{
			// Create an empty attachment group 
			AttachmentGroup = &Scene->AttachmentGroups.Add(PrimitiveComponentId, FAttachmentGroupSceneInfo());
		}

		AttachmentGroup->ParentSceneInfo = this;
	}
}

void FPrimitiveSceneInfo::UnlinkAttachmentGroup()
{
	// Remove the primitive from its attachment group.
	if (LightingAttachmentRoot.IsValid())
	{
		FAttachmentGroupSceneInfo& AttachmentGroup = Scene->AttachmentGroups.FindChecked(LightingAttachmentRoot);
		AttachmentGroup.Primitives.RemoveSwap(this);

		if (AttachmentGroup.Primitives.Num() == 0 && AttachmentGroup.ParentSceneInfo == nullptr)
		{
			// If this was the last primitive attached that uses this attachment group and the root has left the building, free the group.
			Scene->AttachmentGroups.Remove(LightingAttachmentRoot);
		}
	}
	else if (Proxy->LightAttachmentsAsGroup())
	{
		FAttachmentGroupSceneInfo* AttachmentGroup = Scene->AttachmentGroups.Find(PrimitiveComponentId);
		
		if (AttachmentGroup)
		{
			AttachmentGroup->ParentSceneInfo = NULL;
			if (AttachmentGroup->Primitives.Num() == 0)
			{
				// If this was the owner and the group is empty, remove it (otherwise the above will remove when the last attached goes).
				Scene->AttachmentGroups.Remove(PrimitiveComponentId);
			}
		}
	}
}

bool FPrimitiveSceneInfo::RequestGPUSceneUpdate(EPrimitiveDirtyState PrimitiveDirtyState)
{
	if (Scene && IsIndexValid())
	{
		Scene->GPUScene.AddPrimitiveToUpdate(GetPersistentIndex(), PrimitiveDirtyState);
		return true;
	}

	return false;
}

void FPrimitiveSceneInfo::RefreshNaniteRasterBins()
{
	const bool bShouldRenderCustomDepth = Proxy->ShouldRenderCustomDepth();
	if (bShouldRenderCustomDepth == bNaniteRasterBinsRenderCustomDepth)
	{
		// nothing to do
		return;
	}

	TArray<FNaniteRasterBin>& NanitePassRasterBins = NaniteRasterBins[ENaniteMeshPass::BasePass];
	FNaniteRasterPipelines& RasterPipelines = Scene->NaniteRasterPipelines[ENaniteMeshPass::BasePass];
	for (const FNaniteRasterBin& RasterBin : NanitePassRasterBins)
	{
		if (bShouldRenderCustomDepth)
		{
			RasterPipelines.RegisterBinForCustomPass(RasterBin.BinIndex);
		}
		else
		{
			RasterPipelines.UnregisterBinForCustomPass(RasterBin.BinIndex);
		}
	}

	bNaniteRasterBinsRenderCustomDepth = bShouldRenderCustomDepth;
}

void FPrimitiveSceneInfo::GatherLightingAttachmentGroupPrimitives(TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>& OutChildSceneInfos)
{
#if ENABLE_NAN_DIAGNOSTIC
	// local function that returns full name of object
	auto GetObjectName = [](const UObject* InPrimitive)->FString
	{
		return (InPrimitive) ? InPrimitive->GetFullName() : FString("Unknown Object");
	};

	// verify that the current object has a valid bbox before adding it
	const float& BoundsRadius = this->Proxy->GetBounds().SphereRadius;
	if (ensureMsgf(!FMath::IsNaN(BoundsRadius) && FMath::IsFinite(BoundsRadius),
		TEXT("%s had an ill-formed bbox and was skipped during shadow setup, contact DavidH."), *GetObjectName(this->PrimitiveComponentInterfaceForDebuggingOnly->GetUObject())))
	{
		OutChildSceneInfos.Add(this);
	}
	else
	{
		// return, leaving the TArray empty
		return;
	}

#else 
	// add self at the head of this queue
	OutChildSceneInfos.Add(this);
#endif

	if (!LightingAttachmentRoot.IsValid() && Proxy->LightAttachmentsAsGroup())
	{
		const FAttachmentGroupSceneInfo* AttachmentGroup = Scene->AttachmentGroups.Find(PrimitiveComponentId);

		if (AttachmentGroup)
		{
			
			for (int32 ChildIndex = 0, ChildIndexMax = AttachmentGroup->Primitives.Num(); ChildIndex < ChildIndexMax; ChildIndex++)
			{
				FPrimitiveSceneInfo* ShadowChild = AttachmentGroup->Primitives[ChildIndex];
#if ENABLE_NAN_DIAGNOSTIC
				// Only enqueue objects with valid bounds using the normality of the SphereRaduis as criteria.

				const float& ShadowChildBoundsRadius = ShadowChild->Proxy->GetBounds().SphereRadius;

				if (ensureMsgf(!FMath::IsNaN(ShadowChildBoundsRadius) && FMath::IsFinite(ShadowChildBoundsRadius),
					TEXT("%s had an ill-formed bbox and was skipped during shadow setup, contact DavidH."), *GetObjectName(ShadowChild->PrimitiveComponentInterfaceForDebuggingOnly->GetUObject())))
				{
					checkSlow(!OutChildSceneInfos.Contains(ShadowChild))
				    OutChildSceneInfos.Add(ShadowChild);
				}
#else
				// enqueue all objects.
				checkSlow(!OutChildSceneInfos.Contains(ShadowChild))
			    OutChildSceneInfos.Add(ShadowChild);
#endif
			}
		}
	}
}

void FPrimitiveSceneInfo::GatherLightingAttachmentGroupPrimitives(TArray<const FPrimitiveSceneInfo*, SceneRenderingAllocator>& OutChildSceneInfos) const
{
	OutChildSceneInfos.Add(this);

	if (!LightingAttachmentRoot.IsValid() && Proxy->LightAttachmentsAsGroup())
	{
		const FAttachmentGroupSceneInfo* AttachmentGroup = Scene->AttachmentGroups.Find(PrimitiveComponentId);

		if (AttachmentGroup)
		{
			for (int32 ChildIndex = 0, ChildIndexMax = AttachmentGroup->Primitives.Num(); ChildIndex < ChildIndexMax; ChildIndex++)
			{
				const FPrimitiveSceneInfo* ShadowChild = AttachmentGroup->Primitives[ChildIndex];

				checkSlow(!OutChildSceneInfos.Contains(ShadowChild))
			    OutChildSceneInfos.Add(ShadowChild);
			}
		}
	}
}

FBoxSphereBounds FPrimitiveSceneInfo::GetAttachmentGroupBounds() const
{
	FBoxSphereBounds Bounds = Proxy->GetBounds();

	if (!LightingAttachmentRoot.IsValid() && Proxy->LightAttachmentsAsGroup())
	{
		const FAttachmentGroupSceneInfo* AttachmentGroup = Scene->AttachmentGroups.Find(PrimitiveComponentId);

		if (AttachmentGroup)
		{
			for (int32 ChildIndex = 0; ChildIndex < AttachmentGroup->Primitives.Num(); ChildIndex++)
			{
				FPrimitiveSceneInfo* AttachmentChild = AttachmentGroup->Primitives[ChildIndex];
				Bounds = Bounds + AttachmentChild->Proxy->GetBounds();
			}
		}
	}

	return Bounds;
}

uint32 FPrimitiveSceneInfo::GetMemoryFootprint()
{
	return( sizeof( *this ) + HitProxies.GetAllocatedSize() + StaticMeshes.GetAllocatedSize() + StaticMeshRelevances.GetAllocatedSize() );
}

void FPrimitiveSceneInfo::ApplyWorldOffset(FRHICommandListBase& RHICmdList, FVector InOffset)
{
	Proxy->ApplyWorldOffset(RHICmdList, InOffset);
}

void FPrimitiveSceneInfo::UpdateIndirectLightingCacheBuffer(
	FRHICommandListBase& RHICmdList,
	const FIndirectLightingCache* LightingCache,
	const FIndirectLightingCacheAllocation* LightingAllocation,
	FVector VolumetricLightmapLookupPosition,
	uint32 SceneFrameNumber,
	FVolumetricLightmapSceneData* VolumetricLightmapSceneData)
{
	FIndirectLightingCacheUniformParameters Parameters;

	GetIndirectLightingCacheParameters(
		Scene->GetFeatureLevel(),
		Parameters,
		LightingCache,
		LightingAllocation,
		VolumetricLightmapLookupPosition,
		SceneFrameNumber,
		VolumetricLightmapSceneData);

	if (IndirectLightingCacheUniformBuffer)
	{
		IndirectLightingCacheUniformBuffer.UpdateUniformBufferImmediate(RHICmdList, Parameters);
	}
}

void FPrimitiveSceneInfo::UpdateIndirectLightingCacheBuffer(FRHICommandListBase& RHICmdList)
{
	if (bIndirectLightingCacheBufferDirty)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateIndirectLightingCacheBuffer);

		if (Scene->GetFeatureLevel() < ERHIFeatureLevel::SM5
			&& Scene->VolumetricLightmapSceneData.HasData()
			&& (Proxy->IsMovable() || Proxy->NeedsUnbuiltPreviewLighting() || Proxy->GetLightmapType() == ELightmapType::ForceVolumetric)
			&& Proxy->WillEverBeLit())
		{
			UpdateIndirectLightingCacheBuffer(
				RHICmdList,
				nullptr, 
				nullptr,
				Proxy->GetBounds().Origin,
				Scene->GetFrameNumber(),
				&Scene->VolumetricLightmapSceneData);
		}
		// The update is invalid if the lighting cache allocation was not in a functional state.
		else if (IndirectLightingCacheAllocation && (Scene->IndirectLightingCache.IsInitialized() && IndirectLightingCacheAllocation->bHasEverUpdatedSingleSample))
		{
			UpdateIndirectLightingCacheBuffer(
				RHICmdList,
				&Scene->IndirectLightingCache,
				IndirectLightingCacheAllocation,
				FVector(0, 0, 0),
				0,
				nullptr);
		}
		else
		{
			// Fallback to the global empty buffer parameters
			UpdateIndirectLightingCacheBuffer(RHICmdList, nullptr, nullptr, FVector(0.0f, 0.0f, 0.0f), 0, nullptr);
		}

		bIndirectLightingCacheBufferDirty = false;
	}
}

void FPrimitiveSceneInfo::GetStaticMeshesLODRange(int8& OutMinLOD, int8& OutMaxLOD) const
{
	OutMinLOD = MAX_int8;
	OutMaxLOD = 0;

	for (int32 MeshIndex = 0; MeshIndex < StaticMeshRelevances.Num(); ++MeshIndex)
	{
		const FStaticMeshBatchRelevance& MeshRelevance = StaticMeshRelevances[MeshIndex];
		OutMinLOD = FMath::Min(OutMinLOD, MeshRelevance.GetLODIndex());
		OutMaxLOD = FMath::Max(OutMaxLOD, MeshRelevance.GetLODIndex());
	}
}

const FMeshBatch* FPrimitiveSceneInfo::GetMeshBatch(int8 InLODIndex) const
{
	if (StaticMeshes.IsValidIndex(InLODIndex))
	{
		return &StaticMeshes[InLODIndex];
	}

	return nullptr;
}

bool FPrimitiveSceneInfo::NeedsReflectionCaptureUpdate() const
{
	// Cached reflection captures are only consumed on mobile and desktop forward.
	return bNeedsCachedReflectionCaptureUpdate && Scene->ShouldCacheReflectionCaptures();
}

bool FPrimitiveSceneInfo::CacheReflectionCaptures()
{
	// do not use Scene->PrimitiveBounds here, as it may be not initialized yet
	FBoxSphereBounds BoxSphereBounds = Proxy->GetBounds(); 

	const FPlanarReflectionSceneProxy* PreviousPlanarReflectionProxy = CachedPlanarReflectionProxy;
	const FReflectionCaptureProxy* PreviousReflectionCaptureProxy = CachedReflectionCaptureProxy;

	CachedReflectionCaptureProxy = Scene->FindClosestReflectionCapture(BoxSphereBounds.Origin);
	CachedPlanarReflectionProxy = Scene->FindClosestPlanarReflection(BoxSphereBounds);
	
	bNeedsCachedReflectionCaptureUpdate = false;

	return PreviousPlanarReflectionProxy != CachedPlanarReflectionProxy || PreviousReflectionCaptureProxy != CachedReflectionCaptureProxy;
}

void FPrimitiveSceneInfo::RemoveCachedReflectionCapture(const FReflectionCaptureProxy* RemovedProxy)
{
	if (CachedReflectionCaptureProxy && (RemovedProxy == CachedReflectionCaptureProxy || RemovedProxy == nullptr))
	{
		CachedReflectionCaptureProxy = nullptr;
		bNeedsCachedReflectionCaptureUpdate = true;
	}
}

void FPrimitiveSceneInfo::RemoveCachedPlanarReflection(const FPlanarReflectionSceneProxy* RemovedProxy)
{
	if (CachedPlanarReflectionProxy && (RemovedProxy == CachedPlanarReflectionProxy || RemovedProxy == nullptr))
	{
		CachedPlanarReflectionProxy = nullptr;
		bNeedsCachedReflectionCaptureUpdate = true;
	}
}

void FPrimitiveSceneInfo::UpdateComponentLastRenderTime(float CurrentWorldTime, bool bUpdateLastRenderTimeOnScreen)
{
	SceneData->SetLastRenderTime(CurrentWorldTime, bUpdateLastRenderTimeOnScreen);

	// Cheap inline gate: UpdateLastRenderTime is ENGINE_API (cross-DLL call from the Renderer
	// module), so without this the common SSAM-disabled path pays a function call per visible
	// primitive per view per frame just to early-return inside the callee.
	if (bUpdateLastRenderTimeOnScreen
		&& FSimpleStreamableAssetManager::IsEnabled()
		&& FSimpleStreamableAssetManager::ShouldConsiderVisibility())
	{
		FSimpleStreamableAssetManager::UpdateLastRenderTime(Proxy, CurrentWorldTime);
	}

#if UE_WITH_PSO_PRECACHING
	Proxy->BoostPrecachedPSORequestsOnDraw();
#endif
}

void FPrimitiveOctreeSemantics::SetOctreeNodeIndex(const FPrimitiveSceneInfoCompact& Element, FOctreeElementId2 Id)
{
	// When a Primitive is removed from the renderer, it's index will be invalidated.  Only update if the primitive still
	// has a valid index.
	if (Element.PrimitiveSceneInfo->IsIndexValid())
	{
		Element.PrimitiveSceneInfo->Scene->PrimitiveOctreeIndex[Element.PrimitiveSceneInfo->GetIndex()] = Id.GetNodeIndex();
	}
}

FString FPrimitiveSceneInfo::GetFullnameForDebuggingOnly() const
{
	if (PrimitiveComponentInterfaceForDebuggingOnly)
	{
		return PrimitiveComponentInterfaceForDebuggingOnly->GetUObject()->GetFullGroupName(false);
	}

	return FString("Unknown Object");
}

FString FPrimitiveSceneInfo::GetOwnerActorNameOrLabelForDebuggingOnly() const
{
	if (PrimitiveComponentInterfaceForDebuggingOnly)
	{
		return PrimitiveComponentInterfaceForDebuggingOnly->GetOwnerName();
	}
	
	return FString("Unknown Object");
}

UPrimitiveComponent* FPrimitiveSceneInfo::GetComponentForDebugOnly() const
{
	if (PrimitiveComponentInterfaceForDebuggingOnly)
	{
		return Cast<UPrimitiveComponent>(PrimitiveComponentInterfaceForDebuggingOnly->GetUObject());
	}

	return nullptr;
}

IPrimitiveComponent* FPrimitiveSceneInfo::GetComponentInterfaceForDebugOnly() const 
{
	return PrimitiveComponentInterfaceForDebuggingOnly; 
}
