// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Quantization.h: common quantization functions used by renderer.
=============================================================================*/

#include "Quantization.h"

FVector3f ComputePixelFormatQuantizationError(EPixelFormat PixelFormat)
{
	FIntVector ColorMantissaBits = FIntVector(1, 1, 1);
	switch (PixelFormat)
	{
	case PF_FloatR11G11B10:
		ColorMantissaBits = FIntVector(6, 6, 5);
		break;

	case PF_FloatRGBA:
		ColorMantissaBits = FIntVector(10, 10, 10);
		break;

	case PF_R5G6B5_UNORM:
		ColorMantissaBits = FIntVector(5, 6, 5);
		break;

	case PF_B8G8R8A8:
	case PF_R8G8B8A8:
		ColorMantissaBits = FIntVector(8, 8, 8);
		break;

	case PF_A2B10G10R10:
		ColorMantissaBits = FIntVector(10, 10, 10);
		break;

	case PF_A16B16G16R16:
		ColorMantissaBits = FIntVector(16, 16, 16);
		break;


	case PF_A32B32G32R32F:
	default:
		// A few view UBs are created witch SceneTexturesConfig not fully setup, so we gracefully fallback to a default 32bits.
		ColorMantissaBits = FIntVector(23, 23, 23);
	}

	FVector3f Error;
	Error.X = FMath::Pow(0.5f, ColorMantissaBits.X);
	Error.Y = FMath::Pow(0.5f, ColorMantissaBits.Y);
	Error.Z = FMath::Pow(0.5f, ColorMantissaBits.Z);
	return Error;
}
