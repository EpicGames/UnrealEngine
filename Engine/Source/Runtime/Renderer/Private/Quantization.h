// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Quantization.h: common quantization functions used by renderer.
=============================================================================*/

#pragma once

#include "Math/Vector.h"
#include "PixelFormat.h"

FVector3f ComputePixelFormatQuantizationError(EPixelFormat PixelFormat);
