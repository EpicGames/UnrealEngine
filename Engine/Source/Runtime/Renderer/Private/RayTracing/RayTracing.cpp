// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracing.h"

#if RHI_RAYTRACING

#include <type_traits>
#include "CoreMinimal.h"
#include "Async/Fundamental/Scheduler.h"
#include "Async/Fundamental/Task.h"
#include "Async/ParallelFor.h"
#include "Containers/ArrayView.h"
#include "Containers/ChunkedArray.h"
#include "Containers/SetUtilities.h"
#include "Containers/SparseSet.h"
#include "ContentStreaming.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DeferredShadingRendererTypes.h"
#include "DistanceFieldLightingShared.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Experimental/Containers/SherwoodHashTable.h"
#include "GameTime.h"
#include "GlobalRenderResources.h"
#include "GPUScene.h"
#include "HAL/IConsoleManager.h"
#include "HitProxies.h"
#include "Logging/LogMacros.h"
#include "Lumen/LumenHardwareRayTracingCommon.h"
#include "Materials/MaterialRenderProxy.h"
#include "Math/Color.h"
#include "MeshBatch.h"
#include "MeshPassProcessor.h"
#include "Misc/App.h"
#include "Misc/LargeWorldCoordinates.h"
#include "MultiGPU.h"
#include "Nanite/NaniteRayTracing.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RayTracing/RayTracingInstanceMask.h"
#include "RayTracing/RayTracingMaterialHitShaders.h"
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracing/RayTracingShadows.h"
#include "RayTracingDynamicGeometryUpdateManager.h"
#include "RayTracingGeometry.h"
#include "RayTracingGeometryInstance.h"
#include "RayTracingGeometryManagerInterface.h"
#include "RayTracingGeometryManagerTypes.h"
#include "RayTracingInstance.h"
#include "RayTracingMeshDrawCommands.h"
#include "RenderCommandTag.h"
#include "RenderCore.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "Rendering/NaniteCoarseMeshStreamingManager.h"
#include "Rendering/NaniteInterface.h"
#include "Rendering/RayTracingGeometryManager.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIContext.h"
#include "RHIPipeline.h"
#include "RHIResources.h"
#include "RHIShaderBindingLayout.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "SceneRendering.h"
#include "SceneView.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "StereoRendering.h"
#include "Tasks/Pipe.h"
#include "Tasks/Task.h"
#include "SceneViewState.h"
#include "ScenePrivate.h"

bool GRayTracingDynamicInstances = true;
static FAutoConsoleVariableRef CVarRayTracingDynamicInstances(
	TEXT("r.RayTracing.DynamicInstances"),
	GRayTracingDynamicInstances,
	TEXT("Where to include dynamic instances in the ray tracing scene."),
	ECVF_RenderThreadSafe
);

static int32 GParallelGatherDynamicRayTracingInstancesNumTasks = 4;
static FAutoConsoleVariableRef CVarParallelGatherDynamicRayTracingInstancesNumTasks(
	TEXT("r.RayTracing.DynamicInstances.Parallel.NumTasks"),
	GParallelGatherDynamicRayTracingInstancesNumTasks,
	TEXT("Controls the number of gather dynamic ray tracing instances tasks to run asynchronously."),
	ECVF_RenderThreadSafe
);

inline uint32 GetNumDynamicRayTracingInstancesContexts()
{
	if (!IsParallelGatherDynamicRayTracingInstancesEnabled())
	{
		return 1;
	}

	return FMath::Clamp<int32>(GParallelGatherDynamicRayTracingInstancesNumTasks, 1, LowLevelTasks::FScheduler::Get().GetNumWorkers());
}

static int32 GRayTracingParallelMeshBatchSetup = 1;
static FAutoConsoleVariableRef CRayTracingParallelMeshBatchSetup(
	TEXT("r.RayTracing.ParallelMeshBatchSetup"),
	GRayTracingParallelMeshBatchSetup,
	TEXT("Whether to setup ray tracing materials via parallel jobs."),
	ECVF_RenderThreadSafe);

static int32 GRayTracingParallelMeshBatchSize = 1024;
static FAutoConsoleVariableRef CRayTracingParallelMeshBatchSize(
	TEXT("r.RayTracing.ParallelMeshBatchSize"),
	GRayTracingParallelMeshBatchSize,
	TEXT("Batch size for ray tracing materials parallel jobs."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRayTracingDynamicGeometryLastRenderTimeUpdateDistance(
	TEXT("r.RayTracing.DynamicGeometryLastRenderTimeUpdateDistance"),
	5000.0f,
	TEXT("Dynamic geometries within this distance will have their LastRenderTime updated, so that visibility based ticking (like skeletal mesh) can work when the component is not directly visible in the view (but reflected)."));

static TAutoConsoleVariable<int32> CVarRayTracingAutoInstance(
	TEXT("r.RayTracing.AutoInstance"),
	1,
	TEXT("Whether to auto instance static meshes\n"),
	ECVF_RenderThreadSafe
);

static int32 GRayTracingExcludeTranslucent = 0;
static FAutoConsoleVariableRef CRayTracingExcludeTranslucent(
	TEXT("r.RayTracing.ExcludeTranslucent"),
	GRayTracingExcludeTranslucent,
	TEXT("A toggle that modifies the inclusion of translucent objects in the ray tracing scene. This setting is ignored for the Path Tracer.\n")
	TEXT(" 0: Translucent objects included in the ray tracing scene (default)\n")
	TEXT(" 1: Translucent objects excluded from the ray tracing scene"),
	ECVF_RenderThreadSafe);

static int32 GRayTracingExcludeSky = 1;
static FAutoConsoleVariableRef CRayTracingExcludeSky(
	TEXT("r.RayTracing.ExcludeSky"),
	GRayTracingExcludeSky,
	TEXT("A toggle that controls inclusion of sky geometry in the ray tracing scene (excluding sky can make ray tracing faster). This setting is ignored for the Path Tracer.\n")
	TEXT(" 0: Sky objects included in the ray tracing scene\n")
	TEXT(" 1: Sky objects excluded from the ray tracing scene (default)"),
	ECVF_RenderThreadSafe);

static int32 GRayTracingExcludeDecals = 0;
static FAutoConsoleVariableRef CRayTracingExcludeDecals(
	TEXT("r.RayTracing.ExcludeDecals"),
	GRayTracingExcludeDecals,
	TEXT("A toggle that modifies the inclusion of decals in the ray tracing BVH.\n")
	TEXT(" 0: Decals included in the ray tracing BVH (default)\n")
	TEXT(" 1: Decals excluded from the ray tracing BVH"),
	ECVF_RenderThreadSafe);

static int32 GRayTracingDebugDisableTriangleCull = 0;
static FAutoConsoleVariableRef CVarRayTracingDebugDisableTriangleCull(
	TEXT("r.RayTracing.DebugDisableTriangleCull"),
	GRayTracingDebugDisableTriangleCull,
	TEXT("Forces all ray tracing geometry instances to be double-sided by disabling back-face culling. This is useful for debugging and profiling. (default = 0)")
);

static int32 GRayTracingDebugForceOpaque = 0;
static FAutoConsoleVariableRef CVarRayTracingDebugForceOpaque(
	TEXT("r.RayTracing.DebugForceOpaque"),
	GRayTracingDebugForceOpaque,
	TEXT("Forces all ray tracing geometry instances to be opaque, effectively disabling any-hit shaders. This is useful for debugging and profiling. (default = 0)")
);

static int32 GRayTracingParallelPrimitiveGather = 1;
static FAutoConsoleVariableRef CVarRayTracingParallelPrimitiveGather(
	TEXT("r.RayTracing.ParallelPrimitiveGather"),
	GRayTracingParallelPrimitiveGather,
	TEXT("Whether to gather primitives relevant to ray tracing using parallel loops. 0=disabled, 1=enabled (default)"),
	ECVF_RenderThreadSafe);

static bool bUpdateCachedRayTracingState = false;

static FAutoConsoleCommand UpdateCachedRayTracingStateCmd(
	TEXT("r.RayTracing.UpdateCachedState"),
	TEXT("Update cached ray tracing state (mesh commands and instances)."),
	FConsoleCommandDelegate::CreateStatic([] { bUpdateCachedRayTracingState = true; }));

static ERayTracingProxyType GActiveRayTracingProxyTypes = ERayTracingProxyType::All;

static_assert(sizeof(FScene::FPrimitiveRayTracingData) == 8, "FScene::FPrimitiveRayTracingData is packed to 8 bytes to be cache efficient during GatherRelevantPrimitives");

static void RefreshRayTracingInstancesSinkFunction()
{
	static const auto RayTracingStaticMeshesCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.StaticMeshes"));
	static const auto RayTracingHISMCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.HierarchicalInstancedStaticMesh"));
	static const auto RayTracingLandscapeGrassCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.LandscapeGrass"));
	static const auto RayTracingNaniteProxiesCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.NaniteProxies"));

	static const auto RayTracingSkeletalMeshesCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.SkeletalMeshes"));
	static const auto RayTracingISKMCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.Geometry.InstancedSkeletalMeshes"));

	static int32 CachedRayTracingStaticMeshes = RayTracingStaticMeshesCVar->GetValueOnGameThread();
	static int32 CachedRayTracingHISM = RayTracingHISMCVar->GetValueOnGameThread();
	static int32 CachedRayTracingNaniteProxies = RayTracingNaniteProxiesCVar->GetValueOnGameThread();
	static int32 CachedRayTracingSkeletalMeshes = RayTracingSkeletalMeshesCVar->GetValueOnGameThread();	
	static int32 CachedRayTracingISKM = RayTracingISKMCVar->GetValueOnGameThread();
	static int32 CachedRayTracingLandscapeGrass = RayTracingLandscapeGrassCVar->GetValueOnGameThread();

	const int32 RayTracingStaticMeshes = RayTracingStaticMeshesCVar->GetValueOnGameThread();
	const int32 RayTracingHISM = RayTracingHISMCVar->GetValueOnGameThread();
	const int32 RayTracingNaniteProxies = RayTracingNaniteProxiesCVar->GetValueOnGameThread();
	const int32 RayTracingSkeletalMeshes = RayTracingSkeletalMeshesCVar->GetValueOnGameThread();
	const int32 RayTracingISKM = RayTracingISKMCVar->GetValueOnGameThread();
	const int32 RayTracingLandscapeGrass = RayTracingLandscapeGrassCVar->GetValueOnGameThread();

	if (RayTracingStaticMeshes != CachedRayTracingStaticMeshes
		|| RayTracingHISM != CachedRayTracingHISM
		|| RayTracingNaniteProxies != CachedRayTracingNaniteProxies
		|| RayTracingSkeletalMeshes != CachedRayTracingSkeletalMeshes
		|| RayTracingISKM != CachedRayTracingISKM
		|| RayTracingLandscapeGrass != CachedRayTracingLandscapeGrass)
	{		
		bool bRequireUpdateCachedRayTracingState = false;
		ERayTracingProxyType NewActiveRayTracingProxyTypes = ERayTracingProxyType::None;
		auto CheckValue = [&bRequireUpdateCachedRayTracingState, &NewActiveRayTracingProxyTypes](int32& CachedValue, int32 NewValue, ERayTracingProxyType ProxyType)
		{
			if (NewValue == 0 && CachedValue > 0)
			{
				bRequireUpdateCachedRayTracingState = true;
			}
			else if (NewValue == 1)
			{
				EnumAddFlags(NewActiveRayTracingProxyTypes, ProxyType);
				if (CachedValue == 0)
				{
					bRequireUpdateCachedRayTracingState = true;
				}
			}
			CachedValue = NewValue;
		};

		CheckValue(CachedRayTracingStaticMeshes, RayTracingStaticMeshes, ERayTracingProxyType::StaticMesh);
		CheckValue(CachedRayTracingHISM, RayTracingHISM, ERayTracingProxyType::HierarchicalInstancedStaticMesh);
		CheckValue(CachedRayTracingNaniteProxies, RayTracingNaniteProxies, ERayTracingProxyType::NaniteProxy);
		CheckValue(CachedRayTracingSkeletalMeshes, RayTracingSkeletalMeshes, ERayTracingProxyType::SkeletalMesh);
		CheckValue(CachedRayTracingISKM, RayTracingISKM, ERayTracingProxyType::InstanceSkeletalMesh);
		CheckValue(CachedRayTracingLandscapeGrass, RayTracingLandscapeGrass, ERayTracingProxyType::LandscapeGrass);
		
		ENQUEUE_RENDER_COMMAND(RefreshRayTracingInstancesCmd)(
			[bRequireUpdateCachedRayTracingState, NewActiveRayTracingProxyTypes](FRHICommandListImmediate&)
			{
				GActiveRayTracingProxyTypes = NewActiveRayTracingProxyTypes;
				bUpdateCachedRayTracingState = bRequireUpdateCachedRayTracingState;
			}
		);
	}
}

static FAutoConsoleVariableSink CVarRefreshRayTracingInstancesSink(FConsoleCommandDelegate::CreateStatic(&RefreshRayTracingInstancesSinkFunction));

extern bool IsRayTracingVisualizeOpaqueOnly();

namespace RayTracing
{
	// Configure ray tracing scene options based on currently enabled features and their needs
	FSceneOptions::FSceneOptions(
		const FScene& Scene,
		const FViewFamilyInfo& ViewFamily,
		const FViewInfo& View,
		EDiffuseIndirectMethod DiffuseIndirectMethod,
		EReflectionsMethod ReflectionsMethod)
	{
		bTranslucentGeometry = false;

		LumenHardwareRayTracing::SetRayTracingSceneOptions(View, DiffuseIndirectMethod, ReflectionsMethod, *this);
		RayTracingShadows::SetRayTracingSceneOptions(View.bHasRayTracingShadows, *this);

		if (ShouldRenderRayTracingTranslucency(View))
		{
			bTranslucentGeometry = true;
		}

		if (ViewFamily.EngineShowFlags.RayTracingDebug && !IsRayTracingVisualizeOpaqueOnly())
		{
			if (!bTranslucentGeometry)
			{
#if !UE_BUILD_SHIPPING
				GEngine->AddOnScreenDebugMessage(uint64(0x529F427D1), 1.f, FColor::Yellow, TEXT("Translucent geometry is only being included in ray tracing scene due to debug visualization mode."));
#endif
			}
			bTranslucentGeometry = true;
		}

		if (GRayTracingExcludeTranslucent != 0)
		{
			bTranslucentGeometry = false;
		}

        if (ViewFamily.EngineShowFlags.PathTracing)
		{
            // When path tracing, always include translucent geometry
			bTranslucentGeometry = true;
		}

		bIncludeSky = GRayTracingExcludeSky == 0 || ViewFamily.EngineShowFlags.PathTracing;

		bLightingChannelsUsingAHS = MegaLights::IsEnabled(ViewFamily) && MegaLights::IsUsingLightingChannels();
	}

	struct FRelevantCachedPrimitive
	{
		TConstArrayView<FRayTracingShaderBindingData> CachedShaderBindingDataBase;
		TConstArrayView<FRayTracingShaderBindingData> CachedShaderBindingDataDecal;
		int32 SBTAllocationUniqueId = INDEX_NONE;
		uint32 MainRayTracingInstanceIndex = UINT32_MAX;
		uint32 DecalRayTracingInstanceIndex = UINT32_MAX;
#if DO_CHECK
		FRHIRayTracingGeometry* RayTracingGeometry = nullptr;
#endif
	};

	struct FRelevantPrimitive
	{
		const FRayTracingGeometry* RayTracingGeometry = nullptr;
		TConstArrayView<FRayTracingShaderBindingData> CachedShaderBindingDataBase;
		TConstArrayView<FRayTracingShaderBindingData> CachedShaderBindingDataDecal;
		FRayTracingCachedMeshCommandFlags CachedMeshCommandFlags;
		int32 PrimitiveIndex = -1;
		int32 SBTAllocationUniqueId = INDEX_NONE;
		int32 InstanceContributionToHitGroupIndexBase = INDEX_NONE;
		int32 InstanceContributionToHitGroupIndexDecal = INDEX_NONE;
		uint8 LightingChannelMask : 3 = GetDefaultLightingChannelMask();

		uint64 InstancingKey() const
		{
			uint64 Key = CachedMeshCommandFlags.CachedMeshCommandHash;
			Key ^= uint64(CachedMeshCommandFlags.InstanceMask) << 32;
			Key ^= CachedMeshCommandFlags.bAllSegmentsOpaque ? 0x1ull << 40 : 0x0;
			Key ^= CachedMeshCommandFlags.bAllSegmentsCastShadow ? 0x1ull << 41 : 0x0;
			Key ^= CachedMeshCommandFlags.bAnySegmentsCastShadow ? 0x1ull << 42 : 0x0;
			Key ^= CachedMeshCommandFlags.bAnySegmentsDecal ? 0x1ull << 43 : 0x0;
			Key ^= CachedMeshCommandFlags.bAllSegmentsDecal ? 0x1ull << 44 : 0x0;
			Key ^= CachedMeshCommandFlags.bTwoSided ? 0x1ull << 45 : 0x0;
			Key ^= CachedMeshCommandFlags.bIsSky ? 0x1ull << 46 : 0x0;
			Key ^= CachedMeshCommandFlags.bAllSegmentsTranslucent ? 0x1ull << 47 : 0x0;
			Key ^= CachedMeshCommandFlags.bAllSegmentsReverseCulling ? 0x1ull << 48 : 0x0;
			Key ^= uint64(LightingChannelMask) << 49;
			return Key ^ reinterpret_cast<uint64>(RayTracingGeometry->GetRHI());
		}
	};

	struct FDynamicRayTracingPrimitive
	{
		int32 PrimitiveIndex;
		TRange<int32> InstancesRange;
		TRange<int32> GeometriesToUpdateRange;
	};

	struct FDynamicPrimitiveIndex
	{
		FDynamicPrimitiveIndex() = default;

		FDynamicPrimitiveIndex(int32 InIndex, uint8 InViewMask)
			: Index(InIndex)
			, ViewMask(InViewMask)
		{}

		uint32 Index : 24;
		uint32 ViewMask : 8;
	};

	class FDynamicPrimitiveIndexQueue
	{
	public:
		FDynamicPrimitiveIndexQueue(TArray<FDynamicPrimitiveIndex, SceneRenderingAllocator>&& InList)
			: List(InList)
		{
		}

		bool Pop(FDynamicPrimitiveIndex& PrimitiveIndex)
		{
			const int32 Index = NextIndex.fetch_add(1, std::memory_order_relaxed);
			if (Index < List.Num())
			{
				PrimitiveIndex = List[Index];
				return true;
			}
			return false;
		}

		int32 Num()
		{
			return List.Num();
		}

	private:
		TArray<FDynamicPrimitiveIndex, SceneRenderingAllocator> List;
		std::atomic_int32_t NextIndex = { 0 };
	};

	class FDynamicRayTracingInstancesContext
	{
	public:
		FDynamicRayTracingInstancesContext(FScene& InScene, TArrayView<FViewInfo*> InViews, TConstArrayView<const FSceneOptions*> InViewSceneOptions, FSceneRenderingBulkObjectAllocator& InBulkAllocator);

		void GatherDynamicRayTracingInstances(FDynamicPrimitiveIndexQueue* PrimitiveIndexQueue);

		void GatherDynamicRayTracingInstances_RenderThread();

		void Finish(FRHICommandListImmediate& InRHICmdList);

		void AddInstancesToScene(FRayTracingScene& RayTracingScene, FRayTracingShaderBindingTable& RayTracingSBT, int64 SharedBufferGenerationID);

		void CollectRDGResources(FRDGBuilder& RDGBuilder);

		void SetRenderThreadDynamicPrimitives(TArray<FDynamicPrimitiveIndex, SceneRenderingAllocator>&& InDynamicPrimitives);

	private:

		void GatherDynamicRayTracingInstances_Internal(const FDynamicPrimitiveIndex& PrimitiveIndex);

		FScene& Scene;
		TArrayView<FViewInfo*> Views;
		TConstArrayView<const FSceneOptions*> ViewSceneOptions;
		FSceneRenderingBulkObjectAllocator& BulkAllocator;

		const bool bTrackReferencedGeometry;
		bool bAnyViewRequiresTranslucentGeometry;

		FRHICommandList* RHICmdList;
		FGlobalDynamicVertexBuffer DynamicVertexBuffer;
		FGlobalDynamicIndexBuffer DynamicIndexBuffer;

		FRayTracingInstanceCollector RayTracingInstanceCollector;

		TArray<FDynamicRayTracingPrimitive, SceneRenderingAllocator> DynamicRayTracingPrimitives;

		TArray<FDynamicPrimitiveIndex, SceneRenderingAllocator> RenderThreadDynamicPrimitives;
	};

	struct FGatherInstancesViewTaskData
	{
		FGatherInstancesViewTaskData(FScene& InScene, FViewInfo& InView, FSceneRenderingBulkObjectAllocator& InBulkAllocator, FSceneOptions InSceneOptions)
			: Scene(InScene)
			, View(InView)
			, SceneOptions(MoveTemp(InSceneOptions))
		{
			ActiveRayTracingProxyTypes = GActiveRayTracingProxyTypes;

			if (!View.Family->EngineShowFlags.StaticMeshes)
			{
				EnumRemoveFlags(ActiveRayTracingProxyTypes, ERayTracingProxyType::StaticMesh);
			}

			if (!View.Family->EngineShowFlags.SkeletalMeshes)
			{
				EnumRemoveFlags(ActiveRayTracingProxyTypes, ERayTracingProxyType::SkeletalMesh);
			}
		}

		FScene& Scene;
		FViewInfo& View;
		FSceneOptions SceneOptions;
		ERayTracingProxyType ActiveRayTracingProxyTypes;

		// Filtered lists of relevant primitives
		TArray<int32> StaticPrimitivesIndices;
		TArray<int32> DynamicPrimitivesIndices;

		TArray<FRelevantPrimitive> StaticPrimitives;
		TArray<FRelevantCachedPrimitive> CachedStaticPrimitives;

		TArray<TSet<RayTracing::FGeometryGroupHandle>> ReferencedGeometryGroups;

		// Array of primitives that need their cached ray tracing instance updated via FPrimitiveSceneInfo::UpdateCachedRaytracingData()
		TArray<FPrimitiveSceneInfo*> DirtyCachedRayTracingPrimitives;

		// This task must complete before accessing StaticPrimitivesIndices/DynamicPrimitivesIndices.
		UE::Tasks::FTask GatherRelevantPrimitivesTask;

		// This task must complete before accessing RayTracingScene/RaytracingSBT when processing dynamic instances.
		UE::Tasks::FTask FinalizeGatherRelevantPrimitivesTask;

		// This task must complete before accessing StaticPrimitives/CachedStaticPrimitives.
		UE::Tasks::FTask GatherRelevantStaticPrimitivesTask;

		int32 NumCachedStaticVisibleShaderBindings = 0; // TODO: Could remove this but it's used to Reserve

		uint8 LightingChannelAndMask = 0xFF;

#if DO_CHECK
		TArray<TSet<RayTracing::FGeometryGroupHandle>> ReferencedGeometryGroupsToCheck;
#endif
	};

	struct FGatherInstancesTaskData
	{
		UE_NONCOPYABLE(FGatherInstancesTaskData)

		FGatherInstancesTaskData(
			FSceneRenderingBulkObjectAllocator& InAllocator,
			FScene& InScene,
			uint32 NumViews)
			: Scene(InScene)
			, Allocator(InAllocator)
		{
			ViewTaskDatas.Reserve(NumViews);
			Views.Reserve(NumViews);
			ViewSceneOptions.Reserve(NumViews);
		}

		FScene& Scene;
		FSceneRenderingBulkObjectAllocator& Allocator;

		TArray<FGatherInstancesViewTaskData> ViewTaskDatas;
		TArray<FViewInfo*> Views;
		TArray<const FSceneOptions*> ViewSceneOptions;

		UE::Tasks::FPipe AddInstancesPipe{ UE_SOURCE_LOCATION };
		UE::Tasks::FPipe FinalizeGatherRelevantPrimitivesPipe{ UE_SOURCE_LOCATION };

		TArray<FDynamicRayTracingInstancesContext*> DynamicRayTracingInstancesContexts;

		// This task must complete before accessing DynamicRayTracingInstancesContext.
		UE::Tasks::FTask GatherDynamicRayTracingInstancesTask;
		UE::Tasks::FTaskEvent GatherDynamicRayTracingInstancesPrerequisites{ UE_SOURCE_LOCATION };
		bool bGatherDynamicRayTracingInstancesPrerequisitesTriggered = false;

		// This task adds dynamic instances and allocates dynamic ranges in the SBT.
		UE::Tasks::FTask AddDynamicInstancesTask;

		UE::Tasks::FTask VisibleRayTracingShaderBindingsFinalizeTask;
		FRayTracingShaderBindingDataOneFrameArray VisibleShaderBindings;

		// Indicates that this object has been fully produced (for validation)
		bool bValid = false;
	};

	FGatherInstancesTaskData* CreateGatherInstancesTaskData(
		FSceneRenderingBulkObjectAllocator& Allocator,
		FScene& Scene,
		uint32 NumViews)
	{
		return Allocator.Create<FGatherInstancesTaskData>(Allocator, Scene, NumViews);
	}

	void AddView(
		FGatherInstancesTaskData& TaskData,
		FViewInfo& View,
		EDiffuseIndirectMethod DiffuseIndirectMethod,
		EReflectionsMethod ReflectionsMethod)
	{
		if (IStereoRendering::IsStereoEyeView(View) && IStereoRendering::IsASecondaryView(View))
		{
			return;
		}

		const FViewFamilyInfo* ViewFamily = static_cast<const FViewFamilyInfo*>(View.Family);

		TaskData.ViewTaskDatas.Add(FGatherInstancesViewTaskData(TaskData.Scene, View, TaskData.Allocator, FSceneOptions(TaskData.Scene, *ViewFamily, View, DiffuseIndirectMethod, ReflectionsMethod)));
		TaskData.Views.Add(&View);
		TaskData.ViewSceneOptions.Add(&TaskData.ViewTaskDatas.Last().SceneOptions);
	}

	void OnRenderBegin(const FSceneRenderUpdateInputs& SceneUpdateInputs)
	{
		const ERayTracingType CurrentType = EnumHasAnyFlags(SceneUpdateInputs.CommonShowFlags, ESceneRenderCommonShowFlags::PathTracing) ? ERayTracingType::PathTracing : ERayTracingType::RayTracing;

		const bool bNaniteRayTracingModeChanged = Nanite::GRayTracingManager.CheckModeChanged();

		FScene& Scene = *SceneUpdateInputs.Scene;

		bool bAnyViewNeedsInstanceExtraDataBuffer = false;
		bool bAnyViewNeedsRayTracingInstanceDebugData = false;

		for (const FViewInfo* View : SceneUpdateInputs.Views)
		{
			bAnyViewNeedsInstanceExtraDataBuffer |= IsRayTracingInstanceOverlapEnabled(*View);
			bAnyViewNeedsRayTracingInstanceDebugData |= IsRayTracingInstanceDebugDataEnabled(*View);
		}

		if (Scene.RayTracingScene.SetInstanceExtraDataBufferEnabled(bAnyViewNeedsInstanceExtraDataBuffer))
		{
			bUpdateCachedRayTracingState = true;
		}

		if (Scene.RayTracingScene.SetInstanceDebugDataEnabled(bAnyViewNeedsRayTracingInstanceDebugData))
		{
			bUpdateCachedRayTracingState = true;
		}

		bool bAnyViewFamilyUsingRayTracingFeedback = false;

		for (const FViewFamilyInfo* ViewFamily : SceneUpdateInputs.ViewFamilies)
		{
			bAnyViewFamilyUsingRayTracingFeedback |= IsRayTracingFeedbackEnabled(*ViewFamily);
		}

		if (Scene.RayTracingScene.SetTracingFeedbackEnabled(bAnyViewFamilyUsingRayTracingFeedback))
		{
			bUpdateCachedRayTracingState = true;
		}

		if (CurrentType != Scene.CachedRayTracingMeshCommandsType
			|| bNaniteRayTracingModeChanged
			|| bUpdateCachedRayTracingState)
		{
			Scene.WaitForCacheRayTracingPrimitivesTask();

			// In some situations, we need to refresh the cached ray tracing mesh commands because they contain data about the currently bound shader. 
			// This operation is a bit expensive but only happens once as we transition between RT types which should be rare.

			// If any material has been updated process it before re-caching
			FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions();

			Scene.CachedRayTracingMeshCommandsType = CurrentType;
			Scene.RefreshCachedRayTracingData();
			bUpdateCachedRayTracingState = false;
		}

		if (bNaniteRayTracingModeChanged)
		{
			for (FViewInfo* View : SceneUpdateInputs.Views)
			{
				if (View->ViewState != nullptr && !View->bIsOfflineRender)
				{
					// don't invalidate in the offline case because we only get one attempt at rendering each sample
					View->ViewState->PathTracingInvalidate();
				}
			}
		}
	}

	class FRaytracingShaderBindingLayout : public FShaderBindingLayoutContainer
	{
	public:
		static const FShaderBindingLayout& GetInstance(EBindingType BindingType)
		{
			static FRaytracingShaderBindingLayout Instance;
			return Instance.GetLayout(BindingType);
		}
	private:

		FRaytracingShaderBindingLayout()
		{
			// No special binding layout flags required
			EShaderBindingLayoutFlags ShaderBindingLayoutFlags = EShaderBindingLayoutFlags::None;

			// Add scene, view and nanite ray tracing as global/static uniform buffers
			TArray<FShaderParametersMetadata*> StaticUniformBuffers;
			StaticUniformBuffers.Add(FindUniformBufferStructByName(TEXT("Scene")));
			StaticUniformBuffers.Add(FindUniformBufferStructByName(TEXT("View")));
			StaticUniformBuffers.Add(FindUniformBufferStructByName(TEXT("NaniteRayTracing")));
			StaticUniformBuffers.Add(FindUniformBufferStructByName(TEXT("LumenHardwareRayTracingUniformBuffer")));

			BuildShaderBindingLayout(StaticUniformBuffers, ShaderBindingLayoutFlags, *this);
		}
	};

	const FShaderBindingLayout* GetShaderBindingLayout(EShaderPlatform ShaderPlatform)
	{
		if (RHIGetStaticShaderBindingLayoutSupport(ShaderPlatform) != ERHIStaticShaderBindingLayoutSupport::Unsupported)
		{
			// Should support bindless for raytracing at least
			// NOTE: checks disable checks because GConfig which is used to check 
			// runtime binding config can be modified in another thread at the same time
			//check(RHIGetRuntimeBindlessResourcesConfiguration(ShaderPlatform) != ERHIBindlessConfiguration::Disabled);
			//check(RHIGetRuntimeBindlessSamplersConfiguration(ShaderPlatform) != ERHIBindlessConfiguration::Disabled);

			// Retrieve the bindless shader binding table
			return &FRaytracingShaderBindingLayout::GetInstance(FShaderBindingLayoutContainer::EBindingType::Bindless);
		}

		// No binding table supported
		return nullptr;
	}

	TOptional<FScopedUniformBufferStaticBindings> BindStaticUniformBufferBindings(const FViewInfo& View, FRHIUniformBuffer* SceneUniformBuffer, FRHIUniformBuffer* NaniteRayTracingUniformBuffer, FRHICommandList& RHICmdList)
	{
		TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope;

		// Setup the static uniform buffers used by the RTPSO if enabled
		const FShaderBindingLayout* ShaderBindingLayout = GetShaderBindingLayout(View.GetShaderPlatform());
		if (ShaderBindingLayout)
		{
			FUniformBufferStaticBindings StaticUniformBuffers(&ShaderBindingLayout->RHILayout);
			StaticUniformBuffers.AddUniformBuffer(View.ViewUniformBuffer.GetReference());
			StaticUniformBuffers.AddUniformBuffer(SceneUniformBuffer);
			StaticUniformBuffers.AddUniformBuffer(NaniteRayTracingUniformBuffer);
			StaticUniformBuffers.AddUniformBuffer(View.LumenHardwareRayTracingUniformBuffer.GetReference());

			StaticUniformBufferScope.Emplace(RHICmdList, StaticUniformBuffers);
		}

		return StaticUniformBufferScope;
	}

	struct FRayTracingMeshBatchWorkItem
	{
		const FPrimitiveSceneProxy* SceneProxy = nullptr;
		const FRHIRayTracingGeometry* RayTracingGeometry = nullptr;
		TArray<FMeshBatch> MeshBatchesOwned;
		TArrayView<const FMeshBatch> MeshBatchesView;
		FRayTracingSBTAllocation* SBTAllocation = nullptr;
		bool bDynamicRayTracingGeometry = false;

		TArrayView<const FMeshBatch> GetMeshBatches() const
		{
			if (MeshBatchesOwned.Num())
			{
				check(MeshBatchesView.Num() == 0);
				return TArrayView<const FMeshBatch>(MeshBatchesOwned);
			}
			else
			{
				check(MeshBatchesOwned.Num() == 0);
				return MeshBatchesView;
			}
		}
	};

	struct FRayTracingMeshBatchTaskPage
	{
		static constexpr uint32 MaxWorkItems = 128; // Try to keep individual pages small to avoid slow-path memory allocations

		FRayTracingMeshBatchWorkItem WorkItems[MaxWorkItems];
		uint32 NumWorkItems = 0;
		FRayTracingMeshBatchTaskPage* Next = nullptr;
	};

	struct FRayTracingMeshBatchTaskData
	{
		FRayTracingMeshBatchTaskPage* Head = nullptr;
		FRayTracingMeshBatchTaskPage* Page = nullptr;
		uint32 NumPendingMeshBatches = 0;
	};

	void DispatchRayTracingMeshBatchTask(FSceneRenderingBulkObjectAllocator& InBulkAllocator, FScene& Scene, FViewInfo& View, FRayTracingMeshBatchTaskPage* MeshBatchTaskHead, uint32 NumPendingMeshBatches)
	{
		FDynamicRayTracingMeshCommandStorage* TaskDynamicCommandStorage = InBulkAllocator.Create<FDynamicRayTracingMeshCommandStorage>();
		View.DynamicRayTracingMeshCommandStoragePerTask.Add(TaskDynamicCommandStorage);

		FRayTracingShaderBindingDataOneFrameArray* TaskVisibleShaderBindings = InBulkAllocator.Create<FRayTracingShaderBindingDataOneFrameArray>();
		TaskVisibleShaderBindings->Reserve(NumPendingMeshBatches);

		View.DynamicRayTracingShaderBindingsPerTask.Add(TaskVisibleShaderBindings);

		View.AddDynamicRayTracingMeshBatchTaskList.Add(UE::Tasks::Launch(UE_SOURCE_LOCATION,
			[TaskDataHead = MeshBatchTaskHead, &View, &Scene, TaskDynamicCommandStorage, TaskVisibleShaderBindings]()
			{
				FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);
				TRACE_CPUPROFILER_EVENT_SCOPE(RayTracingMeshBatchTask);
				FRayTracingMeshBatchTaskPage* Page = TaskDataHead;
				const int32 ExpectedMaxVisibleCommands = TaskVisibleShaderBindings->Max();
				while (Page)
				{
					for (uint32 ItemIndex = 0; ItemIndex < Page->NumWorkItems; ++ItemIndex)
					{
						const FRayTracingMeshBatchWorkItem& WorkItem = Page->WorkItems[ItemIndex];
						TArrayView<const FMeshBatch> MeshBatches = WorkItem.GetMeshBatches();
						for (const FMeshBatch& MeshBatch : MeshBatches)
						{
							FDynamicRayTracingMeshCommandContext CommandContext(
								*TaskDynamicCommandStorage, *TaskVisibleShaderBindings,
								WorkItem.RayTracingGeometry, MeshBatch.SegmentIndex, WorkItem.bDynamicRayTracingGeometry, WorkItem.SBTAllocation);
							FRayTracingMeshProcessor RayTracingMeshProcessor(&CommandContext, &Scene, &View, Scene.CachedRayTracingMeshCommandsType);
							RayTracingMeshProcessor.AddMeshBatch(MeshBatch, 1, WorkItem.SceneProxy);
						}
					}
					FRayTracingMeshBatchTaskPage* NextPage = Page->Next;
					Page = NextPage;
				}
				check(ExpectedMaxVisibleCommands <= TaskVisibleShaderBindings->Max());
			}, UE::Tasks::ETaskPriority::High));
	};

	FDynamicRayTracingInstancesContext::FDynamicRayTracingInstancesContext(FScene& Scene, TArrayView<FViewInfo*> InViews, TConstArrayView<const FSceneOptions*> InViewSceneOptions, FSceneRenderingBulkObjectAllocator& InBulkAllocator)
		: Scene(Scene)
		, Views(InViews)
		, ViewSceneOptions(InViewSceneOptions)
		, BulkAllocator(InBulkAllocator)
		, bTrackReferencedGeometry(IsRayTracingUsingReferenceBasedResidency())
		, RHICmdList(new FRHICommandList(FRHIGPUMask::All()))
		, DynamicVertexBuffer(*RHICmdList)
		, DynamicIndexBuffer(*RHICmdList)
		, RayTracingInstanceCollector(Scene.GetFeatureLevel(), InBulkAllocator, bTrackReferencedGeometry)
	{
		RHICmdList->SwitchPipeline(ERHIPipeline::Graphics);

		RayTracingInstanceCollector.Start(
			*RHICmdList,
			DynamicVertexBuffer,
			DynamicIndexBuffer,
			FSceneRenderer::DynamicReadBufferForRayTracing
		);

		bAnyViewRequiresTranslucentGeometry = false;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			FViewInfo& View = *Views[ViewIndex];

			RayTracingInstanceCollector.AddViewMeshArrays(&View, &View.RayTracingDynamicPrimitiveCollector);

			bAnyViewRequiresTranslucentGeometry |= ViewSceneOptions[ViewIndex]->bTranslucentGeometry;
		}
	}

	void FDynamicRayTracingInstancesContext::GatherDynamicRayTracingInstances_Internal(const FDynamicPrimitiveIndex& PrimitiveIndex)
	{
		FPrimitiveSceneProxy* SceneProxy = Scene.PrimitiveSceneProxies[PrimitiveIndex.Index];

		RayTracingInstanceCollector.SetPrimitive(SceneProxy, FHitProxyId::InvisibleHitProxyId);
		RayTracingInstanceCollector.SetVisibilityMap(PrimitiveIndex.ViewMask);

		int32 BaseRayTracingInstance = RayTracingInstanceCollector.RayTracingInstances.Num();
		int32 BaseGeometryToUpdate = RayTracingInstanceCollector.RayTracingGeometriesToUpdate.Num();

		if (bAnyViewRequiresTranslucentGeometry || SceneProxy->IsOpaqueOrMasked())
		{
			SceneProxy->GetDynamicRayTracingInstances(RayTracingInstanceCollector);
		}

		FDynamicRayTracingPrimitive Tmp;
		Tmp.PrimitiveIndex = PrimitiveIndex.Index;
		Tmp.InstancesRange = TRange<int32>(BaseRayTracingInstance, RayTracingInstanceCollector.RayTracingInstances.Num());
		Tmp.GeometriesToUpdateRange = TRange<int32>(BaseGeometryToUpdate, RayTracingInstanceCollector.RayTracingGeometriesToUpdate.Num());

		DynamicRayTracingPrimitives.Add(MoveTemp(Tmp));
	}

	void FDynamicRayTracingInstancesContext::GatherDynamicRayTracingInstances(FDynamicPrimitiveIndexQueue* PrimitiveIndexQueue)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracing_GatherDynamicRayTracingInstances);

		check(IsParallelGatherDynamicRayTracingInstancesEnabled());

		// Reserve for worst case even though queue might be processed by multiple tasks
		DynamicRayTracingPrimitives.Reserve(DynamicRayTracingPrimitives.Num() + PrimitiveIndexQueue->Num());

		// TODO: Could filter primitives whose proxy supports ParallelGDRTI during GatherRayTracingRelevantPrimitives_Parallel

		FDynamicPrimitiveIndex PrimitiveIndex;
		while (PrimitiveIndexQueue->Pop(PrimitiveIndex))
		{
			FPrimitiveSceneProxy* SceneProxy = Scene.PrimitiveSceneProxies[PrimitiveIndex.Index];

			if (SceneProxy->SupportsParallelGDRTI())
			{
				GatherDynamicRayTracingInstances_Internal(PrimitiveIndex);
			}
			else
			{
				RenderThreadDynamicPrimitives.Add(PrimitiveIndex);
			}
		}
	}

	void FDynamicRayTracingInstancesContext::GatherDynamicRayTracingInstances_RenderThread()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracing_GatherDynamicRayTracingInstances_RenderThread);

		check(IsInRenderingThread());

		for (const FDynamicPrimitiveIndex& PrimitiveIndex : RenderThreadDynamicPrimitives)
		{
			GatherDynamicRayTracingInstances_Internal(PrimitiveIndex);
		}

		RenderThreadDynamicPrimitives.Empty();
	}

	void FDynamicRayTracingInstancesContext::SetRenderThreadDynamicPrimitives(TArray<FDynamicPrimitiveIndex, SceneRenderingAllocator>&& InDynamicPrimitives)
	{
		check(RenderThreadDynamicPrimitives.IsEmpty());

		RenderThreadDynamicPrimitives = MoveTemp(InDynamicPrimitives);
	}

	void FDynamicRayTracingInstancesContext::Finish(FRHICommandListImmediate& InRHICmdList)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracing_GatherDynamicRayTracingInstances_Finish);

		// TODO: Could process RayTracingGeometriesToUpdate in parallel thread after merging multiple tasks

		FRayTracingDynamicGeometryUpdateManager* DynamicGeometryUpdateManager = Scene.GetRayTracingDynamicGeometryUpdateManager();

		// Can't use RayTracingGeometriesToUpdate directly because need SceneProxy and PersistentPrimitiveIndex
		// TODO: Move those parameters into FRayTracingDynamicGeometryUpdateParams
		for (const FDynamicRayTracingPrimitive& DynamicRayTracingPrimitive : DynamicRayTracingPrimitives)
		{
			FPrimitiveSceneProxy* SceneProxy = Scene.PrimitiveSceneProxies[DynamicRayTracingPrimitive.PrimitiveIndex];
			FPrimitiveSceneInfo* SceneInfo = Scene.Primitives[DynamicRayTracingPrimitive.PrimitiveIndex];
			const FPersistentPrimitiveIndex PersistentPrimitiveIndex = SceneInfo->GetPersistentIndex();

			for (int32 Index = DynamicRayTracingPrimitive.GeometriesToUpdateRange.GetLowerBoundValue(); Index < DynamicRayTracingPrimitive.GeometriesToUpdateRange.GetUpperBoundValue(); ++Index)
			{
				FRayTracingInstanceCollector::FRayTracingDynamicGeometryUpdateRequest& UpdateRequest = RayTracingInstanceCollector.RayTracingGeometriesToUpdate[Index];

				const FViewInfo* View = Views[UpdateRequest.ViewIndex];

				uint32 PrimitiveId = PersistentPrimitiveIndex.Index;

				TConstArrayView<FMeshBatch> MeshBatches = UpdateRequest.Params.GetMeshBatches();
				if (MeshBatches.Num() > 0 && MeshBatches[0].Elements.Num() > 0 && MeshBatches[0].Elements[0].DynamicPrimitiveData != nullptr)
				{
					const int32 ViewDynamicPrimitiveId = View->RayTracingDynamicPrimitiveCollector.GetPrimitiveIdRange().GetLowerBoundValue();
					ensure(MeshBatches[0].Elements[0].DynamicPrimitiveIndex != INDEX_NONE);
					PrimitiveId = ViewDynamicPrimitiveId + MeshBatches[0].Elements[0].DynamicPrimitiveIndex;
				}
				
				DynamicGeometryUpdateManager->AddDynamicGeometryToUpdate(
					*RHICmdList,
					&Scene,
					View,
					SceneProxy,
					MoveTemp(UpdateRequest.Params),
					PrimitiveId
				);
			}
		}

		RayTracingInstanceCollector.RayTracingGeometriesToUpdate.Empty();

		if (bTrackReferencedGeometry)
		{
			// TODO: Could run in parallel thread if properly synchronized with static ray tracing instances tasks
			((FRayTracingGeometryManager*)GRayTracingGeometryManager)->AddReferencedGeometries(RayTracingInstanceCollector.ReferencedGeometries);
			((FRayTracingGeometryManager*)GRayTracingGeometryManager)->AddReferencedGeometryGroups(RayTracingInstanceCollector.ReferencedGeometryGroups);
			((FRayTracingGeometryManager*)GRayTracingGeometryManager)->AddReferencedGeometryGroupsForDynamicUpdate(RayTracingInstanceCollector.ReferencedGeometryGroupsForDynamicUpdate);

			RayTracingInstanceCollector.ReferencedGeometries.Empty();
			RayTracingInstanceCollector.ReferencedGeometryGroups.Empty();
			RayTracingInstanceCollector.ReferencedGeometryGroupsForDynamicUpdate.Empty();
		}
		else
		{
			check(RayTracingInstanceCollector.ReferencedGeometries.IsEmpty());
			check(RayTracingInstanceCollector.ReferencedGeometryGroups.IsEmpty());
			check(RayTracingInstanceCollector.ReferencedGeometryGroupsForDynamicUpdate.IsEmpty());
		}

		RayTracingInstanceCollector.Finish();

		DynamicVertexBuffer.Commit();
		DynamicIndexBuffer.Commit();
		RHICmdList->FinishRecording();

		InRHICmdList.QueueAsyncCommandListSubmit(RHICmdList);
	}

	void FDynamicRayTracingInstancesContext::CollectRDGResources(FRDGBuilder& RDGBuilder)
	{
		for (int32 ViewIndex = 0; ViewIndex < RayTracingInstanceCollector.RDGPooledBuffers.Num(); ++ViewIndex)
		{
			check(RayTracingInstanceCollector.Views[ViewIndex] == Views[ViewIndex]);

			const TSet<FRDGPooledBuffer*>& ViewRDGPooledBuffers = RayTracingInstanceCollector.RDGPooledBuffers[ViewIndex];

			FViewInfo& View = *Views[ViewIndex];

			for (FRDGPooledBuffer* PooledBuffer : ViewRDGPooledBuffers)
			{
				FRDGBufferRef RDGBuffer = RDGBuilder.RegisterExternalBuffer(PooledBuffer);
				View.DynamicRayTracingRDGBuffers.Add(RDGBuffer);
			}
		}
	}

	void FDynamicRayTracingInstancesContext::AddInstancesToScene(FRayTracingScene& RayTracingScene, FRayTracingShaderBindingTable& RayTracingSBT, int64 SharedBufferGenerationID)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracing_AddDynamicInstancesToScene);

		const bool bParallelMeshBatchSetup = GRayTracingParallelMeshBatchSetup && FApp::ShouldUseThreadingForPerformance();
		const float LastRenderTimeUpdateDistance = CVarRayTracingDynamicGeometryLastRenderTimeUpdateDistance.GetValueOnRenderThread();
		const uint32 RayTracingParallelMeshBatchSize = GRayTracingParallelMeshBatchSize;

		const bool bVertexBufferRequired = GRHIGlobals.RayTracing.SupportsShaders || GRHIGlobals.RayTracing.RequiresInlineRayTracingSBT;

		TArray<FRayTracingMeshBatchTaskData, TInlineAllocator<2, SceneRenderingAllocator>> MeshBatchTaskData;
		MeshBatchTaskData.AddDefaulted(Views.Num());

		auto KickRayTracingMeshBatchTask = [this](FRayTracingMeshBatchTaskData& MeshBatchTaskData, FViewInfo& View)
			{
				if (MeshBatchTaskData.Head)
				{
					DispatchRayTracingMeshBatchTask(BulkAllocator, Scene, View, MeshBatchTaskData.Head, MeshBatchTaskData.NumPendingMeshBatches);
				}

				MeshBatchTaskData.Head = nullptr;
				MeshBatchTaskData.Page = nullptr;
				MeshBatchTaskData.NumPendingMeshBatches = 0;
			};

		for (const FDynamicRayTracingPrimitive& DynamicRayTracingPrimitive : DynamicRayTracingPrimitives)
		{
			const int32 PrimitiveIndex = DynamicRayTracingPrimitive.PrimitiveIndex;
			FPrimitiveSceneProxy* SceneProxy = Scene.PrimitiveSceneProxies[PrimitiveIndex];
			FPrimitiveSceneInfo* SceneInfo = Scene.Primitives[PrimitiveIndex];
			const FScene::FPrimitiveRayTracingData& PrimitiveRayTracingData = Scene.PrimitiveRayTracingDatas[PrimitiveIndex];

			if (DynamicRayTracingPrimitive.InstancesRange.Size<int32>() > 0)
			{
				for (int32 DynamicInstanceIndex = DynamicRayTracingPrimitive.InstancesRange.GetLowerBoundValue(); DynamicInstanceIndex < DynamicRayTracingPrimitive.InstancesRange.GetUpperBoundValue(); ++DynamicInstanceIndex)
				{
					FRayTracingInstanceCollector::FRayTracingInstanceAndViewIndex& InstanceAndViewIndex = RayTracingInstanceCollector.RayTracingInstances[DynamicInstanceIndex];
					FRayTracingInstance& Instance = InstanceAndViewIndex.Instance;
					FViewInfo& View = *Views[InstanceAndViewIndex.ViewIndex];

					const int32 ViewInstanceSceneDataOffset = View.RayTracingDynamicPrimitiveCollector.GetInstanceSceneDataOffset();

					const FRayTracingGeometry* Geometry = Instance.Geometry;
					const FRayTracingGeometryInitializer& GeometryInitializer = Geometry->GetInitializer();
					const int32 NumSegments = GeometryInitializer.Segments.Num();

					if (!ensureMsgf(!bVertexBufferRequired
						|| Geometry->DynamicGeometrySharedBufferGenerationID == FRayTracingGeometry::NonSharedVertexBuffers
						|| Geometry->DynamicGeometrySharedBufferGenerationID == SharedBufferGenerationID,
						TEXT("GenerationID %lld, but expected to be %lld or %lld. Geometry debug name: '%s'. ")
						TEXT("When shared vertex buffers are used, the contents is expected to be written every frame. ")
						TEXT("Possibly AddDynamicMeshBatchForGeometryUpdate() was not called for this geometry."),
						Geometry->DynamicGeometrySharedBufferGenerationID, SharedBufferGenerationID, FRayTracingGeometry::NonSharedVertexBuffers,
						*GeometryInitializer.DebugName.ToString()))
					{
						continue;
					}

					((FRayTracingGeometryManager*)GRayTracingGeometryManager)->AddReferencedGeometry(Geometry);

					if (!ensureMsgf(!Geometry->IsEvicted(),
						TEXT("Dynamic ray tracing instance skipped because geometry '%s' is evicted."),
						*GeometryInitializer.DebugName.ToString()))
					{
						continue;
					}

					if (!ensureMsgf(!Geometry->HasPendingBuildRequest(),
						TEXT("Dynamic ray tracing instance skipped because geometry '%s' has a pending build request."),
						*GeometryInitializer.DebugName.ToString()))
					{
						continue;
					}

					TConstArrayView<FMeshBatch> InstanceMaterials = Instance.GetMaterials();

					// Validate the material/segment counts
					if (!ensureMsgf(InstanceMaterials.Num() <= NumSegments,
						TEXT("Ray tracing material assignment validation failed for geometry '%s'. ")
						TEXT("Instance.GetMaterials().Num() = %d, GeometryInitializer.Segments.Num() = %d."),
						*GeometryInitializer.DebugName.ToString(), InstanceMaterials.Num(),
						NumSegments))
					{
						continue;
					}

					if (InstanceMaterials.IsEmpty())
					{
						// If the material list is empty, skip this instance altogether
						continue;
					}

					if (Instance.bInstanceMaskAndFlagsDirty || SceneInfo->bDynamicRayTracingInstanceCachedDataDirty)
					{
						// Build InstanceMaskAndFlags since the data in SceneInfo is not up to date

						FRayTracingMaskAndFlags InstanceMaskAndFlags;

						InstanceMaskAndFlags = BuildRayTracingInstanceMaskAndFlags(Instance, *SceneProxy);

						SceneInfo->DynamicRayTracingInstanceCachedData.Mask = InstanceMaskAndFlags.Mask;
						SceneInfo->DynamicRayTracingInstanceCachedData.bForceOpaque = InstanceMaskAndFlags.bForceOpaque;
						SceneInfo->DynamicRayTracingInstanceCachedData.bDoubleSided = InstanceMaskAndFlags.bDoubleSided;
						SceneInfo->DynamicRayTracingInstanceCachedData.bReverseCulling = InstanceMaskAndFlags.bReverseCulling;
						SceneInfo->DynamicRayTracingInstanceCachedData.bAnySegmentsDecal = InstanceMaskAndFlags.bAnySegmentsDecal;
						SceneInfo->DynamicRayTracingInstanceCachedData.bAllSegmentsDecal = InstanceMaskAndFlags.bAllSegmentsDecal;
						SceneInfo->DynamicRayTracingInstanceCachedData.bAllSegmentsTranslucent = InstanceMaskAndFlags.bAllSegmentsTranslucent;
						SceneInfo->bDynamicRayTracingInstanceCachedDataDirty = false;
					}

					if (!ViewSceneOptions[InstanceAndViewIndex.ViewIndex]->bTranslucentGeometry && SceneInfo->DynamicRayTracingInstanceCachedData.bAllSegmentsTranslucent)
					{
						continue;
					}

					// TODO: Do we want to support dynamic instances in far field?
					const bool bNeedMainInstance = !SceneInfo->DynamicRayTracingInstanceCachedData.bAllSegmentsDecal;

					// if primitive has mixed decal and non-decal segments we need to have two ray tracing instances
					// one containing non-decal segments and the other with decal segments
					// masking of segments is done using "hidden" hitgroups
					// TODO: Debug Visualization to highlight primitives using this?
					const bool bNeedDecalInstance = SceneInfo->DynamicRayTracingInstanceCachedData.bAnySegmentsDecal && !ShouldExcludeDecals();

					if (ShouldExcludeDecals() && SceneInfo->DynamicRayTracingInstanceCachedData.bAllSegmentsDecal)
					{
						continue;
					}

					int32 InstanceSceneDataOffset = SceneInfo->GetInstanceSceneDataOffset();
					uint32 NumInstanceSceneDataEntries = SceneInfo->GetNumInstanceSceneDataEntries();

					if (InstanceMaterials.Num() > 0 && InstanceMaterials[0].Elements.Num() > 0 && InstanceMaterials[0].Elements[0].DynamicPrimitiveData != nullptr)
					{
						InstanceSceneDataOffset = ViewInstanceSceneDataOffset + InstanceMaterials[0].Elements[0].DynamicPrimitiveInstanceSceneDataOffset;
						NumInstanceSceneDataEntries = InstanceMaterials[0].Elements[0].NumInstances;
					}

					FRayTracingGeometryInstance RayTracingInstance;
					RayTracingInstance.GeometryRHI = Geometry->GetRHI();
					checkf(RayTracingInstance.GeometryRHI, TEXT("Ray tracing instance must have a valid geometry."));
					RayTracingInstance.DefaultUserData = InstanceSceneDataOffset;
					RayTracingInstance.bIncrementUserDataPerInstance = true;
					RayTracingInstance.bApplyLocalBoundsTransform = Instance.bApplyLocalBoundsTransform;
					RayTracingInstance.LightingChannelMask = PrimitiveRayTracingData.LightingChannelMask;
					RayTracingInstance.Mask = SceneInfo->DynamicRayTracingInstanceCachedData.Mask;
					if (SceneInfo->DynamicRayTracingInstanceCachedData.bForceOpaque)
					{
						RayTracingInstance.Flags |= ERayTracingInstanceFlags::ForceOpaque;
					}
					if (SceneInfo->DynamicRayTracingInstanceCachedData.bDoubleSided)
					{
						RayTracingInstance.Flags |= ERayTracingInstanceFlags::TriangleCullDisable;
					}
					if (SceneInfo->DynamicRayTracingInstanceCachedData.bReverseCulling)
					{
						RayTracingInstance.Flags |= ERayTracingInstanceFlags::TriangleCullReverse;
					}

					if (!Instance.GetPrimitiveInstanceIndices().IsEmpty())
					{
						TConstArrayView<uint32> PrimitiveInstanceIndices = Instance.GetPrimitiveInstanceIndices();

						// Convert from instance indices to InstanceSceneDataOffsets
						TArrayView<uint32> InstanceSceneDataOffsets = RayTracingScene.Allocate<uint32>(PrimitiveInstanceIndices.Num());
						for (int32 InstanceIndex = 0; InstanceIndex < PrimitiveInstanceIndices.Num(); ++InstanceIndex)
						{
							check(PrimitiveInstanceIndices[InstanceIndex] <= NumInstanceSceneDataEntries);
							InstanceSceneDataOffsets[InstanceIndex] = InstanceSceneDataOffset + PrimitiveInstanceIndices[InstanceIndex];
						}

						RayTracingInstance.InstanceSceneDataOffsets = InstanceSceneDataOffsets;
						RayTracingInstance.UserData = InstanceSceneDataOffsets;
						RayTracingInstance.NumTransforms = PrimitiveInstanceIndices.Num();
					}
					else if (!Instance.GetTransforms().IsEmpty())
					{
						TConstArrayView<FMatrix> TransformsView;
						if (Instance.OwnsTransforms())
						{
							// Slow path: copy transforms to the owned storage
							checkf(Instance.InstanceTransformsView.Num() == 0, TEXT("InstanceTransformsView is expected to be empty if using InstanceTransforms"));
							TArrayView<FMatrix> SceneOwnedTransforms = RayTracingScene.Allocate<FMatrix>(Instance.InstanceTransforms.Num());
							FMemory::Memcpy(SceneOwnedTransforms.GetData(), Instance.InstanceTransforms.GetData(), Instance.InstanceTransforms.Num() * sizeof(RayTracingInstance.Transforms[0]));
							static_assert(std::is_same_v<decltype(SceneOwnedTransforms[0]), decltype(Instance.InstanceTransforms[0])>, "Unexpected transform type");

							TransformsView = SceneOwnedTransforms;
						}
						else
						{
							// Fast path: just reference persistently-allocated transforms and avoid a copy
							checkf(Instance.InstanceTransforms.Num() == 0, TEXT("InstanceTransforms is expected to be empty if using InstanceTransformsView"));
							TransformsView = Instance.InstanceTransformsView;
						}

						RayTracingInstance.NumTransforms = TransformsView.Num();
						RayTracingInstance.Transforms = TransformsView;
					}
					else
					{
						// If array of transforms was not provided, get the instance transforms from GPU Scene
						check(Instance.NumTransforms <= NumInstanceSceneDataEntries);
						RayTracingInstance.NumTransforms = Instance.NumTransforms;
						RayTracingInstance.BaseInstanceSceneDataOffset = InstanceSceneDataOffset;
					}

					ERayTracingShaderBindingLayerMask ActiveLayers = ERayTracingShaderBindingLayerMask::None;
					if (bNeedMainInstance)
					{
						EnumAddFlags(ActiveLayers, ERayTracingShaderBindingLayerMask::Base);
					}
					if (bNeedDecalInstance)
					{
						EnumAddFlags(ActiveLayers, ERayTracingShaderBindingLayerMask::Decals);
					}

					FRayTracingSBTAllocation* SBTAllocation = RayTracingSBT.AllocateDynamicRange(ActiveLayers, NumSegments);
					if (bNeedMainInstance)
					{
						RayTracingInstance.InstanceContributionToHitGroupIndex = SBTAllocation->GetInstanceContributionToHitGroupIndex(ERayTracingShaderBindingLayer::Base);

						const ERayTracingPrimitiveFlags Flags = Scene.PrimitiveRayTracingFlags[PrimitiveIndex];
						const ERayTracingSceneLayer Layer = EnumHasAllFlags(Flags, ERayTracingPrimitiveFlags::FarField) ? ERayTracingSceneLayer::FarField : ERayTracingSceneLayer::Base;

						RayTracingScene.AddTransientInstance(
							RayTracingInstance,
							Layer,
							View.GetRayTracingSceneViewHandle(),
							SceneProxy,
							/*bDynamic*/ true,
							Geometry->GetGeometryHandle());
					}

					if (bNeedDecalInstance)
					{
						FRayTracingGeometryInstance DecalRayTracingInstance = RayTracingInstance;
						DecalRayTracingInstance.InstanceContributionToHitGroupIndex = SBTAllocation->GetInstanceContributionToHitGroupIndex(ERayTracingShaderBindingLayer::Decals);
						RayTracingScene.AddTransientInstance(
							MoveTemp(DecalRayTracingInstance),
							ERayTracingSceneLayer::Decals,
							View.GetRayTracingSceneViewHandle(),
							SceneProxy,
							/*bDynamic*/ true,
							Geometry->GetGeometryHandle());
					}

					if (bNeedMainInstance || bNeedDecalInstance)
					{
						RayTracingScene.LightingChannelAndMask &= PrimitiveRayTracingData.LightingChannelMask;
					}

					if (bParallelMeshBatchSetup)
					{
						FRayTracingMeshBatchTaskData& ViewMeshBatchTaskData = MeshBatchTaskData[InstanceAndViewIndex.ViewIndex];

						if (ViewMeshBatchTaskData.NumPendingMeshBatches >= RayTracingParallelMeshBatchSize)
						{
							KickRayTracingMeshBatchTask(ViewMeshBatchTaskData, View);
						}

						if (ViewMeshBatchTaskData.Page == nullptr || ViewMeshBatchTaskData.Page->NumWorkItems == FRayTracingMeshBatchTaskPage::MaxWorkItems)
						{
							FRayTracingMeshBatchTaskPage* NextPage = BulkAllocator.Create<FRayTracingMeshBatchTaskPage>();
							if (ViewMeshBatchTaskData.Head == nullptr)
							{
								ViewMeshBatchTaskData.Head = NextPage;
							}
							if (ViewMeshBatchTaskData.Page)
							{
								ViewMeshBatchTaskData.Page->Next = NextPage;
							}
							ViewMeshBatchTaskData.Page = NextPage;
						}

						FRayTracingMeshBatchWorkItem& WorkItem = ViewMeshBatchTaskData.Page->WorkItems[ViewMeshBatchTaskData.Page->NumWorkItems];
						ViewMeshBatchTaskData.Page->NumWorkItems++;

						ViewMeshBatchTaskData.NumPendingMeshBatches += InstanceMaterials.Num();

						if (Instance.OwnsMaterials())
						{
							Swap(WorkItem.MeshBatchesOwned, Instance.Materials);
						}
						else
						{
							WorkItem.MeshBatchesView = Instance.MaterialsView;
						}

						WorkItem.SceneProxy = SceneProxy;
						WorkItem.RayTracingGeometry = Geometry->GetRHI();
						WorkItem.bDynamicRayTracingGeometry = GeometryInitializer.bAllowUpdate;
						WorkItem.SBTAllocation = SBTAllocation;
					}
					else
					{
						for (const FMeshBatch& MeshBatch : InstanceMaterials)
						{
							FDynamicRayTracingMeshCommandContext CommandContext(View.DynamicRayTracingMeshCommandStorage, View.VisibleRayTracingShaderBindings, Geometry->GetRHI(), MeshBatch.SegmentIndex, GeometryInitializer.bAllowUpdate, SBTAllocation);
							FRayTracingMeshProcessor RayTracingMeshProcessor(&CommandContext, &Scene, &View, Scene.CachedRayTracingMeshCommandsType);
							RayTracingMeshProcessor.AddMeshBatch(MeshBatch, 1, SceneProxy);
						}
					}
				}

				if (LastRenderTimeUpdateDistance > 0.0f)
				{
					float CurrentWorldTime = 0;
					double DistanceToView = std::numeric_limits<double>::infinity();

					for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
					{
						const FViewInfo& View = *Views[ViewIndex];

						CurrentWorldTime = FMath::Max(CurrentWorldTime, View.Family->Time.GetWorldTimeSeconds());

						if (LastRenderTimeUpdateDistance > 0.0f)
						{
							DistanceToView = FMath::Min(DistanceToView, FVector::Distance(SceneProxy->GetActorPosition(), View.ViewMatrices.GetViewOrigin()));
						}
					}

					if (DistanceToView < LastRenderTimeUpdateDistance)
					{
						// Update LastRenderTime for components so that visibility based ticking (like skeletal meshes) can get updated
						// We are only doing this for dynamic geometries now
						SceneInfo->LastRenderTime = CurrentWorldTime;
						SceneInfo->UpdateComponentLastRenderTime(CurrentWorldTime, /*bUpdateLastRenderTimeOnScreen=*/true);
					}
				}
			}
		}
		
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			KickRayTracingMeshBatchTask(MeshBatchTaskData[ViewIndex], *Views[ViewIndex]);
		}

		RayTracingInstanceCollector.RayTracingInstances.Empty();
	}

	void GatherRelevantPrimitives(FGatherInstancesViewTaskData& TaskData, bool bUsingReferenceBasedResidency)
	{
		FScene& Scene = TaskData.Scene;
		FViewInfo& View = TaskData.View;

		const bool bGameView = View.bIsGameView || View.Family->EngineShowFlags.Game;
		const ERayTracingProxyType ActiveRayTracingProxyTypes = TaskData.ActiveRayTracingProxyTypes;

		const bool bPerformRayTracing = View.State != nullptr && !View.bIsReflectionCapture && View.IsRayTracingAllowedForView();

		if (bPerformRayTracing)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GatherRayTracingRelevantPrimitives);

			struct FGatherRelevantPrimitivesContext
			{
				TChunkedArray<int32> StaticPrimitives;
				TChunkedArray<int32> DynamicPrimitives;
				TChunkedArray<FPrimitiveSceneInfo*> DirtyCachedRayTracingPrimitives;

				TSet<RayTracing::FGeometryGroupHandle> ReferencedGeometryGroups;
			};

			TArray<FGatherRelevantPrimitivesContext> Contexts;
			const int32 MinBatchSize = 128;
			ParallelForWithTaskContext(
				TEXT("GatherRayTracingRelevantPrimitives_Parallel"),
				Contexts,
				Scene.PrimitiveSceneProxies.Num(),
				MinBatchSize,
				[&Scene, &View, bGameView, ActiveRayTracingProxyTypes, bUsingReferenceBasedResidency](FGatherRelevantPrimitivesContext& Context, int32 PrimitiveIndex)
			{
				// Get primitive visibility state from culling
				if (!View.PrimitiveRayTracingVisibilityMap[PrimitiveIndex])
				{
					return;
				}

				const ERayTracingPrimitiveFlags Flags = Scene.PrimitiveRayTracingFlags[PrimitiveIndex];
				const FScene::FPrimitiveRayTracingData& PrimitiveRayTracingData = Scene.PrimitiveRayTracingDatas[PrimitiveIndex];

				check(!EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::Exclude));

				// #dxr_todo: ray tracing in scene captures should re-use the persistent RT scene. (UE-112448)
				if (View.bIsSceneCapture && !PrimitiveRayTracingData.bIsVisibleInSceneCaptures && !PrimitiveRayTracingData.bRetainWhileHidden)
				{
					return;
				}

				if (!View.bIsSceneCapture && PrimitiveRayTracingData.bIsVisibleInSceneCapturesOnly)
				{
					return;
				}

				// Some primitives should only be visible editor mode, however far field geometry 
				// and geometry that retains visibility while hidden (affect indirect while hidden
				// or hidden shadow casters) must still always be added to the RT scene.
				if (bGameView && !PrimitiveRayTracingData.bDrawInGame && !PrimitiveRayTracingData.bRayTracingFarField && !PrimitiveRayTracingData.bRetainWhileHidden)
				{
					return;
				}

				// Check if certain ray tracing proxy types are excluded from the gather
				if (!EnumHasAllFlags(ActiveRayTracingProxyTypes, PrimitiveRayTracingData.ProxyGeometryType))
				{
					return;
				}

				if (bUsingReferenceBasedResidency && !PrimitiveRayTracingData.RayTracingGeometryGroupHandle.IsNull())
				{
					Context.ReferencedGeometryGroups.Add(PrimitiveRayTracingData.RayTracingGeometryGroupHandle);
				}

				// Is the cached data dirty?
				// eg: mesh was streamed in/out
				if (PrimitiveRayTracingData.bCachedRaytracingDataDirty)
				{
					Context.DirtyCachedRayTracingPrimitives.AddElement(Scene.Primitives[PrimitiveIndex]);
				}

				if (EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::Skip))
				{
					return;
				}

				if (EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::Dynamic))
				{
					checkf(!EnumHasAllFlags(Flags, ERayTracingPrimitiveFlags::CacheInstances), TEXT("Only static primitives are expected to use CacheInstances flag."));

					if (GRayTracingDynamicInstances)
					{
						Context.DynamicPrimitives.AddElement(PrimitiveIndex);
					}
				}
				else
				{
					Context.StaticPrimitives.AddElement(PrimitiveIndex);
				}
			}, GRayTracingParallelPrimitiveGather ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

			if (Contexts.Num() > 0)
			{
				SCOPED_NAMED_EVENT(GatherRayTracingRelevantPrimitives_Merge, FColor::Emerald);

				int32 NumStaticPrimitives = 0;
				int32 NumDynamicPrimitives = 0;
				int32 NumDirtyCachedRayTracingPrimitives = 0;

				for (auto& Context : Contexts)
				{
					NumStaticPrimitives += Context.StaticPrimitives.Num();
					NumDynamicPrimitives += Context.DynamicPrimitives.Num();					
					NumDirtyCachedRayTracingPrimitives += Context.DirtyCachedRayTracingPrimitives.Num();
				}

				TaskData.StaticPrimitivesIndices.Reserve(NumStaticPrimitives);
				TaskData.DynamicPrimitivesIndices.Reserve(NumDynamicPrimitives);				
				TaskData.DirtyCachedRayTracingPrimitives.Reserve(NumDirtyCachedRayTracingPrimitives);

				TaskData.ReferencedGeometryGroups.Reserve(Contexts.Num());

				for (auto& Context : Contexts)
				{
					Context.StaticPrimitives.CopyToLinearArray(TaskData.StaticPrimitivesIndices);
					Context.DynamicPrimitives.CopyToLinearArray(TaskData.DynamicPrimitivesIndices);
					Context.DirtyCachedRayTracingPrimitives.CopyToLinearArray(TaskData.DirtyCachedRayTracingPrimitives);

					TaskData.ReferencedGeometryGroups.Add(MoveTemp(Context.ReferencedGeometryGroups));
				}
			}
		}
	}

	void GatherRelevantStaticPrimitives(FGatherInstancesViewTaskData& TaskData, float GlobalLODScale, int32 ForcedLODLevel, bool bUsingReferenceBasedResidency)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GatherRayTracingRelevantStaticPrimitives);

		const bool bExcludeDecals = ShouldExcludeDecals();

		struct FRelevantStaticPrimitivesContext
		{
			TChunkedArray<FRelevantPrimitive> StaticPrimitives;
			TChunkedArray<FRelevantCachedPrimitive> CachedStaticPrimitives;
			TChunkedArray<const FPrimitiveSceneInfo*> VisibleNaniteRayTracingPrimitives;

			int32 NumCachedStaticVisibleShaderBindings = 0;

			uint8 LightingChannelAndMask = 0xFF;

#if DO_CHECK
			TSet<RayTracing::FGeometryGroupHandle> ReferencedGeometryGroupsToCheck;
#endif
		};

		TArray<FRelevantStaticPrimitivesContext> Contexts;
		ParallelForWithTaskContext(
			TEXT("GatherRayTracingRelevantStaticPrimitives_Parallel"),
			Contexts,
			TaskData.StaticPrimitivesIndices.Num(),
			/*MinBatchSize*/ 128,
			[&Scene = TaskData.Scene, &View = TaskData.View, GlobalLODScale, ForcedLODLevel, &StaticPrimitivesIndices = TaskData.StaticPrimitivesIndices,
			bUsingReferenceBasedResidency, bExcludeDecals](FRelevantStaticPrimitivesContext& Context, int32 ItemIndex)
			{
				const int32 PrimitiveIndex = StaticPrimitivesIndices[ItemIndex];

				const FPrimitiveSceneProxy* SceneProxy = Scene.PrimitiveSceneProxies[PrimitiveIndex];
				const FPrimitiveSceneInfo* SceneInfo = Scene.Primitives[PrimitiveIndex];
				const ERayTracingPrimitiveFlags Flags = Scene.PrimitiveRayTracingFlags[PrimitiveIndex];
				const FScene::FPrimitiveRayTracingData& PrimitiveRayTracingData = Scene.PrimitiveRayTracingDatas[PrimitiveIndex];

				ensureMsgf(!PrimitiveRayTracingData.bCachedRaytracingDataDirty, TEXT("Cached ray tracing instances must be up-to-date at this point"));

				const bool bUsingNaniteRayTracing = (Nanite::GetRayTracingMode() != Nanite::ERayTracingMode::Fallback) && SceneProxy->IsNaniteMesh();

				if (bUsingNaniteRayTracing)
				{
					Context.VisibleNaniteRayTracingPrimitives.AddElement(SceneInfo);
				}

				int8 LODIndex = 0;

				if (EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::ComputeLOD))
				{
					const FPrimitiveBounds& Bounds = Scene.PrimitiveBounds[PrimitiveIndex];

					const int8 CurFirstLODIdx = SceneProxy->GetCurrentFirstLODIdx_RenderThread();
					check(CurFirstLODIdx >= 0);

					float MeshScreenSizeSquared = 0;
					float LODScale = GlobalLODScale * View.LODDistanceFactor;
					FLODMask LODToRender = ComputeLODForMeshes(SceneInfo->StaticMeshRelevances, View, Bounds.BoxSphereBounds.Origin, Bounds.BoxSphereBounds.SphereRadius, ForcedLODLevel, MeshScreenSizeSquared, CurFirstLODIdx, LODScale, true);

					LODIndex = LODToRender.GetRayTracedLOD();

					// TODO: Handle !RayTracingProxy->bUsingRenderingLODs
				}

				if (EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::CacheInstances))
				{
					// For primitives with ERayTracingPrimitiveFlags::CacheInstances flag we only cache the instance/mesh commands of the current LOD
					// (see FPrimitiveSceneInfo::UpdateCachedRayTracingInstance(...) and CacheRayTracingMeshCommands(...))
					check(!EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::ComputeLOD));
					LODIndex = 0;

					if (bUsingNaniteRayTracing && !SceneInfo->bIsCachedRayTracingInstanceValid)
					{
						// Nanite ray tracing geometry not ready yet, doesn't include primitive in ray tracing scene
						return;
					}

					const FPrimitiveSceneInfo::FRayTracingLODData& RTLODData = SceneInfo->GetRayTracingLODData(LODIndex);

					for (int32 InstanceIndex = 0; InstanceIndex < RTLODData.Instances.Num(); ++InstanceIndex)
					{
						const FPrimitiveSceneInfo::FRayTracingInstanceData& RTInstanceData = RTLODData.Instances[InstanceIndex];

						if (!bUsingNaniteRayTracing)
						{
							// Currently IsCachedRayTracingGeometryValid() can only be called for non-nanite geometries
							checkf(SceneInfo->IsCachedRayTracingGeometryValid(LODIndex, InstanceIndex), TEXT("Cached ray tracing instance is expected to be valid. Was mesh LOD streamed but cached data was not invalidated?"));
						}

						checkf(SceneInfo->bIsCachedRayTracingInstanceValid, TEXT("Cached ray tracing instance must be valid."));

						if (bExcludeDecals && RTInstanceData.CachedMeshCommandFlags.bAllSegmentsDecal)
						{
							continue;
						}

						check(RTInstanceData.RayTracingInstanceIndexMain == UINT32_MAX || !RTInstanceData.CachedMeshCommandFlags.bAllSegmentsDecal);

						if (RTInstanceData.RayTracingInstanceIndexMain == UINT32_MAX && RTInstanceData.RayTracingInstanceIndexDecal == UINT32_MAX)
						{
							continue;
						}

						ensure(RTInstanceData.SBTAllocation);
						if (RTInstanceData.SBTAllocation == nullptr)
						{
							continue;
						}

						Context.LightingChannelAndMask &= PrimitiveRayTracingData.LightingChannelMask;

						// if primitive has mixed decal and non-decal segments we need to have two ray tracing instances
						// one containing non-decal segments and the other with decal segments
						// masking of segments is done using "hidden" hitgroups
						// TODO: Debug Visualization to highlight primitives using this?

						FRelevantCachedPrimitive* RelevantPrimitive = new (Context.CachedStaticPrimitives) FRelevantCachedPrimitive();
						RelevantPrimitive->CachedShaderBindingDataBase = RTInstanceData.CachedShaderBindingDataBase;
						RelevantPrimitive->CachedShaderBindingDataDecal = RTInstanceData.CachedShaderBindingDataDecal;
						RelevantPrimitive->SBTAllocationUniqueId = RTInstanceData.SBTAllocationUniqueId;
						RelevantPrimitive->MainRayTracingInstanceIndex = RTInstanceData.RayTracingInstanceIndexMain;
						RelevantPrimitive->DecalRayTracingInstanceIndex = RTInstanceData.RayTracingInstanceIndexDecal;
#if DO_CHECK
						if (!bUsingNaniteRayTracing)
						{
							// Currently RTInstanceData.RayTracingGeometry is only available for non-nanite geometries
							RelevantPrimitive->RayTracingGeometry = RTInstanceData.RayTracingGeometry->GetRHI();
						}
#endif

						const int32 NumBindings = RTInstanceData.CachedShaderBindingDataBase.Num() + RTInstanceData.CachedShaderBindingDataDecal.Num();

						Context.NumCachedStaticVisibleShaderBindings += NumBindings;
						checkSlow(NumBindings <= RTInstanceData.SBTAllocation->GetSegmentCount());
					}
				}
				// - DirtyCachedRayTracingPrimitives are only processed after StaticPrimitiveIndices is filled
				// so we can end up with primitives that should be skipped here
				// - once we update flags of primitive with dirty raytracing state before `GatherRayTracingRelevantPrimitives_Parallel`
				// we should replace this condition with an assert instead
				else if (!EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::Skip))
				{
					const uint32 NumRayTracingLODs = SceneInfo->GetRayTracingLODDataNum();

#if DO_CHECK
					// TODO: Should have an assert here but disabled it due to UE-309710
					if (bUsingReferenceBasedResidency && LODIndex < int8(NumRayTracingLODs))
					{
						const FPrimitiveSceneInfo::FRayTracingLODData& RTLODData = SceneInfo->GetRayTracingLODData(LODIndex);

						for (const FPrimitiveSceneInfo::FRayTracingInstanceData& RTInstanceData : RTLODData.Instances)
						{
							if (ensure(RTInstanceData.RayTracingGeometry != nullptr))
							{
								// It is not safe to directly call FRayTracingGeometryManager::IsGeometryGroupReferenced(...) here since other threads might be modifying it.
								// Instead we gather the group handles to validate later.
								Context.ReferencedGeometryGroupsToCheck.Add(RTInstanceData.RayTracingGeometry->GroupHandle);
							}
						}
					}
#endif

					// find highest LOD whose instances all have a valid RayTracingGeometry
					for (; LODIndex < int8(NumRayTracingLODs); ++LODIndex)
					{
						const FPrimitiveSceneInfo::FRayTracingLODData& RTLODData = SceneInfo->GetRayTracingLODData(LODIndex);

						bool bAllInstancesValid = true;
						for (const FPrimitiveSceneInfo::FRayTracingInstanceData& RTInstanceData : RTLODData.Instances)
						{
							const FRayTracingGeometry* RayTracingGeometry = RTInstanceData.RayTracingGeometry;

							if (RayTracingGeometry->HasPendingBuildRequest())
							{
								RayTracingGeometry->BoostBuildPriority();
								bAllInstancesValid = false;
								break;
							}
							else if (!RayTracingGeometry->IsValid() || RayTracingGeometry->IsEvicted())
							{
								bAllInstancesValid = false;
								break;
							}
						}

						if (bAllInstancesValid)
						{
							break;
						}
					}

					// Sometimes LODIndex is out of range because it is clamped by ClampToFirstLOD, like the requested LOD is being streamed in and hasn't been available
					// According to InitViews, we should hide the static mesh instance
					if (LODIndex >= int8(NumRayTracingLODs))
					{
						return;
					}

					const FPrimitiveSceneInfo::FRayTracingLODData& RTLODData = SceneInfo->GetRayTracingLODData(LODIndex);

					for (const FPrimitiveSceneInfo::FRayTracingInstanceData& RTInstanceData : RTLODData.Instances)
					{
						check(RTInstanceData.RayTracingGeometry->LODIndex == LODIndex);

						if (RTInstanceData.SBTAllocation == nullptr)
						{
							// No SBT allocation if no valid segments (see logic FRayTracingShaderBindingTable::AllocateStaticRange)
							ensure(RTInstanceData.CachedMeshCommandFlags.bAllSegmentsDecal && !(RTInstanceData.CachedMeshCommandFlags.bAnySegmentsDecal && !bExcludeDecals));
							continue;
						}

						Context.LightingChannelAndMask &= PrimitiveRayTracingData.LightingChannelMask;

						FRelevantPrimitive* RelevantPrimitive = new (Context.StaticPrimitives) FRelevantPrimitive();
						RelevantPrimitive->PrimitiveIndex = PrimitiveIndex;
						RelevantPrimitive->RayTracingGeometry = RTInstanceData.RayTracingGeometry;
						RelevantPrimitive->SBTAllocationUniqueId = RTInstanceData.SBTAllocationUniqueId;
						RelevantPrimitive->CachedMeshCommandFlags = RTInstanceData.CachedMeshCommandFlags;
						RelevantPrimitive->InstanceContributionToHitGroupIndexBase = RTInstanceData.InstanceContributionToHitGroupIndexBase;
						RelevantPrimitive->InstanceContributionToHitGroupIndexDecal = RTInstanceData.InstanceContributionToHitGroupIndexDecal;
						RelevantPrimitive->CachedShaderBindingDataBase = RTInstanceData.CachedShaderBindingDataBase;
						RelevantPrimitive->CachedShaderBindingDataDecal = RTInstanceData.CachedShaderBindingDataDecal;
						RelevantPrimitive->LightingChannelMask = PrimitiveRayTracingData.LightingChannelMask;
					}
				}
			}, GRayTracingParallelPrimitiveGather ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

		if (Contexts.Num() > 0)
		{
			SCOPED_NAMED_EVENT(GatherRayTracingRelevantStaticPrimitives_Merge, FColor::Emerald);

			uint32 NumStaticPrimitives = 0;
			uint32 NumCachedStaticPrimitives = 0;

			for (auto& Context : Contexts)
			{
				NumStaticPrimitives += Context.StaticPrimitives.Num();
				NumCachedStaticPrimitives += Context.CachedStaticPrimitives.Num();
			}

			TaskData.StaticPrimitives.Reserve(NumStaticPrimitives);
			TaskData.CachedStaticPrimitives.Reserve(NumCachedStaticPrimitives);

			for (int32 ContextIndex = 0; ContextIndex < Contexts.Num(); ++ContextIndex)
			{
				FRelevantStaticPrimitivesContext& Context = Contexts[ContextIndex];

				Context.StaticPrimitives.CopyToLinearArray(TaskData.StaticPrimitives);
				Context.CachedStaticPrimitives.CopyToLinearArray(TaskData.CachedStaticPrimitives);

				TaskData.NumCachedStaticVisibleShaderBindings += Context.NumCachedStaticVisibleShaderBindings;

				TaskData.LightingChannelAndMask &= Context.LightingChannelAndMask;

				for (const FPrimitiveSceneInfo* SceneInfo : Context.VisibleNaniteRayTracingPrimitives)
				{
					Nanite::GRayTracingManager.AddVisiblePrimitive(SceneInfo);
				}

#if DO_CHECK
				TaskData.ReferencedGeometryGroupsToCheck.Add(Context.ReferencedGeometryGroupsToCheck);
#endif
			}
		}
	}

	struct FAutoInstanceBatch
	{
		FRayTracingScene::FInstanceHandle InstanceHandle = FRayTracingScene::INVALID_INSTANCE_HANDLE;
		FRayTracingScene::FInstanceHandle DecalInstanceHandle = FRayTracingScene::INVALID_INSTANCE_HANDLE;

		// Copies the next InstanceSceneDataOffset and user data into the current batch, returns true if arrays were re-allocated.
		bool Add(FRayTracingScene& InRayTracingScene, uint32 InInstanceSceneDataOffset)
		{
			// Adhoc TArray-like resize behavior, in lieu of support for using a custom FMemStackBase in TArray.
			// Idea for future: if batch becomes large enough, we could actually split it into multiple instances to avoid memory waste.

			const bool bNeedReallocation = Cursor == InstanceSceneDataOffsets.Num();

			if (bNeedReallocation)
			{
				int32 PrevCount = InstanceSceneDataOffsets.Num();
				int32 NextCount = FMath::Max(PrevCount * 2, 1);

				TArrayView<uint32> NewInstanceSceneDataOffsets = InRayTracingScene.Allocate<uint32>(NextCount);
				if (PrevCount)
				{
					FMemory::Memcpy(NewInstanceSceneDataOffsets.GetData(), InstanceSceneDataOffsets.GetData(), InstanceSceneDataOffsets.GetTypeSize() * InstanceSceneDataOffsets.Num());
				}
				InstanceSceneDataOffsets = NewInstanceSceneDataOffsets;
			}

			InstanceSceneDataOffsets[Cursor] = InInstanceSceneDataOffset;

			++Cursor;

			return bNeedReallocation;
		}

		bool IsValid() const
		{
			return InstanceSceneDataOffsets.Num() != 0;
		}

		TArrayView<uint32> InstanceSceneDataOffsets;
		uint32 Cursor = 0;
	}; 
	
	void AddStaticInstancesToRayTracingScene(
		const FScene& Scene,
		const FViewInfo& View,
		const RayTracing::FSceneOptions& SceneOptions,
		TConstArrayView<FRelevantPrimitive> RelevantStaticPrimitives,
		FRayTracingScene& RayTracingScene,
		FRayTracingShaderBindingDataOneFrameArray& VisibleShaderBindingData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracingScene_AddStaticInstances);

		VisibleShaderBindingData.Reserve(VisibleShaderBindingData.Num() + VisibleShaderBindingData.Num());

		const bool bAutoInstance = CVarRayTracingAutoInstance.GetValueOnRenderThread() != 0;

		// Instance batches by FRelevantPrimitive::InstancingKey()
		Experimental::TSherwoodMap<uint64, FAutoInstanceBatch> InstanceBatches;

		// scan relevant primitives computing hash data to look for duplicate instances
		for (const FRelevantPrimitive& RelevantPrimitive : RelevantStaticPrimitives)
		{
			const int32 PrimitiveIndex = RelevantPrimitive.PrimitiveIndex;
			FPrimitiveSceneInfo* SceneInfo = Scene.Primitives[PrimitiveIndex];
			FPrimitiveSceneProxy* SceneProxy = Scene.PrimitiveSceneProxies[PrimitiveIndex];
			ERayTracingPrimitiveFlags Flags = Scene.PrimitiveRayTracingFlags[PrimitiveIndex];

			check(!EnumHasAnyFlags(Flags, ERayTracingPrimitiveFlags::CacheInstances));

			const bool bNeedMainInstance = !RelevantPrimitive.CachedMeshCommandFlags.bAllSegmentsDecal;

			// if primitive has mixed decal and non-decal segments we need to have two ray tracing instances
			// one containing non-decal segments and the other with decal segments
			// masking of segments is done using "hidden" hitgroups
			// TODO: Debug Visualization to highlight primitives using this?
			const bool bNeedDecalInstance = RelevantPrimitive.CachedMeshCommandFlags.bAnySegmentsDecal && !ShouldExcludeDecals();

			// skip if not needed for main or decal - default values for bAllSegmentsDecal is true because it's updated with & op for added cached segments
			// but if there are no cached command indices then default value of true is kept but bAnySegmentsDecal will false as well then.
			if (!bNeedMainInstance && !bNeedDecalInstance)
			{
				continue;
			}

			if (ShouldExcludeDecals() && RelevantPrimitive.CachedMeshCommandFlags.bAllSegmentsDecal)
			{
				continue;
			}

			if (!SceneOptions.bTranslucentGeometry && RelevantPrimitive.CachedMeshCommandFlags.bAllSegmentsTranslucent)
			{
				continue;
			}

			if (!SceneOptions.bIncludeSky && RelevantPrimitive.CachedMeshCommandFlags.bIsSky)
			{
				continue;
			}

			const int32 NumInstances = SceneInfo->GetNumInstanceSceneDataEntries();

			// location if this is a new entry
			const uint64 InstanceKey = RelevantPrimitive.InstancingKey();

			FAutoInstanceBatch DummyInstanceBatch = { };
			FAutoInstanceBatch& InstanceBatch = bAutoInstance && (NumInstances == 1) ? InstanceBatches.FindOrAdd(InstanceKey, DummyInstanceBatch) : DummyInstanceBatch;

			if (InstanceBatch.IsValid())
			{
				// Reusing a previous entry, just append to the instance list.

				bool bReallocated = InstanceBatch.Add(RayTracingScene, SceneInfo->GetInstanceSceneDataOffset());

				if (InstanceBatch.InstanceHandle.IsValid())
				{
					FRayTracingGeometryInstance& RayTracingInstance = RayTracingScene.GetInstance(InstanceBatch.InstanceHandle);
					++RayTracingInstance.NumTransforms;
					check(RayTracingInstance.NumTransforms == InstanceBatch.Cursor); // sanity check

					if (bReallocated)
					{
						RayTracingInstance.InstanceSceneDataOffsets = InstanceBatch.InstanceSceneDataOffsets;
						RayTracingInstance.UserData = InstanceBatch.InstanceSceneDataOffsets;
					}
				}

				if (InstanceBatch.DecalInstanceHandle.IsValid())
				{
					FRayTracingGeometryInstance& RayTracingInstance = RayTracingScene.GetInstance(InstanceBatch.DecalInstanceHandle);
					++RayTracingInstance.NumTransforms;
					check(RayTracingInstance.NumTransforms == InstanceBatch.Cursor); // sanity check

					if (bReallocated)
					{
						RayTracingInstance.InstanceSceneDataOffsets = InstanceBatch.InstanceSceneDataOffsets;
						RayTracingInstance.UserData = InstanceBatch.InstanceSceneDataOffsets;
					}
				}
			}
			else
			{
				// Starting new instance batch

				const int32 InstanceSceneDataOffset = SceneInfo->GetInstanceSceneDataOffset();

				InstanceBatch.Add(RayTracingScene, InstanceSceneDataOffset);

				FRayTracingGeometryInstance RayTracingInstance;
				RayTracingInstance.GeometryRHI = RelevantPrimitive.RayTracingGeometry->GetRHI();
				checkf(RayTracingInstance.GeometryRHI, TEXT("Ray tracing instance must have a valid geometry."));
				RayTracingInstance.LightingChannelMask = RelevantPrimitive.LightingChannelMask;

				if (NumInstances == 1)
				{
					RayTracingInstance.NumTransforms = 1;
					RayTracingInstance.InstanceSceneDataOffsets = InstanceBatch.InstanceSceneDataOffsets;
					RayTracingInstance.UserData = InstanceBatch.InstanceSceneDataOffsets;
				}
				else
				{
					RayTracingInstance.NumTransforms = NumInstances;
					RayTracingInstance.BaseInstanceSceneDataOffset = InstanceSceneDataOffset;
					RayTracingInstance.DefaultUserData = InstanceSceneDataOffset;
					RayTracingInstance.bIncrementUserDataPerInstance = true;
				}

				RayTracingInstance.Mask = RelevantPrimitive.CachedMeshCommandFlags.InstanceMask; // When no cached command is found, InstanceMask == 0 and the instance is effectively filtered out

				// Run AHS for alpha masked and meshes with only some sections casting shadows, which require per mesh section filtering in AHS
				if (RelevantPrimitive.CachedMeshCommandFlags.bAllSegmentsOpaque && (RelevantPrimitive.CachedMeshCommandFlags.bAllSegmentsCastShadow || !RelevantPrimitive.CachedMeshCommandFlags.bAnySegmentsCastShadow))
				{
					RayTracingInstance.Flags |= ERayTracingInstanceFlags::ForceOpaque;
				}
				if (RelevantPrimitive.CachedMeshCommandFlags.bTwoSided)
				{
					RayTracingInstance.Flags |= ERayTracingInstanceFlags::TriangleCullDisable;
				}
				if (RelevantPrimitive.CachedMeshCommandFlags.bAllSegmentsReverseCulling)
				{
					RayTracingInstance.Flags |= ERayTracingInstanceFlags::TriangleCullReverse;
				}

				InstanceBatch.InstanceHandle = FRayTracingScene::INVALID_INSTANCE_HANDLE;

				if (bNeedMainInstance)
				{
					RayTracingInstance.InstanceContributionToHitGroupIndex = RelevantPrimitive.InstanceContributionToHitGroupIndexBase;

					const ERayTracingSceneLayer Layer = EnumHasAllFlags(Flags, ERayTracingPrimitiveFlags::FarField) ? ERayTracingSceneLayer::FarField : ERayTracingSceneLayer::Base;
					InstanceBatch.InstanceHandle = RayTracingScene.AddTransientInstance(
						RayTracingInstance,
						Layer,
						View.GetRayTracingSceneViewHandle(),
						SceneProxy,
						/*bDynamic*/ false,
						RelevantPrimitive.RayTracingGeometry->GetGeometryHandle());

					VisibleShaderBindingData.Append(RelevantPrimitive.CachedShaderBindingDataBase);
				}

				InstanceBatch.DecalInstanceHandle = FRayTracingScene::INVALID_INSTANCE_HANDLE;
				if (bNeedDecalInstance)
				{
					FRayTracingGeometryInstance DecalRayTracingInstance = RayTracingInstance;
					DecalRayTracingInstance.InstanceContributionToHitGroupIndex = RelevantPrimitive.InstanceContributionToHitGroupIndexDecal;

					InstanceBatch.DecalInstanceHandle = RayTracingScene.AddTransientInstance(
						MoveTemp(DecalRayTracingInstance),
						ERayTracingSceneLayer::Decals,
						View.GetRayTracingSceneViewHandle(),
						SceneProxy,
						/*bDynamic*/ false,
						RelevantPrimitive.RayTracingGeometry->GetGeometryHandle());

					VisibleShaderBindingData.Append(RelevantPrimitive.CachedShaderBindingDataDecal);
				}
			}
		}
	}

	void AddVisibleCachedInstances(
		const FViewInfo& View,
		FRayTracingScene& RayTracingScene,
		FRayTracingShaderBindingTable& RayTracingSBT,
		TConstArrayView<FRelevantCachedPrimitive> RelevantCachedPrimitives,
		FRayTracingShaderBindingDataOneFrameArray& VisibleShaderBindingData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracing_AddVisibleCachedInstances);

		const bool bExcludeDecals = ShouldExcludeDecals();

		TBitArray<> ProcessedSBTAllocations(false, RayTracingSBT.GetNumGeometrySegments() * RAY_TRACING_NUM_SHADER_SLOTS);
		for (const FRelevantCachedPrimitive& RelevantPrimitive : RelevantCachedPrimitives)
		{
			// Need to call MarkInstanceVisible from single threaded loop
			// to avoid race conditions with multiple threads trying to change bits on the same word

			if (RelevantPrimitive.MainRayTracingInstanceIndex != UINT32_MAX)
			{
#if DO_CHECK
				check(RelevantPrimitive.RayTracingGeometry == nullptr || RayTracingScene.GetCachedInstanceGeometry(RelevantPrimitive.MainRayTracingInstanceIndex) == RelevantPrimitive.RayTracingGeometry);
#endif
				RayTracingScene.MarkInstanceVisible(RelevantPrimitive.MainRayTracingInstanceIndex, View.GetRayTracingSceneViewHandle());
			}

			if (RelevantPrimitive.DecalRayTracingInstanceIndex != UINT32_MAX && !bExcludeDecals)
			{
#if DO_CHECK
				check(RelevantPrimitive.RayTracingGeometry == nullptr || RayTracingScene.GetCachedInstanceGeometry(RelevantPrimitive.DecalRayTracingInstanceIndex) == RelevantPrimitive.RayTracingGeometry);
#endif
				RayTracingScene.MarkInstanceVisible(RelevantPrimitive.DecalRayTracingInstanceIndex, View.GetRayTracingSceneViewHandle());
			}

			FBitReference BitReference = ProcessedSBTAllocations[RelevantPrimitive.SBTAllocationUniqueId];
			if (BitReference)
			{
				continue;
			}
			BitReference = true;

			VisibleShaderBindingData.Append(RelevantPrimitive.CachedShaderBindingDataBase);

			if (!bExcludeDecals)
			{
				VisibleShaderBindingData.Append(RelevantPrimitive.CachedShaderBindingDataDecal);
			}
		}
	}

	void AddDynamicInstancesToRayTracingScene(
		FGatherInstancesTaskData& TaskData,
		FRayTracingScene& RayTracingScene,
		FRayTracingShaderBindingTable& RayTracingSBT,
		int64 SharedBufferGenerationID)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracingScene_AddDynamicInstances);

		for (FDynamicRayTracingInstancesContext* DynamicRayTracingInstancesContext : TaskData.DynamicRayTracingInstancesContexts)
		{
			DynamicRayTracingInstancesContext->AddInstancesToScene(RayTracingScene, RayTracingSBT, SharedBufferGenerationID);
		}
	}

	void BeginGatherInstances(FGatherInstancesTaskData& TaskData, UE::Tasks::FTask FrustumCullTask)
	{
		const bool bUsingReferenceBasedResidency = IsRayTracingUsingReferenceBasedResidency();

		static const auto ICVarStaticMeshLODDistanceScale = IConsoleManager::Get().FindConsoleVariable(TEXT("r.StaticMeshLODDistanceScale"));
		const float LODScaleCVarValue = ICVarStaticMeshLODDistanceScale->GetFloat();
		const int32 ForcedLODLevel = GetCVarForceLOD();

		const bool bMultiView = TaskData.Views.Num() > 1;

		// When there are multiple views, we use a set to avoid duplicate updates of cached ray tracing primitives
		TSet<FPrimitiveSceneInfo*>* UpdatedDirtyCachedRayTracingPrimitives = bMultiView ? TaskData.Allocator.Create<TSet<FPrimitiveSceneInfo*>>() : nullptr;

		// Use high priority tasks to reduce stalls on the critical path.
		const UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High;

		UE::Tasks::FTask CacheRayTracingPrimitivesTask = TaskData.Scene.GetCacheRayTracingPrimitivesTask();

		TArray<UE::Tasks::FTask> GatherRelevantPrimitivesTasks;

		for (FGatherInstancesViewTaskData& ViewTaskData : TaskData.ViewTaskDatas)
		{
			ViewTaskData.GatherRelevantPrimitivesTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&ViewTaskData, bUsingReferenceBasedResidency]
				{
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
					GatherRelevantPrimitives(ViewTaskData, bUsingReferenceBasedResidency);
				}, MakeArrayView({ CacheRayTracingPrimitivesTask, FrustumCullTask }), TaskPriority);

			// Finalize logic can't run in parallel so a pipe is used to serialize work.
			ViewTaskData.FinalizeGatherRelevantPrimitivesTask = TaskData.FinalizeGatherRelevantPrimitivesPipe.Launch(UE_SOURCE_LOCATION,
				[&ViewTaskData, UpdatedDirtyCachedRayTracingPrimitives, bMultiView]
				{
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
					TRACE_CPUPROFILER_EVENT_SCOPE(FinalizeGatherRelevantPrimitives);

					for (const TSet<RayTracing::FGeometryGroupHandle>& ReferencedGeometryGroups : ViewTaskData.ReferencedGeometryGroups)
					{
						((FRayTracingGeometryManager*)GRayTracingGeometryManager)->AddReferencedGeometryGroups(ReferencedGeometryGroups);
					}

					if (bMultiView)
					{
						if (UpdatedDirtyCachedRayTracingPrimitives->IsEmpty())
						{
							// First view simply add DirtyCachedRayTracingPrimitives
							UpdatedDirtyCachedRayTracingPrimitives->Append(ViewTaskData.DirtyCachedRayTracingPrimitives);
						}
						else
						{
							// Then need to filter DirtyCachedRayTracingPrimitives to avoid duplicate updates
							TArray<FPrimitiveSceneInfo*> DirtyCachedRayTracingPrimitives = MoveTemp(ViewTaskData.DirtyCachedRayTracingPrimitives);
							ViewTaskData.DirtyCachedRayTracingPrimitives.Empty(DirtyCachedRayTracingPrimitives.Num());

							for (FPrimitiveSceneInfo* SceneInfo : DirtyCachedRayTracingPrimitives)
							{
								if (!UpdatedDirtyCachedRayTracingPrimitives->Contains(SceneInfo))
								{
									UpdatedDirtyCachedRayTracingPrimitives->Add(SceneInfo);

									ViewTaskData.DirtyCachedRayTracingPrimitives.Add(SceneInfo);
								}
							}
						}
					}

					FPrimitiveSceneInfo::UpdateCachedRaytracingData(&ViewTaskData.Scene, ViewTaskData.DirtyCachedRayTracingPrimitives);
				}, ViewTaskData.GatherRelevantPrimitivesTask);

			ViewTaskData.GatherRelevantStaticPrimitivesTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&ViewTaskData, LODScaleCVarValue, ForcedLODLevel, bUsingReferenceBasedResidency]
				{
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
					GatherRelevantStaticPrimitives(ViewTaskData, LODScaleCVarValue, ForcedLODLevel, bUsingReferenceBasedResidency);
				}, ViewTaskData.FinalizeGatherRelevantPrimitivesTask, TaskPriority);

			TaskData.GatherDynamicRayTracingInstancesPrerequisites.AddPrerequisites(ViewTaskData.GatherRelevantPrimitivesTask);

			GatherRelevantPrimitivesTasks.Add(ViewTaskData.GatherRelevantPrimitivesTask);
		}

		// Dynamic instance gathering
		{
			const int32 NumContexts = GetNumDynamicRayTracingInstancesContexts();
			TaskData.DynamicRayTracingInstancesContexts.Reserve(NumContexts);

			for (int32 Index = 0; Index < NumContexts; ++Index)
			{
				TaskData.DynamicRayTracingInstancesContexts.Add(TaskData.Allocator.Create<FDynamicRayTracingInstancesContext>(TaskData.Scene, TaskData.Views, TaskData.ViewSceneOptions, TaskData.Allocator));
			}

			TaskData.GatherDynamicRayTracingInstancesPrerequisites.AddPrerequisites(TaskData.Scene.GetGPUSkinCacheTask());

			TaskData.GatherDynamicRayTracingInstancesTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [&TaskData]
				{
					FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

					// Build array of FDynamicPrimitiveIndex (includes ViewMasks for each dynamic primitive) by merging ViewTaskData.DynamicPrimitivesIndices.
					//	Alternatively ViewMasks array could be built during GatherRelevantPrimitives(...) with atomic OR operations,
					//	and then here we could just loop over all primitives to collect the relevant dynamic ones.
					//	(loop over all primitives vs loop over dynamic primitives for each view and combine using map + array)

					TArray<FDynamicPrimitiveIndex, SceneRenderingAllocator> DynamicPrimitivesIndices;
					TMap<int32, int32> PrimitiveIndexMap;

					for (int32 ViewIndex = 0; ViewIndex < TaskData.ViewTaskDatas.Num(); ++ViewIndex)
					{
						const FGatherInstancesViewTaskData& ViewTaskData = TaskData.ViewTaskDatas[ViewIndex];

						DynamicPrimitivesIndices.Reserve(DynamicPrimitivesIndices.Num() + ViewTaskData.DynamicPrimitivesIndices.Num());

						for (int32 PrimitiveIndex : ViewTaskData.DynamicPrimitivesIndices)
						{
							if (PrimitiveIndexMap.Contains(PrimitiveIndex))
							{
								const int32 DynamicPrimitiveIndex = PrimitiveIndexMap[PrimitiveIndex];
								DynamicPrimitivesIndices[DynamicPrimitiveIndex].ViewMask |= (1 << ViewIndex);

								check(DynamicPrimitivesIndices[DynamicPrimitiveIndex].Index == PrimitiveIndex);
							}
							else
							{
								const int32 DynamicPrimitiveIndex = DynamicPrimitivesIndices.Emplace(PrimitiveIndex, (1 << ViewIndex));
								PrimitiveIndexMap.Add(PrimitiveIndex, DynamicPrimitiveIndex);
							}
						}
					}

					if (IsParallelGatherDynamicRayTracingInstancesEnabled())
					{
						FDynamicPrimitiveIndexQueue* Queue = TaskData.Allocator.Create<FDynamicPrimitiveIndexQueue>(MoveTemp(DynamicPrimitivesIndices));

						for (FDynamicRayTracingInstancesContext* DynamicRayTracingInstancesContext : TaskData.DynamicRayTracingInstancesContexts)
						{
							UE::Tasks::FTask ContextTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [DynamicRayTracingInstancesContext, Queue]
								{
									FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
									DynamicRayTracingInstancesContext->GatherDynamicRayTracingInstances(Queue);
								});

							UE::Tasks::AddNested(ContextTask);
						}
					}
					else
					{
						TaskData.DynamicRayTracingInstancesContexts.Last()->SetRenderThreadDynamicPrimitives(MoveTemp(DynamicPrimitivesIndices));
					}
				}, TaskData.GatherDynamicRayTracingInstancesPrerequisites, TaskPriority);
		}

		TaskData.bValid = true;
	}

	void BeginGatherDynamicRayTracingInstances(FGatherInstancesTaskData& TaskData)
	{
		if (!TaskData.bGatherDynamicRayTracingInstancesPrerequisitesTriggered)
		{
			TaskData.GatherDynamicRayTracingInstancesPrerequisites.Trigger();
			TaskData.bGatherDynamicRayTracingInstancesPrerequisitesTriggered = true;
		}
	}

	bool FinishGatherInstances(
		FRDGBuilder& GraphBuilder,
		FGatherInstancesTaskData& TaskData,
		FRayTracingScene& RayTracingScene,
		FRayTracingShaderBindingTable& RayTracingSBT)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracing_FinishGatherInstances);
		SCOPE_CYCLE_COUNTER(STAT_RayTracing_FinishGatherInstances);
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, RayTracing_FinishGatherInstances);

		for (FGatherInstancesViewTaskData& ViewTaskData : TaskData.ViewTaskDatas)
		{
			ViewTaskData.FinalizeGatherRelevantPrimitivesTask.Wait();

			INC_DWORD_STAT_BY(STAT_VisibleRayTracingPrimitives, ViewTaskData.StaticPrimitives.Num() + ViewTaskData.CachedStaticPrimitives.Num() + ViewTaskData.DynamicPrimitivesIndices.Num());
		}

		// Prepare ray tracing scene instance list
		checkf(TaskData.bValid, TEXT("Ray tracing relevant primitive list is expected to have been created before GatherRayTracingWorldInstancesForView() is called."));

		// Check that any invalidated cached uniform expressions have been updated on the rendering thread.
		// Normally this work is done through FMaterialRenderProxy::UpdateUniformExpressionCacheIfNeeded,
		// however ray tracing material processing (FMaterialShader::GetShaderBindings, which accesses UniformExpressionCache)
		// is done on task threads, therefore all work must be done here up-front as UpdateUniformExpressionCacheIfNeeded is not free-threaded.
		check(!FMaterialRenderProxy::HasDeferredUniformExpressionCacheRequests());

		RayTracingSBT.ResetDynamicAllocationData();
		RayTracingScene.LockCachedInstances();

		FRayTracingDynamicGeometryUpdateManager* DynamicGeometryUpdateManager = TaskData.Scene.GetRayTracingDynamicGeometryUpdateManager();
		const int64 SharedBufferGenerationID = DynamicGeometryUpdateManager->BeginUpdate();

		{
			TaskData.GatherDynamicRayTracingInstancesTask.Wait();

			for (FDynamicRayTracingInstancesContext* DynamicRayTracingInstancesContext : TaskData.DynamicRayTracingInstancesContexts)
			{
				DynamicRayTracingInstancesContext->GatherDynamicRayTracingInstances_RenderThread();
				DynamicRayTracingInstancesContext->Finish(GraphBuilder.RHICmdList);
			}

			FSceneRenderer::DynamicReadBufferForRayTracing.Commit(GraphBuilder.RHICmdList);

			for (FGatherInstancesViewTaskData& ViewTaskData : TaskData.ViewTaskDatas)
			{
				ViewTaskData.Scene.GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, ViewTaskData.View, /*bRayTracing*/ true);
			}

			for (FDynamicRayTracingInstancesContext* DynamicRayTracingInstancesContext : TaskData.DynamicRayTracingInstancesContexts)
			{
				DynamicRayTracingInstancesContext->CollectRDGResources(GraphBuilder);
			}
		}

		DynamicGeometryUpdateManager->ScheduleUpdates(GraphBuilder, RayTracingScene.IsRayTracingFeedbackEnabled());

		bool bAnyViewLightingChannelsUsingAHS = false;

		UE::Tasks::FTaskEvent AddInstancesTaskEvent{ UE_SOURCE_LOCATION };

		// This adds final dynamic instances to the ray tracing scene and must be done before FRayTracingScene::BuildInitializationData().
		TaskData.AddDynamicInstancesTask = TaskData.AddInstancesPipe.Launch(UE_SOURCE_LOCATION,
			[&TaskData, &RayTracingScene, &RayTracingSBT, SharedBufferGenerationID]
			{
				FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);

				AddDynamicInstancesToRayTracingScene(TaskData, RayTracingScene, RayTracingSBT, SharedBufferGenerationID);
			});

		AddInstancesTaskEvent.AddPrerequisites(TaskData.AddDynamicInstancesTask);

		for (FGatherInstancesViewTaskData& ViewTaskData : TaskData.ViewTaskDatas)
		{
			// This adds final instances to the ray tracing scene and must be done before FRayTracingScene::BuildInitializationData().
			AddInstancesTaskEvent.AddPrerequisites(TaskData.AddInstancesPipe.Launch(UE_SOURCE_LOCATION,
				[&ViewTaskData, &RayTracingScene, &RayTracingSBT, SharedBufferGenerationID]
				{
					FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);

					if (ViewTaskData.SceneOptions.bLightingChannelsUsingAHS)
					{
						RayTracingScene.LightingChannelAndMask &= ViewTaskData.LightingChannelAndMask;
					}

					ViewTaskData.View.VisibleRayTracingShaderBindings.Reserve(ViewTaskData.StaticPrimitives.Num() + ViewTaskData.NumCachedStaticVisibleShaderBindings);

					AddStaticInstancesToRayTracingScene(
						ViewTaskData.Scene,
						ViewTaskData.View,
						ViewTaskData.SceneOptions,
						ViewTaskData.StaticPrimitives,
						RayTracingScene,
						ViewTaskData.View.VisibleRayTracingShaderBindings);

					AddVisibleCachedInstances(
						ViewTaskData.View,
						ViewTaskData.Scene.RayTracingScene,
						ViewTaskData.Scene.RayTracingSBT,
						ViewTaskData.CachedStaticPrimitives,
						ViewTaskData.View.VisibleRayTracingShaderBindings);

#if DO_CHECK
					for(TSet<RayTracing::FGeometryGroupHandle> GeometryGroupHandles : ViewTaskData.ReferencedGeometryGroupsToCheck)
					{
						for (RayTracing::FGeometryGroupHandle GeometryGroupHandle : GeometryGroupHandles)
						{
							ensure(((FRayTracingGeometryManager*)GRayTracingGeometryManager)->IsGeometryGroupReferenced(GeometryGroupHandle));
						}
					}
#endif
				}, ViewTaskData.GatherRelevantStaticPrimitivesTask));

			bAnyViewLightingChannelsUsingAHS |= ViewTaskData.SceneOptions.bLightingChannelsUsingAHS;
		}

		AddInstancesTaskEvent.Trigger();

		// Scene init task can run only when all pre-init tasks are complete (including culling tasks that are spawned while adding instances)
		RayTracingScene.InitTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
			[&RayTracingScene, bLightingChannelsUsingAHS = bAnyViewLightingChannelsUsingAHS]()
			{
				FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);

				TRACE_CPUPROFILER_EVENT_SCOPE(RayTracingSceneInitTask);
				RayTracingScene.BuildInitializationData(bLightingChannelsUsingAHS, GRayTracingDebugForceOpaque != 0, GRayTracingDebugDisableTriangleCull != 0);
			}, AddInstancesTaskEvent);

		// Finalizing VisibleRayTracingShaderBindings can run only when all pre-init tasks are complete (including culling tasks that are spawned while adding instances)
		TaskData.VisibleRayTracingShaderBindingsFinalizeTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
			[&TaskData, &RayTracingSBT]
			{
				FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(DeduplicateVisibleShaderBindings);

					int32 TotalNumBindings = 0;

					for (const FGatherInstancesViewTaskData& ViewTaskData : TaskData.ViewTaskDatas)
					{
						TotalNumBindings += ViewTaskData.View.VisibleRayTracingShaderBindings.Num();
					}

					// Deduplicate all the written SBT record indices by using bit array and checking the written indices into the SBT table
					TBitArray<> ProcessedSBTAllocations(false, RayTracingSBT.GetNumGeometrySegments() * RAY_TRACING_NUM_SHADER_SLOTS);
					TArray<FRayTracingShaderBindingData> DeduplicatedVisibleShaderBindingData;
					DeduplicatedVisibleShaderBindingData.Reserve(TotalNumBindings);

					for (const FGatherInstancesViewTaskData& ViewTaskData : TaskData.ViewTaskDatas)
					{
						for (FRayTracingShaderBindingData& VisibleBinding : ViewTaskData.View.VisibleRayTracingShaderBindings)
						{
							FBitReference BitReference = ProcessedSBTAllocations[VisibleBinding.SBTRecordIndex];
							if (!BitReference)
							{
								BitReference = true;
								DeduplicatedVisibleShaderBindingData.Add(VisibleBinding);
							}
						}
					}
					TaskData.VisibleShaderBindings = MoveTemp(DeduplicatedVisibleShaderBindingData);
				}
			}, AddInstancesTaskEvent);

		return true;
	}

	void WaitForDynamicBindings(FGatherInstancesTaskData& TaskData)
	{
		for (FViewInfo* View : TaskData.Views)
		{
			UE::Tasks::Wait(View->AddDynamicRayTracingMeshBatchTaskList);
			View->AddDynamicRayTracingMeshBatchTaskList.Empty();
		}

		// Wait for AddDynamicInstancesToRayTracingScene to complete since
		// it also allocates dynamic ranges in the SBT (increasing MaxNumDynamicGeometrySegments).
		TaskData.AddDynamicInstancesTask.Wait();
	}

	bool FinishGatherVisibleShaderBindings(FGatherInstancesTaskData& TaskData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RayTracing_FinishGatherVisibleShaderBindings);

		TaskData.VisibleRayTracingShaderBindingsFinalizeTask.Wait();

		// calculate total number of visible shader bindings
		int32 NumVisibleShaderBindings = TaskData.VisibleShaderBindings.Num();

		for (FGatherInstancesViewTaskData& ViewTaskData : TaskData.ViewTaskDatas)
		{
			for (int32 TaskIndex = 0; TaskIndex < ViewTaskData.View.DynamicRayTracingShaderBindingsPerTask.Num(); TaskIndex++)
			{
				NumVisibleShaderBindings += ViewTaskData.View.DynamicRayTracingShaderBindingsPerTask[TaskIndex]->Num();
			}
		}

		// reserve enough space to avoid multiple realloc
		TaskData.VisibleShaderBindings.Reserve(NumVisibleShaderBindings);

		// actually merge dynamic bindings into the main array
		for (FGatherInstancesViewTaskData& ViewTaskData : TaskData.ViewTaskDatas)
		{
			for (int32 TaskIndex = 0; TaskIndex < ViewTaskData.View.DynamicRayTracingShaderBindingsPerTask.Num(); TaskIndex++)
			{
				TaskData.VisibleShaderBindings.Append(*ViewTaskData.View.DynamicRayTracingShaderBindingsPerTask[TaskIndex]);
			}

			ViewTaskData.View.DynamicRayTracingShaderBindingsPerTask.Empty();
		}

		// Even though task dependencies are setup so all work is done by this point, we still have to wait on the
		// pipe to clear out its internal state. Otherwise it can assert that it still has work at shutdown.
		TaskData.AddInstancesPipe.WaitUntilEmpty();
		TaskData.FinalizeGatherRelevantPrimitivesPipe.WaitUntilEmpty();

		return true;
	}

	TConstArrayView<FRayTracingShaderBindingData> GetVisibleShaderBindings(FGatherInstancesTaskData& TaskData)
	{
		return TaskData.VisibleShaderBindings;
	}

	bool ShouldExcludeDecals()
	{
		return GRayTracingExcludeDecals != 0;
	}
}

static_assert(std::is_trivially_destructible_v<RayTracing::FRelevantPrimitive>, "FRelevantPrimitive must be trivially destructible");
template <> struct TIsPODType<RayTracing::FRelevantPrimitive> { enum { Value = true }; }; // Necessary to use TChunkedArray::CopyToLinearArray

#endif //RHI_RAYTRACING
 //RHI_RAYTRACING
