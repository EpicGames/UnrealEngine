// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RendererInterface.h"
#include "RenderGraphDefinitions.h"
#include "RayTracingDefinitions.h"
#include "RayTracingShaderBindingLayout.h"
#include "RHIDefinitions.h"
#include "ShaderCore.h"

enum class EDiffuseIndirectMethod;
enum class EReflectionsMethod;
class FRayTracingScene;
class FRayTracingShaderBindingTable;
class FRayTracingShaderBindingData;
class FScene;
class FViewInfo;
class FViewFamilyInfo;
class FGlobalDynamicReadBuffer;
struct FSceneRenderUpdateInputs;

// Settings controlling ray tracing instance caching
namespace RayTracing
{
	struct FSceneOptions
	{
		bool bTranslucentGeometry = true;
		bool bIncludeSky = true;
		bool bLightingChannelsUsingAHS = true;

		FSceneOptions(
			const FScene& Scene,
			const FViewFamilyInfo& ViewFamily,
			const FViewInfo& View,
			EDiffuseIndirectMethod DiffuseIndirectMethod,
			EReflectionsMethod ReflectionsMethod);
	};
};

#if RHI_RAYTRACING

namespace RayTracing
{
	struct FGatherInstancesTaskData;

	void OnRenderBegin(const FSceneRenderUpdateInputs& SceneUpdateInputs);

	FGatherInstancesTaskData* CreateGatherInstancesTaskData(
		FSceneRenderingBulkObjectAllocator& InAllocator,
		FScene& Scene,
		uint32 NumViews);

	void AddView(
		FGatherInstancesTaskData& TaskData,
		FViewInfo& View,
		EDiffuseIndirectMethod DiffuseIndirectMethod,
		EReflectionsMethod ReflectionsMethod);
	
	void BeginGatherInstances(FGatherInstancesTaskData& TaskData, UE::Tasks::FTask FrustumCullTask);

	void BeginGatherDynamicRayTracingInstances(FGatherInstancesTaskData& TaskData);

	// Fills RayTracingScene instance list for the given View and adds relevant ray tracing data to the view. Does not reset previous scene contents.
	// This function must run on render thread
	bool FinishGatherInstances(
		FRDGBuilder& GraphBuilder,
		FGatherInstancesTaskData& TaskData,
		FRayTracingScene& RayTracingScene,
		FRayTracingShaderBindingTable& RayTracingSBT);

	void WaitForDynamicBindings(FGatherInstancesTaskData& TaskData);

	// Wait for shader bindings related tasks to complete and combine bindings from all views
	// This function must run on render thread
	bool FinishGatherVisibleShaderBindings(FGatherInstancesTaskData& TaskData);

	// Must be called after FinishGatherVisibleShaderBindings(...) completes
	TConstArrayView<FRayTracingShaderBindingData> GetVisibleShaderBindings(FGatherInstancesTaskData& TaskData);

	bool ShouldExcludeDecals();
}

#endif // RHI_RAYTRACING