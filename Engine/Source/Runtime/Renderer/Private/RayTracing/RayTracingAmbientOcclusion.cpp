// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeferredShadingRenderer.h"

#if RHI_RAYTRACING

#include "ClearQuad.h"
#include "SceneRendering.h"
#include "PostProcess/SceneRenderTargets.h"
#include "SceneUtils.h"
#include "RenderTargetPool.h"
#include "RHIResources.h"
#include "UniformBuffer.h"
#include "PipelineStateCache.h"
#include "Nanite/NaniteRayTracing.h"
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracing.h"
#include "RayTracingMaterialHitShaders.h"
#include "RayTracingDefinitions.h"
#include "ScenePrivate.h"
#include "SceneTextureParameters.h"

#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"

static int32 GRayTracingAmbientOcclusion = -1;
static FAutoConsoleVariableRef CVarRayTracingAmbientOcclusion(
	TEXT("r.RayTracing.AmbientOcclusion"),
	GRayTracingAmbientOcclusion,
	TEXT("-1: Value driven by postprocess volume (default) \n")
	TEXT(" 0: ray tracing ambient occlusion off \n")
	TEXT(" 1: ray tracing ambient occlusion enabled"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static TAutoConsoleVariable<int32> CVarUseAODenoiser(
	TEXT("r.AmbientOcclusion.Denoiser"),
	2,
	TEXT("Choose the denoising algorithm.\n")
	TEXT(" 0: Disabled;\n")
	TEXT(" 1: Forces the default denoiser of the renderer;\n")
	TEXT(" 2: GScreenSpaceDenoiser witch may be overriden by a third party plugin (default)."),
	ECVF_RenderThreadSafe);

static int32 GRayTracingAmbientOcclusionSamplesPerPixel = -1;
static FAutoConsoleVariableRef CVarRayTracingAmbientOcclusionSamplesPerPixel(
	TEXT("r.RayTracing.AmbientOcclusion.SamplesPerPixel"),
	GRayTracingAmbientOcclusionSamplesPerPixel,
	TEXT("Sets the samples-per-pixel for ambient occlusion (default = -1 (driven by postprocesing volume))")
);

static TAutoConsoleVariable<int32> CVarRayTracingAmbientOcclusionEnableTwoSidedGeometry(
	TEXT("r.RayTracing.AmbientOcclusion.EnableTwoSidedGeometry"),
	0,
	TEXT("Enables two-sided geometry when tracing shadow rays (default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingAmbientOcclusionEnableMaterials(
	TEXT("r.RayTracing.AmbientOcclusion.EnableMaterials"),
	0,
	TEXT("Enables "),
	ECVF_RenderThreadSafe
);

bool ShouldRenderRayTracingAmbientOcclusion(const FViewInfo& View)
{
	bool bEnabled = GRayTracingAmbientOcclusion < 0
		? View.FinalPostProcessSettings.RayTracingAO > 0
		: GRayTracingAmbientOcclusion != 0;

	bEnabled &= (View.FinalPostProcessSettings.RayTracingAOIntensity > 0.0f);

	return ShouldRenderRayTracingEffect(bEnabled, ERayTracingPipelineCompatibilityFlags::FullPipeline, View);
}

DECLARE_GPU_STAT_NAMED(RayTracingAmbientOcclusion, TEXT("Ray Tracing Ambient Occlusion"));

class FRayTracingAmbientOcclusionRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingAmbientOcclusionRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTracingAmbientOcclusionRGS, FGlobalShader)

	class FEnableTwoSidedGeometryDim : SHADER_PERMUTATION_BOOL("ENABLE_TWO_SIDED_GEOMETRY");
	class FEnableMaterialsDim : SHADER_PERMUTATION_BOOL("ENABLE_MATERIALS");

	using FPermutationDomain = TShaderPermutationDomain<FEnableTwoSidedGeometryDim, FEnableMaterialsDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int, SamplesPerPixel)
		SHADER_PARAMETER(float, MaxRayDistance)
		SHADER_PARAMETER(float, Intensity)
		SHADER_PARAMETER(float, MaxNormalBias)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWAmbientOcclusionMaskUAV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWAmbientOcclusionHitDistanceUAV)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingAmbientOcclusionRGS, "/Engine/Private/RayTracing/RayTracingAmbientOcclusionRGS.usf", "AmbientOcclusionRGS", SF_RayGen);

void FDeferredShadingSceneRenderer::PrepareRayTracingAmbientOcclusion(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders)
{
	if (!ShouldRenderRayTracingAmbientOcclusion(View))
	{
		return;
	}

	// Declare all RayGen shaders that require material closest hit shaders to be bound
	FRayTracingAmbientOcclusionRGS::FPermutationDomain PermutationVector;
	for (uint32 TwoSidedGeometryIndex = 0; TwoSidedGeometryIndex < 2; ++TwoSidedGeometryIndex)
	{
		for (uint32 EnableMaterialsIndex = 0; EnableMaterialsIndex < 2; ++EnableMaterialsIndex)
		{
			PermutationVector.Set<FRayTracingAmbientOcclusionRGS::FEnableTwoSidedGeometryDim>(TwoSidedGeometryIndex != 0);
			PermutationVector.Set<FRayTracingAmbientOcclusionRGS::FEnableMaterialsDim>(EnableMaterialsIndex != 0);
			TShaderMapRef<FRayTracingAmbientOcclusionRGS> RayGenerationShader(View.ShaderMap, PermutationVector);
			OutRayGenShaders.Add(RayGenerationShader.GetRayTracingShader());
		}
	}
}

#endif // RHI_RAYTRACING

void FDeferredShadingSceneRenderer::RenderRayTracingAmbientOcclusion(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	const FSceneTextureParameters& SceneTextures,
	FRDGTextureRef* OutAmbientOcclusionTexture)
#if RHI_RAYTRACING
{
	RDG_EVENT_SCOPE_STAT(GraphBuilder, RayTracingAmbientOcclusion, "Ray Tracing Ambient Occlusion");

	const FRayTracingScene& RayTracingScene = Scene->RayTracingScene;

	// Allocates denoiser inputs.
	IScreenSpaceDenoiser::FAmbientOcclusionInputs DenoiserInputs;
	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(
			SceneTextures.SceneDepthTexture->Desc.Extent,
			PF_R16F,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV);
		DenoiserInputs.Mask = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingAmbientOcclusion"));
		DenoiserInputs.RayHitDistance = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingAmbientOcclusionHitDistance"));
	}
	
	IScreenSpaceDenoiser::FAmbientOcclusionRayTracingConfig RayTracingConfig;
	RayTracingConfig.RayCountPerPixel =  GRayTracingAmbientOcclusionSamplesPerPixel >= 0 ? GRayTracingAmbientOcclusionSamplesPerPixel : View.FinalPostProcessSettings.RayTracingAOSamplesPerPixel;

	// Build RTAO parameters
	FRayTracingAmbientOcclusionRGS::FParameters *PassParameters = GraphBuilder.AllocParameters<FRayTracingAmbientOcclusionRGS::FParameters>();
	PassParameters->SamplesPerPixel = RayTracingConfig.RayCountPerPixel;
	PassParameters->MaxRayDistance = View.FinalPostProcessSettings.RayTracingAORadius;
	PassParameters->Intensity = View.FinalPostProcessSettings.RayTracingAOIntensity;
	PassParameters->MaxNormalBias = GetRaytracingMaxNormalBias();
	PassParameters->TLAS = RayTracingScene.GetLayerView(ERayTracingSceneLayer::Base, View.GetRayTracingSceneViewHandle());
	PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
	PassParameters->RWAmbientOcclusionMaskUAV = GraphBuilder.CreateUAV(DenoiserInputs.Mask);
	PassParameters->RWAmbientOcclusionHitDistanceUAV = GraphBuilder.CreateUAV(DenoiserInputs.RayHitDistance);
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->SceneTextures = SceneTextures;
	PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder);
	PassParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();

	FRayTracingAmbientOcclusionRGS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRayTracingAmbientOcclusionRGS::FEnableTwoSidedGeometryDim>(CVarRayTracingAmbientOcclusionEnableTwoSidedGeometry.GetValueOnRenderThread() != 0);
	PermutationVector.Set<FRayTracingAmbientOcclusionRGS::FEnableMaterialsDim>(CVarRayTracingAmbientOcclusionEnableMaterials.GetValueOnRenderThread() != 0);
	TShaderMapRef<FRayTracingAmbientOcclusionRGS> RayGenerationShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);
	ClearUnusedGraphResources(RayGenerationShader, PassParameters);
	
	FIntPoint RayTracingResolution = View.ViewRect.Size();
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("AmbientOcclusionRayTracing(SamplePerPixels=%d) %dx%d", RayTracingConfig.RayCountPerPixel, RayTracingResolution.X, RayTracingResolution.Y),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, this, &View, RayGenerationShader, RayTracingResolution](FRHICommandList& RHICmdList)
	{
		FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
		SetShaderParameters(GlobalResources, RayGenerationShader, *PassParameters);

		FRHIUniformBuffer* SceneUniformBuffer = PassParameters->Scene->GetRHI();
		FRHIUniformBuffer* NaniteRayTracingUniformBuffer = PassParameters->NaniteRayTracing->GetRHI();
		TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

		// TODO: Provide material support for opacity mask
		FRayTracingPipelineState* Pipeline = View.MaterialRayTracingData.PipelineState;
		FShaderBindingTableRHIRef SBT = View.MaterialRayTracingData.ShaderBindingTable;
		if (CVarRayTracingAmbientOcclusionEnableMaterials.GetValueOnRenderThread() == 0)
		{
			// Declare default pipeline
			FRayTracingPipelineStateInitializer Initializer;
			Initializer.MaxPayloadSizeInBytes = GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType::RayTracingMaterial);
			
			const FShaderBindingLayout* ShaderBindingLayout = RayTracing::GetShaderBindingLayout(ShaderPlatform);
			if (ShaderBindingLayout)
			{
				Initializer.ShaderBindingLayout = &ShaderBindingLayout->RHILayout;
			}

			FRHIRayTracingShader* RayGenShaderTable[] = { RayGenerationShader.GetRayTracingShader() };
			Initializer.SetRayGenShaderTable(RayGenShaderTable);

			FRHIRayTracingShader* HitGroupTable[] = { GetRayTracingDefaultOpaqueShader(View.ShaderMap) };
			Initializer.SetHitGroupTable(HitGroupTable);
						
			FRHIRayTracingShader* MissGroupTable[] = { GetRayTracingDefaultMissShader(View.ShaderMap) };
			Initializer.SetMissShaderTable(MissGroupTable);

			Pipeline = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(RHICmdList, Initializer);

			SBT = Scene->RayTracingSBT.AllocateTransientRHI(RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Disallow, Initializer.GetMaxLocalBindingDataSize());

			RHICmdList.SetDefaultRayTracingHitGroup(SBT, Pipeline, 0);
			RHICmdList.SetRayTracingMissShader(SBT, 0, Pipeline, 0 /* ShaderIndexInPipeline */, 0, nullptr, 0);
			RHICmdList.CommitShaderBindingTable(SBT);
		}

		RHICmdList.RayTraceDispatch(Pipeline, RayGenerationShader.GetRayTracingShader(), SBT, GlobalResources, RayTracingResolution.X, RayTracingResolution.Y);
	});

	int32 DenoiserMode = CVarUseAODenoiser.GetValueOnRenderThread();
	if (DenoiserMode != 0)
	{
		const IScreenSpaceDenoiser* DefaultDenoiser = IScreenSpaceDenoiser::GetDefaultDenoiser();
		const IScreenSpaceDenoiser* DenoiserToUse = DenoiserMode == 1 ? DefaultDenoiser : GScreenSpaceDenoiser;

		RDG_EVENT_SCOPE(GraphBuilder, "%s%s(AmbientOcclusion) %dx%d",
			DenoiserToUse != DefaultDenoiser ? TEXT("ThirdParty ") : TEXT(""),
			DenoiserToUse->GetDebugName(),
			View.ViewRect.Width(), View.ViewRect.Height());

		IScreenSpaceDenoiser::FAmbientOcclusionOutputs DenoiserOutputs = DenoiserToUse->DenoiseAmbientOcclusion(
			GraphBuilder,
			View,
			&View.PrevViewInfo,
			SceneTextures,
			DenoiserInputs,
			RayTracingConfig);

		*OutAmbientOcclusionTexture = DenoiserOutputs.AmbientOcclusionMask;
	}
	else
	{
		*OutAmbientOcclusionTexture = DenoiserInputs.Mask;
	}
}
#else
{
	unimplemented();
}
#endif
