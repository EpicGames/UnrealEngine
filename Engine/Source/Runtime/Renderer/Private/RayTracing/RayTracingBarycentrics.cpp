// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHI.h"

#if RHI_RAYTRACING

#include "BuiltInRayTracingShaders.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DeferredShadingRenderer.h"
#include "GlobalShader.h"
#include "PostProcess/SceneRenderTargets.h"
#include "RenderGraphBuilder.h"
#include "PipelineStateCache.h"
#include "ScenePrivate.h"
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracing/RayTracing.h"
#include "Nanite/NaniteRayTracing.h"

#include "Rendering/NaniteStreamingManager.h"

class FRayTracingBarycentricsRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingBarycentricsRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTracingBarycentricsRGS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Output)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)
		SHADER_PARAMETER(float, TimingScale)
	END_SHADER_PARAMETER_STRUCT()

	class FOutputTiming : SHADER_PERMUTATION_BOOL("OUTPUT_TIMING");
	class FUseNvAPITimestamp : SHADER_PERMUTATION_BOOL("USE_NVAPI_TIMESTAMP");
	using FPermutationDomain = TShaderPermutationDomain<FOutputTiming, FUseNvAPITimestamp>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// TODO: Check this using DDPI
		const bool bUseNvAPITimestamp = PermutationVector.Get<FUseNvAPITimestamp>();
		if (bUseNvAPITimestamp && 
			(IsVulkanPlatform(Parameters.Platform) || IsMetalPlatform(Parameters.Platform)))
		{
			return false;
		}

		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Default;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingBarycentricsRGS, "/Engine/Private/RayTracing/RayTracingBarycentrics.usf", "RayTracingBarycentricsMainRGS", SF_RayGen);

// Example closest hit shader
class FRayTracingBarycentricsCHS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingBarycentricsCHS);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Default;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	FRayTracingBarycentricsCHS() = default;
	FRayTracingBarycentricsCHS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};
IMPLEMENT_SHADER_TYPE(, FRayTracingBarycentricsCHS, TEXT("/Engine/Private/RayTracing/RayTracingBarycentrics.usf"), TEXT("RayTracingBarycentricsMainCHS"), SF_RayHitGroup);

class FRayTracingBarycentricsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingBarycentricsCS)
	SHADER_USE_PARAMETER_STRUCT(FRayTracingBarycentricsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Output)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRasterUniformParameters, NaniteRasterUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteShadingUniformParameters, NaniteShadingUniformBuffer)
		SHADER_PARAMETER(float, RTDebugVisualizationNaniteCutError)
		SHADER_PARAMETER(float, TimingScale)
	END_SHADER_PARAMETER_STRUCT()

	class FSupportProceduralPrimitive : SHADER_PERMUTATION_BOOL("ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE");
	class FOutputTiming : SHADER_PERMUTATION_BOOL("OUTPUT_TIMING");
	using FPermutationDomain = TShaderPermutationDomain<FSupportProceduralPrimitive, FOutputTiming>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
		OutEnvironment.CompilerFlags.Add(CFLAG_InlineRayTracing);

		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_Y"), ThreadGroupSizeY);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform) && RHISupportsRayTracing(Parameters.Platform) && RHISupportsInlineRayTracing(Parameters.Platform);
	}

	// Current inline ray tracing implementation requires 1:1 mapping between thread groups and waves and only supports wave32 mode.
	static constexpr uint32 ThreadGroupSizeX = 8;
	static constexpr uint32 ThreadGroupSizeY = 4;
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingBarycentricsCS, "/Engine/Private/RayTracing/RayTracingBarycentrics.usf", "RayTracingBarycentricsMainCS", SF_Compute);

extern float GetRayTracingDebugTimingScale();

void RenderRayTracingBarycentricsCS(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneColor, ERayTracingSceneLayer LayerToTrace, bool bVisualizeProceduralPrimitives, bool bOutputTiming)
{
	FRayTracingBarycentricsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRayTracingBarycentricsCS::FParameters>();

	PassParameters->TLAS = View.GetRayTracingSceneLayerViewChecked(LayerToTrace);
	PassParameters->Output = GraphBuilder.CreateUAV(SceneColor);
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
	PassParameters->NaniteRasterUniformBuffer = CreateDebugNaniteRasterUniformBuffer(GraphBuilder);
	PassParameters->NaniteShadingUniformBuffer = CreateDebugNaniteShadingUniformBuffer(GraphBuilder);

	PassParameters->RTDebugVisualizationNaniteCutError = 0.0f;
	PassParameters->TimingScale = GetRayTracingDebugTimingScale();

	FIntRect ViewRect = View.ViewRect;

	FRayTracingBarycentricsCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRayTracingBarycentricsCS::FSupportProceduralPrimitive>(bVisualizeProceduralPrimitives);
	PermutationVector.Set<FRayTracingBarycentricsCS::FOutputTiming>(bOutputTiming);

	auto ComputeShader = View.ShaderMap->GetShader<FRayTracingBarycentricsCS>(PermutationVector);

	const FIntPoint GroupSize(FRayTracingBarycentricsCS::ThreadGroupSizeX, FRayTracingBarycentricsCS::ThreadGroupSizeY);
	const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(ViewRect.Size(), GroupSize);

	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("Barycentrics"), ComputeShader, PassParameters, GroupCount);
}

void RenderRayTracingBarycentricsRGS(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View, FRDGTextureRef SceneColor, ERayTracingSceneLayer LayerToTrace, bool bOutputTiming)
{
	FRayTracingBarycentricsRGS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRayTracingBarycentricsRGS::FOutputTiming>(bOutputTiming);
	PermutationVector.Set<FRayTracingBarycentricsRGS::FUseNvAPITimestamp>(IsRHIDeviceNVIDIA());

	auto RayGenShader = View.ShaderMap->GetShader<FRayTracingBarycentricsRGS>(PermutationVector);
	auto ClosestHitShader = View.ShaderMap->GetShader<FRayTracingBarycentricsCHS>();

	FRayTracingPipelineStateInitializer Initializer;

	const FShaderBindingLayout* ShaderBindingLayout = RayTracing::GetShaderBindingLayout(View.GetShaderPlatform());
	if (ShaderBindingLayout)
	{
		Initializer.ShaderBindingLayout = &ShaderBindingLayout->RHILayout;
	}

	FRHIRayTracingShader* RayGenShaderTable[] = { RayGenShader.GetRayTracingShader() };
	Initializer.SetRayGenShaderTable(RayGenShaderTable);

	FRHIRayTracingShader* HitGroupTable[] = { ClosestHitShader.GetRayTracingShader() };
	Initializer.SetHitGroupTable(HitGroupTable);	

	FRHIRayTracingShader* MissTable[] = { View.ShaderMap->GetShader<FDefaultPayloadMS>().GetRayTracingShader() };
	Initializer.SetMissShaderTable(MissTable);

	FRayTracingPipelineState* Pipeline = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(GraphBuilder.RHICmdList, Initializer);

	FShaderBindingTableRHIRef SBT = Scene.RayTracingSBT.AllocateTransientRHI(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Disallow, Initializer.GetMaxLocalBindingDataSize());
	   
	FRayTracingBarycentricsRGS::FParameters* RayGenParameters = GraphBuilder.AllocParameters<FRayTracingBarycentricsRGS::FParameters>();

	RayGenParameters->TLAS = View.GetRayTracingSceneLayerViewChecked(LayerToTrace);
	RayGenParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	RayGenParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
	RayGenParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();
	RayGenParameters->Output = GraphBuilder.CreateUAV(SceneColor);

	RayGenParameters->TimingScale = GetRayTracingDebugTimingScale();

	FIntRect ViewRect = View.ViewRect;
	
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("Barycentrics"),
		RayGenParameters,
		ERDGPassFlags::Compute,
		[RayGenParameters, RayGenShader, &View, SBT, Pipeline, ViewRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
		SetShaderParameters(GlobalResources, RayGenShader, *RayGenParameters);

		FRHIUniformBuffer* SceneUniformBuffer = RayGenParameters->Scene->GetRHI();
		FRHIUniformBuffer* NaniteRayTracingUniformBuffer = RayGenParameters->NaniteRayTracing->GetRHI();
		TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

		// Dispatch rays using default shader binding table
		RHICmdList.SetDefaultRayTracingHitGroup(SBT, Pipeline, 0);
		RHICmdList.SetRayTracingMissShader(SBT, 0, Pipeline, 0 /* ShaderIndexInPipeline */, 0, nullptr, 0);
		RHICmdList.CommitShaderBindingTable(SBT);
		RHICmdList.RayTraceDispatch(Pipeline, RayGenShader.GetRayTracingShader(), SBT, GlobalResources, ViewRect.Size().X, ViewRect.Size().Y);
	});
}

void RenderRayTracingBarycentrics(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View, FRDGTextureRef SceneColor, ERayTracingSceneLayer LayerToTrace, bool bVisualizeProceduralPrimitives, bool bOutputTiming)
{
	const bool bRayTracingInline = ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::Inline, View);
	const bool bRayTracingPipeline = ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::FullPipeline, View);

	// currently bOutputTiming is only supported on Nvidia when using RGS
	if (bRayTracingInline && (!bOutputTiming || !IsRHIDeviceNVIDIA()))
	{
		RenderRayTracingBarycentricsCS(GraphBuilder, View, SceneColor, LayerToTrace, bVisualizeProceduralPrimitives, bOutputTiming);
	}
	else if (bRayTracingPipeline)
	{
		RenderRayTracingBarycentricsRGS(GraphBuilder, Scene, View, SceneColor, LayerToTrace, bOutputTiming);
	}
}
#endif
