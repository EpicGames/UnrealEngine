// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracing/RayTracingBasicShaders.h"

#if RHI_RAYTRACING

#include "PipelineStateCache.h"
#include "ShaderParameterStruct.h"
#include "RayTracingDefinitions.h"

IMPLEMENT_GLOBAL_SHADER( FBasicOcclusionMainRGS, "/Engine/Private/RayTracing/RayTracingBuiltInShaders.usf", "OcclusionMainRGS", SF_RayGen);
IMPLEMENT_GLOBAL_SHADER( FBasicIntersectionMainRGS, "/Engine/Private/RayTracing/RayTracingBuiltInShaders.usf", "IntersectionMainRGS", SF_RayGen);
IMPLEMENT_GLOBAL_SHADER( FBasicIntersectionMainCHS, "/Engine/Private/RayTracing/RayTracingBuiltInShaders.usf", "IntersectionMainCHS", SF_RayHitGroup);

struct FBasicRayTracingPipeline
{
	FRayTracingPipelineState* PipelineState = nullptr;
	FShaderBindingTableRHIRef SBT;
	TShaderRef<FBasicOcclusionMainRGS> OcclusionRGS;
	TShaderRef<FBasicIntersectionMainRGS> IntersectionRGS;
};

/**
* Returns a ray tracing pipeline with FBasicOcclusionMainRGS, FBasicIntersectionMainRGS, FBasicIntersectionMainCHS and FDefaultPayloadMS.
* This can be used to perform basic "fixed function" occlusion and intersection ray tracing.
*/
FBasicRayTracingPipeline GetBasicRayTracingPipeline(FRHICommandList& RHICmdList, ERHIFeatureLevel::Type FeatureLevel)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

	FRayTracingPipelineStateInitializer PipelineInitializer;

	auto OcclusionRGS = ShaderMap->GetShader<FBasicOcclusionMainRGS>();
	auto IntersectionRGS = ShaderMap->GetShader<FBasicIntersectionMainRGS>();

	FRHIRayTracingShader* RayGenShaderTable[] = { OcclusionRGS.GetRayTracingShader(), IntersectionRGS.GetRayTracingShader() };
	PipelineInitializer.SetRayGenShaderTable(RayGenShaderTable);

	auto ClosestHitShader = ShaderMap->GetShader<FBasicIntersectionMainCHS>();
	FRHIRayTracingShader* HitShaderTable[] = { ClosestHitShader.GetRayTracingShader() };
	PipelineInitializer.SetHitGroupTable(HitShaderTable);

	auto MissShader = ShaderMap->GetShader<FDefaultPayloadMS>();
	FRHIRayTracingShader* MissShaderTable[] = { MissShader.GetRayTracingShader() };
	PipelineInitializer.SetMissShaderTable(MissShaderTable);
		
	FRayTracingShaderBindingTableInitializer SBTInitializer;
	SBTInitializer.ShaderBindingMode = ERayTracingShaderBindingMode::RTPSO;
	SBTInitializer.HitGroupIndexingMode = ERayTracingHitGroupIndexingMode::Disallow;
	SBTInitializer.NumGeometrySegments = 1;
	SBTInitializer.NumShaderSlotsPerGeometrySegment = RAY_TRACING_NUM_SHADER_SLOTS;
	SBTInitializer.NumMissShaderSlots = 1;
	SBTInitializer.NumCallableShaderSlots = 0;
	SBTInitializer.LocalBindingDataSize = PipelineInitializer.GetMaxLocalBindingDataSize();

	FBasicRayTracingPipeline Result;

	Result.PipelineState = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(RHICmdList, PipelineInitializer);
	Result.SBT = RHICmdList.CreateRayTracingShaderBindingTable(SBTInitializer);
	Result.OcclusionRGS = OcclusionRGS;
	Result.IntersectionRGS = IntersectionRGS;

	return Result;
}

void DispatchBasicOcclusionRays(FRHICommandList& RHICmdList, FRHIShaderResourceView* SceneView, FRHIRayTracingGeometry* Geometry, FRHIShaderResourceView* RayBufferView, FRHIUnorderedAccessView* ResultView, uint32 NumRays)
{
	FBasicRayTracingPipeline RayTracingPipeline = GetBasicRayTracingPipeline(RHICmdList, GMaxRHIFeatureLevel);
	
	RHICmdList.SetDefaultRayTracingHitGroup(RayTracingPipeline.SBT, RayTracingPipeline.PipelineState, 0);
	RHICmdList.SetRayTracingMissShader(RayTracingPipeline.SBT, 0, RayTracingPipeline.PipelineState, 0, 0, nullptr, 0);
	RHICmdList.CommitShaderBindingTable(RayTracingPipeline.SBT);

	FBasicOcclusionMainRGS::FParameters OcclusionParameters;
	OcclusionParameters.TLAS = SceneView;
	OcclusionParameters.Rays = RayBufferView;
	OcclusionParameters.OcclusionOutput = ResultView;

	FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
	SetShaderParameters(GlobalResources, RayTracingPipeline.OcclusionRGS, OcclusionParameters);
	RHICmdList.RayTraceDispatch(RayTracingPipeline.PipelineState, RayTracingPipeline.OcclusionRGS.GetRayTracingShader(), RayTracingPipeline.SBT, GlobalResources, NumRays, 1);
}

void DispatchBasicIntersectionRays(FRHICommandList& RHICmdList, FRHIShaderResourceView* SceneView, FRHIRayTracingGeometry* Geometry, FRHIShaderResourceView* RayBufferView, FRHIUnorderedAccessView* ResultView, uint32 NumRays)
{
	FBasicRayTracingPipeline RayTracingPipeline = GetBasicRayTracingPipeline(RHICmdList, GMaxRHIFeatureLevel);
	   
	RHICmdList.SetDefaultRayTracingHitGroup(RayTracingPipeline.SBT, RayTracingPipeline.PipelineState, 0);
	RHICmdList.SetRayTracingMissShader(RayTracingPipeline.SBT, 0, RayTracingPipeline.PipelineState, 0, 0, nullptr, 0);
	RHICmdList.CommitShaderBindingTable(RayTracingPipeline.SBT);

	FBasicIntersectionMainRGS::FParameters OcclusionParameters;
	OcclusionParameters.TLAS = SceneView;
	OcclusionParameters.Rays = RayBufferView;
	OcclusionParameters.IntersectionOutput = ResultView;

	FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
	SetShaderParameters(GlobalResources, RayTracingPipeline.IntersectionRGS, OcclusionParameters);
	RHICmdList.RayTraceDispatch(RayTracingPipeline.PipelineState, RayTracingPipeline.IntersectionRGS.GetRayTracingShader(), RayTracingPipeline.SBT, GlobalResources, NumRays, 1);
}

#endif // RHI_RAYTRACING
