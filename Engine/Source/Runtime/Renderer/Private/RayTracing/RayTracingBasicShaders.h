// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuiltInRayTracingShaders.h"

#if RHI_RAYTRACING

#include "RayTracingPayloadType.h"
#include "ShaderParameterStruct.h"

// C++ counter-part of FBasicRayData declared in RayTracingCommon.ush
struct FBasicRayTracingRay
{
	float Origin[3];
	uint32 Mask;
	float Direction[3];
	float TFar;
};

// C++ counter-part of FIntersectionPayload declared in RayTracingCommon.ush
struct FBasicRayTracingIntersectionResult
{
	float  HitT;            // Distance from ray origin to the intersection point in the ray direction. Negative on miss.
	uint32 PrimitiveIndex;  // Index of the primitive within the geometry inside the bottom-level acceleration structure instance. Undefined on miss.
	uint32 InstanceIndex;   // Index of the current instance in the top-level structure. Undefined on miss.
	float  Barycentrics[2]; // Primitive barycentric coordinates of the intersection point. Undefined on miss.
};

class FBasicOcclusionMainRGS : public FBuiltInRayTracingShader
{
	DECLARE_GLOBAL_SHADER(FBasicOcclusionMainRGS);
	SHADER_USE_ROOT_PARAMETER_STRUCT(FBasicOcclusionMainRGS, FBuiltInRayTracingShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_SRV(StructuredBuffer<FBasicRayData>, Rays)
		SHADER_PARAMETER_UAV(RWStructuredBuffer<uint>, OcclusionOutput)
	END_SHADER_PARAMETER_STRUCT()

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Default;
	}
};

class FBasicIntersectionMainRGS : public FBuiltInRayTracingShader
{
	DECLARE_GLOBAL_SHADER(FBasicIntersectionMainRGS);
	SHADER_USE_ROOT_PARAMETER_STRUCT(FBasicIntersectionMainRGS, FBuiltInRayTracingShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_SRV(StructuredBuffer<FBasicRayData>, Rays)
		SHADER_PARAMETER_UAV(RWStructuredBuffer<FIntersectionPayload>, IntersectionOutput)
	END_SHADER_PARAMETER_STRUCT()

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Default;
	}
};

class FBasicIntersectionMainCHS : public FBuiltInRayTracingShader
{
	DECLARE_GLOBAL_SHADER(FBasicIntersectionMainCHS);
public:

	FBasicIntersectionMainCHS() = default;
	FBasicIntersectionMainCHS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FBuiltInRayTracingShader(Initializer)
	{}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Default;
	}
};

/**
* Trace rays from an input buffer of FBasicRayTracingRay.
* Binary intersection results are written to output buffer as R32_UINTs.
* 0xFFFFFFFF is written if ray intersects any scene triangle, 0 otherwise.
*/
void DispatchBasicOcclusionRays(FRHICommandList& RHICmdList, FRHIShaderResourceView* SceneView, FRHIRayTracingGeometry* Geometry, FRHIShaderResourceView* RayBufferView, FRHIUnorderedAccessView* ResultView, uint32 NumRays);

/**
* Trace rays from an input buffer of FBasicRayTracingRay.
* Primitive intersection results are written to output buffer as FBasicRayTracingIntersectionResult.
*/
void DispatchBasicIntersectionRays(FRHICommandList& RHICmdList, FRHIShaderResourceView* SceneView, FRHIRayTracingGeometry* Geometry, FRHIShaderResourceView* RayBufferView, FRHIUnorderedAccessView* ResultView, uint32 NumRays);

#endif // RHI_RAYTRACING
