// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHI.h"
#include "ScenePrivate.h"
#include "ScreenPass.h"

#if RHI_RAYTRACING

#include "DataDrivenShaderPlatformInfo.h"
#include "DeferredShadingRenderer.h"
#include "GlobalShader.h"
#include "PostProcess/SceneRenderTargets.h"
#include "RenderGraphBuilder.h"
#include "SceneUtils.h"
#include "RayTracingVisualizationData.h"
#include "RaytracingDebugDefinitions.h"
#include "RayTracingDebugTypes.h"
#include "RayTracing/RayTracingLighting.h"
#include "RayTracing/RayTracing.h"
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracingTraversalStatistics.h"
#include "RHIResourceUtils.h"
#include "Nanite/NaniteRayTracing.h"
#include "PixelShaderUtils.h"
#include "SystemTextures.h"
#include "BlueNoise.h"
#include "SceneViewState.h"

#define LOCTEXT_NAMESPACE "RayTracingDebugVisualizationMenuCommands"

DECLARE_GPU_STAT(RayTracingDebug);

static uint32 GetRaytracingDebugViewModeID(const FSceneView& View);

namespace RayTracingDebug
{
	bool UseInlineHardwareRayTracing(const FSceneViewFamily& ViewFamily)
	{
		if (ViewFamily.Views.IsEmpty() || !ViewFamily.EngineShowFlags.RayTracingDebug)
		{
			return false;
		}

		// Any of the views require inline raytracing
		for (const FSceneView* View : ViewFamily.Views)
		{
			if (View && ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::Inline, *View) 
				&& RayTracingDebugModeSupportsInline(GetRaytracingDebugViewModeID(*View)))
			{
				return true;
			}
		}

		return false;
	}
}

TAutoConsoleVariable<int32> CVarRayTracingVisualizePickerDomain(
	TEXT("r.RayTracing.Visualize.PickerDomain"),
	0,
	TEXT("Changes the picker domain to highlight:\n")
	TEXT("0 - Triangles (default)\n")
	TEXT("1 - Instances\n")
	TEXT("2 - Segment\n")
	TEXT("3 - Flags\n")
	TEXT("4 - Mask\n"),
	ECVF_RenderThreadSafe
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugPickerDomain_Deprecated(TEXT("r.RayTracing.Debug.PickerDomain"), TEXT("r.RayTracing.Visualize.PickerDomain"), TEXT("5.6"));

static TAutoConsoleVariable<bool> CVarRayTracingVisualizeOpaqueOnly(
	TEXT("r.RayTracing.Visualize.OpaqueOnly"),
	true,
	TEXT("Whether to only show opaque instances in ray tracing debug visualizations."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarRayTracingVisualizeFarField(
	TEXT("r.RayTracing.Visualize.FarField"),
	false,
	TEXT("Whether to show far field geometry instead of near field in ray tracing visualizations."),
	ECVF_RenderThreadSafe
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugModeOpaqueOnly_Deprecated(TEXT("r.RayTracing.DebugVisualizationMode.OpaqueOnly"), TEXT("r.RayTracing.Visualize.OpaqueOnly"), TEXT("5.6"));

static TAutoConsoleVariable<float> CVarRayTracingVisualizeTimingScale(
	TEXT("r.RayTracing.Visualize.TimingScale"),
	1.0f,
	TEXT("Scaling factor for ray timing heat map visualization. (default = 1)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugTimingScale_Deprecated(TEXT("r.RayTracing.DebugTimingScale"), TEXT("r.RayTracing.Visualize.TimingScale"), TEXT("5.6"));

static TAutoConsoleVariable<float> CVarRayTracingVisualizeTraversalBoxScale(
	TEXT("r.RayTracing.Visualize.Traversal.BoxScale"),
	150.0f,
	TEXT("Scaling factor for box traversal heat map visualization. (default = 150)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugTraversalBoxScale_Deprecated(TEXT("r.RayTracing.DebugTraversalScale.Box"), TEXT("r.RayTracing.Visualize.Traversal.BoxScale"), TEXT("5.6"));

static TAutoConsoleVariable<float> CVarRayTracingVisualizeTraversalClusterScale(
	TEXT("r.RayTracing.Visualize.Traversal.ClusterScale"),
	2500.0f,
	TEXT("Scaling factor for cluster traversal heat map visualization. (default = 2500)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugTraversalClusterScale_Deprecated(TEXT("r.RayTracing.DebugTraversalScale.Cluster"), TEXT("r.RayTracing.Visualize.Traversal.ClusterScale"), TEXT("5.6"));

static TAutoConsoleVariable<float> CVarRayTracingVisualizeInstanceOverlapScale(
	TEXT("r.RayTracing.Visualize.InstanceOverlap.Scale"),
	16.0f,
	TEXT("Scaling factor for instance traversal heat map visualization. (default = 16)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugInstanceOverlapScale_Deprecated(TEXT("r.RayTracing.Debug.InstanceOverlap.Scale"), TEXT("r.RayTracing.Visualize.InstanceOverlap.Scale"), TEXT("5.6"));

static TAutoConsoleVariable<float> CVarRayTracingVisualizeInstanceOverlapBoundingBoxScale(
	TEXT("r.RayTracing.Visualize.InstanceOverlap.BoundingBoxScale"),
	1.001f,
	TEXT("Scaling factor for instance bounding box extent for avoiding z-fighting. (default = 1.001)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugInstanceOverlapBoundingBoxScale_Deprecated(TEXT("r.RayTracing.Debug.InstanceOverlap.BoundingBoxScale"), TEXT("r.RayTracing.Visualize.InstanceOverlap.BoundingBoxScale"), TEXT("5.6"));

static TAutoConsoleVariable<int32> CVarRayTracingVisualizeInstanceOverlapShowWireframe(
	TEXT("r.RayTracing.Visualize.InstanceOverlap.ShowWireframe"),
	1,
	TEXT("Show instance bounding boxes in wireframe in Instances Overlap mode. (default = 1)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugInstanceOverlapShowWireframe_Deprecated(TEXT("r.RayTracing.Debug.InstanceOverlap.ShowWireframe"), TEXT("r.RayTracing.Visualize.InstanceOverlap.ShowWireframe"), TEXT("5.6"));

static TAutoConsoleVariable<float> CVarRayTracingVisualizeTraversalTriangleScale(
	TEXT("r.RayTracing.Visualize.Traversal.TriangleScale"),
	30.0f,
	TEXT("Scaling factor for triangle traversal heat map visualization. (default = 30)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugTraversalTriangleScale_Deprecated(TEXT("r.RayTracing.DebugTraversalScale.Triangle"), TEXT("r.RayTracing.Visualize.Traversal.TriangleScale"), TEXT("5.6"));

static TAutoConsoleVariable<int32> CVarRayTracingVisualizeHitCountMaxThreshold(
	TEXT("r.RayTracing.Visualize.TriangleHitCount.MaxThreshold"),
	6,
	TEXT("Maximum hit count threshold for debug ray tracing triangle hit count heat map visualization. (default = 6)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugHitCountMaxThreshold_Deprecated(TEXT("r.RayTracing.DebugTriangleHitCount.MaxThreshold"), TEXT("r.RayTracing.Visualize.TriangleHitCount.MaxThreshold"), TEXT("5.6"));

static TAutoConsoleVariable<int32> CVarRayTracingVisualizeHitCountPerInstanceMaxThreshold(
	TEXT("r.RayTracing.Visualize.HitCountPerInstance.MaxThreshold"),
	100000,
	TEXT("Maximum hit count threshold for debug ray tracing hit count per instance heat map visualization. (default = 100000)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugHitCountPerInstanceMaxThreshold_Deprecated(TEXT("r.RayTracing.DebugTriangleHitCountPerInstance.MaxThreshold"), TEXT("r.RayTracing.Visualize.HitCountPerInstance.MaxThreshold"), TEXT("5.6"));

static TAutoConsoleVariable<int32> CVarRayTracingVisualizeHitCountTopKHits(
	TEXT("r.RayTracing.Visualize.TriangleHitCount.TopKMostHits"),
	10,
	TEXT("Highlight top k most hit instances in the view. (default = 10)\n")
);

static FAutoConsoleVariableDeprecated CVarRayTracingDebugHitCountTopKHits_Deprecated(TEXT("r.RayTracing.DebugTriangleHitCount.TopKMostHits"), TEXT("r.RayTracing.Visualize.TriangleHitCount.TopKMostHits"), TEXT("5.6"));

static int32 GVisualizeProceduralPrimitives = 0;
static FAutoConsoleVariableRef CVarVisualizeProceduralPrimitives(
	TEXT("r.RayTracing.Visualize.ProceduralPrimitives"),
	GVisualizeProceduralPrimitives,
	TEXT("Whether to include procedural primitives in visualization modes.\n")
	TEXT("Currently only supports Nanite primitives in inline barycentrics mode."),
	ECVF_RenderThreadSafe
);

static FAutoConsoleVariableDeprecated CVarVisualizeProceduralPrimitives_Deprecated(TEXT("r.RayTracing.DebugVisualizationMode.ProceduralPrimitives"), TEXT("r.RayTracing.Visualize.ProceduralPrimitives"), TEXT("5.6"));

bool IsRayTracingVisualizeOpaqueOnly()
{
	return CVarRayTracingVisualizeOpaqueOnly.GetValueOnRenderThread();
}

float GetRayTracingDebugTimingScale()
{
	return CVarRayTracingVisualizeTimingScale.GetValueOnRenderThread() / 25000.0f;
}

IMPLEMENT_RT_PAYLOAD_TYPE(ERayTracingPayloadType::RayTracingDebug, 44);

BEGIN_UNIFORM_BUFFER_STRUCT(FRayTracingDebugHitStatsUniformBufferParameters, )
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<FRayTracingHitStatsEntry>, HitStatsOutput)
END_UNIFORM_BUFFER_STRUCT()

IMPLEMENT_UNIFORM_BUFFER_STRUCT(FRayTracingDebugHitStatsUniformBufferParameters, "RayTracingDebugHitStatsUniformBuffer");

BEGIN_SHADER_PARAMETER_STRUCT(FRayTracingDebugParameters, )
	SHADER_PARAMETER(uint32, VisualizationMode)
	SHADER_PARAMETER(uint32, PickerDomain)
	SHADER_PARAMETER(float, TimingScale)
	SHADER_PARAMETER(uint32, OpaqueOnly)
	SHADER_PARAMETER(float, TriangleHitCountMaxThreshold)
	SHADER_PARAMETER(float, TriangleHitCountPerInstanceMaxThreshold)
	SHADER_PARAMETER(uint32, TopKMostHitInstances)
	SHADER_PARAMETER(uint32, NumTotalInstances)
	SHADER_PARAMETER(uint32, SubstrateDebugDataSizeInUints)
	SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Output)
	SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, OutputDepth)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InstancesExtraData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InstancesDebugData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InstanceBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRayTracingPickingFeedback>, PickingBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint2>, TopKHitStats)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, SceneUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracingUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRayTracingDebugHitStatsUniformBufferParameters, RayTracingDebugHitStatsUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRayTracingLightGrid, LightGridPacked)
	SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<int>, SubstrateDebugDataUAV)
	// Inline data
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, RayTracingSceneMetadata)
END_SHADER_PARAMETER_STRUCT()

class FRayTracingDebugRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTracingDebugRGS, FGlobalShader)
	
	class FUseDebugCHSType : SHADER_PERMUTATION_BOOL("USE_DEBUG_CHS");
	class FUseNvAPITimestamp : SHADER_PERMUTATION_BOOL("USE_NVAPI_TIMESTAMP");
	using FPermutationDomain = TShaderPermutationDomain<FUseDebugCHSType, FUseNvAPITimestamp>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRayTracingDebugParameters, SharedParameters)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		// TODO: Check this using DDPI
		const bool bUseNvAPITimestamp = PermutationVector.Get<FUseNvAPITimestamp>();
		if (bUseNvAPITimestamp && 
			(IsVulkanPlatform(Parameters.Platform) || IsMetalPlatform(Parameters.Platform)))
		{
			return false;
		}

		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		FPermutationDomain PermutationVector(PermutationId);
		if (PermutationVector.Get<FUseDebugCHSType>())
		{
			return ERayTracingPayloadType::RayTracingDebug;
		}
		else
		{
			return ERayTracingPayloadType::RayTracingMaterial;
		}
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugRGS, "/Engine/Private/RayTracing/RayTracingDebug.usf", "RayTracingDebugMainRGS", SF_RayGen);

class FRayTracingDebugCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugCS)
	SHADER_USE_PARAMETER_STRUCT(FRayTracingDebugCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform) && FDataDrivenShaderPlatformInfo::GetSupportsInlineRayTracing(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
		OutEnvironment.CompilerFlags.Add(CFLAG_InlineRayTracing);

		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_Y"), ThreadGroupSizeY);
	
		// Nanite RayTracing
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRayTracingDebugParameters, SharedParameters)
	END_SHADER_PARAMETER_STRUCT()

	// Current inline ray tracing implementation requires 1:1 mapping between thread groups and waves and only supports wave32 mode.
	static constexpr uint32 ThreadGroupSizeX = 8;
	static constexpr uint32 ThreadGroupSizeY = 4;
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugCS, "/Engine/Private/RayTracing/RayTracingDebug.usf", "RayTracingDebugMainCS", SF_Compute);

class FRayTracingDebugCHS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugCHS);

public:

	FRayTracingDebugCHS() = default;
	FRayTracingDebugCHS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}

	class FNaniteRayTracing : SHADER_PERMUTATION_BOOL("NANITE_RAY_TRACING");
	using FPermutationDomain = TShaderPermutationDomain<FNaniteRayTracing>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FNaniteRayTracing>())
		{
			OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		}
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingDebug;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugCHS, "/Engine/Private/RayTracing/RayTracingDebugCHS.usf", "closesthit=RayTracingDebugMainCHS anyhit=RayTracingDebugAHS", SF_RayHitGroup);

class FRayTracingDebugMS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugMS);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	FRayTracingDebugMS() = default;
	FRayTracingDebugMS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingDebug;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugMS, "/Engine/Private/RayTracing/RayTracingDebugMS.usf", "RayTracingDebugMS", SF_RayMiss);
class FRayTracingDebugHitStatsRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugHitStatsRGS);
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTracingDebugHitStatsRGS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER(int32, OpaqueOnly)
		SHADER_PARAMETER(uint32, VisualizationMode)
		SHADER_PARAMETER(uint32, TriangleHitCountForceNonOpaque)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, SceneUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracingUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRayTracingDebugHitStatsUniformBufferParameters, RayTracingDebugHitStatsUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingDebug;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugHitStatsRGS, "/Engine/Private/RayTracing/RayTracingDebugHitStats.usf", "RayTracingDebugHitStatsRGS", SF_RayGen);


class FRayTracingDebugHitStatsCHS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugHitStatsCHS);
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTracingDebugHitStatsCHS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRayTracingDebugHitStatsUniformBufferParameters, RayTracingDebugHitStatsUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// Temporary workaround for "unbound parameters not represented in the parameter struct" when disabling optimizations
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceOptimization);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingDebug;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugHitStatsCHS, "/Engine/Private/RayTracing/RayTracingDebugHitStatsCHS.usf", "closesthit=RayTracingDebugHitStatsCHS anyhit=RayTracingDebugHitStatsAHS", SF_RayHitGroup);

class FRayTracingDebugTraversalCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugTraversalCS)
	SHADER_USE_PARAMETER_STRUCT(FRayTracingDebugTraversalCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, Output)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRasterUniformParameters, NaniteRasterUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteShadingUniformParameters, NaniteShadingUniformBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(RaytracingTraversalStatistics::FShaderParameters, TraversalStatistics)

		SHADER_PARAMETER(uint32, VisualizationMode)
		SHADER_PARAMETER(float, TraversalBoxScale)
		SHADER_PARAMETER(float, TraversalClusterScale)
		SHADER_PARAMETER(float, TraversalTriangleScale)

		SHADER_PARAMETER(float, RTDebugVisualizationNaniteCutError)

		SHADER_PARAMETER_STRUCT_REF(FBlueNoise, BlueNoise)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)

		SHADER_PARAMETER(float, NormalBias)
		SHADER_PARAMETER(float, TraceDistance)
	END_SHADER_PARAMETER_STRUCT()

	class FSupportProceduralPrimitive : SHADER_PERMUTATION_BOOL("ENABLE_TRACE_RAY_INLINE_PROCEDURAL_PRIMITIVE");
	class FPrintTraversalStatistics : SHADER_PERMUTATION_BOOL("PRINT_TRAVERSAL_STATISTICS");
	using FPermutationDomain = TShaderPermutationDomain<FSupportProceduralPrimitive, FPrintTraversalStatistics>;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
		OutEnvironment.CompilerFlags.Add(CFLAG_InlineRayTracing);

		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_Y"), ThreadGroupSizeY);
		OutEnvironment.SetDefine(TEXT("ENABLE_TRACE_RAY_INLINE_TRAVERSAL_STATISTICS"), 1);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		bool bTraversalStats = PermutationVector.Get<FPrintTraversalStatistics>();
		bool bSupportsTraversalStats = FDataDrivenShaderPlatformInfo::GetSupportsRayTracingTraversalStatistics(Parameters.Platform);
		if (bTraversalStats && !bSupportsTraversalStats)
		{
			return false;
		}

		return IsRayTracingEnabledForProject(Parameters.Platform) && RHISupportsRayTracing(Parameters.Platform) && RHISupportsInlineRayTracing(Parameters.Platform);
	}

	static constexpr uint32 ThreadGroupSizeX = 8;
	static constexpr uint32 ThreadGroupSizeY = 4;
	static_assert(ThreadGroupSizeX*ThreadGroupSizeY == 32, "Current inline ray tracing implementation requires 1:1 mapping between thread groups and waves and only supports wave32 mode.");
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugTraversalCS, "/Engine/Private/RayTracing/RayTracingDebugTraversal.usf", "RayTracingDebugTraversalCS", SF_Compute);

BEGIN_SHADER_PARAMETER_STRUCT(FRayTracingPickingParameters, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
	SHADER_PARAMETER(int32, OpaqueOnly)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, PickingOutput)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InstancesExtraData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InstancesDebugData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InstanceBuffer)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, SceneUniformBuffer)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracingUniformBuffer)
END_SHADER_PARAMETER_STRUCT()

class FRayTracingPickingRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingPickingRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTracingPickingRGS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRayTracingPickingParameters, SharedParameters)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingDebug;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingPickingRGS, "/Engine/Private/RayTracing/RayTracingDebugPicking.usf", "RayTracingDebugPickingRGS", SF_RayGen);

class FRayTracingPickingCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingPickingCS)
	SHADER_USE_PARAMETER_STRUCT(FRayTracingPickingCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform) && FDataDrivenShaderPlatformInfo::GetSupportsInlineRayTracing(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
		OutEnvironment.CompilerFlags.Add(CFLAG_InlineRayTracing);

		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_Y"), ThreadGroupSizeY);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FRayTracingPickingParameters, SharedParameters)
	END_SHADER_PARAMETER_STRUCT()
	
	static constexpr uint32 ThreadGroupSizeX = 1;
	static constexpr uint32 ThreadGroupSizeY = 1;
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingPickingCS, "/Engine/Private/RayTracing/RayTracingDebugPicking.usf", "RayTracingDebugPickingCS", SF_Compute);

class FRayTracingDebugInstanceOverlapVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugInstanceOverlapVS);
	SHADER_USE_PARAMETER_STRUCT(FRayTracingDebugInstanceOverlapVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InstanceExtraDataBuffer)
		SHADER_PARAMETER(float, BoundingBoxExtentScale)
	END_SHADER_PARAMETER_STRUCT()

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugInstanceOverlapVS, "/Engine/Private/RayTracing/RayTracingDebugInstanceOverlap.usf", "InstanceOverlapMainVS", SF_Vertex);

class FRayTracingDebugInstanceOverlapPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugInstanceOverlapPS);
	SHADER_USE_PARAMETER_STRUCT(FRayTracingDebugInstanceOverlapPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)		
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugInstanceOverlapPS, "/Engine/Private/RayTracing/RayTracingDebugInstanceOverlap.usf", "InstanceOverlapMainPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FRayTracingDebugInstanceOverlapVSPSParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FRayTracingDebugInstanceOverlapVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FRayTracingDebugInstanceOverlapPS::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

class FRayTracingDebugConvertToDeviceDepthPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugConvertToDeviceDepthPS);
	SHADER_USE_PARAMETER_STRUCT(FRayTracingDebugConvertToDeviceDepthPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, InputDepth)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugConvertToDeviceDepthPS, "/Engine/Private/RayTracing/RayTracingDebugInstanceOverlap.usf", "ConvertToDeviceDepthPS", SF_Pixel);

class FRayTracingDebugBlendInstanceOverlapPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingDebugBlendInstanceOverlapPS);
	SHADER_USE_PARAMETER_STRUCT(FRayTracingDebugBlendInstanceOverlapPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, InstanceOverlap)
		SHADER_PARAMETER(float, HeatmapScale)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingDebugBlendInstanceOverlapPS, "/Engine/Private/RayTracing/RayTracingDebugInstanceOverlap.usf", "BlendInstanceOverlapPS", SF_Pixel);

class FRayTracingDebugLineAABBIndexBuffer : public FIndexBuffer
{
public:
	/**
	* Initialize the RHI for this rendering resource
	*/
	void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		static const uint16 LineIndices[12 * 2] =
		{
			0, 1,
			0, 2,
			0, 4,
			2, 3,
			3, 1,
			1, 5,
			3, 7,
			2, 6,
			6, 7,
			6, 4,
			7, 5,
			4, 5
		};

		// Create index buffer. Fill buffer with initial data upon creation
		IndexBufferRHI = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("FRayTracingDebugLineAABBIndexBuffer"), EBufferUsageFlags::Static, MakeConstArrayView(LineIndices));
	}
};

TGlobalResource<FRayTracingDebugLineAABBIndexBuffer> GRayTracingInstanceLineAABBIndexBuffer;

struct FRayTracingDebugResources : public FRenderResource
{
	const int32 MaxPickingBuffers = 4;
	int32 PickingBufferWriteIndex = 0;
	int32 PickingBufferNumPending = 0;
	TArray<FRHIGPUBufferReadback*> PickingBuffers;

	const int32 MaxHitStatsBuffers = 4;
	int32 HitStatsBufferWriteIndex = 0;
	int32 HitStatsBufferNumPending = 0;
	TArray<FRHIGPUBufferReadback*> HitStatsBuffers;

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		PickingBuffers.AddZeroed(MaxPickingBuffers);
		HitStatsBuffers.AddZeroed(MaxHitStatsBuffers);
	}

	virtual void ReleaseRHI() override
	{
		for (int32 BufferIndex = 0; BufferIndex < PickingBuffers.Num(); ++BufferIndex)
		{
			if (PickingBuffers[BufferIndex])
			{
				delete PickingBuffers[BufferIndex];
				PickingBuffers[BufferIndex] = nullptr;
			}
		}

		for (int32 BufferIndex = 0; BufferIndex < HitStatsBuffers.Num(); ++BufferIndex)
		{
			if (HitStatsBuffers[BufferIndex])
			{
				delete HitStatsBuffers[BufferIndex];
				HitStatsBuffers[BufferIndex] = nullptr;
			}
		}

		PickingBuffers.Reset();
		HitStatsBuffers.Reset();
	}
};

TGlobalResource<FRayTracingDebugResources> GRayTracingDebugResources;

static void BindRayTracingDebugHitStatsCHSMaterialBindings(
	FRHICommandList& RHICmdList,
	FRHIShaderBindingTable* SBT,
	const FRayTracingMeshCommandStorage& RayTracingMeshCommands,
	TConstArrayView<FRayTracingShaderBindingData> VisibleRayTracingShaderBindings,
	const FViewInfo& View,
	FRHIUniformBuffer* SceneUniformBuffer,
	FRHIUniformBuffer* NaniteRayTracingUniformBuffer,
	FRHIUniformBuffer* HitStatsUniformBuffer,
	FRayTracingPipelineState* PipelineState)
{
	FSceneRenderingBulkObjectAllocator Allocator;

	auto Alloc = [&](uint32 Size, uint32 Align)
	{
		return RHICmdList.Bypass()
			? Allocator.Malloc(Size, Align)
			: RHICmdList.Alloc(Size, Align);
	};

	const int32 NumTotalBindings = VisibleRayTracingShaderBindings.Num();
	const uint32 MergedBindingsSize = sizeof(FRayTracingLocalShaderBindings) * NumTotalBindings;
	FRayTracingLocalShaderBindings* Bindings = (FRayTracingLocalShaderBindings*)Alloc(MergedBindingsSize, alignof(FRayTracingLocalShaderBindings));

	struct FBinding
	{
		int32 ShaderIndexInPipeline;
		uint32 NumUniformBuffers;
		FRHIUniformBuffer** UniformBufferArray;
	};

	auto SetupBinding = [&](FRayTracingDebugHitStatsCHS::FPermutationDomain PermutationVector)
	{
		auto Shader = View.ShaderMap->GetShader<FRayTracingDebugHitStatsCHS>(PermutationVector);
		auto HitGroupShader = Shader.GetRayTracingShader();

		FBinding Binding;
		Binding.ShaderIndexInPipeline = FindRayTracingHitGroupIndex(PipelineState, HitGroupShader, true);
		Binding.NumUniformBuffers = Shader->ParameterMapInfo.UniformBuffers.Num();
		Binding.UniformBufferArray = (FRHIUniformBuffer**)Alloc(sizeof(FRHIUniformBuffer*) * Binding.NumUniformBuffers, alignof(FRHIUniformBuffer*));

		const auto& HitStatsUniformBufferParameter = Shader->GetUniformBufferParameter<FRayTracingDebugHitStatsUniformBufferParameters>();
		const auto& ViewUniformBufferParameter = Shader->GetUniformBufferParameter<FViewUniformShaderParameters>();
		const auto& SceneUniformBufferParameter = Shader->GetUniformBufferParameter<FSceneUniformParameters>();
		const auto& NaniteUniformBufferParameter = Shader->GetUniformBufferParameter<FNaniteRayTracingUniformParameters>();

		if (HitStatsUniformBufferParameter.IsBound())
		{
			check(HitStatsUniformBufferParameter.GetBaseIndex() < Binding.NumUniformBuffers);
			Binding.UniformBufferArray[HitStatsUniformBufferParameter.GetBaseIndex()] = HitStatsUniformBuffer;
		}

		if (ViewUniformBufferParameter.IsBound())
		{
			check(ViewUniformBufferParameter.GetBaseIndex() < Binding.NumUniformBuffers);
			Binding.UniformBufferArray[ViewUniformBufferParameter.GetBaseIndex()] = View.ViewUniformBuffer.GetReference();
		}

		if (SceneUniformBufferParameter.IsBound())
		{
			check(SceneUniformBufferParameter.GetBaseIndex() < Binding.NumUniformBuffers);
			Binding.UniformBufferArray[SceneUniformBufferParameter.GetBaseIndex()] = SceneUniformBuffer;
		}

		if (NaniteUniformBufferParameter.IsBound())
		{
			check(NaniteUniformBufferParameter.GetBaseIndex() < Binding.NumUniformBuffers);
			Binding.UniformBufferArray[NaniteUniformBufferParameter.GetBaseIndex()] = NaniteRayTracingUniformBuffer;
		}

		return Binding;
	};

	FRayTracingDebugHitStatsCHS::FPermutationDomain PermutationVector;

	FBinding ShaderBinding = SetupBinding(PermutationVector);
	
	const uint32 NumShaderSlotsPerGeometrySegment = SBT->GetInitializer().NumShaderSlotsPerGeometrySegment;

	uint32 BindingIndex = 0;
	for (const FRayTracingShaderBindingData DirtyShaderBinding : VisibleRayTracingShaderBindings)
	{
		const FRayTracingMeshCommand& MeshCommand = DirtyShaderBinding.GetRayTracingMeshCommand(RayTracingMeshCommands);

		const FBinding& HelperBinding = ShaderBinding;

		FRayTracingLocalShaderBindings Binding = {};
		Binding.ShaderIndexInPipeline = HelperBinding.ShaderIndexInPipeline;
		Binding.RecordIndex = DirtyShaderBinding.SBTRecordIndex;
		Binding.Geometry = DirtyShaderBinding.RayTracingGeometry;
		Binding.SegmentIndex = MeshCommand.GeometrySegmentIndex;
		Binding.UniformBuffers = HelperBinding.UniformBufferArray;
		Binding.NumUniformBuffers = HelperBinding.NumUniformBuffers;

		Bindings[BindingIndex] = Binding;
		BindingIndex++;
	}

	const bool bCopyDataToInlineStorage = false; // Storage is already allocated from RHICmdList, no extra copy necessary
	RHICmdList.SetRayTracingHitGroups(
		SBT,
		PipelineState,
		NumTotalBindings, Bindings,
		bCopyDataToInlineStorage);
}

static void BindRayTracingDebugCHSMaterialBindings(
	FRHICommandList& RHICmdList,
	FRHIShaderBindingTable* SBT,
	const FRayTracingMeshCommandStorage& RayTracingMeshCommands,
	TConstArrayView<FRayTracingShaderBindingData> VisibleRayTracingShaderBindings,
	const FViewInfo& View,
	FRHIUniformBuffer* SceneUniformBuffer,
	FRHIUniformBuffer* NaniteRayTracingUniformBuffer,
	FRayTracingPipelineState* PipelineState)
{
	FSceneRenderingBulkObjectAllocator Allocator;

	auto Alloc = [&](uint32 Size, uint32 Align)
	{
		return RHICmdList.Bypass()
			? Allocator.Malloc(Size, Align)
			: RHICmdList.Alloc(Size, Align);
	};

	const int32 NumTotalBindings = VisibleRayTracingShaderBindings.Num();
	const uint32 MergedBindingsSize = sizeof(FRayTracingLocalShaderBindings) * NumTotalBindings;
	FRayTracingLocalShaderBindings* Bindings = (FRayTracingLocalShaderBindings*)Alloc(MergedBindingsSize, alignof(FRayTracingLocalShaderBindings));

	struct FBinding
	{
		int32 ShaderIndexInPipeline;
		uint32 NumUniformBuffers;
		FRHIUniformBuffer** UniformBufferArray;
	};

	auto SetupBinding = [&](FRayTracingDebugCHS::FPermutationDomain PermutationVector)
	{
		auto Shader = View.ShaderMap->GetShader<FRayTracingDebugCHS>(PermutationVector);
		auto HitGroupShader = Shader.GetRayTracingShader();

		FBinding Binding;
		Binding.ShaderIndexInPipeline = FindRayTracingHitGroupIndex(PipelineState, HitGroupShader, true);
		Binding.NumUniformBuffers = Shader->ParameterMapInfo.UniformBuffers.Num();
		Binding.UniformBufferArray = (FRHIUniformBuffer**)Alloc(sizeof(FRHIUniformBuffer*) * Binding.NumUniformBuffers, alignof(FRHIUniformBuffer*));

		const auto& ViewUniformBufferParameter = Shader->GetUniformBufferParameter<FViewUniformShaderParameters>();
		const auto& SceneUniformBufferParameter = Shader->GetUniformBufferParameter<FSceneUniformParameters>();
		const auto& NaniteUniformBufferParameter = Shader->GetUniformBufferParameter<FNaniteRayTracingUniformParameters>();

		if (ViewUniformBufferParameter.IsBound())
		{
			check(ViewUniformBufferParameter.GetBaseIndex() < Binding.NumUniformBuffers);
			Binding.UniformBufferArray[ViewUniformBufferParameter.GetBaseIndex()] = View.ViewUniformBuffer.GetReference();
		}

		if (SceneUniformBufferParameter.IsBound())
		{
			check(SceneUniformBufferParameter.GetBaseIndex() < Binding.NumUniformBuffers);
			Binding.UniformBufferArray[SceneUniformBufferParameter.GetBaseIndex()] = SceneUniformBuffer;
		}

		if (NaniteUniformBufferParameter.IsBound())
		{
			check(NaniteUniformBufferParameter.GetBaseIndex() < Binding.NumUniformBuffers);
			Binding.UniformBufferArray[NaniteUniformBufferParameter.GetBaseIndex()] = NaniteRayTracingUniformBuffer;
		}

		return Binding;
	};

	FRayTracingDebugCHS::FPermutationDomain PermutationVector;

	PermutationVector.Set<FRayTracingDebugCHS::FNaniteRayTracing>(false);
	FBinding ShaderBinding = SetupBinding(PermutationVector);

	PermutationVector.Set<FRayTracingDebugCHS::FNaniteRayTracing>(true);
	FBinding ShaderBindingNaniteRT = SetupBinding(PermutationVector);

	const uint32 NumShaderSlotsPerGeometrySegment = SBT->GetInitializer().NumShaderSlotsPerGeometrySegment;

	uint32 BindingIndex = 0;
	for (const FRayTracingShaderBindingData DirtyShaderBinding : VisibleRayTracingShaderBindings)
	{
		const FRayTracingMeshCommand& MeshCommand = DirtyShaderBinding.GetRayTracingMeshCommand(RayTracingMeshCommands);

		const FBinding& HelperBinding = MeshCommand.IsUsingNaniteRayTracing() ? ShaderBindingNaniteRT : ShaderBinding;

		FRayTracingLocalShaderBindings Binding = {};
		Binding.ShaderIndexInPipeline = HelperBinding.ShaderIndexInPipeline;
		Binding.RecordIndex = DirtyShaderBinding.SBTRecordIndex;
		Binding.Geometry = DirtyShaderBinding.RayTracingGeometry;
		Binding.SegmentIndex = MeshCommand.GeometrySegmentIndex;
		Binding.UniformBuffers = HelperBinding.UniformBufferArray;
		Binding.NumUniformBuffers = HelperBinding.NumUniformBuffers;

		Bindings[BindingIndex] = Binding;
		BindingIndex++;
	}

	const bool bCopyDataToInlineStorage = false; // Storage is already allocated from RHICmdList, no extra copy necessary
	RHICmdList.SetRayTracingHitGroups(
		SBT,
		PipelineState,
		NumTotalBindings, Bindings,
		bCopyDataToInlineStorage);
}

static bool IsRayTracingPickingEnabled(uint32 DebugVisualizationMode)
{
	return DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_PICKER;
}

void FDeferredShadingSceneRenderer::PrepareRayTracingDebug(const FSceneViewFamily& ViewFamily, TArray<FRHIRayTracingShader*>& OutRayGenShaders)
{
	// Declare all RayGen shaders that require material closest hit shaders to be bound
	bool bEnabled = ViewFamily.EngineShowFlags.RayTracingDebug && ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::FullPipeline, ViewFamily);
	if (bEnabled)
	{
		{
			FRayTracingDebugRGS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FRayTracingDebugRGS::FUseDebugCHSType>(false);
			PermutationVector.Set<FRayTracingDebugRGS::FUseNvAPITimestamp>(GRHIGlobals.SupportsShaderTimestamp && IsRHIDeviceNVIDIA());
			auto RayGenShader = GetGlobalShaderMap(ViewFamily.GetShaderPlatform())->GetShader<FRayTracingDebugRGS>(PermutationVector);
			OutRayGenShaders.Add(RayGenShader.GetRayTracingShader());
		}
	}
}

static FRDGBufferRef RayTracingPerformPicking(
	FRDGBuilder& GraphBuilder,
	const FScene& Scene,
	const FViewInfo& View,
	TConstArrayView<FRayTracingShaderBindingData> VisibleRayTracingShaderBindings,
	FRayTracingPickingFeedback& PickingFeedback, ERayTracingSceneLayer LayerToTrace, bool bInlineRayTracing)
{
	const FRayTracingScene& RayTracingScene = Scene.RayTracingScene;

	FRDGBufferDesc PickingBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FRayTracingPickingFeedback), 1);
	PickingBufferDesc.Usage = EBufferUsageFlags(PickingBufferDesc.Usage | BUF_SourceCopy);
	FRDGBufferRef PickingBuffer = GraphBuilder.CreateBuffer(PickingBufferDesc, TEXT("RayTracingDebug.PickingBuffer"));

	FRayTracingPickingParameters SharedParameters = {};
	SharedParameters.InstancesExtraData = GraphBuilder.CreateSRV(RayTracingScene.GetInstanceExtraDataBuffer(LayerToTrace, View.GetRayTracingSceneViewHandle()));
	SharedParameters.InstancesDebugData = GraphBuilder.CreateSRV(RayTracingScene.GetInstanceDebugBuffer(LayerToTrace));
	SharedParameters.TLAS = RayTracingScene.GetLayerView(LayerToTrace, View.GetRayTracingSceneViewHandle());
	SharedParameters.OpaqueOnly = IsRayTracingVisualizeOpaqueOnly() ? 1 : 0;
	SharedParameters.InstanceBuffer = GraphBuilder.CreateSRV(RayTracingScene.GetInstanceBuffer(LayerToTrace, View.GetRayTracingSceneViewHandle()));
	SharedParameters.ViewUniformBuffer = View.ViewUniformBuffer;
	SharedParameters.SceneUniformBuffer = GetSceneUniformBufferRef(GraphBuilder, View); // TODO: use a separate params structure
	SharedParameters.NaniteRayTracingUniformBuffer = Nanite::GRayTracingManager.GetUniformBuffer();
	SharedParameters.PickingOutput = GraphBuilder.CreateUAV(PickingBuffer);

	if (bInlineRayTracing)
	{
		FRayTracingPickingCS::FParameters* InlinePassParameters = GraphBuilder.AllocParameters<FRayTracingPickingCS::FParameters>();

		InlinePassParameters->SharedParameters = SharedParameters;

		TShaderRef<FRayTracingPickingCS> ComputeShader = View.ShaderMap->GetShader<FRayTracingPickingCS>();

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RayTracingPicking (INLINE)"),
			InlinePassParameters,
			ERDGPassFlags::Compute,
			[InlinePassParameters, ComputeShader](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
			{
				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *InlinePassParameters, FIntVector(1,1,1));
			});
	}
	else
	{
		FRayTracingPickingRGS::FParameters* RayGenParameters = GraphBuilder.AllocParameters<FRayTracingPickingRGS::FParameters>();
		RayGenParameters->SharedParameters = SharedParameters;

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		auto RayGenShader = ShaderMap->GetShader<FRayTracingPickingRGS>();

		FRayTracingPipelineStateInitializer Initializer;
		Initializer.MaxPayloadSizeInBytes = GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType::RayTracingDebug);

		const FShaderBindingLayout* ShaderBindingLayout = RayTracing::GetShaderBindingLayout(Scene.GetShaderPlatform());
		if (ShaderBindingLayout)
		{
			Initializer.ShaderBindingLayout = &ShaderBindingLayout->RHILayout;
		}

		FRHIRayTracingShader* RayGenShaderTable[] = { RayGenShader.GetRayTracingShader() };
		Initializer.SetRayGenShaderTable(RayGenShaderTable);

		FRayTracingDebugCHS::FPermutationDomain PermutationVector;

		PermutationVector.Set<FRayTracingDebugCHS::FNaniteRayTracing>(false);
		auto HitGroupShader = View.ShaderMap->GetShader<FRayTracingDebugCHS>(PermutationVector);

		PermutationVector.Set<FRayTracingDebugCHS::FNaniteRayTracing>(true);
		auto HitGroupShaderNaniteRT = View.ShaderMap->GetShader<FRayTracingDebugCHS>(PermutationVector);

		FRHIRayTracingShader* HitGroupTable[] = { HitGroupShader.GetRayTracingShader(), HitGroupShaderNaniteRT.GetRayTracingShader() };
		Initializer.SetHitGroupTable(HitGroupTable);

		auto MissShader = ShaderMap->GetShader<FRayTracingDebugMS>();
		FRHIRayTracingShader* MissTable[] = { MissShader.GetRayTracingShader() };
		Initializer.SetMissShaderTable(MissTable);

		FRayTracingPipelineState* PickingPipeline = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(GraphBuilder.RHICmdList, Initializer);

		FShaderBindingTableRHIRef PickingSBT = Scene.RayTracingSBT.AllocateTransientRHI(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Allow, Initializer.GetMaxLocalBindingDataSize());

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RayTracingPicking"),
			RayGenParameters,
			ERDGPassFlags::Compute,
			[RayGenParameters, RayGenShader, &View, PickingSBT, PickingPipeline, &RayTracingMeshCommands = Scene.CachedRayTracingMeshCommands, VisibleRayTracingShaderBindings](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
				SetShaderParameters(GlobalResources, RayGenShader, *RayGenParameters);

				FRHIUniformBuffer* SceneUniformBuffer = RayGenParameters->SharedParameters.SceneUniformBuffer->GetRHI();
				FRHIUniformBuffer* NaniteRayTracingUniformBuffer = RayGenParameters->SharedParameters.NaniteRayTracingUniformBuffer->GetRHI();
				TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

				BindRayTracingDebugCHSMaterialBindings(RHICmdList, PickingSBT, RayTracingMeshCommands, VisibleRayTracingShaderBindings, View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, PickingPipeline);
				RHICmdList.SetRayTracingMissShader(PickingSBT, 0, PickingPipeline, 0 /* ShaderIndexInPipeline */, 0, nullptr, 0);
				RHICmdList.CommitShaderBindingTable(PickingSBT);

				RHICmdList.RayTraceDispatch(PickingPipeline, RayGenShader.GetRayTracingShader(), PickingSBT, GlobalResources, 1, 1);
			});
	}	

	const int32 MaxPickingBuffers = GRayTracingDebugResources.MaxPickingBuffers;

	int32& PickingBufferWriteIndex = GRayTracingDebugResources.PickingBufferWriteIndex;
	int32& PickingBufferNumPending = GRayTracingDebugResources.PickingBufferNumPending;

	TArray<FRHIGPUBufferReadback*>& PickingBuffers = GRayTracingDebugResources.PickingBuffers;

	{
		FRHIGPUBufferReadback* LatestPickingBuffer = nullptr;

		// Find latest buffer that is ready
		while (PickingBufferNumPending > 0)
		{
			uint32 Index = (PickingBufferWriteIndex + MaxPickingBuffers - PickingBufferNumPending) % MaxPickingBuffers;
			if (PickingBuffers[Index]->IsReady())
			{
				--PickingBufferNumPending;
				LatestPickingBuffer = PickingBuffers[Index];
			}
			else
			{
				break;
			}
		}

		if (LatestPickingBuffer != nullptr)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LockBuffer);
			PickingFeedback = *((const FRayTracingPickingFeedback*)LatestPickingBuffer->Lock(sizeof(FRayTracingPickingFeedback)));
			LatestPickingBuffer->Unlock();
		}
	}

	// Skip when queue is full. It is NOT safe to EnqueueCopy on a buffer that already has a pending copy
	if (PickingBufferNumPending != MaxPickingBuffers)
	{
		if (PickingBuffers[PickingBufferWriteIndex] == nullptr)
		{
			FRHIGPUBufferReadback* GPUBufferReadback = new FRHIGPUBufferReadback(TEXT("RayTracingDebug.PickingFeedback"));
			PickingBuffers[PickingBufferWriteIndex] = GPUBufferReadback;
		}

		FRHIGPUBufferReadback* PickingReadback = PickingBuffers[PickingBufferWriteIndex];

		AddEnqueueCopyPass(GraphBuilder, PickingReadback, PickingBuffer, 0u);

		PickingBufferWriteIndex = (PickingBufferWriteIndex + 1) % MaxPickingBuffers;
		PickingBufferNumPending = FMath::Min(PickingBufferNumPending + 1, MaxPickingBuffers);
	}	

	return PickingBuffer;
}

static TRDGUniformBufferRef<FRayTracingDebugHitStatsUniformBufferParameters> DebugHitStatsUniformBuffer;

struct FRayTracingSceneDebugHitStatsNameInfo
{
	uint32 PrimitiveID;
	uint32 Count;
	uint16 Offset;
	uint8  Length;
	uint8  Pad0;
};

class FRayTracingSceneHitStatsDebugRenderCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingSceneHitStatsDebugRenderCS);
	SHADER_USE_PARAMETER_STRUCT(FRayTracingSceneHitStatsDebugRenderCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, SelectedNameInfoCount)
		SHADER_PARAMETER(int32, SelectedNameCharacterCount)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint8>, SelectedPrimitiveNames)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint3>, SelectedPrimitiveNameInfos)
	END_SHADER_PARAMETER_STRUCT()

public:
	static constexpr uint32 NumThreadsPerGroup = 32U;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShaderPrint::IsSupported(Parameters.Platform) && IsRayTracingEnabledForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRayTracingSceneHitStatsDebugRenderCS, "/Engine/Private/RayTracing/RayTracingDebugHitStatsUtils.usf", "RayTracingSceneDebugHitStatsRenderCS", SF_Compute);

static void PrintTopKMostHitMessage(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View, const TArray<FRayTracingHitStatsEntry>& HitStatsArray)
{
	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);

	int32 NumPrimitives = CVarRayTracingVisualizeHitCountTopKHits.GetValueOnRenderThread();
	if (ShaderPrint::IsEnabled(View.ShaderPrintData) && NumPrimitives > 0)
	{
		// This lags by one frame, so may miss some in one frame, also overallocates since we will cull a lot.
		ShaderPrint::RequestSpaceForLines(NumPrimitives * 12);

		const uint32 MaxPrimitiveNameCount = 128u;
		check(sizeof(FRayTracingSceneDebugHitStatsNameInfo) == 12);
		TArray<FRayTracingSceneDebugHitStatsNameInfo> SelectedNameInfos;
		TArray<uint8> SelectedNames;
		SelectedNames.Reserve(NumPrimitives * 30u);

		uint32 SelectedCount = 0;
		const int32 BitsPerWord = (sizeof(uint32) * 8U);
		for (int32 HitStatsID = 0; HitStatsID < NumPrimitives; ++HitStatsID)
		{
			const uint32 PrimitiveID = HitStatsArray[HitStatsID].PrimitiveID;

			FPersistentPrimitiveIndex PersistentPrimitiveIndex;
			PersistentPrimitiveIndex.Index = PrimitiveID;

			FPrimitiveSceneInfo* SceneInfo = Scene.GetPrimitiveSceneInfo(PersistentPrimitiveIndex);

			if (SceneInfo == nullptr)
			{
				continue;
			}

			const FString OwnerName = SceneInfo->GetFullnameForDebuggingOnly();
			const uint32 NameOffset = SelectedNames.Num();
			const uint32 NameLength = OwnerName.Len();
			for (TCHAR C : OwnerName)
			{
				SelectedNames.Add(uint8(C));
			}

			FRayTracingSceneDebugHitStatsNameInfo& NameInfo = SelectedNameInfos.AddDefaulted_GetRef();
			NameInfo.PrimitiveID = PrimitiveID;
			NameInfo.Count = HitStatsArray[HitStatsID].Count;
			NameInfo.Length = NameLength;
			NameInfo.Offset = NameOffset;
			++SelectedCount;
		}

		if (SelectedNameInfos.IsEmpty())
		{
			FRayTracingSceneDebugHitStatsNameInfo& NameInfo = SelectedNameInfos.AddDefaulted_GetRef();
			NameInfo.PrimitiveID = ~0;
			NameInfo.Count = -1;
			NameInfo.Length = 4;
			NameInfo.Offset = 0;
			SelectedNames.Add(uint8('N'));
			SelectedNames.Add(uint8('o'));
			SelectedNames.Add(uint8('n'));
			SelectedNames.Add(uint8('e'));
		}

		// Request more characters for printing if needed
		ShaderPrint::RequestSpaceForCharacters(SelectedNames.Num() + SelectedCount * 48u);

		FRDGBufferRef SelectedPrimitiveNames = CreateVertexBuffer(GraphBuilder, TEXT("RayTracingDebug.HitStats.SelectedPrimitiveNames"), 
			FRDGBufferDesc::CreateBufferDesc(1, SelectedNames.Num()), SelectedNames.GetData(), SelectedNames.Num());
		FRDGBufferRef SelectedPrimitiveNameInfos = CreateStructuredBuffer(GraphBuilder, TEXT("RayTracingDebug.HitStats.SelectedPrimitiveNameInfos"), SelectedNameInfos);

		FRayTracingSceneHitStatsDebugRenderCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRayTracingSceneHitStatsDebugRenderCS::FParameters>();
		ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, PassParameters->ShaderPrintUniformBuffer);
		PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder, View);
		PassParameters->SelectedNameInfoCount = SelectedCount;
		PassParameters->SelectedNameCharacterCount = SelectedCount > 0 ? SelectedNames.Num() : 0;
		PassParameters->SelectedPrimitiveNameInfos = GraphBuilder.CreateSRV(SelectedPrimitiveNameInfos);
		PassParameters->SelectedPrimitiveNames = GraphBuilder.CreateSRV(SelectedPrimitiveNames, PF_R8_UINT);

		auto ComputeShader = View.ShaderMap->GetShader<FRayTracingSceneHitStatsDebugRenderCS>();

		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("RayTracingDebug::TopKHitStatsInfo"),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(NumPrimitives, FRayTracingSceneHitStatsDebugRenderCS::NumThreadsPerGroup)
		);
	}
}

static FRDGBufferRef RayTracingPerformHitStatsPerPrimitive(
	FRDGBuilder& GraphBuilder,
	const FScene& Scene,
	const FViewInfo& View,
	TConstArrayView<FRayTracingShaderBindingData> VisibleRayTracingShaderBindings,
	ERayTracingSceneLayer LayerToTrace)
{
	const FRayTracingScene& RayTracingScene = Scene.RayTracingScene;

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	auto RayGenShader = ShaderMap->GetShader<FRayTracingDebugHitStatsRGS>();

	FRayTracingPipelineStateInitializer Initializer;
	Initializer.MaxPayloadSizeInBytes = GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType::RayTracingDebug);
	
	const FShaderBindingLayout* ShaderBindingLayout = RayTracing::GetShaderBindingLayout(Scene.GetShaderPlatform());
	if (ShaderBindingLayout)
	{
		Initializer.ShaderBindingLayout = &ShaderBindingLayout->RHILayout;
	}

	FRHIRayTracingShader* RayGenShaderTable[] = { RayGenShader.GetRayTracingShader() };
	Initializer.SetRayGenShaderTable(RayGenShaderTable);

	FRayTracingDebugHitStatsCHS::FPermutationDomain PermutationVector;

	auto HitGroupShader = View.ShaderMap->GetShader<FRayTracingDebugHitStatsCHS>(PermutationVector);

	FRHIRayTracingShader* HitGroupTable[] = { HitGroupShader.GetRayTracingShader() };
	Initializer.SetHitGroupTable(HitGroupTable);

	auto MissShader = ShaderMap->GetShader<FRayTracingDebugMS>();
	FRHIRayTracingShader* MissTable[] = { MissShader.GetRayTracingShader() };
	Initializer.SetMissShaderTable(MissTable);

	FRayTracingPipelineState* HitStatsPerPrimitivePipeline = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(GraphBuilder.RHICmdList, Initializer);

	FShaderBindingTableRHIRef HitStatsSBT = Scene.RayTracingSBT.AllocateTransientRHI(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Allow, Initializer.GetMaxLocalBindingDataSize());

	// TODO: Should check RayTracingScene for actual number of instances instead of max number in FRHIRayTracingScene initializer
	const uint32 NumInstancesInTLAS = FMath::Max(RayTracingScene.GetRHIRayTracingSceneChecked(LayerToTrace, View.GetRayTracingSceneViewHandle())->GetInitializer().MaxNumInstances, (uint32)CVarRayTracingVisualizeHitCountTopKHits.GetValueOnRenderThread());
	FRDGBufferDesc HitStatsPerPrimitiveBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FRayTracingHitStatsEntry), NumInstancesInTLAS);
	HitStatsPerPrimitiveBufferDesc.Usage = EBufferUsageFlags(HitStatsPerPrimitiveBufferDesc.Usage | BUF_SourceCopy);
	FRDGBufferRef HitStatsBuffer = GraphBuilder.CreateBuffer(HitStatsPerPrimitiveBufferDesc, TEXT("RayTracingDebug.HitStatsBuffer"));
	
	FRayTracingDebugHitStatsUniformBufferParameters* DebugHitStatsUniformBufferParameters = GraphBuilder.AllocParameters<FRayTracingDebugHitStatsUniformBufferParameters>();
	DebugHitStatsUniformBufferParameters->HitStatsOutput = GraphBuilder.CreateUAV(HitStatsBuffer);
	DebugHitStatsUniformBuffer = GraphBuilder.CreateUniformBuffer(DebugHitStatsUniformBufferParameters);

	FRayTracingDebugHitStatsRGS::FParameters* RayGenParameters = GraphBuilder.AllocParameters<FRayTracingDebugHitStatsRGS::FParameters>();
	RayGenParameters->TLAS = RayTracingScene.GetLayerView(LayerToTrace, View.GetRayTracingSceneViewHandle());
	RayGenParameters->OpaqueOnly = IsRayTracingVisualizeOpaqueOnly() ? 1 : 0;
	RayGenParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	RayGenParameters->SceneUniformBuffer = GetSceneUniformBufferRef(GraphBuilder, View); // TODO: use a separate params structure
	RayGenParameters->NaniteRayTracingUniformBuffer = Nanite::GRayTracingManager.GetUniformBuffer();
	RayGenParameters->RayTracingDebugHitStatsUniformBuffer = DebugHitStatsUniformBuffer;

	AddClearUAVPass(GraphBuilder, DebugHitStatsUniformBufferParameters->HitStatsOutput, 0);
	
	FIntRect ViewRect = View.ViewRect;
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("RayTracingHitStats"),
		RayGenParameters,
		ERDGPassFlags::Compute,
		[RayGenParameters, RayGenShader, &View, HitStatsSBT, HitStatsPerPrimitivePipeline, ViewRect, &RayTracingMeshCommands = Scene.CachedRayTracingMeshCommands, VisibleRayTracingShaderBindings](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
			SetShaderParameters(GlobalResources, RayGenShader, *RayGenParameters);

			FRHIUniformBuffer* SceneUniformBuffer = RayGenParameters->SceneUniformBuffer->GetRHI();
			FRHIUniformBuffer* NaniteRayTracingUniformBuffer = RayGenParameters->NaniteRayTracingUniformBuffer->GetRHI();
			TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

			BindRayTracingDebugHitStatsCHSMaterialBindings(RHICmdList, HitStatsSBT, RayTracingMeshCommands, VisibleRayTracingShaderBindings, View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, DebugHitStatsUniformBuffer->GetRHI(), HitStatsPerPrimitivePipeline);
			RHICmdList.SetRayTracingMissShader(HitStatsSBT, 0, HitStatsPerPrimitivePipeline, 0 /* ShaderIndexInPipeline */, 0, nullptr, 0);
			RHICmdList.CommitShaderBindingTable(HitStatsSBT);

			RHICmdList.RayTraceDispatch(HitStatsPerPrimitivePipeline, RayGenShader.GetRayTracingShader(), HitStatsSBT, GlobalResources, ViewRect.Size().X, ViewRect.Size().Y);
		});

	TArray<FRayTracingHitStatsEntry> HitStatsKeyValuePairs;	
	FRHIGPUBufferReadback* CPUHitStatsBuffer = nullptr;
	const int32 MaxHitStatsBuffers = GRayTracingDebugResources.MaxHitStatsBuffers;

	int32& HitStatsBufferWriteIndex = GRayTracingDebugResources.HitStatsBufferWriteIndex;
	int32& HitStatsBufferNumPending = GRayTracingDebugResources.HitStatsBufferNumPending;

	TArray<FRHIGPUBufferReadback*>& HitStatsBuffers = GRayTracingDebugResources.HitStatsBuffers;

	{
		FRHIGPUBufferReadback* LatestHitStatsBuffer = nullptr;

		// Find latest buffer that is ready
		while (HitStatsBufferNumPending > 0)
		{
			uint32 Index = (HitStatsBufferWriteIndex + MaxHitStatsBuffers - HitStatsBufferNumPending) % MaxHitStatsBuffers;
			if (HitStatsBuffers[Index]->IsReady())
			{
				--HitStatsBufferNumPending;
				LatestHitStatsBuffer = HitStatsBuffers[Index];
			}
			else
			{
				break;
			}
		}

		if (LatestHitStatsBuffer != nullptr)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LockBuffer);
			const uint32 BufferSize = LatestHitStatsBuffer->GetGPUSizeBytes();
			const FRayTracingHitStatsEntry* BufferAddr = (const FRayTracingHitStatsEntry*)LatestHitStatsBuffer->Lock(BufferSize);
			for (uint32 Index = 0; Index < BufferSize / sizeof(FRayTracingHitStatsEntry); Index++)
			{
				HitStatsKeyValuePairs.Add(BufferAddr[Index]);
			}
			LatestHitStatsBuffer->Unlock();
		}
	}

	// Skip when queue is full. It is NOT safe to EnqueueCopy on a buffer that already has a pending copy
	if (HitStatsBufferNumPending != MaxHitStatsBuffers)
	{
		if (HitStatsBuffers[HitStatsBufferWriteIndex] == nullptr)
		{
			FRHIGPUBufferReadback* GPUBufferReadback = new FRHIGPUBufferReadback(TEXT("RayTracingDebug.HitStatsFeedback"));
			HitStatsBuffers[HitStatsBufferWriteIndex] = GPUBufferReadback;
		}

		FRHIGPUBufferReadback* HitStatsReadback = HitStatsBuffers[HitStatsBufferWriteIndex];

		AddEnqueueCopyPass(GraphBuilder, HitStatsReadback, HitStatsBuffer, 0u);
		 
		HitStatsBufferWriteIndex = (HitStatsBufferWriteIndex + 1) % MaxHitStatsBuffers;
		HitStatsBufferNumPending = FMath::Min(HitStatsBufferNumPending + 1, MaxHitStatsBuffers);
	}


	if (HitStatsKeyValuePairs.Num() > 0) 
	{
		HitStatsKeyValuePairs.Sort([](const FRayTracingHitStatsEntry& A, const FRayTracingHitStatsEntry& B) {
			return A.Count > B.Count;
			});
		PrintTopKMostHitMessage(GraphBuilder, Scene, View, HitStatsKeyValuePairs);
		return HitStatsBuffer;
	}

	return HitStatsBuffer;
}


static void RayTracingDrawInstances(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef OutputTexture, FRDGTextureRef SceneDepthTexture, FRDGBufferRef InstanceExtraDataBuffer, uint32 NumInstances, bool bWireframe)
{
	TShaderMapRef<FRayTracingDebugInstanceOverlapVS> VertexShader(View.ShaderMap);
	TShaderMapRef<FRayTracingDebugInstanceOverlapPS> PixelShader(View.ShaderMap);

	FRayTracingDebugInstanceOverlapVSPSParameters* PassParameters = GraphBuilder.AllocParameters<FRayTracingDebugInstanceOverlapVSPSParameters>();
	PassParameters->VS.View = View.ViewUniformBuffer;
	PassParameters->VS.Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
	PassParameters->VS.InstanceExtraDataBuffer = GraphBuilder.CreateSRV(InstanceExtraDataBuffer);
	PassParameters->VS.BoundingBoxExtentScale = CVarRayTracingVisualizeInstanceOverlapBoundingBoxScale.GetValueOnRenderThread();

	PassParameters->PS.View = View.ViewUniformBuffer;

	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, bWireframe ? ERenderTargetLoadAction::ELoad : ERenderTargetLoadAction::EClear);
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilNop);

	ValidateShaderParameters(PixelShader, PassParameters->PS);
	ClearUnusedGraphResources(PixelShader, &PassParameters->PS);
	ValidateShaderParameters(VertexShader, PassParameters->VS);
	ClearUnusedGraphResources(VertexShader, &PassParameters->VS);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("RayTracingDebug::DrawInstances"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, VertexShader, PixelShader, PassParameters, NumInstances, bWireframe](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
			GraphicsPSOInit.BlendState = bWireframe ? TStaticBlendState<CW_RGB>::GetRHI() : TStaticBlendState<CW_RED, BO_Add, BF_One, BF_One>::GetRHI();
			GraphicsPSOInit.RasterizerState = bWireframe ? TStaticRasterizerState<FM_Wireframe, CM_None>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
			GraphicsPSOInit.PrimitiveType = bWireframe ? PT_LineList : PT_TriangleList;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

			RHICmdList.SetStreamSource(0, GetUnitCubeVertexBuffer(), 0);

			const FBufferRHIRef IndexBufferRHI = bWireframe ? GRayTracingInstanceLineAABBIndexBuffer.IndexBufferRHI  : GetUnitCubeIndexBuffer();
			RHICmdList.DrawIndexedPrimitive(IndexBufferRHI, 0, 0, 8, 0, 12, NumInstances);
		});
}

static void DrawInstanceOverlap(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View, FRDGTextureRef SceneColorTexture, FRDGTextureRef InputDepthTexture, ERayTracingSceneLayer LayerToTrace)
{
	FRDGTextureRef SceneDepthTexture = GraphBuilder.CreateTexture(
		FRDGTextureDesc::Create2D(
			SceneColorTexture->Desc.Extent,
			PF_DepthStencil,
			FClearValueBinding::DepthFar,
			TexCreate_DepthStencilTargetable | TexCreate_InputAttachmentRead | TexCreate_ShaderResource),
		TEXT("RayTracingDebug::SceneDepth"));

	// Convert from depth texture to depth buffer for depth testing
	{
		FRayTracingDebugConvertToDeviceDepthPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRayTracingDebugConvertToDeviceDepthPS::FParameters>();
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthTexture, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilNop);

		PassParameters->InputDepth = GraphBuilder.CreateSRV(InputDepthTexture);

		TShaderMapRef<FRayTracingDebugConvertToDeviceDepthPS> PixelShader(View.ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("RayTracingDebug::ConvertToDeviceDepth"),
			PixelShader,
			PassParameters,
			View.ViewRect,
			TStaticBlendState<>::GetRHI(),
			TStaticRasterizerState<FM_Solid, CM_None>::GetRHI(),
			TStaticDepthStencilState<true, CF_Always>::GetRHI());
	}

	// Accumulate instance overlap
	FRDGTextureDesc InstanceOverlapTextureDesc = FRDGTextureDesc::Create2D(SceneColorTexture->Desc.Extent, PF_R32_FLOAT, FClearValueBinding::Black, TexCreate_ShaderResource | TexCreate_RenderTargetable);
	FRDGTextureRef InstanceOverlapTexture = GraphBuilder.CreateTexture(InstanceOverlapTextureDesc, TEXT("RayTracingDebug::InstanceOverlap"));
	
	RayTracingDrawInstances(
		GraphBuilder,
		View,
		InstanceOverlapTexture,
		SceneDepthTexture,
		Scene.RayTracingScene.GetInstanceExtraDataBuffer(LayerToTrace, View.GetRayTracingSceneViewHandle()),
		Scene.RayTracingScene.GetNumNativeInstances(LayerToTrace, View.GetRayTracingSceneViewHandle()),
		false);

	// Calculate heatmap of instance overlap and blend it on top of ray tracing debug output
	{
		FRayTracingDebugBlendInstanceOverlapPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRayTracingDebugBlendInstanceOverlapPS::FParameters>();

		PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColorTexture, ERenderTargetLoadAction::ELoad, 0);

		PassParameters->InstanceOverlap = GraphBuilder.CreateSRV(InstanceOverlapTexture);
		PassParameters->HeatmapScale = CVarRayTracingVisualizeInstanceOverlapScale.GetValueOnRenderThread();

		TShaderMapRef<FRayTracingDebugBlendInstanceOverlapPS> PixelShader(View.ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("RayTracingDebug::BlendInstanceOverlap"),
			PixelShader,
			PassParameters,
			View.ViewRect,
			TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha>::GetRHI(),
			TStaticRasterizerState<FM_Solid, CM_None>::GetRHI(),
			TStaticDepthStencilState<false, CF_Always>::GetRHI());
	}

	// Draw instance AABB with lines
	if (CVarRayTracingVisualizeInstanceOverlapShowWireframe.GetValueOnRenderThread() != 0)
	{
		RayTracingDrawInstances(
			GraphBuilder,
			View,
			SceneColorTexture,
			SceneDepthTexture,
			Scene.RayTracingScene.GetInstanceExtraDataBuffer(LayerToTrace, View.GetRayTracingSceneViewHandle()),
			Scene.RayTracingScene.GetNumNativeInstances(LayerToTrace, View.GetRayTracingSceneViewHandle()),
			true);
	}
}

static FName GetRaytracingDebugViewMode(const FSceneView& View)
{
	check(IsInParallelRenderingThread());
	FRayTracingVisualizationData& RayTracingVisualizationData = GetRayTracingVisualizationData();
	FName CurrentMode = RayTracingVisualizationData.ApplyOverrides(View.CurrentRayTracingDebugVisualizationMode);

	// Use barycentrics as default when mode is not specified
	return CurrentMode != NAME_None ? CurrentMode : TEXT("Barycentrics");
}

static uint32 GetRaytracingDebugViewModeID(const FSceneView& View)
{
	return GetRayTracingVisualizationData().GetModeID(GetRaytracingDebugViewMode(View));
}

bool RaytracingDebugViewModeNeedsTonemapping(const FSceneView& View)
{
	return GetRayTracingVisualizationData().GetModeTonemapped(GetRaytracingDebugViewMode(View));
}

bool HasRaytracingDebugViewModeRaytracedOverlay(const FSceneViewFamily& ViewFamily)
{
	bool bAnySubstrate = false;
	bool bAnyTraversalSecondary = false;

	for (const FSceneView* View : ViewFamily.Views)
	{
		const uint32 Mode = View != nullptr ? GetRaytracingDebugViewModeID(*View) : UINT32_MAX;

		bAnySubstrate |= Mode == RAY_TRACING_DEBUG_VIZ_SUBSTRATE_DATA;

		// can't get WorldNormal in inline ray tracing so need GBuffer Depth/WorldNormal rendered by raster passes to generate secondary rays
		bAnyTraversalSecondary |= Mode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_NODE;
		bAnyTraversalSecondary |= Mode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_TRIANGLE;
		bAnyTraversalSecondary |= Mode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_ALL;
		bAnyTraversalSecondary |= Mode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_CLUSTER;
		bAnyTraversalSecondary |= Mode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_STATISTICS;
	}

	return !bAnySubstrate && !bAnyTraversalSecondary;
}

extern void RenderRayTracingBarycentrics(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View, FRDGTextureRef SceneColor, ERayTracingSceneLayer LayerToTrace, bool bVisualizeProceduralPrimitives, bool bOutputTiming);

extern void RenderRayTracingPrimaryRaysView(
	FRDGBuilder& GraphBuilder, const FViewInfo& View, const FSceneTextures& SceneTextures,
	FRDGTextureRef* InOutColorTexture, FRDGTextureRef* InOutRayHitDistanceTexture,
	ERayTracingSceneLayer LayerToTrace, int32 SamplePerPixel, int32 HeightFog, float ResolutionFraction, ERayTracingPrimaryRaysFlag Flags);

void RenderRayTracingDebug(FRDGBuilder& GraphBuilder, const FScene& Scene, const FViewInfo& View, FSceneTextures& SceneTextures, TConstArrayView<FRayTracingShaderBindingData> VisibleRayTracingShaderBindings, FRayTracingPickingFeedback& PickingFeedback)
{
	const EShaderPlatform ShaderPlatform = Scene.GetShaderPlatform();

	FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures.UniformBuffer);
	FRDGTextureRef SceneColorTexture = SceneTextures.Color.Target;

	const ERayTracingSceneLayer LayerToTrace = CVarRayTracingVisualizeFarField.GetValueOnRenderThread() ? ERayTracingSceneLayer::FarField : ERayTracingSceneLayer::Base;

	const uint32 DebugVisualizationMode = GetRaytracingDebugViewModeID(View);
	const bool bSubstratePixelDebugEnable = DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_SUBSTRATE_DATA && View.ViewState && Substrate::IsSubstrateEnabled();

	if (bSubstratePixelDebugEnable)
	{
		ShaderPrint::SetEnabled(true);
		ShaderPrint::RequestSpaceForLines(1024);
		ShaderPrint::RequestSpaceForCharacters(1024);
	}

	if (DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_BARYCENTRICS || DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TIMING_TRAVERSAL)
	{
		return RenderRayTracingBarycentrics(GraphBuilder, Scene, View, SceneColorTexture, LayerToTrace, (bool)GVisualizeProceduralPrimitives, /*bOutputTiming*/ DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TIMING_TRAVERSAL);
	}

	const FRayTracingScene& RayTracingScene = Scene.RayTracingScene;

	if (IsRayTracingDebugTraversalMode(DebugVisualizationMode) && ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::Inline, View))
	{
		const bool bPrintTraversalStats = FDataDrivenShaderPlatformInfo::GetSupportsRayTracingTraversalStatistics(GMaxRHIShaderPlatform)
			&& (DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_STATISTICS || DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_STATISTICS);

		FRayTracingDebugTraversalCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRayTracingDebugTraversalCS::FParameters>();
		PassParameters->Output = GraphBuilder.CreateUAV(SceneColorTexture);
		PassParameters->TLAS = RayTracingScene.GetLayerView(LayerToTrace, View.GetRayTracingSceneViewHandle());
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->Scene = View.GetSceneUniforms().GetBuffer(GraphBuilder);
		PassParameters->NaniteRasterUniformBuffer = CreateDebugNaniteRasterUniformBuffer(GraphBuilder);
		PassParameters->NaniteShadingUniformBuffer = CreateDebugNaniteShadingUniformBuffer(GraphBuilder);

		PassParameters->VisualizationMode = DebugVisualizationMode;
		PassParameters->TraversalBoxScale = CVarRayTracingVisualizeTraversalBoxScale.GetValueOnAnyThread();
		PassParameters->TraversalClusterScale = CVarRayTracingVisualizeTraversalClusterScale.GetValueOnAnyThread();
		PassParameters->TraversalTriangleScale = CVarRayTracingVisualizeTraversalTriangleScale.GetValueOnAnyThread();

		PassParameters->RTDebugVisualizationNaniteCutError = 0.0f;

		FBlueNoise BlueNoise = GetBlueNoiseGlobalParameters();
		PassParameters->BlueNoise = CreateUniformBufferImmediate(BlueNoise, EUniformBufferUsage::UniformBuffer_SingleDraw);

		PassParameters->SceneTextures = SceneTextureParameters;
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->NormalBias = GetRaytracingMaxNormalBias();
		PassParameters->TraceDistance = 20000.0f;

		RaytracingTraversalStatistics::FTraceRayInlineStatisticsData TraversalData;
		if (bPrintTraversalStats)
		{
			RaytracingTraversalStatistics::Init(GraphBuilder, TraversalData);
			RaytracingTraversalStatistics::SetParameters(GraphBuilder, TraversalData, PassParameters->TraversalStatistics);
		}

		FIntRect ViewRect = View.ViewRect;

		RDG_EVENT_SCOPE_STAT(GraphBuilder, RayTracingDebug, "RayTracingDebug");

		const FIntPoint GroupSize(FRayTracingDebugTraversalCS::ThreadGroupSizeX, FRayTracingDebugTraversalCS::ThreadGroupSizeY);
		const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(ViewRect.Size(), GroupSize);

		FRayTracingDebugTraversalCS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRayTracingDebugTraversalCS::FSupportProceduralPrimitive>((bool)GVisualizeProceduralPrimitives);
		PermutationVector.Set<FRayTracingDebugTraversalCS::FPrintTraversalStatistics>(bPrintTraversalStats);
		
		TShaderRef<FRayTracingDebugTraversalCS> ComputeShader = View.ShaderMap->GetShader<FRayTracingDebugTraversalCS>(PermutationVector);

		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RayTracingDebug"), ComputeShader, PassParameters, GroupCount);

		if (bPrintTraversalStats)
		{
			RaytracingTraversalStatistics::AddPrintPass(GraphBuilder, View, TraversalData);
		}

		return;
	}

	const bool bInlineRayTracing = ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::Inline, View);
	const bool bRayTracingPipeline = ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::FullPipeline, View);
	if (!bRayTracingPipeline && !(bInlineRayTracing && RayTracingDebugModeSupportsInline(DebugVisualizationMode)))
	{
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(SceneColorTexture), FLinearColor::Black);
		return;
	}

	if (DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_PRIMARY_RAYS)
	{
		FRDGTextureRef OutputColor = nullptr;
		FRDGTextureRef HitDistanceTexture = nullptr;

		RenderRayTracingPrimaryRaysView(
			GraphBuilder, View, SceneTextures, 
			&OutputColor, &HitDistanceTexture, LayerToTrace, 1, 1, 1, ERayTracingPrimaryRaysFlag::PrimaryView);

		AddDrawTexturePass(GraphBuilder, View, OutputColor, SceneColorTexture, View.ViewRect.Min, View.ViewRect.Min, View.ViewRect.Size());
		return;
	}

	FRDGBufferRef PickingBuffer = nullptr;
	FRDGBufferRef StatsBuffer = nullptr;
	if (IsRayTracingPickingEnabled(DebugVisualizationMode)
		&& RayTracingScene.GetInstanceExtraDataBuffer(LayerToTrace, View.GetRayTracingSceneViewHandle()) != nullptr
		&& RayTracingScene.GetInstanceDebugBuffer(LayerToTrace) != nullptr)
	{
		PickingBuffer = RayTracingPerformPicking(GraphBuilder, Scene, View, VisibleRayTracingShaderBindings, PickingFeedback, LayerToTrace, bInlineRayTracing);
	}
	else
	{
		PickingBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FRayTracingPickingFeedback));
	}

	if (DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_HITCOUNT_PER_INSTANCE)
	{
		StatsBuffer = RayTracingPerformHitStatsPerPrimitive(GraphBuilder, Scene, View, VisibleRayTracingShaderBindings, LayerToTrace);
	}
	else 
	{
		FRDGBufferDesc StatsBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32) * 2, 1);
		StatsBuffer = GraphBuilder.CreateBuffer(StatsBufferDesc, TEXT("RayTracingDebug.HitStatsBuffer"));

		FRayTracingDebugHitStatsUniformBufferParameters* DebugHitStatsUniformBufferParameters = GraphBuilder.AllocParameters<FRayTracingDebugHitStatsUniformBufferParameters>();
		DebugHitStatsUniformBufferParameters->HitStatsOutput = GraphBuilder.CreateUAV(StatsBuffer);
		DebugHitStatsUniformBuffer = GraphBuilder.CreateUniformBuffer(DebugHitStatsUniformBufferParameters);
		AddClearUAVPass(GraphBuilder, DebugHitStatsUniformBufferParameters->HitStatsOutput, 0);
	}

	FRDGBufferRef InstanceExtraDataBuffer = RayTracingScene.GetInstanceExtraDataBuffer(LayerToTrace, View.GetRayTracingSceneViewHandle());
	if (InstanceExtraDataBuffer == nullptr)
	{
		InstanceExtraDataBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FRayTracingInstanceExtraData));
	}

	FRDGBufferRef InstanceDebugBuffer = RayTracingScene.GetInstanceDebugBuffer(LayerToTrace);
	if (InstanceDebugBuffer == nullptr)
	{
		InstanceDebugBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FRayTracingInstanceDebugData));
	}

	FRDGBufferRef InstanceBuffer = RayTracingScene.GetInstanceBuffer(LayerToTrace, View.GetRayTracingSceneViewHandle());
	if (InstanceBuffer == nullptr)
	{
		InstanceBuffer = GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32));
	}

	const bool bRequiresDebugCHS = RequiresRayTracingDebugCHS(DebugVisualizationMode);

	const uint32 NumInstances = RayTracingScene.GetNumNativeInstances(LayerToTrace, View.GetRayTracingSceneViewHandle());

	FRayTracingDebugParameters SharedParameters;
	SharedParameters.VisualizationMode = DebugVisualizationMode;
	SharedParameters.PickerDomain = CVarRayTracingVisualizePickerDomain.GetValueOnRenderThread();
	SharedParameters.TimingScale = GetRayTracingDebugTimingScale();
	SharedParameters.OpaqueOnly = IsRayTracingVisualizeOpaqueOnly() ? 1 : 0;
	SharedParameters.TriangleHitCountMaxThreshold = FMath::Clamp((float)CVarRayTracingVisualizeHitCountMaxThreshold.GetValueOnRenderThread(), 1, 100000);
	SharedParameters.TriangleHitCountPerInstanceMaxThreshold = FMath::Max(1, CVarRayTracingVisualizeHitCountPerInstanceMaxThreshold.GetValueOnRenderThread());
	SharedParameters.RayTracingDebugHitStatsUniformBuffer = DebugHitStatsUniformBuffer;
	SharedParameters.LightGridPacked = View.RayTracingLightGridUniformBuffer;
	SharedParameters.TopKMostHitInstances = CVarRayTracingVisualizeHitCountTopKHits.GetValueOnRenderThread();
	SharedParameters.NumTotalInstances = NumInstances;
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, SharedParameters.ShaderPrintUniformBuffer);

	SharedParameters.RayTracingSceneMetadata = View.InlineRayTracingBindingDataBuffer ? GraphBuilder.CreateSRV(View.InlineRayTracingBindingDataBuffer) : nullptr;

	FSubstrateViewDebugData::FTransientPixelDebugBuffer SubstratePixelDebugBuffer;
	if (bSubstratePixelDebugEnable)
	{
		FSubstrateViewDebugData& SubstrateViewDebugData = View.ViewState->GetSubstrateViewDebugData();
		SubstratePixelDebugBuffer = SubstrateViewDebugData.CreateTransientPixelDebugBuffer(GraphBuilder);
	}
	else
	{
		SubstratePixelDebugBuffer = FSubstrateViewDebugData::CreateDummyPixelDebugBuffer(GraphBuilder);
	}
	SharedParameters.SubstrateDebugDataSizeInUints = SubstratePixelDebugBuffer.DebugDataSizeInUints;
	SharedParameters.SubstrateDebugDataUAV = SubstratePixelDebugBuffer.DebugDataUAV;

	// If we don't output depth, create dummy 1x1 texture
	const bool bOutputDepth = DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP;

	FRDGTextureDesc OutputDepthTextureDesc = FRDGTextureDesc::Create2D(
		bOutputDepth ? SceneColorTexture->Desc.Extent : FIntPoint(1, 1),
		PF_R32_FLOAT,
		FClearValueBinding::Black,
		TexCreate_ShaderResource | TexCreate_UAV);
	FRDGTextureRef OutputDepthTexture = GraphBuilder.CreateTexture(OutputDepthTextureDesc, TEXT("RayTracingDebug::Depth"));
	SharedParameters.OutputDepth = GraphBuilder.CreateUAV(OutputDepthTexture);
	
	SharedParameters.InstancesExtraData = GraphBuilder.CreateSRV(InstanceExtraDataBuffer);
	SharedParameters.InstancesDebugData = GraphBuilder.CreateSRV(InstanceDebugBuffer);
	SharedParameters.InstanceBuffer = GraphBuilder.CreateSRV(InstanceBuffer);
	SharedParameters.PickingBuffer = GraphBuilder.CreateSRV(PickingBuffer);
	SharedParameters.TLAS = RayTracingScene.GetLayerView(LayerToTrace, View.GetRayTracingSceneViewHandle());
	SharedParameters.ViewUniformBuffer = View.ViewUniformBuffer;
	SharedParameters.Output = GraphBuilder.CreateUAV(SceneColorTexture);
	SharedParameters.TopKHitStats = GraphBuilder.CreateSRV(StatsBuffer);

	SharedParameters.SceneUniformBuffer = GetSceneUniformBufferRef(GraphBuilder, View); // TODO: use a separate params structure
	SharedParameters.NaniteRayTracingUniformBuffer = Nanite::GRayTracingManager.GetUniformBuffer();

	FIntRect ViewRect = View.ViewRect;

	RDG_EVENT_SCOPE_STAT(GraphBuilder, RayTracingDebug, "RayTracingDebug");

	if (bRayTracingPipeline)
	{
		FRayTracingDebugRGS::FParameters* RayGenParameters = GraphBuilder.AllocParameters<FRayTracingDebugRGS::FParameters>();

		RayGenParameters->SharedParameters = SharedParameters;

		FRayTracingDebugRGS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRayTracingDebugRGS::FUseDebugCHSType>(bRequiresDebugCHS);
		PermutationVector.Set<FRayTracingDebugRGS::FUseNvAPITimestamp>(GRHIGlobals.SupportsShaderTimestamp && IsRHIDeviceNVIDIA());

		auto RayGenShader = View.ShaderMap->GetShader<FRayTracingDebugRGS>(PermutationVector);

		FRayTracingPipelineState* Pipeline = View.MaterialRayTracingData.PipelineState;
		FShaderBindingTableRHIRef SBT = View.MaterialRayTracingData.ShaderBindingTable;
		bool bRequiresBindings = false;

		if (bRequiresDebugCHS)
		{
			FRayTracingPipelineStateInitializer Initializer;
			Initializer.MaxPayloadSizeInBytes = GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType::RayTracingDebug);

			const FShaderBindingLayout* ShaderBindingLayout = RayTracing::GetShaderBindingLayout(ShaderPlatform);
			if (ShaderBindingLayout)
			{
				Initializer.ShaderBindingLayout = &ShaderBindingLayout->RHILayout;
			}

			FRHIRayTracingShader* RayGenShaderTable[] = { RayGenShader.GetRayTracingShader() };
			Initializer.SetRayGenShaderTable(RayGenShaderTable);

			FRayTracingDebugCHS::FPermutationDomain PermutationVectorCHS;

			PermutationVectorCHS.Set<FRayTracingDebugCHS::FNaniteRayTracing>(false);
			auto HitGroupShader = View.ShaderMap->GetShader<FRayTracingDebugCHS>(PermutationVectorCHS);

			PermutationVectorCHS.Set<FRayTracingDebugCHS::FNaniteRayTracing>(true);
			auto HitGroupShaderNaniteRT = View.ShaderMap->GetShader<FRayTracingDebugCHS>(PermutationVectorCHS);

			// auto AnyHitShader = ShaderMap->GetShader<FRayTracingDebugAHS>();

			FRHIRayTracingShader* HitGroupTable[] = { HitGroupShader.GetRayTracingShader(), HitGroupShaderNaniteRT.GetRayTracingShader() };
			Initializer.SetHitGroupTable(HitGroupTable);

			auto MissShader = View.ShaderMap->GetShader<FRayTracingDebugMS>();
			FRHIRayTracingShader* MissTable[] = { MissShader.GetRayTracingShader() };
			Initializer.SetMissShaderTable(MissTable);

			Pipeline = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(GraphBuilder.RHICmdList, Initializer);

			SBT = Scene.RayTracingSBT.AllocateTransientRHI(GraphBuilder.RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Allow, Initializer.GetMaxLocalBindingDataSize());

			bRequiresBindings = true;
		}

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RayTracingDebug"),
			RayGenParameters,
			ERDGPassFlags::Compute,
			[RayGenParameters, RayGenShader, &View, Pipeline, SBT, ViewRect, &RayTracingMeshCommands = Scene.CachedRayTracingMeshCommands, VisibleRayTracingShaderBindings, bRequiresBindings](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
				SetShaderParameters(GlobalResources, RayGenShader, *RayGenParameters);

				FRHIUniformBuffer* SceneUniformBuffer = RayGenParameters->SharedParameters.SceneUniformBuffer->GetRHI();
				FRHIUniformBuffer* NaniteRayTracingUniformBuffer = RayGenParameters->SharedParameters.NaniteRayTracingUniformBuffer->GetRHI();
				TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

				if (bRequiresBindings)
				{
					BindRayTracingDebugCHSMaterialBindings(RHICmdList, SBT, RayTracingMeshCommands, VisibleRayTracingShaderBindings, View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, Pipeline);
					RHICmdList.SetRayTracingMissShader(SBT, 0, Pipeline, 0 /* ShaderIndexInPipeline */, 0, nullptr, 0);
					RHICmdList.CommitShaderBindingTable(SBT);
				}

				RHICmdList.RayTraceDispatch(Pipeline, RayGenShader.GetRayTracingShader(), SBT, GlobalResources, ViewRect.Size().X, ViewRect.Size().Y);
			});
	}
	else if (bInlineRayTracing && RayTracingDebugModeSupportsInline(DebugVisualizationMode))
	{
		FRayTracingDebugCS::FParameters* InlinePassParameters = GraphBuilder.AllocParameters<FRayTracingDebugCS::FParameters>();

		InlinePassParameters->SharedParameters = SharedParameters;

		TShaderRef<FRayTracingDebugCS> ComputeShader = View.ShaderMap->GetShader<FRayTracingDebugCS>();

		FIntPoint Resolution(View.ViewRect.Width(), View.ViewRect.Height());

		const FIntPoint GroupSize(FRayTracingDebugCS::ThreadGroupSizeX, FRayTracingDebugCS::ThreadGroupSizeY);
		const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(ViewRect.Size(), GroupSize);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RayTracingDebug (INLINE)"),
			InlinePassParameters,
			ERDGPassFlags::Compute,
			[InlinePassParameters, ComputeShader, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
			{
				FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *InlinePassParameters, GroupCount);
			});
	}
	else
	{
		checkNoEntry(); // should have earlied out above
	}	

	if (DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP)
	{
		DrawInstanceOverlap(GraphBuilder, Scene, View, SceneColorTexture, OutputDepthTexture, LayerToTrace);
	}

	if (bSubstratePixelDebugEnable)
	{
		Substrate::AddProcessAndPrintSubstrateMaterialPropertiesPasses(GraphBuilder, View, SceneColorTexture, ShaderPlatform, SubstratePixelDebugBuffer);
	}
}

void RayTracingDebugDisplayOnScreenMessages(FScreenMessageWriter& Writer, const FViewInfo& View, bool bViewHasFarFieldInstances)
{
	const uint32 DebugVisualizationMode = GetRaytracingDebugViewModeID(View);

	switch (DebugVisualizationMode)
	{
	case RAY_TRACING_DEBUG_VIZ_TIMING_TRAVERSAL:
	case RAY_TRACING_DEBUG_VIZ_TIMING_ANY_HIT:
	case RAY_TRACING_DEBUG_VIZ_TIMING_MATERIAL:
	{
		static const FText Message = NSLOCTEXT("Renderer", "RayTracingDebugVizPerformance", "Use r.RayTracing.Visualize.TimingScale to adjust visualization.");
		Writer.DrawLine(Message, 10, FColor::White);
		break;
	}
	default:
		break;
	}

	if (CVarRayTracingVisualizeFarField.GetValueOnRenderThread())
	{
		if (!View.RayTracingCullingParameters.bIsRayTracingFarField)
		{
			static const FText Message = NSLOCTEXT("Renderer", "RayTracingDebugVizFarFieldNotEnabled", "Ray Tracing Far Field is automatically disabled as there are no features using it.");
			Writer.DrawLine(Message);
		}
		else if (!bViewHasFarFieldInstances)
		{
			static const FText Message = NSLOCTEXT("Renderer", "RayTracingDebugVizFarFieldEmpty", "Ray Tracing Far Field doesn't have any instances.");
			Writer.DrawLine(Message);
		}
	}
}

void FDeferredShadingSceneRenderer::RayTracingDisplayPicking(const FRayTracingPickingFeedback& PickingFeedback, FScreenMessageWriter& Writer)
{
	if (PickingFeedback.InstanceIndex == ~uint32(0))
	{
		return;
	}

	const ERayTracingSceneLayer LayerToTrace = CVarRayTracingVisualizeFarField.GetValueOnRenderThread() ? ERayTracingSceneLayer::FarField : ERayTracingSceneLayer::Base;
	
	int32 PickerDomain = CVarRayTracingVisualizePickerDomain.GetValueOnRenderThread();
	switch (PickerDomain)
	{
	case RAY_TRACING_DEBUG_PICKER_DOMAIN_TRIANGLE:
		Writer.DrawLine(FText::FromString(TEXT("Domain [Triangle]")), 10, FColor::Yellow);
		break;

	case RAY_TRACING_DEBUG_PICKER_DOMAIN_SEGMENT:
		Writer.DrawLine(FText::FromString(TEXT("Domain [Segment]")), 10, FColor::Yellow);
		break;

	case RAY_TRACING_DEBUG_PICKER_DOMAIN_INSTANCE:
		Writer.DrawLine(FText::FromString(TEXT("Domain [Instance]")), 10, FColor::Yellow);
		break;

	case RAY_TRACING_DEBUG_PICKER_DOMAIN_FLAGS:
		Writer.DrawLine(FText::FromString(TEXT("Domain [Flags]")), 10, FColor::Yellow);
		break;

	case RAY_TRACING_DEBUG_PICKER_DOMAIN_MASK:
		Writer.DrawLine(FText::FromString(TEXT("Domain [Mask]")), 10, FColor::Yellow);
		break;

	default:
		break; // Invalid picking domain
	}

	Writer.EmptyLine();

	Writer.DrawLine(FText::FromString(TEXT("(Use r.RayTracing.Visualize.PickerDomain to change domain)")), 10, FColor::Yellow);

	Writer.EmptyLine();

	Writer.DrawLine(FText::FromString(TEXT("[Hit]")), 10, FColor::Yellow);	

	Writer.EmptyLine();

	Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Instance Index: %u"), PickingFeedback.InstanceIndex)), 10, FColor::Yellow);
	Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Segment Index: %u"), PickingFeedback.GeometryIndex)), 10, FColor::Yellow);
	Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Triangle Index: %u"), PickingFeedback.TriangleIndex)), 10, FColor::Yellow);

	Writer.EmptyLine();

	FRHIRayTracingGeometry* Geometry = nullptr;
	for (const FRayTracingGeometryInstance& Instance : Scene->RayTracingScene.GetInstances(LayerToTrace))
	{
		if (Instance.GeometryRHI)
		{
			const uint64 GeometryAddress = uint64(Instance.GeometryRHI);
			if (PickingFeedback.GeometryAddress == GeometryAddress)
			{
				Geometry = Instance.GeometryRHI;
				break;
			}
		}
	}
	
	Writer.DrawLine(FText::FromString(TEXT("[BLAS]")), 10, FColor::Yellow);
	Writer.EmptyLine();

	if (Geometry)
	{
		const FRayTracingGeometryInitializer& Initializer = Geometry->GetInitializer();
		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Name: %s"), *Initializer.DebugName.ToString())), 10, FColor::Yellow);
		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Num Segments: %d"), Initializer.Segments.Num())), 10, FColor::Yellow);
		if (PickingFeedback.GeometryIndex < uint32(Initializer.Segments.Num()))
		{
			const FRayTracingGeometrySegment& Segment = Initializer.Segments[PickingFeedback.GeometryIndex];
			Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Segment %d Primitive Count: %u"), PickingFeedback.GeometryIndex, Segment.NumPrimitives)), 10, FColor::Yellow);
		}
		else
		{
			Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Segment %d UNKNOWN"), PickingFeedback.GeometryIndex)), 10, FColor::Yellow);
		}
		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Total Primitive Count: %u"), Initializer.TotalPrimitiveCount)), 10, FColor::Yellow);
		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Fast Build: %d"), Initializer.bFastBuild)), 10, FColor::Yellow);
		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Allow Update: %d"), Initializer.bAllowUpdate)), 10, FColor::Yellow);
		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Allow Compaction: %d"), Initializer.bAllowCompaction)), 10, FColor::Yellow);

		Writer.EmptyLine();

		FRayTracingAccelerationStructureSize SizeInfo = Geometry->GetSizeInfo();
		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Result Size: %" UINT64_FMT), SizeInfo.ResultSize)), 10, FColor::Yellow);
		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Build Scratch Size: %" UINT64_FMT), SizeInfo.BuildScratchSize)), 10, FColor::Yellow);
		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Update Scratch Size: %" UINT64_FMT), SizeInfo.UpdateScratchSize)), 10, FColor::Yellow);
	}
	else
	{
		Writer.DrawLine(FText::FromString(TEXT("UNKNOWN")), 10, FColor::Yellow);
	}	

	Writer.EmptyLine();

	Writer.DrawLine(FText::FromString(TEXT("[TLAS]")), 10, FColor::Yellow);

	Writer.EmptyLine();

	Writer.DrawLine(FText::FromString(FString::Printf(TEXT("InstanceId: %u"), PickingFeedback.InstanceId)), 10, FColor::Yellow);
	Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Mask: %u"), PickingFeedback.Mask)), 10, FColor::Yellow);
	if (PickingFeedback.Mask & RAY_TRACING_MASK_OPAQUE)
	{
		Writer.DrawLine(FText::FromString(TEXT("   RAY_TRACING_MASK_OPAQUE")), 10, FColor::Yellow);
	}
	if (PickingFeedback.Mask & RAY_TRACING_MASK_TRANSLUCENT)
	{
		Writer.DrawLine(FText::FromString(TEXT("   RAY_TRACING_MASK_TRANSLUCENT")), 10, FColor::Yellow);
	}
	if (PickingFeedback.Mask & RAY_TRACING_MASK_OPAQUE_SHADOW)
	{
		Writer.DrawLine(FText::FromString(TEXT("   RAY_TRACING_MASK_OPAQUE_SHADOW")), 10, FColor::Yellow);
	}
	if (PickingFeedback.Mask & RAY_TRACING_MASK_TRANSLUCENT_SHADOW)
	{
		Writer.DrawLine(FText::FromString(TEXT("   RAY_TRACING_MASK_TRANSLUCENT_SHADOW")), 10, FColor::Yellow);
	}
	if (PickingFeedback.Mask & RAY_TRACING_MASK_THIN_SHADOW)
	{
		Writer.DrawLine(FText::FromString(TEXT("   RAY_TRACING_MASK_THIN_SHADOW")), 10, FColor::Yellow);
	}
	if (PickingFeedback.Mask & RAY_TRACING_MASK_HAIR_STRANDS)
	{
		Writer.DrawLine(FText::FromString(TEXT("   RAY_TRACING_MASK_HAIR_STRANDS")), 10, FColor::Yellow);
	}
	if (PickingFeedback.Mask & RAY_TRACING_MASK_OPAQUE_FP_WORLD_SPACE)
	{
		Writer.DrawLine(FText::FromString(TEXT("   RAY_TRACING_MASK_OPAQUE_FP_WORLD_SPACE")), 10, FColor::Yellow);
	}
	if (PickingFeedback.Mask & RAY_TRACING_MASK_OPAQUE_REFLECTION)
	{
		Writer.DrawLine(FText::FromString(TEXT("   RAY_TRACING_MASK_OPAQUE_REFLECTION")), 10, FColor::Yellow);
	}

	Writer.DrawLine(FText::FromString(FString::Printf(TEXT("ContributionToHitGroup: %u"), PickingFeedback.InstanceContributionToHitGroupIndex)), 10, FColor::Yellow);
	{
		const ERayTracingInstanceFlags Flags = (ERayTracingInstanceFlags)PickingFeedback.Flags;
		FString FlagNames;
		if (EnumHasAnyFlags(Flags, ERayTracingInstanceFlags::TriangleCullDisable))
		{
			FlagNames += FString("CullDisable ");
		}

		if (EnumHasAnyFlags(Flags, ERayTracingInstanceFlags::TriangleCullReverse))
		{
			FlagNames += FString("CullReverse ");
		}

		if (EnumHasAnyFlags(Flags, ERayTracingInstanceFlags::ForceOpaque))
		{
			FlagNames += FString("ForceOpaque ");
		}

		if (EnumHasAnyFlags(Flags, ERayTracingInstanceFlags::ForceNonOpaque))
		{
			FlagNames += FString("ForceNonOpaque ");
		}

		Writer.DrawLine(FText::FromString(FString::Printf(TEXT("Flags: %u - %s"), PickingFeedback.Flags, *FlagNames)), 10, FColor::Yellow);
	}	

	Writer.EmptyLine();
}

bool IsRayTracingInstanceDebugDataEnabled(const FViewInfo& View)
{
#if !UE_BUILD_SHIPPING
	const uint32 Mode = GetRaytracingDebugViewModeID(View);

	return Mode == RAY_TRACING_DEBUG_VIZ_DYNAMIC_INSTANCES
		|| Mode == RAY_TRACING_DEBUG_VIZ_PROXY_TYPE
		|| Mode == RAY_TRACING_DEBUG_VIZ_PICKER;
#else
	return false;
#endif
}

bool IsRayTracingInstanceOverlapEnabled(const FViewInfo& View)
{
#if !UE_BUILD_SHIPPING	
	return GetRaytracingDebugViewModeID(View) == RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP;
#else
	return false;
#endif
}

#undef LOCTEXT_NAMESPACE

#endif
