// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingDecals.h"

#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#include "RayTracingMaterialHitShaders.h"
#include "DecalRenderingShared.h"
#include "RayTracingTypes.h"
#include "RayTracingDefinitions.h"
#include "SceneProxies/DeferredDecalProxy.h"
#include <limits>
#include "ScenePrivate.h"

TAutoConsoleVariable<int32> CVarDecalGridResolution(
	TEXT("r.RayTracing.DecalGrid.Resolution"),
	256,
	TEXT("Controls the resolution of the 2D decal grid used to cull irrelevant decal from calculations (default = 256)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarDecalGridMaxCount(
	TEXT("r.RayTracing.DecalGrid.MaxCount"),
	128,
	TEXT("Controls the maximum number of decals per cell in the 2D decal grid. The minimum of this value and the number of decal in the scene is used. (default = 128)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarDecalGridAxis(
	TEXT("r.RayTracing.DecalGrid.Axis"),
	-1,
	TEXT("Choose the coordinate axis along which to project the decal grid (default = -1, automatic)\n"),
	ECVF_RenderThreadSafe
);


enum class EDecalsWriteFlags : uint32
{
	None						= 0,
	BaseColor					= (1 << 0),
	Normal						= (1 << 1),
	RoughnessSpecularMetallic	= (1 << 2),
	Emissive					= (1 << 3),
	AmbientOcclusion			= (1 << 4),
};

static uint32 CalculateDecalWriteFlags(FDecalBlendDesc DecalBlendDesc)
{
	uint32 Flags = (uint32)EDecalsWriteFlags::None;
#define SET_FLAG(BOOL_VAL, WRITE_FLAG) Flags |= uint32(DecalBlendDesc.BOOL_VAL ? EDecalsWriteFlags::WRITE_FLAG : EDecalsWriteFlags::None)
	SET_FLAG(bWriteBaseColor, BaseColor);
	SET_FLAG(bWriteNormal, Normal);
	SET_FLAG(bWriteRoughnessSpecularMetallic, RoughnessSpecularMetallic);
	SET_FLAG(bWriteEmissive, Emissive);
	SET_FLAG(bWriteAmbientOcclusion, AmbientOcclusion);
#undef SET_FLAG

	return Flags;
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FRayTracingDecals, "RayTracingDecals");

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FDecalParametersRayTracing, )
	SHADER_PARAMETER(FMatrix44f, WorldToDecal)
	SHADER_PARAMETER(FMatrix44f, DecalToWorld)
	SHADER_PARAMETER(FVector3f, DecalPositionHigh)
	SHADER_PARAMETER(FVector3f, DecalToWorldInvScale)
	SHADER_PARAMETER(FVector3f, DecalOrientation)
	SHADER_PARAMETER(FVector2f, DecalParams)
	SHADER_PARAMETER(FVector4f, DecalColorParam)
	SHADER_PARAMETER(uint32, DecalWriteFlags)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FDecalParametersRayTracing, "RayTracingDecalParameters");

static TUniformBufferRef<FDecalParametersRayTracing> CreateDecalParametersBuffer(
	const FViewInfo& View,
	EShaderPlatform Platform,
	const FVisibleDecal& VisibleDecal,
	EUniformBufferUsage Usage)
{
	FDecalParametersRayTracing Parameters;

	const FMatrix DecalToWorldMatrix = VisibleDecal.ComponentTrans.ToMatrixWithScale();
	const FMatrix WorldToDecalMatrix = VisibleDecal.ComponentTrans.ToInverseMatrixWithScale();
	const FDFVector3 AbsoluteOrigin(DecalToWorldMatrix.GetOrigin());
	const FVector3f PositionHigh = AbsoluteOrigin.High;
	const FMatrix44f RelativeDecalToWorldMatrix = FDFMatrix::MakeToRelativeWorldMatrix(PositionHigh, DecalToWorldMatrix).M;
	const FVector3f OrientationVector = (FVector3f)VisibleDecal.ComponentTrans.GetUnitAxis(EAxis::X);

	Parameters.DecalPositionHigh = PositionHigh;
	Parameters.WorldToDecal = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToDecalMatrix);
	Parameters.DecalToWorld = RelativeDecalToWorldMatrix;
	Parameters.DecalToWorldInvScale = static_cast<FVector3f>(DecalToWorldMatrix.GetScaleVector().Reciprocal());
	Parameters.DecalOrientation = OrientationVector;
	Parameters.DecalColorParam = VisibleDecal.DecalColor;

	float LifetimeAlpha = 1.0f;

	// Certain engine captures (e.g. environment reflection) don't have a tick. Default to fully opaque.
	if (View.Family->Time.GetWorldTimeSeconds())
	{
		LifetimeAlpha = FMath::Clamp(FMath::Min(View.Family->Time.GetWorldTimeSeconds() * -VisibleDecal.InvFadeDuration + VisibleDecal.FadeStartDelayNormalized, View.Family->Time.GetWorldTimeSeconds() * VisibleDecal.InvFadeInDuration + VisibleDecal.FadeInStartDelayNormalized), 0.0f, 1.0f);
	}

	Parameters.DecalParams = FVector2f(VisibleDecal.FadeAlpha, LifetimeAlpha);
	Parameters.DecalColorParam = VisibleDecal.DecalColor;

	Parameters.DecalWriteFlags = CalculateDecalWriteFlags(VisibleDecal.BlendDesc);

	return CreateUniformBufferImmediate(Parameters, Usage);
}

class FRayTracingBuildDecalGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingBuildDecalGridCS)
	SHADER_USE_PARAMETER_STRUCT(FRayTracingBuildDecalGridCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRayTracingDecal>, SceneDecals)
		SHADER_PARAMETER(uint32, SceneDecalCount)
		SHADER_PARAMETER(FVector3f, SceneDecalsTranslatedBoundMin)
		SHADER_PARAMETER(FVector3f, SceneDecalsTranslatedBoundMax)
		SHADER_PARAMETER(unsigned, DecalGridResolution)
		SHADER_PARAMETER(unsigned, DecalGridMaxCount)
		SHADER_PARAMETER(int, DecalGridAxis)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, RWDecalGrid)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWDecalGridData)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_SHADER_TYPE(, FRayTracingBuildDecalGridCS, TEXT("/Engine/Private/RayTracing/RayTracingBuildDecalGrid.usf"), TEXT("BuildDecalGridCS"), SF_Compute);

class FRayTracingDecalMaterialShader : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FRayTracingDecalMaterialShader, Material);
public:
	FRayTracingDecalMaterialShader() = default;
	FRayTracingDecalMaterialShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMaterialShader(Initializer)
	{
		DecalParameter.Bind(Initializer.ParameterMap, TEXT("RaytracingDecalParameters"));
	}

	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		if (!ShouldCompileRayTracingCallableShadersForProject(Parameters.Platform))
		{
			// is raytracing enabled at all?
			return false;
		}
		if (!FDataDrivenShaderPlatformInfo::GetSupportsPathTracing(Parameters.Platform))
		{
			// this shader is currently only used by the path tracer
			return false;
		}
		if (Parameters.MaterialParameters.MaterialDomain != MD_DeferredDecal)
		{
			// only compile callable shader permutation of deferred decal material
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		// DECAL_PRIMITIVE informs material templates which functions to expose when rendering decals.
		OutEnvironment.SetDefine(TEXT("DECAL_PRIMITIVE"), 1);
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		if (ParameterMap.ContainsParameterAllocation(FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
		{
			OutError.Add(TEXT("Ray tracing callable shaders cannot read from the SceneTexturesStruct."));
			return false;
		}

		for (const auto& It : ParameterMap.GetParameterMap())
		{
			const FParameterAllocation& ParamAllocation = It.Value;
			if (ParamAllocation.Type != EShaderParameterType::UniformBuffer)
			{
				OutError.Add(FString::Printf(TEXT("Invalid ray tracing callable shader parameter '%s'. Only uniform buffers and loose data parameters are supported."), *(It.Key)));
				return false;
			}
		}

		return true;
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Decals;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FViewInfo& View,
		const TUniformBufferRef<FDecalParametersRayTracing>& DecalParameters,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMaterialShader::GetShaderBindings(Scene, FeatureLevel, MaterialRenderProxy, Material, ShaderBindings);

		ShaderBindings.Add(GetUniformBufferParameter<FViewUniformShaderParameters>(), View.ViewUniformBuffer);
		// Similar to DecalRendering::SetShader(...) uses GIdentityPrimitiveUniformBuffer
		// We could potentially bind the actual primitive uniform buffer
		ShaderBindings.Add(GetUniformBufferParameter<FPrimitiveUniformShaderParameters>(), GIdentityPrimitiveUniformBuffer);
		ShaderBindings.Add(DecalParameter, DecalParameters);
	}

private:
	LAYOUT_FIELD(FShaderUniformBufferParameter, DecalParameter);
};

IMPLEMENT_RT_PAYLOAD_TYPE(ERayTracingPayloadType::Decals, 48);

IMPLEMENT_SHADER_TYPE(, FRayTracingDecalMaterialShader, TEXT("/Engine/Private/RayTracing/RayTracingDecalMaterialShader.usf"), TEXT("RayTracingDecalMaterialShader"), SF_RayCallable);

static bool DecalNeedsAnyHitShader(EBlendMode BlendMode)
{
	return BlendMode != BLEND_Opaque;
}

template<bool bUseAnyHitShader>
class TRayTracingDecalMaterial : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(TRayTracingDecalMaterial, MeshMaterial);
public:
	TRayTracingDecalMaterial() = default;

	TRayTracingDecalMaterial(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		if (!ShouldCompileRayTracingShadersForProject(Parameters.Platform))
		{
			// is raytracing enabled at all?
			return false;
		}
		if (!FDataDrivenShaderPlatformInfo::GetSupportsPathTracing(Parameters.Platform))
		{
			// this shader is currently only used by the path tracer
			return false;
		}
		if (!Parameters.VertexFactoryType->SupportsRayTracing())
		{
			// does the VF support ray tracing at all?
			return false;
		}
		if (Parameters.MaterialParameters.MaterialDomain != MD_DeferredDecal)
		{
			return false;
		}
		if (DecalNeedsAnyHitShader(Parameters.MaterialParameters.BlendMode) != bUseAnyHitShader)
		{
			// the anyhit permutation is only required if the material is masked or has a non-opaque blend mode
			return false;
		}

		return true;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		check(Parameters.MaterialParameters.MaterialDomain == MD_DeferredDecal);

		FDecalBlendDesc DecalBlendDesc = DecalRendering::ComputeDecalBlendDesc(Parameters.Platform, Parameters.MaterialParameters);

		OutEnvironment.SetDefine(TEXT("DECAL_PAYLOAD_FLAGS"), CalculateDecalWriteFlags(DecalBlendDesc));
		OutEnvironment.SetDefine(TEXT("USE_MATERIAL_ANY_HIT_SHADER"), bUseAnyHitShader ? 1 : 0);
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		if (ParameterMap.ContainsParameterAllocation(FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
		{
			OutError.Add(TEXT("Ray tracing closest hit shaders cannot read from the SceneTexturesStruct."));
			return false;
		}

		for (const auto& It : ParameterMap.GetParameterMap())
		{
			const FParameterAllocation& ParamAllocation = It.Value;
			if (ParamAllocation.Type != EShaderParameterType::UniformBuffer
				&& ParamAllocation.Type != EShaderParameterType::LooseData)
			{
				OutError.Add(FString::Printf(TEXT("Invalid ray tracing shader parameter '%s'. Only uniform buffers and loose data parameters are supported."), *(It.Key)));
				return false;
			}
		}

		return true;
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Decals;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};

using FRayTracingDecalMaterialCHS = TRayTracingDecalMaterial<false>;
using FRayTracingDecalMaterialCHS_AHS = TRayTracingDecalMaterial<true>;

IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FRayTracingDecalMaterialCHS, TEXT("/Engine/Private/RayTracing/RayTracingDecalMaterialShader.usf"), TEXT("closesthit=RayTracingDecalMaterialCHS"), SF_RayHitGroup);
IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, FRayTracingDecalMaterialCHS_AHS, TEXT("/Engine/Private/RayTracing/RayTracingDecalMaterialShader.usf"), TEXT("closesthit=RayTracingDecalMaterialCHS anyhit=RayTracingDecalMaterialAHS"), SF_RayHitGroup);

FShaderType* GetRayTracingDecalMaterialShaderType(EBlendMode BlendMode)
{
	if (DecalNeedsAnyHitShader(BlendMode))
	{
		return &FRayTracingDecalMaterialCHS_AHS::GetStaticType();
	}
	else
	{
		return &FRayTracingDecalMaterialCHS::GetStaticType();
	}
}

class FDefaultOpaqueMeshDecalHitGroup : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDefaultOpaqueMeshDecalHitGroup)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FDefaultOpaqueMeshDecalHitGroup, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Decals;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	using FParameters = FEmptyShaderParameters;
};

IMPLEMENT_SHADER_TYPE(, FDefaultOpaqueMeshDecalHitGroup, TEXT("/Engine/Private/RayTracing/RayTracingDefaultDecalHitShader.usf"), TEXT("closesthit=RayTracingDefaultOpaqueDecalMaterialCHS"), SF_RayHitGroup);

class FDefaultHiddenMeshDecalHitGroup : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDefaultHiddenMeshDecalHitGroup)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FDefaultHiddenMeshDecalHitGroup, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Decals;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	using FParameters = FEmptyShaderParameters;
};

IMPLEMENT_SHADER_TYPE(, FDefaultHiddenMeshDecalHitGroup, TEXT("/Engine/Private/RayTracing/RayTracingDefaultDecalHitShader.usf"), TEXT("closesthit=RayTracingDefaultHiddenDecalMaterialCHS anyhit=RayTracingDefaultHiddenDecalMaterialAHS"), SF_RayHitGroup);

FRHIRayTracingShader* GetDefaultOpaqueMeshDecalHitShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FDefaultOpaqueMeshDecalHitGroup>().GetRayTracingShader();
}

FRHIRayTracingShader* GetDefaultHiddenMeshDecalHitShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FDefaultHiddenMeshDecalHitGroup>().GetRayTracingShader();
}

void BuildDecalGrid(FRDGBuilder& GraphBuilder, uint32 NumDecals, FRDGBufferSRVRef DecalsSRV, FRayTracingDecals& OutParameters)
{
	// if we have some decals -- build a decal grid
	if (NumDecals > 0)
	{
		const uint32 Resolution = FMath::Clamp(CVarDecalGridResolution.GetValueOnRenderThread(), 1, 2048);
		const uint32 MaxCount = FMath::Clamp(CVarDecalGridMaxCount.GetValueOnRenderThread(), 1, FMath::Min((int32)NumDecals, RAY_TRACING_DECAL_COUNT_MAXIMUM));
		OutParameters.GridResolution = Resolution;
		OutParameters.GridMaxCount = MaxCount;
		OutParameters.GridAxis = CVarDecalGridAxis.GetValueOnRenderThread();
		
		FRDGTextureDesc DecalGridDesc = FRDGTextureDesc::Create2DArray(
			FIntPoint(Resolution, Resolution),
			PF_R32_UINT,
			FClearValueBinding::None,
			TexCreate_ShaderResource | TexCreate_UAV, 3);
		FRDGTexture* DecalGridTexture = GraphBuilder.CreateTexture(DecalGridDesc, TEXT("PathTracer.DecalGrid"), ERDGTextureFlags::None);

		EPixelFormat DecalGridDataFormat = PF_R32_UINT;
		size_t DecalGridDataNumBytes = sizeof(uint32);
		if (NumDecals <= (MAX_uint8 + 1))
		{
			DecalGridDataFormat = PF_R8_UINT;
			DecalGridDataNumBytes = sizeof(uint8);
		}
		else if (NumDecals <= (MAX_uint16 + 1))
		{
			DecalGridDataFormat = PF_R16_UINT;
			DecalGridDataNumBytes = sizeof(uint16);
		}
		FRDGBufferDesc DecalGridDataDesc = FRDGBufferDesc::CreateBufferDesc(DecalGridDataNumBytes, 3 * MaxCount * Resolution * Resolution);
		FRDGBuffer* DecalGridData = GraphBuilder.CreateBuffer(DecalGridDataDesc, TEXT("PathTracer.DecalGridData"));

		auto* PassParameters = GraphBuilder.AllocParameters<FRayTracingBuildDecalGridCS::FParameters>();
		PassParameters->RWDecalGrid = GraphBuilder.CreateUAV(DecalGridTexture);
		PassParameters->RWDecalGridData = GraphBuilder.CreateUAV(DecalGridData, DecalGridDataFormat);
		
		PassParameters->SceneDecals = DecalsSRV;
		PassParameters->SceneDecalCount = OutParameters.Count;
		PassParameters->SceneDecalsTranslatedBoundMin = OutParameters.TranslatedBoundMin;
		PassParameters->SceneDecalsTranslatedBoundMax = OutParameters.TranslatedBoundMax;
		PassParameters->DecalGridResolution = OutParameters.GridResolution;
		PassParameters->DecalGridMaxCount = OutParameters.GridMaxCount;
		PassParameters->DecalGridAxis = OutParameters.GridAxis;

		TShaderMapRef<FRayTracingBuildDecalGridCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("Decal Grid Create (%u decals)", NumDecals),
			ComputeShader,
			PassParameters,
			FComputeShaderUtils::GetGroupCount(FIntVector(Resolution, Resolution, 3), FIntVector(FComputeShaderUtils::kGolden2DGroupSize, FComputeShaderUtils::kGolden2DGroupSize, 1)));

		// hookup to the actual rendering pass
		OutParameters.Grid = DecalGridTexture;
		OutParameters.GridData = GraphBuilder.CreateSRV(DecalGridData, DecalGridDataFormat);
	}
	else
	{
		// light grid is not needed - just hookup dummy data

		FRDGBufferDesc DecalGridDataDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1);
		FRDGBuffer* DecalGridData = GraphBuilder.CreateBuffer(DecalGridDataDesc, TEXT("PathTracer.DecalGridData"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(DecalGridData, PF_R32_UINT), 0);

		OutParameters.GridResolution = 0;
		OutParameters.GridMaxCount = 0;
		OutParameters.GridAxis = 0;
		OutParameters.Grid = GSystemTextures.GetBlackDummy(GraphBuilder);
		OutParameters.GridData = GraphBuilder.CreateSRV(DecalGridData, PF_R32_UINT);
	}
}

FRelevantDecalList GetSortedDecals(FVisibleDecalList& VisibleDecals, TConstArrayView<FDeferredDecalProxy*> Decals, FScene& Scene, const FViewInfo& View)
{
	const float FadeMultiplier = DecalRendering::GetDecalFadeScreenSizeMultiplier();
	const bool bIsPerspectiveProjection = View.IsPerspectiveProjection();

	FRelevantDecalList SortedDecals;

	// NOTE: Reserve is critical here to avoid reallocation of the underlying array, since the sorted decals array holds a pointer!
	SortedDecals.Reserve(Decals.Num());
	VisibleDecals.Reserve(Decals.Num());

	for (auto DecalProxy : Decals)
	{
		if (!DecalProxy->DecalMaterial || !DecalProxy->DecalMaterial->IsValidLowLevelFast())
		{
			continue;
		}

		bool bIsShown = true;

		if (!DecalProxy->IsShown(&View))
		{
			bIsShown = false;
		}

		if (bIsShown)
		{
			float FadeAlpha = 1.0f;

			if (bIsPerspectiveProjection && DecalProxy->FadeScreenSize != 0.0f)
			{
				const FMatrix ComponentToWorldMatrix = DecalProxy->ComponentTrans.ToMatrixWithScale();
				FadeAlpha = DecalRendering::CalculateDecalFadeAlpha(DecalProxy->FadeScreenSize, ComponentToWorldMatrix, View, FadeMultiplier);
			}

			const bool bShouldRender = FadeAlpha > 0.0f;

			if (bShouldRender)
			{
				VisibleDecals.Emplace(*DecalProxy, 0.0f, FadeAlpha, Scene.GetShaderPlatform(), Scene.GetFeatureLevel());
				SortedDecals.Emplace(&VisibleDecals.Last());
			}
		}
	}

	DecalRendering::SortDecalList(SortedDecals);
	return SortedDecals;
}

TRDGUniformBufferRef<FRayTracingDecals> CreateNullRayTracingDecalsUniformBuffer(FRDGBuilder& GraphBuilder)
{
	auto* OutParameters = GraphBuilder.AllocParameters<FRayTracingDecals>();
	OutParameters->Count = 0;
	OutParameters->TranslatedBoundMin = FVector3f::ZeroVector;
	OutParameters->TranslatedBoundMax = FVector3f::ZeroVector;
	
	BuildDecalGrid(GraphBuilder, 0, nullptr, *OutParameters);
	
	return GraphBuilder.CreateUniformBuffer(OutParameters);
}

TRDGUniformBufferRef<FRayTracingDecals> CreateRayTracingDecalData(FRDGBuilder& GraphBuilder, FScene& Scene, const FViewInfo& View, uint32 BaseCallableSlotIndex)
{
	const EShaderPlatform ShaderPlatform = View.Family->GetShaderPlatform();

	FVisibleDecalList VisibleDecals;
	FRelevantDecalList SortedDecals = GetSortedDecals(VisibleDecals, Scene.Decals, Scene, View);

	// Allocate from the graph builder so that we don't need to copy the data again when queuing the upload
	auto& RayTracingDecals = GraphBuilder.AllocArray<FRayTracingDecal>();
	RayTracingDecals.Reserve(SortedDecals.Num());

	const float Inf = std::numeric_limits<float>::infinity();
	FVector3f TranslatedBoundMin(+Inf, +Inf, +Inf);
	FVector3f TranslatedBoundMax(-Inf, -Inf, -Inf);

	for (int32 Index = 0; Index < SortedDecals.Num(); ++Index)
	{
		const FVisibleDecal& VisibleDecal = *SortedDecals[Index];

		const FMaterialRenderProxy* MaterialProxy = VisibleDecal.MaterialProxy;
		const FMaterial* MaterialResource = &MaterialProxy->GetMaterialWithFallback(View.GetFeatureLevel(), MaterialProxy);
		const FMaterialShaderMap* MaterialShaderMap = MaterialResource->GetRenderingThreadShaderMap();

		auto CallableShader = MaterialShaderMap->GetShader<FRayTracingDecalMaterialShader>();

		if (!CallableShader.IsValid())
		{
			continue;
		}

		TUniformBufferRef<FDecalParametersRayTracing> DecalParameters = CreateDecalParametersBuffer(View, ShaderPlatform, VisibleDecal, EUniformBufferUsage::UniformBuffer_SingleFrame);

		Scene.RayTracingSBT.TransientUniformBuffers.Add(DecalParameters); // Hold uniform buffer ref in RayTracingSBT since FMeshDrawSingleShaderBindings doesn't

		FRayTracingShaderCommand& Command = Scene.RayTracingSBT.CallableCommands.AddDefaulted_GetRef();
		Command.SetShader(CallableShader);
		Command.SlotInScene = BaseCallableSlotIndex + Index;

		FMeshDrawSingleShaderBindings SingleShaderBindings = Command.ShaderBindings.GetSingleShaderBindings(SF_RayCallable);
		CallableShader->GetShaderBindings(&Scene, Scene.GetFeatureLevel(), *MaterialProxy, *MaterialResource, View, DecalParameters, SingleShaderBindings);

		const FBox BoxBounds = VisibleDecal.BoxBounds;

		FRayTracingDecal& DestDecal = RayTracingDecals.AddDefaulted_GetRef();
		DestDecal.TranslatedBoundMin = FVector3f(BoxBounds.Min + View.ViewMatrices.GetPreViewTranslation());
		DestDecal.TranslatedBoundMax = FVector3f(BoxBounds.Max + View.ViewMatrices.GetPreViewTranslation());
		DestDecal.CallableSlotIndex = BaseCallableSlotIndex + Index;
		{
			const FMatrix WorldToDecalMatrix = VisibleDecal.ComponentTrans.ToInverseMatrixWithScale();
			DestDecal.TranslatedWorldToDecal = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToDecalMatrix);
		}
		TranslatedBoundMin = FVector3f::Min(TranslatedBoundMin, DestDecal.TranslatedBoundMin);
		TranslatedBoundMax = FVector3f::Max(TranslatedBoundMax, DestDecal.TranslatedBoundMax);
	}

	// Upload the buffer of decals to the GPU
	const uint32 NumCopyDecals = FMath::Max(1u, (uint32)RayTracingDecals.Num()); // need at least one since zero-sized buffers are not allowed
	const size_t DataSize = sizeof(FRayTracingDecal) * RayTracingDecals.Num();
	FRDGBufferSRVRef DecalsSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(CreateStructuredBuffer(GraphBuilder, TEXT("PathTracer.DecalsBuffer"), sizeof(FRayTracingDecal), NumCopyDecals, RayTracingDecals.GetData(), DataSize, ERDGInitialDataFlags::NoCopy)));

	auto* OutParameters = GraphBuilder.AllocParameters<FRayTracingDecals>();
	OutParameters->Count = RayTracingDecals.Num();
	OutParameters->TranslatedBoundMin = TranslatedBoundMin;
	OutParameters->TranslatedBoundMax = TranslatedBoundMax;

	BuildDecalGrid(GraphBuilder, RayTracingDecals.Num(), DecalsSRV, *OutParameters);
	
	return GraphBuilder.CreateUniformBuffer(OutParameters);
}

#endif // RHI_RAYTRACING
