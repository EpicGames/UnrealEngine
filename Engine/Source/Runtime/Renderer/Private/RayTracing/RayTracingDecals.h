// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderGraphResources.h"
#include "RenderGraphDefinitions.h"

#if RHI_RAYTRACING

class FScene;
class FViewInfo;
class FShaderType;
class FRHIRayTracingShader;
class FGlobalShaderMap;

enum EBlendMode : int;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FRayTracingDecals, RENDERER_API)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, Grid)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, GridData)
	SHADER_PARAMETER(FVector3f, TranslatedBoundMin)
	SHADER_PARAMETER(FVector3f, TranslatedBoundMax)
	SHADER_PARAMETER(uint32, GridResolution)
	SHADER_PARAMETER(uint32, GridMaxCount)
	SHADER_PARAMETER(uint32, GridAxis)
	SHADER_PARAMETER(uint32, Count)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

TRDGUniformBufferRef<FRayTracingDecals> CreateNullRayTracingDecalsUniformBuffer(FRDGBuilder& GraphBuilder);
TRDGUniformBufferRef<FRayTracingDecals> CreateRayTracingDecalData(FRDGBuilder& GraphBuilder, FScene& Scene, const FViewInfo& View, uint32 BaseCallableSlotIndex);

FShaderType* GetRayTracingDecalMaterialShaderType(EBlendMode BlendMode);

FRHIRayTracingShader* GetDefaultOpaqueMeshDecalHitShader(const FGlobalShaderMap* ShaderMap);
FRHIRayTracingShader* GetDefaultHiddenMeshDecalHitShader(const FGlobalShaderMap* ShaderMap);

#endif
