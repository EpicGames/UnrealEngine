// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingDynamicGeometryUpdateManager.h"
#include "MeshMaterialShader.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ScenePrivate.h"
#include "RayTracingInstance.h"
#include "RayTracingGeometry.h"
#include "RenderGraphBuilder.h"
#include "PSOPrecacheMaterial.h"
#include "PSOPrecacheValidation.h"
#include "Rendering/RayTracingGeometryManager.h"
#include "UnrealEngine.h"
#include "RayTracing/RaytracingOptions.h"
#include "ProfilingDebugging/CsvProfiler.h"

#if RHI_RAYTRACING

#include "Materials/MaterialRenderProxy.h"

DECLARE_LOG_CATEGORY_CLASS(LogRayTracingDynamicGeometryManager, Log, All);

DECLARE_GPU_STAT(RayTracingDynamicGeometry);

DECLARE_DWORD_COUNTER_STAT(TEXT("Ray tracing dynamic build primitives"), STAT_RayTracingDynamicBuildPrimitives, STATGROUP_SceneRendering);
DECLARE_DWORD_COUNTER_STAT(TEXT("Ray tracing dynamic update primitives"), STAT_RayTracingDynamicUpdatePrimitives, STATGROUP_SceneRendering);
DECLARE_DWORD_COUNTER_STAT(TEXT("Ray tracing dynamic skipped primitives"), STAT_RayTracingDynamicSkippedPrimitives, STATGROUP_SceneRendering);
DECLARE_MEMORY_STAT(TEXT("Ray tracing dynamic shared vertex buffer memory"), STAT_RayTracingDynamicSharedVertexBufferMemory, STATGROUP_SceneRendering);

CSV_DEFINE_CATEGORY(RayTracing, true);

static int32 GRTDynGeomSharedVertexBufferSizeInMB = 4;
static FAutoConsoleVariableRef CVarRTDynGeomSharedVertexBufferSizeInMB(
	TEXT("r.RayTracing.DynamicGeometry.SharedVertexBufferSizeInMB"),
	GRTDynGeomSharedVertexBufferSizeInMB,
	TEXT("Size of the a single shared vertex buffer used during the BLAS update of dynamic geometries (default 4MB)"),
	ECVF_RenderThreadSafe
);

static int32 GRTDynGeomSharedVertexBufferGarbageCollectLatency = 30;
static FAutoConsoleVariableRef CVarRTDynGeomSharedVertexBufferGarbageCollectLatency(
	TEXT("r.RayTracing.DynamicGeometry.SharedVertexBufferGarbageCollectLatency"),
	GRTDynGeomSharedVertexBufferGarbageCollectLatency,
	TEXT("Amount of update cycles before a heap is deleted when not used (default 30)."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRTDynGeomMaxUpdatePrimitivesPerFrame(
	TEXT("r.RayTracing.DynamicGeometry.MaxUpdatePrimitivesPerFrame"),
	-1,
	TEXT("Sets the dynamic ray tracing acceleration structure build budget in terms of maximum number of updated triangles per frame (<= 0 then disabled and all acceleration structures are updated - default)"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static TAutoConsoleVariable<int32> CVarRTDynGeomForceBuildMaxUpdatePrimitivesPerFrame(
	TEXT("r.RayTracing.DynamicGeometry.ForceBuild.MaxPrimitivesPerFrame"),
	0,
	TEXT("Sets the dynamic ray tracing acceleration structure build budget in terms of maximum number of triangles that are rebuild per frame (default 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRTDynGeomForceBuildMinUpdatesSinceLastBuild(
	TEXT("r.RayTracing.DynamicGeometry.ForceBuild.MinUpdatesSinceLastBuild"),
	-1,
	TEXT("Sets minimum number of updates before the dynamic geometry acceleration structure will be considered for rebuild (default INT_MAX)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRTDynGeomPriorityBasedUpdate(
	TEXT("r.RayTracing.DynamicGeometry.PriorityBasedUpdate"),
	0,
	TEXT("Whether dynamic geometry should be update based on priority."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRTDynGeomPriorityBasedUpdateMaxFramesSinceLastUpdate(
	TEXT("r.RayTracing.DynamicGeometry.PriorityBasedUpdate.MaxFramesSinceLastUpdate"),
	60,
	TEXT("Sets the maximum number of frames since the last update after which it is assigned the highest update priority."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarRTDynGeomPriorityBasedUpdateLastUpdateWeight(
	TEXT("r.RayTracing.DynamicGeometry.PriorityBasedUpdate.LastUpdateWeight"),
	0.01f,
	TEXT("Sets how time (frames since last update) affects the overall geometry priority weight (value between 0.0 - 1.0)."),
	ECVF_RenderThreadSafe
);

// Lower value means higher priority - max priority is 0.0.
float CalculatePriority(const FSceneView* View,	const FPrimitiveSceneProxy* PrimitiveSceneProxy, const FRayTracingDynamicGeometryUpdateParams& UpdateParams)
{
	const float MaxDistance = GetRayTracingCullingRadius();
	const float InvMaxDistanceSq = 1.0f / (MaxDistance * MaxDistance);
	const FBoxSphereBounds& ProxyBounds = PrimitiveSceneProxy->GetBounds();

	const float InvMaxAge = 1.0f / float(CVarRTDynGeomPriorityBasedUpdateMaxFramesSinceLastUpdate.GetValueOnRenderThread());

	const float Distance = ProxyBounds.ComputeSquaredDistanceFromBoxToPoint(View->ViewMatrices.GetViewOrigin());
	const float Age = GFrameCounterRenderThread - UpdateParams.Geometry->LastUpdatedFrame;

	const float DistancePriority = FMath::Clamp(Distance * InvMaxDistanceSq, 0.0f, 1.0f);
	const float AgePriority = 1.0f - FMath::Clamp(Age * InvMaxAge, 0.0f, 1.0f);

	const float AgeWeight = FMath::Clamp(CVarRTDynGeomPriorityBasedUpdateLastUpdateWeight.GetValueOnRenderThread(), 0.0f, 1.0f);
	float Priority = AgeWeight * AgePriority + (1.0f - AgeWeight) * DistancePriority;

	const bool bInFrustum = View->GetCullingFrustum().IntersectBox(ProxyBounds.Origin, ProxyBounds.BoxExtent);
	if (bInFrustum)
	{
		Priority *= 0.5;
	}
	return FMath::Clamp(Priority, 0.0f, 1.0f);
}

class FRayTracingDynamicGeometryConverterCS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FRayTracingDynamicGeometryConverterCS, MeshMaterial);

	class FVertexMask : SHADER_PERMUTATION_BOOL("USE_VERTEX_MASK");

	using FPermutationDomain = TShaderPermutationDomain<FVertexMask>;

public:
	FRayTracingDynamicGeometryConverterCS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());

		RWVertexPositions.Bind(Initializer.ParameterMap, TEXT("RWVertexPositions"));
		UsingIndirectDraw.Bind(Initializer.ParameterMap, TEXT("UsingIndirectDraw"));
		NumVertices.Bind(Initializer.ParameterMap, TEXT("NumVertices"));
		MaxNumThreads.Bind(Initializer.ParameterMap, TEXT("MaxNumThreads"));
		MinVertexIndex.Bind(Initializer.ParameterMap, TEXT("MinVertexIndex"));
		PrimitiveId.Bind(Initializer.ParameterMap, TEXT("PrimitiveId"));
		OutputVertexBaseIndex.Bind(Initializer.ParameterMap, TEXT("OutputVertexBaseIndex"));
		bApplyWorldPositionOffset.Bind(Initializer.ParameterMap, TEXT("bApplyWorldPositionOffset"));
		InstanceId.Bind(Initializer.ParameterMap, TEXT("InstanceId"));
		WorldToInstance.Bind(Initializer.ParameterMap, TEXT("WorldToInstance"));
		IndexBuffer.Bind(Initializer.ParameterMap, TEXT("IndexBuffer"));
		IndexBufferOffset.Bind(Initializer.ParameterMap, TEXT("IndexBufferOffset"));
	}

	FRayTracingDynamicGeometryConverterCS() = default;

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (!RHISupportsManualVertexFetch(Parameters.Platform))
		{
			return false;
		}

		if (!IsRayTracingEnabledForProject(Parameters.Platform))
		{
			return false;
		}

		if (!Parameters.VertexFactoryType->SupportsRayTracingDynamicGeometry())
		{
			return false;
		}

		if (PermutationVector.Get<FVertexMask>())
		{
			return Parameters.MaterialParameters.BlendMode == BLEND_Masked;
		}
		
		return true;
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1);
		OutEnvironment.SetDefine(TEXT("RAYTRACING_DYNAMIC_GEOMETRY_CONVERTER"), 1);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FMeshMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
	}

	void GetElementShaderBindings(
		const FShaderMapPointerTable& PointerTable,
		const FScene* Scene,
		const FSceneView* ViewIfDynamicMeshCommand,
		const FVertexFactory* VertexFactory,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMeshBatch& MeshBatch, 
		const FMeshBatchElement& BatchElement,
		const FMeshMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FMeshMaterialShader::GetElementShaderBindings(PointerTable, Scene, ViewIfDynamicMeshCommand, VertexFactory, InputStreamType, FeatureLevel, PrimitiveSceneProxy, MeshBatch, BatchElement, ShaderElementData, ShaderBindings, VertexStreams);
	}

	LAYOUT_FIELD(FShaderResourceParameter, RWVertexPositions);
	LAYOUT_FIELD(FShaderParameter, UsingIndirectDraw);
	LAYOUT_FIELD(FShaderParameter, MaxNumThreads);
	LAYOUT_FIELD(FShaderParameter, NumVertices);
	LAYOUT_FIELD(FShaderParameter, MinVertexIndex);
	LAYOUT_FIELD(FShaderParameter, PrimitiveId);
	LAYOUT_FIELD(FShaderParameter, bApplyWorldPositionOffset);
	LAYOUT_FIELD(FShaderParameter, OutputVertexBaseIndex);
	LAYOUT_FIELD(FShaderParameter, InstanceId);
	LAYOUT_FIELD(FShaderParameter, WorldToInstance);
	LAYOUT_FIELD(FShaderResourceParameter, IndexBuffer);
	LAYOUT_FIELD(FShaderParameter, IndexBufferOffset);
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FRayTracingDynamicGeometryConverterCS, TEXT("/Engine/Private/RayTracing/RayTracingDynamicMesh.usf"), TEXT("RayTracingDynamicGeometryConverterCS"), SF_Compute);

static const TCHAR* RayTracingDynamicGeometryPSOCollectorName = TEXT("RayTracingDynamicGeometry");

class FRayTracingDynamicGeometryPSOCollector : public IPSOCollector
{
public:
	FRayTracingDynamicGeometryPSOCollector(ERHIFeatureLevel::Type InFeatureLevel) : 
		IPSOCollector(FPSOCollectorCreateManager::GetIndex(GetFeatureLevelShadingPath(InFeatureLevel), RayTracingDynamicGeometryPSOCollectorName)),
		FeatureLevel(InFeatureLevel)
	{
	}

	virtual void CollectPSOInitializers(
		const FSceneTexturesConfig& SceneTexturesConfig,
		const FMaterial& Material,
		const FPSOPrecacheVertexFactoryData& VertexFactoryData,
		const FPSOPrecacheParams& PreCacheParams,
		TArray<FPSOPrecacheData>& PSOInitializers
	) override final;

private:

	ERHIFeatureLevel::Type FeatureLevel;
};


void FRayTracingDynamicGeometryPSOCollector::CollectPSOInitializers(
	const FSceneTexturesConfig& SceneTexturesConfig,
	const FMaterial& Material,
	const FPSOPrecacheVertexFactoryData& VertexFactoryData,
	const FPSOPrecacheParams& PreCacheParams,
	TArray<FPSOPrecacheData>& PSOInitializers
)
{
	// Check if RT dynamic geometry update is needed or not for this component
	if (!PreCacheParams.bRequireRTDynamicGeometryUpdate)
	{
		return;
	}
		
	if (!VertexFactoryData.VertexFactoryType->SupportsRayTracingDynamicGeometry())
	{
		return;
	}

	TShaderRef<FRayTracingDynamicGeometryConverterCS> Shader;
	auto GetShader = [&](bool bMasked)
	{
		FRayTracingDynamicGeometryConverterCS::FPermutationDomain PermutationVectorCS;
		PermutationVectorCS.Set<FRayTracingDynamicGeometryConverterCS::FVertexMask>(bMasked);

		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FRayTracingDynamicGeometryConverterCS>(PermutationVectorCS.ToDimensionValueId());

		FMaterialShaders MaterialShaders;
		if (!Material.TryGetShaders(ShaderTypes, VertexFactoryData.VertexFactoryType, MaterialShaders))
		{
			return false;
		}

		if (!MaterialShaders.TryGetShader(SF_Compute, Shader))
		{
			return false;
		}

		return true;
	};

	// Try with masked setup and otherwise fallback to reverse
	const bool bAlphaMasked = Material.IsMasked() && !PreCacheParams.bIgnoreMaskMaterial;
	if (!GetShader(bAlphaMasked))
	{
		GetShader(!bAlphaMasked);
	}

	if (!Shader.IsValid())
	{
		return;
	}

	FPSOPrecacheData RTPrecacheData;
	RTPrecacheData.Type = FPSOPrecacheData::EType::Compute;
	if (RTPrecacheData.SetComputeShader(Shader))
	{
#if PSO_PRECACHING_VALIDATE
		RTPrecacheData.ResourceName = Material.GetAssetName();
		RTPrecacheData.PSOCollectorIndex = PSOCollectorIndex;
		RTPrecacheData.VertexFactoryType = VertexFactoryData.VertexFactoryType;
#endif // PSO_PRECACHING_VALIDATE
		PSOInitializers.Add(MoveTemp(RTPrecacheData));
	}
}

IPSOCollector* CreateRayTracingDynamicGeometryPSOCollector(ERHIFeatureLevel::Type FeatureLevel)
{
	// check if raytracing is runtime enabled
	static IConsoleVariable* CVarRayTracingEnable = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing.Enable"));
	return (CVarRayTracingEnable->GetInt() != 0) ? new FRayTracingDynamicGeometryPSOCollector(FeatureLevel) : nullptr;
}
FRegisterPSOCollectorCreateFunction RegisterRayTracingDynamicGeometryPSOCollector(&CreateRayTracingDynamicGeometryPSOCollector, EShadingPath::Deferred, RayTracingDynamicGeometryPSOCollectorName);

struct FMeshComputeDispatchCommand
{
	FMeshDrawShaderBindings ShaderBindings;
	TShaderRef<FRayTracingDynamicGeometryConverterCS> MaterialShader;

	uint32 NumThreads;
	uint32 NumCPUVertices;
	FRWBuffer* TargetBuffer;

#if WANTS_DRAW_MESH_EVENTS
	FRayTracingGeometry* Geometry;
	uint32 MinVertexIndex;
	bool bApplyWorldPositionOffset;
#endif 
};

FRayTracingDynamicGeometryUpdateManager::FRayTracingDynamicGeometryUpdateManager() 
{
}

FRayTracingDynamicGeometryUpdateManager::~FRayTracingDynamicGeometryUpdateManager()
{
	for (FSharedBuffer* Buffer : VertexPositionBuffers)
	{
		delete Buffer;
	}
	VertexPositionBuffers.Empty();
}

void FRayTracingDynamicGeometryUpdateManager::Clear()
{
	DispatchCommandsPerView = {};
	InstancedViewUniformBuffers = {};

	BuildParams = nullptr;

	// Clear working arrays - keep max size allocated
	BuildRequests.Empty(BuildRequests.Max());
	UpdateRequests.Empty(UpdateRequests.Max());

	ScratchBufferSize = 0;
}

int64 FRayTracingDynamicGeometryUpdateManager::BeginUpdate()
{
	check(DispatchCommandsPerView.IsEmpty());
	check(InstancedViewUniformBuffers.IsEmpty());
	check(BuildParams == nullptr);
	check(ReferencedUniformBuffers.IsEmpty());
	check(BuildRequests.IsEmpty());
	check(UpdateRequests.IsEmpty());

	// Garbage collect unused buffers for n generations (never GC pinned buffers).
	for (int32 BufferIndex = 0; BufferIndex < VertexPositionBuffers.Num(); ++BufferIndex)
	{
		FSharedBuffer* Buffer = VertexPositionBuffers[BufferIndex];

		if (!Buffer->bPinned && Buffer->LastUsedGenerationID + GRTDynGeomSharedVertexBufferGarbageCollectLatency <= SharedBufferGenerationID)
		{
			VertexPositionBuffers.RemoveAtSwap(BufferIndex);
			delete Buffer;
			BufferIndex--;
		}
	}

	// Increment generation ID used for validation
	SharedBufferGenerationID++;

	return SharedBufferGenerationID;
}

FRayTracingDynamicGeometryUpdateManager::FSharedBuffer* FRayTracingDynamicGeometryUpdateManager::FindVertexPositionBuffer(FRHIBuffer* Buffer) const
{
	// TODO: could replace with map if this becomes a bottleneck

	for (FSharedBuffer* VertexPositionBuffer : VertexPositionBuffers)
	{
		if (VertexPositionBuffer->RWBuffer.Buffer == Buffer)
		{
			return VertexPositionBuffer;
		}
	}

	return nullptr;
}

bool FRayTracingDynamicGeometryUpdateManager::PinSharedBuffer(FRayTracingGeometry& Geometry)
{
	// Only pin if the geometry's shared buffer data is from last frame (still valid).
	// If the geometry has been absent for multiple frames, the data has been overwritten so there's nothing to preserve.
	if (Geometry.DynamicGeometrySharedBufferGenerationID != SharedBufferGenerationID - 1)
	{
		return false;
	}

	const TArray<FRayTracingGeometrySegment>& Segments = Geometry.GetInitializer().Segments;
	if (Segments.Num() > 0 && Segments[0].VertexBuffer != nullptr)
	{
#if DO_CHECK
		for (const FRayTracingGeometrySegment& Segment : Segments)
		{
			checkf(Segment.VertexBuffer == Segments[0].VertexBuffer, TEXT("All segments are expected to use the same shared vertex buffer."));
		}
#endif

		FSharedBuffer* PinnedBuffer = FindVertexPositionBuffer(Segments[0].VertexBuffer);
		if (PinnedBuffer)
		{
			PinnedBuffer->bPinned = true;
			PinnedBuffer->LastUsedGenerationID = SharedBufferGenerationID;
			Geometry.DynamicGeometrySharedBufferGenerationID = SharedBufferGenerationID;
			return true;
		}
	}

	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// TODO: Shouldn't directly modify FRayTracingGeometry initializer

void FRayTracingDynamicGeometryUpdateManager::AddDynamicGeometryToUpdate(
	FRHICommandListBase& RHICmdList,
	const FScene* Scene,
	const FSceneView* View,
	const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	FRayTracingDynamicGeometryUpdateParams UpdateParams,
	uint32 PrimitiveId
)
{
	// Keep View and InstancedView UB alive until EndUpdate()
	ReferencedUniformBuffers.Add(View->ViewUniformBuffer);
	ReferencedUniformBuffers.Add(View->GetInstancedViewUniformBuffer());

	DispatchCommandsPerView.FindOrAdd(View->ViewUniformBuffer, nullptr);
	InstancedViewUniformBuffers.FindOrAdd(View->ViewUniformBuffer, View->GetInstancedViewUniformBuffer());

	FRayTracingGeometry& Geometry = *UpdateParams.Geometry;
	check(Geometry.IsInitialized());

	if (UpdateParams.bAlphaMasked)
	{
		check(UpdateParams.IndexBuffer);
		check(!UpdateParams.bUsingIndirectDraw);

		UpdateParams.NumVertices = UpdateParams.NumTriangles * 3;
		UpdateParams.VertexBufferSize = UpdateParams.NumVertices * (uint32)sizeof(FVector3f);
	}

	bool bRecreateGeometry = false;

	if (!Geometry.IsValid() || Geometry.IsEvicted())
	{
		bRecreateGeometry = true;
	}

	// Primitive count or triangle count changed then setup the initializer and segment data again
	// TODO: Move this to higher level code since it only works with 1 segment. These "resizes" should be handled by the systems that request dynamic geometry updates.
	const bool bTotalPrimitiveCountChanges = Geometry.Initializer.TotalPrimitiveCount != UpdateParams.NumTriangles;
	const bool bVertexCountChanges = Geometry.Initializer.Segments.Num() == 1 ? Geometry.Initializer.Segments[0].MaxVertices != UpdateParams.NumVertices : false;
	if ((bTotalPrimitiveCountChanges || bVertexCountChanges) && UpdateParams.NumTriangles > 0)
	{
		checkf(Geometry.Initializer.Segments.Num() <= 1, TEXT("Dynamic ray tracing geometry '%s' has an unexpected number of segments."), *Geometry.Initializer.DebugName.ToString());
		Geometry.Initializer.TotalPrimitiveCount = UpdateParams.NumTriangles;
		Geometry.Initializer.Segments.Empty();
		FRayTracingGeometrySegment Segment;
		Segment.NumPrimitives = UpdateParams.NumTriangles;
		Segment.MaxVertices = UpdateParams.NumVertices;
		Geometry.Initializer.Segments.Add(Segment);
		bRecreateGeometry = true;
	}

	if (UpdateParams.bAlphaMasked)
	{
		Geometry.Initializer.IndexBuffer = nullptr;
	}

	FUpdateRequest UpdateRequest;
	UpdateRequest.Scene = Scene;
	UpdateRequest.View = View;
	UpdateRequest.PrimitiveSceneProxy = PrimitiveSceneProxy;
	UpdateRequest.UpdateParams = MoveTemp(UpdateParams);
	UpdateRequest.PrimitiveId = PrimitiveId;
	// recreating geometry is delayed until ScheduleUpdates since it needs VB to be assigned
	UpdateRequest.bRecreateGeometry = bRecreateGeometry;

	if (CVarRTDynGeomPriorityBasedUpdate.GetValueOnRenderThread())
	{
		UpdateRequest.Priority = CalculatePriority(View, PrimitiveSceneProxy, UpdateRequest.UpdateParams);
	}

	if (Geometry.GetRequiresBuild() || bRecreateGeometry)
	{
		BuildRequests.Add(MoveTemp(UpdateRequest));
	}
	else
	{
		UpdateRequests.Add(MoveTemp(UpdateRequest));
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FRWBuffer* FRayTracingDynamicGeometryUpdateManager::AllocateSharedBuffer(FRHICommandListBase& RHICmdList, uint32 VertexBufferSize, uint32& OutVertexBufferOffset)
{
	FSharedBuffer* VertexPositionBuffer = nullptr;
	for (FSharedBuffer* Buffer : VertexPositionBuffers)
	{
		// Don't allocate from pinned buffers - their data must be preserved for budget/visibility-skipped geometry
		if (Buffer->bPinned)
		{
			continue;
		}

		if (Buffer->RWBuffer.NumBytes >= (VertexBufferSize + Buffer->UsedSize))
		{
			VertexPositionBuffer = Buffer;
			break;
		}
	}

	// Allocate a new buffer?
	if (VertexPositionBuffer == nullptr)
	{
		VertexPositionBuffer = new FSharedBuffer;
		VertexPositionBuffers.Add(VertexPositionBuffer);

		static const uint32 VertexBufferCacheSize = GRTDynGeomSharedVertexBufferSizeInMB * 1024 * 1024;
		uint32 AllocationSize = FMath::Max(VertexBufferCacheSize, VertexBufferSize);

		VertexPositionBuffer->RWBuffer.Initialize(RHICmdList, TEXT("FRayTracingDynamicGeometryUpdateManager::RayTracingDynamicVertexBuffer"), sizeof(float), AllocationSize / sizeof(float), PF_R32_FLOAT, BUF_UnorderedAccess | BUF_ShaderResource);
		VertexPositionBuffer->UsedSize = 0;
	}

	// Update the last used generation ID
	VertexPositionBuffer->LastUsedGenerationID = SharedBufferGenerationID;

	// Get the offset and update used size
	OutVertexBufferOffset = VertexPositionBuffer->UsedSize;
	VertexPositionBuffer->UsedSize += VertexBufferSize;

	// Make sure vertex buffer offset is aligned to 16 (required for Raw SRV views)
	VertexPositionBuffer->UsedSize = Align(VertexPositionBuffer->UsedSize, RHI_RAW_VIEW_ALIGNMENT);

	return &VertexPositionBuffer->RWBuffer;
}

void FRayTracingDynamicGeometryUpdateManager::AddDispatchCommands(
	FRHICommandListBase& RHICmdList,
	const FScene* Scene, 
	const FSceneView* View, 
	const FPrimitiveSceneProxy* PrimitiveSceneProxy, 
	const FRayTracingDynamicGeometryUpdateParams& UpdateParams,
	uint32 PrimitiveId,
	FRWBuffer* RWBuffer,
	uint32 VertexBufferOffset,
	TArray<FMeshComputeDispatchCommand, SceneRenderingAllocator>& OutDispatchCommands)
{	
	const int32 PSOCollectorIndex = FPSOCollectorCreateManager::GetIndex(EShadingPath::Deferred, RayTracingDynamicGeometryPSOCollectorName);

	for (const FMeshBatch& MeshBatch : UpdateParams.GetMeshBatches())
	{
		if (!ensureMsgf(MeshBatch.VertexFactory->GetType()->SupportsRayTracingDynamicGeometry(),
		                TEXT("FRayTracingDynamicGeometryConverterCS doesn't support %s. Skipping rendering of %s.  This can happen when the skinning cache runs out of space and falls back to GPUSkinVertexFactory."),
		                MeshBatch.VertexFactory->GetType()->GetName(), *PrimitiveSceneProxy->GetOwnerName().ToString()))
		{
			continue;
		}

		const FMaterialRenderProxy* MaterialRenderProxyPtr = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxyPtr)
		{
			const FMaterial* MaterialPtr = MaterialRenderProxyPtr->GetMaterialNoFallback(Scene->GetFeatureLevel());
			if (MaterialPtr && MaterialPtr->GetRenderingThreadShaderMap())
			{
				const FMaterial& Material = *MaterialPtr;
				const FMaterialRenderProxy& MaterialRenderProxy = *MaterialRenderProxyPtr;

				auto* MaterialInterface = Material.GetMaterialInterface();

				FMeshComputeDispatchCommand DispatchCmd;

				FRayTracingDynamicGeometryConverterCS::FPermutationDomain PermutationVectorCS;
				PermutationVectorCS.Set<FRayTracingDynamicGeometryConverterCS::FVertexMask>(UpdateParams.bAlphaMasked);

				FMaterialShaderTypes ShaderTypes;
				ShaderTypes.AddShaderType<FRayTracingDynamicGeometryConverterCS>(PermutationVectorCS.ToDimensionValueId());

				FMaterialShaders MaterialShaders;
				if (Material.TryGetShaders(ShaderTypes, MeshBatch.VertexFactory->GetType(), MaterialShaders))
				{
					TShaderRef<FRayTracingDynamicGeometryConverterCS> Shader;
					MaterialShaders.TryGetShader(SF_Compute, Shader);

					FMeshProcessorShaders MeshProcessorShaders;
					MeshProcessorShaders.ComputeShader = Shader;

					DispatchCmd.MaterialShader = Shader;
					FMeshDrawShaderBindings& ShaderBindings = DispatchCmd.ShaderBindings;
					ShaderBindings.Initialize(MeshProcessorShaders);

					FMeshMaterialShaderElementData ShaderElementData;
					ShaderElementData.InitializeMeshMaterialData(View, PrimitiveSceneProxy, MeshBatch, -1, false);

					FMeshDrawSingleShaderBindings SingleShaderBindings = ShaderBindings.GetSingleShaderBindings(SF_Compute);
					Shader->GetShaderBindings(Scene, Scene->GetFeatureLevel(), PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, SingleShaderBindings);

					FVertexInputStreamArray DummyArray;
					FMeshMaterialShader::GetElementShaderBindings(Shader, Scene, View, MeshBatch.VertexFactory, EVertexInputStreamType::Default, Scene->GetFeatureLevel(), PrimitiveSceneProxy, MeshBatch, MeshBatch.Elements[0], ShaderElementData, SingleShaderBindings, DummyArray);

					DispatchCmd.TargetBuffer = RWBuffer;					

					// Setup the loose parameters directly on the binding
					uint32 OutputVertexBaseIndex = VertexBufferOffset / sizeof(float);
					uint32 MinVertexIndex = MeshBatch.Elements[0].MinVertexIndex;
					uint32 NumCPUVertices = UpdateParams.NumVertices;
					if (MeshBatch.Elements[0].MinVertexIndex < MeshBatch.Elements[0].MaxVertexIndex)
					{
						NumCPUVertices = 1 + MeshBatch.Elements[0].MaxVertexIndex - MeshBatch.Elements[0].MinVertexIndex;
					}

					const uint32 VertexBufferNumElements = UpdateParams.VertexBufferSize / sizeof(FVector3f) - MinVertexIndex;
					if (!ensureMsgf(NumCPUVertices <= VertexBufferNumElements,
					                TEXT("%s: Vertex buffer contains %d vertices, but RayTracingDynamicGeometryConverterCS dispatch command expects at least %d."),
					                *PrimitiveSceneProxy->GetOwnerName().ToString(), VertexBufferNumElements, NumCPUVertices))
					{
						NumCPUVertices = VertexBufferNumElements;
					}

					DispatchCmd.NumCPUVertices = NumCPUVertices;

					SingleShaderBindings.Add(Shader->UsingIndirectDraw, UpdateParams.bUsingIndirectDraw ? 1 : 0);
					SingleShaderBindings.Add(Shader->NumVertices, NumCPUVertices);
					SingleShaderBindings.Add(Shader->MinVertexIndex, MinVertexIndex);
					SingleShaderBindings.Add(Shader->PrimitiveId, PrimitiveId);
					SingleShaderBindings.Add(Shader->OutputVertexBaseIndex, OutputVertexBaseIndex);
					SingleShaderBindings.Add(Shader->bApplyWorldPositionOffset, UpdateParams.bApplyWorldPositionOffset ? 1 : 0);
					SingleShaderBindings.Add(Shader->InstanceId, UpdateParams.InstanceId);
					SingleShaderBindings.Add(Shader->WorldToInstance, UpdateParams.WorldToInstance);

					if (UpdateParams.bAlphaMasked)
					{
						FRHIBuffer* IndexBufferRHI = UpdateParams.IndexBuffer;

						const uint32 IndexStride = IndexBufferRHI->GetStride();
						const uint32 NumTriangles = UpdateParams.NumTriangles;
						const uint32 IndexBufferOffset = UpdateParams.Geometry->GetInitializer().IndexBufferOffset / IndexStride + MeshBatch.Elements[0].FirstIndex;

						SingleShaderBindings.Add(Shader->IndexBuffer,
							RHICmdList.CreateShaderResourceView(IndexBufferRHI,
								FRHIViewDesc::CreateBufferSRV()
								.SetType(FRHIViewDesc::EBufferType::Typed)
								.SetFormat(IndexStride == 4 ? PF_R32_UINT : PF_R16_UINT))
						);

						SingleShaderBindings.Add(Shader->MaxNumThreads, NumTriangles);
						SingleShaderBindings.Add(Shader->IndexBufferOffset, IndexBufferOffset);

						DispatchCmd.NumThreads = NumTriangles;
					}
					else
					{
						SingleShaderBindings.Add(Shader->MaxNumThreads, NumCPUVertices);
						SingleShaderBindings.Add(Shader->IndexBufferOffset, 0);

						DispatchCmd.NumThreads = UpdateParams.NumVertices;
					}				

				#if MESH_DRAW_COMMAND_DEBUG_DATA
					ShaderBindings.Finalize(&MeshProcessorShaders);
				#endif

				#if PSO_PRECACHING_VALIDATE
					FRHIComputeShader* ComputeShader = DispatchCmd.MaterialShader.GetComputeShader();
					if (ComputeShader != nullptr)
					{
						EPSOPrecacheResult PSOPrecacheResult = PipelineStateCache::CheckPipelineStateInCache(ComputeShader);
						PSOCollectorStats::CheckComputePipelineStateInCache(*ComputeShader, PSOPrecacheResult, &MaterialRenderProxy, PSOCollectorIndex);
					}
				#endif

#if WANTS_DRAW_MESH_EVENTS
					DispatchCmd.Geometry = UpdateParams.Geometry;
					DispatchCmd.MinVertexIndex = MinVertexIndex;
					DispatchCmd.bApplyWorldPositionOffset = UpdateParams.bApplyWorldPositionOffset;
#endif

					OutDispatchCommands.Add(DispatchCmd);

					break;
				}
			}

			MaterialRenderProxyPtr = MaterialRenderProxyPtr->GetFallback(Scene->GetFeatureLevel());
		}
	}
}

void FRayTracingDynamicGeometryUpdateManager::ClassifyRequests(bool bUseTracingFeedback, bool bVertexBufferRequired)
{
	const uint32 MaxUpdatePrimitivesPerFrame = uint32(CVarRTDynGeomMaxUpdatePrimitivesPerFrame.GetValueOnRenderThread());
	const uint32 MaxForceBuildPrimitivesPerFrame = uint32(CVarRTDynGeomForceBuildMaxUpdatePrimitivesPerFrame.GetValueOnRenderThread());
	const uint32 MinUpdatesSinceLastBuild = uint32(CVarRTDynGeomForceBuildMinUpdatesSinceLastBuild.GetValueOnRenderThread());

	// Sort UpdateRequests for budget prioritization
	const bool bNeedsSorting = (int32(MaxUpdatePrimitivesPerFrame) != -1) || (MaxForceBuildPrimitivesPerFrame != 0);
	if (bNeedsSorting)
	{
		// TODO: If moving FUpdateRequest turns out too be too slow, add indirection and sort indices instead.

		if (CVarRTDynGeomPriorityBasedUpdate.GetValueOnRenderThread())
		{
			UpdateRequests.Sort([](const FUpdateRequest& InLHS, const FUpdateRequest& InRHS)
				{
					return InLHS.Priority < InRHS.Priority;
				});
		}
		else
		{
			UpdateRequests.Sort([](const FUpdateRequest& InLHS, const FUpdateRequest& InRHS)
				{
					if (InLHS.UpdateParams.Geometry->LastUpdatedFrame == InRHS.UpdateParams.Geometry->LastUpdatedFrame)
					{
						return InLHS.UpdateParams.Geometry->NumUpdatesSinceLastBuild > InRHS.UpdateParams.Geometry->NumUpdatesSinceLastBuild;
					}

					return InLHS.UpdateParams.Geometry->LastUpdatedFrame < InRHS.UpdateParams.Geometry->LastUpdatedFrame;
				});
		}
	}

	// Classify each UpdateRequest as scheduled (build/update) or skipped based on budget and visibility.
	// When vertex buffers must survive past the BLAS build (SBT references them), skipped geometries
	// have their shared vertex buffers pinned to prevent reuse. This must happen before any new VB
	// allocations so that AllocateSharedBuffer() knows which buffers to avoid.

	uint32 NumUpdatedPrimitives = 0;
	uint32 NumForceBuildPrimitives = 0;
	bool bBudgetExhausted = false;

	auto ClassifyBuildMode = [&](FUpdateRequest& UpdateRequest, FRayTracingGeometry* RayTracingGeometry) -> EAccelerationStructureBuildMode
		{
			const FRayTracingGeometryInitializer& GeometryInitializer = RayTracingGeometry->GetInitializer();
			const uint32 TotalPrimitiveCount = GeometryInitializer.TotalPrimitiveCount;

			if (MaxForceBuildPrimitivesPerFrame > 0 && RayTracingGeometry->NumUpdatesSinceLastBuild > MinUpdatesSinceLastBuild && NumForceBuildPrimitives <= MaxForceBuildPrimitivesPerFrame)
			{
				NumForceBuildPrimitives += TotalPrimitiveCount;
				return EAccelerationStructureBuildMode::Build;
			}
			else if (!GeometryInitializer.bAllowUpdate)
			{
				return EAccelerationStructureBuildMode::Build;
			}

			return EAccelerationStructureBuildMode::Update;
		};

	for (FUpdateRequest& UpdateRequest : UpdateRequests)
	{
		FRayTracingGeometry* RayTracingGeometry = UpdateRequest.UpdateParams.Geometry;
		const FRayTracingGeometryInitializer& GeometryInitializer = RayTracingGeometry->GetInitializer();
		const uint32 TotalPrimitiveCount = GeometryInitializer.TotalPrimitiveCount;

		UpdateRequest.bSkipped = false;

		if (bBudgetExhausted)
		{
			UpdateRequest.bSkipped = true;
		}
		else if (bUseTracingFeedback && !GRayTracingGeometryManager->IsGeometryVisible(RayTracingGeometry->GetGeometryHandle()))
		{
			UpdateRequest.bSkipped = true;
		}

		// If update will be skipped and the geometry uses shared vertex buffer then we need to pin the buffer to avoid vertex data being overwritten.
		// Only applies when the geometry actually uses shared buffers (no external buffer provided) 
		// AND MeshBatches is non-empty (compute shader will write vertex data into the shared buffer).
		// Geometries with their own buffer or with no dispatch work don't use shared buffers and can safely stay skipped.
		if (UpdateRequest.bSkipped && bVertexBufferRequired && UpdateRequest.UpdateParams.Buffer == nullptr && !UpdateRequest.UpdateParams.GetMeshBatches().IsEmpty())
		{
			// Try to pin the shared buffer to preserve its vertex data.
			// If pinning fails (geometry absent too long, buffer already recycled), force the update to be processed regardless of budget,
			// since there's no valid fallback data so the geometry would be invisible otherwise.
			if (!PinSharedBuffer(*RayTracingGeometry))
			{
				UpdateRequest.bSkipped = false;
			}
		}

		if (UpdateRequest.bSkipped)
		{
			INC_DWORD_STAT_BY(STAT_RayTracingDynamicSkippedPrimitives, TotalPrimitiveCount);
		}
		else
		{
			UpdateRequest.ScheduledBuildMode = ClassifyBuildMode(UpdateRequest, RayTracingGeometry);

			NumUpdatedPrimitives += TotalPrimitiveCount;

			if (!bBudgetExhausted && NumUpdatedPrimitives > MaxUpdatePrimitivesPerFrame)
			{
				bBudgetExhausted = true;
			}
		}
	}

	INC_DWORD_STAT_BY(STAT_RayTracingDynamicUpdatePrimitives, NumUpdatedPrimitives);
}

void FRayTracingDynamicGeometryUpdateManager::AddRequestsToBuildList(FRDGBuilder& GraphBuilder)
{
	const int32 TotalNumGeometryBuilds = BuildRequests.Num() + UpdateRequests.Num();

	checkf(BuildParams == nullptr, TEXT("BuildParams is not empty. Previous frame updates were not dispatched."));
	BuildParams = &GraphBuilder.AllocArray<FRayTracingGeometryBuildParams>();
	BuildParams->Reserve(TotalNumGeometryBuilds);

	// reserve for worst case (might be wasteful if there are too many views)
	for (auto& ViewDispatchCommandsPair : DispatchCommandsPerView)
	{
		checkf(ViewDispatchCommandsPair.Value == nullptr, TEXT("DispatchCommandsPerView is not empty. Previous frame updates were not dispatched."));
		ViewDispatchCommandsPair.Value = &GraphBuilder.AllocArray<FMeshComputeDispatchCommand>();
		ViewDispatchCommandsPair.Value->Reserve(TotalNumGeometryBuilds);
	}

	uint32 BLASScratchSize = 0;
	uint32 NumBuildPrimitives = 0;

	auto AddRequestToBuildList = [this, &GraphBuilder, &BLASScratchSize, &NumBuildPrimitives](FUpdateRequest& UpdateRequest, EAccelerationStructureBuildMode BuildMode)
		{
			FRayTracingGeometry& RayTracingGeometry = *UpdateRequest.UpdateParams.Geometry;

			FRWBuffer* RWBuffer = UpdateRequest.UpdateParams.Buffer;
			uint32 VertexBufferOffset = 0;
			bool bUseSharedVertexBuffer = false;

			// Only update when we have mesh batches
			if (!UpdateRequest.UpdateParams.GetMeshBatches().IsEmpty())
			{
				// If update params didn't provide a buffer then use a shared vertex position buffer
				if (RWBuffer == nullptr)
				{
					RWBuffer = AllocateSharedBuffer(GraphBuilder.RHICmdList, UpdateRequest.UpdateParams.VertexBufferSize, VertexBufferOffset);
					bUseSharedVertexBuffer = true;
				}
				check(IsAligned(VertexBufferOffset, RHI_RAW_VIEW_ALIGNMENT));

				AddDispatchCommands(
					GraphBuilder.RHICmdList,
					UpdateRequest.Scene,
					UpdateRequest.View,
					UpdateRequest.PrimitiveSceneProxy,
					UpdateRequest.UpdateParams,
					UpdateRequest.PrimitiveId,
					RWBuffer,
					VertexBufferOffset,
					*DispatchCommandsPerView[UpdateRequest.View->ViewUniformBuffer]);
			}

			// Optionally resize the buffer when not shared (could also be lazy allocated and still empty)
			if (!bUseSharedVertexBuffer && RWBuffer && RWBuffer->NumBytes != UpdateRequest.UpdateParams.VertexBufferSize)
			{
				RWBuffer->Initialize(GraphBuilder.RHICmdList, TEXT("FRayTracingDynamicGeometryUpdateManager::RayTracingDynamicVertexBuffer"), sizeof(float), UpdateRequest.UpdateParams.VertexBufferSize / sizeof(float), PF_R32_FLOAT, BUF_UnorderedAccess | BUF_ShaderResource);
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS // TODO: Shouldn't directly modify FRayTracingGeometry initializer
			if (RWBuffer)
			{
				for (FRayTracingGeometrySegment& Segment : RayTracingGeometry.Initializer.Segments)
				{
					Segment.VertexBuffer = RWBuffer->Buffer;
					Segment.VertexBufferOffset = VertexBufferOffset;
				}
			}
#if DO_CHECK
			else
			{
				for (FRayTracingGeometrySegment& Segment : RayTracingGeometry.Initializer.Segments)
				{
					checkf(Segment.VertexBuffer != nullptr, TEXT("Dynamic ray tracing geometry '%s' has a segment without a valid VertexBuffer."), *RayTracingGeometry.Initializer.DebugName.ToString());
				}
			}
#endif
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			const FRayTracingGeometryInitializer& GeometryInitializer = RayTracingGeometry.GetInitializer();

			if (UpdateRequest.bRecreateGeometry)
			{
				checkf(RayTracingGeometry.RawData.IsEmpty() && GeometryInitializer.OfflineData == nullptr, TEXT("Dynamic ray tracing geometry '%s' is not expected to have offline acceleration structure data."), *GeometryInitializer.DebugName.ToString());
				RayTracingGeometry.CreateRayTracingGeometry(GraphBuilder.RHICmdList, ERTAccelerationStructureBuildPriority::Skip);
			}

			if (bUseSharedVertexBuffer)
			{
				RayTracingGeometry.DynamicGeometrySharedBufferGenerationID = SharedBufferGenerationID;
			}
			else
			{
				RayTracingGeometry.DynamicGeometrySharedBufferGenerationID = FRayTracingGeometry::NonSharedVertexBuffers;
			}

			RayTracingGeometry.LastUpdatedFrame = GFrameCounterRenderThread;

			FRHIRayTracingGeometry* RayTracingGeometryRHI = RayTracingGeometry.GetRHI();

			if (BuildMode == EAccelerationStructureBuildMode::Build)
			{
				RayTracingGeometry.NumUpdatesSinceLastBuild = 0;
				RayTracingGeometry.SetRequiresBuild(false);
				RayTracingGeometry.SetRequiresUpdate(false);

				const uint32 ScratchSize = RayTracingGeometryRHI->GetSizeInfo().BuildScratchSize;
				BLASScratchSize = Align(BLASScratchSize + ScratchSize, GRHIRayTracingScratchBufferAlignment);

				NumBuildPrimitives += GeometryInitializer.TotalPrimitiveCount;
			}
			else
			{
				RayTracingGeometry.NumUpdatesSinceLastBuild++;
				RayTracingGeometry.SetRequiresUpdate(false);

				const uint32 ScratchSize = RayTracingGeometryRHI->GetSizeInfo().UpdateScratchSize;
				BLASScratchSize = Align(BLASScratchSize + ScratchSize, GRHIRayTracingScratchBufferAlignment);
			}

			// Make a copy of GeometryInitializer.Segments that can be passed to RDG pass
			TArray<FRayTracingGeometrySegment, SceneRenderingAllocator>& Segments = GraphBuilder.AllocArray<FRayTracingGeometrySegment>();
			Segments.Append(GeometryInitializer.Segments);

			FRayTracingGeometryBuildParams RTGeoBuildParams;
			RTGeoBuildParams.Geometry = RayTracingGeometryRHI;
			RTGeoBuildParams.BuildMode = BuildMode;
			RTGeoBuildParams.Segments = Segments;

			BuildParams->Add(MoveTemp(RTGeoBuildParams));
		};

	for (FUpdateRequest& BuildRequest : BuildRequests)
	{
		checkf(!BuildRequest.bSkipped, TEXT("Build requests can't be skipped."));
		AddRequestToBuildList(BuildRequest, EAccelerationStructureBuildMode::Build);
	}

	// Process scheduled UpdateRequests (decisions made in the classification loop above)
	for (FUpdateRequest& UpdateRequest : UpdateRequests)
	{
		if (!UpdateRequest.bSkipped)
		{
			AddRequestToBuildList(UpdateRequest, UpdateRequest.ScheduledBuildMode);
		}
	}

	ScratchBufferSize = BLASScratchSize;

	INC_DWORD_STAT_BY(STAT_RayTracingDynamicBuildPrimitives, NumBuildPrimitives);
}

void FRayTracingDynamicGeometryUpdateManager::ScheduleUpdates(FRDGBuilder& GraphBuilder, bool bUseTracingFeedback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRayTracingDynamicGeometryUpdateManager::ScheduleUpdates);
	
	// Early out of nothing to do
	if (BuildRequests.IsEmpty() && UpdateRequests.IsEmpty())
	{
		return;
	}

	// Vertex buffers only need to be kept alive after BLAS build when the SBT references them.
	const bool bVertexBufferRequired = GRHIGlobals.RayTracing.SupportsShaders || GRHIGlobals.RayTracing.RequiresInlineRayTracingSBT;

	if (bVertexBufferRequired)
	{
		// Clear previous frame's pins
		for (FSharedBuffer* Buffer : VertexPositionBuffers)
		{
			Buffer->bPinned = false;
		}
	}

	ClassifyRequests(bUseTracingFeedback, bVertexBufferRequired);

	if (bVertexBufferRequired)
	{
		// Reset UsedSize on buffers that are not pinned
		for (FSharedBuffer* Buffer : VertexPositionBuffers)
		{
			if (!Buffer->bPinned)
			{
				Buffer->UsedSize = 0;
			}
		}
	}
	else
	{
		// When vertex buffers don't need to survive past the BLAS build,
		// no pinning is needed so just reset all shared buffers for reuse.
		for (FSharedBuffer* Buffer : VertexPositionBuffers)
		{
			Buffer->bPinned = false;
			Buffer->UsedSize = 0;
		}
	}

	AddRequestsToBuildList(GraphBuilder);

	// Track total shared vertex buffer memory (after all allocations are complete)
	{
		int64 TotalSharedVertexBufferMemory = 0;
		for (const FSharedBuffer* Buffer : VertexPositionBuffers)
		{
			TotalSharedVertexBufferMemory += Buffer->RWBuffer.NumBytes;
		}

		SET_MEMORY_STAT(STAT_RayTracingDynamicSharedVertexBufferMemory, TotalSharedVertexBufferMemory);
		CSV_CUSTOM_STAT(RayTracing, DynamicGeometrySharedVertexBufferMB, float(TotalSharedVertexBufferMemory) / (1024.0f * 1024.0f), ECsvCustomStatOp::Set);
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FRayTracingDynamicGeometryUpdatePassParams, )
	RDG_BUFFER_ACCESS(DynamicGeometryScratchBuffer, ERHIAccess::UAVCompute)

	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
END_SHADER_PARAMETER_STRUCT()

void FRayTracingDynamicGeometryUpdateManager::AddDynamicGeometryUpdatePass(FRDGBuilder& GraphBuilder, ERDGPassFlags ComputePassFlags, const TRDGUniformBufferRef<FSceneUniformParameters>& SceneUB, ERHIPipeline ResourceAccessPipelines, FRDGBufferRef& OutDynamicGeometryScratchBuffer)
{
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());
	RDG_EVENT_SCOPE_STAT(GraphBuilder, RayTracingDynamicGeometry, "RayTracingDynamicGeometry");

	// Slow path when scratch buffer exceeds 2GBs
	const uint32 MaxBLASScratchSize = 2147483647u;
	const bool bSlowPath = ScratchBufferSize >= MaxBLASScratchSize;

	if (bSlowPath)
	{
		UE_LOGF(LogRayTracingDynamicGeometryManager, Warning, "Required scratch buffer size (%u) is larger than 2GB. Consider reducing dynamic geometry complexity or number of primitives updated per frame.", ScratchBufferSize);
	}

	if (ScratchBufferSize > 0)
	{
		FRDGBufferDesc ScratchBufferDesc;
		ScratchBufferDesc.Usage = EBufferUsageFlags::RayTracingScratch | EBufferUsageFlags::StructuredBuffer;
		ScratchBufferDesc.BytesPerElement = GRHIRayTracingScratchBufferAlignment;

		// Even when using the slow path we need to allocate scratch buffer in RDG to get correct pass order.
		ScratchBufferDesc.NumElements = bSlowPath ? 1 : FMath::DivideAndRoundUp(ScratchBufferSize, GRHIRayTracingScratchBufferAlignment);

		OutDynamicGeometryScratchBuffer = GraphBuilder.CreateBuffer(ScratchBufferDesc, TEXT("DynamicGeometry.BLASSharedScratchBuffer"));
	}

	const ERHIPipeline SrcResourceAccessPipelines = ComputePassFlags == ERDGPassFlags::AsyncCompute ? ERHIPipeline::AsyncCompute : ERHIPipeline::Graphics;

	for (TPair<FRHIUniformBuffer*, TArray<FMeshComputeDispatchCommand, SceneRenderingAllocator>*>& ViewDispatchCommands : DispatchCommandsPerView)
	{
		if (ViewDispatchCommands.Value == nullptr || ViewDispatchCommands.Value->IsEmpty())
		{
			continue;
		}

		FRayTracingDynamicGeometryUpdatePassParams* PassParams = GraphBuilder.AllocParameters<FRayTracingDynamicGeometryUpdatePassParams>();
		PassParams->View.View = TUniformBufferRef<FViewUniformShaderParameters>(ViewDispatchCommands.Key);
		PassParams->View.InstancedView = TUniformBufferRef<FInstancedViewUniformShaderParameters>(InstancedViewUniformBuffers[ViewDispatchCommands.Key]);
		PassParams->Scene = SceneUB;

		// DynamicGeometryScratchBuffer is not directly used in this pass but set so RDG orders passes correctly
		// (TODO: this might also prevent dispatches for different views from overlapping, so investigate better solution)
		PassParams->DynamicGeometryScratchBuffer = OutDynamicGeometryScratchBuffer; 

		GraphBuilder.AddPass(RDG_EVENT_NAME("RayTracingDynamicUpdate"), PassParams, ComputePassFlags | ERDGPassFlags::NeverCull,
			[SrcResourceAccessPipelines, ResourceAccessPipelines, DispatchCommands = MakeConstArrayView(*ViewDispatchCommands.Value)](FRHICommandList& RHICmdList)
			{
				DispatchUpdates(RHICmdList, DispatchCommands, SrcResourceAccessPipelines, ResourceAccessPipelines);
			});
	}

	if (BuildParams != nullptr && BuildParams->Num() > 0)
	{
		FRayTracingDynamicGeometryUpdatePassParams* PassParams = GraphBuilder.AllocParameters<FRayTracingDynamicGeometryUpdatePassParams>();
		PassParams->View = {};
		PassParams->Scene = nullptr;
		PassParams->DynamicGeometryScratchBuffer = OutDynamicGeometryScratchBuffer;

		GraphBuilder.AddPass(RDG_EVENT_NAME("RayTracingDynamicUpdateBuild"), PassParams, ComputePassFlags | ERDGPassFlags::NeverCull,
			[PassParams, BuildParams = MakeConstArrayView(*BuildParams), bSlowPath](FRHICommandList& RHICmdList)
			{
				// Can't use parallel command list because we have to make sure we are not building BVH data
				// on the same RTGeometry on multiple threads at the same time. Ideally move the build
				// requests over to the RaytracingGeometry manager so they can be correctly scheduled
				// with other build requests in the engine (see UE-106982)
				SCOPED_DRAW_EVENT(RHICmdList, Build);

				if (!bSlowPath)
				{
					FRHIBufferRange ScratchBufferRange;
					ScratchBufferRange.Buffer = PassParams->DynamicGeometryScratchBuffer ? PassParams->DynamicGeometryScratchBuffer->GetRHI() : nullptr;
					ScratchBufferRange.Offset = 0;
					RHICmdList.BuildAccelerationStructures(BuildParams, ScratchBufferRange);
				}
				else
				{
					RHICmdList.BuildAccelerationStructures(BuildParams);
				}
			});
	}

	Clear();

	// TODO: Is it safe to use a regular task that waits on FRDGBuilder::GetAsyncExecuteTask() here instead?
	// which would allow the passes above to be tagged with FRDGAsyncTask
	GraphBuilder.AddPostExecuteCallback([this]
		{
			EndUpdate();
		});
}

void FRayTracingDynamicGeometryUpdateManager::DispatchUpdates(FRHICommandList& RHICmdList, TConstArrayView<FMeshComputeDispatchCommand> DispatchCommands, ERHIPipeline SrcResourceAccessPipelines, ERHIPipeline DstResourceAccessPipelines)
{
	if (DispatchCommands.Num() > 0)
	{
		SCOPED_DRAW_EVENT(RHICmdList, RayTracingDynamicGeometryUpdate);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SortDispatchCommands);

			// This can be optimized by using sorted insert or using map on shaders
			// There are only a handful of unique shaders and a few target buffers so we want to swap state as little as possible
			// to reduce RHI thread overhead
			DispatchCommands.Sort([](const FMeshComputeDispatchCommand& InLHS, const FMeshComputeDispatchCommand& InRHS)
			                      {
									  if (InLHS.MaterialShader.GetComputeShader() != InRHS.MaterialShader.GetComputeShader())
										  return InLHS.MaterialShader.GetComputeShader() < InRHS.MaterialShader.GetComputeShader();

									  return InLHS.TargetBuffer < InRHS.TargetBuffer;
								  });
		}

		FMemMark Mark(FMemStack::Get());

		TArray<FRHITransitionInfo, TMemStackAllocator<>> TransitionsBefore, TransitionsAfter;
		TArray<FRHIUnorderedAccessView*, TMemStackAllocator<>> OverlapUAVs;
		TransitionsBefore.Reserve(DispatchCommands.Num());
		TransitionsAfter.Reserve(DispatchCommands.Num());
		OverlapUAVs.Reserve(DispatchCommands.Num());
		const FRWBuffer* LastBuffer = nullptr;
		TSet<const FRWBuffer*> TransitionedBuffers;
		for (const FMeshComputeDispatchCommand& Cmd : DispatchCommands)
		{
			if (Cmd.TargetBuffer == nullptr)
			{
				continue;
			}
			FRHIUnorderedAccessView* UAV = Cmd.TargetBuffer->UAV.GetReference();

			// The list is sorted by TargetBuffer, so we can remove duplicates by simply looking at the previous value we've processed.
			if (LastBuffer == Cmd.TargetBuffer)
			{
				// This UAV is used by more than one dispatch, so tell the RHI it's OK to overlap the dispatches, because
				// we're updating disjoint regions.
				if (OverlapUAVs.Num() == 0 || OverlapUAVs.Last() != UAV)
				{
					OverlapUAVs.Add(UAV);
				}
				continue;
			}

			LastBuffer = Cmd.TargetBuffer;

			// In case different shaders use different TargetBuffer we want to add transition only once
			bool bAlreadyInSet = false;
			TransitionedBuffers.FindOrAdd(LastBuffer, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				// Looks like the resource can get here in either UAVCompute or SRVMask mode, so we'll have to use Unknown until we can have better tracking.
				TransitionsBefore.Add(FRHITransitionInfo(UAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
				TransitionsAfter.Add(FRHITransitionInfo(UAV, ERHIAccess::UAVCompute, ERHIAccess::SRVMask));
			}
		}

		{
			FRHIComputeShader* CurrentShader = nullptr;
			FRWBuffer* CurrentBuffer = nullptr;

			// Transition to writeable for each cmd list and enable UAV overlap, because several dispatches can update non-overlapping portions of the same buffer.
			// Mark as no fence because these resources have been fenced already at the beginning of the frame by RDG
			RHICmdList.Transition(TransitionsBefore, ERHITransitionCreateFlags::AllowDecayPipelines);
			RHICmdList.BeginUAVOverlap(OverlapUAVs);

			// Cache the bound uniform buffers because a lot are the same between dispatches
			FShaderBindingState ShaderBindingState;

			for (const FMeshComputeDispatchCommand& Cmd : DispatchCommands)
			{
				const TShaderRef<FRayTracingDynamicGeometryConverterCS>& Shader = Cmd.MaterialShader;
				FRHIComputeShader* ComputeShader = Shader.GetComputeShader();
				if (CurrentShader != ComputeShader)
				{
					SetComputePipelineState(RHICmdList, ComputeShader);
					CurrentBuffer = nullptr;
					CurrentShader = ComputeShader;

					// Reset binding state
					ShaderBindingState = FShaderBindingState();
				}

#if WANTS_DRAW_MESH_EVENTS				
				SCOPED_CONDITIONAL_DRAW_EVENTF(RHICmdList, RayTracingDynamicGeometryDispatch, GShowMaterialDrawEvents != 0, 
					TEXT("%s - NumVertices:%d MinVertexIndex:%d WPO:%d")
					, Cmd.Geometry->GetInitializer().DebugName
					, Cmd.NumCPUVertices
					, Cmd.MinVertexIndex
					, (Cmd.bApplyWorldPositionOffset ? 1 : 0)
				);
#endif

				FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();

				FRWBuffer* TargetBuffer = Cmd.TargetBuffer;

				// Always rebind the target buffer because we have to make sure that the bindless index is always written
				// otherwise it might miss in the cbuffer data
				//if (CurrentBuffer != TargetBuffer)
				{
					CurrentBuffer = TargetBuffer;

					SetUAVParameter(BatchedParameters, Shader->RWVertexPositions, Cmd.TargetBuffer->UAV);
				}

				Cmd.ShaderBindings.SetParameters(BatchedParameters, &ShaderBindingState);
				RHICmdList.SetBatchedShaderParameters(CurrentShader, BatchedParameters);

				const FIntVector NumWrappedThreadGroups = FComputeShaderUtils::GetGroupCountWrapped(Cmd.NumThreads, 64);
				RHICmdList.DispatchComputeShader(NumWrappedThreadGroups.X, NumWrappedThreadGroups.Y, NumWrappedThreadGroups.Z);
			}

			// Make sure buffers are readable again and disable UAV overlap.
			RHICmdList.EndUAVOverlap(OverlapUAVs);

			// Transition to SRV state and mark readable on requested pipelines
			RHICmdList.Transition(TransitionsAfter, SrcResourceAccessPipelines, DstResourceAccessPipelines);
		}
	}
}

void FRayTracingDynamicGeometryUpdateManager::EndUpdate()
{
	ReferencedUniformBuffers.Reset();
}

#undef USE_RAY_TRACING_DYNAMIC_GEOMETRY_PARALLEL_COMMAND_LISTS

#endif // RHI_RAYTRACING
