// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RayTracingInstance.cpp: Helper functions for creating a ray tracing instance.
=============================================================================*/

#include "RayTracingInstance.h"
#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#endif
