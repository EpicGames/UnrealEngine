// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingInstanceBufferUtil.h"

#include "Lumen/Lumen.h"

#include "RayTracingDefinitions.h"
#include "GPUScene.h"

#include "RenderGraphBuilder.h"
#include "ShaderParameterUtils.h"
#include "RendererInterface.h"
#include "RenderCore.h"
#include "ShaderParameterStruct.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderCompilerCore.h"

#include "SceneRendering.h"

#include "Async/ParallelFor.h"

#include "Experimental/Containers/SherwoodHashTable.h"

#if RHI_RAYTRACING

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/*
*
* Each FRayTracingGeometryInstance can translate to multiple native TLAS instances (see FRayTracingGeometryInstance::NumTransforms).
*
* The FRayTracingGeometryInstance array (ie: FRayTracingScene::Instances) used to create FRayTracingSceneRHI
* can have mix of instances using GPUScene or CPU transforms.
* In order to reduce the number of dispatches to build the native RayTracing Instance Buffer,
* the upload buffer containing FRayTracingInstanceDescriptor is split in 2 sections, [GPUSceneInstances] [CPUInstances].
* This way native GPUScene and CPU instance descriptors can be built in a single dispatch per type.
*
* If the ray tracing scene contains multiple layers, the instance buffer is divided into multiple subranges as expected by the RHI.
*
*/

static TAutoConsoleVariable<bool> CVarRayTracingInstanceBufferRLE(
	TEXT("r.RayTracing.InstanceBuffer.RLE"),
	true,
	TEXT("Whether to use RLE to build ray tracing instance buffer."),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static TAutoConsoleVariable<bool> CVarRayTracingInstanceBufferParallelFill(
	TEXT("r.RayTracing.InstanceBuffer.ParallelFill"),
	true,
	TEXT("Whether to fill the ray tracing instance upload buffer using parallel loops. 0=disabled, 1=enabled (default)"),
	ECVF_RenderThreadSafe);

struct FRayTracingInstanceDescriptor
{
	uint32 GPUSceneInstanceOrTransformIndex;
	uint32 OutputDescriptorIndex;
	uint32 AccelerationStructureIndex;
	uint32 InstanceId;
	uint32 InstanceMaskAndFlags;
	uint32 InstanceContributionToHitGroupIndex;
	uint32 SceneInstanceIndexAndApplyLocalBoundsTransform;
};

struct FRayTracingInstanceGroup
{
	uint32 BaseInstanceIndex : 30;
	uint32 bIncrementUserDataPerInstance : 1;
	uint32 bReuseInstance : 1;
};

static_assert(sizeof(FRayTracingInstanceGroup) == sizeof(uint32), "FRayTracingInstanceGroup is expected be same size as uint32.");

static const uint32 GRayTracingInstanceGroupSize = 64;

// Helper structure to assign instances to FRayTracingInstanceGroup depending on whether the primitive is compatible with RLE
// TODO: Investigate better schemes to maximize RLE usage
// The current implementation fills incomplete "head" group (before generating RLE groups) and might also generate a "tail" group (neither of which can use RLE since they contain instances from different primitives)
// which means in practice only ISMs with >128 instances benefit from RLE unless they happen to end up at group boundaries.
// An alternative approach is to allow incomplete groups instead of packing so aggressively to maximize the number of groups using RLE,
// although that can lead to a lot of inactive threads depending on specific heuristics.
// Primitives could also be sorted by number of instances to reduce fragmentation, etc.
struct FGroupHelper
{
	uint32 CurrentGroupIndex = 0;
	uint32 CurrentIndexInGroup = 0;

	uint32 NumInstanceDescriptors = 0;
	uint32 OptimalNumInstanceDescriptors = 0;

	void AddInstances(uint32 NumInstances, bool bRLECompatible)
	{
		if (bRLECompatible)
		{
			uint32 NumInstancesRemaining = NumInstances;

			if (CurrentIndexInGroup != 0)
			{
				// first N instances are used to fill the current (partial) group
				const uint32 N = FMath::Min(GRayTracingInstanceGroupSize - CurrentIndexInGroup, NumInstancesRemaining);
				NumInstancesRemaining -= N;

				CurrentIndexInGroup += N;
				CurrentGroupIndex += CurrentIndexInGroup / GRayTracingInstanceGroupSize;
				CurrentIndexInGroup %= GRayTracingInstanceGroupSize;

				NumInstanceDescriptors += N;
			}

			if (NumInstancesRemaining > 0)
			{
				check(CurrentIndexInGroup == 0);

				// remaining instances go into packed groups + tail group

				CurrentIndexInGroup += NumInstancesRemaining;
				CurrentGroupIndex += CurrentIndexInGroup / GRayTracingInstanceGroupSize;
				CurrentIndexInGroup %= GRayTracingInstanceGroupSize;

				const uint32 NumPackedGroups = NumInstancesRemaining / GRayTracingInstanceGroupSize;
				NumInstanceDescriptors += NumPackedGroups;
				NumInstanceDescriptors += CurrentIndexInGroup;
			}

			OptimalNumInstanceDescriptors += FMath::DivideAndRoundUp(NumInstances, GRayTracingInstanceGroupSize);
		}
		else
		{
			NumInstanceDescriptors += NumInstances;
			OptimalNumInstanceDescriptors += NumInstances;

			CurrentIndexInGroup += NumInstances;
			CurrentGroupIndex += CurrentIndexInGroup / GRayTracingInstanceGroupSize;
			CurrentIndexInGroup %= GRayTracingInstanceGroupSize;
		}
	}
};

void FRayTracingInstanceBufferBuilder::BuildRayTracingSceneInitializationData()
{
	const bool bRLEAllowed = CVarRayTracingInstanceBufferRLE.GetValueOnRenderThread();

	const uint32 NumSceneInstances = Instances.Num();
	
	Data.NumNativeGPUSceneInstances = 0;
	Data.NumNativeCPUInstances = 0;
	Data.InstanceGeometryIndices.SetNumUninitialized(NumSceneInstances);
	Data.BaseUploadBufferOffsets.SetNumUninitialized(NumSceneInstances);
	Data.BaseInstancePrefixSum.SetNumUninitialized(NumSceneInstances);
	Data.InstanceGroupEntryRefs.SetNumUninitialized(NumSceneInstances);

	TArray<uint32> InstanceGroups;

	Experimental::TSherwoodMap<FRHIRayTracingGeometry*, uint32> UniqueGeometries;

	uint32 NumNativeInstances = 0;

	FGroupHelper GPUGroupHelper;
	FGroupHelper CPUGroupHelper;

	for (TConstSetBitIterator BitIt(VisibleInstances); BitIt; ++BitIt)
	{
		const uint32 InstanceIndex = BitIt.GetIndex();
		const FRayTracingGeometryInstance& InstanceDesc = Instances[InstanceIndex];

		const bool bGpuSceneInstance = InstanceDesc.BaseInstanceSceneDataOffset != -1 || !InstanceDesc.InstanceSceneDataOffsets.IsEmpty();
		const bool bCpuInstance = !bGpuSceneInstance;

		checkf(!bGpuSceneInstance || InstanceDesc.BaseInstanceSceneDataOffset != -1 || InstanceDesc.NumTransforms <= uint32(InstanceDesc.InstanceSceneDataOffsets.Num()),
			TEXT("Expected at least %d ray tracing geometry instance scene data offsets, but got %d."),
			InstanceDesc.NumTransforms, InstanceDesc.InstanceSceneDataOffsets.Num());
		checkf(!bCpuInstance || InstanceDesc.NumTransforms <= uint32(InstanceDesc.Transforms.Num()),
			TEXT("Expected at least %d ray tracing geometry instance transforms, but got %d."),
			InstanceDesc.NumTransforms, InstanceDesc.Transforms.Num());

		if (!EnumHasAnyFlags(InstanceDesc.Flags, ERayTracingInstanceFlags::NaniteRayTracing))
		{
			// we expect visible instances to have a valid ray tracing geometry
			// (unless it's using Nanite Ray Tracing, in which case the ray tracing geometry is determined in RayTracingBuildInstanceBufferCS)
			checkf(InstanceDesc.GeometryRHI, TEXT("Ray tracing instance must have a valid geometry."));
		}

		const uint32 GeometryIndex = InstanceDesc.GeometryRHI != nullptr ? UniqueGeometries.FindOrAdd(InstanceDesc.GeometryRHI, Data.ReferencedGeometries.Num()) : UINT32_MAX;		
		Data.InstanceGeometryIndices[InstanceIndex] = GeometryIndex;
		if (GeometryIndex == Data.ReferencedGeometries.Num())
		{
			Data.ReferencedGeometries.Add(InstanceDesc.GeometryRHI);
		}

		if (bGpuSceneInstance)
		{
			check(InstanceDesc.Transforms.IsEmpty());
			Data.BaseUploadBufferOffsets[InstanceIndex] = GPUGroupHelper.NumInstanceDescriptors;
			Data.NumNativeGPUSceneInstances += InstanceDesc.NumTransforms;
		}
		else if (bCpuInstance)
		{
			Data.BaseUploadBufferOffsets[InstanceIndex] = CPUGroupHelper.NumInstanceDescriptors;
			Data.NumNativeCPUInstances += InstanceDesc.NumTransforms;
		}
		else
		{
			checkNoEntry();
		}
		
		Data.BaseInstancePrefixSum[InstanceIndex] = NumNativeInstances;

		NumNativeInstances += InstanceDesc.NumTransforms;

		const bool bUseUniqueUserData = InstanceDesc.UserData.Num() != 0;

		if (bGpuSceneInstance)
		{
			Data.InstanceGroupEntryRefs[InstanceIndex].GroupIndex = GPUGroupHelper.CurrentGroupIndex;
			Data.InstanceGroupEntryRefs[InstanceIndex].BaseIndexInGroup = GPUGroupHelper.CurrentIndexInGroup;

			const bool bRLECompatible = bRLEAllowed && (InstanceDesc.BaseInstanceSceneDataOffset != -1) && !bUseUniqueUserData;
			GPUGroupHelper.AddInstances(InstanceDesc.NumTransforms, bRLECompatible);
		}
		else
		{
			Data.InstanceGroupEntryRefs[InstanceIndex].GroupIndex = CPUGroupHelper.CurrentGroupIndex;
			Data.InstanceGroupEntryRefs[InstanceIndex].BaseIndexInGroup = CPUGroupHelper.CurrentIndexInGroup;

			const bool bRLECompatible = bRLEAllowed && !bUseUniqueUserData;
			CPUGroupHelper.AddInstances(InstanceDesc.NumTransforms, bRLECompatible);
		}
	}

	Data.NumGPUInstanceGroups = GPUGroupHelper.CurrentGroupIndex + (GPUGroupHelper.CurrentIndexInGroup > 0 ? 1 : 0);
	Data.NumCPUInstanceGroups = CPUGroupHelper.CurrentGroupIndex + (CPUGroupHelper.CurrentIndexInGroup > 0 ? 1 : 0);

	Data.NumGPUInstanceDescriptors = GPUGroupHelper.NumInstanceDescriptors;
	Data.NumCPUInstanceDescriptors = CPUGroupHelper.NumInstanceDescriptors;
}

void WriteInstanceDescriptor(
	const FRayTracingGeometryInstance& SceneInstance,
	uint32 SceneInstanceIndex,
	uint32 TransformIndex,
	uint32 AccelerationStructureIndex,
	bool bGpuSceneInstance,
	bool bUseUniqueUserData,
	bool bUseLightingChannels,
	bool bForceOpaque,
	bool bDisableTriangleCull,
	uint32 BaseInstanceIndex,
	uint32 BaseTransformIndex,
	FRayTracingInstanceDescriptor& OutInstanceDescriptor)
{
	FRayTracingInstanceDescriptor InstanceDesc;

	if (bGpuSceneInstance)
	{
		if (SceneInstance.BaseInstanceSceneDataOffset != -1)
		{
			InstanceDesc.GPUSceneInstanceOrTransformIndex = SceneInstance.BaseInstanceSceneDataOffset + TransformIndex;
		}
		else
		{
			InstanceDesc.GPUSceneInstanceOrTransformIndex = SceneInstance.InstanceSceneDataOffsets[TransformIndex];
		}
	}
	else
	{
		InstanceDesc.GPUSceneInstanceOrTransformIndex = BaseTransformIndex + TransformIndex;
	}

	uint32 UserData;

	if (bUseUniqueUserData)
	{
		UserData = SceneInstance.UserData[TransformIndex];
	}
	else
	{
		UserData = SceneInstance.DefaultUserData;

		if (SceneInstance.bIncrementUserDataPerInstance)
		{
			UserData += TransformIndex;
		}
	}

	ERayTracingInstanceFlags InstanceFlags = SceneInstance.Flags;

	// Run AHS for instances without a default lighting channel for lighting channel filtering
	if (bUseLightingChannels && !(SceneInstance.LightingChannelMask & GetDefaultLightingChannelMask()))
	{
		InstanceFlags &= ~ERayTracingInstanceFlags::ForceOpaque;
	}

	if (bForceOpaque)
	{
		InstanceFlags |= ERayTracingInstanceFlags::ForceOpaque;
	}

	if (bDisableTriangleCull)
	{
		InstanceFlags |= ERayTracingInstanceFlags::TriangleCullDisable;
	}

	InstanceDesc.OutputDescriptorIndex = BaseInstanceIndex + TransformIndex;
	InstanceDesc.AccelerationStructureIndex = AccelerationStructureIndex;
	InstanceDesc.InstanceId = UserData;
	InstanceDesc.InstanceMaskAndFlags = SceneInstance.Mask | ((uint32)InstanceFlags << 8);
	InstanceDesc.InstanceContributionToHitGroupIndex = SceneInstance.InstanceContributionToHitGroupIndex;
	InstanceDesc.SceneInstanceIndexAndApplyLocalBoundsTransform = (SceneInstance.bApplyLocalBoundsTransform ? 0x80000000 : 0) | SceneInstanceIndex;

	ensureMsgf(InstanceDesc.InstanceId <= 0xFFFFFF, TEXT("InstanceId must fit in 24 bits."));
	ensureMsgf(InstanceDesc.InstanceContributionToHitGroupIndex <= 0xFFFFFF, TEXT("InstanceContributionToHitGroupIndex must fit in 24 bits."));

	// copy at the end to avoid reading from OutInstanceDescriptor in the checks above
	OutInstanceDescriptor = InstanceDesc;
}


// Helper function to fill upload buffers required by BuildRayTracingInstanceBuffer with instance descriptors
// Transforms of CPU instances are copied to OutTransformData
void FRayTracingInstanceBufferBuilder::FillRayTracingInstanceUploadBuffer(
	FVector PreViewTranslation,
	TConstArrayView<FRayTracingGeometryInstance> Instances,
	const TBitArray<>& VisibleInstances,
	bool bUseLightingChannels,
	bool bForceOpaque,
	bool bDisableTriangleCull,
	TConstArrayView<uint32> InstanceGeometryIndices,
	TConstArrayView<uint32> BaseUploadBufferOffsets,
	TConstArrayView<uint32> BaseInstancePrefixSum,
	TConstArrayView<FInstanceGroupEntryRef> InstanceGroupEntryRefs,
	uint32 NumGPUInstanceGroups,
	uint32 NumCPUInstanceGroups,
	uint32 NumGPUInstanceDescriptors,
	uint32 NumCPUInstanceDescriptors,
	TArrayView<FRayTracingInstanceGroup> OutInstanceGroupUploadData,
	TArrayView<FRayTracingInstanceDescriptor> OutInstanceUploadData,
	TArrayView<FVector4f> OutTransformData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FillRayTracingInstanceUploadBuffer);

	const bool bRLEAllowed = CVarRayTracingInstanceBufferRLE.GetValueOnRenderThread();

	const int32 NumSceneInstances = VisibleInstances.Num();
	const int32 MinBatchSize = 128;
	ParallelFor(
		TEXT("FillRayTracingInstanceUploadBuffer_Parallel"),
		NumSceneInstances,
		MinBatchSize,
		[
			OutInstanceGroupUploadData,
			OutInstanceUploadData,
			OutTransformData,
			NumGPUInstanceGroups,
			NumCPUInstanceGroups,
			NumGPUInstanceDescriptors,
			NumCPUInstanceDescriptors,
			Instances,
			&VisibleInstances,
			bUseLightingChannels,
			bForceOpaque,
			bDisableTriangleCull,
			InstanceGeometryIndices,
			BaseUploadBufferOffsets,
			BaseInstancePrefixSum,
			InstanceGroupEntryRefs,
			PreViewTranslation,
			bRLEAllowed
		](int32 SceneInstanceIndex)
		{
			if(!VisibleInstances[SceneInstanceIndex])
			{
				return;
			}

			const FRayTracingGeometryInstance& SceneInstance = Instances[SceneInstanceIndex];

			const uint32 NumTransforms = SceneInstance.NumTransforms;

			checkf(SceneInstance.UserData.Num() == 0 || SceneInstance.UserData.Num() >= int32(NumTransforms),
				TEXT("User data array must be either be empty (Instance.DefaultUserData is used), or contain one entry per entry in Transforms array."));			

			const bool bUseUniqueUserData = SceneInstance.UserData.Num() != 0;

			const bool bGpuSceneInstance = SceneInstance.BaseInstanceSceneDataOffset != -1 || !SceneInstance.InstanceSceneDataOffsets.IsEmpty();
			const bool bCpuInstance = !bGpuSceneInstance;

			checkf(bGpuSceneInstance + bCpuInstance == 1, TEXT("Instance can only get transforms from one of GPUScene, or Transforms array."));

			const uint32 AccelerationStructureIndex = InstanceGeometryIndices[SceneInstanceIndex];
			const uint32 BaseInstanceIndex = BaseInstancePrefixSum[SceneInstanceIndex];
			const uint32 BaseTransformIndex = bCpuInstance ? BaseUploadBufferOffsets[SceneInstanceIndex] : 0;

			uint32 BaseDescriptorIndex = BaseUploadBufferOffsets[SceneInstanceIndex];
			uint32 BaseDescriptorOffset = 0;

			// Upload buffer is split into 2 sections [GPUSceneInstances][CPUInstances]
			if (!bGpuSceneInstance)
			{
				BaseDescriptorOffset += NumGPUInstanceDescriptors;
			}

			const bool bRLECompatible = bRLEAllowed && (!bGpuSceneInstance || SceneInstance.BaseInstanceSceneDataOffset != -1) && !bUseUniqueUserData;

			const FInstanceGroupEntryRef& GroupEntryRef = InstanceGroupEntryRefs[SceneInstanceIndex];

			uint32 GroupIndex = GroupEntryRef.GroupIndex;
			uint32 BaseIndexInGroup = GroupEntryRef.BaseIndexInGroup;

			if (bCpuInstance)
			{
				GroupIndex += NumGPUInstanceGroups;
			}

			uint32 TransformIndex = 0;

			if (BaseIndexInGroup > 0)
			{
				// write N instances to fill (partial) head group
				const uint32 N = FMath::Min(GRayTracingInstanceGroupSize - BaseIndexInGroup, NumTransforms);

				for (; TransformIndex < N; ++TransformIndex)
				{
					WriteInstanceDescriptor(
						SceneInstance,
						SceneInstanceIndex,
						TransformIndex,
						AccelerationStructureIndex,
						bGpuSceneInstance,
						bUseUniqueUserData,
						bUseLightingChannels,
						bForceOpaque,
						bDisableTriangleCull,
						BaseInstanceIndex,
						BaseTransformIndex,
						OutInstanceUploadData[BaseDescriptorOffset + BaseDescriptorIndex]);

					++BaseDescriptorIndex;
				}

				++GroupIndex;
			}

			if (bRLECompatible)
			{
				const uint32 NumPackedGroups = (NumTransforms - TransformIndex) / GRayTracingInstanceGroupSize;

				// write packed groups
				for (uint32 PackedGroupIndex = 0; PackedGroupIndex < NumPackedGroups; ++PackedGroupIndex)
				{
					// write packed group
					FRayTracingInstanceGroup Group;
					Group.BaseInstanceIndex = BaseDescriptorIndex;
					Group.bIncrementUserDataPerInstance = SceneInstance.bIncrementUserDataPerInstance ? 1 : 0;
					Group.bReuseInstance = 1;

					OutInstanceGroupUploadData[GroupIndex] = Group;

					++GroupIndex;

					// and corresponding instance
					WriteInstanceDescriptor(
						SceneInstance,
						SceneInstanceIndex,
						TransformIndex,
						AccelerationStructureIndex,
						bGpuSceneInstance,
						bUseUniqueUserData,
						bUseLightingChannels,
						bForceOpaque,
						bDisableTriangleCull,
						BaseInstanceIndex,
						BaseTransformIndex,
						OutInstanceUploadData[BaseDescriptorOffset + BaseDescriptorIndex]);

					++BaseDescriptorIndex;

					TransformIndex += GRayTracingInstanceGroupSize;
				}
			}

			if (TransformIndex < NumTransforms)
			{
				// write tail groups (not packed)

				const uint32 NumTailGroups = FMath::DivideAndRoundUp(NumTransforms - TransformIndex, GRayTracingInstanceGroupSize);

				for (uint32 TailGroupIndex = 0; TailGroupIndex < NumTailGroups; ++TailGroupIndex)
				{
					FRayTracingInstanceGroup Group;
					Group.BaseInstanceIndex = BaseDescriptorIndex + TailGroupIndex * GRayTracingInstanceGroupSize;
					Group.bIncrementUserDataPerInstance = 0;
					Group.bReuseInstance = 0;

					OutInstanceGroupUploadData[GroupIndex] = Group;

					++GroupIndex;
				}

				// and instances
				for (; TransformIndex < NumTransforms; ++TransformIndex)
				{
					WriteInstanceDescriptor(
						SceneInstance,
						SceneInstanceIndex,
						TransformIndex,
						AccelerationStructureIndex,
						bGpuSceneInstance,
						bUseUniqueUserData,
						bUseLightingChannels,
						bForceOpaque,
						bDisableTriangleCull,
						BaseInstanceIndex,
						BaseTransformIndex,
						OutInstanceUploadData[BaseDescriptorOffset + BaseDescriptorIndex]);

					++BaseDescriptorIndex;
				}
			}

			if (bCpuInstance)
			{
				for (uint32 TransformIndex2 = 0; TransformIndex2 < NumTransforms; ++TransformIndex2)
				{
					const uint32 TransformDataOffset = (BaseTransformIndex + TransformIndex2) * 3;
					FMatrix LocalToTranslatedWorld = SceneInstance.Transforms[TransformIndex2].ConcatTranslation(PreViewTranslation);
					const FMatrix44f LocalToTranslatedWorldF = FMatrix44f(LocalToTranslatedWorld.GetTransposed());
					OutTransformData[TransformDataOffset + 0] = *(FVector4f*)&LocalToTranslatedWorldF.M[0];
					OutTransformData[TransformDataOffset + 1] = *(FVector4f*)&LocalToTranslatedWorldF.M[1];
					OutTransformData[TransformDataOffset + 2] = *(FVector4f*)&LocalToTranslatedWorldF.M[2];
				}
			}
		}, CVarRayTracingInstanceBufferParallelFill.GetValueOnRenderThread() ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
}

struct FRayTracingBuildInstanceBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingBuildInstanceBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FRayTracingBuildInstanceBufferCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParametersRHI, GPUSceneParameters)

		SHADER_PARAMETER_UAV(RWStructuredBuffer, OutPlatformInstanceDescriptors)
		SHADER_PARAMETER_UAV(RWStructuredBuffer, OutHitGroupContributions)

		SHADER_PARAMETER_SRV(StructuredBuffer, InstanceGroupDescriptors)
		SHADER_PARAMETER_SRV(StructuredBuffer, InstanceDescriptors)
		SHADER_PARAMETER_SRV(ByteAddressBuffer, AccelerationStructureAddresses)
		SHADER_PARAMETER_SRV(StructuredBuffer, InstanceTransforms)

		SHADER_PARAMETER(uint32, MaxNumInstances)
		SHADER_PARAMETER(uint32, NumGroups)
		SHADER_PARAMETER(uint32, NumInstanceDescriptors)
		SHADER_PARAMETER(uint32, BaseGroupDescriptorIndex)
		SHADER_PARAMETER(uint32, BaseInstanceDescriptorIndex)

		SHADER_PARAMETER(FVector3f, PreViewTranslationHigh)
		SHADER_PARAMETER(FVector3f, PreViewTranslationLow)

		// Instance culling params
		SHADER_PARAMETER(float, CullingRadius)
		SHADER_PARAMETER(float, FarFieldCullingRadius)
		SHADER_PARAMETER(float, AngleThresholdRatioSq)
		SHADER_PARAMETER(FVector3f, ViewOrigin)
		SHADER_PARAMETER(uint32, CullingMode)
		SHADER_PARAMETER(uint32, CullUsingGroups)
		SHADER_PARAMETER(int32, bUseDistanceToBox)

		SHADER_PARAMETER_SRV(ByteAddressBuffer, NaniteBLASData)

		SHADER_PARAMETER_UAV(RWStructuredBuffer<uint>, RWOutputStats)
		SHADER_PARAMETER(uint32, OutputStatsOffset)

		// Debug parameters
		SHADER_PARAMETER_UAV(RWStructuredBuffer, RWInstanceExtraData)
	END_SHADER_PARAMETER_STRUCT()

	class FSupportInstanceGroupsDim : SHADER_PERMUTATION_BOOL("SUPPORT_INSTANCE_GROUPS");
	class FUseGPUSceneDim : SHADER_PERMUTATION_BOOL("USE_GPUSCENE");
	class FOutputInstanceExtraDataDim : SHADER_PERMUTATION_BOOL("OUTPUT_INSTANCE_EXTRA_DATA");
	class FGpuCullingDim : SHADER_PERMUTATION_BOOL("GPU_CULLING");
	class FNaniteRayTracingDim : SHADER_PERMUTATION_BOOL("NANITE_RAYTRACING");
	class FOutputStatsDim : SHADER_PERMUTATION_BOOL("OUTPUT_STATS");
	class FCompactOutputDim : SHADER_PERMUTATION_BOOL("COMPACT_OUTPUT");
	class FUseWaveOpsDim : SHADER_PERMUTATION_BOOL("USE_WAVE_OPS");
	class FUseSeparateHitContributionBuffer : SHADER_PERMUTATION_BOOL("USE_SEPARATE_HIT_CONTRIBUTION_BUFFER");
	using FPermutationDomain = TShaderPermutationDomain<FSupportInstanceGroupsDim, FUseGPUSceneDim, FOutputInstanceExtraDataDim, FGpuCullingDim, FNaniteRayTracingDim, FOutputStatsDim, FCompactOutputDim, FUseWaveOpsDim, FUseSeparateHitContributionBuffer>;
		
	static constexpr uint32 ThreadGroupSize = GRayTracingInstanceGroupSize;

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);

		// Force DXC to avoid shader reflection issues.
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (PermutationVector.Get<FUseWaveOpsDim>() && !RHISupportsWaveOperations(Parameters.Platform))
		{
			return false;
		}

		// TODO: Once we update FRayTracingInstanceBufferBuilder to use FRDGBuilder
		// we could replace this permutation with a dynamic branch and simply bind a dummy buffer when none is provided
		if (PermutationVector.Get<FUseSeparateHitContributionBuffer>() != IsMetalPlatform(Parameters.Platform))
		{
			return false;
		}
		
		if (PermutationVector.Get<FNaniteRayTracingDim>() && (!PermutationVector.Get<FUseGPUSceneDim>() || !DoesPlatformSupportNanite(Parameters.Platform)))
		{
			// Nanite RayTracing requires GPUScene to access PrimitiveData.NaniteResourceID
			return false;
		}

		if (PermutationVector.Get<FGpuCullingDim>() && !PermutationVector.Get<FUseGPUSceneDim>())
		{
			// GPU culling requires GPUScene
			return false;
		}

		return IsRayTracingEnabledForProject(Parameters.Platform);
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		if (PermutationVector.Get<FUseWaveOpsDim>() != GRHIGlobals.SupportsWaveOperations)
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}

		if (PermutationVector.Get<FUseSeparateHitContributionBuffer>() != GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer)
		{
			return EShaderPermutationPrecacheRequest::NotPrecached;
		}

		return FGlobalShader::ShouldPrecachePermutation(Parameters);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingBuildInstanceBufferCS, "/Engine/Private/RayTracing/RayTracingInstanceBufferUtil.usf", "RayTracingBuildInstanceBufferCS", SF_Compute);

void BuildRayTracingInstanceBuffer(
	FRHICommandList& RHICmdList,
	const FGPUSceneResourceParametersRHI* GPUSceneParameters,
	const FDFVector3& PreViewTranslation,
	uint32 MaxNumInstances,
	uint32 NumGroups,
	uint32 NumInstanceDescriptors,
	FRHIUnorderedAccessView* InstancesUAV,
	FRHIUnorderedAccessView* HitGroupContributionsUAV,
	FRHIShaderResourceView* InstanceGroupUploadSRV,
	uint32 InstanceGroupUploadOffset,
	FRHIShaderResourceView* InstanceUploadSRV,
	uint32 InstanceUploadOffset,
	FRHIShaderResourceView* AccelerationStructureAddressesSRV,
	FRHIShaderResourceView* InstanceTransformSRV,
	const FRayTracingCullingParameters* CullingParameters,
	FRHIShaderResourceView* NaniteBLASDataSRV,
	bool bCompactOutput,
	FRHIUnorderedAccessView* OutputStatsUAV,
	uint32 OutputStatsOffset,
	FRHIUnorderedAccessView* InstanceExtraDataUAV)
{
	FRayTracingBuildInstanceBufferCS::FParameters PassParams;
	PassParams.OutPlatformInstanceDescriptors = InstancesUAV;
	PassParams.OutHitGroupContributions = HitGroupContributionsUAV;
	PassParams.InstanceGroupDescriptors = InstanceGroupUploadSRV;
	PassParams.InstanceDescriptors = InstanceUploadSRV;
	PassParams.AccelerationStructureAddresses = AccelerationStructureAddressesSRV;
	PassParams.InstanceTransforms = InstanceTransformSRV;
	PassParams.MaxNumInstances = MaxNumInstances;
	PassParams.NumGroups = NumGroups;
	PassParams.NumInstanceDescriptors = NumInstanceDescriptors;
	PassParams.BaseGroupDescriptorIndex = InstanceGroupUploadOffset;
	PassParams.BaseInstanceDescriptorIndex = InstanceUploadOffset;
	PassParams.PreViewTranslationHigh = PreViewTranslation.High;
	PassParams.PreViewTranslationLow = PreViewTranslation.Low;

	if (GPUSceneParameters)
	{
		PassParams.GPUSceneParameters = *GPUSceneParameters;
	}

	if (CullingParameters)
	{
		PassParams.CullingRadius = CullingParameters->CullingRadius;
		PassParams.FarFieldCullingRadius = CullingParameters->FarFieldCullingRadius;
		PassParams.AngleThresholdRatioSq = CullingParameters->AngleThresholdRatioSq;
		PassParams.ViewOrigin = CullingParameters->TranslatedViewOrigin;
		PassParams.CullingMode = uint32(CullingParameters->CullingMode);
		PassParams.CullUsingGroups = uint32(CullingParameters->bCullUsingGroupIds);
		PassParams.bUseDistanceToBox = CullingParameters->bUseDistanceToBox ? 1 : 0;
	}

	PassParams.NaniteBLASData = NaniteBLASDataSRV;

	PassParams.RWOutputStats = OutputStatsUAV;
	PassParams.OutputStatsOffset = OutputStatsOffset;

	PassParams.RWInstanceExtraData = InstanceExtraDataUAV;

	const bool bUseGPUScene = InstanceTransformSRV == nullptr;

	FRayTracingBuildInstanceBufferCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRayTracingBuildInstanceBufferCS::FSupportInstanceGroupsDim>(InstanceGroupUploadSRV != nullptr);
	PermutationVector.Set<FRayTracingBuildInstanceBufferCS::FUseGPUSceneDim>(bUseGPUScene);
	PermutationVector.Set<FRayTracingBuildInstanceBufferCS::FNaniteRayTracingDim>(bUseGPUScene && NaniteBLASDataSRV != nullptr);
	PermutationVector.Set<FRayTracingBuildInstanceBufferCS::FOutputInstanceExtraDataDim>(InstanceExtraDataUAV != nullptr);
	PermutationVector.Set<FRayTracingBuildInstanceBufferCS::FOutputStatsDim>(OutputStatsUAV != nullptr);
	PermutationVector.Set<FRayTracingBuildInstanceBufferCS::FCompactOutputDim>(bCompactOutput);
	PermutationVector.Set<FRayTracingBuildInstanceBufferCS::FUseWaveOpsDim>(GRHIGlobals.SupportsWaveOperations);
	PermutationVector.Set<FRayTracingBuildInstanceBufferCS::FGpuCullingDim>(CullingParameters != nullptr);
	PermutationVector.Set<FRayTracingBuildInstanceBufferCS::FUseSeparateHitContributionBuffer>(GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer);

	auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FRayTracingBuildInstanceBufferCS>(PermutationVector);

	const FIntVector GroupCount = FComputeShaderUtils::GetGroupCountWrapped(PassParams.NumGroups);

	//ClearUnusedGraphResources(ComputeShader, &PassParams);

	SetComputePipelineState(RHICmdList, ComputeShader.GetComputeShader());

	SetShaderParameters(RHICmdList, ComputeShader, ComputeShader.GetComputeShader(), PassParams);

	DispatchComputeShader(RHICmdList, ComputeShader.GetShader(), GroupCount.X, GroupCount.Y, GroupCount.Z);

	UnsetShaderUAVs(RHICmdList, ComputeShader, ComputeShader.GetComputeShader());
}

void BuildRayTracingInstanceBuffer(
	FRHICommandList& RHICmdList,
	const FGPUSceneResourceParametersRHI* GPUSceneParameters,
	const FDFVector3& PreViewTranslation,
	FRHIUnorderedAccessView* InstancesUAV,
	FRHIUnorderedAccessView* HitGroupContributionsUAV,
	FRHIShaderResourceView* InstanceGroupUploadSRV,
	FRHIShaderResourceView* InstanceUploadSRV,
	FRHIShaderResourceView* AccelerationStructureAddressesSRV,
	FRHIShaderResourceView* CPUInstanceTransformSRV,
	uint32 MaxNumInstances,
	uint32 NumGPUGroups,
	uint32 NumCPUGroups,
	uint32 NumGPUInstanceDescriptors,
	uint32 NumCPUInstanceDescriptors,
	const FRayTracingCullingParameters* CullingParameters,
	FRHIShaderResourceView* NaniteBLASDataSRV,
	bool bCompactOutput,
	FRHIUnorderedAccessView* OutputStatsUAV,
	uint32 OutputStatsOffset,
	FRHIUnorderedAccessView* InstanceExtraDataUAV)
{
	if (NumGPUInstanceDescriptors > 0)
	{
		BuildRayTracingInstanceBuffer(
			RHICmdList,
			GPUSceneParameters,
			PreViewTranslation,
			MaxNumInstances,
			NumGPUGroups,
			NumGPUInstanceDescriptors,
			InstancesUAV,
			HitGroupContributionsUAV,
			InstanceGroupUploadSRV,
			0,
			InstanceUploadSRV,
			0,
			AccelerationStructureAddressesSRV,
			nullptr,
			CullingParameters,
			NaniteBLASDataSRV,
			bCompactOutput,
			OutputStatsUAV,
			OutputStatsOffset,
			InstanceExtraDataUAV);
	}

	if (NumCPUInstanceDescriptors > 0)
	{
		BuildRayTracingInstanceBuffer(
			RHICmdList,
			GPUSceneParameters,
			PreViewTranslation,
			MaxNumInstances,
			NumCPUGroups,
			NumCPUInstanceDescriptors,
			InstancesUAV,
			HitGroupContributionsUAV,
			InstanceGroupUploadSRV,
			NumGPUGroups, // CPU instance group descriptors are stored after GPU Scene instance groups
			InstanceUploadSRV,
			NumGPUInstanceDescriptors, // CPU input instance descriptors are stored after GPU Scene instances
			AccelerationStructureAddressesSRV,
			CPUInstanceTransformSRV,
			nullptr,
			NaniteBLASDataSRV,
			bCompactOutput,
			OutputStatsUAV,
			OutputStatsOffset,
			InstanceExtraDataUAV);
	}
}

void FRayTracingInstanceBufferBuilder::Init(TConstArrayView<FRayTracingGeometryInstance> InInstances, FVector InPreViewTranslation)
{
	checkf(Instances.IsEmpty(), TEXT("RayTracingInstanceBufferBuilder was already initialized."));

	Instances = InInstances;
	PreViewTranslation = InPreViewTranslation;

	// Deprecated code path assumes all provided instances are visible.
	VisibleInstances.SetNum(Instances.Num(), true);

	BuildRayTracingSceneInitializationData();
}

void FRayTracingInstanceBufferBuilder::Init(FRayTracingInstanceBufferBuilderInitializer Initializer)
{
	Instances = Initializer.Instances;
	VisibleInstances = MoveTemp(Initializer.VisibleInstances);
	PreViewTranslation = Initializer.PreViewTranslation;
	
	bUseLightingChannels = Initializer.bUseLightingChannels;
	bForceOpaque = Initializer.bForceOpaque;
	bDisableTriangleCull = Initializer.bDisableTriangleCull;

	BuildRayTracingSceneInitializationData();
}

void FRayTracingInstanceBufferBuilder::FillRayTracingInstanceUploadBuffer(FRHICommandList& RHICmdList)
{
	// Round up buffer sizes to some multiple to avoid pathological growth reallocations.
	static constexpr uint32 AllocationGranularity = 8 * 1024;
	static constexpr uint64 BufferAllocationGranularity = 16 * 1024 * 1024;

	const uint32 NumInstanceGroups = Data.NumGPUInstanceGroups + Data.NumCPUInstanceGroups;
	const uint32 NumInstanceGroupsAligned = FMath::DivideAndRoundUp(FMath::Max(NumInstanceGroups, 1U), AllocationGranularity) * AllocationGranularity;

	const uint32 NumInstanceDescriptors = Data.NumGPUInstanceDescriptors + Data.NumCPUInstanceDescriptors;
	const uint32 NumInstanceDescriptorsAligned = FMath::DivideAndRoundUp(FMath::Max(NumInstanceDescriptors, 1U), AllocationGranularity) * AllocationGranularity;

	const uint32 NumTransformsAligned = FMath::DivideAndRoundUp(FMath::Max(Data.NumNativeCPUInstances, 1U), AllocationGranularity) * AllocationGranularity;

	{
		// Create/resize instance group upload buffer (if necessary)
		const uint32 UploadBufferSize = NumInstanceGroupsAligned * sizeof(FRayTracingInstanceGroup);

		if (!InstanceGroupUploadBuffer.IsValid()
			|| UploadBufferSize > InstanceGroupUploadBuffer->GetSize()
			|| UploadBufferSize < InstanceGroupUploadBuffer->GetSize() / 2)
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("FRayTracingScene::InstanceGroupUploadBuffer"), UploadBufferSize, sizeof(FRayTracingInstanceGroup))
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Volatile)
				.DetermineInitialState();

			InstanceGroupUploadBuffer = RHICmdList.CreateBuffer(CreateDesc);
			InstanceGroupUploadSRV = RHICmdList.CreateShaderResourceView(InstanceGroupUploadBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(InstanceGroupUploadBuffer));
		}
	}

	{
		// Create/resize instance upload buffer (if necessary)
		const uint32 UploadBufferSize = NumInstanceDescriptorsAligned * sizeof(FRayTracingInstanceDescriptor);

		if (!InstanceUploadBuffer.IsValid()
			|| UploadBufferSize > InstanceUploadBuffer->GetSize()
			|| UploadBufferSize < InstanceUploadBuffer->GetSize() / 2)
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("FRayTracingScene::InstanceUploadBuffer"), UploadBufferSize, sizeof(FRayTracingInstanceDescriptor))
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Volatile)
				.DetermineInitialState();

			InstanceUploadBuffer = RHICmdList.CreateBuffer(CreateDesc);
			InstanceUploadSRV = RHICmdList.CreateShaderResourceView(InstanceUploadBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(InstanceUploadBuffer));
		}
	}

	{
		const uint32 UploadBufferSize = NumTransformsAligned * sizeof(FVector4f) * 3;

		// Create/resize transform upload buffer (if necessary)
		if (!TransformUploadBuffer.IsValid()
			|| UploadBufferSize > TransformUploadBuffer->GetSize()
			|| UploadBufferSize < TransformUploadBuffer->GetSize() / 2)
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateStructured(TEXT("FRayTracingScene::TransformUploadBuffer"), UploadBufferSize, sizeof(FVector4f))
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Volatile)
				.DetermineInitialState();

			TransformUploadBuffer = RHICmdList.CreateBuffer(CreateDesc);
			TransformUploadSRV = RHICmdList.CreateShaderResourceView(TransformUploadBuffer, FRHIViewDesc::CreateBufferSRV().SetTypeFromBuffer(TransformUploadBuffer));
		}
	}

	const uint32 InstanceGroupUploadBytes = NumInstanceGroups * sizeof(FRayTracingInstanceGroup);
	const uint32 InstanceUploadBytes = NumInstanceDescriptors * sizeof(FRayTracingInstanceDescriptor);
	const uint32 TransformUploadBytes = Data.NumNativeCPUInstances * 3 * sizeof(FVector4f);

	FRayTracingInstanceGroup* InstanceGroupUploadData = (FRayTracingInstanceGroup*)RHICmdList.LockBuffer(InstanceGroupUploadBuffer, 0, InstanceGroupUploadBytes, RLM_WriteOnly);
	FRayTracingInstanceDescriptor* InstanceUploadData = (FRayTracingInstanceDescriptor*)RHICmdList.LockBuffer(InstanceUploadBuffer, 0, InstanceUploadBytes, RLM_WriteOnly);
	FVector4f* TransformUploadData = (Data.NumNativeCPUInstances > 0) ? (FVector4f*)RHICmdList.LockBuffer(TransformUploadBuffer, 0, TransformUploadBytes, RLM_WriteOnly) : nullptr;

	FillRayTracingInstanceUploadBuffer(
		PreViewTranslation,
		Instances,
		VisibleInstances,
		bUseLightingChannels,
		bForceOpaque,
		bDisableTriangleCull,
		Data.InstanceGeometryIndices,
		Data.BaseUploadBufferOffsets,
		Data.BaseInstancePrefixSum,
		Data.InstanceGroupEntryRefs,
		Data.NumGPUInstanceGroups,
		Data.NumCPUInstanceGroups,
		Data.NumGPUInstanceDescriptors,
		Data.NumCPUInstanceDescriptors,
		MakeArrayView(InstanceGroupUploadData, NumInstanceGroups),
		MakeArrayView(InstanceUploadData, NumInstanceDescriptors),
		MakeArrayView(TransformUploadData, Data.NumNativeCPUInstances * 3));

	RHICmdList.UnlockBuffer(InstanceGroupUploadBuffer);
	RHICmdList.UnlockBuffer(InstanceUploadBuffer);

	if (Data.NumNativeCPUInstances > 0)
	{
		RHICmdList.UnlockBuffer(TransformUploadBuffer);
	}
}

void FRayTracingInstanceBufferBuilder::FillAccelerationStructureAddressesBuffer(FRHICommandList& RHICmdList)
{
	const uint32 NumGeometries = FMath::RoundUpToPowerOfTwo(Data.ReferencedGeometries.Num());

	{
		// Round to PoT to avoid resizing too often
		const uint32 NumGeometriesAligned = FMath::RoundUpToPowerOfTwo(NumGeometries);
		const uint32 AccelerationStructureAddressesBufferSize = NumGeometriesAligned * sizeof(uint64);

		if (AccelerationStructureAddressesBuffer.NumBytes < AccelerationStructureAddressesBufferSize)
		{
			// Need to pass "BUF_MultiGPUAllocate", as virtual addresses are different per GPU
			AccelerationStructureAddressesBuffer.Initialize(RHICmdList, TEXT("FRayTracingScene::AccelerationStructureAddressesBuffer"), AccelerationStructureAddressesBufferSize, BUF_Volatile | BUF_MultiGPUAllocate);
		}
	}

	TConstArrayView<FRHIResource*> ReferencedResources = MakeConstArrayView(reinterpret_cast<FRHIResource**>(Data.ReferencedGeometries.GetData()), Data.ReferencedGeometries.Num());
	

	RHICmdList.WriteResourceGPUAddresses(ReferencedResources, AccelerationStructureAddressesBuffer.Buffer);
}

void FRayTracingInstanceBufferBuilder::BuildRayTracingInstanceBuffer(
	FRHICommandList& RHICmdList,
	const FGPUScene* GPUScene,
	const FRayTracingCullingParameters* CullingParameters,
	FRHIUnorderedAccessView* InstancesUAV,
	FRHIUnorderedAccessView* HitGroupContributionsUAV,
	uint32 MaxNumInstances,
	bool bCompactOutput,
	FRHIUnorderedAccessView* OutputStatsUAV,
	uint32 OutputStatsOffset,
	FRHIUnorderedAccessView* InstanceExtraDataUAV)
{
	FGPUSceneResourceParametersRHI GPUSceneParameters;
	if (GPUScene)
	{
		GPUSceneParameters = GPUScene->GetShaderParametersRHI();
	}

	::BuildRayTracingInstanceBuffer(
		RHICmdList,
		GPUScene ? &GPUSceneParameters : nullptr,
		FDFVector3(PreViewTranslation),
		InstancesUAV,
		HitGroupContributionsUAV,
		InstanceGroupUploadSRV,
		InstanceUploadSRV,
		AccelerationStructureAddressesBuffer.SRV,
		TransformUploadSRV,
		MaxNumInstances,
		Data.NumGPUInstanceGroups,
		Data.NumCPUInstanceGroups,
		Data.NumGPUInstanceDescriptors,
		Data.NumCPUInstanceDescriptors,
		CullingParameters,
		nullptr,
		bCompactOutput,
		OutputStatsUAV,
		OutputStatsOffset,
		InstanceExtraDataUAV);
}

void FRayTracingInstanceBufferBuilder::BuildRayTracingInstanceBuffer(
	FRHICommandList& RHICmdList,
	const FGPUScene* GPUScene,
	const FGPUSceneResourceParametersRHI* GPUSceneParameters,
	const FRayTracingCullingParameters* CullingParameters,
	FRHIUnorderedAccessView* InstancesUAV,
	FRHIUnorderedAccessView* HitGroupContributionsUAV,
	uint32 MaxNumInstances,
	FRHIShaderResourceView* NaniteBLASDataSRV,
	bool bCompactOutput,
	FRHIUnorderedAccessView* OutputStatsUAV,
	uint32 OutputStatsOffset,
	FRHIUnorderedAccessView* InstanceExtraDataUAV)
{
	::BuildRayTracingInstanceBuffer(
		RHICmdList,
		GPUSceneParameters,
		FDFVector3(PreViewTranslation),
		InstancesUAV,
		HitGroupContributionsUAV,
		InstanceGroupUploadSRV,
		InstanceUploadSRV,
		AccelerationStructureAddressesBuffer.SRV,
		TransformUploadSRV,
		MaxNumInstances,
		Data.NumGPUInstanceGroups,
		Data.NumCPUInstanceGroups,
		Data.NumGPUInstanceDescriptors,
		Data.NumCPUInstanceDescriptors,
		CullingParameters,
		NaniteBLASDataSRV,
		bCompactOutput,
		OutputStatsUAV,
		OutputStatsOffset,
		InstanceExtraDataUAV);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif //RHI_RAYTRACING
