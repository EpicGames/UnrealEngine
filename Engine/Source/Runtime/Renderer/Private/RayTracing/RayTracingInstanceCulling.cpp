// Copyright Epic Games, Inc.All Rights Reserved.

#include "RayTracingInstanceCulling.h"

#if RHI_RAYTRACING

#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Lumen/Lumen.h"
#include "Math/BoxSphereBounds.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "MegaLights/MegaLights.h"
#include "PrimitiveSceneProxy.h"
#include "RayTracing/RaytracingOptions.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneView.h"
#include "ShowFlags.h"

static TAutoConsoleVariable<int32> CVarRayTracingCulling(
	TEXT("r.RayTracing.Culling"),
	3,
	TEXT("Enable culling in ray tracing for objects that are behind the camera\n")
	TEXT(" 0: Culling disabled (default)\n")
	TEXT(" 1: Culling by distance and solid angle enabled. Only cull objects behind camera.\n")
	TEXT(" 2: Culling by distance and solid angle enabled. Cull objects in front and behind camera.\n")
	TEXT(" 3: Culling by distance OR solid angle enabled. Cull objects in front and behind camera."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingCullingPerInstance(
	TEXT("r.RayTracing.Culling.PerInstance"),
	1,
	TEXT(""),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRayTracingCullingRadius(
	TEXT("r.RayTracing.Culling.Radius"),
	30000.0f,
	TEXT("Do camera culling for objects behind the camera outside of this radius in ray tracing effects (default = 30000 (300m))"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarRayTracingCullingAngle(
	TEXT("r.RayTracing.Culling.Angle"),
	1.0f,
	TEXT("Do camera culling for objects behind the camera with a projected angle smaller than this threshold in ray tracing effects (default = 1 degrees)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingCullingUseMinDrawDistance(
	TEXT("r.RayTracing.Culling.UseMinDrawDistance"),
	1,
	TEXT("Use min draw distance for culling"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingCullingGroupIds(
	TEXT("r.RayTracing.Culling.UseGroupIds"),
	0,
	TEXT("Cull using aggregate ray tracing group id bounds when defined instead of primitive or instance bounds."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRayTracingCullingUseDistanceToBox(
	TEXT("r.RayTracing.Culling.UseDistanceToBox"),
	1,
	TEXT("Whether to compute distance to objects as a distance to their bbox or bounding sphere."),
	ECVF_RenderThreadSafe);

RayTracing::ECullingMode RayTracing::GetCullingMode(const FEngineShowFlags& ShowFlags)
{
	// Disable culling if path tracer is used, so that path tracer matches raster view
	if (ShowFlags.PathTracing)
	{
		return ECullingMode::Disabled;
	}
	
	return (ECullingMode) FMath::Clamp(CVarRayTracingCulling.GetValueOnRenderThread(), 0, int32(ECullingMode::MAX) - 1);
}

static bool CullingUseDistanceToBox()
{
	return CVarRayTracingCullingUseDistanceToBox.GetValueOnRenderThread() != 0;
}

float GetRayTracingCullingRadius()
{
	return CVarRayTracingCullingRadius.GetValueOnRenderThread();
}

void FRayTracingCullingParameters::Init(FViewInfo& View)
{
	CullingMode = RayTracing::GetCullingMode(View.Family->EngineShowFlags);
	CullingRadius = CVarRayTracingCullingRadius.GetValueOnRenderThread();
	FarFieldCullingRadius = Lumen::GetFarFieldMaxTraceDistance();
	CullAngleThreshold = CVarRayTracingCullingAngle.GetValueOnRenderThread();
	AngleThresholdRatio = FMath::Tan(FMath::Min(89.99f, CullAngleThreshold) * PI / 180.0f);
	AngleThresholdRatioSq = FMath::Square(AngleThresholdRatio);
	ViewOrigin = View.ViewMatrices.GetViewOrigin();
	TranslatedViewOrigin = FVector3f(ViewOrigin + View.ViewMatrices.GetPreViewTranslation());
	ViewDirection = View.GetViewDirection();
	bCullAllObjects = CullingMode == RayTracing::ECullingMode::DistanceAndSolidAngle || CullingMode == RayTracing::ECullingMode::DistanceOrSolidAngle;
	bCullByRadiusOrDistance = CullingMode == RayTracing::ECullingMode::DistanceOrSolidAngle;
	bIsRayTracingFarField = Lumen::UseFarField(*View.Family) || MegaLights::UseFarField(*View.Family);
	bCullUsingGroupIds = CVarRayTracingCullingGroupIds.GetValueOnRenderThread() != 0;
	bCullMinDrawDistance = CVarRayTracingCullingUseMinDrawDistance.GetValueOnRenderThread() != 0;
	bUseInstanceCulling = CVarRayTracingCullingPerInstance.GetValueOnRenderThread() != 0 && bCullAllObjects;
	bUseDistanceToBox = CullingUseDistanceToBox();
}

namespace RayTracing
{
bool ShouldConsiderCulling(const FRayTracingCullingParameters& CullingParameters, const FBoxSphereBounds& ObjectBounds, float MinDrawDistance)
{
	FVector CameraToObjectCenter = ObjectBounds.Origin - CullingParameters.ViewOrigin;
	bool bBehindCamera = FVector::DotProduct(CullingParameters.ViewDirection, CameraToObjectCenter) < -ObjectBounds.SphereRadius;
	bool bNearEnoughToCull = CullingParameters.bCullMinDrawDistance && (FVector::DistSquared(ObjectBounds.Origin, CullingParameters.ViewOrigin) < FMath::Square(MinDrawDistance));
	return bBehindCamera || bNearEnoughToCull;
}

bool CullPrimitiveByFlags(const FRayTracingCullingParameters& CullingParameters, const FScene* RESTRICT Scene, int32 PrimitiveIndex)
{
	if (EnumHasAnyFlags(Scene->PrimitiveRayTracingFlags[PrimitiveIndex], ERayTracingPrimitiveFlags::UnsupportedProxyType))
	{
		checkf(Scene->PrimitiveRayTracingFlags[PrimitiveIndex] == ERayTracingPrimitiveFlags::UnsupportedProxyType, TEXT("ERayTracingPrimitiveFlags::UnsupportedProxyType should not be combined with other flags"));
		return true;
	}

	if (EnumHasAnyFlags(Scene->PrimitiveRayTracingFlags[PrimitiveIndex], ERayTracingPrimitiveFlags::Exclude))
	{
		return true;
	}

	// Skip far field if not enabled
	const bool bIsFarFieldPrimitive = EnumHasAnyFlags(Scene->PrimitiveRayTracingFlags[PrimitiveIndex], ERayTracingPrimitiveFlags::FarField);
	if (!CullingParameters.bIsRayTracingFarField && bIsFarFieldPrimitive)
	{
		return true;
	}

	return false;
}

// Tests if the given primitive should be culled, given the pre-calculated inputs for the primitive, returns true if the primitive SHOULD be culled
template<bool bCullByRadiusOrDistance>
bool CullBounds(const FRayTracingCullingParameters& CullingParameters, const FBoxSphereBounds& RESTRICT ObjectBounds, float MinDrawDistance, bool bIsFarFieldPrimitive)
{
	const float ObjectRadius = ObjectBounds.SphereRadius;
	const FVector ObjectCenter = ObjectBounds.Origin;

	const FVector3f TranslatedObjectCenter = FVector3f(ObjectCenter - CullingParameters.ViewOrigin);
	const float CameraToObjectCenterLengthSq = TranslatedObjectCenter.SquaredLength();

	float CameraToObjectDistanceSq;
	if (CullingParameters.bUseDistanceToBox)
	{
		const FVector3f BoxExtent = FVector3f(ObjectBounds.BoxExtent);
		const FVector3f BoxMin = TranslatedObjectCenter - BoxExtent;
		const FVector3f BoxMax = TranslatedObjectCenter + BoxExtent;
		CameraToObjectDistanceSq = ComputeSquaredDistanceFromBoxToPoint(BoxMin, BoxMax, FVector3f::Zero());
	}
	else
	{
		const float DistanceToSphere = FMath::Max(FMath::Sqrt(CameraToObjectCenterLengthSq) - ObjectRadius, 0.0f);
		CameraToObjectDistanceSq = FMath::Square(DistanceToSphere);
	}

	if (bIsFarFieldPrimitive)
	{
		if (CameraToObjectDistanceSq > FMath::Square(CullingParameters.FarFieldCullingRadius))
		{
			return true;
		}
	}
	else
	{
		const bool bIsFarEnoughToCull = CameraToObjectDistanceSq > FMath::Square(CullingParameters.CullingRadius);
		const bool bIsNearEnoughToCull = CameraToObjectCenterLengthSq < FMath::Square(MinDrawDistance);

		if (CullingParameters.bCullMinDrawDistance && bIsNearEnoughToCull)
		{
			return true;
		}
		else if (bCullByRadiusOrDistance && bIsFarEnoughToCull)
		{
			return true;
		}
		else
		{
			// Cull by solid angle: check the radius of bounding sphere against angle threshold
			const bool bAngleIsSmallEnoughToCull = FMath::Square(ObjectRadius) / CameraToObjectCenterLengthSq < CullingParameters.AngleThresholdRatioSq;
			if (bCullByRadiusOrDistance && bAngleIsSmallEnoughToCull)
			{
				return true;
			}
			else if (bIsFarEnoughToCull && bAngleIsSmallEnoughToCull)
			{
				return true;
			}
		}
	}

	return false;
}

bool ShouldCullBounds(const FRayTracingCullingParameters& CullingParameters, const FBoxSphereBounds& RESTRICT ObjectBounds, float MinDrawDistance, bool bIsFarFieldPrimitive)
{
	if (CullingParameters.CullingMode != RayTracing::ECullingMode::Disabled)
	{
		const bool bConsiderCulling = CullingParameters.bCullAllObjects || ShouldConsiderCulling(CullingParameters, ObjectBounds, MinDrawDistance);

		if (bConsiderCulling)
		{
			if (CullingParameters.bCullByRadiusOrDistance)
			{
				return CullBounds<true>(CullingParameters, ObjectBounds, MinDrawDistance, bIsFarFieldPrimitive);
			}
			else
			{
				return CullBounds<false>(CullingParameters, ObjectBounds, MinDrawDistance, bIsFarFieldPrimitive);
			}
		}
	}

	return false;
}

} // namspace RayTracing

#endif
