// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#include "Containers/ArrayView.h"
#include "Math/MathFwd.h"

class FPrimitiveSceneInfo;
class FScene;
struct FEngineShowFlags;

struct FRayTracingCullingParameters;

float GetRayTracingCullingRadius();

namespace RayTracing
{

enum class ECullingMode : uint8;

ECullingMode GetCullingMode(const FEngineShowFlags& ShowFlags);

// Tests if a primitive with the given inputs should be considered for culling.  Does NOT test configuration values.  The assumption is that the config values have already been tested.
bool ShouldConsiderCulling(const FRayTracingCullingParameters& CullingParameters, const FBoxSphereBounds& ObjectBounds, float MinDrawDistance);

// Returns true if the primitive should be culled out due to it's ray tracing flags
bool CullPrimitiveByFlags(const FRayTracingCullingParameters& CullingParameters, const FScene* RESTRICT Scene, int32 PrimitiveIndex);

// Completely test if the bounds should be culled for ray tracing. This includes all configuration values.
bool ShouldCullBounds(const FRayTracingCullingParameters& CullingParameters, const FBoxSphereBounds& ObjectBounds, float MinDrawDistance, bool bIsFarFieldPrimitive);
}

#endif
