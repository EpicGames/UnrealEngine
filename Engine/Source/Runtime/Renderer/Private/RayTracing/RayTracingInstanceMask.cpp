// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingInstanceMask.h"

#if RHI_RAYTRACING

#include "DataDrivenShaderPlatformInfo.h"
#include "RayTracingDefinitions.h"
#include "PathTracingDefinitions.h"
#include "PrimitiveSceneProxy.h"
#include "Materials/MaterialRenderProxy.h"
#include "MeshPassProcessor.h"
#include "ScenePrivate.h"
#include "Rendering/NaniteResources.h"

namespace {

// Helper class so we can refer to both RAY_TRACING_MASK_* and PATH_TRACING_MASK_* in a unified way within this file
enum class ERayTracingInstanceMaskType : uint8
{
	// General mask type for primary and secondary rays
	Opaque,
	Translucent,

	// Mask types for shadow rays
	OpaqueShadow,
	TranslucentShadow,
	ThinShadow,

	// Geometry specific ray types
	HairStrands,

	// Special purpose ray types
	FirstPersonWorldSpaceRepresentation,

	// Ray types according to illumination type
	VisibleInPrimaryRay,
	VisibleInIndirectRay,
	VisibleInReflectionRay,

};

} // anonymous namespace

static uint8 ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType MaskType, ERayTracingType RayTracingType)
{
	if (RayTracingType == ERayTracingType::RayTracing)
	{
		switch (MaskType)
		{
		case ERayTracingInstanceMaskType::Opaque:
			return RAY_TRACING_MASK_OPAQUE | RAY_TRACING_MASK_OPAQUE_REFLECTION;
		case ERayTracingInstanceMaskType::Translucent:
			return RAY_TRACING_MASK_TRANSLUCENT;
		case ERayTracingInstanceMaskType::OpaqueShadow:
			return RAY_TRACING_MASK_OPAQUE_SHADOW;
		case ERayTracingInstanceMaskType::TranslucentShadow:
			return RAY_TRACING_MASK_TRANSLUCENT_SHADOW;
		case ERayTracingInstanceMaskType::ThinShadow:
			return RAY_TRACING_MASK_THIN_SHADOW;
		case ERayTracingInstanceMaskType::HairStrands:
			return RAY_TRACING_MASK_HAIR_STRANDS | RAY_TRACING_MASK_THIN_SHADOW;
		case ERayTracingInstanceMaskType::FirstPersonWorldSpaceRepresentation:
			return RAY_TRACING_MASK_OPAQUE_FP_WORLD_SPACE;
		case ERayTracingInstanceMaskType::VisibleInPrimaryRay:
			return 0; // There is no distinct notion of primary ray visibility for ray tracing
		case ERayTracingInstanceMaskType::VisibleInIndirectRay:
			return RAY_TRACING_MASK_OPAQUE | RAY_TRACING_MASK_TRANSLUCENT | RAY_TRACING_MASK_HAIR_STRANDS;
		case ERayTracingInstanceMaskType::VisibleInReflectionRay:
			return RAY_TRACING_MASK_OPAQUE_REFLECTION;
		default:
			break;
		}
	}
	else if (RayTracingType == ERayTracingType::PathTracing ||
			 RayTracingType == ERayTracingType::LightMapTracing)
	{
		switch (MaskType)
		{
		case ERayTracingInstanceMaskType::Opaque:
			return PATHTRACER_MASK_CAMERA_OPAQUE | PATHTRACER_MASK_INDIRECT_OPAQUE | PATHTRACER_MASK_REFLECTION_OPAQUE | PATHTRACER_MASK_SSS_RANDOM_WALK;
		case ERayTracingInstanceMaskType::Translucent:
			return PATHTRACER_MASK_CAMERA_TRANSLUCENT | PATHTRACER_MASK_INDIRECT_TRANSLUCENT | PATHTRACER_MASK_REFLECTION_TRANSLUCENT | PATHTRACER_MASK_SSS_RANDOM_WALK;
		case ERayTracingInstanceMaskType::OpaqueShadow:
		case ERayTracingInstanceMaskType::TranslucentShadow:
		case ERayTracingInstanceMaskType::ThinShadow:
			return PATHTRACER_MASK_SHADOW;
		case ERayTracingInstanceMaskType::FirstPersonWorldSpaceRepresentation:
			return PATHTRACER_MASK_IGNORE;
		case ERayTracingInstanceMaskType::HairStrands:
			return PATHTRACER_MASK_CAMERA_OPAQUE | PATHTRACER_MASK_INDIRECT_OPAQUE | PATHTRACER_MASK_REFLECTION_OPAQUE | PATHTRACER_MASK_SHADOW;
		case ERayTracingInstanceMaskType::VisibleInPrimaryRay:
			return PATHTRACER_MASK_CAMERA_OPAQUE | PATHTRACER_MASK_CAMERA_TRANSLUCENT;
		case ERayTracingInstanceMaskType::VisibleInIndirectRay:
			return PATHTRACER_MASK_INDIRECT_OPAQUE | PATHTRACER_MASK_INDIRECT_TRANSLUCENT;
		case ERayTracingInstanceMaskType::VisibleInReflectionRay:
			return PATHTRACER_MASK_REFLECTION_OPAQUE | PATHTRACER_MASK_REFLECTION_TRANSLUCENT;
		default:
			break;
		}
	}
	checkNoEntry();
	return 0;
}

uint8 BlendModeToRayTracingInstanceMask(const EBlendMode BlendMode, bool bIsDitherMasked, bool bCastShadow, ERayTracingType RayTracingType)
{
	ERayTracingInstanceMaskType MaskTypePrimary = ERayTracingInstanceMaskType::Opaque;
	ERayTracingInstanceMaskType MaskTypeShadows = ERayTracingInstanceMaskType::OpaqueShadow;

	const bool bIsOpaqueOrMasked = IsOpaqueOrMaskedBlendMode(BlendMode);

	// RayTracing treats dithered masked materials the same as regular masked materials for speed
	// PathTracing/LightmapTracing both upgrade dithered masking to translucent internally and therefore need to take them with those bits
	if ((RayTracingType == ERayTracingType::RayTracing && !bIsOpaqueOrMasked) ||
		(RayTracingType != ERayTracingType::RayTracing && (!bIsOpaqueOrMasked || bIsDitherMasked)))
	{
		MaskTypePrimary = ERayTracingInstanceMaskType::Translucent;
		MaskTypeShadows = ERayTracingInstanceMaskType::TranslucentShadow;
	}

	return ComputeRayTracingInstanceMask(MaskTypePrimary, RayTracingType) |	(bCastShadow ? ComputeRayTracingInstanceMask(MaskTypeShadows, RayTracingType) : 0);
}

namespace {

/** Util struct and function to derive mask related info from scene proxy*/
struct FSceneProxyRayTracingMaskInfo
{
	bool bVisibleToCamera = false;
	bool bVisibleToShadow = false;
	bool bVisibleToIndirect = false;
	bool bVisibleToReflections = false;
	bool bIsFirstPersonWorldSpaceRepresentation = false;
	ERayTracingType RayTracingType = ERayTracingType::RayTracing;

    void UpdateMask(uint8* Mask) const
    {
        if (!bVisibleToCamera)
        {
            // If the object is not visible to camera, remove all direct visibility bits.
            *Mask &= ~ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::VisibleInPrimaryRay, RayTracingType);
        }

        if (!bVisibleToIndirect)
        {
            // If the object does not affect indirect lighting, remove all indirect bits.
            *Mask &= ~ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::VisibleInIndirectRay, RayTracingType);
        }

        if (!bVisibleToReflections)
        {
            // If the object does not affect reflections, remove all reflection bits.
            *Mask &= ~ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::VisibleInReflectionRay, RayTracingType);
        }

        if (!bVisibleToShadow)
        {
            // Not casting shadows, remove any set shadow flags
            *Mask &= ~(
                ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::OpaqueShadow, RayTracingType) |
                ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::TranslucentShadow, RayTracingType) |
                ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::ThinShadow, RayTracingType)
            );
        }

        if (bIsFirstPersonWorldSpaceRepresentation)
        {
            const uint8 OpaqueMaskType = 
                ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::Opaque, RayTracingType) |
                ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::OpaqueShadow, RayTracingType);
            const bool bIsOpaque = (*Mask & OpaqueMaskType) != 0;
            // Tag world space representations of first person meshes so rays originating from first person meshes can skip them.
            // We currently only support opaque world space representations of first person objects, so set the mask to 0 otherwise.
            *Mask = bIsOpaque ? ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::FirstPersonWorldSpaceRepresentation, RayTracingType) : 0;
        }
    }
};

} // anonymous namespace

static FSceneProxyRayTracingMaskInfo GetSceneProxyRayTracingMaskInfo(const FPrimitiveSceneProxy& PrimitiveSceneProxy)
{
	FSceneProxyRayTracingMaskInfo MaskInfo = {};

	const FScene* RenderScene = PrimitiveSceneProxy.GetScene().GetRenderScene();
	MaskInfo.RayTracingType = RenderScene->CachedRayTracingMeshCommandsType;

	if (PrimitiveSceneProxy.IsRayTracingFarField())
	{
		MaskInfo.bVisibleToCamera = true;
		MaskInfo.bVisibleToShadow = true;
		MaskInfo.bVisibleToIndirect = true;
	}
	else if (PrimitiveSceneProxy.IsDrawnInGame())
	{
		MaskInfo.bVisibleToCamera = true;
		MaskInfo.bVisibleToShadow = true;
		// NOTE: For backwards compatibility, only path tracing obeys the AffectsDynamicIndirectLighting flag
		MaskInfo.bVisibleToIndirect = MaskInfo.RayTracingType == ERayTracingType::RayTracing ? true : PrimitiveSceneProxy.AffectsDynamicIndirectLighting();
	}
	else
	{
		MaskInfo.bVisibleToCamera = false;
		MaskInfo.bVisibleToShadow = PrimitiveSceneProxy.CastsHiddenShadow();
		MaskInfo.bVisibleToIndirect = PrimitiveSceneProxy.AffectsIndirectLightingWhileHidden();
	}

	MaskInfo.bVisibleToReflections = MaskInfo.bVisibleToIndirect && PrimitiveSceneProxy.IsVisibleInReflections();
	MaskInfo.bIsFirstPersonWorldSpaceRepresentation = PrimitiveSceneProxy.IsFirstPersonWorldSpaceRepresentation();

	return MaskInfo;
}

FRayTracingMaskAndFlags BuildRayTracingInstanceMaskAndFlags(const FRayTracingInstance& Instance, const FPrimitiveSceneProxy& PrimitiveSceneProxy)
{
	FSceneProxyRayTracingMaskInfo MaskInfo = GetSceneProxyRayTracingMaskInfo(PrimitiveSceneProxy);
	const ERHIFeatureLevel::Type FeatureLevel = PrimitiveSceneProxy.GetScene().GetFeatureLevel();
	const ERayTracingType RayTracingType = MaskInfo.RayTracingType;

	FRayTracingMaskAndFlags Result;

	ensureMsgf(Instance.GetMaterials().Num() > 0, TEXT("You need to add MeshBatches first for instance mask and flags to build upon."));

	bool bAllSegmentsOpaque = true;
	bool bAnySegmentsCastShadow = false;
	bool bAllSegmentsCastShadow = true;
	bool bAnySegmentsDecal = false;
	bool bAllSegmentsDecal = true;
	bool bDoubleSided = false;
	bool bAllSegmentsReverseCulling = true;

	Result.Mask = 0;
	for (const FMeshBatch& MeshBatch : Instance.GetMaterials())
	{
		// Mesh Batches can "null" when they have zero triangles.  Check the MaterialRenderProxy before accessing.
		if (MeshBatch.bUseForMaterial && MeshBatch.MaterialRenderProxy)
		{
			const FMaterial& Material = MeshBatch.MaterialRenderProxy->GetIncompleteMaterialWithFallback(FeatureLevel);
			const EBlendMode BlendMode = Material.GetBlendMode();
			const bool bSegmentCastsShadow = MaskInfo.bVisibleToShadow && MeshBatch.CastRayTracedShadow && Material.CastsRayTracedShadows() && BlendMode != BLEND_Additive;

			Result.Mask |= BlendModeToRayTracingInstanceMask(BlendMode, Material.IsDitherMasked(), bSegmentCastsShadow, RayTracingType);
			bAllSegmentsOpaque &= BlendMode == EBlendMode::BLEND_Opaque;
			bAnySegmentsCastShadow |= bSegmentCastsShadow;
			bAllSegmentsCastShadow &= bSegmentCastsShadow;
			bAnySegmentsDecal |= Material.IsDeferredDecal();
			bAllSegmentsDecal &= Material.IsDeferredDecal();
			bDoubleSided |= MeshBatch.bDisableBackfaceCulling || Material.IsTwoSided();
			bAllSegmentsReverseCulling &= MeshBatch.ReverseCulling;
		}
	}
    MaskInfo.bVisibleToShadow = bAnySegmentsCastShadow;

	// Run AHS for alpha masked and meshes with only some sections casting shadows, which require per mesh section filtering in AHS
	Result.bForceOpaque = bAllSegmentsOpaque && (bAllSegmentsCastShadow || !bAnySegmentsCastShadow);
	// Consider that all segments are translucent if none of the mask bits contain Opaque or OpaqueShadow
	const uint8 OpaqueMask =
		ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::Opaque      , RayTracingType) |
		ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::OpaqueShadow, RayTracingType);
	Result.bAllSegmentsTranslucent = Result.Mask != 0 && (Result.Mask & OpaqueMask) == 0;

	Result.bDoubleSided = bDoubleSided;	
	Result.bAnySegmentsDecal = bAnySegmentsDecal;
	Result.bAllSegmentsDecal = bAllSegmentsDecal;
	Result.bReverseCulling = bAllSegmentsReverseCulling;

	const bool bIsHairStrands = Instance.bThinGeometry;
	if (bIsHairStrands)
	{
		// Reset all hair strands bits "on"
		Result.Mask = ComputeRayTracingInstanceMask(ERayTracingInstanceMaskType::HairStrands, RayTracingType);
		Result.bForceOpaque = true;
		Result.bAllSegmentsTranslucent = false;
	}

    MaskInfo.UpdateMask(&Result.Mask);

	return Result;
}

void SetupRayTracingMeshCommandMaskAndStatus(FRayTracingMeshCommand& MeshCommand, const FMeshBatch& MeshBatch, const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterial& MaterialResource, ERayTracingType RayTracingType)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	MeshCommand.bCastRayTracedShadows = MeshBatch.CastRayTracedShadow && MaterialResource.CastsRayTracedShadows() && MaterialResource.GetBlendMode() != BLEND_Additive;
	MeshCommand.bOpaque = MaterialResource.GetBlendMode() == EBlendMode::BLEND_Opaque && !(VertexFactory->GetType()->SupportsRayTracingProceduralPrimitive() && FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(GMaxRHIShaderPlatform));
	MeshCommand.bAlphaMasked = MaterialResource.GetBlendMode() == EBlendMode::BLEND_Masked; // Used by Lumen only
	MeshCommand.bDecal = MaterialResource.IsDeferredDecal();
	MeshCommand.bIsSky = MaterialResource.IsSky();
	MeshCommand.bTwoSided = MaterialResource.IsTwoSided() || MeshBatch.bForceTwoSidedInRayTracing;
	MeshCommand.bIsTranslucent = MaterialResource.GetBlendMode() == EBlendMode::BLEND_Translucent;
	MeshCommand.bReverseCulling = MeshBatch.ReverseCulling;

	MeshCommand.InstanceMask = BlendModeToRayTracingInstanceMask(MaterialResource.GetBlendMode(), MaterialResource.IsDitherMasked(), MeshCommand.bCastRayTracedShadows, RayTracingType);

	if (!PrimitiveSceneProxy)
	{
		return;
	}

	// MeshBatch.ReverseCulling is generally not what we want as the value could be set including the transform's orientation.
	// This is because cached mesh commands are shared with rasterization.
	// For ray tracing, only the user decision of wanting reversed culling matters, so query this directly here.
	// In the case that that this mesh command is not associated with a primitive, the mesh batch value will still apply.
	MeshCommand.bReverseCulling = PrimitiveSceneProxy->IsCullingReversedByComponent();

	MeshCommand.bNaniteRayTracing = (Nanite::GetRayTracingMode() != Nanite::ERayTracingMode::Fallback) && PrimitiveSceneProxy->IsNaniteMesh();

	FSceneProxyRayTracingMaskInfo MaskInfo = GetSceneProxyRayTracingMaskInfo(*PrimitiveSceneProxy);
    MaskInfo.RayTracingType = RayTracingType;
    MaskInfo.bVisibleToShadow &= MeshCommand.bCastRayTracedShadows;

    // TODO: Should this be done once all mesh commands for a mesh are combined? (similar to BuildRayTracingInstanceMaskAndFlags above)
    MaskInfo.UpdateMask(&MeshCommand.InstanceMask);
}

#endif
