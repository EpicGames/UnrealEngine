// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"

#if RHI_RAYTRACING

// Forward declarations used in this header
struct FMeshBatch;
class FPrimitiveSceneProxy;
class FSceneViewFamily;
struct FRayTracingInstance;
class FRayTracingMeshCommand;
class FMaterial;
enum EBlendMode : int;

struct FRayTracingMaskAndFlags
{
	/** Instance mask that can be used to exclude the instance from specific effects (eg. ray traced shadows). */
	uint8 Mask = 0xFF;

	/** Whether the instance is forced opaque, i.e. anyhit shaders are disabled on this instance. */
	uint8 bForceOpaque : 1 = false;

	/** Whether ray hits should be registered for front and back faces. */
	uint8 bDoubleSided : 1 = false;
	
	/** Whether front and back facings should be reversed. */
	uint8 bReverseCulling : 1 = false;

	/** Whether any or all of the segments in the instance are decals. */
	uint8 bAnySegmentsDecal : 1 = false;
	uint8 bAllSegmentsDecal : 1 = false;

	/** Whether all of the segments in the instance are translucent. */
	uint8 bAllSegmentsTranslucent : 1 = false;
};


/** Describes what type of ray tracing we are doing. This is used to know which set of ray flags
 * to use, and is also stored in the mesh command so we know when to invalidate them. */
enum class ERayTracingType : uint8
{
	RayTracing,
	PathTracing,
	LightMapTracing,
};

/** Compute the mask based on blend mode for different ray tracing mode*/
RENDERER_API uint8 BlendModeToRayTracingInstanceMask(const EBlendMode BlendMode, bool bIsDitherMasked, bool bCastShadow, ERayTracingType RayTracingType);


//-------------------------------------------------------
//	Build Instance mask and flags (if needed)
//-------------------------------------------------------

// Build mask and flags without modification of RayTracingInstance
FRayTracingMaskAndFlags BuildRayTracingInstanceMaskAndFlags(const FRayTracingInstance& Instance, const FPrimitiveSceneProxy& PrimitiveSceneProxy);

//-------------------------------------------------------
//	FRayTracingMeshCommand related mask setup and update
//-------------------------------------------------------
void SetupRayTracingMeshCommandMaskAndStatus(FRayTracingMeshCommand& MeshCommand, const FMeshBatch& MeshBatch, const FPrimitiveSceneProxy* PrimitiveSceneProxy,
	const FMaterial& MaterialResource, ERayTracingType RayTracingType);

#endif