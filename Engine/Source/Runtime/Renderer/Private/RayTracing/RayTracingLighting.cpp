// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingLighting.h"
#include "RHIDefinitions.h"
#include "DeferredShadingRenderer.h"

#if RHI_RAYTRACING

#include "LightFunctionRendering.h"
#include "LightRendering.h"
#include "LightSceneProxy.h"
#include "SceneRendering.h"
#include "RayTracingMaterialHitShaders.h"
#include "RayTracingTypes.h"
#include "Lumen/LumenHardwareRayTracingCommon.h"
#include "ScenePrivate.h"

static TAutoConsoleVariable<int32> CVarRayTracingLightFunction(
	TEXT("r.RayTracing.LightFunction"),
	1,
	TEXT("Whether to support light material functions in ray tracing effects. (default = 1)"),
	ECVF_RenderThreadSafe);

TAutoConsoleVariable<int32> CVarRayTracingLightGridResolution(
	TEXT("r.RayTracing.LightGridResolution"),
	256,
	TEXT("Controls the resolution of the 2D light grid used to cull irrelevant lights from lighting calculations (default = 256)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarRayTracingLightGridMaxCount(
	TEXT("r.RayTracing.LightGridMaxCount"),
	128,
	TEXT("Controls the maximum number of lights per cell in the 2D light grid. The minimum of this value and the number of lights in the scene is used. (default = 128)\n"),
	ECVF_RenderThreadSafe
);

TAutoConsoleVariable<int32> CVarRayTracingLightGridAxis(
	TEXT("r.RayTracing.LightGridAxis"),
	-1,
	TEXT("Choose the coordinate axis along which to project the light grid (default = -1, automatic)\n"),
	ECVF_RenderThreadSafe
);


IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FRayTracingLightGrid, "RaytracingLightGridData");

class FRayTracingBuildLightGridCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingBuildLightGridCS)
	SHADER_USE_PARAMETER_STRUCT(FRayTracingBuildLightGridCS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_X"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("THREADGROUPSIZE_Y"), FComputeShaderUtils::kGolden2DGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, SceneLightCount)
		SHADER_PARAMETER(uint32, SceneInfiniteLightCount)
		SHADER_PARAMETER(FVector3f, SceneLightsTranslatedBoundMin)
		SHADER_PARAMETER(FVector3f, SceneLightsTranslatedBoundMax)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRTLightingData>, SceneLights)
		SHADER_PARAMETER(unsigned, LightGridResolution)
		SHADER_PARAMETER(unsigned, LightGridMaxCount)
		SHADER_PARAMETER(unsigned, LightGridAxis)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<uint>, RWLightGrid)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWLightGridData)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_SHADER_TYPE(, FRayTracingBuildLightGridCS, TEXT("/Engine/Private/RayTracing/RayTracingBuildLightGrid.usf"), TEXT("RayTracingBuildLightGridCS"), SF_Compute);


FBox3f GetPointLightBounds(const FVector3f& Center, float Radius)
{
	const FVector3f R(Radius, Radius, Radius);
	return FBox3f(Center - R, Center + R);
}

FBox3f GetSpotLightBounds(const FVector3f& Center, const FVector3f& Normal, float Radius, float CosOuter)
{
	// box around ray from light center to tip of the cone
	const FVector3f Tip = Center + Normal * Radius;
	FVector3f Lo = FVector3f::Min(Center, Tip);
	FVector3f Hi = FVector3f::Max(Center, Tip);

	const float SinOuter = FMath::Sqrt(1.0f - CosOuter * CosOuter);

	// expand by disc around the farthest part of the cone
	const FVector3f Disc = FVector3f(
		FMath::Sqrt(FMath::Clamp(1.0f - Normal.X * Normal.X, 0.0f, 1.0f)),
		FMath::Sqrt(FMath::Clamp(1.0f - Normal.Y * Normal.Y, 0.0f, 1.0f)),
		FMath::Sqrt(FMath::Clamp(1.0f - Normal.Z * Normal.Z, 0.0f, 1.0f))
	);
	Lo = FVector3f::Min(Lo, Center + Radius * (Normal * CosOuter - Disc * SinOuter));
	Hi = FVector3f::Max(Hi, Center + Radius * (Normal * CosOuter + Disc * SinOuter));

	// Check if any of the coordinate axis points lie inside the cone and include them if they do
	// This is the only case which is not captured by the AABB above
	const FVector3f E = FVector3f(
		FMath::Abs(Normal.X) > CosOuter ? Center.X + copysignf(Radius, Normal.X) : Center.X,
		FMath::Abs(Normal.Y) > CosOuter ? Center.Y + copysignf(Radius, Normal.Y) : Center.Y,
		FMath::Abs(Normal.Z) > CosOuter ? Center.Z + copysignf(Radius, Normal.Z) : Center.Z
	);
	Lo = FVector3f::Min(Lo, E);
	Hi = FVector3f::Max(Hi, E);
	return FBox3f(Lo, Hi);
}

FBox3f GetRectLightBounds(const FVector3f& Center, const FVector3f& Normal, const FVector3f& Tangent, float HalfWidth, float HalfHeight, float Radius, float BarnCos, float BarnLen)
{
	const FVector3f Corner = FVector3f(
		copysignf(Radius, Normal.X),
		copysignf(Radius, Normal.Y),
		copysignf(Radius, Normal.Z)
	);
	const FVector3f Disc = FVector3f(
		FMath::Sqrt(FMath::Clamp(1.0f - Normal.X * Normal.X, 0.0f, 1.0f)),
		FMath::Sqrt(FMath::Clamp(1.0f - Normal.Y * Normal.Y, 0.0f, 1.0f)),
		FMath::Sqrt(FMath::Clamp(1.0f - Normal.Z * Normal.Z, 0.0f, 1.0f))
	);

	// rect bbox is the bbox of the disc + furthest corner of the Radius sized box in the direction of the normal
	FVector3f Lo = FVector3f::Min(Center + Corner, Center - Radius * Disc);
	FVector3f Hi = FVector3f::Max(Center + Corner, Center + Radius * Disc);

	// Take into account barndoor frustum if enabled
	if (BarnCos > 0.035f)
	{
		const FVector3f dPdv = Tangent;
		const FVector3f dPdu = Normal.Cross(Tangent);
		const float BarnSin = FMath::Sqrt(1 - BarnCos * BarnCos);

		FVector3f BoundingPlane = FVector3f(
			2 * HalfWidth  + BarnLen * BarnSin,
			2 * HalfHeight + BarnLen * BarnSin,
			BarnLen * BarnCos
		);
		FVector3f BLo = Center, BHi = Center;
		// loop through 9 points to get extremes of the "rounded" pyramid defined by the barndoor penumbra + radius
		for (int Dy = -1; Dy <= 1; Dy++)
		for (int Dx = -1; Dx <= 1; Dx++)
		{
			// Get point on rectangle
			const FVector3f Rxy = Center + Dx * HalfWidth * dPdu + Dy * HalfHeight * dPdv;
			// Get penumbra plane vector, normalize it and scale to edge of the sphere (roughly, since we aren't starting from the center)
			const FVector3f Bxy = Rxy + (Dx * BoundingPlane.X * dPdu + Dy * BoundingPlane.Y * dPdv + BoundingPlane.Z * Normal).GetUnsafeNormal() * Radius;
			BLo = FVector3f::Min(BLo, Rxy); BHi = FVector3f::Max(BHi, Rxy);
			BLo = FVector3f::Min(BLo, Bxy); BHi = FVector3f::Max(BHi, Bxy);
		}

		// Include "axis" points if they lie inside the barndoor penumbra (similar to spot light test above, but the apex is behind the Center and different in X and Y due to the rectangle size)
		// A 2D visualization of this is here: https://www.desmos.com/calculator/15zh9boeqz
		const float TanAlphaX = BoundingPlane.X / BoundingPlane.Z, CosAlphaX = FMath::InvSqrt(1 + TanAlphaX * TanAlphaX), ApexX = HalfWidth  / TanAlphaX;
		const float TanAlphaY = BoundingPlane.Y / BoundingPlane.Z, CosAlphaY = FMath::InvSqrt(1 + TanAlphaY * TanAlphaY), ApexY = HalfHeight / TanAlphaY;

		// Take the extreme point along each axis, then rotate it to local space
		const FVector3f Px = FVector3f(dPdu.X * Corner.X, dPdv.X * Corner.X, Normal.X * Corner.X);
		const FVector3f Py = FVector3f(dPdu.Y * Corner.Y, dPdv.Y * Corner.Y, Normal.Y * Corner.Y);
		const FVector3f Pz = FVector3f(dPdu.Z * Corner.Z, dPdv.Z * Corner.Z, Normal.Z * Corner.Z);

		// Now - check if dot product betwen this point and normal lies within the cone
		// We do two 2D cone tests as we have a different cone in X and Y
		// This is the analog of the simpler implementation possible for spotlights above
		if ((Px.Z - ApexX) > FVector2f(Px.X, Px.Z - ApexX).Length() * CosAlphaX &&
			(Px.Z - ApexY) > FVector2f(Px.Y, Px.Z - ApexY).Length() * CosAlphaY)
		{
			BLo.X = FMath::Min(BLo.X, Center.X + Corner.X);
			BHi.X = FMath::Max(BHi.X, Center.X + Corner.X);
		}
		if ((Py.Z - ApexX) > FVector2f(Py.X, Py.Z - ApexX).Length() * CosAlphaX &&
			(Py.Z - ApexY) > FVector2f(Py.Y, Py.Z - ApexY).Length() * CosAlphaY)
		{
			BLo.Y = FMath::Min(BLo.Y, Center.Y + Corner.Y);
			BHi.Y = FMath::Max(BHi.Y, Center.Y + Corner.Y);
		}
		if ((Pz.Z - ApexX) > FVector2f(Pz.X, Pz.Z - ApexX).Length() * CosAlphaX &&
			(Pz.Z - ApexY) > FVector2f(Pz.Y, Pz.Z - ApexY).Length() * CosAlphaY)
		{
			BLo.Z = FMath::Min(BLo.Z, Center.Z + Corner.Z);
			BHi.Z = FMath::Max(BHi.Z, Center.Z + Corner.Z);
		}
		// Now clip the new BBox against the old (conservative one) for the entire half-space
		Lo = FVector3f::Max(Lo, BLo);
		Hi = FVector3f::Min(Hi, BHi);
	}
	return FBox3f(Lo, Hi);
}



static void PrepareLightGrid(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRayTracingLightGrid* LightGridParameters, const FRTLightingData* Lights)
{
	// number of lights and infinite lights is provided by caller
	const uint32 NumLights = LightGridParameters->SceneLightCount;
	const uint32 NumInfiniteLights = LightGridParameters->SceneInfiniteLightCount;
	
	// Set all grid related parameters
	const float Inf = std::numeric_limits<float>::infinity();
	LightGridParameters->SceneLightsTranslatedBoundMin = FVector3f(+Inf, +Inf, +Inf);
	LightGridParameters->SceneLightsTranslatedBoundMax = FVector3f(-Inf, -Inf, -Inf);
	LightGridParameters->LightGrid = nullptr;
	LightGridParameters->LightGridData = nullptr;

	int NumFiniteLights = NumLights - NumInfiniteLights;
	if (NumFiniteLights == 0)
	{
		// light grid is not needed - just hookup dummy data and exit
		LightGridParameters->LightGridResolution = 0;
		LightGridParameters->LightGridMaxCount = 0;
		LightGridParameters->LightGridAxis = 0;
		LightGridParameters->LightGrid = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		FRDGBuffer* LightGridData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1), TEXT("RayTracing.LightGridData"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(LightGridData, PF_R32_UINT), 0);
		LightGridParameters->LightGridData = GraphBuilder.CreateSRV(LightGridData, PF_R32_UINT);
		return;
	}
	// if we have some finite lights -- build a light grid
	check(NumFiniteLights > 0);
	// get bounding box of all finite lights
	const FRTLightingData* FiniteLights = Lights + NumInfiniteLights;
	for (int Index = 0; Index < NumFiniteLights; Index++)
	{
		const FRTLightingData& Light = FiniteLights[Index];
		FBox3f Box;

		const FVector3f Center = Light.TranslatedLightPosition;
		const FVector3f Normal = -Light.Direction;
		const float Radius = 1.0f / Light.InvRadius;
		switch (Light.Type)
		{
			case LightType_Point:
			{
				Box = GetPointLightBounds(Center, Radius);
				break;
			}
			case LightType_Spot:
			{
				Box = GetSpotLightBounds(Center, Normal, Radius, Light.SpotAngles.X);
				break;
			}
			case LightType_Rect:
			{
				const float HalfWidth = Light.SourceRadius;
				const float HalfHeight = Light.SourceLength;
				Box = GetRectLightBounds(Center, Normal, Light.Tangent, HalfWidth, HalfHeight, Radius, Light.RectLightBarnCosAngle, Light.RectLightBarnLength);
				break;
			}
			default:
			{
				// non-finite lights should not appear in this case
				checkNoEntry();
				break;
			}
		}
		LightGridParameters->SceneLightsTranslatedBoundMin = FVector3f::Min(LightGridParameters->SceneLightsTranslatedBoundMin, Box.Min);
		LightGridParameters->SceneLightsTranslatedBoundMax = FVector3f::Max(LightGridParameters->SceneLightsTranslatedBoundMax, Box.Max);
	}

	const uint32 Resolution = FMath::Clamp(CVarRayTracingLightGridResolution.GetValueOnRenderThread(), 1, 2048);
	const uint32 MaxCount = FMath::Clamp(CVarRayTracingLightGridMaxCount.GetValueOnRenderThread(), 1, NumFiniteLights);
	LightGridParameters->LightGridResolution = Resolution;
	LightGridParameters->LightGridMaxCount = MaxCount;

	LightGridParameters->LightGridAxis = CVarRayTracingLightGridAxis.GetValueOnRenderThread();

	// The light grid stores indexes in the range [0,NumLights-1]
	EPixelFormat LightGridDataFormat = PF_R32_UINT;
	size_t LightGridDataNumBytes = sizeof(uint32);
	if (NumLights <= (MAX_uint8 + 1))
	{
		LightGridDataFormat = PF_R8_UINT;
		LightGridDataNumBytes = sizeof(uint8);
	}
	else if (NumLights <= (MAX_uint16 + 1))
	{
		LightGridDataFormat = PF_R16_UINT;
		LightGridDataNumBytes = sizeof(uint16);
	}
	// The texture stores a number of lights in the range [0,NumLights]
	EPixelFormat TextureDataFormat = PF_R32_UINT;
	if (NumLights <= MAX_uint8)
	{
		TextureDataFormat = PF_R8_UINT;
	}
	else if (NumLights <= MAX_uint16)
	{
		TextureDataFormat = PF_R16_UINT;
	}

	FRDGTextureDesc LightGridDesc = FRDGTextureDesc::Create2DArray(
		FIntPoint(Resolution, Resolution),
		TextureDataFormat,
		FClearValueBinding::None,
		TexCreate_ShaderResource | TexCreate_UAV, 3);

	// Run the build compute shader
	FRDGTexture* LightGridTexture = GraphBuilder.CreateTexture(LightGridDesc, TEXT("RayTracing.LightGrid"), ERDGTextureFlags::None);
	FRDGBuffer* LightGridData = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(LightGridDataNumBytes, 3 * MaxCount * Resolution * Resolution), TEXT("RayTracing.LightGridData"));


	FRayTracingBuildLightGridCS::FParameters* BuilderParams = GraphBuilder.AllocParameters<FRayTracingBuildLightGridCS::FParameters>();
	BuilderParams->SceneLightCount = LightGridParameters->SceneLightCount;
	BuilderParams->SceneInfiniteLightCount = LightGridParameters->SceneInfiniteLightCount;
	BuilderParams->SceneLights = LightGridParameters->SceneLights;
	BuilderParams->SceneLightsTranslatedBoundMin = LightGridParameters->SceneLightsTranslatedBoundMin;
	BuilderParams->SceneLightsTranslatedBoundMax = LightGridParameters->SceneLightsTranslatedBoundMax;
	BuilderParams->LightGridResolution = LightGridParameters->LightGridResolution;
	BuilderParams->LightGridMaxCount = LightGridParameters->LightGridMaxCount;
	BuilderParams->LightGridAxis = LightGridParameters->LightGridAxis;
	BuilderParams->RWLightGrid = GraphBuilder.CreateUAV(LightGridTexture);
	BuilderParams->RWLightGridData = GraphBuilder.CreateUAV(LightGridData, LightGridDataFormat);

	TShaderMapRef<FRayTracingBuildLightGridCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("Light Grid Create (%u lights)", NumFiniteLights),
		ComputeShader,
		BuilderParams,
		FComputeShaderUtils::GetGroupCount(FIntVector(Resolution, Resolution, 3), FIntVector(FComputeShaderUtils::kGolden2DGroupSize, FComputeShaderUtils::kGolden2DGroupSize, 1)));

	// hookup to the actual rendering pass
	LightGridParameters->LightGrid = LightGridTexture;
	LightGridParameters->LightGridData = GraphBuilder.CreateSRV(LightGridData, LightGridDataFormat);
}

static bool ShouldIncludeRayTracingLight(const FLightSceneInfoCompact& Light, const FEngineShowFlags& ShowFlags)
{
	if (Light.LightType == LightType_Directional && !ShowFlags.DirectionalLights) return false;
	if (Light.LightType == LightType_Spot        && !ShowFlags.SpotLights       ) return false;
	if (Light.LightType == LightType_Point       && !ShowFlags.PointLights      ) return false;
	if (Light.LightType == LightType_Rect        && !ShowFlags.RectLights       ) return false;
	const bool bHasStaticLighting = Light.LightSceneInfo->Proxy->HasStaticLighting() && Light.LightSceneInfo->IsPrecomputedLightingValid();
	const bool bAffectReflection = Light.LightSceneInfo->Proxy->AffectReflection();
	return !bHasStaticLighting && bAffectReflection;
}

uint32 PackRG16(float In0, float In1);

TRDGUniformBufferRef<FRayTracingLightGrid> CreateRayTracingLightData(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FSceneView& View,
	FGlobalShaderMap* ShaderMap,
	bool bBuildLightGrid)
{
	FRayTracingLightGrid* LightGridParameters = GraphBuilder.AllocParameters<FRayTracingLightGrid>();

	if (bBuildLightGrid)
	{
		// Count the number of lights we want to include by type
		int NumLightsByType[LightType_MAX] = {};

		for (const FLightSceneInfoCompact& Light : Scene->Lights)
		{
			if (!ShouldIncludeRayTracingLight(Light, View.Family->EngineShowFlags))
				continue;
			check(Light.LightType < LightType_MAX);
			NumLightsByType[Light.LightType]++;
		}

		// Figure out offset in the target light buffer where each light type will start
		int LightTypeOffsets[LightType_MAX + 1];
		LightTypeOffsets[0] = 0;
		for (int TypeIndex = 1; TypeIndex <= LightType_MAX; TypeIndex++)
		{
			LightTypeOffsets[TypeIndex] = LightTypeOffsets[TypeIndex - 1] + NumLightsByType[TypeIndex - 1];
		}

		LightGridParameters->SceneLightCount = LightTypeOffsets[LightType_MAX];

		FRDGUploadData<FRTLightingData> LightDataArray(GraphBuilder, LightGridParameters->SceneLightCount);

		const FRayTracingLightFunctionMap* RayTracingLightFunctionMap = GraphBuilder.Blackboard.Get<FRayTracingLightFunctionMap>();
		for (const FLightSceneInfoCompact& Light : Scene->Lights)
		{
			if (!ShouldIncludeRayTracingLight(Light, View.Family->EngineShowFlags))
				continue;

			FLightRenderParameters LightParameters;
			Light.LightSceneInfo->Proxy->GetLightShaderParameters(LightParameters);

			if (Light.LightType == LightType_Directional && Light.LightSceneInfo->Proxy->GetUsePerPixelAtmosphereTransmittance())
			{
				// When using PerPixelTransmittance, transmittance is evaluated per pixel by sampling the transmittance texture. It gives better gradient on large scale objects such as mountains.
				// However, to skip doing that texture sampling in ray tracing, we use the simple planet top ground transmittance as a simplification.
				// That will work for most of the cases for most of the map/terrain at the top of the virtual planet.
				LightParameters.Color *= Light.LightSceneInfo->Proxy->GetAtmosphereTransmittanceTowardSun();
			}

			if (Light.LightSceneInfo->Proxy->IsInverseSquared())
			{
				LightParameters.FalloffExponent = 0;
			}

			// Figure out where in the target light array this light goes (so that all lights will be sorted by type)
			int32 Offset = LightTypeOffsets[Light.LightType];
			LightTypeOffsets[Light.LightType]++; // increment offset for next light


			FRTLightingData& LightDataElement = LightDataArray[Offset];

			LightDataElement.Type = Light.LightType;

			LightDataElement.Direction = LightParameters.Direction;
			LightDataElement.TranslatedLightPosition = FVector3f(LightParameters.WorldPosition + View.ViewMatrices.GetPreViewTranslation());
			LightDataElement.LightColor = FVector3f(LightParameters.Color) * LightParameters.GetLightExposureScale(View.GetLastEyeAdaptationExposure());
			LightDataElement.Tangent = LightParameters.Tangent;

			// Ray tracing should compute fade parameters ignoring lightmaps
			const FVector2D FadeParams = Light.LightSceneInfo->Proxy->GetDirectionalLightDistanceFadeParameters(View.GetFeatureLevel(), false, View.MaxShadowCascades);
			
			LightDataElement.SpotAngles = LightParameters.SpotAngles;
			LightDataElement.DistanceFadeMAD = PackRG16(FadeParams.Y, -FadeParams.X * FadeParams.Y);
			LightDataElement.IndirectLightScale = Light.LightSceneInfo->Proxy->GetIndirectLightingScale();

			LightDataElement.InvRadius = LightParameters.InvRadius;
			LightDataElement.DiffuseSpecularScale = PackRG16(LightParameters.DiffuseScale, LightParameters.SpecularScale);
			LightDataElement.FalloffExponent = LightParameters.FalloffExponent;
			LightDataElement.SourceRadius = LightParameters.SourceRadius;
			LightDataElement.SourceLength = LightParameters.SourceLength;
			LightDataElement.SoftSourceRadius = LightParameters.SoftSourceRadius;
			LightDataElement.RectLightBarnCosAngle = LightParameters.RectLightBarnCosAngle;
			LightDataElement.RectLightBarnLength = LightParameters.RectLightBarnLength;
			LightDataElement.IESAtlasIndex = LightParameters.IESAtlasIndex;
			LightDataElement.RectLightAtlasUVOffset[0] = LightParameters.RectLightAtlasUVOffset.X;
			LightDataElement.RectLightAtlasUVOffset[1] = LightParameters.RectLightAtlasUVOffset.Y;
			LightDataElement.RectLightAtlasUVScale[0] = LightParameters.RectLightAtlasUVScale.X;
			LightDataElement.RectLightAtlasUVScale[1] = LightParameters.RectLightAtlasUVScale.Y;
			LightDataElement.RectLightAtlasMaxLevel = LightParameters.RectLightAtlasMaxLevel;
			LightDataElement.LightMissShaderIndex = RAY_TRACING_MISS_SHADER_SLOT_LIGHTING;

			// Stuff directional light's shadow angle factor into a RectLight parameter
			if (Light.LightType == LightType_Directional)
			{
				LightDataElement.RectLightBarnCosAngle = Light.LightSceneInfo->Proxy->GetShadowSourceAngleFactor();
			}

			// NOTE: This map will be empty if the light functions are disabled for some reason
			if (RayTracingLightFunctionMap)
			{
				const int32* LightFunctionIndex = RayTracingLightFunctionMap->Find(Light.LightSceneInfo);
				if (LightFunctionIndex)
				{
					check(uint32(*LightFunctionIndex) > RAY_TRACING_MISS_SHADER_SLOT_LIGHTING);
					check(uint32(*LightFunctionIndex) < Scene->RayTracingSBT.NumMissShaderSlots);
					LightDataElement.LightMissShaderIndex = *LightFunctionIndex;
				}
			}
		}
		// last light type should not match the total scene light count
		check(LightGridParameters->SceneLightCount == LightTypeOffsets[LightType_MAX - 1]);

		LightGridParameters->SceneLights = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("LightBuffer"), LightDataArray));
		LightGridParameters->SceneInfiniteLightCount = NumLightsByType[LightType_Directional];
		PrepareLightGrid(GraphBuilder, ShaderMap, LightGridParameters, LightDataArray.GetData());
	}
	else
	{
		LightGridParameters->SceneLightCount = 0;
		LightGridParameters->SceneInfiniteLightCount = 0;
		LightGridParameters->SceneLightsTranslatedBoundMin = FVector3f::ZeroVector;
		LightGridParameters->SceneLightsTranslatedBoundMax = FVector3f::ZeroVector;
		LightGridParameters->SceneLights = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FRTLightingData), 0u));
		LightGridParameters->LightGrid = GSystemTextures.GetDefaultTexture2D(GraphBuilder, PF_R32_UINT, 0u);
		LightGridParameters->LightGridData = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(uint32), 0u), PF_R32_UINT);
		LightGridParameters->LightGridResolution = 0;
		LightGridParameters->LightGridMaxCount = 0;
		LightGridParameters->LightGridAxis = 0;
	}

	return GraphBuilder.CreateUniformBuffer(LightGridParameters);
}

class FRayTracingLightingMS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingLightingMS);
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTracingLightingMS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRayTracingLightGrid, LightDataPacked)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_REF(FLumenHardwareRayTracingUniformBufferParameters, LumenHardwareRayTracingUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingLightingMS, "/Engine/Private/RayTracing/RayTracingLightingMS.usf", "RayTracingLightingMS", SF_RayMiss);

/**
 * FLightFunctionParametersRayTracing
 * Global constant buffer derived from loose parameters of standard light function materials. Note that it lacks
 * the screen to world transform, as the RT version never have screen as a reference frame
 * This is nearly identical to the one found in Lumen and ultimately should be converted to a shared solution
 *
 * function to create the constant buffer is deirved from the LightFunctionMaterial SetParameters code
 */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLightFunctionParametersRayTracing, )
	SHADER_PARAMETER(FMatrix44f, LightFunctionTranslatedWorldToLight)
	SHADER_PARAMETER(FVector4f, LightFunctionParameters)
	SHADER_PARAMETER(FVector3f, LightFunctionParameters2)
	SHADER_PARAMETER(FVector3f, CameraRelativeLightPosition)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLightFunctionParametersRayTracing, "RaytracingLightFunctionParameters");

static TUniformBufferRef<FLightFunctionParametersRayTracing> CreateLightFunctionParametersBufferRT(
	const FLightSceneInfo* LightSceneInfo,
	const FSceneView& View,
	EUniformBufferUsage Usage)
{
	FLightFunctionParametersRayTracing LightFunctionParameters;

	const FVector Scale = LightSceneInfo->Proxy->GetLightFunctionScale();
	// Switch x and z so that z of the user specified scale affects the distance along the light direction
	const FVector InverseScale = FVector(1.f / Scale.Z, 1.f / Scale.Y, 1.f / Scale.X);
	const FMatrix WorldToLight = LightSceneInfo->Proxy->GetWorldToLight() * FScaleMatrix(FVector(InverseScale));

	LightFunctionParameters.LightFunctionTranslatedWorldToLight = FMatrix44f(FTranslationMatrix(-View.ViewMatrices.GetPreViewTranslation()) * WorldToLight);

	const bool bIsSpotLight = LightSceneInfo->Proxy->GetLightType() == LightType_Spot;
	const bool bIsPointLight = LightSceneInfo->Proxy->GetLightType() == LightType_Point;
	const float TanOuterAngle = bIsSpotLight ? FMath::Tan(LightSceneInfo->Proxy->GetOuterConeAngle()) : 1.0f;

	// should this match raster?
	const float ShadowFadeFraction = 1.0f;

	LightFunctionParameters.LightFunctionParameters = FVector4f(TanOuterAngle, ShadowFadeFraction, bIsSpotLight ? 1.0f : 0.0f, bIsPointLight ? 1.0f : 0.0f);

	// do we need this?
	const bool bRenderingPreviewShadowIndicator = false;

	LightFunctionParameters.LightFunctionParameters2 = FVector3f(
		LightSceneInfo->Proxy->GetLightFunctionFadeDistance(),
		LightSceneInfo->Proxy->GetLightFunctionDisabledBrightness(),
		bRenderingPreviewShadowIndicator ? 1.0f : 0.0f);

	LightFunctionParameters.CameraRelativeLightPosition = GetCamRelativeLightPosition(View.ViewMatrices, *LightSceneInfo);

	return CreateUniformBufferImmediate(LightFunctionParameters, Usage);
}

/**
 * Generic light function for ray tracing compilable as miss shader with lighting
 */
class FLightFunctionRayTracingShader : public FMaterialShader
{
	DECLARE_SHADER_TYPE(FLightFunctionRayTracingShader, Material);
public:

	/**
	  * Makes sure only shaders for materials that are explicitly flagged
	  * as 'UsedAsLightFunction' in the Material Editor gets compiled into
	  * the shader cache.
	  */
	static bool ShouldCompilePermutation(const FMaterialShaderPermutationParameters& Parameters)
	{
		return Parameters.MaterialParameters.MaterialDomain == MD_LightFunction && ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	FLightFunctionRayTracingShader() {}
	FLightFunctionRayTracingShader(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMaterialShader(Initializer)
	{
		LightMaterialsParameter.Bind(Initializer.ParameterMap, TEXT("RaytracingLightFunctionParameters"));
		LightDataPacked.Bind(Initializer.ParameterMap, TEXT("RaytracingLightGridData"));
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FViewInfo& View,
		const TUniformBufferRef<FDeferredLightUniformStruct>& DeferredLightBuffer,
		const TUniformBufferRef<FLightFunctionParametersRayTracing>& LightFunctionParameters,
		const TUniformBufferRef<FRayTracingLightGrid>& LightGridBuffer,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMaterialShader::GetShaderBindings(Scene, FeatureLevel, MaterialRenderProxy, Material, ShaderBindings);

		// Bind view
		ShaderBindings.Add(GetUniformBufferParameter<FViewUniformShaderParameters>(), View.ViewUniformBuffer);

		// Bind light parameters
		ShaderBindings.Add(GetUniformBufferParameter<FDeferredLightUniformStruct>(), DeferredLightBuffer);

		//bind Lightfunction parameters
		ShaderBindings.Add(LightMaterialsParameter, LightFunctionParameters);

		//bind light data
		ShaderBindings.Add(LightDataPacked, LightGridBuffer);

		// LightFunctions can use primitive data, set identity so we do not crash on a missing binding
		ShaderBindings.Add(GetUniformBufferParameter<FPrimitiveUniformShaderParameters>(), GIdentityPrimitiveUniformBuffer);

		// bind Lumen Hardware RT buffer
		ShaderBindings.Add(GetUniformBufferParameter<FLumenHardwareRayTracingUniformBufferParameters>(), View.LumenHardwareRayTracingUniformBuffer);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("SUPPORT_LIGHT_FUNCTION"), 1);
		FMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

private:

	LAYOUT_FIELD(FShaderUniformBufferParameter, LightMaterialsParameter);
	LAYOUT_FIELD(FShaderUniformBufferParameter, LightDataPacked);
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FLightFunctionRayTracingShader, TEXT("/Engine/Private/RayTracing/RayTracingLightingMS.usf"), TEXT("RayTracingLightingMS"), SF_RayMiss);

FRayTracingLightFunctionMap GatherLightFunctionLights(FScene* Scene, const FEngineShowFlags EngineShowFlags, ERHIFeatureLevel::Type InFeatureLevel)
{
	checkf(EngineShowFlags.LightFunctions, TEXT("This function should not be called if light functions are disabled"));

	// gives control over lighting functions in raytraced effects, independently of the show flag (for performance testing / debug)
	if (CVarRayTracingLightFunction.GetValueOnRenderThread() == 0)
	{
		return {};
	}

	FRayTracingLightFunctionMap RayTracingLightFunctionMap;
	for (const FLightSceneInfoCompact& Light : Scene->Lights)
	{
		FLightSceneInfo* LightSceneInfo = Light.LightSceneInfo;
		auto MaterialProxy = LightSceneInfo->Proxy->GetLightFunctionMaterial();
		if (MaterialProxy)
		{
			const FMaterialRenderProxy* FallbackMaterialRenderProxyPtr = nullptr;
			const FMaterial& Material = MaterialProxy->GetMaterialWithFallback(InFeatureLevel, FallbackMaterialRenderProxyPtr);
			if (Material.IsLightFunction())
			{
				const FMaterialShaderMap* MaterialShaderMap = Material.GetRenderingThreadShaderMap();
				// Getting the shader here has the side-effect of populating the raytracing miss shader library which is used when building the raytracing pipeline
				MaterialShaderMap->GetShader<FLightFunctionRayTracingShader>().GetRayTracingShader();

				int32 Index = Scene->RayTracingSBT.NumMissShaderSlots;
				Scene->RayTracingSBT.NumMissShaderSlots++;
				RayTracingLightFunctionMap.Add(LightSceneInfo, Index);
			}
		}
	}
	return RayTracingLightFunctionMap;
}

static void BindLightFunction(
	FRHICommandList& RHICmdList,
	const FScene* Scene,
	const FViewInfo& View,
	const FMaterial& Material,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const TUniformBufferRef<FDeferredLightUniformStruct>& DeferredLightBuffer,
	const TUniformBufferRef<FLightFunctionParametersRayTracing>& LightFunctionParameters,
	int32 Index
	)
{
	FRayTracingPipelineState* Pipeline = View.MaterialRayTracingData.PipelineState;
	FRHIShaderBindingTable* SBT = View.MaterialRayTracingData.ShaderBindingTable;
	const FMaterialShaderMap* MaterialShaderMap = Material.GetRenderingThreadShaderMap();

	TShaderRef<FLightFunctionRayTracingShader> Shader = MaterialShaderMap->GetShader<FLightFunctionRayTracingShader>();

	FMeshDrawShaderBindings ShaderBindings;
	ShaderBindings.Initialize(Shader);

	FMeshDrawSingleShaderBindings SingleShaderBindings = ShaderBindings.GetSingleShaderBindings( SF_RayMiss);

	Shader->GetShaderBindings(Scene, Scene->GetFeatureLevel(), MaterialRenderProxy, Material, View, DeferredLightBuffer, LightFunctionParameters, View.RayTracingLightGridUniformBuffer->GetRHIRef(), SingleShaderBindings);

	int32 MissShaderPipelineIndex = FindRayTracingMissShaderIndex(View.MaterialRayTracingData.PipelineState, Shader.GetRayTracingShader(), false);

	if (MissShaderPipelineIndex == INDEX_NONE)
	{
		MissShaderPipelineIndex = FindRayTracingMissShaderIndex(View.MaterialRayTracingData.PipelineState, GetRayTracingDefaultMissShader(View.ShaderMap), true);
	}

	ShaderBindings.SetRayTracingShaderBindingsForMissShader(RHICmdList, SBT, Index, Pipeline, MissShaderPipelineIndex);
}

FRHIRayTracingShader* GetRayTracingLightingMissShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FRayTracingLightingMS>().GetRayTracingShader();
}

void BindLightFunctionShaders(
	FRHICommandList& RHICmdList,
	const FScene* Scene,
	const FRayTracingLightFunctionMap* RayTracingLightFunctionMap,
	const class FViewInfo& View)
{
	if (RayTracingLightFunctionMap == nullptr)
	{
		return;
	}

	for (const FRayTracingLightFunctionMap::ElementType& LightAndIndex : *RayTracingLightFunctionMap)
	{
		const FLightSceneInfo* LightSceneInfo = LightAndIndex.Key;

		const FMaterialRenderProxy* MaterialProxy = LightSceneInfo->Proxy->GetLightFunctionMaterial();
		check(MaterialProxy != nullptr);
		// Catch the fallback material case
		const FMaterialRenderProxy* FallbackMaterialRenderProxyPtr = nullptr;
		const FMaterial& Material = MaterialProxy->GetMaterialWithFallback(Scene->GetFeatureLevel(), FallbackMaterialRenderProxyPtr);

		check(Material.IsLightFunction());

		const FMaterialRenderProxy& MaterialRenderProxy = FallbackMaterialRenderProxyPtr ? *FallbackMaterialRenderProxyPtr : *MaterialProxy;

		//create the uniform buffers we need
		TUniformBufferRef<FDeferredLightUniformStruct> DeferredLightBuffer = CreateUniformBufferImmediate(GetDeferredLightParameters(View, *LightSceneInfo), EUniformBufferUsage::UniformBuffer_SingleFrame);
		TUniformBufferRef<FLightFunctionParametersRayTracing> LightFunctionParameters = CreateLightFunctionParametersBufferRT(LightSceneInfo, View, EUniformBufferUsage::UniformBuffer_SingleFrame);

		int32 MissIndex = LightAndIndex.Value;
		BindLightFunction(RHICmdList, Scene, View, Material, MaterialRenderProxy, DeferredLightBuffer, LightFunctionParameters, MissIndex);
	}
}

template< typename ShaderClass>
static int32 BindParameters(const TShaderRef<ShaderClass>& Shader, typename ShaderClass::FParameters & Parameters, int32 MaxParams, const FRHIUniformBuffer **OutUniformBuffers)
{
	// Allow FRayTracingShaderBindingsWriter
	// #yuriy-todo: ResourceBinder here appears to only be used to fill OutUniformBuffers. Add a dedicated helper for this? 
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRayTracingShaderBindingsWriter ResourceBinder;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	auto &ParameterMap = Shader->ParameterMapInfo;

	// all parameters should be in uniform buffers
	check(ParameterMap.LooseParameterBuffers.Num() == 0);
	check(ParameterMap.SRVs.Num() == 0);
	check(ParameterMap.TextureSamplers.Num() == 0);

	SetShaderParameters(ResourceBinder, Shader, Parameters);

	FMemory::Memzero(OutUniformBuffers, sizeof(FRHIUniformBuffer *)*MaxParams);

	const int32 NumUniformBuffers = ParameterMap.UniformBuffers.Num();

	int32 MaxUniformBufferUsed = -1;
	for (int32 UniformBufferIndex = 0; UniformBufferIndex < NumUniformBuffers; UniformBufferIndex++)
	{
		const FShaderUniformBufferParameterInfo Parameter = ParameterMap.UniformBuffers[UniformBufferIndex];
		checkSlow(Parameter.BaseIndex < MaxParams);
		const FRHIUniformBuffer* UniformBuffer = ResourceBinder.UniformBuffers[UniformBufferIndex];
		if (Parameter.BaseIndex < MaxParams)
		{
			OutUniformBuffers[Parameter.BaseIndex] = UniformBuffer;
			MaxUniformBufferUsed = FMath::Max((int32)Parameter.BaseIndex, MaxUniformBufferUsed);
		}
	}

	return MaxUniformBufferUsed + 1;
}

void FDeferredShadingSceneRenderer::SetupRayTracingDefaultMissShader(FRHICommandList& RHICmdList, const FViewInfo& View)
{
	int32 MissShaderPipelineIndex = FindRayTracingMissShaderIndex(View.MaterialRayTracingData.PipelineState, GetRayTracingDefaultMissShader(View.ShaderMap), true);

	RHICmdList.SetRayTracingMissShader(
		View.MaterialRayTracingData.ShaderBindingTable,
		RAY_TRACING_MISS_SHADER_SLOT_DEFAULT,
		View.MaterialRayTracingData.PipelineState,
		MissShaderPipelineIndex,
		0, nullptr, 0);
}

void FDeferredShadingSceneRenderer::SetupRayTracingLightingMissShader(FRHICommandList& RHICmdList, const FViewInfo& View)
{
	FRayTracingLightingMS::FParameters MissParameters;
	MissParameters.ViewUniformBuffer = View.ViewUniformBuffer;
	MissParameters.LightDataPacked = View.RayTracingLightGridUniformBuffer;
	MissParameters.LumenHardwareRayTracingUniformBuffer = View.LumenHardwareRayTracingUniformBuffer;

	// Allow FRayTracingShaderBindingsWriter
	// #yuriy-todo: ResourceBinder here appears to only be used to fill OutUniformBuffers. Add a dedicated helper for this? 
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static constexpr uint32 MaxUniformBuffers = UE_ARRAY_COUNT(FRayTracingShaderBindings::UniformBuffers);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const FRHIUniformBuffer* MissData[MaxUniformBuffers] = {};
	auto MissShader = View.ShaderMap->GetShader<FRayTracingLightingMS>();

	int32 ParameterSlots = BindParameters(MissShader, MissParameters, MaxUniformBuffers, MissData);

	int32 MissShaderPipelineIndex = FindRayTracingMissShaderIndex(View.MaterialRayTracingData.PipelineState, MissShader.GetRayTracingShader(), true);

	RHICmdList.SetRayTracingMissShader(
		View.MaterialRayTracingData.ShaderBindingTable,
		RAY_TRACING_MISS_SHADER_SLOT_LIGHTING,
		View.MaterialRayTracingData.PipelineState,
		MissShaderPipelineIndex,
		ParameterSlots, (FRHIUniformBuffer**)MissData, 0);
}

#endif // RHI_RAYTRACING
