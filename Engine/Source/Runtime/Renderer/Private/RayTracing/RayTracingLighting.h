// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderParameters.h"
#include "ShaderParameterMacros.h"
#include "SceneRendering.h"
#include "LightSceneInfo.h"
#include "RayTracingDefinitions.h"
#include "RayTracingTypes.h"
#include "Containers/DynamicRHIResourceArray.h"

#if RHI_RAYTRACING

FRHIRayTracingShader* GetRayTracingLightingMissShader(const FGlobalShaderMap* ShaderMap);

// This struct holds a light grid and list of raytracing lights for both building and rendering
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FRayTracingLightGrid, RENDERER_API)
	SHADER_PARAMETER(uint32, SceneLightCount)
	SHADER_PARAMETER(uint32, SceneInfiniteLightCount)
	SHADER_PARAMETER(FVector3f, SceneLightsTranslatedBoundMin)
	SHADER_PARAMETER(FVector3f, SceneLightsTranslatedBoundMax)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FRTLightingData>, SceneLights)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2DArray<uint>, LightGrid)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, LightGridData)
	SHADER_PARAMETER(unsigned, LightGridResolution)
	SHADER_PARAMETER(unsigned, LightGridMaxCount)
	SHADER_PARAMETER(unsigned, LightGridAxis)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

using FRayTracingLightFunctionMap = TMap<const FLightSceneInfo*, int32>;

// Register this in the graph builder so we can easily move it around and access it from both the main rendering thread and RDG passes
RDG_REGISTER_BLACKBOARD_STRUCT(FRayTracingLightFunctionMap)

FRayTracingLightFunctionMap GatherLightFunctionLights(FScene* Scene, const FEngineShowFlags EngineShowFlags, ERHIFeatureLevel::Type InFeatureLevel);
FRayTracingLightFunctionMap GatherLightFunctionLightsPathTracing(FScene* Scene, const FEngineShowFlags EngineShowFlags, ERHIFeatureLevel::Type InFeatureLevel);

TRDGUniformBufferRef<FRayTracingLightGrid> CreateRayTracingLightData(
	FRDGBuilder& GraphBuilder,
	const FScene* Scene,
	const FSceneView& View,
	FGlobalShaderMap* ShaderMap,
	bool bBuildLightGrid);

void BindLightFunctionShaders(
	FRHICommandList& RHICmdList,
	const FScene* Scene,
	const FRayTracingLightFunctionMap* RayTracingLightFunctionMap,
	const class FViewInfo& View);

void BindLightFunctionShadersPathTracing(
	FRHICommandList& RHICmdList,
	const FScene* Scene,
	const FRayTracingLightFunctionMap* RayTracingLightFunctionMap,
	const class FViewInfo& View);

// Helper routines, useful when building a light grid
FBox3f GetPointLightBounds(const FVector3f& Center, float Radius);
FBox3f GetSpotLightBounds(const FVector3f& Center, const FVector3f& Normal, float Radius, float CosOuter);
FBox3f GetRectLightBounds(const FVector3f& Center, const FVector3f& Normal, const FVector3f& Tangent, float HalfWidth, float HalfHeight, float Radius, float BarnCos, float BarnLen);

#endif
