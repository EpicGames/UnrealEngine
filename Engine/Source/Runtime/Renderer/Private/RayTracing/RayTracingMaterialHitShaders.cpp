// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracing/RayTracingMaterialHitShaders.h"

#if RHI_RAYTRACING

#include "BasePassRendering.h"
#include "DeferredShadingRenderer.h"
#include "PipelineStateCache.h"
#include "RenderCore.h"
#include "ScenePrivate.h"

#include "Nanite/NaniteRayTracing.h"

#include "RayTracingDefinitions.h"
#include "RayTracingInstance.h"
#include "BuiltInRayTracingShaders.h"
#include "RaytracingOptions.h"
#include "RayTracingLighting.h"
#include "RayTracingDecals.h"
#include "PathTracing.h"
#include "RayTracing.h"
#include "LogRenderer.h"
#include "ShaderPlatformCachedIniValue.h"

int32 GEnableRayTracingMaterials = 1;
static FAutoConsoleVariableRef CVarEnableRayTracingMaterials(
	TEXT("r.RayTracing.EnableMaterials"),
	GEnableRayTracingMaterials,
	TEXT(" 0: bind default material shader that outputs placeholder data\n")
	TEXT(" 1: bind real material shaders (default)\n"),
	ECVF_RenderThreadSafe
);

int32 GCompileRayTracingMaterialCHS = 1;
static FAutoConsoleVariableRef CVarCompileRayTracingMaterialCHS(
	TEXT("r.RayTracing.CompileMaterialCHS"),
	GCompileRayTracingMaterialCHS,
	TEXT(" 0: skip compilation of closest-hit shaders for materials (useful if only shadows or ambient occlusion effects are needed)\n")
	TEXT(" 1: compile closest hit shaders for all ray tracing materials (default)\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

int32 GCompileRayTracingMaterialAHS = 1;
static FAutoConsoleVariableRef CVarCompileRayTracingMaterialAHS(
	TEXT("r.RayTracing.CompileMaterialAHS"),
	GCompileRayTracingMaterialAHS,
	TEXT(" 0: skip compilation of any-hit shaders for materials (useful if alpha masked or translucent materials are not needed)\n")
	TEXT(" 1: compile any hit shaders for all ray tracing materials (default)\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe
);

static int32 GRayTracingNonBlockingPipelineCreation = 1;
static FAutoConsoleVariableRef CVarRayTracingNonBlockingPipelineCreation(
	TEXT("r.RayTracing.NonBlockingPipelineCreation"),
	GRayTracingNonBlockingPipelineCreation,
	TEXT("Enable background ray tracing pipeline creation, without blocking RHI or Render thread.\n")
	TEXT("Fallback opaque black material will be used for missing shaders meanwhile.\n")
	TEXT(" 0: off (rendering will always use correct requested material)\n")
	TEXT(" 1: on (default, non-blocking mode may sometimes use the fallback opaque black material outside of offline rendering scenarios)\n"),
	ECVF_RenderThreadSafe);

static bool IsSupportedVertexFactoryType(const FVertexFactoryType* VertexFactoryType)
{
	return VertexFactoryType->SupportsRayTracing();
}

static bool AreRayTracingMaterialsCompiled(EShaderPlatform Platform)
{
	static FShaderPlatformCachedIniValue<int32> CVarCompileMaterialCHS(TEXT("r.RayTracing.CompileMaterialCHS"));
	static FShaderPlatformCachedIniValue<int32> CVarCompileMaterialAHS(TEXT("r.RayTracing.CompileMaterialAHS"));

	return CVarCompileMaterialCHS.Get(Platform) || CVarCompileMaterialAHS.Get(Platform);
}

class FMaterialCHS : public FMeshMaterialShader, public FUniformLightMapPolicyShaderParametersType
{
	DECLARE_INLINE_TYPE_LAYOUT_EXPLICIT_BASES(FMaterialCHS, NonVirtual, FMeshMaterialShader, FUniformLightMapPolicyShaderParametersType);
public:
	FMaterialCHS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
		FUniformLightMapPolicyShaderParametersType::Bind(Initializer.ParameterMap);
	}

	FMaterialCHS() {}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const TBasePassShaderElementData<FUniformLightMapPolicy>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);
		
		FUniformLightMapPolicy::GetPixelShaderBindings(
			PrimitiveSceneProxy,
			ShaderElementData.LightMapPolicyElementData,
			this,
			ShaderBindings);
	}

	void GetElementShaderBindings(
		const FShaderMapPointerTable& PointerTable,
		const FScene* Scene,
		const FSceneView* ViewIfDynamicMeshCommand,
		const FVertexFactory* VertexFactory,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMeshBatch& MeshBatch, 
		const FMeshBatchElement& BatchElement,
		const TBasePassShaderElementData<FUniformLightMapPolicy>& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams) const
	{
		FMeshMaterialShader::GetElementShaderBindings(PointerTable, Scene, ViewIfDynamicMeshCommand, VertexFactory, InputStreamType, FeatureLevel, PrimitiveSceneProxy, MeshBatch, BatchElement, ShaderElementData, ShaderBindings, VertexStreams);
	}
};

static bool RTNeedsAnyHitShader(EBlendMode BlendMode)
{
	switch (BlendMode)
	{
		case BLEND_Opaque: 							return false; // always hit
		case BLEND_Masked: 							return true;  // runs shader (NOTE: dithered masking gets turned into translucent for the path tracer)
		case BLEND_Translucent: 					return true;  // casts transparent (colored) shadows depending on the shading model setup (fake caustics or transparent shadows)
		case BLEND_Additive: 						return false; // never hit for shadows, goes through the default shader instead, so no need to use AHS for primary rays
		case BLEND_Modulate: 						return true;  // casts colored shadows
		case BLEND_AlphaComposite: 					return true;
		case BLEND_AlphaHoldout: 					return false; // treat as opaque for shadows
		case BLEND_TranslucentColoredTransmittance: return true;  // NOTE: Substrate only
		default: checkf(false, TEXT("Unhandled blend mode %d"), int(BlendMode)); return false;
	}
}

template<typename LightMapPolicyType, bool UseAnyHitShader, bool UseIntersectionShader, bool UseRayConeTextureLod>
class TMaterialCHS : public FMaterialCHS
{
	DECLARE_SHADER_TYPE(TMaterialCHS, MeshMaterial);
public:

	TMaterialCHS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMaterialCHS(Initializer)
	{}

	TMaterialCHS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		if (!AreRayTracingMaterialsCompiled(Parameters.Platform))
		{
			return false;
		}

		if (Parameters.MaterialParameters.MaterialDomain != MD_Surface)
		{
			return false;
		}

		static FShaderPlatformCachedIniValue<int32> CVarCompileMaterialAHS(TEXT("r.RayTracing.CompileMaterialAHS"));
		const bool bWantAnyHitShader = CVarCompileMaterialAHS.Get(Parameters.Platform) != 0 && RTNeedsAnyHitShader(Parameters.MaterialParameters.BlendMode);
		const bool bSupportProceduralPrimitive = Parameters.VertexFactoryType->SupportsRayTracingProceduralPrimitive() && FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(Parameters.Platform);

		static const auto CVarRayTracingUseTextureLod = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.UseTextureLod"));
		const bool bRayTracingUseTextureLod = CVarRayTracingUseTextureLod && CVarRayTracingUseTextureLod->GetValueOnAnyThread() != 0;

		return IsSupportedVertexFactoryType(Parameters.VertexFactoryType)
			&& (bWantAnyHitShader == UseAnyHitShader)
			&& LightMapPolicyType::ShouldCompilePermutation(Parameters)
			&& ShouldCompileRayTracingShadersForProject(Parameters.Platform)
			&& bRayTracingUseTextureLod == UseRayConeTextureLod
			&& (UseIntersectionShader == bSupportProceduralPrimitive);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		// NOTE: Any CVars that are used in this function must be handled in ShaderMapAppendKeyString() to ensure shaders are recompiled when necessary.
		static FShaderPlatformCachedIniValue<int32> CVarCompileMaterialCHS(TEXT("r.RayTracing.CompileMaterialCHS"));

		OutEnvironment.SetDefine(TEXT("USE_MATERIAL_CLOSEST_HIT_SHADER"), CVarCompileMaterialCHS.Get(Parameters.Platform) ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("USE_MATERIAL_ANY_HIT_SHADER"), UseAnyHitShader ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("USE_MATERIAL_INTERSECTION_SHADER"), UseIntersectionShader ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("USE_RAYTRACED_TEXTURE_RAYCONE_LOD"), UseRayConeTextureLod ? 1 : 0);
		OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1);
		OutEnvironment.SetDefine(TEXT("HAIRSTRANDS_HAS_NORMAL_CONNECTED"), Parameters.MaterialParameters.bHasNormalConnected ? 1 : 0);
		LightMapPolicyType::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		FMeshMaterialShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		const bool VirtualTextureLightmaps = UseVirtualTextureLightmap(Parameters.Platform);
		OutEnvironment.SetDefine(TEXT("LIGHTMAP_VT_ENABLED"), VirtualTextureLightmaps);
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		if (ParameterMap.ContainsParameterAllocation(FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName()))
		{
			OutError.Add(TEXT("Ray tracing closest hit shaders cannot read from the SceneTexturesStruct."));
			return false;
		}

		for (const auto& It : ParameterMap.GetParameterMap())
		{
			const FParameterAllocation& ParamAllocation = It.Value;
			if (ParamAllocation.Type != EShaderParameterType::UniformBuffer
				&& ParamAllocation.Type != EShaderParameterType::LooseData)
			{
				OutError.Add(FString::Printf(TEXT("Invalid ray tracing shader parameter '%s'. Only uniform buffers and loose data parameters are supported."), *(It.Key)));
				return false;
			}
		}

		return true;
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};

class FTrivialMaterialCHS : public FMaterialCHS
{
	DECLARE_SHADER_TYPE(FTrivialMaterialCHS, MeshMaterial);
public:

	FTrivialMaterialCHS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMaterialCHS(Initializer)
	{}

	FTrivialMaterialCHS() {}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		if (AreRayTracingMaterialsCompiled(Parameters.Platform))
		{
			return false;
		}

		return IsSupportedVertexFactoryType(Parameters.VertexFactoryType)
			&& ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FMaterialShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}

	static bool ValidateCompiledResult(EShaderPlatform Platform, const FShaderParameterMap& ParameterMap, TArray<FString>& OutError)
	{
		return true;
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FTrivialMaterialCHS, TEXT("/Engine/Private/RayTracing/RayTracingMaterialDefaultHitShaders.usf"), TEXT("closesthit=OpaqueShadowCHS"), SF_RayHitGroup);

#define IMPLEMENT_MATERIALCHS_TYPE(LightMapPolicyType, LightMapPolicyName, AnyHitShaderName) \
	typedef TMaterialCHS<LightMapPolicyType, false, false, false> TMaterialCHS##LightMapPolicyName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMaterialCHS##LightMapPolicyName, TEXT("/Engine/Private/RayTracing/RayTracingMaterialHitShaders.usf"), TEXT("closesthit=MaterialCHS"), SF_RayHitGroup); \
	typedef TMaterialCHS<LightMapPolicyType, true, false, false> TMaterialCHS##LightMapPolicyName##AnyHitShaderName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMaterialCHS##LightMapPolicyName##AnyHitShaderName, TEXT("/Engine/Private/RayTracing/RayTracingMaterialHitShaders.usf"), TEXT("closesthit=MaterialCHS anyhit=MaterialAHS"), SF_RayHitGroup) \
	typedef TMaterialCHS<LightMapPolicyType, false, false, true> TMaterialCHSLod##LightMapPolicyName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMaterialCHSLod##LightMapPolicyName, TEXT("/Engine/Private/RayTracing/RayTracingMaterialHitShaders.usf"), TEXT("closesthit=MaterialCHS"), SF_RayHitGroup); \
	typedef TMaterialCHS<LightMapPolicyType, true, false, true> TMaterialCHSLod##LightMapPolicyName##AnyHitShaderName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMaterialCHSLod##LightMapPolicyName##AnyHitShaderName, TEXT("/Engine/Private/RayTracing/RayTracingMaterialHitShaders.usf"), TEXT("closesthit=MaterialCHS anyhit=MaterialAHS"), SF_RayHitGroup); \
	typedef TMaterialCHS<LightMapPolicyType, false, true, false> TMaterialCHS_IS_##LightMapPolicyName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMaterialCHS_IS_##LightMapPolicyName, TEXT("/Engine/Private/RayTracing/RayTracingMaterialHitShaders.usf"), TEXT("closesthit=MaterialCHS intersection=MaterialIS"), SF_RayHitGroup); \
	typedef TMaterialCHS<LightMapPolicyType, true, true, false> TMaterialCHS_IS_##LightMapPolicyName##AnyHitShaderName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMaterialCHS_IS_##LightMapPolicyName##AnyHitShaderName, TEXT("/Engine/Private/RayTracing/RayTracingMaterialHitShaders.usf"), TEXT("closesthit=MaterialCHS anyhit=MaterialAHS intersection=MaterialIS"), SF_RayHitGroup) \
	typedef TMaterialCHS<LightMapPolicyType, false, true, true> TMaterialCHS_IS_Lod##LightMapPolicyName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMaterialCHS_IS_Lod##LightMapPolicyName, TEXT("/Engine/Private/RayTracing/RayTracingMaterialHitShaders.usf"), TEXT("closesthit=MaterialCHS intersection=MaterialIS"), SF_RayHitGroup); \
	typedef TMaterialCHS<LightMapPolicyType, true, true, true> TMaterialCHS_IS_Lod##LightMapPolicyName##AnyHitShaderName; \
	IMPLEMENT_MATERIAL_SHADER_TYPE(template<>, TMaterialCHS_IS_Lod##LightMapPolicyName##AnyHitShaderName, TEXT("/Engine/Private/RayTracing/RayTracingMaterialHitShaders.usf"), TEXT("closesthit=MaterialCHS anyhit=MaterialAHS intersection=MaterialIS"), SF_RayHitGroup);

IMPLEMENT_MATERIALCHS_TYPE(TUniformLightMapPolicy<LMP_NO_LIGHTMAP>, FNoLightMapPolicy, FAnyHitShader);
IMPLEMENT_MATERIALCHS_TYPE(TUniformLightMapPolicy<LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING>, FPrecomputedVolumetricLightmapLightingPolicy, FAnyHitShader);
IMPLEMENT_MATERIALCHS_TYPE(TUniformLightMapPolicy<LMP_LQ_LIGHTMAP>, TLightMapPolicyLQ, FAnyHitShader);
IMPLEMENT_MATERIALCHS_TYPE(TUniformLightMapPolicy<LMP_HQ_LIGHTMAP>, TLightMapPolicyHQ, FAnyHitShader);
IMPLEMENT_MATERIALCHS_TYPE(TUniformLightMapPolicy<LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP>, TDistanceFieldShadowsAndLightMapPolicyHQ, FAnyHitShader);

IMPLEMENT_GLOBAL_SHADER(FHiddenMaterialHitGroup, "/Engine/Private/RayTracing/RayTracingMaterialDefaultHitShaders.usf", "closesthit=HiddenMaterialCHS anyhit=HiddenMaterialAHS", SF_RayHitGroup);
IMPLEMENT_GLOBAL_SHADER(FOpaqueShadowHitGroup, "/Engine/Private/RayTracing/RayTracingMaterialDefaultHitShaders.usf", "closesthit=OpaqueShadowCHS", SF_RayHitGroup);
IMPLEMENT_GLOBAL_SHADER(FDefaultCallableShader, "/Engine/Private/RayTracing/RayTracingMaterialDefaultHitShaders.usf", "DefaultCallableShader", SF_RayCallable);

// Select TextureLOD
template<typename LightMapPolicyType, bool bUseAnyHitShader, bool bUseIntersectionShader>
inline void GetMaterialHitShader_TextureLOD(FMaterialShaderTypes& ShaderTypes, bool bUseTextureLod)
{
	if (bUseTextureLod)
	{
		ShaderTypes.AddShaderType<TMaterialCHS<LightMapPolicyType, bUseAnyHitShader, bUseIntersectionShader, true>>();
	}
	else
	{
		ShaderTypes.AddShaderType<TMaterialCHS<LightMapPolicyType, bUseAnyHitShader, bUseIntersectionShader, false>>();
	}
}

// Select Intersection shader
template<typename LightMapPolicyType, bool bUseAnyHitShader>
inline void GetMaterialHitShader_Intersection_TextureLOD(FMaterialShaderTypes& ShaderTypes, bool bUseIntersectionShader, bool bUseTextureLod)
{
	if (bUseIntersectionShader)
	{
		GetMaterialHitShader_TextureLOD<LightMapPolicyType, bUseAnyHitShader, true>(ShaderTypes, bUseTextureLod);
	}
	else
	{
		GetMaterialHitShader_TextureLOD<LightMapPolicyType, bUseAnyHitShader, false>(ShaderTypes, bUseTextureLod);
	}
}

// Select AnyHit shader
template<typename LightMapPolicyType>
inline void GetMaterialHitShader_AnyHit_Intersection_TextureLOD(FMaterialShaderTypes& ShaderTypes, bool bUseAnyHitShader, bool bUseIntersectionShader, bool bUseTextureLod)
{
	if (bUseAnyHitShader)
	{
		GetMaterialHitShader_Intersection_TextureLOD<LightMapPolicyType, true>(ShaderTypes, bUseIntersectionShader, bUseTextureLod);
	}
	else
	{
		GetMaterialHitShader_Intersection_TextureLOD<LightMapPolicyType, false>(ShaderTypes, bUseIntersectionShader, bUseTextureLod);
	}
}

template<typename LightMapPolicyType>
static bool GetMaterialHitShader(const FMaterial& RESTRICT MaterialResource, const FVertexFactory* VertexFactory, bool UseTextureLod, EShaderPlatform Platform, TShaderRef<FMaterialCHS>& OutShader)
{
	const bool bMaterialsCompiled = AreRayTracingMaterialsCompiled(Platform);
	checkf(bMaterialsCompiled, TEXT("Material hit shaders are requested but they were not compiled for current platform [%s]"), *LexToString(Platform));

	FMaterialShaderTypes ShaderTypes;
	const FVertexFactoryType* VFType = VertexFactory->GetType();
	const bool bUseIntersectionShader = VFType->HasFlags(EVertexFactoryFlags::SupportsRayTracingProceduralPrimitive) && FDataDrivenShaderPlatformInfo::GetSupportsRayTracingProceduralPrimitive(GMaxRHIShaderPlatform);
	const bool UseAnyHitShader = (MaterialResource.IsMasked() || RTNeedsAnyHitShader(MaterialResource.GetBlendMode())) && GCompileRayTracingMaterialAHS;

	GetMaterialHitShader_AnyHit_Intersection_TextureLOD<LightMapPolicyType>(ShaderTypes, UseAnyHitShader, bUseIntersectionShader, UseTextureLod);

	FMaterialShaders Shaders;
	if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
	{
		return false;
	}

	Shaders.TryGetShader(SF_RayHitGroup, OutShader);
	return true;
}

static bool GetRayTracingMeshProcessorShaders(
	const FUniformLightMapPolicy& RESTRICT LightMapPolicy,
	const FVertexFactory* VertexFactory,
	const FMaterial& RESTRICT MaterialResource,
	EShaderPlatform Platform,
	TShaderRef<FMaterialCHS>& OutRayHitGroupShader)
{
	check(GRHISupportsRayTracingShaders);

	const bool bMaterialsCompiled = AreRayTracingMaterialsCompiled(Platform);

	if (bMaterialsCompiled)
	{
		static const auto CVarRayTracingUseTextureLod = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.RayTracing.UseTextureLod"));
		const bool bUseTextureLOD = CVarRayTracingUseTextureLod && CVarRayTracingUseTextureLod->GetValueOnAnyThread() != 0;

		switch (LightMapPolicy.GetIndirectPolicy())
		{
		case LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING:
			if (!GetMaterialHitShader<TUniformLightMapPolicy<LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING>>(MaterialResource, VertexFactory, bUseTextureLOD, Platform, OutRayHitGroupShader))
			{
				return false;
			}
			break;
		case LMP_LQ_LIGHTMAP:
			if (!GetMaterialHitShader<TUniformLightMapPolicy<LMP_LQ_LIGHTMAP>>(MaterialResource, VertexFactory, bUseTextureLOD, Platform, OutRayHitGroupShader))
			{
				return false;
			}
			break;
		case LMP_HQ_LIGHTMAP:
			if (!GetMaterialHitShader<TUniformLightMapPolicy<LMP_HQ_LIGHTMAP>>(MaterialResource, VertexFactory, bUseTextureLOD, Platform, OutRayHitGroupShader))
			{
				return false;
			}
			break;
		case LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP:
			if (!GetMaterialHitShader<TUniformLightMapPolicy<LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP>>(MaterialResource, VertexFactory, bUseTextureLOD, Platform, OutRayHitGroupShader))
			{
				return false;
			}
			break;
		case LMP_NO_LIGHTMAP:
			if (!GetMaterialHitShader<TUniformLightMapPolicy<LMP_NO_LIGHTMAP>>(MaterialResource, VertexFactory, bUseTextureLOD, Platform, OutRayHitGroupShader))
			{
				return false;
			}
			break;
		default:
			check(false);
		}
	}
	else
	{
		FMaterialShaderTypes ShaderTypes;
		ShaderTypes.AddShaderType<FTrivialMaterialCHS>();

		FMaterialShaders Shaders;
		if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
		{
			return false;
		}

		Shaders.TryGetShader(SF_RayHitGroup, OutRayHitGroupShader);
	}

	return true;
}

FRayTracingMeshProcessor::FRayTracingMeshProcessor(FRayTracingMeshCommandContext* InCommandContext, const FScene* InScene, const FSceneView* InViewIfDynamicMeshCommand, ERayTracingType InRayTracingType)
	:
	CommandContext(InCommandContext),
	Scene(InScene),
	ViewIfDynamicMeshCommand(InViewIfDynamicMeshCommand),
	FeatureLevel(InScene ? InScene->GetFeatureLevel() : GMaxRHIFeatureLevel),
	RayTracingType(InRayTracingType)
{
}

FRayTracingMeshProcessor::~FRayTracingMeshProcessor() = default;

bool FRayTracingMeshProcessor::Process(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	const FUniformLightMapPolicy& RESTRICT LightMapPolicy)
{
	TShaderRef<FMaterialCHS> RayTracingShader;
	if (GRHISupportsRayTracingShaders)
	{
		if (!GetRayTracingMeshProcessorShaders(LightMapPolicy, MeshBatch.VertexFactory, MaterialResource, Scene->GetShaderPlatform(), RayTracingShader))
		{
			return false;
		}
	}

	TBasePassShaderElementData<FUniformLightMapPolicy> ShaderElementData(MeshBatch.LCI);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, -1, true);

	BuildRayTracingMeshCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		RayTracingShader,
		ShaderElementData);

	return true;
}

void FRayTracingMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy)
{
	if (!MeshBatch.bUseForMaterial || !IsSupportedVertexFactoryType(MeshBatch.VertexFactory->GetType()))
	{
		return;
	}

	const FMaterialRenderProxy* FallbackMaterialRenderProxyPtr = MeshBatch.MaterialRenderProxy;
	while (FallbackMaterialRenderProxyPtr)
	{
		const FMaterial* Material = FallbackMaterialRenderProxyPtr->GetMaterialNoFallback(FeatureLevel);
		if (Material && Material->GetRenderingThreadShaderMap())
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, -1, *FallbackMaterialRenderProxyPtr, *Material))
			{
				break;
			}
		}
		FallbackMaterialRenderProxyPtr = FallbackMaterialRenderProxyPtr->GetFallback(FeatureLevel);
	}
}

bool FRayTracingMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& Material
)
{
	// Only draw opaque materials.
	if ((!PrimitiveSceneProxy || PrimitiveSceneProxy->ShouldRenderInMainPass())
		&& ShouldIncludeDomainInMeshPass(Material.GetMaterialDomain()))
	{
		if (RayTracingType == ERayTracingType::PathTracing ||
			RayTracingType == ERayTracingType::LightMapTracing)
		{
			// Path Tracer has its own process call so that it can attach its own material permutation
			return ProcessPathTracing(MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, Material);
		}

		// Check for a cached light-map.
		const bool bIsLitMaterial = Material.GetShadingModels().IsLit();
		const bool bAllowStaticLighting = IsStaticLightingAllowed();

		const FLightMapInteraction LightMapInteraction = (bAllowStaticLighting && MeshBatch.LCI && bIsLitMaterial)
			? MeshBatch.LCI->GetLightMapInteraction(FeatureLevel)
			: FLightMapInteraction();

		// force LQ lightmaps based on system settings
		const bool bPlatformAllowsHighQualityLightMaps = AllowHighQualityLightmaps(FeatureLevel);
		const bool bAllowHighQualityLightMaps = bPlatformAllowsHighQualityLightMaps && LightMapInteraction.AllowsHighQualityLightmaps();

		const bool bAllowIndirectLightingCache = Scene && Scene->PrecomputedLightVolumes.Num() > 0;
		const bool bUseVolumetricLightmap = Scene && Scene->VolumetricLightmapSceneData.HasData();

		{
			static const auto CVarSupportLowQualityLightmap = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SupportLowQualityLightmaps"));
			const bool bAllowLowQualityLightMaps = (!CVarSupportLowQualityLightmap) || (CVarSupportLowQualityLightmap->GetValueOnAnyThread() != 0);

			switch (LightMapInteraction.GetType())
			{
			case LMIT_Texture:
				if (bAllowHighQualityLightMaps)
				{
					const FShadowMapInteraction ShadowMapInteraction = (bAllowStaticLighting && MeshBatch.LCI && bIsLitMaterial)
						? MeshBatch.LCI->GetShadowMapInteraction(FeatureLevel)
						: FShadowMapInteraction();

					if (ShadowMapInteraction.GetType() == SMIT_Texture)
					{
						return Process(
							MeshBatch,
							BatchElementMask,
							PrimitiveSceneProxy,
							MaterialRenderProxy,
							Material,
							FUniformLightMapPolicy(LMP_DISTANCE_FIELD_SHADOWS_AND_HQ_LIGHTMAP));
					}
					else
					{
						return Process(
							MeshBatch,
							BatchElementMask,
							PrimitiveSceneProxy,
							MaterialRenderProxy,
							Material,
							FUniformLightMapPolicy(LMP_HQ_LIGHTMAP));
					}
				}
				else if (bAllowLowQualityLightMaps)
				{
					return Process(
						MeshBatch,
						BatchElementMask,
						PrimitiveSceneProxy,
						MaterialRenderProxy,
						Material,
						FUniformLightMapPolicy(LMP_LQ_LIGHTMAP));
				}
				else
				{
					return Process(
						MeshBatch,
						BatchElementMask,
						PrimitiveSceneProxy,
						MaterialRenderProxy,
						Material,
						FUniformLightMapPolicy(LMP_NO_LIGHTMAP));
				}
			default:
				if (bIsLitMaterial
					&& bAllowStaticLighting
					&& Scene
					&& Scene->VolumetricLightmapSceneData.HasData()
					&& PrimitiveSceneProxy
					&& (PrimitiveSceneProxy->IsMovable()
						|| PrimitiveSceneProxy->NeedsUnbuiltPreviewLighting()
						|| PrimitiveSceneProxy->GetLightmapType() == ELightmapType::ForceVolumetric))
				{
					return Process(
						MeshBatch,
						BatchElementMask,
						PrimitiveSceneProxy,
						MaterialRenderProxy,
						Material,
						FUniformLightMapPolicy(LMP_PRECOMPUTED_IRRADIANCE_VOLUME_INDIRECT_LIGHTING));
				}
				else
				{
					return Process(
						MeshBatch,
						BatchElementMask,
						PrimitiveSceneProxy,
						MaterialRenderProxy,
						Material,
						FUniformLightMapPolicy(LMP_NO_LIGHTMAP));
				}
			};
		}
	}

	return true;
}

static bool IsCompatibleFallbackPipelineSignature(FRayTracingPipelineStateSignature& B, FRayTracingPipelineStateSignature& A)
{
	// Compare everything except hit group table
	return A.MaxPayloadSizeInBytes == B.MaxPayloadSizeInBytes
		&& A.GetRayGenHash() == B.GetRayGenHash()
		&& A.GetRayMissHash() == B.GetRayMissHash()
		&& A.GetCallableHash() == B.GetCallableHash();
}

static bool PipelineContainsHitShaders(FRayTracingPipelineState* Pipeline, const TArrayView<FRHIRayTracingShader*>& Shaders)
{
	for (FRHIRayTracingShader* Shader : Shaders)
	{
		int32 Index = FindRayTracingHitGroupIndex(Pipeline, Shader, false);
		if (Index == INDEX_NONE)
		{
			return false;
		}
	}
	return true;
}

FRHIRayTracingShader* GetRayTracingDefaultMissShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FPackedMaterialClosestHitPayloadMS>().GetRayTracingShader();
}

FRHIRayTracingShader* GetRayTracingDefaultOpaqueShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FOpaqueShadowHitGroup>().GetRayTracingShader();
}

FRHIRayTracingShader* GetRayTracingDefaultHiddenShader(const FGlobalShaderMap* ShaderMap)
{
	return ShaderMap->GetShader<FHiddenMaterialHitGroup>().GetRayTracingShader();
}

void FDeferredShadingSceneRenderer::CreateMaterialRayTracingMaterialPipeline(
	FRDGBuilder& GraphBuilder,
	const TArrayView<FRHIRayTracingShader*>& RayGenShaderTable,
	uint32& OutMaxLocalBindingDataSize,
	bool& bOutIsUsingFallbackRTPSO)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDeferredShadingSceneRenderer::CreateRayTracingMaterialPipeline);
	SCOPE_CYCLE_COUNTER(STAT_CreateRayTracingPipeline);

	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(ShaderPlatform);

	FRHICommandList& RHICmdList = GraphBuilder.RHICmdList;

	const bool bIsPathTracing = ViewFamily.EngineShowFlags.PathTracing;
	const bool bSupportMeshDecals = bIsPathTracing;

	ERayTracingPayloadType PayloadType = bIsPathTracing
		? (ERayTracingPayloadType::PathTracingMaterial | ERayTracingPayloadType::Decals)
		: ERayTracingPayloadType::RayTracingMaterial;

	FRayTracingPipelineStateInitializer Initializer;
	Initializer.MaxPayloadSizeInBytes = GetRayTracingPayloadTypeMaxSize(PayloadType);

	const FShaderBindingLayout* ShaderBindingLayout = RayTracing::GetShaderBindingLayout(ShaderPlatform);
	if (ShaderBindingLayout)
	{
		Initializer.ShaderBindingLayout = &ShaderBindingLayout->RHILayout;
	}

	FRHIRayTracingShader* DefaultMissShader = bIsPathTracing ? GetPathTracingDefaultMissShader(ShaderMap) : GetRayTracingDefaultMissShader(ShaderMap);

	TArray<FRHIRayTracingShader*> RayTracingMissShaderLibrary;
	FShaderMapResource::GetRayTracingMissShaderLibrary(ShaderPlatform, RayTracingMissShaderLibrary, DefaultMissShader);

	// make sure we have at least one miss shader present
	check(RayTracingMissShaderLibrary.Num() > 0);

	Initializer.SetMissShaderTable(RayTracingMissShaderLibrary);

	Initializer.SetRayGenShaderTable(RayGenShaderTable);

	const bool bMaterialsCompiled = AreRayTracingMaterialsCompiled(ShaderPlatform);
	const bool bEnableMaterials = bMaterialsCompiled && GEnableRayTracingMaterials != 0;
	static auto CVarEnableShadowMaterials = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing.Shadows.EnableMaterials"));
	const bool bEnableShadowMaterials = bMaterialsCompiled && (CVarEnableShadowMaterials ? CVarEnableShadowMaterials->GetInt() != 0 : true);

	FRHIRayTracingShader* OpaqueShadowShader   = bIsPathTracing ? GetPathTracingDefaultOpaqueHitShader(ShaderMap) : GetRayTracingDefaultOpaqueShader(ShaderMap);
	FRHIRayTracingShader* HiddenMaterialShader = bIsPathTracing ? GetPathTracingDefaultHiddenHitShader(ShaderMap) : GetRayTracingDefaultHiddenShader(ShaderMap);

	FRHIRayTracingShader* OpaqueMeshDecalHitShader = bSupportMeshDecals ? GetDefaultOpaqueMeshDecalHitShader(ShaderMap) : nullptr;
	FRHIRayTracingShader* HiddenMeshDecalHitShader = bSupportMeshDecals ? GetDefaultHiddenMeshDecalHitShader(ShaderMap) : nullptr;
	
	TArray<FRHIRayTracingShader*> RayTracingHitGroupLibrary;
	if (bEnableMaterials)
	{
		FShaderMapResource::GetRayTracingHitGroupLibrary(ShaderPlatform, RayTracingHitGroupLibrary, OpaqueShadowShader);

		if (bSupportMeshDecals)
		{
			FShaderMapResource::GetRayTracingHitGroupLibrary(ShaderPlatform, RayTracingHitGroupLibrary, OpaqueMeshDecalHitShader);
		}
	}

	FRHIRayTracingShader* RequiredHitShaders[] = { OpaqueShadowShader, HiddenMaterialShader };
	FRHIRayTracingShader* RequiredHitDecalShaders[] = { OpaqueMeshDecalHitShader, HiddenMeshDecalHitShader };

	RayTracingHitGroupLibrary.Append(RequiredHitShaders);
	if (bSupportMeshDecals)
	{
		RayTracingHitGroupLibrary.Append(RequiredHitDecalShaders);
	}

	Initializer.SetHitGroupTable(RayTracingHitGroupLibrary);

	// For now, only path tracing uses callable shaders (for decals). This is only enabled if the current platform supports callable shaders.
	const bool bCallableShadersRequired = bIsPathTracing && RHISupportsRayTracingCallableShaders(ShaderPlatform);
	TArray<FRHIRayTracingShader*> RayTracingCallableShaderLibrary;
	FRHIRayTracingShader* DefaultCallableShader = nullptr;

	if (bCallableShadersRequired)
	{
		DefaultCallableShader = ShaderMap->GetShader<FDefaultCallableShader>().GetRayTracingShader();
		check(DefaultCallableShader != nullptr);

		if (bEnableMaterials)
		{
			FShaderMapResource::GetRayTracingCallableShaderLibrary(ShaderPlatform, RayTracingCallableShaderLibrary, DefaultCallableShader);
		}
		else
		{
			RayTracingCallableShaderLibrary.Add(DefaultCallableShader);
		}

		Initializer.SetCallableTable(RayTracingCallableShaderLibrary);
	}

	bool bIsOfflineRender = false;

	for(const FViewInfo& View : Views)
	{
		if (View.bIsOfflineRender)
		{
			bIsOfflineRender = true;
			break;
		}
	}
	
	const bool bAllowNonBlockingPipelineCreation = GRayTracingNonBlockingPipelineCreation && !bIsOfflineRender;
	FRayTracingPipelineState* FallbackPipelineState = bAllowNonBlockingPipelineCreation
		? PipelineStateCache::GetRayTracingPipelineState(Scene->LastRayTracingMaterialPipelineSignature)
		: nullptr;

	ERayTracingPipelineCacheFlags PipelineCacheFlags = ERayTracingPipelineCacheFlags::Default;
	const bool bCompatiblePipelineSignatures = FallbackPipelineState ? IsCompatibleFallbackPipelineSignature(Scene->LastRayTracingMaterialPipelineSignature, Initializer) : false;
	if (FallbackPipelineState
		&& bCompatiblePipelineSignatures
		&& PipelineContainsHitShaders(FallbackPipelineState, RequiredHitShaders)
		&& (!bSupportMeshDecals || PipelineContainsHitShaders(FallbackPipelineState, RequiredHitDecalShaders))
		&& FindRayTracingMissShaderIndex(FallbackPipelineState, DefaultMissShader, false) != INDEX_NONE
		&& (!bCallableShadersRequired || FindRayTracingCallableShaderIndex(FallbackPipelineState, DefaultCallableShader, false) != INDEX_NONE))
	{
		PipelineCacheFlags |= ERayTracingPipelineCacheFlags::NonBlocking;
	}

	FRayTracingPipelineState* PipelineState = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(RHICmdList, Initializer, PipelineCacheFlags);

	if (PipelineState)
	{
		// Save the current pipeline to be used as fallback in future frames
		Scene->LastRayTracingMaterialPipelineSignature = static_cast<FRayTracingPipelineStateSignature&>(Initializer);
        Scene->bLastFrameUsedFallback = false;
	}
	else
	{
		// If pipeline was not found in cache, use the fallback from previous frame
		check(FallbackPipelineState);
		PipelineState = FallbackPipelineState;
		bOutIsUsingFallbackRTPSO = true;
#if WITH_EDITOR
        static const FText Message = NSLOCTEXT("Renderer", "RayTracingPSONotReady", "Ray Tracing PSO is not ready yet, using fallback.");
		OnGetOnScreenMessages.AddLambda([](FScreenMessageWriter& ScreenMessageWriter) -> void
        {
            ScreenMessageWriter.DrawLine(Message);
        });
        if (!Scene->bLastFrameUsedFallback)
        {
            UE_LOGF(LogRenderer, Log, "%ls", *Message.ToString());
            Scene->bLastFrameUsedFallback = true;
        }
#endif
	}

	// Retrieve the binding data size from the actual used RTPSO because the requested RTPSO could still be non blocking async compiling
	// and then we are using the RTPSO from the previous frame
	OutMaxLocalBindingDataSize = FMath::Max(OutMaxLocalBindingDataSize, GetRHIRayTracingPipelineStateMaxLocalBindingDataSize(PipelineState));

	if (FallbackPipelineState != nullptr && PipelineState != FallbackPipelineState && bIsPathTracing && !bIsOfflineRender)
	{
		// When using path tracing, a change in pipeline state compared to the previous frame means some new materials got added to the RTPSO
		// and we should restart sample accumulation.
		// Only do this if the pipeline signatures are compatible, otherwise we might be toggling between Lit and PathTraced views and don't want to invalidate the state
		if (bCompatiblePipelineSignatures)
		{
			Scene->InvalidatePathTracedOutput();
		}
	}

	check(PipelineState);

	// Send RTPSO to all views since they all share the same one
	EnumerateLinkedViews([PipelineState](FViewInfo& View)
		{
			if (View.bHasAnyRayTracingPass)
			{
				View.MaterialRayTracingData.PipelineState = PipelineState;
			}
			return true;
		});
}

void FDeferredShadingSceneRenderer::SetupMaterialRayTracingHitGroupBindings(FRDGBuilder& GraphBuilder, FViewInfo& View, TConstArrayView<FRayTracingShaderBindingData> RayTracingShaderBindings)
{
	FRayTracingPipelineState* PipelineState = View.MaterialRayTracingData.PipelineState;

	const bool bIsPathTracing = ViewFamily.EngineShowFlags.PathTracing;
	const bool bSupportMeshDecals = bIsPathTracing;
	const bool bMaterialsCompiled = AreRayTracingMaterialsCompiled(View.GetShaderPlatform());
	const bool bEnableMaterials = bMaterialsCompiled && GEnableRayTracingMaterials != 0;
	static auto CVarEnableShadowMaterials = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing.Shadows.EnableMaterials"));
	const bool bEnableShadowMaterials = bMaterialsCompiled && (CVarEnableShadowMaterials ? CVarEnableShadowMaterials->GetInt() != 0 : true);

	FRHIRayTracingShader* OpaqueShadowShader = bIsPathTracing ? GetPathTracingDefaultOpaqueHitShader(View.ShaderMap) : GetRayTracingDefaultOpaqueShader(View.ShaderMap);
	FRHIRayTracingShader* HiddenMaterialShader = bIsPathTracing ? GetPathTracingDefaultHiddenHitShader(View.ShaderMap) : GetRayTracingDefaultHiddenShader(View.ShaderMap);

	const int32 OpaqueShadowMaterialIndex = FindRayTracingHitGroupIndex(PipelineState, OpaqueShadowShader, true);
	const int32 HiddenMaterialIndex = FindRayTracingHitGroupIndex(PipelineState, HiddenMaterialShader, true);

	const int32 OpaqueMeshDecalHitGroupIndex = bSupportMeshDecals ? FindRayTracingHitGroupIndex(PipelineState, GetDefaultOpaqueMeshDecalHitShader(View.ShaderMap), true) : INDEX_NONE;
	const int32 HiddenMeshDecalHitGroupIndex = bSupportMeshDecals ? FindRayTracingHitGroupIndex(PipelineState, GetDefaultHiddenMeshDecalHitShader(View.ShaderMap), true) : INDEX_NONE;

	// Scene UB is only needed when shader binding layout is not used because then it's bound via the global bindings
	// Should ideally be lazy fetched during binding if needed
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRHIUniformBuffer* SceneUB = GetSceneUniforms().GetBufferRHI(GraphBuilder);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	FRHIUniformBuffer* NaniteRayTracingUB = Nanite::GRayTracingManager.GetUniformBufferRHI(GraphBuilder);

	// material hit groups
	AddRayTracingLocalShaderBindingWriterTasks(GraphBuilder, RayTracingShaderBindings, View.MaterialRayTracingData.MaterialBindings, 
		[&View, PipelineState, OpaqueShadowMaterialIndex, HiddenMaterialIndex, OpaqueMeshDecalHitGroupIndex, HiddenMeshDecalHitGroupIndex, SceneUB, NaniteRayTracingUB, bIsPathTracing, bSupportMeshDecals,
		bEnableMaterials, bEnableShadowMaterials, &RayTracingSBT = Scene->RayTracingSBT, &RayTracingMeshCommands = Scene->CachedRayTracingMeshCommands](const FRayTracingShaderBindingData& DirtyShaderBinding, FRayTracingLocalShaderBindingWriter* BindingWriter)
	{
		const FRayTracingMeshCommand& MeshCommand = DirtyShaderBinding.GetRayTracingMeshCommand(RayTracingMeshCommands);
	
		const bool bIsMeshDecalShader = MeshCommand.MaterialShader->RayTracingPayloadType == (uint32)ERayTracingPayloadType::Decals;
	
		// TODO: Following check is disabled since FRayTracingMeshProcessor non-path-tracing code paths still don't assign the appropriate shader to decal mesh commands.
		// We could also potentially use regular materials to approximate decals in ray tracing in some situations.
		// check(bIsMeshDecalShader == MeshCommand.bDecal);
	
		// Force the same shader to be used on all geometry unless materials are enabled
		int32 HitGroupIndex;
	
		if (bIsMeshDecalShader)
		{
			checkf(bSupportMeshDecals && MeshCommand.bDecal, TEXT("Unexpected ray tracing mesh command using Mesh Decal payload. Fix logic adding the command or update bSupportMeshDecals as appropriate."));
			HitGroupIndex = DirtyShaderBinding.bHidden ? HiddenMeshDecalHitGroupIndex : OpaqueMeshDecalHitGroupIndex;
		}
		else
		{
			checkf((!bIsPathTracing && MeshCommand.MaterialShader->RayTracingPayloadType == (uint32)ERayTracingPayloadType::RayTracingMaterial)
				|| (bIsPathTracing && MeshCommand.MaterialShader->RayTracingPayloadType == (uint32)ERayTracingPayloadType::PathTracingMaterial),
				TEXT("Incorrectly using RayTracingMaterial when path tracer is enabled or vice-versa."));
			HitGroupIndex = DirtyShaderBinding.bHidden ? HiddenMaterialIndex : OpaqueShadowMaterialIndex;
		}
	
		if (bEnableMaterials && !DirtyShaderBinding.bHidden)
		{
			const int32 FoundIndex = FindRayTracingHitGroupIndex(PipelineState, MeshCommand.MaterialShader, false);
			if (FoundIndex != INDEX_NONE)
			{
				HitGroupIndex = FoundIndex;
			}
			else if (RayTracingSBT.IsPersistent())
			{
				check(DirtyShaderBinding.BindingType == ERayTracingLocalShaderBindingType::Transient);
				check(RayTracingSBT.IsDirty(DirtyShaderBinding.SBTRecordIndex));
			}
		}
	
		uint32 BaseRecordIndex = DirtyShaderBinding.SBTRecordIndex;
	
		// Bind primary material shader
	
		{
			MeshCommand.SetRayTracingShaderBindingsForHitGroup(BindingWriter,
				View.ViewUniformBuffer,
				SceneUB,
				NaniteRayTracingUB,
				BaseRecordIndex + RAY_TRACING_SHADER_SLOT_MATERIAL,
				DirtyShaderBinding.RayTracingGeometry,
				MeshCommand.GeometrySegmentIndex,
				HitGroupIndex,
				DirtyShaderBinding.BindingType);
		}
	
		// Bind shadow shader
		if (bIsMeshDecalShader)
		{
			// mesh decals do not use the shadow slot, so do minimal work
			FRayTracingLocalShaderBindings& Binding = BindingWriter->AddWithExternalParameters();
			Binding.RecordIndex = BaseRecordIndex + RAY_TRACING_SHADER_SLOT_SHADOW;
			Binding.Geometry = DirtyShaderBinding.RayTracingGeometry;
			Binding.SegmentIndex = MeshCommand.GeometrySegmentIndex;
			Binding.ShaderIndexInPipeline = HiddenMeshDecalHitGroupIndex;
			Binding.BindingType = DirtyShaderBinding.BindingType;
	
		}
		else if (MeshCommand.bCastRayTracedShadows && !DirtyShaderBinding.bHidden)
		{
			if (MeshCommand.bOpaque || !bEnableShadowMaterials)
			{
				FRayTracingLocalShaderBindings& Binding = BindingWriter->AddWithExternalParameters();
				Binding.RecordIndex = BaseRecordIndex + RAY_TRACING_SHADER_SLOT_SHADOW;
				Binding.Geometry = DirtyShaderBinding.RayTracingGeometry;
				Binding.SegmentIndex = MeshCommand.GeometrySegmentIndex;
				Binding.ShaderIndexInPipeline = OpaqueShadowMaterialIndex;
				Binding.BindingType = DirtyShaderBinding.BindingType;
			}
			else
			{
				// Masked materials require full material evaluation with any-hit shader.
				// Full CHS is bound, however material evaluation is skipped for shadow rays using a dynamic branch on a ray payload flag.
				MeshCommand.SetRayTracingShaderBindingsForHitGroup(BindingWriter,
					View.ViewUniformBuffer,
					SceneUB,
					NaniteRayTracingUB,
					BaseRecordIndex + RAY_TRACING_SHADER_SLOT_SHADOW,
					DirtyShaderBinding.RayTracingGeometry,
					MeshCommand.GeometrySegmentIndex,
					HitGroupIndex,
					DirtyShaderBinding.BindingType);
			}
		}
		else
		{
			FRayTracingLocalShaderBindings& Binding = BindingWriter->AddWithExternalParameters();
			Binding.RecordIndex = BaseRecordIndex + RAY_TRACING_SHADER_SLOT_SHADOW;
			Binding.Geometry = DirtyShaderBinding.RayTracingGeometry;
			Binding.SegmentIndex = MeshCommand.GeometrySegmentIndex;
			Binding.ShaderIndexInPipeline = HiddenMaterialIndex;
			Binding.BindingType = DirtyShaderBinding.BindingType;
		}
	});

	// For now, only path tracing uses callable shaders (for decals). This is only enabled if the current platform supports callable shaders.
	const bool bCallableShadersRequired = bIsPathTracing && RHISupportsRayTracingCallableShaders(View.Family->GetShaderPlatform());
	if (bCallableShadersRequired)
	{
		FRHIRayTracingShader* DefaultCallableShader = View.ShaderMap->GetShader<FDefaultCallableShader>().GetRayTracingShader();
		const int32 DefaultCallableShaderIndex = FindRayTracingCallableShaderIndex(PipelineState, DefaultCallableShader, true);

		const uint32 TargetCommandsPerTask = 4096;

		const uint32 NumTotalCallableCommands = Scene->RayTracingSBT.CallableCommands.Num();
		const uint32 NumTasks = FMath::Max(1u, FMath::DivideAndRoundUp(NumTotalCallableCommands, TargetCommandsPerTask));
		const uint32 CommandsPerTask = FMath::DivideAndRoundUp(NumTotalCallableCommands, NumTasks); // Evenly divide commands between tasks (avoiding potential short last task)

		View.MaterialRayTracingData.CallableBindings.SetNum(NumTasks);

		for (uint32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			const uint32 TaskBaseCommandIndex = TaskIndex * CommandsPerTask;
			const FRayTracingShaderCommand* TaskCallableCommands = Scene->RayTracingSBT.CallableCommands.GetData() + TaskBaseCommandIndex;
			const uint32 NumCommands = FMath::Min(CommandsPerTask, NumTotalCallableCommands - TaskBaseCommandIndex);

			FRayTracingLocalShaderBindingWriter* BindingWriter = new FRayTracingLocalShaderBindingWriter();
			View.MaterialRayTracingData.CallableBindings[TaskIndex] = BindingWriter;

			GraphBuilder.AddSetupTask(
				[&View, SceneUB, NaniteRayTracingUB, PipelineState, BindingWriter, TaskCallableCommands, NumCommands, bEnableMaterials, DefaultCallableShaderIndex, TaskIndex]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(BindRayTracingMaterialPipelineTask);

					for (uint32 CommandIndex = 0; CommandIndex < NumCommands; ++CommandIndex)
					{
						const FRayTracingShaderCommand& CallableCommand = TaskCallableCommands[CommandIndex];

						int32 CallableShaderIndex = DefaultCallableShaderIndex; // Force the same shader to be used on all geometry unless materials are enabled

						if (bEnableMaterials)
						{
							const int32 FoundIndex = FindRayTracingCallableShaderIndex(PipelineState, CallableCommand.Shader, false);
							if (FoundIndex != INDEX_NONE)
							{
								CallableShaderIndex = FoundIndex;
							}
						}

						CallableCommand.SetRayTracingShaderBindings(
							BindingWriter, 
							View.ViewUniformBuffer, SceneUB, NaniteRayTracingUB,
							CallableShaderIndex, CallableCommand.SlotInScene);
					}
				});
		}
	}
}

void MergeAndSetRayTracingBindings(
	FRHICommandList& RHICmdList,
	FSceneRenderingBulkObjectAllocator& Allocator,
	FRHIShaderBindingTable* SBT,
	FRayTracingPipelineState* Pipeline,
	TConstArrayView<FRayTracingLocalShaderBindingWriter*> Bindings,
	ERayTracingBindingType BindingType)
{
	// Gather bindings from all chunks and submit them all as a single batch to allow RHI to bind all shader parameters in parallel.

	uint32 NumTotalBindings = 0;

	for (FRayTracingLocalShaderBindingWriter* BindingWriter : Bindings)
	{
		const FRayTracingLocalShaderBindingWriter::FChunk* Chunk = BindingWriter->GetFirstChunk();
		while (Chunk)
		{
			NumTotalBindings += Chunk->Num;
			Chunk = Chunk->Next;
		}
	}

	if (NumTotalBindings == 0)
	{
		return;
	}

	const uint32 MergedBindingsSize = sizeof(FRayTracingLocalShaderBindings) * NumTotalBindings;
	FRayTracingLocalShaderBindings* MergedBindings = (FRayTracingLocalShaderBindings*)(RHICmdList.Bypass()
		? Allocator.Malloc(MergedBindingsSize, alignof(FRayTracingLocalShaderBindings))
		: RHICmdList.Alloc(MergedBindingsSize, alignof(FRayTracingLocalShaderBindings)));

	uint32 MergedBindingIndex = 0;
	for (FRayTracingLocalShaderBindingWriter* BindingWriter : Bindings)
	{
		const FRayTracingLocalShaderBindingWriter::FChunk* Chunk = BindingWriter->GetFirstChunk();
		while (Chunk)
		{
			const uint32 Num = Chunk->Num;
			for (uint32_t i = 0; i < Num; ++i)
			{
				MergedBindings[MergedBindingIndex] = Chunk->Bindings[i];
				MergedBindingIndex++;
			}
			Chunk = Chunk->Next;
		}
	}

	const bool bCopyDataToInlineStorage = false; // Storage is already allocated from RHICmdList, no extra copy necessary
	RHICmdList.SetBindingsOnShaderBindingTable(
		SBT,
		Pipeline,
		NumTotalBindings, MergedBindings,
		BindingType,
		bCopyDataToInlineStorage);
}

void SetRayTracingShaderBindings(FRHICommandList& RHICmdList, FSceneRenderingBulkObjectAllocator& Allocator, FViewInfo::FRayTracingData& RayTracingData)
{
	if (!RayTracingData.MaterialBindings.IsEmpty())
	{
		MergeAndSetRayTracingBindings(RHICmdList, Allocator, RayTracingData.ShaderBindingTable, RayTracingData.PipelineState, RayTracingData.MaterialBindings, ERayTracingBindingType::HitGroup);
	}
	if (!RayTracingData.CallableBindings.IsEmpty())
	{
		MergeAndSetRayTracingBindings(RHICmdList, Allocator, RayTracingData.ShaderBindingTable, RayTracingData.PipelineState, RayTracingData.CallableBindings, ERayTracingBindingType::CallableShader);
	}

	// Move the ray tracing binding container ownership to the command list, so that memory will be
	// released on the RHI thread timeline, after the commands that reference it are processed.
	RHICmdList.EnqueueLambda([PtrsA = MoveTemp(RayTracingData.MaterialBindings), 
	                          PtrsB = MoveTemp(RayTracingData.CallableBindings),
	                          Mem = MoveTemp(RayTracingData.MaterialBindingsMemory)](FRHICommandList&)
	{
		for (auto Ptr : PtrsA)
	 	{
			delete Ptr;
	 	}
	 	for (auto Ptr : PtrsB)
	 	{
			delete Ptr;
	 	}
	});
}

#endif // RHI_RAYTRACING
