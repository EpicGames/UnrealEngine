// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#include "DataDrivenShaderPlatformInfo.h"
#include "LightMapRendering.h"
#include "MaterialDomain.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.inl"
#include "RayTracingMeshDrawCommands.h"
#include "RayTracingInstanceMask.h"
#include "RayTracingPayloadType.h"
#include "RayTracing/RayTracing.h"
#include "ShaderParameterStruct.h"
#include "SceneRendering.h"
#include <type_traits>

FRHIRayTracingShader* GetRayTracingDefaultMissShader(const FGlobalShaderMap* ShaderMap);
FRHIRayTracingShader* GetRayTracingDefaultOpaqueShader(const FGlobalShaderMap* ShaderMap);
FRHIRayTracingShader* GetRayTracingDefaultHiddenShader(const FGlobalShaderMap* ShaderMap);

class FRayTracingMeshProcessor
{
public:
	RENDERER_API FRayTracingMeshProcessor(FRayTracingMeshCommandContext* InCommandContext, const FScene* InScene, const FSceneView* InViewIfDynamicMeshCommand, ERayTracingType InRayTracingType);
	RENDERER_API ~FRayTracingMeshProcessor();

	RENDERER_API void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy);

private:
	FRayTracingMeshCommandContext* CommandContext;
	const FScene* Scene;
	const FSceneView* ViewIfDynamicMeshCommand;
	ERHIFeatureLevel::Type FeatureLevel;
	ERayTracingType RayTracingType;

	bool Process(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		const FUniformLightMapPolicy& RESTRICT LightMapPolicy);

	template<typename RayTracingShaderType, typename ShaderElementDataType>
	void BuildRayTracingMeshCommands(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		const TShaderRef<RayTracingShaderType>& RayTracingShader,
		const ShaderElementDataType& ShaderElementData)
	{
		const FVertexFactory* RESTRICT VertexFactory = MeshBatch.VertexFactory;

		FRayTracingMeshCommand SharedCommand;

		SetupRayTracingMeshCommandMaskAndStatus(SharedCommand, MeshBatch, PrimitiveSceneProxy, MaterialResource, RayTracingType);

		if (GRHISupportsRayTracingShaders)
		{
			SharedCommand.SetShader(RayTracingShader);
		}

		FVertexInputStreamArray VertexStreams;
		VertexFactory->GetStreams(FeatureLevel, EVertexInputStreamType::Default, VertexStreams);

		if (RayTracingShader.IsValid())
		{
			FMeshDrawSingleShaderBindings ShaderBindings = SharedCommand.ShaderBindings.GetSingleShaderBindings(SF_RayHitGroup);
			RayTracingShader->GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, MaterialResource, ShaderElementData, ShaderBindings);
		}

		const int32 NumElements = MeshBatch.Elements.Num();

		for (int32 BatchElementIndex = 0; BatchElementIndex < NumElements; BatchElementIndex++)
		{
			if ((1ull << BatchElementIndex) & BatchElementMask)
			{
				const FMeshBatchElement& BatchElement = MeshBatch.Elements[BatchElementIndex];
				FRayTracingMeshCommand& RayTracingMeshCommand = CommandContext->AddCommand(SharedCommand);

				if (RayTracingShader.IsValid())
				{
					FMeshDrawSingleShaderBindings RayHitGroupShaderBindings = RayTracingMeshCommand.ShaderBindings.GetSingleShaderBindings(SF_RayHitGroup);
					FMeshMaterialShader::GetElementShaderBindings(RayTracingShader, Scene, ViewIfDynamicMeshCommand, VertexFactory, EVertexInputStreamType::Default, FeatureLevel, PrimitiveSceneProxy, MeshBatch, BatchElement, ShaderElementData, RayHitGroupShaderBindings, VertexStreams);
				}

				// Command can only be cached if no global/static uniform buffers are used - if all platforms use SBTLayout for all RT shaders then this could be a check
				RayTracingMeshCommand.bCanBeCached = !RayTracingMeshCommand.HasGlobalUniformBufferBindings();

				RayTracingMeshCommand.GeometrySegmentIndex = uint32(MeshBatch.SegmentIndex) + BatchElementIndex;
				RayTracingMeshCommand.bIsTranslucent = MeshBatch.IsTranslucent(MaterialResource.GetFeatureLevel());
				CommandContext->FinalizeCommand(RayTracingMeshCommand);
			}
		}
	}

	bool ProcessPathTracing(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource);

	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material
	);
};

class FHiddenMaterialHitGroup : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FHiddenMaterialHitGroup, RENDERER_API)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FHiddenMaterialHitGroup, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	using FParameters = FEmptyShaderParameters;
};

class FOpaqueShadowHitGroup : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FOpaqueShadowHitGroup, RENDERER_API)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FOpaqueShadowHitGroup, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	using FParameters = FEmptyShaderParameters;
};

class FDefaultCallableShader : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FDefaultCallableShader, RENDERER_API)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FDefaultCallableShader, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingCallableShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::Decals;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	using FParameters = FEmptyShaderParameters;
};

class FRayTracingLocalShaderBindingWriter
{
public:

	FRayTracingLocalShaderBindingWriter()
	{}

	FRayTracingLocalShaderBindingWriter(const FRayTracingLocalShaderBindingWriter&) = delete;
	FRayTracingLocalShaderBindingWriter& operator = (const FRayTracingLocalShaderBindingWriter&) = delete;
	FRayTracingLocalShaderBindingWriter(FRayTracingLocalShaderBindingWriter&&) = delete;
	FRayTracingLocalShaderBindingWriter& operator = (FRayTracingLocalShaderBindingWriter&&) = delete;

	~FRayTracingLocalShaderBindingWriter() = default;

	FRayTracingLocalShaderBindings& AddWithInlineParameters(uint32 NumUniformBuffers, uint32 LooseDataSize = 0)
	{
		FRayTracingLocalShaderBindings* Result = AllocateInternal();

		if (NumUniformBuffers)
		{
			uint32 AllocSize = sizeof(FRHIUniformBuffer*) * NumUniformBuffers;
			Result->UniformBuffers = (FRHIUniformBuffer**)ParameterMemory.Alloc(AllocSize, alignof(FRHIUniformBuffer*));
			FMemory::Memset(Result->UniformBuffers, 0, AllocSize);
		}
		Result->NumUniformBuffers = NumUniformBuffers;

		if (LooseDataSize)
		{
			Result->LooseParameterData = (uint8*)ParameterMemory.Alloc(LooseDataSize, alignof(void*));
		}
		Result->LooseParameterDataSize = LooseDataSize;

		return *Result;
	}

	FRayTracingLocalShaderBindings& AddWithExternalParameters()
	{
		return *AllocateInternal();
	}

	void Commit(FRHICommandList& RHICmdList, FRHIShaderBindingTable* SBT, FRayTracingPipelineState* Pipeline, bool bCopyDataToInlineStorage) const
	{
		const FChunk* Chunk = FirstChunk;
		while (Chunk)
		{
			RHICmdList.SetRayTracingHitGroups(SBT, Pipeline, Chunk->Num, Chunk->Bindings, bCopyDataToInlineStorage);
			Chunk = Chunk->Next;
		}
	}

	struct FChunk
	{
		static constexpr uint32 MaxNum = 1024;

		// Note: constructors for elements of this array are called explicitly in AllocateInternal(). Destructors are not called.
		static_assert(std::is_trivially_destructible_v<FRayTracingLocalShaderBindings>, "FRayTracingLocalShaderBindings must be trivially destructible, as no destructor will be called.");
		FRayTracingLocalShaderBindings Bindings[MaxNum];
		FChunk* Next;
		uint32 Num;
	};

	const FChunk* GetFirstChunk() const
	{
		return FirstChunk;
	}

private:

	FChunk* FirstChunk = nullptr;
	FChunk* CurrentChunk = nullptr;

	FMemStackBase ParameterMemory;

	friend class FRHICommandList;

	FRayTracingLocalShaderBindings* AllocateInternal()
	{
		if (!CurrentChunk || CurrentChunk->Num == FChunk::MaxNum)
		{
			FChunk* OldChunk = CurrentChunk;

			static_assert(std::is_trivially_destructible_v<FChunk>, "Chunk must be trivially destructible, as no destructor will be called.");
			CurrentChunk = (FChunk*)ParameterMemory.Alloc(sizeof(FChunk), alignof(FChunk));
			CurrentChunk->Next = nullptr;
			CurrentChunk->Num = 0;

			if (FirstChunk == nullptr)
			{
				FirstChunk = CurrentChunk;
			}

			if (OldChunk)
			{
				OldChunk->Next = CurrentChunk;
			}
		}

		FRayTracingLocalShaderBindings* ResultMemory = &CurrentChunk->Bindings[CurrentChunk->Num++];
		return new(ResultMemory) FRayTracingLocalShaderBindings;
	}
};

template <typename FunctionType>
void AddRayTracingLocalShaderBindingWriterTasks(
	FRDGBuilder& GraphBuilder,
	TConstArrayView<FRayTracingShaderBindingData> DirtyPersistentRayTracingShaderBindings,
	TArray<FRayTracingLocalShaderBindingWriter*, SceneRenderingAllocator>& ShaderBindingWriters,
	FunctionType SetupBindingsFunction)
{	
	const uint32 NumTotalDirtyBindings = DirtyPersistentRayTracingShaderBindings.Num();
	const uint32 TargetBindingsPerTask = 1024;
	const uint32 NumTasks = FMath::Max(1u, FMath::DivideAndRoundUp(NumTotalDirtyBindings, TargetBindingsPerTask));
	const uint32 BindingsPerTask = FMath::DivideAndRoundUp(NumTotalDirtyBindings, NumTasks); // Evenly divide commands between tasks (avoiding potential short last task)

	ShaderBindingWriters.SetNum(NumTasks);

	for (uint32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
	{
		const uint32 FirstTaskBindingIndex = TaskIndex * BindingsPerTask;
		const FRayTracingShaderBindingData* RTShaderBindings = DirtyPersistentRayTracingShaderBindings.GetData() + FirstTaskBindingIndex;
		const uint32 NumBindings = FMath::Min(BindingsPerTask, NumTotalDirtyBindings - FirstTaskBindingIndex);

		FRayTracingLocalShaderBindingWriter* BindingWriter = new FRayTracingLocalShaderBindingWriter();
		ShaderBindingWriters[TaskIndex] = BindingWriter;

		GraphBuilder.AddSetupTask(
			[BindingWriter, RTShaderBindings, NumBindings, SetupBindingsFunction]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(BuildRayTracingMaterialBindingsTask);

				for (uint32 BindingIndex = 0; BindingIndex < NumBindings; ++BindingIndex)
				{
					const FRayTracingShaderBindingData& RTShaderBindingData = RTShaderBindings[BindingIndex];
					SetupBindingsFunction(RTShaderBindingData, BindingWriter);
				}
			});
	}
}

void SetRayTracingShaderBindings(FRHICommandList& RHICmdList, FSceneRenderingBulkObjectAllocator& Allocator, FViewInfo::FRayTracingData& RayTracingData);

#endif // RHI_RAYTRACING
