// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	MeshPassProcessor.cpp: 
=============================================================================*/

#include "RayTracingMeshDrawCommands.h"
#include "SceneUniformBuffer.h"
#include "Nanite/NaniteShared.h"
#include "RayTracingDefinitions.h"
#include "RayTracingShaderBindingTable.h"

#if RHI_RAYTRACING

void FDynamicRayTracingMeshCommandContext::FinalizeCommand(FRayTracingMeshCommand& RayTracingMeshCommand)
{
	check(GeometrySegmentIndex == RayTracingMeshCommand.GeometrySegmentIndex);

	if (SBTAllocation)
	{
		if (SBTAllocation->HasLayer(ERayTracingShaderBindingLayer::Base))
		{
			const bool bHidden = RayTracingMeshCommand.bDecal;
			const uint32 RecordIndex = SBTAllocation->GetRecordIndex(ERayTracingShaderBindingLayer::Base, RayTracingMeshCommand.GeometrySegmentIndex);
			FRayTracingShaderBindingData ShaderBinding(&RayTracingMeshCommand, RayTracingGeometry, bDynamicRayTracingGeometry, RecordIndex, ERayTracingLocalShaderBindingType::Transient, bHidden);
			ShaderBindings.Add(ShaderBinding);
		}

		if (SBTAllocation->HasLayer(ERayTracingShaderBindingLayer::Decals))
		{
			const bool bHidden = !RayTracingMeshCommand.bDecal;
			const uint32 RecordIndex = SBTAllocation->GetRecordIndex(ERayTracingShaderBindingLayer::Decals, RayTracingMeshCommand.GeometrySegmentIndex);
			FRayTracingShaderBindingData ShaderBinding(&RayTracingMeshCommand, RayTracingGeometry, bDynamicRayTracingGeometry, RecordIndex, ERayTracingLocalShaderBindingType::Transient, bHidden);
			ShaderBindings.Add(ShaderBinding);
		}
	}
}

void FRayTracingMeshCommand::SetRayTracingShaderBindingsForHitGroup(
	FRayTracingLocalShaderBindingWriter* BindingWriter,
	const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
	FRHIUniformBuffer* SceneUniformBuffer,
	FRHIUniformBuffer* NaniteUniformBuffer,
	uint32 RecordIndex,
	const FRHIRayTracingGeometry* RayTracingGeometry,
	uint32 SegmentIndex,
	uint32 HitGroupIndexInPipeline,
	ERayTracingLocalShaderBindingType BindingType) const
{
	FRayTracingLocalShaderBindings* Bindings = ShaderBindings.SetRayTracingShaderBindingsForHitGroup(BindingWriter, RecordIndex, RayTracingGeometry, SegmentIndex, HitGroupIndexInPipeline, BindingType);

	if (ViewUniformBufferParameter.IsBound())
	{
		check(ViewUniformBuffer);
		Bindings->UniformBuffers[ViewUniformBufferParameter.GetBaseIndex()] = ViewUniformBuffer;
	}

	if (SceneUniformBufferParameter.IsBound())
	{
		check(SceneUniformBuffer);
		Bindings->UniformBuffers[SceneUniformBufferParameter.GetBaseIndex()] = SceneUniformBuffer;
	}

	if (NaniteUniformBufferParameter.IsBound())
	{
		check(NaniteUniformBuffer);
		Bindings->UniformBuffers[NaniteUniformBufferParameter.GetBaseIndex()] = NaniteUniformBuffer;
	}
}

void FRayTracingMeshCommand::SetShader(const TShaderRef<FShader>& Shader)
{
	check(Shader.IsValid());

	// Retrieve shader first to make sure the library index is set (set the RHI shader)
	MaterialShader = Shader.GetRayTracingShader();
	MaterialShaderIndex = Shader.GetRayTracingHitGroupLibraryIndex();
	ViewUniformBufferParameter = Shader->GetUniformBufferParameter<FViewUniformShaderParameters>();
	SceneUniformBufferParameter = Shader->GetUniformBufferParameter<FSceneUniformParameters>();
	NaniteUniformBufferParameter = Shader->GetUniformBufferParameter<FNaniteRayTracingUniformParameters>();
	ShaderBindings.Initialize(Shader);

	if (NaniteUniformBufferParameter.IsBound())
	{
		check(bNaniteRayTracing);
	}
}

bool FRayTracingMeshCommand::IsUsingNaniteRayTracing() const
{
	return bNaniteRayTracing;
}

void FRayTracingMeshCommand::UpdateFlags(FRayTracingCachedMeshCommandFlags& Flags) const
{
	Flags.InstanceMask |= InstanceMask;
	Flags.bAllSegmentsOpaque &= bOpaque;
	Flags.bAllSegmentsCastShadow &= bCastRayTracedShadows;
	Flags.bAnySegmentsCastShadow |= bCastRayTracedShadows;
	Flags.bAnySegmentsDecal |= bDecal;
	Flags.bAllSegmentsDecal &= bDecal;
	Flags.bTwoSided |= bTwoSided;
	Flags.bIsSky |= bIsSky;
	Flags.bAllSegmentsTranslucent &= bIsTranslucent;
	Flags.bAllSegmentsReverseCulling &= bReverseCulling;
}

void FRayTracingShaderCommand::SetRayTracingShaderBindings(
	FRayTracingLocalShaderBindingWriter* BindingWriter,
	const TUniformBufferRef<FViewUniformShaderParameters>& ViewUniformBuffer,
	FRHIUniformBuffer* SceneUniformBuffer,
	FRHIUniformBuffer* NaniteUniformBuffer,
	uint32 ShaderIndexInPipeline,
	uint32 ShaderSlot) const
{
	FRayTracingLocalShaderBindings* Bindings = ShaderBindings.SetRayTracingShaderBindings(BindingWriter, ShaderIndexInPipeline, ShaderSlot, 0, ERayTracingLocalShaderBindingType::Transient);

	if (ViewUniformBufferParameter.IsBound())
	{
		check(ViewUniformBuffer);
		Bindings->UniformBuffers[ViewUniformBufferParameter.GetBaseIndex()] = ViewUniformBuffer;
	}

	if (SceneUniformBufferParameter.IsBound())
	{
		check(SceneUniformBuffer);
		Bindings->UniformBuffers[SceneUniformBufferParameter.GetBaseIndex()] = SceneUniformBuffer;
	}

	if (NaniteUniformBufferParameter.IsBound())
	{
		check(NaniteUniformBuffer);
		Bindings->UniformBuffers[NaniteUniformBufferParameter.GetBaseIndex()] = NaniteUniformBuffer;
	}
}

void FRayTracingShaderCommand::SetShader(const TShaderRef<FShader>& InShader)
{
	check(InShader->GetFrequency() == SF_RayCallable || InShader->GetFrequency() == SF_RayMiss);

	// Retrieve shader first to make sure the library index is set (set the RHI shader)
	Shader = InShader.GetRayTracingShader();
	ShaderIndex = InShader.GetRayTracingCallableShaderLibraryIndex();
	ViewUniformBufferParameter = InShader->GetUniformBufferParameter<FViewUniformShaderParameters>();
	SceneUniformBufferParameter = InShader->GetUniformBufferParameter<FSceneUniformParameters>();
	NaniteUniformBufferParameter = InShader->GetUniformBufferParameter<FNaniteRayTracingUniformParameters>();

	ShaderBindings.Initialize(InShader);
}
#endif // RHI_RAYTRACING