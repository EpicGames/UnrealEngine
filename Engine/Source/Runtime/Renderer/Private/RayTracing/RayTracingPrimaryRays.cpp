// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalShader.h"
#include "DeferredShadingRenderer.h"
#include "SceneTextureParameters.h"

#if RHI_RAYTRACING

#include "ClearQuad.h"
#include "FogRendering.h"
#include "SceneRendering.h"
#include "PostProcess/SceneRenderTargets.h"
#include "RHIResources.h"
#include "PostProcess/PostProcessing.h"
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracingLighting.h"
#include "RayTracing/RayTracing.h"
#include "Nanite/NaniteRayTracing.h"

DECLARE_GPU_STAT(RayTracingPrimaryRays);

class FRayTracingPrimaryRaysRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingPrimaryRaysRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTracingPrimaryRaysRGS, FGlobalShader)

	class FEnableTwoSidedGeometryForShadowDim : SHADER_PERMUTATION_BOOL("ENABLE_TWO_SIDED_GEOMETRY");

	using FPermutationDomain = TShaderPermutationDomain<FEnableTwoSidedGeometryForShadowDim>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, SamplesPerPixel)
		SHADER_PARAMETER(int32, MaxRefractionRays)
		SHADER_PARAMETER(int32, HeightFog)
		SHADER_PARAMETER(int32, ForceOpaqueRays)
		SHADER_PARAMETER(int32, ShouldDoDirectLighting)
		SHADER_PARAMETER(int32, ShadowsType)
		SHADER_PARAMETER(int32, ShadowsTranslucencyType)
		SHADER_PARAMETER(int32, ShouldDoEmissiveAndIndirectLighting)
		SHADER_PARAMETER(int32, UpscaleFactor)
		SHADER_PARAMETER(uint32, PrimaryRayFlags)
		SHADER_PARAMETER(float, TranslucencyMinRayDistance)
		SHADER_PARAMETER(float, TranslucencyMaxRayDistance)
		SHADER_PARAMETER(float, TranslucencyMaxRoughness)
		SHADER_PARAMETER(int32, TranslucencyRefraction)
		SHADER_PARAMETER(float, MaxNormalBias)

		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FRayTracingLightGrid, LightGridPacked)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FReflectionUniformParameters, ReflectionStruct)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogUniformParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)

		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FLumenTranslucencyLightingUniforms, LumenGIVolumeStruct)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)

		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, ColorOutput)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RayHitDistanceOutput)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingPrimaryRaysRGS, "/Engine/Private/RayTracing/RayTracingPrimaryRays.usf", "RayTracingPrimaryRaysRGS", SF_RayGen);

void FDeferredShadingSceneRenderer::PrepareRayTracingTranslucency(const FViewInfo& View, TArray<FRHIRayTracingShader*>& OutRayGenShaders)
{
	// Translucency and primary ray tracing requires the full ray tracing pipeline with material bindings.
	if (!ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::FullPipeline, View))
	{
		return;
	}

	// Declare all RayGen shaders that require material closest hit shaders to be bound.
	// NOTE: Translucency shader may be used for primary ray debug view mode.
	if (GetRayTracingTranslucencyOptions(View).bEnabled || View.Family->EngineShowFlags.RayTracingDebug)
	{
		FRayTracingPrimaryRaysRGS::FPermutationDomain PermutationVector;

		PermutationVector.Set<FRayTracingPrimaryRaysRGS::FEnableTwoSidedGeometryForShadowDim>(EnableRayTracingShadowTwoSidedGeometry());

		auto RayGenShader = View.ShaderMap->GetShader<FRayTracingPrimaryRaysRGS>(PermutationVector);
		OutRayGenShaders.Add(RayGenShader.GetRayTracingShader());
	}
}

void RenderRayTracingPrimaryRaysView(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FSceneTextures& SceneTextures,
	FRDGTextureRef* InOutColorTexture,
	FRDGTextureRef* InOutRayHitDistanceTexture,
	ERayTracingSceneLayer LayerToTrace,
	int32 SamplePerPixel,
	int32 HeightFog,
	float ResolutionFraction,
	ERayTracingPrimaryRaysFlag Flags)
{
	const FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures);

	int32 UpscaleFactor = int32(1.0f / ResolutionFraction);
	ensure(ResolutionFraction == 1.0 / UpscaleFactor);
	ensureMsgf(FComputeShaderUtils::kGolden2DGroupSize % UpscaleFactor == 0, TEXT("PrimaryRays ray tracing will have uv misalignement."));
	FIntPoint RayTracingResolution = FIntPoint::DivideAndRoundUp(View.ViewRect.Size(), UpscaleFactor);

	{
		FRDGTextureDesc Desc = SceneTextures.Color.Target->Desc;
		Desc.Format = PF_FloatRGBA;
		Desc.Flags &= ~(TexCreate_FastVRAM);
		Desc.Flags |= TexCreate_UAV;
		Desc.Extent /= UpscaleFactor;

		if(*InOutColorTexture == nullptr) 
		{
			*InOutColorTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingPrimaryRays"));
		}

		Desc.Format = PF_R16F;
		if(*InOutRayHitDistanceTexture == nullptr) 
		{
			*InOutRayHitDistanceTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingPrimaryRaysHitDistance"));
		}
	}

	const FRayTracingPrimaryRaysOptions TranslucencyOptions = GetRayTracingTranslucencyOptions(View);

	FRayTracingPrimaryRaysRGS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRayTracingPrimaryRaysRGS::FParameters>();
	PassParameters->SamplesPerPixel = SamplePerPixel;
	PassParameters->MaxRefractionRays = TranslucencyOptions.MaxRefractionRays > -1 ? TranslucencyOptions.MaxRefractionRays : View.FinalPostProcessSettings.RayTracingTranslucencyRefractionRays;
	PassParameters->HeightFog = HeightFog;
	PassParameters->ShouldDoDirectLighting = TranslucencyOptions.EnableDirectLighting;
	PassParameters->ForceOpaqueRays = TranslucencyOptions.bForceOpaque;
	PassParameters->ShadowsType = TranslucencyOptions.EnableShadows > -1 ? TranslucencyOptions.EnableShadows : (int32)View.FinalPostProcessSettings.RayTracingTranslucencyShadows;
	PassParameters->ShadowsTranslucencyType = TranslucencyOptions.ShadowsTranslucencyType;
	PassParameters->ShouldDoEmissiveAndIndirectLighting = TranslucencyOptions.EnableEmmissiveAndIndirectLighting;
	PassParameters->UpscaleFactor = UpscaleFactor;
	PassParameters->TranslucencyMinRayDistance = FMath::Min(TranslucencyOptions.MinRayDistance, TranslucencyOptions.MaxRayDistance);
	PassParameters->TranslucencyMaxRayDistance = TranslucencyOptions.MaxRayDistance;
	PassParameters->TranslucencyMaxRoughness = FMath::Clamp(TranslucencyOptions.MaxRoughness >= 0 ? TranslucencyOptions.MaxRoughness : View.FinalPostProcessSettings.RayTracingTranslucencyMaxRoughness, 0.01f, 1.0f);
	PassParameters->TranslucencyRefraction = TranslucencyOptions.EnableRefraction >= 0 ? TranslucencyOptions.EnableRefraction : View.FinalPostProcessSettings.RayTracingTranslucencyRefraction;
	PassParameters->MaxNormalBias = GetRaytracingMaxNormalBias();
	PassParameters->PrimaryRayFlags = (uint32)Flags;
	PassParameters->TLAS = View.GetRayTracingSceneLayerViewChecked(LayerToTrace);
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;

	PassParameters->LightGridPacked = View.RayTracingLightGridUniformBuffer;

	auto* LumenUniforms = GraphBuilder.AllocParameters<FLumenTranslucencyLightingUniforms>();
	LumenUniforms->Parameters = GetLumenTranslucencyLightingParameters(GraphBuilder, View.GetLumenTranslucencyGIVolume());
	PassParameters->LumenGIVolumeStruct = GraphBuilder.CreateUniformBuffer(LumenUniforms);

	PassParameters->SceneTextures = SceneTextureParameters;
	PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder, View);
	PassParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();

	PassParameters->SceneColorTexture = GetIfProduced(SceneTextures.Color.Resolve, FRDGSystemTextures::Get(GraphBuilder).Black);

	PassParameters->ReflectionStruct = CreateReflectionUniformBuffer(GraphBuilder, View);
	PassParameters->FogUniformParameters = CreateFogUniformBuffer(GraphBuilder, View);

	PassParameters->ColorOutput = GraphBuilder.CreateUAV(*InOutColorTexture);
	PassParameters->RayHitDistanceOutput = GraphBuilder.CreateUAV(*InOutRayHitDistanceTexture);

	FRayTracingPrimaryRaysRGS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRayTracingPrimaryRaysRGS::FEnableTwoSidedGeometryForShadowDim>(EnableRayTracingShadowTwoSidedGeometry());

	auto RayGenShader = View.ShaderMap->GetShader<FRayTracingPrimaryRaysRGS>(PermutationVector);

	ClearUnusedGraphResources(RayGenShader, PassParameters);

	RDG_EVENT_SCOPE_STAT(GraphBuilder, RayTracingPrimaryRays, "RayTracingPrimaryRays");

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("RayTracingPrimaryRays %dx%d", RayTracingResolution.X, RayTracingResolution.Y),
		PassParameters,
		ERDGPassFlags::Compute,
		[PassParameters, &View, RayGenShader, RayTracingResolution](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FRHIShaderBindingTable* SBT = View.MaterialRayTracingData.ShaderBindingTable;
		FRayTracingPipelineState* Pipeline = View.MaterialRayTracingData.PipelineState;

		FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
		SetShaderParameters(GlobalResources, RayGenShader, *PassParameters);

		FRHIUniformBuffer* SceneUniformBuffer = PassParameters->Scene->GetRHI();
		FRHIUniformBuffer* NaniteRayTracingUniformBuffer = PassParameters->NaniteRayTracing->GetRHI();
		TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

		RHICmdList.RayTraceDispatch(Pipeline, RayGenShader.GetRayTracingShader(), SBT, GlobalResources, RayTracingResolution.X, RayTracingResolution.Y);
	});
}

#endif // RHI_RAYTRACING
