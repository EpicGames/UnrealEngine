// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingScene.h"

#if RHI_RAYTRACING

#include "RayTracingInstanceBufferUtil.h"
#include "RenderCore.h"
#include "RayTracingDefinitions.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RaytracingOptions.h"
#include "PrimitiveSceneProxy.h"
#include "SceneUniformBuffer.h"
#include "SceneRendering.h"
#include "RayTracingInstanceCulling.h"
#include "Rendering/RayTracingGeometryManager.h"
#include "Nanite/NaniteRayTracing.h"

static TAutoConsoleVariable<int32> CVarRayTracingSceneBuildMode(
	TEXT("r.RayTracing.Scene.BuildMode"),
	1,
	TEXT("Controls the mode in which ray tracing scene is built:\n")
	TEXT(" 0: Fast build\n")
	TEXT(" 1: Fast trace (default)\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static TAutoConsoleVariable<bool> CVarRayTracingSceneUseTracingFeedback(
	TEXT("r.RayTracing.Scene.UseTracingFeedback"),
	false,
	TEXT("When set to true, will only schedule updates of dynamic geometry instances that were hit in the previous frame."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarRayTracingSceneBatchedBuild(
	TEXT("r.RayTracing.Scene.BatchedBuild"),
	true,
	TEXT("Whether to batch TLAS builds. Should be kept enabled since batched builds reduce barriers on GPU."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<bool> CVarRayTracingSceneCompactInstances(
	TEXT("r.RayTracing.Scene.CompactInstances"),
	false,
	TEXT("Whether to compact the instance buffer so it only contains active instances.\n")
	TEXT("On platforms that don't support indirect TLAS build this requires doing a GPU->CPU readback, ")
	TEXT("which lead so instances missing from TLAS due to the extra latency.\n")
	TEXT("r.RayTracing.Scene.CompactInstances.Min and r.RayTracing.Scene.CompactInstances.Margin can be used to avoid those issues."),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static TAutoConsoleVariable<int32> CVarRayTracingSceneCompactInstancesMin(
	TEXT("r.RayTracing.Scene.CompactInstances.Min"),
	0,
	TEXT("Minimum of instances in the instance buffer when using compaction.\n")
	TEXT("Should be set to the expected high water mark to avoid issues on platforms that don't support indirect TLAS build."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingSceneCompactInstancesMargin(
	TEXT("r.RayTracing.Scene.CompactInstances.Margin"),
	5000,
	TEXT("Margin applied on top of lastest number of active instances readback from GPU to avoid issues when number instances increases from frame to frame."),
	ECVF_RenderThreadSafe
);

#if !UE_BUILD_SHIPPING

static bool GRayTracingSerializeSceneNextFrame = false;

static FAutoConsoleCommand RayTracingSerializeSceneCmd(
	TEXT("r.RayTracing.Scene.SerializeOnce"),
	TEXT("Serialize Ray Tracing Scene to disk."),
	FConsoleCommandDelegate::CreateStatic([] { GRayTracingSerializeSceneNextFrame = true; }));

#endif

bool IsRayTracingFeedbackEnabled(const FSceneViewFamily& ViewFamily)
{
	// TODO: For now Feedback is limited to inline passes
	return !HasRayTracedOverlay(ViewFamily) && CVarRayTracingSceneUseTracingFeedback.GetValueOnRenderThread() && GRHISupportsInlineRayTracing;
}

bool IsRayTracingSceneCompactInstancesEnabled()
{
	return CVarRayTracingSceneCompactInstances.GetValueOnRenderThread();
}

BEGIN_SHADER_PARAMETER_STRUCT(FBuildInstanceBufferPassParams, )
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, InstanceBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, HitGroupContributionsBuffer)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutputStats)
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, InstanceExtraDataBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, NaniteBLASData)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUSceneParameters)
END_SHADER_PARAMETER_STRUCT()

const FRayTracingScene::FInstanceHandle FRayTracingScene::INVALID_INSTANCE_HANDLE = FInstanceHandle();
const FRayTracingScene::FViewHandle FRayTracingScene::INVALID_VIEW_HANDLE = FViewHandle();

using FInstanceBufferStats = uint32;

FRayTracingScene::FRayTracingScene()
{
	const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);
	Layers.AddDefaulted(NumLayers);

	for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		FLayer& Layer = Layers[LayerIndex];
		Layer.Name = FName(FString::Printf(TEXT("RayTracingScene_Layer%u"), LayerIndex));
	}
}

FRayTracingScene::~FRayTracingScene()
{
	ReleaseFeedbackReadbackBuffers();
	ReleaseReadbackBuffers();
}

void FRayTracingScene::BuildInitializationData(bool bUseLightingChannels, bool bForceOpaque, bool bDisableTriangleCull)
{
	const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);

	for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		FLayer& Layer = Layers[LayerIndex];

		for (int32 ViewIndex : ActiveViews)
		{
			FLayerView& LayerView = Layer.Views[ViewIndex];

			FRayTracingInstanceBufferBuilderInitializer Initializer;
			Initializer.Instances = Layer.Instances;
			Initializer.VisibleInstances = LayerView.VisibleInstances;
			Initializer.PreViewTranslation = ViewParameters[ViewIndex].PreViewTranslation;
			Initializer.bUseLightingChannels = bUseLightingChannels;
			Initializer.bForceOpaque = bForceOpaque;
			Initializer.bDisableTriangleCull = bDisableTriangleCull;

			LayerView.InstanceBufferBuilder.Init(MoveTemp(Initializer));
		}
	}

	bInitializationDataBuilt = true;
}

FRayTracingScene::FViewHandle FRayTracingScene::AddView(uint32 ViewKey)
{
	if (ViewIndexMap.Contains(ViewKey))
	{
		const int32 ViewIndex = ViewIndexMap[ViewKey];
		check(ActiveViews.IsValidIndex(ViewIndex) && ActiveViews[ViewIndex] == ViewIndex && ViewParameters[ViewIndex].ViewKey == ViewKey);
		return FViewHandle(ViewIndex);
	}

	const int32 ViewIndex = ActiveViews.Add(0);
	ActiveViews[ViewIndex] = ViewIndex;

	if (ViewKey == 0)
	{
		// Transient View (eg: no ViewState) are removed at the end of the frame
		TransientViewIndices.Add(ViewIndex);
	}
	else
	{
		ViewIndexMap.Add(ViewKey, ViewIndex);
	}

	if (ViewParameters.Num() < ViewIndex + 1)
	{
		ViewParameters.SetNum(ViewIndex + 1);
	}

	ViewParameters[ViewIndex].ViewKey = ViewKey;

	const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);
	for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		FLayer& Layer = Layers[LayerIndex];

		if (Layer.Views.Num() < ViewIndex + 1)
		{
			Layer.Views.SetNum(ViewIndex + 1);
		}
	}

	return FViewHandle(ViewIndex);
}

void FRayTracingScene::RemoveView(uint32 ViewKey)
{
	if (!ViewIndexMap.Contains(ViewKey))
	{
		return;
	}

	const int32 ViewIndex = ViewIndexMap[ViewKey];
	check(ActiveViews.IsValidIndex(ViewIndex) && ActiveViews[ViewIndex] == ViewIndex);

	// clear ViewIndex in Layers Views
	const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);
	for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		FLayer& Layer = Layers[LayerIndex];
		Layer.Views[ViewIndex] = {};
	}

	ViewParameters[ViewIndex] = {};

	ActiveViews.RemoveAt(ViewIndex);
	ViewIndexMap.Remove(ViewKey);
}

void FRayTracingScene::SetViewParams(FViewHandle ViewHandle, const FViewMatrices& ViewMatrices, const FRayTracingCullingParameters& CullingParameters)
{
	ViewParameters[ViewHandle].CullingParameters = &CullingParameters;
	ViewParameters[ViewHandle].PreViewTranslation = ViewMatrices.GetPreViewTranslation();
}

void FRayTracingScene::Update(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer, const FGPUScene* GPUScene, ERDGPassFlags ComputePassFlags)
{
	// Round up buffer sizes to some multiple to avoid pathological growth reallocations.
	static constexpr uint32 AllocationGranularity = 8 * 1024;
	static constexpr uint64 BufferAllocationGranularity = 16 * 1024 * 1024;

	TRACE_CPUPROFILER_EVENT_SCOPE(FRayTracingScene::Update);
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RayTracingScene_Update);

	const ERayTracingAccelerationStructureFlags BuildFlags = CVarRayTracingSceneBuildMode.GetValueOnRenderThread()
		? ERayTracingAccelerationStructureFlags::FastTrace
		: ERayTracingAccelerationStructureFlags::FastBuild;

	checkf(bInitializationDataBuilt, TEXT("BuildInitializationData(...) must be called before Update(...)."));

	bUsedThisFrame = true;

	FRHICommandListBase& RHICmdList = GraphBuilder.RHICmdList;

	const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);
	const uint32 MaxNumViews = ActiveViews.GetMaxIndex();

	FRDGBufferUAVRef InstanceStatsBufferUAV = nullptr;
	{
		// one counter per layer in the stats buffer
		FRDGBufferDesc InstanceStatsBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FInstanceBufferStats), NumLayers * MaxNumViews);
		InstanceStatsBufferDesc.Usage |= BUF_SourceCopy;

		InstanceStatsBuffer = GraphBuilder.CreateBuffer(InstanceStatsBufferDesc, TEXT("FRayTracingScene::InstanceStatsBuffer"));
		InstanceStatsBufferUAV = GraphBuilder.CreateUAV(InstanceStatsBuffer);

		AddClearUAVPass(GraphBuilder, InstanceStatsBufferUAV, 0, ComputePassFlags);
	}

	const bool bCompactInstanceBuffer = IsRayTracingSceneCompactInstancesEnabled();

	for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		FLayer& Layer = Layers[LayerIndex];

		for (int32 ViewIndex : ActiveViews)
		{
			FLayerView& LayerView = Layer.Views[ViewIndex];

			uint32 NumNativeInstances = LayerView.InstanceBufferBuilder.GetMaxNumInstances();

			if (bCompactInstanceBuffer && !GRHIGlobals.RayTracing.SupportsBuildTopLevelAccelerationStructureIndirect)
			{
				NumNativeInstances = FMath::Max<uint32>(CVarRayTracingSceneCompactInstancesMin.GetValueOnRenderThread(), LayerView.NumActiveInstances + CVarRayTracingSceneCompactInstancesMargin.GetValueOnRenderThread());
				NumNativeInstances = FMath::Min<uint32>(NumNativeInstances, LayerView.InstanceBufferBuilder.GetMaxNumInstances());
			}

			LayerView.MaxNumInstances = NumNativeInstances;

			const uint32 NumNativeInstancesAligned = FMath::DivideAndRoundUp(FMath::Max(NumNativeInstances, 1U), AllocationGranularity) * AllocationGranularity;

			// Allocate RHI object if current one is too small or significantly larger than what we need.
			if(LayerView.RayTracingSceneRHI == nullptr
				|| NumNativeInstancesAligned > LayerView.RayTracingSceneRHI->GetInitializer().MaxNumInstances
				|| NumNativeInstancesAligned < LayerView.RayTracingSceneRHI->GetInitializer().MaxNumInstances / 2)
			{
				FRayTracingSceneInitializer Initializer;
				Initializer.DebugName = Layer.Name; // TODO: also include ViewIndex in the name
				Initializer.MaxNumInstances = NumNativeInstancesAligned;
				Initializer.BuildFlags = BuildFlags;
				LayerView.RayTracingSceneRHI = RHICreateRayTracingScene(MoveTemp(Initializer));
			}

			FRayTracingAccelerationStructureSize SizeInfo = LayerView.RayTracingSceneRHI->GetSizeInfo();
			SizeInfo.ResultSize = FMath::DivideAndRoundUp(FMath::Max(SizeInfo.ResultSize, 1ull), BufferAllocationGranularity) * BufferAllocationGranularity;

			// Allocate GPU buffer if current one is too small or significantly larger than what we need.
			if (!LayerView.RayTracingScenePooledBuffer.IsValid()
				|| SizeInfo.ResultSize > LayerView.RayTracingScenePooledBuffer->GetSize()
				|| SizeInfo.ResultSize < LayerView.RayTracingScenePooledBuffer->GetSize() / 2)
			{
				FRDGBufferDesc Desc = FRDGBufferDesc::CreateBufferDesc(1, uint32(SizeInfo.ResultSize));
				Desc.Usage = EBufferUsageFlags::AccelerationStructure;

				LayerView.RayTracingScenePooledBuffer = AllocatePooledBuffer(Desc, TEXT("FRayTracingScene::SceneBuffer"));
			}

			LayerView.RayTracingSceneBufferRDG = GraphBuilder.RegisterExternalBuffer(LayerView.RayTracingScenePooledBuffer);
			LayerView.RayTracingSceneBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(LayerView.RayTracingSceneBufferRDG, LayerView.RayTracingSceneRHI, 0));

			{
				const uint64 ScratchAlignment = GRHIRayTracingScratchBufferAlignment;
				FRDGBufferDesc ScratchBufferDesc;
				ScratchBufferDesc.Usage = EBufferUsageFlags::RayTracingScratch | EBufferUsageFlags::StructuredBuffer;
				ScratchBufferDesc.BytesPerElement = uint32(ScratchAlignment);
				ScratchBufferDesc.NumElements = uint32(FMath::DivideAndRoundUp(SizeInfo.BuildScratchSize, ScratchAlignment));

				LayerView.BuildScratchBuffer = GraphBuilder.CreateBuffer(ScratchBufferDesc, TEXT("FRayTracingScene::ScratchBuffer"));
			}

			{
				FRDGBufferDesc InstanceBufferDesc;
				InstanceBufferDesc.Usage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer;
				InstanceBufferDesc.BytesPerElement = GRHIRayTracingInstanceDescriptorSize;
				InstanceBufferDesc.NumElements = NumNativeInstancesAligned;

				LayerView.InstanceBuffer = GraphBuilder.CreateBuffer(InstanceBufferDesc, TEXT("FRayTracingScene::InstanceBuffer"));

				if (bCompactInstanceBuffer)
				{
					// need to clear since FRayTracingBuildInstanceBufferCS will only write active instances
					AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(LayerView.InstanceBuffer), 0, ComputePassFlags);
				}
			}
			
			if(GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer)
			{
				FRDGBufferDesc HitGroupContributionsDesc;
				HitGroupContributionsDesc.Usage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer;
				HitGroupContributionsDesc.BytesPerElement = 4;
				HitGroupContributionsDesc.NumElements = NumNativeInstancesAligned;
				
				LayerView.HitGroupContributionsBuffer = GraphBuilder.CreateBuffer(HitGroupContributionsDesc, TEXT("FRayTracingScene::HitGroupContributionsBuffer"));
			}

			// Feedback
			if (bTracingFeedbackEnabled)
			{
				{
					FRDGBufferDesc InstanceHitCountBufferDesc;
					InstanceHitCountBufferDesc.Usage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer;
					InstanceHitCountBufferDesc.BytesPerElement = sizeof(uint32);
					InstanceHitCountBufferDesc.NumElements = NumNativeInstancesAligned;

					LayerView.InstanceHitCountBuffer = GraphBuilder.CreateBuffer(InstanceHitCountBufferDesc, TEXT("FRayTracingScene::InstanceHitCount"));
					LayerView.InstanceHitCountBufferUAV = GraphBuilder.CreateUAV(LayerView.InstanceHitCountBuffer, ERDGUnorderedAccessViewFlags::SkipBarrier);
					AddClearUAVPass(GraphBuilder, LayerView.InstanceHitCountBufferUAV, 0, ComputePassFlags);
				}

				{
					FRDGBufferDesc AccelerationStructureIndexBufferDesc;
					AccelerationStructureIndexBufferDesc.Usage = EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer;
					AccelerationStructureIndexBufferDesc.BytesPerElement = sizeof(uint32);
					AccelerationStructureIndexBufferDesc.NumElements = NumNativeInstancesAligned;

					LayerView.AccelerationStructureIndexBuffer = GraphBuilder.CreateBuffer(AccelerationStructureIndexBufferDesc, TEXT("FRayTracingScene::AccelerationStructureIndexBuffer"));

					TConstArrayView<uint32> InstanceGeometryIndices = LayerView.InstanceBufferBuilder.GetInstanceGeometryIndices();
					GraphBuilder.QueueBufferUpload(LayerView.AccelerationStructureIndexBuffer, InstanceGeometryIndices.GetData(), InstanceGeometryIndices.GetTypeSize() * InstanceGeometryIndices.Num());
				}
			}

			FRDGBufferUAVRef InstanceExtraDataBufferUAV = nullptr;
			if (bInstanceExtraDataBufferEnabled || bTracingFeedbackEnabled || bInstanceDebugDataEnabled)
			{
				FRDGBufferDesc InstanceExtraDataBufferDesc;
				InstanceExtraDataBufferDesc.Usage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer;
				InstanceExtraDataBufferDesc.BytesPerElement = sizeof(FRayTracingInstanceExtraData);
				InstanceExtraDataBufferDesc.NumElements = FMath::Max(NumNativeInstances, 1u);

				LayerView.InstanceExtraDataBuffer = GraphBuilder.CreateBuffer(InstanceExtraDataBufferDesc, TEXT("FRayTracingScene::InstanceExtraDataBuffer"));
				InstanceExtraDataBufferUAV = GraphBuilder.CreateUAV(LayerView.InstanceExtraDataBuffer);

				AddClearUAVPass(GraphBuilder, InstanceExtraDataBufferUAV, 0xFFFFFFFF, ComputePassFlags);
			}

			if (NumNativeInstances > 0)
			{
				// Fill instance upload buffer on separate thread since results are only needed in RHI thread
				GraphBuilder.AddCommandListSetupTask([&InstanceBufferBuilder = LayerView.InstanceBufferBuilder](FRHICommandList& RHICmdList)
					{
						FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);

						InstanceBufferBuilder.FillRayTracingInstanceUploadBuffer(RHICmdList);
					});

				GraphBuilder.AddCommandListSetupTask([&InstanceBufferBuilder = LayerView.InstanceBufferBuilder](FRHICommandList& RHICmdList)
					{
						FTaskTagScope TaskTagScope(ETaskTag::EParallelRenderingThread);

						InstanceBufferBuilder.FillAccelerationStructureAddressesBuffer(RHICmdList);
					});

	#if STATS
				const bool bStatsEnabled = true;
	#else
				const bool bStatsEnabled = false;
	#endif

				{
					FBuildInstanceBufferPassParams* PassParams = GraphBuilder.AllocParameters<FBuildInstanceBufferPassParams>();
					PassParams->InstanceBuffer = GraphBuilder.CreateUAV(LayerView.InstanceBuffer);
					
					if(GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer)
					{
						PassParams->HitGroupContributionsBuffer = GraphBuilder.CreateUAV(LayerView.HitGroupContributionsBuffer);
					}
					
					PassParams->InstanceExtraDataBuffer = InstanceExtraDataBufferUAV;
					PassParams->Scene = SceneUniformBuffer.GetBuffer(GraphBuilder);
					PassParams->GPUSceneParameters = GPUScene->GetShaderParameters(GraphBuilder);
					PassParams->OutputStats = bCompactInstanceBuffer || bStatsEnabled ? InstanceStatsBufferUAV : nullptr;
					PassParams->NaniteBLASData = Nanite::GRayTracingManager.GetBLASDataSRV(GraphBuilder);

					const uint32 OutputStatsOffset = LayerIndex * MaxNumViews + ViewIndex;

					GraphBuilder.AddPass(
						RDG_EVENT_NAME("RayTracingBuildInstanceBuffer"),
						PassParams,
						ComputePassFlags,
						[PassParams,
						&InstanceBufferBuilder = LayerView.InstanceBufferBuilder,
						OutputStatsOffset,
						GPUScene,
						CullingParameters = ViewParameters[ViewIndex].CullingParameters,
						NumNativeInstances,
						bCompactInstanceBuffer
						](FRHICommandList& RHICmdList)
						{
							FGPUSceneResourceParametersRHI GPUSceneParameters;
							GPUSceneParameters.CommonParameters = PassParams->GPUSceneParameters.CommonParameters;
							GPUSceneParameters.GPUSceneInstanceSceneData = PassParams->GPUSceneParameters.GPUSceneInstanceSceneData->GetRHI();
							GPUSceneParameters.GPUSceneInstancePayloadData = PassParams->GPUSceneParameters.GPUSceneInstancePayloadData->GetRHI();
							GPUSceneParameters.GPUScenePrimitiveSceneData = PassParams->GPUSceneParameters.GPUScenePrimitiveSceneData->GetRHI();
							GPUSceneParameters.GPUSceneLightmapData = PassParams->GPUSceneParameters.GPUSceneLightmapData->GetRHI();
							GPUSceneParameters.GPUSceneLightData = PassParams->GPUSceneParameters.GPUSceneLightData->GetRHI();

							InstanceBufferBuilder.BuildRayTracingInstanceBuffer(
								RHICmdList,
								GPUScene,
								&GPUSceneParameters,
								CullingParameters,
								PassParams->InstanceBuffer->GetRHI(),
								GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer ? PassParams->HitGroupContributionsBuffer->GetRHI() : nullptr,
								NumNativeInstances,
								PassParams->NaniteBLASData ? PassParams->NaniteBLASData->GetRHI() : nullptr,
								bCompactInstanceBuffer,
								PassParams->OutputStats ? PassParams->OutputStats->GetRHI() : nullptr,
								OutputStatsOffset,
								PassParams->InstanceExtraDataBuffer ? PassParams->InstanceExtraDataBuffer->GetRHI() : nullptr);
						});
				}
			}
			else
			{
				// when RayTracingBuildInstanceBuffer pass is skipped need to clear buffers so RDG doesn't complain they have never been written to
				AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(LayerView.InstanceBuffer), 0, ComputePassFlags);
				
				if (GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer)
				{
					AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(LayerView.HitGroupContributionsBuffer), 0, ComputePassFlags);
				}
			}

			// Feedback
			if (bTracingFeedbackEnabled)
			{
				static_assert(sizeof(typename decltype(Layer.GeometryHandles)::ElementType) == sizeof(uint32));

				FRDGBufferDesc GeometryHandleBufferDesc;
				GeometryHandleBufferDesc.Usage = EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer;
				GeometryHandleBufferDesc.BytesPerElement = sizeof(uint32);
				GeometryHandleBufferDesc.NumElements = FMath::Max(Layer.GeometryHandles.Num(), 1);

				Layer.GeometryHandleBuffer = GraphBuilder.CreateBuffer(GeometryHandleBufferDesc, TEXT("FRayTracingScene::GeometryHandleBuffer"));
				GraphBuilder.QueueBufferUpload(Layer.GeometryHandleBuffer, Layer.GeometryHandles.GetData(), Layer.GeometryHandles.GetTypeSize() * Layer.GeometryHandles.Num());
			}

			if (Layer.InstancesDebugData.Num() > 0)
			{
				check(Layer.InstancesDebugData.Num() == Layer.Instances.Num());

				Layer.InstanceDebugBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("FRayTracingScene::InstanceDebugData"), Layer.InstancesDebugData);
			}
		}
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FRayTracingSceneBuildPassParams, )
	RDG_BUFFER_ACCESS_ARRAY(TLASBuildBuffers)
	RDG_BUFFER_ACCESS_ARRAY(NaniteDependencies)
	RDG_BUFFER_ACCESS(DynamicGeometryScratchBuffer, ERHIAccess::UAVCompute)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FRayTracingSceneSerializePassParams, )
	RDG_BUFFER_ACCESS(TLASBuffer, ERHIAccess::BVHRead)
END_SHADER_PARAMETER_STRUCT()

void FRayTracingScene::Build(FRDGBuilder& GraphBuilder, ERDGPassFlags ComputePassFlags, FRDGBufferRef DynamicGeometryScratchBuffer)
{
	const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);

	FRayTracingSceneBuildPassParams* PassParams = GraphBuilder.AllocParameters<FRayTracingSceneBuildPassParams>();
	PassParams->DynamicGeometryScratchBuffer = DynamicGeometryScratchBuffer; // TODO: Is this necessary?

	for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		FLayer& Layer = Layers[LayerIndex];

		for (int32 ViewIndex : ActiveViews)
		{
			FLayerView& LayerView = Layer.Views[ViewIndex];

			PassParams->TLASBuildBuffers.Emplace(LayerView.BuildScratchBuffer, ERHIAccess::UAVCompute);
			PassParams->TLASBuildBuffers.Emplace(LayerView.InstanceBuffer, ERHIAccess::SRVCompute);
			
			if(GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer)
			{
				PassParams->TLASBuildBuffers.Emplace(LayerView.HitGroupContributionsBuffer, ERHIAccess::SRVCompute);
			}
			
			PassParams->TLASBuildBuffers.Emplace(LayerView.RayTracingSceneBufferRDG, ERHIAccess::BVHWrite);
		}
	}

	// Add Nanite acceleration structure buffers as dependencies so RDG inserts proper
	// BVHWrite->BVHRead barriers between BLAS/CLAS builds/moves and the TLAS build.
	Nanite::GRayTracingManager.GetTLASBuildDependencies(GraphBuilder, PassParams->NaniteDependencies);

	GraphBuilder.AddPass(RDG_EVENT_NAME("RayTracingBuildScene"), PassParams, ComputePassFlags,
		[PassParams, this](FRHICommandList& RHICmdList)
		{
			const bool bUseBatchedBuild = CVarRayTracingSceneBatchedBuild.GetValueOnRenderThread();
			const bool bCompactInstanceBuffer = IsRayTracingSceneCompactInstancesEnabled();
			const uint32 MaxNumViews = ActiveViews.GetMaxIndex();

			TArray<FRHIBuffer*> ReferencedBuffers;
			ReferencedBuffers.Reserve(PassParams->NaniteDependencies.Num());

			for (const FRDGBufferAccess& BufferAccess : PassParams->NaniteDependencies)
			{
				ReferencedBuffers.Add(BufferAccess->GetRHI());
			}

			TArray<FRayTracingSceneBuildParams> BatchedBuildParams;
			BatchedBuildParams.Reserve(NumLayers); // TODO: should also take num views into account

			for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
			{
				FLayer& Layer = Layers[LayerIndex];

				for (int32 ViewIndex : ActiveViews)
				{
					FLayerView& LayerView = Layer.Views[ViewIndex];

					FRayTracingSceneBuildParams BuildParams;
					BuildParams.Scene = LayerView.RayTracingSceneRHI;
					BuildParams.ScratchBuffer = LayerView.BuildScratchBuffer->GetRHI();
					BuildParams.ScratchBufferOffset = 0;
					BuildParams.InstanceBuffer = LayerView.InstanceBuffer->GetRHI();
					BuildParams.InstanceBufferOffset = 0;
					
					if(GRHIGlobals.RayTracing.RequiresSeparateHitGroupContributionsBuffer)
					{
						check(LayerView.HitGroupContributionsBuffer);
						BuildParams.HitGroupContributionsBuffer = LayerView.HitGroupContributionsBuffer->GetRHI();
						BuildParams.HitGroupContributionsBufferOffset = 0;
					}

					if (bCompactInstanceBuffer && GRHIGlobals.RayTracing.SupportsBuildTopLevelAccelerationStructureIndirect)
					{
						BuildParams.InstanceCountBuffer = InstanceStatsBuffer ? InstanceStatsBuffer->GetRHI() : nullptr;
						BuildParams.InstanceCountBufferOffset = (LayerIndex * MaxNumViews + ViewIndex) * sizeof(FInstanceBufferStats);
					}
					
					BuildParams.NumInstances = LayerView.MaxNumInstances;
					BuildParams.ReferencedGeometries = LayerView.InstanceBufferBuilder.GetReferencedGeometries();
					BuildParams.ReferencedBuffers = ReferencedBuffers;

					RHICmdList.BindAccelerationStructureMemory(LayerView.RayTracingSceneRHI, LayerView.RayTracingSceneBufferRDG->GetRHI(), 0);

					if (bUseBatchedBuild)
					{
						BatchedBuildParams.Add(BuildParams);
					}
					else
					{
						RHICmdList.BuildAccelerationStructure(BuildParams);
					}
				}
			}

			if (bUseBatchedBuild)
			{
				RHICmdList.BuildAccelerationStructures(BatchedBuildParams);
			}
		});

#if !UE_BUILD_SHIPPING
	if (GRayTracingSerializeSceneNextFrame && GRHIGlobals.RayTracing.SupportsSerializeAccelerationStructure)
	{
		for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
		{
			FLayer& Layer = Layers[LayerIndex];

			for (int32 ViewIndex : ActiveViews)
			{
				FLayerView& LayerView = Layer.Views[ViewIndex];

				FRayTracingSceneSerializePassParams* SerializePassParams = GraphBuilder.AllocParameters<FRayTracingSceneSerializePassParams>();
				SerializePassParams->TLASBuffer = LayerView.RayTracingSceneBufferRDG;

				GraphBuilder.AddPass(RDG_EVENT_NAME("RayTracingSerializeScene"), SerializePassParams, ERDGPassFlags::Readback,
					[SerializePassParams, &Layer, &LayerView, ViewIndex](FRHICommandListImmediate& RHICmdList)
					{
						FString Filename = FString::Printf(TEXT("%s_%d_(%s)"), *Layer.Name.ToString(), ViewIndex, *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")));
						FString RootPath = FPaths::ScreenShotDir() + TEXT("BVH/"); // Save BVH dumps to ScreenShot directory
						FString OutputFilename = RootPath + Filename + TEXT(".bvh");

						RHICmdList.SerializeAccelerationStructure(LayerView.RayTracingSceneRHI, *OutputFilename);
					});
			}
		}
	}

	GRayTracingSerializeSceneNextFrame = false;
#endif // !UE_BUILD_SHIPPING
}

struct FRayTracingProcessFeedbackCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingProcessFeedbackCS);
	SHADER_USE_PARAMETER_STRUCT(FRayTracingProcessFeedbackCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )		
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, GeometryHitCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, RWGeometryHandleBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWGeometryHandleAllocator)
		SHADER_PARAMETER(uint32, NumGeometries)
	END_SHADER_PARAMETER_STRUCT()

	static constexpr uint32 ThreadGroupSize = 64;

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);

		// Force DXC to avoid shader reflection issues.
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingProcessFeedbackCS, "/Engine/Private/RayTracing/RayTracingFeedback.usf", "RayTracingProcessFeedbackCS", SF_Compute);

struct FRayTracingUpdateGeometryHitCountCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingUpdateGeometryHitCountCS);
	SHADER_USE_PARAMETER_STRUCT(FRayTracingUpdateGeometryHitCountCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, GeometryHandleBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InstanceHitCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, RWGeometryHitCountBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, RWGeometryHandleBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, AccelerationStructureIndexBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, InstanceExtraDataBuffer)

		SHADER_PARAMETER(uint32, NumInstances)
	END_SHADER_PARAMETER_STRUCT()

	static constexpr uint32 ThreadGroupSize = 64;

	static inline void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);

		// Force DXC to avoid shader reflection issues.
		OutEnvironment.CompilerFlags.Add(CFLAG_ForceDXC);
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingUpdateGeometryHitCountCS, "/Engine/Private/RayTracing/RayTracingFeedback.usf", "RayTracingUpdateGeometryHitCountCS", SF_Compute);

BEGIN_SHADER_PARAMETER_STRUCT(FFeedbackReadbackPassParameters, )
	RDG_BUFFER_ACCESS(HandleBuffer, ERHIAccess::CopySrc)
	RDG_BUFFER_ACCESS(CountBuffer, ERHIAccess::CopySrc)	
END_SHADER_PARAMETER_STRUCT()

void FRayTracingScene::FinishTracingFeedback(FRDGBuilder& GraphBuilder, ERDGPassFlags ComputePassFlags)
{
	if (!bTracingFeedbackEnabled)
	{
		return;
	}

	RDG_EVENT_SCOPE(GraphBuilder, "RayTracingScene::FinishTracingFeedback");
	
	const FLayer& Layer = Layers[0];
	const FLayerView& LayerView = Layer.Views[0];
	const uint32 NumGeometries = (uint32)LayerView.InstanceBufferBuilder.GetReferencedGeometries().Num();
	const uint32 NumInstances = LayerView.InstanceBufferBuilder.GetMaxNumInstances();

	if (NumGeometries == 0)
	{
		return;
	}

	FRDGBufferRef GeometryHandleBuffer;

	FRDGBufferDesc GeometryHandleBufferDesc;
	GeometryHandleBufferDesc.Usage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer | EBufferUsageFlags::SourceCopy;
	GeometryHandleBufferDesc.BytesPerElement = sizeof(int32);
	GeometryHandleBufferDesc.NumElements = NumGeometries;

	GeometryHandleBuffer = GraphBuilder.CreateBuffer(GeometryHandleBufferDesc, TEXT("FRayTracingScene::GeometryHandleBuffer"));

	// Update geometry hit count
	FRDGBufferRef GeometryHitCountBuffer;
	{
		FRDGBufferDesc GeometryHitCountBufferDesc;
		GeometryHitCountBufferDesc.Usage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::StructuredBuffer;
		GeometryHitCountBufferDesc.BytesPerElement = sizeof(uint32);
		GeometryHitCountBufferDesc.NumElements = NumGeometries;

		GeometryHitCountBuffer = GraphBuilder.CreateBuffer(GeometryHitCountBufferDesc, TEXT("FRayTracingScene::GeometryHitCountBuffer"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(GeometryHitCountBuffer), 0, ComputePassFlags);

		FRayTracingUpdateGeometryHitCountCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRayTracingUpdateGeometryHitCountCS::FParameters>();

		PassParameters->GeometryHandleBuffer = GraphBuilder.CreateSRV(Layer.GeometryHandleBuffer);
		PassParameters->AccelerationStructureIndexBuffer = GraphBuilder.CreateSRV(LayerView.AccelerationStructureIndexBuffer);
		PassParameters->InstanceHitCountBuffer = GraphBuilder.CreateSRV(LayerView.InstanceHitCountBuffer);
		PassParameters->RWGeometryHitCountBuffer = GraphBuilder.CreateUAV(GeometryHitCountBuffer);
		PassParameters->RWGeometryHandleBuffer = GraphBuilder.CreateUAV(GeometryHandleBuffer);
		PassParameters->InstanceExtraDataBuffer = GraphBuilder.CreateSRV(LayerView.InstanceExtraDataBuffer);
		PassParameters->NumInstances = NumInstances;

		const FIntVector GroupSize = FComputeShaderUtils::GetGroupCountWrapped(NumInstances, FRayTracingUpdateGeometryHitCountCS::ThreadGroupSize);

		TShaderRef<FRayTracingUpdateGeometryHitCountCS> ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FRayTracingUpdateGeometryHitCountCS>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FRayTracingScene::RayTracingUpdateGeometryHitCount"),
			ComputePassFlags,
			ComputeShader,
			PassParameters,
			GroupSize);
	}	

	// Fill geometry handle buffer	
	FRDGBufferRef GeometryHandleAllocatorBuffer;
	{
		FRDGBufferDesc GeometryHandleAllocatorBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(uint32), 1);
		GeometryHandleAllocatorBufferDesc.Usage = EBufferUsageFlags(GeometryHandleAllocatorBufferDesc.Usage | BUF_SourceCopy);
		GeometryHandleAllocatorBuffer = GraphBuilder.CreateBuffer(GeometryHandleAllocatorBufferDesc, TEXT("FRayTracingScene::GeometryHandleAllocator"));
		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(GeometryHandleAllocatorBuffer, PF_R32_UINT), 0, ComputePassFlags);

		FRayTracingProcessFeedbackCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FRayTracingProcessFeedbackCS::FParameters>();
		PassParameters->GeometryHitCountBuffer = GraphBuilder.CreateSRV(GeometryHitCountBuffer);
		PassParameters->RWGeometryHandleBuffer = GraphBuilder.CreateUAV(GeometryHandleBuffer);
		PassParameters->RWGeometryHandleAllocator = GraphBuilder.CreateUAV(GeometryHandleAllocatorBuffer, PF_R32_UINT);
		PassParameters->NumGeometries = NumGeometries;

		const FIntVector GroupSize = FComputeShaderUtils::GetGroupCountWrapped(NumGeometries, FRayTracingProcessFeedbackCS::ThreadGroupSize);

		TShaderRef<FRayTracingProcessFeedbackCS> ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FRayTracingProcessFeedbackCS>();
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("FRayTracingScene::FinishTracingFeedback"),			
			ComputePassFlags,
			ComputeShader,
			PassParameters,
			GroupSize);
	}	

	// Readback
	//  if necessary create readback buffers
	if (FeedbackReadback.IsEmpty())
	{
		FeedbackReadback.SetNum(MaxReadbackBuffers);

		for (uint32 Index = 0; Index < MaxReadbackBuffers; ++Index)
		{
			FeedbackReadback[Index].GeometryHandleReadbackBuffer = new FRHIGPUBufferReadback(TEXT("FRayTracingScene::FeedbackReadbackBuffer::GeometryHandles"));
			FeedbackReadback[Index].GeometryCountReadbackBuffer = new FRHIGPUBufferReadback(TEXT("FRayTracingScene::FeedbackReadbackBuffer::GeometryCount"));			
		}
	}

	// process ready results

	while (FeedbackReadbackNumPending > 0)
	{
		uint32 ReadbackIndex = (FeedbackReadbackWriteIndex + MaxReadbackBuffers - FeedbackReadbackNumPending) % MaxReadbackBuffers;
		FRHIGPUBufferReadback* GeometryHandleReadbackBuffer = FeedbackReadback[ReadbackIndex].GeometryHandleReadbackBuffer;
		FRHIGPUBufferReadback* GeometryCountReadbackBuffer = FeedbackReadback[ReadbackIndex].GeometryCountReadbackBuffer;
		check(GeometryHandleReadbackBuffer->IsReady() == GeometryCountReadbackBuffer->IsReady());
		if (GeometryHandleReadbackBuffer->IsReady() && GeometryCountReadbackBuffer->IsReady())
		{
			FeedbackReadbackNumPending--;

			const uint32* GeometryCountPtr = (const uint32*)GeometryCountReadbackBuffer->Lock(sizeof(uint32));
			const uint32 GeometryCount = GeometryCountPtr[0];
			GeometryCountReadbackBuffer->Unlock();

			const uint32* GeometryHandlesPtr = (const uint32*)GeometryHandleReadbackBuffer->Lock(sizeof(uint32) * GeometryCount);

			for (uint32 Index = 0; Index < GeometryCount; Index++)
			{
				const RayTracing::FGeometryHandle GeometryHandle = RayTracing::FGeometryHandle(GeometryHandlesPtr[Index]);
				if (ensure(!GeometryHandle.IsNull()))
				{
					GRayTracingGeometryManager->AddVisibleGeometry(GeometryHandle);
				}
			}

			GeometryHandleReadbackBuffer->Unlock();
		}
		else
		{
			break;
		}
	}

	//if (FeedbackReadbackNumPending < MaxReadbackBuffers) // TODO: need to prevent overwritng results?
	{
		// copy feedback to readback buffer

		FFeedbackReadbackPassParameters* PassParameters = GraphBuilder.AllocParameters<FFeedbackReadbackPassParameters>();
		PassParameters->HandleBuffer = GeometryHandleBuffer;
		PassParameters->CountBuffer = GeometryHandleAllocatorBuffer;

		GraphBuilder.AddPass(RDG_EVENT_NAME("FRayTracingScene::FeedbackReadback"), PassParameters, ERDGPassFlags::Readback,
			[HandleReadbackBuffer = FeedbackReadback[FeedbackReadbackWriteIndex].GeometryHandleReadbackBuffer,
			CountReadbackBuffer = FeedbackReadback[FeedbackReadbackWriteIndex].GeometryCountReadbackBuffer,
			PassParameters]
			(FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				HandleReadbackBuffer->EnqueueCopy(RHICmdList, PassParameters->HandleBuffer->GetRHI(), 0u);
				CountReadbackBuffer->EnqueueCopy(RHICmdList, PassParameters->CountBuffer->GetRHI(), 0u);
			});

		FeedbackReadbackWriteIndex = (FeedbackReadbackWriteIndex + 1u) % MaxReadbackBuffers;
		FeedbackReadbackNumPending = FMath::Min(FeedbackReadbackNumPending + 1u, MaxReadbackBuffers);
	}
}

void FRayTracingScene::FinishStats(FRDGBuilder& GraphBuilder, ERDGPassFlags ComputePassFlags)
{
	//  if necessary create readback buffers
	if (StatsReadback.IsEmpty())
	{
		StatsReadback.SetNum(MaxReadbackBuffers);

		for (uint32 Index = 0; Index < MaxReadbackBuffers; ++Index)
		{
			StatsReadback[Index].ReadbackBuffer = new FRHIGPUBufferReadback(TEXT("FRayTracingScene::StatsReadbackBuffer"));
		}
	}

	uint32 TotalNumNativeInstances = 0;
	uint32 TotalNumActiveInstances = 0;

	const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);

	// process ready results
	while (StatsReadbackNumPending > 0)
	{
		uint32 Index = (StatsReadbackWriteIndex + MaxReadbackBuffers - StatsReadbackNumPending) % MaxReadbackBuffers;
		FStatsReadbackData& ReadbackData = StatsReadback[Index];
		if (ReadbackData.ReadbackBuffer->IsReady())
		{
			StatsReadbackNumPending--;

			auto ReadbackBufferPtr = (const FInstanceBufferStats*)ReadbackData.ReadbackBuffer->Lock(sizeof(FInstanceBufferStats) * NumLayers * ReadbackData.MaxNumViews);

			for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
			{
				FLayer& Layer = Layers[LayerIndex];

				for (uint32 ViewKey : ReadbackData.ViewKeys)
				{
					if (!ViewIndexMap.Contains(ViewKey))
					{
						continue;
					}

					const int32 ViewIndex = ViewIndexMap[ViewKey];

					FLayerView& LayerView = Layer.Views[ViewIndex];

					const uint32 LayerViewNumNativeInstances = LayerView.InstanceBufferBuilder.GetMaxNumInstances();

					LayerView.NumActiveInstances = FMath::Min(ReadbackBufferPtr[LayerIndex * ReadbackData.MaxNumViews + ViewIndex], LayerViewNumNativeInstances);

					TotalNumNativeInstances += LayerViewNumNativeInstances;
					TotalNumActiveInstances += LayerView.NumActiveInstances;
				}
			}

			ReadbackData.ReadbackBuffer->Unlock();
		}
		else
		{
			break;
		}
	}

	SET_DWORD_STAT(STAT_RayTracingTotalInstances, TotalNumNativeInstances);
	SET_DWORD_STAT(STAT_RayTracingActiveInstances, TotalNumActiveInstances);

	// copy stats to readback buffer
	if (InstanceStatsBuffer != nullptr && StatsReadbackNumPending < MaxReadbackBuffers)
	{
		AddReadbackBufferPass(GraphBuilder, RDG_EVENT_NAME("FRayTracingScene::StatsReadback"), InstanceStatsBuffer,
			[&ReadbackData = StatsReadback[StatsReadbackWriteIndex], InstanceStatsBuffer = InstanceStatsBuffer](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				ReadbackData.ReadbackBuffer->EnqueueCopy(RHICmdList, InstanceStatsBuffer->GetRHI(), 0u);
			});

		StatsReadback[StatsReadbackWriteIndex].MaxNumViews = ActiveViews.GetMaxIndex();

		StatsReadback[StatsReadbackWriteIndex].ViewKeys.Reset(ActiveViews.GetMaxIndex());

		for (int32 ViewIndex : ActiveViews)
		{
			StatsReadback[StatsReadbackWriteIndex].ViewKeys.Add(ViewParameters[ViewIndex].ViewKey);
		}

		StatsReadbackWriteIndex = (StatsReadbackWriteIndex + 1u) % MaxReadbackBuffers;
		StatsReadbackNumPending = FMath::Min(StatsReadbackNumPending + 1u, MaxReadbackBuffers);
	}
}

void FRayTracingScene::PostRender(FRDGBuilder& GraphBuilder, ERDGPassFlags ComputePassFlags)
{
	FinishTracingFeedback(GraphBuilder, ComputePassFlags);
	FinishStats(GraphBuilder, ComputePassFlags);
}

bool FRayTracingScene::IsCreated() const
{
	return bUsedThisFrame;
}

FRHIRayTracingScene* FRayTracingScene::GetRHIRayTracingScene(ERayTracingSceneLayer InLayer, FViewHandle InViewHandle) const
{
	return Layers[uint8(InLayer)].Views[InViewHandle].RayTracingSceneRHI.GetReference();
}

FRHIRayTracingScene* FRayTracingScene::GetRHIRayTracingSceneChecked(ERayTracingSceneLayer InLayer, FViewHandle InViewHandle) const
{
	FRHIRayTracingScene* Result = GetRHIRayTracingScene(InLayer, InViewHandle);
	checkf(Result, TEXT("Ray tracing scene was not created. Perhaps Update() was not called."));
	return Result;
}

FShaderResourceViewRHIRef FRayTracingScene::CreateLayerViewRHI(FRHICommandListBase& RHICmdList, ERayTracingSceneLayer InLayer, FViewHandle InViewHandle) const
{
	const FLayerView& LayerView = Layers[uint8(InLayer)].Views[InViewHandle];
	checkf(LayerView.RayTracingScenePooledBuffer, TEXT("Ray tracing scene was not created. Perhaps Update() was not called."));
	return RHICmdList.CreateShaderResourceView(FShaderResourceViewInitializer(LayerView.RayTracingScenePooledBuffer->GetRHI(), LayerView.RayTracingSceneRHI, 0));
}

FRDGBufferSRVRef FRayTracingScene::GetLayerView(ERayTracingSceneLayer InLayer, FViewHandle InViewHandle) const
{
	const FLayerView& LayerView = Layers[uint8(InLayer)].Views[InViewHandle];
	checkf(LayerView.RayTracingSceneBufferSRV, TEXT("Ray tracing scene SRV was not created. Perhaps Update() was not called."));
	return LayerView.RayTracingSceneBufferSRV;
}

FRDGBufferUAVRef FRayTracingScene::GetInstanceHitCountBufferUAV(ERayTracingSceneLayer InLayer, FViewHandle InViewHandle) const
{
	return bTracingFeedbackEnabled ? Layers[uint8(InLayer)].Views[InViewHandle].InstanceHitCountBufferUAV : nullptr;
}

uint32 FRayTracingScene::GetNumNativeInstances(ERayTracingSceneLayer InLayer, FViewHandle InViewHandle) const
{
	const FLayerView& LayerView = Layers[uint8(InLayer)].Views[InViewHandle];
	checkf(bInitializationDataBuilt, TEXT("Must call BuildInitializationData() or Update() before using GetNumNativeInstances()."));
	return LayerView.InstanceBufferBuilder.GetMaxNumInstances();
}

FRayTracingScene::FInstanceHandle FRayTracingScene::AddCachedInstance(FRayTracingGeometryInstance Instance, ERayTracingSceneLayer InLayer, const FPrimitiveSceneProxy* Proxy, bool bDynamic, RayTracing::FGeometryHandle InGeometryHandle)
{
	UE_MT_SCOPED_WRITE_ACCESS(InstancesAccessDetector);

	ensure(!bCachedInstancesLocked);

	FLayer& Layer = Layers[uint8(InLayer)];

	FRHIRayTracingGeometry* GeometryRHI = Instance.GeometryRHI;

	uint32 InstanceIndex = UINT32_MAX;

	if (Layer.CachedInstancesFreeList.IsEmpty())
	{
		InstanceIndex = Layer.Instances.Add(MoveTemp(Instance));
	}
	else
	{
		InstanceIndex = Layer.CachedInstancesFreeList.Pop(EAllowShrinking::No);
		Layer.Instances[InstanceIndex] = MoveTemp(Instance);
	}

	++Layer.NumCachedInstances;

	if (bTracingFeedbackEnabled)
	{
		RayTracing::FGeometryHandle& GeometryHandle = Layer.GeometryHandles.IsValidIndex(InstanceIndex) ? Layer.GeometryHandles[InstanceIndex] : Layer.GeometryHandles.AddDefaulted_GetRef();
		GeometryHandle = InGeometryHandle;
		check(Layer.Instances.Num() == Layer.GeometryHandles.Num());
	}

	if (bInstanceDebugDataEnabled)
	{
		FRayTracingInstanceDebugData& InstanceDebugData = Layer.InstancesDebugData.IsValidIndex(InstanceIndex) ? Layer.InstancesDebugData[InstanceIndex] : Layer.InstancesDebugData.AddDefaulted_GetRef();
		InstanceDebugData.Flags = bDynamic ? 1 : 0;
		InstanceDebugData.GeometryAddress = uint64(GeometryRHI);
		InstanceDebugData.ProxyHash = Proxy ? Proxy->GetTypeHash() : 0;

		check(Layer.Instances.Num() == Layer.InstancesDebugData.Num());
	}

	return { InLayer, InstanceIndex };
}

void FRayTracingScene::FreeCachedInstance(FInstanceHandle Handle)
{
	UE_MT_SCOPED_READ_ACCESS(InstancesAccessDetector);

	ensure(!bCachedInstancesLocked);

	if (!Handle.IsValid())
	{
		return;
	}

	FLayer& Layer = Layers[uint8(Handle.Layer)];

	Layer.Instances[Handle.Index] = {};
	Layer.CachedInstancesFreeList.Push(Handle.Index);
	--Layer.NumCachedInstances;
}

void FRayTracingScene::FreeCachedInstance(uint32 PackedHandle)
{
	if (PackedHandle == UINT32_MAX)
	{
		return;
	}

	FreeCachedInstance(FInstanceHandle(PackedHandle));
}

void FRayTracingScene::UpdateCachedInstanceGeometry(FInstanceHandle Handle, FRHIRayTracingGeometry* GeometryRHI, int32 InstanceContributionToHitGroupIndex)
{
	UE_MT_SCOPED_READ_ACCESS(InstancesAccessDetector);

	FLayer& Layer = Layers[uint8(Handle.Layer)];

	Layer.Instances[Handle.Index].GeometryRHI = GeometryRHI;
	Layer.Instances[Handle.Index].InstanceContributionToHitGroupIndex = InstanceContributionToHitGroupIndex;

	if (bInstanceDebugDataEnabled)
	{
		FRayTracingInstanceDebugData& InstanceDebugData = Layer.InstancesDebugData[Handle.Index];
		InstanceDebugData.GeometryAddress = uint64(GeometryRHI);
	}
}

void FRayTracingScene::UpdateCachedInstanceGeometry(uint32 PackedHandle, FRHIRayTracingGeometry* GeometryRHI, int32 InstanceContributionToHitGroupIndex)
{
	UpdateCachedInstanceGeometry(FInstanceHandle(PackedHandle), GeometryRHI, InstanceContributionToHitGroupIndex);
}

FRHIRayTracingGeometry* FRayTracingScene::GetCachedInstanceGeometry(FInstanceHandle Handle) const
{
	UE_MT_SCOPED_READ_ACCESS(InstancesAccessDetector);

	const FLayer& Layer = Layers[uint8(Handle.Layer)];
	return Layer.Instances[Handle.Index].GeometryRHI;
}

FRHIRayTracingGeometry* FRayTracingScene::GetCachedInstanceGeometry(uint32 PackedHandle) const
{
	return GetCachedInstanceGeometry(FInstanceHandle(PackedHandle));
}

FRayTracingScene::FInstanceHandle FRayTracingScene::AddTransientInstance(FRayTracingGeometryInstance Instance, ERayTracingSceneLayer InLayer, FViewHandle InViewHandle, const FPrimitiveSceneProxy* Proxy, bool bDynamic, RayTracing::FGeometryHandle GeometryHandle)
{
	UE_MT_SCOPED_WRITE_ACCESS(InstancesAccessDetector);

	ensure(bCachedInstancesLocked);

	FLayer& Layer = Layers[uint8(InLayer)];

	FRHIRayTracingGeometry* GeometryRHI = Instance.GeometryRHI;

	const uint32 InstanceIndex = Layer.Instances.Add(MoveTemp(Instance));

	if (bTracingFeedbackEnabled)
	{
		Layer.GeometryHandles.Add(GeometryHandle);
		check(Layer.Instances.Num() == Layer.GeometryHandles.Num());
	}

	if (bInstanceDebugDataEnabled)
	{
		FRayTracingInstanceDebugData& InstanceDebugData = Layer.InstancesDebugData.AddDefaulted_GetRef();
		InstanceDebugData.Flags = bDynamic ? 1 : 0;
		InstanceDebugData.GeometryAddress = uint64(GeometryRHI);
		InstanceDebugData.ProxyHash = Proxy ? Proxy->GetTypeHash() : 0;

		check(Layer.Instances.Num() == Layer.InstancesDebugData.Num());
	}

	FLayerView& LayerView = Layer.Views[InViewHandle];

	int32 MinNumInstances = InstanceIndex + 1;
	if (LayerView.VisibleInstances.Num() < MinNumInstances)
	{
		LayerView.VisibleInstances.SetNum(MinNumInstances, false);
	}

	LayerView.VisibleInstances[InstanceIndex] = true;

	return { InLayer, InstanceIndex };
}

void FRayTracingScene::MarkInstanceVisible(FInstanceHandle Handle, FViewHandle InViewHandle)
{
	UE_MT_SCOPED_READ_ACCESS(InstancesAccessDetector);

	FLayer& Layer = Layers[uint8(Handle.Layer)];

	if(!EnumHasAnyFlags(Layer.Instances[Handle.Index].Flags, ERayTracingInstanceFlags::NaniteRayTracing))
	{
		// we expect visible instances to have a valid ray tracing geometry
		// (unless it's using Nanite Ray Tracing, in which case the ray tracing geometry is determined in RayTracingBuildInstanceBufferCS)
		check(Layer.Instances[Handle.Index].GeometryRHI != nullptr);
	}

	FLayerView& LayerView = Layer.Views[InViewHandle];

	int32 MinNumInstances = Handle.Index + 1;
	if (LayerView.VisibleInstances.Num() < MinNumInstances)
	{
		LayerView.VisibleInstances.SetNum(MinNumInstances, false);
	}

	LayerView.VisibleInstances[Handle.Index] = true;
}

void FRayTracingScene::MarkInstanceVisible(uint32 Handle, FViewHandle InViewHandle)
{
	MarkInstanceVisible(FInstanceHandle(ERayTracingSceneLayer(Handle >> 24), Handle & 0xFFFFFF), InViewHandle);
}

uint32 FRayTracingScene::FLayer::GetCachedInstanceSectionSize()
{
	return NumCachedInstances + CachedInstancesFreeList.Num();
}

void FRayTracingScene::Reset()
{
	UE_MT_SCOPED_WRITE_ACCESS(InstancesAccessDetector);

	const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);

	for (int32 ViewIndex : TransientViewIndices)
	{
		check(ActiveViews.IsValidIndex(ViewIndex) && ActiveViews[ViewIndex] == ViewIndex);

		// clear ViewIndex in Layers Views
		for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
		{
			FLayer& Layer = Layers[LayerIndex];
			Layer.Views[ViewIndex] = {};
		}

		ActiveViews.RemoveAt(ViewIndex);
		ViewParameters[ViewIndex] = {};
	}

	TransientViewIndices.Empty();

	for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		FLayer& Layer = Layers[LayerIndex];

		const uint32 CachedInstanceSectionSize = Layer.GetCachedInstanceSectionSize();

		Layer.Instances.SetNum(CachedInstanceSectionSize);
		Layer.InstancesDebugData.SetNum(bInstanceDebugDataEnabled ? CachedInstanceSectionSize : 0);

		Layer.GeometryHandleBuffer = nullptr;
		Layer.GeometryHandles.SetNum(bTracingFeedbackEnabled ? CachedInstanceSectionSize : 0);

		Layer.InstanceDebugBuffer = nullptr;

		for (int32 ViewIndex = 0; ViewIndex < Layer.Views.Num(); ++ViewIndex)
		{
			FLayerView& LayerView = Layer.Views[ViewIndex];

			LayerView.VisibleInstances.Reset();
			LayerView.VisibleInstances.SetNum(CachedInstanceSectionSize, false);

			LayerView.InstanceBufferBuilder = {};

			LayerView.RayTracingSceneBufferRDG = nullptr;
			LayerView.RayTracingSceneBufferSRV = nullptr;

			LayerView.InstanceBuffer = nullptr;
			LayerView.HitGroupContributionsBuffer = nullptr;
			LayerView.BuildScratchBuffer = nullptr;
			LayerView.InstanceExtraDataBuffer = nullptr;

			LayerView.InstanceHitCountBuffer = nullptr;
			LayerView.InstanceHitCountBufferUAV = nullptr;
			LayerView.AccelerationStructureIndexBuffer = nullptr;
		}
	}

	InstanceStatsBuffer = nullptr;

	Allocator.Flush();

	LightingChannelAndMask = 0xFF;

	check(InitTask.IsCompleted());
	InitTask = {};

	bCachedInstancesLocked = false;
}

void FRayTracingScene::EndFrame()
{
	Reset();

	// Release the resources if ray tracing wasn't used
	if (!bUsedThisFrame)
	{
		const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);

		for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
		{
			FLayer& Layer = Layers[LayerIndex];

			for (int32 ViewIndex = 0; ViewIndex < Layer.Views.Num(); ++ViewIndex)
			{
				FLayerView& LayerView = Layer.Views[ViewIndex];
				LayerView.RayTracingSceneRHI = nullptr;
				LayerView.RayTracingScenePooledBuffer = nullptr;
			}
		}

		ReleaseFeedbackReadbackBuffers();
		ReleaseReadbackBuffers();
	}

	bUsedThisFrame = false;
	bInitializationDataBuilt = false;
}

bool FRayTracingScene::SetInstanceExtraDataBufferEnabled(bool bInEnabled)
{
	bInstanceExtraDataBufferEnabled = bInEnabled;

	return false;
}

bool FRayTracingScene::SetTracingFeedbackEnabled(bool bInEnabled)
{
	bool bChanged = (bTracingFeedbackEnabled != bInEnabled);

	bTracingFeedbackEnabled = bInEnabled;

	if (bChanged)
	{
		const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);

		if (bTracingFeedbackEnabled)
		{
			for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
			{
				FLayer& Layer = Layers[LayerIndex];
				Layer.GeometryHandles.SetNum(Layer.Instances.Num());
			}
		}
		else
		{
			for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
			{
				FLayer& Layer = Layers[LayerIndex];
				Layer.GeometryHandles.Empty();
			}
		}
	}

	return bChanged;
}

bool FRayTracingScene::SetInstanceDebugDataEnabled(bool bInEnabled)
{
	bool bChanged = (bInstanceDebugDataEnabled != bInEnabled);

	bInstanceDebugDataEnabled = bInEnabled;

	if (bChanged)
	{
		const uint8 NumLayers = uint8(ERayTracingSceneLayer::NUM);

		if (bInstanceDebugDataEnabled)
		{
			for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
			{
				FLayer& Layer = Layers[LayerIndex];
				Layer.InstancesDebugData.SetNum(Layer.Instances.Num());
			}
		}
		else
		{
			for (uint32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
			{
				FLayer& Layer = Layers[LayerIndex];
				Layer.InstancesDebugData.Empty();
			}
		}
	}

	return bChanged;
}

void FRayTracingScene::ReleaseReadbackBuffers()
{
	for (auto& ReadbackData : StatsReadback)
	{
		delete ReadbackData.ReadbackBuffer;
	}

	StatsReadback.Empty();

	StatsReadbackWriteIndex = 0;
	StatsReadbackNumPending = 0;
}

void FRayTracingScene::ReleaseFeedbackReadbackBuffers()
{
	for (auto& ReadbackBuffer : FeedbackReadback)
	{
		delete ReadbackBuffer.GeometryHandleReadbackBuffer;
		delete ReadbackBuffer.GeometryCountReadbackBuffer;
	}
	FeedbackReadback.Empty();

	FeedbackReadbackWriteIndex = 0;
	FeedbackReadbackNumPending = 0;
}

#endif // RHI_RAYTRACING
