// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#include "Async/TaskGraphInterfaces.h"
#include "Math/DoubleFloat.h"
#include "RHI.h"
#include "RHIUtilities.h"
#include "RHIGPUReadback.h"
#include "RenderGraphResources.h"
#include "Misc/MemStack.h"
#include "Misc/MTAccessDetector.h"
#include "Containers/ArrayView.h"
#include "MeshPassProcessor.h"
#include "RayTracingShaderBindingTable.h"
#include "RayTracingInstanceBufferUtil.h"
#include "RayTracingGeometryInstance.h"
#include "RayTracingGeometryManagerTypes.h"
#include "RayTracingDebugTypes.h"

class FGPUScene;
class FRHIRayTracingScene;
class FRHIShaderResourceView;
class FRayTracingGeometry;
class FRDGBuilder;
class FPrimitiveSceneProxy;

enum class ERayTracingSceneLayer : uint8
{
	Base = 0,
	Decals,
	FarField,

	NUM
};

/**
* Persistent representation of the scene for ray tracing.
* Manages top level acceleration structure instances, memory and build process.
*/
class FRayTracingScene
{
public:

	struct FInstanceHandle
	{
		FInstanceHandle()
			: Layer(ERayTracingSceneLayer::NUM)
			, Index(UINT32_MAX)
		{}

		bool IsValid() const
		{
			return Layer < ERayTracingSceneLayer::NUM && Index != UINT32_MAX;
		}

		// We currently need to store these handles in FPrimitiveSceneInfo but since that's a public header we can't use FRayTracingScene::FInstanceHandle directly.
		// For now we provide a way to cast FInstanceHandle to uint32 and then FRayTracingScene methods also accept uint32 "PackedHandle".
		// TODO: Consider moving this handle type to a public header to avoid this.
		uint32 AsUint32() const
		{
			return Index | (int32(Layer) << 24);
		}

	private:
		FInstanceHandle(ERayTracingSceneLayer InLayer, uint32 InIndex)
			: Layer(InLayer)
			, Index(InIndex)
		{}

		explicit FInstanceHandle(uint32 InPackedHandle)
			: Layer(ERayTracingSceneLayer(InPackedHandle >> 24))
			, Index(InPackedHandle & 0xFFFFFF)
		{}

		ERayTracingSceneLayer Layer;
		uint32 Index;

		friend class FRayTracingScene;
	};

	static const FInstanceHandle INVALID_INSTANCE_HANDLE;

	struct FViewHandle
	{
		FViewHandle()
			: Index(UINT32_MAX)
		{}

		bool IsValid() const
		{
			return Index != UINT32_MAX;
		}

	private:
		explicit FViewHandle(uint32 InIndex)
			: Index(InIndex)
		{}

		operator uint32()
		{
			return Index;
		}

		uint32 Index;

		friend class FRayTracingScene;
	};

	static const FViewHandle INVALID_VIEW_HANDLE;

	FRayTracingScene();
	~FRayTracingScene();

	FInstanceHandle AddCachedInstance(FRayTracingGeometryInstance Instance, ERayTracingSceneLayer Layer, const FPrimitiveSceneProxy* Proxy = nullptr, bool bDynamic = false, RayTracing::FGeometryHandle GeometryHandle = {});
	void FreeCachedInstance(FInstanceHandle Handle);
	void FreeCachedInstance(uint32 PackedHandle);

	void UpdateCachedInstanceGeometry(FInstanceHandle Handle, FRHIRayTracingGeometry* GeometryRHI, int32 InstanceContributionToHitGroupIndex);
	void UpdateCachedInstanceGeometry(uint32 Handle, FRHIRayTracingGeometry* GeometryRHI, int32 InstanceContributionToHitGroupIndex);

	FRHIRayTracingGeometry* GetCachedInstanceGeometry(FInstanceHandle Handle) const;
	FRHIRayTracingGeometry* GetCachedInstanceGeometry(uint32 PackedHandle) const;

	FInstanceHandle AddTransientInstance(FRayTracingGeometryInstance Instance, ERayTracingSceneLayer Layer, FViewHandle ViewHandle, const FPrimitiveSceneProxy* Proxy = nullptr, bool bDynamic = false, RayTracing::FGeometryHandle GeometryHandle = {});

	void MarkInstanceVisible(FInstanceHandle Handle, FViewHandle ViewHandle);
	void MarkInstanceVisible(uint32 PackedHandle, FViewHandle ViewHandle);

	// Builds various metadata required to create the final scene.
	// Must be done before calling Create(...).
	void BuildInitializationData(bool bUseLightingChannels, bool bForceOpaque, bool bDisableTriangleCull);

	FViewHandle AddView(uint32 ViewKey);
	void RemoveView(uint32 ViewKey);

	// Allocates GPU memory to fit at least the current number of instances.
	// Kicks off instance buffer build to parallel thread along with RDG pass.
	void Update(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniformBuffer, const FGPUScene* GPUScene, ERDGPassFlags ComputePassFlags);

	void Build(FRDGBuilder& GraphBuilder, ERDGPassFlags ComputePassFlags, FRDGBufferRef DynamicGeometryScratchBuffer);

	void PostRender(FRDGBuilder& GraphBuilder, ERDGPassFlags ComputePassFlags = ERDGPassFlags::Compute);

	// Reset transient state/resources
	void Reset();

	void EndFrame();

	// Prevent cached instances being added/freed (via validation checks) until Reset() or EndFrame()
	// Must be done before adding transient instances.
	void LockCachedInstances()
	{
		bCachedInstancesLocked = true;
	}

	bool SetInstanceExtraDataBufferEnabled(bool bEnabled);
	bool SetTracingFeedbackEnabled(bool bEnabled);
	bool SetInstanceDebugDataEnabled(bool bEnabled);

	bool IsRayTracingFeedbackEnabled() const { return bTracingFeedbackEnabled; }

	// Allocates temporary memory that will be valid until the next Reset().
	// Can be used to store temporary instance transforms, user data, etc.
	template <typename T>
	TArrayView<T> Allocate(int32 Count)
	{
		return MakeArrayView(new(Allocator) T[Count], Count);
	}

	// Returns true if RHI ray tracing scene has been created.
	// i.e. returns true after BeginCreate() and before Reset().
	RENDERER_API bool IsCreated() const;

	// Returns RayTracingSceneRHI object (may return null).
	RENDERER_API  FRHIRayTracingScene* GetRHIRayTracingScene(ERayTracingSceneLayer Layer, FViewHandle ViewHandle) const;

	// Similar to GetRayTracingScene, but checks that ray tracing scene RHI object is valid.
	RENDERER_API  FRHIRayTracingScene* GetRHIRayTracingSceneChecked(ERayTracingSceneLayer Layer, FViewHandle ViewHandle) const;

	// Creates new RHI view of a layer. Can only be used on valid ray tracing scene. 
	RENDERER_API FShaderResourceViewRHIRef CreateLayerViewRHI(FRHICommandListBase& RHICmdList, ERayTracingSceneLayer Layer, FViewHandle ViewHandle) const;

	// Returns RDG view of a layer. Can only be used on valid ray tracing scene.
	RENDERER_API FRDGBufferSRVRef GetLayerView(ERayTracingSceneLayer Layer, FViewHandle ViewHandle) const;

	// Feedback	
	RENDERER_API FRDGBufferUAVRef GetInstanceHitCountBufferUAV(ERayTracingSceneLayer Layer, FViewHandle ViewHandle) const;

	FRDGBufferRef GetInstanceBuffer(ERayTracingSceneLayer Layer, FViewHandle ViewHandle) const { return Layers[uint8(Layer)].Views[ViewHandle].InstanceBuffer; }

	TConstArrayView<FRayTracingGeometryInstance> GetInstances(ERayTracingSceneLayer Layer) const { return MakeArrayView(Layers[uint8(Layer)].Instances); }

	FRayTracingGeometryInstance& GetInstance(FInstanceHandle Handle) { return Layers[uint8(Handle.Layer)].Instances[Handle.Index]; }

	uint32 GetNumNativeInstances(ERayTracingSceneLayer Layer, FViewHandle ViewHandle) const;

	FRDGBufferRef GetInstanceDebugBuffer(ERayTracingSceneLayer Layer) const { return Layers[uint8(Layer)].InstanceDebugBuffer; }

	FRDGBufferRef GetInstanceExtraDataBuffer(ERayTracingSceneLayer Layer, FViewHandle ViewHandle) const { return Layers[uint8(Layer)].Views[ViewHandle].InstanceExtraDataBuffer; }

	void SetViewParams(FViewHandle ViewHandle, const FViewMatrices& ViewMatrices, const FRayTracingCullingParameters& CullingParameters);

	FVector GetPreViewTranslation(FViewHandle ViewHandle) const { return ViewParameters[ViewHandle].PreViewTranslation; }

public:

	// Public members for initial refactoring step (previously were public members of FViewInfo).

	uint8 LightingChannelAndMask = 0xFF;

	UE::Tasks::FTask InitTask; // Task to asynchronously call BuildInitializationData()

private:

	void FinishTracingFeedback(FRDGBuilder& GraphBuilder, ERDGPassFlags ComputePassFlags);
	void FinishStats(FRDGBuilder& GraphBuilder, ERDGPassFlags ComputePassFlags);

	void ReleaseReadbackBuffers();
	void ReleaseFeedbackReadbackBuffers();

	struct FLayerView
	{
		FRayTracingInstanceBufferBuilder InstanceBufferBuilder;

		FRayTracingSceneRHIRef RayTracingSceneRHI;

		FRDGBufferRef InstanceBuffer = nullptr;
		FRDGBufferRef HitGroupContributionsBuffer = nullptr;
		FRDGBufferRef BuildScratchBuffer = nullptr;

		// Feedback		
		FRDGBufferRef InstanceHitCountBuffer = nullptr;
		FRDGBufferUAVRef InstanceHitCountBufferUAV;
		FRDGBufferRef AccelerationStructureIndexBuffer = nullptr;

		TRefCountPtr<FRDGPooledBuffer> RayTracingScenePooledBuffer;
		FRDGBufferRef RayTracingSceneBufferRDG = nullptr;
		FRDGBufferSRVRef RayTracingSceneBufferSRV = nullptr;

		FRDGBufferRef InstanceExtraDataBuffer = nullptr;

		TBitArray<> VisibleInstances;

		uint32 NumActiveInstances = 0;
		uint32 MaxNumInstances = 0;
	};

	struct FLayer
	{
		uint32 GetCachedInstanceSectionSize();

		// Feedback
		FRDGBufferRef GeometryHandleBuffer = nullptr;
		TArray<RayTracing::FGeometryHandle> GeometryHandles;

		// Special data for debugging purposes
		FRDGBufferRef InstanceDebugBuffer = nullptr;

		// Persistent storage for ray tracing instance descriptors.
		// The array is divided into two sections [Cached instances | Transient Instances]
		// Transient instances are cleared every frame.
		TArray<FRayTracingGeometryInstance> Instances;

		TArray<FRayTracingInstanceDebugData> InstancesDebugData;

		TArray<int32> CachedInstancesFreeList;

		uint32 NumCachedInstances = 0;

		TArray<FLayerView> Views;

		FName Name;
	};

	TArray<FLayer> Layers;

	// Transient memory allocator
	FMemStackBase Allocator;

	struct FViewParameters
	{
		const FRayTracingCullingParameters* CullingParameters;

		// Used for transforming to translated world space in which TLAS was built.
		FVector PreViewTranslation;

		uint32 ViewKey = 0;
	};

	TArray<FViewParameters> ViewParameters;

	TSparseArray<int32> ActiveViews;
	TMap<uint32, int32> ViewIndexMap;
	TArray<int32> TransientViewIndices;

	bool bInstanceExtraDataBufferEnabled = false;
	bool bTracingFeedbackEnabled = false;
	bool bInstanceDebugDataEnabled = false;

	bool bInitializationDataBuilt = false;
	bool bUsedThisFrame = false;

	// Adding/freeing cached instances is not allowed when this bool is set (used for validation)
	bool bCachedInstancesLocked = false;

	// Validates concurrent access to Layer.Instances (debug-only, no actual locking).
	// WRITE = structural mutation (Add/resize). READ = element-level access (safe to overlap if different indices).
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(InstancesAccessDetector);

	struct FFeedbackReadbackData
	{
		FRHIGPUBufferReadback* GeometryHandleReadbackBuffer = nullptr;
		FRHIGPUBufferReadback* GeometryCountReadbackBuffer = nullptr;
	};

	const uint32 MaxReadbackBuffers = 4;

	FRDGBufferRef InstanceStatsBuffer = nullptr;

	TArray<FFeedbackReadbackData> FeedbackReadback;
	uint32 FeedbackReadbackWriteIndex = 0;
	uint32 FeedbackReadbackNumPending = 0;

	struct FStatsReadbackData
	{
		FRHIGPUBufferReadback* ReadbackBuffer = nullptr;
		TArray<uint32> ViewKeys;
		uint32 MaxNumViews = 0;
	};

	TArray<FStatsReadbackData> StatsReadback;
	uint32 StatsReadbackWriteIndex = 0;
	uint32 StatsReadbackNumPending = 0;
};

#endif // RHI_RAYTRACING
