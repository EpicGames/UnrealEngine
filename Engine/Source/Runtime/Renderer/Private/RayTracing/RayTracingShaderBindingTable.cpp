// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingShaderBindingTable.h"
#include "RayTracingDefinitions.h"
#include "RayTracingScene.h"
#include "RayTracing.h"
#include "RayTracingGeometry.h"
#include "LogRenderer.h"

#if RHI_RAYTRACING

static int32 GPersistentSBTEnabled = 1;
static FAutoConsoleVariableRef CVarRayTracingPersistentSBT(
	TEXT("r.RayTracing.PersistentSBT"),
	GPersistentSBTEnabled,
	TEXT("Enable persistent RayTracing ShaderBindingTables."),
	ECVF_RenderThreadSafe
);

static int32 GForceAlwaysDirty = 0;
static FAutoConsoleVariableRef CVarRayTracingPersistentSBTForceAlwaysDirty(
	TEXT("r.RayTracing.PersistentSBT.ForceAlwaysDirty"),
	GForceAlwaysDirty,
	TEXT("Force all visible shader bindings as dirty (debug mode)."),
	ECVF_RenderThreadSafe
);

static int32 GMinLocalBindingDataSize = 96;
static FAutoConsoleVariableRef CVarRayTracingPersistentSBTMinLocalBindingDataSize(
	TEXT("r.RayTracing.PersistentSBT.MinLocalBindingDataSize"),
	GMinLocalBindingDataSize,
	TEXT("Minimum local binding data size of the persistent SBT (can dynamically grow if need by hit shaders used in the RTPSO)."),
	ECVF_ReadOnly
);

static int32 GMinMissShaderSlots = 128;
static FAutoConsoleVariableRef CVarRayTracingPersistentSBTMinMissShaderSlots(
	TEXT("r.RayTracing.PersistentSBT.MinMissShaderSlots"),
	GMinMissShaderSlots,
	TEXT("Minimum amount of miss shader slots reserved in the persistent SBT (can dynamically grow if need by number of miss shaders used in the RTPSO)."),
	ECVF_ReadOnly
);

static int32 GMinStaticGeometrySegments = 256;
static FAutoConsoleVariableRef CVarRayTracingPersistentSBTMinStaticGeometrySegments(
	TEXT("r.RayTracing.PersistentSBT.MinStaticGeometrySegments"),
	GMinStaticGeometrySegments,
	TEXT("Minimum amount of static geometry segments reserved in the persistent SBT (can dynamically grow if need by number of allocated static SBT allocations in the scene)."),
	ECVF_ReadOnly
);

static int32 GMinDynamicGeometrySegments = 256;
static FAutoConsoleVariableRef CVarRayTracingPersistentSBTMinDynamicGeometrySegments(
	TEXT("r.RayTracing.PersistentSBT.MinDynamicGeometrySegments"),
	GMinDynamicGeometrySegments,
	TEXT("Minimum amount of dynamic geometry segments reserved in the persistent SBT (can dynamically grow if need by number of allocated static SBT allocations in the scene)."),
	ECVF_ReadOnly
);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static int32 GValidatePersistentBindings = 1;
#else 
static int32 GValidatePersistentBindings = 0;
#endif
static FAutoConsoleVariableRef CVarRayTracingPersistentSBTValidateBindings(
	TEXT("r.RayTracing.PersistentSBT.ValidateBindings"),
	GValidatePersistentBindings,
	TEXT("Force all visible shader bindings as dirty (debug mode)."),
	ECVF_RenderThreadSafe
);

static bool UseRayTracingPersistentSBTs()
{
	return GRHIGlobals.RayTracing.SupportsPersistentSBTs && GPersistentSBTEnabled;
}

uint32 FRayTracingSBTAllocation::GetRecordIndex(ERayTracingShaderBindingLayer Layer, uint32 SegmentIndex) const
{
	check(HasLayer(Layer));

	// Find out all the bits set below the given layer
	// and count the set bits to know the offset
	uint32 LayerMask = (1 << (uint32)Layer) - 1;
	LayerMask = (uint32)AllocatedLayers & LayerMask;
	uint32 RecordTypeBaseOffset = FMath::CountBits(LayerMask) * RecordsPerLayer;

	// Cast SegmentIndex to uint64 so the check still works when SegmentIndex == UINT32_MAX
	check(RecordTypeBaseOffset + uint64(SegmentIndex) * RAY_TRACING_NUM_SHADER_SLOTS + RAY_TRACING_NUM_SHADER_SLOTS <= NumRecords);
	return BaseRecordIndex + RecordTypeBaseOffset + SegmentIndex * RAY_TRACING_NUM_SHADER_SLOTS;
}

int32 FRayTracingSBTAllocation::GetSegmentCount() const
{
	return NumRecords / RAY_TRACING_NUM_SHADER_SLOTS;
}

int32 FRayTracingSBTAllocation::GetUniqueId() const
{
	return BaseRecordIndex;
}

FRayTracingShaderBindingTable::FRayTracingShaderBindingTable() : NumShaderSlotsPerGeometrySegment(RAY_TRACING_NUM_SHADER_SLOTS), StaticRangeAllocator(true /*bInGrowOnly*/)
{
	PersistentSBTInitializer.ShaderBindingMode = ERayTracingShaderBindingMode::RTPSO;
	PersistentSBTInitializer.Lifetime = UseRayTracingPersistentSBTs() ? ERayTracingShaderBindingTableLifetime::Persistent : ERayTracingShaderBindingTableLifetime::Transient;
	PersistentSBTInitializer.LocalBindingDataSize = GMinLocalBindingDataSize;
	PersistentSBTInitializer.NumMissShaderSlots = GMinMissShaderSlots;
}

/**
 * Make sure all dynamic allocation objects are freed and assure all static allocations have been requested deleted already
 */
FRayTracingShaderBindingTable::~FRayTracingShaderBindingTable()
{
	ResetDynamicAllocationData();
	
	// Assume empty?
	check(TrackedAllocationMap.IsEmpty());
	check(StaticRangeAllocator.GetSparselyAllocatedSize() == 0);
}

/**
 * Mark all currently allocated dynamic ranges as free again so they can be allocated
 * Setup the CurrentDynamicRangeOffset from where dynamic SBT records will be stored 
 * After this call no static SBT ranges can be allocated anymore until the end of the 'frame'
 */
void FRayTracingShaderBindingTable::ResetDynamicAllocationData()
{				
	// Release all dynamic allocation back to the pool
	NumUsedDynamicAllocations = 0;
	NumDynamicGeometrySegments = 0;
	
	// Static allocations are not allowed anymore because dynamic allocations are stored right after all static allocations
	bStaticAllocationsLocked = true;

	// Dynamic segments will be stored right after the currently allocated 
	uint32 AllocatedStaticSegmentSize = GetMaxAllocatedStaticSegmentCount();
	StartDynamicRangeOffset = AllocatedStaticSegmentSize * NumShaderSlotsPerGeometrySegment;
	CurrentDynamicRangeOffset = StartDynamicRangeOffset;
	StartDynamicRangeOffsetToMarkDirty = StartDynamicRangeOffset;
}

void FRayTracingShaderBindingTable::ResetMissAndCallableShaders()
{
	CallableCommands.Reset();

	NumMissShaderSlots = 1;
	NumCallableShaderSlots = 0;
}

void FRayTracingShaderBindingTable::EndFrame()
{
	TransientUniformBuffers.Reset();

	ResetMissAndCallableShaders();
}

FRayTracingShaderBindingDataOneFrameArray FRayTracingShaderBindingTable::GetDirtyBindings(TConstArrayView<FRayTracingShaderBindingData> VisibleBindings, bool bForceNonPersistent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRayTracingShaderBindingTable::GetDirtyBindings);
	
	ensure(bStaticAllocationsLocked);

	// Flush dynamic records that are pending being marked dirty
	{
		check(StartDynamicRangeOffsetToMarkDirty <= CurrentDynamicRangeOffset);
	
		// Grow the bit array if necessary
		const int32 MaxRecordIndex = CurrentDynamicRangeOffset;
		if (DirtyPersistentRecords.Num() < MaxRecordIndex)
		{
			DirtyPersistentRecords.SetNumUninitialized(FMath::RoundUpToPowerOfTwo(MaxRecordIndex));
		}

		// Mark bits in dynamic range as dirty
		DirtyPersistentRecords.SetRange(StartDynamicRangeOffsetToMarkDirty, CurrentDynamicRangeOffset - StartDynamicRangeOffsetToMarkDirty, true);

		StartDynamicRangeOffsetToMarkDirty = CurrentDynamicRangeOffset;
	}

	FRayTracingShaderBindingDataOneFrameArray DirtyBindings;

	// Persistent SBTs disabled (none of the cached SBTs have any persistent bindings so safe to just copy visible bindings and update type inside binding will be ignored)
	if (!UseRayTracingPersistentSBTs())
	{
		DirtyBindings = VisibleBindings;
	}
	else
	{
		// Check if one of the cached persistent SBTs doesn't support persistent bindings (eg pathtracer) or requested via cvar or via param 
		// (RTPSO is still compiling and bindings should be marked as persistent yet)
		bool bForceAllTransient = bForceNonPersistent || GForceAlwaysDirty || PersistentSBTInitializer.Lifetime == ERayTracingShaderBindingTableLifetime::Transient;
		
		DirtyBindings.Reserve(VisibleBindings.Num());
		for (const FRayTracingShaderBindingData& VisibleBinding : VisibleBindings)
		{
			if (bForceAllTransient)
			{
				FRayTracingShaderBindingData ValidBinding = VisibleBinding;
				ValidBinding.BindingType = ERayTracingLocalShaderBindingType::Transient;
				DirtyBindings.Add(ValidBinding);
			}
			else
			{
				check(VisibleBinding.BindingType != ERayTracingLocalShaderBindingType::Persistent || SetPersistentRecords[VisibleBinding.SBTRecordIndex]);

				FBitReference DirtyRecordIndex = DirtyPersistentRecords[VisibleBinding.SBTRecordIndex];
				if (VisibleBinding.BindingType == ERayTracingLocalShaderBindingType::Transient || DirtyRecordIndex)
				{
					DirtyBindings.Add(VisibleBinding);
					DirtyRecordIndex = false;

					//UE_LOGF(LogRenderer, Log, "Marking SBT Record %d NOT dirty anymore", VisibleBinding.SBTRecordIndex);
				}
				else if (GValidatePersistentBindings)
				{
					FRayTracingShaderBindingData ValidBinding = VisibleBinding;
					ValidBinding.BindingType = ERayTracingLocalShaderBindingType::Validation;
					DirtyBindings.Add(ValidBinding);
				}
			}
		}
	}

	return MoveTemp(DirtyBindings);
}

void FRayTracingShaderBindingTable::AllocatePersistentShaderBindingTable(FRHICommandListBase& RHICmdList, FRayTracingPersistentShaderBindingTableID PersistentSBTID, ERayTracingShaderBindingMode ShaderBindingMode)
{
	// Update the shader binding mode on the shared peristent initializer
	FRayTracingShaderBindingTableInitializer SBTInitializer = PersistentSBTInitializer;
	SBTInitializer.ShaderBindingMode = ShaderBindingMode;

	PersistentSBTs[PersistentSBTID].ShaderBindingTable = RHICmdList.CreateRayTracingShaderBindingTable(SBTInitializer);
	
	FRHISizeAndStride InlineBindingDataSizeAndStride = PersistentSBTs[PersistentSBTID].ShaderBindingTable->GetInlineBindingDataSizeAndStride();
	if (InlineBindingDataSizeAndStride.Size > 0)
	{	
		const uint32 InlineBindingDataElementCount = InlineBindingDataSizeAndStride.Size / InlineBindingDataSizeAndStride.Stride;

		PersistentSBTs[PersistentSBTID].InlineBindingDataPooledBuffer = AllocatePooledBuffer(FRDGBufferDesc::CreateStructuredDesc(InlineBindingDataSizeAndStride.Stride, InlineBindingDataElementCount), TEXT("InlineRayTracingBindingData"));
	}
}
	
FRayTracingPersistentShaderBindingTableID FRayTracingShaderBindingTable::AllocatePersistentSBTID(FRHICommandListBase& RHICmdList, ERayTracingShaderBindingMode ShaderBindingMode)
{
	ensure(bStaticAllocationsLocked);

	FRayTracingPersistentShaderBindingTableID ResultID = INDEX_NONE;
	for (int32 Index = 0; Index < PersistentSBTs.Num(); ++Index)
	{
		if (PersistentSBTs[Index].ShaderBindingTable == nullptr)
		{
			ResultID = Index;
			break;
		}
	}
	if (ResultID == INDEX_NONE)
	{
		ResultID = PersistentSBTs.AddDefaulted();
	}

	// Also recreate all current SBTs because valid records will be cleared with clearing the current valid persistent records
	for (int32 Index = 0; Index < PersistentSBTs.Num(); ++Index)
	{
		if (PersistentSBTs[Index].ShaderBindingTable)
		{
			AllocatePersistentShaderBindingTable(RHICmdList, Index, PersistentSBTs[Index].ShaderBindingMode);
		}
	}
	
	// Allocate the RHI object with current initializer settings and store the current shader binding mode
	PersistentSBTs[ResultID].ShaderBindingMode = ShaderBindingMode;
	AllocatePersistentShaderBindingTable(RHICmdList, ResultID, ShaderBindingMode);

	DirtyPersistentRecords.Init(true, PersistentSBTInitializer.NumGeometrySegments * PersistentSBTInitializer.NumShaderSlotsPerGeometrySegment);

	return ResultID;
}

void FRayTracingShaderBindingTable::CheckPersistentRHI(FRHICommandListBase& RHICmdList, uint32 LocalBindingDataSize)
{		
	ensure(bStaticAllocationsLocked);

	uint32 NumPersistentStaticGeometrySegments = FMath::Max(uint32(GMinStaticGeometrySegments), FMath::RoundUpToPowerOfTwo(GetMaxAllocatedStaticSegmentCount()));
	uint32 NumPersistentDynamicGeometrySegments = FMath::Max(uint32(GMinDynamicGeometrySegments), FMath::RoundUpToPowerOfTwo(MaxNumDynamicGeometrySegments));
	uint32 NumMissShaderSlotsAligned = FMath::RoundUpToPowerOfTwo(NumMissShaderSlots);

	// Build the new SBT initializer
	FRayTracingShaderBindingTableInitializer NewSBTInitializer;
	NewSBTInitializer.Lifetime = UseRayTracingPersistentSBTs() ? ERayTracingShaderBindingTableLifetime::Persistent : ERayTracingShaderBindingTableLifetime::Transient;
	NewSBTInitializer.ShaderBindingMode = PersistentSBTInitializer.ShaderBindingMode;
	NewSBTInitializer.HitGroupIndexingMode = PersistentSBTInitializer.HitGroupIndexingMode;
	NewSBTInitializer.NumShaderSlotsPerGeometrySegment = NumShaderSlotsPerGeometrySegment;
	NewSBTInitializer.NumGeometrySegments = FMath::Max(NumPersistentStaticGeometrySegments + NumPersistentDynamicGeometrySegments, PersistentSBTInitializer.NumGeometrySegments);
	NewSBTInitializer.NumMissShaderSlots = FMath::Max(NumMissShaderSlotsAligned, PersistentSBTInitializer.NumMissShaderSlots);
	NewSBTInitializer.NumCallableShaderSlots = FMath::Max(NumCallableShaderSlots, PersistentSBTInitializer.NumCallableShaderSlots);
	NewSBTInitializer.LocalBindingDataSize = FMath::Max(LocalBindingDataSize, PersistentSBTInitializer.LocalBindingDataSize);

	// Always force recreate when persistent SBT is not enabled
	bool bRecreate = !UseRayTracingPersistentSBTs();
	if (UseRayTracingPersistentSBTs())
	{
		TStringBuilder<256> Reason;
		auto NewDifferent = [&Reason](const TCHAR* MemberName, uint32 CurrentValue, uint32 NewValue)
		{
			if (CurrentValue != NewValue)
			{
				Reason.Appendf(TEXT("\n\t\t%s changed: current: %d - new: %d"), MemberName, CurrentValue, NewValue);
				return true;
			}
			return false;
		};
		auto NewBigger = [&Reason](const TCHAR* MemberName, uint32 CurrentValue, uint32 NewValue)
		{
			if (CurrentValue < NewValue)
			{
				Reason.Appendf(TEXT("\n\t\t%s changed: current: %d - new: %d"), MemberName, CurrentValue, NewValue);
				return true;
			}
			return false;
		};

		bRecreate = NewDifferent(TEXT("Lifetime"), (uint32)PersistentSBTInitializer.Lifetime, (uint32)NewSBTInitializer.Lifetime);
		bRecreate = NewDifferent(TEXT("ShaderBindingMode"), (uint32)PersistentSBTInitializer.ShaderBindingMode, (uint32)NewSBTInitializer.ShaderBindingMode) || bRecreate;
		bRecreate = NewDifferent(TEXT("HitGroupIndexingMode"), (uint32)PersistentSBTInitializer.HitGroupIndexingMode, (uint32)NewSBTInitializer.HitGroupIndexingMode) || bRecreate;
		bRecreate = NewDifferent(TEXT("NumShaderSlotsPerGeometrySegment"), PersistentSBTInitializer.NumShaderSlotsPerGeometrySegment, NewSBTInitializer.NumShaderSlotsPerGeometrySegment) || bRecreate;
		bRecreate = NewBigger(TEXT("NumGeometrySegments"), PersistentSBTInitializer.NumGeometrySegments, NewSBTInitializer.NumGeometrySegments) || bRecreate;
		bRecreate = NewBigger(TEXT("NumMissShaderSlots"), PersistentSBTInitializer.NumMissShaderSlots, NewSBTInitializer.NumMissShaderSlots) || bRecreate;
		bRecreate = NewBigger(TEXT("NumCallableShaderSlots"), PersistentSBTInitializer.NumCallableShaderSlots, NewSBTInitializer.NumCallableShaderSlots) || bRecreate;
		bRecreate = NewBigger(TEXT("LocalBindingDataSize"), PersistentSBTInitializer.LocalBindingDataSize, NewSBTInitializer.LocalBindingDataSize) || bRecreate;

		if (bRecreate)
		{			
			UE_LOGF(LogRenderer, Log, "Recreating Persistent SBTs due to initializer changes: %ls", *Reason);
		}
	}

	// Recreate new RHI object if either persistent is not enabled or current allocated RHI object doesn't match the new initializer or doesn't have enough space to store all bindings
	// (number of bindings stored in SBT only grows rights now)
	if (bRecreate)
	{
		PersistentSBTInitializer = NewSBTInitializer;

		// Reallocate all the persistent SBTs because all valid records will be reset without clearing the already used records first
		// Another option is supporting persistent state overwrite on the SBT records but doesn't allow for correct record state validation (first clear before persisntent record can be written again)
		// All used records could be cleared on the other persistent SBTs as well but will be a lot records and recreating the SBT is easier
		for (int32 Index = 0; Index < PersistentSBTs.Num(); ++Index)
		{
			if (PersistentSBTs[Index].ShaderBindingTable)
			{
				AllocatePersistentShaderBindingTable(RHICmdList, Index, PersistentSBTs[Index].ShaderBindingMode);
			}
		}
		DirtyPersistentRecords.Init(true, NewSBTInitializer.NumGeometrySegments * NewSBTInitializer.NumShaderSlotsPerGeometrySegment);
	}
}

FShaderBindingTableRHIRef FRayTracingShaderBindingTable::AllocateTransientRHI(
	FRHICommandListBase& RHICmdList, 
	ERayTracingShaderBindingMode ShaderBindingMode, 
	ERayTracingHitGroupIndexingMode HitGroupIndexingMode,
	uint32 LocalBindingDataSize) const
{
	uint32 AllocatedStaticSegmentSize = GetMaxAllocatedStaticSegmentCount();
	
	FRayTracingShaderBindingTableInitializer SBTInitializer;
	SBTInitializer.ShaderBindingMode = ShaderBindingMode;
	SBTInitializer.HitGroupIndexingMode = HitGroupIndexingMode;
	SBTInitializer.NumGeometrySegments = AllocatedStaticSegmentSize + NumDynamicGeometrySegments;
	SBTInitializer.NumShaderSlotsPerGeometrySegment = NumShaderSlotsPerGeometrySegment;
	SBTInitializer.NumMissShaderSlots = NumMissShaderSlots;
	SBTInitializer.NumCallableShaderSlots = NumCallableShaderSlots;
	SBTInitializer.LocalBindingDataSize = LocalBindingDataSize;
	
	return RHICmdList.CreateRayTracingShaderBindingTable(SBTInitializer);
}

uint32 FRayTracingShaderBindingTable::GetNumGeometrySegments() const
{
	return GetMaxAllocatedStaticSegmentCount() + NumDynamicGeometrySegments;
}

uint32 FRayTracingShaderBindingTable::GetMaxAllocatedStaticSegmentCount() const
{
	return StaticRangeAllocator.GetMaxSize() / NumShaderSlotsPerGeometrySegment;
}

void FRayTracingShaderBindingTable::MarkDirty(FRayTracingSBTAllocation* SBTAllocation)
{
	int32 MaxRecordIndex = SBTAllocation->BaseRecordIndex + SBTAllocation->NumRecords;
	if (DirtyPersistentRecords.Num() < MaxRecordIndex)
	{
		DirtyPersistentRecords.SetNum(FMath::RoundUpToPowerOfTwo(MaxRecordIndex), true);
	}
	for (uint32 Index = 0; Index < SBTAllocation->NumRecords; ++Index)
	{
		DirtyPersistentRecords[SBTAllocation->BaseRecordIndex + Index] = true;
		//UE_LOGF(LogRenderer, Log, "Marking SBT Record %d dirty", SBTAllocation->BaseRecordIndex + Index);
	}
}

void FRayTracingShaderBindingTable::MarkSet(FRayTracingSBTAllocation* SBTAllocation, bool bValue)
{
	int32 MaxRecordIndex = SBTAllocation->BaseRecordIndex + SBTAllocation->NumRecords;
	if (SetPersistentRecords.Num() < MaxRecordIndex)
	{
		SetPersistentRecords.SetNum(FMath::RoundUpToPowerOfTwo(MaxRecordIndex), false);
		PersistentRecordsToClear.SetNum(FMath::RoundUpToPowerOfTwo(MaxRecordIndex), false);
	}
	for (uint32 Index = 0; Index < SBTAllocation->NumRecords; ++Index)
	{
		uint32 RecordIndex = SBTAllocation->BaseRecordIndex + Index;
		check(SetPersistentRecords[RecordIndex] != bValue);
		SetPersistentRecords[RecordIndex] = bValue;

		// Mark record to clear and validate that it's not marked for clear when we are trying to clear it out (double clear?)
		check(bValue || !PersistentRecordsToClear[RecordIndex]);
		PersistentRecordsToClear[RecordIndex] = !bValue;

		//UE_LOGF(LogRenderer, Log, "Update set SBT record data for RecordIndex %d with new value: %d", RecordIndex, bValue ? 1 : 0);
	}
}

FRayTracingSBTAllocation* FRayTracingShaderBindingTable::AllocateStaticRangeInternal(
	ERayTracingShaderBindingLayerMask AllocatedLayers,
	uint32 SegmentCount, 
	const FRHIRayTracingGeometry* Geometry, 
	FRayTracingCachedMeshCommandFlags Flags)
{
	// Should be allowed to make static SBT allocations
	ensure(!bStaticAllocationsLocked);

	uint32 LayersCount = FMath::CountBits((uint32)AllocatedLayers);
	uint32 RecordsPerLayer = SegmentCount * NumShaderSlotsPerGeometrySegment;
	uint32 RecordCount = RecordsPerLayer * LayersCount;
	uint32 BaseIndex = StaticRangeAllocator.Allocate(RecordCount);
			
	FRayTracingSBTAllocation* Allocation = new FRayTracingSBTAllocation();
	Allocation->InitStatic(AllocatedLayers, BaseIndex, RecordsPerLayer, RecordCount, Geometry, Flags);

	MarkDirty(Allocation);
	MarkSet(Allocation, true);

	AllocatedStaticSegmentCount += SegmentCount * LayersCount;
	
	return Allocation;
}

FRayTracingSBTAllocation* FRayTracingShaderBindingTable::AllocateStaticRange(uint32 SegmentCount, const FRHIRayTracingGeometry* Geometry, FRayTracingCachedMeshCommandFlags Flags)
{
	ensure(!bStaticAllocationsLocked);

	// No allocation if we are not rendering decals and all segments are decals
	if (RayTracing::ShouldExcludeDecals() && Flags.bAllSegmentsDecal)
	{
		return nullptr;
	}

	ERayTracingShaderBindingLayerMask AllocatedLayers = ERayTracingShaderBindingLayerMask::None;
	if (!Flags.bAllSegmentsDecal)
	{
		EnumAddFlags(AllocatedLayers, ERayTracingShaderBindingLayerMask::Base);
	}
	if (Flags.bAnySegmentsDecal && !RayTracing::ShouldExcludeDecals())
	{
		EnumAddFlags(AllocatedLayers, ERayTracingShaderBindingLayerMask::Decals);
	}
	if (AllocatedLayers == ERayTracingShaderBindingLayerMask::None)
	{
		return nullptr;
	}

	FScopeLock ScopeLock(&StaticAllocationCS);

	// Setup the key needed for deduplication
	FAllocationKey Key;
	Key.Geometry = Geometry;
	Key.Flags = Flags;
	Key.SegmentCount = SegmentCount;

	// Already allocated for given hash
	FRefCountedAllocation& Allocation = TrackedAllocationMap.FindOrAdd(Key);
	if (Allocation.RefCount == 0)
	{		
		Allocation.Allocation = AllocateStaticRangeInternal(AllocatedLayers, SegmentCount, Geometry, Flags);
		Allocation.Allocation->bDeduplicate = true;
	}
	else
	{
		TotalStaticAllocationCount++;
	}
	check(Allocation.Allocation->AllocatedLayers == AllocatedLayers);
	check(Allocation.Allocation->GetSegmentCount() == FMath::CountBits((uint32)AllocatedLayers) * SegmentCount);
	
	Allocation.RefCount++;
	return Allocation.Allocation;
}

void FRayTracingShaderBindingTable::FreeStaticRange(FRayTracingSBTAllocation* Allocation)
{
	if (Allocation == nullptr)
	{
		return;
	}

	FScopeLock ScopeLock(&StaticAllocationCS);

	TotalStaticAllocationCount--;

	if (Allocation->bDeduplicate)
	{
		FAllocationKey Key;
		Key.Geometry = Allocation->Geometry;
		Key.Flags = Allocation->Flags;
		Key.SegmentCount = Allocation->RecordsPerLayer / NumShaderSlotsPerGeometrySegment;

		FRefCountedAllocation* RefAllocation = TrackedAllocationMap.Find(Key);
		check(RefAllocation);
		RefAllocation->RefCount--;

		if (RefAllocation->RefCount == 0)
		{
			AllocatedStaticSegmentCount -= (RefAllocation->Allocation->NumRecords / NumShaderSlotsPerGeometrySegment);

			// Already mark unused but only defer the free on the free range allocator until the clear op is actually processed
			// so the free can happen free threaded while the static allocations are still locked
			MarkSet(RefAllocation->Allocation, false);
			PendingStaticAllocationsToFree.Add(RefAllocation->Allocation);

			TrackedAllocationMap.Remove(Key);
		}
	}
	else
	{
		AllocatedStaticSegmentCount -= (Allocation->NumRecords / NumShaderSlotsPerGeometrySegment);

		// Already mark unused but only defer the free on the free range allocator until the clear op is actually processed
		// so the free can happen free threaded while the static allocations are still locked
		MarkSet(Allocation, false);
		PendingStaticAllocationsToFree.Add(Allocation);
	}
}

FRayTracingSBTAllocation* FRayTracingShaderBindingTable::AllocateDynamicRange(ERayTracingShaderBindingLayerMask AllocatedLayers, uint32 SegmentCount)
{	
	ensure(bStaticAllocationsLocked);

	// Don't need lock right now because all dynamic allocation are allocated linearly on the same thread
	// So the FreeDynamicAllocationPool can't be shared with the static allocations right now because those would require a lock
	FRayTracingSBTAllocation* Allocation = nullptr;
	if (NumUsedDynamicAllocations < (uint32)DynamicAllocationPool.Num())
	{
		Allocation = &DynamicAllocationPool[NumUsedDynamicAllocations];
	}
	else
	{
		Allocation = new FRayTracingSBTAllocation();
		DynamicAllocationPool.Add(Allocation);
	}
	NumUsedDynamicAllocations++;

	uint32 LayersCount = FMath::CountBits((uint32)AllocatedLayers);
	uint32 BaseIndex = CurrentDynamicRangeOffset;
	uint32 RecordsPerLayer = SegmentCount * NumShaderSlotsPerGeometrySegment;
	uint32 RecordCount = RecordsPerLayer * LayersCount;
	CurrentDynamicRangeOffset += RecordCount;
	Allocation->InitDynamic(AllocatedLayers, BaseIndex, RecordsPerLayer, RecordCount);

	// Don't need to call MarkDirty(Allocation) here since it will be batched with other allocations using StartDynamicRangeOffsetToMarkDirty

	NumDynamicGeometrySegments += SegmentCount * LayersCount;

	MaxNumDynamicGeometrySegments = FMath::Max(MaxNumDynamicGeometrySegments, NumDynamicGeometrySegments);

	return Allocation;
}

bool FRayTracingShaderBindingTable::IsDirty(uint32 RecordIndex) const
{
	const bool bIsDynamicPendingMarkDirty = RecordIndex >= StartDynamicRangeOffsetToMarkDirty;
	return bIsDynamicPendingMarkDirty || DirtyPersistentRecords[RecordIndex];
}

bool FRayTracingShaderBindingTable::IsPersistent() const
{
	return PersistentSBTInitializer.Lifetime == ERayTracingShaderBindingTableLifetime::Persistent;
}

void FRayTracingShaderBindingTable::FlushAllocationsToClear(FRHICommandList& RHICmdList)
{
	FScopeLock ScopeLock(&StaticAllocationCS);

	// Don't clear outside of current allocated range - could have allocated and cleared ranges before calling CheckPersistentRHI
	// and then the RHI SBT doesn't have those new ranges allocated yet
	uint32 MaxNumValidRecords = PersistentSBTInitializer.NumGeometrySegments * PersistentSBTInitializer.NumShaderSlotsPerGeometrySegment;
	check(MaxNumValidRecords <= (uint32)DirtyPersistentRecords.Num());
	
	uint32 TotalRecordsToClear = PersistentRecordsToClear.CountSetBits();
	if (TotalRecordsToClear == 0)
	{
		return;
	}

	FSceneRenderingBulkObjectAllocator Allocator;
	const uint32 LocalBindingsSize = sizeof(FRayTracingLocalShaderBindings) * TotalRecordsToClear;
	FRayTracingLocalShaderBindings* LocalBindings = (FRayTracingLocalShaderBindings*)(RHICmdList.Bypass()
		? Allocator.Malloc(LocalBindingsSize, alignof(FRayTracingLocalShaderBindings))
	: RHICmdList.Alloc(LocalBindingsSize, alignof(FRayTracingLocalShaderBindings)));

	uint32 LocalBindingIndex = 0;
	for (TConstSetBitIterator<> It(PersistentRecordsToClear); It; ++It)
	{
		uint32 RecordIndexToClear = It.GetIndex();
		if (RecordIndexToClear < MaxNumValidRecords)
		{
			FRayTracingLocalShaderBindings& RecordBindingToClear = LocalBindings[LocalBindingIndex++];
			RecordBindingToClear = FRayTracingLocalShaderBindings();
			RecordBindingToClear.RecordIndex = RecordIndexToClear;
			RecordBindingToClear.BindingType = ERayTracingLocalShaderBindingType::Clear;
		}
	}

	const bool bCopyDataToInlineStorage = false; // Storage is already allocated from RHICmdList, no extra copy necessary
	for (FPersistentSBTData& PersistentSBT : PersistentSBTs)
	{
		if (PersistentSBT.ShaderBindingTable)
		{
			RHICmdList.SetBindingsOnShaderBindingTable(
				PersistentSBT.ShaderBindingTable,
				nullptr,
				LocalBindingIndex, LocalBindings,
				ERayTracingBindingType::HitGroup,
				bCopyDataToInlineStorage);
		}
	}

	PersistentRecordsToClear.Init(false, PersistentRecordsToClear.Num());

	// Only make ranges as free now on the SBT after the clear ops have been processed by the RHI SBT
	// to make sure we don't reuse records which haven't been freed yet (due to out of order freeing of records)
	uint32 FreedRecords = 0;
	for (FRayTracingSBTAllocation* AllocationToFree : PendingStaticAllocationsToFree)
	{
		FreedRecords += AllocationToFree->NumRecords;
		StaticRangeAllocator.Free(AllocationToFree->BaseRecordIndex, AllocationToFree->NumRecords);
		// TODO: pool static allocations as well?
		delete AllocationToFree;
	}
	check(FreedRecords == TotalRecordsToClear);
	PendingStaticAllocationsToFree.Reset();
}

#endif // RHI_RAYTRACING