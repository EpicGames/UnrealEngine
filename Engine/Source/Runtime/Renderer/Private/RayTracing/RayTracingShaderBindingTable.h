// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"
#include "SpanAllocator.h"
#include "RayTracingMeshDrawCommands.h"
#include "RenderGraph.h"

#if RHI_RAYTRACING

enum class ERayTracingShaderBindingLayer : uint8
{
	Base = 0,
	Decals,

	NUM
};

enum class ERayTracingShaderBindingLayerMask
{
	None = 0,
	Base = 1 << (uint32)ERayTracingShaderBindingLayer::Base,
	Decals = 1 << (uint32)ERayTracingShaderBindingLayer::Decals,
	All = Base | Decals,
};

struct FRayTracingSBTAllocation
{
public:

	FRayTracingSBTAllocation() = default;

	bool IsValid() const
	{
		return NumRecords > 0;
	}

	/**
	 * Get the InstanceContributionToHitGroupIndex for the given layer which is stored in the RayTracingInstance data
	 */
	uint32 GetInstanceContributionToHitGroupIndex(ERayTracingShaderBindingLayer Layer) const
	{
		// InstanceContributionToHitGroupIndex is stored at the first segment index because all other segments are directly allocated after this one
		return GetRecordIndex(Layer, 0);
	}

	/**
	 * Get the base SBT record index for the given layer and segment index
	 */
	RENDERER_API uint32 GetRecordIndex(ERayTracingShaderBindingLayer Layer, uint32 SegmentIndex) const;
	RENDERER_API int32 GetSegmentCount() const;
	RENDERER_API int32 GetUniqueId() const;
	bool HasLayer(ERayTracingShaderBindingLayer Layer) const { return EnumHasAnyFlags(ERayTracingShaderBindingLayerMask(1 << (uint32)Layer), AllocatedLayers); }

private:

	void InitStatic(ERayTracingShaderBindingLayerMask InAllocatedLayers, uint32 InBaseRecordIndex, uint32 InRecordsPerLayer, uint32 InNumRecords, const FRHIRayTracingGeometry* InGeometry, const FRayTracingCachedMeshCommandFlags& InFlags)
	{
		check(InAllocatedLayers != ERayTracingShaderBindingLayerMask::None);
		AllocatedLayers = InAllocatedLayers;
		BaseRecordIndex = InBaseRecordIndex;
		RecordsPerLayer = InRecordsPerLayer;
		NumRecords = InNumRecords;
		Geometry = InGeometry;
		Flags = InFlags;
	}

	void InitDynamic(ERayTracingShaderBindingLayerMask InAllocatedLayers, uint32 InBaseRecordIndex, uint32 InRecordsPerLayer, uint32 InNumRecords)
	{
		check(InAllocatedLayers != ERayTracingShaderBindingLayerMask::None);
		AllocatedLayers = InAllocatedLayers;
		BaseRecordIndex = InBaseRecordIndex;
		RecordsPerLayer = InRecordsPerLayer;
		NumRecords = InNumRecords;
	}

	friend class FRayTracingShaderBindingTable;

	uint32 BaseRecordIndex = 0;
	uint32 RecordsPerLayer = 0;
	uint32 NumRecords = 0;
	ERayTracingShaderBindingLayerMask AllocatedLayers;

	// Store the original geometry and flags in the allocation object so it can be used to build the lookup key again used for deduplication
	const FRHIRayTracingGeometry* Geometry = nullptr;
	FRayTracingCachedMeshCommandFlags Flags;
	bool bDeduplicate = false;
};

using FRayTracingPersistentShaderBindingTableID = uint32;

/**
* Shader binding table use for raytracing
*/
class FRayTracingShaderBindingTable
{
public:
			
	RENDERER_API FRayTracingShaderBindingTable();
	RENDERER_API ~FRayTracingShaderBindingTable();
	
	/**
	 * Retrive the current SBT version - needed to collect the current dirty shader bindings for all the persistent SBTs
	 */
	RENDERER_API FRayTracingShaderBindingDataOneFrameArray GetDirtyBindings(TConstArrayView<FRayTracingShaderBindingData> VisibleBindings, bool bForceAllDirty);

	/**
	 * Allocate persistent SBT ID - can be used to retrieve the rhi object during rendering
	 */	
	RENDERER_API FRayTracingPersistentShaderBindingTableID AllocatePersistentSBTID(FRHICommandListBase& RHICmdList, ERayTracingShaderBindingMode ShaderBindingMode);
	
	/**
	 * Release previously allocated persistent SBT ID
	 */	
	void ReleasePersistentSBT(FRayTracingPersistentShaderBindingTableID PersistentSBTID)
	{
		// Just release the RHI object so the slot can be reused for the next persistent SBT
		PersistentSBTs[PersistentSBTID].Reset();
	}
		
	FRHIShaderBindingTable* GetPersistentSBT(FRayTracingPersistentShaderBindingTableID PersistentSBTID) { return PersistentSBTs[PersistentSBTID].ShaderBindingTable; }
	FRDGBufferRef GetPersistentInlineBindingDataBuffer(FRDGBuilder& GraphBuilder, FRayTracingPersistentShaderBindingTableID PersistentSBTID) { return GraphBuilder.RegisterExternalBuffer(PersistentSBTs[PersistentSBTID].InlineBindingDataPooledBuffer); }

	FRHIShaderResourceView* GetPersistentInlineBindingDataBufferSRV(FRayTracingPersistentShaderBindingTableID PersistentSBTID) const
	{ 
		FRHIShaderResourceView* InlineBindingDataBufferSRV = nullptr;
		if (PersistentSBTs.IsValidIndex(PersistentSBTID))
		{
			InlineBindingDataBufferSRV = PersistentSBTs[PersistentSBTID].InlineBindingDataPooledBuffer->GetSRV();
		}
		return InlineBindingDataBufferSRV;
	}

	uint32 GetNumShaderSlotsPerSegment() const { return NumShaderSlotsPerGeometrySegment; }

	/**
	 * Get the persistent SBT RHI shader binding table for rendering and might recreate the RHI object if needed (will mark all currently cached bindings as dirty in all persistent SBTs on recreate)
	 */	
	RENDERER_API void CheckPersistentRHI(FRHICommandListBase& RHICmdList, uint32 MaxLocalBindingDataSize);
		 
	/**
	 * Allocate RHI shader binding table which can contain all static allocations and all current dynamic allocations - transient single frame SBT
	 */
	RENDERER_API FShaderBindingTableRHIRef AllocateTransientRHI(
		FRHICommandListBase& RHICmdList, 
		ERayTracingShaderBindingMode ShaderBindingMode, 
		ERayTracingHitGroupIndexingMode HitGroupIndexingMode,
		uint32 LocalBindingDataSize) const;
	 
	/**
	 * Get the total number of allocated geometry segments (static and dynamic)
	 */
	RENDERER_API uint32 GetNumGeometrySegments() const;

	/**
	 * Allocate single static range of records for the given SegmentCount for all layers in the AllocatedLayersMask
	 */
	FRayTracingSBTAllocation* AllocateStaticRange(ERayTracingShaderBindingLayerMask AllocatedLayers, uint32 SegmentCount)
	{
		FScopeLock ScopeLock(&StaticAllocationCS);
		FRayTracingCachedMeshCommandFlags DefaultFlags;
		return AllocateStaticRangeInternal(AllocatedLayers, SegmentCount, nullptr, DefaultFlags);
	}
	
	/**
	 * Allocate or share static allocation range - sharing can happen if geometry and cached RT MDC flags are the same (will result in exactly the same binding data written in the SBT)
	 */
	RENDERER_API FRayTracingSBTAllocation* AllocateStaticRange(uint32 SegmentCount, const FRHIRayTracingGeometry* Geometry, FRayTracingCachedMeshCommandFlags Flags);
	RENDERER_API void FreeStaticRange(FRayTracingSBTAllocation* Allocation);
	 
	/**
	 * Allocate dynamic SBT range which can be reused again when ResetDynamicAllocationData is called
	 */	
	RENDERER_API FRayTracingSBTAllocation* AllocateDynamicRange(ERayTracingShaderBindingLayerMask AllocatedLayers, uint32 SegmentCount);

	/**
	 * Mark all currently allocated dynamic ranges as free again so they can be allocated
	 */		
	RENDERER_API void ResetDynamicAllocationData();

	/**
	 * Resets the arrays and counters of miss and callable shaders since they're not persistent
	 */
	void ResetMissAndCallableShaders();

	/**
	 * Clears data / resources tied to a single frame.
	 */
	void EndFrame();

	/**
	 * Reset the static allocation lock again (used for validation)
	 */
	void ResetStaticAllocationLock()
	{
		bStaticAllocationsLocked = false;
	}
	
	/**
	 * Flush all pending persistent allocation to clear to cached persistent SBTs
	 */	
	RENDERER_API void FlushAllocationsToClear(FRHICommandList& RHICmdList);

	/**
	 * Check if given record index is dirty - used for validation
	 */	
	bool IsDirty(uint32 RecordIndex) const;

	/**
	 * Check if SBTs are persistently allocated
	 */
	bool IsPersistent() const;

public:

	uint32 NumMissShaderSlots = 1; // we must have a default miss shader, so always include it from the start
	uint32 NumCallableShaderSlots = 0;
	TArray<FRayTracingShaderCommand> CallableCommands;

	// Helper array to hold references to single frame uniform buffers used in SBTs
	TArray<FUniformBufferRHIRef> TransientUniformBuffers;

private:

	/**
	 * Get the maximum amount of static allocated segments (highest allocation index with free ranges included)
	 */
	uint32 GetMaxAllocatedStaticSegmentCount() const;
	
	/**
	 * Allocate persistent SBT and optional inline binding data possibly overwriting the old data
	 */	
	void AllocatePersistentShaderBindingTable(FRHICommandListBase& RHICmdList, FRayTracingPersistentShaderBindingTableID PersistentSBTID, ERayTracingShaderBindingMode ShaderBindingMode);

	/**
	 * Allocate single static range of records for the given SegmentCount for all layers in the AllocatedLayersMask
	 */
	RENDERER_API FRayTracingSBTAllocation* AllocateStaticRangeInternal(ERayTracingShaderBindingLayerMask AllocatedLayers, uint32 SegmentCount, const FRHIRayTracingGeometry* Geometry, FRayTracingCachedMeshCommandFlags Flags);

	/**
	 * Mark all records used by the SBTAllocation as dirty again
	 */
	void MarkDirty(FRayTracingSBTAllocation* SBTAllocation);
	void MarkSet(FRayTracingSBTAllocation* SBTAllocation, bool bValue);

	struct FAllocationKey
	{
		const FRHIRayTracingGeometry* Geometry;
		FRayTracingCachedMeshCommandFlags Flags;
		uint32 SegmentCount = 0;

		bool operator==(const FAllocationKey& Other) const
		{
			return Geometry == Other.Geometry
				&& Flags == Other.Flags
				&& SegmentCount == Other.SegmentCount;
		}

		bool operator!=(const FAllocationKey& Other) const
		{
			return !(*this == Other);
		}

		friend uint32 GetTypeHash(const FAllocationKey& Key)
		{
			return HashCombine(HashCombine(GetTypeHash(Key.Geometry), GetTypeHash(Key.Flags)), GetTypeHash(Key.SegmentCount));
		}
	};
	
	struct FRefCountedAllocation
	{
		FRayTracingSBTAllocation* Allocation;
		uint32 RefCount = 0;
	};

	struct FPersistentSBTData
	{
		void Reset()
		{
			ShaderBindingMode = ERayTracingShaderBindingMode::Disabled;
			ShaderBindingTable = nullptr;
			InlineBindingDataPooledBuffer = nullptr;
		}

		ERayTracingShaderBindingMode ShaderBindingMode = ERayTracingShaderBindingMode::Disabled;
		FShaderBindingTableRHIRef ShaderBindingTable;					//< Actual persistent RHI shader binding table
		TRefCountPtr<FRDGPooledBuffer> InlineBindingDataPooledBuffer;	//< Optional inline binding data buffer - size is retrieved from the RHI SBT after creation
	};

	const uint32 NumShaderSlotsPerGeometrySegment = 0;					//< Number of slots per geometry segment (engine wide fixed)

	FRayTracingShaderBindingTableInitializer PersistentSBTInitializer;	 //< Shared initializer used for all persistent SBTs - so they can all be versioned together
	TArray<FPersistentSBTData> PersistentSBTs;							 //< All currently allocated persistent SBTs (FRayTracingPersistentShaderBindingTableID contains index into this array so can be sparse)
	TBitArray<> SetPersistentRecords;									 //< BitArray containing which bits are valid in the cached persistent SBTs (use for validation)
	TBitArray<> DirtyPersistentRecords;									 //< BitArray containing which bits are dirty in the cached persistent SBTs (used during dirty binding collection from currently visible bindings)
	TBitArray<> PersistentRecordsToClear;								 //< BitArray containing which bits need to be cleared in the persisten SBTs
	
	TArray<FRayTracingSBTAllocation*> PendingStaticAllocationsToFree;	 //< All pending allocations to free during next flush

	FCriticalSection StaticAllocationCS;								 //< Critical section used to access all static allocation data
	bool bStaticAllocationsLocked = false;								 //< Static allocations are not allowed when this bool is set (used for validation)
	FSpanAllocator StaticRangeAllocator;								 //< Range allocator to find free static record ranges
	TMap<FAllocationKey, FRefCountedAllocation> TrackedAllocationMap;	 //< All static allocation with refcount tracking
	
	TIndirectArray<FRayTracingSBTAllocation> DynamicAllocationPool;      //< Dynamic allocation pool (for faster allocations)
	uint32 NumUsedDynamicAllocations = 0;                                //< Number of dynamic allocations currently used

	uint32 TotalStaticAllocationCount = 0;								 //< Total amount of static allocations (without deduplications)
	uint32 AllocatedStaticSegmentCount = 0;								 //< Total amount of allocated static segments (with deduplication)
		
	uint32 MaxNumDynamicGeometrySegments = 0;							 //< Maximum number of allocated dynamic segments required (peek number)
	uint32 NumDynamicGeometrySegments = 0;								 //< Current number of allocated dynamic segments
	uint32 StartDynamicRangeOffset = 0;								 	 //< Start SBT record offset for the first dynamic allocation
	uint32 CurrentDynamicRangeOffset = 0;								 //< Current working SBT record offset for the next dynamic allocation
	uint32 StartDynamicRangeOffsetToMarkDirty = 0;						 //< Start SBT record offset that needs to be marked dirty
};

#endif // RHI_RAYTRACING

