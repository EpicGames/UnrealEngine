// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingShadows.h"
#include "DeferredShadingRenderer.h"
#include "PostProcess/SceneRenderTargets.h"
#include "ScenePrivate.h"
#include "RayTracing.h"

#if RHI_RAYTRACING

#include "ClearQuad.h"
#include "LightSceneProxy.h"
#include "SceneRendering.h"
#include "RenderGraphBuilder.h"
#include "RenderTargetPool.h"
#include "RHIResources.h"
#include "UniformBuffer.h"
#include "VisualizeTexture.h"
#include "RayGenShaderUtils.h"
#include "RaytracingOptions.h"
#include "BuiltInRayTracingShaders.h"
#include "RayTracing/RayTracingMaterialHitShaders.h"
#include "Containers/DynamicRHIResourceArray.h"
#include "SceneTextureParameters.h"
#include "RayTracingDefinitions.h"
#include "Nanite/NaniteRayTracing.h"

static float GRayTracingMaxNormalBias = 0.1f;
static FAutoConsoleVariableRef CVarRayTracingNormalBias(
	TEXT("r.RayTracing.NormalBias"),
	GRayTracingMaxNormalBias,
	TEXT("Sets the max. normal bias used for offseting the ray start position along the normal (default = 0.1, i.e., 1mm)")
);

static int32 GRayTracingShadowsEnableMaterials = 1;
static FAutoConsoleVariableRef CVarRayTracingShadowsEnableMaterials(
	TEXT("r.RayTracing.Shadows.EnableMaterials"),
	GRayTracingShadowsEnableMaterials,
	TEXT("Enables material shader binding for shadow rays. If this is disabled, then a default trivial shader is used. (default = 1)"),
	ECVF_RenderThreadSafe
);

static float GRayTracingShadowsAvoidSelfIntersectionTraceDistance = 1.0f;
static FAutoConsoleVariableRef CVarRayTracingShadowsAvoidSelfIntersectionTraceDistance(
	TEXT("r.RayTracing.Shadows.AvoidSelfIntersectionTraceDistance"),
	GRayTracingShadowsAvoidSelfIntersectionTraceDistance,
	TEXT("Max trace distance of epsilon trace to avoid self intersections. If set to 0, epsilon trace will not be used.")
);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsEnableTwoSidedGeometry(
	TEXT("r.RayTracing.Shadows.EnableTwoSidedGeometry"),
	1,
	TEXT("Enables two-sided geometry when tracing shadow rays (default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingTransmissionSamplingDistanceCulling(
	TEXT("r.RayTracing.Transmission.TransmissionSamplingDistanceCulling"),
	1,
	TEXT("Enables visibility testing to cull transmission sampling distance (default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingTransmissionSamplingTechnique(
	TEXT("r.RayTracing.Transmission.SamplingTechnique"),
	1,
	TEXT("0: Uses constant tracking of an infinite homogeneous medium\n")
	TEXT("1: Uses constant tracking of a finite homogeneous medium whose extent is determined by transmission sampling distance (default)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable CVarRayTracingTransmissionMeanFreePathType(
	TEXT("r.RayTracing.Transmission.MeanFreePathType"),
	0,
	TEXT("0: Use the extinction scale from subsurface profile as MFP.")
	TEXT("1: Use the max MFP from Subsurface profile to generate samples for transmission (Substrate is not supported)."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingTransmissionRejectionSamplingTrials(
	TEXT("r.RayTracing.Transmission.RejectionSamplingTrials"),
	0,
	TEXT("Determines the number of rejection-sampling trials (default = 0)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsEnableHairVoxel(
	TEXT("r.RayTracing.Shadows.EnableHairVoxel"),
	1,
	TEXT("Enables use of hair voxel data for tracing shadow (default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsLODTransitionStart(
	TEXT("r.RayTracing.Shadows.LODTransitionStart"),
	4000.0, // 40 m
	TEXT("The start of an LOD transition range (default = 4000)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsLODTransitionEnd(
	TEXT("r.RayTracing.Shadows.LODTransitionEnd"),
	5000.0f, // 50 m
	TEXT("The end of an LOD transition range (default = 5000)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsAcceptFirstHit(
	TEXT("r.RayTracing.Shadows.AcceptFirstHit"),
	1,
	TEXT("Whether to allow shadow rays to terminate early, on first intersected primitive. This may result in worse denoising quality in some cases. (default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsTranslucency(
	TEXT("r.RayTracing.Shadows.Translucency"),
	0,
	TEXT("0: Translucent material will not cast shadow (by default).")
	TEXT("1: Translucent material cast approximate translucent shadows based on opacity (Very expensive)."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingShadowsMaxTranslucencyHitCount(
	TEXT("r.RayTracing.Shadows.MaxTranslucencyHitCount"),
	-1,
	TEXT("-1: Evaluate all intersections (default).")
	TEXT(" 0: Disable translucent shadow testing.")
	TEXT(">0: Limit the number of intersections."),
	ECVF_RenderThreadSafe
);

void RayTracingShadows::SetRayTracingSceneOptions(bool bSceneHasLightsWithRayTracedShadows, RayTracing::FSceneOptions& SceneOptions)
{
	if (bSceneHasLightsWithRayTracedShadows && CVarRayTracingShadowsTranslucency.GetValueOnRenderThread() != 0)
	{
		SceneOptions.bTranslucentGeometry = true;
	}
};

int32 GetRayTracingShadowsMaxTranslucencyHitCount()
{
	return CVarRayTracingShadowsMaxTranslucencyHitCount.GetValueOnRenderThread();
}

bool EnableRayTracingShadowTwoSidedGeometry()
{
	return CVarRayTracingShadowsEnableTwoSidedGeometry.GetValueOnRenderThread() != 0;
}

uint32 GetRayTracingTransmissionMeanFreePathType()
{	
	uint32 MeanFreePathType = FMath::Clamp(CVarRayTracingTransmissionMeanFreePathType.GetValueOnRenderThread(), 0, 1);

	if (Substrate::IsSubstrateEnabled())
	{
		MeanFreePathType = 0u;
	}

	return MeanFreePathType;
}

class FOcclusionRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOcclusionRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FOcclusionRGS, FGlobalShader)

	class FLightTypeDim : SHADER_PERMUTATION_INT("LIGHT_TYPE", LightType_MAX);
	class FAvoidSelfIntersectionTraceDim : SHADER_PERMUTATION_BOOL("AVOID_SELF_INTERSECTION_TRACE");
	class FDenoiserOutputDim : SHADER_PERMUTATION_INT("DIM_DENOISER_OUTPUT", 3);
	class FEnableMultipleSamplesPerPixel : SHADER_PERMUTATION_BOOL("ENABLE_MULTIPLE_SAMPLES_PER_PIXEL");
	class FHairLighting : SHADER_PERMUTATION_INT("USE_HAIR_LIGHTING", 2);
	class FEnableTransmissionDim : SHADER_PERMUTATION_INT("ENABLE_TRANSMISSION", 2);

	using FPermutationDomain = TShaderPermutationDomain<FLightTypeDim, FAvoidSelfIntersectionTraceDim, FDenoiserOutputDim, FHairLighting, FEnableMultipleSamplesPerPixel, FEnableTransmissionDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("UE_RAY_TRACING_DYNAMIC_CLOSEST_HIT_SHADER"), 0);
		OutEnvironment.SetDefine(TEXT("UE_RAY_TRACING_DYNAMIC_ANY_HIT_SHADER"),     1);
		OutEnvironment.SetDefine(TEXT("UE_RAY_TRACING_DYNAMIC_MISS_SHADER"),        0);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FLightTypeDim>() == LightType_Directional &&
			PermutationVector.Get<FHairLighting>() == 0)
		{
			OutEnvironment.SetDefine(TEXT("UE_RAY_TRACING_COHERENT_RAYS"), 1);
		}
		else
		{
			OutEnvironment.SetDefine(TEXT("UE_RAY_TRACING_COHERENT_RAYS"), 0);
		}
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER(uint32, SamplesPerPixel)
		SHADER_PARAMETER(float, NormalBias)
		SHADER_PARAMETER(uint32, LightingChannelMask)
		SHADER_PARAMETER(FIntRect, LightScissor)
		SHADER_PARAMETER(FIntPoint, PixelOffset)
		SHADER_PARAMETER(uint32, bUseHairVoxel)
		SHADER_PARAMETER(float, TraceDistance)
		SHADER_PARAMETER(float, LODTransitionStart)
		SHADER_PARAMETER(float, LODTransitionEnd)
		SHADER_PARAMETER(float, AvoidSelfIntersectionTraceDistance)
		SHADER_PARAMETER(uint32, bTransmissionSamplingDistanceCulling)
		SHADER_PARAMETER(uint32, TransmissionSamplingTechnique)
		SHADER_PARAMETER(uint32, TransmissionMeanFreePathType)
		SHADER_PARAMETER(uint32, RejectionSamplingTrials)
		SHADER_PARAMETER(uint32, bAcceptFirstHit)
		SHADER_PARAMETER(uint32, bTwoSidedGeometry)
		SHADER_PARAMETER(uint32, bTranslucentShadow)
		SHADER_PARAMETER(uint32, MaxTranslucencyHitCount)

		SHADER_PARAMETER_STRUCT(FLightShaderParameters, Light)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneLightingChannelParameters, SceneLightingChannels)

		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HairLightChannelMaskTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWOcclusionMaskUAV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, RWRayDistanceUAV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWSubPixelOcclusionMaskUAV)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FHairStrandsViewUniformParameters, HairStrands)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualVoxelParameters, VirtualVoxel)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FOcclusionRGS, "/Engine/Private/RayTracing/RayTracingOcclusionRGS.usf", "OcclusionRGS", SF_RayGen);

class FOcclusionCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOcclusionCS)
	SHADER_USE_PARAMETER_STRUCT(FOcclusionCS, FGlobalShader)

	class FLightTypeDim : SHADER_PERMUTATION_INT("LIGHT_TYPE", LightType_MAX);
	class FDenoiserOutputDim : SHADER_PERMUTATION_INT("DIM_DENOISER_OUTPUT", 3);
	class FEnableMultipleSamplesPerPixel : SHADER_PERMUTATION_BOOL("ENABLE_MULTIPLE_SAMPLES_PER_PIXEL");
	class FHairLighting : SHADER_PERMUTATION_INT("USE_HAIR_LIGHTING", 2);
	class FEnableTransmissionDim : SHADER_PERMUTATION_INT("ENABLE_TRANSMISSION", 2);

	using FPermutationDomain = TShaderPermutationDomain<FLightTypeDim, FDenoiserOutputDim, FHairLighting, FEnableMultipleSamplesPerPixel, FEnableTransmissionDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform) && FDataDrivenShaderPlatformInfo::GetSupportsInlineRayTracing(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.CompilerFlags.Add(CFLAG_Wave32);
		OutEnvironment.CompilerFlags.Add(CFLAG_InlineRayTracing);

		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("INLINE_RAY_TRACING_THREAD_GROUP_SIZE_Y"), ThreadGroupSizeY);

		FPermutationDomain PermutationVector(Parameters.PermutationId);
		if (PermutationVector.Get<FLightTypeDim>() == LightType_Directional &&
			PermutationVector.Get<FHairLighting>() == 0)
		{
			OutEnvironment.SetDefine(TEXT("UE_RAY_TRACING_COHERENT_RAYS"), 1);
		}
		else
		{
			OutEnvironment.SetDefine(TEXT("UE_RAY_TRACING_COHERENT_RAYS"), 0);
		}
	}

	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		// Disable precaching for occlusion CS - should use project setting if it's used?
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FOcclusionRGS::FParameters, CommonParameters)
	END_SHADER_PARAMETER_STRUCT()

	// Current inline ray tracing implementation requires 1:1 mapping between thread groups and waves and only supports wave32 mode.
	static constexpr uint32 ThreadGroupSizeX = 8;
	static constexpr uint32 ThreadGroupSizeY = 4;

};

IMPLEMENT_GLOBAL_SHADER(FOcclusionCS, "/Engine/Private/RayTracing/RayTracingOcclusionRGS.usf", "OcclusionCS", SF_Compute);

float GetRaytracingMaxNormalBias()
{
	return FMath::Max(0.01f, GRayTracingMaxNormalBias);
}

FOcclusionCS::FPermutationDomain ToComputePermutationVector(const FOcclusionRGS::FPermutationDomain& RGSPermutationVector)
{
	FOcclusionCS::FPermutationDomain PermutationVector;

	PermutationVector.Set<FOcclusionCS::FLightTypeDim>(RGSPermutationVector.Get<FOcclusionRGS::FLightTypeDim>());
	PermutationVector.Set<FOcclusionCS::FDenoiserOutputDim>(RGSPermutationVector.Get<FOcclusionRGS::FDenoiserOutputDim>());
	PermutationVector.Set<FOcclusionCS::FEnableMultipleSamplesPerPixel>(RGSPermutationVector.Get<FOcclusionRGS::FEnableMultipleSamplesPerPixel>());
	PermutationVector.Set<FOcclusionCS::FHairLighting>(RGSPermutationVector.Get<FOcclusionRGS::FHairLighting>());
	PermutationVector.Set<FOcclusionCS::FEnableTransmissionDim>(RGSPermutationVector.Get<FOcclusionRGS::FEnableTransmissionDim>());

	return PermutationVector;
}

void FDeferredShadingSceneRenderer::PrepareRayTracingShadows(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders)
{
	// Ray tracing shadows shaders should be properly configured even if r.RayTracing.Shadows is 0 because lights can have raytracing shadows enabled independently of that CVar
	// We have to check if ray tracing is enabled on any of the scene lights. The Scene.bHasRayTracedLights is computed using ShouldRenderRayTracingShadowsForLight() helper, 
	// which handles various override conditions.

	if (!ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::FullPipeline, View))
	{
		return;
	}

	if (!View.bHasRayTracingShadows)
	{
		return;
	}

	const IScreenSpaceDenoiser::EShadowRequirements DenoiserRequirements[] =
	{
		IScreenSpaceDenoiser::EShadowRequirements::Bailout,
		IScreenSpaceDenoiser::EShadowRequirements::PenumbraAndAvgOccluder,
		IScreenSpaceDenoiser::EShadowRequirements::PenumbraAndClosestOccluder,
	};

	for (int32 MultiSPP = 0; MultiSPP < 2; ++MultiSPP)
	{
		for (int32 EnableTransmissionDim = 0; EnableTransmissionDim < 2; ++EnableTransmissionDim)
		{
			for (int32 HairLighting = 0; HairLighting < 2; ++HairLighting)
			{
				for (int32 AvoidSelfIntersectionTrace = 0; AvoidSelfIntersectionTrace < 2; ++AvoidSelfIntersectionTrace)
				{
					for (int32 LightType = 0; LightType < LightType_MAX; ++LightType)
					{
						for (IScreenSpaceDenoiser::EShadowRequirements DenoiserRequirement : DenoiserRequirements)
						{
							FOcclusionRGS::FPermutationDomain PermutationVector;
							PermutationVector.Set<FOcclusionRGS::FLightTypeDim>(LightType);
							PermutationVector.Set<FOcclusionRGS::FAvoidSelfIntersectionTraceDim>((bool)AvoidSelfIntersectionTrace);
							PermutationVector.Set<FOcclusionRGS::FDenoiserOutputDim>((int32)DenoiserRequirement);
							PermutationVector.Set<FOcclusionRGS::FHairLighting>(HairLighting);
							PermutationVector.Set<FOcclusionRGS::FEnableMultipleSamplesPerPixel>(MultiSPP != 0);
							PermutationVector.Set<FOcclusionRGS::FEnableTransmissionDim>(EnableTransmissionDim);

							TShaderMapRef<FOcclusionRGS> RayGenerationShader(View.ShaderMap, PermutationVector);
							OutRayGenShaders.Add(RayGenerationShader.GetRayTracingShader());
						}
					}
				}
			}
		}
	}
}

#endif // RHI_RAYTRACING

void FDeferredShadingSceneRenderer::RenderRayTracingShadows(
	FRDGBuilder& GraphBuilder,
	const FSceneTextureParameters& SceneTextures,
	const FViewInfo& View,
	const FLightSceneInfo& LightSceneInfo,
	const IScreenSpaceDenoiser::FShadowRayTracingConfig& RayTracingConfig,
	const IScreenSpaceDenoiser::EShadowRequirements DenoiserRequirements,
	FRDGTextureRef LightingChannelsTexture,
	FRDGTextureUAV* OutShadowMaskUAV,
	FRDGTextureUAV* OutRayHitDistanceUAV,
	FRDGTextureUAV* SubPixelRayTracingShadowMaskUAV)
#if RHI_RAYTRACING
{
	FLightSceneProxy* LightSceneProxy = LightSceneInfo.Proxy;
	check(LightSceneProxy);

	const bool bInlineRayTracing = !GRHISupportsRayTracingShaders && GRHISupportsInlineRayTracing;

	FIntRect ScissorRect = View.ViewRect;
	FIntPoint PixelOffset = { 0, 0 };

	//#UE-95409: Implement support for scissor in multi-view 
	const bool bClipDispatch = View.Family->Views.Num() == 1;

	if (LightSceneProxy->GetScissorRect(ScissorRect, View, View.ViewRect))
	{
		// Account for scissor being defined on the whole frame viewport while the trace is only on the view subrect
		// ScissorRect.Min = ScissorRect.Min;
		// ScissorRect.Max = ScissorRect.Max;
	}
	else
	{
		ScissorRect = View.ViewRect;
	}

	if (bClipDispatch)
	{
		PixelOffset = ScissorRect.Min;
	}

	// Ray generation pass for shadow occlusion.
	{
		const bool bUseHairLighting   = HairStrands::HasViewHairStrandsData(View) && HairStrands::HasViewHairStrandsVoxelData(View);
		const bool bUseHairDeepShadow = HairStrands::HasViewHairStrandsData(View) && LightSceneProxy->CastsHairStrandsDeepShadow();

		FOcclusionRGS::FParameters* CommonPassParameters = GraphBuilder.AllocParameters<FOcclusionRGS::FParameters>();

		CommonPassParameters->RWOcclusionMaskUAV = OutShadowMaskUAV;
		CommonPassParameters->RWRayDistanceUAV = OutRayHitDistanceUAV;
		CommonPassParameters->RWSubPixelOcclusionMaskUAV = SubPixelRayTracingShadowMaskUAV;
		CommonPassParameters->SamplesPerPixel = RayTracingConfig.RayCountPerPixel;
		CommonPassParameters->NormalBias = GetRaytracingMaxNormalBias();
		CommonPassParameters->LightingChannelMask = LightSceneProxy->GetLightingChannelMask();
		
		{
			FLightRenderParameters LightParameters;
			LightSceneProxy->GetLightShaderParameters(LightParameters);
			LightParameters.MakeShaderParameters(View.ViewMatrices, View.GetLastEyeAdaptationExposure(), CommonPassParameters->Light);
			CommonPassParameters->Light.SourceRadius *= LightSceneProxy->GetShadowSourceAngleFactor();
		}

		CommonPassParameters->TraceDistance = LightSceneProxy->GetTraceDistance();
		CommonPassParameters->LODTransitionStart = CVarRayTracingShadowsLODTransitionStart.GetValueOnRenderThread();
		CommonPassParameters->LODTransitionEnd = CVarRayTracingShadowsLODTransitionEnd.GetValueOnRenderThread();
		CommonPassParameters->AvoidSelfIntersectionTraceDistance = GRayTracingShadowsAvoidSelfIntersectionTraceDistance;
		CommonPassParameters->bAcceptFirstHit = CVarRayTracingShadowsAcceptFirstHit.GetValueOnRenderThread();
		CommonPassParameters->bTwoSidedGeometry = EnableRayTracingShadowTwoSidedGeometry() ? 1 : 0;
		CommonPassParameters->bTranslucentShadow = CVarRayTracingShadowsTranslucency.GetValueOnRenderThread() != 0;
		CommonPassParameters->MaxTranslucencyHitCount = GetRayTracingShadowsMaxTranslucencyHitCount();
		CommonPassParameters->TLAS = View.GetRayTracingSceneLayerViewChecked(ERayTracingSceneLayer::Base);
		CommonPassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		CommonPassParameters->SceneTextures = SceneTextures;
		CommonPassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder);
		CommonPassParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();
		CommonPassParameters->SceneLightingChannels = GetSceneLightingChannelParameters(GraphBuilder, View, LightingChannelsTexture);
		CommonPassParameters->LightScissor = ScissorRect;
		CommonPassParameters->PixelOffset = PixelOffset;
		CommonPassParameters->bTransmissionSamplingDistanceCulling = CVarRayTracingTransmissionSamplingDistanceCulling.GetValueOnRenderThread();
		CommonPassParameters->TransmissionSamplingTechnique = CVarRayTracingTransmissionSamplingTechnique.GetValueOnRenderThread();
		CommonPassParameters->TransmissionMeanFreePathType = GetRayTracingTransmissionMeanFreePathType();
		CommonPassParameters->RejectionSamplingTrials = CVarRayTracingTransmissionRejectionSamplingTrials.GetValueOnRenderThread();
		CommonPassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

		if (bUseHairLighting)
		{
			const bool bUseHairVoxel = CVarRayTracingShadowsEnableHairVoxel.GetValueOnRenderThread() > 0;
			CommonPassParameters->bUseHairVoxel = !bUseHairDeepShadow && bUseHairVoxel ? 1 : 0;
			CommonPassParameters->HairLightChannelMaskTexture = View.HairStrandsViewData.VisibilityData.LightChannelMaskTexture;
			CommonPassParameters->HairStrands = HairStrands::BindHairStrandsViewUniformParameters(View);
			CommonPassParameters->VirtualVoxel = HairStrands::BindHairStrandsVoxelUniformParameters(View);

			if (ShaderPrint::IsValid(View.ShaderPrintData))
			{
				ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, CommonPassParameters->ShaderPrintParameters);
			}
		}

		FOcclusionRGS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FOcclusionRGS::FLightTypeDim>(LightSceneProxy->GetLightType());
		PermutationVector.Set<FOcclusionRGS::FAvoidSelfIntersectionTraceDim>(GRayTracingShadowsAvoidSelfIntersectionTraceDistance > 0.0f);
		if (DenoiserRequirements == IScreenSpaceDenoiser::EShadowRequirements::PenumbraAndAvgOccluder)
		{
			PermutationVector.Set<FOcclusionRGS::FDenoiserOutputDim>(1);
		}
		else if (DenoiserRequirements == IScreenSpaceDenoiser::EShadowRequirements::PenumbraAndClosestOccluder)
		{
			PermutationVector.Set<FOcclusionRGS::FDenoiserOutputDim>(2);
		}
		else
		{
			PermutationVector.Set<FOcclusionRGS::FDenoiserOutputDim>(0);
		}
		PermutationVector.Set<FOcclusionRGS::FHairLighting>(bUseHairLighting? 1 : 0);

		PermutationVector.Set<FOcclusionRGS::FEnableMultipleSamplesPerPixel>(RayTracingConfig.RayCountPerPixel > 1);
		PermutationVector.Set<FOcclusionRGS::FEnableTransmissionDim>(LightSceneProxy->Transmission() ? 1 : 0);

		if (bInlineRayTracing)
		{
			FOcclusionCS::FParameters* InlinePassParameters = GraphBuilder.AllocParameters<FOcclusionCS::FParameters>();

			InlinePassParameters->CommonParameters = *CommonPassParameters;

			TShaderRef<FOcclusionCS> ComputeShader = View.ShaderMap->GetShader<FOcclusionCS>(ToComputePermutationVector(PermutationVector));

			FIntPoint Resolution(View.ViewRect.Width(), View.ViewRect.Height());

			if (bClipDispatch)
			{
				Resolution = ScissorRect.Size();
			}

			const FIntPoint GroupSize(FOcclusionCS::ThreadGroupSizeX, FOcclusionCS::ThreadGroupSizeY);
			const FIntVector GroupCount = FComputeShaderUtils::GetGroupCount(View.ViewRect.Size(), GroupSize);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("RayTracedShadow (INLINE) (spp=%d) %dx%d", RayTracingConfig.RayCountPerPixel, Resolution.X, Resolution.Y),
				InlinePassParameters,
				ERDGPassFlags::Compute,
				[InlinePassParameters, ComputeShader, GroupCount](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
				{
					FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *InlinePassParameters, GroupCount);
				});

			//FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RayTracedShadow (INLINE) (spp=%d) %dx%d", RayTracingConfig.RayCountPerPixel, Resolution.X, Resolution.Y), ComputeShader, InlinePassParameters, GroupCount);
		}
		else
		{
			const FRayTracingScene& RayTracingScene = Scene->RayTracingScene;

			TShaderMapRef<FOcclusionRGS> RayGenerationShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);

			ClearUnusedGraphResources(RayGenerationShader, CommonPassParameters);

			FIntPoint Resolution(View.ViewRect.Width(), View.ViewRect.Height());

			if (bClipDispatch)
			{
				Resolution = ScissorRect.Size();
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("RayTracedShadow (spp=%d) %dx%d", RayTracingConfig.RayCountPerPixel, Resolution.X, Resolution.Y),
				CommonPassParameters,
				ERDGPassFlags::Compute,
				[this, &View, RayGenerationShader, CommonPassParameters, Resolution](FRHICommandList& RHICmdList)
				{
					FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
					SetShaderParameters(GlobalResources, RayGenerationShader, *CommonPassParameters);

					FRHIUniformBuffer* SceneUniformBuffer = CommonPassParameters->Scene->GetRHI();
					FRHIUniformBuffer* NaniteRayTracingUniformBuffer = CommonPassParameters->NaniteRayTracing->GetRHI();
					TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

					if (GRayTracingShadowsEnableMaterials)
					{
						RHICmdList.RayTraceDispatch(View.MaterialRayTracingData.PipelineState, RayGenerationShader.GetRayTracingShader(), View.MaterialRayTracingData.ShaderBindingTable, GlobalResources, Resolution.X, Resolution.Y);
					}
					else
					{
						FRayTracingPipelineStateInitializer Initializer;

						Initializer.MaxPayloadSizeInBytes = GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType::RayTracingMaterial);
						
						const FShaderBindingLayout* ShaderBindingLayout = RayTracing::GetShaderBindingLayout(ShaderPlatform);
						if (ShaderBindingLayout)
						{
							Initializer.ShaderBindingLayout = &ShaderBindingLayout->RHILayout;
						}

						FRHIRayTracingShader* RayGenShaderTable[] = { RayGenerationShader.GetRayTracingShader() };
						Initializer.SetRayGenShaderTable(RayGenShaderTable);

						FRHIRayTracingShader* HitGroupTable[] = { GetRayTracingDefaultOpaqueShader(View.ShaderMap) };
						Initializer.SetHitGroupTable(HitGroupTable);

						FRHIRayTracingShader* MissGroupTable[] = { GetRayTracingDefaultMissShader(View.ShaderMap) };
						Initializer.SetMissShaderTable(MissGroupTable);

						FRayTracingPipelineState* Pipeline = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(RHICmdList, Initializer);

						FShaderBindingTableRHIRef SBT = Scene->RayTracingSBT.AllocateTransientRHI(RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Disallow, Initializer.GetMaxLocalBindingDataSize());
						
						RHICmdList.SetDefaultRayTracingHitGroup(SBT, Pipeline, 0);
						RHICmdList.SetRayTracingMissShader(SBT, 0, Pipeline, 0 /* ShaderIndexInPipeline */, 0, nullptr, 0);
						RHICmdList.CommitShaderBindingTable(SBT);
						RHICmdList.RayTraceDispatch(Pipeline, RayGenerationShader.GetRayTracingShader(), SBT, GlobalResources, Resolution.X, Resolution.Y);
					}
				}
			);
		}
	}
}
#else // !RHI_RAYTRACING
{
	unimplemented();
}
#endif

BEGIN_SHADER_PARAMETER_STRUCT(FDitheredLODFadingOutMaskParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

void FDeferredShadingSceneRenderer::RenderDitheredLODFadingOutMask(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneDepthTexture)
{
	if (auto* Pass = const_cast<FViewInfo&>(View).ParallelMeshDrawCommandPasses[EMeshPass::DitheredLODFadingOutMaskPass])
	{
		auto* PassParameters = GraphBuilder.AllocParameters<FDitheredLODFadingOutMaskParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);

		Pass->BuildRenderingCommands(GraphBuilder, Scene->GPUScene, PassParameters->InstanceCullingDrawParams);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("DitheredLODFadingOutMask"),
			PassParameters,
			ERDGPassFlags::Raster,
			[&View, Pass, PassParameters](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
			Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
		});
	}
}
