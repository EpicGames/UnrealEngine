// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
RaytracingOptions.h declares ray tracing options for use in rendering
=============================================================================*/

#pragma once

#include "UniformBuffer.h"
#include "RenderGraph.h"
#include "SceneRendering.h"

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FSkyLightData, RENDERER_API)
	SHADER_PARAMETER(uint32, SamplesPerPixel)
	SHADER_PARAMETER(float, MaxRayDistance)
	SHADER_PARAMETER(float, MaxNormalBias)
	SHADER_PARAMETER(float, MaxShadowThickness)
	SHADER_PARAMETER(int, bTransmission)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FWritableSkyLightVisibilityRaysData, )
	SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<SkyLightVisibilityRays>, OutSkyLightVisibilityRays)
	SHADER_PARAMETER(FIntVector, SkyLightVisibilityRaysDimensions)
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FSkyLightVisibilityRaysData, )
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<SkyLightVisibilityRays>, SkyLightVisibilityRays)
	SHADER_PARAMETER(FIntVector, SkyLightVisibilityRaysDimensions)
END_SHADER_PARAMETER_STRUCT()

#if RHI_RAYTRACING

int32 GetRayTracingSkyLightDecoupleSampleGenerationCVarValue();

class FPathTracingSkylight;

bool SetupSkyLightParameters(
	FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View,
	bool bEnableSkylight,
	FPathTracingSkylight* SkylightParameters,
	FSkyLightData* SkyLight);

void SetupSkyLightVisibilityRaysParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FSkyLightVisibilityRaysData* OutSkyLightVisibilityRaysData);

#endif // RHI_RAYTRACING
