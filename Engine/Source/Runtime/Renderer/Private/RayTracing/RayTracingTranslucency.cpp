// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalShader.h"
#include "DeferredShadingRenderer.h"
#include "SceneTextureParameters.h"

#if RHI_RAYTRACING

#include "ClearQuad.h"
#include "SceneRendering.h"
#include "PostProcess/SceneRenderTargets.h"
#include "RHIResources.h"
#include "SystemTextures.h"
#include "ScreenSpaceDenoise.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PipelineStateCache.h"
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracingLighting.h"


static TAutoConsoleVariable<int32> CVarRayTracingTranslucency(
	TEXT("r.RayTracing.Translucency"),
	-1,
	TEXT("-1: Value driven by postprocess volume (default) \n")
	TEXT(" 0: ray tracing translucency off (use raster) \n")
	TEXT(" 1: ray tracing translucency enabled"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static bool GRayTracingTranslucencyForceOpaque = false;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyForceOpaque(
	TEXT("r.RayTracing.Translucency.ForceOpaque"),
	GRayTracingTranslucencyForceOpaque,
	TEXT("Allow forcing rays traced during translucency to be marked as opaque so they do not execute the Any Hit Shader:\n")
	TEXT("0 - Rays will execute the any hit shader, allowing masked materials to be seen correctly (default) \n")
	TEXT("1 - Rays are forced to be marked opaque which improves performance but may incorrectly deal with masked materials."),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static float GRayTracingTranslucencyMaxRoughness = -1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyMaxRoughness(
	TEXT("r.RayTracing.Translucency.MaxRoughness"),
	GRayTracingTranslucencyMaxRoughness,
	TEXT("Sets the maximum roughness until which ray tracing reflections will be visible (default = -1 (max roughness driven by postprocessing volume))")
);

static int32 GRayTracingTranslucencyMaxRefractionRays = -1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyMaxRefractionRays(
	TEXT("r.RayTracing.Translucency.MaxRefractionRays"),
	GRayTracingTranslucencyMaxRefractionRays,
	TEXT("Sets the maximum number of refraction rays for ray traced translucency (default = -1 (max bounces driven by postprocessing volume)"));

static int32 GRayTracingTranslucencyEmissiveAndIndirectLighting = 1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyEmissiveAndIndirectLighting(
	TEXT("r.RayTracing.Translucency.EmissiveAndIndirectLighting"),
	GRayTracingTranslucencyEmissiveAndIndirectLighting,
	TEXT("Enables ray tracing translucency emissive and indirect lighting (default = 1)")
);

static int32 GRayTracingTranslucencyDirectLighting = 1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyDirectLighting(
	TEXT("r.RayTracing.Translucency.DirectLighting"),
	GRayTracingTranslucencyDirectLighting,
	TEXT("Enables ray tracing translucency direct lighting (default = 1)")
);

static int32 GRayTracingTranslucencyShadows = -1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyShadows(
	TEXT("r.RayTracing.Translucency.Shadows"),
	GRayTracingTranslucencyShadows,
	TEXT("Enables shadows in ray tracing translucency:\n")
	TEXT(" -1: Shadows driven by postprocessing volume (default)\n")
	TEXT(" 0: Shadows disabled\n")
	TEXT(" 1: Hard shadows\n")
	TEXT(" 2: Soft area shadows")
);

static int32 GRayTracingTranslucencyShadowsTranslucencyType = RAY_TRACING_SHADOWS_TRANSLUCENCY_TYPE_MASKED;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyShadowsTranslucencyType(
	TEXT("r.RayTracing.Translucency.ShadowsTranslucencyType"),
	GRayTracingTranslucencyShadowsTranslucencyType,
	TEXT("Controls translucency of shadows in reflections and refractions in ray traced translucency:\n")
	TEXT(" 0: Shadows ignore the any-hit shader\n")
	TEXT(" 1: Shadows use any-hit shader for masked materials (default)\n")
	TEXT(" 2: Shadows use any-hit shader for masked and translucent materials")
);


static float GRayTracingTranslucencyMinRayDistance = -1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyMinRayDistance(
	TEXT("r.RayTracing.Translucency.MinRayDistance"),
	GRayTracingTranslucencyMinRayDistance,
	TEXT("Sets the minimum ray distance for ray traced translucency rays. Actual translucency ray length is computed as Lerp(MaxRayDistance, MinRayDistance, Roughness), i.e. translucency rays become shorter when traced from rougher surfaces. (default = -1 (infinite rays))")
);

static float GRayTracingTranslucencyMaxRayDistance = -1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyMaxRayDistance(
	TEXT("r.RayTracing.Translucency.MaxRayDistance"),
	GRayTracingTranslucencyMaxRayDistance,
	TEXT("Sets the maximum ray distance for ray traced translucency rays. When ray shortening is used, skybox will not be sampled in RT translucency pass and will be composited later, together with local reflection captures. Negative values turn off this optimization. (default = -1 (infinite rays))")
);

static int32 GRayTracingTranslucencySamplesPerPixel = 1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencySamplesPerPixel(
	TEXT("r.RayTracing.Translucency.SamplesPerPixel"),
	GRayTracingTranslucencySamplesPerPixel,
	TEXT("Sets the samples-per-pixel for Translucency (default = 1)"));

static int32 GRayTracingTranslucencyHeightFog = 1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyHeightFog(
	TEXT("r.RayTracing.Translucency.HeightFog"),
	GRayTracingTranslucencyHeightFog,
	TEXT("Enables height fog in ray traced Translucency (default = 1)"));

static int32 GRayTracingTranslucencyRefraction = -1;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyRefraction(
	TEXT("r.RayTracing.Translucency.Refraction"),
	GRayTracingTranslucencyRefraction,
	TEXT("Enables refraction in ray traced Translucency (default = 1)"));

static float GRayTracingTranslucencyPrimaryRayBias = 1e-5f;
static FAutoConsoleVariableRef CVarRayTracingTranslucencyPrimaryRayBias(
	TEXT("r.RayTracing.Translucency.PrimaryRayBias"),
	GRayTracingTranslucencyPrimaryRayBias,
	TEXT("Sets the bias to be subtracted from the primary ray TMax in ray traced Translucency. Larger bias reduces the chance of opaque objects being intersected in ray traversal, saving performance, but at the risk of skipping some thin translucent objects in proximity of opaque objects. (recommended range: 0.00001 - 0.1) (default = 0.00001)"));

DECLARE_GPU_STAT_NAMED(RayTracingTranslucency, TEXT("Ray Tracing Translucency"));

#if RHI_RAYTRACING

FRayTracingPrimaryRaysOptions GetRayTracingTranslucencyOptions(const FViewInfo& View)
{
	FRayTracingPrimaryRaysOptions Options = {};

	Options.bEnabled = ShouldRenderRayTracingTranslucency(View);
	Options.bForceOpaque = GRayTracingTranslucencyForceOpaque;
	Options.SamplerPerPixel = GRayTracingTranslucencySamplesPerPixel >= 0 ? GRayTracingTranslucencySamplesPerPixel : View.FinalPostProcessSettings.RayTracingTranslucencySamplesPerPixel;
	Options.ApplyHeightFog = GRayTracingTranslucencyHeightFog;
	Options.PrimaryRayBias = GRayTracingTranslucencyPrimaryRayBias;
	Options.MaxRoughness = GRayTracingTranslucencyMaxRoughness >= 0 ? GRayTracingTranslucencyMaxRoughness : View.FinalPostProcessSettings.RayTracingTranslucencyMaxRoughness;
	Options.MaxRefractionRays = GRayTracingTranslucencyMaxRefractionRays >= 0 ? GRayTracingTranslucencyMaxRefractionRays : View.FinalPostProcessSettings.RayTracingTranslucencyRefractionRays;
	Options.EnableEmmissiveAndIndirectLighting = GRayTracingTranslucencyEmissiveAndIndirectLighting;
	Options.EnableDirectLighting = GRayTracingTranslucencyDirectLighting;
	Options.EnableShadows = GRayTracingTranslucencyShadows >= 0 ? GRayTracingTranslucencyShadows : (int) View.FinalPostProcessSettings.RayTracingTranslucencyShadows;
	Options.ShadowsTranslucencyType = GRayTracingTranslucencyShadowsTranslucencyType;
	Options.MinRayDistance = GRayTracingTranslucencyMinRayDistance;
	Options.MaxRayDistance = GRayTracingTranslucencyMaxRayDistance;
	Options.EnableRefraction = GRayTracingTranslucencyRefraction >= 0 ? GRayTracingTranslucencyRefraction : View.FinalPostProcessSettings.RayTracingTranslucencyRefraction;;

	return Options;
}

bool ShouldRenderRayTracingTranslucency(const FViewInfo& View)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const bool bViewWithRaytracingTranslucency = View.FinalPostProcessSettings.TranslucencyType == ETranslucencyType::RayTraced_Deprecated;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	const int32 RayTracingTranslucencyMode = CVarRayTracingTranslucency.GetValueOnRenderThread();
	
	const bool bTranslucencyEnabled = RayTracingTranslucencyMode < 0
		? bViewWithRaytracingTranslucency
		: RayTracingTranslucencyMode != 0;

	return ShouldRenderRayTracingEffect(bTranslucencyEnabled, ERayTracingPipelineCompatibilityFlags::FullPipeline, View);
}
#endif // RHI_RAYTRACING

extern void RenderRayTracingPrimaryRaysView(
	FRDGBuilder& GraphBuilder, const FViewInfo& View, const FSceneTextures& SceneTextures,
	FRDGTextureRef* InOutColorTexture, FRDGTextureRef* InOutRayHitDistanceTexture,
	ERayTracingSceneLayer LayerToTrace, int32 SamplePerPixel, int32 HeightFog, float ResolutionFraction, ERayTracingPrimaryRaysFlag Flags);

void FDeferredShadingSceneRenderer::RenderRayTracingTranslucency(FRDGBuilder& GraphBuilder, FRDGTextureMSAA SceneColorTexture)
{
	if (   !ShouldRenderTranslucency(ETranslucencyPass::TPT_TranslucencyStandard, Views)
		&& !ShouldRenderTranslucency(ETranslucencyPass::TPT_TranslucencyStandardModulate, Views)
		&& !ShouldRenderTranslucency(ETranslucencyPass::TPT_TranslucencyAfterDOF, Views)
		&& !ShouldRenderTranslucency(ETranslucencyPass::TPT_TranslucencyAfterDOFModulate, Views)
		&& !ShouldRenderTranslucency(ETranslucencyPass::TPT_TranslucencyAfterMotionBlur, Views)
		&& !ShouldRenderTranslucency(ETranslucencyPass::TPT_AllTranslucency, Views)
		)
	{
		return; // Early exit if nothing needs to be done.
	}

	AddResolveSceneColorPass(GraphBuilder, Views, SceneColorTexture);

	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, RayTracingTranslucency, "RayTracingTranslucency");

		for (int32 ViewIndex = 0, Num = Views.Num(); ViewIndex < Num; ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];

			if (!View.ShouldRenderView() || !ShouldRenderRayTracingTranslucency(View))
			{
				continue;
			}

			const FScreenPassRenderTarget Output(SceneColorTexture.Target, View.ViewRect, ERenderTargetLoadAction::ELoad);

			//#dxr_todo: UE-72581 do not use reflections denoiser structs but separated ones
			IScreenSpaceDenoiser::FReflectionsInputs DenoiserInputs;
			float ResolutionFraction = 1.0f;
			int32 TranslucencySPP = GRayTracingTranslucencySamplesPerPixel > -1 ? GRayTracingTranslucencySamplesPerPixel : View.FinalPostProcessSettings.RayTracingTranslucencySamplesPerPixel;
		
			RenderRayTracingPrimaryRaysView(
				GraphBuilder, View, GetActiveSceneTextures(),
				&DenoiserInputs.Color, &DenoiserInputs.RayHitDistance,
				ERayTracingSceneLayer::Base, TranslucencySPP, GRayTracingTranslucencyHeightFog, ResolutionFraction,
				ERayTracingPrimaryRaysFlag::AllowSkipSkySample | ERayTracingPrimaryRaysFlag::UseGBufferForMaxDistance);

			const FScreenPassTexture SceneColor(DenoiserInputs.Color, View.ViewRect);
			AddDrawTexturePass(GraphBuilder, View, SceneColor, Output);
		}
	}

	AddResolveSceneColorPass(GraphBuilder, Views, SceneColorTexture);
}

#endif // RHI_RAYTRACING
