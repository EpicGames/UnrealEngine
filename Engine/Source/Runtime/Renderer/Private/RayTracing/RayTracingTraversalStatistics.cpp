// Copyright Epic Games, Inc. All Rights Reserved.

#include "RayTracingTraversalStatistics.h"

#if RHI_RAYTRACING

#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "ShaderPrint.h"
#include "ShaderPrintParameters.h"
#include "SceneRendering.h"
#include "GlobalShader.h"
#include "DataDrivenShaderPlatformInfo.h"

// Needs to match the size of FTraceRayInlineTraversalStatistics in TraceRayInlineCommon.ush
#define TRACE_RAY_INLINE_TRAVERSAL_STATISTICS_SIZE_IN_BYTES 40

class FTraceRayInlinePrintStatisticsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTraceRayInlinePrintStatisticsCS);
	SHADER_USE_PARAMETER_STRUCT(FTraceRayInlinePrintStatisticsCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintStruct)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer, TraversalStatistics)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsRayTracingEnabledForProject(Parameters.Platform) && RHISupportsRayTracing(Parameters.Platform) && RHISupportsInlineRayTracing(Parameters.Platform) && FDataDrivenShaderPlatformInfo::GetSupportsRayTracingTraversalStatistics(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);		
	}
};

IMPLEMENT_GLOBAL_SHADER(FTraceRayInlinePrintStatisticsCS, "/Engine/Private/RayTracing/TraceRayInlineStatistics.usf", "TraceRayInlinePrintStatisticsCS", SF_Compute);

namespace RaytracingTraversalStatistics
{
	bool IsEnabled()
	{
		return GRHISupportsInlineRayTracing && FDataDrivenShaderPlatformInfo::GetSupportsRayTracingTraversalStatistics(GMaxRHIShaderPlatform);
	}

	void Init(FRDGBuilder& GraphBuilder, FTraceRayInlineStatisticsData& OutTraversalData)
	{
		if (!IsEnabled())
		{
			return;
		}

		OutTraversalData.TraversalStatisticsBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateStructuredDesc(TRACE_RAY_INLINE_TRAVERSAL_STATISTICS_SIZE_IN_BYTES, 1),
			TEXT("TraversalStatisticsBuffer"));

		AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(OutTraversalData.TraversalStatisticsBuffer), 0);
	}

	void SetParameters(FRDGBuilder& GraphBuilder, const FTraceRayInlineStatisticsData& TraversalData, FShaderParameters& OutParameters)
	{		
		if (IsEnabled() && TraversalData.TraversalStatisticsBuffer)
		{
			OutParameters.TraceRayInlineTraversalStatistics_Accumulator = GraphBuilder.CreateUAV(TraversalData.TraversalStatisticsBuffer);
		}		
	}

	void AddPrintPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FTraceRayInlineStatisticsData& TraversalData)
	{
		if (IsEnabled() && TraversalData.TraversalStatisticsBuffer)
		{
			FTraceRayInlinePrintStatisticsCS::FParameters* Parameters = GraphBuilder.AllocParameters<FTraceRayInlinePrintStatisticsCS::FParameters>();
			ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintStruct);
			Parameters->TraversalStatistics = GraphBuilder.CreateSRV(TraversalData.TraversalStatisticsBuffer);
			TShaderRef<FTraceRayInlinePrintStatisticsCS> ComputeShader = View.ShaderMap->GetShader<FTraceRayInlinePrintStatisticsCS>();
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("TraceRayInlinePrintStatisticsCS"),
				ComputeShader,
				Parameters,
				FIntVector(1, 1, 1));
		}
	}
}

#endif // RHI_RAYTRACING
