// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"

#if RHI_RAYTRACING

#include "RenderGraphDefinitions.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameters.h"

class FViewInfo;

namespace RaytracingTraversalStatistics
{
	BEGIN_SHADER_PARAMETER_STRUCT(FShaderParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, TraceRayInlineTraversalStatistics_Accumulator)
	END_SHADER_PARAMETER_STRUCT()

	struct FTraceRayInlineStatisticsData
	{
		FRDGBufferRef TraversalStatisticsBuffer;
	};
	
	void Init(FRDGBuilder& GraphBuilder, FTraceRayInlineStatisticsData& OutTraversalData);
	void SetParameters(FRDGBuilder& GraphBuilder, const FTraceRayInlineStatisticsData& TraversalData, FShaderParameters& OutParameters);
	void AddPrintPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FTraceRayInlineStatisticsData& TraversalData);
}

#endif // RHI_RAYTRACING
