// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RaytracingOptions.h declares ray tracing options for use in rendering
=============================================================================*/

#pragma once

#include "RHIDefinitions.h"
#include "RayTracingPayloadType.h"

class FSkyLightSceneProxy;
class FViewInfo;
class FLightSceneInfoCompact;
class FLightSceneInfo;
class FScene;
class FSceneView;
class FSceneViewFamily;
class FLightSceneProxy;
enum EShaderPlatform : uint16;
struct FEngineShowFlags;

// be sure to also update the definition in the `RayTracingPrimaryRays.usf`
enum class ERayTracingPrimaryRaysFlag: uint32 
{
	None                      =      0,
	UseGBufferForMaxDistance  = 1 << 0,
	PrimaryView	              = 1 << 1,
	AllowSkipSkySample        = 1 << 2,
};

ENUM_CLASS_FLAGS(ERayTracingPrimaryRaysFlag);

struct FRayTracingPrimaryRaysOptions
{
	bool bEnabled;
	bool bForceOpaque;
	int32 SamplerPerPixel;
	int32 ApplyHeightFog;
	float PrimaryRayBias;
	float MaxRoughness;
	int32 MaxRefractionRays;
	int32 EnableEmmissiveAndIndirectLighting;
	int32 EnableDirectLighting;
	int32 EnableShadows;
	int32 ShadowsTranslucencyType;
	float MinRayDistance;
	float MaxRayDistance;
	int32 EnableRefraction;
};

enum class ERayTracingPipelineCompatibilityFlags
{
	// Rendering feature can use the full ray tracing pipeline, with raygen, hit and miss shaders.
	FullPipeline = 1 << 0,

	// Rendering feature can use inline ray tracing
	Inline  = 1 << 1,
};
ENUM_CLASS_FLAGS(ERayTracingPipelineCompatibilityFlags);

namespace RayTracing
{
	// Keep in sync with r.RayTracing.Culling
	enum class ECullingMode : uint8
	{
		Disabled,
		BehindCameraByDistanceAndSolidAngle,
		DistanceAndSolidAngle,
		DistanceOrSolidAngle,

		MAX
	};
}

#if RHI_RAYTRACING

// Whether a particular effect should be used, taking into account debug override
extern bool ShouldRenderRayTracingEffect(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, EShaderPlatform ShaderPlatform);
extern bool ShouldRenderRayTracingEffect(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, const FSceneView& View);
extern bool ShouldRenderRayTracingEffect(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, const FSceneViewFamily& ViewFamily);

extern bool AnyInlineRayTracingPassEnabled(const FScene* Scene, const FViewInfo& View);
extern FRayTracingPrimaryRaysOptions GetRayTracingTranslucencyOptions(const FViewInfo& View);

extern bool ShouldRenderRayTracingSkyLight(const FSkyLightSceneProxy* SkyLightSceneProxy, EShaderPlatform ShaderPlatform);
extern bool ShouldRenderRayTracingAmbientOcclusion(const FViewInfo& View);
extern bool ShouldRenderRayTracingTranslucency(const FViewInfo& View);
extern bool ShouldRenderRayTracingShadows(const FSceneViewFamily& ViewFamily);
extern bool ShouldRenderRayTracingShadowsForLight(const FSceneViewFamily& ViewFamily, const FLightSceneProxy& LightProxy);
extern bool ShouldRenderRayTracingShadowsForLight(const FSceneViewFamily& ViewFamily, const FLightSceneInfoCompact& LightInfo);
extern bool ShouldRenderPluginRayTracingGlobalIllumination(const FViewInfo& View);
extern bool HasRayTracedOverlay(const FSceneViewFamily& ViewFamily);

namespace RayTracing
{
	extern ECullingMode GetCullingMode(const FEngineShowFlags& ShowFlags);
}

extern bool EnableRayTracingShadowTwoSidedGeometry();
extern float GetRaytracingMaxNormalBias();
extern float GetRayTracingCullingRadius();
extern bool IsRayTracingInstanceDebugDataEnabled(const FViewInfo& View);
extern bool IsRayTracingInstanceOverlapEnabled(const FViewInfo& View);
extern bool RaytracingDebugViewModeNeedsTonemapping(const FSceneView& View);
extern bool IsRayTracingFeedbackEnabled(const FSceneViewFamily& ViewFamily);

extern bool CanUseRayTracingAMDHitToken();

#else // RHI_RAYTRACING

FORCEINLINE bool ShouldRenderRayTracingEffect(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, EShaderPlatform ShaderPlatform)
{
	return false;
}

FORCEINLINE bool ShouldRenderRayTracingEffect(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, const FSceneView& View)
{
	return false;
}

FORCEINLINE bool ShouldRenderRayTracingEffect(bool bEffectEnabled, ERayTracingPipelineCompatibilityFlags CompatibilityFlags, const FSceneViewFamily& ViewFamily)
{
	return false;
}

FORCEINLINE bool ShouldRenderRayTracingSkyLight(const FSkyLightSceneProxy* SkyLightSceneProxy, EShaderPlatform ShaderPlatform)
{
	return false;
}

FORCEINLINE bool ShouldRenderRayTracingAmbientOcclusion(const FViewInfo& View)
{
	return false;
}

FORCEINLINE bool ShouldRenderRayTracingTranslucency(const FViewInfo& View)
{
	return false;
}

FORCEINLINE bool ShouldRenderRayTracingShadows(const FSceneViewFamily& ViewFamily)
{
	return false;
}

FORCEINLINE bool ShouldRenderRayTracingShadowsForLight(const FSceneViewFamily& ViewFamily, const FLightSceneProxy& LightProxy)
{
	return false;
}

FORCEINLINE bool ShouldRenderRayTracingShadowsForLight(const FSceneViewFamily& ViewFamily, const FLightSceneInfoCompact& LightInfo)
{
	return false;
}

FORCEINLINE bool ShouldRenderPluginRayTracingGlobalIllumination(const FViewInfo& View)
{
	return false;
}

FORCEINLINE bool HasRayTracedOverlay(const FSceneViewFamily& ViewFamily)
{
	return false;
}

namespace RayTracing
{
	FORCEINLINE ECullingMode GetCullingMode(const FEngineShowFlags& ShowFlags)
	{
		return ECullingMode::Disabled;
	}
}

FORCEINLINE float GetRayTracingCullingRadius()
{
	return 0.0;
}

FORCEINLINE bool CanUseRayTracingAMDHitToken()
{
	return false;
}

FORCEINLINE bool IsRayTracingInstanceDebugDataEnabled(const FViewInfo& View)
{
	return false;
}

FORCEINLINE bool IsRayTracingInstanceOverlapEnabled(const FViewInfo& View)
{
	return false;
}

FORCEINLINE bool IsRayTracingFeedbackEnabled(const FSceneViewFamily& ViewFamily)
{
	return false;
}

FORCEINLINE bool RaytracingDebugViewModeNeedsTonemapping(const FViewInfo& View)
{
	return false;
}

#endif // RHI_RAYTRACING
