// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RaytracingSkyLight.cpp implements sky lighting with ray tracing
=============================================================================*/

#include "DeferredShadingRenderer.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "SceneTextureParameters.h"
#include "DataDrivenShaderPlatformInfo.h"

static int32 GRayTracingSkyLight = 0;

static FAutoConsoleVariableRef CVarRayTracingSkyLight(
	TEXT("r.RayTracing.SkyLight"),
	GRayTracingSkyLight,
	TEXT("Enables ray tracing SkyLight (default = 0)"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

#if RHI_RAYTRACING

#include "RayTracingSkyLight.h"
#include "RayTracingMaterialHitShaders.h"
#include "ClearQuad.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "SceneRendering.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "PostProcess/SceneRenderTargets.h"
#include "RenderGraphBuilder.h"
#include "RenderTargetPool.h"
#include "VisualizeTexture.h"
#include "RayGenShaderUtils.h"
#include "SceneTextureParameters.h"
#include "ScreenSpaceDenoise.h"
#include "RayTracingDefinitions.h"
#include "PathTracing.h"
#include "Nanite/NaniteRayTracing.h"

#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracing.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/SceneFilterRendering.h"
#include "HairStrands/HairStrandsRendering.h"
#include "Substrate/Substrate.h"
#include "SceneViewState.h"


static int32 GRayTracingSkyLightSamplesPerPixel = -1;
static FAutoConsoleVariableRef CVarRayTracingSkyLightSamplesPerPixel(
	TEXT("r.RayTracing.SkyLight.SamplesPerPixel"),
	GRayTracingSkyLightSamplesPerPixel,
	TEXT("Sets the samples-per-pixel for ray tracing SkyLight (default = -1)")
);

static float GRayTracingSkyLightMaxRayDistance = 1.0e7;
static FAutoConsoleVariableRef CVarRayTracingSkyLightMaxRayDistance(
	TEXT("r.RayTracing.SkyLight.MaxRayDistance"),
	GRayTracingSkyLightMaxRayDistance,
	TEXT("Sets the max ray distance for ray tracing SkyLight (default = 1.0e7)")
);

static float GRayTracingSkyLightMaxShadowThickness = 1.0e3;
static FAutoConsoleVariableRef CVarRayTracingSkyLightMaxShadowThickness(
	TEXT("r.RayTracing.SkyLight.MaxShadowThickness"),
	GRayTracingSkyLightMaxShadowThickness,
	TEXT("Sets the max shadow thickness for translucent materials for ray tracing SkyLight (default = 1.0e3)")
);

static int32 GRayTracingSkyLightSamplingStopLevel = 0;
static FAutoConsoleVariableRef CVarRayTracingSkyLightSamplingStopLevel(
	TEXT("r.RayTracing.SkyLight.Sampling.StopLevel"),
	GRayTracingSkyLightSamplingStopLevel,
	TEXT("Sets the stop level for MIP-sampling (default = 0)")
);

static int32 GRayTracingSkyLightDenoiser = 1;
static FAutoConsoleVariableRef CVarRayTracingSkyLightDenoiser(
	TEXT("r.RayTracing.SkyLight.Denoiser"),
	GRayTracingSkyLightDenoiser,
	TEXT("Denoising options (default = 1)")
);

static TAutoConsoleVariable<int32> CVarRayTracingSkyLightEnableTwoSidedGeometry(
	TEXT("r.RayTracing.SkyLight.EnableTwoSidedGeometry"),
	1,
	TEXT("Enables two-sided geometry when tracing shadow rays (default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingSkyLightEnableMaterials(
	TEXT("r.RayTracing.SkyLight.EnableMaterials"),
	1,
	TEXT("Enables material shader binding for shadow rays. If this is disabled, then a default trivial shader is used. (default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingSkyLightDecoupleSampleGeneration(
	TEXT("r.RayTracing.SkyLight.DecoupleSampleGeneration"),
	1,
	TEXT("Decouples sample generation from ray traversal (default = 1)"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingSkyLightEnableHairVoxel(
	TEXT("r.RayTracing.SkyLight.HairVoxel"),
	1,
	TEXT("Include hair voxel representation to estimate sky occlusion"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<float> CVarRayTracingSkyLightScreenPercentage(
	TEXT("r.RayTracing.SkyLight.ScreenPercentage"),
	100.0f,
	TEXT("Screen percentage at which to evaluate sky occlusion"),
	ECVF_RenderThreadSafe
);

int32 GetRayTracingSkyLightDecoupleSampleGenerationCVarValue()
{
	return CVarRayTracingSkyLightDecoupleSampleGeneration.GetValueOnRenderThread();
}

int32 GetSkyLightSamplesPerPixel(const FSkyLightSceneProxy* SkyLightSceneProxy)
{
	check(SkyLightSceneProxy != nullptr);

	// Clamp to 2spp minimum due to poor 1spp denoised quality
	return GRayTracingSkyLightSamplesPerPixel >= 0 ? GRayTracingSkyLightSamplesPerPixel : FMath::Max(SkyLightSceneProxy->SamplesPerPixel, 2);
}

bool ShouldRenderRayTracingSkyLight(const FSkyLightSceneProxy* SkyLightSceneProxy, EShaderPlatform ShaderPlatform)
{
	if (SkyLightSceneProxy == nullptr || !IsRayTracingEnabled(ShaderPlatform))
	{
		return false;
	}

	bool bRayTracingSkyEnabled = (GRayTracingSkyLight  > 0 && SkyLightSceneProxy->CastRayTracedShadow == ECastRayTracedShadow::UseProjectSetting)
								||  SkyLightSceneProxy->CastRayTracedShadow == ECastRayTracedShadow::Enabled;

	return bRayTracingSkyEnabled && ShouldRenderRayTracingEffect(true, ERayTracingPipelineCompatibilityFlags::FullPipeline, ShaderPlatform) && (GetSkyLightSamplesPerPixel(SkyLightSceneProxy) > 0);
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSkyLightData, "SkyLight");

struct FSkyLightVisibilityRays
{
	FVector4f DirectionAndPdf;
};

bool SetupSkyLightParameters(
	FRDGBuilder& GraphBuilder, FScene* Scene, const FViewInfo& View,
	bool bEnableSkylight,
	FPathTracingSkylight* SkylightParameters,
	FSkyLightData* SkyLightData)
{
	// Check if parameters should be set based on if the sky light's texture has been processed and if its mip tree has been built yet

	const bool bUseMISCompensation = true;
	if (PrepareSkyTexture(GraphBuilder, Scene, View, bEnableSkylight, bUseMISCompensation, SkylightParameters))
	{

		SkyLightData->SamplesPerPixel = GetSkyLightSamplesPerPixel(Scene->SkyLight);
		SkyLightData->MaxRayDistance = GRayTracingSkyLightMaxRayDistance;
		SkyLightData->MaxNormalBias = GetRaytracingMaxNormalBias();
		SkyLightData->bTransmission = Scene->SkyLight->bTransmission;
		SkyLightData->MaxShadowThickness = GRayTracingSkyLightMaxShadowThickness;
		ensure(SkyLightData->SamplesPerPixel > 0);
		return true;
	}
	else
	{
		// skylight is not enabled
		SkyLightData->SamplesPerPixel = -1;
		SkyLightData->MaxRayDistance = 0.0f;
		SkyLightData->MaxNormalBias = 0.0f;
		SkyLightData->MaxShadowThickness = 0.0f;
		return false;
	}
}

void SetupSkyLightVisibilityRaysParameters(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FSkyLightVisibilityRaysData* OutSkyLightVisibilityRaysData)
{
	// Get the Scene View State
	FSceneViewState* SceneViewState = (FSceneViewState*)View.State;

	FRDGBufferRef SkyLightVisibilityRaysBuffer = nullptr;
	FIntVector SkyLightVisibilityRaysDimensions;

	// Check if the Sky Light Visibility Ray Data should be set based on if decoupled sample generation is being used
	if (
		(SceneViewState != nullptr) &&
		(SceneViewState->SkyLightVisibilityRaysBuffer != nullptr) &&
		(CVarRayTracingSkyLightDecoupleSampleGeneration.GetValueOnRenderThread() == 1))
	{
		// Set the Sky Light Visibility Ray pooled buffer to the stored pooled buffer
		SkyLightVisibilityRaysBuffer = GraphBuilder.RegisterExternalBuffer(SceneViewState->SkyLightVisibilityRaysBuffer);

		// Set the Sky Light Visibility Ray Dimensions from the stored dimensions
		SkyLightVisibilityRaysDimensions = SceneViewState->SkyLightVisibilityRaysDimensions;
	}
	else
	{
		// Create a dummy Sky Light Visibility Ray buffer in a dummy RDG
		FRDGBufferDesc DummyBufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FSkyLightVisibilityRays), 1);
		SkyLightVisibilityRaysBuffer = GraphBuilder.CreateBuffer(DummyBufferDesc, TEXT("DummySkyLightVisibilityRays"));
		FRDGBufferUAVRef DummyRDGBufferUAV = GraphBuilder.CreateUAV(SkyLightVisibilityRaysBuffer, EPixelFormat::PF_R32_UINT);

		// Clear the dummy Sky Light Visibility Ray buffer
		AddClearUAVPass(GraphBuilder, DummyRDGBufferUAV, 0);

		// Set the Sky Light Visibility Ray Dimensions to a dummy value
		SkyLightVisibilityRaysDimensions = FIntVector(1);
	}

	// Set Sky Light Visibility Ray Data information
	OutSkyLightVisibilityRaysData->SkyLightVisibilityRays = GraphBuilder.CreateSRV(SkyLightVisibilityRaysBuffer, EPixelFormat::PF_R32_UINT);
	OutSkyLightVisibilityRaysData->SkyLightVisibilityRaysDimensions = SkyLightVisibilityRaysDimensions;
}

class FRayTracingSkyLightRGS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRayTracingSkyLightRGS)
	SHADER_USE_ROOT_PARAMETER_STRUCT(FRayTracingSkyLightRGS, FGlobalShader)

	class FEnableTwoSidedGeometryDim : SHADER_PERMUTATION_BOOL("ENABLE_TWO_SIDED_GEOMETRY");
	class FEnableMaterialsDim : SHADER_PERMUTATION_BOOL("ENABLE_MATERIALS");
	class FDecoupleSampleGeneration : SHADER_PERMUTATION_BOOL("DECOUPLE_SAMPLE_GENERATION");
	class FHairLighting : SHADER_PERMUTATION_INT("USE_HAIR_LIGHTING", 2);

	using FPermutationDomain = TShaderPermutationDomain<FEnableTwoSidedGeometryDim, FEnableMaterialsDim, FDecoupleSampleGeneration, FHairLighting>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static ERayTracingPayloadType GetRayTracingPayloadType(const int32 PermutationId)
	{
		return ERayTracingPayloadType::RayTracingMaterial;
	}

	static const FShaderBindingLayout* GetShaderBindingLayout(const FShaderPermutationParameters& Parameters)
	{
		return RayTracing::GetShaderBindingLayout(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, UpscaleFactor)
		SHADER_PARAMETER_RDG_BUFFER_SRV(RaytracingAccelerationStructure, TLAS)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWSkyOcclusionMaskUAV)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float2>, RWSkyOcclusionRayDistanceUAV)

		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_STRUCT_REF(FSkyLightData, SkyLightData)
		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingSkylight, SkyLightParameters)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FVirtualVoxelParameters, VirtualVoxel)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FNaniteRayTracingUniformParameters, NaniteRayTracing)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)
		
		SHADER_PARAMETER_STRUCT_INCLUDE(FSkyLightVisibilityRaysData, SkyLightVisibilityRaysData)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FRayTracingSkyLightRGS, "/Engine/Private/RayTracing/RayTracingSkyLightRGS.usf", "SkyLightRGS", SF_RayGen);

void FDeferredShadingSceneRenderer::PrepareRayTracingSkyLight(const FViewInfo& View, const FScene& Scene, TArray<FRHIRayTracingShader*>& OutRayGenShaders)
{
	if (!ShouldRenderRayTracingSkyLight(Scene.SkyLight, View.GetShaderPlatform()))
	{
		return;
	}

	// Declare all RayGen shaders that require material closest hit shaders to be bound
	FRayTracingSkyLightRGS::FPermutationDomain PermutationVector;
	for (uint32 TwoSidedGeometryIndex = 0; TwoSidedGeometryIndex < 2; ++TwoSidedGeometryIndex)
	{
		for (uint32 EnableMaterialsIndex = 0; EnableMaterialsIndex < 2; ++EnableMaterialsIndex)
		{
			for (uint32 DecoupleSampleGeneration = 0; DecoupleSampleGeneration < 2; ++DecoupleSampleGeneration)
			{
				for (int32 HairLighting = 0; HairLighting < 2; ++HairLighting)
				{
					PermutationVector.Set<FRayTracingSkyLightRGS::FEnableTwoSidedGeometryDim>(TwoSidedGeometryIndex != 0);
					PermutationVector.Set<FRayTracingSkyLightRGS::FEnableMaterialsDim>(EnableMaterialsIndex != 0);
					PermutationVector.Set<FRayTracingSkyLightRGS::FDecoupleSampleGeneration>(DecoupleSampleGeneration != 0);
					PermutationVector.Set<FRayTracingSkyLightRGS::FHairLighting>(HairLighting);
					TShaderMapRef<FRayTracingSkyLightRGS> RayGenerationShader(View.ShaderMap, PermutationVector);
					OutRayGenShaders.Add(RayGenerationShader.GetRayTracingShader());
				}
			}
		}
	}
}

class FGenerateSkyLightVisibilityRaysCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGenerateSkyLightVisibilityRaysCS);
	SHADER_USE_PARAMETER_STRUCT(FGenerateSkyLightVisibilityRaysCS, FGlobalShader);

	static const uint32 kGroupSize = 16;
	using FPermutationDomain = FShaderPermutationNone;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("TILE_SIZE"), kGroupSize);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, SamplesPerPixel)

		SHADER_PARAMETER_STRUCT_INCLUDE(FPathTracingSkylight, SkylightParameters)
		SHADER_PARAMETER_STRUCT_REF(FSkyLightData, SkyLightData)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)

		// Writable variant to allow for Sky Light Visibility Ray output
		SHADER_PARAMETER_STRUCT_INCLUDE(FWritableSkyLightVisibilityRaysData, WritableSkyLightVisibilityRaysData)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FGenerateSkyLightVisibilityRaysCS, "/Engine/Private/RayTracing/GenerateSkyLightVisibilityRaysCS.usf", "MainCS", SF_Compute);

static void GenerateSkyLightVisibilityRays(
	FRDGBuilder& GraphBuilder,
	FViewInfo& View,
	FPathTracingSkylight& SkylightParameters,
	FSkyLightData& SkyLightData,
	FRDGBufferRef& SkyLightVisibilityRaysBuffer,
	FIntVector& Dimensions
)
{
	// Allocating mask of 256 x 256 rays
	Dimensions = FIntVector(256, 256, 0);

	// Compute Pass parameter definition
	FGenerateSkyLightVisibilityRaysCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGenerateSkyLightVisibilityRaysCS::FParameters>();
	PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
	PassParameters->SkyLightData = CreateUniformBufferImmediate(SkyLightData, EUniformBufferUsage::UniformBuffer_SingleDraw);
	PassParameters->SkylightParameters = SkylightParameters;

	// Output structured buffer creation
	FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FSkyLightVisibilityRays), Dimensions.X * Dimensions.Y * SkyLightData.SamplesPerPixel);
	SkyLightVisibilityRaysBuffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("SkyLightVisibilityRays"));

	PassParameters->WritableSkyLightVisibilityRaysData.SkyLightVisibilityRaysDimensions = FIntVector(Dimensions.X, Dimensions.Y, 0);
	PassParameters->WritableSkyLightVisibilityRaysData.OutSkyLightVisibilityRays = GraphBuilder.CreateUAV(SkyLightVisibilityRaysBuffer, EPixelFormat::PF_R32_UINT);

	auto ComputeShader = View.ShaderMap->GetShader<FGenerateSkyLightVisibilityRaysCS>();

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("GenerateSkyLightVisibilityRays"),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(FIntPoint(Dimensions.X, Dimensions.Y), FGenerateSkyLightVisibilityRaysCS::kGroupSize)
	);
}

DECLARE_GPU_STAT_NAMED(RayTracingSkyLight, TEXT("Ray Tracing SkyLight"));

void FDeferredShadingSceneRenderer::RenderRayTracingSkyLight(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef SceneColorTexture,
	FRDGTextureRef& OutSkyLightTexture,
	FRDGTextureRef& OutHitDistanceTexture)
{
	FSkyLightSceneProxy* SkyLight = Scene->SkyLight;
	
	// Fill Sky Light parameters
	const bool bShouldRenderRayTracingSkyLight = ShouldRenderRayTracingSkyLight(SkyLight, Scene->GetShaderPlatform());
	FPathTracingSkylight SkylightParameters;
	FSkyLightData SkyLightData;
	if (!SetupSkyLightParameters(GraphBuilder, Scene, Views[0], bShouldRenderRayTracingSkyLight, &SkylightParameters, &SkyLightData))
	{
		OutSkyLightTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		OutHitDistanceTexture = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, RayTracingSkyLight, "RayTracingSkyLight");

	check(SceneColorTexture);

	float ResolutionFraction = 1.0f;
	if (GRayTracingSkyLightDenoiser != 0)
	{
		ResolutionFraction = FMath::Clamp(CVarRayTracingSkyLightScreenPercentage.GetValueOnRenderThread() / 100.0f, 0.25f, 1.0f);
	}

	int32 UpscaleFactor = int32(1.0 / ResolutionFraction);
	ResolutionFraction = 1.0f / UpscaleFactor;


	{
		FRDGTextureDesc Desc = SceneColorTexture->Desc;
		Desc.Format = PF_FloatRGBA;
		Desc.Flags &= ~(TexCreate_FastVRAM);
		Desc.Extent /= UpscaleFactor;
		OutSkyLightTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingSkylight"));

		Desc.Format = PF_G16R16;
		OutHitDistanceTexture = GraphBuilder.CreateTexture(Desc, TEXT("RayTracingSkyLightHitDistance"));
	}


	FRDGBufferRef SkyLightVisibilityRaysBuffer;
	FIntVector SkyLightVisibilityRaysDimensions;
	if (CVarRayTracingSkyLightDecoupleSampleGeneration.GetValueOnRenderThread() == 1)
	{
		GenerateSkyLightVisibilityRays(GraphBuilder, Views[0], SkylightParameters, SkyLightData, SkyLightVisibilityRaysBuffer, SkyLightVisibilityRaysDimensions);
	}
	else
	{
		FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FSkyLightVisibilityRays), 1);
		SkyLightVisibilityRaysBuffer = GraphBuilder.CreateBuffer(BufferDesc, TEXT("SkyLightVisibilityRays"));
		SkyLightVisibilityRaysDimensions = FIntVector(1);
	}

	const FRDGTextureDesc& SceneColorDesc = SceneColorTexture->Desc;
	FRDGTextureUAV* SkyLightkUAV = GraphBuilder.CreateUAV(OutSkyLightTexture);
	FRDGTextureUAV* RayDistanceUAV = GraphBuilder.CreateUAV(OutHitDistanceTexture);

	// Fill Scene Texture parameters
	FSceneTextureParameters SceneTextures = GetSceneTextureParameters(GraphBuilder, Views[0]);
	const FRayTracingScene& RayTracingScene = Scene->RayTracingScene;

	for (FViewInfo& View : Views)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		FRayTracingSkyLightRGS::FParameters *PassParameters = GraphBuilder.AllocParameters<FRayTracingSkyLightRGS::FParameters>();
		PassParameters->RWSkyOcclusionMaskUAV = SkyLightkUAV;
		PassParameters->RWSkyOcclusionRayDistanceUAV = RayDistanceUAV;
		PassParameters->SkyLightParameters = SkylightParameters;
		PassParameters->SkyLightData = CreateUniformBufferImmediate(SkyLightData, EUniformBufferUsage::UniformBuffer_SingleDraw);
		PassParameters->SkyLightVisibilityRaysData.SkyLightVisibilityRaysDimensions = SkyLightVisibilityRaysDimensions;
		if (CVarRayTracingSkyLightDecoupleSampleGeneration.GetValueOnRenderThread() == 1)
		{
			PassParameters->SkyLightVisibilityRaysData.SkyLightVisibilityRays = GraphBuilder.CreateSRV(SkyLightVisibilityRaysBuffer, EPixelFormat::PF_R32_UINT);
		}
		PassParameters->SceneTextures = SceneTextures;
		PassParameters->UpscaleFactor = UpscaleFactor;

		PassParameters->TLAS = View.GetRayTracingSceneLayerViewChecked(ERayTracingSceneLayer::Base);
		PassParameters->ViewUniformBuffer = View.ViewUniformBuffer;
		PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder);
		PassParameters->NaniteRayTracing = Nanite::GRayTracingManager.GetUniformBuffer();
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);

		const bool bUseHairLighting = 
			HairStrands::HasViewHairStrandsData(View) && 
			HairStrands::HasViewHairStrandsVoxelData(View) &&
			CVarRayTracingSkyLightEnableHairVoxel.GetValueOnRenderThread() > 0;
		if (bUseHairLighting)
		{
			PassParameters->VirtualVoxel = HairStrands::BindHairStrandsVoxelUniformParameters(View);
		}

		FRayTracingSkyLightRGS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FRayTracingSkyLightRGS::FEnableTwoSidedGeometryDim>(CVarRayTracingSkyLightEnableTwoSidedGeometry.GetValueOnRenderThread() != 0);
		PermutationVector.Set<FRayTracingSkyLightRGS::FEnableMaterialsDim>(CVarRayTracingSkyLightEnableMaterials.GetValueOnRenderThread() != 0);
		PermutationVector.Set<FRayTracingSkyLightRGS::FDecoupleSampleGeneration>(CVarRayTracingSkyLightDecoupleSampleGeneration.GetValueOnRenderThread() != 0);
		PermutationVector.Set<FRayTracingSkyLightRGS::FHairLighting>(bUseHairLighting ? 1 : 0);
		TShaderMapRef<FRayTracingSkyLightRGS> RayGenerationShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);
		ClearUnusedGraphResources(RayGenerationShader, PassParameters);
		
		FIntPoint RayTracingResolution = View.ViewRect.Size() / UpscaleFactor;
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("SkyLightRayTracing %dx%d", RayTracingResolution.X, RayTracingResolution.Y),
			PassParameters,
			ERDGPassFlags::Compute,
			[PassParameters, this, &View, RayGenerationShader, RayTracingResolution](FRHICommandList& RHICmdList)
		{
			FRHIBatchedShaderParameters& GlobalResources = RHICmdList.GetScratchShaderParameters();
			SetShaderParameters(GlobalResources, RayGenerationShader, *PassParameters);

			FRHIUniformBuffer* SceneUniformBuffer = PassParameters->Scene->GetRHI();
			FRHIUniformBuffer* NaniteRayTracingUniformBuffer = PassParameters->NaniteRayTracing->GetRHI();
			TOptional<FScopedUniformBufferStaticBindings> StaticUniformBufferScope = RayTracing::BindStaticUniformBufferBindings(View, SceneUniformBuffer, NaniteRayTracingUniformBuffer, RHICmdList);

			FRayTracingPipelineState* Pipeline = View.MaterialRayTracingData.PipelineState;
			FShaderBindingTableRHIRef SBT = View.MaterialRayTracingData.ShaderBindingTable;
			if (CVarRayTracingSkyLightEnableMaterials.GetValueOnRenderThread() == 0)
			{
				// Declare default pipeline
				FRayTracingPipelineStateInitializer Initializer;

				const FShaderBindingLayout* ShaderBindingLayout = RayTracing::GetShaderBindingLayout(ShaderPlatform);
				if (ShaderBindingLayout)
				{
					Initializer.ShaderBindingLayout = &ShaderBindingLayout->RHILayout;
				}

				Initializer.MaxPayloadSizeInBytes = GetRayTracingPayloadTypeMaxSize(ERayTracingPayloadType::RayTracingMaterial);
				FRHIRayTracingShader* RayGenShaderTable[] = { RayGenerationShader.GetRayTracingShader() };
				Initializer.SetRayGenShaderTable(RayGenShaderTable);

				FRHIRayTracingShader* HitGroupTable[] = { GetRayTracingDefaultOpaqueShader(View.ShaderMap) };
				Initializer.SetHitGroupTable(HitGroupTable);

				FRHIRayTracingShader* MissGroupTable[] = { GetRayTracingDefaultMissShader(View.ShaderMap) };
				Initializer.SetMissShaderTable(MissGroupTable);

				Pipeline = PipelineStateCache::GetAndOrCreateRayTracingPipelineState(RHICmdList, Initializer);

				SBT = Scene->RayTracingSBT.AllocateTransientRHI(RHICmdList, ERayTracingShaderBindingMode::RTPSO, ERayTracingHitGroupIndexingMode::Disallow, Initializer.GetMaxLocalBindingDataSize());

				RHICmdList.SetDefaultRayTracingHitGroup(SBT, Pipeline, 0);
				RHICmdList.SetRayTracingMissShader(SBT, 0, Pipeline, 0 /* ShaderIndexInPipeline */, 0, nullptr, 0);
				RHICmdList.CommitShaderBindingTable(SBT);
			}

			RHICmdList.RayTraceDispatch(Pipeline, RayGenerationShader.GetRayTracingShader(), SBT, GlobalResources, RayTracingResolution.X, RayTracingResolution.Y);
		});
	}

	// Denoising
	if (GRayTracingSkyLightDenoiser != 0)
	{
		const IScreenSpaceDenoiser* DefaultDenoiser = IScreenSpaceDenoiser::GetDefaultDenoiser();
		const IScreenSpaceDenoiser* DenoiserToUse = DefaultDenoiser;

		IScreenSpaceDenoiser::FDiffuseIndirectInputs DenoiserInputs;
		DenoiserInputs.Color = OutSkyLightTexture;
		DenoiserInputs.RayHitDistance = OutHitDistanceTexture;

		IScreenSpaceDenoiser::FAmbientOcclusionRayTracingConfig RayTracingConfig;
		RayTracingConfig.ResolutionFraction = ResolutionFraction;
		RayTracingConfig.RayCountPerPixel = GetSkyLightSamplesPerPixel(SkyLight);

		bool bAllViewsSameGPU = true;
#if WITH_MGPU
		for (int32 ViewIndex = 1; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (Views[ViewIndex].GPUMask != Views[0].GPUMask)
			{
				bAllViewsSameGPU = false;
			}
		}
#endif

		if (bAllViewsSameGPU && DenoiserToUse == DefaultDenoiser)
		{
			RDG_GPU_MASK_SCOPE(GraphBuilder, Views[0].GPUMask);

			RDG_EVENT_SCOPE(GraphBuilder, "%s(SkyLight) %s",
				DenoiserToUse->GetDebugName(),
				Views.Num() > 1 ?
					*FString::Printf(TEXT("%d views"), Views.Num()) :
					*FString::Printf(TEXT("%dx%d"), Views[0].ViewRect.Width(), Views[0].ViewRect.Height()));

			// Multi-view version of DenoiseSkyLight, which saves memory by sharing persistent render targets across views.
			// Persistent render targets are stored on the first view, so PrevViewInfo for the first view is passed in.
			IScreenSpaceDenoiser::FDiffuseIndirectOutputs DenoiserOutputs = IScreenSpaceDenoiser::DenoiseSkyLight(
				GraphBuilder,
				Views,
				&Views[0].PrevViewInfo,
				SceneTextures,
				DenoiserInputs,
				RayTracingConfig);

			OutSkyLightTexture = DenoiserOutputs.Color;
		}
		else
		{
			int32 ViewIndex = 0;
			int32 LastViewIndex = Views.Num() - 1;

			for (FViewInfo& View : Views)
			{
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

				RDG_EVENT_SCOPE(GraphBuilder, "%s%s(SkyLight) %dx%d",
					DenoiserToUse != DefaultDenoiser ? TEXT("ThirdParty ") : TEXT(""),
					DenoiserToUse->GetDebugName(),
					View.ViewRect.Width(), View.ViewRect.Height());

				IScreenSpaceDenoiser::FDiffuseIndirectOutputs DenoiserOutputs = DenoiserToUse->DenoiseSkyLight(
					GraphBuilder,
					View,
					&View.PrevViewInfo,
					SceneTextures,
					DenoiserInputs,
					RayTracingConfig);

				// Need to set output used by the caller on the last iteration of the view loop
				if (ViewIndex == LastViewIndex)
				{
					OutSkyLightTexture = DenoiserOutputs.Color;
				}

				++ViewIndex;
			}
		}
	}

	for (FViewInfo& View : Views)
	{
		FSceneViewState* SceneViewState = (FSceneViewState*)View.State;
		if (SceneViewState != nullptr)
		{
			if (CVarRayTracingSkyLightDecoupleSampleGeneration.GetValueOnRenderThread() == 1)
			{
				// Set the Sky Light Visibility Ray dimensions and its extracted pooled RDG buffer on the scene view state
				GraphBuilder.QueueBufferExtraction(SkyLightVisibilityRaysBuffer, &SceneViewState->SkyLightVisibilityRaysBuffer);
				SceneViewState->SkyLightVisibilityRaysDimensions = SkyLightVisibilityRaysDimensions;
			}
			else
			{
				// Set invalid Sky Light Visibility Ray dimensions and pooled RDG buffer
				SceneViewState->SkyLightVisibilityRaysBuffer = nullptr;
				SceneViewState->SkyLightVisibilityRaysDimensions = FIntVector(1);
			}
		}
	}
}

class FCompositeSkyLightPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCompositeSkyLightPS)
	SHADER_USE_PARAMETER_STRUCT(FCompositeSkyLightPS, FGlobalShader)

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return ShouldCompileRayTracingShadersForProject(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		RENDER_TARGET_BINDING_SLOTS()
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SkyLightTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SkyLightTextureSampler)
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstrateGlobalUniformParameters, Substrate)

		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureParameters, SceneTextures)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCompositeSkyLightPS, "/Engine/Private/RayTracing/CompositeSkyLightPS.usf", "CompositeSkyLightPS", SF_Pixel);


#endif // RHI_RAYTRACING

void FDeferredShadingSceneRenderer::CompositeRayTracingSkyLight(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef SkyLightRT,
	FRDGTextureRef HitDistanceRT)
#if RHI_RAYTRACING
{
	check(SkyLightRT);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		FSceneTextureParameters SceneTextureParameters = GetSceneTextureParameters(GraphBuilder, SceneTextures.UniformBuffer);

		FCompositeSkyLightPS::FParameters *PassParameters = GraphBuilder.AllocParameters<FCompositeSkyLightPS::FParameters>();
		PassParameters->SkyLightTexture = SkyLightRT;
		PassParameters->SkyLightTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->View = Views[ViewIndex].GetShaderParameters();
		PassParameters->Substrate = Substrate::BindSubstrateGlobalUniformParameters(View);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneTextures.Color.Target, ERenderTargetLoadAction::ELoad);
		PassParameters->SceneTextures = SceneTextureParameters;

		// dxr_todo: Unify with RTGI compositing workflow
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("GlobalIlluminationComposite"),
			PassParameters,
			ERDGPassFlags::Raster,
			[this, &View, PassParameters, SceneTextureExtent = SceneTextures.Config.Extent](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			TShaderMapRef<FPostProcessVS> VertexShader(View.ShaderMap);
			TShaderMapRef<FCompositeSkyLightPS> PixelShader(View.ShaderMap);
			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			// Additive blending
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_One>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			RHICmdList.SetViewport((float)View.ViewRect.Min.X, (float)View.ViewRect.Min.Y, 0.0f, (float)View.ViewRect.Max.X, (float)View.ViewRect.Max.Y, 1.0f);

			DrawRectangle(
				RHICmdList,
				0, 0,
				View.ViewRect.Width(), View.ViewRect.Height(),
				View.ViewRect.Min.X, View.ViewRect.Min.Y,
				View.ViewRect.Width(), View.ViewRect.Height(),
				FIntPoint(View.ViewRect.Width(), View.ViewRect.Height()),
				SceneTextureExtent,
				VertexShader
			);
		});
	}
}
#else
{
	unimplemented();
}
#endif
