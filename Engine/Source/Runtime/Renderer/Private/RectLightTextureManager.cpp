// Copyright Epic Games, Inc. All Rights Reserved.

#include "RectLightTextureManager.h"
#include "Engine/Texture.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterStruct.h"
#include "RenderGraphUtils.h"
#include "ShaderPrintParameters.h"
#include "ShaderPrint.h"
#include "RenderingThread.h"
#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "SceneView.h"
#include "RHIDefinitions.h"
#include "SceneRendering.h"
#include "SystemTextures.h"
#include "TextureLayout.h"
#include "CommonRenderResources.h"
#include "ScreenPass.h"
#include "RectLightTexture.h"
#include "DataDrivenShaderPlatformInfo.h"

///////////////////////////////////////////////////////////////////////////////////////////////////
// Possible improvements:
// * When copying & filtering the texture, add RGBE/BCH6 encoding to reduce footprint and improve 
//   fetch perf
// * Support non-square atlas

static TAutoConsoleVariable<int32> CVarRectLightTextureResolution(
	TEXT("r.RectLightAtlas.MaxResolution"),
	4096,
	TEXT("The maximum resolution for storing rect. light textures.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRectLighTextureDebug(
	TEXT("r.RectLightAtlas.Debug"),
	0,
	TEXT("Enable rect. light atlas debug information."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRectLighTextureDebugMipLevel(
	TEXT("r.RectLightAtlas.Debug.MipLevel"),
	0,
	TEXT("Set MIP level for visualizing atlas texture in debug mode."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRectLighForceUpdate(
	TEXT("r.RectLightAtlas.ForceUpdate"),
	0,
	TEXT("Force rect. light atlas update very frame."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRectLighResetOnChange(
	TEXT("r.RectLightAtlas.ResetOnChange"),
	0,
	TEXT("Recreate the atlas layout from scratch each time an update is made to the atlas."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRectLighClearAtlas(
	TEXT("r.RectLightAtlas.ClearAtlas"),
	0,
	TEXT("Clear rect light texture atlas each time the atlas is recreated/updated."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRectLighFilterQuality(
	TEXT("r.RectLightAtlas.FilterQuality"),
	0,
	TEXT("Define the filtering quality used for filtering texture (0:Box filter, 1:Gaussian filter)."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRectLighMaxTextureRatio(
	TEXT("r.RectLightAtlas.MaxTextureRatio"),
	2,
	TEXT("Define the max Width/Height or Height/Width ratio that a texture can have."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRectLighAtlasFormat(
	TEXT("r.RectLightAtlas.Format"),
	0,
	TEXT("Define the atlas format used.\n - 0: R11G11B10 (faster but can caused hue shift). \n - 1: RGA16F (more accurate but slower)"),
	ECVF_RenderThreadSafe);

static bool PlatformSupportsRectLightAtlas(FStaticShaderPlatform Platform)
{
	// Mobile platforms currently do not suppport rect light source texture
	// See USE_SOURCE_TEXTURE in RectLight.ush
	return !IsMobilePlatform(Platform);
}

namespace RectLightAtlas
{

///////////////////////////////////////////////////////////////////////////////////////////////////
// Configuration

// Packing algo.
// Reference: "A Thousand Ways to Pack the Bin. A Practical Approach to Two-Dimensional Rectangle Bin Packing"
// 
// 0 : use a Shelf/Row packing algo       -> faster, but produces low-quality packing. 
// 1 : use a Skyline/Horizon packing algo -> more expensive, but provides better packing.
// 2 : use a FTextureLayout
#define USE_PACKING_MODE 1

// If enabled, track the deallocated free rects (i.e. 'holes' in the atlas), and try to allocate 
// slots from them when possible rather than expanding the atlas
#define USE_WASTE_RECT 1

///////////////////////////////////////////////////////////////////////////////////////////////////
// Structs & constants

static const uint32 InvalidSlotIndex = ~0u;
static const FIntPoint InvalidOrigin = FIntPoint(-1, -1);

static bool SupportWaveQuadPassing(EShaderPlatform In)
{
	return GRHISupportsWaveOperations && RHISupportsWaveOperations(In);
}

static FIntPoint GetSourceResolution(const FRHITexture* Tex, const FVector4f& ScaleOffset)
{
	FIntPoint Out(1, 1);
	if (Tex)
	{
		const FIntVector SourceOriginalResolution = Tex->GetSizeXYZ();

		float ScaledResolutionX = ScaleOffset.X * SourceOriginalResolution.X;
		float ScaledResolutionY = ScaleOffset.Y * SourceOriginalResolution.Y;

		const float RatioYX = ScaledResolutionY / ScaledResolutionX;
		const float RatioXY = ScaledResolutionX / ScaledResolutionY;

		Out = FIntPoint(ScaledResolutionX, ScaledResolutionY);
		// Max Ratio of 2
		const float RatioThreshold = FMath::Clamp(CVarRectLighMaxTextureRatio.GetValueOnAnyThread(), 1, 16);
		if (RatioYX > RatioThreshold)
		{
			Out.X *= RatioYX / RatioThreshold;
		}
		else if (RatioXY > RatioThreshold)
		{
			Out.Y *= RatioXY / RatioThreshold;
		}
	}
	return Out;
}

struct FAtlasRect
{
	FIntPoint Origin = InvalidOrigin;
	FIntPoint Resolution = FIntPoint::ZeroValue;
};

struct FAtlasHorizon
{
	FAtlasRect Line;
	FAtlasRect ExtendedLine;
};

// Store the current atlas layout, i.e. data for knowing where next insertion can be done
struct FAtlasLayout
{
	FAtlasLayout(const FIntPoint& MaxResolution = FIntPoint(256, 256))
		: AtlasResolution(MaxResolution)
		#if USE_PACKING_MODE == 2
		, Packer(256, 256, MaxResolution.X, MaxResolution.Y, true, ETextureLayoutAspectRatio::ForceSquare, true)
		#endif
	{

	}

	FIntPoint AtlasResolution = FIntPoint::ZeroValue;
	int32 SourceTextureMIPBias = 0;
#if USE_PACKING_MODE == 0
	int32 SplitX = 0;
	int32 SplitY = 0;
	int32 MaxY = 0;
#elif USE_PACKING_MODE == 1
	TArray<FAtlasHorizon> Horizons;
#elif USE_PACKING_MODE == 2
	FTextureLayout Packer;
#endif

#if USE_WASTE_RECT
	TArray<FAtlasRect> FreeRects;
#endif
};

// Store info of a current slot within the atlas
struct FAtlasSlot
{
	uint32 Id = InvalidSlotIndex;
	FAtlasRect Rect;
	FVector4f SourceScaleOffset;
	FTextureReferenceRHIRef SourceTexture = nullptr;
	FIntPoint CachedSourceTextureResolution = FIntPoint(0, 0);
	uint32 RefCount = 0;
	bool bForceRefresh = false;
	bool IsValid() const { return SourceTexture != nullptr; }
	FRHITexture* GetTextureRHI() const { return SourceTexture->GetReferencedTexture(); }

	FIntPoint GetSourceResolution() const 
	{ 
		return RectLightAtlas::GetSourceResolution(SourceTexture ? SourceTexture->GetReferencedTexture() : nullptr, SourceScaleOffset);
	}
};

// Store info for copying one atlas slot to another one, when a new layout is created
struct FAtlasCopySlot
{
	uint32 Id = InvalidSlotIndex;
	FIntPoint SrcOrigin = InvalidOrigin;
	FIntPoint DstOrigin = InvalidOrigin;
	FIntPoint Resolution = FIntPoint::ZeroValue;
	FTextureReferenceRHIRef SourceTexture = nullptr;
};

// Texture manager, holding all atlas data & description
struct FRectLightTextureManager : public FRenderResource
{
	TRefCountPtr<IPooledRenderTarget> AtlasTexture = nullptr;
	TArray<FAtlasSlot> AtlasSlots;
	TArray<FAtlasRect> DeletedSlots;
	TQueue<uint32> FreeSlots;
	FAtlasLayout AtlasLayout;

	bool bLock = false;
	bool bHasPendingAdds = false;
	bool bHasPendingDeletes = false;

	virtual void ReleaseRHI()
	{
		AtlasTexture.SafeRelease();
	}
};

// lives on the render thread
TGlobalResource<FRectLightTextureManager> GRectLightTextureManager;

///////////////////////////////////////////////////////////////////////////////////////////////////
// Utils functions

bool CanContain(const FAtlasRect& Outside, const FAtlasRect& Inside)
{
	return Inside.Resolution.X <= Outside.Resolution.X && Inside.Resolution.Y <= Outside.Resolution.Y;
}

static FIntPoint ToMIP(const FIntPoint& InMip, uint32 MipIndex)
{
	const int32 Div = 1 << MipIndex;

	FIntPoint Out;
	Out.X = FMath::DivideAndRoundUp(InMip.X,Div);
	Out.Y = FMath::DivideAndRoundUp(InMip.Y,Div);
	return Out;
}

static int32 GetSlotMaxMIPLevel(const FAtlasSlot& In)
{
	const uint32 MaxMIP = FMath::FloorLog2(FMath::Min(In.Rect.Resolution.X, In.Rect.Resolution.Y));
	return MaxMIP > 0u ? MaxMIP-1u : 0u;
}

// Align the slot resolution to prevent bilinear filtering to leak on neighbors' slot
static void AlignSlotResolution(FAtlasSlot& In)
{
	const uint32 MaxMIP = GetSlotMaxMIPLevel(In);
	const FIntPoint ResolutionMIP = ToMIP(In.Rect.Resolution, MaxMIP);
	In.Rect.Resolution = FIntPoint(ResolutionMIP.X << MaxMIP, ResolutionMIP.Y << MaxMIP);
}

static EPixelFormat GetRectLightAtlasFormat()
{
	return CVarRectLighAtlasFormat.GetValueOnRenderThread() > 0 ? PF_FloatRGBA : PF_FloatR11G11B10;
}

static bool Traits_IsValid(const FAtlasRect& In)					{ return true; }
static bool Traits_IsValid(const FAtlasHorizon& In)					{ return true; }
static bool Traits_IsValid(const FAtlasSlot& In)					{ return In.IsValid(); }
static const FAtlasRect& Traits_GetRect(const FAtlasRect& In)		{ return In; }
static const FAtlasRect& Traits_GetRect(const FAtlasHorizon& In)	{ return In.ExtendedLine; }
static const FAtlasRect& Traits_GetRect(const FAtlasSlot& In)		{ return In.Rect; }
static const FVector4f&	 Traits_GetScaleOffset(const FAtlasSlot& In) { return In.SourceScaleOffset; }

template<typename T>
static FRDGBufferRef CreateSlotBuffer(FRDGBuilder& GraphBuilder, const TArray<T>& In, const TCHAR* InBufferName)
{
	struct FAtlasGPUSlot
	{
		uint16 OffsetX;
		uint16 OffsetY;
		uint16 ResolutionX;
		uint16 ResolutionY;
	};

	if (In.IsEmpty())
	{
		return GSystemTextures.GetDefaultBuffer(GraphBuilder, sizeof(FAtlasGPUSlot), 0u);
	}

	TArray<FAtlasGPUSlot> SlotData;
	SlotData.Reserve(In.Num());
	for (const T& I : In)
	{
		if (Traits_IsValid(I))
		{
			const FAtlasRect& Rect = Traits_GetRect(I);
			FAtlasGPUSlot& GPUSlot	= SlotData.AddDefaulted_GetRef();
			GPUSlot.OffsetX			= Rect.Origin.X;
			GPUSlot.OffsetY			= Rect.Origin.Y;
			GPUSlot.ResolutionX		= Rect.Resolution.X;
			GPUSlot.ResolutionY		= Rect.Resolution.Y;
		}
	}
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FAtlasGPUSlot), SlotData.Num()), InBufferName);
	GraphBuilder.QueueBufferUpload(Buffer, SlotData.GetData(), SlotData.Num() * sizeof(FAtlasGPUSlot));
	return Buffer;
}

template<typename T>
static FRDGBufferRef CreateScaleOffsetBuffer(FRDGBuilder& GraphBuilder, const TArray<T>& In, const TCHAR* InBufferName)
{
	struct FAtlasGPUScaleOffset
	{
		float  SourceScaleX;
		float  SourceScaleY;
		float  SourceOffsetX;
		float  SourceOffsetY;
	};

	if (In.IsEmpty())
	{
		return GSystemTextures.GetDefaultBuffer(GraphBuilder, sizeof(FAtlasGPUScaleOffset), 0u);
	}

	TArray<FAtlasGPUScaleOffset> ScaleOffsetData;
	ScaleOffsetData.Reserve(In.Num());
	for (const T& I : In)
	{
		if (Traits_IsValid(I))
		{
			//Traits_GetRect(I);
			const FVector4f& ScaleOffset = Traits_GetScaleOffset(I);
			FAtlasGPUScaleOffset& GPUSlot	= ScaleOffsetData.AddDefaulted_GetRef();
			GPUSlot.SourceScaleX	= ScaleOffset.X;
			GPUSlot.SourceScaleY	= ScaleOffset.Y;
			GPUSlot.SourceOffsetX	= ScaleOffset.Z;
			GPUSlot.SourceOffsetY	= ScaleOffset.W;
		}
	}
	FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(FAtlasGPUScaleOffset), ScaleOffsetData.Num()), InBufferName);
	GraphBuilder.QueueBufferUpload(Buffer, ScaleOffsetData.GetData(), ScaleOffsetData.Num() * sizeof(FAtlasGPUScaleOffset));
	return Buffer;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Clear
#define RECT_ATLAS_DEBUG_CLEAR 0

#if RECT_ATLAS_DEBUG_CLEAR
class FRectLightAtlasClearCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRectLightAtlasClearCS);
	SHADER_USE_PARAMETER_STRUCT(FRectLightAtlasClearCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, Resolution)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return PlatformSupportsRectLightAtlas(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_CLEAR_TEXTURE"), 1);
	}
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FRectLightAtlasClearCS, "/Engine/Private/RectLightAtlas.usf", "MainCS", SF_Compute);

static FRDGTextureRef AddRectLightClearPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	FRDGTextureRef OutputTexture)
{
	const FIntPoint OutputResolution(OutputTexture->Desc.Extent);
	const uint32 MipCount = FMath::FloorLog2(OutputResolution.X);
	for (uint32 MipIt = 0; MipIt < MipCount; ++MipIt)
	{
		FRectLightAtlasClearCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRectLightAtlasClearCS::FParameters>();
		Parameters->Resolution.X = OutputResolution.X>>MipIt;
		Parameters->Resolution.Y = OutputResolution.Y>>MipIt;
		Parameters->OutputTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutputTexture, MipIt));

		TShaderMapRef<FRectLightAtlasClearCS> ComputeShader(ShaderMap);
		const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(OutputResolution.X, OutputResolution.Y, 1), FIntVector(8, 8, 1));
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RectLightAtlas::Clear"), ComputeShader, Parameters, DispatchCount);
	}

	return OutputTexture;
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////
// Debug
class FRectLightAtlasDebugInfoCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRectLightAtlasDebugInfoCS);
	SHADER_USE_PARAMETER_STRUCT(FRectLightAtlasDebugInfoCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, AtlasResolution)
		SHADER_PARAMETER(float, AtlasMaxMipLevel)
		SHADER_PARAMETER(float, Occupancy)
		SHADER_PARAMETER(uint32, SlotCount)
		SHADER_PARAMETER(uint32, HorizonCount)
		SHADER_PARAMETER(uint32, FreeCount)
		SHADER_PARAMETER(FIntPoint, OutputResolution)
		SHADER_PARAMETER(uint32, AtlasMIPIndex)
		SHADER_PARAMETER(uint32, AtlasSourceTextureMIPBias)
		SHADER_PARAMETER(uint32, AtlasFormat)
		SHADER_PARAMETER(uint32, ForceUpdate)
		SHADER_PARAMETER(uint32, ResetOnChange)
		SHADER_PARAMETER(uint32, ClearAtlas)
		SHADER_PARAMETER_SAMPLER(SamplerState, AtlasSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, AtlasTexture)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutputTexture)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, SlotBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, HorizonBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, FreeBuffer)
		SHADER_PARAMETER_STRUCT_INCLUDE(ShaderPrint::FShaderParameters, ShaderPrintParameters)
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return ShaderPrint::IsSupported(Parameters.Platform) && PlatformSupportsRectLightAtlas(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		ShaderPrint::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("SHADER_DEBUG"), 1);
	}
	static EShaderPermutationPrecacheRequest ShouldPrecachePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return EShaderPermutationPrecacheRequest::NotPrecached;
	}
};

IMPLEMENT_GLOBAL_SHADER(FRectLightAtlasDebugInfoCS, "/Engine/Private/RectLightAtlas.usf", "MainCS", SF_Compute);

static FRDGTextureRef AddRectLightDebugInfoPass(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	const FRDGTextureDesc& OutputDesc)
{
	// Force ShaderPrint on.
	ShaderPrint::SetEnabled(true);

	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(OutputDesc.Extent, OutputDesc.Format, FClearValueBinding::Black, TexCreate_UAV | TexCreate_ShaderResource), TEXT("RectLight.DebugTexture"));

	FGlobalShaderMap* ShaderMap = View.ShaderMap;

	FRDGTextureRef AtlasTexture = GRectLightTextureManager.AtlasTexture ? GraphBuilder.RegisterExternalTexture(GRectLightTextureManager.AtlasTexture) : GSystemTextures.GetBlackDummy(GraphBuilder);
	
	uint32 ValidSlotCount = 0;
	uint32 OccupiedPixels = 0;
	TArray<FAtlasSlot> ValidSlots;
	ValidSlots.Reserve(GRectLightTextureManager.AtlasSlots.Num());
	for (const FAtlasSlot& Slot : GRectLightTextureManager.AtlasSlots)
	{
		if (Slot.IsValid())
		{
			ValidSlots.Add(Slot);
			OccupiedPixels += Slot.Rect.Resolution.X * Slot.Rect.Resolution.Y;
		}
	}

#if USE_PACKING_MODE == 1
	const TArray<FAtlasHorizon>& Horizons = GRectLightTextureManager.AtlasLayout.Horizons;
	const TArray<FAtlasRect>& FreeRects = GRectLightTextureManager.AtlasLayout.FreeRects;
#else
	const TArray<FAtlasHorizon> Horizons;
	const TArray<FAtlasRect> FreeRects;
#endif

	// Create a buffer with all the valid slot to highlight them on the debug view
	FRDGBufferRef SlotBuffer = CreateSlotBuffer(GraphBuilder, ValidSlots, TEXT("RectLight.AtlasSlotBuffer"));
	FRDGBufferRef HorizonBuffer = CreateSlotBuffer(GraphBuilder, Horizons, TEXT("RectLight.HorizonBuffer"));
	FRDGBufferRef FreeBuffer = CreateSlotBuffer(GraphBuilder, FreeRects, TEXT("RectLight.FreeBuffer"));

	uint32 AtlasFormat = 2;
	switch (AtlasTexture->Desc.Format)
	{
		case PF_FloatR11G11B10 : AtlasFormat = 0; break;
		case PF_FloatRGBA : AtlasFormat = 1; break;
		default: AtlasFormat = 2; break;
	}	

	const FIntPoint OutputResolution(OutputTexture->Desc.Extent);
	FRectLightAtlasDebugInfoCS::FParameters* Parameters = GraphBuilder.AllocParameters<FRectLightAtlasDebugInfoCS::FParameters>();
	Parameters->AtlasResolution = AtlasTexture->Desc.Extent;
	Parameters->AtlasMaxMipLevel = AtlasTexture->Desc.NumMips;
	Parameters->AtlasSourceTextureMIPBias = GRectLightTextureManager.AtlasLayout.SourceTextureMIPBias;
	Parameters->AtlasFormat = AtlasFormat;
	Parameters->Occupancy = OccupiedPixels / float(AtlasTexture->Desc.Extent.X * AtlasTexture->Desc.Extent.Y);
	Parameters->SlotCount = ValidSlots.Num();
	Parameters->HorizonCount = Horizons.Num();
	Parameters->FreeCount = FreeRects.Num();
	Parameters->OutputResolution = OutputResolution;
	Parameters->ForceUpdate = CVarRectLighForceUpdate.GetValueOnRenderThread() > 0;
	Parameters->ResetOnChange = CVarRectLighResetOnChange.GetValueOnRenderThread() > 0;
	Parameters->ClearAtlas = CVarRectLighClearAtlas.GetValueOnRenderThread() > 0;
	Parameters->AtlasSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	Parameters->AtlasTexture = AtlasTexture;
	Parameters->SlotBuffer = GraphBuilder.CreateSRV(SlotBuffer, PF_R16G16B16A16_UINT);
	Parameters->HorizonBuffer = GraphBuilder.CreateSRV(HorizonBuffer, PF_R16G16B16A16_UINT);
	Parameters->FreeBuffer = GraphBuilder.CreateSRV(FreeBuffer, PF_R16G16B16A16_UINT);
	Parameters->ViewUniformBuffer = View.ViewUniformBuffer;
	Parameters->AtlasMIPIndex = FMath::Clamp(CVarRectLighTextureDebugMipLevel.GetValueOnRenderThread(), 0u, Parameters->AtlasMaxMipLevel-1);
	ShaderPrint::SetParameters(GraphBuilder, View.ShaderPrintData, Parameters->ShaderPrintParameters);
	Parameters->OutputTexture = GraphBuilder.CreateUAV(OutputTexture);

	TShaderMapRef<FRectLightAtlasDebugInfoCS> ComputeShader(ShaderMap);
	const FIntVector DispatchCount = DispatchCount.DivideAndRoundUp(FIntVector(OutputResolution.X, OutputResolution.Y, 1), FIntVector(8, 8, 1));
	FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("RectLightAtlas::DebugInfo"), ComputeShader, Parameters, DispatchCount);

	return OutputTexture;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Rect VS

class FRectLightAtlasVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRectLightAtlasVS);
	SHADER_USE_PARAMETER_STRUCT(FRectLightAtlasVS, FGlobalShader);
	
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, SlotBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, ScaleOffsetBuffer)
		SHADER_PARAMETER(uint32, SlotBufferOffset)
		SHADER_PARAMETER(FIntPoint, AtlasResolution)
		SHADER_PARAMETER(int32, bHasScaleOffset)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return PlatformSupportsRectLightAtlas(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_RECT_VS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRectLightAtlasVS, "/Engine/Private/RectLightAtlas.usf", "MainVS", SF_Vertex);

///////////////////////////////////////////////////////////////////////////////////////////////////
// Add pass

class FRectAtlasAddTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRectAtlasAddTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FRectAtlasAddTexturePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, InTextureMIPBias)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture0)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture1)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture2)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture3)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture4)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture5)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture6)
		SHADER_PARAMETER_TEXTURE(Texture2D, InTexture7)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSampler0)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSampler1)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSampler2)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSampler3)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSampler4)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSampler5)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSampler6)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSampler7)
		SHADER_PARAMETER_STRUCT_INCLUDE(FRectLightAtlasVS::FParameters, VS)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutAtlasTextureMip1)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return PlatformSupportsRectLightAtlas(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_ADD_TEXTURE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRectAtlasAddTexturePS, "/Engine/Private/RectLightAtlas.usf", "MainPS", SF_Pixel);

// Insert a texture into the atlas
static void AddSlotsPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const int32 TextureMIPBias,
	const TArray<FAtlasSlot>& Slots,
	FRDGTextureRef& OutAtlas)
{
	const FIntRect Viewport(FIntPoint::ZeroValue, OutAtlas->Desc.Extent);
	const FIntPoint Resolution(Viewport.Width(), Viewport.Height());

	FRDGBufferRef SlotBuffer		= CreateSlotBuffer(GraphBuilder, Slots, TEXT("RectLight.AtlasSlotBuffer"));
	FRDGBufferRef ScaleOffsetBuffer = CreateScaleOffsetBuffer(GraphBuilder, Slots, TEXT("RectLight.AtlasScaleOffsetBuffer"));

	// If compiler support quad pass, use it to generate first MIP during atlas copy
	const bool bUseQuadPassing = SupportWaveQuadPassing(ShaderMap->GetPlatform());

	// Batch new slots into several passes
	const uint32 SlotCountPerPass = 8u;
	const uint32 PassCount = FMath::DivideAndRoundUp(uint32(Slots.Num()), SlotCountPerPass);
	for (uint32 PassIt = 0; PassIt < PassCount; ++PassIt)
	{
		const uint32 SlotOffset = PassIt * SlotCountPerPass;
		const uint32 SlotCount = SlotCountPerPass * (PassIt+1) <= uint32(Slots.Num()) ? SlotCountPerPass : uint32(Slots.Num()) - (SlotCountPerPass * PassIt);
		
		FRectAtlasAddTexturePS::FParameters* Parameters = GraphBuilder.AllocParameters<FRectAtlasAddTexturePS::FParameters>();
		// Init. source texure
		{
			Parameters->InTexture0 = GSystemTextures.BlackDummy->GetRHI();
			Parameters->InTexture1 = GSystemTextures.BlackDummy->GetRHI();
			Parameters->InTexture2 = GSystemTextures.BlackDummy->GetRHI();
			Parameters->InTexture3 = GSystemTextures.BlackDummy->GetRHI();
			Parameters->InTexture4 = GSystemTextures.BlackDummy->GetRHI();
			Parameters->InTexture5 = GSystemTextures.BlackDummy->GetRHI();
			Parameters->InTexture6 = GSystemTextures.BlackDummy->GetRHI();
			Parameters->InTexture7 = GSystemTextures.BlackDummy->GetRHI();
		}
		for (uint32 SlotIt = 0; SlotIt<SlotCount;++SlotIt)
		{
			const FAtlasSlot& Slot = Slots[SlotOffset + SlotIt];
			check(Slot.SourceTexture);

			switch (SlotIt)
			{
			case 0: Parameters->InTexture0 = Slot.GetTextureRHI(); break;
			case 1: Parameters->InTexture1 = Slot.GetTextureRHI(); break;
			case 2: Parameters->InTexture2 = Slot.GetTextureRHI(); break;
			case 3: Parameters->InTexture3 = Slot.GetTextureRHI(); break;
			case 4: Parameters->InTexture4 = Slot.GetTextureRHI(); break;
			case 5: Parameters->InTexture5 = Slot.GetTextureRHI(); break;
			case 6: Parameters->InTexture6 = Slot.GetTextureRHI(); break;
			case 7: Parameters->InTexture7 = Slot.GetTextureRHI(); break;
			}
		}

		FRHISamplerState* SamplerState = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		Parameters->InSampler0 = SamplerState;
		Parameters->InSampler1 = SamplerState;
		Parameters->InSampler2 = SamplerState;
		Parameters->InSampler3 = SamplerState;
		Parameters->InSampler4 = SamplerState;
		Parameters->InSampler5 = SamplerState;
		Parameters->InSampler6 = SamplerState;
		Parameters->InSampler7 = SamplerState;

		Parameters->InTextureMIPBias = TextureMIPBias;
		Parameters->VS.AtlasResolution = Resolution;
		Parameters->VS.SlotBufferOffset = SlotOffset;
		Parameters->VS.SlotBuffer = GraphBuilder.CreateSRV(SlotBuffer, PF_R16G16B16A16_UINT);
		Parameters->VS.ScaleOffsetBuffer = GraphBuilder.CreateSRV(ScaleOffsetBuffer, PF_A32B32G32R32F);
		Parameters->VS.bHasScaleOffset   = true;
		if (bUseQuadPassing)
		{
			Parameters->OutAtlasTextureMip1 = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutAtlas, 1));
		}
		Parameters->RenderTargets[0] = FRenderTargetBinding(OutAtlas, ERenderTargetLoadAction::ELoad, 0);

		TShaderMapRef<FRectLightAtlasVS> VertexShader(ShaderMap);
		TShaderMapRef<FRectAtlasAddTexturePS> PixelShader(ShaderMap);
	
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("RectLightAtlas::AddTexturePass(Slot:%d)", SlotCount),
			Parameters,
			ERDGPassFlags::Raster,
			[Parameters, VertexShader, PixelShader, Viewport, Resolution, SlotCount](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
				SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);
				SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), Parameters->VS);

				RHICmdList.SetStreamSource(0, nullptr, 0);
				RHICmdList.DrawPrimitive(0, 2, SlotCount);
			});
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Copy pass

class FRectAtlasCopyTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRectAtlasCopyTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FRectAtlasCopyTexturePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MipLevel)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, SrcSlotBuffer)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, SourceAtlasTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(FRectLightAtlasVS::FParameters, VS)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return PlatformSupportsRectLightAtlas(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_COPY_TEXTURE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRectAtlasCopyTexturePS, "/Engine/Private/RectLightAtlas.usf", "MainPS", SF_Pixel);

// Copy all slots (and their mip) from InAtlas to OutAtlas
// This is done when atlas need a full repacking, changing the slot layout 
static void CopySlotsPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const TArray<FAtlasCopySlot>& InSlots,
	const FRDGTextureRef& InAtlas,
	FRDGTextureRef& OutAtlas)
{
	FRDGBufferSRVRef DummyScaleOffsetBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 16u, 1u), PF_A32B32G32R32F);

	// For each MIP, copy the slot from the old atlas (InAtlas) to the new atlas (OutAtlas)	
	const uint32 MipCount = InAtlas->Desc.NumMips; // FMath::Log2(FMath::Min(InAtlas->Desc.Extent.X, InAtlas->Desc.Extent.Y));
	for (uint32 MipIt = 0; MipIt < MipCount; ++MipIt)
	{
		TArray<FAtlasSlot> SrcSlots;
		TArray<FAtlasSlot> DstSlots;
		SrcSlots.Reserve(InSlots.Num());
		DstSlots.Reserve(InSlots.Num());
		for (const FAtlasCopySlot& Slot : InSlots)
		{
			const bool bIsValidMIP = (Slot.Resolution.X >> MipIt) > 1 && (Slot.Resolution.Y >> MipIt) > 1;
			if (bIsValidMIP)
			{
				{
					FAtlasSlot& DstSlot = DstSlots.AddDefaulted_GetRef();
					DstSlot.Rect.Origin = ToMIP(Slot.DstOrigin, MipIt);
					DstSlot.Rect.Resolution = ToMIP(Slot.Resolution, MipIt);
					DstSlot.SourceTexture = Slot.SourceTexture;
				}

				{
					FAtlasSlot& SrcSlot = SrcSlots.AddDefaulted_GetRef();
					SrcSlot.Rect.Origin = ToMIP(Slot.SrcOrigin, MipIt);
					SrcSlot.Rect.Resolution = ToMIP(Slot.Resolution, MipIt);
					SrcSlot.SourceTexture = Slot.SourceTexture;
				}
			}
		}

		const uint32 SlotCount = DstSlots.Num();
		if (SlotCount > 0)
		{
			FRDGBufferRef SrcSlotBuffer = CreateSlotBuffer(GraphBuilder, SrcSlots, TEXT("RectLight.SrcSlotBuffer"));
			FRDGBufferRef DstSlotBuffer = CreateSlotBuffer(GraphBuilder, DstSlots, TEXT("RectLight.DstSlotBuffer"));

			const FIntPoint Resolution = ToMIP(OutAtlas->Desc.Extent.X, MipIt);
			const FIntRect Viewport(FIntPoint::ZeroValue, Resolution);

			FRectAtlasCopyTexturePS::FParameters* Parameters = GraphBuilder.AllocParameters<FRectAtlasCopyTexturePS::FParameters>();
			Parameters->SourceAtlasTexture = InAtlas;
			Parameters->MipLevel = MipIt;
			Parameters->SrcSlotBuffer = GraphBuilder.CreateSRV(SrcSlotBuffer, PF_R16G16B16A16_UINT);
			Parameters->VS.AtlasResolution = Resolution;
			Parameters->VS.SlotBufferOffset = 0;
			Parameters->VS.SlotBuffer = GraphBuilder.CreateSRV(DstSlotBuffer, PF_R16G16B16A16_UINT);
			Parameters->VS.ScaleOffsetBuffer = DummyScaleOffsetBuffer;
			Parameters->VS.bHasScaleOffset   = false;
		
			Parameters->RenderTargets[0] = FRenderTargetBinding(OutAtlas, ERenderTargetLoadAction::ELoad, MipIt);

			TShaderMapRef<FRectLightAtlasVS> VertexShader(ShaderMap);
			TShaderMapRef<FRectAtlasCopyTexturePS> PixelShader(ShaderMap);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("RectLightAtlas::CopyTexturePass(MIP:%d,Slots:%d)", MipIt, SlotCount),
				Parameters,
				ERDGPassFlags::Raster,
				[Parameters, VertexShader, PixelShader, Viewport, Resolution, SlotCount](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);
					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), Parameters->VS);

					RHICmdList.SetStreamSource(0, nullptr, 0);
					RHICmdList.DrawPrimitive(0, 2, SlotCount);
				});
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Filtering pass

class FRectAtlasFilterTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRectAtlasFilterTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FRectAtlasFilterTexturePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, SrcAtlasResolution)
		SHADER_PARAMETER(uint32, FilterQuality)
		SHADER_PARAMETER(uint32, bWriteMIP1)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, DstSlotBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer, SrcSlotBuffer)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceAtlasSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, SourceAtlasTexture)
		SHADER_PARAMETER_STRUCT_INCLUDE(FRectLightAtlasVS::FParameters, VS)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutAtlasTextureMip1)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return PlatformSupportsRectLightAtlas(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SHADER_FILTER_TEXTURE"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRectAtlasFilterTexturePS, "/Engine/Private/RectLightAtlas.usf", "MainPS", SF_Pixel);

// Filter all provided slots
static void FilterSlotsPass(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const TArray<FAtlasSlot>& InSlots,
	FRDGTextureRef& InAtlas)
{
	FRDGBufferSRVRef DummyScaleOffsetBuffer = GraphBuilder.CreateSRV(GSystemTextures.GetDefaultBuffer(GraphBuilder, 16u, 1u), PF_A32B32G32R32F);

	// If QuadPass is available generate two MIPs at a time when using box filtering
	const uint32 FilteringQuality = FMath::Clamp(CVarRectLighFilterQuality.GetValueOnRenderThread(), 0, 1);
	const bool bUseQuadPassing = SupportWaveQuadPassing(ShaderMap->GetPlatform()) && FilteringQuality == 0;

	FRDGTextureRef DummyTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(1, 1), InAtlas->Desc.Format, FClearValueBinding::Black, TexCreate_UAV), TEXT("RectLight.DummyAtlasMIP"));
	FRDGTextureUAVRef DummyTextureUAV = GraphBuilder.CreateUAV(DummyTexture);

	const uint32 MipCount = InAtlas->Desc.NumMips; // FMath::Log2(FMath::Min(InAtlas->Desc.Extent.X, InAtlas->Desc.Extent.Y));
	for (uint32 DstMip = (bUseQuadPassing ? 2u : 1u); DstMip < MipCount; DstMip += (bUseQuadPassing ? 2u : 1u))
	{
		const uint32 SrcMip = DstMip-1;

		TArray<FAtlasSlot> SrcMIPSlots;
		TArray<FAtlasSlot> DstMIPSlots;
		SrcMIPSlots.Reserve(InSlots.Num());
		DstMIPSlots.Reserve(InSlots.Num());
		for (const FAtlasSlot& Slot : InSlots)
		{
			const bool bIsValidMIP = (Slot.Rect.Resolution.X >> DstMip) > 1 && (Slot.Rect.Resolution.Y >> DstMip) > 1;
			if (bIsValidMIP)
			{
				{
					FAtlasSlot& DstMIPSlot		= DstMIPSlots.AddDefaulted_GetRef();
					DstMIPSlot.Rect.Origin		= ToMIP(Slot.Rect.Origin, DstMip);
					DstMIPSlot.Rect.Resolution	= ToMIP(Slot.Rect.Resolution, DstMip);
					DstMIPSlot.SourceTexture	= Slot.SourceTexture;
				}

				{
					FAtlasSlot& SrcMIPSlot		= SrcMIPSlots.AddDefaulted_GetRef();
					SrcMIPSlot.Rect.Origin		= ToMIP(Slot.Rect.Origin, SrcMip);
					SrcMIPSlot.Rect.Resolution	= ToMIP(Slot.Rect.Resolution, SrcMip);
					SrcMIPSlot.SourceTexture	= Slot.SourceTexture;
				}
			}
		}

		const uint32 SlotCount = DstMIPSlots.Num();
		if (SlotCount > 0)
		{
			FRDGBufferRef SrcSlotBuffer = CreateSlotBuffer(GraphBuilder, SrcMIPSlots, TEXT("RectLight.SrcMIPSlotBuffer"));
			FRDGBufferRef DstSlotBuffer = CreateSlotBuffer(GraphBuilder, DstMIPSlots, TEXT("RectLight.DstMIPSlotBuffer"));

			FRectAtlasFilterTexturePS::FParameters* Parameters = GraphBuilder.AllocParameters<FRectAtlasFilterTexturePS::FParameters>();
			Parameters->FilterQuality		= FilteringQuality;
			Parameters->SrcAtlasResolution	= ToMIP(InAtlas->Desc.Extent, SrcMip);
			Parameters->DstSlotBuffer		= GraphBuilder.CreateSRV(DstSlotBuffer, PF_R16G16B16A16_UINT);
			Parameters->SrcSlotBuffer		= GraphBuilder.CreateSRV(SrcSlotBuffer, PF_R16G16B16A16_UINT);
			Parameters->SourceAtlasTexture	= GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(InAtlas, SrcMip));
			Parameters->SourceAtlasSampler  = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters->VS.AtlasResolution	= ToMIP(InAtlas->Desc.Extent, DstMip);
			Parameters->VS.SlotBufferOffset = 0;
			Parameters->VS.SlotBuffer		= GraphBuilder.CreateSRV(DstSlotBuffer, PF_R16G16B16A16_UINT);
			Parameters->VS.ScaleOffsetBuffer= DummyScaleOffsetBuffer;
			Parameters->VS.bHasScaleOffset  = false;
			Parameters->bWriteMIP1			= (bUseQuadPassing && DstMip+1 < InAtlas->Desc.NumMips) ? 1u : 0u;
			Parameters->OutAtlasTextureMip1 = bUseQuadPassing && Parameters->bWriteMIP1 ? GraphBuilder.CreateUAV(FRDGTextureUAVDesc(InAtlas, DstMip+1)) : DummyTextureUAV;
			Parameters->RenderTargets[0] = FRenderTargetBinding(InAtlas, ERenderTargetLoadAction::ELoad, DstMip);

			TShaderMapRef<FRectLightAtlasVS> VertexShader(ShaderMap);
			TShaderMapRef<FRectAtlasFilterTexturePS> PixelShader(ShaderMap);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("RectLightAtlas::FilterTexturePass(Mip:%d,Slots:%d)", DstMip, SlotCount),
				Parameters,
				ERDGPassFlags::Raster,
				[Parameters, VertexShader, PixelShader, SlotCount](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					const FIntPoint Resolution = Parameters->VS.AtlasResolution;
					const FIntRect Viewport(FIntPoint::ZeroValue, Resolution);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					RHICmdList.SetViewport(Viewport.Min.X, Viewport.Min.Y, 0.0f, Viewport.Max.X, Viewport.Max.Y, 1.0f);
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *Parameters);
					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), Parameters->VS);

					RHICmdList.SetStreamSource(0, nullptr, 0);
					RHICmdList.DrawPrimitive(0, 2, SlotCount);
				});
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Internal update

// Compute an aligned version of a rect. 
// The output rect will be smaller, but its origin will respect the alignement requirement
static FAtlasRect ComputeAlignedRect(const FAtlasRect& In, int32 Alignement)
{
	FAtlasRect Out;
	Out.Origin = FIntPoint(
		FMath::DivideAndRoundUp(In.Origin.X, Alignement) * Alignement,
		FMath::DivideAndRoundUp(In.Origin.Y, Alignement) * Alignement);
	Out.Resolution = (In.Resolution + In.Origin) - Out.Origin;
	return Out;
}

// Find a valid slot among the free rects
static bool FindFreeRect(FAtlasLayout& Layout, FAtlasSlot& Slot, int32 Alignement)
{
#if USE_WASTE_RECT
	// 1. Find the smallest free rect than can contains the slot request
	int32 BestFreeRectIt = -1;
	int32 BestFreeRectResX = Layout.AtlasResolution.X * 2;
	FAtlasRect BestAlignedFreeRect;
	for (int32 FreeIt = 0, FreeCount = Layout.FreeRects.Num(); FreeIt < FreeCount; ++FreeIt)
	{
		// Compute the aligned free rect, to respect the slot alignement requirement
		const FAtlasRect& FreeRect = Layout.FreeRects[FreeIt];
		const FAtlasRect AlignedFreeRect = ComputeAlignedRect(FreeRect, Alignement);			
		if (CanContain(AlignedFreeRect, Slot.Rect) && AlignedFreeRect.Resolution.X < BestFreeRectResX)
		{
			BestFreeRectIt = FreeIt;
			BestFreeRectResX = FreeRect.Resolution.X;
			BestAlignedFreeRect = AlignedFreeRect;
		}
	}

	// 2. If found, split the rest of the valid free rect into two parts
	//  ---------                  -----
	// |Slot|    |                |     |
	// |    |    |   ->           |Free0|
	// |----     |         -----  |     |
	// |         |        |Free1| |     |
	//  ---------          -----   -----
	// Fitted new			  New free
	//   slot                  rects
	const bool bFit = BestFreeRectIt != -1;
	if (bFit)
	{
		// Assign the slot origin
		Slot.Rect.Origin = BestAlignedFreeRect.Origin;

		// Find the split for the two new rects
		const FAtlasRect BestFreeRect = Layout.FreeRects[BestFreeRectIt];
		check(BestFreeRect.Origin.X >= 0 && BestFreeRect.Origin.Y >= 0);
		Layout.FreeRects.RemoveAtSwap(BestFreeRectIt);

		// If the new Slot doesn't fit the rect, split the remaining empty space into new free rects
		#if 1
		const bool bFitX = Slot.Rect.Origin.X + Slot.Rect.Resolution.X == BestFreeRect.Origin.X + BestFreeRect.Resolution.X;
		const bool bFitY = Slot.Rect.Origin.Y + Slot.Rect.Resolution.Y == BestFreeRect.Origin.Y + BestFreeRect.Resolution.Y;
		// 1. Perfect fit, or right-fit
		//  ---------  
		// ||'''''''|| 
		// ||       || 
		// ||       || 
		// ||.......|| 
		//  ---------  
		if (bFitX && bFitY)
		{
			// The slot takes all the free. rect space. Nothing to do.
		}
		// 2. Width fit
		//  ---------  
		// ||'''''''|| 
		// ||.......|| 
		// |         | 
		// |         | 
		//  ---------  
		else if (bFitX)
		{
			FAtlasRect& FreeRect  = Layout.FreeRects.AddDefaulted_GetRef();
			FreeRect.Origin.X     = Slot.Rect.Origin.X + Slot.Rect.Resolution.X;
			FreeRect.Origin.Y     = BestFreeRect.Origin.Y;
			FreeRect.Resolution.X =(BestFreeRect.Origin.X+ BestFreeRect.Resolution.X) - FreeRect.Origin.X;
			FreeRect.Resolution.Y = BestFreeRect.Resolution.Y;
		}
		// 3. Height fit
		//  ---------  
		// ||''''|   | 
		// ||    |   | 
		// ||    |   | 
		// ||....|   | 
		//  ---------  
		else if (bFitY)
		{
			FAtlasRect& FreeRect  = Layout.FreeRects.AddDefaulted_GetRef();
			FreeRect.Origin.X     = BestFreeRect.Origin.X;
			FreeRect.Origin.Y     = Slot.Rect.Origin.Y + Slot.Rect.Resolution.Y;
			FreeRect.Resolution.X = BestFreeRect.Resolution.X;
			FreeRect.Resolution.Y = (BestFreeRect.Origin.Y + BestFreeRect.Resolution.Y) - FreeRect.Origin.Y;
		}
		// 4. Two splits
		//  ---------  
		// ||''''|   | 
		// ||    |   | 
		// ||....|   |
		// |         | 
		//  ---------  
		else
		{
			{
				FAtlasRect& FreeRect  = Layout.FreeRects.AddDefaulted_GetRef();
				FreeRect.Origin.X     = BestFreeRect.Origin.X;
				FreeRect.Origin.Y     = Slot.Rect.Origin.Y + Slot.Rect.Resolution.Y;
				FreeRect.Resolution.X =(Slot.Rect.Origin.X + Slot.Rect.Resolution.X) - FreeRect.Origin.X;
				FreeRect.Resolution.Y =(BestFreeRect.Origin.Y + BestFreeRect.Resolution.Y) - FreeRect.Origin.Y;
			}
			{
				FAtlasRect& FreeRect  = Layout.FreeRects.AddDefaulted_GetRef();
				FreeRect.Origin.X     = Slot.Rect.Origin.X + Slot.Rect.Resolution.X;
				FreeRect.Origin.Y     = BestFreeRect.Origin.Y;
				FreeRect.Resolution.X =(BestFreeRect.Origin.X + BestFreeRect.Resolution.X) - FreeRect.Origin.X;
				FreeRect.Resolution.Y = BestFreeRect.Resolution.Y;
			}
		}
		#endif
	}

	return bFit;
#endif // USE_WASTE_RECT
}


// Fit a new slot into the curent layout. Returns true if the slot can be fitted, false otherwise
static bool FitAlignedSlot(FAtlasLayout& Layout, FAtlasSlot& Slot)
#if USE_PACKING_MODE == 0
// Use Shelf packing algo.
{
	// Due to MIPs we need to ensure that the rect is aligned onto its pow2 resolution
	const uint32 MaxMIP = GetSlotMaxMIPLevel(Slot);
	const int32 Alignement = 1u << MaxMIP;

	// 1. Find a valid slot among the free rects
	bool bFit = FindFreeRect(Layout, Slot, Alignement);
	
	// 2. Try to fit slot 
	if (!bFit)
	{

		const FIntPoint AlignedSplit(
			FMath::DivideAndRoundUp(Layout.SplitX, Alignement) * Alignement,
			FMath::DivideAndRoundUp(Layout.SplitY, Alignement) * Alignement);
		const int32 AlignedMaxY = FMath::DivideAndRoundUp(Layout.MaxY, Alignement) * Alignement;

		// 2.1. Try to fit into the current row
		if (
			AlignedSplit.X + Slot.Rect.Resolution.X <= Layout.AtlasResolution.X &&
			AlignedSplit.Y + Slot.Rect.Resolution.Y <= Layout.AtlasResolution.Y)
		{
			//Slot.Origin = FIntPoint(Layout.SplitX, Layout.SplitY);
			Slot.Rect.Origin = FIntPoint(AlignedSplit.X, AlignedSplit.Y);
			Layout.SplitX += Slot.Rect.Resolution.X;
			Layout.MaxY = FMath::Max(Layout.MaxY, AlignedSplit.Y + Slot.Rect.Resolution.Y);
		}
		// 2.2. Try to fit in a the next row
		else if (
			Slot.Rect.Resolution.X <= Layout.AtlasResolution.X &&
			AlignedMaxY + Slot.Rect.Resolution.Y <= Layout.AtlasResolution.Y)
		{
			Layout.SplitX = 0;
			Layout.SplitY = Layout.MaxY;
			Slot.Rect.Origin = FIntPoint(Layout.SplitX, AlignedMaxY);

			Layout.SplitX = Slot.Rect.Resolution.X;
			Layout.MaxY = AlignedMaxY + Slot.Rect.Resolution.Y;
		}
		// 2.3. It doesn't fit with the current atlas resolution
		else
		{
			bFit = false;
		}
	}

	return bFit;
}
#elif USE_PACKING_MODE == 1
// Use skyline packing algo.
{
	// Due to MIPs we need to ensure that the rect is aligned onto its pow2 resolution
	const uint32 MaxMIP = GetSlotMaxMIPLevel(Slot);
	const int32 Alignement = 1u << MaxMIP;

	// 0. If the layout is verbatim, initialize the horizon data
	if (Layout.Horizons.IsEmpty())
	{
		FAtlasHorizon& H = Layout.Horizons.AddDefaulted_GetRef();
		H.Line.Origin = FIntPoint(0, 0);
		H.Line.Resolution = Layout.AtlasResolution;
		H.ExtendedLine = H.Line;
	}

	// 1. Find a valid slot among the free rects
	bool bFit = FindFreeRect(Layout, Slot, Alignement);

	// 2. Fit rectangle within the existing horizon
	if (!bFit)
	{
		// 2.2 Find the horizon line which result in the lowest horizon after update
		int32 MinHeight = Layout.AtlasResolution.Y * 2; // *2 to ensure MinHeight is initialized with a larger height than the default horizon (i.e., full atlas)
		int32 BestHorizonIt = -1;
		FAtlasRect BestAlignedHorizon;
		for (int32 HorizonIt = 0, HorizonCount = Layout.Horizons.Num(); HorizonIt < HorizonCount; ++HorizonIt)
		{
			// Compute the aligned horizon for taking into account filtering/MIP-mapping requirement
			const FAtlasRect AlignedHorizon = ComputeAlignedRect(Layout.Horizons[HorizonIt].ExtendedLine, Alignement);
			if (CanContain(AlignedHorizon, Slot.Rect))
			{
				if (AlignedHorizon.Origin.Y + Slot.Rect.Resolution.Y < MinHeight)
				{
					MinHeight = AlignedHorizon.Origin.Y + Slot.Rect.Resolution.Y;
					BestAlignedHorizon = AlignedHorizon;
					BestHorizonIt = HorizonIt;
				}
			}
		}
	
		// 2.3. Insert & update horizon
		bFit = BestHorizonIt != -1;
		if (bFit)
		{
			// The slot is inserted with Left-most insertion.
			// TODO compute waste space heuristic left/right to pick the best
			Slot.Rect.Origin = BestAlignedHorizon.Origin;

			// 2.3.1 Update Horizon segments
			const uint32 SlotY = Slot.Rect.Origin.Y + Slot.Rect.Resolution.Y;
			const uint32 StartX = Slot.Rect.Origin.X;
			const uint32 EndX = Slot.Rect.Origin.X + Slot.Rect.Resolution.X;
			{
				TArray<FAtlasHorizon> NewHorizonRects;
				NewHorizonRects.Reserve(Layout.Horizons.Num());
				for (int32 HorizonIt = 0, HorizonCount = Layout.Horizons.Num(); HorizonIt < HorizonCount; ++HorizonIt)
				{
					const FAtlasHorizon& Horizon = Layout.Horizons[HorizonIt];
					const uint32 HorizonStartX = Horizon.Line.Origin.X;
					const uint32 HorizonEndX = Horizon.Line.Origin.X + Horizon.Line.Resolution.X;
					const uint32 HorizonY = Horizon.Line.Origin.Y;

					const bool bIntersection = !(EndX < HorizonStartX || StartX > HorizonEndX);
					if (bIntersection)
					{
						// Sanity check
						//check(HorizonY <= SlotY);

						// 1. Complete cover
						// Slot    |----------|  ->  Output  |----------|
						// Horizon    |---|	     ->          
						if (StartX <= HorizonStartX && EndX >= HorizonEndX)
						{
							if ((StartX == HorizonStartX && EndX == HorizonEndX) || StartX == HorizonStartX)
							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = StartX;
								H.Line.Origin.Y = SlotY;
								H.Line.Resolution.X = EndX - StartX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - SlotY;
								H.ExtendedLine = H.Line;
							}

							// Skip horizon line, as it is totally occluded
						}
						// 2. Inside cover
						// Slot       |---|	     ->  Output     |---|
						// Horizon |----------|  ->          |--|   |---|
						else if (StartX > HorizonStartX && EndX < HorizonEndX)
						{
							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = HorizonStartX;
								H.Line.Origin.Y = HorizonY;
								H.Line.Resolution.X = StartX - HorizonStartX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - HorizonY;
								H.ExtendedLine = H.Line;
							}

							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = StartX;
								H.Line.Origin.Y = SlotY;
								H.Line.Resolution.X = EndX - StartX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - SlotY;
								H.ExtendedLine = H.Line;
							}

							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = EndX;
								H.Line.Origin.Y = HorizonY;
								H.Line.Resolution.X = HorizonEndX - EndX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - HorizonY;
								H.ExtendedLine = H.Line;
							}
						}
						// 3. Left cover
						// Slot    |---|		 ->  Output  |--|
						// Horizon |----------|	 ->             |-------|
						else if (StartX <= HorizonStartX && EndX < HorizonEndX)
						{
							//if (StartX == HorizonStartX && SlotY < uint32(Layout.AtlasResolution.Y))
							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = StartX;
								H.Line.Origin.Y = SlotY;
								H.Line.Resolution.X = EndX - StartX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - SlotY;
								H.ExtendedLine = H.Line;
							}

							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = EndX;
								H.Line.Origin.Y = HorizonY;
								H.Line.Resolution.X = HorizonEndX - EndX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - HorizonY;
								H.ExtendedLine = H.Line;
							}
						}
						// 4. Right cover
						// Slot           |---|  ->  Output          |--|
						// Horizon |----------|  ->          |-------|
						else if (StartX > HorizonStartX && EndX <= HorizonEndX)
						{
							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = HorizonStartX;
								H.Line.Origin.Y = HorizonY;
								H.Line.Resolution.X = StartX - HorizonStartX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - HorizonY;
								H.ExtendedLine = H.Line;
							}

							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = StartX;
								H.Line.Origin.Y = SlotY;
								H.Line.Resolution.X = EndX - StartX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - SlotY;
								H.ExtendedLine = H.Line;
							}
						}
						// 5. Right cover with overflow (merge this case with 4.)
						// Slot           |------|  ->  Output         |------|
						// Horizon |----------|     ->          |------|       
						else if (StartX > HorizonStartX && StartX < HorizonEndX)
						{
							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = HorizonStartX;
								H.Line.Origin.Y = HorizonY;
								H.Line.Resolution.X = StartX - HorizonStartX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - HorizonY;
								H.ExtendedLine = H.Line;
							}

							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = StartX;
								H.Line.Origin.Y = SlotY;
								H.Line.Resolution.X = EndX - StartX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - SlotY;
								H.ExtendedLine = H.Line;
							}
						}
						// 6. Left cover with overflow (merge this case with 3.)
						// Slot    |-----|		    ->  Output  |-----|
						// Horizon   |----------|	->                |----|
						else if (EndX > StartX && EndX < HorizonEndX)
						{
							{
								FAtlasHorizon& H = NewHorizonRects.AddDefaulted_GetRef();
								H.Line.Origin.X = EndX;
								H.Line.Origin.Y = HorizonY;
								H.Line.Resolution.X = HorizonEndX - EndX;
								H.Line.Resolution.Y = Layout.AtlasResolution.Y - HorizonY;
								H.ExtendedLine = H.Line;
							}
						}
						else
						{
							// Sanity check
							check(StartX == HorizonEndX);
							NewHorizonRects.Add(Horizon);
						}
					}
					else
					{
						FAtlasHorizon& NewHorizon = NewHorizonRects.Add_GetRef(Horizon);
						NewHorizon.ExtendedLine = NewHorizon.Line;
					}
				}
				Layout.Horizons = NewHorizonRects;

				// 2.3.2 Rebuild extended horizon data
				// The first step (Left->Right) could be done during the horizon building)
				if (Layout.Horizons.Num())
				{
					// Extend left-horizon by sweeping: Left -> Right
					{
						TArray<FIntPoint> VisibilityStack;
						VisibilityStack.Add(FIntPoint(0, Layout.AtlasResolution.Y+1));
						for (int32 HorizonIt = 0, HorizonCount = Layout.Horizons.Num(); HorizonIt < HorizonCount; ++HorizonIt)
						{
							FAtlasHorizon& H = Layout.Horizons[HorizonIt];

							if (H.Line.Origin.Y >= VisibilityStack.Last().Y)
							{
								while (H.Line.Origin.Y >= VisibilityStack.Last().Y)
								{
									VisibilityStack.Pop(EAllowShrinking::No);
								}

								// Sanity 
								check(!VisibilityStack.IsEmpty());
							}

							H.ExtendedLine.Origin.X = VisibilityStack.Last().X;
							H.ExtendedLine.Resolution.X += (H.Line.Origin.X - H.ExtendedLine.Origin.X);
							VisibilityStack.Add(FIntPoint(H.Line.Origin.X + H.Line.Resolution.X, H.Line.Origin.Y));
						}
					}

					// Extend right-horizon by sweeping: Right -> Left
					{
						TArray<FIntPoint> VisibilityStack;
						VisibilityStack.Add(FIntPoint(Layout.AtlasResolution.X, Layout.AtlasResolution.Y+1));
						for (int32 HorizonIt = Layout.Horizons.Num() - 1; HorizonIt >= 0; --HorizonIt)
						{
							FAtlasHorizon& H = Layout.Horizons[HorizonIt];
							
							if (H.Line.Origin.Y >= VisibilityStack.Last().Y)
							{
								while (H.Line.Origin.Y >= VisibilityStack.Last().Y)
								{
									VisibilityStack.Pop(EAllowShrinking::No);
								}

								// Sanity 
								check(!VisibilityStack.IsEmpty());
							}

							H.ExtendedLine.Resolution.X = VisibilityStack.Last().X - H.ExtendedLine.Origin.X;
							VisibilityStack.Add(H.Line.Origin);

						}
					}
				}
			}
		}
	}

	return bFit;
}
#elif USE_PACKING_MODE == 2
// Use FTextureLayout packer
{
	uint32 OriginX = 0;
	uint32 OriginY = 0;
	const bool bFit = Layout.Packer.AddElement(OriginX, OriginY, Slot.Rect.Resolution.X, Slot.Rect.Resolution.Y);
	Slot.Rect.Origin.X = OriginX;
	Slot.Rect.Origin.Y = OriginY;
	return bFit;
}
#endif 

// Update the atlas layout (book-keeping) with the remove slots
static void CleanAtlas(
	FAtlasLayout& InLayout,
	const TArray<FAtlasRect>& InDeleteSlots)
{
	for (const FAtlasRect& Rect : InDeleteSlots)
	{
		#if USE_WASTE_RECT && (USE_PACKING_MODE == 1 || USE_PACKING_MODE == 2)
		if (Rect.Origin.X >= 0 && Rect.Origin.Y >= 0)
		{
			InLayout.FreeRects.Add(Rect);
		}
		#elif USE_PACKING_MODE ==2 
		InLayout.Packer.RemoveElement(Rect.Origin.X, Rect.Origin.Y, Rect.Resolution.X, Rect.Resolution.Y);
		#endif
	}
}

// Fit a set of slots within an existing altas layout, or create a new layout.
// * Try to fit the new slot within the existing layout. 
// * If it fails, produce a new atlas layout.
static void PackAtlas(
	FAtlasLayout& Layout,
	const TArray<FAtlasSlot>& InSlots, 
	TArray<FAtlasCopySlot>& CopySlots,
	TArray<FAtlasSlot>& NewSlots)
{ 
	const int32 MaxAtlasResolution = CVarRectLightTextureResolution.GetValueOnRenderThread();

	// Extract slots which are already parts of the current layout from the new/requested slots.
	// Estimate of the target resolution;
	uint32 TargetPixelCount = 0;
	TArray<FAtlasSlot> ValidSlots;
	TArray<FAtlasSlot> ValidNewSlots;
	ValidSlots.Reserve(InSlots.Num());
	ValidNewSlots.Reserve(InSlots.Num());
	for (const FAtlasSlot& Slot : InSlots)
	{
		if (Slot.IsValid())
		{
			FAtlasSlot& ValidSlot = ValidSlots.Add_GetRef(Slot);

			// Check if the texture resolution has changed (due to streaming)
			// If the texture resolution has change, reset the slot to be handled as a new slot
			const FIntPoint TextureResolution = ValidSlot.GetSourceResolution();
			const bool bHasTextureResolutionChanged = ValidSlot.CachedSourceTextureResolution != TextureResolution;
			if (bHasTextureResolutionChanged || ValidSlot.bForceRefresh)
			{
				ValidSlot.Rect.Origin = InvalidOrigin;
				ValidSlot.Rect.Resolution = TextureResolution;
				AlignSlotResolution(ValidSlot);
				ValidSlot.CachedSourceTextureResolution = TextureResolution;
			}

			TargetPixelCount += ValidSlot.Rect.Resolution.X * ValidSlot.Rect.Resolution.Y;

			if (ValidSlot.Rect.Origin == InvalidOrigin)
			{
				ValidNewSlots.Add(ValidSlot);
			}
		}
	}

	// Initialize atlas resolution the first time
	if (Layout.AtlasResolution == FIntPoint::ZeroValue)
	{
		const float HeuristicFactor = 1.0f;
		Layout.AtlasResolution = FMath::RoundUpToPowerOfTwo(FMath::Sqrt(float(TargetPixelCount)) * HeuristicFactor);
		Layout.AtlasResolution.X = FMath::Min(MaxAtlasResolution, Layout.AtlasResolution.X);
		Layout.AtlasResolution.Y = FMath::Min(MaxAtlasResolution, Layout.AtlasResolution.Y);
	}
	
	// Sort rect by perimeter
	ValidSlots.Sort		([](const FAtlasSlot& A, const FAtlasSlot& B) { return (A.Rect.Resolution.X + A.Rect.Resolution.Y) > (B.Rect.Resolution.X + B.Rect.Resolution.Y); });
	ValidNewSlots.Sort  ([](const FAtlasSlot& A, const FAtlasSlot& B) { return (A.Rect.Resolution.X + A.Rect.Resolution.Y) > (B.Rect.Resolution.X + B.Rect.Resolution.Y); });

	// 1. Try to fit the new slots into the current layout
	bool bNeedRefit = false;
	{
		for (const FAtlasSlot& Slot : ValidNewSlots)
		{
			FAtlasSlot NewSlot = Slot;
			if (!FitAlignedSlot(Layout, NewSlot))
			{
				bNeedRefit = true;
				break;
			}
			NewSlots.Add(NewSlot);
		}
	}

	// 2. Couldn't fit the new textures into the current layout. Refit the atlas layout
	//    * Use Simple row packing algo.
	//    * First iteration try to repack the texure within the existing resolution
	//    * Subsequent iteration increaes atlas resolution to fit the textures
	if (bNeedRefit)
	{
		CopySlots.Reserve(ValidSlots.Num());
		CopySlots.SetNum(0, EAllowShrinking::No);
		NewSlots.SetNum(0, EAllowShrinking::No);
		
		FIntPoint CurrentAtlasResolution = Layout.AtlasResolution;
		int32 CurrentSourceTextureMIPBias = 0; // When a refit is done, restart with a MIP bias of 0, to ensure we use the full potential space of the atlas
		bool bIsPackingValid = false;
		while (!bIsPackingValid)
		{
			// 2.0 Reset atlas layout
			Layout = FAtlasLayout(CurrentAtlasResolution);
			Layout.AtlasResolution = CurrentAtlasResolution;
			Layout.SourceTextureMIPBias = CurrentSourceTextureMIPBias;

			// 2.1 Try to fit all slots
			bool bFit = true;
			for (const FAtlasSlot& SrcSlot : ValidSlots)
			{
				FAtlasSlot DstSlot = SrcSlot;
				if (!FitAlignedSlot(Layout, DstSlot))
				{
					bFit = false;
					break;
				}	

				const bool bIsNewSlot = SrcSlot.Rect.Origin == InvalidOrigin;
				if (bIsNewSlot)
				{
					NewSlots.Add(DstSlot);
				}
				else
				{
					FAtlasCopySlot& CopySlot = CopySlots.AddDefaulted_GetRef();
					CopySlot.Id				 = SrcSlot.Id;
					CopySlot.Resolution		 = SrcSlot.Rect.Resolution;
					CopySlot.SrcOrigin		 = SrcSlot.Rect.Origin;
					CopySlot.DstOrigin		 = DstSlot.Rect.Origin;
					CopySlot.SourceTexture	 = SrcSlot.SourceTexture;
				}
			}

			// 2.2 If the current slots don't fit, increase atlas resolution
			if (!bFit)
			{	
				// Ensure the atlas resolution fit under the user requirement.
				// Otherwise starts to drop SourceTexture resolution
				if (Layout.AtlasResolution.X * 2 <= MaxAtlasResolution && Layout.AtlasResolution.Y * 2 <= MaxAtlasResolution)
				{
					CurrentAtlasResolution = Layout.AtlasResolution * 2;
				}
				else
				{
					// Mark all textures as invalid to force copy them into the atlas with a lower mip index
					CurrentSourceTextureMIPBias++;
					for (FAtlasSlot& Slot : ValidSlots)
					{
						const FIntPoint SourceResolution = Slot.GetSourceResolution();
						Slot.Rect.Origin = InvalidOrigin;
						Slot.Rect.Resolution = ToMIP(SourceResolution, CurrentSourceTextureMIPBias);
						Slot.CachedSourceTextureResolution = Slot.Rect.Resolution;
						AlignSlotResolution(Slot);
					}
				}

				CopySlots.SetNum(0, EAllowShrinking::No);
				NewSlots.SetNum(0, EAllowShrinking::No);
			}
			else
			{
				bIsPackingValid = true;
			}
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// API

uint32 AddTexture(FStaticShaderPlatform Platform, UTexture* In, const FVector4f& ScaleOffset)
{
	check(IsInRenderingThread());

	uint32 SlotIndex = InvalidSlotIndex;
	if (In && PlatformSupportsRectLightAtlas(Platform))
	{
		// 1. Find if the texture already exist (simple linear search assuming a low number of textures)
		bool bFound = false;
		for (FAtlasSlot& Slot : GRectLightTextureManager.AtlasSlots)
		{
			if ((Slot.SourceTexture == In->TextureReference.TextureReferenceRHI) &&
			    (Slot.SourceScaleOffset == ScaleOffset))
			{
				Slot.RefCount++;
				SlotIndex = Slot.Id;
				bFound = true;
				break;
			}
		}

		// 2. If not found, then add an atlas slot for this new texture
		if (!bFound)
		{
			FAtlasSlot* Slot = nullptr;
			if (GRectLightTextureManager.FreeSlots.Dequeue(SlotIndex))
			{
				Slot = &GRectLightTextureManager.AtlasSlots[SlotIndex];
			}
			else
			{
				SlotIndex = GRectLightTextureManager.AtlasSlots.Num();
				Slot = &GRectLightTextureManager.AtlasSlots.AddDefaulted_GetRef();
			}

			const FRHITexture* Tex = In->TextureReference.TextureReferenceRHI;

			*Slot = FAtlasSlot();
			Slot->SourceTexture = In->TextureReference.TextureReferenceRHI;
			Slot->SourceScaleOffset = ScaleOffset;
			Slot->Id = SlotIndex;
			Slot->Rect.Origin = InvalidOrigin;
			Slot->Rect.Resolution = GetSourceResolution(Tex, ScaleOffset);
			Slot->CachedSourceTextureResolution = Slot->Rect.Resolution;
			Slot->RefCount = 1;

			// Force alignement based on the max. MIP-level
			AlignSlotResolution(*Slot);

			GRectLightTextureManager.bHasPendingAdds = true;
		}
	}
	return SlotIndex;
}

uint32 AddTexture(UTexture* In, const FVector4f& ScaleOffset)
{
	return AddTexture(GMaxRHIShaderPlatform, In, ScaleOffset);
}

void RemoveTexture(uint32 InSlotIndex)
{
	check(IsInRenderingThread());

	if (InSlotIndex != InvalidSlotIndex && InSlotIndex < uint32(GRectLightTextureManager.AtlasSlots.Num()))
	{
		// If it is the last light referencing this texture, we retires the atlas slot
		if (--GRectLightTextureManager.AtlasSlots[InSlotIndex].RefCount == 0)
		{
			// Add pending slots to clean-up the layout during the next update call
			GRectLightTextureManager.DeletedSlots.Add(GRectLightTextureManager.AtlasSlots[InSlotIndex].Rect);

			const int32 SlotCount = GRectLightTextureManager.AtlasSlots.Num();
			if (InSlotIndex == SlotCount-1)
			{
				GRectLightTextureManager.AtlasSlots.SetNum(SlotCount-1);
			}
			else
			{
				GRectLightTextureManager.FreeSlots.Enqueue(InSlotIndex);
				GRectLightTextureManager.AtlasSlots[InSlotIndex] = FAtlasSlot();
			}

			GRectLightTextureManager.bHasPendingDeletes = true;
		}
	}
}

FAtlasSlotDesc GetAtlasSlot(uint32 InSlotIndex)
{
	FAtlasSlotDesc Out;
	Out.UVOffset = FVector2f(0,0);
	Out.UVScale = FVector2f(0, 0);
	Out.MaxMipLevel = FLightRenderParameters::GetRectLightAtlasInvalidMIPLevel();

	if (InSlotIndex < uint32(GRectLightTextureManager.AtlasSlots.Num()) && GRectLightTextureManager.AtlasTexture)
	{
		const FAtlasSlot& Slot = GRectLightTextureManager.AtlasSlots[InSlotIndex];
		const FIntPoint Resolution = GRectLightTextureManager.AtlasTexture->GetDesc().Extent;
		const FVector2f InvResolution(1.f / Resolution.X, 1.f / Resolution.Y);

		// UV rect shrinking, to avoid leaking during tri/bi-linear filtering, is done within the shader, as it depends on the selected MIP value.
		// UVOffset/UVScale gives the rect value without any border/clamping
		Out.UVOffset = FVector2f(Slot.Rect.Origin.X * InvResolution.X, Slot.Rect.Origin.Y * InvResolution.Y);
		Out.UVScale  = FVector2f(Slot.Rect.Resolution.X * InvResolution.X, Slot.Rect.Resolution.Y * InvResolution.Y);
		Out.MaxMipLevel = GetSlotMaxMIPLevel(Slot);
	}

	return Out;
}

static FRDGTextureRef CreateRectLightAtlasTexture(FRDGBuilder& GraphBuilder, const FIntPoint& Resolution, EPixelFormat AtlasFormat)
{
	const uint32 MipCount = FMath::Log2(float(FMath::Min(Resolution.X, Resolution.Y)));
	return GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(
		Resolution,
		AtlasFormat,
		FClearValueBinding::Transparent,
		ETextureCreateFlags::UAV | ETextureCreateFlags::ShaderResource | ETextureCreateFlags::RenderTargetable,
		MipCount),
		TEXT("RectLight.AtlasTexture"),
		ERDGTextureFlags::MultiFrame);
}

void UpdateAtlasTexture(FRDGBuilder& GraphBuilder, const ERHIFeatureLevel::Type FeatureLevel)
{
	if (GRectLightTextureManager.bLock || !PlatformSupportsRectLightAtlas(GetFeatureLevelShaderPlatform(FeatureLevel)))
	{
		return;
	}

	// Force update by resetting the atlas layout
	static int32 CachedMaxAtlasResolution = CVarRectLightTextureResolution.GetValueOnRenderThread();
	static EPixelFormat CachedAtlasFormat = GetRectLightAtlasFormat();
	static int32 CachedFilteringMethod = CVarRectLighFilterQuality.GetValueOnRenderThread();
	const EPixelFormat AtlasFormat = GetRectLightAtlasFormat();
	const bool bForceUpdate = CVarRectLighForceUpdate.GetValueOnRenderThread() > 0 || CachedMaxAtlasResolution != CVarRectLightTextureResolution.GetValueOnRenderThread() || CachedAtlasFormat != AtlasFormat || CachedFilteringMethod != CVarRectLighFilterQuality.GetValueOnRenderThread();
	const bool bResetOnChange = CVarRectLighResetOnChange.GetValueOnRenderThread() > 0;
	const bool bClearAtlas = CVarRectLighClearAtlas.GetValueOnRenderThread() > 0;
	auto ResetAtlas = [&]()
	{
		CachedMaxAtlasResolution = CVarRectLightTextureResolution.GetValueOnRenderThread();
		CachedAtlasFormat = AtlasFormat;
		CachedFilteringMethod = CVarRectLighFilterQuality.GetValueOnRenderThread();
		GRectLightTextureManager.bHasPendingAdds = true;
		GRectLightTextureManager.AtlasLayout = FAtlasLayout(FIntPoint(CachedMaxAtlasResolution, CachedMaxAtlasResolution));
		GRectLightTextureManager.DeletedSlots.Empty();
		GRectLightTextureManager.FreeSlots.Empty();
		for (FAtlasSlot& Slot : GRectLightTextureManager.AtlasSlots)
		{
			Slot.Rect.Origin = InvalidOrigin;
		}
	};

	if (bForceUpdate)
	{
		ResetAtlas();
	}

	// Force update if among the existing valid slot a streamed a higher resolution than the existing one
	uint32 RefreshRequestCount = 0;
	{
		for (FAtlasSlot& Slot : GRectLightTextureManager.AtlasSlots)
		{
			if (Slot.IsValid() && Slot.GetTextureRHI())
			{
				const FIntPoint SourceResolutionMIPed = ToMIP(Slot.GetSourceResolution(), GRectLightTextureManager.AtlasLayout.SourceTextureMIPBias);
				if (SourceResolutionMIPed.X > Slot.Rect.Resolution.X || SourceResolutionMIPed.Y > Slot.Rect.Resolution.Y)
				{
					// Invalid the slot
					Slot.Rect.Origin = InvalidOrigin;
					GRectLightTextureManager.bHasPendingAdds = true;
				}
				if (Slot.bForceRefresh)
				{
					++RefreshRequestCount;
				}
			}
		}
	}

	// Force the atlas to be reset when some change are detected (optional)
	if (GRectLightTextureManager.bHasPendingAdds && !bForceUpdate && bResetOnChange)
	{
		ResetAtlas();
	}

	// Update the atlas layout with the delete slots
	if (GRectLightTextureManager.DeletedSlots.Num() > 0)
	{
		CleanAtlas(GRectLightTextureManager.AtlasLayout, GRectLightTextureManager.DeletedSlots);
		GRectLightTextureManager.DeletedSlots.SetNum(0);
		GRectLightTextureManager.bHasPendingDeletes = false;
	}

	// Process new atlas entries
	if (GRectLightTextureManager.bHasPendingAdds)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);

		// 1. Compute new layout
		TArray<FAtlasSlot> NewSlots;
		TArray<FAtlasCopySlot> CopySlots;
		PackAtlas(
			GRectLightTextureManager.AtlasLayout,
			GRectLightTextureManager.AtlasSlots,
			CopySlots,
			NewSlots);

		// 2. Apply layout modification
		const bool bHasChanged = !NewSlots.IsEmpty() || !CopySlots.IsEmpty();
		if (bHasChanged)
		{
			// 2.0 Register or create the texture atlas
			FRDGTextureRef AtlasTexture = nullptr;
			bool bNeedExtraction = false;
			const bool bRecreateAtlasTexture = GRectLightTextureManager.AtlasTexture == nullptr || (CopySlots.Num() == 0 && GRectLightTextureManager.AtlasTexture->GetDesc().Extent != GRectLightTextureManager.AtlasLayout.AtlasResolution);
			if (bRecreateAtlasTexture)
			{
				AtlasTexture = CreateRectLightAtlasTexture(GraphBuilder, GRectLightTextureManager.AtlasLayout.AtlasResolution, AtlasFormat);

				#if RECT_ATLAS_DEBUG_CLEAR
				AddRectLightClearPass(GraphBuilder, ShaderMap, AtlasTexture);
				#endif

				bNeedExtraction = true;
				// Sanity check
				check(CopySlots.Num() == 0);
			}
			else
			{
				AtlasTexture = GraphBuilder.RegisterExternalTexture(GRectLightTextureManager.AtlasTexture);

				// Clear the atlas (optional)
				if (bClearAtlas && bResetOnChange && CopySlots.Num() == 0)
				{
					#if RECT_ATLAS_DEBUG_CLEAR
					AddRectLightClearPass(GraphBuilder, ShaderMap, AtlasTexture);
					#else
					AddClearRenderTargetPass(GraphBuilder, AtlasTexture, FLinearColor::Transparent);
					#endif
				}
			}

			// 2.1 Copy slots from previous to new atlas texture
			if (CopySlots.Num() > 0)
			{
				FRDGTextureRef NewAtlasTexture = CreateRectLightAtlasTexture(GraphBuilder, GRectLightTextureManager.AtlasLayout.AtlasResolution, AtlasFormat);
				bNeedExtraction = true;

				#if RECT_ATLAS_DEBUG_CLEAR
				AddRectLightClearPass(GraphBuilder, ShaderMap, NewAtlasTexture);
				#endif

				CopySlotsPass(GraphBuilder, ShaderMap, CopySlots, AtlasTexture, NewAtlasTexture);

				for (FAtlasCopySlot& Slot : CopySlots)
				{
					// Update:
					// * the slot Origin based on the new layout & Resolution with new layout data
					// * the slot Resolution in case the texture has streamed a higher-res version
					GRectLightTextureManager.AtlasSlots[Slot.Id].Rect.Origin	 = Slot.DstOrigin;
					GRectLightTextureManager.AtlasSlots[Slot.Id].Rect.Resolution = Slot.Resolution;
					GRectLightTextureManager.AtlasSlots[Slot.Id].bForceRefresh   = false;
				}
				AtlasTexture = NewAtlasTexture;
			}

			// 2.2 Insert & filter new slots into the atlas texture
			if (NewSlots.Num() > 0)
			{
				AddSlotsPass(GraphBuilder, ShaderMap, GRectLightTextureManager.AtlasLayout.SourceTextureMIPBias, NewSlots, AtlasTexture);
				FilterSlotsPass(GraphBuilder, ShaderMap, NewSlots, AtlasTexture);

				for (FAtlasSlot& Slot : NewSlots)
				{
					// Update:
					// * the slot Origin based on the new layout & Resolution with new layout data
					// * the slot Resolution in case the texture has streamed a higher-res version
					GRectLightTextureManager.AtlasSlots[Slot.Id].Rect.Origin	 = Slot.Rect.Origin;
					GRectLightTextureManager.AtlasSlots[Slot.Id].Rect.Resolution = Slot.Rect.Resolution;
					GRectLightTextureManager.AtlasSlots[Slot.Id].bForceRefresh   = false;
				}
			}

			if (bNeedExtraction)
			{
				GRectLightTextureManager.AtlasTexture = GraphBuilder.ConvertToExternalTexture(AtlasTexture);
			}
		}

		GRectLightTextureManager.bHasPendingAdds = false;
	}
	// Process forced refresh slots
	else if (RefreshRequestCount > 0)
	{
		TArray<FAtlasSlot> RefreshSlots;
		RefreshSlots.Reserve(FMath::Max(1u, RefreshRequestCount));
		for (FAtlasSlot& Slot : GRectLightTextureManager.AtlasSlots)
		{
			if (Slot.bForceRefresh)
			{
				Slot.bForceRefresh = false;

				RefreshSlots.Add(Slot);
			}
		}

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);
		FRDGTextureRef AtlasTexture = GraphBuilder.RegisterExternalTexture(GRectLightTextureManager.AtlasTexture);
		AddSlotsPass(GraphBuilder, ShaderMap, GRectLightTextureManager.AtlasLayout.SourceTextureMIPBias, RefreshSlots, AtlasTexture);
		FilterSlotsPass(GraphBuilder, ShaderMap, RefreshSlots, AtlasTexture);
	}

	// If the atlas was registered into RDG make sure it gets transitioned to external access on all pipes.
	if (FRDGTexture* AtlasTexture = GraphBuilder.FindExternalTexture(GRectLightTextureManager.AtlasTexture))
	{
		GraphBuilder.UseExternalAccessMode(AtlasTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}
}

void AddDebugPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef OutputTexture)
{
	if (CVarRectLighTextureDebug.GetValueOnRenderThread() > 0 
		&& ShaderPrint::IsSupported(View.Family->GetShaderPlatform()) 
		&& PlatformSupportsRectLightAtlas(View.Family->GetShaderPlatform()))
	{
		if (FRDGTextureRef DebugOutput = AddRectLightDebugInfoPass(GraphBuilder, View, OutputTexture->Desc))
		{
			// Debug output is blend on top of the SceneColor/OutputTexture, as debug pass is a CS pass, and SceneColor/OutputTexture might not have a UAV flag
			FCopyRectPS::FParameters* Parameters = GraphBuilder.AllocParameters<FCopyRectPS::FParameters>();
			Parameters->InputTexture = DebugOutput;
			Parameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ELoad);

			const FScreenPassTextureViewport InputViewport(DebugOutput->Desc.Extent);
			const FScreenPassTextureViewport OutputViewport(OutputTexture);
			TShaderMapRef<FCopyRectPS> PixelShader(View.ShaderMap);

			TShaderMapRef<FScreenPassVS> VertexShader(View.ShaderMap);
			FRHIBlendState* BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
			FRHIDepthStencilState* DepthStencilState = FScreenPassPipelineState::FDefaultDepthStencilState::GetRHI();

			AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("RectLightAtlas::BlitDebug"), View, OutputViewport, InputViewport, VertexShader, PixelShader, BlendState, DepthStencilState, Parameters, EScreenPassDrawFlags::None);
		}
	}
}

FRHITexture* GetAtlasTexture()
{
	return GRectLightTextureManager.AtlasTexture ? GRectLightTextureManager.AtlasTexture->GetRHI() : nullptr;
}

// Scope object allowing to force refresh of a slot texture, and locking/preventing 
// the atlas update during the 'update'/'capture'
FAtlasTextureInvalidationScope::FAtlasTextureInvalidationScope(const UTexture* In)
{
	if (In)
	{
		// Several rect lights can reference the same animated texture, but with different offset/scale. 
		// In this case we need to refresh all rects referencing this texture.
		bool bHasBeenLockedLocally = false;
		for (FAtlasSlot& Slot : GRectLightTextureManager.AtlasSlots)
		{
			// If the input texture is actually part of the atlas, 
			// then we lock the atlas to not update the atlas during 
			// the capture, and force the target texture to refresh 
			// its data during the next update
			if (Slot.SourceTexture == In->TextureReference.TextureReferenceRHI)
			{
				if (!bHasBeenLockedLocally)
				{
					// Sanity check, allow a single lock/capture refresh at a time.
					check(GRectLightTextureManager.bLock == false);
					GRectLightTextureManager.bLock = true;
					bHasBeenLockedLocally = true;
				}

				bLocked = true;
				Slot.bForceRefresh = true;
			}
		}
	}
}

FAtlasTextureInvalidationScope::~FAtlasTextureInvalidationScope()
{
	if (bLocked)
	{
		// Sanity check, allow a single lock/capture refresh at a time.
		check(GRectLightTextureManager.bLock == true);

		GRectLightTextureManager.bLock = false;
		bLocked = false;
	}
}

} // namespace RectLightAtlas
