// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"
#include "RenderGraphDefinitions.h"

class FRHITexture;
class UTexture;
class FViewInfo;

namespace RectLightAtlas
{
// Atlas slot description in terms of UV coordinates
struct FAtlasSlotDesc
{
	FVector2f UVOffset;
	FVector2f UVScale;
	float MaxMipLevel;
};
	
// Add a rect light source texture to the atlas
RENDERER_API uint32 AddTexture(FStaticShaderPlatform Platform, UTexture* Texture, const FVector4f& ScaleOffset = FVector4f::One());
RENDERER_API uint32 AddTexture(UTexture* Texture, const FVector4f& ScaleOffset = FVector4f::One());

// Remove a rect light source texture from the atlas
RENDERER_API void RemoveTexture(uint32 InSlotId);

// Return the atlas coordinate for a particular slot
RENDERER_API FAtlasSlotDesc GetAtlasSlot(uint32 InSlotId);

// Return the atlas texture
RENDERER_API FRHITexture* GetAtlasTexture();

// Update the rect light atlas texture
RENDERER_API void UpdateAtlasTexture(FRDGBuilder& GraphBuilder, const ERHIFeatureLevel::Type FeatureLevel);

// Return the rect light atlas debug pass
RENDERER_API void AddDebugPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef OutputTexture);

} 
