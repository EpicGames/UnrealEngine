// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Reflection Environment - feature that provides HDR glossy reflections on any surfaces, leveraging precomputation to prefilter cubemaps of the scene
=============================================================================*/

#include "ReflectionEnvironment.h"
#include "Components/ReflectionCaptureComponent.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "RHI.h"
#include "UniformBuffer.h"
#include "ShaderParameters.h"
#include "RendererInterface.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "SceneUtils.h"
#include "RHIStaticStates.h"
#include "PostProcess/SceneRenderTargets.h"
#include "GlobalShader.h"
#include "SceneRenderTargetParameters.h"
#include "DeferredShadingRenderer.h"
#include "BasePassRendering.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "LightRendering.h"
#include "PipelineStateCache.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "SceneTextureParameters.h"
#include "ScreenSpaceDenoise.h"
#include "ScreenSpaceRayTracing.h"
#include "RayTracing/RaytracingOptions.h"
#include "RenderGraph.h"
#include "PixelShaderUtils.h"
#include "SceneTextureParameters.h"
#include "HairStrands/HairStrandsRendering.h"
#include "RenderTargetPool.h"

static TAutoConsoleVariable<int32> CVarReflectionEnvironment(
	TEXT("r.ReflectionEnvironment"),
	1,
	TEXT("Whether to render the reflection environment feature, which implements local reflections through Reflection Capture actors.\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on and blend with scene (default)")
	TEXT(" 2: on and overwrite scene (only in non-shipping builds)"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

int32 GReflectionEnvironmentLightmapMixing = 1;
FAutoConsoleVariableRef CVarReflectionEnvironmentLightmapMixing(
	TEXT("r.ReflectionEnvironmentLightmapMixing"),
	GReflectionEnvironmentLightmapMixing,
	TEXT("Whether to mix indirect specular from reflection captures with indirect diffuse from lightmaps for rough surfaces."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GReflectionEnvironmentLightmapMixBasedOnRoughness = 1;
FAutoConsoleVariableRef CVarReflectionEnvironmentLightmapMixBasedOnRoughness(
	TEXT("r.ReflectionEnvironmentLightmapMixBasedOnRoughness"),
	GReflectionEnvironmentLightmapMixBasedOnRoughness,
	TEXT("Whether to reduce lightmap mixing with reflection captures for very smooth surfaces.  This is useful to make sure reflection captures match SSR / planar reflections in brightness."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GReflectionEnvironmentBeginMixingRoughness = .1f;
FAutoConsoleVariableRef CVarReflectionEnvironmentBeginMixingRoughness(
	TEXT("r.ReflectionEnvironmentBeginMixingRoughness"),
	GReflectionEnvironmentBeginMixingRoughness,
	TEXT("Min roughness value at which to begin mixing reflection captures with lightmap indirect diffuse."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

float GReflectionEnvironmentEndMixingRoughness = .3f;
FAutoConsoleVariableRef CVarReflectionEnvironmentEndMixingRoughness(
	TEXT("r.ReflectionEnvironmentEndMixingRoughness"),
	GReflectionEnvironmentEndMixingRoughness,
	TEXT("Min roughness value at which to end mixing reflection captures with lightmap indirect diffuse."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GReflectionEnvironmentLightmapMixLargestWeight = 10000;
FAutoConsoleVariableRef CVarReflectionEnvironmentLightmapMixLargestWeight(
	TEXT("r.ReflectionEnvironmentLightmapMixLargestWeight"),
	GReflectionEnvironmentLightmapMixLargestWeight,
	TEXT("When set to 1 can be used to clamp lightmap mixing such that only darkening from lightmaps are applied to reflection captures."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarDoTiledReflections(
	TEXT("r.DoTiledReflections"),
	1,
	TEXT("Compute Reflection Environment with Tiled compute shader..\n")
	TEXT(" 0: off\n")
	TEXT(" 1: on (default)"),
	ECVF_RenderThreadSafe);


// to avoid having direct access from many places
int GetReflectionEnvironmentCVar()
{
	int32 RetVal = CVarReflectionEnvironment.GetValueOnAnyThread();

#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Disabling the debug part of this CVar when in shipping
	if (RetVal == 2)
	{
		RetVal = 1;
	}
#endif

	return RetVal;
}

FVector GetReflectionEnvironmentRoughnessMixingScaleBiasAndLargestWeight()
{
	float RoughnessMixingRange = 1.0f / FMath::Max(GReflectionEnvironmentEndMixingRoughness - GReflectionEnvironmentBeginMixingRoughness, .001f);

	if (GReflectionEnvironmentLightmapMixing == 0)
	{
		return FVector(0, 0, GReflectionEnvironmentLightmapMixLargestWeight);
	}

	if (GReflectionEnvironmentEndMixingRoughness == 0.0f && GReflectionEnvironmentBeginMixingRoughness == 0.0f)
	{
		// Make sure a Roughness of 0 results in full mixing when disabling roughness-based mixing
		return FVector(0, 1, GReflectionEnvironmentLightmapMixLargestWeight);
	}

	if (!GReflectionEnvironmentLightmapMixBasedOnRoughness)
	{
		return FVector(0, 1, GReflectionEnvironmentLightmapMixLargestWeight);
	}

	return FVector(RoughnessMixingRange, -GReflectionEnvironmentBeginMixingRoughness * RoughnessMixingRange, GReflectionEnvironmentLightmapMixLargestWeight);
}

bool IsReflectionEnvironmentAvailable(ERHIFeatureLevel::Type InFeatureLevel)
{
	return (SupportsTextureCubeArray(InFeatureLevel)) && (GetReflectionEnvironmentCVar() != 0);
}

bool IsReflectionCaptureAvailable()
{
	return IsStaticLightingAllowed();
}

void FReflectionEnvironmentCubemapArray::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (SupportsTextureCubeArray(GetFeatureLevel()))
	{
		check(MaxCubemaps > 0);
		check(CubemapSize > 0);

		const int32 NumReflectionCaptureMips = FMath::CeilLogTwo(CubemapSize) + 1;

		ReleaseCubeArray();

		FPooledRenderTargetDesc Desc(
			FPooledRenderTargetDesc::CreateCubemapDesc(
				CubemapSize,
				// Alpha stores sky mask
				PF_FloatRGBA, 
				FClearValueBinding::None,
				TexCreate_None,
				TexCreate_None,
				false, 
				// Cubemap array of 1 produces a regular cubemap, so guarantee it will be allocated as an array
				FMath::Max<uint32>(MaxCubemaps, 2),
				NumReflectionCaptureMips
				)
			);

		// Allocate TextureCubeArray for the scene's reflection captures
		GRenderTargetPool.FindFreeElement(RHICmdList, Desc, ReflectionEnvs, TEXT("ReflectionEnvs"));
	}
}

void FReflectionEnvironmentCubemapArray::ReleaseCubeArray()
{
	// it's unlikely we can reuse the TextureCubeArray so when we release it we want to really remove it
	GRenderTargetPool.FreeUnusedResource(ReflectionEnvs);
}

void FReflectionEnvironmentCubemapArray::ReleaseRHI()
{
	ReleaseCubeArray();
}

float FCaptureComponentSceneState::ComputeCurrentFade(double Now, float Duration) const
{
	if (FadeStartValue == FadeTargetValue)
	{
		return FadeTargetValue;
	}
	if (Duration <= 0.0f)
	{
		return FadeTargetValue;
	}
	const float Elapsed = static_cast<float>(Now - FadeStartTime);
	const float Delta = Elapsed / Duration;
	const float Direction = (FadeTargetValue >= FadeStartValue) ? 1.0f : -1.0f;
	const float Value = FadeStartValue + Direction * Delta;
	return FMath::Clamp(Value,
		FMath::Min(FadeStartValue, FadeTargetValue),
		FMath::Max(FadeStartValue, FadeTargetValue));
}


const FCaptureComponentSceneState* FReflectionCaptureCache::Find(const FGuid& MapBuildDataId) const
{
	const FReflectionCaptureCacheEntry* Entry = CaptureData.Find(MapBuildDataId);
	if (Entry == nullptr)
		return nullptr;

	return &(Entry->SceneState);
}

FCaptureComponentSceneState* FReflectionCaptureCache::Find(const FGuid& MapBuildDataId)
{
	FReflectionCaptureCacheEntry* Entry = CaptureData.Find(MapBuildDataId);
	if (Entry == nullptr)
		return nullptr;

	return &(Entry->SceneState);
}

const FCaptureComponentSceneState* FReflectionCaptureCache::Find(const UReflectionCaptureComponent* Component) const
{
	if (!Component)	// Intentionally not IsValid(Component), as this often occurs when Component is explicitly PendingKill.
		return nullptr;

	const FCaptureComponentSceneState* SceneState = Find(Component->MapBuildDataId);
	if (SceneState)
	{
		return SceneState;
	}

	const FGuid* Guid = RegisteredComponentMapBuildDataIds.Find(Component);
	if (Guid)
	{
		return Find(*Guid);
	}

	return nullptr;
}

FCaptureComponentSceneState* FReflectionCaptureCache::Find(const UReflectionCaptureComponent* Component)
{
	if (!Component)	// Intentionally not IsValid(Component), as this often occurs when Component is explicitly PendingKill.
		return nullptr;

	FCaptureComponentSceneState* SceneState = Find(Component->MapBuildDataId);
	if (SceneState)
	{
		return SceneState;
	}

	const FGuid* Guid = RegisteredComponentMapBuildDataIds.Find(Component);
	if (Guid)
	{
		return Find(*Guid);
	}
	return nullptr;
}


const FCaptureComponentSceneState& FReflectionCaptureCache::FindChecked(const UReflectionCaptureComponent* Component) const
{
	const FCaptureComponentSceneState* Found = Find(Component);
	check(Found);

	return *Found;
}

FCaptureComponentSceneState& FReflectionCaptureCache::FindChecked(const UReflectionCaptureComponent* Component)
{
	FCaptureComponentSceneState* Found = Find(Component);
	check(Found);

	return *Found;
}

FCaptureComponentSceneState& FReflectionCaptureCache::Add(const UReflectionCaptureComponent* Component, const FCaptureComponentSceneState& Value)
{
	// During Reflection Capture Placement in editor, this is potentially not IsValid
	//  So just check to make sure that the pointer is non-null
	check(Component)

	FCaptureComponentSceneState* Existing = AddReference(Component);
	if (Existing != nullptr)
	{
		return *Existing;
	}
	else
	{
		FReflectionCaptureCacheEntry& Entry = CaptureData.Add(Component->MapBuildDataId, { 1, Value });
		RegisterComponentMapBuildDataId(Component);
		return Entry.SceneState;
	}
}

FCaptureComponentSceneState* FReflectionCaptureCache::AddReference(const UReflectionCaptureComponent* Component)
{
	// During Reflection Capture Placement in editor, this is potentially not IsValid
	//  So just check to make sure that the pointer is non-null
	check(Component)

	bool Remap = RemapRegisteredComponentMapBuildDataId(Component);
	FReflectionCaptureCacheEntry* Found = CaptureData.Find(Component->MapBuildDataId);
	if (Found == nullptr)
		return nullptr;

	// Should not add reference count if this is caused by capture rebuilt
	if (!Remap)
	{
		Found->RefCount += 1;
	}

	return &(Found->SceneState);
}

int32 FReflectionCaptureCache::GetKeys(TArray<FGuid>& OutKeys) const
{
	return CaptureData.GetKeys(OutKeys);
}

int32 FReflectionCaptureCache::GetKeys(TSet<FGuid>& OutKeys) const
{
	return CaptureData.GetKeys(OutKeys);
}

void FReflectionCaptureCache::Empty()
{
	CaptureData.Empty();
	RegisteredComponentMapBuildDataIds.Empty();
}

int32 FReflectionCaptureCache::Prune(const TSet<FGuid> KeysToKeep, TArray<int32>& ReleasedIndices)
{
	TSet<FGuid> ExistingKeys;
	CaptureData.GetKeys(ExistingKeys);

	TSet<FGuid> KeysToRemove = ExistingKeys.Difference(KeysToKeep);
	ReleasedIndices.Empty();
	ReleasedIndices.Reserve(KeysToRemove.Num());

	for (const FGuid& Key : KeysToRemove)
	{
		FReflectionCaptureCacheEntry* Found = CaptureData.Find(Key);
		if (Found == nullptr)
			continue;

		int32 CubemapIndex = Found->SceneState.CubemapIndex;
		if (CubemapIndex != -1)
			ReleasedIndices.Add(CubemapIndex);

		CaptureData.Remove(Key);
	}

	return ReleasedIndices.Num();
}

bool FReflectionCaptureCache::Remove(const UReflectionCaptureComponent* Component)
{
	if (!Component)	// Intentionally not IsValid(Component), as this often occurs when Component is explicitly PendingKill.
		return false;

	RemapRegisteredComponentMapBuildDataId(Component);
	FReflectionCaptureCacheEntry* Found = CaptureData.Find(Component->MapBuildDataId);
	if (Found == nullptr)
		return false;

	if (Found->RefCount > 1)
	{
		Found->RefCount -= 1;
		return false;
	}
	else
	{
		CaptureData.Remove(Component->MapBuildDataId);
		UnregisterComponentMapBuildDataId(Component);
		return true;
	}
}

void FReflectionCaptureCache::RegisterComponentMapBuildDataId(const UReflectionCaptureComponent* Component)
{
	RegisteredComponentMapBuildDataIds.Add(Component, Component->MapBuildDataId);
}

bool FReflectionCaptureCache::RemapRegisteredComponentMapBuildDataId(const UReflectionCaptureComponent* Component)
{
	check(Component);

	// Remap old guid to new guid when the component is the same pointer.
	FGuid* OldBuildId = RegisteredComponentMapBuildDataIds.Find(Component);
	if (OldBuildId &&
		*OldBuildId != Component->MapBuildDataId)
	{
		FGuid OldBuildIdCopy = *OldBuildId;
		const FReflectionCaptureCacheEntry* Current = CaptureData.Find(OldBuildIdCopy);
		if (Current == nullptr)
			return false; // No current entry to remap to, so no remap to perform

		// Remap all pointers that point to the old guid to the new one.
		int32 ReferenceCount = 0;
		for (TPair<const UReflectionCaptureComponent*, FGuid>& item : RegisteredComponentMapBuildDataIds)
		{
			if (item.Value == OldBuildIdCopy)
			{
				item.Value = Component->MapBuildDataId;
				ReferenceCount++;
			}
		}

		FReflectionCaptureCacheEntry Entry = *Current;
		Entry.RefCount = ReferenceCount;

		CaptureData.Remove(OldBuildIdCopy);
		CaptureData.Shrink();
		CaptureData.Add(Component->MapBuildDataId, Entry);

		return true;
	}

	return false;
}

void FReflectionCaptureCache::UnregisterComponentMapBuildDataId(const UReflectionCaptureComponent* Component)
{
	RegisteredComponentMapBuildDataIds.Remove(Component);
}

void FReflectionEnvironmentSceneData::SetGameThreadTrackingData(int32 MaxAllocatedCubemaps, int32 CaptureSize, int32 DesiredCaptureSize)
{
	MaxAllocatedReflectionCubemapsGameThread = MaxAllocatedCubemaps;
	ReflectionCaptureSizeGameThread = CaptureSize;
	DesiredReflectionCaptureSizeGameThread = DesiredCaptureSize;
}

bool FReflectionEnvironmentSceneData::DoesAllocatedDataNeedUpdate(int32 DesiredMaxCubemaps, int32 DesiredCaptureSize) const
{
	if (DesiredMaxCubemaps != MaxAllocatedReflectionCubemapsGameThread)
		return true;

	if (DesiredCaptureSize != ReflectionCaptureSizeGameThread)
		return true;

	return false;
}

void FReflectionEnvironmentSceneData::ResizeCubemapArrayGPU(uint32 InMaxCubemaps, uint32 InNumReserved, int32 InCubemapSize)
{
	check(IsInRenderingThread());
	check(InNumReserved <= InMaxCubemaps);

	const int32 NewRegularEnd = (int32)InMaxCubemaps - (int32)InNumReserved;

	// If the cubemap array isn't setup yet then no copying/reallocation is necessary. Just go through the old path
	if (!CubemapArray.IsInitialized())
	{
		CubemapArraySlotsUsed.Init(false, NewRegularEnd);
		CubemapArray.UpdateMaxCubemaps(InMaxCubemaps, InNumReserved, InCubemapSize);
		return;
	}

	const int32 OldNumReserved = CubemapArray.GetNumReservedCubemaps();
	const int32 OldRegularEnd = CubemapArray.GetMaxRegularCubemaps();

	// Generate a remapping table for regular slot positions.  IndexRemapping applies to the regular slots, ignoring reserved
	// slots at the end of the cube map array used for smooth blending of refreshed captures.  Runtime reflection captures can add
	// elements to CubemapArraySlotsUsed, exceeding the original GetMaxRegularCubemaps(), so it's necessary to take the maximum.
	// In particular, runtime captures allocate new slots in the "PruneReflectionCaptures" render command BEFORE this function is
	// called, while baked captures allocate their slots AFTER this call in CaptureOrUploadReflectionCapture.  The reserved slots
	// at the tail of the array are separately remapped below.
	const int32 RegularRemapCount = FMath::Max(OldRegularEnd, CubemapArraySlotsUsed.Num());
	TArray<int32> IndexRemapping;
	IndexRemapping.Reserve(RegularRemapCount);
	int32 Count = 0;
	for (int i = 0; i < RegularRemapCount; i++)
	{
		bool bUsed = i < CubemapArraySlotsUsed.Num() ? CubemapArraySlotsUsed[i] : false;
		if (bUsed)
		{
			IndexRemapping.Add(Count);
			Count++;
		}
		else
		{
			IndexRemapping.Add(-1);
		}
	}

	// Reset the CubemapArraySlotsUsed array (we'll recompute it below)
	CubemapArraySlotsUsed.Init(false, NewRegularEnd);

	// Spin through the AllocatedReflectionCaptureState map and remap the indices based on the LUT
	TArray<FGuid> Components;
	AllocatedReflectionCaptureState.GetKeys(Components);
	int32 UsedCubemapCount = 0;
	for (int32 i=0; i<Components.Num(); i++ )
	{
		FCaptureComponentSceneState* ComponentStatePtr = AllocatedReflectionCaptureState.Find(Components[i]);
		check(ComponentStatePtr->CubemapIndex < IndexRemapping.Num());
		int32 NewIndex = IndexRemapping[ComponentStatePtr->CubemapIndex];
		CubemapArraySlotsUsed[NewIndex] = true;
		ComponentStatePtr->CubemapIndex = NewIndex;
		check(ComponentStatePtr->CubemapIndex > -1);
		UsedCubemapCount = FMath::Max(UsedCubemapCount, ComponentStatePtr->CubemapIndex + 1);
	}

	// Clear elements in the remapping array which are outside the range of the used components (these were allocated but not used)
	for (int i = 0; i < IndexRemapping.Num(); i++)
	{
		if (IndexRemapping[i] >= UsedCubemapCount)
		{
			IndexRemapping[i] = -1;
		}
	}

	// Build the remapping for reserved elements at the tail of the array.
	TArray<int32> ReservedTailRemapping;
	ReservedTailRemapping.Reserve(OldNumReserved);
	for (int32 i = 0; i < OldNumReserved; i++)
	{
		// Note that the number of reserved elements may have been reduced if the CVar shrank, so we need to check for
		// elements no longer in range, and give them an invalid remap index.
		ReservedTailRemapping.Add(i < (int32)InNumReserved ? NewRegularEnd + i : -1);
	}

	// Apply the reserved-tail remap to RuntimeSmoothBlendSlots.  Each occupied entry's SourceCubemapArrayIndex stores the
	// physical slice index of its source cubemap.  Drop the blend if the slot is being evicted by a CVar shrink.
	for (int32 i = 0; i < RuntimeSmoothBlendSlots.Num(); i++)
	{
		FSmoothBlendEntry& Entry = RuntimeSmoothBlendSlots[i];
		if (Entry.SourceCubemapArrayIndex != INDEX_NONE)
		{
			const int32 LogicalReservedIndex = Entry.SourceCubemapArrayIndex - OldRegularEnd;
			check(LogicalReservedIndex >= 0 && LogicalReservedIndex < OldNumReserved);
			Entry.SourceCubemapArrayIndex = ReservedTailRemapping[LogicalReservedIndex];
			if (Entry.SourceCubemapArrayIndex == INDEX_NONE)
			{
				Entry.OwningComponent = nullptr;
				Entry.BlendStartTime = -1.0;
			}
		}
	}
	RuntimeSmoothBlendSlots.SetNum((int32)InNumReserved);

	CubemapArray.ResizeCubemapArrayGPU(InMaxCubemaps, InNumReserved, InCubemapSize, IndexRemapping, ReservedTailRemapping);
}

void FReflectionEnvironmentSceneData::Reset(FScene* Scene)
{
	for (TSparseArray<UReflectionCaptureComponent*>::TIterator It(AllocatedReflectionCapturesGameThread); It; ++It)
	{
		UReflectionCaptureComponent* CurrentCapture = *It;

		// Unused slots will have a null component pointer
		if (CurrentCapture)
		{
			Scene->RemoveReflectionCapture(CurrentCapture);
		}
	}

	// Fields to reset in the game thread
	AllocatedReflectionCapturesGameThread.Empty();
	RuntimeCaptureBudgetCandidates.Empty();
	MaxAllocatedReflectionCubemapsGameThread = 0;
	ReflectionCaptureSizeGameThread = 0;
	DesiredReflectionCaptureSizeGameThread = 0;

	ENQUEUE_RENDER_COMMAND(ReflectionEnvironmentSceneDataReset)(
		[this](FRHICommandList& RHICmdList)
		{
			// Fields to reset in the render thread
			bRegisteredReflectionCapturesHasChanged = true;
			AllocatedReflectionCaptureStateHasChanged = false;

			ReflectionCaptureUniformBuffer.SafeRelease();
			MobileReflectionCaptureUniformBuffer.SafeRelease();

			RegisteredReflectionCaptures.Empty();
			RegisteredReflectionCapturePositionAndRadius.Empty();
			CubemapArray.Reset();
			AllocatedReflectionCaptureState.Empty();
			CubemapArraySlotsUsed.Empty();
			SortedCaptures.Empty();
			PreChangeSortOrder.Empty();
			NumBoxCaptures = 0;
			NumSphereCaptures = 0;
			RuntimeSmoothBlendSlots.Empty();
		});
}

void FReflectionEnvironmentSceneData::SetRegisteredReflectionCapturesHasChanged()
{
	bRegisteredReflectionCapturesHasChanged = true;

	// Snapshot the current sort order (key data only) for post-change sort order comparison in UpdateReflectionSceneData.
	if (PreChangeSortOrder.IsEmpty())
	{
		for (const FReflectionCaptureSortData& Entry : SortedCaptures)
		{
			PreChangeSortOrder.Add(Entry);
		}
	}
}

void FReflectionEnvironmentCubemapArray::ResizeCubemapArrayGPU(uint32 InMaxCubemaps, uint32 InNumReserved, int32 InCubemapSize, const TArray<int32>& IndexRemapping, const TArray<int32>& ReservedTailRemapping)
{
	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	check(InMaxCubemaps > 0);
	check(InCubemapSize > 0);
	check(InNumReserved <= InMaxCubemaps);
	check(IsInitialized());
	check(InCubemapSize == CubemapSize);

	// Take a reference to the old cubemap array and then release it to prevent it getting destroyed during InitRHI
	TRefCountPtr<IPooledRenderTarget> OldReflectionEnvs = ReflectionEnvs;
	ReflectionEnvs = nullptr;
	const int32 OldMaxCubemaps = (int32)MaxCubemaps;
	const int32 OldRegularEnd = (int32)MaxCubemaps - (int32)NumReservedCubemaps;
	MaxCubemaps = InMaxCubemaps;
	NumReservedCubemaps = InNumReserved;

	InitRHI(RHICmdList);

	const int32 NumMips = FMath::CeilLogTwo(InCubemapSize) + 1;

	{
		SCOPED_DRAW_EVENT(RHICmdList, ReflectionEnvironment_ResizeCubemapArray);

		RHICmdList.Transition({
			FRHITransitionInfo(OldReflectionEnvs->GetRHI(), ERHIAccess::Unknown, ERHIAccess::CopySrc),
			FRHITransitionInfo(ReflectionEnvs->GetRHI(), ERHIAccess::Unknown, ERHIAccess::CopyDest)
		});

		// Copy the regular cubemaps, remapping the elements as necessary
		const int32 RegularSourceCount = FMath::Min(OldRegularEnd, IndexRemapping.Num());
		for (int32 SourceCubemapIndex = 0; SourceCubemapIndex < RegularSourceCount; SourceCubemapIndex++)
		{
			const int32 DestCubemapIndex = IndexRemapping[SourceCubemapIndex];
			if (DestCubemapIndex != -1)
			{
				check(DestCubemapIndex < (int32)MaxCubemaps);

				for (int32 Face = 0; Face < CubeFace_MAX; Face++)
				{
					FRHICopyTextureInfo CopyInfo;
					CopyInfo.SourceSliceIndex = SourceCubemapIndex * CubeFace_MAX + Face;
					CopyInfo.DestSliceIndex   = DestCubemapIndex   * CubeFace_MAX + Face;
					CopyInfo.NumMips          = NumMips;

					RHICmdList.CopyTexture(OldReflectionEnvs->GetRHI(), ReflectionEnvs->GetRHI(), CopyInfo);
				}
			}
		}

		// Copy the reserved tail cubemaps, remapping the elements as necessary
		for (int32 ReservedTailIndex = 0; ReservedTailIndex < ReservedTailRemapping.Num(); ReservedTailIndex++)
		{
			const int32 SourceCubemapIndex = OldRegularEnd + ReservedTailIndex;
			const int32 DestCubemapIndex = ReservedTailRemapping[ReservedTailIndex];
			if (DestCubemapIndex != -1)
			{
				check(SourceCubemapIndex < OldMaxCubemaps);
				check(DestCubemapIndex < (int32)MaxCubemaps);

				for (int32 Face = 0; Face < CubeFace_MAX; Face++)
				{
					FRHICopyTextureInfo CopyInfo;
					CopyInfo.SourceSliceIndex = SourceCubemapIndex * CubeFace_MAX + Face;
					CopyInfo.DestSliceIndex   = DestCubemapIndex   * CubeFace_MAX + Face;
					CopyInfo.NumMips          = NumMips;

					RHICmdList.CopyTexture(OldReflectionEnvs->GetRHI(), ReflectionEnvs->GetRHI(), CopyInfo);
				}
			}
		}

		RHICmdList.Transition({
			FRHITransitionInfo(OldReflectionEnvs->GetRHI(), ERHIAccess::CopySrc, ERHIAccess::SRVMask),
			FRHITransitionInfo(ReflectionEnvs->GetRHI(), ERHIAccess::CopyDest, ERHIAccess::SRVMask)
		});
	}
	GRenderTargetPool.FreeUnusedResource(OldReflectionEnvs);
}

void FReflectionEnvironmentCubemapArray::UpdateMaxCubemaps(uint32 InMaxCubemaps, uint32 InNumReserved, int32 InCubemapSize)
{
	check(InMaxCubemaps > 0);
	check(InCubemapSize > 0);
	check(InNumReserved <= InMaxCubemaps);
	MaxCubemaps = InMaxCubemaps;
	NumReservedCubemaps = InNumReserved;
	CubemapSize = InCubemapSize;

	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	// Reallocate the cubemap array
	if (IsInitialized())
	{
		UpdateRHI(RHICmdList);
	}
	else
	{
		InitResource(RHICmdList);
	}
}

void FReflectionEnvironmentCubemapArray::Reset()
{
	ReleaseRHI();
	MaxCubemaps = 0;
	NumReservedCubemaps = 0;
	CubemapSize = 0;
}

IMPLEMENT_STATIC_UNIFORM_BUFFER_SLOT(ReflectionCapture);
IMPLEMENT_STATIC_AND_SHADER_UNIFORM_BUFFER_STRUCT(FReflectionCaptureShaderData, "ReflectionCaptureSM5", ReflectionCapture);
IMPLEMENT_STATIC_AND_SHADER_UNIFORM_BUFFER_STRUCT_EX(FMobileReflectionCaptureShaderData, "ReflectionCaptureES31", ReflectionCapture, FShaderParametersMetadata::EUsageFlags::NoEmulatedUniformBuffer);

void SetupSkyIrradianceEnvironmentMapConstantsFromSkyIrradiance(FVector4f* OutSkyIrradianceEnvironmentMap, const FSHVectorRGB3 SkyIrradiance)
{
	const float SqrtPI = FMath::Sqrt(PI);
	const float Coefficient0 = 1.0f / (2 * SqrtPI);
	const float Coefficient1 = FMath::Sqrt(3.f) / (3 * SqrtPI);
	const float Coefficient2 = FMath::Sqrt(15.f) / (8 * SqrtPI);
	const float Coefficient3 = FMath::Sqrt(5.f) / (16 * SqrtPI);
	const float Coefficient4 = .5f * Coefficient2;

	// Pack the SH coefficients in a way that makes applying the lighting use the least shader instructions
	// This has the diffuse convolution coefficients baked in
	// See "Stupid Spherical Harmonics (SH) Tricks"
	OutSkyIrradianceEnvironmentMap[0].X = -Coefficient1 * SkyIrradiance.R.V[3];
	OutSkyIrradianceEnvironmentMap[0].Y = -Coefficient1 * SkyIrradiance.R.V[1];
	OutSkyIrradianceEnvironmentMap[0].Z = Coefficient1 * SkyIrradiance.R.V[2];
	OutSkyIrradianceEnvironmentMap[0].W = Coefficient0 * SkyIrradiance.R.V[0] - Coefficient3 * SkyIrradiance.R.V[6];

	OutSkyIrradianceEnvironmentMap[1].X = -Coefficient1 * SkyIrradiance.G.V[3];
	OutSkyIrradianceEnvironmentMap[1].Y = -Coefficient1 * SkyIrradiance.G.V[1];
	OutSkyIrradianceEnvironmentMap[1].Z = Coefficient1 * SkyIrradiance.G.V[2];
	OutSkyIrradianceEnvironmentMap[1].W = Coefficient0 * SkyIrradiance.G.V[0] - Coefficient3 * SkyIrradiance.G.V[6];

	OutSkyIrradianceEnvironmentMap[2].X = -Coefficient1 * SkyIrradiance.B.V[3];
	OutSkyIrradianceEnvironmentMap[2].Y = -Coefficient1 * SkyIrradiance.B.V[1];
	OutSkyIrradianceEnvironmentMap[2].Z = Coefficient1 * SkyIrradiance.B.V[2];
	OutSkyIrradianceEnvironmentMap[2].W = Coefficient0 * SkyIrradiance.B.V[0] - Coefficient3 * SkyIrradiance.B.V[6];

	OutSkyIrradianceEnvironmentMap[3].X = Coefficient2 * SkyIrradiance.R.V[4];
	OutSkyIrradianceEnvironmentMap[3].Y = -Coefficient2 * SkyIrradiance.R.V[5];
	OutSkyIrradianceEnvironmentMap[3].Z = 3 * Coefficient3 * SkyIrradiance.R.V[6];
	OutSkyIrradianceEnvironmentMap[3].W = -Coefficient2 * SkyIrradiance.R.V[7];

	OutSkyIrradianceEnvironmentMap[4].X = Coefficient2 * SkyIrradiance.G.V[4];
	OutSkyIrradianceEnvironmentMap[4].Y = -Coefficient2 * SkyIrradiance.G.V[5];
	OutSkyIrradianceEnvironmentMap[4].Z = 3 * Coefficient3 * SkyIrradiance.G.V[6];
	OutSkyIrradianceEnvironmentMap[4].W = -Coefficient2 * SkyIrradiance.G.V[7];

	OutSkyIrradianceEnvironmentMap[5].X = Coefficient2 * SkyIrradiance.B.V[4];
	OutSkyIrradianceEnvironmentMap[5].Y = -Coefficient2 * SkyIrradiance.B.V[5];
	OutSkyIrradianceEnvironmentMap[5].Z = 3 * Coefficient3 * SkyIrradiance.B.V[6];
	OutSkyIrradianceEnvironmentMap[5].W = -Coefficient2 * SkyIrradiance.B.V[7];

	OutSkyIrradianceEnvironmentMap[6].X = Coefficient4 * SkyIrradiance.R.V[8];
	OutSkyIrradianceEnvironmentMap[6].Y = Coefficient4 * SkyIrradiance.G.V[8];
	OutSkyIrradianceEnvironmentMap[6].Z = Coefficient4 * SkyIrradiance.B.V[8];
	OutSkyIrradianceEnvironmentMap[6].W = 1;
}

void UpdateSkyIrradianceGpuBuffer(FRDGBuilder& GraphBuilder, const FEngineShowFlags& EngineShowFlags, const FSkyLightSceneProxy* SkyLight, TRefCountPtr<FRDGPooledBuffer>& Buffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateSkyIrradianceGpuBuffer);

	const bool bUploadIrradiance =
		SkyLight
		// Skylights with static lighting already had their diffuse contribution baked into lightmaps
		&& !SkyLight->bHasStaticLighting
		&& EngineShowFlags.SkyLighting
		&& !SkyLight->bRealTimeCaptureEnabled; // When bRealTimeCaptureEnabled is true, the buffer will be setup on GPU directly in this case

	FRDGBufferDesc BufferDesc = FRDGBufferDesc::CreateStructuredDesc(sizeof(FVector4f), SKY_IRRADIANCE_ENVIRONMENT_MAP_VEC4_COUNT);
	BufferDesc.Usage |= EBufferUsageFlags::SourceCopy;
	const bool bNewAlloc = AllocatePooledBuffer(
		BufferDesc,
		Buffer,
		TEXT("SkyIrradianceEnvironmentMap"),
		ERDGPooledBufferAlignment::None);

	FRDGBufferRef BufferRDG = GraphBuilder.RegisterExternalBuffer(Buffer, TEXT("SkyIrradianceEnvironmentMap"));
	const uint32 BufferSizeInBytes = BufferRDG->GetSize();
	check(BufferRDG);

	if (bNewAlloc && !bUploadIrradiance)
	{
		// Ensure that sky irradiance SH buffer contains sensible initial values (zero init).
		// If there is no sky in the level, then nothing else may fill this buffer.
		void* InitialValue = GraphBuilder.Alloc(BufferSizeInBytes);
		FMemory::Memset(InitialValue, 0, BufferSizeInBytes);
		GraphBuilder.QueueBufferUpload(BufferRDG, InitialValue, BufferSizeInBytes, ERDGInitialDataFlags::NoCopy);
	}

	if (bUploadIrradiance)
	{
		FVector4f* UploadData = (FVector4f*)GraphBuilder.Alloc(BufferSizeInBytes);
		SetupSkyIrradianceEnvironmentMapConstantsFromSkyIrradiance(UploadData, SkyLight->IrradianceEnvironmentMap);

		const float SkyLightAverageBrightness = SkyLight->AverageBrightness;
		UploadData[7] = FVector4f(SkyLightAverageBrightness, SkyLightAverageBrightness, SkyLightAverageBrightness, SkyLightAverageBrightness);

		GraphBuilder.QueueBufferUpload(BufferRDG, UploadData, BufferSizeInBytes, ERDGInitialDataFlags::NoCopy);
	}

	Buffer = ConvertToExternalAccessBuffer(GraphBuilder, BufferRDG, ERHIAccess::SRVMask, ERHIPipeline::All);
}