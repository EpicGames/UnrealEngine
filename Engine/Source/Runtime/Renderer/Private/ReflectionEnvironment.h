// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Reflection Environment common declarations
=============================================================================*/

#pragma once

#include "CoreTypes.h"
#include "Math/Vector4.h"
#include "RenderGraphFwd.h"
#include "RHIFwd.h"
#include "ShaderParameterMacros.h"

class FSkyLightSceneProxy;
class FViewInfo;
struct FEngineShowFlags;

template<int32 MaxSHOrder> class TSHVectorRGB;
typedef TSHVectorRGB<3> FSHVectorRGB3;

extern bool IsReflectionEnvironmentAvailable(ERHIFeatureLevel::Type InFeatureLevel);
extern bool IsReflectionCaptureAvailable();

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FReflectionUniformParameters,)
	SHADER_PARAMETER(FVector4f, SkyLightParameters)
	SHADER_PARAMETER_RDG_TEXTURE(TextureCube, SkyLightCubemap)
	SHADER_PARAMETER_SAMPLER(SamplerState, SkyLightCubemapSampler)
	SHADER_PARAMETER_TEXTURE(TextureCube, SkyLightBlendDestinationCubemap)
	SHADER_PARAMETER_SAMPLER(SamplerState, SkyLightBlendDestinationCubemapSampler)
	SHADER_PARAMETER_TEXTURE(TextureCubeArray, ReflectionCubemap)
	SHADER_PARAMETER_SAMPLER(SamplerState, ReflectionCubemapSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, PreIntegratedGF)
	SHADER_PARAMETER_SAMPLER(SamplerState, PreIntegratedGFSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

extern void SetupReflectionUniformParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FReflectionUniformParameters& OutParameters);
TRDGUniformBufferRef<FReflectionUniformParameters> CreateReflectionUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& View);

RENDERER_API void SetupSkyIrradianceEnvironmentMapConstantsFromSkyIrradiance(FVector4f* OutSkyIrradianceEnvironmentMap, const FSHVectorRGB3 SkyIrradiance);

extern void UpdateSkyIrradianceGpuBuffer(FRDGBuilder& GraphBuilder, const FEngineShowFlags& EngineShowFlags, const FSkyLightSceneProxy* SkyLight, TRefCountPtr<FRDGPooledBuffer>& Buffer);
