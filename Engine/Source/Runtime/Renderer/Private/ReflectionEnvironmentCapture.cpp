// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Functionality for capturing the scene into reflection capture cubemaps, and prefiltering
=============================================================================*/

#include "ReflectionEnvironmentCapture.h"
#include "CoreMinimal.h"

#include "CommonRenderResources.h"
#include "Components/ActorComponent.h"
#include "Components/BoxReflectionCaptureComponent.h"
#include "Components/PlaneReflectionCaptureComponent.h"
#include "Components/ReflectionCaptureComponent.h"
#include "Components/SphereReflectionCaptureComponent.h"
#include "Components/SceneCaptureComponent.h"
#include "Components/SceneCaptureComponentCube.h"
#include "Components/SceneComponent.h"
#include "Components/SkyLightComponent.h"
#include "Containers/SetUtilities.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DynamicRHI.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/MapBuildDataRegistry.h"
#include "Engine/SceneCaptureCube.h"
#include "Engine/TextureCube.h"
#include "Engine/World.h"
#include "EngineModule.h"
#include "GameFramework/Actor.h"
#include "GameFramework/PlayerController.h"
#include "GameTime.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "LegacyScreenPercentageDriver.h"
#include "Logging/LogMacros.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Math/Color.h"
#include "Math/Float16Color.h"
#include "Math/SHMath.h"
#include "Misc/App.h"
#include "Misc/FeedbackContext.h"
#include "Misc/Guid.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Misc/ScopeExit.h"
#include "MobileReflectionEnvironmentCapture.h"
#include "OneColorShader.h"
#include "PipelineStateCache.h"
#include "PixelShaderUtils.h"
#include "PostProcess/SceneFilterRendering.h"
#include "ReflectionEnvironment.h"
#include "RenderCommandTag.h"
#include "RendererInterface.h"
#include "LogRenderer.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "RenderResource.h"
#include "RenderTargetPool.h"
#include "RenderUtils.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIPipeline.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "RHIStats.h"
#include "RHITypes.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "SceneProxies/ReflectionCaptureProxy.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneRenderBuilder.h"
#include "SceneRendering.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShadowRendering.h"
#include "ShowFlags.h"
#include "Stats/Stats.h"
#include "TextureResource.h"
#include "UnrealClient.h"
#include "VisualizeTexture.h"

/** Near plane to use when capturing the scene. */
float GReflectionCaptureNearPlane = 5;

constexpr int32 MinSupersampleCaptureFactor = 1;
constexpr int32 MaxSupersampleCaptureFactor = 8;

int32 GSupersampleCaptureFactor = 1;
static FAutoConsoleVariableRef CVarGSupersampleCaptureFactor(
	TEXT("r.ReflectionCaptureSupersampleFactor"),
	GSupersampleCaptureFactor,
	TEXT("Super sample factor when rendering reflection captures.\n")
	TEXT("Default = 1, no super sampling\n")
	TEXT("Maximum clamped to 8."),
	ECVF_RenderThreadSafe
	);

float GSkylightCaptureLODDistanceScale = 1.f;
static FAutoConsoleVariableRef CVarSkylightCaptureLODDistanceScale(
	TEXT("r.SkylightCapture.LODDistanceScale"),
	GSkylightCaptureLODDistanceScale,
	TEXT("LODDistanceScale for the Sky Light Capture. Default is 1")
	TEXT("Negative values will be clamped to 1"),
	ECVF_Scalability);

TAutoConsoleVariable<int32> CVarReflectionCaptureRuntimeTimeslice(
	TEXT("r.ReflectionCapture.Runtime.Timeslice"),
	1,
	TEXT("Number of timesliced faces to render per frame (1 - 6)\n")
);

TAutoConsoleVariable<int32> CVarReflectionCaptureRuntimeTimesliceEditor(
	TEXT("r.ReflectionCapture.Runtime.TimesliceEditor"),
	3,
	TEXT("Number of timesliced faces to render per frame (1 - 6), in the editor.\n")
	TEXT("Setting to 6 may be useful for debugging, at a cost in editor performance.")
);

static TAutoConsoleVariable<int32> CVarReflectionCaptureRuntimeTimesliceSlow(
	TEXT("r.ReflectionCapture.Runtime.TimesliceSlow"),
	0,
	TEXT("Whether timesliced runtime reflection captures should update slower -- every other frame.\n")
	TEXT("Only applies when r.ReflectionCapture.Runtime.Timeslice (or TimesliceEditor) is set to 1.\n")
);

int32 GReflectionCaptureRuntimeDebugLock = INDEX_NONE;
static FAutoConsoleVariableRef CVarReflectionCaptureRuntimeDebugLock(
	TEXT("r.ReflectionCapture.Runtime.DebugLock"),
	GReflectionCaptureRuntimeDebugLock,
	TEXT("If positive, number is decremented per frame.  When it reaches zero, timesliced rendering is locked on the current face")
	TEXT("or faces being rendered.  Useful for graphical debugging of a specific timeslice.  Decrements to allow cycling around")
	TEXT("to the same face again by setting a count equal to the number of timeslices, or set to one to advance a single face.")
);

static TAutoConsoleVariable<bool> CVarReflectionCaptureRuntimeFoliage(
	TEXT("r.ReflectionCapture.Runtime.Foliage"),
	false,
	TEXT("Whether foliage should be enabled for runtime reflection captures.\n")
);

static TAutoConsoleVariable<bool> CVarReflectionCaptureRuntimeDFShadows(
	TEXT("r.ReflectionCapture.Runtime.DFShadows"),
	false,
	TEXT("Whether distance field shadows should be enabled for runtime reflection captures.\n")
);

static TAutoConsoleVariable<bool> CVarReflectionCaptureRuntimeTranslucency(
	TEXT("r.ReflectionCapture.Runtime.Translucency"),
	false,
	TEXT("Whether translucent rendering should be enabled for runtime reflection captures.\n")
);

static TAutoConsoleVariable<int32> CVarReflectionCaptureRuntimeMode(
	TEXT("r.ReflectionCapture.Runtime.Mode"),
	1,
	TEXT("Controls the runtime reflection capture update mode (ignored in editor, only active in PIE/runtime).\n")
	TEXT("0: Continuous - captures update indefinitely, nearest to camera first (default)\n")
	TEXT("1: Once - each runtime capture renders once.  Can optionally use RefreshCapture() blueprint function to re-trigger.\n")
);

static TAutoConsoleVariable<int32> CVarReflectionCaptureRuntimeBudget(
	TEXT("r.ReflectionCapture.Runtime.Budget"),
	0,
	TEXT("Maximum number of runtime reflection captures active simultaneously (0 = unlimited).\n")
	TEXT("Only active in PIE/runtime, ignored in editor. Nearest captures to camera are prioritized.\n")
);

static TAutoConsoleVariable<float> CVarReflectionCaptureRuntimeBudgetHysteresis(
	TEXT("r.ReflectionCapture.Runtime.BudgetHysteresis"),
	1000.0f,
	TEXT("Distance in Unreal units added to inactive runtime capture sort keys to prevent thrashing.\n")
	TEXT("An inactive capture must be this much closer than an active capture to replace it.\n")
);

TAutoConsoleVariable<float> CVarReflectionCaptureRuntimeFadeInTime(
	TEXT("r.ReflectionCapture.Runtime.FadeInTime"),
	0.5f,
	TEXT("Seconds over which a runtime reflection capture is faded in after its cubemap finishes rendering.\n")
	TEXT("0 disables the fade (capture pops to full contribution).\n")
);

static TAutoConsoleVariable<int32> CVarReflectionCaptureRuntimeFastRenderOnLoad(
	TEXT("r.ReflectionCapture.Runtime.FastRenderOnLoad"),
	3,
	TEXT("Number of runtime reflection captures to fast-render (all 6 faces, in one frame) the first time the\n")
	TEXT("budget system runs after level load.  0 disables the level-load burst.\n")
);

static TAutoConsoleVariable<int32> CVarReflectionCaptureRuntimeFastRenderOnTeleport(
	TEXT("r.ReflectionCapture.Runtime.FastRenderOnTeleport"),
	3,
	TEXT("Number of runtime reflection captures to fast-render when a camera teleport is detected.  Paired with\n")
	TEXT("r.ReflectionCapture.Runtime.TeleportDistance to classify a teleport.  0 disables teleport fast-render.\n")
);

static TAutoConsoleVariable<float> CVarReflectionCaptureRuntimeTeleportDistance(
	TEXT("r.ReflectionCapture.Runtime.TeleportDistance"),
	5000.0f,
	TEXT("Minimum single-frame camera jump (Unreal units) that counts as a teleport for the runtime capture\n")
	TEXT("budget system.  Out-of-budget captures are evicted instantly (no fade) and new captures are fast-rendered.\n")
);

TAutoConsoleVariable<int32> CVarReflectionCaptureRuntimeSmoothBlendSlots(
	TEXT("r.ReflectionCapture.Runtime.SmoothBlendSlots"),
	0,
	TEXT("Number of cubemap array slots reserved for smoothly blending manually refreshed runtime captures from\n")
	TEXT("their previous content to their new content.  Maximum is 4, default of 0 disables smooth blending.\n")
);

TAutoConsoleVariable<float> CVarReflectionCaptureRuntimeSmoothBlendTime(
	TEXT("r.ReflectionCapture.Runtime.SmoothBlendTime"),
	0.5f,
	TEXT("Seconds over which a smoothly-refreshed runtime reflection capture cross-fades from its old cubemap to\n")
	TEXT("its newly-rendered cubemap, after the new content is fully rendered.  0 collapses the fade to a pop.\n")
);

/**
 * Mip map used by a Roughness of 0, counting down from the lowest resolution mip (MipCount - 1).  
 * This has been tweaked along with ReflectionCaptureRoughnessMipScale to make good use of the resolution in each mip, especially the highest resolution mips.
 * This value is duplicated in ReflectionEnvironmentShared.usf!
 */
float ReflectionCaptureRoughestMip = 1;

/** 
 * Scales the log2 of Roughness when computing which mip to use for a given roughness.
 * Larger values make the higher resolution mips sharper.
 * This has been tweaked along with ReflectionCaptureRoughnessMipScale to make good use of the resolution in each mip, especially the highest resolution mips.
 * This value is duplicated in ReflectionEnvironmentShared.usf!
 */
float ReflectionCaptureRoughnessMipScale = 1.2f;

// Chaos addition
static TAutoConsoleVariable<int32> CVarReflectionCaptureStaticSceneOnly(
	TEXT("r.chaos.ReflectionCaptureStaticSceneOnly"),
	1,
	TEXT("")
	TEXT(" 0 is off, 1 is on (default)"),
	ECVF_ReadOnly);

/**
* This CVar might affect the quality and performance for: 
* (1) Captured reflection: Increase volume and light function quality and the cost at lighting build time.
* (2) Sky light scene capture (non real time): Increase quality and the cost at the start of a level when the scene is captured.
* (3) Real time sky light capture: this one does not render volumetric fog or anything that reads a light function. It increases the cost only.
*
* It might also create mismatch when light function is time dependent (e.g., sun light simulating time-varying cloud shadows). Different level building can look inconsistent builds after builds.
*/
static int32 GReflectionCaptureEnableLightFunctions = 0;
static FAutoConsoleVariableRef CVarReflectionCaptureEnableLightFunctions(
	TEXT("r.ReflectionCapture.EnableLightFunctions"),
	GReflectionCaptureEnableLightFunctions,
	TEXT("0. Disable light functions in reflection/sky light capture (default).\n")
	TEXT("Others. Enable light functions."));

BEGIN_SHADER_PARAMETER_STRUCT(FCubeShaderParameters, )
	SHADER_PARAMETER_RDG_TEXTURE_SRV(TextureCube, SourceCubemapTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
	SHADER_PARAMETER(FVector2f, SvPositionToUVScale)
	SHADER_PARAMETER(int32, CubeFace)
	SHADER_PARAMETER(int32, SourceMipIndex)
	SHADER_PARAMETER(int32, MipIndex)
	SHADER_PARAMETER(int32, NumMips)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

/** Pixel shader used for filtering a mip. */
class FCubeDownsamplePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCubeDownsamplePS);
	SHADER_USE_PARAMETER_STRUCT(FCubeDownsamplePS, FGlobalShader);
	using FParameters = FCubeShaderParameters;
};

IMPLEMENT_GLOBAL_SHADER(FCubeDownsamplePS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "DownsamplePS", SF_Pixel);

class FCubeDownsampleMaxPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCubeDownsampleMaxPS);
	SHADER_USE_PARAMETER_STRUCT(FCubeDownsampleMaxPS, FGlobalShader);
	using FParameters = FCubeShaderParameters;
};

IMPLEMENT_GLOBAL_SHADER(FCubeDownsampleMaxPS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "DownsampleMaxPS", SF_Pixel);

class FCubeFilterPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCubeFilterPS);
	SHADER_USE_PARAMETER_STRUCT(FCubeFilterPS, FGlobalShader);
	using FParameters = FCubeShaderParameters;
};

IMPLEMENT_GLOBAL_SHADER(FCubeFilterPS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "FilterPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FCubeFinalGatherPS, )
	SHADER_PARAMETER_RDG_TEXTURE(TextureCube, ReflectionEnvironmentColorTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, ReflectionEnvironmentColorSampler)
	SHADER_PARAMETER(int32, NumCaptureArrayMips)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

/** Computes the average brightness of a 1x1 mip of a cubemap. */
class FComputeBrightnessPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComputeBrightnessPS);
	SHADER_USE_PARAMETER_STRUCT(FComputeBrightnessPS, FGlobalShader);
	using FParameters = FCubeFinalGatherPS;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTEBRIGHTNESS_PIXELSHADER"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FComputeBrightnessPS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "ComputeBrightnessMain", SF_Pixel);

class FComputeCubeMaxLuminancePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComputeCubeMaxLuminancePS);
	SHADER_USE_PARAMETER_STRUCT(FComputeCubeMaxLuminancePS, FGlobalShader);
	using FParameters = FCubeFinalGatherPS;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("COMPUTEMAXLUMINANCE_PIXELSHADER"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FComputeCubeMaxLuminancePS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "ComputeCubeMaxLuminancePS", SF_Pixel);

/** Vertex shader used when writing to a cubemap. */
class FCopyToCubeFaceVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyToCubeFaceVS);

	FCopyToCubeFaceVS() = default;
	FCopyToCubeFaceVS(const CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

IMPLEMENT_GLOBAL_SHADER(FCopyToCubeFaceVS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "CopyToCubeFaceVS", SF_Vertex);

/** Pixel shader used when copying scene color from a scene render into a face of a reflection capture cubemap. */
class FCopySceneColorToCubeFacePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopySceneColorToCubeFacePS);
	SHADER_USE_PARAMETER_STRUCT(FCopySceneColorToCubeFacePS, FGlobalShader);

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		if (IsMobilePlatform(Parameters.Platform))
		{
			// SceneDepth is memoryless on mobile
			OutEnvironment.SetDefine(TEXT("SCENE_TEXTURES_DISABLED"), 1u);
		}
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneColorTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneColorSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SceneDepthSampler)
		SHADER_PARAMETER(FVector4f, SkyLightCaptureParameters)
		SHADER_PARAMETER(FVector4f, LowerHemisphereColor)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCopySceneColorToCubeFacePS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "CopySceneColorToCubeFaceColorPS", SF_Pixel);

/** Pixel shader used when copying a cubemap into a face of a reflection capture cubemap. */
class FCopyCubemapToCubeFacePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FCopyCubemapToCubeFacePS);
	SHADER_USE_PARAMETER_STRUCT(FCopyCubemapToCubeFacePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER(FVector4f, SkyLightCaptureParameters)
		SHADER_PARAMETER(FVector4f, LowerHemisphereColor)
		SHADER_PARAMETER(FVector4f, ColorAlphaMultiplier)
		SHADER_PARAMETER(FVector2f, SinCosSourceCubemapRotation)
		SHADER_PARAMETER(FVector2f, SvPositionToUVScale)
		SHADER_PARAMETER(int32, CubeFace)
		SHADER_PARAMETER(float, ClampToFP16)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCopyCubemapToCubeFacePS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "CopyCubemapToCubeFaceColorPS", SF_Pixel);

class FReflectionCubemapTexture : public FRenderThreadStructBase
{
public:
	FReflectionCubemapTexture(uint32 InCubemapSize)
		: CubemapSize(InCubemapSize)
	{
		check(GSupportsRenderTargetFormat_PF_FloatRGBA);
	}

	void InitRHI(FRHICommandListImmediate& RHICmdList)
	{
		const int32 NumReflectionCaptureMips = GetNumMips(CubemapSize);

		const FRDGTextureDesc TextureDesc = FRDGTextureDesc::CreateCube(
			CubemapSize, PF_FloatRGBA, FClearValueBinding(FLinearColor(0, 10000, 0, 0)), TexCreate_RenderTargetable | TexCreate_TargetArraySlicesIndependently | TexCreate_DisableDCC, NumReflectionCaptureMips);

		RenderTarget = AllocatePooledTexture(TextureDesc, TEXT("ReflectionCubeTexture"));

		FRDGBuilder GraphBuilder(RHICmdList);

		{
			RDG_EVENT_SCOPE(GraphBuilder, "ClearReflectionCubemap");

			FRDGTextureRef OutputTexture = GetRDG(GraphBuilder);

			for (int32 MipIndex = 0; MipIndex < NumReflectionCaptureMips; MipIndex++)
			{
				for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
				{
					FRenderTargetParameters* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
					PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::EClear, MipIndex, CubeFace);

					GraphBuilder.AddPass(
						RDG_EVENT_NAME("Clear (Mip: %d, Face : %d)", MipIndex, CubeFace),
						PassParameters,
						ERDGPassFlags::Raster,
						[](FRDGAsyncTask, FRHICommandList&) {});
				}
			}
		}

		GraphBuilder.Execute();
	}

	FRDGTexture* GetRDG(FRDGBuilder& GraphBuilder) const
	{
		return GraphBuilder.RegisterExternalTexture(RenderTarget);
	}

private:
	TRefCountPtr<IPooledRenderTarget> RenderTarget;
	uint32 CubemapSize;
};

void CreateCubeMips(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture)
{
	RDG_EVENT_SCOPE(GraphBuilder, "CreateCubeMips");

	TShaderMapRef<FCubeDownsamplePS> PixelShader(ShaderMap);

	const int32 NumMips = CubemapTexture->Desc.NumMips;

	// Downsample all the mips, each one reads from the mip above it
	for (int32 MipIndex = 1; MipIndex < NumMips; MipIndex++)
	{
		const int32 SourceMipIndex = (MipIndex - 1);
		const int32 MipSize = 1 << (NumMips - MipIndex - 1);
		const FIntRect ViewRect(0, 0, MipSize, MipSize);

		FRDGTextureSRVDesc SourceSRVDesc = FRDGTextureSRVDesc::CreateForMipLevel(CubemapTexture, SourceMipIndex);
		if (GRHISupportsTextureViews == false)
		{
			SourceSRVDesc = FRDGTextureSRVDesc::Create(CubemapTexture);
		}

		for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
		{
			auto* PassParameters = GraphBuilder.AllocParameters<FCubeDownsamplePS::FParameters>();
			PassParameters->SourceCubemapTexture = GraphBuilder.CreateSRV(SourceSRVDesc);
			PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->CubeFace = CubeFace;
			PassParameters->SourceMipIndex = (GRHISupportsTextureViews ? 0 : SourceMipIndex);
			PassParameters->MipIndex = MipIndex;
			PassParameters->NumMips = NumMips;
			PassParameters->SvPositionToUVScale = FVector2f(1.0f / MipSize, 1.0f / MipSize);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(CubemapTexture, ERenderTargetLoadAction::ENoAction, MipIndex, CubeFace);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				ShaderMap,
				RDG_EVENT_NAME("CreateCubeMips (Mip: %d, Face: %d)", MipIndex, CubeFace),
				PixelShader,
				PassParameters,
				ViewRect);
		}
	}
}

void ComputeSingleAverageBrightnessFromCubemap(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, float* OutAverageBrightness)
{
	RDG_EVENT_SCOPE(GraphBuilder, "ComputeSingleAverageBrightnessFromCubemap");

	FRDGTexture* ReflectionBrightnessTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_FloatRGBA, FClearValueBinding::None, TexCreate_RenderTargetable), TEXT("ReflectionBrightness"));

	auto* PassParameters = GraphBuilder.AllocParameters<FComputeBrightnessPS::FParameters>();

	PassParameters->ReflectionEnvironmentColorTexture = CubemapTexture;
	PassParameters->ReflectionEnvironmentColorSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->NumCaptureArrayMips = CubemapTexture->Desc.NumMips;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(ReflectionBrightnessTexture, ERenderTargetLoadAction::ENoAction);

	TShaderMapRef<FComputeBrightnessPS> PixelShader(ShaderMap);

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		RDG_EVENT_NAME("ReflectionBrightness"),
		PixelShader,
		PassParameters,
		FIntRect(FIntPoint::ZeroValue, FIntPoint(1, 1)));

	AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("ReadbackTexture"), ReflectionBrightnessTexture, [ReflectionBrightnessTexture, OutAverageBrightness](FRHICommandListImmediate& RHICmdList)
	{
		TArray<FFloat16Color> SurfaceData;
		RHICmdList.ReadSurfaceFloatData(ReflectionBrightnessTexture->GetRHI(), FIntRect(0, 0, 1, 1), SurfaceData, CubeFace_PosX, 0, 0);

		// Shader outputs luminance to R
		*OutAverageBrightness = SurfaceData[0].R.GetFloat();
	});
}

void ComputeAverageBrightness(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, float* OutAverageBrightness)
{
	CreateCubeMips(GraphBuilder, ShaderMap, CubemapTexture);
	ComputeSingleAverageBrightnessFromCubemap(GraphBuilder, ShaderMap, CubemapTexture, OutAverageBrightness);
}

FRDGTexture* FilterCubeMap(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* SourceTexture)
{
	RDG_EVENT_SCOPE(GraphBuilder, "FilterCubeMap");

	FRDGTexture* FilteredCubemapTexture = GraphBuilder.CreateTexture(SourceTexture->Desc, TEXT("FilteredCubemapTexture"));

	const int32 NumMips = SourceTexture->Desc.NumMips;

	TShaderMapRef<FCubeFilterPS> PixelShader(ShaderMap);

	for (int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
	{
		const int32 MipSize = 1 << (NumMips - MipIndex - 1);
		const FIntRect ViewRect(0, 0, MipSize, MipSize);

		for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
		{
			auto* PassParameters = GraphBuilder.AllocParameters<FCubeFilterPS::FParameters>();
			PassParameters->SourceCubemapTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceTexture));
			PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->CubeFace = CubeFace;
			PassParameters->MipIndex = MipIndex;
			PassParameters->NumMips = NumMips;
			PassParameters->SvPositionToUVScale = FVector2f(1.0f / MipSize, 1.0f / MipSize);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(FilteredCubemapTexture, ERenderTargetLoadAction::ENoAction, MipIndex, CubeFace);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				ShaderMap,
				RDG_EVENT_NAME("FilterCubeMap (Mip: %d, CubeFace: %d)", MipIndex, CubeFace),
				PixelShader,
				PassParameters,
				ViewRect);
		}
	}

	return FilteredCubemapTexture;
}

void ConvolveCubeMap(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, uint32 CubeMipStart, uint32 CubeMipEnd, uint32 FaceStart, uint32 FaceCount, FRDGTexture* RDGSrcRenderTarget, FRDGTexture* RDGDstRenderTarget)
{
	TShaderMapRef<FCubeFilterPS> PixelShader(ShaderMap);

	const int32 NumMips = RDGSrcRenderTarget->Desc.NumMips;

	for (uint32 MipIndex = CubeMipStart; MipIndex <= CubeMipEnd; MipIndex++)
	{
		const int32 MipSize = 1 << (NumMips - MipIndex - 1);
		const FIntRect ViewRect(0, 0, MipSize, MipSize);

		for (uint32 CubeFace = FaceStart; CubeFace < FaceStart+FaceCount; CubeFace++)
		{
			auto* PassParameters = GraphBuilder.AllocParameters<FCubeFilterPS::FParameters>();
			PassParameters->SourceCubemapTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(RDGSrcRenderTarget));
			PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->CubeFace = CubeFace;
			PassParameters->MipIndex = MipIndex;
			PassParameters->NumMips = NumMips;
			PassParameters->SvPositionToUVScale = FVector2f(1.0f / MipSize, 1.0f / MipSize);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(RDGDstRenderTarget, ERenderTargetLoadAction::ENoAction, MipIndex, CubeFace);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				ShaderMap,
				RDG_EVENT_NAME("ConvolveCubeMap (Mip: %u, CubeFace: %u)", MipIndex, CubeFace),
				PixelShader,
				PassParameters,
				ViewRect);
		}
	}
}

// Premultiply alpha in-place using alpha blending

void PremultiplyCubeMipAlpha(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, int32 MipIndex)
{
	const int32 NumMips = CubemapTexture->Desc.NumMips;
	const int32 MipSize = 1 << (NumMips - MipIndex - 1);
	const FIntRect ViewRect(0, 0, MipSize, MipSize);

	for (uint32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
	{
		auto* PassParameters = GraphBuilder.AllocParameters<FOneColorPS::FParameters>();
		PassParameters->DrawColorMRT[0] = FLinearColor::Black;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(CubemapTexture, ERenderTargetLoadAction::ELoad, MipIndex, CubeFace);

		TShaderMapRef<FOneColorPS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("PremultipliedAlpha (Mip: %d, Face %d", MipIndex, CubeFace),
			PixelShader,
			PassParameters,
			ViewRect,
			TStaticBlendState<CW_RGBA, BO_Add, BF_Zero, BF_DestAlpha, BO_Add, BF_Zero, BF_One>::GetRHI());
	}
}

/** Generates mips for glossiness and filters the cubemap for a given reflection. */
FRDGTexture* FilterReflectionEnvironment(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture, FSHVectorRGB3* OutIrradianceEnvironmentMap)
{
	RDG_EVENT_SCOPE(GraphBuilder, "FilterReflectionEnvironment");

	PremultiplyCubeMipAlpha(GraphBuilder, ShaderMap, CubemapTexture, 0);
	CreateCubeMips(GraphBuilder, ShaderMap, CubemapTexture);

	if (OutIrradianceEnvironmentMap)
	{
		ComputeDiffuseIrradiance(GraphBuilder, ShaderMap, CubemapTexture, OutIrradianceEnvironmentMap);
	}

	return FilterCubeMap(GraphBuilder, ShaderMap, CubemapTexture);
}

int32 FindOrAllocateCubemapIndex(FScene* Scene, const UReflectionCaptureComponent* Component)
{
	int32 CubemapIndex = -1;

	// Try to find an existing capture index for this component
	const FCaptureComponentSceneState* CaptureSceneStatePtr = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.AddReference(Component);

	if (CaptureSceneStatePtr)
	{
		CubemapIndex = CaptureSceneStatePtr->CubemapIndex;
	}
	else
	{
		// Reuse a freed index if possible
		CubemapIndex = Scene->ReflectionSceneData.CubemapArraySlotsUsed.FindAndSetFirstZeroBit();
		if (CubemapIndex == INDEX_NONE)
		{
			// If we didn't find a free index, allocate a new one from the CubemapArraySlotsUsed bitfield
			CubemapIndex = Scene->ReflectionSceneData.CubemapArraySlotsUsed.Num();
			Scene->ReflectionSceneData.CubemapArraySlotsUsed.Add(true);
		}

		Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Add(Component, FCaptureComponentSceneState(CubemapIndex));
		Scene->ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged = true;

		check(CubemapIndex < GetMaxNumReflectionCaptures(Scene->GetShaderPlatform()) - Scene->ReflectionSceneData.CubemapArray.GetNumReservedCubemaps());
	}

	check(CubemapIndex >= 0);
	return CubemapIndex;
}

/** Captures the scene for a reflection capture by rendering the scene multiple times and copying into a cubemap texture. */
void CaptureSceneToScratchCubemap(
	FRDGBuilder& GraphBuilder,
	FSceneRenderer* SceneRenderer,
	FSceneRenderUpdateInputs* SceneUpdateInputs,
	const FReflectionCubemapTexture& ReflectionCubemapTexture,
	ECubeFace CubeFace,
	int32 CubemapSize,
	bool bCapturingForSkyLight,
	bool bLowerHemisphereIsBlack,
	const FLinearColor& LowerHemisphereColor,
	bool bCapturingForMobile)
{
	const ERHIFeatureLevel::Type FeatureLevel = SceneRenderer->FeatureLevel;

	UE_CLOGF(FSceneCaptureLogUtils::bEnableSceneCaptureLogging, LogSceneCapture, Log, "Starting CaptureSceneToScratchCubemap Face: %d, Size: %d, bCapturingForSkylight: %d", CubeFace, CubemapSize, (int32) bCapturingForSkyLight);

	// Render the scene normally for one face of the cubemap
	SceneRenderer->Render(GraphBuilder, SceneUpdateInputs);

	AddPass(GraphBuilder, RDG_EVENT_NAME("FlushGPU"), [](FRHICommandListImmediate& InRHICmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_CaptureSceneToScratchCubemap_Flush);
		InRHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);

		// some platforms may not be able to keep enqueueing commands like crazy, this will
		// allow them to restart their command buffers
		InRHICmdList.SubmitAndBlockUntilGPUIdle();
	});

	const FViewInfo& View = SceneRenderer->Views[0];

	FRDGTextureRef OutputTexture = ReflectionCubemapTexture.GetRDG(GraphBuilder);

	auto* PassParameters = GraphBuilder.AllocParameters<FCopySceneColorToCubeFacePS::FParameters>();
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ENoAction, 0, CubeFace);
	PassParameters->LowerHemisphereColor = LowerHemisphereColor;

	{
		FVector4f SkyLightParametersValue(ForceInitToZero);
		FScene* Scene = SceneRenderer->Scene;

		if (bCapturingForSkyLight)
		{
			// When capturing reflection captures, support forcing all low hemisphere lighting to be black
			SkyLightParametersValue = FVector4f(0, 0, bLowerHemisphereIsBlack ? 1.0f : 0.0f, 0);
		}
		else if (!bCapturingForMobile && Scene->SkyLight && !Scene->SkyLight->bHasStaticLighting)	
		{
			// Mobile renderer can't blend reflections with a sky at runtime, so we dont use this path when capturing for a mobile renderer
				
			// When capturing reflection captures and there's a stationary sky light, mask out any pixels whose depth classify it as part of the sky
			// This will allow changing the stationary sky light at runtime
			SkyLightParametersValue = FVector4f(1, Scene->SkyLight->SkyDistanceThreshold, 0, 0);
		}
		else
		{
			// When capturing reflection captures and there's no sky light, or only a static sky light, capture all depth ranges
			SkyLightParametersValue = FVector4f(2, 0, 0, 0);
		}

		PassParameters->SkyLightCaptureParameters = SkyLightParametersValue;

		UE_CLOGF(FSceneCaptureLogUtils::bEnableSceneCaptureLogging, LogSceneCapture, Log, "Ending CaptureSceneToScratchCubemap Face: %d, Size: %d, bCapturingForSkylight: %d", CubeFace, CubemapSize, (int32) bCapturingForSkyLight);
	}

	const FMinimalSceneTextures& SceneTextures = View.GetSceneTextures();

	PassParameters->View = View.ViewUniformBuffer;
	PassParameters->SceneColorSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	PassParameters->SceneColorTexture = SceneTextures.Color.Resolve;
        
    if (!IsMobilePlatform(SceneRenderer->ShaderPlatform))
    {
        PassParameters->SceneDepthSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
        PassParameters->SceneDepthTexture = SceneTextures.Depth.Resolve;
    }

	const int32 EffectiveSize = CubemapSize;
	const FIntPoint SceneTextureExtent = SceneTextures.Config.Extent;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("CopySceneToCubeFace"),
		PassParameters,
		ERDGPassFlags::Raster,
		[EffectiveSize, SceneTextureExtent, FeatureLevel, PassParameters](FRDGAsyncTask, FRHICommandList& InRHICmdList)
	{
		const FIntRect ViewRect(0, 0, EffectiveSize, EffectiveSize);
		InRHICmdList.SetViewport(0.0f, 0.0f, 0.0f, (float)EffectiveSize, (float)EffectiveSize, 1.0f);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		InRHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

		TShaderMapRef<FCopyToCubeFaceVS> VertexShader(GetGlobalShaderMap(FeatureLevel));
		TShaderMapRef<FCopySceneColorToCubeFacePS> PixelShader(GetGlobalShaderMap(FeatureLevel));

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);
		SetShaderParameters(InRHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

		const int32 SupersampleCaptureFactor = FMath::Clamp(GSupersampleCaptureFactor, MinSupersampleCaptureFactor, MaxSupersampleCaptureFactor);

		DrawRectangle( 
			InRHICmdList,
			ViewRect.Min.X, ViewRect.Min.Y, 
			ViewRect.Width(), ViewRect.Height(),
			ViewRect.Min.X, ViewRect.Min.Y, 
			ViewRect.Width() * SupersampleCaptureFactor, ViewRect.Height() * SupersampleCaptureFactor,
			FIntPoint(ViewRect.Width(), ViewRect.Height()),
			SceneTextureExtent,
			VertexShader);
	});
}


static void CopyCubemapToScratchCubemapInner(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	const FTexture* SourceCubemapResource,
	FRDGTextureRef OutputTexture,
	int32 CubemapSize,
	bool bIsSkyLight,
	bool bLowerHemisphereIsBlack,
	float SourceCubemapRotation,
	const FLinearColor& LowerHemisphereColorValue,
	const FLinearColor& ColorAlphaMultiplier,
	const bool bClampToFP16)
{
	RDG_EVENT_SCOPE(GraphBuilder, "CopyCubemapToScratchCubemap");

	TShaderMapRef<FCopyCubemapToCubeFacePS> PixelShader(ShaderMap);

	for (uint32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
	{
		auto* PassParameters = GraphBuilder.AllocParameters<FCopyCubemapToCubeFacePS::FParameters>();
		PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->SourceCubemapTexture = SourceCubemapResource->TextureRHI;
		PassParameters->LowerHemisphereColor = LowerHemisphereColorValue;
		PassParameters->SkyLightCaptureParameters = FVector3f(bIsSkyLight ? 1.0f : 0.0f, 0.0f, bLowerHemisphereIsBlack ? 1.0f : 0.0f);
		PassParameters->SinCosSourceCubemapRotation = FVector2f(FMath::Sin(SourceCubemapRotation), FMath::Cos(SourceCubemapRotation));
		PassParameters->SvPositionToUVScale = FVector2f(1.0f / CubemapSize, 1.0f / CubemapSize);
		PassParameters->CubeFace = CubeFace;
		PassParameters->ColorAlphaMultiplier = ColorAlphaMultiplier;
		PassParameters->ClampToFP16 = bClampToFP16 ? 1.0f : 0.0f;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ENoAction, 0, CubeFace);

		const FIntRect ViewRect(0, 0, CubemapSize, CubemapSize);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("CopyCubemapToCubeFace"),
			PixelShader,
			PassParameters,
			ViewRect);
	}
}

void CopyCubemapToScratchCubemap(
	FRHICommandListImmediate& RHICmdList,
	ERHIFeatureLevel::Type FeatureLevel,
	UTextureCube* SourceCubemap,
	const FReflectionCubemapTexture& ReflectionCubemapTexture,
	int32 CubemapSize,
	bool bIsSkyLight,
	bool bLowerHemisphereIsBlack,
	float SourceCubemapRotation,
	const FLinearColor& LowerHemisphereColorValue,
	const FLinearColor& ColorAlphaMultiplier)
{
	check(SourceCubemap);

	const FTexture* SourceCubemapResource = SourceCubemap->GetResource();

	if (SourceCubemapResource == nullptr)
	{
		UE_LOGF(LogRenderer, Warning, "Unable to copy from cubemap %ls, it's RHI resource is null", *SourceCubemap->GetPathName());
		return;
	}

	FRDGBuilder GraphBuilder(RHICmdList);

	auto ShaderMap = GetGlobalShaderMap(FeatureLevel);
	FRDGTextureRef OutputTexture = ReflectionCubemapTexture.GetRDG(GraphBuilder);
	const bool bClampToFP16 = true; // Rendering into an FP16 texture.
	CopyCubemapToScratchCubemapInner(
		GraphBuilder,
		ShaderMap,
		SourceCubemapResource,
		OutputTexture,
		CubemapSize,
		bIsSkyLight,
		bLowerHemisphereIsBlack,
		SourceCubemapRotation,
		LowerHemisphereColorValue,
		ColorAlphaMultiplier,
		bClampToFP16);

	GraphBuilder.Execute();
}

const int32 MinCapturesForSlowTask = 20;

void BeginReflectionCaptureSlowTask(int32 NumCaptures, const TCHAR* CaptureReason)
{
	if (NumCaptures > MinCapturesForSlowTask)
	{
		FText Status;
		
		if (CaptureReason)
		{
			Status = FText::Format(NSLOCTEXT("Engine", "UpdateReflectionCapturesForX", "Building reflection captures for {0}"), FText::FromString(FString(CaptureReason)));
		}
		else
		{
			Status = FText(NSLOCTEXT("Engine", "UpdateReflectionCaptures", "Building reflection captures..."));
		}

		GWarn->BeginSlowTask(Status, true);
		GWarn->StatusUpdate(0, NumCaptures, Status);
	}
}

void UpdateReflectionCaptureSlowTask(int32 CaptureIndex, int32 NumCaptures)
{
	const int32 UpdateDivisor = FMath::Max(NumCaptures / 5, 1);

	if (NumCaptures > MinCapturesForSlowTask && (CaptureIndex % UpdateDivisor) == 0)
	{
		GWarn->UpdateProgress(CaptureIndex, NumCaptures);
	}
}

void EndReflectionCaptureSlowTask(int32 NumCaptures)
{
	if (NumCaptures > MinCapturesForSlowTask)
	{
		GWarn->EndSlowTask();
	}
}

static int32 GetMaxReflectionCapturesAtSize(int32 CaptureSize)
{
	FRHITextureDesc RHITexDesc(
		ETextureDimension::TextureCube,
		ETextureCreateFlags::None,
		PF_FloatRGBA,
		FClearValueBinding::Black,
		FIntPoint(CaptureSize, CaptureSize),
		1,										// Depth
		1,										// ArraySize
		FMath::CeilLogTwo(CaptureSize) + 1,		// InNumMips
		1,										// Samples
		0);

	SIZE_T TexMemRequiredPerCapture = RHICalcTexturePlatformSize(RHITexDesc).Size;

	// Attempt to limit the resource size to within percentage (3/4) of total video memory to give consistent stable results.
	// Also limit max size to 4 GB (technically 4 GB minus one), as D3D12 allocation fails for individual resources 4 GB or over,
	// and we'll assume the same is true for other platforms.
	FTextureMemoryStats TextureMemStats;
	RHIGetTextureMemoryStats(TextureMemStats);
		
	SIZE_T DedicatedVideoMemoryLimit = ((SIZE_T)TextureMemStats.DedicatedVideoMemory * (SIZE_T)3) / (SIZE_T)4;
	if (!DedicatedVideoMemoryLimit)
	{
		DedicatedVideoMemoryLimit = (SIZE_T)UINT_MAX;
	}

	const SIZE_T MaxResourceVideoMemoryFootprint = FMath::Min(DedicatedVideoMemoryLimit, (SIZE_T)UINT_MAX);

	return MaxResourceVideoMemoryFootprint / TexMemRequiredPerCapture;
}

int32 NumUniqueReflectionCaptures(const TSparseArray<UReflectionCaptureComponent*>& CaptureComponents)
{
	TSet<FGuid> Guids;
	for (TSparseArray<UReflectionCaptureComponent*>::TConstIterator It(CaptureComponents); It; ++It)
	{
		Guids.Add((*It)->MapBuildDataId);
	}

	return Guids.Num();
}

// Signed distance from CameraLocation to the influence shape of Comp.  Returns 0 or negative when the camera is inside the
// influence volume, and positive distance to the surface otherwise.
static float ComputeRuntimeBudgetSignedDistance(const UReflectionCaptureComponent* Comp, const FVector& CameraLocation)
{
	if (const USphereReflectionCaptureComponent* SphereComp = Cast<const USphereReflectionCaptureComponent>(Comp))
	{
		const float CenterDist = (float)FVector::Dist(CameraLocation, SphereComp->GetComponentLocation());
		return CenterDist - SphereComp->InfluenceRadius;
	}

	if (const UBoxReflectionCaptureComponent* BoxComp = Cast<const UBoxReflectionCaptureComponent>(Comp))
	{
		const FTransform& Transform = BoxComp->GetComponentTransform();
		const FVector LocalCamera = Transform.InverseTransformPositionNoScale(CameraLocation);
		const FVector Extent = Transform.GetScale3D();

		const FVector SignedOffset = LocalCamera.GetAbs() - Extent;
		if (SignedOffset.X <= 0.0f && SignedOffset.Y <= 0.0f && SignedOffset.Z <= 0.0f)
		{
			// Inside: signed distance is smallest negative penetration depth (closest face).
			return (float)SignedOffset.GetMax();
		}

		// Outside: distance to the closest point on the OBB surface.
		return (float)SignedOffset.ComponentMax(FVector::Zero()).Length();
	}

	// Plane (or unknown): fall back to center distance -- shouldn't be reached in practice.
	return (float)FVector::Dist(CameraLocation, Comp->GetComponentLocation());
}

/**
 * Allocates reflection captures in the scene's reflection cubemap array and updates them by recapturing the scene.
 * Existing captures will only be uploaded.  Must be called from the game thread.
 */
void FScene::AllocateReflectionCaptures(const TArray<UReflectionCaptureComponent*>& NewCaptures, const TCHAR* CaptureReason, bool bVerifyOnlyCapturing, bool bCapturingForMobile, bool bInsideTick)
{
	// We potentially need to add to the NewCaptures list if the runtime capture state changed for a given component
	TArray<UReflectionCaptureComponent*> NewCapturesLocal = NewCaptures;

	UReflectionCaptureComponent* RuntimeCaptureToUpdate = nullptr;
	int32 RuntimeCaptureTimesliceFirst = 0;
	int32 RuntimeCaptureTimesliceCount = CubeFace_MAX;

	// Captures that should render all 6 faces this frame (fast render feature).  Items may be added on teleport,
	// level load, or from explicit RefreshCapture(bFastRender=true) requests.
	TSet<UReflectionCaptureComponent*> RuntimeFastRenderSet;

	// Captures that requested a smooth cross-fade refresh this frame (RefreshCapture(bSmoothBlend=true)).  Each entry
	// will get a reserved-tail blend slot allocated on the render thread (if free), the current cubemap copied into it,
	// and the cross-fade clock started when the new render lands.  If no slot is free or the feature is disabled,
	// the refresh pops without blending.
	TArray<UReflectionCaptureComponent*> SmoothBlendQueue;

	static const auto CVarReflectionCaptureRuntime = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ReflectionCapture.Runtime"));
	bool bReflectionCaptureRuntime = CVarReflectionCaptureRuntime ? !!CVarReflectionCaptureRuntime->GetValueOnGameThread() : 0;

	// The editor world keeps ticking in the background while PIE is running.  Skip runtime capture refresh work
	// in the editor scene whenever any PIE world is active, to eliminate the performance hit, and simplify debugging
	// the runtime capture system by preventing breakpoints from also being hit for the editor scene.
	bool bSkipRuntimeCaptureTickForEditor = false;
	if (bIsEditorScene && bInsideTick && GEngine)
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE && Context.World() != nullptr)
			{
				bSkipRuntimeCaptureTickForEditor = true;
				break;
			}
		}
	}

	if (bInsideTick && !bSkipRuntimeCaptureTickForEditor)
	{
		// Reset the tracked time slice component, if we rendered the last of the faces, and we aren't locked.  This handles the case
		// where rendering was locked rendering the last face on the previous frame, but is no longer locked (the capture is locked when
		// GReflectionCaptureRuntimeDebugLock == 0).
		if (ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT >= CubeFace_MAX && GReflectionCaptureRuntimeDebugLock)
		{
			ReflectionSceneData.RuntimeCaptureTimesliceComponent = nullptr;
		}

		// In non-editor scenes, use distance-based priority and respect the runtime capture mode CVar.
		// Editor scenes always use frame-age priority.
		const int32 RuntimeCaptureMode = bIsEditorScene ? 0 : CVarReflectionCaptureRuntimeMode.GetValueOnGameThread();

		// Get camera position for distance-based priority.
		FVector CameraLocation = FVector::ZeroVector;
		bool bUseDistancePriority = false;
		if (!bIsEditorScene)
		{
			if (const UWorld* CaptureWorld = GetWorld())
			{
				if (APlayerController* PC = CaptureWorld->GetFirstPlayerController())
				{
					FRotator CameraRotation;
					PC->GetPlayerViewPoint(CameraLocation, CameraRotation);
					bUseDistancePriority = true;
				}
				else if (CaptureWorld->ViewLocationsRenderedLastFrame.Num() > 0)
				{
					CameraLocation = CaptureWorld->ViewLocationsRenderedLastFrame[0];
					bUseDistancePriority = true;
				}
			}
		}

		// If a runtime capture budget is active, activate/deactivate captures to stay within the limit.
		const int32 RuntimeCaptureBudget = bIsEditorScene ? 0 : CVarReflectionCaptureRuntimeBudget.GetValueOnGameThread();

		if (RuntimeCaptureBudget > 0 && bUseDistancePriority)
		{
			const float BudgetHysteresis = FMath::Max(0.0f, CVarReflectionCaptureRuntimeBudgetHysteresis.GetValueOnGameThread());
			const float FadeInDuration = FMath::Max(0.0f, CVarReflectionCaptureRuntimeFadeInTime.GetValueOnGameThread());
			const double Now = FApp::GetCurrentTime();

			// A single-frame camera jump beyond TeleportDistance, or the first render on scene load, can optionally trigger a
			// fast-render burst and evict out-of-budget captures instantly (no fade) so the budget opens up for the new set.
			// The goal is to have lighting on top of the new camera location to start out initialized, at the cost of a one frame
			// performance hitch.  We defer location triggered fast renders on frames where the camera location is still at its
			// initial value of zero, as this usually is a glitch where the camera has not yet been updated on the first frame
			// or two.  We don't want to waste time doing a bunch of extra renders near the origin.
			int32 AutoFastRenderCount = 0;

			if (!CameraLocation.IsNearlyZero())
			{
				if (ReflectionSceneData.bRuntimeFastRenderHasLastCameraLocation)
				{
					const float TeleportDistance = CVarReflectionCaptureRuntimeTeleportDistance.GetValueOnGameThread();
					const float Jump = (float)FVector::Dist(ReflectionSceneData.RuntimeFastRenderLastCameraLocation, CameraLocation);
					if (TeleportDistance > 0.0f && Jump > TeleportDistance)
					{
						// Fast render on teleport.
						AutoFastRenderCount = CVarReflectionCaptureRuntimeFastRenderOnTeleport.GetValueOnGameThread();
					}
				}

				ReflectionSceneData.RuntimeFastRenderLastCameraLocation = CameraLocation;
				ReflectionSceneData.bRuntimeFastRenderHasLastCameraLocation = true;

				if (ReflectionSceneData.bRuntimeFastRenderPendingLevelLoad)
				{
					// Fast render on level load.
					AutoFastRenderCount = FMath::Max(AutoFastRenderCount, CVarReflectionCaptureRuntimeFastRenderOnLoad.GetValueOnGameThread());
					ReflectionSceneData.bRuntimeFastRenderPendingLevelLoad = false;
				}

				AutoFastRenderCount = FMath::Min(AutoFastRenderCount, RuntimeCaptureBudget);
			}

			struct FBudgetCandidate
			{
				UReflectionCaptureComponent* Component;
				float RawDistance;
				float Penalty;
				float SortKey() const { return RawDistance + Penalty; }
			};
			TArray<FBudgetCandidate> BudgetCandidates;
			BudgetCandidates.Reserve(ReflectionSceneData.RuntimeCaptureBudgetCandidates.Num());

			int32 NumFadingOut = 0;
			for (UReflectionCaptureComponent* Candidate : ReflectionSceneData.RuntimeCaptureBudgetCandidates)
			{
				if (!IsValid(Candidate))
				{
					continue;
				}

				// Signed distance to the influence shape -- camera inside the box/sphere yields a negative key so those
				// captures sort before captures the camera location is outside of.
				const float Dist = ComputeRuntimeBudgetSignedDistance(Candidate, CameraLocation);

				// Penalize captures that are "not visible" for priority purposes: inactive (no proxy) and fading out (proxy exists but
				// headed toward blend 0).  An inactive candidate needs to be closer by the hysteresis amount than a steady active
				// capture to claim its slot.
				const bool bHasProxy = (Candidate->SceneProxy != nullptr);
				const bool bIsFadingOut = bHasProxy && Candidate->IsRuntimeCaptureFadingOut();
				const float Penalty = (!bHasProxy || bIsFadingOut) ? BudgetHysteresis : 0.0f;

				BudgetCandidates.Add({ Candidate, Dist, Penalty });

				if (bIsFadingOut)
				{
					NumFadingOut++;
				}
			}

			BudgetCandidates.Sort([](const FBudgetCandidate& A, const FBudgetCandidate& B) { return A.SortKey() < B.SortKey(); });

			// Remove the hysteresis penalty from the "next best" candidates when one or more components are fading out.
			// Fade out is initiated when an active capture's priority sorts outside the budget, and we want to favor
			// candidates that should displace a fading capture, as if they were already visible and have no penalty.
			if (NumFadingOut && BudgetCandidates.Num() > RuntimeCaptureBudget)
			{
				int32 RemovePenaltyCount = NumFadingOut;
				for (int32 i = 0; i < BudgetCandidates.Num(); i++)
				{
					if (BudgetCandidates[i].Penalty && !BudgetCandidates[i].Component->IsRuntimeCaptureFadingOut())
					{
						BudgetCandidates[i].Penalty = 0.0f;
						RemovePenaltyCount--;

						if (RemovePenaltyCount == 0)
						{
							break;
						}
					}
				}
				BudgetCandidates.Sort([](const FBudgetCandidate& A, const FBudgetCandidate& B) { return A.SortKey() < B.SortKey(); });
			}

			// Check if there are any fast captures in budget.  Done in a separate loop, as fast captures force any out-of-budget captures
			// to be evicted immediately, affecting whether other in-budget captures need to check that all fades are complete before
			// allocating.  Also check for out-of-budget captures that would *start* fading out, as they should be treated as already fading.
			for (int32 i = 0; i < BudgetCandidates.Num(); i++)
			{
				UReflectionCaptureComponent* Comp = BudgetCandidates[i].Component;
				if (i < RuntimeCaptureBudget)
				{
					const bool bInAutoTop = (i < AutoFastRenderCount);
					const bool bUpToDate = Comp->RuntimeLastRenderedFrame && Comp != ReflectionSceneData.RuntimeCaptureTimesliceComponent;

					if ((bInAutoTop && !bUpToDate) || Comp->bFastRenderRequested)
					{
						RuntimeFastRenderSet.Add(Comp);
					}
				}
				else
				{
					if (Comp->SceneProxy && !Comp->IsRuntimeCaptureFadingOut())
					{
						// Capture would be set to start fading out below.  Treat it as fading.
						NumFadingOut++;
					}
				}
			}

			// Process list of budget candidates.
			TArray<UReflectionCaptureComponent::FRuntimeFadeEntry> FadeEntries;

			for (int32 i = 0; i < BudgetCandidates.Num(); i++)
			{
				UReflectionCaptureComponent* Comp = BudgetCandidates[i].Component;
				const bool bInAutoTop = (i < AutoFastRenderCount);
				const bool bUpToDate = Comp->RuntimeLastRenderedFrame && Comp != ReflectionSceneData.RuntimeCaptureTimesliceComponent;
				if (i < RuntimeCaptureBudget)
				{
					// Capture is in budget!
					if (!Comp->SceneProxy)
					{
						// Don't create new proxies if a fade out is active -- we want to wait until the fade out completes, so its
						// budget slot is freed up first.  If there are fast renders, budget slots will all be freed immediately.
						if (NumFadingOut == 0 || !RuntimeFastRenderSet.IsEmpty())
						{
							CreateReflectionCaptureProxy(Comp);
							ReflectionSceneData.AllocatedReflectionCapturesGameThread.Add(Comp);
						}
					}

					if (bInAutoTop || Comp->bFastRenderRequested)
					{
						// Fast render components will be fully rendered this frame, and don't fade in.
						Comp->RuntimeCaptureFadeSet(1.0f, FadeEntries);
					}
					else if (!bUpToDate && !Comp->bRefreshInFlight)
					{
						// Kick off a fade in if we are going to be bringing this item up to date.  This may be called for frames its
						// capture is in flight, but extra calls are harmless, since it won't render until it has valid data anyway.
						Comp->RuntimeCaptureFadeIn(Now, 0.0f, FadeEntries);
					}
					else if (Comp->IsRuntimeCaptureFadingOut())
					{
						// Reversal of an in-flight fade out.  Restart the fade toward 1 from the current partial blend value so the
						// visible blend is continuous.
						const float CurrentBlend = Comp->ComputeCurrentRuntimeCaptureFade(Now, FadeInDuration);
						Comp->RuntimeCaptureFadeIn(Now, CurrentBlend, FadeEntries);
					}
					// else: already fading in or steady -- nothing to do.
				}
				else
				{
					if (Comp->SceneProxy)
					{
						// Destroy the proxy when the fade-out reaches 0, or immediately if any fast renders are queued.
						float CurrentBlend = Comp->ComputeCurrentRuntimeCaptureFade(Now, FadeInDuration);

						if (!RuntimeFastRenderSet.IsEmpty() || CurrentBlend <= 0.0f)
						{
							if (ReflectionSceneData.RuntimeCaptureTimesliceComponent == Comp)
							{
								ReflectionSceneData.RuntimeCaptureTimesliceComponent = nullptr;
							}

							for (auto It = ReflectionSceneData.AllocatedReflectionCapturesGameThread.CreateIterator(); It; ++It)
							{
								if (*It == Comp)
								{
									It.RemoveCurrent();
									break;
								}
							}

							DestroyReflectionCaptureProxy(Comp);
						}
						else if (!Comp->IsRuntimeCaptureFadingOut())
						{
							// Start a fresh fade-out from wherever the capture currently sits (1 for a steady
							// active capture, or a partial value if a fade-in was still in progress).
							Comp->RuntimeCaptureFadeOut(Now, CurrentBlend, FadeEntries);
						}
					}
					// else: inactive candidate that isn't in top-N -- nothing to do.
				}

				// Explicit requests for fast render are consumed, whether the capture is in budget or not, so they don't stay queued indefinitely.
				Comp->bFastRenderRequested = false;
			}

			// Mirror any changed component's current fade state to the render thread.
			if (!FadeEntries.IsEmpty())
			{
				FScene* LocalScene = this;
				ENQUEUE_RENDER_COMMAND(SyncRuntimeCaptureFadeState)(
					[LocalScene, FadeEntries = MoveTemp(FadeEntries)](FRHICommandListBase&)
					{
						for (const UReflectionCaptureComponent::FRuntimeFadeEntry& FadeEntry : FadeEntries)
						{
							if (FCaptureComponentSceneState* State = LocalScene->ReflectionSceneData.AllocatedReflectionCaptureState.Find(FadeEntry.Comp))
							{
								State->FadeStartValue = FadeEntry.StartValue;
								State->FadeTargetValue = FadeEntry.TargetValue;
								State->FadeStartTime = FadeEntry.StartTime;
								LocalScene->ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged = true;
							}
						}
					});
			}
		}

		// Selection pass: forced picks (currently timeslicing, debug viz, manual refresh) always win.
		// Otherwise, pick by distance (runtime) or frame age (editor).
		UReflectionCaptureComponent* ForcedCapture = nullptr;
		UReflectionCaptureComponent* BestCapture = nullptr;
		float BestDistanceSq = FLT_MAX;
		uint32 BestFrameAge = 0;

		for (UReflectionCaptureComponent* Capture : ReflectionSceneData.AllocatedReflectionCapturesGameThread)
		{
			if (Capture && Capture->SceneProxy && !Capture->IsA<UPlaneReflectionCaptureComponent>())
			{
				if (bReflectionCaptureRuntime && Capture->IsRuntimeCapture())
				{
					// Always finish the capture currently being timesliced, or honor debug visualization.
					if (Capture == ReflectionSceneData.RuntimeCaptureTimesliceComponent
						|| GVisualizeTexture.IsNamedView(*Capture->GetOwner()->GetActorNameOrLabel()))
					{
						ForcedCapture = Capture;
					}
					// Manual refresh requested via blueprint takes next priority.
					else if (Capture->bRefreshRequested)
					{
						if (!ForcedCapture)
						{
							ForcedCapture = Capture;
						}
					}
					else
					{
						// In "once" mode, skip captures that have already been rendered.
						if (RuntimeCaptureMode == 1 && Capture->RuntimeLastRenderedFrame != 0)
						{
							continue;
						}

						if (bUseDistancePriority)
						{
							// Distance-based priority: nearest capture updates first.
							const float DistSq = FVector::DistSquared(CameraLocation, Capture->GetComponentLocation());
							if (!BestCapture || DistSq < BestDistanceSq)
							{
								BestCapture = Capture;
								BestDistanceSq = DistSq;
							}
						}
						else
						{
							// Frame-age priority: oldest capture updates first (editor behavior).
							const uint32 TestFrameAge = GFrameNumber - Capture->RuntimeLastRenderedFrame;
							if (!BestCapture || TestFrameAge > BestFrameAge)
							{
								BestCapture = Capture;
								BestFrameAge = TestFrameAge;
							}
						}
					}
				}
				else if (Capture->RuntimeLastRenderedFrame)
				{
					// If the state of this capture changed from runtime to not runtime, it will have a frame set where it last rendered, and we need to update it
					// back to being a baked reflection capture.
					NewCapturesLocal.AddUnique(Capture);
					Capture->RuntimeCaptureResetState();
				}
			}
		}

		RuntimeCaptureToUpdate = ForcedCapture ? ForcedCapture : BestCapture;

		// Optional slow mode that updates timesliced captures every other frame, for further reduced amortized cost.
		if (!bIsEditorScene && (GFrameNumber & 1) &&
			CVarReflectionCaptureRuntimeTimesliceSlow.GetValueOnGameThread() &&
			CVarReflectionCaptureRuntimeTimeslice.GetValueOnGameThread() == 1)
		{
			RuntimeCaptureToUpdate = nullptr;
		}

		// Clear the manual refresh flag now that this capture has been selected.
		if (RuntimeCaptureToUpdate && RuntimeCaptureToUpdate->bRefreshRequested)
		{
			RuntimeCaptureToUpdate->bRefreshRequested = false;
			RuntimeCaptureToUpdate->bRefreshInFlight = true;
		}

		// Consume bSmoothBlendRequested when the refresh actually fires.  Only queue a blend when the feature is enabled
		// and the capture has rendered before -- a freshly-activated capture has no source content to blend from.
		if (RuntimeCaptureToUpdate && RuntimeCaptureToUpdate->bSmoothBlendRequested)
		{
			if (CVarReflectionCaptureRuntimeSmoothBlendSlots.GetValueOnGameThread() > 0 && RuntimeCaptureToUpdate->RuntimeLastRenderedFrame != 0)
			{
				SmoothBlendQueue.Add(RuntimeCaptureToUpdate);
			}
			RuntimeCaptureToUpdate->bSmoothBlendRequested = false;
		}

		// Fast-render override: if any captures were flagged for fast render this frame (level-load burst, teleport, or
		// explicit RefreshCapture), they render all 6 faces here instead of the usual timesliced per-frame update.  Skip
		// the normal single-capture path entirely -- the timesliced rotation resumes on the next frame.  Each fast-render
		// capture needs RuntimeLastRenderedFrame stamped so the prune command allocates a cubemap slot for it, and
		// bRefreshRequested cleared since we're rendering it now.
		if (RuntimeFastRenderSet.Num() > 0)
		{
			const bool bSmoothBlendEnabled = CVarReflectionCaptureRuntimeSmoothBlendSlots.GetValueOnGameThread() > 0;
			for (UReflectionCaptureComponent* FastComp : RuntimeFastRenderSet)
			{
				if (FastComp->bSmoothBlendRequested)
				{
					// Only start a smooth blend if the capture has previously been rendered.  This needs to be checked
					// before RuntimeLastRenderedFrame is written below.
					if (FastComp->RuntimeLastRenderedFrame != 0 && bSmoothBlendEnabled)
					{
						SmoothBlendQueue.Add(FastComp);
					}
					FastComp->bSmoothBlendRequested = false;
				}

				FastComp->RuntimeLastRenderedFrame = GFrameNumber;
				FastComp->bRefreshInFlight = FastComp->bRefreshRequested;
				FastComp->bRefreshRequested = false;
			}

			RuntimeCaptureToUpdate = nullptr;
			ReflectionSceneData.RuntimeCaptureTimesliceComponent = nullptr;
		}

		// Time slicing.
		if (RuntimeCaptureToUpdate)
		{
			RuntimeCaptureToUpdate->RuntimeLastRenderedFrame = GFrameNumber;

			int32 TimesliceCount = FMath::Clamp(bIsEditorScene ? CVarReflectionCaptureRuntimeTimesliceEditor.GetValueOnGameThread() : CVarReflectionCaptureRuntimeTimeslice.GetValueOnGameThread(), 1, 6);
			bool bShadowsOnlyFirstTimeslice = false;

			// Start up the new tracked time slice component, if it changed
			if (ReflectionSceneData.RuntimeCaptureTimesliceComponent != RuntimeCaptureToUpdate)
			{
				ReflectionSceneData.RuntimeCaptureTimesliceComponent = RuntimeCaptureToUpdate;

				// If we are sharing shadow cascades across frames, and only rendering a single timeslice, we do a separate shadows only render as a first timeslice.
				// To indicate this to downstream code, the count of faces rendered is set to zero (instead of TimesliceCount) on this first frame.
				bShadowsOnlyFirstTimeslice = TimesliceCount == 1 && IsSceneCaptureCubeShareCascadesEnabled(GetShaderPlatform());

				RuntimeCaptureTimesliceFirst = 0;
				RuntimeCaptureTimesliceCount = bShadowsOnlyFirstTimeslice ? 0 : TimesliceCount;
				
				ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT = RuntimeCaptureTimesliceCount;

				// Mark runtime capture shadows as not rendered yet for this new capture we are starting, and propagate the face count to the render thread.
				// The face count is primarily used on the render thread to detect that a shadows only render should be done for the current timeslice,
				// when the face count is zero.
				ENQUEUE_RENDER_COMMAND(BeginRuntimeCaptureShadows)([this, FaceCount = ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT](FRHICommandListImmediate& RHICmdList)
				{
					ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame = 0;
					ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFailed = false;
					ReflectionSceneData.RuntimeCaptureShadows.Shadows.Empty();
					ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_RT = FaceCount;
				});

				// If this isn't INDEX_NONE (-1 or any negative for that matter), decrement it each frame -- when it gets to zero, it will lock the capture on the current face.
				if (GReflectionCaptureRuntimeDebugLock > 0)
				{
					GReflectionCaptureRuntimeDebugLock--;
				}
			}
			else
			{
				if (GReflectionCaptureRuntimeDebugLock == 0)
				{
					// Repeat the same timeslices we did last frame when capture is locked.  This is useful for graphical debugging of a specific frame of a multi-frame
					// timesliced render.  Note that RuntimeCaptureTimesliceFaceCount_GT will be zero when the first timeslice is a shadows only render, in which case
					// both RuntimeCaptureTimesliceFirst and RuntimeCaptureTimesliceCount will end up zero here.
					RuntimeCaptureTimesliceFirst = FMath::Max(ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT - TimesliceCount, 0);
					RuntimeCaptureTimesliceCount = ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT - RuntimeCaptureTimesliceFirst;
				}
				else
				{
					RuntimeCaptureTimesliceFirst = ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT;
					RuntimeCaptureTimesliceCount = TimesliceCount;

					ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT += TimesliceCount;

					ENQUEUE_RENDER_COMMAND(UpdateRuntimeCaptureFaceCount)([this, FaceCount = ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT](FRHICommandListImmediate& RHICmdList)
					{
						ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_RT = FaceCount;
					});

					// If this isn't INDEX_NONE (-1 or any negative for that matter), decrement it each frame -- when it gets to zero, it will lock the capture on the current face.
					if (GReflectionCaptureRuntimeDebugLock > 0)
					{
						GReflectionCaptureRuntimeDebugLock--;
					}

					// Reset the tracked time slice component, if we're rendering the last of the faces.  Don't reset if we have the capture locked on the last face.
					if (ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT >= CubeFace_MAX && GReflectionCaptureRuntimeDebugLock)
					{
						ReflectionSceneData.RuntimeCaptureTimesliceComponent = nullptr;
					}
				}
			}
		}
	}

	if (NewCapturesLocal.Num() > 0 || RuntimeCaptureToUpdate || RuntimeFastRenderSet.Num() > 0)
	{
		// Create encoded HDR texture resources if platform requires it
		// This has to be done before capture data is uploaded to GPU and discarded, see UpdateAllReflectionCaptures
		EShaderPlatform Platform = GetShaderPlatform();
		if (UReflectionCaptureComponent::IsEncodedHDRCubemapTextureRequired(Platform))
		{
			for (UReflectionCaptureComponent* Component : NewCapturesLocal)
			{
				if (Component)
				{
					Component->AllocateEncodedHDRCubemapTexture(Platform);
				}
			}
		}
				
		if (SupportsTextureCubeArray(GetFeatureLevel()))
		{
			int32_t PlatformMaxNumReflectionCaptures = FMath::Min(FMath::FloorToInt(GMaxTextureArrayLayers / 6.0f), GetMaxNumReflectionCaptures(GetShaderPlatform()));

			for (int32 CaptureIndex = 0; CaptureIndex < NewCapturesLocal.Num(); CaptureIndex++)
			{
				bool bAlreadyExists = false;

				// Try to find an existing allocation
				for (TSparseArray<UReflectionCaptureComponent*>::TIterator It(ReflectionSceneData.AllocatedReflectionCapturesGameThread); It; ++It)
				{
					UReflectionCaptureComponent* OtherComponent = *It;

					if (OtherComponent == NewCapturesLocal[CaptureIndex])
					{
						bAlreadyExists = true;
					}
				}
				
				// Add the capture to the allocated list
				if (!bAlreadyExists && ReflectionSceneData.AllocatedReflectionCapturesGameThread.Num() < PlatformMaxNumReflectionCaptures)
				{
					ReflectionSceneData.AllocatedReflectionCapturesGameThread.Add(NewCapturesLocal[CaptureIndex]);
				}
			}

			// Request the exact amount and size needed by default.  DesiredMaxCubemaps is the total array size:
			// regular slots (one per active capture) + a fixed reserved tail used as cross-fade source slots for
			// manually-refreshed runtime captures.  The reserved count comes from the CVar, clamped to the hard cap.
			const int32 DesiredNumReserved = FMath::Clamp(CVarReflectionCaptureRuntimeSmoothBlendSlots.GetValueOnGameThread(), 0, GMaxRuntimeReflectionCaptureSmoothBlendSlots);
			int32 DesiredMaxCubemaps = NumUniqueReflectionCaptures(ReflectionSceneData.AllocatedReflectionCapturesGameThread) + DesiredNumReserved;
			int32 DesiredCaptureSize = UReflectionCaptureComponent::GetReflectionCaptureSize();
			int32 ReflectionCaptureSize = DesiredCaptureSize;

			DesiredMaxCubemaps = FMath::Min(DesiredMaxCubemaps, PlatformMaxNumReflectionCaptures);

#if WITH_EDITOR
			// Reduce the capture size (resolution) until we can fit all the captures we need in a single texture cube array resource
			int32 MaxCapturesAtSize;
			for (MaxCapturesAtSize = GetMaxReflectionCapturesAtSize(ReflectionCaptureSize);
				 MaxCapturesAtSize < DesiredMaxCubemaps;
				 MaxCapturesAtSize = GetMaxReflectionCapturesAtSize(ReflectionCaptureSize))
			{
				ReflectionCaptureSize >>= 1;
			}

			if (ReflectionCaptureSize != DesiredCaptureSize)
			{
				UE_LOGF(LogRenderer, Error, "Requested reflection capture cube size of %d with %d elements results in too large a resource for host machine, limiting reflection capture cube size to %d", DesiredCaptureSize, DesiredMaxCubemaps, ReflectionCaptureSize);
			}
#else
			// When not in editor, let the code proceed with the desired size and number, but warn the user an OOM failure is likely and why
			int32 MaxCapturesAtSize = GetMaxReflectionCapturesAtSize(ReflectionCaptureSize);
			if (MaxCapturesAtSize < DesiredMaxCubemaps)
			{
				UE_LOGF(LogRenderer, Error, "Reflection capture of size %d with %d elements exceeds estimated GPU memory limit of %d elements, OOM likely", DesiredCaptureSize, DesiredMaxCubemaps, MaxCapturesAtSize);
				
				// We expect an OOM, but set the limit to the exact number desired to give the best chance of it succeeding anyway
				MaxCapturesAtSize = DesiredMaxCubemaps;
			}
#endif

			if (DesiredMaxCubemaps > 0)
			{
				// In the editor, some captures might have been marked dirty and requested new guids.
				// So remove any probes that can no longer be referenced from the cache.  Also ensure
				// a cubemap slot is allocated for all runtime captures.

				TSet<FGuid> ToKeep;
				TArray<UReflectionCaptureComponent::FRuntimeFadeEntry> RuntimeCaptures;
				ToKeep.Reserve(DesiredMaxCubemaps);
				for (TSparseArray<UReflectionCaptureComponent*>::TIterator It(ReflectionSceneData.AllocatedReflectionCapturesGameThread); It; ++It)
				{
					if (IsValid(*It) && (*It)->SceneProxy)
					{
						if ((*It)->IsRuntimeCapture() && (*It)->RuntimeLastRenderedFrame)
						{
							RuntimeCaptures.Add({ (*It)->RuntimeFade, *It });
						}

						if ((*It)->MapBuildDataId.IsValid())
						{
							ToKeep.Add((*It)->MapBuildDataId);
						}
					}
				}

				FScene* Scene = this;
				ENQUEUE_RENDER_COMMAND(PruneReflectionCaptures)(
					[Scene, ToKeep = MoveTemp(ToKeep), RuntimeCaptures = MoveTemp(RuntimeCaptures)](FRHICommandListImmediate& RHICmdList)
				{
					TArray<int32> ReleasedIndices;
					Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Prune(ToKeep, ReleasedIndices);

					for (int32 Index : ReleasedIndices)
					{
						Scene->ReflectionSceneData.CubemapArraySlotsUsed[Index] = false;
					}

					// Ensure a cubemap slot is allocated for all runtime captures.  This is necessary for undo of deletion,
					// which reregisters an existing component in the undo stack, bypassing the creation logic that normally
					// triggers captures.  This is also the point where a freshly-activated runtime capture first gets a
					// state entry.  It's necessary to mirror the component's game-thread fade state to the render thread
					// here, as the state entry wouldn't have existed.  We still need the fade state sync above, as this
					// code isn't reached in cases that don't trigger a capture (RuntimeCaptureToUpdate == null).
					for (const UReflectionCaptureComponent::FRuntimeFadeEntry& FadeEntry : RuntimeCaptures)
					{
						FindOrAllocateCubemapIndex(Scene, FadeEntry.Comp);

						if (FCaptureComponentSceneState* State = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Find(FadeEntry.Comp))
						{
							// Don't modify the render state if the target value and start time already match, implying the values
							// were already mirrored.  Needs to be conditional, as we don't want to modify StartValue, which
							// is set to TargetValue when the fade completes.
							if (State->FadeTargetValue != FadeEntry.TargetValue || State->FadeStartTime != FadeEntry.StartTime)
							{
								State->FadeStartValue = FadeEntry.StartValue;
								State->FadeTargetValue = FadeEntry.TargetValue;
								State->FadeStartTime = FadeEntry.StartTime;
							}
						}
					}
				});
			}

			// If this is not the first time the scene has allocated the cubemap array, include slack to reduce reallocations
			const float MaxCubemapsRoundUpBase = 1.5f;
			if (ReflectionSceneData.MaxAllocatedReflectionCubemapsGameThread > 0)
			{
				float Exponent = FMath::LogX(MaxCubemapsRoundUpBase, DesiredMaxCubemaps);

				// Round up to the next integer exponent to provide stability and reduce reallocations
				DesiredMaxCubemaps = FMath::Pow(MaxCubemapsRoundUpBase, FMath::TruncToInt(Exponent) + 1);
			}

			// After slack calculation, need to clamp again at our hard limits.
			DesiredMaxCubemaps = FMath::Min3(DesiredMaxCubemaps, PlatformMaxNumReflectionCaptures, MaxCapturesAtSize);

			bool bNeedsUpdateAllCaptures = ReflectionSceneData.DoesAllocatedDataNeedUpdate(DesiredMaxCubemaps, ReflectionCaptureSize);

			if (bNeedsUpdateAllCaptures)
			{
				// If we're not in the editor, we discard the CPU-side reflection capture data after loading to save memory, so we can't resize if the resolution changes. If this happens, we assert
				check(GIsEditor || ReflectionCaptureSize == ReflectionSceneData.CubemapArray.GetCubemapSize() || ReflectionSceneData.CubemapArray.GetCubemapSize() == 0);

				if (ReflectionCaptureSize == ReflectionSceneData.CubemapArray.GetCubemapSize())
				{
					// We can do a fast GPU copy to realloc the array, so we don't need to update all captures
					ReflectionSceneData.SetGameThreadTrackingData(DesiredMaxCubemaps, ReflectionCaptureSize, DesiredCaptureSize);

					FScene* Scene = this;
					uint32 MaxSize = ReflectionSceneData.MaxAllocatedReflectionCubemapsGameThread;
					uint32 NumReserved = (uint32)FMath::Min(DesiredNumReserved, (int32)MaxSize);
					ENQUEUE_RENDER_COMMAND(GPUResizeArrayCommand)(
						[Scene, MaxSize, NumReserved, ReflectionCaptureSize](FRHICommandListImmediate& RHICmdList)
						{
							// Update the scene's cubemap array, preserving the original contents with a GPU-GPU copy
							Scene->ReflectionSceneData.ResizeCubemapArrayGPU(MaxSize, NumReserved, ReflectionCaptureSize);
						});

					bNeedsUpdateAllCaptures = false;
				}
			}

			if (bNeedsUpdateAllCaptures)
			{
				ReflectionSceneData.SetGameThreadTrackingData(DesiredMaxCubemaps, ReflectionCaptureSize, DesiredCaptureSize);

				// Reset runtime capture game thread state after the resize, as all runtime cubemaps have been discarded.
				for (UReflectionCaptureComponent* Candidate : ReflectionSceneData.RuntimeCaptureBudgetCandidates)
				{
					if (IsValid(Candidate))
					{
						Candidate->RuntimeCaptureResetState();
					}
				}
				ReflectionSceneData.RuntimeCaptureTimesliceComponent = nullptr;
				ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_GT = 0;

				FScene* Scene = this;
				uint32 MaxSize = ReflectionSceneData.MaxAllocatedReflectionCubemapsGameThread;
				uint32 NumReserved = (uint32)FMath::Min(DesiredNumReserved, (int32)MaxSize);
				ENQUEUE_RENDER_COMMAND(ResizeArrayCommand)(
					[Scene, MaxSize, NumReserved, ReflectionCaptureSize](FRHICommandListImmediate& RHICmdList)
					{
						// Update the scene's cubemap array, which will reallocate it, so we no longer have the contents of existing entries
						Scene->ReflectionSceneData.CubemapArray.UpdateMaxCubemaps(MaxSize, NumReserved, ReflectionCaptureSize);

						// Also need to reset render thread state, included bRenderedForShading, which suppresses rendering when false,
						// to avoid rendering with uninitialized garbage.
						TArray<FGuid> CachedKeys;
						Scene->ReflectionSceneData.AllocatedReflectionCaptureState.GetKeys(CachedKeys);
						for (const FGuid& Key : CachedKeys)
						{
							if (FCaptureComponentSceneState* State = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Find(Key))
							{
								State->bRenderedForShading = false;
							}
						}
						Scene->ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged = true;
						Scene->ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_RT = 0;

						// UpdateMaxCubemaps reallocates without copying, so any in-flight blend slot's source content is lost; reset.
						Scene->ReflectionSceneData.RuntimeSmoothBlendSlots.Reset();
						Scene->ReflectionSceneData.RuntimeSmoothBlendSlots.SetNum((int32)NumReserved);
					});

				// Recapture all reflection captures now that we have reallocated the cubemap array
				UpdateAllReflectionCaptures(CaptureReason, ReflectionCaptureSize, bVerifyOnlyCapturing, bCapturingForMobile, bInsideTick);
			}
			else if (NewCapturesLocal.Num())
			{
				const int32 NumCapturesForStatus = bVerifyOnlyCapturing ? NewCapturesLocal.Num() : 0;
				BeginReflectionCaptureSlowTask(NumCapturesForStatus, CaptureReason);

				// No teardown of the cubemap array was needed, just update the captures that were requested
				for (int32 CaptureIndex = 0; CaptureIndex < NewCapturesLocal.Num(); CaptureIndex++)
				{
					UReflectionCaptureComponent* CurrentComponent = NewCapturesLocal[CaptureIndex];
					UpdateReflectionCaptureSlowTask(CaptureIndex, NumCapturesForStatus);

					bool bAllocated = false;

					for (TSparseArray<UReflectionCaptureComponent*>::TIterator It(ReflectionSceneData.AllocatedReflectionCapturesGameThread); It; ++It)
					{
						if (*It == CurrentComponent)
						{
							bAllocated = true;
						}
					}

					if (bAllocated)
					{
						CaptureOrUploadReflectionCapture(CurrentComponent, ReflectionCaptureSize, bVerifyOnlyCapturing, bCapturingForMobile, bInsideTick);
					}
				}

				EndReflectionCaptureSlowTask(NumCapturesForStatus);
			}

			// Allocate refresh blend slots and copy old cubemap content into the reserved tail before the new render overwrites
			// the regular slot.  Done as a render command so it sequences after PruneReflectionCaptures (which guarantees a
			// CubemapIndex for the component) and the cubemap-array resize, but before the runtime capture render below.
			if (SmoothBlendQueue.Num() > 0)
			{
				FScene* Scene = this;
				ENQUEUE_RENDER_COMMAND(AllocateRefreshBlendSlots)(
					[Scene, SmoothBlendQueue = MoveTemp(SmoothBlendQueue)](FRHICommandListImmediate& RHICmdList)
					{
						FReflectionEnvironmentSceneData& Data = Scene->ReflectionSceneData;
						if (!Data.CubemapArray.IsValid() || Data.RuntimeSmoothBlendSlots.Num() == 0)
						{
							return;
						}
						FRHITexture* CubeMapArray = Data.CubemapArray.GetRenderTarget()->GetRHI();
						const int32 NewRegularEnd = Data.CubemapArray.GetMaxRegularCubemaps();
						const int32 CubemapSize = Data.CubemapArray.GetCubemapSize();
						const int32 NumMips = FMath::CeilLogTwo(CubemapSize) + 1;

						struct FCopyJob { int32 SrcSlot; int32 DstSlot; };
						TArray<FCopyJob, TInlineAllocator<GMaxRuntimeReflectionCaptureSmoothBlendSlots>> CopyJobs;

						for (UReflectionCaptureComponent* Comp : SmoothBlendQueue)
						{
							const FCaptureComponentSceneState* State = Data.AllocatedReflectionCaptureState.Find(Comp);
							if (!State || State->CubemapIndex < 0)
							{
								continue;  // capture not allocated; pop without blend
							}

							// Multi-refresh: if this component already owns a blend slot from an earlier refresh, free it.
							// The new blend starts from whatever currently sits in the regular slot (the prior render's content),
							// discarding the in-flight cross-fade rather than stacking blends.
							for (FReflectionEnvironmentSceneData::FSmoothBlendEntry& Existing : Data.RuntimeSmoothBlendSlots)
							{
								if (Existing.OwningComponent == Comp)
								{
									Existing = {};
								}
							}

							int32 FreeSlotIdx = INDEX_NONE;
							for (int32 i = 0; i < Data.RuntimeSmoothBlendSlots.Num(); i++)
							{
								if (Data.RuntimeSmoothBlendSlots[i].OwningComponent == nullptr)
								{
									FreeSlotIdx = i;
									break;
								}
							}

							if (FreeSlotIdx == INDEX_NONE)
							{
								continue;  // no slot available; pop without blend
							}

							const int32 ReservedSlotPhysical = NewRegularEnd + FreeSlotIdx;

							FReflectionEnvironmentSceneData::FSmoothBlendEntry& Entry = Data.RuntimeSmoothBlendSlots[FreeSlotIdx];
							Entry.OwningComponent = Comp;
							Entry.SourceCubemapArrayIndex = ReservedSlotPhysical;
							Entry.BlendStartTime = -1.0;  // stamped on render-thread once the new render lands

							CopyJobs.Add({ State->CubemapIndex, ReservedSlotPhysical });
						}

						if (CopyJobs.IsEmpty())
						{
							return;
						}

						SCOPED_DRAW_EVENT(RHICmdList, ReflectionEnvironment_RefreshBlendCopy);

						// Same-texture slice copies aren't expressible as whole-resource transitions, so route each blend
						// through a single-cube intermediate render target: cubemap array[src] -> intermediate -> cubemap array[dst].
						// One intermediate cube is enough -- we process blends serially, releasing it back to the pool at the end.
						const FPooledRenderTargetDesc IntermediateDesc = FPooledRenderTargetDesc::CreateCubemapDesc(
							CubemapSize,
							Data.CubemapArray.GetRenderTarget()->GetDesc().Format,
							FClearValueBinding::None,
							TexCreate_None,
							TexCreate_ShaderResource,
							/*bForceSeparateTargetAndShaderResource*/ false,
							/*ArraySize*/ 1,
							(uint8)NumMips);

						TRefCountPtr<IPooledRenderTarget> Intermediate;
						GRenderTargetPool.FindFreeElement(RHICmdList, IntermediateDesc, Intermediate, TEXT("RefreshBlendIntermediate"));
						FRHITexture* IntermediateTex = Intermediate->GetRHI();

						for (const FCopyJob& Job : CopyJobs)
						{
							// Phase 1: cubemap array -> intermediate (read source cube)
							RHICmdList.Transition({
								FRHITransitionInfo(CubeMapArray, ERHIAccess::Unknown, ERHIAccess::CopySrc),
								FRHITransitionInfo(IntermediateTex, ERHIAccess::Unknown, ERHIAccess::CopyDest)
							});

							for (int32 Face = 0; Face < CubeFace_MAX; Face++)
							{
								FRHICopyTextureInfo CopyInfo;
								CopyInfo.SourceSliceIndex = Job.SrcSlot * CubeFace_MAX + Face;
								CopyInfo.DestSliceIndex   = Face;
								CopyInfo.NumMips          = NumMips;
								RHICmdList.CopyTexture(CubeMapArray, IntermediateTex, CopyInfo);
							}

							// Phase 2: intermediate -> cubemap array (write dest cube)
							RHICmdList.Transition({
								FRHITransitionInfo(CubeMapArray, ERHIAccess::CopySrc, ERHIAccess::CopyDest),
								FRHITransitionInfo(IntermediateTex, ERHIAccess::CopyDest, ERHIAccess::CopySrc)
							});

							for (int32 Face = 0; Face < CubeFace_MAX; Face++)
							{
								FRHICopyTextureInfo CopyInfo;
								CopyInfo.SourceSliceIndex = Face;
								CopyInfo.DestSliceIndex   = Job.DstSlot * CubeFace_MAX + Face;
								CopyInfo.NumMips          = NumMips;
								RHICmdList.CopyTexture(IntermediateTex, CubeMapArray, CopyInfo);
							}
						}

						RHICmdList.Transition({
							FRHITransitionInfo(CubeMapArray, ERHIAccess::CopyDest, ERHIAccess::SRVMask),
							FRHITransitionInfo(IntermediateTex, ERHIAccess::CopySrc, ERHIAccess::SRVMask)
						});
						GRenderTargetPool.FreeUnusedResource(Intermediate);
					});
			}

			if (RuntimeCaptureToUpdate || RuntimeFastRenderSet.Num() > 0)
			{
				if (!ReflectionSceneData.RuntimeCaptureActor)
				{
					FVector Location(0, 0, 0);
					FRotator Rotation(0, 0, 0);
					FActorSpawnParameters SpawnParams;
					SpawnParams.ObjectFlags = RF_Transient;
#if WITH_EDITOR
					SpawnParams.bHideFromSceneOutliner = true;
					SpawnParams.bCreateActorPackage = false;
#endif

					ReflectionSceneData.RuntimeCaptureActor = World->SpawnActor<ASceneCaptureCube>(Location, Rotation, SpawnParams);

					// Can this fail?
					if (ReflectionSceneData.RuntimeCaptureActor)
					{
						ASceneCaptureCube* CaptureActor = ReflectionSceneData.RuntimeCaptureActor.Get();
						USceneCaptureComponentCube* CaptureComponent = CaptureActor->GetCaptureComponentCube();
						CaptureComponent->PrimitiveRenderMode = ESceneCapturePrimitiveRenderMode::PRM_RenderScenePrimitives;
						CaptureComponent->CaptureSource = ESceneCaptureSource::SCS_SceneColorHDRNoAlpha;
						CaptureComponent->bCaptureEveryFrame = false;
						CaptureComponent->bCaptureOnMovement = false;
						CaptureComponent->bAlwaysPersistRenderingState = false;
						CaptureComponent->ShowFlags.GlobalIllumination = false;
						CaptureComponent->ShowFlags.LightFunctions = false;
						CaptureComponent->ShowFlags.LumenReflections = false;
						CaptureComponent->ShowFlags.Particles = false;
						CaptureComponent->ShowFlags.ReflectionEnvironment = false;			// Disable the reflection environment, to avoid feedback
						CaptureComponent->ShowFlags.ScreenSpaceReflections = false;
						CaptureComponent->ShowFlags.SetCompositeEditorPrimitives(false);
						CaptureComponent->ShowFlags.SkeletalMeshes = false;
						CaptureComponent->ShowFlags.DisableOcclusionQueries = true;			// Timesliced one cube face at a time, occlusion queries aren't useful
						CaptureComponent->bCaptureRotation = false;
					}
				}

				if (ReflectionSceneData.RuntimeCaptureActor)
				{
					USceneCaptureComponentCube* SceneCaptureComponent = ReflectionSceneData.RuntimeCaptureActor->GetCaptureComponentCube();

					// Build the render list for this frame.  In the normal path there's a single entry: the
					// timesliced RuntimeCaptureToUpdate.  In the fast-render path, every capture in the set is
					// rendered with TimesliceFirst=0 / Count=6 so all 6 faces complete in one frame each.
					struct FRenderEntry { UReflectionCaptureComponent* Component; int32 TimesliceFirst; int32 TimesliceCount; };
					TArray<FRenderEntry, TInlineAllocator<4>> RenderList;
					if (RuntimeFastRenderSet.Num() > 0)
					{
						for (UReflectionCaptureComponent* FastComp : RuntimeFastRenderSet)
						{
							RenderList.Add({ FastComp, 0, CubeFace_MAX });
						}
					}
					else
					{
						RenderList.Add({ RuntimeCaptureToUpdate, RuntimeCaptureTimesliceFirst, RuntimeCaptureTimesliceCount });
					}

					// Cache these once -- they don't depend on the capture being rendered, so we don't need to
					// re-read the CVars on every loop iteration.
					const bool bRuntimeFoliage = CVarReflectionCaptureRuntimeFoliage.GetValueOnGameThread();
					const bool bRuntimeDFShadows = CVarReflectionCaptureRuntimeDFShadows.GetValueOnGameThread();
					const bool bRuntimeTranslucency = CVarReflectionCaptureRuntimeTranslucency.GetValueOnGameThread();

					for (const FRenderEntry& Entry : RenderList)
					{
						UReflectionCaptureComponent* RenderComp = Entry.Component;
						SceneCaptureComponent->ReflectionComponent = RenderComp;
						SceneCaptureComponent->ReflectionTimeSliceFirst = Entry.TimesliceFirst;
						SceneCaptureComponent->ReflectionTimeSliceCount = Entry.TimesliceCount;
						SceneCaptureComponent->SkylightScale = RenderComp->RuntimeSkylightScale;
						SceneCaptureComponent->MaxViewDistanceOverride = RenderComp->MaxViewDistance > 0.0f ? RenderComp->MaxViewDistance : -1.0f;
						SceneCaptureComponent->bFiniteFarPlane = RenderComp->bFiniteFarPlane;

						// Propagate post process material to the scene capture's PostProcessSettings
						SceneCaptureComponent->PostProcessSettings.WeightedBlendables.Array.Reset();
						SceneCaptureComponent->PostProcessBlendWeight = 0.0f;

						if (RenderComp->PostProcessMaterial)
						{
							UMaterialInterface* PostMaterial = RenderComp->PostProcessMaterial;
							UMaterial* BaseMaterial = PostMaterial->GetMaterial();

							if (BaseMaterial->IsPostProcessMaterial() &&
								PostMaterial->GetUserSceneTextureOutput(BaseMaterial).IsNone() &&
								PostMaterial->GetBlendableLocation(BaseMaterial) == BL_SceneColorBeforeBloom)
							{
								FWeightedBlendable& Blendable = SceneCaptureComponent->PostProcessSettings.WeightedBlendables.Array.AddDefaulted_GetRef();
								Blendable.Weight = 1.0f;
								Blendable.Object = PostMaterial;
								SceneCaptureComponent->PostProcessBlendWeight = 1.0f;
							}
						}

						// Set these every frame, to allow for the CVars to be changed at runtime
						SceneCaptureComponent->ShowFlags.InstancedFoliage = bRuntimeFoliage;
						SceneCaptureComponent->ShowFlags.InstancedGrass = bRuntimeFoliage;
						SceneCaptureComponent->ShowFlags.RayTracedDistanceFieldShadows = bRuntimeDFShadows;
						SceneCaptureComponent->ShowFlags.Translucency = bRuntimeTranslucency;

						ReflectionSceneData.RuntimeCaptureActor->SetActorLocation(RenderComp->GetOwner()->GetActorLocation());

						// Each fast-render capture starts from a clean shadow atlas / face-count state so it
						// renders all 6 faces in one go without inheriting the previous capture's partial state.
						if (RuntimeFastRenderSet.Num() > 0)
						{
							FScene* SceneForShadowReset = this;
							ENQUEUE_RENDER_COMMAND(BeginFastRenderCapture)(
								[SceneForShadowReset](FRHICommandListImmediate&)
								{
									SceneForShadowReset->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame = 0;
									SceneForShadowReset->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFailed = false;
									SceneForShadowReset->ReflectionSceneData.RuntimeCaptureShadows.Shadows.Empty();
									SceneForShadowReset->ReflectionSceneData.RuntimeCaptureTimesliceFaceCount_RT = CubeFace_MAX;
								});
						}

						SceneCaptureComponent->CaptureScene();
					}

					// Stamp BlendStartTime for any in-flight refresh blend whose render lands this frame.  For fast renders
					// every face is in this frame; for timesliced renders, completion is the frame whose timeslice ends at
					// face CubeFace_MAX.  Until BlendStartTime is positive the shader keeps the source at 100%.
					TArray<UReflectionCaptureComponent*, TInlineAllocator<4>> CompletedRefreshComponents;
					for (const FRenderEntry& Entry : RenderList)
					{
						if (Entry.TimesliceFirst + Entry.TimesliceCount >= CubeFace_MAX)
						{
							CompletedRefreshComponents.Add(Entry.Component);
							Entry.Component->bRefreshInFlight = false;
						}
					}

					if (CompletedRefreshComponents.Num() > 0)
					{
						FScene* Scene = this;
						const double Now = FApp::GetCurrentTime();
						ENQUEUE_RENDER_COMMAND(StampRefreshBlendStartTime)(
							[Scene, CompletedComponents = TArray<UReflectionCaptureComponent*>(CompletedRefreshComponents), Now](FRHICommandListBase&)
							{
								FReflectionEnvironmentSceneData& Data = Scene->ReflectionSceneData;
								for (UReflectionCaptureComponent* Comp : CompletedComponents)
								{
									for (FReflectionEnvironmentSceneData::FSmoothBlendEntry& Entry : Data.RuntimeSmoothBlendSlots)
									{
										if (Entry.OwningComponent == Comp && Entry.BlendStartTime < 0.0)
										{
											Entry.BlendStartTime = Now;
										}
									}
								}
							});
					}
				}
			}
		}

		for (int32 CaptureIndex = 0; CaptureIndex < NewCapturesLocal.Num(); CaptureIndex++)
		{
			UReflectionCaptureComponent* Component = NewCapturesLocal[CaptureIndex];

			Component->SetCaptureCompleted();

			if (Component->SceneProxy)
			{
				// Update the transform of the reflection capture
				// This is not done earlier by the reflection capture when it detects that it is dirty,
				// To ensure that the RT sees both the new transform and the new contents on the same frame.
				Component->SendRenderTransform_Concurrent();
			}
		}
	}
}

/** Updates the contents of all reflection captures in the scene.  Must be called from the game thread. */
void FScene::UpdateAllReflectionCaptures(const TCHAR* CaptureReason, int32 ReflectionCaptureSize, bool bVerifyOnlyCapturing, bool bCapturingForMobile, bool bInsideTick)
{
	if (IsReflectionEnvironmentAvailable(GetFeatureLevel()))
	{
		FScene* Scene = this;
		ENQUEUE_RENDER_COMMAND(CaptureCommand)(
			[Scene](FRHICommandListImmediate& RHICmdList)
			{
				Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Empty();
				Scene->ReflectionSceneData.CubemapArraySlotsUsed.Reset();
			});

		// Only display status during building reflection captures, otherwise we may interrupt a editor widget manipulation of many captures
		const int32 NumCapturesForStatus = bVerifyOnlyCapturing ? ReflectionSceneData.AllocatedReflectionCapturesGameThread.Num() : 0;
		BeginReflectionCaptureSlowTask(NumCapturesForStatus, CaptureReason);

		int32 CaptureIndex = 0;

		for (TSparseArray<UReflectionCaptureComponent*>::TIterator It(ReflectionSceneData.AllocatedReflectionCapturesGameThread); It; ++It)
		{
			UpdateReflectionCaptureSlowTask(CaptureIndex, NumCapturesForStatus);

			CaptureIndex++;
			UReflectionCaptureComponent* CurrentComponent = *It;
			CaptureOrUploadReflectionCapture(CurrentComponent, ReflectionCaptureSize, bVerifyOnlyCapturing, bCapturingForMobile, bInsideTick);
		}

		EndReflectionCaptureSlowTask(NumCapturesForStatus);
	}
}

void FScene::ResetReflectionCaptures(bool bOnlyIfOOM)
{
	if (bOnlyIfOOM == false || ReflectionSceneData.ReflectionCaptureSizeGameThread != ReflectionSceneData.DesiredReflectionCaptureSizeGameThread)
	{
		ReflectionSceneData.Reset(this);
	}
}

void GetReflectionCaptureData_RenderingThread(FRHICommandListImmediate& RHICmdList, FScene* Scene, const UReflectionCaptureComponent* Component, FReflectionCaptureData* OutCaptureData)
{
	const FCaptureComponentSceneState* ComponentStatePtr = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Find(Component);

	if (ComponentStatePtr)
	{
		FRHITexture* EffectiveDest = Scene->ReflectionSceneData.CubemapArray.GetRenderTarget()->GetRHI();

		const int32 CubemapIndex = ComponentStatePtr->CubemapIndex;
		const int32 NumMips = EffectiveDest->GetNumMips();
		const int32 EffectiveTopMipSize = FMath::Pow(2.f, NumMips - 1);

		int32 CaptureDataSize = 0;

		for (int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
		{
			const int32 MipSize = 1 << (NumMips - MipIndex - 1);

			for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
			{
				CaptureDataSize += MipSize * MipSize * sizeof(FFloat16Color);
			}
		}

		OutCaptureData->FullHDRCapturedData.Empty(CaptureDataSize);
		OutCaptureData->FullHDRCapturedData.AddZeroed(CaptureDataSize);
		int32 MipBaseIndex = 0;

		for (int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
		{
			check(EffectiveDest->GetFormat() == PF_FloatRGBA);
			const int32 MipSize = 1 << (NumMips - MipIndex - 1);
			const int32 CubeFaceBytes = MipSize * MipSize * sizeof(FFloat16Color);

			for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
			{
				TArray<FFloat16Color> SurfaceData;
				// Read each mip face
				//@todo - do this without blocking the GPU so many times
				//@todo - pool the temporary textures in RHIReadSurfaceFloatData instead of always creating new ones
				RHICmdList.ReadSurfaceFloatData(EffectiveDest, FIntRect(0, 0, MipSize, MipSize), SurfaceData, (ECubeFace)CubeFace, CubemapIndex, MipIndex);
				const int32 DestIndex = MipBaseIndex + CubeFace * CubeFaceBytes;
				uint8* FaceData = &OutCaptureData->FullHDRCapturedData[DestIndex];
				check(SurfaceData.Num() * SurfaceData.GetTypeSize() == CubeFaceBytes);
				FMemory::Memcpy(FaceData, SurfaceData.GetData(), CubeFaceBytes);
			}

			MipBaseIndex += CubeFaceBytes * CubeFace_MAX;
		}

		OutCaptureData->CubemapSize = EffectiveTopMipSize;

		OutCaptureData->AverageBrightness = ComponentStatePtr->AverageBrightness;
	}
}

void FScene::GetReflectionCaptureData(UReflectionCaptureComponent* Component, FReflectionCaptureData& OutCaptureData) 
{
	check(GetFeatureLevel() >= ERHIFeatureLevel::SM5);

	FScene* Scene = this;
	FReflectionCaptureData* OutCaptureDataPtr = &OutCaptureData;
	ENQUEUE_RENDER_COMMAND(GetReflectionDataCommand)(
		[Scene, Component, OutCaptureDataPtr](FRHICommandListImmediate& RHICmdList)
		{
			GetReflectionCaptureData_RenderingThread(RHICmdList, Scene, Component, OutCaptureDataPtr);
		});

	// Necessary since the RT is writing to OutDerivedData directly
	FlushRenderingCommands();
}

void UploadReflectionCapture_RenderingThread(FRHICommandListImmediate& RHICmdList, FScene* Scene, const FReflectionCaptureData* CaptureData, const UReflectionCaptureComponent* CaptureComponent)
{
	// Due to memory limitations, it's possible for the in-memory size to be smaller than the originally captured size.
	const int32 EffectiveTopMipSize = Scene->ReflectionSceneData.CubemapArray.GetCubemapSize();
	const int32 NumMipsSource = FMath::CeilLogTwo(CaptureData->CubemapSize) + 1;
	const int32 NumMipsDest = FMath::CeilLogTwo(EffectiveTopMipSize) + 1;

	const int32 CaptureIndex = FindOrAllocateCubemapIndex(Scene, CaptureComponent);
	check(CaptureData->CubemapSize >= Scene->ReflectionSceneData.CubemapArray.GetCubemapSize());
	check(CaptureIndex < Scene->ReflectionSceneData.CubemapArray.GetMaxRegularCubemaps());
	FRHITexture* CubeMapArray = Scene->ReflectionSceneData.CubemapArray.GetRenderTarget()->GetRHI();
	check(CubeMapArray->GetFormat() == PF_FloatRGBA);

	int32 MipBaseIndex = 0;

	// Skip over mips in originally captured data, based on what we can fit in memory.
	for (int32 MipIndex = 0; MipIndex < NumMipsSource - NumMipsDest; MipIndex++)
	{
		const int32 MipSize = 1 << (NumMipsSource - MipIndex - 1);
		const int32 CubeFaceBytes = MipSize * MipSize * sizeof(FFloat16Color);

		MipBaseIndex += CubeFaceBytes * CubeFace_MAX;
	}

	for (int32 MipIndex = 0; MipIndex < NumMipsDest; MipIndex++)
	{
		const int32 MipSize = 1 << (NumMipsDest - MipIndex - 1);
		const int32 CubeFaceBytes = MipSize * MipSize * sizeof(FFloat16Color);

		for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
		{
			uint32 DestStride = 0;
			uint8* DestBuffer = (uint8*)RHICmdList.LockTextureCubeFace(CubeMapArray, CubeFace, CaptureIndex, MipIndex, RLM_WriteOnly, DestStride, false);

			// Handle DestStride by copying each row
			for (int32 Y = 0; Y < MipSize; Y++)
			{
				FFloat16Color* DestPtr = (FFloat16Color*)((uint8*)DestBuffer + Y * DestStride);
				const int32 SourceIndex = MipBaseIndex + CubeFace * CubeFaceBytes + Y * MipSize * sizeof(FFloat16Color);
				const uint8* SourcePtr = &CaptureData->FullHDRCapturedData[SourceIndex];
				FMemory::Memcpy(DestPtr, SourcePtr, MipSize * sizeof(FFloat16Color));
			}

			RHICmdList.UnlockTextureCubeFace(CubeMapArray, CubeFace, CaptureIndex, MipIndex, false);
		}

		MipBaseIndex += CubeFaceBytes * CubeFace_MAX;
	}

	FCaptureComponentSceneState& FoundState = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.FindChecked(CaptureComponent);
	FoundState.AverageBrightness = CaptureData->AverageBrightness;

	// Baked data has been uploaded into the cubemap slot; the capture is now valid for shading.
	if (!FoundState.bRenderedForShading)
	{
		FoundState.bRenderedForShading = true;
		Scene->ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged = true;
	}
}

/** Creates a transformation for a cubemap face, following the D3D cubemap layout. */
FMatrix CalcCubeFaceViewRotationMatrix(ECubeFace Face)
{
	FMatrix Result(FMatrix::Identity);

	static const FVector XAxis(1.f,0.f,0.f);
	static const FVector YAxis(0.f,1.f,0.f);
	static const FVector ZAxis(0.f,0.f,1.f);

	// vectors we will need for our basis
	FVector vUp(YAxis);
	FVector vDir;

	switch( Face )
	{
	case CubeFace_PosX:
		vDir = XAxis;
		break;
	case CubeFace_NegX:
		vDir = -XAxis;
		break;
	case CubeFace_PosY:
		vUp = -ZAxis;
		vDir = YAxis;
		break;
	case CubeFace_NegY:
		vUp = ZAxis;
		vDir = -YAxis;
		break;
	case CubeFace_PosZ:
		vDir = ZAxis;
		break;
	case CubeFace_NegZ:
		vDir = -ZAxis;
		break;
	}

	// derive right vector
	FVector vRight( vUp ^ vDir );
	// create matrix from the 3 axes
	Result = FBasisVectorMatrix( vRight, vUp, vDir, FVector::ZeroVector );	

	return Result;
}

FMatrix GetCubeProjectionMatrix(float HalfFovDeg, float CubeMapSize, float NearPlane)
{
	return FReversedZPerspectiveMatrix(HalfFovDeg * float(PI) / 180.0f, CubeMapSize, CubeMapSize, NearPlane);
}

void CaptureSceneIntoScratchCubemap(
	FScene* Scene, 
	const FReflectionCubemapTexture& ReflectionCubemapTexture,
	FVector CapturePosition,
	int32 CubemapSize,
	bool bCapturingForSkyLight,
	bool bStaticSceneOnly, 
	float SkyLightNearPlane,
	bool bLowerHemisphereIsBlack, 
	bool bCaptureEmissiveOnly,
	const FLinearColor& LowerHemisphereColor,
	bool bCapturingForMobile,
	bool bInsideTick
	)
{
	int32 SupersampleCaptureFactor = FMath::Clamp(GSupersampleCaptureFactor, MinSupersampleCaptureFactor, MaxSupersampleCaptureFactor);

	class FDummyRenderTarget final : public FRenderTarget, public FRenderThreadStructBase
	{
	public:
		FDummyRenderTarget() = default;

		const FTextureRHIRef& GetRenderTargetTexture() const override
		{
			static FTextureRHIRef DummyTexture;
			return DummyTexture;
		}

		void SetSize(int32 TargetSize) { Size = TargetSize; }
		FIntPoint GetSizeXY() const override { return FIntPoint(Size, Size); }
		float GetDisplayGamma() const override { return 1.0f; }

	private:
		int32 Size = 0;
	};

	TRenderThreadStruct<FDummyRenderTarget> DummyRenderTarget;

	if (!bCapturingForSkyLight && !bInsideTick)
	{
		ENQUEUE_RENDER_COMMAND(BeginCubemapCapture)([] (FRHICommandListImmediate&)
		{
			GFrameNumberRenderThread++;
		});
	}

	FSceneRenderBuilder SceneRenderBuilder(Scene);

	ON_SCOPE_EXIT
	{
		SceneRenderBuilder.Execute();

		if (!bCapturingForSkyLight && !bInsideTick)
		{
			ENQUEUE_RENDER_COMMAND(EndCubemapCapture)([] (FRHICommandListImmediate& RHICmdList)
			{
				RHICmdList.EndFrame();
			});
		}
	};

	SCENE_RENDER_GROUP_SCOPE(SceneRenderBuilder, TEXT("CubemapCapture"), ESceneRenderGroupFlags::None);

	for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
	{
		DummyRenderTarget->SetSize(CubemapSize);

		auto ViewFamilyInit = FSceneViewFamily::ConstructionValues(
			DummyRenderTarget.Get(),
			Scene,
			FEngineShowFlags(ESFIM_Game)
			)
			.SetResolveScene(false);

		if( bStaticSceneOnly )
		{
			ViewFamilyInit.SetTime(FGameTime());
		}

		FSceneViewFamilyContext ViewFamily( ViewFamilyInit );

		// Disable features that are not desired when capturing the scene
		ViewFamily.EngineShowFlags.PostProcessing = 0;
		ViewFamily.EngineShowFlags.MotionBlur = 0;
		ViewFamily.EngineShowFlags.SetOnScreenDebug(false);
		ViewFamily.EngineShowFlags.HMDDistortion = 0;
		// Conditionally exclude particles and light functions as they are usually dynamic, and can't be captured well
		ViewFamily.EngineShowFlags.Particles = 0;
		ViewFamily.EngineShowFlags.LightFunctions = abs(GReflectionCaptureEnableLightFunctions) ? 1 : 0;
		ViewFamily.EngineShowFlags.SetCompositeEditorPrimitives(false);
		// These are highly dynamic and can't be captured effectively
		ViewFamily.EngineShowFlags.LightShafts = 0;
		// Don't apply sky lighting diffuse when capturing the sky light source, or we would have feedback
		ViewFamily.EngineShowFlags.SkyLighting = !bCapturingForSkyLight;
		// Skip lighting for emissive only
		ViewFamily.EngineShowFlags.Lighting = !bCaptureEmissiveOnly;
		// Never do screen percentage in reflection environment capture.
		ViewFamily.EngineShowFlags.ScreenPercentage = false;

		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = &ViewFamily;
		ViewInitOptions.BackgroundColor = FLinearColor::Black;
		ViewInitOptions.OverlayColor = FLinearColor::Black;
		ViewInitOptions.SetViewRectangle(FIntRect(0, 0, CubemapSize * SupersampleCaptureFactor, CubemapSize * SupersampleCaptureFactor));

		const float NearPlane = bCapturingForSkyLight ? SkyLightNearPlane : GReflectionCaptureNearPlane;

		// Projection matrix based on the fov, near / far clip settings
		// Each face always uses a 90 degree field of view
		ViewInitOptions.ProjectionMatrix = GetCubeProjectionMatrix(45.0f, (float)CubemapSize * SupersampleCaptureFactor, NearPlane);

		ViewInitOptions.ViewOrigin = CapturePosition;
		ViewInitOptions.ViewRotationMatrix = CalcCubeFaceViewRotationMatrix((ECubeFace)CubeFace);
		ViewInitOptions.bIsReflectionCapture = true;

		FSceneView* View = new FSceneView(ViewInitOptions);

		// Force all surfaces diffuse
		View->RoughnessOverrideParameter = FVector2f(1.0f, 0.0f);

		if (bCaptureEmissiveOnly)
		{
			View->DiffuseOverrideParameter = FVector4f(0, 0, 0, 0);
			View->SpecularOverrideParameter = FVector4f(0, 0, 0, 0);
		}

		View->bStaticSceneOnly = bStaticSceneOnly;
		View->StartFinalPostprocessSettings(CapturePosition);
		View->EndFinalPostprocessSettings(ViewInitOptions);

		if (bCapturingForSkyLight)
		{
			const float SkylightCaptureLODDistanceScale = GSkylightCaptureLODDistanceScale > 0.f ? GSkylightCaptureLODDistanceScale : 1.f;
			View->LODDistanceFactor *= SkylightCaptureLODDistanceScale;
		}
		ViewFamily.Views.Add(View);

		ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
			ViewFamily, /* GlobalResolutionFraction = */ 1.0f));

		FSceneViewExtensionContext ViewExtensionContext(Scene);
		ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(ViewExtensionContext);
		for (const FSceneViewExtensionRef& Extension : ViewFamily.ViewExtensions)
		{
			Extension->SetupViewFamily(ViewFamily);
			Extension->SetupView(ViewFamily, *View);
		}

		FSceneRenderer* SceneRenderer = SceneRenderBuilder.CreateSceneRenderer(&ViewFamily);

		SceneRenderBuilder.AddRenderer(SceneRenderer,
			[
				  &ReflectionCubemapTexture
				, CubeFace
				, CubemapSize
				, bCapturingForSkyLight
				, bLowerHemisphereIsBlack
				, LowerHemisphereColor
				, bCapturingForMobile
				, bInsideTick
			] (FRDGBuilder& GraphBuilder, const FSceneRenderFunctionInputs& Inputs)
		{
			CaptureSceneToScratchCubemap(GraphBuilder, Inputs.Renderer, Inputs.SceneUpdateInputs, ReflectionCubemapTexture, (ECubeFace)CubeFace, CubemapSize, bCapturingForSkyLight, bLowerHemisphereIsBlack, LowerHemisphereColor, bCapturingForMobile);
			return true;
		});
	}
}

void CopyToSceneArray(FRDGBuilder& GraphBuilder, FScene* Scene, FRDGTexture* FilteredCubeTexture, FReflectionCaptureProxy* ReflectionProxy, int32 CaptureIndex)
{
	RDG_EVENT_SCOPE(GraphBuilder, "CopyToSceneArray");

	const int32 NumMips = GetNumMips(Scene->ReflectionSceneData.CubemapArray.GetCubemapSize());

	FRDGTexture* DestCubeTexture = GraphBuilder.RegisterExternalTexture(Scene->ReflectionSceneData.CubemapArray.GetRenderTarget());

	// GPU copy back to the scene's texture array, which is not a render target
	for (int32 MipIndex = 0; MipIndex < NumMips; MipIndex++)
	{
		for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
		{
			FRHICopyTextureInfo CopyInfo;
			CopyInfo.SourceMipIndex   = MipIndex;
			CopyInfo.DestMipIndex     = MipIndex;
			CopyInfo.SourceSliceIndex = CubeFace;
			CopyInfo.DestSliceIndex   = CaptureIndex * CubeFace_MAX + CubeFace;

			AddCopyTexturePass(GraphBuilder, FilteredCubeTexture, DestCubeTexture, CopyInfo);
		}
	}
}

/** 
 * Updates the contents of the given reflection capture by rendering the scene. 
 * This must be called on the game thread.
 */
void FScene::CaptureOrUploadReflectionCapture(UReflectionCaptureComponent* CaptureComponent, int32 ReflectionCaptureSize, bool bVerifyOnlyCapturing, bool bCapturingForMobile, bool bInsideTick)
{
	// Runtime captures render their cubemap from scratch, so skip upload or render of a baked version of the same capture.
	// Avoids wasted work in the editor, and prevents a glitch of baked or uninitialized data that would occur before the runtime
	// render replaces it.
	if (CaptureComponent->IsRuntimeCapture())
	{
		return;
	}

	if (IsReflectionEnvironmentAvailable(GetFeatureLevel()))
	{
		FReflectionCaptureData* CaptureData = CaptureComponent->GetMapBuildData();

		// Upload existing derived data if it exists, instead of capturing
		if (CaptureData)
		{
			// Safety check during the reflection capture build, there should not be any map build data
			ensure(!bVerifyOnlyCapturing);

			check(SupportsTextureCubeArray(GetFeatureLevel()));

			FScene* Scene = this;

			ENQUEUE_RENDER_COMMAND(UploadCaptureCommand)
				([Scene, CaptureData, CaptureComponent](FRHICommandListImmediate& RHICmdList)
			{
				// After the final upload we cannot upload again because we tossed the source MapBuildData,
				// After uploading it into the scene's texture array, to guaratee there's only one copy in memory.
				// This means switching between LightingScenarios only works if the scenario level is reloaded (not simply made hidden / visible again)
				if (!CaptureData->HasBeenUploadedFinal())
				{
					UploadReflectionCapture_RenderingThread(RHICmdList, Scene, CaptureData, CaptureComponent);

					CaptureData->OnDataUploadedToGPUFinal();
				}
				else
				{
					const FCaptureComponentSceneState* CaptureSceneStatePtr = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Find(CaptureComponent);
					
					if (!CaptureSceneStatePtr)
					{
						ensureMsgf(CaptureSceneStatePtr, TEXT("Reflection capture %s uploaded twice without reloading its lighting scenario level.  The Lighting scenario level must be loaded once for each time the reflection capture is uploaded."), *CaptureComponent->GetPathName());
					}
				}
			});
		}
		// Capturing only supported in the editor.  Game can only use built reflection captures.
		else if (bIsEditorScene)
		{
			if (CaptureComponent->ReflectionSourceType == EReflectionSourceType::SpecifiedCubemap && !CaptureComponent->Cubemap)
			{
				return;
			}

			if (FPlatformProperties::RequiresCookedData())
			{
				UE_LOGF(LogRenderer, Warning, "No built data for %ls, skipping generation in cooked build.", *CaptureComponent->GetPathName());
				return;
			}

			UE::RenderCommandPipe::FSyncScope SyncScope;

			// Prefetch all virtual textures so that we have content available
			if (UseVirtualTexturing(GetShaderPlatform()))
			{
				const ERHIFeatureLevel::Type InFeatureLevel = FeatureLevel;
				const FVector2D ScreenSpaceSize(ReflectionCaptureSize, ReflectionCaptureSize);

				ENQUEUE_RENDER_COMMAND(LoadTiles)(
					[InFeatureLevel, ScreenSpaceSize](FRHICommandListImmediate& RHICmdList)
				{
					GetRendererModule().RequestVirtualTextureTiles(ScreenSpaceSize, -1);
					GetRendererModule().LoadPendingVirtualTextureTiles(RHICmdList, InFeatureLevel);
				});

				FlushRenderingCommands();
			}

			TRenderThreadStruct<FReflectionCubemapTexture> ReflectionCubemapTexture(ReflectionCaptureSize);

			if (CaptureComponent->ReflectionSourceType == EReflectionSourceType::CapturedScene)
			{
				const bool bAllowStaticLighting = IsStaticLightingAllowed();

				// Reflection Captures are a form of static lighting, so only capture scene elements that are static
				// However if the project has static lighting disabled, Reflection Captures can still be made to work by capturing Movable lights
				bool const bCaptureStaticSceneOnly = CVarReflectionCaptureStaticSceneOnly.GetValueOnGameThread() != 0 && bAllowStaticLighting;
				CaptureSceneIntoScratchCubemap(this, *ReflectionCubemapTexture, CaptureComponent->GetComponentLocation() + CaptureComponent->CaptureOffset, ReflectionCaptureSize, false, bCaptureStaticSceneOnly, 0, false, false, FLinearColor(), bCapturingForMobile, bInsideTick);
			}
			else if (CaptureComponent->ReflectionSourceType == EReflectionSourceType::SpecifiedCubemap)
			{
				UTextureCube* SourceCubemap = CaptureComponent->Cubemap;
				float SourceCubemapRotation = CaptureComponent->SourceCubemapAngle * (PI / 180.f);
				ENQUEUE_RENDER_COMMAND(CopyCubemapCommand)(
					[FeatureLevel = FeatureLevel, SourceCubemap, ReflectionCubemapTexture = ReflectionCubemapTexture.Get(), ReflectionCaptureSize, SourceCubemapRotation](FRHICommandListImmediate& RHICmdList)
				{
					CopyCubemapToScratchCubemap(RHICmdList, FeatureLevel, SourceCubemap, *ReflectionCubemapTexture, ReflectionCaptureSize, false, false, SourceCubemapRotation, FLinearColor(), FLinearColor::White);
				});
			}
			else
			{
				check(!TEXT("Unknown reflection source type"));
			}

			// Create a proxy to represent the reflection capture to the rendering thread
			// The rendering thread will be responsible for deleting this when done with the filtering operation
			// We can't use the component's SceneProxy here because the component may not be registered with the scene
			FReflectionCaptureProxy* ReflectionProxy = new FReflectionCaptureProxy(CaptureComponent);

			ENQUEUE_RENDER_COMMAND(FilterCommand)(
				[Scene = this, FeatureLevel = FeatureLevel, ReflectionCubemapTexture = ReflectionCubemapTexture.Get(), ReflectionCaptureSize, CaptureComponent, ReflectionProxy](FRHICommandListImmediate& RHICmdList)
			{
				const int32 CubemapIndex = FindOrAllocateCubemapIndex(Scene, CaptureComponent);
				FCaptureComponentSceneState& FoundState = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.FindChecked(CaptureComponent);

				FRDGBuilder GraphBuilder(RHICmdList);

				auto* ShaderMap = GetGlobalShaderMap(FeatureLevel);

				FRDGTexture* SceneCubemapTexture = ReflectionCubemapTexture->GetRDG(GraphBuilder);

				ComputeAverageBrightness(GraphBuilder, ShaderMap, SceneCubemapTexture, &FoundState.AverageBrightness);

				FRDGTexture* FilteredSceneCubemapTexture = FilterReflectionEnvironment(GraphBuilder, ShaderMap, SceneCubemapTexture, nullptr);

				if (FeatureLevel >= ERHIFeatureLevel::SM5)
				{
					CopyToSceneArray(GraphBuilder, Scene, FilteredSceneCubemapTexture, ReflectionProxy, CubemapIndex);

					// Baked data has been rendered into the cubemap slot; the capture is now valid for shading.
					if (!FoundState.bRenderedForShading)
					{
						FoundState.bRenderedForShading = true;
						Scene->ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged = true;
					}
				}

				GraphBuilder.Execute();

				// Clean up the proxy now that the rendering thread is done with it
				delete ReflectionProxy;
			});
		}
	}
}

void ReadbackRadianceMap(FRDGBuilder& GraphBuilder, FRDGTexture* InputTexture, TArray<FFloat16Color>* OutRadianceMap)
{
	check(InputTexture->Desc.Format == PF_FloatRGBA);

	AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("ReadbackRadianceMap"), InputTexture, [InputTexture, &OutRadianceMap = *OutRadianceMap](FRHICommandListImmediate& RHICmdList)
	{
		const FIntPoint Extent = InputTexture->Desc.Extent;

		const int32 MipIndex = 0;
		const int32 CubeFaceBytes = Extent.X * Extent.Y * OutRadianceMap.GetTypeSize();

		OutRadianceMap.Empty(Extent.X* Extent.Y * 6);
		OutRadianceMap.AddZeroed(Extent.X* Extent.Y * 6);

		for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
		{
			TArray<FFloat16Color> SurfaceData;

			// Read each mip face
			RHICmdList.ReadSurfaceFloatData(InputTexture->GetRHI(), FIntRect(FIntPoint::ZeroValue, Extent), SurfaceData, (ECubeFace)CubeFace, 0, MipIndex);
			const int32 DestIndex = CubeFace * Extent.X * Extent.Y;
			FFloat16Color* FaceData = &OutRadianceMap[DestIndex];
			check(SurfaceData.Num() * SurfaceData.GetTypeSize() == CubeFaceBytes);
			FMemory::Memcpy(FaceData, SurfaceData.GetData(), CubeFaceBytes);
		}
	});
}

void CopyToSkyTexture(FRDGBuilder& GraphBuilder, FScene* Scene, FRDGTexture* InputTexture, FTexture* ProcessedTexture)
{
	if (ProcessedTexture->TextureRHI)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "CopyToSkyTexture");

		FRDGTexture* OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ProcessedTexture->TextureRHI, TEXT("SkyTexture")));

		// GPU copy back to the skylight's texture, which is not a render target
		FRHICopyTextureInfo CopyInfo;
		CopyInfo.Size.X = InputTexture->Desc.Extent.X;
		CopyInfo.Size.Y = InputTexture->Desc.Extent.Y;
		CopyInfo.NumSlices = 6;
		CopyInfo.NumMips = GetNumMips(ProcessedTexture->GetSizeX());

		AddCopyTexturePass(GraphBuilder, InputTexture, OutputTexture, CopyInfo);
		GraphBuilder.UseExternalAccessMode(OutputTexture, ERHIAccess::SRVMask, ERHIPipeline::All);
	}
}

void ComputeSpecifiedCubemapColorScale(
	FRDGBuilder& GraphBuilder, 
	FGlobalShaderMap* ShaderMap, 
	UTextureCube* SourceCubemap,
	int32 CubemapSize,
	bool bIsSkyLight,
	bool bLowerHemisphereIsBlack,
	float SourceCubemapRotation,
	const FLinearColor& LowerHemisphereColorValue,
	float& MaxFloatLuminance)
{
	const FTexture* SourceCubemapResource = SourceCubemap->GetResource();
	if (SourceCubemapResource == nullptr)
	{
		UE_LOGF(LogRenderer, Warning, "Unable to copy from cubemap %ls, it's RHI resource is null", *SourceCubemap->GetPathName());
		return;
	}
;
	const int32 NumReflectionCaptureMips = GetNumMips(CubemapSize);
	const FRDGTextureDesc TextureDesc = FRDGTextureDesc::CreateCube(
		CubemapSize, PF_A32B32G32R32F, FClearValueBinding(FLinearColor(0, 10000, 0, 0)), TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_TargetArraySlicesIndependently | TexCreate_DisableDCC, NumReflectionCaptureMips);
	FRDGTextureRef ReflectionMaxLuminanceCubeTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("ReflectionMaxLuminanceCubeTexture"));

	// Copy the source texture into the first mip level of each faces of the cubemap.
	{
		RDG_EVENT_SCOPE(GraphBuilder, "CopyCubeTextureToFP32CubemapMip0");
		const bool bClampToFP16 = false; // Rendering into an FP32 texture.
		CopyCubemapToScratchCubemapInner(
			GraphBuilder,
			ShaderMap,
			SourceCubemapResource,
			ReflectionMaxLuminanceCubeTexture,
			CubemapSize,
			bIsSkyLight,
			bLowerHemisphereIsBlack,
			SourceCubemapRotation,
			LowerHemisphereColorValue,
			FLinearColor::White,
			bClampToFP16);
	}

	// Generate all the mips of the texture, each time storing the maximum luminance value.
	{
		RDG_EVENT_SCOPE(GraphBuilder, "CreateFP32CubeMipsAsMaxLuminance");
		TShaderMapRef<FCubeDownsampleMaxPS> PixelShader(ShaderMap);
		const int32 NumMips = ReflectionMaxLuminanceCubeTexture->Desc.NumMips;

		// Downsample all the mips, each one reads from the mip above it
		for (int32 MipIndex = 1; MipIndex < NumMips; MipIndex++)
		{
			const int32 MipSize = 1 << (NumMips - MipIndex - 1);
			const FIntRect ViewRect(0, 0, MipSize, MipSize);

			for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
			{
				auto* PassParameters = GraphBuilder.AllocParameters<FCubeDownsampleMaxPS::FParameters>();
				PassParameters->SourceCubemapTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(ReflectionMaxLuminanceCubeTexture, MipIndex - 1));
				PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->CubeFace = CubeFace;
				PassParameters->MipIndex = MipIndex;
				PassParameters->NumMips = NumMips;
				PassParameters->SvPositionToUVScale = FVector2f(1.0f / MipSize, 1.0f / MipSize);
				PassParameters->RenderTargets[0] = FRenderTargetBinding(ReflectionMaxLuminanceCubeTexture, ERenderTargetLoadAction::ENoAction, MipIndex, CubeFace);

				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					ShaderMap,
					RDG_EVENT_NAME("CreateCubeMips (Mip: %d, Face: %d)", MipIndex, CubeFace),
					PixelShader,
					PassParameters,
					ViewRect);
			}
		}
	}

	// Fetch the maximum luminance (acounting for each face) from GPU to CPU.
	{
		RDG_EVENT_SCOPE(GraphBuilder, "FetchFP32CubeMaxLuminance");

		FRDGTexture* MaxLuminanceTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(1, 1), PF_A32B32G32R32F, FClearValueBinding::None, TexCreate_RenderTargetable), TEXT("ReflectionBrightness"));

		auto* PassParameters = GraphBuilder.AllocParameters<FComputeCubeMaxLuminancePS::FParameters>();

		PassParameters->ReflectionEnvironmentColorTexture = ReflectionMaxLuminanceCubeTexture;
		PassParameters->ReflectionEnvironmentColorSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->NumCaptureArrayMips = ReflectionMaxLuminanceCubeTexture->Desc.NumMips;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(MaxLuminanceTexture, ERenderTargetLoadAction::ENoAction);

		TShaderMapRef<FComputeCubeMaxLuminancePS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("ReflectionBrightness"),
			PixelShader,
			PassParameters,
			FIntRect(FIntPoint::ZeroValue, FIntPoint(1, 1)));

		float* MaxFloatLuminancePtr = &MaxFloatLuminance;
		AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("ReadbackTexture"), MaxLuminanceTexture, [MaxLuminanceTexture, MaxFloatLuminancePtr](FRHICommandListImmediate& RHICmdList)
		{
			FReadSurfaceDataFlags ReadDataFlags(RCM_MinMax);
			ReadDataFlags.SetLinearToGamma(false);

			FIntRect SourceRect = FIntRect(0, 0, 1, 1);

			TArray<FLinearColor> RawPixels;
			RawPixels.SetNum(SourceRect.Width() * SourceRect.Height());

			RHICmdList.ReadSurfaceData(MaxLuminanceTexture->GetRHI(), SourceRect, RawPixels, ReadDataFlags);

			// Shader outputs luminance to R
			*MaxFloatLuminancePtr = RawPixels[0].R;
		});
	}
}

// Warning: returns before writes to OutIrradianceEnvironmentMap have completed, as they are queued on the rendering thread
void FScene::UpdateSkyCaptureContents(
	const USkyLightComponent* CaptureComponent, 
	bool bCaptureEmissiveOnly, 
	UTextureCube* SourceCubemap, 
	FTexture* OutProcessedTexture, 
	float& OutAverageBrightness, 
	FSHVectorRGB3& OutIrradianceEnvironmentMap,
	TArray<FFloat16Color>* OutRadianceMap,
	FLinearColor* SpecifiedCubemapColorScale)
{
	if (GSupportsRenderTargetFormat_PF_FloatRGBA || FeatureLevel >= ERHIFeatureLevel::SM5)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateSkyCaptureContents);

		if (UWorld* CurrentWorld = GetWorld())
		{
			//guarantee that all render proxies are up to date before kicking off this render
			CurrentWorld->SendAllEndOfFrameUpdates();
		}

		const int32 CubemapResolution = CaptureComponent->CubemapResolution;
		const bool bLowerHemisphereIsBlack = CaptureComponent->bLowerHemisphereIsBlack;
		const FLinearColor LowerHemisphereColor = CaptureComponent->LowerHemisphereColor;

		// For FP32 textures, we are going to compute the maximum luminance accross each pixels and components, to rescale the luminance to fit in the FP16 range.
		// Later when fetched, SpecifiedCubemapColorScale will be used to rescale the sky light to the correct original luminance.
		const bool bSkySpecifiedCubemapUses32bitFloat = SpecifiedCubemapColorScale != nullptr && CaptureComponent->SourceType == SLS_SpecifiedCubemap && CaptureComponent->Cubemap->GetPixelFormat() == PF_A32B32G32R32F;

		TRenderThreadStruct<FReflectionCubemapTexture> ReflectionCubemapTexture(CubemapResolution);

		UE::RenderCommandPipe::FSyncScope SyncScope;

		if (CaptureComponent->SourceType == SLS_CapturedScene)
		{
			const bool bStaticSceneOnly = CaptureComponent->Mobility == EComponentMobility::Static;
			const bool bCapturingForMobile = false;
			CaptureSceneIntoScratchCubemap(this, *ReflectionCubemapTexture, CaptureComponent->GetComponentLocation(), CubemapResolution, true, bStaticSceneOnly, CaptureComponent->SkyDistanceThreshold, bLowerHemisphereIsBlack, bCaptureEmissiveOnly, LowerHemisphereColor, bCapturingForMobile, false);
		}
		else if (CaptureComponent->SourceType == SLS_SpecifiedCubemap)
		{
			const float SourceCubemapRotation = CaptureComponent->SourceCubemapAngle * (PI / 180.f);
			ENQUEUE_RENDER_COMMAND(CopyCubemapCommand)(
				[FeatureLevel = FeatureLevel, SourceCubemap, ReflectionCubemapTexture = ReflectionCubemapTexture.Get(), CubemapResolution, bLowerHemisphereIsBlack, 
				SourceCubemapRotation, LowerHemisphereColor, bSkySpecifiedCubemapUses32bitFloat, SpecifiedCubemapColorScale](FRHICommandListImmediate& RHICmdList)
			{
				float MaxFloatLuminance = 1.0f;
				if (bSkySpecifiedCubemapUses32bitFloat)
				{
					FRDGBuilder GraphBuilder(RHICmdList);

					FRDGTexture* SceneCubemapTexture = ReflectionCubemapTexture->GetRDG(GraphBuilder);
					auto ShaderMap = GetGlobalShaderMap(FeatureLevel); 
					ComputeSpecifiedCubemapColorScale(GraphBuilder, ShaderMap, SourceCubemap, CubemapResolution, true, bLowerHemisphereIsBlack, SourceCubemapRotation, LowerHemisphereColor, MaxFloatLuminance);

					GraphBuilder.Execute(); // Execute the graph to actually fetch MaxFloatLuminance from the GPU processed texture.
				}

				// Now copy into the scratch cubemap and rescale so that the maximum values is MaxFP16, 
				// because our runtime cubemap ReflectionCubemapTexture only uses FP16 for performance sake.
				FLinearColor CubeLuminanceScale = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
				if (bSkySpecifiedCubemapUses32bitFloat && MaxFloatLuminance > FFloat16::MaxF16Float)
				{
					const float LuminanceScale = FFloat16::MaxF16Float / MaxFloatLuminance;
					CubeLuminanceScale = FLinearColor(LuminanceScale, LuminanceScale, LuminanceScale, 1.0f);
				}

				CopyCubemapToScratchCubemap(RHICmdList, FeatureLevel, SourceCubemap, *ReflectionCubemapTexture, CubemapResolution, true, bLowerHemisphereIsBlack, SourceCubemapRotation, LowerHemisphereColor, CubeLuminanceScale);

				// Now update SpecifiedCubemapColorScale so that it can expand a rescaled FP16 texture into its original FP32 range.
				if (bSkySpecifiedCubemapUses32bitFloat && MaxFloatLuminance > FFloat16::MaxF16Float)
				{
					check(SpecifiedCubemapColorScale);
					const float LuminanceScale = MaxFloatLuminance / FFloat16::MaxF16Float;
					*SpecifiedCubemapColorScale = FLinearColor(LuminanceScale, LuminanceScale, LuminanceScale, 1.0);
				}

				// Now that we have rescale an FP32 input texture to fit in an FP16 range, the rest of the process is unchanged.
			});
		}
		else if (CaptureComponent->IsRealTimeCaptureEnabled())
		{
			ensureMsgf(false, TEXT("A sky light with RealTimeCapture enabled cannot be scheduled for a cubemap update. This will be done dynamically each frame by the renderer."));
			return;
		}
		else
		{
			checkNoEntry();
		}

		ENQUEUE_RENDER_COMMAND(UpdateCaptureContents)(
			[Scene = this, FeatureLevel = FeatureLevel, ReflectionCubemapTexture = ReflectionCubemapTexture.Get(), OutAverageBrightness = &OutAverageBrightness, OutIrradianceEnvironmentMap = &OutIrradianceEnvironmentMap, 
			OutProcessedTexture, OutRadianceMap](FRHICommandListImmediate& RHICmdList)
		{
			auto ShaderMap = GetGlobalShaderMap(FeatureLevel);

			FRDGBuilder GraphBuilder(RHICmdList);

			FRDGTexture* SceneCubemapTexture = ReflectionCubemapTexture->GetRDG(GraphBuilder);

			if (OutRadianceMap)
			{
				ReadbackRadianceMap(GraphBuilder, SceneCubemapTexture, OutRadianceMap);
			}

			FRDGTexture* FilteredSceneCubemapTexture;

			if (FeatureLevel <= ERHIFeatureLevel::ES3_1)
			{
				MobileReflectionEnvironmentCapture::ComputeAverageBrightness(GraphBuilder, ShaderMap, SceneCubemapTexture, OutAverageBrightness);
				FilteredSceneCubemapTexture = MobileReflectionEnvironmentCapture::FilterReflectionEnvironment(GraphBuilder, ShaderMap, SceneCubemapTexture, OutIrradianceEnvironmentMap);
			}
			else
			{
				ComputeAverageBrightness(GraphBuilder, ShaderMap, SceneCubemapTexture, OutAverageBrightness);
				FilteredSceneCubemapTexture = FilterReflectionEnvironment(GraphBuilder, ShaderMap, SceneCubemapTexture, OutIrradianceEnvironmentMap);
			}

			if (OutProcessedTexture)
			{
				CopyToSkyTexture(GraphBuilder, Scene, FilteredSceneCubemapTexture, OutProcessedTexture);
			}

			GraphBuilder.Execute();

			Scene->PathTracingSkylightTexture = nullptr;
			Scene->PathTracingSkylightPdf = nullptr;
		});
	}
}
