// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Functionality for capturing the scene into reflection capture cubemaps, and prefiltering
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Math/SHMath.h"
#include "RHI.h"
#include "GlobalShader.h"
#include "RenderGraphDefinitions.h"

// OutIrradianceEnvironmentMap is optional -- if not set, diffuse irradiance is written back to the low mips of LightingSource
extern void ComputeDiffuseIrradiance(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* LightingSource,  FSHVectorRGB3* OutIrradianceEnvironmentMap);

// LightingSource and LightingDest can be the same texture
extern void ComputeDiffuseIrradianceReference(
	FRDGBuilder& GraphBuilder,
	ERHIFeatureLevel::Type FeatureLevel,
	FGlobalShaderMap* ShaderMap,
	FRDGTexture* LightingSource,
	FRDGTexture* LightingDest);

FMatrix CalcCubeFaceViewRotationMatrix(ECubeFace Face);
FMatrix GetCubeProjectionMatrix(float HalfFovDeg, float CubeMapSize, float NearPlane);

inline uint32 GetNumMips(uint32 MipSize)
{
	return FMath::CeilLogTwo(MipSize) + 1;
}

void ConvolveCubeMap(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, uint32 CubeMipStart, uint32 CubeMipEnd, uint32 FaceStart, uint32 FaceCount, FRDGTexture* RDGSrcRenderTarget, FRDGTexture* RDGDstRenderTarget);
void CreateCubeMips(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* CubemapTexture);
