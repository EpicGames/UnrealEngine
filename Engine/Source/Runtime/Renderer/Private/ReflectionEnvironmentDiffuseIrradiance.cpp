// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Functionality for computing SH diffuse irradiance from a cubemap
=============================================================================*/

#include "CoreMinimal.h"
#include "RHIDefinitions.h"
#include "RHI.h"
#include "ShaderParameters.h"
#include "Shader.h"
#include "StaticBoundShaderState.h"
#include "RHIStaticStates.h"
#include "ReflectionEnvironmentCapture.h"
#include "GlobalShader.h"
#include "PostProcess/SceneFilterRendering.h"
#include "ScreenRendering.h"
#include "PipelineStateCache.h"
#include "VisualizeTexture.h"
#include "CommonRenderResources.h"
#include "PixelShaderUtils.h"
#include "GenerateMips.h"

// For non-reference implementation, Mipmap size diffuse irradiance SH is computed from
static const int32 GDiffuseIrradianceCubemapSize = 32;

// For reference implementation, Mipmap index diffuse irradiance cube map is computed from (relative to lowest mip, so 0 == 1x1 mip, 3 == 8x8 mip).
// In testing, no visual difference was noted between using the 32x32 and 8x8 mip.
static const int32 GDiffuseIrradianceReferenceMip = 3;

// The number of tail mips diffuse irradiance is written back to.
static const int32 GDiffuseIrradianceResultMips = 4;

int32 GetDiffuseConvolutionSourceMip(FRDGTexture* Texture)
{
	const int32 NumMips = Texture->Desc.NumMips;
	const int32 NumDiffuseMips = GetNumMips(GDiffuseIrradianceCubemapSize);
	return FMath::Max(0, NumMips - NumDiffuseMips);
}

/** Pixel shader used for copying to diffuse irradiance texture. */
class FCopyDiffuseIrradiancePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyDiffuseIrradiancePS);
	SHADER_USE_PARAMETER_STRUCT(FCopyDiffuseIrradiancePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER(FVector4f, CoefficientMask0)
		SHADER_PARAMETER(FVector4f, CoefficientMask1)
		SHADER_PARAMETER(FVector2f, SvPositionToUVScale)
		SHADER_PARAMETER(float, CoefficientMask2)
		SHADER_PARAMETER(int32, CubeFace)
		SHADER_PARAMETER(int32, SourceMipIndex)
		SHADER_PARAMETER(int32, NumSamples)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FCopyDiffuseIrradiancePS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "DiffuseIrradianceCopyPS", SF_Pixel)

class FAccumulateDiffuseIrradiancePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulateDiffuseIrradiancePS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulateDiffuseIrradiancePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER(FVector4f, Sample01)
		SHADER_PARAMETER(FVector4f, Sample23)
		SHADER_PARAMETER(FVector2f, SvPositionToUVScale)
		SHADER_PARAMETER(int32, CubeFace)
		SHADER_PARAMETER(int32, SourceMipIndex)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FAccumulateDiffuseIrradiancePS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "DiffuseIrradianceAccumulatePS", SF_Pixel)

class FAccumulateCubeFacesPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FAccumulateCubeFacesPS);
	SHADER_USE_PARAMETER_STRUCT(FAccumulateCubeFacesPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER(int32, SourceMipIndex)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FAccumulateCubeFacesPS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "AccumulateCubeFacesPS", SF_Pixel)

class FSHCoefficientsToCubemapCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSHCoefficientsToCubemapCS);
	SHADER_USE_PARAMETER_STRUCT(FSHCoefficientsToCubemapCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, OutCubeFaces)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SHTexture)
		SHADER_PARAMETER(float, MipInvSize)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FSHCoefficientsToCubemapCS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "SHCoefficientsToCubemapCS", SF_Compute)

class FDiffuseIrradianceReferenceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDiffuseIrradianceReferenceCS);
	SHADER_USE_PARAMETER_STRUCT(FDiffuseIrradianceReferenceCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<float4>, InCubeFaces)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2DArray<float4>, OutCubeFaces)
		SHADER_PARAMETER(int, InMipSizeMinusOne)	// InCubeFaces
		SHADER_PARAMETER(float, InMipInvSize)		// InCubeFaces
		SHADER_PARAMETER(float, MipInvSize)			// OutCubeFaces
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FDiffuseIrradianceReferenceCS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "DiffuseIrradianceReferenceCS", SF_Compute)

void ComputeDiffuseIrradiance(FRDGBuilder& GraphBuilder, FGlobalShaderMap* ShaderMap, FRDGTexture* LightingSource, FSHVectorRGB3* OutIrradianceEnvironmentMap)
{
	RDG_EVENT_SCOPE(GraphBuilder, "ComputeDiffuseIrradiance");

	const int32 LightingSourceMipIndex = GetDiffuseConvolutionSourceMip(LightingSource);
	const int32 NumMips = GetNumMips(GDiffuseIrradianceCubemapSize);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();

	FRDGTexture* SkySHIrradianceTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(FIntPoint(FSHVector3::MaxSHBasis, 1), PF_FloatRGBA, FClearValueBinding(FLinearColor(0, 10000, 0, 0)), TexCreate_RenderTargetable), TEXT("SkySHIrradiance"));

	TShaderMapRef<FCopyDiffuseIrradiancePS> CopyDiffuseIrradiancePS(ShaderMap);
	TShaderMapRef<FAccumulateDiffuseIrradiancePS> AccumulateDiffuseIrradiancePS(ShaderMap);
	TShaderMapRef<FAccumulateCubeFacesPS> AccumulateCubeFacesPS(ShaderMap);

	for (int32 CoefficientIndex = 0; CoefficientIndex < FSHVector3::MaxSHBasis; CoefficientIndex++)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "Coefficient %d", CoefficientIndex);

		const FRDGTextureDesc IrradianceCubemapDesc(
			FRDGTextureDesc::CreateCube(GDiffuseIrradianceCubemapSize, PF_FloatRGBA, FClearValueBinding(FLinearColor(0, 10000, 0, 0)), TexCreate_TargetArraySlicesIndependently | TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_DisableDCC, NumMips));

		FRDGTexture* IrradianceCubemapTexture = GraphBuilder.CreateTexture(IrradianceCubemapDesc, TEXT("DiffuseIrradianceCubemap"));

		// Copy the starting mip from the lighting texture, apply texel area weighting and appropriate SH coefficient
		{
			RDG_EVENT_SCOPE(GraphBuilder, "CopyDiffuseIrradiance");

			const FVector4f Mask0(CoefficientIndex == 0, CoefficientIndex == 1, CoefficientIndex == 2, CoefficientIndex == 3);
			const FVector4f Mask1(CoefficientIndex == 4, CoefficientIndex == 5, CoefficientIndex == 6, CoefficientIndex == 7);
			const float Mask2 = CoefficientIndex == 8;

			const int32 MipIndex = 0;
			const int32 MipSize = GDiffuseIrradianceCubemapSize;
			const FIntRect ViewRect(0, 0, MipSize, MipSize);

			for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
			{
				auto* PassParameters = GraphBuilder.AllocParameters<FCopyDiffuseIrradiancePS::FParameters>();
				PassParameters->SourceCubemapTexture = LightingSource;
				PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->CoefficientMask0 = Mask0;
				PassParameters->CoefficientMask1 = Mask1;
				PassParameters->CoefficientMask2 = Mask2;
				PassParameters->SvPositionToUVScale = FVector2f(1.0f / MipSize, 1.0f / MipSize);
				PassParameters->CubeFace = CubeFace;
				PassParameters->SourceMipIndex = LightingSourceMipIndex;
				PassParameters->NumSamples = MipSize * MipSize * 6;
				PassParameters->RenderTargets[0] = FRenderTargetBinding(IrradianceCubemapTexture, ERenderTargetLoadAction::ENoAction, 0, CubeFace);

				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					ShaderMap,
					RDG_EVENT_NAME("Face: %d", CubeFace),
					CopyDiffuseIrradiancePS,
					PassParameters,
					ViewRect);
			}
		}

		{
			RDG_EVENT_SCOPE(GraphBuilder, "AccumulateDiffuseIrradiance");

			// Accumulate all the texel values through downsampling to 1x1 mip
			for (int32 MipIndex = 1; MipIndex < NumMips; MipIndex++)
			{
				const int32 MipSize = 1 << (NumMips - MipIndex - 1);
				const FIntRect ViewRect(0, 0, MipSize, MipSize);

				const int32 SourceMipIndex = FMath::Max(MipIndex - 1, 0);
				const int32 SourceMipSize = 1 << (NumMips - SourceMipIndex - 1);

				const float HalfSourceTexelSize = 0.5f / SourceMipSize;
				const FVector4f Sample01(-HalfSourceTexelSize, -HalfSourceTexelSize, HalfSourceTexelSize, -HalfSourceTexelSize);
				const FVector4f Sample23(-HalfSourceTexelSize, HalfSourceTexelSize, HalfSourceTexelSize, HalfSourceTexelSize);

				FRDGTextureSRVDesc SourceSRVDesc = FRDGTextureSRVDesc::CreateForMipLevel(IrradianceCubemapTexture, MipIndex - 1); 
				if (GRHISupportsTextureViews == false)
				{
					SourceSRVDesc = FRDGTextureSRVDesc::Create(IrradianceCubemapTexture);
				}

				for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
				{
					auto* PassParameters = GraphBuilder.AllocParameters<FAccumulateDiffuseIrradiancePS::FParameters>();
					PassParameters->SourceCubemapTexture = GraphBuilder.CreateSRV(SourceSRVDesc);
					PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
					PassParameters->Sample01 = Sample01;
					PassParameters->Sample23 = Sample23;
					PassParameters->SvPositionToUVScale = FVector2f(1.0f / MipSize, 1.0f / MipSize);
					PassParameters->CubeFace = CubeFace;
					PassParameters->SourceMipIndex = SourceMipIndex;
					PassParameters->RenderTargets[0] = FRenderTargetBinding(IrradianceCubemapTexture, ERenderTargetLoadAction::ELoad, MipIndex, CubeFace);

					FPixelShaderUtils::AddFullscreenPass(
						GraphBuilder,
						ShaderMap,
						RDG_EVENT_NAME("Mip: %d, Face: %d", MipIndex, CubeFace),
						AccumulateDiffuseIrradiancePS,
						PassParameters,
						ViewRect);
				}
			}
		}

		{
			const int32 SourceMipIndex = NumMips - 1;
			const int32 MipSize = 1;
			const FIntRect ViewRect(CoefficientIndex, 0, CoefficientIndex + 1, 1);

			auto* PassParameters = GraphBuilder.AllocParameters<FAccumulateCubeFacesPS::FParameters>();
			PassParameters->SourceCubemapTexture = IrradianceCubemapTexture;
			PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			PassParameters->SourceMipIndex = SourceMipIndex;
			PassParameters->RenderTargets[0] = FRenderTargetBinding(SkySHIrradianceTexture, ERenderTargetLoadAction::ELoad);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				ShaderMap,
				RDG_EVENT_NAME("GatherCoefficients"),
				AccumulateCubeFacesPS,
				PassParameters,
				ViewRect);
		}
	}

	if (OutIrradianceEnvironmentMap)
	{
		AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("ReadbackCoefficients"), SkySHIrradianceTexture, [SkySHIrradianceTexture, OutIrradianceEnvironmentMap](FRHICommandListImmediate& RHICmdList)
		{
			TArray<FFloat16Color> SurfaceData;
			RHICmdList.ReadSurfaceFloatData(SkySHIrradianceTexture->GetRHI(), FIntRect(0, 0, FSHVector3::MaxSHBasis, 1), SurfaceData, CubeFace_PosX, 0, 0);
			check(SurfaceData.Num() == FSHVector3::MaxSHBasis);

			for (int32 CoefficientIndex = 0; CoefficientIndex < FSHVector3::MaxSHBasis; CoefficientIndex++)
			{
				const FLinearColor CoefficientValue(SurfaceData[CoefficientIndex]);
				OutIrradianceEnvironmentMap->R.V[CoefficientIndex] = CoefficientValue.R;
				OutIrradianceEnvironmentMap->G.V[CoefficientIndex] = CoefficientValue.G;
				OutIrradianceEnvironmentMap->B.V[CoefficientIndex] = CoefficientValue.B;
			}
		});
	}
	else
	{
		// If no SH result OutIrradianceEnvironmentMap pointer is provided, diffuse irradiance is written back to low mips of LightingSource
		int32 LightingNumMips = LightingSource->Desc.NumMips;
		if (LightingNumMips == 1)
		{
			// No mips to write to!
			return;
		}

		for (int32 MipLog2 = 0; MipLog2 < FMath::Min(GDiffuseIrradianceResultMips, LightingNumMips); MipLog2++)
		{
			int32 MipSize = 1 << MipLog2;

			FSHCoefficientsToCubemapCS::FParameters* Parameters = GraphBuilder.AllocParameters<FSHCoefficientsToCubemapCS::FParameters>();
			Parameters->OutCubeFaces = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(LightingSource, LightingNumMips - 1 - MipLog2, PF_Unknown, 0, 1));
			Parameters->SHTexture = SkySHIrradianceTexture;
			Parameters->MipInvSize = 1.0f / MipSize;

			TShaderMapRef<FSHCoefficientsToCubemapCS> ComputeShader(ShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("SHCoefficientsToCubemap"),
				ComputeShader,
				Parameters,
				FIntVector(FMath::DivideAndRoundUp(MipSize, 8), FMath::DivideAndRoundUp(MipSize, 8), 6));
		}
	}
}

void ComputeDiffuseIrradianceReference(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FGlobalShaderMap* ShaderMap, FRDGTexture* LightingSource, FRDGTexture* LightingDest)
{
	
	const int32 SourceMipLog2 = FMath::Clamp(GDiffuseIrradianceReferenceMip, 0, LightingSource->Desc.NumMips - 1);
	int32 SourceMipSize = 1 << SourceMipLog2;
	int32 SourceNumMips = LightingSource->Desc.NumMips;

	const int32 DestMipLog2 = FMath::Min(GDiffuseIrradianceResultMips - 1, LightingDest->Desc.NumMips - 1);		// Log2 size of max mip actually written to
	int32 DestMipSize = 1 << DestMipLog2;
	int32 DestNumMips = LightingDest->Desc.NumMips;

	FRDGTextureDesc TemporaryDestDesc = FRDGTextureDesc::CreateCube(
		DestMipSize,
		LightingDest->Desc.Format,
		FClearValueBinding::None,
		ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV,
		DestNumMips > 1 ? DestMipLog2 + 1 : 1);
	FRDGTexture* TemporaryDest = GraphBuilder.CreateTexture(TemporaryDestDesc, TEXT("DiffuseIrradianceTemp"));

	FRDGTextureSRVDesc SourceDesc(LightingSource);
	SourceDesc.MipLevel = SourceNumMips - 1 - SourceMipLog2;
	SourceDesc.NumMipLevels = 1;
	SourceDesc.FirstArraySlice = 0;
	SourceDesc.NumArraySlices = 6;
	SourceDesc.DimensionOverride = ETextureDimension::Texture2DArray;

	FDiffuseIrradianceReferenceCS::FParameters* Parameters = GraphBuilder.AllocParameters<FDiffuseIrradianceReferenceCS::FParameters>();
	Parameters->InCubeFaces = GraphBuilder.CreateSRV(SourceDesc);
	Parameters->OutCubeFaces = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(TemporaryDest, 0, PF_Unknown));
	Parameters->InMipSizeMinusOne = SourceMipSize - 1;
	Parameters->InMipInvSize = 1.0f / SourceMipSize;
	Parameters->MipInvSize = 1.0f / DestMipSize;

	TShaderMapRef<FDiffuseIrradianceReferenceCS> ComputeShader(ShaderMap);
	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("DiffuseIrradianceRef"),
		ComputeShader,
		Parameters,
		FIntVector(FMath::DivideAndRoundUp(DestMipSize, 8), FMath::DivideAndRoundUp(DestMipSize, 8), 6));

	if (DestNumMips > 1)
	{
		FGenerateMipsParams GenerateMipsParams;
		FGenerateMips::Execute(GraphBuilder, FeatureLevel, TemporaryDest, GenerateMipsParams);
	}

	// Iterate and copy all mips, or one mip if destination only has one
	int32 FirstDestMipLog2 = DestNumMips > 1 ? 0 : DestMipLog2;
	for (int32 CopyMipLog2 = FirstDestMipLog2; CopyMipLog2 <= DestMipLog2; CopyMipLog2++)
	{
		FRHICopyTextureInfo CopyInfo;
		CopyInfo.SourceMipIndex = DestMipLog2 - CopyMipLog2;							// Mip in TemporaryDest
		CopyInfo.DestMipIndex = DestNumMips > 1 ? DestNumMips - 1 - CopyMipLog2 : 0;	// Mip in LightingDest
		CopyInfo.DestSliceIndex = 0;
		CopyInfo.NumSlices = 6;
		AddCopyTexturePass(GraphBuilder, TemporaryDest, LightingDest, CopyInfo);
	}
}
