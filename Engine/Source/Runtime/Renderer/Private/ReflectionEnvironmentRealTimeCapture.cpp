// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Functionality for capturing and pre-filtering a sky env map in real time.
=============================================================================*/

#include "ReflectionEnvironmentCapture.h"
#include "BasePassRendering.h"
#include "ClearQuad.h"
#include "MeshPassProcessor.h"
#include "PrimitiveSceneProxy.h"
#include "MeshPassProcessor.inl"
#include "ScenePrivate.h"
#include "SceneProxies/SkyAtmosphereSceneProxy.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SkyPassRendering.h"
#include "RenderGraphUtils.h"
#include "VolumetricCloudRendering.h"
#include "VolumetricCloudProxy.h"
#include "FogRendering.h"
#include "GPUScene.h"
#include "ScreenPass.h"
#include "SkyAtmosphereRendering.h"
#include "MobileBasePassRendering.h"
#include "MobileReflectionEnvironmentCapture.h"
#include "SimpleMeshDrawCommandPass.h"
#include "SceneCore.h"
#include "RenderTargetPool.h"

#if WITH_EDITOR
#include "CanvasTypes.h"
#endif

extern float GReflectionCaptureNearPlane;


DECLARE_GPU_STAT(CaptureConvolveSkyEnvMap);


static TAutoConsoleVariable<int32> CVarRealTimeReflectionCaptureTimeSlicing(
	TEXT("r.SkyLight.RealTimeReflectionCapture.TimeSlice"), 1,
	TEXT("When enabled, the real-time sky light capture and convolutions will by distributed over several frames to lower the per-frame cost."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarRealTimeReflectionCaptureTimeSlicingSkyCloudCubeFacePerFrame(
	TEXT("r.SkyLight.RealTimeReflectionCapture.TimeSlice.SkyCloudCubeFacePerFrame"), 2, // Default preferred for 60hz. 30hz applications can use 6 to not have any different sky lighting in different faces when the sun moves fast.
	TEXT("When enabled, the real-time sky light capture, when time sliced, will not render cloud in all cube face in a single frame; but one face per frame. That is to distribute the cloud tracing cost even more, but will add latency and potentially can result in lighting discrepancy between faces if the sun is moving fast. Value in [1,6]."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarRealTimeReflectionCaptureShadowFromOpaque(
	TEXT("r.SkyLight.RealTimeReflectionCapture.ShadowFromOpaque"), 0,
	TEXT("Opaque meshes cast shadow from directional lights onto sky and clouds when enabled.\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRealTimeReflectionCaptureDepthBuffer(
	TEXT("r.SkyLight.RealTimeReflectionCapture.DepthBuffer"), 1,
	TEXT("When enabled, the real-time sky light capture will have a depth buffer, this is for multiple meshes to be cover each other correctly. The height fog will also be applied according to the depth buffer."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRealTimeReflectionCaptureAlwaysClearColorBuffer(
	TEXT("r.SkyLight.RealTimeReflectionCapture.AlwaysClearColorBuffer"), 0,
	TEXT("Always clear the color buffer to black before rendering SkyAtmosphere, SkyPass, Fog, etc."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRealTimeReflectionCaptureVolumetricCloudResolutionDivider(
	TEXT("r.SkyLight.RealTimeReflectionCapture.VolumetricCloudResolutionDivider"), 2, // Default preferred for 60hz.
	TEXT("The divider applied on the resolution when capturing cloud in the real time sky light. For instance, a value of 2 will render cloud at half resolution and thus will be faster to render."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarRealTimeReflectionCaptureResolutionOverride(
	TEXT("r.SkyLight.RealTimeReflectionCapture.ResolutionOverride"), 0,
	TEXT("Override the real time sky light capture resolution if this CVar is greater than 16."),
	ECVF_RenderThreadSafe | ECVF_Scalability);	// Should be ECVF_ReadOnly because it is not really toggable mid-update when time slicing is used (a black convolved sky target will be used instead). 
												// TODO update on FRealTimeSlicedReflectionCapture only of first frame of capture.

// If this message on screen message is not enough to prevent TDRs, we should implement half resolution + upsampling tracing for cloud in sky light capture.
static TAutoConsoleVariable<int32> CVarRealTimeReflectionCaptureDisableExpenssiveCaptureMessage(
	TEXT("r.SkyLight.RealTimeReflectionCapture.DisableExpenssiveCaptureMessage"), 0,
	TEXT("Disable the message reporting expenssive sky light capture due to high resolution with volumetric cloud tracing that could cause TDR."),
	ECVF_RenderThreadSafe);

class FDownsampleCubeFaceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FDownsampleCubeFaceCS);
	SHADER_USE_PARAMETER_STRUCT(FDownsampleCubeFaceCS, FGlobalShader);

	static const uint32 ThreadGroupSize = 8;

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MipIndex)
		SHADER_PARAMETER(uint32, NumMips)
		SHADER_PARAMETER(int32, CubeFace)
		SHADER_PARAMETER(int32, FaceThreadGroupSize)
		SHADER_PARAMETER(FIntPoint, ValidDispatchCoord)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTextureMipColor)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return !IsMobilePlatform(Parameters.Platform); }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("USE_COMPUTE"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FDownsampleCubeFaceCS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "DownsampleCS", SF_Compute);


class FConvolveSpecularFaceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FConvolveSpecularFaceCS);
	SHADER_USE_PARAMETER_STRUCT(FConvolveSpecularFaceCS, FGlobalShader);

	static const uint32 ThreadGroupSize = 8;

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, MipIndex)
		SHADER_PARAMETER(uint32, NumMips)
		SHADER_PARAMETER(int32, CubeFaceOffset)
		SHADER_PARAMETER(int32, CubeFace)
		SHADER_PARAMETER(int32, FaceThreadGroupSize)
		SHADER_PARAMETER(FIntPoint, ValidDispatchCoord)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D, OutTextureMipColor)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::ES3_1; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE"), ThreadGroupSize);
		OutEnvironment.SetDefine(TEXT("USE_COMPUTE"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FConvolveSpecularFaceCS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "FilterCS", SF_Compute);


class FComputeSkyEnvMapDiffuseIrradianceCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComputeSkyEnvMapDiffuseIrradianceCS);
	SHADER_USE_PARAMETER_STRUCT(FComputeSkyEnvMapDiffuseIrradianceCS, FGlobalShader);

	// 8*8=64 threads in a group.
	// Each thread uses 4*7*RGB sh float => 84 bytes shared group memory. 
	// 64 * 84 = 5376 bytes which fits dx11 16KB shared memory limitation. 6144 with vector alignement in shared memory and it still fits
	// Low occupancy on a single CU.
	static const uint32 ThreadGroupSizeX = 8;
	static const uint32 ThreadGroupSizeY = 8;

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(TextureCube, SourceCubemapTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, SourceCubemapSampler)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer, OutIrradianceEnvMapSH)
		SHADER_PARAMETER(float, UniformSampleSolidAngle)
		SHADER_PARAMETER(uint32, MipIndex)
	END_SHADER_PARAMETER_STRUCT()

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return GetMaxSupportedFeatureLevel(Parameters.Platform) >= ERHIFeatureLevel::ES3_1; }
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), ThreadGroupSizeY);
		OutEnvironment.SetDefine(TEXT("SHADER_DIFFUSE_TO_SH"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FComputeSkyEnvMapDiffuseIrradianceCS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "ComputeSkyEnvMapDiffuseIrradianceCS", SF_Compute);


class FApplyLowerHemisphereColorPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FApplyLowerHemisphereColorPS);
	SHADER_USE_PARAMETER_STRUCT(FApplyLowerHemisphereColorPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER(int32, ApplyLowerHemisphereColor)
		SHADER_PARAMETER(FVector4f, LowerHemisphereSolidColor)
		SHADER_PARAMETER(FVector2f, SvPositionToUVScale)
		SHADER_PARAMETER(int32, CubeFace)
		SHADER_PARAMETER(int32, ApplyLowResCloudTexture)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, LowResCloudTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, LowResCloudSampler)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("APPLY_LOWER_HEMISPHERE_COLOR_PIXELSHADER"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FApplyLowerHemisphereColorPS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "ApplyLowerHemisphereColorPS", SF_Pixel);


class FRenderRealTimeReflectionHeightFogVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderRealTimeReflectionHeightFogVS);
	SHADER_USE_PARAMETER_STRUCT(FRenderRealTimeReflectionHeightFogVS, FGlobalShader);;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("REALTIME_REFLECTION_HEIGHT_FOG"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderRealTimeReflectionHeightFogVS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "RenderRealTimeReflectionHeightFogVS", SF_Vertex);


class FRenderRealTimeReflectionHeightFogPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderRealTimeReflectionHeightFogPS);
	SHADER_USE_PARAMETER_STRUCT(FRenderRealTimeReflectionHeightFogPS, FGlobalShader);

	class FDepthTexture : SHADER_PERMUTATION_BOOL("PERMUTATION_DEPTHTEXTURE");
	using FPermutationDomain = TShaderPermutationDomain<FDepthTexture>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, ViewUniformBuffer)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FFogUniformParameters, FogStruct)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DepthTexture)
		SHADER_PARAMETER(FVector3f, SkyLightPosition)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static FPermutationDomain RemapPermutation(FPermutationDomain PermutationVector)
	{
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("REALTIME_REFLECTION_HEIGHT_FOG"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FRenderRealTimeReflectionHeightFogPS, "/Engine/Private/ReflectionEnvironmentShaders.usf", "RenderRealTimeReflectionHeightFogPS", SF_Pixel);


void FScene::ValidateSkyLightRealTimeCapture(
	FRDGBuilder& GraphBuilder,
	const FViewInfo& View,
	FRDGTextureRef SceneColorTexture)
{
#if WITH_EDITOR
	if (!GAreScreenMessagesEnabled)
	{
		return;
	}

	bool bSkyMeshInMainPassExist = false;
	bool bSkyMeshInRealTimeSkyCaptureExtist = false;

	const int32 SkyRealTimeReflectionOnlyMeshBatcheCount = View.SkyMeshBatches.Num();
	for (int32 MeshBatchIndex = 0; MeshBatchIndex < SkyRealTimeReflectionOnlyMeshBatcheCount; ++MeshBatchIndex)
	{
		const FSkyMeshBatch& SkyMeshBatch = View.SkyMeshBatches[MeshBatchIndex];
		bSkyMeshInMainPassExist |= SkyMeshBatch.bVisibleInMainPass;
		bSkyMeshInRealTimeSkyCaptureExtist |= SkyMeshBatch.bVisibleInRealTimeSkyCapture;
	}

	if (!bSkyMeshInMainPassExist || !bSkyMeshInRealTimeSkyCaptureExtist)
	{
		AddDrawCanvasPass(GraphBuilder, {}, View, FScreenPassRenderTarget(SceneColorTexture, View.ViewRect, ERenderTargetLoadAction::ELoad), [this, &View, bSkyMeshInMainPassExist, bSkyMeshInRealTimeSkyCaptureExtist](FCanvas& Canvas)
		{
			FLinearColor TextColor(1.0f, 0.5f, 0.0f);

			if (View.bSceneHasSkyMaterial && !bSkyMeshInMainPassExist)
			{
				const FString WarningString = NSLOCTEXT("RealTimeReflectionCapture", "RealTimeReflectionCaptureMainView",			"At least one mesh with a sky material is in the scene but none are rendered in main view.").ToString();
				Canvas.DrawShadowedString(100.0f, 100.0f, WarningString, GetStatsFont(), TextColor);
			}
			if (View.bSceneHasSkyMaterial && !bSkyMeshInRealTimeSkyCaptureExtist && SkyLight && SkyLight->bRealTimeCaptureEnabled)
			{
				const FString WarningString = NSLOCTEXT("RealTimeReflectionCapture", "RealTimeReflectionCaptureREReflectionView",	"At least one mesh with a sky material is in the scene but none are rendered in the real-time sky light reflection.").ToString();
				Canvas.DrawShadowedString(100.0f, 100.0f, WarningString, GetStatsFont(), TextColor);
			}
		});
	}
#endif
}

BEGIN_SHADER_PARAMETER_STRUCT(FCaptureSkyMeshReflectionPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FOpaqueBasePassUniformParameters, BasePass)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

BEGIN_SHADER_PARAMETER_STRUCT(FMobileCaptureSkyMeshReflectionPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileBasePassUniformParameters, BasePass)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()


void FScene::AllocateAndCaptureFrameSkyEnvMap(
	FRDGBuilder& GraphBuilder, FSceneRenderer& SceneRenderer, FViewInfo& MainView,
	bool bShouldRenderSkyAtmosphere, bool bShouldRenderVolumetricCloud, FInstanceCullingManager& InstanceCullingManager, FRDGExternalAccessQueue& ExternalAccessQueue)
{
	check(SkyLight && SkyLight->bRealTimeCaptureEnabled && !SkyLight->bHasStaticLighting);

	// Ignore viewfamilies without the Atmosphere showflag enabled as the sky capture may fail otherwise 
	// as well as all views being "scene captures" which cannot be used to update the sky light data.
	if (MainView.bIsSceneCapture || !MainView.Family->EngineShowFlags.Atmosphere)
	{
		return;
	}

	FRealTimeSlicedReflectionCapture& Capture = RealTimeSlicedReflectionCapture;

	const bool bIsNewFrame = GFrameNumberRenderThread != Capture.FrameNumber;
	Capture.FrameNumber = GFrameNumberRenderThread;

	// Clear record of GPUs handled this frame if this is a new frame
	if (bIsNewFrame)
	{
		Capture.GpusHandledThisFrame = 0;
	}

	// If this GPU has already been handled this frame, return, because we want to process the
	// sky capture update for each RenderScene, but only once per GPU.
	if ((Capture.GpusHandledThisFrame & MainView.GPUMask.GetNative()) == MainView.GPUMask.GetNative())
	{
		return;
	}

	// Record that we are handling the GPU in the MainView
	Capture.GpusHandledThisFrame |= MainView.GPUMask.GetNative();

	SCOPED_NAMED_EVENT(AllocateAndCaptureFrameSkyEnvMap, FColor::Emerald);
	RDG_EVENT_SCOPE_STAT(GraphBuilder, CaptureConvolveSkyEnvMap, "CaptureConvolveSkyEnvMap");
	RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, SkyAtmosphere);

	const int32 CaptureResolutionOverride = CVarRealTimeReflectionCaptureResolutionOverride.GetValueOnRenderThread();
	const uint32 CubeWidth = CaptureResolutionOverride >= 16 ? CaptureResolutionOverride : SkyLight->CaptureCubeMapResolution;
	const uint32 CubeMipCount = FMath::CeilLogTwo(CubeWidth) + 1;

	auto CreateMainViewSnapshotForRealTimeCapture = [&MainView](FViewInfo*& OutView, FMatrix& OutCubeProjectionMatrix, float CubeViewWidth)
	{
		// Make a snapshot we are going to use for the 6 cubemap faces and set it up.
		// Note: cube view is not meant to be sent to lambdas because we only create a single one. You should only send the ViewUniformBuffer around.
		OutView = MainView.CreateSnapshot();
		FViewInfo& CubeView = *OutView;
		CubeView.FOV = 90.0f;
		// Note: We cannot override exposure because sky input texture are using exposure

		// Other view data clean up
		CubeView.StereoPass = EStereoscopicPass::eSSP_FULL;
		CubeView.DrawDynamicFlags = EDrawDynamicFlags::ForceLowestLOD;
		CubeView.MaterialTextureMipBias = 0;

		FBox VolumeBounds[TVC_MAX];
		CubeView.CachedViewUniformShaderParameters = MakeUnique<FViewUniformShaderParameters>();
		CubeView.SetupUniformBufferParameters(
			VolumeBounds,
			TVC_MAX,
			*CubeView.CachedViewUniformShaderParameters);

		OutCubeProjectionMatrix = GetCubeProjectionMatrix(CubeView.FOV * 0.5f, (float)CubeViewWidth, GReflectionCaptureNearPlane);
		CubeView.UpdateProjectionMatrix(OutCubeProjectionMatrix);
	};

	FMatrix CubeProjectionMatrix;
	FViewInfo* CubeViewPtr = nullptr;
	CreateMainViewSnapshotForRealTimeCapture(CubeViewPtr, CubeProjectionMatrix, CubeWidth);
	FViewInfo& CubeView = *CubeViewPtr;

	FPooledRenderTargetDesc SkyCubeTexDesc = Translate(FSkyPassMeshProcessor::GetCaptureFrameSkyEnvMapTextureDesc(CubeWidth, CubeMipCount));

	const bool bTimeSlicedRealTimeCapture = CVarRealTimeReflectionCaptureTimeSlicing.GetValueOnRenderThread() > 0 && !MainView.Family->bCurrentlyBeingEdited;

	const bool CubeResolutionInvalidated = ConvolvedSkyRenderTargetReadyIndex < 0 || (ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetReadyIndex].IsValid() && ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetReadyIndex]->GetDesc().GetSize().X != CubeWidth);
	if (!ConvolvedSkyRenderTarget[0].IsValid() || CubeResolutionInvalidated)
	{
		// Always allocated
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, SkyCubeTexDesc, ConvolvedSkyRenderTarget[0], TEXT("SkyLight.ConvolvedSkyRenderTarget0"));
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, SkyCubeTexDesc, CapturedSkyRenderTarget, TEXT("SkyLight.CapturedSkyRenderTarget"));
	}
	if (bTimeSlicedRealTimeCapture && (!ConvolvedSkyRenderTarget[1].IsValid() || CubeResolutionInvalidated))
	{
		// Additional allocation for time slicing
		GRenderTargetPool.FindFreeElement(GraphBuilder.RHICmdList, SkyCubeTexDesc, ConvolvedSkyRenderTarget[1], TEXT("SkyLight.ConvolvedSkyRenderTarget1"));
	}

	auto ClearCubeFace = [&](FRDGTextureRef SkyCubeTexture, int32 CubeFace)
	{
		FRenderTargetParameters* Parameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
		Parameters->RenderTargets[0] = FRenderTargetBinding(SkyCubeTexture, ERenderTargetLoadAction::ENoAction, 0, CubeFace);

		FLinearColor ClearColor = FLinearColor::Black;
		GraphBuilder.AddPass(
			RDG_EVENT_NAME("ClearSkyRenderTarget"),
			Parameters,
			ERDGPassFlags::Raster,
			[Parameters, ClearColor](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				DrawClearQuad(RHICmdList, ClearColor);
			});
	};

	auto RenderCubeFaces_SkyCloud = [&](bool bExecuteSky, bool bExecuteCloud, TRefCountPtr<IPooledRenderTarget>& SkyRenderTarget, int32 StartCubeFace, int32 EndCubeFace)
	{
		FScene* Scene = MainView.Family->Scene->GetRenderScene();

		FRDGTextureRef SkyCubeTexture = GraphBuilder.RegisterExternalTexture(SkyRenderTarget, TEXT("SkyRenderTarget"));

		if (bExecuteSky || bExecuteCloud)
		{
			FRDGTextureRef BlackDummy2dTex = GraphBuilder.RegisterExternalTexture(GSystemTextures.BlackDummy);
			FRDGTextureRef BlackDummy3dTex = GraphBuilder.RegisterExternalTexture(GSystemTextures.VolumetricBlackDummy);
			const bool CaptureShadowFromOpaque = CVarRealTimeReflectionCaptureShadowFromOpaque.GetValueOnRenderThread() > 0;

			FSkyAtmosphereRenderContext SkyRC;
			const FAtmosphereSetup* AtmosphereSetup = nullptr;
			if (bShouldRenderSkyAtmosphere)
			{
				FSkyAtmosphereRenderSceneInfo& SkyInfo = *GetSkyAtmosphereSceneInfo();
				const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyInfo.GetSkyAtmosphereSceneProxy();

				// Global data constant between faces
				AtmosphereSetup = &SkyAtmosphereSceneProxy.GetAtmosphereSetup();

				SkyRC.bFastSky = false;
				SkyRC.bFastAerialPerspective = false;
				SkyRC.bFastAerialPerspectiveDepthTest = false;
				SkyRC.bSecondAtmosphereLightEnabled = IsSecondAtmosphereLightEnabled();

				// Enable opaque shadow on sky if needed
				SkyRC.bShouldSampleOpaqueShadow = false;
				if (CaptureShadowFromOpaque)
				{
					SkyAtmosphereLightShadowData LightShadowData;
					SkyRC.bShouldSampleOpaqueShadow = ShouldSkySampleAtmosphereLightsOpaqueShadow(*Scene, SceneRenderer.VisibleLightInfos, LightShadowData);
					GetSkyAtmosphereLightsUniformBuffers(GraphBuilder, SkyRC.LightShadowShaderParams0UniformBuffer, SkyRC.LightShadowShaderParams1UniformBuffer,
						LightShadowData, CubeView, SkyRC.bShouldSampleOpaqueShadow, UniformBuffer_SingleDraw);
				}

				SkyRC.bUseDepthBoundTestIfPossible = false;
				SkyRC.bForceRayMarching = true;				// We do not have any valid view LUT
				SkyRC.bDepthReadDisabled = true;
				SkyRC.bDisableBlending = true;

				SkyRC.TransmittanceLut = GraphBuilder.RegisterExternalTexture(SkyInfo.GetTransmittanceLutTexture());
				SkyRC.MultiScatteredLuminanceLut = GraphBuilder.RegisterExternalTexture(SkyInfo.GetMultiScatteredLuminanceLutTexture());
			}

			FCloudRenderContext CloudRC;
			if (bShouldRenderVolumetricCloud)
			{
				FVolumetricCloudRenderSceneInfo& CloudInfo = *GetVolumetricCloudSceneInfo();
				FVolumetricCloudSceneProxy& CloudSceneProxy = CloudInfo.GetVolumetricCloudSceneProxy();

				if (CloudSceneProxy.GetCloudVolumeMaterial())
				{
					FMaterialRenderProxy* CloudVolumeMaterialProxy = CloudSceneProxy.GetCloudVolumeMaterial()->GetRenderProxy();
					CloudRC.CloudInfo = &CloudInfo;
					CloudRC.CloudVolumeMaterialProxy = CloudVolumeMaterialProxy;
					CloudRC.SceneDepthZ = GSystemTextures.GetMaxFP16Depth(GraphBuilder);

					CloudRC.MainView = &CubeView; /// This is only accessing data that is not changing between view orientation. Such data are accessed from the ViewUniformBuffer. See CubeView comment above.

					CloudRC.bShouldViewRenderVolumetricRenderTarget = false;
					CloudRC.bIsReflectionRendering = true;
					CloudRC.bIsSkyRealTimeReflectionRendering = true;
					CloudRC.bSecondAtmosphereLightEnabled = IsSecondAtmosphereLightEnabledForCloud();

					CloudRC.bSkipAtmosphericLightShadowmap = !CaptureShadowFromOpaque;
					if (CaptureShadowFromOpaque)
					{
						FLightSceneInfo* AtmosphericLight0Info = Scene->AtmosphereLights[0];
						FLightSceneProxy* AtmosphericLight0 = AtmosphericLight0Info ? AtmosphericLight0Info->Proxy : nullptr;
						const FProjectedShadowInfo* ProjectedShadowInfo0 = nullptr;
						if (AtmosphericLight0Info)
						{
							ProjectedShadowInfo0 = GetFirstWholeSceneShadowMap(SceneRenderer.VisibleLightInfos[AtmosphericLight0Info->Id]);
						}

						// Get the main view shadow info for the cloud shadows in reflection.
						if (!CloudRC.bSkipAtmosphericLightShadowmap && AtmosphericLight0 && ProjectedShadowInfo0)
						{
							SetVolumeShadowingShaderParameters(GraphBuilder, CloudRC.LightShadowShaderParams0, MainView, AtmosphericLight0Info, ProjectedShadowInfo0);
						}
						else
						{
							SetVolumeShadowingDefaultShaderParameters(GraphBuilder, CloudRC.LightShadowShaderParams0);
						}
					}
					else
					{
						SetVolumeShadowingDefaultShaderParameters(GraphBuilder, CloudRC.LightShadowShaderParams0);
					}

					// Create default textures once for each faces
					CloudRC.CreateDefaultTexturesIfNeeded(GraphBuilder);
				}
				else
				{
					bShouldRenderVolumetricCloud = false; // Disable cloud rendering
				}
			}

			const FSkyLightSceneProxy* SkyLightProxy = SkyLight;
			auto SetupCommonViewUniformBufferParameters = [SkyLightProxy](FViewInfo& CubeView, FMatrix CubeProjectionMatrix, FViewMatrices& OutCubeViewMatrices, float CubeViewWidth, int32 CubeFace)
			{
				const FMatrix CubeViewRotationMatrix = CalcCubeFaceViewRotationMatrix((ECubeFace)CubeFace);

				FViewMatrices::FMinimalInitializer SceneCubeViewInitOptions;
				SceneCubeViewInitOptions.ConstrainedViewRect = FIntRect(FIntPoint(0, 0), FIntPoint(CubeViewWidth, CubeViewWidth));
				SceneCubeViewInitOptions.ViewRotationMatrix = CubeViewRotationMatrix;
				SceneCubeViewInitOptions.ViewOrigin = SkyLightProxy->CapturePosition;
				SceneCubeViewInitOptions.ProjectionMatrix = CubeProjectionMatrix;
				OutCubeViewMatrices = FViewMatrices(SceneCubeViewInitOptions);

				CubeView.SetupCommonViewUniformBufferParameters(
					*CubeView.CachedViewUniformShaderParameters,
					FIntPoint(CubeViewWidth, CubeViewWidth),
					1,
					FIntRect(FIntPoint(0, 0), FIntPoint(CubeViewWidth, CubeViewWidth)),
					OutCubeViewMatrices,
					OutCubeViewMatrices);

				// Notify the fact that we render a reflection, e.g. remove sun disk.
				CubeView.CachedViewUniformShaderParameters->RenderingReflectionCaptureMask = 1.0f;
				// Force-disable primitive alpha holdout for reflection captures
				CubeView.CachedViewUniformShaderParameters->bPrimitiveAlphaHoldoutEnabled = 0;
				// Notify the fact that we render a reflection, e.g. use special exposure.
				CubeView.CachedViewUniformShaderParameters->RealTimeReflectionCapture = 1.0f;
				// Clamp emissive to max10bits for real time capture
				const static float Max10BitsFloat = 64512.0f;
				CubeView.CachedViewUniformShaderParameters->MaterialMaxEmissiveValue = Max10BitsFloat;
			};

			auto SetupViewSkyAtmosphereParametersAndResources = [&](FViewInfo& OutView)
			{
				// We have rendered a sky dome with identity rotation at the SkyLight position for the capture.
				if (AtmosphereSetup)
				{
					FVector3f SkyCameraTranslatedWorldOrigin;
					FMatrix44f SkyViewLutReferential;
					FVector4f TempSkyPlanetData;
					if (MainView.bSceneHasSkyMaterial)
					{
						// Setup a constant referential for each of the faces of the dynamic reflection capture.
						// This is to have the FastSkyViewLUT match the one generated specifically for the capture point of view.
						const FVector3f SkyViewLutReferentialForward = FVector3f(1.0f, 0.0f, 0.0f);
						const FVector3f SkyViewLutReferentialRight = FVector3f(0.0f, 0.0f, -1.0f);
						AtmosphereSetup->ComputeViewData(
							SkyLight->CapturePosition, MainView.ViewMatrices.GetPreViewTranslation(), SkyViewLutReferentialForward, SkyViewLutReferentialRight,
							SkyCameraTranslatedWorldOrigin, TempSkyPlanetData, SkyViewLutReferential);
						OutView.CachedViewUniformShaderParameters->SkyViewLutTexture = RealTimeReflectionCaptureSkyAtmosphereViewLutTexture->GetRHI();
					}
					else
					{
						// Else if there is no sky material, we assume that no material is sampling the FastSkyViewLUT texture in the sky light reflection (bFastSky=bFastAerialPerspective=false).
						// But, we still need to udpate the sky parameters on the view according to the sky light capture position
						const FVector3f SkyViewLutReferentialForward = FVector3f(1.0f, 0.0f, 0.0f);
						const FVector3f SkyViewLutReferentialRight = FVector3f(0.0f, 0.0f, -1.0f);
						// LWC_TODO: SkyPlanetTranslatedWorldCenterAndViewHeight is FVector4f because it's from a shader, and will have lost precision already.
						AtmosphereSetup->ComputeViewData(
							SkyLight->CapturePosition, MainView.ViewMatrices.GetPreViewTranslation(), SkyViewLutReferentialForward, SkyViewLutReferentialRight,
							SkyCameraTranslatedWorldOrigin, TempSkyPlanetData, SkyViewLutReferential);
					}

					OutView.CachedViewUniformShaderParameters->SkyPlanetTranslatedWorldCenterAndViewHeight = TempSkyPlanetData;
					OutView.CachedViewUniformShaderParameters->SkyCameraTranslatedWorldOrigin = SkyCameraTranslatedWorldOrigin;
					OutView.CachedViewUniformShaderParameters->SkyViewLutReferential = SkyViewLutReferential;
				}

				if (HasSkyAtmosphere() && (MainView.bSceneHasSkyMaterial || HasVolumetricCloud())
					&& RealTimeReflectionCaptureCamera360APLutTexture.IsValid())	// we also check that because it seems it can happen for some view setup UE-107270, TODO find a repro for a proper fix.
				{
					OutView.CachedViewUniformShaderParameters->CameraAerialPerspectiveVolume = RealTimeReflectionCaptureCamera360APLutTexture->GetRHI();
				}
				else
				{
					OutView.CachedViewUniformShaderParameters->CameraAerialPerspectiveVolume = GSystemTextures.VolumetricBlackDummy->GetRHI();
				}
			};

			// Render clouds in separate textures all at once in parallel with compute work that overlaps on GPU to be faster.
			// The texture will be composited later using the ApplyLowerHemisphereColor pass.
			const int32 CloudResolutionDivider = FMath::Clamp(CVarRealTimeReflectionCaptureVolumetricCloudResolutionDivider.GetValueOnRenderThread(), 1, 8);
			FRDGTextureRef LowResCloudTextureArray[6] = { nullptr, nullptr, nullptr, nullptr, nullptr, nullptr };
			if (bShouldRenderVolumetricCloud && bExecuteCloud)
			{
				for (int32 CubeFace = StartCubeFace; CubeFace < EndCubeFace; CubeFace++)
				{
					RDG_EVENT_SCOPE(GraphBuilder, "Capture Volumetric Cloud Face=%d", CubeFace);
					const uint32 CubeWidthLow = CubeWidth / CloudResolutionDivider;

					// Create a texture to render the clouds at a given resolution
					FRDGTextureRef& LowResCloudTexture = LowResCloudTextureArray[CubeFace];
					FRDGTextureDesc LowResCloudTexDesc = FRDGTextureDesc::Create2D(FIntPoint(CubeWidthLow, CubeWidthLow), PF_FloatRGBA,
						FClearValueBinding::Black,
						TexCreate_RenderTargetable | TexCreate_UAV | TexCreate_ShaderResource);
					LowResCloudTexture = GraphBuilder.CreateTexture(LowResCloudTexDesc, TEXT("SkyLight.LowResCloudTexture"));

					if (CloudRC.CloudInfo->GetVolumetricCloudSceneProxy().bVisibleInRealTimeSkyCaptures)
					{
						FMatrix LowResCubeProjectionMatrix;
						FViewInfo* LowResCubeViewPtr = nullptr;
						CreateMainViewSnapshotForRealTimeCapture(LowResCubeViewPtr, LowResCubeProjectionMatrix, CubeWidthLow);
						FViewInfo& LowResCubeView = *LowResCubeViewPtr;

						FViewMatrices CubeViewMatrices;
						SetupCommonViewUniformBufferParameters(LowResCubeView, LowResCubeProjectionMatrix, CubeViewMatrices, CubeWidthLow, CubeFace);

						// We have rendered a sky dome with identity rotation at the SkyLight position for the capture.
						SetupViewSkyAtmosphereParametersAndResources(LowResCubeView);

						LowResCubeView.CreateViewUniformBuffers(*LowResCubeView.CachedViewUniformShaderParameters);
						CloudRC.ViewUniformBuffer = LowResCubeView.ViewUniformBuffer;


						// Render
						CloudRC.RenderTargets[0] = FRenderTargetBinding(LowResCloudTexture, ERenderTargetLoadAction::EClear);
						CloudRC.bDisableCloudBlending = true;

						FCloudShadowAOData CloudShadowAOData;
						GetCloudShadowAOData(GetVolumetricCloudSceneInfo(), CubeView, GraphBuilder, CloudShadowAOData);
						SkyRC.bShouldSampleCloudShadow = CloudShadowAOData.bShouldSampleCloudShadow;
						SkyRC.VolumetricCloudShadowMap[0] = CloudShadowAOData.VolumetricCloudShadowMap[0];
						SkyRC.VolumetricCloudShadowMap[1] = CloudShadowAOData.VolumetricCloudShadowMap[1];
						SkyRC.bShouldSampleCloudSkyAO = CloudShadowAOData.bShouldSampleCloudSkyAO;
						SkyRC.VolumetricCloudSkyAO = CloudShadowAOData.VolumetricCloudSkyAO;
						// TODO this CloudShadowAOData management looks a bit heavy, simplify or make common
						CloudRC.VolumetricCloudShadowTexture[0] = CloudShadowAOData.VolumetricCloudShadowMap[0];
						CloudRC.VolumetricCloudShadowTexture[1] = CloudShadowAOData.VolumetricCloudShadowMap[1];

						SceneRenderer.RenderVolumetricCloudsInternal(GraphBuilder, CloudRC, InstanceCullingManager, FIntPoint(CubeWidthLow, CubeWidthLow));
					}
					else
					{
						AddClearRenderTargetPass(GraphBuilder, LowResCloudTexture);
					}
				}
			}

			const bool bAlwaysClearColorBuffer = CVarRealTimeReflectionCaptureAlwaysClearColorBuffer.GetValueOnRenderThread() > 0;
			for (int32 CubeFace = StartCubeFace; CubeFace < EndCubeFace; CubeFace++)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "Capture Face=%d", CubeFace);

				SkyRC.RenderTargets[0] = FRenderTargetBinding(SkyCubeTexture, ERenderTargetLoadAction::ELoad, 0, CubeFace);

				FViewMatrices CubeViewMatrices;
				SetupCommonViewUniformBufferParameters(CubeView, CubeProjectionMatrix, CubeViewMatrices, CubeWidth, CubeFace);

				// We have rendered a sky dome with identity rotation at the SkyLight position for the capture.
				SetupViewSkyAtmosphereParametersAndResources(CubeView);

				CubeView.CreateViewUniformBuffers(*CubeView.CachedViewUniformShaderParameters);

				SkyRC.ViewUniformBuffer = CubeView.ViewUniformBuffer;

				SkyRC.SceneUniformBuffer = SceneRenderer.GetSceneUniforms().GetBuffer(GraphBuilder);

				SkyRC.ViewMatrices = &CubeViewMatrices;
				SkyRC.bSceneHasSkyMaterial = MainView.bSceneHasSkyMaterial;

				SkyRC.SkyAtmosphereViewLutTexture = BlackDummy2dTex;
				SkyRC.SkyAtmosphereCameraAerialPerspectiveVolume = BlackDummy3dTex;
				SkyRC.SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly = BlackDummy3dTex;
				SkyRC.SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly = BlackDummy3dTex;

				SkyRC.Viewport = FIntRect(FIntPoint(0, 0), FIntPoint(CubeWidth, CubeWidth));
				SkyRC.bIsReflectionCapture = true;
				SkyRC.bRenderSkyPixel = true;
				SkyRC.AerialPerspectiveStartDepthInCm = 0.01f;
				SkyRC.NearClippingDistance = 0.01f;
				SkyRC.FeatureLevel = FeatureLevel;

				FCloudShadowAOData CloudShadowAOData;
				GetCloudShadowAOData(GetVolumetricCloudSceneInfo(), CubeView, GraphBuilder, CloudShadowAOData);
				SkyRC.bShouldSampleCloudShadow = CloudShadowAOData.bShouldSampleCloudShadow;
				SkyRC.VolumetricCloudShadowMap[0] = CloudShadowAOData.VolumetricCloudShadowMap[0];
				SkyRC.VolumetricCloudShadowMap[1] = CloudShadowAOData.VolumetricCloudShadowMap[1];
				SkyRC.bShouldSampleCloudSkyAO = CloudShadowAOData.bShouldSampleCloudSkyAO;
				SkyRC.VolumetricCloudSkyAO = CloudShadowAOData.VolumetricCloudSkyAO;

				// Note: 
				//	- The depth texture is here so that the ordering of IsSky material meshes is correct when rendering the sky into cube map.
				//	- This depth is also used to apply distance based fog (usefull when one want to capture a terrain at the bottom of the skylight).
				//	- If that behavior is not needed, use r.SkyLight.RealTimeReflectionCapture.DepthBuffer 0.
				const bool bUseDepthBuffer = CVarRealTimeReflectionCaptureDepthBuffer.GetValueOnRenderThread() > 0;
				FRDGTextureRef CubeDepthTexture = nullptr;

				const bool bIsMobilePlatform = IsMobilePlatform(MainView.GetShaderPlatform());
				if (bExecuteSky)
				{
					if (bAlwaysClearColorBuffer)
					{
						ClearCubeFace(SkyCubeTexture, CubeFace);
					}

					if(MainView.bSceneHasSkyMaterial || bShouldRenderSkyAtmosphere)
					{
						// If there are any mesh tagged as IsSky then we render them only, otherwise we simply render the sky atmosphere itself.
						if (MainView.bSceneHasSkyMaterial)	// TODO skypass using FMobileBasePassUniformParameters
						{
							if (bIsMobilePlatform)
							{
								RDG_EVENT_SCOPE(GraphBuilder, "Capture Sky Materials", CubeFace);
								auto* PassParameters = GraphBuilder.AllocParameters<FMobileCaptureSkyMeshReflectionPassParameters>();
								PassParameters->View = CubeView.GetShaderParameters();
								PassParameters->RenderTargets = SkyRC.RenderTargets;
								const EMobileBasePass BasePass = EMobileBasePass::Opaque;
								PassParameters->BasePass = CreateMobileBasePassUniformBuffer(GraphBuilder, MainView, BasePass, EMobileSceneTextureSetupMode::SceneDepth, {}, true);
					
								if (bUseDepthBuffer)
								{
									FRDGTextureDesc CubeDepthTextureDesc = FRDGTextureDesc::Create2D(FIntPoint(CubeWidth, CubeWidth), PF_ShadowDepth,
										MainView.GetSceneTexturesConfig().DepthClearValue,
										TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
									CubeDepthTexture = GraphBuilder.CreateTexture(CubeDepthTextureDesc, TEXT("SkyLight.CubeDepthTexture"));
									PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(CubeDepthTexture, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilNop);
								}
					
								AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, MainView, &InstanceCullingManager, RDG_EVENT_NAME("CaptureSkyMeshReflection"), SkyRC.Viewport,
								[&MainView, CubeViewUniformBuffer = CubeView.ViewUniformBuffer, bUseDepthBuffer, Scene](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
								{
									FMeshPassProcessorRenderState DrawRenderState;
									FSkyPassMeshProcessor::ESkyPassType SkyPassType = FSkyPassMeshProcessor::ESkyPassType::SPT_Default;
					
									DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA>::GetRHI());
									if (bUseDepthBuffer)
									{
										DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilNop);
										DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());

										SkyPassType = FSkyPassMeshProcessor::ESkyPassType::SPT_RealTimeCapture_DepthWrite;
									}
									else
									{
										DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthNop_StencilNop);
										DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

										SkyPassType = FSkyPassMeshProcessor::ESkyPassType::SPT_RealTimeCapture_DepthNop;
									}
					
									FSkyPassMeshProcessor PassMeshProcessor(Scene, Scene->GetFeatureLevel(), nullptr, DrawRenderState, DynamicMeshPassContext);
									PassMeshProcessor.SkyPassType = SkyPassType;

									const int32 SkyRealTimeReflectionOnlyMeshBatcheCount = MainView.SkyMeshBatches.Num();
									for (int32 MeshBatchIndex = 0; MeshBatchIndex < SkyRealTimeReflectionOnlyMeshBatcheCount; ++MeshBatchIndex)
									{
										FSkyMeshBatch& SkyMeshBatch = MainView.SkyMeshBatches[MeshBatchIndex];
										if (!SkyMeshBatch.bVisibleInRealTimeSkyCapture)
										{
											continue;
										}
					
										const FMeshBatch* MeshBatch = SkyMeshBatch.Mesh;
										const FPrimitiveSceneProxy* PrimitiveSceneProxy = SkyMeshBatch.Proxy;
										const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
					
										// Real time sky light capture cannot render dynamic meshes for now.
										// For those to be rendered we would need to specify a view to the PassMeshProcessor creation above.
										// Dynamic draws uses temporary per frame & per view data (appended at the end of the GPUScene buffer).
										// But the view is transient and data on it can morph, and correct data would need to be added to FGPUScenePrimitiveCollector (see UploadDynamicPrimitiveShaderDataForViewInternal)
										bool bSkipDynamicMesh = false;
										for (auto& Element : MeshBatch->Elements)
										{
											if (Element.PrimitiveIdMode == PrimID_DynamicPrimitiveShaderData)
											{
												bSkipDynamicMesh = true;
											}
										}
										if (bSkipDynamicMesh)
										{
											continue;
										}
					
										const uint64 DefaultBatchElementMask = ~0ull;
										PassMeshProcessor.AddMeshBatch(*MeshBatch, DefaultBatchElementMask, PrimitiveSceneProxy);
									}
								});
							}
							else
							{
								RDG_EVENT_SCOPE(GraphBuilder, "Capture Sky Materials", CubeFace);
								auto* PassParameters = GraphBuilder.AllocParameters<FCaptureSkyMeshReflectionPassParameters>();
								PassParameters->View = CubeView.GetShaderParameters();
								PassParameters->RenderTargets = SkyRC.RenderTargets;

								const int32 ViewIndex = 0;
								const bool bEnableSkylight = Scene->ShouldRenderSkylightInBasePass(/*bIsTranslucent*/false);
								const bool bForRealtimeSkyCapture = true;
								PassParameters->BasePass = CreateOpaqueBasePassUniformBuffer(GraphBuilder, MainView, ViewIndex, {}, {}, bEnableSkylight, bForRealtimeSkyCapture);
					
								// Setup the depth buffer
								if (bUseDepthBuffer)
								{
									FRDGTextureDesc CubeDepthTextureDesc = FRDGTextureDesc::Create2D(FIntPoint(CubeWidth, CubeWidth), PF_ShadowDepth,
										MainView.GetSceneTexturesConfig().DepthClearValue,
										TexCreate_DepthStencilTargetable | TexCreate_ShaderResource);
									CubeDepthTexture = GraphBuilder.CreateTexture(CubeDepthTextureDesc, TEXT("SkyLight.CubeDepthTexture"));
									PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(CubeDepthTexture, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilNop);
								}
					
								AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, MainView, &InstanceCullingManager, RDG_EVENT_NAME("CaptureSkyMeshReflection"), SkyRC.Viewport,
								[&MainView, CubeViewUniformBuffer = CubeView.ViewUniformBuffer, bUseDepthBuffer, Scene](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
								{
									FMeshPassProcessorRenderState DrawRenderState;

									FExclusiveDepthStencil::Type BasePassDepthStencilAccess_Sky = bUseDepthBuffer ? FExclusiveDepthStencil::Type(Scene->DefaultBasePassDepthStencilAccess | FExclusiveDepthStencil::DepthWrite)
										: FExclusiveDepthStencil::Type(Scene->DefaultBasePassDepthStencilAccess & ~FExclusiveDepthStencil::DepthWrite);
									SetupBasePassState(BasePassDepthStencilAccess_Sky, false, DrawRenderState);

									DrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_RGBA>::GetRHI());
									if (bUseDepthBuffer)
									{
										DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthWrite_StencilNop);
										DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
									}
									else
									{
										DrawRenderState.SetDepthStencilAccess(FExclusiveDepthStencil::DepthNop_StencilNop);
										DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
									}
					
									FSkyPassMeshProcessor PassMeshProcessor(Scene, Scene->GetFeatureLevel(), nullptr, DrawRenderState, DynamicMeshPassContext);
									const int32 SkyRealTimeReflectionOnlyMeshBatcheCount = MainView.SkyMeshBatches.Num();
									for (int32 MeshBatchIndex = 0; MeshBatchIndex < SkyRealTimeReflectionOnlyMeshBatcheCount; ++MeshBatchIndex)
									{
										FSkyMeshBatch& SkyMeshBatch = MainView.SkyMeshBatches[MeshBatchIndex];
										if (!SkyMeshBatch.bVisibleInRealTimeSkyCapture)
										{
											continue;
										}
					
										const FMeshBatch* MeshBatch = SkyMeshBatch.Mesh;
										const FPrimitiveSceneProxy* PrimitiveSceneProxy = SkyMeshBatch.Proxy;
										const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
					
										// Real time sky light capture cannot render dynamic meshes for now.
										// For those to be rendered we would need to specify a view to the PassMeshProcessor creation above.
										// Dynamic draws uses temporary per frame & per view data (appended at the end of the GPUScene buffer).
										// But the view is transient and data on it can morph, and correct data would need to be added to FGPUScenePrimitiveCollector (see UploadDynamicPrimitiveShaderDataForViewInternal)
										bool bSkipDynamicMesh = false;
										for (auto& Element : MeshBatch->Elements)
										{
											if (Element.PrimitiveIdMode == PrimID_DynamicPrimitiveShaderData)
											{
												bSkipDynamicMesh = true;
											}
										}
										if (bSkipDynamicMesh)
										{
											continue;
										}
					
										const uint64 DefaultBatchElementMask = ~0ull;
										PassMeshProcessor.AddMeshBatch(*MeshBatch, DefaultBatchElementMask, PrimitiveSceneProxy);
									}
								});
								}
						}
						else if (!bIsMobilePlatform)
						{
							// TODO: mobile should never get there
							RDG_EVENT_SCOPE(GraphBuilder, "Capture Sky Raw", CubeFace);
							FSceneTextureShaderParameters SceneTextures = CreateSceneTextureShaderParameters(GraphBuilder, &SceneRenderer.GetActiveSceneTextures(), SceneRenderer.FeatureLevel, ESceneTextureSetupMode::SceneDepth);
							SceneRenderer.RenderSkyAtmosphereInternal(GraphBuilder, SceneTextures, SkyRC);
						}
						else if (bIsMobilePlatform)	// The SkyAtmosphere full screen pass shader is not compiled on mobile, so clear the capture to black.
						{
							// No sky dome mesh and no sky rendering on mobile so let's cleat to black.
							ClearCubeFace(SkyCubeTexture, CubeFace);
						}

						// Also render the height fog as part of the sky render pass when time slicing is enabled.
						
						if (Scene && Scene->HasAnyExponentialHeightFog() && Scene->ExponentialFogs[0].bVisibleInRealTimeSkyCaptures)
						{
							FRenderRealTimeReflectionHeightFogVS::FPermutationDomain VsPermutationVector;
							TShaderMapRef<FRenderRealTimeReflectionHeightFogVS> VertexShader(GetGlobalShaderMap(SkyRC.FeatureLevel), VsPermutationVector);

							FRenderRealTimeReflectionHeightFogPS::FPermutationDomain PsPermutationVector;
							PsPermutationVector.Set<FRenderRealTimeReflectionHeightFogPS::FDepthTexture>(bUseDepthBuffer && CubeDepthTexture);
							TShaderMapRef<FRenderRealTimeReflectionHeightFogPS> PixelShader(GetGlobalShaderMap(SkyRC.FeatureLevel), PsPermutationVector);

							FRenderRealTimeReflectionHeightFogPS::FParameters* PsPassParameters = GraphBuilder.AllocParameters<FRenderRealTimeReflectionHeightFogPS::FParameters>();
							PsPassParameters->ViewUniformBuffer = CubeView.ViewUniformBuffer;
							PsPassParameters->RenderTargets = SkyRC.RenderTargets;
							PsPassParameters->DepthTexture = (bUseDepthBuffer && CubeDepthTexture) ? CubeDepthTexture : BlackDummy2dTex;
							PsPassParameters->FogStruct = CreateFogUniformBuffer(GraphBuilder, CubeView);
							PsPassParameters->SkyLightPosition = FVector3f(SkyLightProxy->CapturePosition);

							ClearUnusedGraphResources(PixelShader, PsPassParameters);

							// Render height fog at an infinite distance since real time reflections does not have a depth buffer for now.
							// Volumetric fog is not supported in such reflections.
							GraphBuilder.AddPass(
								RDG_EVENT_NAME("DistantHeightFog"),
								PsPassParameters,
								ERDGPassFlags::Raster,
								[PsPassParameters, VertexShader, PixelShader, CubeWidth](FRDGAsyncTask, FRHICommandList& RHICmdListLambda)
								{
									RHICmdListLambda.SetViewport(0.0f, 0.0f, 0.0f, CubeWidth, CubeWidth, 1.0f);

									FGraphicsPipelineStateInitializer GraphicsPSOInit;
									RHICmdListLambda.ApplyCachedRenderTargets(GraphicsPSOInit);

									GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_One>::GetRHI();
									GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
									GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
									GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
									GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
									GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
									GraphicsPSOInit.PrimitiveType = PT_TriangleList;
									SetGraphicsPipelineState(RHICmdListLambda, GraphicsPSOInit, 0);

									FRenderRealTimeReflectionHeightFogVS::FParameters VsPassParameters;
									VsPassParameters.ViewUniformBuffer = PsPassParameters->ViewUniformBuffer;
									SetShaderParameters(RHICmdListLambda, VertexShader, VertexShader.GetVertexShader(), VsPassParameters);
									SetShaderParameters(RHICmdListLambda, PixelShader, PixelShader.GetPixelShader(), *PsPassParameters);

									RHICmdListLambda.DrawPrimitive(0, 1, 1);
								});
						}
					}
					else if(!bAlwaysClearColorBuffer) // Only clear if not done before.
					{
						ClearCubeFace(SkyCubeTexture, CubeFace);
					}
				}

				// Now composited: (1) clouds rendered in isolated texture, and (2) blend over the lower hemisphere color.
				FRDGTextureRef& LowResCloudTexture = LowResCloudTextureArray[CubeFace];
				if ((SkyLight->bLowerHemisphereIsSolidColor || LowResCloudTexture !=nullptr) && bExecuteCloud)
				{
					FRenderRealTimeReflectionHeightFogVS::FPermutationDomain VsPermutationVector;
					TShaderMapRef<FRenderRealTimeReflectionHeightFogVS> VertexShader(GetGlobalShaderMap(SkyRC.FeatureLevel), VsPermutationVector);

					FApplyLowerHemisphereColorPS::FPermutationDomain PsPermutationVector;
					TShaderMapRef<FApplyLowerHemisphereColorPS> PixelShader(GetGlobalShaderMap(SkyRC.FeatureLevel), PsPermutationVector);

					FApplyLowerHemisphereColorPS::FParameters* PsPassParameters = GraphBuilder.AllocParameters<FApplyLowerHemisphereColorPS::FParameters>();
					PsPassParameters->RenderTargets = SkyRC.RenderTargets;
					PsPassParameters->ViewUniformBuffer = CubeView.ViewUniformBuffer;
					PsPassParameters->ApplyLowerHemisphereColor = SkyLight->bLowerHemisphereIsSolidColor ? 1 : 0;
					PsPassParameters->LowerHemisphereSolidColor = SkyLight->LowerHemisphereColor;
					PsPassParameters->ApplyLowResCloudTexture = LowResCloudTexture ? 1 : 0;
					PsPassParameters->LowResCloudTexture = LowResCloudTexture ? LowResCloudTexture : GSystemTextures.GetBlackAlphaOneDummy(GraphBuilder);
					PsPassParameters->LowResCloudSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();;
					PsPassParameters->CubeFace = CubeFace;
					PsPassParameters->SvPositionToUVScale = FVector2f(1.0f / float(CubeWidth), 1.0f / float(CubeWidth));

					// Render height fog at an infinite distance since real time reflections does not have a depth buffer for now.
					// Volumetric fog is not supported in such reflections.
					GraphBuilder.AddPass(
						RDG_EVENT_NAME("ApplyLowerHemisphereColor"),
						PsPassParameters,
						ERDGPassFlags::Raster,
						[PsPassParameters, VertexShader, PixelShader, CubeWidth](FRDGAsyncTask, FRHICommandList& RHICmdListLambda)
						{
							RHICmdListLambda.SetViewport(0.0f, 0.0f, 0.0f, CubeWidth, CubeWidth, 1.0f);

							FGraphicsPipelineStateInitializer GraphicsPSOInit;
							RHICmdListLambda.ApplyCachedRenderTargets(GraphicsPSOInit);

							GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGB, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_Zero>::GetRHI();
							GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
							GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
							GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
							GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
							GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
							GraphicsPSOInit.PrimitiveType = PT_TriangleList;
							SetGraphicsPipelineState(RHICmdListLambda, GraphicsPSOInit, 0);

							FRenderRealTimeReflectionHeightFogVS::FParameters VsPassParameters;
							VsPassParameters.ViewUniformBuffer = PsPassParameters->ViewUniformBuffer;
							SetShaderParameters(RHICmdListLambda, VertexShader, VertexShader.GetVertexShader(), VsPassParameters);
							SetShaderParameters(RHICmdListLambda, PixelShader, PixelShader.GetPixelShader(), *PsPassParameters);

							RHICmdListLambda.DrawPrimitive(0, 1, 1);
						});
				}
			}
		}
		else
		{
			for (int32 CubeFace = 0; CubeFace < CubeFace_MAX; CubeFace++)
			{
				ClearCubeFace(SkyCubeTexture, CubeFace);
			}
		}
	};

	auto RenderCubeFaces_GenCubeMips = [&](uint32 CubeMipStart, uint32 CubeMipEnd, TRefCountPtr<IPooledRenderTarget>& SkyRenderTarget)
	{
		// on mobile platforms use the pixel shader implementation as compute shaders might not be optimal
		const bool bIsMobilePlatform = IsMobilePlatform(MainView.GetShaderPlatform());
		if (bIsMobilePlatform)
		{
			FRDGTextureRef SkyCubeTexture = GraphBuilder.RegisterExternalTexture(SkyRenderTarget, TEXT("SkyRenderTarget"));
			CreateCubeMips(GraphBuilder, GetGlobalShaderMap(FeatureLevel), SkyCubeTexture);
		}
		else
		{
			check(CubeMipStart > 0);	// Never write to mip0 as it has just been redered into

			FRDGTextureRef SkyCubeTexture = GraphBuilder.RegisterExternalTexture(SkyRenderTarget, TEXT("SkyRenderTarget"));

			FDownsampleCubeFaceCS::FPermutationDomain PermutationVector;
			TShaderMapRef<FDownsampleCubeFaceCS> ComputeShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);

			for (uint32 MipIndex = CubeMipStart; MipIndex <= CubeMipEnd; MipIndex++)
			{
				const uint32 MipResolution = 1 << (CubeMipCount - MipIndex - 1);
				FRDGTextureSRVRef SkyCubeTextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SkyCubeTexture, MipIndex - 1)); // slice/face selection is useless so remove from CreateForMipLevel

				FDownsampleCubeFaceCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDownsampleCubeFaceCS::FParameters>();
				PassParameters->MipIndex = MipIndex;
				PassParameters->NumMips = CubeMipCount;
				PassParameters->CubeFace = 0; // unused
				PassParameters->ValidDispatchCoord = FIntPoint(MipResolution, MipResolution);
				PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Point>::GetRHI();

				PassParameters->SourceCubemapTexture = SkyCubeTextureSRV;
				FRDGTextureUAVDesc OutTextureMipColorDesc(SkyCubeTexture, MipIndex);
				OutTextureMipColorDesc.DimensionOverride = ETextureDimension::Texture2DArray;
				PassParameters->OutTextureMipColor = GraphBuilder.CreateUAV(OutTextureMipColorDesc);

				FIntVector NumGroups = FIntVector::DivideAndRoundUp(FIntVector(MipResolution, MipResolution, 1), FIntVector(FDownsampleCubeFaceCS::ThreadGroupSize, FDownsampleCubeFaceCS::ThreadGroupSize, 1));

				// The groupd size per face with padding
				PassParameters->FaceThreadGroupSize = NumGroups.X * FDownsampleCubeFaceCS::ThreadGroupSize;

				// We are going to dispatch once for all faces 
				NumGroups.X *= 6;

				// Dispatch with GenerateMips: reading from a slice through SRV and writing into lower mip through UAV.
				ClearUnusedGraphResources(ComputeShader, PassParameters);
				GraphBuilder.AddPass(
					Forward<FRDGEventName>(RDG_EVENT_NAME("MipGen")),
					PassParameters,
					ERDGPassFlags::Compute,
				[PassParameters, ComputeShader, NumGroups](FRDGAsyncTask, FRHICommandList& RHICmdList)
				{
					FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, *PassParameters, NumGroups);
				});
			}
		}
		
	};

	auto RenderCubeFaces_SpecularConvolution = [&](uint32 CubeMipStart, uint32 CubeMipEnd, uint32 FaceStart, uint32 FaceCount, TRefCountPtr<IPooledRenderTarget>& DstRenderTarget, TRefCountPtr<IPooledRenderTarget>& SrcRenderTarget)
	{
		check((FaceStart + FaceCount) <= 6);
		FRDGTextureRef RDGSrcRenderTarget = GraphBuilder.RegisterExternalTexture(SrcRenderTarget);
		FRDGTextureRef RDGDstRenderTarget = GraphBuilder.RegisterExternalTexture(DstRenderTarget);

		// on mobile platforms use the pixel shader convolution as compute shaders might not be optimal
		const bool bIsMobilePlatform = IsMobilePlatform(MainView.GetShaderPlatform());
		if (bIsMobilePlatform)
		{
			ConvolveCubeMap(GraphBuilder, GetGlobalShaderMap(FeatureLevel), CubeMipStart, CubeMipEnd, FaceStart, FaceCount, RDGSrcRenderTarget, RDGDstRenderTarget);
		}
		else
		{
			FRDGTextureSRVRef RDGSrcRenderTargetSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(RDGSrcRenderTarget));

			FDownsampleCubeFaceCS::FPermutationDomain PermutationVector;
			TShaderMapRef<FConvolveSpecularFaceCS> ComputeShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);
			for (uint32 MipIndex = CubeMipStart; MipIndex <= CubeMipEnd; MipIndex++)
			{
				const uint32 MipResolution = 1 << (CubeMipCount - MipIndex - 1);
		
				FConvolveSpecularFaceCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FConvolveSpecularFaceCS::FParameters>();
				PassParameters->MipIndex = MipIndex;
				PassParameters->NumMips = CubeMipCount;
				PassParameters->CubeFace = 0; // unused
				PassParameters->CubeFaceOffset = int(FaceStart);
				PassParameters->ValidDispatchCoord = FIntPoint(MipResolution, MipResolution);
				PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Point>::GetRHI();
		
				PassParameters->SourceCubemapTexture = RDGSrcRenderTargetSRV;
				FRDGTextureUAVDesc OutTextureMipColorDesc(RDGDstRenderTarget, MipIndex);
				OutTextureMipColorDesc.DimensionOverride = ETextureDimension::Texture2DArray;
				PassParameters->OutTextureMipColor = GraphBuilder.CreateUAV(OutTextureMipColorDesc);
		
				FIntVector NumGroups = FIntVector::DivideAndRoundUp(FIntVector(MipResolution, MipResolution, 1), FIntVector(FConvolveSpecularFaceCS::ThreadGroupSize, FConvolveSpecularFaceCS::ThreadGroupSize, 1));
		
				// The groupd size per face with padding
				PassParameters->FaceThreadGroupSize = NumGroups.X * FConvolveSpecularFaceCS::ThreadGroupSize;
		
				// We are going to dispatch once for all faces 
				NumGroups.X *= FaceCount;
		
				FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("Convolve"), ComputeShader, PassParameters, NumGroups);
			}
		}
	};

	auto RenderCubeFaces_DiffuseIrradiance = [&](TRefCountPtr<IPooledRenderTarget>& SourceCubemap)
	{
		FRDGTextureRef SourceCubemapTexture = GraphBuilder.RegisterExternalTexture(SourceCubemap);
		FRDGTextureSRVRef SourceCubemapTextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceCubemapTexture));
		
		// ForceImmediateFirstBarrier is required because the RHI resource is used as an SRV outside of RDG prior to this UAV pass. Without
		// the flag, RDG will split the transition to UAV to the start of the graph, which results in a validation error. With the flag, RDG
		// will transition to UAV at the start of the pass instead.
		FRDGBuffer* SkyIrradianceEnvironmentMapRDG = GraphBuilder.RegisterExternalBuffer(SkyIrradianceEnvironmentMap, ERDGBufferFlags::ForceImmediateFirstBarrier); // TODO SkyIrradianceEnvironmentMap is nullptr
		GraphBuilder.UseInternalAccessMode(SkyIrradianceEnvironmentMapRDG);

		TShaderMapRef<FComputeSkyEnvMapDiffuseIrradianceCS> ComputeShader(GetGlobalShaderMap(FeatureLevel));

		const float SampleCount = FComputeSkyEnvMapDiffuseIrradianceCS::ThreadGroupSizeX * FComputeSkyEnvMapDiffuseIrradianceCS::ThreadGroupSizeY;
		const float UniformSampleSolidAngle = 4.0f * PI / SampleCount; // uniform distribution

		FComputeSkyEnvMapDiffuseIrradianceCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComputeSkyEnvMapDiffuseIrradianceCS::FParameters>();
		PassParameters->SourceCubemapSampler = TStaticSamplerState<SF_Point>::GetRHI();
		PassParameters->SourceCubemapTexture = SourceCubemapTextureSRV;
		PassParameters->OutIrradianceEnvMapSH = GraphBuilder.CreateUAV(SkyIrradianceEnvironmentMapRDG);
		PassParameters->UniformSampleSolidAngle = UniformSampleSolidAngle;

		// For 64 uniform samples on the unit sphere, we roughly have 10 samples per face.
		// Considering mip generation and bilinear sampling, we can assume 10 samples is enough to integrate 10*4=40 texels.
		// With that, we target integration of 16*16 face.
		const uint32 Log2_16 = 4; // FMath::Log2(16.0f)
		PassParameters->MipIndex = uint32(FMath::Log2(float(CapturedSkyRenderTarget->GetDesc().GetSize().X))) - Log2_16;

		const FIntVector NumGroups = FIntVector(1, 1, 1);
		FComputeShaderUtils::AddPass(GraphBuilder, RDG_EVENT_NAME("ComputeSkyEnvMapDiffuseIrradianceCS"), ComputeShader, PassParameters, NumGroups);

		const bool bIsMobilePlatform = IsMobilePlatform(MainView.GetShaderPlatform());
		if (bIsMobilePlatform)
		{
			TSharedPtr<FRHIGPUBufferReadback> NewReadBack = MakeShared<FRHIGPUBufferReadback>(TEXT("MobileSkyLightRealTimeCaptureIrradianceReadBack"));
			AddEnqueueCopyPass(GraphBuilder, NewReadBack.Get(), SkyIrradianceEnvironmentMapRDG, SkyIrradianceEnvironmentMapRDG->GetSize());

			FScene* Scene = MainView.Family->Scene->GetRenderScene();
			Scene->MobileSkyLightRealTimeCaptureIrradianceReadBackQueries.Enqueue(NewReadBack);
		}

		ExternalAccessQueue.Add(SkyIrradianceEnvironmentMapRDG, ERHIAccess::SRVMask, ERHIPipeline::All);
	};


	auto Mobile_ReadBackSkyIrradianceEnvironmentMap = [&]()
	{
		const bool bIsMobilePlatform = IsMobilePlatform(MainView.GetShaderPlatform());
		if (bIsMobilePlatform)
		{
			FScene* Scene = MainView.Family->Scene->GetRenderScene();

			// Add a readback real tiem capture irradiance buffer

			TSharedPtr<FRHIGPUBufferReadback> AvailableReadback;
			Scene->MobileSkyLightRealTimeCaptureIrradianceReadBackQueries.Peek(AvailableReadback);
			if (AvailableReadback.IsValid() && AvailableReadback->IsReady())
			{
				// Update the irradiance as soon as possible
				Scene->MobileSkyLightRealTimeCaptureIrradianceReadBackQueries.Dequeue(AvailableReadback);

				// Access the data and copy to a frame transient buffer for rendering pass.
				uint64 SizeBytes = AvailableReadback->GetGPUSizeBytes();
				void* SrcSkyIrradianceEnvironmentMap = AvailableReadback->Lock(SizeBytes);
				if (SrcSkyIrradianceEnvironmentMap == nullptr)
				{
					// Reset to 0 in case this happens...
					//SkyLight->IrradianceEnvironmentMap.R = TSHVector<3>();
					//SkyLight->IrradianceEnvironmentMap.G = TSHVector<3>();
					//SkyLight->IrradianceEnvironmentMap.B = TSHVector<3>();
					// Keep the last value
				}
				else
				{
					check(sizeof(Scene->MobileSkyLightRealTimeCaptureIrradianceEnvironmentMap) <= SizeBytes);
					memcpy(Scene->MobileSkyLightRealTimeCaptureIrradianceEnvironmentMap, SrcSkyIrradianceEnvironmentMap, sizeof(Scene->MobileSkyLightRealTimeCaptureIrradianceEnvironmentMap));
				}
			}
			// else, we keep the last value
		}
	};

	const uint32 LastMipLevel = CubeMipCount - 1;

	// Ensure the main view got the full cubemap by running all the capture operations for the first frame.
	// This ensures a proper initial state when time-slicing the steps.

	// Update the firt frame detection state variable
	if (bTimeSlicedRealTimeCapture)
	{
		// Go to next state iff this is a new frame
		if (bIsNewFrame)
		{
			switch (Capture.FirstFrameState)
			{
			case FRealTimeSlicedReflectionCapture::EFirstFrameState::INIT:
				Capture.FirstFrameState = FRealTimeSlicedReflectionCapture::EFirstFrameState::FIRST_FRAME;
				Capture.GpusWithFullCube = 0;
				break;

			case FRealTimeSlicedReflectionCapture::EFirstFrameState::FIRST_FRAME:
				Capture.FirstFrameState = FRealTimeSlicedReflectionCapture::EFirstFrameState::BEYOND_FIRST_FRAME;
				break;

			default:
				break;
			}
		}
	}
	else
	{
		// Reset the time-slicing first frame detection state when not time-slicing.
		Capture.FirstFrameState = FRealTimeSlicedReflectionCapture::EFirstFrameState::INIT;
	}

	const bool bGpuNeedsFullCube = Capture.GpusWithFullCube != (Capture.GpusWithFullCube | MainView.GPUMask.GetNative());

	if (!bTimeSlicedRealTimeCapture 
		|| (Capture.FirstFrameState < FRealTimeSlicedReflectionCapture::EFirstFrameState::BEYOND_FIRST_FRAME)
		|| bGpuNeedsFullCube)
	{
		Capture.GpusWithFullCube |= MainView.GPUMask.GetNative();

		// Generate a full cube map in a single frame for the first frame.
		// Perf number are for a 128x128x6 a cubemap on PS4 with sky and cloud and default settings

		// Since it is entirely generated each frame when time slicing is not enabled, we always use cubemap index 0 always allocated above
		ConvolvedSkyRenderTargetReadyIndex = 0;

		// 0.60ms (0.12ms for faces with the most clouds)
		RenderCubeFaces_SkyCloud(true, true, CapturedSkyRenderTarget, 0, CubeFace_MAX);

		// 0.05ms
		RenderCubeFaces_GenCubeMips(1, LastMipLevel, CapturedSkyRenderTarget);

		// 0.80ms total (0.30ms for mip0, 0.20ms for mip1+2, 0.30ms for remaining mips)
		RenderCubeFaces_SpecularConvolution(0, LastMipLevel, 0, 6, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetReadyIndex], CapturedSkyRenderTarget);

		// 0.015ms
		RenderCubeFaces_DiffuseIrradiance(ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetReadyIndex]);

		Mobile_ReadBackSkyIrradianceEnvironmentMap();

		// Reset Scene time slicing state so that it starts from the beginning if/when we get out of non-time-sliced.
		Capture.State = -1; // Value of -1 indicates this is the first time-sliced iteration.
		Capture.StateSubStep = 0;

		// The sky just changed, so invalidate these textures, so that the path tracer can rebuild them
		PathTracingSkylightTexture.SafeRelease();
		PathTracingSkylightPdf.SafeRelease();
	}
	else
	{
		// Each frame we capture the sky and work in ProcessedSkyRenderTarget to generate the specular convolution.
		// Once done, we copy the result into ConvolvedSkyRenderTarget and generate the sky irradiance SH from there.

		// On the first frame, we always fully initialise the convolution so ConvolvedSkyRenderTargetReadyIndex should already be valid.
		check(ConvolvedSkyRenderTargetReadyIndex >= 0 && ConvolvedSkyRenderTargetReadyIndex <= 1);
		const int32 TimeSliceCount = 12;

#define DEBUG_TIME_SLICE 0
#if DEBUG_TIME_SLICE
		Capture = FRealTimeSlicedReflectionCapture();
		Capture.FirstFrameState = FRealTimeSlicedReflectionCapture::EFirstFrameState::BEYOND_FIRST_FRAME;
		Capture.GpusWithFullCube |= MainView.GPUMask.GetNative();
		while(true)
		{
			if (Capture.State+1 >= TimeSliceCount)
			{
				break;
			}
#endif 

		const int32 SkyCloudFrameStepCount = FMath::Clamp(CVarRealTimeReflectionCaptureTimeSlicingSkyCloudCubeFacePerFrame.GetValueOnRenderThread(), int32(1), int32(CubeFace_MAX));

		// Because we want all GPUs to do the time slicing in lockstep, we only update the state when a new frame is starting
		if (bIsNewFrame)
		{
			int32 LastSkyCloudEndSubStep = FMath::Clamp(Capture.StateSubStep + SkyCloudFrameStepCount, int32(0), int32(CubeFace_MAX));

			bool bStateFaceStepsDone = true;

			if (Capture.State == 0 || Capture.State == 1)
			{
				bStateFaceStepsDone = LastSkyCloudEndSubStep >= CubeFace_MAX;
				Capture.StateSubStep = bStateFaceStepsDone ? 0 : LastSkyCloudEndSubStep;
			}

			// Update the current time-slicing state if this is a new frame and if the current step is done.
			// Note: Capture.State will initially be -1.
			if (bStateFaceStepsDone)
			{
				if (++Capture.State >= TimeSliceCount)
				{
					// Now use the new cubemap
					ConvolvedSkyRenderTargetReadyIndex = 1 - ConvolvedSkyRenderTargetReadyIndex;

					// The sky just changed, so invalidate these textures, so that the path tracer can rebuild them
					PathTracingSkylightTexture.SafeRelease();
					PathTracingSkylightPdf.SafeRelease();

					Capture.State = 0;
					Capture.StateSubStep = 0;
				}
			}
		}

		const int32 ConvolvedSkyRenderTargetWorkIndex = 1 - ConvolvedSkyRenderTargetReadyIndex;

		const int32 SkyCloudStartSubStep = FMath::Clamp(Capture.StateSubStep, int32(0), int32(CubeFace_MAX - 1));
		const int32 SkyCloudEndSubStep = FMath::Clamp(Capture.StateSubStep + SkyCloudFrameStepCount, int32(0), int32(CubeFace_MAX));

		if (Capture.State <= 0)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "RenderSky StartFace=%d EndFace=%d", SkyCloudStartSubStep, SkyCloudEndSubStep);
			RenderCubeFaces_SkyCloud(true, false, CapturedSkyRenderTarget, SkyCloudStartSubStep, SkyCloudEndSubStep);
		}
		else if (Capture.State == 1)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "RenderCloud StartFace=%d EndFace=%d", SkyCloudStartSubStep, SkyCloudEndSubStep);
			RenderCubeFaces_SkyCloud(false, true, CapturedSkyRenderTarget, SkyCloudStartSubStep, SkyCloudEndSubStep);
		}
		else if (Capture.State == 2)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "GenCubeMips");
			RenderCubeFaces_GenCubeMips(1, LastMipLevel, CapturedSkyRenderTarget);
		}
		else if (Capture.State == 3)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionMip0Face01");
			RenderCubeFaces_SpecularConvolution(0, 0, 0, 2, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex], CapturedSkyRenderTarget); // convolution of mip0, face 0, 1
		}
		else if (Capture.State == 4)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionMip0Face23");
			RenderCubeFaces_SpecularConvolution(0, 0, 2, 2, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex], CapturedSkyRenderTarget); // convolution of mip0, face 2, 3
		}
		else if (Capture.State == 5)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionMip0Face45");
			RenderCubeFaces_SpecularConvolution(0, 0, 4, 2, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex], CapturedSkyRenderTarget); // convolution of mip0, face 4, 5
		}
		else if (Capture.State == 6)
		{
			if (LastMipLevel >= 1)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionMip1");
				RenderCubeFaces_SpecularConvolution(1, 1, 0, 6, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex], CapturedSkyRenderTarget);
			}
		}
		else if (Capture.State == 7)
		{
			if (LastMipLevel >= 2)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionMip2");
				RenderCubeFaces_SpecularConvolution(2, 2, 0, 6, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex], CapturedSkyRenderTarget);
			}
		}
		else if (Capture.State == 8)
		{
			if (LastMipLevel >= 3)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionMip3");
				RenderCubeFaces_SpecularConvolution(3, 3, 0, 6, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex], CapturedSkyRenderTarget);
			}
		}
		else if (Capture.State == 9)
		{
			if (LastMipLevel >= 5)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionMip45");
				RenderCubeFaces_SpecularConvolution(4, 5, 0, 6, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex], CapturedSkyRenderTarget);
			}
			else if (LastMipLevel >= 4)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionMip4");
				RenderCubeFaces_SpecularConvolution(4, 4, 0, 6, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex], CapturedSkyRenderTarget);
			}
		}
		else if (Capture.State == 10)
		{
			if (LastMipLevel >= 6)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "ConvolutionMip6Etc");
				RenderCubeFaces_SpecularConvolution(6, LastMipLevel, 0, 6, ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex], CapturedSkyRenderTarget);
			}
		}
		else if (Capture.State == 11)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "DiffuseIrradiance");

			// Update the sky irradiance SH buffer.
			RenderCubeFaces_DiffuseIrradiance(ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetWorkIndex]);
		}

		Mobile_ReadBackSkyIrradianceEnvironmentMap();

#if DEBUG_TIME_SLICE
		}
		ConvolvedSkyRenderTargetReadyIndex = 1 - ConvolvedSkyRenderTargetReadyIndex;
		Capture.State = 0;
		Capture.StateSubStep = 0;
#endif
	}

	if (ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetReadyIndex])
	{
		ExternalAccessQueue.Add(GraphBuilder.RegisterExternalTexture(ConvolvedSkyRenderTarget[ConvolvedSkyRenderTargetReadyIndex]), ERHIAccess::SRVMask, ERHIPipeline::All);
	}
}


