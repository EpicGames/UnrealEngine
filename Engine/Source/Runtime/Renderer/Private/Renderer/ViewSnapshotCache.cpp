// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewSnapshotCache.h"
#include "SceneRendering.h"

class FViewSnapshotCache
{
public:
	FViewSnapshotCache() = default;

	FViewInfo* Create(const FViewInfo* InView, EViewSnapshotType Type)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ViewSnapshotCache::Create);
		check(IsInParallelRenderingThread()); // we do not want this popped before the end of the scene and it better be the scene allocator

		FViewInfo* Result = FreeSnapshots.Pop();

		if (Result)
		{
			NumFreeSnapshots--;
		}
		else
		{
			Result = (FViewInfo*)FMemory::Malloc(sizeof(FViewInfo), alignof(FViewInfo));
		}

		Snapshots.Push(Result);
		NumSnapshots++;

		if (Type == EViewSnapshotType::Main)
		{
			FMemory::Memcpy(*Result, *InView);

			static_assert(std::is_trivially_destructible_v<FGPUScenePrimitiveCollector> != 0, "The destructor is not invoked properly because of FMemory::Memcpy above");
			Result->DynamicPrimitiveCollector = FGPUScenePrimitiveCollector(InView->DynamicPrimitiveCollector);
		}
		else // Shadow view
		{
			struct FSkipRange
			{
				size_t Offset;
				size_t Size;
			};

			// These members are unused by the shadow view but are modified on the main view in parallel on the render thread.
			const FSkipRange SkipRanges[] =
			{
				// Modified in FGPUOcclusionParallel::Finalize
				{ offsetof(FViewInfo, IndividualOcclusionQueries), sizeof(FViewInfo::IndividualOcclusionQueries) + sizeof(FViewInfo::GroupedOcclusionQueries) },
				// Modified in FSceneRenderer::ComputeLightGrid
				{ offsetof(FViewInfo, ForwardLightingResources),   sizeof(FViewInfo::ForwardLightingResources)  },
				// Modified in FGPUScene::UploadDynamicPrimitiveShaderDataForView()
				{ offsetof(FViewInfo, DynamicPrimitiveCollector),  sizeof(FViewInfo::DynamicPrimitiveCollector) },
			};

			const uint8* SrcData = reinterpret_cast<const uint8*>(InView);
			uint8* DstData = reinterpret_cast<uint8*>(Result);
			size_t CopyOffset = 0;

			for (const FSkipRange& Skip : SkipRanges)
			{
				// Keep ranges in order.
				check(Skip.Offset >= CopyOffset);

				if (Skip.Offset > CopyOffset)
				{
					FMemory::Memcpy(DstData + CopyOffset, SrcData + CopyOffset, Skip.Offset - CopyOffset);
				}

				FMemory::Memzero(DstData + Skip.Offset, Skip.Size);
				CopyOffset = Skip.Offset + Skip.Size;
			}

			if (CopyOffset < sizeof(FViewInfo))
			{
				FMemory::Memcpy(DstData + CopyOffset, SrcData + CopyOffset, sizeof(FViewInfo) - CopyOffset);
			}

			// We need a full new operator here to recreate the vtable that was memset to 0.
			new (&Result->DynamicPrimitiveCollector) FGPUScenePrimitiveCollector();
		}

		FMemory::Memzero(&Result->StereoCullingFrustum, sizeof(Result->StereoCullingFrustum));

		// we want these to start null without a reference count, since we clear a ref later
		TUniformBufferRef<FViewUniformShaderParameters> NullViewUniformBuffer;
		FMemory::Memcpy(Result->ViewUniformBuffer, NullViewUniformBuffer);
		TUniformBufferRef<FInstancedViewUniformShaderParameters> NullInstancedViewUniformBuffer;
		FMemory::Memcpy(Result->InstancedViewUniformBuffer, NullInstancedViewUniformBuffer);

		TUniquePtr<FViewUniformShaderParameters> NullViewParameters;
		FMemory::Memcpy(Result->CachedViewUniformShaderParameters, NullViewParameters); 

		TArray<FPrimitiveUniformShaderParameters> NullDynamicPrimitiveShaderData;
		FMemory::Memset(&Result->ParallelMeshDrawCommandPasses, 0, sizeof(Result->ParallelMeshDrawCommandPasses));

		Result->SnapshotOriginView = InView;
		return Result;
	}

	void Deallocate()
	{
		check(IsInRenderingThread());

		// Only keep double the number actually used, plus a few
		NumSnapshotsToRemove = FMath::Max(NumFreeSnapshots - (NumSnapshots + 2), 0);
		NumSnapshots = 0;

		while (FViewInfo* Snapshot = Snapshots.Pop())
		{
			DeallocatedSnapshots.Add(Snapshot);
		}
	}

	void Destroy()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FViewInfo::DestroyAllSnapshots);

		if (NumSnapshotsToRemove > 0)
		{
			while (FViewInfo* Snapshot = FreeSnapshots.Pop())
			{
				NumFreeSnapshots.fetch_sub(1, std::memory_order_relaxed);
				FMemory::Free(Snapshot);

				if (--NumSnapshotsToRemove == 0)
				{
					break;
				}
			}

			// It's possible that this will not be 0 if allocation happened concurrently, which is safe.
			NumSnapshotsToRemove = 0;
		}

		for (FViewInfo* Snapshot : DeallocatedSnapshots)
		{
			Snapshot->ViewUniformBuffer.SafeRelease();
			Snapshot->InstancedViewUniformBuffer.SafeRelease();
			Snapshot->CachedViewUniformShaderParameters.Reset();

			for (auto* Pass : Snapshot->ParallelMeshDrawCommandPasses)
			{
				if (Pass)
				{
					Pass->Cleanup();
				}
			}

			Snapshot->SnapshotOriginView = nullptr;

			FreeSnapshots.Push(Snapshot);
			NumFreeSnapshots.fetch_add(1, std::memory_order_relaxed);
		}
		DeallocatedSnapshots.Empty();
	}

private:
	// These are not real view infos, just dumb memory blocks
	TLockFreePointerListLIFOPad<FViewInfo, PLATFORM_CACHE_LINE_SIZE> Snapshots;
	// these are never freed, even at program shutdown
	TLockFreePointerListLIFOPad<FViewInfo, PLATFORM_CACHE_LINE_SIZE> FreeSnapshots;

	TArray<FViewInfo*> DeallocatedSnapshots;
	std::atomic_int32_t NumSnapshots{ 0 };
	std::atomic_int32_t NumFreeSnapshots{ 0 };
	int32 NumSnapshotsToRemove = 0;
};

namespace ViewSnapshotCache
{
	static FViewSnapshotCache Cache;

	FViewInfo* Create(const FViewInfo* View, EViewSnapshotType Type)
	{
		return Cache.Create(View, Type);
	}

	void Deallocate()
	{
		Cache.Deallocate();
	}

	void Destroy()
	{
		Cache.Destroy();
	}
}