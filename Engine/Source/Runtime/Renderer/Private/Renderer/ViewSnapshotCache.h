// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FViewInfo;

enum class EViewSnapshotType;

namespace ViewSnapshotCache
{
	// Creates and returns a snapshot from the input view.
	extern FViewInfo* Create(const FViewInfo* View, EViewSnapshotType Type);

	// Called on the render thread after rendering to prepare snapshots for destruction.
	extern void Deallocate();

	// Called on a worker thread to destroy snapshots.
	extern void Destroy();
}