// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RendererPrivate.h: Renderer interface private definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "Shader.h"

class FHitProxyId;
class FLightCacheInterface;
class FMaterial;
class FPrimitiveSceneInfo;
class FSceneInterface;
class FSceneView;
class FSceneViewFamily;
class FSceneViewStateInterface;
class FViewInfo;
class FSceneUniformBuffer;
class FStateStreamManagerImpl;
struct FMeshBatch;
struct FSynthBenchmarkResults;
struct FSceneTextures;

template<class ResourceType, FRenderResource::EInitPhase> class TGlobalResource;

/** The renderer module implementation. */
class FRendererModule final : public IRendererModule
{
public:
	FRendererModule();
	virtual bool SupportsDynamicReloading() override { return true; }

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual void BeginRenderingViewFamily(FCanvas* Canvas,FSceneViewFamily* ViewFamily) override;
	virtual void CreateAndInitSingleView(FRHICommandListImmediate& RHICmdList, class FSceneViewFamily* ViewFamily, const struct FSceneViewInitOptions* ViewInitOptions) override;
	virtual FSceneInterface* AllocateScene(UWorld* World, bool bInRequiresHitProxies, bool bCreateFXSystem, ERHIFeatureLevel::Type InFeatureLevel) override;
	virtual void RemoveScene(FSceneInterface* Scene) override;
	virtual FSceneRemovedEvent::RegistrationType& GetSceneRemovedEvent() override;
	virtual void UpdateStaticDrawLists() override;
	virtual void UpdateStaticDrawListsForMaterials(const TArray<const FMaterial*>& Materials) override;
	virtual FSceneViewStateInterface* AllocateViewState(ERHIFeatureLevel::Type FeatureLevel) override;
	virtual FSceneViewStateInterface* AllocateViewState(ERHIFeatureLevel::Type FeatureLevel, FSceneViewStateInterface* ShareOriginTarget) override;
	virtual uint32 GetNumDynamicLightsAffectingPrimitive(const FPrimitiveSceneInfo* PrimitiveSceneInfo,const FLightCacheInterface* LCI) override;
	virtual void OnWorldCleanup(UWorld* World, bool bSessionEnded, bool bCleanupResources, bool bWorldChanged) override;
	virtual void InitializeSystemTextures(FRHICommandListImmediate& RHICmdList);
	virtual FSceneUniformBuffer* CreateSinglePrimitiveSceneUniformBuffer(FRDGBuilder& GraphBuilder, const FPrimitiveSceneProxy* PrimitiveSceneProxy, ERHIFeatureLevel::Type FeatureLevel, FMeshBatch& Mesh) override;
	virtual FSceneUniformBuffer* CreateSinglePrimitiveSceneUniformBuffer(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FMeshBatch& Mesh) override;
	virtual FSceneUniformBuffer* CreateSinglePrimitiveSceneUniformBuffer(FRDGBuilder& GraphBuilder, const FViewInfo& SceneView, FMeshBatch& Mesh) override;
	virtual TRDGUniformBufferRef<FBatchedPrimitiveParameters> CreateSinglePrimitiveUniformView(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, EShaderPlatform ShaderPlatform, FMeshBatch& Mesh) override;
	virtual TRDGUniformBufferRef<FBatchedPrimitiveParameters> CreateSinglePrimitiveUniformView(FRDGBuilder& GraphBuilder, const FViewInfo& SceneView, FMeshBatch& Mesh) override;
	virtual void DrawTileMesh(FCanvasRenderContext& RenderContext, FMeshPassProcessorRenderState& DrawRenderState, const FSceneView& View, FMeshBatch& Mesh, bool bIsHitTesting, const FHitProxyId& HitProxyId, bool bUse128bitRT = false) override;
	virtual void DebugLogOnCrash() override;
	virtual void GPUBenchmark(FSynthBenchmarkResults& InOut, float WorkScale) override;
	virtual void ExecVisualizeTextureCmd(const FString& Cmd) override;
	virtual void UpdateMapNeedsLightingFullyRebuiltState(UWorld* World) override;
	virtual void DrawRectangle(
		FRHICommandList& RHICmdList,
		float X,
		float Y,
		float SizeX,
		float SizeY,
		float U,
		float V,
		float SizeU,
		float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize,
		const TShaderRef<FShader>& VertexShader,
		EDrawRectangleFlags Flags = EDRF_Default
		) override;

	virtual const TSet<FSceneInterface*>& GetAllocatedScenes() override
	{
		check(IsInGameThread());
		return AllocatedScenes;
	}

	virtual void RegisterCustomCullingImpl(ICustomCulling* impl) override;
	virtual void UnregisterCustomCullingImpl(ICustomCulling* impl) override;

	virtual FDelegateHandle RegisterPostOpaqueRenderDelegate(const FPostOpaqueRenderDelegate& InPostOpaqueRenderDelegate) override;
	virtual void RemovePostOpaqueRenderDelegate(FDelegateHandle InPostOpaqueRenderDelegate) override;
	virtual FDelegateHandle RegisterOverlayRenderDelegate(const FPostOpaqueRenderDelegate& InOverlayRenderDelegate) override;
	virtual void RemoveOverlayRenderDelegate(FDelegateHandle InOverlayRenderDelegate) override;

	virtual FOnResolvedSceneColor& GetResolvedSceneColorCallbacks() override
	{
		return PostResolvedSceneColorCallbacks;
	}

	virtual void PostRenderAllViewports() override;

	virtual void PerFrameCleanupIfSkipRenderer() override;

	virtual IMaterialCacheTagProvider* GetMaterialCacheTagProvider() override;
	virtual IMaterialCacheVirtualTextureAllocator* GetMaterialCacheVirtualTextureAllocator() override;

	virtual IAllocatedVirtualTexture* AllocateVirtualTexture(FRHICommandListBase& RHICmdList, const FAllocatedVTDescription& Desc) override;
	virtual void DestroyVirtualTexture(IAllocatedVirtualTexture* AllocatedVT) override;
	virtual IAdaptiveVirtualTexture* AllocateAdaptiveVirtualTexture(FRHICommandListBase& RHICmdList, const FAdaptiveVTDescription& AdaptiveVTDesc, const FAllocatedVTDescription& AllocatedVTDesc) override;
	virtual void DestroyAdaptiveVirtualTexture(IAdaptiveVirtualTexture* AdaptiveVT) override;
	virtual FVirtualTextureProducerHandle RegisterVirtualTextureProducer(FRHICommandListBase& RHICmdList, const FVTProducerDescription& Desc, IVirtualTexture* Producer) override;
	virtual void ReleaseVirtualTextureProducer(const FVirtualTextureProducerHandle& Handle) override;
	virtual void AddVirtualTextureProducerDestroyedCallback(const FVirtualTextureProducerHandle& Handle, FVTProducerDestroyedFunction* Function, void* Baton) override;
	virtual uint32 RemoveAllVirtualTextureProducerDestroyedCallbacks(const void* Baton) override;
	virtual void ReleaseVirtualTexturePendingResources() override;
	virtual void RequestVirtualTextureTiles(const FVector2D& InScreenSpaceSize, int32 InMipLevel) override;
	virtual void RequestVirtualTextureTiles(const FMaterialRenderProxy* InMaterialRenderProxy, const FVector2D& InScreenSpaceSize, ERHIFeatureLevel::Type InFeatureLevel) override;
	virtual void RequestVirtualTextureTiles(IAllocatedVirtualTexture* AllocatedVT, const FVector2D& InScreenSpaceSize, const FVector2D& InViewportPosition, const FVector2D& InViewportSize, const FVector2D& InUV0, const FVector2D& InUV1, int32 InMipLevel) override;
	virtual IVirtualTexture* FindProducer(const FVirtualTextureProducerHandle& Handle) override;

	virtual void LoadPendingVirtualTextureTiles(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel) override;
	virtual void LockVirtualTextureTiles(FVirtualTextureProducerHandle ProducerHandle, int32 InMipLevel) override;
	virtual void FlushVirtualTextureCache() override;
	virtual void FlushVirtualTextureCache(IAllocatedVirtualTexture* AllocatedVT, const FVector2f& InUV0, const FVector2f& InUV1) override;
	virtual void SyncVirtualTextureUpdates(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel) override;

	virtual void SetVirtualTextureRequestRecordBuffer(uint64 Handle) override;
	virtual uint64 GetVirtualTextureRequestRecordBuffer(TMap<uint64, uint32>& OutRequestData) override;
	virtual void PackVirtualTextureRequestsToStream(TMap<uint64, uint32> const& InPageRequests, TArray<uint32>& OutRequestStream) override;
	virtual void RequestVirtualTextureTiles(TArray<uint32>&& InRequestStream, FIntPoint const& InCaptureResolution) override;

	virtual void SetNaniteRequestRecordBuffer(uint64 Handle) override;
	virtual uint64 GetNaniteRequestRecordBuffer(TMap<uint64, uint32>& OutPageRequests) override; 	
	virtual void PackNaniteRequestsToStream(TMap<uint64, uint32> const& InPageRequests, TArray<uint32>& OutRequestStream) override;
	virtual void RequestNanitePages(TArray<uint32>&& InRequestStream) override;
	virtual void PrefetchNaniteResource(const Nanite::FResources* Resource, uint32 NumFramesUntilRender) override;
	
	virtual void RegisterPersistentViewUniformBufferExtension(IPersistentViewUniformBufferExtension* Extension) override;

	void RenderPostOpaqueExtensions(
		FRDGBuilder& GraphBuilder,
		TArrayView<const FViewInfo> Views,
		const FSceneTextures& SceneTextures);

	void RenderOverlayExtensions(
		FRDGBuilder& GraphBuilder,
		TArrayView<const FViewInfo> Views,
		const FSceneTextures& SceneTextures);

	void RenderPostResolvedSceneColorExtension(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

	virtual IScenePrimitiveRenderingContext* BeginScenePrimitiveRendering(FRDGBuilder& GraphBuilder, FSceneViewFamily* ViewFamily, EBeginScenePrimitiveRenderingFlags BeginScenePrimitiveRenderingFlags = EBeginScenePrimitiveRenderingFlags::All) override;
	virtual IScenePrimitiveRenderingContext* BeginScenePrimitiveRendering(FRDGBuilder& GraphBuilder, FSceneInterface& Scene, EBeginScenePrimitiveRenderingFlags BeginScenePrimitiveRenderingFlags = EBeginScenePrimitiveRenderingFlags::All) override;

	virtual void InvalidatePathTracedOutput(PathTracing::EInvalidateReason InvalidateReason = PathTracing::EInvalidateReason::Uncategorized) override;

	virtual void BeginRenderingViewFamilies(FCanvas* Canvas, TConstArrayView<FSceneViewFamily*> ViewFamilies) override;

	virtual void ResetSceneTextureExtentHistory() override;

	virtual const FViewMatrices& GetPreviousViewMatrices(const FSceneView& View) override;
	virtual const FGlobalDistanceFieldParameterData* GetGlobalDistanceFieldParameterData(const FSceneView& View) override;
	virtual void RequestStaticMeshUpdate(FPrimitiveSceneInfo* Info) override;
	virtual void AddMeshBatchToGPUScene(FGPUScenePrimitiveCollector* Collector, FMeshBatch& MeshBatch) override;

#if WITH_STATE_STREAM
	virtual IStateStreamManager* GetStateStreamManager() override;
	virtual void DebugLogRenderProxies(FName Type, uint32 Num) override;
#endif

	virtual TUniquePtr<ISceneRenderBuilder> CreateSceneRenderBuilder(FSceneInterface* Scene) override;

private:
	TSet<FSceneInterface*> AllocatedScenes;
	FOnPostOpaqueRender PostOpaqueRenderDelegate;
	FOnPostOpaqueRender OverlayRenderDelegate;
	FOnResolvedSceneColor PostResolvedSceneColorCallbacks;
	FSceneRemovedEvent SceneRemovedEvent;
	FDelegateHandle StopRenderingThreadDelegate;

#if WITH_STATE_STREAM
	class FRendererStateStreamManager;
	FRendererStateStreamManager* StateStreamManager = nullptr;
#endif
};

extern ICustomCulling* GCustomCullingImpl;
