// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Misc/CoreDelegates.h"
#include "Misc/LazySingleton.h"

/** 
 * Class to wrap FCoreDelegates::FGetOnScreenMessagesDelegate for use on the render thread. 
 * This avoids race conditions in registration/unregistration that would happen if using FCoreDelegates::FGetOnScreenMessagesDelegate not on the game thread.
 * Registration/Unregistration can happen on any thread and a proxy delegate calls Broadcast() on the render thread.
 */
class FRendererOnScreenNotification
{
public:
	/** 
	 * Create or get singleton instance. 
	 * First call should be on game thread. After that any thread will do.
	 */
	static FRendererOnScreenNotification& Get()
	{
		return TLazySingleton<FRendererOnScreenNotification>::Get();
	}

	/** 
	 * Tear down singleton instance. 
	 * Must be called on game thread. 
	 */
	static void TearDown()
	{
		TLazySingleton<FRendererOnScreenNotification>::TearDown();
	}

	/** 
	 * Relay to AddLambda() of underlying proxy delegate. 
	 * Thia takes a lock so that it can be called from any thread.
	 */
	template<typename FunctorType, typename... VarTypes>
	FDelegateHandle AddLambda(FunctorType&& InFunctor, VarTypes... Vars)
	{
		FScopeLock Lock(&DelgateCS);
		return ProxyDelegate.AddLambda(MoveTemp(InFunctor), Vars...);
	}

	/** 
	 * Relay to Remove() of underlying proxy delegate. 
	 * Thia takes a lock so that it can be called from any thread.
	 */
	bool Remove(FDelegateHandle InHandle)
	{
		FScopeLock Lock(&DelgateCS);
		return ProxyDelegate.Remove(InHandle);
	}

	/** 
	 * Calls Broadcast on the proxy delegate.
	 * The results are buffered for later collection by the base game thread delegate.
	 */
	void Broadcast()
	{
		// Broadcast using temporary container to reduce any lock contention.
		FCoreDelegates::FSeverityMessageMap MessagesTmp;
		MessagesTmp.Empty(Messages.Num());
		
		{
			FScopeLock Lock(&DelgateCS);
			ProxyDelegate.Broadcast(MessagesTmp);
		}
		{
			FScopeLock Lock(&MessageCS);
			Messages = MoveTemp(MessagesTmp);
		}
	}

private:
	friend FLazySingleton;

	FRendererOnScreenNotification()
	{
		BaseDelegateHandle = FCoreDelegates::OnGetOnScreenMessages.AddLambda([this](FCoreDelegates::FSeverityMessageMap& OutMessages)
		{
			FScopeLock Lock(&MessageCS);
			OutMessages.Append(Messages);
		});
	}

	~FRendererOnScreenNotification()
	{
		FCoreDelegates::OnGetOnScreenMessages.Remove(BaseDelegateHandle);
	}

private:
	FCriticalSection DelgateCS;
	FCriticalSection MessageCS;
	TMulticastDelegate<void(FCoreDelegates::FSeverityMessageMap&)> ProxyDelegate;
	FDelegateHandle BaseDelegateHandle;
	FCoreDelegates::FSeverityMessageMap Messages;
};
