// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RendererPrivate.h: Renderer interface private definitions.
=============================================================================*/

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.8, "")

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#include "Async/ParallelFor.h"
#include "DeferredShadingRenderer.h"
#include "DynamicPrimitiveDrawing.h"
#include "TranslucentRendering.h"
#include "LightMapDensityRendering.h"
#include "ClearQuad.h"
#include "ScenePrivate.h"
#include "RendererModule.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

