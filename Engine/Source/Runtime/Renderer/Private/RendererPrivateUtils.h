// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderResource.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RenderGraphUtils.h"

/**
 * Returns true if Nanite primitives are to be skipped in Light Primitive Interactions (LPI) and the primitive acceleration structure (octree).
 */
bool ShouldSkipNaniteLPIs(EShaderPlatform ShaderPlatform);


class FTileTexCoordVertexBuffer : public FVertexBuffer
{
public:
	FTileTexCoordVertexBuffer(int32 InNumTileQuadsInBuffer)
		: NumTileQuadsInBuffer(InNumTileQuadsInBuffer)
	{
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector2f>(TEXT("FTileTexCoordVertexBuffer"), 4 * NumTileQuadsInBuffer)
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FVector2f> Vertices = RHICmdList.CreateBufferInitializer(CreateDesc);

		for (uint32 SpriteIndex = 0; SpriteIndex < NumTileQuadsInBuffer; ++SpriteIndex)
		{
			Vertices[SpriteIndex * 4 + 0] = FVector2f(0.0f, 0.0f);
			Vertices[SpriteIndex * 4 + 1] = FVector2f(0.0f, 1.0f);
			Vertices[SpriteIndex * 4 + 2] = FVector2f(1.0f, 1.0f);
			Vertices[SpriteIndex * 4 + 3] = FVector2f(1.0f, 0.0f);
		}

		VertexBufferRHI = Vertices.Finalize();
	}

	const uint32 NumTileQuadsInBuffer;
};

class FTileIndexBuffer : public FIndexBuffer
{
public:
	FTileIndexBuffer(int32 InNumTileQuadsInBuffer)
		: NumTileQuadsInBuffer(InNumTileQuadsInBuffer)
	{
	}

	/** Initialize the RHI for this rendering resource */
	void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex<uint16>(TEXT("FTileIndexBuffer"), 6 * NumTileQuadsInBuffer)
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<uint16> Indices = RHICmdList.CreateBufferInitializer(CreateDesc);

		for (uint32 SpriteIndex = 0; SpriteIndex < NumTileQuadsInBuffer; ++SpriteIndex)
		{
			Indices[SpriteIndex * 6 + 0] = SpriteIndex * 4 + 0;
			Indices[SpriteIndex * 6 + 1] = SpriteIndex * 4 + 1;
			Indices[SpriteIndex * 6 + 2] = SpriteIndex * 4 + 2;
			Indices[SpriteIndex * 6 + 3] = SpriteIndex * 4 + 0;
			Indices[SpriteIndex * 6 + 4] = SpriteIndex * 4 + 2;
			Indices[SpriteIndex * 6 + 5] = SpriteIndex * 4 + 3;
		}

		IndexBufferRHI = Indices.Finalize();
	}

	const uint32 NumTileQuadsInBuffer;
};

/** One Tile Quad Vertex Buffer*/
RENDERER_API FBufferRHIRef& GetOneTileQuadVertexBuffer();
/** One Tile Quad Index Buffer*/
RENDERER_API FBufferRHIRef& GetOneTileQuadIndexBuffer();

template <typename ReadbackProcessingLambdaType>
void AddBufferLockReadbackPass(FRDGBuilder& GraphBuilder, TRefCountPtr<FRDGPooledBuffer> SourceBuffer, uint32 NumBytes, ReadbackProcessingLambdaType &&ReadbackProcessingLambda)
{
	FRHIBuffer* SourceBufferRHI = SourceBuffer->GetRHI();
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("BufferLockReadbackPass"),
		ERDGPassFlags::None,
		[ReadbackProcessingLambdaType=MoveTemp(ReadbackProcessingLambda), SourceBufferRHI, NumBytes](FRHICommandListImmediate& RHICmdList)
	{
		const void *GPUData = (uint32*)RHICmdList.LockBuffer(SourceBufferRHI, 0, NumBytes, RLM_ReadOnly);
		ReadbackProcessingLambdaType(GPUData);
		RHICmdList.UnlockBuffer(SourceBufferRHI);
	});
}

class FBufferScatterUploader
{
public:
	// Used to capture GPU scatter buffer and num, to use for example to chain a compute shader doing some per-updated-thing work
	struct FScatterInfo
	{
		FRDGBuffer *ScatterOffsetsRDG;
		int32 NumScatters;
	};

	RENDERER_API void UploadTo(FRDGBuilder& GraphBuilder, FRDGBuffer *DestBuffer, FRDGBuffer *ScatterOffsets, FRDGBuffer *Values, uint32 NumScatters, uint32 NumBytesPerElement, int32 NumValuesPerScatter, EPixelFormat ValuePixelFormat);
	RENDERER_API void UploadTo(FRDGBuilder& GraphBuilder, FRDGBuffer* DestBuffer, FRDGBuffer* ScatterOffsets, FRDGBuffer* Values, TFunction<uint64()>&& GetNumScatters, uint32 NumBytesPerElement, int32 NumValuesPerScatter);
};

namespace UE::RendererPrivateUtils::Implementation
{

/** 
 * Helper class to manage a persistent buffer.
 */
class FPersistentBuffer
{
public:
	RENDERER_API FPersistentBuffer(int32 InMinimumNumElementsReserved, const TCHAR *InName, bool bInRoundUpToPOT = true);

	RENDERER_API FRDGBuffer* Register(FRDGBuilder& GraphBuilder) const;

	RENDERER_API void Empty();

protected:

	RENDERER_API FRDGBuffer* ResizeBufferIfNeeded(FRDGBuilder& GraphBuilder, const FRDGBufferDesc& BufferDesc);

	RENDERER_API FRDGBuffer* ResizeAndClearBufferIfNeeded(FRDGBuilder& GraphBuilder, const FRDGBufferDesc& BufferDesc);

	int32 MinimumNumElementsReserved = 0;
	const TCHAR *Name = nullptr;
	bool bRoundUpToPOT = true;
	TRefCountPtr<FRDGPooledBuffer> PooledBuffer;
};

struct FFloat4BufferTraits
{
	template <typename ElementType, typename AllocatorType>
	static FRDGBuffer *CreateValueUploadBuffer(FRDGBuilder& GraphBuilder, const TCHAR* Name, const TArray<ElementType, AllocatorType> &InitialData)
	{
		static_assert(sizeof(ElementType) % OverrideBytesPerElement == 0);
		const int32 NumElements = (InitialData.Num() * sizeof(ElementType)) / OverrideBytesPerElement;
		FRDGBufferRef Buffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredUploadDesc(OverrideBytesPerElement, NumElements), Name);
		GraphBuilder.QueueBufferUpload(Buffer, TConstArrayView<ElementType>(InitialData), ERDGInitialDataFlags::NoCopy);
		return Buffer;
	}
	
	template <typename AllocatorType>
	static FRDGBuffer *CreateOffsetUploadBuffer(FRDGBuilder& GraphBuilder, const TCHAR* Name, const TArray<uint32, AllocatorType> &InitialData)
	{
		return CreateStructuredUploadBuffer(GraphBuilder, Name, InitialData, ERDGInitialDataFlags::NoCopy);
	}

	static FRDGBufferDesc CreateDesc(uint32 BytesPerElement, uint32 NumElements)
	{
		return FRDGBufferDesc::CreateBufferDesc(BytesPerElement, NumElements);
	}

	static constexpr bool bAutoValuesPerScatter = false;
	static constexpr uint32 OverrideBytesPerElement = 16;
	static constexpr EPixelFormat PixelFormat = PF_A32B32G32R32F;
};

struct FStructuredBufferTraits
{
	template <typename ElementType, typename AllocatorType>
	static FRDGBuffer *CreateValueUploadBuffer(FRDGBuilder& GraphBuilder, const TCHAR* Name, const TArray<ElementType, AllocatorType> &InitialData)
	{
		return CreateStructuredUploadBuffer(GraphBuilder, Name, InitialData, ERDGInitialDataFlags::NoCopy);
	}

	template <typename AllocatorType>
	static FRDGBuffer *CreateOffsetUploadBuffer(FRDGBuilder& GraphBuilder, const TCHAR* Name, const TArray<uint32, AllocatorType> &InitialData)
	{
		return CreateStructuredUploadBuffer(GraphBuilder, Name, InitialData, ERDGInitialDataFlags::NoCopy);
	}

	static FRDGBufferDesc CreateDesc(uint32 BytesPerElement, uint32 NumElements)
	{
		return FRDGBufferDesc::CreateStructuredDesc(BytesPerElement, NumElements);
	}

	static constexpr bool bAutoValuesPerScatter = false;
	static constexpr uint32 OverrideBytesPerElement = 0;
	static constexpr EPixelFormat PixelFormat = PF_Unknown;
};

struct FByteAddressBufferTraits
{
	template <typename ElementType, typename AllocatorType>
	static FRDGBuffer *CreateValueUploadBuffer(FRDGBuilder& GraphBuilder, const TCHAR* Name, const TArray<ElementType, AllocatorType> &InitialData)
	{
		return CreateByteAddressUploadBuffer(GraphBuilder, Name, InitialData, ERDGInitialDataFlags::NoCopy);
	}

	template <typename AllocatorType>
	static FRDGBuffer *CreateOffsetUploadBuffer(FRDGBuilder& GraphBuilder, const TCHAR* Name, const TArray<uint32, AllocatorType> &InitialData)
	{
		return CreateByteAddressUploadBuffer(GraphBuilder, Name, InitialData, ERDGInitialDataFlags::NoCopy);
	}

	template <typename GetArrayRefCallback>
	static FRDGBuffer* CreateUploadBuffer(FRDGBuilder& GraphBuilder, const TCHAR* Name, GetArrayRefCallback&& ArrayCallback)
	{
		return CreateByteAddressUploadBuffer(GraphBuilder, Name, MoveTemp(ArrayCallback));
	}

	static FRDGBufferDesc CreateDesc(uint32 BytesPerElement, uint32 NumElements)
	{
		uint32 NumBytes = BytesPerElement * NumElements;
		// Needs to be aligned to 16 bytes to MemcpyResource to work correctly (otherwise it skips last unaligned elements of the buffer during resize)
		check((NumBytes & 15) == 0);
		return FRDGBufferDesc::CreateByteAddressDesc(NumBytes);	
	}

	static constexpr bool bAutoValuesPerScatter = true;
	static constexpr uint32 OverrideBytesPerElement = 0;
	static constexpr EPixelFormat PixelFormat = PF_Unknown;
};

/**
 * Typed version of FPersistentStructuredBuffer 
 */
template <typename InValueType, typename InBufferTraits>
class TPersistentBuffer : public FPersistentBuffer
{
public:
	using ValueType = InValueType;
	using BufferTraits = InBufferTraits;

	static constexpr int32 BytesPerElement = BufferTraits::OverrideBytesPerElement != 0 ? BufferTraits::OverrideBytesPerElement : sizeof(ValueType);

	TPersistentBuffer(int32 InMinimumNumElementsReserved, const TCHAR *InName, bool bInRoundUpToPOT = true)
	: FPersistentBuffer(InMinimumNumElementsReserved, InName, bInRoundUpToPOT)
	{
	}

	FRDGBuffer* ResizeBufferIfNeeded(FRDGBuilder& GraphBuilder, int32 InNewMinNumElements)
	{
		int32 NewMinNumElements = GetNumElements(FMath::Max(MinimumNumElementsReserved, bRoundUpToPOT ? int32(FMath::RoundUpToPowerOfTwo(InNewMinNumElements)) : InNewMinNumElements));
		return FPersistentBuffer::ResizeBufferIfNeeded(GraphBuilder, BufferTraits::CreateDesc(BytesPerElement, NewMinNumElements));
	}

	FRDGBuffer* ResizeAndClearBufferIfNeeded(FRDGBuilder& GraphBuilder, int32 InNewMinNumElements)
	{
		int32 NewMinNumElements = GetNumElements(FMath::Max(MinimumNumElementsReserved, bRoundUpToPOT ? int32(FMath::RoundUpToPowerOfTwo(InNewMinNumElements)) : InNewMinNumElements));
		return FPersistentBuffer::ResizeAndClearBufferIfNeeded(GraphBuilder, BufferTraits::CreateDesc(BytesPerElement, NewMinNumElements));
	}

	template <typename ValueCheckFuncType>
	void ValidateGPUData(FRDGBuilder& GraphBuilder, TConstArrayView<ValueType> HostValues, ValueCheckFuncType &&ValueCheckFunc)
	{
#if DO_CHECK
		const int32 NumValues = (PooledBuffer->Desc.NumElements * BytesPerElement) / sizeof(ValueType);

		check(HostValues.Num() == 0 || PooledBuffer.IsValid() && HostValues.Num() <= NumValues);

		if (!HostValues.IsEmpty())
		{
			// TODO: should perhaps copy the host values? A lot of overhead but very useful to be sure they are alive...
			AddBufferLockReadbackPass(GraphBuilder, PooledBuffer, uint32(HostValues.GetTypeSize() * HostValues.Num()), [ValueCheckFunc=MoveTemp(ValueCheckFunc), HostValues](const void *LockedData)
			{
				const ValueType *GPUValuePtr = reinterpret_cast<const ValueType*>(LockedData);
				for (int32 Index = 0; Index < HostValues.Num(); ++Index)
				{
					ValueCheckFunc(Index, HostValues[Index], GPUValuePtr[Index]);
				}
			});
		}
#endif
	}

	TRefCountPtr<FRDGPooledBuffer> &GetPooledBuffer() { return PooledBuffer; }
	const TRefCountPtr<FRDGPooledBuffer> &GetPooledBuffer() const { return PooledBuffer; }

private:
	int32 GetNumElements(int32 InNumValues) const
	{
		return (InNumValues * sizeof(ValueType)) / BytesPerElement;
	}
};

template <typename InValueType, typename InBufferTraits,  int32 InNumValuesPerScatter = 1>
class TBufferScatterUploader : public FBufferScatterUploader
{
public:
	using BufferTraits = InBufferTraits;
	using ValueType = InValueType;
	static constexpr int32 NumBytesPerElement = BufferTraits::OverrideBytesPerElement != 0 ? BufferTraits::OverrideBytesPerElement : sizeof(ValueType);
	static constexpr int32 NumValuesPerScatter = InNumValuesPerScatter;

	static_assert((NumBytesPerElement % 4) == 0, "The struct used must be 4-byte aligned");

	/**
	 * Optionally reserve space for NumScatters items.
	 */
	TBufferScatterUploader(int32 NumScatters = 0)
	{
		Reserve(NumScatters);
	}

	virtual ~TBufferScatterUploader() = default;

	/**
	 * Pre-size the value and scatter arrays, allocates space for InNumValuesPerScatter * NumScatters values & NumScatters scatter offsets.
	 */
	void Reserve(int32 NumScatters)
	{
		check(UploadDataProxy == nullptr);

		UploadData.Values.Reserve(NumScatters * NumValuesPerScatter);
		UploadData.ScatterOffsets.Reserve(NumScatters);
	}

	/**
	 * Add single value to scatter to the destination offset.
	 */
	void Add(const ValueType &Value, int32 ScatterOffset)
	{
		check(1 == NumValuesPerScatter);
		check(UploadDataProxy == nullptr);

		UploadData.Values.Add(Value);
		UploadData.ScatterOffsets.Add(ScatterOffset);
	}

	/**
	 * Add a number of values to scatter to a common destination start offset, 
	 * NOTE: the destination start offset is ScatterOffset * InNumValuesPerScatter (not ScatterOffset)
	 */
	void Add(const TConstArrayView<ValueType> &InValues, int32 ScatterOffset)
	{
		check(InValues.Num() == NumValuesPerScatter);
		check(UploadDataProxy == nullptr);

		UploadData.Values.Append(InValues);
		UploadData.ScatterOffsets.Add(ScatterOffset);
	}

	/**
	 * Add a number of values to scatter to consecutive destination offsets with a common start offset, 
	 * NOTE: this allocates a new scatter offset for each NumValuesPerScatter elements.
	 */
	void AddMultiple(const TConstArrayView<ValueType> &InValues, int32 FirstScatterOffset)
	{
		check(InValues.Num() % NumValuesPerScatter == 0);
		check(UploadDataProxy == nullptr);

		UploadData.Values.Append(InValues);
		for (int32 Index = 0; Index < InValues.Num() / NumValuesPerScatter; ++Index)
		{
			UploadData.ScatterOffsets.Add(FirstScatterOffset + Index);
		}
	}

	/**
	 * Add a number of values to scatter to consecutive destination offsets with a common start offset, 
	 * NOTE: this allocates a new scatter offset for each NumValuesPerScatter elements.
	 */
	TArrayView<ValueType> AddMultiple_GetRef(int32 FirstScatterOffset, int32 NumValues)
	{
		check(NumValues % NumValuesPerScatter == 0);
		check(UploadDataProxy == nullptr);

		int32 SrcOffset = UploadData.Values.Num();
		UploadData.Values.AddUninitialized(NumValues);
		for (int32 Index = 0; Index < NumValues / NumValuesPerScatter; ++Index)
		{
			UploadData.ScatterOffsets.Add(FirstScatterOffset + Index);
		}

		return MakeArrayView(&UploadData.Values[SrcOffset], NumValues);
	}

	int32 GetNumScatters() const { return UploadDataProxy != nullptr ? UploadDataProxy->ScatterOffsets.Num() : UploadData.ScatterOffsets.Num(); }

	/**
	 * Resize the destination persistent buffer (if needed) and upload & scatter the collected data to it.
	 * This locks the uploader to prevent accidental resize (and thus realloc) of the buffer by adding more elements.
	 */
	FRDGBuffer *ResizeAndUploadTo(FRDGBuilder& GraphBuilder, TPersistentBuffer<ValueType, BufferTraits> &DestDataBuffer, int32 DestDataMinimumSize, FScatterInfo &OutScatterInfo)
	{
		check(UploadDataProxy == nullptr);

		FRDGBuffer *DestBufferRDG = DestDataBuffer.ResizeBufferIfNeeded(GraphBuilder, DestDataMinimumSize);

		OutScatterInfo.NumScatters = UploadData.ScatterOffsets.Num();
		OutScatterInfo.ScatterOffsetsRDG = nullptr;
		if (OutScatterInfo.NumScatters != 0u)
		{
			// Move the data arrays to a proxy object owned by RDG to guarantee life-times for upload.
			UploadDataProxy = GraphBuilder.AllocObject<FUploadData>(MoveTemp(UploadData));

			// upload the values & offsets
			OutScatterInfo.ScatterOffsetsRDG = BufferTraits::CreateOffsetUploadBuffer(GraphBuilder, TEXT("ScatterUploader.Offsets"), UploadDataProxy->ScatterOffsets);
			FRDGBuffer *ValuesRDG = BufferTraits::CreateValueUploadBuffer(GraphBuilder, TEXT("ScatterUploader.Values"), UploadDataProxy->Values);

			uint32 ElementSize = NumBytesPerElement;
			uint32 ElementsPerScatter = NumValuesPerScatter;
			if (BufferTraits::bAutoValuesPerScatter)
			{
				// Let the implementation determine the optimal way to divide up the scatter
				ElementSize *= ElementsPerScatter;
				ElementsPerScatter = INDEX_NONE;
			}
			FBufferScatterUploader::UploadTo(GraphBuilder, DestBufferRDG, OutScatterInfo.ScatterOffsetsRDG, ValuesRDG, OutScatterInfo.NumScatters, ElementSize, ElementsPerScatter, BufferTraits::PixelFormat);
		}

		return DestBufferRDG;
	}
	
	FRDGBuffer *ResizeAndUploadTo(FRDGBuilder& GraphBuilder, TPersistentBuffer<ValueType, BufferTraits> &DestDataBuffer, int32 DestDataMinimumSize)
	{
		FScatterInfo ScatterInfo = { nullptr, 0 };
		return ResizeAndUploadTo(GraphBuilder, DestDataBuffer, DestDataMinimumSize, ScatterInfo);
	}

protected:
	struct FUploadData
	{
		TArray<ValueType> Values;
		TArray<uint32> ScatterOffsets;
	};
	FUploadData *UploadDataProxy = nullptr;
	FUploadData UploadData;
};

template <typename InValueType, typename InBufferTraits, int32 InNumValuesPerScatter = 1>
class TAsyncBufferScatterUploader : public TBufferScatterUploader<InValueType, InBufferTraits, InNumValuesPerScatter>
{
	using Base = TBufferScatterUploader<InValueType, InBufferTraits, InNumValuesPerScatter>;

public:

	/**
	* Allocate an instance of TAsyncBufferScatterUploader using an allocator tied to the lifetime of the graph
	*/
	static TAsyncBufferScatterUploader* Allocate(FRDGBuilder& GraphBuilder, int32 NumScatters = 0)
	{
		return GraphBuilder.AllocObject<TAsyncBufferScatterUploader>(NumScatters);
	}

	/**
	 * Resize the destination persistent buffer (if needed) and upload & scatter the collected data to it.
	 * Callbacks are used to create upload buffers and scheduling scatter dispatch since data might not be ready yet.
	 */
	FRDGBuffer* ResizeAndUploadToAsync(FRDGBuilder& GraphBuilder, TPersistentBuffer<InValueType, InBufferTraits>& DestDataBuffer, int32 DestDataMinimumSize)
	{
		check(Base::UploadDataProxy == nullptr);

		FRDGBuffer* DestBufferRDG = DestDataBuffer.ResizeBufferIfNeeded(GraphBuilder, DestDataMinimumSize);

		// upload the values & offsets
		FRDGBufferRef ScatterOffsetsRDG = Base::BufferTraits::CreateUploadBuffer(GraphBuilder, TEXT("ScatterUploader.Offsets"), [&]() -> auto& { return Base::UploadData.ScatterOffsets; });
		FRDGBufferRef ValuesRDG = Base::BufferTraits::CreateUploadBuffer(GraphBuilder, TEXT("ScatterUploader.Values"), [&]() -> auto& { return Base::UploadData.Values; });

		uint32 ElementSize = sizeof(typename Base::ValueType);
		uint32 ElementsPerScatter = Base::NumValuesPerScatter;
		if (Base::BufferTraits::bAutoValuesPerScatter)
		{
			// Let the implementation determine the optimal way to divide up the scatter
			ElementSize *= ElementsPerScatter;
			ElementsPerScatter = INDEX_NONE;
		}
		FBufferScatterUploader::UploadTo(GraphBuilder, DestBufferRDG, ScatterOffsetsRDG, ValuesRDG, [&]() -> int64 { return Base::UploadData.ScatterOffsets.Num(); }, ElementSize, ElementsPerScatter);

		return DestBufferRDG;
	}

private:

	/**
	* Optionally reserve space for NumScatters items.
	* 
	* Constructor is private since instances should be allocated directly from GraphBuilder or using the static helper function Allocate(...)
	* because ResizeAndUploadToAsync(...) requires that UploadData is tied to the lifetime of the graph
	*/
	TAsyncBufferScatterUploader(int32 NumScatters = 0) : Base(NumScatters)
	{
	}

	friend class FRDGAllocator;
	friend class FRDGAllocator::TObject<TAsyncBufferScatterUploader>;
};

}

template <typename InValueType>
using TPersistentFloat4Buffer = UE::RendererPrivateUtils::Implementation::TPersistentBuffer<InValueType, UE::RendererPrivateUtils::Implementation::FFloat4BufferTraits>;

template <typename InValueType, int32 InNumValuesPerScatter = 1>
using TFloat4BufferScatterUploader = UE::RendererPrivateUtils::Implementation::TBufferScatterUploader<InValueType, UE::RendererPrivateUtils::Implementation::FFloat4BufferTraits, InNumValuesPerScatter>;

template <typename InValueType>
using TPersistentStructuredBuffer = UE::RendererPrivateUtils::Implementation::TPersistentBuffer<InValueType, UE::RendererPrivateUtils::Implementation::FStructuredBufferTraits>;

template <typename InValueType, int32 InNumValuesPerScatter = 1>
using TStructuredBufferScatterUploader = UE::RendererPrivateUtils::Implementation::TBufferScatterUploader<InValueType, UE::RendererPrivateUtils::Implementation::FStructuredBufferTraits, InNumValuesPerScatter>;

template <typename InValueType>
using TPersistentByteAddressBuffer = UE::RendererPrivateUtils::Implementation::TPersistentBuffer<InValueType, UE::RendererPrivateUtils::Implementation::FByteAddressBufferTraits>;

template <typename InValueType, int32 InNumValuesPerScatter = 1>
using TByteAddressBufferScatterUploader = UE::RendererPrivateUtils::Implementation::TBufferScatterUploader<InValueType, UE::RendererPrivateUtils::Implementation::FByteAddressBufferTraits, InNumValuesPerScatter>;

template <typename InValueType, int32 InNumValuesPerScatter = 1>
using TAsyncByteAddressBufferScatterUploader = UE::RendererPrivateUtils::Implementation::TAsyncBufferScatterUploader<InValueType, UE::RendererPrivateUtils::Implementation::FByteAddressBufferTraits, InNumValuesPerScatter>;

/**
 * Clear indirect args for GPU-side building, initializes the XYZ dimensions of the dispatch arg to DimClearValue and everything else in the strided range to zero. 
 * Can clear one or more args (NumIndirectArgs). 
 * IndirectArgStride gives the size (in uints) of each argument, this may be larger than the required size. E.g., to make space for an atomic counter.
 */
void AddClearIndirectDispatchArgsPass(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGBufferRef IndirectArgsRDG, const FIntVector3 &DimClearValue, uint32 NumIndirectArgs, uint32 IndirectArgStride);

/**
 * Clear indirect for 1D kernel launch, sets YZ grid dimensions to 1.
 */
inline void AddClearIndirectDispatchArgs1DPass(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGBufferRef IndirectArgsRDG, uint32 NumIndirectArgs, uint32 IndirectArgStride)
{
	AddClearIndirectDispatchArgsPass(GraphBuilder, FeatureLevel, IndirectArgsRDG, FIntVector3(0,1,1), NumIndirectArgs, IndirectArgStride);
}
/**
 * Create an indirect args buffer and clear it using AddClearIndirectDispatchArgsPass.
 */
FRDGBufferRef CreateAndClearIndirectDispatchArgs(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, const TCHAR* Name, const FIntVector3 &DimClearValue, uint32 NumIndirectArgs, uint32 IndirectArgStride);

/**
 * Create and initialize a singular indirect args buffer with size derived from FRHIDispatchIndirectParameters.
 */
inline FRDGBufferRef CreateAndClearIndirectDispatchArgs1D(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, const TCHAR* Name)
{
	return CreateAndClearIndirectDispatchArgs(GraphBuilder, FeatureLevel, Name, FIntVector3(0,1,1), 1u, sizeof(FRHIDispatchIndirectParameters) / 4u);
}
