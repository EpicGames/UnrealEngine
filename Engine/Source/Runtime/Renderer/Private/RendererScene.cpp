// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Scene.cpp: Scene manager implementation.
=============================================================================*/

#include "CoreMinimal.h"
#include "Engine/Level.h"
#include "Engine/TextureLightProfile.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/PlatformFileManager.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "EngineDefines.h"
#include "EngineGlobals.h"
#include "Components/ActorComponent.h"
#include "RHI.h"
#include "RenderingThread.h"
#include "RenderResource.h"
#include "UniformBuffer.h"
#include "SceneTypes.h"
#include "SceneInterface.h"
#include "Components/PrimitiveComponent.h"
#include "PhysicsField/PhysicsFieldComponent.h"
#include "MaterialShared.h"
#include "PrimitiveDrawingUtils.h"
#include "PrecomputedLightVolume.h"
#include "PrecomputedVolumetricLightmap.h"
#include "Components/LightComponent.h"
#include "GameFramework/WorldSettings.h"
#include "Components/DecalComponent.h"
#include "Components/ReflectionCaptureComponent.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "ScenePrivateBase.h"
#include "SceneCore.h"
#include "Rendering/RayTracingGeometryManager.h"
#include "Rendering/MotionVectorSimulation.h"
#include "PrimitiveSceneInfo.h"
#include "LogRenderer.h"
#include "LightSceneInfo.h"
#include "LightMapRendering.h"
#include "SkyAtmosphereRendering.h"
#include "BasePassRendering.h"
#include "MobileBasePassRendering.h"
#include "PrimitiveSceneDesc.h"
#include "InstancedStaticMeshSceneProxyDesc.h"
#include "LightSceneDesc.h"
#include "ScenePrivate.h"
#include "SceneProxies/DeferredDecalProxy.h"
#include "SceneProxies/ReflectionCaptureProxy.h"
#include "SceneProxies/SkyAtmosphereSceneProxy.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneProxies/WindSourceSceneProxy.h"
#include "RendererModule.h"
#include "StaticMeshResources.h"
#include "ParameterCollection.h"
#include "DistanceFieldAmbientOcclusion.h"
#include "DistanceFieldAtlas.h"
#include "EngineModule.h"
#include "FXSystem.h"
#include "DistanceFieldLightingShared.h"
#include "SpeedTreeWind.h"
#include "Components/WindDirectionalSourceComponent.h"
#include "Lumen/LumenSceneData.h"
#include "PlanarReflectionSceneProxy.h"
#include "Engine/StaticMesh.h"
#include "GPUSkinCache.h"
#include "SkeletalMeshUpdater.h"
#include "ComputeSystemInterface.h"
#include "DynamicShadowMapChannelBindingHelper.h"
#include "GPUScene.h"
#include "HAL/LowLevelMemStats.h"
#include "HAL/LowLevelMemTracker.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "VT/RuntimeVirtualTextureSceneProxy.h"
#include "VT/VirtualTextureSystem.h"
#include "HairStrandsInterface.h"
#include "VelocityRendering.h"
#include "RectLightSceneProxy.h"
#include "RectLightTextureManager.h"
#include "RenderCore.h"
#include "SceneRenderBuilder.h"
#include "IESTextureManager.h"
#include "Materials/MaterialRenderProxy.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "StateStream/SkinnedMeshStateStreamImpl.h"
#include "StateStream/StaticMeshStateStreamImpl.h"
#include "StateStream/ParticleSystemStateStreamImpl.h"
#include "StateStreamCreator.h"
#include "StateStreamManagerImpl.h"
#include "SceneCulling/SceneCulling.h"
#include "TransformStateStreamImpl.h"
#include "InstanceCulling/InstanceCullingOcclusionQuery.h"
#include "ComputeWorkerInterface.h"
#include "Nanite/NaniteMaterialsSceneExtension.h"
#include "Skinning/SkinningSceneExtension.h"
#include "ObjectCacheContext.h"
#include "Animation/AnimBank.h"
#include "Skinning/AnimBankTransformProvider.h"
#include "Skinning/AnimRuntimeTransformProvider.h"
#include "Skinning/AnimSequenceTransformProvider.h"
#include "Skinning/SkinningTransformProvider.h"
#include "PathTracing.h"
#include "SceneViewState.h"
#include "RayTracing/RaytracingOptions.h"
#include "RenderTargetPool.h"

#if RHI_RAYTRACING
#include "Nanite/NaniteRayTracing.h"
#include "RayTracingDynamicGeometryUpdateManager.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracing/RayTracingInstanceMask.h"
#endif
#include "RHIGPUReadback.h"
#include "ShaderPrint.h"

#include "VirtualShadowMaps/VirtualShadowMapCacheManager.h"
#include "Shadows/ShadowScene.h"
#include "VariableRateShadingImageManager.h"
#include "Streaming/SimpleStreamableAssetManager.h"

#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#endif
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "PrimitiveSceneShaderData.h"

#define VALIDATE_PRIMITIVE_PACKED_INDEX 0

/** Affects BasePassPixelShader.usf so must relaunch editor to recompile shaders. */
static TAutoConsoleVariable<int32> CVarEarlyZPassOnlyMaterialMasking(
	TEXT("r.EarlyZPassOnlyMaterialMasking"),
	1,
	TEXT("Whether to compute materials' mask opacity only in early Z pass. Changing this setting requires restarting the editor.\n")
	TEXT("Note: Needs r.EarlyZPass == 2 && r.EarlyZPassMovable == 1"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

TAutoConsoleVariable<int32> CVarEarlyZPass(
	TEXT("r.EarlyZPass"),
	3,
	TEXT("Whether to use a depth only pass to initialize Z culling for the base pass. Cannot be changed at runtime.\n")
	TEXT("Note: also look at r.EarlyZPassMovable\n")
	TEXT("  0: off\n")
	TEXT("  1: good occluders only: not masked, and large on screen\n")
	TEXT("  2: all opaque (including masked)\n")
	TEXT("  x: use built in heuristic (default is 3)"),
	ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarMobileEarlyZPass(
	TEXT("r.Mobile.EarlyZPass"),
	0,
	TEXT("Whether to use a depth only pass to initialize Z culling for the mobile base pass. Changing this setting requires restarting the editor.\n")
	TEXT("  0: off\n")
	TEXT("  1: all opaque \n")
	TEXT("  2: masked primitives only \n"),
	ECVF_ReadOnly
);

static TAutoConsoleVariable<int32> CVarBasePassWriteDepthEvenWithFullPrepass(
	TEXT("r.BasePassWriteDepthEvenWithFullPrepass"),
	0,
	TEXT("0 to allow a readonly base pass, which skips an MSAA depth resolve, and allows masked materials to get EarlyZ (writing to depth while doing clip() disables EarlyZ) (default)\n")
	TEXT("1 to force depth writes in the base pass.  Useful for debugging when the prepass and base pass don't match what they render."));

int32 GVisibilitySkipAlwaysVisible = 1;
static FAutoConsoleVariableRef CVarVisibilitySkipAlwaysVisible(
	TEXT("r.Visibility.SkipAlwaysVisible"),
	GVisibilitySkipAlwaysVisible,
	TEXT("Whether visibility passes should skip primitives marked always visible")
	TEXT("0: All primitives are processed by visibility passes")
	TEXT("1: Only primitives not marked with bAlwaysVisible will be processed by visibility passes"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarVisibilityLocalLightPrimitiveInteraction(
	TEXT("r.Visibility.LocalLightPrimitiveInteraction"),
	2,
	TEXT("Whether to allow computing local primitive interactions. May greatly speedup render thread time if not needed.\n")
	TEXT("0 - disabled\n")
	TEXT("1 - enabled\n")
	TEXT("2 - enabled if MegaLights is disabled"),
	ECVF_RenderThreadSafe
);

bool GSafeCullDistanceUpdate = true;
static FAutoConsoleVariableRef CVarSafeCullDistanceUpdate(
	TEXT("r.Render.SafeCullDistanceUpdate"),
	GSafeCullDistanceUpdate,
	TEXT("Update primitive cull distances before static meshes are added to the scene.\n")
	TEXT("0: Update while static meshes are potentially asynchronously being added\n")
	TEXT("1: Force update before static meshes are added\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarLumenSceneSurfaceCacheAllowPersistOnVisibilityChange(
	TEXT("r.LumenScene.SurfaceCache.AllowPersistOnVisibilityChange"),
	1,
	TEXT("Whether to allow primitives that affect indirect lighting while hidden to persist their surface cache on visibility change.\n")
	TEXT("Only supported through FScene::UpdatePrimitivesIsDrawn_RenderThread currently."),
	ECVF_RenderThreadSafe
);

DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer MotionBlurStartFrame"), STAT_FDeferredShadingSceneRenderer_MotionBlurStartFrame, STATGROUP_SceneRendering);

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FDistanceCullFadeUniformShaderParameters, "PrimitiveFade");

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FDitherUniformShaderParameters, "PrimitiveDither");

TRACE_DECLARE_INT_COUNTER(ScenePrimitives, TEXT("Scene/Primitives"));
TRACE_DECLARE_INT_COUNTER(ScenePrimitivesAdds, TEXT("Scene/Primitives/Adds"));
TRACE_DECLARE_INT_COUNTER(ScenePrimitivesRemoves, TEXT("Scene/Primitives/Removes"));
TRACE_DECLARE_INT_COUNTER(ScenePrimitivesUpdatesTransforms, TEXT("Scene/Primitives/Updates/Transforms"));
TRACE_DECLARE_INT_COUNTER(ScenePrimitivesUpdatesInstances, TEXT("Scene/Primitives/Updates/Instances"));
TRACE_DECLARE_INT_COUNTER(ScenePrimitivesArrayMax, TEXT("Scene/PrimitiveArrayMax"));
TRACE_DECLARE_INT_COUNTER(SceneLights, TEXT("Scene/Lights"));

/** Global primitive uniform buffer resource containing distance cull faded in */
TGlobalResource< FGlobalDistanceCullFadeUniformBuffer > GDistanceCullFadedInUniformBuffer;

/** Global primitive uniform buffer resource containing dither faded in */
TGlobalResource< FGlobalDitherUniformBuffer > GDitherFadedInUniformBuffer;

static FThreadSafeCounter FSceneViewState_UniqueID;
// Maintained on the render thread, must not be accessed on any other thread (except worker threads within the scope of a RT command).
static TSet<uint32> FSceneViewState_ActiveUniqueIDs;

/**
 * Holds the info to update SpeedTree wind per unique tree object in the scene, instead of per instance
 */
struct FSpeedTreeWindComputation
{
	explicit FSpeedTreeWindComputation() :
		ReferenceCount(1)
	{
	}

	/** SpeedTree wind object */
	FSpeedTreeWind Wind;

	/** Uniform buffer shared between trees of the same type. */
	TUniformBufferRef<FSpeedTreeUniformParameters> UniformBuffer;

	int32 ReferenceCount;
};

FPersistentSkyAtmosphereData::FPersistentSkyAtmosphereData()
	: bInitialised(false)
	, CurrentScreenResolution(0)
	, CurrentDepthResolution(0)
	, CurrentTextureAerialLUTFormat(PF_Unknown)
	, CameraAerialPerspectiveVolumeIndex(0)
	, bSeparatedAtmosphereMieRayLeigh(false)
{
}
void FPersistentSkyAtmosphereData::InitialiseOrNextFrame(ERHIFeatureLevel::Type FeatureLevel, FPooledRenderTargetDesc& AerialPerspectiveDesc, FRHICommandListImmediate& RHICmdList, bool bSeparatedAtmosphereMieRayLeighIn)
{
	if (!bInitialised || (bInitialised && ((AerialPerspectiveDesc.Extent.X != CurrentScreenResolution) || (AerialPerspectiveDesc.Depth != CurrentDepthResolution) 
		|| (AerialPerspectiveDesc.Format != CurrentTextureAerialLUTFormat) || (bSeparatedAtmosphereMieRayLeigh != bSeparatedAtmosphereMieRayLeighIn))))
	{
		bSeparatedAtmosphereMieRayLeigh = bSeparatedAtmosphereMieRayLeighIn;
		CameraAerialPerspectiveVolumeCount = FeatureLevel == ERHIFeatureLevel::ES3_1 ? 2 : 1;
		for (int i = 0; i < CameraAerialPerspectiveVolumeCount; ++i)
		{
			GRenderTargetPool.FindFreeElement(RHICmdList, AerialPerspectiveDesc, CameraAerialPerspectiveVolumes[i], 
				i==0 ? TEXT("SkyAtmosphere.CameraAPVolume0") : TEXT("SkyAtmosphere.CameraAPVolume1"));
			if (bSeparatedAtmosphereMieRayLeigh)
			{
				GRenderTargetPool.FindFreeElement(RHICmdList, AerialPerspectiveDesc, CameraAerialPerspectiveVolumesMieOnly[i],
					i == 0 ? TEXT("SkyAtmosphere.CameraAPVolumeMieOnly0") : TEXT("SkyAtmosphere.CameraAPVolumeMieOnly1"));
				GRenderTargetPool.FindFreeElement(RHICmdList, AerialPerspectiveDesc, CameraAerialPerspectiveVolumesRayOnly[i],
					i == 0 ? TEXT("SkyAtmosphere.CameraAPVolumeRayOnly0") : TEXT("SkyAtmosphere.CameraAPVolumeRayOnly1"));
			}
			else
			{
				CameraAerialPerspectiveVolumesMieOnly[i] = nullptr;
				CameraAerialPerspectiveVolumesRayOnly[i] = nullptr;
			}
		}
		bInitialised = true;
		CurrentScreenResolution = AerialPerspectiveDesc.Extent.X;
		CurrentDepthResolution = AerialPerspectiveDesc.Depth;
		CurrentTextureAerialLUTFormat = AerialPerspectiveDesc.Format;
	}

	CameraAerialPerspectiveVolumeIndex = (CameraAerialPerspectiveVolumeIndex + 1) % CameraAerialPerspectiveVolumeCount;
}
TRefCountPtr<IPooledRenderTarget> FPersistentSkyAtmosphereData::GetCurrentCameraAerialPerspectiveVolume()
{
	check(CameraAerialPerspectiveVolumes[CameraAerialPerspectiveVolumeIndex].IsValid());
	return CameraAerialPerspectiveVolumes[CameraAerialPerspectiveVolumeIndex];
}
TRefCountPtr<IPooledRenderTarget> FPersistentSkyAtmosphereData::GetCurrentCameraAerialPerspectiveVolumeMieOnly()
{
	check(CameraAerialPerspectiveVolumesMieOnly[CameraAerialPerspectiveVolumeIndex].IsValid());
	return CameraAerialPerspectiveVolumesMieOnly[CameraAerialPerspectiveVolumeIndex];
}
TRefCountPtr<IPooledRenderTarget> FPersistentSkyAtmosphereData::GetCurrentCameraAerialPerspectiveVolumeRayOnly()
{
	check(CameraAerialPerspectiveVolumesRayOnly[CameraAerialPerspectiveVolumeIndex].IsValid());
	return CameraAerialPerspectiveVolumesRayOnly[CameraAerialPerspectiveVolumeIndex];
}

void FSceneViewState::ResetVolumetricFogState()
{
	LightScatteringHistoryPreExposure = 1.0f;
	PrevLightScatteringViewGridUVToViewRectVolumeUV = FVector2f::One();
	VolumetricFogPrevViewGridRectUVToResourceUV = FVector2f::One();
	VolumetricFogPrevUVMax = FVector2f::One();
	VolumetricFogPrevUVMaxForTemporalBlend = FVector2f::One();
	VolumetricFogPrevResourceGridSize = FIntVector(1, 1, 1);
	LightScatteringHistory = nullptr;
	PrevLightScatteringConservativeDepthTexture = nullptr;
}

/** Default constructor. */
FSceneViewState::FSceneViewState(ERHIFeatureLevel::Type FeatureLevel, FSceneViewState* ShareOriginTarget)
	: OcclusionQueryPool(RHICreateRenderQueryPool(RQT_Occlusion))
{
	// Set FeatureLevel to a valid value, so we get Init/ReleaseRHI calls on FeatureLevel changes
	SetFeatureLevel(FeatureLevel);
	
	UniqueID = FSceneViewState_UniqueID.Increment();

	ENQUEUE_RENDER_COMMAND(SceneViewStateAdddUniqueID)(
		[UniqueID = UniqueID] (FRHICommandListBase&)
		{
			FSceneViewState_ActiveUniqueIDs.Add(UniqueID);
		});	

	Scene = nullptr;
	OcclusionFrameCounter = 0;
	LastRenderTime = -FLT_MAX;
	MotionBlurTimeScale = 1.0f;
	MotionBlurTargetDeltaTime = 1.0f / 60.0f; // Start with a reasonable default of 60hz.
	PrevViewMatrixForOcclusionQuery.SetIdentity();
	PrevViewOriginForOcclusionQuery = FVector::ZeroVector;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bIsFreezing = false;
	bIsFrozen = false;
	bIsFrozenViewMatricesCached = false;
#endif
	// Register this object as a resource, so it will receive device reset notifications.
	if ( IsInGameThread() )
	{
		BeginInitResource(this);
	}
	else
	{
		InitResource(FRHICommandListImmediate::Get());
	}
	CachedVisibilityChunk = NULL;
	CachedVisibilityHandlerId = INDEX_NONE;
	CachedVisibilityBucketIndex = INDEX_NONE;
	CachedVisibilityChunkIndex = INDEX_NONE;
	MIDUsedCount = 0;
	TemporalAASampleIndex = 0;
	FrameIndex = 0;
	OutputFrameIndex = 0;
	DistanceFieldTemporalSampleIndex = 0;
	
	// Sets the mipbias to invalid large number.
	MaterialTextureCachedMipBias = BIG_NUMBER;
	LandscapeCachedMipBias = BIG_NUMBER;

	SequencerState = ESS_None;

	bIsStereoView = false;

	bRoundRobinOcclusionEnabled = false;

	if (ShareOriginTarget)
	{
		GlobalDistanceFieldData = ShareOriginTarget->GlobalDistanceFieldData;
		ShareOriginUniqueID = ShareOriginTarget->UniqueID;
	}
	else
	{
		GlobalDistanceFieldData = new FPersistentGlobalDistanceFieldData;
		ShareOriginUniqueID = UniqueID;
	}

	ShadowOcclusionQueryMaps.Empty(FOcclusionQueryHelpers::MaxBufferedOcclusionFrames);
	ShadowOcclusionQueryMaps.AddZeroed(FOcclusionQueryHelpers::MaxBufferedOcclusionFrames);	

	PreExposure = 1.f;
	bUpdateLastExposure = false;

	bLumenSceneDataAdded = false;
	LumenSurfaceCacheResolution = 1.0f;

	// OcclusionFeedback works only with mobile rendering atm
	if (FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		extern int32 GOcclusionFeedback_Enable;
		if (GOcclusionFeedback_Enable != 0)
		{
			BeginInitResource(&OcclusionFeedback);
		}
	}

	Occlusion.LastOcclusionQueryArray.SetNumZeroed(FOcclusionQueryHelpers::MaxBufferedOcclusionFrames);

	ResetVolumetricFogState();
}

FSceneViewState::~FSceneViewState()
{
	check(IsInRenderingThread());
	FSceneViewState_ActiveUniqueIDs.Remove(UniqueID);

	CachedVisibilityChunk = NULL;
	ShadowOcclusionQueryMaps.Reset();

	HairStrandsViewStateData.Release();
	ShaderPrintStateData.Release();

	if (Scene)
	{
		Scene->RemoveViewState_RenderThread(this);
	}

#if WITH_EDITORONLY_DATA
	if (ExposureCompensationCurveLUT.Curve_GameThread.IsValid()
		&& ExposureCompensationCurveLUT.CurveUpdateDelegateHandle.IsValid())
	{
		ExposureCompensationCurveLUT.Curve_GameThread->OnUpdateCurve.Remove(ExposureCompensationCurveLUT.CurveUpdateDelegateHandle);
		ExposureCompensationCurveLUT.CurveUpdateDelegateHandle.Reset();
	}
#endif
}

void FSceneViewState::ReleaseRHI()
{
	// FSceneViewState contains multiple sub-components that have GPU readback buffers which can have in-flight async copy
	// passes registered with RDG (e.g. AddEnqueueCopyPass), so we need to sync the execution tasks prior to releasing those buffers.
	FRDGBuilder::WaitForAsyncExecuteTask();

	HZBOcclusionTests.ReleaseRHI();
	EyeAdaptationManager.SafeRelease(); 
	SubstrateViewDebugData.SafeRelease();
	OcclusionFeedback.ReleaseResource();
	bValidEyeAdaptationBuffer = false;
}


inline FScene::FPrimitiveSceneProxyType::FPrimitiveSceneProxyType(const FPrimitiveSceneProxy *PrimitiveSceneProxy) 
	: ProxyTypeHash(PrimitiveSceneProxy->GetTypeHash())
	, bIsAlwaysVisible(PrimitiveSceneProxy->IsAlwaysVisible())
{
}

void FScene::RemoveViewLumenSceneData_RenderThread(FSceneViewStateInterface* ViewState)
{
	FLumenSceneDataKey ByViewKey = { ViewState->GetViewKey(), (uint32)INDEX_NONE };
	FLumenSceneData* const* Found = PerViewOrGPULumenSceneData.Find(ByViewKey);
	if (Found)
	{
		delete* Found;
		PerViewOrGPULumenSceneData.Remove(ByViewKey);
	}
}

void FScene::RemoveViewState_RenderThread(FSceneViewStateInterface* ViewState)
{
	for (int32 ViewStateIndex = 0; ViewStateIndex < ViewStates.Num(); ViewStateIndex++)
	{
		if (ViewStates[ViewStateIndex] == ViewState)
		{
			ViewStates.RemoveAt(ViewStateIndex);
			break;
		}
	}

	RemoveViewLumenSceneData_RenderThread(ViewState);

#if RHI_RAYTRACING
	{
		const uint32 ViewKey = ViewState->GetViewKey();
		RayTracingScene.RemoveView(ViewKey);
	}
#endif // RHI_RAYTRACING
}


#if WITH_EDITOR

FPixelInspectorData::FPixelInspectorData()
{
	for (int32 i = 0; i < 2; ++i)
	{
		RenderTargetBufferFinalColor[i] = nullptr;
		RenderTargetBufferDepth[i] = nullptr;
		RenderTargetBufferSceneColor[i] = nullptr;
		RenderTargetBufferHDR[i] = nullptr;
		RenderTargetBufferA[i] = nullptr;
		RenderTargetBufferBCDEF[i] = nullptr;
	}
}

void FPixelInspectorData::InitializeBuffers(FRenderTarget* BufferFinalColor, FRenderTarget* BufferSceneColor, FRenderTarget* BufferDepth, FRenderTarget* BufferHDR, FRenderTarget* BufferA, FRenderTarget* BufferBCDEF, int32 BufferIndex)
{
	RenderTargetBufferFinalColor[BufferIndex] = BufferFinalColor;
	RenderTargetBufferDepth[BufferIndex] = BufferDepth;
	RenderTargetBufferSceneColor[BufferIndex] = BufferSceneColor;
	RenderTargetBufferHDR[BufferIndex] = BufferHDR;
	RenderTargetBufferA[BufferIndex] = BufferA;
	RenderTargetBufferBCDEF[BufferIndex] = BufferBCDEF;

	check(RenderTargetBufferBCDEF[BufferIndex] != nullptr);
	
	FIntPoint BufferSize = RenderTargetBufferBCDEF[BufferIndex]->GetSizeXY();
	check(BufferSize.X == 4 && BufferSize.Y == 1);

	if (RenderTargetBufferA[BufferIndex] != nullptr)
	{
		BufferSize = RenderTargetBufferA[BufferIndex]->GetSizeXY();
		check(BufferSize.X == 1 && BufferSize.Y == 1);
	}
	
	if (RenderTargetBufferFinalColor[BufferIndex] != nullptr)
	{
		BufferSize = RenderTargetBufferFinalColor[BufferIndex]->GetSizeXY();
		//The Final color grab an area and can change depending on the setup
		//It should at least contain 1 pixel but can be 3x3 or more
		check(BufferSize.X > 0 && BufferSize.Y > 0);
	}

	if (RenderTargetBufferDepth[BufferIndex] != nullptr)
	{
		BufferSize = RenderTargetBufferDepth[BufferIndex]->GetSizeXY();
		check(BufferSize.X == 1 && BufferSize.Y == 1);
	}

	if (RenderTargetBufferSceneColor[BufferIndex] != nullptr)
	{
		BufferSize = RenderTargetBufferSceneColor[BufferIndex]->GetSizeXY();
		check(BufferSize.X == 1 && BufferSize.Y == 1);
	}

	if (RenderTargetBufferHDR[BufferIndex] != nullptr)
	{
		BufferSize = RenderTargetBufferHDR[BufferIndex]->GetSizeXY();
		check(BufferSize.X == 1 && BufferSize.Y == 1);
	}
}

bool FPixelInspectorData::AddPixelInspectorRequest(FPixelInspectorRequest *PixelInspectorRequest)
{
	if (PixelInspectorRequest == nullptr)
		return false;
	FVector2f ViewportUV = PixelInspectorRequest->SourceViewportUV;
	if (Requests.Contains(ViewportUV))
		return false;
	
	//Remove the oldest request since the new request use the buffer
	if (Requests.Num() > 1)
	{
		auto It = Requests.CreateIterator();
		It.RemoveCurrent();
	}
	Requests.Add(ViewportUV, PixelInspectorRequest);
	return true;
}

#endif //WITH_EDITOR

bool IncludePrimitiveInDistanceFieldSceneData(bool bTrackAllPrimitives, const FPrimitiveSceneProxy* Proxy)
{
	return PrimitiveNeedsDistanceFieldSceneData(
		bTrackAllPrimitives, 
		Proxy->CastsDynamicIndirectShadow(), 
		Proxy->AffectsDistanceFieldLighting(), 
		Proxy->IsDrawnInGame(),  
		Proxy->CastsHiddenShadow(), 
		Proxy->CastsDynamicShadow(),
		Proxy->AffectsDynamicIndirectLighting(),
		Proxy->AffectsIndirectLightingWhileHidden());
}

void FDistanceFieldSceneData::AddPrimitive(FPrimitiveSceneInfo* InPrimitive)
{
	FPrimitiveSceneProxy* Proxy = InPrimitive->Proxy;

	if (IncludePrimitiveInDistanceFieldSceneData(bTrackAllPrimitives, Proxy))
	{
		if (Proxy->SupportsHeightfieldRepresentation())
		{
			UTexture2D* HeightAndNormal;
			UTexture2D* Visibility;
			FHeightfieldComponentDescription Desc(FMatrix::Identity, InPrimitive->GetInstanceSceneDataOffset());
			Proxy->GetHeightfieldRepresentation(HeightAndNormal, Visibility, Desc);
			GHeightFieldTextureAtlas.AddAllocation(HeightAndNormal);

			if (Visibility)
			{
				check(Desc.VisibilityChannel >= 0 && Desc.VisibilityChannel < 4);
				GHFVisibilityTextureAtlas.AddAllocation(Visibility, Desc.VisibilityChannel);
			}

			checkSlow(!PendingHeightFieldAddOps.Contains(InPrimitive));
			PendingHeightFieldAddOps.Add(InPrimitive);
		}

		if (Proxy->SupportsDistanceFieldRepresentation())
		{
			checkSlow(!PendingAddOperations.Contains(InPrimitive));
			checkSlow(!PendingUpdateOperations.Contains(InPrimitive));
			PendingAddOperations.Add(InPrimitive);
		}
	}
}

void FDistanceFieldSceneData::UpdatePrimitive(FPrimitiveSceneInfo* InPrimitive)
{
	const FPrimitiveSceneProxy* Proxy = InPrimitive->Proxy;

	if (IncludePrimitiveInDistanceFieldSceneData(bTrackAllPrimitives, Proxy)
		&& Proxy->SupportsDistanceFieldRepresentation() 
		&& !PendingAddOperations.Contains(InPrimitive)
		// This is needed to prevent infinite buildup when DF features are off such that the pending operations don't get consumed
		&& !PendingUpdateOperations.Contains(InPrimitive)
		// This can happen when the primitive fails to allocate from the SDF atlas
		&& InPrimitive->DistanceFieldInstanceIndices.Num() > 0)
	{
		PendingUpdateOperations.Add(InPrimitive);
	}
}

void FDistanceFieldSceneData::RemovePrimitive(FPrimitiveSceneInfo* InPrimitive)
{
	FPrimitiveSceneProxy* Proxy = InPrimitive->Proxy;

	if (IncludePrimitiveInDistanceFieldSceneData(bTrackAllPrimitives, Proxy))
	{
		if (Proxy->SupportsDistanceFieldRepresentation())
		{
			PendingAddOperations.Remove(InPrimitive);
			PendingUpdateOperations.Remove(InPrimitive);

			if (InPrimitive->DistanceFieldInstanceIndices.Num() > 0)
			{
				PendingRemoveOperations.Add(FPrimitiveRemoveInfo(InPrimitive));
			}
			
			InPrimitive->DistanceFieldInstanceIndices.Empty();
		}

		if (Proxy->SupportsHeightfieldRepresentation())
		{
			UTexture2D* HeightAndNormal;
			UTexture2D* Visibility;
			FHeightfieldComponentDescription Desc(FMatrix::Identity, InPrimitive->GetInstanceSceneDataOffset());
			Proxy->GetHeightfieldRepresentation(HeightAndNormal, Visibility, Desc);
			GHeightFieldTextureAtlas.RemoveAllocation(HeightAndNormal);

			if (Visibility)
			{
				GHFVisibilityTextureAtlas.RemoveAllocation(Visibility);
			}

			PendingHeightFieldAddOps.Remove(InPrimitive);

			if (InPrimitive->DistanceFieldInstanceIndices.Num() > 0)
			{
				PendingHeightFieldRemoveOps.Add(FHeightFieldPrimitiveRemoveInfo(InPrimitive));
			}

			InPrimitive->DistanceFieldInstanceIndices.Empty();
		}
	}

	checkf(!PendingAddOperations.Contains(InPrimitive), TEXT("Primitive is being removed from the scene, but didn't remove from Distance Field Scene properly - a crash will occur when processing PendingAddOperations.  This can happen if the proxy's properties have changed without recreating its render state."));
	checkf(!PendingUpdateOperations.Contains(InPrimitive), TEXT("Primitive is being removed from the scene, but didn't remove from Distance Field Scene properly - a crash will occur when processing PendingUpdateOperations.  This can happen if the proxy's properties have changed without recreating its render state."));

	checkf(!PendingHeightFieldAddOps.Contains(InPrimitive), TEXT("Primitive is being removed from the scene, but didn't remove from Distance Field Scene properly - a crash will occur when processing PendingHeightFieldAddOps.  This can happen if the proxy's properties have changed without recreating its render state."));
}

void FDistanceFieldSceneData::Release()
{
	if (ObjectBuffers != nullptr)
	{
		ObjectBuffers->Release();
	}

	for (int32 BufferIndex = 0; BufferIndex < StreamingRequestReadbackBuffers.Num(); ++BufferIndex)
	{
		if (StreamingRequestReadbackBuffers[BufferIndex])
	{
			delete StreamingRequestReadbackBuffers[BufferIndex];
			StreamingRequestReadbackBuffers[BufferIndex] = nullptr;
		}
	}
}

void FDistanceFieldSceneData::VerifyIntegrity()
{
#if DO_CHECK
	check(NumObjectsInBuffer == PrimitiveInstanceMapping.Num());

	for (int32 PrimitiveInstanceIndex = 0; PrimitiveInstanceIndex < PrimitiveInstanceMapping.Num(); PrimitiveInstanceIndex++)
	{
		const FPrimitiveAndInstance& PrimitiveAndInstance = PrimitiveInstanceMapping[PrimitiveInstanceIndex];

		check(PrimitiveAndInstance.Primitive && PrimitiveAndInstance.Primitive->DistanceFieldInstanceIndices.Num() > 0);
		check(PrimitiveAndInstance.Primitive->DistanceFieldInstanceIndices.IsValidIndex(PrimitiveAndInstance.InstanceIndex));

		const int32 InstanceIndex = PrimitiveAndInstance.Primitive->DistanceFieldInstanceIndices[PrimitiveAndInstance.InstanceIndex];
		check(InstanceIndex == PrimitiveInstanceIndex || InstanceIndex == -1);
	}
#endif
}

void FScene::UpdateSceneSettings(AWorldSettings* WorldSettings)
{
	FScene* Scene = this;
	float InDefaultMaxDistanceFieldOcclusionDistance = WorldSettings->DefaultMaxDistanceFieldOcclusionDistance;
	float InGlobalDistanceFieldViewDistance = WorldSettings->GlobalDistanceFieldViewDistance;
	float InDynamicIndirectShadowsSelfShadowingIntensity = FMath::Clamp(WorldSettings->DynamicIndirectShadowsSelfShadowingIntensity, 0.0f, 1.0f);
	ENQUEUE_RENDER_COMMAND(UpdateSceneSettings)(
		[Scene, InDefaultMaxDistanceFieldOcclusionDistance, InGlobalDistanceFieldViewDistance, InDynamicIndirectShadowsSelfShadowingIntensity] (FRHICommandListBase&)
	{
		Scene->DefaultMaxDistanceFieldOcclusionDistance = InDefaultMaxDistanceFieldOcclusionDistance;
		Scene->GlobalDistanceFieldViewDistance = InGlobalDistanceFieldViewDistance;
		Scene->DynamicIndirectShadowsSelfShadowingIntensity = InDynamicIndirectShadowsSelfShadowingIntensity;
	});
}

/**
 * Sets the FX system associated with the scene.
 */
void FScene::SetFXSystem( class FFXSystemInterface* InFXSystem )
{
	FXSystem = InFXSystem;
}

/**
 * Get the FX system associated with the scene.
 */
FFXSystemInterface* FScene::GetFXSystem()
{
	return FXSystem;
}

FLumenSceneData* FScene::FindLumenSceneData(uint32 ViewKey, uint32 GPUIndex) const
{
	// First search by ViewKey
	FLumenSceneDataKey ByViewKey = { ViewKey, (uint32)INDEX_NONE };
	FLumenSceneData* const* Found = PerViewOrGPULumenSceneData.Find(ByViewKey);
	if (Found)
	{
		return *Found;
	}

	// Then search by GPU
	FLumenSceneDataKey ByGPUIndex = { 0, GPUIndex };
	Found = PerViewOrGPULumenSceneData.Find(ByGPUIndex);
	if (Found)
	{
		return *Found;
	}

	// If both fail, return default
	return DefaultLumenSceneData;
}

uint32 FScene::GetShareOriginViewKey(const FSceneViewState& ViewState)
{
	return ViewState.GetShareOriginViewKey();
}

void FScene::UpdateParameterCollections(const TArray<FMaterialParameterCollectionInstanceResource*>& InParameterCollections)
{
	ENQUEUE_RENDER_COMMAND(UpdateParameterCollectionsCommand)(
		[this, InParameterCollections] (FRHICommandListBase&)
	{
		// Async RDG tasks can call FMaterialShader::SetParameters which touch material parameter collections.
		FRDGBuilder::WaitForAsyncExecuteTask();

		// Empty the scene's map so any unused uniform buffers will be released
		ParameterCollections.Empty();

		// Add each existing parameter collection id and its uniform buffer
		for (int32 CollectionIndex = 0; CollectionIndex < InParameterCollections.Num(); CollectionIndex++)
		{
			FMaterialParameterCollectionInstanceResource* InstanceResource = InParameterCollections[CollectionIndex];
			ParameterCollections.Add(InstanceResource->GetId(), InstanceResource->GetUniformBuffer());
		}
	});
}

bool FScene::RequestGPUSceneUpdate(FPrimitiveSceneInfo& PrimitiveSceneInfo, EPrimitiveDirtyState PrimitiveDirtyState)
{
	return PrimitiveSceneInfo.RequestGPUSceneUpdate(PrimitiveDirtyState);
}

bool FScene::RequestUniformBufferUpdate(FPrimitiveSceneInfo& PrimitiveSceneInfo)
{
	return PrimitiveSceneInfo.RequestUniformBufferUpdate();
}

void FScene::RefreshNaniteRasterBins(FPrimitiveSceneInfo& PrimitiveSceneInfo)
{
	PrimitiveSceneInfo.RefreshNaniteRasterBins();
}

void FScene::ReloadNaniteFixedFunctionBins()
{
	for (int32 NanitePass = 0; NanitePass < ENaniteMeshPass::Num; ++NanitePass)
	{
		NaniteRasterPipelines[NanitePass].ReloadFixedFunctionBins();
	}
}

SIZE_T FScene::GetSizeBytes() const
{
	return sizeof(*this) 
		+ Primitives.GetAllocatedSize()
		+ Lights.GetAllocatedSize()
		+ StaticMeshes.GetAllocatedSize()
		+ ExponentialFogs.GetAllocatedSize()
		+ WindSources.GetAllocatedSize()
		+ SpeedTreeVertexFactoryMap.GetAllocatedSize()
		+ SpeedTreeWindComputationMap.GetAllocatedSize()
		+ LocalShadowCastingLightOctree.GetSizeBytes()
		+ PrimitiveOctree.GetSizeBytes();
}

void FScene::OnWorldCleanup()
{
	UniformBuffers.Clear();
}

void FScene::CheckPrimitiveArrays(int MaxTypeOffsetIndex)
{
	check(Primitives.Num() == PrimitiveTransforms.Num());
	check(Primitives.Num() == PrimitiveSceneProxies.Num());
	check(Primitives.Num() == PrimitiveBounds.Num());
	check(Primitives.Num() == PrimitiveFlagsCompact.Num());
	check(Primitives.Num() == PrimitiveVisibilityIds.Num());
	check(Primitives.Num() == PrimitiveOctreeIndex.Num());
	check(Primitives.Num() == PrimitiveOcclusionFlags.Num());
	check(Primitives.Num() == PrimitiveComponentIds.Num());
	check(Primitives.Num() == PrimitiveOcclusionBounds.Num());
#if WITH_EDITOR
	check(Primitives.Num() == PrimitivesSelected.Num());
#endif
#if RHI_RAYTRACING
	check(Primitives.Num() == PrimitiveRayTracingFlags.Num());
	check(Primitives.Num() == PrimitiveRayTracingDatas.Num());
	check(Primitives.Num() == PrimitiveRayTracingGroupIds.Num());
#endif
	check(Primitives.Num() == PrimitivesNeedingStaticMeshUpdate.Num());
	check(Primitives.Num() == PrimitivesNeedingUniformBufferUpdate.Num());

#if UE_BUILD_DEBUG
	MaxTypeOffsetIndex = MaxTypeOffsetIndex == INDEX_NONE ? TypeOffsetTable.Num() : MaxTypeOffsetIndex;
	for (int32 i = 0; i < MaxTypeOffsetIndex; i++)
	{
		for (int32 j = i + 1; j < MaxTypeOffsetIndex; j++)
		{
			check(TypeOffsetTable[i].PrimitiveSceneProxyType != TypeOffsetTable[j].PrimitiveSceneProxyType);
			check(TypeOffsetTable[i].Offset <= TypeOffsetTable[j].Offset);
		}
	}

	uint32 NextOffset = 0;
	for (int32 i = 0; i < MaxTypeOffsetIndex; i++)
	{
		const FTypeOffsetTableEntry& Entry = TypeOffsetTable[i];
		for (uint32 Index = NextOffset; Index < Entry.Offset; Index++)
		{
			checkSlow(Primitives[Index]->Proxy == PrimitiveSceneProxies[Index]);
			FPrimitiveSceneProxyType PrimitiveSceneProxyType = FPrimitiveSceneProxyType(PrimitiveSceneProxies[Index]);
			checkfSlow(PrimitiveSceneProxyType == Entry.PrimitiveSceneProxyType, TEXT("TypeHash: %zu not matching, expected: %zu"), PrimitiveSceneProxyType.ProxyTypeHash, Entry.PrimitiveSceneProxyType.ProxyTypeHash);
		}
		NextOffset = Entry.Offset;
	}
#endif
}

static void UpdateEarlyZPassModeCVarSinkFunction()
{
	static auto* CVarAntiAliasingMethod = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AntiAliasingMethod"));
	static auto* CVarMSAACount = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.MSAACount"));
	static int32 CachedAntiAliasingMethod = CVarAntiAliasingMethod->GetValueOnGameThread();
	static int32 CachedMSAACount = CVarMSAACount->GetValueOnGameThread();
	static int32 CachedEarlyZPass = CVarEarlyZPass.GetValueOnGameThread();
	static int32 CachedBasePassWriteDepthEvenWithFullPrepass = CVarBasePassWriteDepthEvenWithFullPrepass.GetValueOnGameThread();

	const int32 AntiAliasingMethod = CVarAntiAliasingMethod->GetValueOnGameThread();
	const int32 MSAACount = CVarMSAACount->GetValueOnGameThread();
	const int32 EarlyZPass = CVarEarlyZPass.GetValueOnGameThread();
	const int32 BasePassWriteDepthEvenWithFullPrepass = CVarBasePassWriteDepthEvenWithFullPrepass.GetValueOnGameThread();

	// Switching between MSAA and another AA in forward shading mode requires EarlyZPassMode to update.
	if (AntiAliasingMethod != CachedAntiAliasingMethod
		|| MSAACount != CachedMSAACount
		|| EarlyZPass != CachedEarlyZPass
		|| BasePassWriteDepthEvenWithFullPrepass != CachedBasePassWriteDepthEvenWithFullPrepass)
	{
		for (TObjectIterator<UWorld> It; It; ++It)
		{
			UWorld* World = *It;
			if (World && World->Scene)
			{
				World->Scene->UpdateEarlyZPassMode();
			}
		}

		CachedAntiAliasingMethod = AntiAliasingMethod;
		CachedMSAACount = MSAACount;
		CachedEarlyZPass = EarlyZPass;
		CachedBasePassWriteDepthEvenWithFullPrepass = BasePassWriteDepthEvenWithFullPrepass;
	}
}

static FAutoConsoleVariableSink CVarUpdateEarlyZPassModeSink(FConsoleCommandDelegate::CreateStatic(&UpdateEarlyZPassModeCVarSinkFunction));

void FScene::DumpMeshDrawCommandMemoryStats()
{
	SIZE_T TotalCachedMeshDrawCommands = 0;
	SIZE_T TotalStaticMeshCommandInfos = 0;

	struct FPassStats
	{
		SIZE_T CachedMeshDrawCommandBytes = 0;
		SIZE_T PSOBytes = 0;
		SIZE_T ShaderBindingInlineBytes = 0;
		SIZE_T ShaderBindingHeapBytes = 0;
		SIZE_T VertexStreamsInlineBytes = 0;
		SIZE_T DebugDataBytes = 0;
		SIZE_T DrawCommandParameterBytes = 0;
		uint32 NumCommands = 0;
	};

	FPassStats AllPassStats[EMeshPass::Num];
	TArray<bool> StateBucketAccounted[EMeshPass::Num];
	for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; PassIndex++)
	{
		StateBucketAccounted[PassIndex].Empty(CachedMeshDrawCommandStateBuckets[PassIndex].GetMaxIndex());
		StateBucketAccounted[PassIndex].AddZeroed(CachedMeshDrawCommandStateBuckets[PassIndex].GetMaxIndex());
	}

	for (int32 i = 0; i < Primitives.Num(); i++)
	{
		FPrimitiveSceneInfo* PrimitiveSceneInfo = Primitives[i];

		TotalStaticMeshCommandInfos += PrimitiveSceneInfo->StaticMeshCommandInfos.GetAllocatedSize();

		for (int32 CommandIndex = 0; CommandIndex < PrimitiveSceneInfo->StaticMeshCommandInfos.Num(); ++CommandIndex)
		{
			const FCachedMeshDrawCommandInfo& CachedCommand = PrimitiveSceneInfo->StaticMeshCommandInfos[CommandIndex];
			int PassIndex = CachedCommand.MeshPass;
			const FMeshDrawCommand* MeshDrawCommandPtr = nullptr;

			if (CachedCommand.StateBucketId != INDEX_NONE)
			{
				if (!StateBucketAccounted[PassIndex][CachedCommand.StateBucketId])
				{
					StateBucketAccounted[PassIndex][CachedCommand.StateBucketId] = true;
					MeshDrawCommandPtr = &CachedMeshDrawCommandStateBuckets[PassIndex].GetByElementId(CachedCommand.StateBucketId).Key;
				}
			}
			else if (CachedCommand.CommandIndex >= 0)
			{
				FCachedPassMeshDrawList& PassDrawList = CachedDrawLists[CachedCommand.MeshPass];
				MeshDrawCommandPtr = &PassDrawList.MeshDrawCommands[CachedCommand.CommandIndex];
			}

			if (MeshDrawCommandPtr)
			{
				const FMeshDrawCommand& MeshDrawCommand = *MeshDrawCommandPtr;
				FPassStats& PassStats = AllPassStats[CachedCommand.MeshPass];
				SIZE_T CommandBytes = sizeof(MeshDrawCommand) + MeshDrawCommand.GetAllocatedSize();
				PassStats.CachedMeshDrawCommandBytes += CommandBytes;
				TotalCachedMeshDrawCommands += MeshDrawCommand.GetAllocatedSize();
				PassStats.PSOBytes += sizeof(MeshDrawCommand.CachedPipelineId);
				PassStats.ShaderBindingInlineBytes += sizeof(MeshDrawCommand.ShaderBindings);
				PassStats.ShaderBindingHeapBytes += MeshDrawCommand.ShaderBindings.GetAllocatedSize();
				PassStats.VertexStreamsInlineBytes += sizeof(MeshDrawCommand.VertexStreams);
				PassStats.DebugDataBytes += MeshDrawCommand.GetDebugDataSize();
				PassStats.DrawCommandParameterBytes += sizeof(MeshDrawCommand.IndexBuffer) + sizeof(MeshDrawCommand.FirstIndex) + sizeof(MeshDrawCommand.NumPrimitives) + sizeof(MeshDrawCommand.NumInstances) + sizeof(MeshDrawCommand.VertexParams); //-V568
				PassStats.NumCommands++;
			}
		}
	}

	for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; PassIndex++)
	{
		TotalCachedMeshDrawCommands += CachedMeshDrawCommandStateBuckets[PassIndex].GetAllocatedSize();
	}

	for (int32 i = 0; i < EMeshPass::Num; i++)
	{
		TotalCachedMeshDrawCommands += CachedDrawLists[i].MeshDrawCommands.GetAllocatedSize();
	}

	for (int32 i = 0; i < EMeshPass::Num; i++)
	{
		const FPassStats& PassStats = AllPassStats[i];

		if (PassStats.NumCommands > 0)
		{
			UE_LOGF(LogRenderer, Log, "%ls: %.1fKb for %u CachedMeshDrawCommands", GetMeshPassName((EMeshPass::Type)i), PassStats.CachedMeshDrawCommandBytes / 1024.0f, PassStats.NumCommands);

			if (PassStats.CachedMeshDrawCommandBytes > 1024 && i <= EMeshPass::BasePass)
			{
				UE_LOGF(LogRenderer, Log, "     avg %.1f bytes PSO", PassStats.PSOBytes / (float)PassStats.NumCommands);
				UE_LOGF(LogRenderer, Log, "     avg %.1f bytes ShaderBindingInline", PassStats.ShaderBindingInlineBytes / (float)PassStats.NumCommands);
				UE_LOGF(LogRenderer, Log, "     avg %.1f bytes ShaderBindingHeap", PassStats.ShaderBindingHeapBytes / (float)PassStats.NumCommands);
				UE_LOGF(LogRenderer, Log, "     avg %.1f bytes VertexStreamsInline", PassStats.VertexStreamsInlineBytes / (float)PassStats.NumCommands);
				UE_LOGF(LogRenderer, Log, "     avg %.1f bytes DebugData", PassStats.DebugDataBytes / (float)PassStats.NumCommands);
				UE_LOGF(LogRenderer, Log, "     avg %.1f bytes DrawCommandParameters", PassStats.DrawCommandParameterBytes / (float)PassStats.NumCommands);

				const SIZE_T Other = PassStats.CachedMeshDrawCommandBytes -
					(PassStats.PSOBytes +
					PassStats.ShaderBindingInlineBytes +
					PassStats.ShaderBindingHeapBytes +
					PassStats.VertexStreamsInlineBytes +
					PassStats.DebugDataBytes +
					PassStats.DrawCommandParameterBytes);

				UE_LOGF(LogRenderer, Log, "     avg %.1f bytes Other", Other / (float)PassStats.NumCommands);
			}
		}
	}

	UE_LOGF(LogRenderer, Log, "sizeof(FMeshDrawCommand) %zu", sizeof(FMeshDrawCommand));
	UE_LOGF(LogRenderer, Log, "Total cached MeshDrawCommands %.3fMb", TotalCachedMeshDrawCommands / 1024.0f / 1024.0f);
	UE_LOGF(LogRenderer, Log, "Primitive StaticMeshCommandInfos %.1fKb", TotalStaticMeshCommandInfos / 1024.0f);
	UE_LOGF(LogRenderer, Log, "GPUScene CPU structures %.1fKb", GPUScene.GetAllocatedSize() / 1024.0f);
	UE_LOGF(LogRenderer, Log, "PSO persistent Id table %.1fKb %d elements", FGraphicsMinimalPipelineStateId::GetPersistentIdTableSize() / 1024.0f, FGraphicsMinimalPipelineStateId::GetPersistentIdNum());
	UE_LOGF(LogRenderer, Log, "PSO one frame Id %.1fKb", FGraphicsMinimalPipelineStateId::GetLocalPipelineIdTableSize() / 1024.0f);
}

template<typename T>
static void TArraySwapElements(TArray<T>& Array, int i1, int i2)
{
	T tmp = Array[i1];
	Array[i1] = Array[i2];
	Array[i2] = tmp;
}

template<typename T>
static void TArraySwapElements(TScenePrimitiveArray<T>& Array, int i1, int i2)
{
	T tmp = Array[i1];
	Array[i1] = Array[i2];
	Array[i2] = tmp;
}

static void TBitArraySwapElements(TBitArray<>& Array, int32 i1, int32 i2)
{
	FBitReference BitRef1 = Array[i1];
	FBitReference BitRef2 = Array[i2];
	bool Bit1 = BitRef1;
	bool Bit2 = BitRef2;
	BitRef1 = Bit2;
	BitRef2 = Bit1;
}

void FScene::AddPrimitiveSceneInfo_RenderThread(FPrimitiveSceneInfo* PrimitiveSceneInfo, const TOptional<FTransform>& PreviousTransform)
{
	// Must always be a novel primitive that is added
	check(PrimitiveSceneInfo->PackedIndex == INDEX_NONE);
	PrimitiveUpdates.EnqueueAdd(PrimitiveSceneInfo);

	if (PreviousTransform.IsSet())
	{
		PrimitiveUpdates.Enqueue<FUpdateOverridePreviousTransformData>(PrimitiveSceneInfo, FUpdateOverridePreviousTransformData(PreviousTransform.GetValue().ToMatrixWithScale()));
	}
}

/**
 * Verifies that a component is added to the proper scene
 *
 * @param Component Component to verify
 * @param World World who's scene the primitive is being attached to
 */
FORCEINLINE static void VerifyProperPIEScene(UObject* Component, UWorld* World)
{
	if (!Component)
	{
		return;
	}

	checkfSlow(Component->GetOuter() == GetTransientPackage() || 
		(FPackageName::GetLongPackageAssetName(Component->GetOutermostObject()->GetPackage()->GetName()).StartsWith(PLAYWORLD_PACKAGE_PREFIX) == 
		FPackageName::GetLongPackageAssetName(World->GetPackage()->GetName()).StartsWith(PLAYWORLD_PACKAGE_PREFIX)),
		TEXT("The component %s was added to the wrong world's scene (due to PIE). The callstack should tell you why"), 
		*Component->GetFullName()
		);
}

void FPersistentUniformBuffers::Clear()
{
	for (auto& UniformBuffer : MobileDirectionalLightUniformBuffers)
	{
		UniformBuffer.SafeRelease();
	}
	MobileSkyReflectionUniformBuffer.SafeRelease();

	Initialize();
}

void FPersistentUniformBuffers::Initialize()
{
	FViewUniformShaderParameters ViewUniformBufferParameters;


	FMobileDirectionalLightShaderParameters MobileDirectionalLightShaderParameters = {};
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(MobileDirectionalLightUniformBuffers); ++Index)
	{
		// UniformBuffer_SingleFrame here is an optimization as this buffer gets uploaded everyframe
		MobileDirectionalLightUniformBuffers[Index] = TUniformBufferRef<FMobileDirectionalLightShaderParameters>::CreateUniformBufferImmediate(MobileDirectionalLightShaderParameters, UniformBuffer_MultiFrame, EUniformBufferValidation::None);
	}

	const FMobileReflectionCaptureShaderParameters* DefaultMobileSkyReflectionParameters = (const FMobileReflectionCaptureShaderParameters*)GDefaultMobileReflectionCaptureUniformBuffer.GetContents();
	MobileSkyReflectionUniformBuffer = TUniformBufferRef<FMobileReflectionCaptureShaderParameters>::CreateUniformBufferImmediate(*DefaultMobileSkyReflectionParameters, UniformBuffer_MultiFrame, EUniformBufferValidation::None);
}

TSet<IPersistentViewUniformBufferExtension*> PersistentViewUniformBufferExtensions;

void FRendererModule::RegisterPersistentViewUniformBufferExtension(IPersistentViewUniformBufferExtension* Extension)
{
	PersistentViewUniformBufferExtensions.Add(Extension);
}

FScene::FScene(UWorld* InWorld, bool bInRequiresHitProxies, bool bInIsEditorScene, bool bCreateFXSystem, ERHIFeatureLevel::Type InFeatureLevel)
:	FSceneInterface(InFeatureLevel)
,	World(InWorld)
,	FXSystem(nullptr)
,	bCachedShouldRenderSkylightInBasePass(false)
,	bCachedSkyLightRealTimeCapture(false)
,	bScenesPrimitivesNeedStaticMeshElementUpdate(false)
,   PathTracingInvalidationCounter(0)
#if RHI_RAYTRACING
,   CachedRayTracingMeshCommandsType(ERayTracingType::RayTracing)
#endif
,	SkyLight(NULL)
,	ConvolvedSkyRenderTargetReadyIndex(-1)
,   PathTracingSkylightColor(0, 0, 0, 0)
,	SimpleDirectionalLight(NULL)
,	ReflectionSceneData(InFeatureLevel)
,	IndirectLightingCache(InFeatureLevel)
,	VolumetricLightmapSceneData(this)
,	GPUScene(*this)
,	DistanceFieldSceneData(GShaderPlatformForFeatureLevel[InFeatureLevel])
,	DefaultLumenSceneData(nullptr)
,	PreshadowCacheLayout(0, 0, 0, 0, false)
,	SkyAtmosphere(NULL)
,	CachedTransmittanceAndMultiScatteringLUTsVersion(0)
,	VolumetricCloud(NULL)
,	PrecomputedVisibilityHandler(NULL)
,	LocalShadowCastingLightOctree(FVector::ZeroVector, UE_OLD_HALF_WORLD_MAX)
,	PrimitiveOctree(FVector::ZeroVector, UE_OLD_HALF_WORLD_MAX)
,	bRequiresHitProxies(bInRequiresHitProxies)
,	bIsEditorScene(bInIsEditorScene)
,	bRuntimeVirtualTexturePrimitiveHideEditor(false)
,	bRuntimeVirtualTexturePrimitiveHideGame(false)
,	NumUncachedStaticLightingInteractions(0)
,	NumUnbuiltReflectionCaptures(0)
,	NumMobileStaticAndCSMLights_RenderThread(0)
,	NumMobileMovableDirectionalLights_RenderThread(0)
,	GPUSkinCache(nullptr)
,	SceneLODHierarchy(this)
,	DefaultMaxDistanceFieldOcclusionDistance(InWorld->GetWorldSettings()->DefaultMaxDistanceFieldOcclusionDistance)
,	GlobalDistanceFieldViewDistance(InWorld->GetWorldSettings()->GlobalDistanceFieldViewDistance)
,	DynamicIndirectShadowsSelfShadowingIntensity(FMath::Clamp(InWorld->GetWorldSettings()->DynamicIndirectShadowsSelfShadowingIntensity, 0.0f, 1.0f))
#if RHI_RAYTRACING
,	RayTracingDynamicGeometryUpdateManager(nullptr)
#endif
,	PersistentViewStateDebugFlags(0)
,	NumVisibleLights_AnyThread(0)
,	NumEnabledSkylights_GameThread(0)
,	SceneFrameNumber(0)
,	SceneFrameNumberRenderThread(0)
,	bForceNoPrecomputedLighting(InWorld->GetWorldSettings()->bForceNoPrecomputedLighting)
{
	FMemory::Memzero(MobileDirectionalLights);
	FMemory::Memzero(AtmosphereLights);
	FMemory::Memzero(MobileSkyLightRealTimeCaptureIrradianceEnvironmentMap);

	FullWorldName = World->GetFullName();

	check(World);
	World->Scene = this;

	FeatureLevel = World->GetFeatureLevel();
	ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

	checkf((uint32)FeatureLevel < (uint32)ERHIFeatureLevel::Num, TEXT("World provided an invalid feature level (%d) to FScene."), FeatureLevel);
	checkf(ShaderPlatform != SP_NumPlatforms, TEXT("Invalid feature level %s for platform (max feature level %s)"), *LexToString(FeatureLevel), *LexToString(GMaxRHIFeatureLevel));

	for (int32 NanitePass = 0; NanitePass < ENaniteMeshPass::Num; ++NanitePass)
	{
		const bool bBasePass = (ENaniteMeshPass::Type)NanitePass == ENaniteMeshPass::BasePass;	// Only basepass for now
		NaniteShadingPipelines[NanitePass].Init(UseNaniteBindlessShading(ShaderPlatform) && bBasePass);
	}

	GPUScene.SetEnabled(ShaderPlatform);

	if (GPUScene.IsEnabled())
	{
		InstanceCullingOcclusionQueryRenderer = new FInstanceCullingOcclusionQueryRenderer;
	}

	if (World->FXSystem)
	{
		FFXSystemInterface::MarkPendingKill(World->FXSystem.Get());
	}

	if (bCreateFXSystem)
	{
		World->CreateFXSystem();
	}
	else
	{
		World->FXSystem = NULL;
		SetFXSystem(NULL);
	}

	if (IsGPUSkinCacheAvailable(GetFeatureLevelShaderPlatform(InFeatureLevel)))
	{
		GPUSkinCache = new FGPUSkinCache(InFeatureLevel, World);
	}

	SkeletalMeshUpdater = new FSkeletalMeshUpdater(this, GPUSkinCache);

	ComputeSystemInterface::CreateWorkers(this, ComputeTaskWorkers);

#if RHI_RAYTRACING
	if (IsRayTracingAllowed())
	{
		RayTracingDynamicGeometryUpdateManager = new FRayTracingDynamicGeometryUpdateManager();
	}
#endif

	World->UpdateParameterCollectionInstances(false, false);

	FPersistentUniformBuffers* PersistentUniformBuffers = &UniformBuffers;
	ENQUEUE_RENDER_COMMAND(InitializeUniformBuffers)(
		[PersistentUniformBuffers] (FRHICommandListBase&)
	{
		PersistentUniformBuffers->Initialize();
	});

	UpdateEarlyZPassMode();

	DefaultLumenSceneData = new FLumenSceneData(GShaderPlatformForFeatureLevel[InFeatureLevel], World->WorldType, World->AllowLumenPrimitiveTrackingInPreviewWorld());

	SceneLightInfoUpdates = new FSceneLightInfoUpdates;

	// Make sure we initialize the SceneRenderExtensions last, when the rest of the scene is initialized
	SceneExtensions.Init(*this);
}

FScene::~FScene()
{
#if 0 // if you have component that has invalid scene, try this code to see this is reason. 
	for (FThreadSafeObjectIterator Iter(UActorComponent::StaticClass()); Iter; ++Iter)
	{
		UActorComponent * ActorComp = CastChecked<UActorComponent>(*Iter);
		if (ActorComp->GetScene() == this)
		{
			UE_LOGF(LogRenderer, Log, "%ls's scene is going to get invalidated", *ActorComp->GetName());
		}
	}
#endif

	checkf(PrimitiveUpdates.IsEmpty(), TEXT("All pending primitive addition operations are expected to be flushed when the scene is destroyed. Remaining operations are likely to cause a memory leak."));
	checkf(Primitives.Num() == 0, TEXT("All primitives are expected to be removed before the scene is destroyed. Remaining primitives are likely to cause a memory leak."));

	delete InstanceCullingOcclusionQueryRenderer;

	// Unlink any view states from the scene
	for (FSceneViewState* ViewState : ViewStates)
	{
		check(ViewState->Scene == this);
		ViewState->Scene = nullptr;
	}
	ViewStates.Empty();

	if (DefaultLumenSceneData)
	{
		delete DefaultLumenSceneData;
		DefaultLumenSceneData = nullptr;
	}

	for (FLumenSceneDataMap::TConstIterator LumenSceneData(PerViewOrGPULumenSceneData); LumenSceneData; ++LumenSceneData)
	{
		delete LumenSceneData.Value();
	}
	PerViewOrGPULumenSceneData.Empty();

	ReflectionSceneData.CubemapArray.ReleaseResource();
	IndirectLightingCache.ReleaseResource();
	DistanceFieldSceneData.Release();

	delete GPUSkinCache;
	GPUSkinCache = nullptr;

	delete SkeletalMeshUpdater;
	SkeletalMeshUpdater = nullptr;

	ComputeSystemInterface::DestroyWorkers(this, ComputeTaskWorkers);

#if RHI_RAYTRACING
	delete RayTracingDynamicGeometryUpdateManager;
	RayTracingDynamicGeometryUpdateManager = nullptr;
#endif // RHI_RAYTRACING

	delete SceneLightInfoUpdates;

	TSharedPtr<FRHIGPUBufferReadback> It;
	while (MobileSkyLightRealTimeCaptureIrradianceReadBackQueries.Dequeue(It))
	{
		It.Reset();
	}
}

// Helpers for internal templates
UObject* ToUObject(FPrimitiveSceneDesc* Desc)
{
	return Desc->PrimitiveUObject;
}

UObject* ToUObject(UPrimitiveComponent* Prim)
{
	return Prim;
}


void FScene::AddPrimitive(UPrimitiveComponent* Primitive)
{	
	// If the bulk reregister flag is set, add / remove will be handled in bulk by the FStaticMeshComponentBulkReregisterContext
	if (Primitive->bBulkReregister)
	{
		return;
	}
	BatchAddPrimitivesInternal(MakeArrayView(&Primitive, 1));
}

void FScene::AddPrimitive(FPrimitiveSceneDesc* Primitive)
{
	// If the bulk reregister flag is set, add / remove will be handled in bulk by the FStaticMeshComponentBulkReregisterContext
	if (Primitive->bBulkReregister)
	{
		return;
	}

	BatchAddPrimitivesInternal(MakeArrayView(&Primitive, 1));
}

template<typename PrimitiveType>
static void CheckAndSanitizePrimitiveBounds(FBoxSphereBounds &InOutWorldBounds, PrimitiveType *Primitive)
{
	// Help track down primitive with bad bounds way before the it gets to the Renderer
	if (!ensureMsgf(!InOutWorldBounds.ContainsNaN(),
		TEXT("NaNs found on Bounds for Primitive %s: Owner: %s, Resource: %s, Level: %s, Origin: %s, BoxExtent: %s, SphereRadius: %f"),
		*Primitive->GetName(),
		*Primitive->GetSceneProxy()->GetOwnerName().ToString(),
		*Primitive->GetSceneProxy()->GetResourceName().ToString(),
		*Primitive->GetSceneProxy()->GetLevelName().ToString(),
		*InOutWorldBounds.Origin.ToString(),
		*InOutWorldBounds.BoxExtent.ToString(),
		InOutWorldBounds.SphereRadius
		))
	{
		InOutWorldBounds = FBoxSphereBounds(ForceInit);
	}
}

template<class T> 	
void FScene::BatchAddPrimitivesInternal(TArrayView<T*> InPrimitives)
{
	check(InPrimitives.Num() > 0);
	
#if WITH_STATE_STREAM
	if (IsInGameThread() || IsInParallelGameThread())
	{
		if (UObject* PrimitiveObject = ToUObject(InPrimitives.Last()))
		{
			GetRendererModule().DebugLogRenderProxies(PrimitiveObject->GetClass()->GetFName(), InPrimitives.Num());
		}
	}
#endif

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	// If detailed per-tag asset memory stats are active, don't batch primitives, so the memory tags can be independent
	if (FLowLevelMemTracker::Get().IsTagSetActive(ELLMTagSet::Assets) && InPrimitives.Num() > 1)
	{
		for (T* Primitive : InPrimitives)
		{
			BatchAddPrimitivesInternal(MakeArrayView(TArrayView<T*>(&Primitive, 1)));
		}
		return;
	}
#endif
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(ToUObject(InPrimitives[0]) ? InPrimitives[0]->GetOutermost()->GetFName() : NAME_None, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(ToUObject(InPrimitives[0]) ? FName(UPackage::StaticClass()->GetPathName()) : NAME_None, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, ToUObject(InPrimitives[0]) ? FName(UPackage::StaticClass()->GetPathName()) : NAME_None,
		ToUObject(InPrimitives[0]) ? InPrimitives[0]->GetOutermost()->GetFName() : NAME_None);

	SCOPE_CYCLE_COUNTER(STAT_AddScenePrimitiveGT);

	struct FCreateCommand
	{
		FCreateCommand(
			FPrimitiveSceneInfo* InPrimitiveSceneInfo,
			FPrimitiveSceneProxy* InPrimitiveSceneProxy,
			TOptional<FTransform> InPreviousTransform,
			FMatrix InRenderMatrix,
			FBoxSphereBounds InWorldBounds,
			FVector InAttachmentRootPosition,
			FBoxSphereBounds InLocalBounds)
			: PrimitiveSceneInfo(InPrimitiveSceneInfo)
			, PrimitiveSceneProxy(InPrimitiveSceneProxy)
			, PreviousTransform(InPreviousTransform)
			, RenderMatrix(InRenderMatrix)
			, WorldBounds(InWorldBounds)
			, AttachmentRootPosition(InAttachmentRootPosition)
			, LocalBounds(InLocalBounds)
		{}

		FPrimitiveSceneInfo* PrimitiveSceneInfo;
		FPrimitiveSceneProxy* PrimitiveSceneProxy;
		TOptional<FTransform> PreviousTransform;
		FMatrix RenderMatrix;
		FBoxSphereBounds WorldBounds;
		FVector AttachmentRootPosition;
		FBoxSphereBounds LocalBounds;
	};

	TArray<FCreateCommand, SceneRenderingAllocator> CreateCommands;
	CreateCommands.Reserve(InPrimitives.Num());
	
	for (T* Primitive : InPrimitives)
	{
		FPrimitiveSceneInfoData& SceneData = Primitive->GetSceneData();
	
		checkf(!Primitive->IsUnreachable(), TEXT("%s"), *Primitive->GetFullName());

		FPrimitiveSceneProxy* PrimitiveSceneProxy  = nullptr;

		if (Primitive->GetPrimitiveComponentInterface() && !Primitive->GetSceneProxy())
		{
			PrimitiveSceneProxy = Primitive->GetPrimitiveComponentInterface()->CreateSceneProxy();
			check(SceneData.SceneProxy == PrimitiveSceneProxy); // CreateSceneProxy has access to the shared SceneData and should set it properly
		}
		else
		{
			check(!Primitive->ShouldRecreateProxyOnUpdateTransform()); // recreating proxies when updating the transform requires a IPrimitiveComponentInterface
			PrimitiveSceneProxy = Primitive->GetSceneProxy();
		}
	
		if(!PrimitiveSceneProxy)
		{
			// Primitives which don't have a proxy are irrelevant to the scene manager.
			continue;
		}

		// Create the primitive scene info.
		FPrimitiveSceneInfo* PrimitiveSceneInfo = new FPrimitiveSceneInfo(Primitive, this);
		PrimitiveSceneProxy->PrimitiveSceneInfo = PrimitiveSceneInfo;

		// Cache the primitives initial transform.
		FMatrix RenderMatrix = Primitive->GetRenderMatrix();
		FVector AttachmentRootPosition = Primitive->GetActorPositionForRenderer();
		FBoxSphereBounds WorldBounds = Primitive->Bounds;

		CheckAndSanitizePrimitiveBounds(WorldBounds, Primitive);

		CreateCommands.Emplace(
			PrimitiveSceneInfo,
			PrimitiveSceneProxy,
			// If this primitive has a simulated previous transform, ensure that the velocity data for the scene representation is correct.
			FMotionVectorSimulation::Get().GetPreviousTransform(ToUObject(Primitive)),
			RenderMatrix,
			WorldBounds,
			AttachmentRootPosition,
			Primitive->GetLocalBounds()
		);


		INC_DWORD_STAT_BY( STAT_GameToRendererMallocTotal, PrimitiveSceneProxy->GetMemoryFootprint() + PrimitiveSceneInfo->GetMemoryFootprint() );

		// Verify the primitive is valid
		VerifyProperPIEScene(ToUObject(Primitive), World);

		FSimpleStreamableAssetManager::Register(Primitive->GetPrimitiveComponentInterface(), PrimitiveSceneProxy, RenderMatrix, WorldBounds);

		// Increment the attachment counter, the primitive is about to be attached to the scene.
		SceneData.AttachmentCounter.Increment();
	}

	if (!CreateCommands.IsEmpty())
	{
		// Must enqueue RT commands to ensure SetTransform & CreateRenderThreadResources are executed in-order.
		ENQUEUE_RENDER_COMMAND(AddPrimitiveCommand)(
			[this, CreateCommands = MoveTemp(CreateCommands)](FRHICommandListBase& RHICmdList)
		{
			for (const FCreateCommand& Command : CreateCommands)
			{
				FScopeCycleCounter Context(Command.PrimitiveSceneProxy->GetStatId());
				Command.PrimitiveSceneProxy->SetTransform(RHICmdList, Command.RenderMatrix, Command.WorldBounds, Command.LocalBounds, Command.AttachmentRootPosition);
				Command.PrimitiveSceneProxy->CreateRenderThreadResources(RHICmdList);

				AddPrimitiveSceneInfo_RenderThread(Command.PrimitiveSceneInfo, Command.PreviousTransform);
			}
		});
	}
}

void FScene::BatchAddPrimitives(TArrayView<UPrimitiveComponent*> InPrimitives)
{
	BatchAddPrimitivesInternal(InPrimitives);
}

void FScene::BatchAddPrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives)
{
	BatchAddPrimitivesInternal(InPrimitives);
}

static int32 GWarningOnRedundantTransformUpdate = 0;
static FAutoConsoleVariableRef CVarWarningOnRedundantTransformUpdate(
	FCVarDeprecationArgs{ TEXT("5.8") },
	TEXT("r.WarningOnRedundantTransformUpdate"),
	GWarningOnRedundantTransformUpdate,
	TEXT("Produce a warning when UpdatePrimitiveTransform is called redundantly."),
	ECVF_Default
);

static int32 GSkipRedundantTransformUpdate = 0;
static FAutoConsoleVariableRef CVarSkipRedundantTransformUpdate(
	FCVarDeprecationArgs{ TEXT("5.8") },
	TEXT("r.SkipRedundantTransformUpdate"),
	GSkipRedundantTransformUpdate,
	TEXT("Skip updates UpdatePrimitiveTransform is called redundantly, if the proxy allows it."),
	ECVF_Default
);

#if VALIDATE_PRIMITIVE_PACKED_INDEX
inline void ValidatePackedPrimitiveIndexForUpdate(FPrimitiveSceneInfo* PrimitiveSceneInfo, const FScenePrimitiveUpdate& PrimitiveUpdates)
{
	if (const FPrimitiveUpdateCommand* Cmd = PrimitiveUpdates.FindCommand(PrimitiveSceneInfo))
	{
		if (Cmd->bAdded)
		{
			check(PrimitiveSceneInfo->GetIndex() == INDEX_NONE);
		}
		else
		{
			check(PrimitiveSceneInfo->GetIndex() != INDEX_NONE);
		}				
	}
	else
	{
		// sending update, and no queued command - must have a valid index
		check(PrimitiveSceneInfo->GetIndex() != INDEX_NONE);
	}
}
#endif

void FScene::UpdatePrimitiveTransform_RenderThread(FPrimitiveSceneProxy* PrimitiveSceneProxy, const FBoxSphereBounds& WorldBounds, const FBoxSphereBounds& LocalBounds, const FMatrix& LocalToWorld, const FVector& AttachmentRootPosition, const TOptional<FTransform>& PreviousTransform)
{
	FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
#if VALIDATE_PRIMITIVE_PACKED_INDEX
	ValidatePackedPrimitiveIndexForUpdate(PrimitiveSceneInfo, PrimitiveUpdates);
#endif

	PrimitiveUpdates.Enqueue(PrimitiveSceneInfo, FUpdateTransformCommand { .WorldBounds = WorldBounds, .LocalBounds = LocalBounds, .LocalToWorld = LocalToWorld, .AttachmentRootPosition = AttachmentRootPosition });

	if (PreviousTransform.IsSet())
	{
		PrimitiveUpdates.Enqueue(PrimitiveSceneInfo, FUpdateOverridePreviousTransformData(PreviousTransform.GetValue().ToMatrixWithScale()));
	}
}

template <typename UpdatePayloadType>
void FScene::UpdatePrimitiveInternal(FPrimitiveSceneProxy* SceneProxy, UpdatePayloadType&& InUpdatePayload)
{
	if (SceneProxy != nullptr)
	{
		ENQUEUE_RENDER_COMMAND(UpdatePrimitiveCmd)(
		[this, PrimitiveSceneInfo = SceneProxy->PrimitiveSceneInfo, UpdatePayload = MoveTemp(InUpdatePayload)] (FRHICommandListBase&) mutable
		{
#if VALIDATE_PRIMITIVE_PACKED_INDEX
			ValidatePackedPrimitiveIndexForUpdate(PrimitiveSceneInfo, PrimitiveUpdates);
#endif
			PrimitiveUpdates.Enqueue<UpdatePayloadType>(PrimitiveSceneInfo, MoveTemp(UpdatePayload));
		});
	}
}

void FScene::UpdatePrimitiveTransform(UPrimitiveComponent* Primitive)
{
	UpdatePrimitiveTransformInternal(Primitive);
}

void FScene::UpdatePrimitiveTransform(FPrimitiveSceneDesc* Primitive)
{
	UpdatePrimitiveTransformInternal(Primitive);	
}

template<class T> 	
void FScene::UpdatePrimitiveTransformInternal(T* Primitive)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdatePrimitiveTransformGT);

	FPrimitiveSceneInfoData& SceneData = Primitive->GetSceneData();

	if (Primitive->GetSceneProxy())
	{
		// Check if the primitive needs to recreate its proxy for the transform update.
		if (Primitive->ShouldRecreateProxyOnUpdateTransform())
		{
			check(Primitive->GetPrimitiveComponentInterface()); // required to execute the Remove/Add sequence inside this method
			// Re-add the primitive from scratch to recreate the primitive's proxy.
			RemovePrimitive(Primitive);
			AddPrimitive(Primitive);
		}
		else
		{
			FVector AttachmentRootPosition = Primitive->GetActorPositionForRenderer();

			FPrimitiveUpdateParams UpdateParams;
			UpdateParams.Scene = this;
			UpdateParams.PrimitiveSceneProxy = Primitive->GetSceneProxy();
			UpdateParams.WorldBounds = Primitive->Bounds;
			UpdateParams.LocalToWorld = Primitive->GetRenderMatrix();
			UpdateParams.AttachmentRootPosition = AttachmentRootPosition;
			UpdateParams.LocalBounds = Primitive->GetLocalBounds();
			UpdateParams.PreviousTransform = FMotionVectorSimulation::Get().GetPreviousTransform(ToUObject(Primitive));

			CheckAndSanitizePrimitiveBounds(UpdateParams.WorldBounds, Primitive);

			bool bPerformUpdate = true;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			const bool bAllowSkip = GSkipRedundantTransformUpdate && Primitive->GetSceneProxy()->CanSkipRedundantTransformUpdates();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (bAllowSkip || GWarningOnRedundantTransformUpdate)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (Primitive->GetSceneProxy()->WouldSetTransformBeRedundant_AnyThread(
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					UpdateParams.LocalToWorld,
					UpdateParams.WorldBounds,
					UpdateParams.LocalBounds,
					UpdateParams.AttachmentRootPosition))
				{
					if (bAllowSkip)
					{
						// Do not perform the transform update!
						bPerformUpdate = false;
					}
					else
					{
						// Not skipping, and warnings are enabled.
						UE_LOGF(LogRenderer, Warning,
							"Redundant UpdatePrimitiveTransform for Primitive %ls: Owner: %ls, Resource: %ls, Level: %ls",
							*Primitive->GetName(),
							*Primitive->GetSceneProxy()->GetOwnerName().ToString(),
							*Primitive->GetSceneProxy()->GetResourceName().ToString(),
							*Primitive->GetSceneProxy()->GetLevelName().ToString()
						);
					}
				}
			}

			if (bPerformUpdate)
			{
				bool bNeedsTransformCommand = true;

				// Accumulate all transform updates and enqueue them as once
				if (auto* Updater = static_cast<FPrimitiveTransformUpdater*>(IPrimitiveTransformUpdater::GetInstanceTLS()))
				{
					const int32 Index = Updater->Index.fetch_add(1, std::memory_order_relaxed);
					if (ensure(Index < Updater->Params.Num()))
					{
						Updater->Params[Index] = MoveTemp(UpdateParams);
						bNeedsTransformCommand = false;
					}
				}

				if (bNeedsTransformCommand)
				{
					ENQUEUE_RENDER_COMMAND(UpdateTransformCommand)(
						[UpdateParams](FRHICommandListBase&)
						{
							FScopeCycleCounter Context(UpdateParams.PrimitiveSceneProxy->GetStatId());
							UpdateParams.Scene->UpdatePrimitiveTransform_RenderThread(
								UpdateParams.PrimitiveSceneProxy,
								UpdateParams.WorldBounds,
								UpdateParams.LocalBounds,
								UpdateParams.LocalToWorld,
								UpdateParams.AttachmentRootPosition,
								UpdateParams.PreviousTransform
							);

							FSimpleStreamableAssetManager::UpdateStreamingState(UpdateParams.PrimitiveSceneProxy, UpdateParams.LocalToWorld, UpdateParams.WorldBounds);
						}
					);
				}
			}
		}
	}
	else
	{
		// If the primitive doesn't have a scene info object yet, it must be added from scratch.
		AddPrimitive(Primitive);
	}
}

FScene::IPrimitiveTransformUpdater* FScene::CreatePrimitiveTransformUpdater(int32 MaxPrimitives)
{
	if (MaxPrimitives > 0)
	{
		FPrimitiveTransformUpdater* Updater = new FPrimitiveTransformUpdater;
		Updater->Params.SetNum(MaxPrimitives);
		return Updater;
	}
	return nullptr;
}

void FScene::UpdatePrimitiveTransforms(IPrimitiveTransformUpdater* InUpdater)
{
	if (auto* Updater = static_cast<FPrimitiveTransformUpdater*>(InUpdater))
	{
		ENQUEUE_RENDER_COMMAND(UpdateTransformCommand)([Updater](FRHICommandListBase&)
		{
			const int32 NumPrimitiveUpdates = FMath::Min(Updater->Params.Num(), Updater->Index.load(std::memory_order_relaxed));

			for (int32 Index = 0; Index < NumPrimitiveUpdates; ++Index)
			{
				const auto& UpdateParams = Updater->Params[Index];
				FScopeCycleCounter Context(UpdateParams.PrimitiveSceneProxy->GetStatId());
				UpdateParams.Scene->UpdatePrimitiveTransform_RenderThread(
					UpdateParams.PrimitiveSceneProxy,
					UpdateParams.WorldBounds,
					UpdateParams.LocalBounds,
					UpdateParams.LocalToWorld,
					UpdateParams.AttachmentRootPosition,
					UpdateParams.PreviousTransform
				);

				FSimpleStreamableAssetManager::UpdateStreamingState(UpdateParams.PrimitiveSceneProxy, UpdateParams.LocalToWorld, UpdateParams.WorldBounds);
			}

			delete Updater;
		});
	}
}

void FScene::UpdatePrimitiveOcclusionBoundsSlack(UPrimitiveComponent* Primitive, float NewSlack)
{
	UpdatePrimitiveInternal(Primitive->GetSceneProxy(), FUpdateOcclusionBoundsSlacksData(NewSlack));
}

void FScene::UpdatePrimitiveDrawDistance(UPrimitiveComponent* Primitive, float MinDrawDistance, float MaxDrawDistance, float VirtualTextureMaxDrawDistance)
{
	UpdatePrimitiveInternal(Primitive->GetSceneProxy(), FUpdateDrawDistanceData(FVector3f(MinDrawDistance, MaxDrawDistance, VirtualTextureMaxDrawDistance)));
}

void FScene::UpdateInstanceCullDistance(UPrimitiveComponent* Primitive, float StartCullDistance, float EndCullDistance)
{
	UpdatePrimitiveInternal(Primitive->GetSceneProxy(), FUpdateInstanceCullDistanceData(FVector2f(StartCullDistance, EndCullDistance)));
}

void FScene::UpdatePrimitiveInstances(UPrimitiveComponent* Primitive)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdatePrimitiveInstanceGT);

	// If the primitive doesn't have a scene info object yet, it must be added from scratch.
	if (!Primitive->GetSceneProxy())
	{
		AddPrimitive(Primitive);
		return;
	}

	FUpdateInstanceCommand UpdateParams;
	UpdateParams.PrimitiveSceneProxy = Primitive->GetSceneProxy();
	UpdateParams.WorldBounds = Primitive->Bounds;
	UpdateParams.LocalBounds = static_cast<UPrimitiveComponent*>(Primitive)->GetLocalBounds();
	CheckAndSanitizePrimitiveBounds(UpdateParams.WorldBounds, Primitive);
	UpdatePrimitiveInstances(UpdateParams);
}

void FScene::UpdatePrimitiveInstancesFromCompute(FPrimitiveSceneDesc* Primitive, FGPUSceneWriteDelegate&& DataWriterGPU)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdatePrimitiveInstanceFromComputeGT);
	check(Primitive);

	FPrimitiveSceneProxy* SceneProxy = Primitive->GetSceneProxy();

	if (!ensureMsgf(SceneProxy, TEXT("Primitive must be added to scene prior to updating its instances from compute.")))
	{
		return;
	}

	FUpdateInstanceFromComputeCommand UpdateCommand;
	UpdateCommand.PrimitiveSceneProxy = SceneProxy;
	UpdateCommand.GPUSceneWriter = MoveTemp(DataWriterGPU);

	UpdatePrimitiveInternal(SceneProxy, MoveTemp(UpdateCommand));
}

static bool AllowPersistSurfaceCacheOnVisibilityChange(const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	return CVarLumenSceneSurfaceCacheAllowPersistOnVisibilityChange.GetValueOnRenderThread() != 0
		&& PrimitiveSceneProxy->AffectsIndirectLightingWhileHidden()
		// It doesn't seem useful to persist emissive light source surface cache as most lighting comes from the material
		// emissive property instead of bounce lighting
		&& !PrimitiveSceneProxy->IsEmissiveLightSource();
}

void FScene::UpdatePrimitivesIsDrawn_RenderThread(TArrayView<FPrimitiveSceneProxy*> InPrimitiveSceneProxies, bool bInIsDrawn)
{
	check(IsInRenderingThread());

	for (FPrimitiveSceneProxy* PrimitiveSceneProxy : InPrimitiveSceneProxies)
	{
		if ((PrimitiveSceneProxy->IsDrawnInGame() != PrimitiveSceneProxy->IsDrawnInEditor()) || (PrimitiveSceneProxy->IsDrawnInGame() != bInIsDrawn))
		{
			FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
			const int32 PrimitiveIndex = PrimitiveSceneInfo->GetIndex();
			const bool bPrimitiveIndexValid = PrimitiveSceneInfo->IsIndexValid();
			check(bPrimitiveIndexValid);
			const bool bPersistSurfaceCache = AllowPersistSurfaceCacheOnVisibilityChange(PrimitiveSceneProxy);

			if (!bInIsDrawn && bPrimitiveIndexValid)
			{
				DistanceFieldSceneData.RemovePrimitive(PrimitiveSceneInfo);

				if (!bPersistSurfaceCache)
				{
					LumenRemovePrimitive(PrimitiveSceneInfo, PrimitiveIndex);
				}
			}

			PrimitiveSceneProxy->SetIsDrawn_RenderThread(bInIsDrawn);

			if (bInIsDrawn && bPrimitiveIndexValid)
			{
				DistanceFieldSceneData.AddPrimitive(PrimitiveSceneInfo);
				// 5.6.1 Temporary Fix
				// IsDrawInGame() is not sufficient to know if primitive wasn't already added in Lumen (see UpdateVisibleInLumenScene).
				// Validate that proxy was not already added.
				if (PrimitiveSceneProxy->IsVisibleInLumenScene() && PrimitiveSceneInfo->LumenPrimitiveGroupIndices.IsEmpty())
				{
					LumenAddPrimitive(PrimitiveSceneInfo);
				}
			}
				
#if RHI_RAYTRACING
			if (bPrimitiveIndexValid)
			{
				UpdateCachedRayTracingState(PrimitiveSceneProxy, EPrimitiveDirtyRayTracingState::Default | EPrimitiveDirtyRayTracingState::DrawInGame);
			}
#endif
		}
	}
}

void FScene::UpdatePrimitiveInstances(FPrimitiveSceneDesc* Primitive)
{
	SCOPE_CYCLE_COUNTER(STAT_UpdatePrimitiveInstanceGT);

	// If the primitive doesn't have a scene info object yet, it must be added from scratch.
	if (!Primitive->GetSceneProxy())
	{
		AddPrimitive(Primitive);
		return;
	}
	
	FUpdateInstanceCommand UpdateParams;
	UpdateParams.PrimitiveSceneProxy = Primitive->GetSceneProxy();
	UpdateParams.WorldBounds = Primitive->GetBounds();
	UpdateParams.LocalBounds = Primitive->GetLocalBounds();
	CheckAndSanitizePrimitiveBounds(UpdateParams.WorldBounds, Primitive);
	UpdatePrimitiveInstances(UpdateParams);
}

void FScene::UpdatePrimitiveInstances(FUpdateInstanceCommand& UpdateParams)
{	
	UpdatePrimitiveInternal(UpdateParams.PrimitiveSceneProxy, MoveTemp(UpdateParams));
}

void FScene::UpdatePrimitiveSelectedState_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bIsSelected)
{
	check(IsInParallelRenderingThread());

#if WITH_EDITOR
	if (PrimitiveSceneInfo)
	{
		if (PrimitiveSceneInfo->GetIndex() != INDEX_NONE)
		{
			PrimitivesSelected[PrimitiveSceneInfo->GetIndex()].AtomicSet(bIsSelected);
		}
	}
#endif // WITH_EDITOR
}

void FScene::UpdatePrimitiveLightingAttachmentRoot(UPrimitiveComponent* Primitive)
{
	const UPrimitiveComponent* NewLightingAttachmentRoot = Primitive->GetLightingAttachmentRoot();

	if (NewLightingAttachmentRoot == Primitive)
	{
		NewLightingAttachmentRoot = nullptr;
	}

	FPrimitiveComponentId NewComponentId = NewLightingAttachmentRoot ? NewLightingAttachmentRoot->GetPrimitiveSceneId() : FPrimitiveComponentId();
	UpdatePrimitiveInternal(Primitive->GetSceneProxy(), FUpdateAttachmentRootData(NewComponentId));
}

void FScene::UpdatePrimitiveAttachment(UPrimitiveComponent* Primitive)
{
	TArray<USceneComponent*, TInlineAllocator<1> > ProcessStack;
	ProcessStack.Push(Primitive);

	// Walk down the tree updating, because the scene's attachment data structures must be updated if the root of the attachment tree changes
	while (ProcessStack.Num() > 0)
	{
		USceneComponent* Current = ProcessStack.Pop(EAllowShrinking::No);
		if (Current)
		{
			UPrimitiveComponent* CurrentPrimitive = Cast<UPrimitiveComponent>(Current);

			if (CurrentPrimitive
				&& CurrentPrimitive->GetWorld() 
				&& CurrentPrimitive->GetWorld()->Scene == this
				&& CurrentPrimitive->ShouldComponentAddToScene())
			{
				UpdatePrimitiveLightingAttachmentRoot(CurrentPrimitive);
			}

			ProcessStack.Append(Current->GetAttachChildren());
		}
	}
}

void FScene::UpdateCustomPrimitiveData(UPrimitiveComponent* Primitive)
{
	UpdateCustomPrimitiveData(Primitive->GetSceneProxy(), Primitive->GetCustomPrimitiveData());
}

void FScene::UpdateCustomPrimitiveData(FPrimitiveSceneDesc* Primitive, const FCustomPrimitiveData& CustomPrimitiveData)
{
	UpdateCustomPrimitiveData(Primitive->GetSceneProxy(), CustomPrimitiveData);
}

void FScene::UpdateCustomPrimitiveData(FPrimitiveSceneProxy* SceneProxy, const FCustomPrimitiveData& CustomPrimitiveData)
{
	UpdatePrimitiveInternal(SceneProxy, FUpdateCustomPrimitiveData(CustomPrimitiveData));
}

void FScene::UpdatePrimitiveDistanceFieldSceneData_GameThread(UPrimitiveComponent* Primitive)
{
	check(IsInGameThread());
	if (Primitive->SceneProxy)
	{
		UpdatePrimitiveInternal(Primitive->GetSceneProxy(), FUpdateDistanceFieldSceneData{});
	}
}

FPrimitiveSceneInfo* FScene::GetPrimitiveSceneInfo(int32 PrimitiveIndex) const
{
	if (Primitives.IsValidIndex(PrimitiveIndex))
	{
		return Primitives[PrimitiveIndex];
	}
	return nullptr;
}

FPrimitiveSceneInfo* FScene::GetPrimitiveSceneInfo(FPrimitiveComponentId PrimitiveId) const
{
	FPrimitiveSceneInfo* const* Found = PrimitiveComponentIdToInfoMap.Find(PrimitiveId);
	return Found ? *Found : nullptr;
}

FPrimitiveSceneInfo* FScene::GetPrimitiveSceneInfo(const FPersistentPrimitiveIndex& PersistentPrimitiveIndex) const
{
	int32 PrimitiveIndex = GetPrimitiveIndex(PersistentPrimitiveIndex);
	return GetPrimitiveSceneInfo(PrimitiveIndex);
}

void FScene::RemovePrimitiveSceneInfo_RenderThread(FPrimitiveSceneInfo* PrimitiveSceneInfo)
{
	PrimitiveUpdates.EnqueueDelete(PrimitiveSceneInfo);
}

void FScene::RemovePrimitive(UPrimitiveComponent* Primitive)
{
	// If the bulk reregister flag is set, add / remove will be handled in bulk by the FStaticMeshComponentBulkReregisterContext
	if (Primitive->bBulkReregister)
	{
		return;
	}
	BatchRemovePrimitives(MakeArrayView(&Primitive, 1));
}

void FScene::RemovePrimitive(FPrimitiveSceneDesc* Primitive)
{
	// If the bulk reregister flag is set, add / remove will be handled in bulk by the FStaticMeshComponentBulkReregisterContext
	if (Primitive->bBulkReregister)
	{
		return;
	}

	BatchRemovePrimitives(MakeArrayView(&Primitive, 1));
}

template <class T>
void FScene::BatchRemovePrimitivesInternal(TArrayView<T*> InPrimitives)
{
	SCOPE_CYCLE_COUNTER(STAT_RemoveScenePrimitiveGT);

	struct FDetachCommand
	{
		FPrimitiveSceneInfo* PrimitiveSceneInfo;
		FPrimitiveSceneProxy* PrimitiveSceneProxy;
		FAttachmentCounter* AttachmentCounter;
	};

	TArray<FDetachCommand, SceneRenderingAllocator> DestroyCommands;

	for (T* Primitive : InPrimitives)
	{
		FPrimitiveSceneProxy* PrimitiveSceneProxy = Primitive->GetSceneProxy();
		if (PrimitiveSceneProxy)
		{
			FSimpleStreamableAssetManager::Unregister(Primitive->GetPrimitiveComponentInterface(), PrimitiveSceneProxy);

			FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();

			// Disassociate the primitive's scene proxy.
			Primitive->ReleaseSceneProxy();
			DestroyCommands.Add({ PrimitiveSceneInfo, PrimitiveSceneProxy, Primitive->GetAttachmentCounter() });
		}
	}

	if (!DestroyCommands.IsEmpty())
	{
		// must run RT cmds in order here too
		ENQUEUE_RENDER_COMMAND(FRemovePrimitiveCommand)(
			[this, DestroyCommands = MoveTemp(DestroyCommands)](FRHICommandListBase&)
		{
			for (const FDetachCommand& Command : DestroyCommands)
			{
				RemovePrimitiveSceneInfo_RenderThread(Command.PrimitiveSceneInfo);
				Command.PrimitiveSceneProxy->DestroyRenderThreadResources();
				if(Command.AttachmentCounter)
				{
					Command.AttachmentCounter->Decrement();
				}
			}
		});
	}
}

void FScene::BatchRemovePrimitives(TArrayView<UPrimitiveComponent*> InPrimitives)
{
	BatchRemovePrimitivesInternal(InPrimitives);
}

void FScene::BatchRemovePrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives)
{
	BatchRemovePrimitivesInternal(InPrimitives);
}

void FScene::BatchRemovePrimitives(TArray<FPrimitiveSceneProxy*>&& InPrimitives)
{
	if (!InPrimitives.IsEmpty())
	{
		ENQUEUE_RENDER_COMMAND(BatchRemovePrimitives)(
			[this, InPrimitives = MoveTemp(InPrimitives)](FRHICommandListBase&)
		{
			for (FPrimitiveSceneProxy* PrimitiveSceneProxy : InPrimitives)
			{
				// Proxy-only removal: no component to touch, so Unregister's component-field write
				// is skipped via the nullptr arg (which also lets the GT check short-circuit
				// -- we're on RT here inside the render command).
				FSimpleStreamableAssetManager::Unregister(nullptr, PrimitiveSceneProxy);

				RemovePrimitiveSceneInfo_RenderThread(PrimitiveSceneProxy->GetPrimitiveSceneInfo());
				PrimitiveSceneProxy->DestroyRenderThreadResources();
			}
		});
	}
}

void FScene::ReleasePrimitive(UPrimitiveComponent* PrimitiveComponent)
{
	// Check if this components was already bulk released on the render side
	if (PrimitiveComponent->bBulkReregister)
	{
		return;
	}
	BatchReleasePrimitives(MakeArrayView(&PrimitiveComponent, 1));
}

void FScene::ReleasePrimitive(FPrimitiveSceneDesc* Primitive)
{
	// Check if this components was already bulk released on the render side
	if (Primitive->bBulkReregister)
	{
		return;
	}
	BatchReleasePrimitives(MakeArrayView(&Primitive, 1));
}

template<class T> 	
void FScene::BatchReleasePrimitivesInternal(TArrayView<T*> InPrimitives)
{
	// Send a command to the rendering thread to clean up any state dependent on this primitive
	FScene* Scene = this;
	TArray<FPrimitiveComponentId, TInlineAllocator<1>> ReleaseComponentIds;
	ReleaseComponentIds.SetNumUninitialized(InPrimitives.Num());

	for (int32 ComponentIndex = 0; ComponentIndex < InPrimitives.Num(); ComponentIndex++)
	{
		ReleaseComponentIds[ComponentIndex] = InPrimitives[ComponentIndex]->GetPrimitiveSceneId();
	}

	ENQUEUE_RENDER_COMMAND(FReleasePrimitiveCommand)(
		[Scene, ReleaseComponentIds = MoveTemp(ReleaseComponentIds)] (FRHICommandListBase&)
		{
			for (FPrimitiveComponentId PrimitiveComponentId : ReleaseComponentIds)
			{
				// Free the space in the indirect lighting cache
				Scene->IndirectLightingCache.ReleasePrimitive(PrimitiveComponentId);
			}
		});
}

void FScene::BatchReleasePrimitives(TArrayView<UPrimitiveComponent*> InPrimitives)
{
	BatchReleasePrimitivesInternal(InPrimitives);
}

void FScene::BatchReleasePrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives)
{
	BatchReleasePrimitivesInternal(InPrimitives);
}

void FScene::AssignAvailableShadowMapChannelForLight(FLightSceneInfo* LightSceneInfo)
{
	FDynamicShadowMapChannelBindingHelper Helper;
	check(LightSceneInfo && LightSceneInfo->Proxy);

	// For lights with static shadowing, only check for lights intersecting the preview channel if any.
	if (LightSceneInfo->Proxy->HasStaticShadowing())
	{
		Helper.DisableAllOtherChannels(LightSceneInfo->GetDynamicShadowMapChannel());

		// If this static shadowing light does not need a (preview) channel, skip it.
		if (!Helper.HasAnyChannelEnabled())
		{
			return;
		}
	}
	else if (LightSceneInfo->Proxy->GetLightType() == LightType_Directional)
	{
		// The implementation of forward lighting in ShadowProjectionPixelShader.usf does not support binding the directional light to channel 3.
		// This is related to the USE_FADE_PLANE feature that encodes the CSM blend factor the alpha channel.
		Helper.DisableChannel(3);
	}

	Helper.UpdateAvailableChannels(Lights, LightSceneInfo);

	const int32 NewChannelIndex = Helper.GetBestAvailableChannel();
	if (NewChannelIndex != INDEX_NONE)
	{
		// Unbind the channels previously allocated to lower priority lights.
		for (FLightSceneInfo* OtherLight : Helper.GetLights(NewChannelIndex))
		{
			OtherLight->SetDynamicShadowMapChannel(INDEX_NONE);
		}

		LightSceneInfo->SetDynamicShadowMapChannel(NewChannelIndex);

		// Try to assign new channels to lights that were just unbound.
		// Sort the lights so that they only get inserted once (prevents recursion).
		Helper.SortLightByPriority(NewChannelIndex);
		for (FLightSceneInfo* OtherLight : Helper.GetLights(NewChannelIndex))
		{
			AssignAvailableShadowMapChannelForLight(OtherLight);
		}
	}
	else
	{
		LightSceneInfo->SetDynamicShadowMapChannel(INDEX_NONE);
		OverflowingDynamicShadowedLights.AddUnique(LightSceneInfo->Proxy->GetOwnerNameOrLabel());
	}
}

void FScene::AddLightSceneInfo_RenderThread(FLightSceneInfo* LightSceneInfo)
{
	SCOPE_CYCLE_COUNTER(STAT_AddSceneLightTime);

	check(LightSceneInfo->bVisible);

	// Add the light to the light list.
	LightSceneInfo->Id = Lights.Add(FLightSceneInfoCompact(LightSceneInfo));
	const FLightSceneInfoCompact& LightSceneInfoCompact = Lights[LightSceneInfo->Id];
	const ELightComponentType LightType = ELightComponentType(LightSceneInfoCompact.LightType);
	const bool bDirectionalLight = LightType == LightType_Directional;

	if (bDirectionalLight)
	{
		DirectionalLights.Add(LightSceneInfo);
	}

	if (bDirectionalLight &&
		// Only use a stationary or movable light
		!(LightSceneInfo->Proxy->HasStaticLighting() 
		// if it is a Static DirectionalLight and the light has not been built, add it to MobileDirectionalLights for mobile preview.
		&& LightSceneInfo->IsPrecomputedLightingValid())
		)
	{
		// Set SimpleDirectionalLight
		if(!SimpleDirectionalLight)
		{
			SimpleDirectionalLight = LightSceneInfo;
		}

		if(GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Mobile)
		{
			const bool bUseCSMForDynamicObjects = LightSceneInfo->Proxy->UseCSMForDynamicObjects();
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			// these are tracked for disabled shader permutation warnings
			if (LightSceneInfo->Proxy->IsMovable())
			{
				NumMobileMovableDirectionalLights_RenderThread++;
			}
			if (bUseCSMForDynamicObjects)
			{
				NumMobileStaticAndCSMLights_RenderThread++;
			}
#endif
		    // Set MobileDirectionalLights entry
		    int32 FirstLightingChannel = GetFirstLightingChannelFromMask(LightSceneInfo->Proxy->GetLightingChannelMask());
		    if (FirstLightingChannel >= 0 && MobileDirectionalLights[FirstLightingChannel] == nullptr)
		    {
			    MobileDirectionalLights[FirstLightingChannel] = LightSceneInfo;
    
			    // if this light is a dynamic shadowcast then we need to update the static draw lists to pick a new lighting policy:
				const bool bUsesDirectionalLightForLighmapPolicySelection = IsStaticLightingAllowed() && !IsMobileDeferredShadingEnabled(GetShaderPlatform());
			    if (bUsesDirectionalLightForLighmapPolicySelection && (!LightSceneInfo->Proxy->HasStaticShadowing() || bUseCSMForDynamicObjects))
				{
		    		bScenesPrimitivesNeedStaticMeshElementUpdate = true;
					UE_CLOGF(!GIsEditor, LogRenderer, Log, "Forcing update for all mesh draw commands: Add directional light");
				}
		    }
		}
	}

	// Register rect. light texture
	if (LightType == LightType_Rect)
	{
		FRectLightSceneProxy* RectProxy = (FRectLightSceneProxy*)LightSceneInfo->Proxy;
		RectProxy->RectAtlasId = RectLightAtlas::AddTexture(ShaderPlatform, RectProxy->SourceTexture, RectProxy->SourceTextureScaleOffset);
	}

	// Register IES texture
	if (UTextureLightProfile* IESTexture = LightSceneInfo->Proxy->GetIESTexture())
	{
		LightSceneInfo->Proxy->IESAtlasId = IESAtlas::AddTexture(IESTexture);
	}

	if (IsMobilePlatform(ShaderPlatform) && LightSceneInfo->Proxy->CastsDynamicShadow())
	{
		if (IsStaticLightingAllowed() && MobileUsesShadowMaskTexture(ShaderPlatform))
		{
			AssignAvailableShadowMapChannelForLight(LightSceneInfo);
		}
		else
		{
			// Mobile deferred and forward use single channel for shadows
			LightSceneInfo->SetDynamicShadowMapChannel(0);
		}
	}
	else if (IsForwardShadingEnabled(ShaderPlatform) && (LightSceneInfo->Proxy->CastsDynamicShadow() || LightSceneInfo->Proxy->GetLightFunctionMaterial()))
	{
		AssignAvailableShadowMapChannelForLight(LightSceneInfo);
	}

	ProcessAtmosphereLightAddition_RenderThread(LightSceneInfo);

	InvalidatePathTracedOutput();

	// Add the light to the scene.
	LightSceneInfo->AddToScene();
}

void FScene::AddLight(ULightComponent* Light)
{
	LLM_SCOPE(ELLMTag::SceneRender);

	if (!Light->SceneProxy)
	{
		// Create the light's scene proxy.
		Light->SceneProxy = Light->CreateSceneProxy();
	}

	FLightSceneDesc LightSceneDesc{ .Transform = Light->GetComponentTransform(), .SceneProxy = Light->SceneProxy };
	BatchAddLights(MakeArrayView(&LightSceneDesc, 1));
}

void FScene::BatchAddLights(TArrayView<FLightSceneDesc> InLights)
{
	LLM_SCOPE(ELLMTag::SceneRender);

	TArray<FLightSceneInfo*, SceneRenderingAllocator> LightsToAdd;
	LightsToAdd.Reserve(InLights.Num());

	for (const FLightSceneDesc& LightDesc : InLights)
	{
		if (FLightSceneProxy* Proxy = LightDesc.GetSceneProxy())
		{
			// Update the light's transform and position.
			Proxy->SetTransform(LightDesc.GetTransform().ToMatrixNoScale(), LightDesc.GetPosition());

			// Create the light scene info.
			Proxy->LightSceneInfo = new FLightSceneInfo(Proxy, true);

			INC_DWORD_STAT(STAT_SceneLights);

			// Adding a new light
			++NumVisibleLights_AnyThread;

			LightsToAdd.Emplace(Proxy->LightSceneInfo);
		}
	}

	if (!LightsToAdd.IsEmpty())
	{
		// Send a command to the rendering thread to add the lights to the scene.
		ENQUEUE_RENDER_COMMAND(BatchAddLightsCommand)(
			[this, LightsToAdd = MoveTemp(LightsToAdd)](FRHICommandListBase& RHICmdList)
			{
				CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Scene_AddLight);
				for (FLightSceneInfo* LightSceneInfo : LightsToAdd)
				{
					FScopeCycleCounter Context(LightSceneInfo->Proxy->GetStatId());
					// It should not have been added to the scene already
					check(!LightSceneInfo->Id.IsValid());
					SceneLightInfoUpdates->EnqueueAdd(LightSceneInfo);
				}
			});
	}
}

void FScene::AddInvisibleLight(ULightComponent* Light)
{
	if (!Light->SceneProxy)
	{
		// Create the light's scene proxy.
		Light->SceneProxy = Light->CreateSceneProxy();
	}

	FLightSceneDesc LightSceneDesc{ .Transform = Light->GetComponentTransform(), .SceneProxy = Light->SceneProxy };
	BatchAddInvisibleLights(MakeArrayView(&LightSceneDesc, 1));
}

void FScene::BatchAddInvisibleLights(TArrayView<FLightSceneDesc> InLights)
{
	TArray<FLightSceneInfo*, SceneRenderingAllocator> LightsToAdd;
	LightsToAdd.Reserve(InLights.Num());

	for (const FLightSceneDesc& LightDesc : InLights)
	{
		if (FLightSceneProxy* Proxy = LightDesc.GetSceneProxy())
		{
			// Update the light's transform and position.
			Proxy->SetTransform(LightDesc.GetTransform().ToMatrixNoScale(), LightDesc.GetPosition());

			// Create the light scene info.
			Proxy->LightSceneInfo = new FLightSceneInfo(Proxy, false);

			INC_DWORD_STAT(STAT_SceneLights);

			// Send a command to the rendering thread to add the light to the scene.
			LightsToAdd.Emplace(Proxy->LightSceneInfo);
		}
	}

	if (!LightsToAdd.IsEmpty())
	{
		// Send a command to the rendering thread to add the lights to the scene.
		ENQUEUE_RENDER_COMMAND(BatchAddInvisibleLightsCommand)(
			[this, LightsToAdd = MoveTemp(LightsToAdd)](FRHICommandListBase& RHICmdList)
			{
				for (FLightSceneInfo* LightSceneInfo : LightsToAdd)
				{
					FScopeCycleCounter Context(LightSceneInfo->Proxy->GetStatId());
					LightSceneInfo->Id = InvisibleLights.Add(FLightSceneInfoCompact(LightSceneInfo));
				}
			});
	}
}

void FScene::SetSkyLight(FSkyLightSceneProxy* LightProxy)
{
	check(LightProxy);
	NumEnabledSkylights_GameThread++;

	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FSetSkyLightCommand)(
		[Scene, LightProxy] (FRHICommandListBase&)
		{
			check(!Scene->SkyLightStack.Contains(LightProxy));
			Scene->SkyLightStack.Push(LightProxy);

			// Use the most recently enabled skylight
			Scene->SkyLight = LightProxy;
			
			Scene->ReflectionSceneData.bSkylightHasChanged = true;
			Scene->InvalidatePathTracedOutput();
		});
}

void FScene::DisableSkyLight(FSkyLightSceneProxy* LightProxy)
{
	check(LightProxy);
	NumEnabledSkylights_GameThread--;

	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FDisableSkyLightCommand)(
		[Scene, LightProxy] (FRHICommandListBase&)
	{
		Scene->SkyLightStack.RemoveSingle(LightProxy);

		if (Scene->SkyLightStack.Num() > 0)
		{
			// Use the most recently enabled skylight
			Scene->SkyLight = Scene->SkyLightStack.Last();
		}
		else
		{
			Scene->SkyLight = NULL;
		}

		Scene->ReflectionSceneData.bSkylightHasChanged = true;
		Scene->InvalidatePathTracedOutput();
	});
}

bool FScene::HasSkyLightRequiringLightingBuild() const
{
	return SkyLight != nullptr && !SkyLight->IsMovable();
}

bool FScene::HasAtmosphereLightRequiringLightingBuild() const
{
	bool AnySunLightNotMovable = false;
	for (uint8 Index = 0; Index < NUM_ATMOSPHERE_LIGHTS; ++Index)
	{
		AnySunLightNotMovable |= AtmosphereLights[Index] != nullptr && !AtmosphereLights[Index]->Proxy->IsMovable();
	}
	return AnySunLightNotMovable;
}

void FScene::AddOrRemoveDecal_RenderThread(FDeferredDecalProxy* Proxy, bool bAdd)
{
	if (bAdd)
	{
		Decals.Add(Proxy);
		InvalidatePathTracedOutput();
	}
	else
	{
		// can be optimized
		for (int32 Index = 0; Index < Decals.Num(); ++Index)
		{
			if (Decals[Index] == Proxy)
			{
				InvalidatePathTracedOutput();
				Decals.RemoveAtSwap(Index, EAllowShrinking::No);
				delete Proxy;
				break;
			}
		}
	}
}

void FScene::SetPhysicsField(FPhysicsFieldSceneProxy* PhysicsFieldSceneProxy)
{
	check(PhysicsFieldSceneProxy);
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FSetPhysicsFieldCommand)(
		[Scene, PhysicsFieldSceneProxy] (FRHICommandListBase&)
		{
			Scene->PhysicsField = PhysicsFieldSceneProxy;
		});
}

void FScene::ShowPhysicsField()
{
	// Set the shader print/debug values if
	// physics field visualisation has been enabled
	if (PhysicsField && PhysicsField->FieldResource && PhysicsField->FieldResource->FieldInfos.bShowFields)
	{
		// Force ShaderPrint on.
		ShaderPrint::SetEnabled(true);
		ShaderPrint::RequestSpaceForLines(128000);
	}
}

void FScene::ResetPhysicsField()
{
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(FResetPhysicsFieldCommand)(
		[Scene] (FRHICommandListBase&)
		{
			Scene->PhysicsField = nullptr;
		});
}

void FScene::UpdatePhysicsField(FRDGBuilder& GraphBuilder, FViewInfo& View)
{
	if (PhysicsField && PhysicsField->FieldResource)
	{
		PhysicsField->FieldResource->FieldInfos.ViewOrigin = View.ViewMatrices.GetViewOrigin();
		if (View.Family )
		{
			PhysicsField->FieldResource->FieldInfos.bShowFields = View.Family->EngineShowFlags.PhysicsField;
			ShowPhysicsField();
		}
	}
}

void FScene::AddDecal(UDecalComponent* Component)
{
	if(!Component->SceneProxy)
	{
		// Create the decals's scene proxy.
		Component->SceneProxy = Component->CreateSceneProxy();

		// If there is no scene proxy then don't add to scene
		if (!Component->SceneProxy)
		{
			return;
		}

		INC_DWORD_STAT(STAT_SceneDecals);

		// Send a command to the rendering thread to add the light to the scene.
		FScene* Scene = this;
		FDeferredDecalProxy* Proxy = Component->SceneProxy;
		ENQUEUE_RENDER_COMMAND(FAddDecalCommand)(
			[Scene, Proxy] (FRHICommandListBase&)
			{
				Scene->AddOrRemoveDecal_RenderThread(Proxy, true);
			});
	}
}

void FScene::RemoveDecal(UDecalComponent* Component)
{
	if(Component->SceneProxy)
	{
		DEC_DWORD_STAT(STAT_SceneDecals);

		// Send a command to the rendering thread to remove the light from the scene.
		FScene* Scene = this;
		FDeferredDecalProxy* Proxy = Component->SceneProxy;
		ENQUEUE_RENDER_COMMAND(FRemoveDecalCommand)(
			[Scene, Proxy] (FRHICommandListBase&)
			{
				Scene->AddOrRemoveDecal_RenderThread(Proxy, false);
			});

		// Disassociate the primitive's scene proxy.
		Component->SceneProxy = NULL;
	}
}

void FScene::UpdateDecalTransform(UDecalComponent* Decal)
{
	if(Decal->SceneProxy)
	{
		//Send command to the rendering thread to update the decal's transform.
		FScene* Scene = this;
		FDeferredDecalProxy* DecalSceneProxy = Decal->SceneProxy;
		FTransform ComponentToWorldIncludingDecalSize = Decal->GetTransformIncludingDecalSize();
		FBoxSphereBounds Bounds = Decal->CalcBounds(Decal->GetComponentTransform());
		ENQUEUE_RENDER_COMMAND(UpdateTransformCommand)(
			[DecalSceneProxy, ComponentToWorldIncludingDecalSize, Bounds, Scene] (FRHICommandListBase&)
			{
				// Invalidate the path tracer only if the decal was sufficiently moved
				if (!ComponentToWorldIncludingDecalSize.Equals(DecalSceneProxy->ComponentTrans, SMALL_NUMBER))
				{
					Scene->InvalidatePathTracedOutput();
				}
				// Update the primitive's transform.
				DecalSceneProxy->SetTransformIncludingDecalSize(ComponentToWorldIncludingDecalSize, Bounds);
			});
	}
}

void FScene::UpdateDecalFadeOutTime(UDecalComponent* Decal)
{
	FDeferredDecalProxy* Proxy = Decal->SceneProxy;
	if(Proxy)
	{
		float CurrentTime = GetWorld()->GetTimeSeconds();
		float DecalFadeStartDelay = Decal->FadeStartDelay;
		float DecalFadeDuration = Decal->FadeDuration;

		ENQUEUE_RENDER_COMMAND(FUpdateDecalFadeInTimeCommand)(
			[Proxy, CurrentTime, DecalFadeStartDelay, DecalFadeDuration] (FRHICommandListBase&)
		{
			if (DecalFadeDuration > 0.0f)
			{
				Proxy->InvFadeDuration = 1.0f / DecalFadeDuration;
				Proxy->FadeStartDelayNormalized = (CurrentTime + DecalFadeStartDelay + DecalFadeDuration) * Proxy->InvFadeDuration;
			}
			else
			{
				Proxy->InvFadeDuration = -1.0f;
				Proxy->FadeStartDelayNormalized = 1.0f;
			}
		});
	}
}

void FScene::UpdateDecalFadeInTime(UDecalComponent* Decal)
{
	FDeferredDecalProxy* Proxy = Decal->SceneProxy;
	if (Proxy)
	{
		float CurrentTime = GetWorld()->GetTimeSeconds();
		float DecalFadeStartDelay = Decal->FadeInStartDelay;
		float DecalFadeDuration = Decal->FadeInDuration;

		ENQUEUE_RENDER_COMMAND(FUpdateDecalFadeInTimeCommand)(
			[Proxy, CurrentTime, DecalFadeStartDelay, DecalFadeDuration] (FRHICommandListBase&)
		{
			if (DecalFadeDuration > 0.0f)
			{
				Proxy->InvFadeInDuration = 1.0f / DecalFadeDuration;
				Proxy->FadeInStartDelayNormalized = (CurrentTime + DecalFadeStartDelay) * -Proxy->InvFadeInDuration;
			}
			else
			{
				Proxy->InvFadeInDuration = 1.0f;
				Proxy->FadeInStartDelayNormalized = 0.0f;
			}
		});
	}
}

void FScene::BatchUpdateDecals(TArray<FDeferredDecalUpdateParams>&& UpdateParams)
{
	ENQUEUE_RENDER_COMMAND(FBatchUpdateDecalsCommand)(
		[Scene=this, UpdateParams_RT=MoveTemp(UpdateParams)] (FRHICommandListBase&)
		{
			for (const FDeferredDecalUpdateParams& DecalUpdate : UpdateParams_RT )
			{
				if (DecalUpdate.OperationType == FDeferredDecalUpdateParams::EOperationType::RemoveFromSceneAndDelete)
				{
					Scene->AddOrRemoveDecal_RenderThread(DecalUpdate.DecalProxy, false);
					continue;
				}

				if (DecalUpdate.OperationType == FDeferredDecalUpdateParams::EOperationType::AddToSceneAndUpdate )
				{
					Scene->AddOrRemoveDecal_RenderThread(DecalUpdate.DecalProxy, true);
				}

				DecalUpdate.DecalProxy->SetTransformIncludingDecalSize(DecalUpdate.Transform, DecalUpdate.Bounds);

				// When FadeDuration is intentionally set to 0 the user expects the decal to not fade automatically
				if (DecalUpdate.FadeDuration == 0.0f)
				{
					DecalUpdate.DecalProxy->InvFadeDuration = -1.0f;
				}
				else
				{
					DecalUpdate.DecalProxy->InitializeFadingParameters(DecalUpdate.AbsSpawnTime, DecalUpdate.FadeDuration, DecalUpdate.FadeStartDelay, DecalUpdate.FadeInDuration, DecalUpdate.FadeInStartDelay);
				}

				DecalUpdate.DecalProxy->FadeScreenSize = DecalUpdate.FadeScreenSize;
				DecalUpdate.DecalProxy->SortOrder = DecalUpdate.SortOrder;
				DecalUpdate.DecalProxy->DecalColor = DecalUpdate.DecalColor;
			}
		}
	);
}

void FScene::AddHairStrands(FHairStrandsInstance* Proxy)
{
	if (Proxy)
	{
		const int32 PackedIndex = HairStrandsSceneData.RegisteredProxies.Add(TRefCountPtr<FHairStrandsInstance>(Proxy));
		Proxy->RegisteredIndex = PackedIndex;
	}
}

void FScene::RemoveHairStrands(FHairStrandsInstance* Proxy)
{
	if (Proxy)
	{
		int32 ProxyIndex = Proxy->RegisteredIndex;
		if (HairStrandsSceneData.RegisteredProxies.IsValidIndex(ProxyIndex))
		{
			HairStrandsSceneData.RegisteredProxies.RemoveAtSwap(ProxyIndex);
		}
		Proxy->RegisteredIndex = -1;
		if (HairStrandsSceneData.RegisteredProxies.IsValidIndex(ProxyIndex))
		{
			HairStrandsSceneData.RegisteredProxies[ProxyIndex]->RegisteredIndex = ProxyIndex;
		}
	}
}

IAnimBankTransformProvider* FScene::GetAnimBankTransformProvider()
{
	return SceneExtensions.GetExtensionPtr<FAnimBankTransformProvider>();
}

IAnimRuntimeTransformProvider* FScene::GetAnimRuntimeTransformProvider()
{
	return SceneExtensions.GetExtensionPtr<FAnimRuntimeTransformProvider>();
}

IAnimSequenceTransformProvider* FScene::GetAnimSequenceTransformProvider()
{
	return SceneExtensions.GetExtensionPtr<FAnimSequenceTransformProvider>();
}

void FScene::GetLightIESAtlasSlot(const FLightSceneProxy* Proxy, FLightRenderParameters* Out)
{
	if (Proxy)
	{
		//check(!IsInGameThread());
		Out->IESAtlasIndex = IESAtlas::GetAtlasSlot(Proxy->IESAtlasId);
	}
}

void FScene::GetRectLightAtlasSlot(const FRectLightSceneProxy* Proxy, FLightRenderParameters* Out)
{
	if (Proxy)
	{
		//check(IsInRenderingThread());
		const RectLightAtlas::FAtlasSlotDesc Slot = RectLightAtlas::GetAtlasSlot(Proxy->RectAtlasId);
		Out->RectLightAtlasUVOffset = Slot.UVOffset;
		Out->RectLightAtlasUVScale = Slot.UVScale;
		Out->RectLightAtlasMaxLevel = Slot.MaxMipLevel;
	}
}

void FScene::CreateReflectionCaptureProxy(UReflectionCaptureComponent* Component)
{
	if (!Component->SceneProxy)
	{
		Component->SceneProxy = Component->CreateSceneProxy();

		FScene* Scene = this;
		FReflectionCaptureProxy* Proxy = Component->SceneProxy;
		const FVector Position = Component->GetComponentLocation();

		ENQUEUE_RENDER_COMMAND(FAddCaptureCommand)(
			[Scene, Component, bIsRuntime = Component->IsRuntimeCapture(), Proxy, Position](FRHICommandListBase& RHICmdList)
		{
			if (Proxy->bUsingPreviewCaptureData)
			{
				FPlatformAtomics::InterlockedIncrement(&Scene->NumUnbuiltReflectionCaptures);
			}

			Scene->ReflectionSceneData.SetRegisteredReflectionCapturesHasChanged();
			const int32 PackedIndex = Scene->ReflectionSceneData.RegisteredReflectionCaptures.Add(Proxy);

			Proxy->PackedIndex = PackedIndex;
			Scene->ReflectionSceneData.RegisteredReflectionCapturePositionAndRadius.Add(FSphere(Position, Proxy->InfluenceRadius));
			
			if (Scene->GetFeatureLevel() <= ERHIFeatureLevel::ES3_1)
			{
				Proxy->UpdateMobileUniformBuffer(RHICmdList);
			}

			// For runtime captures, suppress shading until its cubemap has been rendered.
			if (bIsRuntime)
			{
				FCaptureComponentSceneState* State = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Find(Component);
				if (State && State->bRenderedForShading)
				{
					State->bRenderedForShading = false;
					Scene->ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged = true;
				}
			}

			checkSlow(Scene->ReflectionSceneData.RegisteredReflectionCaptures.Num() == Scene->ReflectionSceneData.RegisteredReflectionCapturePositionAndRadius.Num());
		});
	}
}

void FScene::DestroyReflectionCaptureProxy(UReflectionCaptureComponent* Component)
{
	if (Component->SceneProxy)
	{
		FScene* Scene = this;
		FReflectionCaptureProxy* Proxy = Component->SceneProxy;

		ENQUEUE_RENDER_COMMAND(FRemoveCaptureCommand)(
			[Scene, Proxy] (FRHICommandListBase&)
		{
			if (Proxy->bUsingPreviewCaptureData)
			{
				FPlatformAtomics::InterlockedDecrement(&Scene->NumUnbuiltReflectionCaptures);
			}

			Scene->ReflectionSceneData.SetRegisteredReflectionCapturesHasChanged();

			// Need to clear out all reflection captures on removal to avoid dangling pointers.
			// Only mobile and desktop forward populate CachedReflectionCaptureProxy; on deferred there is nothing to clear.
			if (Scene->ShouldCacheReflectionCaptures())
			{
				for (int32 PrimitiveIndex = 0; PrimitiveIndex < Scene->Primitives.Num(); ++PrimitiveIndex)
				{
					Scene->Primitives[PrimitiveIndex]->RemoveCachedReflectionCapture(Proxy);
				}
			}

			int32 CaptureIndex = Proxy->PackedIndex;
			Scene->ReflectionSceneData.RegisteredReflectionCaptures.RemoveAtSwap(CaptureIndex);
			Scene->ReflectionSceneData.RegisteredReflectionCapturePositionAndRadius.RemoveAtSwap(CaptureIndex);

			if (Scene->ReflectionSceneData.RegisteredReflectionCaptures.IsValidIndex(CaptureIndex))
			{
				FReflectionCaptureProxy* OtherCapture = Scene->ReflectionSceneData.RegisteredReflectionCaptures[CaptureIndex];
				OtherCapture->PackedIndex = CaptureIndex;
			}

			delete Proxy;

			checkSlow(Scene->ReflectionSceneData.RegisteredReflectionCaptures.Num() == Scene->ReflectionSceneData.RegisteredReflectionCapturePositionAndRadius.Num());
		});

		// Disassociate the primitive's scene proxy.
		Component->SceneProxy = NULL;

		// Reset runtime capture related values to defaults, so it starts at a fresh state if its proxy is re-created.
		Component->RuntimeCaptureResetState();
	}
}

void FScene::AddReflectionCapture(UReflectionCaptureComponent* Component)
{
	// Always track runtime captures as budget candidates so the budget CVar can be changed at runtime.
	if (Component->IsRuntimeCapture())
	{
		ReflectionSceneData.RuntimeCaptureBudgetCandidates.AddUnique(Component);

		// If budgeting is active, defer proxy creation.  The budget evaluation in AllocateReflectionCaptures will activate it later.
		static const auto CVarBudget = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.ReflectionCapture.Runtime.Budget"));
		const int32 Budget = CVarBudget ? CVarBudget->GetValueOnGameThread() : 0;
		if (Budget > 0 && !bIsEditorScene)
		{
			return;
		}
	}

	CreateReflectionCaptureProxy(Component);
}

void FScene::RemoveReflectionCapture(UReflectionCaptureComponent* Component)
{
	ReflectionSceneData.RuntimeCaptureBudgetCandidates.Remove(Component);
	DestroyReflectionCaptureProxy(Component);
}

void FScene::UpdateReflectionCaptureTransform(UReflectionCaptureComponent* Component)
{
	if (Component->SceneProxy)
	{
		const FReflectionCaptureMapBuildData* MapBuildData = Component->GetMapBuildData();
		bool bUsingPreviewCaptureData = MapBuildData == NULL;

		FScene* Scene = this;
		FReflectionCaptureProxy* Proxy = Component->SceneProxy;
		FMatrix Transform = Component->GetComponentTransform().ToMatrixWithScale();
		float InfluenceRadius = Component->GetInfluenceBoundingRadius();

		ENQUEUE_RENDER_COMMAND(UpdateReflectionCaptureTransformCommand)(
			[Scene, Proxy, Transform, InfluenceRadius, bUsingPreviewCaptureData](FRHICommandListBase& RHICmdList)
		{
			if (Proxy->bUsingPreviewCaptureData)
			{
				FPlatformAtomics::InterlockedDecrement(&Scene->NumUnbuiltReflectionCaptures);
			}

			Proxy->bUsingPreviewCaptureData = bUsingPreviewCaptureData;

			if (Proxy->bUsingPreviewCaptureData)
			{
				FPlatformAtomics::InterlockedIncrement(&Scene->NumUnbuiltReflectionCaptures);
			}

			Scene->ReflectionSceneData.SetRegisteredReflectionCapturesHasChanged();
			Proxy->SetTransform(Transform);
			Proxy->InfluenceRadius = InfluenceRadius;

			int32 CaptureIndex = Proxy->PackedIndex;
			Scene->ReflectionSceneData.RegisteredReflectionCapturePositionAndRadius[CaptureIndex] = FSphere(Transform.GetOrigin(), InfluenceRadius);

			if (Scene->GetFeatureLevel() <= ERHIFeatureLevel::ES3_1)
			{
				Proxy->UpdateMobileUniformBuffer(RHICmdList);
			}
		});
	}
}

void FScene::ReleaseReflectionCubemap(UReflectionCaptureComponent* CaptureComponent)
{
	bool bRemoved = false;
	for (TSparseArray<UReflectionCaptureComponent*>::TIterator It(ReflectionSceneData.AllocatedReflectionCapturesGameThread); It; ++It)
	{
		UReflectionCaptureComponent* CurrentCapture = *It;

		if (CurrentCapture == CaptureComponent)
		{
			It.RemoveCurrent();
			bRemoved = true;
			break;
		}
	}

	if (bRemoved)
	{
		FScene* Scene = this;
		ENQUEUE_RENDER_COMMAND(RemoveCaptureCommand)(
			[CaptureComponent, Scene] (FRHICommandListBase&)
			{
				int32 IndexToFree = -1;

				const FCaptureComponentSceneState* ComponentStatePtr = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Find(CaptureComponent);
				if (ComponentStatePtr)
				{
					// We track removed captures so we can remap them when reallocating the cubemap array
					check(ComponentStatePtr->CubemapIndex != -1);
					IndexToFree = ComponentStatePtr->CubemapIndex;
				}

				const bool bDidRemove = Scene->ReflectionSceneData.AllocatedReflectionCaptureState.Remove(CaptureComponent);
				if (bDidRemove && (IndexToFree != -1))
				{
					Scene->ReflectionSceneData.CubemapArraySlotsUsed[IndexToFree] = false;
				}

				// Free any in-flight blends owned by this component so the slot becomes available for other blends.
				for (FReflectionEnvironmentSceneData::FSmoothBlendEntry& Entry : Scene->ReflectionSceneData.RuntimeSmoothBlendSlots)
				{
					if (Entry.OwningComponent == CaptureComponent)
					{
						Entry = {};
					}
				}
			});
	}
}

const FReflectionCaptureProxy* FScene::FindClosestReflectionCapture(FVector Position) const
{
	checkSlow(IsInParallelRenderingThread());
	float ClosestDistanceSquared = FLT_MAX;
	int32 ClosestInfluencingCaptureIndex = INDEX_NONE;

	// Linear search through the scene's reflection captures
	// ReflectionSceneData.RegisteredReflectionCapturePositionAndRadius has been packed densely to make this coherent in memory
	for (int32 CaptureIndex = 0; CaptureIndex < ReflectionSceneData.RegisteredReflectionCapturePositionAndRadius.Num(); CaptureIndex++)
	{
		const FSphere& ReflectionCapturePositionAndRadius = ReflectionSceneData.RegisteredReflectionCapturePositionAndRadius[CaptureIndex];

		const float DistanceSquared = (ReflectionCapturePositionAndRadius.Center - Position).SizeSquared();

		// If the Position is inside the InfluenceRadius of a ReflectionCapture
		if (DistanceSquared <= FMath::Square(ReflectionCapturePositionAndRadius.W))
		{
			// Choose the closest ReflectionCapture or record the first one found.
			if (ClosestInfluencingCaptureIndex == INDEX_NONE || DistanceSquared < ClosestDistanceSquared)
			{
				ClosestDistanceSquared = DistanceSquared;
				ClosestInfluencingCaptureIndex = CaptureIndex;
			}
		}
	}

	return ClosestInfluencingCaptureIndex != INDEX_NONE ? ReflectionSceneData.RegisteredReflectionCaptures[ClosestInfluencingCaptureIndex] : NULL;
}

const FPlanarReflectionSceneProxy* FScene::FindClosestPlanarReflection(const FBoxSphereBounds& Bounds) const
{
	checkSlow(IsInParallelRenderingThread());
	const FPlanarReflectionSceneProxy* ClosestPlanarReflection = NULL;
	float ClosestDistance = FLT_MAX;
	FBox PrimitiveBoundingBox(Bounds.Origin - Bounds.BoxExtent, Bounds.Origin + Bounds.BoxExtent);

	// Linear search through the scene's planar reflections
	for (int32 CaptureIndex = 0; CaptureIndex < PlanarReflections.Num(); CaptureIndex++)
	{
		FPlanarReflectionSceneProxy* CurrentPlanarReflection = PlanarReflections[CaptureIndex];
		const FBox ReflectionBounds = CurrentPlanarReflection->WorldBounds;

		if (PrimitiveBoundingBox.Intersect(ReflectionBounds))
		{
			const float Distance = FMath::Abs(CurrentPlanarReflection->ReflectionPlane.PlaneDot(Bounds.Origin));

			if (Distance < ClosestDistance)
			{
				ClosestDistance = Distance;
				ClosestPlanarReflection = CurrentPlanarReflection;
			}
		}
	}

	return ClosestPlanarReflection;
}

const FPlanarReflectionSceneProxy* FScene::GetForwardPassGlobalPlanarReflection() const
{
	// For the forward pass just pick first planar reflection.

	if (PlanarReflections.Num() > 0)
	{
		return PlanarReflections[0];
	}

	return nullptr;
}

void FScene::AddPrecomputedLightVolume(const FPrecomputedLightVolume* Volume)
{
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(AddVolumeCommand)(
		[Scene, Volume] (FRHICommandListBase&)
		{
			Scene->PrecomputedLightVolumes.Add(Volume);
			Scene->IndirectLightingCache.SetLightingCacheDirty(Scene, Volume);
		});
}

void FScene::RemovePrecomputedLightVolume(const FPrecomputedLightVolume* Volume)
{
	FScene* Scene = this; 

	ENQUEUE_RENDER_COMMAND(RemoveVolumeCommand)(
		[Scene, Volume] (FRHICommandListBase&)
		{
			Scene->PrecomputedLightVolumes.Remove(Volume);
			Scene->IndirectLightingCache.SetLightingCacheDirty(Scene, Volume);
		});
}

void FVolumetricLightmapSceneData::AddLevelVolume(FRHICommandListImmediate& RHICmdList, const FPrecomputedVolumetricLightmap* InVolume, EShadingPath ShadingPath, bool bIsPersistentLevel)
{
	LevelVolumetricLightmaps.Add(InVolume);

	if (bIsPersistentLevel)
	{
		PersistentLevelVolumetricLightmap = InVolume;
	}

	InVolume->Data->AddToSceneData(RHICmdList, &GlobalVolumetricLightmapData);
	
	// Invalidate CPU lightmap lookup cache
	CPUInterpolationCache.Empty();
}

void FVolumetricLightmapSceneData::RemoveLevelVolume(FRHICommandListImmediate& RHICmdList, const FPrecomputedVolumetricLightmap* InVolume)
{
	LevelVolumetricLightmaps.Remove(InVolume);

	InVolume->Data->RemoveFromSceneData(RHICmdList, &GlobalVolumetricLightmapData, PersistentLevelVolumetricLightmap ? PersistentLevelVolumetricLightmap->Data->BrickDataBaseOffsetInAtlas : 0);

	if (PersistentLevelVolumetricLightmap == InVolume)
	{
		PersistentLevelVolumetricLightmap = nullptr;
	}

	// Invalidate CPU lightmap lookup cache
	CPUInterpolationCache.Empty();
}

const FPrecomputedVolumetricLightmap* FVolumetricLightmapSceneData::GetLevelVolumetricLightmap() const
{
#if WITH_EDITOR
	if (FStaticLightingSystemInterface::GetPrecomputedVolumetricLightmap(Scene->GetWorld()))
	{
		return FStaticLightingSystemInterface::GetPrecomputedVolumetricLightmap(Scene->GetWorld());
	}
#endif
	return &GlobalVolumetricLightmap;
}

bool FVolumetricLightmapSceneData::HasData() const
{
#if WITH_EDITOR
	if (FStaticLightingSystemInterface::GetPrecomputedVolumetricLightmap(Scene->GetWorld()))
	{
		return true;
	}
#endif
	if (LevelVolumetricLightmaps.Num() > 0)
	{
		if (Scene->GetFeatureLevel() >= ERHIFeatureLevel::SM5)
		{
			return GlobalVolumetricLightmapData.IndirectionTexture.Texture.IsValid();
		}
		else
		{
			return GlobalVolumetricLightmapData.IndirectionTexture.Data.Num() > 0;
		}
	}

	return false;
}

bool FScene::HasPrecomputedVolumetricLightmap_RenderThread() const
{
#if WITH_EDITOR
	if (FStaticLightingSystemInterface::GetPrecomputedVolumetricLightmap(GetWorld()))
	{
		return true;
	}
#endif
	return VolumetricLightmapSceneData.HasData();
}

void FScene::AddPrecomputedVolumetricLightmap(const FPrecomputedVolumetricLightmap* Volume, bool bIsPersistentLevel)
{
	FScene* Scene = this;

	ENQUEUE_RENDER_COMMAND(AddVolumeCommand)
		([Scene, Volume, bIsPersistentLevel] (FRHICommandListImmediate& RHICmdList)
	{
		Scene->VolumetricLightmapSceneData.AddLevelVolume(RHICmdList, Volume, Scene->GetShadingPath(), bIsPersistentLevel);
	});
}

void FScene::RemovePrecomputedVolumetricLightmap(const FPrecomputedVolumetricLightmap* Volume)
{
	FScene* Scene = this; 

	ENQUEUE_RENDER_COMMAND(RemoveVolumeCommand)
		([Scene, Volume] (FRHICommandListImmediate& RHICmdList)
	{
		Scene->VolumetricLightmapSceneData.RemoveLevelVolume(RHICmdList, Volume);
	});
}

static void UpdateRuntimeVirtualTextureHidePrimitiveMasks(TSparseArray<FRuntimeVirtualTextureSceneProxy*> const& InProxies, bool& bOutEditor, bool& bOutGame)
{
	// Set the global flag to hide when any one volume sets it.
	bOutEditor = false;
	bOutGame = false;
	for (TSparseArray<FRuntimeVirtualTextureSceneProxy*>::TConstIterator It(InProxies); It && !bOutEditor && !bOutGame; ++It)
	{
		if ((*It)->RuntimeVirtualTextureId != -1)
		{
			bOutEditor |= (*It)->bHidePrimitivesInEditor;
			bOutGame |= (*It)->bHidePrimitivesInGame;
		}
	}
}

void FScene::AddRuntimeVirtualTexture(URuntimeVirtualTextureComponent* Component)
{
	if (Component->SceneProxy == nullptr)
	{
		Component->SceneProxy = new FRuntimeVirtualTextureSceneProxy(Component);

		FScene* Scene = this;
		FRuntimeVirtualTextureSceneProxy* SceneProxy = Component->SceneProxy;

		ENQUEUE_RENDER_COMMAND(AddRuntimeVirtualTextureCommand)(
			[Scene, SceneProxy] (FRHICommandListBase&)
		{
			SceneProxy->SceneIndex = Scene->RuntimeVirtualTextures.Add(SceneProxy);
			
			UpdateRuntimeVirtualTextureHidePrimitiveMasks(Scene->RuntimeVirtualTextures, Scene->bRuntimeVirtualTexturePrimitiveHideEditor, Scene->bRuntimeVirtualTexturePrimitiveHideGame);
		});
	}
	else
	{
		// This is a component update.
		// Delete old proxy and add new one on the render thread.
		FRuntimeVirtualTextureSceneProxy* SceneProxyToRemove = Component->SceneProxy;
		SceneProxyToRemove->Release();

		Component->SceneProxy = new FRuntimeVirtualTextureSceneProxy(Component);

		FScene* Scene = this;
		FRuntimeVirtualTextureSceneProxy* SceneProxyToAdd = Component->SceneProxy;

		ENQUEUE_RENDER_COMMAND(UpdateRuntimeVirtualTextureCommand)(
			[Scene, SceneProxyToAdd, SceneProxyToRemove] (FRHICommandListBase&)
		{
			Scene->RuntimeVirtualTextures.RemoveAt(SceneProxyToRemove->SceneIndex);
			delete SceneProxyToRemove;

			SceneProxyToAdd->SceneIndex = Scene->RuntimeVirtualTextures.Add(SceneProxyToAdd);

			UpdateRuntimeVirtualTextureHidePrimitiveMasks(Scene->RuntimeVirtualTextures, Scene->bRuntimeVirtualTexturePrimitiveHideEditor, Scene->bRuntimeVirtualTexturePrimitiveHideGame);
		});
	}
}

void FScene::RemoveRuntimeVirtualTexture(URuntimeVirtualTextureComponent* Component)
{
	FRuntimeVirtualTextureSceneProxy* SceneProxy = Component->SceneProxy;
	if (SceneProxy != nullptr)
	{
		// Release now but defer any deletion to the render thread
		Component->SceneProxy->Release();
		Component->SceneProxy = nullptr;

		FScene* Scene = this;
		ENQUEUE_RENDER_COMMAND(RemoveRuntimeVirtualTextureCommand)(
			[Scene, SceneProxy] (FRHICommandListBase&)
		{
			Scene->RuntimeVirtualTextures.RemoveAt(SceneProxy->SceneIndex);
			delete SceneProxy;

			UpdateRuntimeVirtualTextureHidePrimitiveMasks(Scene->RuntimeVirtualTextures, Scene->bRuntimeVirtualTexturePrimitiveHideEditor, Scene->bRuntimeVirtualTexturePrimitiveHideGame);
		});
	}
}

void FScene::GetRuntimeVirtualTextureHidePrimitiveMask(uint8& bHideMaskEditor, uint8& bHideMaskGame) const
{
	bHideMaskEditor = bRuntimeVirtualTexturePrimitiveHideEditor ? 0xff : 0;
	bHideMaskGame = bRuntimeVirtualTexturePrimitiveHideGame ? 0xff : 0;
}

void FScene::InvalidateRuntimeVirtualTexture(URuntimeVirtualTextureComponent* Component, FBoxSphereBounds const& WorldBounds, EVTInvalidatePriority InvalidatePriority)
{
	if (Component->SceneProxy != nullptr)
	{
		FRuntimeVirtualTextureSceneProxy* SceneProxy = Component->SceneProxy;
		ENQUEUE_RENDER_COMMAND(InvalidateRuntimeVirtualTextureCommand)(
			[SceneProxy, WorldBounds, InvalidatePriority] (FRHICommandListBase&)
		{
			SceneProxy->Dirty(WorldBounds, InvalidatePriority);
		});
	}
}

void FScene::FlushDirtyRuntimeVirtualTextures()
{
	checkSlow(IsInRenderingThread());
	for (TSparseArray<FRuntimeVirtualTextureSceneProxy*>::TIterator It(RuntimeVirtualTextures); It; ++It)
	{
		(*It)->FlushDirtyPages();
	}
}

void FScene::RequestPreloadRuntimeVirtualTexture(class URuntimeVirtualTextureComponent* Component, FBoxSphereBounds const& WorldBounds, int32 Level)
{
	if (Component->SceneProxy != nullptr)
	{
		FRuntimeVirtualTextureSceneProxy* SceneProxy = Component->SceneProxy;
		ENQUEUE_RENDER_COMMAND(PreloadRuntimeVirtualTextureCommand)(
			[SceneProxy, WorldBounds, Level](FRHICommandListBase&)
		{
			SceneProxy->RequestPreload(WorldBounds, Level);
		});
	}
}

void FScene::InvalidatePathTracedOutput(PathTracing::EInvalidateReason InvalidateReason)
{
#if RHI_RAYTRACING
	if (PathTracing::IsOutputInvalidateAllowed(InvalidateReason))
	{
		// NOTE: this is an atomic, so this function is ok to call from any thread
		++PathTracingInvalidationCounter;
	}
#endif
}

void FScene::InvalidateLumenSurfaceCache_GameThread(UPrimitiveComponent* Component)
{
	check(IsInGameThread());

	if (Component->SceneProxy)
	{
		ENQUEUE_RENDER_COMMAND(InvalidateLumenSurfaceCacheCmd)(
			[this, PrimitiveSceneProxy = Component->SceneProxy] (FRHICommandListBase&)
		{
			if (PrimitiveSceneProxy && PrimitiveSceneProxy->GetPrimitiveSceneInfo())
			{
				LumenInvalidateSurfaceCacheForPrimitive(PrimitiveSceneProxy->GetPrimitiveSceneInfo());
			}
		});
	}
}

bool FScene::GetPreviousLocalToWorld(const FPrimitiveSceneInfo* PrimitiveSceneInfo, FMatrix& OutPreviousLocalToWorld) const
{
	return VelocityData.GetComponentPreviousLocalToWorld(PrimitiveSceneInfo->PrimitiveComponentId, OutPreviousLocalToWorld);
}

void FSceneVelocityData::StartFrame(FScene* Scene)
{
	InternalFrameIndex++;

	const bool bTrimOld = InternalFrameIndex % 100 == 0;

	for (TMap<FPrimitiveComponentId, FComponentVelocityData>::TIterator It(ComponentData); It; ++It)
	{
		FComponentVelocityData& VelocityData = It.Value();
		VelocityData.PreviousLocalToWorld = VelocityData.LocalToWorld;
		VelocityData.bPreviousLocalToWorldValid = true;

		if ((InternalFrameIndex - VelocityData.LastFrameUpdated == 1) && VelocityData.PrimitiveSceneInfo)
		{
			// Force an update of the primitive data on the frame after the primitive moved, since it contains PreviousLocalToWorld
			VelocityData.PrimitiveSceneInfo->MarkGPUStateDirty(EPrimitiveDirtyState::ChangedTransform);
		}

		if (bTrimOld && (InternalFrameIndex - VelocityData.LastFrameUsed) > 10)
		{
			if (VelocityData.PrimitiveSceneInfo)
			{
				Scene->GPUScene.AddPrimitiveToUpdate(VelocityData.PrimitiveSceneInfo->GetPersistentIndex(), EPrimitiveDirtyState::ChangedOther);
			}

			It.RemoveCurrent();
		}
	}
}

void FScene::GetPrimitiveUniformShaderParameters_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool& bHasPrecomputedVolumetricLightmap, FMatrix& PreviousLocalToWorld, int32& SingleCaptureIndex, bool& bOutputVelocity) const 
{
	SCOPED_NAMED_EVENT(GetPrimitiveUniformShaderParameters_RenderThread, FColor::Yellow);

	// NOTE: The logic here is duplicated in BuildPrimitiveUniformShaderParameters_RenderThread since that allows reordering the logic to avoid transform copies & needlessly setting Builder.PreviousLocalToWorld - which avoids 
	//       a matrix inverse.
	const FMatrix LocalToWorld = PrimitiveSceneInfo->Proxy->GetLocalToWorld();
	PreviousLocalToWorld = LocalToWorld;
	bOutputVelocity = false;

	const bool bHasPreviousLocalToWorld = VelocityData.GetComponentPreviousLocalToWorld(PrimitiveSceneInfo->PrimitiveComponentId, PreviousLocalToWorld);
	if (bHasPreviousLocalToWorld)
	{
		bOutputVelocity = !LocalToWorld.Equals(PreviousLocalToWorld, 0.0001f);
	}

	bHasPrecomputedVolumetricLightmap = VolumetricLightmapSceneData.HasData();

	// Get index if proxy exists, otherwise fall back to index 0 which will contain the default black cubemap
	SingleCaptureIndex = PrimitiveSceneInfo->CachedReflectionCaptureProxy ? PrimitiveSceneInfo->CachedReflectionCaptureProxy->SortedCaptureIndex : 0;
}

void FScene::BuildPrimitiveUniformShaderParameters_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, FPrimitiveUniformShaderParametersBuilder& Builder) const
{
	const FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;

	const FMatrix LocalToWorld = Proxy->GetLocalToWorld();

	bool bOutputVelocity = Proxy->AlwaysHasVelocity() || (Proxy->IsNaniteMesh() && Proxy->IsSkinnedMesh());
	const bool bHasPrecomputedVolumetricLightmap = VolumetricLightmapSceneData.HasData();

	// Get index if proxy exists, otherwise fall back to index 0 which will contain the default black cubemap
	const int32 SingleCaptureIndex = PrimitiveSceneInfo->CachedReflectionCaptureProxy ? PrimitiveSceneInfo->CachedReflectionCaptureProxy->SortedCaptureIndex : 0;

	Builder.LocalToWorld(LocalToWorld)
			.SingleCaptureIndex(SingleCaptureIndex)
			.UseVolumetricLightmap(bHasPrecomputedVolumetricLightmap);

	// Only set up previous transform if we actually have one.
	if (const FMatrix* RESTRICT PreviousLocalToWorld = VelocityData.GetComponentPreviousLocalToWorld(PrimitiveSceneInfo->PrimitiveComponentId))
	{
		Builder.PreviousLocalToWorld(*PreviousLocalToWorld);
		bOutputVelocity = bOutputVelocity || !LocalToWorld.Equals(*PreviousLocalToWorld, 0.0001f);
	}

	Builder.OutputVelocity(bOutputVelocity)
			.LightmapDataIndex(PrimitiveSceneInfo->GetLightmapDataOffset())
			.InstanceSceneDataOffset(PrimitiveSceneInfo->GetInstanceSceneDataOffset())
			.NumInstanceSceneDataEntries(PrimitiveSceneInfo->GetNumInstanceSceneDataEntries())
			.InstancePayloadDataOffset(PrimitiveSceneInfo->GetInstancePayloadDataOffset())
			.InstancePayloadDataStride(PrimitiveSceneInfo->GetInstancePayloadDataStride())
			.PersistentPrimitiveIndex(PrimitiveSceneInfo->GetPersistentIndex().Index);
}

bool DoesPlatformNeedLocalLightPrimitiveInteraction(EShaderPlatform ShaderPlatform)
{
	if (CVarVisibilityLocalLightPrimitiveInteraction.GetValueOnRenderThread() == 0)
	{
		return false;
	}

	if (CVarVisibilityLocalLightPrimitiveInteraction.GetValueOnRenderThread() == 2 && MegaLights::IsEnabledInProject(ShaderPlatform))
	{
		return false;
	}

	extern bool MobileLocalLightsUseSinglePermutation(EShaderPlatform ShaderPlatform);
	return !IsMobilePlatform(ShaderPlatform) || !MobileLocalLightsUseSinglePermutation(ShaderPlatform) || IsMobileMovableSpotlightShadowsEnabled(ShaderPlatform);
}

void FScene::UpdateLightTransform_RenderThread(FLightSceneId LightId, FLightSceneInfo* LightSceneInfo, const FUpdateLightTransformParameters& Parameters)
{
	SCOPED_NAMED_EVENT(FScene_UpdateLightTransform_RenderThread, FColor::Yellow);

	// This is called without a valid ID when the update is fused with an 'add' command (saves redundant scene updates to do the update first)
	const bool bHasId = LightId.IsValid();
	// Don't Update Primitive Interactions for directional lights
	const bool bUpdatePrimitiveInteractions = bHasId && (Lights[LightId].LightType != LightType_Directional);

	// Invalidate the path tracer if the transform actually changed
	// NOTE: Position is derived from the Matrix, so there is no need to check it separately
	if (!Parameters.LightToWorld.Equals(LightSceneInfo->Proxy->LightToWorld, SMALL_NUMBER))
	{
		InvalidatePathTracedOutput();
	}

	// Update the light's transform and position.
	LightSceneInfo->Proxy->SetTransform(Parameters.LightToWorld, Parameters.Position);

	// Also update the LightSceneInfoCompact (if one exists)
	if (bHasId)
	{
		checkSlow(Lights[LightId].LightSceneInfo == LightSceneInfo);
		Lights[LightId].Init(LightSceneInfo);

		if (bUpdatePrimitiveInteractions && DoesPlatformNeedLocalLightPrimitiveInteraction(GetShaderPlatform()))
		{
			using PrimitiveSceneInfoSet = TSet<FPrimitiveSceneInfo*, DefaultKeyFuncs<FPrimitiveSceneInfo*>, SceneRenderingSetAllocator>;
			PrimitiveSceneInfoSet PrevPrimitivesInBounds;

			TMap<FPrimitiveSceneInfo*, FLightPrimitiveInteraction*, SceneRenderingSetAllocator> PrimitivesToInteractions;
			for (FLightPrimitiveInteraction* Interaction = LightSceneInfo->GetDynamicInteractionOftenMovingPrimitiveList();
				Interaction;
				Interaction = Interaction->GetNextPrimitive()
				)
			{
				PrevPrimitivesInBounds.Add(Interaction->GetPrimitiveSceneInfo());
				PrimitivesToInteractions.Add(Interaction->GetPrimitiveSceneInfo(), Interaction);
			}

			for (FLightPrimitiveInteraction* Interaction = LightSceneInfo->GetDynamicInteractionStaticPrimitiveList();
				Interaction;
				Interaction = Interaction->GetNextPrimitive()
				)
			{
				PrevPrimitivesInBounds.Add(Interaction->GetPrimitiveSceneInfo());
				PrimitivesToInteractions.Add(Interaction->GetPrimitiveSceneInfo(), Interaction);
			}

			PrimitiveSceneInfoSet CurrentPrimitivesInBounds;
			const FLightSceneInfoCompact& LightSceneInfoCompact = Lights[LightId];

			if (LightSceneInfo->OctreeId.IsValidId())
			{
				// Re-add the light to the octree after transform update.
				LocalShadowCastingLightOctree.RemoveElement(LightSceneInfo->OctreeId);
				LightSceneInfo->OctreeId = FOctreeElementId2();
				LocalShadowCastingLightOctree.AddElement(LightSceneInfoCompact);
			}

			PrimitiveOctree.FindElementsWithBoundsTest(LightSceneInfo->GetBoundingBox(), [&LightSceneInfoCompact, &CurrentPrimitivesInBounds, this](const FPrimitiveSceneInfoCompact& PrimitiveSceneInfoCompact)
				{
					CurrentPrimitivesInBounds.Add(PrimitiveSceneInfoCompact.PrimitiveSceneInfo);
				});

			PrimitiveSceneInfoSet PrimitivesToBeRemoved = PrevPrimitivesInBounds.Difference(CurrentPrimitivesInBounds);
			PrimitiveSceneInfoSet PrimitivesToAdd = CurrentPrimitivesInBounds.Difference(PrevPrimitivesInBounds);

			for (FPrimitiveSceneInfo* PrimitiveToRemove : PrimitivesToBeRemoved)
			{
				FLightPrimitiveInteraction::Destroy(PrimitivesToInteractions[PrimitiveToRemove]);
			}

			for (FPrimitiveSceneInfo* PrimitiveToAdd : PrimitivesToAdd)
			{
				LightSceneInfo->CreateLightPrimitiveInteraction(LightSceneInfoCompact, PrimitiveToAdd);
			}
		}
	}
}

template <typename UpdatePayloadType>
void FScene::UpdateLightInternal_RenderThread(FLightSceneProxy* LightSceneProxy, UpdatePayloadType&& InUpdatePayload)
{
	FLightSceneInfo* LightSceneInfo = LightSceneProxy->GetLightSceneInfo();
	if (LightSceneInfo->bVisible)
	{
		FScopeCycleCounter Context(LightSceneProxy->GetStatId());
		SceneLightInfoUpdates->Enqueue(LightSceneInfo, MoveTemp(InUpdatePayload));
	}
}

template <typename UpdatePayloadType>
void FScene::UpdateLightInternal(FLightSceneProxy* LightSceneProxy, UpdatePayloadType&& InUpdatePayload)
{
	if (LightSceneProxy)
	{
		FLightSceneInfo* LightSceneInfo = LightSceneProxy->GetLightSceneInfo();
		if (LightSceneInfo->bVisible)
		{
			ENQUEUE_RENDER_COMMAND(UpdateLightTransform)(
				[this, LightSceneInfo, LightSceneProxy, UpdatePayload = MoveTemp(InUpdatePayload)] (FRHICommandListBase&) mutable
				{
					FScopeCycleCounter Context(LightSceneProxy->GetStatId());
					SceneLightInfoUpdates->Enqueue(LightSceneInfo, MoveTemp(UpdatePayload));
				});
		}
	}
}

void FScene::UpdateLightTransform(ULightComponent* Light)
{
	UpdateLightInternal(Light->SceneProxy, FUpdateLightTransformParameters{.LightToWorld = Light->GetComponentTransform().ToMatrixNoScale(), .Position = Light->GetLightPosition()});
}

void FScene::UpdateLightColorAndBrightness(ULightComponent* Light)
{
	UpdateLightInternal(Light->SceneProxy, FUpdateLightColorParameters{
		.NewColor = Light->GetColoredLightBrightness(),
		.NewIndirectLightingScale = Light->IndirectLightingIntensity,
		.NewVolumetricScatteringIntensity = Light->VolumetricScatteringIntensity});
}

void FScene::UpdateLightProxy(ULightComponent* Light, TFunction<void(FLightSceneProxy*)>&& Func)
{
	FLightSceneProxy* LightSceneProxy = Light->SceneProxy;
	if (LightSceneProxy)
	{
		ENQUEUE_RENDER_COMMAND(UpdateLightProxy)(
			[this, LightSceneProxy, Func](FRHICommandListBase&)
			{
				Func(LightSceneProxy);
				UpdateLightInternal_RenderThread(LightSceneProxy, FUpdateLightInvalidateGPUState{});
			});
	}
}

void FScene::RemoveLightSceneInfo_RenderThread(FLightSceneInfo* LightSceneInfo)
{
	SCOPE_CYCLE_COUNTER(STAT_RemoveSceneLightTime);

	check(LightSceneInfo->bVisible);

	const bool bDirectionalLight = LightSceneInfo->Proxy->GetLightType() == LightType_Directional;

	if (bDirectionalLight)
	{
		DirectionalLights.Remove(LightSceneInfo);
	}

	// check SimpleDirectionalLight
	if (LightSceneInfo == SimpleDirectionalLight)
	{
		SimpleDirectionalLight = nullptr;
	}

	if(GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Mobile)
	{
		const bool bUseCSMForDynamicObjects = LightSceneInfo->Proxy->UseCSMForDynamicObjects();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Tracked for disabled shader permutation warnings.
		// Condition must match that in AddLightSceneInfo_RenderThread
		if (LightSceneInfo->Proxy->GetLightType() == LightType_Directional && !LightSceneInfo->Proxy->HasStaticLighting())
		{
			if (LightSceneInfo->Proxy->IsMovable())
			{
				NumMobileMovableDirectionalLights_RenderThread--;
			}
			if (bUseCSMForDynamicObjects)
			{
				NumMobileStaticAndCSMLights_RenderThread--;
			}
		}
#endif

		// check MobileDirectionalLights
		for (int32 LightChannelIdx = 0; LightChannelIdx < UE_ARRAY_COUNT(MobileDirectionalLights); LightChannelIdx++)
		{
			if (LightSceneInfo == MobileDirectionalLights[LightChannelIdx])
			{
				MobileDirectionalLights[LightChannelIdx] = nullptr;

				// find another light that could be the new MobileDirectionalLight for this channel
				for (const FLightSceneInfoCompact& OtherLight : Lights)
				{
					if (OtherLight.LightSceneInfo != LightSceneInfo &&
						OtherLight.LightType == LightType_Directional &&
						!OtherLight.bStaticLighting &&
						GetFirstLightingChannelFromMask(OtherLight.LightSceneInfo->Proxy->GetLightingChannelMask()) == LightChannelIdx)
					{
						MobileDirectionalLights[LightChannelIdx] = OtherLight.LightSceneInfo;
						break;
					}
				}

				// if this light is a dynamic shadowcast then we need to update the static draw lists to pick a new lightingpolicy
				const bool bUsesDirectionalLightForLighmapPolicySelection = IsStaticLightingAllowed() && !IsMobileDeferredShadingEnabled(GetShaderPlatform());
				if (bUsesDirectionalLightForLighmapPolicySelection && (!LightSceneInfo->Proxy->HasStaticShadowing() || bUseCSMForDynamicObjects))
				{
					bScenesPrimitivesNeedStaticMeshElementUpdate = true;
					UE_CLOGF(!GIsEditor, LogRenderer, Log, "Forcing update for all mesh draw commands: Remove directional light");
				}
				break;
			}
		}
	}

	ProcessAtmosphereLightRemoval_RenderThread(LightSceneInfo);

	// Remove the light from the scene.
	LightSceneInfo->RemoveFromScene();

	// Remove the light from the lights list.
	Lights.RemoveAt(LightSceneInfo->Id);

	if (!LightSceneInfo->Proxy->HasStaticShadowing()
		&& LightSceneInfo->Proxy->CastsDynamicShadow()
		&& LightSceneInfo->GetDynamicShadowMapChannel() == -1)
	{
		OverflowingDynamicShadowedLights.Remove(LightSceneInfo->Proxy->GetOwnerNameOrLabel());
	}

	InvalidatePathTracedOutput();

	if (LightSceneInfo->Proxy->GetLightType() == LightType_Rect)
	{
		const FRectLightSceneProxy* RectProxy = (const FRectLightSceneProxy*)LightSceneInfo->Proxy;
		RectLightAtlas::RemoveTexture(RectProxy->RectAtlasId);
	}

	if (UTextureLightProfile* IESTexture = LightSceneInfo->Proxy->GetIESTexture())
	{
		IESAtlas::RemoveTexture(LightSceneInfo->Proxy->IESAtlasId);
	}

	// Free the light scene info and proxy.
	delete LightSceneInfo->Proxy;
	delete LightSceneInfo;
}

void FScene::RemoveLight(ULightComponent* Light)
{
	FLightSceneDesc LightSceneDesc{ .SceneProxy = Light->SceneProxy };
	BatchRemoveLights(MakeArrayView(&LightSceneDesc, 1));

	// Disassociate the primitive's render info.
	Light->SceneProxy = nullptr;
}

void FScene::BatchRemoveLights(TArrayView<FLightSceneDesc> InLights)
{
	TArray<FLightSceneInfo*, SceneRenderingAllocator> LightsToRemove;
	LightsToRemove.Reserve(InLights.Num());

	for (const FLightSceneDesc& LightDesc : InLights)
	{
		if (LightDesc.SceneProxy)
		{
			FLightSceneInfo* LightSceneInfo = LightDesc.SceneProxy->GetLightSceneInfo();

			DEC_DWORD_STAT(STAT_SceneLights);

			if (LightSceneInfo->bVisible)
			{
				// Removing one visible light
				--NumVisibleLights_AnyThread;
			}

			LightsToRemove.Emplace(LightSceneInfo);
		}
	}

	if (!LightsToRemove.IsEmpty())
	{
		// Send a command to the rendering thread to queue the light for removal from the scene.
		ENQUEUE_RENDER_COMMAND(BatchRemoveLightsCommand)(
			[this, LightsToRemove = MoveTemp(LightsToRemove)](FRHICommandListBase& RHICmdList)
			{
				for (FLightSceneInfo* LightSceneInfo : LightsToRemove)
				{
					FScopeCycleCounter Context(LightSceneInfo->Proxy->GetStatId());

					if (LightSceneInfo->bVisible)
					{
						// Note: all the deletes are deferred now (except the invisible ones) as this simpifies the update buffer.
						SceneLightInfoUpdates->EnqueueDelete(LightSceneInfo);
					}
					else
					{
						// There should never be updates queued for lights that are not visible
						check(!SceneLightInfoUpdates->HasCommand(LightSceneInfo));
						// The "invisible lights" are removed at once.
						InvisibleLights.RemoveAt(LightSceneInfo->Id);

						// Free the light scene info and proxy.
						delete LightSceneInfo->Proxy;
						delete LightSceneInfo;
					}
				}
			});
	}
}

void FScene::AddExponentialHeightFog(uint64 Id, const FExponentialHeightFogDynamicState& State)
{
	FExponentialHeightFogSceneInfo HeightFogSceneInfo(Id, State);
	ENQUEUE_RENDER_COMMAND(FAddFogCommand)(
		[Scene = this, HeightFogSceneInfo] (FRHICommandListBase&)
		{
			// Create a FExponentialHeightFogSceneInfo for the component in the scene's fog array.
			new(Scene->ExponentialFogs) FExponentialHeightFogSceneInfo(HeightFogSceneInfo);
			Scene->InvalidatePathTracedOutput();
		});
}

void FScene::RemoveExponentialHeightFog(uint64 Id)
{
	ENQUEUE_RENDER_COMMAND(FRemoveFogCommand)(
		[Scene = this, Id] (FRHICommandListBase&)
		{
			// Remove the given component's FExponentialHeightFogSceneInfo from the scene's fog array.
			for(int32 FogIndex = 0;FogIndex < Scene->ExponentialFogs.Num();FogIndex++)
			{
				if(Scene->ExponentialFogs[FogIndex].Id == Id)
				{
					Scene->ExponentialFogs.RemoveAt(FogIndex);
					Scene->InvalidatePathTracedOutput();
					break;
				}
			}
		});
}

bool FScene::HasAnyExponentialHeightFog() const
{
	return this->ExponentialFogs.Num() > 0;
}

void FScene::AddWindSource(UWindDirectionalSourceComponent* WindComponent)
{
	// if this wind component is not activated (or Auto Active is set to false), then don't add to WindSources
	if(!WindComponent->IsActive())
	{
		return;
	}
	ensure(IsInGameThread());
	WindComponents_GameThread.Add(WindComponent);

	FWindSourceSceneProxy* SceneProxy = WindComponent->CreateSceneProxy();
	WindComponent->SceneProxy = SceneProxy;

	FScene* Scene = this;
	ENQUEUE_RENDER_COMMAND(FAddWindSourceCommand)(
		[Scene, SceneProxy] (FRHICommandListBase&)
		{
			Scene->WindSources.Add(SceneProxy);
		});
}

void FScene::RemoveWindSource(UWindDirectionalSourceComponent* WindComponent)
{
	ensure(IsInGameThread());
	WindComponents_GameThread.Remove(WindComponent);

	FWindSourceSceneProxy* SceneProxy = WindComponent->SceneProxy;
	WindComponent->SceneProxy = NULL;

	if(SceneProxy)
	{
		FScene* Scene = this;
		ENQUEUE_RENDER_COMMAND(FRemoveWindSourceCommand)(
			[Scene, SceneProxy] (FRHICommandListBase&)
			{
				Scene->WindSources.Remove(SceneProxy);

				delete SceneProxy;
			});
	}
}

void FScene::UpdateWindSource(UWindDirectionalSourceComponent* WindComponent)
{
	// Recreate the scene proxy without touching WindComponents_GameThread
	// so that this function is kept thread safe when iterating in parallel
	// over components (unlike AddWindSource and RemoveWindSource)
	FWindSourceSceneProxy* const OldSceneProxy = WindComponent->SceneProxy;
	if (OldSceneProxy)
	{
		WindComponent->SceneProxy = nullptr;

		ENQUEUE_RENDER_COMMAND(FRemoveWindSourceCommand)(
			[Scene = this, OldSceneProxy] (FRHICommandListBase&)
			{
				Scene->WindSources.Remove(OldSceneProxy);

				delete OldSceneProxy;
			});
	}

	if (WindComponent->IsActive())
	{
		FWindSourceSceneProxy* const NewSceneProxy = WindComponent->CreateSceneProxy();
		WindComponent->SceneProxy = NewSceneProxy;

		ENQUEUE_RENDER_COMMAND(FAddWindSourceCommand)(
			[Scene = this, NewSceneProxy] (FRHICommandListBase&)
			{
				Scene->WindSources.Add(NewSceneProxy);
			});
	}

}

const TArray<FWindSourceSceneProxy*>& FScene::GetWindSources_RenderThread() const
{
	checkSlow(IsInRenderingThread());
	return WindSources;
}

void FScene::GetWindParameters(const FVector& Position, FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const
{
	FWindData AccumWindData;
	AccumWindData.PrepareForAccumulate();

	int32 NumActiveWindSources = 0;
	FVector4f AccumulatedDirectionAndSpeed(0,0,0,0);
	float TotalWeight = 0.0f;
	for (int32 i = 0; i < WindSources.Num(); i++)
	{
		
		FVector4f CurrentDirectionAndSpeed;
		float Weight;
		const FWindSourceSceneProxy* CurrentSource = WindSources[i];
		FWindData CurrentSourceData;
		if (CurrentSource->GetWindParameters(Position, CurrentSourceData, Weight))
		{
			AccumWindData.AddWeighted(CurrentSourceData, Weight);
			TotalWeight += Weight;
			NumActiveWindSources++;
		}
	}

	AccumWindData.NormalizeByTotalWeight(TotalWeight);

	if (NumActiveWindSources == 0)
	{
		AccumWindData.Direction = FVector(1.0f, 0.0f, 0.0f);
	}
	OutDirection	= AccumWindData.Direction;
	OutSpeed		= AccumWindData.Speed;
	OutMinGustAmt	= AccumWindData.MinGustAmt;
	OutMaxGustAmt	= AccumWindData.MaxGustAmt;
}

void FScene::GetWindParameters_GameThread(const FVector& Position, FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const
{
	FWindData AccumWindData;
	AccumWindData.PrepareForAccumulate();

	const int32 NumSources = WindComponents_GameThread.Num();
	int32 NumActiveSources = 0;
	float TotalWeight = 0.0f;

	// read the wind component array, this is safe for the game thread
	for(UWindDirectionalSourceComponent* Component : WindComponents_GameThread)
	{
		float Weight = 0.0f;
		FWindData CurrentComponentData;
		if(Component && Component->GetWindParameters(Position, CurrentComponentData, Weight))
		{
			AccumWindData.AddWeighted(CurrentComponentData, Weight);
			TotalWeight += Weight;
			++NumActiveSources;
		}
	}

	AccumWindData.NormalizeByTotalWeight(TotalWeight);

	if(NumActiveSources == 0)
	{
		AccumWindData.Direction = FVector(1.0f, 0.0f, 0.0f);
	}

	OutDirection = AccumWindData.Direction;
	OutSpeed = AccumWindData.Speed;
	OutMinGustAmt = AccumWindData.MinGustAmt;
	OutMaxGustAmt = AccumWindData.MaxGustAmt;
}

void FScene::GetDirectionalWindParameters(FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const
{
	FWindData AccumWindData;
	AccumWindData.PrepareForAccumulate();

	int32 NumActiveWindSources = 0;
	FVector4f AccumulatedDirectionAndSpeed(0,0,0,0);
	float TotalWeight = 0.0f;
	for (int32 i = 0; i < WindSources.Num(); i++)
	{
		FVector4f CurrentDirectionAndSpeed;
		float Weight;
		const FWindSourceSceneProxy* CurrentSource = WindSources[i];
		FWindData CurrentSourceData;
		if (CurrentSource->GetDirectionalWindParameters(CurrentSourceData, Weight))
		{
			AccumWindData.AddWeighted(CurrentSourceData, Weight);			
			TotalWeight += Weight;
			NumActiveWindSources++;
		}
	}

	AccumWindData.NormalizeByTotalWeight(TotalWeight);	

	if (NumActiveWindSources == 0)
	{
		AccumWindData.Direction = FVector(1.0f, 0.0f, 0.0f);
	}
	OutDirection = AccumWindData.Direction;
	OutSpeed = AccumWindData.Speed;
	OutMinGustAmt = AccumWindData.MinGustAmt;
	OutMaxGustAmt = AccumWindData.MaxGustAmt;
}

void FScene::AddSpeedTreeWind(FVertexFactory* VertexFactory, const UStaticMesh* StaticMesh)
{
	if (StaticMesh != NULL && StaticMesh->SpeedTreeWind.IsValid() && StaticMesh->GetRenderData())
	{
		FScene* Scene = this;
		ENQUEUE_RENDER_COMMAND(FAddSpeedTreeWindCommand)(
			[Scene, StaticMesh, VertexFactory] (FRHICommandListBase&)
			{
				Scene->SpeedTreeVertexFactoryMap.Add(VertexFactory, StaticMesh);

				if (Scene->SpeedTreeWindComputationMap.Contains(StaticMesh))
				{
					(*(Scene->SpeedTreeWindComputationMap.Find(StaticMesh)))->ReferenceCount++;
				}
				else
				{
					FSpeedTreeWindComputation* WindComputation = new FSpeedTreeWindComputation;
					WindComputation->Wind = *(StaticMesh->SpeedTreeWind.Get( ));

					FSpeedTreeUniformParameters UniformParameters;
					FPlatformMemory::Memzero(&UniformParameters, sizeof(UniformParameters));
					WindComputation->UniformBuffer = TUniformBufferRef<FSpeedTreeUniformParameters>::CreateUniformBufferImmediate(UniformParameters, UniformBuffer_MultiFrame);
					Scene->SpeedTreeWindComputationMap.Add(StaticMesh, WindComputation);
				}
			});
	}
}

void FScene::RemoveSpeedTreeWind_RenderThread(class FVertexFactory* VertexFactory, const class UStaticMesh* StaticMesh)
{
	FSpeedTreeWindComputation** WindComputationRef = SpeedTreeWindComputationMap.Find(StaticMesh);
	if (WindComputationRef != NULL)
	{
		FSpeedTreeWindComputation* WindComputation = *WindComputationRef;

		WindComputation->ReferenceCount--;
		if (WindComputation->ReferenceCount < 1)
		{
			for (auto Iter = SpeedTreeVertexFactoryMap.CreateIterator(); Iter; ++Iter )
			{
				if (Iter.Value() == StaticMesh)
				{
					Iter.RemoveCurrent();
				}
			}

			SpeedTreeWindComputationMap.Remove(StaticMesh);
			delete WindComputation;
		}
	}
}

void FScene::UpdateSpeedTreeWind(double CurrentTime)
{
#define SET_SPEEDTREE_TABLE_FLOAT4V(name, offset) \
	UniformParameters.name = *(FVector4f*)(WindShaderValues + FSpeedTreeWind::offset); \
	UniformParameters.Prev##name = *(FVector4f*)(WindShaderValues + FSpeedTreeWind::offset + FSpeedTreeWind::NUM_SHADER_VALUES);

	FScene* Scene = this;
	ENQUEUE_RENDER_COMMAND(FUpdateSpeedTreeWindCommand)(
		[Scene, CurrentTime] (FRHICommandListBase& RHICmdList)
		{
			FVector WindDirection;
			float WindSpeed;
			float WindMinGustAmt;
			float WindMaxGustAmt;
			Scene->GetDirectionalWindParameters(WindDirection, WindSpeed, WindMinGustAmt, WindMaxGustAmt);

			for (TMap<const UStaticMesh*, FSpeedTreeWindComputation*>::TIterator It(Scene->SpeedTreeWindComputationMap); It; ++It )
			{
				const UStaticMesh* StaticMesh = It.Key();
				FSpeedTreeWindComputation* WindComputation = It.Value();

				if( !(StaticMesh->GetRenderData() && StaticMesh->SpeedTreeWind.IsValid()) )
				{
					It.RemoveCurrent();
					continue;
				}

				if (GIsEditor && StaticMesh->SpeedTreeWind->NeedsReload( ))
				{
					// reload the wind since it may have changed or been scaled differently during reimport
					StaticMesh->SpeedTreeWind->SetNeedsReload(false);
					WindComputation->Wind = *(StaticMesh->SpeedTreeWind.Get( ));
				}

				// advance the wind object
				WindComputation->Wind.SetDirection(WindDirection);
				WindComputation->Wind.SetStrength(WindSpeed);
				WindComputation->Wind.SetGustMin(WindMinGustAmt);
				WindComputation->Wind.SetGustMax(WindMaxGustAmt);
				WindComputation->Wind.Advance(true, CurrentTime);

				// copy data into uniform buffer
				const float* WindShaderValues = WindComputation->Wind.GetShaderTable();

				FSpeedTreeUniformParameters UniformParameters;
				UniformParameters.WindAnimation.Set(CurrentTime, 0.0f, 0.0f, 0.0f);
			
				SET_SPEEDTREE_TABLE_FLOAT4V(WindVector, SH_WIND_DIR_X);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindGlobal, SH_GLOBAL_TIME);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindBranch, SH_BRANCH_1_TIME);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindBranchTwitch, SH_BRANCH_1_TWITCH);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindBranchWhip, SH_BRANCH_1_WHIP);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindBranchAnchor, SH_WIND_ANCHOR_X);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindBranchAdherences, SH_GLOBAL_DIRECTION_ADHERENCE);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindTurbulences, SH_BRANCH_1_TURBULENCE);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindLeaf1Ripple, SH_LEAF_1_RIPPLE_TIME);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindLeaf1Tumble, SH_LEAF_1_TUMBLE_TIME);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindLeaf1Twitch, SH_LEAF_1_TWITCH_THROW);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindLeaf2Ripple, SH_LEAF_2_RIPPLE_TIME);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindLeaf2Tumble, SH_LEAF_2_TUMBLE_TIME);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindLeaf2Twitch, SH_LEAF_2_TWITCH_THROW);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindFrondRipple, SH_FROND_RIPPLE_TIME);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindRollingBranch, SH_ROLLING_BRANCH_FIELD_MIN);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindRollingLeafAndDirection, SH_ROLLING_LEAF_RIPPLE_MIN);
				SET_SPEEDTREE_TABLE_FLOAT4V(WindRollingNoise, SH_ROLLING_NOISE_PERIOD);

				WindComputation->UniformBuffer.UpdateUniformBufferImmediate(RHICmdList, UniformParameters);
			}
		});
	#undef SET_SPEEDTREE_TABLE_FLOAT4V
}

FRHIUniformBuffer* FScene::GetSpeedTreeUniformBuffer(const FVertexFactory* VertexFactory) const
{
	if (VertexFactory != NULL)
	{
		const UStaticMesh* const* StaticMesh = SpeedTreeVertexFactoryMap.Find(VertexFactory);
		if (StaticMesh != NULL)
		{
			const FSpeedTreeWindComputation* const * WindComputation = SpeedTreeWindComputationMap.Find(*StaticMesh);
			if (WindComputation != NULL)
			{
				return (*WindComputation)->UniformBuffer;
			}
		}
	}

	return nullptr;
}

FLightSceneInfo* FScene::GetMobileDirectionalLightForView(int32 ChannelIdx, const FViewInfo& View) const
{
	// TODO:  Should this call "ShouldRenderLight(View)" instead of just checking the view lighting channel mask?  Main
	// difference is that ShouldRenderLight filters out dynamic lights when the view is set to render static only, typically
	// for sky and reflection captures.  The immediate goal of this function is to make the view lighting channel mask work
	// on mobile, and otherwise avoid changes to existing behavior, so we'll err on the side of caution for now.  If this
	// gets changed, the other view lighting channel mask checks in FMobileSceneRenderer::InitDynamicShadows and
	// FMobileSceneRenderer::BuildCSMVisibilityState should also be updated to use "ShouldRenderLight"!
	if (MobileDirectionalLights[ChannelIdx] &&
		MobileDirectionalLights[ChannelIdx]->Proxy &&
		(MobileDirectionalLights[ChannelIdx]->Proxy->GetViewLightingChannelMask() & View.ViewLightingChannelMask))
	{
		return MobileDirectionalLights[ChannelIdx];
	}
	return nullptr;
}

/**
 * Retrieves the lights interacting with the passed in primitive and adds them to the out array.
 *
 * Render thread version of function.
 * 
 * @param	Primitive				Primitive to retrieve interacting lights for
 * @param	RelevantLights	[out]	Array of lights interacting with primitive
 */
void FScene::GetRelevantLights_RenderThread( const FPrimitiveSceneProxy* PrimitiveSceneProxy, TArray<const FLightSceneProxy*> &OutRelevantLights ) const
{
	if (ShouldSkipNaniteLPIs(GetShaderPlatform()) && PrimitiveSceneProxy->IsNaniteMesh())
	{
		if (PrimitiveSceneProxy->GetLightingChannelMask() != 0)
		{
			const FBoxSphereBounds& Bounds = PrimitiveSceneProxy->GetBounds();
			const FPrimitiveSceneInfoCompact PrimitiveSceneInfoCompact(PrimitiveSceneProxy->GetPrimitiveSceneInfo());
			auto TestAddLight = [&](const FLightSceneInfoCompact& LightSceneInfoCompact)
			{
				if (LightSceneInfoCompact.LightSceneInfo->ShouldCreateLightPrimitiveInteraction(LightSceneInfoCompact, PrimitiveSceneInfoCompact))
				{
					OutRelevantLights.Add( LightSceneInfoCompact.LightSceneInfo->Proxy);
				}
			};

			if(DoesPlatformNeedLocalLightPrimitiveInteraction(GetShaderPlatform()))
			{
				// Find local lights that affect the primitive in the light octree.
				LocalShadowCastingLightOctree.FindElementsWithBoundsTest(Bounds.GetBox(), TestAddLight);
			}
			// Also loop through non-local (directional) shadow-casting lights
			for (FLightSceneId LightID : DirectionalShadowCastingLightIDs)
			{
				TestAddLight(Lights[LightID]);
			}
		}
	}
	else
	{
		for( const FLightPrimitiveInteraction* Interaction = PrimitiveSceneProxy->GetPrimitiveSceneInfo()->LightList; Interaction; Interaction=Interaction->GetNextLight() )
		{
			OutRelevantLights.Add( Interaction->GetLight()->Proxy);
		}
	}
}

/**
 * Retrieves the lights interacting with the passed in primitive and adds them to the out array.
 *
 * @param	Primitive				Primitive to retrieve interacting lights for
 * @param	RelevantLights	[out]	Array of lights interacting with primitive
 */
void FScene::GetRelevantLights( UPrimitiveComponent* Primitive, TArray<const ULightComponent*>* RelevantLights ) const
{
	if( Primitive && RelevantLights && Primitive->GetSceneProxy() != nullptr)
	{
		// Add interacting lights to the array.
		ENQUEUE_RENDER_COMMAND(FGetRelevantLightsCommand)(
			[this, PrimitiveSceneProxy = Primitive->GetSceneProxy(), RelevantLights] (FRHICommandListBase&)
			{
				TArray<const FLightSceneProxy*> RelevantLightProxies;
				GetRelevantLights_RenderThread( PrimitiveSceneProxy, RelevantLightProxies );
				for (const FLightSceneProxy* LightSceneProxy : RelevantLightProxies)
				{
					check(LightSceneProxy);
					if (const ULightComponent* LightComponent = LightSceneProxy->LightComponent)
					{
						RelevantLights->Add(LightComponent);
					}
				}
			});

		// We need to block the main thread as the rendering thread needs to finish modifying the array before we can continue.
		FlushRenderingCommands();
	}
}

int32 FScene::GetRelevantLightCount(UPrimitiveComponent* Primitive) const
{
	int32 Count = 0;
	if (Primitive && Primitive->GetSceneProxy() != nullptr)
	{
		// Add interacting lights to the array.
		ENQUEUE_RENDER_COMMAND(FGetRelevantLightsCommand)(
			[this, PrimitiveSceneProxy = Primitive->GetSceneProxy(), &Count](FRHICommandListBase&)
			{
				TArray<const FLightSceneProxy*> RelevantLightProxies;
				GetRelevantLights_RenderThread(PrimitiveSceneProxy, RelevantLightProxies);
				for (const FLightSceneProxy* LightSceneProxy : RelevantLightProxies)
				{
					if (LightSceneProxy->GetLightComponentId().IsValid())
					{
						++Count;
					}
				}
			});
	}

	// Flush rendering commands to wait for command to execute
	FlushRenderingCommands();

	return Count;
}

/** Sets the precomputed visibility handler for the scene, or NULL to clear the current one. */
void FScene::SetPrecomputedVisibility(const FPrecomputedVisibilityHandler* NewPrecomputedVisibilityHandler)
{
	FScene* Scene = this;
	ENQUEUE_RENDER_COMMAND(UpdatePrecomputedVisibility)(
		[Scene, NewPrecomputedVisibilityHandler] (FRHICommandListBase&)
		{
			Scene->PrecomputedVisibilityHandler = NewPrecomputedVisibilityHandler;
		});
}

void FScene::UpdateCachedRenderStates(FPrimitiveSceneProxy* SceneProxy)
{
	if (SceneProxy->GetPrimitiveSceneInfo())
	{
		SceneProxy->GetPrimitiveSceneInfo()->RequestStaticMeshUpdate();
	}
}

#if RHI_RAYTRACING

void FScene::UpdateCachedRayTracingState(FPrimitiveSceneProxy* SceneProxy, EPrimitiveDirtyRayTracingState PrimitiveDirtyState)
{
	check(IsInRenderingThread());

	if (FPrimitiveSceneInfo* PrimitiveSceneInfo = SceneProxy->GetPrimitiveSceneInfo())
	{
		if (EnumHasAnyFlags(PrimitiveDirtyState, EPrimitiveDirtyRayTracingState::MeshCommandsAndInstances))
		{
			PrimitiveSceneInfo->RemoveCachedRayTracingPrimitives();
		}

		const int32 PrimitiveIndex = PrimitiveSceneInfo->GetIndex();
		if (PrimitiveIndex != INDEX_NONE)
		{
			if (EnumHasAnyFlags(PrimitiveDirtyState, EPrimitiveDirtyRayTracingState::DrawInGame))
			{
				PrimitiveRayTracingDatas[PrimitiveIndex].bDrawInGame = SceneProxy->IsDrawnInGame();
			}

			if (EnumHasAnyFlags(PrimitiveDirtyState, EPrimitiveDirtyRayTracingState::LightingChannelMask))
			{
				PrimitiveRayTracingDatas[PrimitiveIndex].LightingChannelMask = SceneProxy->GetLightingChannelMask();
			}

			if (EnumHasAnyFlags(PrimitiveDirtyState, EPrimitiveDirtyRayTracingState::MeshCommandsAndInstances))
			{
				PrimitiveRayTracingDatas[PrimitiveIndex].bCachedRaytracingDataDirty = true;
			}
		}
	}
}

extern int32 GRayTracingSBTImmediateRelease;

void FScene::ReleaseUnusedPersistentSBTs()
{
	// When immediate release is enabled, it is no longer the responsibility of this function to release SBTs.
	if (GRayTracingSBTImmediateRelease)
	{
		RayTracingPersistentSBTState.MaterialSBT.bUsedThisFrame = false;
		RayTracingPersistentSBTState.PathTracingSBT.bUsedThisFrame = false;
		RayTracingPersistentSBTState.LumenSBT.bUsedThisFrame = false;
		RayTracingPersistentSBTState.InlineSBT.bUsedThisFrame = false;

		return;
	}

	// This function is called at the end of frame, after all viewfamilies being rendered
	// had a chance to mark used SBTs as such.

	// Helper to release if unused this frame
	auto ReleaseSBTIfUnused = [&](FRayTracingPersistentSBTState::FTrackedSBT& TrackedSBT)
	{
		if (TrackedSBT.ID != INDEX_NONE && !TrackedSBT.bUsedThisFrame)
		{
			RayTracingSBT.ReleasePersistentSBT(TrackedSBT.ID);
			TrackedSBT.Reset();
		}
	};

	// Check and release each SBT type
	ReleaseSBTIfUnused(RayTracingPersistentSBTState.MaterialSBT);
	ReleaseSBTIfUnused(RayTracingPersistentSBTState.PathTracingSBT);
	ReleaseSBTIfUnused(RayTracingPersistentSBTState.LumenSBT);
	ReleaseSBTIfUnused(RayTracingPersistentSBTState.InlineSBT);

	// Reset our frame usage flags since this function is meant to be called once per scene at the end of the frame.
	RayTracingPersistentSBTState.MaterialSBT.bUsedThisFrame = false;
	RayTracingPersistentSBTState.PathTracingSBT.bUsedThisFrame = false;
	RayTracingPersistentSBTState.LumenSBT.bUsedThisFrame = false;
	RayTracingPersistentSBTState.InlineSBT.bUsedThisFrame = false;
}

#endif // RHI_RAYTRACING

void FScene::EndFrame(FRHICommandListImmediate& RHICmdList)
{
	// Performs a final update of any queued scene primitives in the case where the scene wasn't rendered to avoid a build-up of queued data that is never flushed.
#if !WITH_STATE_STREAM
	if (LastUpdateFrameCounter != GFrameCounterRenderThread)
	{
		UpdateAllPrimitiveSceneInfos(RHICmdList);
	}
#endif

#if RHI_RAYTRACING
	ReleaseUnusedPersistentSBTs();
#endif
}

/**
 * @return		true if hit proxies should be rendered in this scene.
 */
bool FScene::RequiresHitProxies() const
{
	return (GIsEditor && bRequiresHitProxies);
}

void FScene::Release()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FScene::Release);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Verify that no components reference this scene being destroyed
	static bool bTriggeredOnce = false;

	if (!bTriggeredOnce)
	{
		for (auto* ActorComponent : TObjectRange<UActorComponent>())
		{
			if ( !ensureMsgf(!ActorComponent->IsRegistered() || ActorComponent->GetScene() != this,
					TEXT("Component Name: %s World Name: %s Component Asset: %s"),
										*ActorComponent->GetFullName(),
										*GetWorld()->GetFullName(),
										*ActorComponent->AdditionalStatObject()->GetPathName()) )
			{
				bTriggeredOnce = true;
				break;
			}
		}
	}
#endif

	GetRendererModule().RemoveScene(this);

	UE::RenderCommandPipe::FSyncScope SyncScope;

	FScene* Scene = this;

#if WITH_STATE_STREAM
	if (FStateStreamManagerImpl* StateStreamManager = static_cast<FStateStreamManagerImpl*>(World->GetStateStreamManager()))
	{
		ENQUEUE_RENDER_COMMAND(FReleaseStateStream)(
			[Scene, StateStreamManager](FRHICommandListImmediate& RHICmdList)
			{
				//StateStreamManager->Render_Exit();
				//GetRendererModule().FreeStateStream(StateStreamManager);
				if (FSkeletalMeshUpdater* Smu = Scene->SkeletalMeshUpdater)
				{
					Smu->Shutdown();
				}
			});
	}
	else
#endif
	if (SkeletalMeshUpdater)
	{
		SkeletalMeshUpdater->Shutdown();
	}

	// Send a command to the rendering thread to release the scene.
	ENQUEUE_RENDER_COMMAND(FReleaseCommand)(
		[Scene
		](FRHICommandListImmediate& RHICmdList)
		{
			// Flush any remaining batched primitive update commands before deleting the scene.
			FUpdateParameters UpdateParameters;
			UpdateParameters.bDestruction = true;

			// Scope required so that the GraphBuilder is destructed before this Scene
			{
				FRDGBuilder GraphBuilder(RHICmdList, FRDGEventName(TEXT("UpdateAllPrimitiveSceneInfos")));
				Scene->Update(GraphBuilder, UpdateParameters);
				GraphBuilder.Execute();
			}

			// Wait for RDG to complete async deletion as scene extensions can be allocated through RDG.
			FRDGBuilder::WaitForAsyncDeleteTask();

			delete Scene;
		});
}

void FScene::UpdateEarlyZPassMode()
{
	checkSlow(IsInGameThread());

	DefaultBasePassDepthStencilAccess = GetDefaultBasePassDepthStencilAccess(GetFeatureLevel());
	GetEarlyZPassMode(GetFeatureLevel(), EarlyZPassMode, bEarlyZPassMovable);
}

FExclusiveDepthStencil::Type FScene::GetDefaultBasePassDepthStencilAccess(ERHIFeatureLevel::Type InFeatureLevel)
{
	FExclusiveDepthStencil::Type BasePassDepthStencilAccess = FExclusiveDepthStencil::DepthWrite_StencilWrite;

	if (GetFeatureLevelShadingPath(InFeatureLevel) == EShadingPath::Deferred)
	{
		const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);
		if (ShouldForceFullDepthPass(ShaderPlatform)
			&& CVarBasePassWriteDepthEvenWithFullPrepass.GetValueOnAnyThread() == 0)
		{
			BasePassDepthStencilAccess = FExclusiveDepthStencil::DepthRead_StencilWrite;
		}
	}

	return BasePassDepthStencilAccess;
}

void FScene::GetEarlyZPassMode(ERHIFeatureLevel::Type InFeatureLevel, EDepthDrawingMode & OutZPassMode, bool& bOutEarlyZPassMovable)
{
	OutZPassMode = DDM_NonMaskedOnly;
	bOutEarlyZPassMovable = false;

	const EShaderPlatform ShaderPlatform = GetFeatureLevelShaderPlatform(InFeatureLevel);
	if (GetFeatureLevelShadingPath(InFeatureLevel) == EShadingPath::Deferred)
	{
		// developer override, good for profiling, can be useful as project setting
		{
			const int32 CVarValue = CVarEarlyZPass.GetValueOnAnyThread();

				switch (CVarValue)
				{
				case 0: OutZPassMode = DDM_None; break;
				case 1: OutZPassMode = DDM_NonMaskedOnly; break;
				case 2: OutZPassMode = DDM_AllOccluders; break;
				case 3: break;	// Note: 3 indicates "default behavior" and does not specify an override
				}
		}

		if (ShouldForceFullDepthPass(ShaderPlatform))
		{
			// DBuffer decals and stencil LOD dithering force a full prepass
			const bool bDepthPassCanOutputVelocity = FVelocityRendering::DepthPassCanOutputVelocity(InFeatureLevel);
			OutZPassMode = bDepthPassCanOutputVelocity ? DDM_AllOpaqueNoVelocity : DDM_AllOpaque;
			bOutEarlyZPassMovable = bDepthPassCanOutputVelocity ? false : true;
		}
	}
	else if (GetFeatureLevelShadingPath(InFeatureLevel) == EShadingPath::Mobile)
	{
		OutZPassMode = DDM_None;
				 
		const bool bMaskedOnlyPrePass = FReadOnlyCVARCache::MobileEarlyZPass(ShaderPlatform) == 2;
		if (bMaskedOnlyPrePass)
		{
			OutZPassMode = DDM_MaskedOnly;
		}

		if (MobileUsesFullDepthPrepass(ShaderPlatform))
		{
			const bool bDepthPassCanOutputVelocity = PlatformSupportsVelocityRendering(ShaderPlatform) && FVelocityRendering::DepthPassCanOutputVelocity(InFeatureLevel);
			OutZPassMode = bDepthPassCanOutputVelocity ? DDM_AllOpaqueNoVelocity : DDM_AllOpaque;
		}
	}
}

void FScene::DumpUnbuiltLightInteractions( FOutputDevice& Ar ) const
{
	FlushRenderingCommands();

	TSet<FString> LightsWithUnbuiltInteractions;
	TSet<FString> PrimitivesWithUnbuiltInteractions;

	// if want to print out all of the lights
	for(const FLightSceneInfoCompact& LightCompactInfo : Lights)
	{
		FLightSceneInfo* LightSceneInfo = LightCompactInfo.LightSceneInfo;

		bool bLightHasUnbuiltInteractions = false;

		for(FLightPrimitiveInteraction* Interaction = LightSceneInfo->GetDynamicInteractionOftenMovingPrimitiveList();
			Interaction;
			Interaction = Interaction->GetNextPrimitive())
		{
			if (Interaction->IsUncachedStaticLighting())
			{
				bLightHasUnbuiltInteractions = true;
				PrimitivesWithUnbuiltInteractions.Add(Interaction->GetPrimitiveSceneInfo()->GetComponentForDebugOnly()->GetFullName());
			}
		}

		for(FLightPrimitiveInteraction* Interaction = LightSceneInfo->GetDynamicInteractionStaticPrimitiveList();
			Interaction;
			Interaction = Interaction->GetNextPrimitive())
		{
			if (Interaction->IsUncachedStaticLighting())
			{
				bLightHasUnbuiltInteractions = true;
				PrimitivesWithUnbuiltInteractions.Add(Interaction->GetPrimitiveSceneInfo()->GetComponentForDebugOnly()->GetFullName());
			}
		}

		if (bLightHasUnbuiltInteractions)
		{
			LightsWithUnbuiltInteractions.Add(LightSceneInfo->Proxy->GetOwnerNameOrLabel());
		}
	}

	Ar.Logf( TEXT( "DumpUnbuiltLightIteractions" ) );
	Ar.Logf( TEXT( "Lights with unbuilt interactions: %d" ), LightsWithUnbuiltInteractions.Num() );
	for (auto &LightName : LightsWithUnbuiltInteractions)
	{
		Ar.Logf(TEXT("    Light %s"), *LightName);
	}

	Ar.Logf( TEXT( "" ) );
	Ar.Logf( TEXT( "Primitives with unbuilt interactions: %d" ), PrimitivesWithUnbuiltInteractions.Num() );
	for (auto &PrimitiveName : PrimitivesWithUnbuiltInteractions)
	{
		Ar.Logf(TEXT("    Primitive %s"), *PrimitiveName);
	}
}

/**
 * Exports the scene.
 *
 * @param	Ar		The Archive used for exporting.
 **/
void FScene::Export( FArchive& Ar ) const
{
	
}

void FScene::ApplyWorldOffset(const FVector& InOffset)
{
	UE::RenderCommandPipe::FSyncScope SyncScope;

	// Send a command to the rendering thread to shift scene data
	FScene* Scene = this;
	FVector Offset = InOffset;
	ENQUEUE_RENDER_COMMAND(FApplyWorldOffset)(
		[Scene, Offset](FRHICommandListImmediate& RHICmdList)
		{
			Scene->UpdateAllPrimitiveSceneInfos(RHICmdList);
			Scene->ApplyWorldOffset_RenderThread(RHICmdList, Offset);
		});
}

void FScene::ApplyWorldOffset_RenderThread(FRHICommandListBase& RHICmdList, const FVector& InOffset)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_SceneApplyWorldOffset);
	
	GPUScene.bUpdateAllPrimitives = true;

	// Primitives
	for (int32 Idx = 0; Idx < Primitives.Num(); ++Idx)
	{
		Primitives[Idx]->ApplyWorldOffset(RHICmdList, InOffset);
	}

	// Primitive transforms
	for (int32 Idx = 0; Idx < PrimitiveTransforms.Num(); ++Idx)
	{
		PrimitiveTransforms[Idx].SetOrigin(PrimitiveTransforms[Idx].GetOrigin() + InOffset);
	}

	// Primitive bounds
	for (int32 Idx = 0; Idx < PrimitiveBounds.Num(); ++Idx)
	{
		PrimitiveBounds[Idx].BoxSphereBounds.Origin+= InOffset;
	}

#if RHI_RAYTRACING
	for (auto& BoundsPair : PrimitiveRayTracingGroups)
	{
		BoundsPair.Value.Bounds.Origin += InOffset;
	}
#endif

	// Primitive occlusion bounds
	for (int32 Idx = 0; Idx < PrimitiveOcclusionBounds.Num(); ++Idx)
	{
		PrimitiveOcclusionBounds[Idx].Origin+= InOffset;
	}
	
	// Precomputed light volumes
	for (const FPrecomputedLightVolume* It : PrecomputedLightVolumes)
	{
		const_cast<FPrecomputedLightVolume*>(It)->ApplyWorldOffset(InOffset);
	}

	// Precomputed visibility
	if (PrecomputedVisibilityHandler)
	{
		const_cast<FPrecomputedVisibilityHandler*>(PrecomputedVisibilityHandler)->ApplyWorldOffset(InOffset);
	}
	
	// Invalidate indirect lighting cache
	IndirectLightingCache.SetLightingCacheDirty(this, NULL);

	// Primitives octree
	PrimitiveOctree.ApplyOffset(InOffset, /*bGlobalOctee*/ true);

	// Lights
	VectorRegister OffsetReg = VectorLoadFloat3_W0(&InOffset);
	for(FLightSceneInfoCompact& LightSceneInfoCompact : Lights)
	{
		LightSceneInfoCompact.BoundingSphereVector = VectorAdd(LightSceneInfoCompact.BoundingSphereVector, OffsetReg);
		LightSceneInfoCompact.LightSceneInfo->Proxy->ApplyWorldOffset(InOffset);
	}

	LocalShadowCastingLightOctree.ApplyOffset(InOffset, /*bGlobalOctee*/ true);

	// Cached preshadows
	for (auto It = CachedPreshadows.CreateIterator(); It; ++It)
	{
		(*It)->PreShadowTranslation-= InOffset;
		(*It)->ShadowBounds.Center+= InOffset;
	}

	// Decals
	for (auto It = Decals.CreateIterator(); It; ++It)
	{
		(*It)->ComponentTrans.AddToTranslation(InOffset);
	}

	// Wind sources
	for (auto It = WindSources.CreateIterator(); It; ++It)
	{
		(*It)->ApplyWorldOffset(InOffset);
	}

	// Reflection captures
	for (auto It = ReflectionSceneData.RegisteredReflectionCaptures.CreateIterator(); It; ++It)
	{
		FMatrix NewTransform = FMatrix((*It)->BoxTransform.Inverse().ConcatTranslation((FVector3f)InOffset));
		(*It)->SetTransform(NewTransform);
	}

	// Planar reflections
	for (auto It = PlanarReflections.CreateIterator(); It; ++It)
	{
		(*It)->ApplyWorldOffset(InOffset);
	}
	
	// Exponential Fog
	for (FExponentialHeightFogSceneInfo& FogInfo : ExponentialFogs)
	{
		for (FExponentialHeightFogSceneInfo::FExponentialHeightFogSceneData& FogData : FogInfo.FogData)
		{
			FogData.Height += InOffset.Z;
		}
	}

	// SkyAtmospheres
	for (FSkyAtmosphereSceneProxy* SkyAtmosphereProxy : SkyAtmosphereStack)
	{
		SkyAtmosphereProxy->ApplyWorldOffset((FVector3f)InOffset);
	}
	
	
	VelocityData.ApplyOffset(InOffset);
}

void FScene::OnLevelAddedToWorld(const FName& InLevelAddedName, UWorld* InWorld, bool bIsLightingScenario)
{
	if (bIsLightingScenario)
	{
		InWorld->PropagateLightingScenarioChange();
	}

	FScene* Scene = this;
	FName LevelAddedName = InLevelAddedName;
	ENQUEUE_RENDER_COMMAND(FLevelAddedToWorld)(
		[Scene, LevelAddedName] (FRHICommandListBase&)
		{
			FLevelCommand Cmd;
			Cmd.Name = LevelAddedName;
			Cmd.Op = FLevelCommand::EOp::Add;
			Scene->LevelCommands.Add(Cmd);
		});
}

void FScene::OnLevelRemovedFromWorld(const FName& InLevelRemovedName, UWorld* InWorld, bool bIsLightingScenario)
{
	if (bIsLightingScenario)
	{
		InWorld->PropagateLightingScenarioChange();
	}

	FScene* Scene = this;
	FName LevelRemovedName = InLevelRemovedName;
	ENQUEUE_RENDER_COMMAND(FLevelRemovedFromWorld)(
		[Scene, LevelRemovedName] (FRHICommandListBase&)
		{
			FLevelCommand Cmd;
			Cmd.Name = LevelRemovedName;
			Cmd.Op = FLevelCommand::EOp::Remove;
			Scene->LevelCommands.Add(Cmd);
		});
}

void FScene::ProcessAtmosphereLightAddition_RenderThread(FLightSceneInfo* LightSceneInfo)
{
	if (LightSceneInfo->Proxy->IsUsedAsAtmosphereSunLight())
	{
		const uint8 Index = LightSceneInfo->Proxy->GetAtmosphereSunLightIndex();
		if (!AtmosphereLights[Index] ||																								// Set it if null
			LightSceneInfo->Proxy->GetColor().GetLuminance() > AtmosphereLights[Index]->Proxy->GetColor().GetLuminance())	// Or choose the brightest sun light
		{
			AtmosphereLights[Index] = LightSceneInfo;
		}
	}
}

void FScene::ProcessAtmosphereLightRemoval_RenderThread(FLightSceneInfo* LightSceneInfo)
{
	// When a light has its intensity or index changed, it will be removed first, then re-added. So we only need to check the index of the removed light.
	const uint8 Index = LightSceneInfo->Proxy->GetAtmosphereSunLightIndex();
	if (AtmosphereLights[Index] == LightSceneInfo)
	{
		AtmosphereLights[Index] = nullptr;
		float SelectedLightLuminance = 0.0f;

		for (const FLightSceneInfoCompact& LightInfo : Lights)
		{
			float LightLuminance = LightInfo.LightSceneInfo->Proxy->GetColor().GetLuminance();

			if (LightInfo.LightSceneInfo != LightSceneInfo
				&& LightInfo.LightSceneInfo->Proxy->IsUsedAsAtmosphereSunLight() && LightInfo.LightSceneInfo->Proxy->GetAtmosphereSunLightIndex() == Index
				&& (!AtmosphereLights[Index] || SelectedLightLuminance < LightLuminance))
			{
				AtmosphereLights[Index] = LightInfo.LightSceneInfo;
				SelectedLightLuminance = LightLuminance;
			}
		}
	}
}

#if WITH_EDITOR
bool FScene::InitializePixelInspector(FRenderTarget* BufferFinalColor, FRenderTarget* BufferSceneColor, FRenderTarget* BufferDepth, FRenderTarget* BufferHDR, FRenderTarget* BufferA, FRenderTarget* BufferBCDEF, int32 BufferIndex)
{
	//Initialize the buffers
	PixelInspectorData.InitializeBuffers(BufferFinalColor, BufferSceneColor, BufferDepth, BufferHDR, BufferA, BufferBCDEF, BufferIndex);
	//return true when the interface is implemented
	return true;
}

bool FScene::AddPixelInspectorRequest(FPixelInspectorRequest *PixelInspectorRequest)
{
	return PixelInspectorData.AddPixelInspectorRequest(PixelInspectorRequest);
}
#endif //WITH_EDITOR


struct FPrimitiveArraySortKey
{
	inline bool operator()(const FPrimitiveSceneInfo& A, const FPrimitiveSceneInfo& B) const
	{
		const uint32 A_TypeHash = A.Proxy->GetTypeHash();
		const uint32 B_TypeHash = B.Proxy->GetTypeHash();

		const uint32 A_AlwaysVisible = A.Proxy->IsAlwaysVisible() ? 1u : 0u;
		const uint32 B_AlwaysVisible = B.Proxy->IsAlwaysVisible() ? 1u : 0u;

		// First group all proxies by test visibility vs. always visible (at the end)
		if (A_AlwaysVisible != B_AlwaysVisible)
		{
			return A_AlwaysVisible > B_AlwaysVisible;
		}

		// Then group up all proxies in the two ranges by type for better cache coherency
		if (A_TypeHash != B_TypeHash)
		{
			return A_TypeHash > B_TypeHash;
		}

		// Finally, sort by primitive component ID to add more determinism/stability to the sort
		return A.PrimitiveComponentId.PrimIDValue > B.PrimitiveComponentId.PrimIDValue;
	}
};

static bool ShouldPrimitiveOutputVelocity(const FPrimitiveSceneProxy* Proxy, const FStaticShaderPlatform ShaderPlatform)
{
	bool bShouldPrimitiveOutputVelocity = Proxy->HasDynamicTransform();

	bool bPlatformSupportsVelocityRendering = PlatformSupportsVelocityRendering(ShaderPlatform);

	return bPlatformSupportsVelocityRendering && bShouldPrimitiveOutputVelocity;
}

void FScene::UpdatePrimitiveVelocityState_RenderThread(FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bIsBeingMoved)
{
	if (bIsBeingMoved)
	{
		if (ShouldPrimitiveOutputVelocity(PrimitiveSceneInfo->Proxy, GetShaderPlatform()))
		{
			if (PrimitiveSceneInfo->IsIndexValid())
			{
				PrimitiveSceneInfo->bRegisteredWithVelocityData = true;
				// We must register the initial LocalToWorld with the velocity state. 
				int32 PrimitiveIndex = PrimitiveSceneInfo->PackedIndex;
				VelocityData.UpdateTransform(PrimitiveSceneInfo, PrimitiveTransforms[PrimitiveIndex], PrimitiveTransforms[PrimitiveIndex]);
			}
		}
	}
	else if (PrimitiveSceneInfo->bRegisteredWithVelocityData)
	{
		PrimitiveSceneInfo->bRegisteredWithVelocityData = false;
		VelocityData.RemoveFromScene(PrimitiveSceneInfo->PrimitiveComponentId, true);
	}
}

#if RHI_RAYTRACING

void FScene::UpdateRayTracingGroupBounds_AddPrimitives(const TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>& PrimitiveSceneInfos)
{
	for (FPrimitiveSceneInfo* const PrimitiveSceneInfo : PrimitiveSceneInfos)
	{
		const int32 GroupId = PrimitiveSceneInfo->Proxy->GetRayTracingGroupId();
		if (GroupId != -1)
		{
			bool bInMap = false;
			static const FRayTracingCullingGroup DefaultGroup = {};
			FRayTracingCullingGroup* const Group = PrimitiveRayTracingGroups.FindOrAdd(GroupId, DefaultGroup, bInMap);
			if (bInMap)
			{
				Group->Bounds = Group->Bounds + PrimitiveSceneInfo->Proxy->GetBounds();
				Group->MinDrawDistance = FMath::Max(Group->MinDrawDistance, PrimitiveSceneInfo->Proxy->GetMinDrawDistance());
			}
			else
			{
				Group->Bounds = PrimitiveSceneInfo->Proxy->GetBounds();
				Group->MinDrawDistance = PrimitiveSceneInfo->Proxy->GetMinDrawDistance();
			}
			Group->Primitives.Add(PrimitiveSceneInfo);
		}
	}
}

static void UpdateRayTracingGroupBounds(Experimental::TRobinHoodHashSet<FScene::FRayTracingCullingGroup*>& GroupsToUpdate)
{
	for (FScene::FRayTracingCullingGroup* const Group : GroupsToUpdate)
	{
		bool bFirstBounds = false;
		for (FPrimitiveSceneInfo* const Primitive : Group->Primitives)
		{
			if (!bFirstBounds)
			{
				Group->Bounds = Primitive->Proxy->GetBounds();
				bFirstBounds = true;
			}
			else
			{
				Group->Bounds = Group->Bounds + Primitive->Proxy->GetBounds();
			}
		}
	}
}

void FScene::UpdateRayTracingGroupBounds_RemovePrimitives(const TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>& PrimitiveSceneInfos)
{
	Experimental::TRobinHoodHashSet<FRayTracingCullingGroup*> GroupsToUpdate;
	for (FPrimitiveSceneInfo* const PrimitiveSceneInfo : PrimitiveSceneInfos)
	{
		const int32 RayTracingGroupId = PrimitiveSceneInfo->Proxy->GetRayTracingGroupId();
		const Experimental::FHashElementId GroupId = (RayTracingGroupId != -1) ? PrimitiveRayTracingGroups.FindId(RayTracingGroupId) : Experimental::FHashElementId();
		if (GroupId.IsValid())
		{
			FRayTracingCullingGroup& Group = PrimitiveRayTracingGroups.GetByElementId(GroupId).Value;
			Group.Primitives.RemoveSingleSwap(PrimitiveSceneInfo);
			if (Group.Primitives.Num() == 0)
			{
				PrimitiveRayTracingGroups.RemoveByElementId(GroupId);
			}
			else
			{
				GroupsToUpdate.FindOrAdd(&Group);
			}
		}
	}

	UpdateRayTracingGroupBounds(GroupsToUpdate);
}

template<typename RangeType>
inline void FScene::UpdateRayTracingGroupBounds_UpdatePrimitives(const RangeType& InUpdatedTransforms)
{
	Experimental::TRobinHoodHashSet<FRayTracingCullingGroup*> GroupsToUpdate;
	for (const auto& Transform : InUpdatedTransforms)
	{
		FPrimitiveSceneProxy* const PrimitiveSceneProxy = Transform.SceneInfo->Proxy;
		const int32 RayTracingGroupId = PrimitiveSceneProxy->GetRayTracingGroupId();
		const Experimental::FHashElementId GroupId = (RayTracingGroupId != -1) ? PrimitiveRayTracingGroups.FindId(RayTracingGroupId) : Experimental::FHashElementId();
		if (GroupId.IsValid())
		{
			FRayTracingCullingGroup& Group = PrimitiveRayTracingGroups.GetByElementId(GroupId).Value;
			GroupsToUpdate.FindOrAdd(&Group);
		}
	}

	UpdateRayTracingGroupBounds(GroupsToUpdate);
}
#endif

void FScene::UpdateLights(FRDGBuilder& GraphBuilder, FSceneExtensionsUpdaters& SceneExtensionsUpdaters)
{
	SCOPED_NAMED_EVENT(FScene_UpdateAllLightSceneInfos, FColor::Orange);
	struct FFLightSceneChangeSetAllocation
	{
		TArray<FLightSceneId, SceneRenderingAllocator> RemovedLightIds;
		TArray<FLightSceneId, SceneRenderingAllocator> AddedLightIds;
		FFLightSceneChangeSetAllocation(FSceneLightInfoUpdates&& InSceneLightInfoUpdates, int32 InPreUpdateMaxIndex) 
			: SceneLightInfoUpdates(MoveTemp(InSceneLightInfoUpdates)) 
			, RemovedLightsMask(false, InPreUpdateMaxIndex)
			, PreUpdateChangeSet{TConstArrayView<FLightSceneId>(RemovedLightIds), TConstArrayView<FLightSceneId>(), SceneLightInfoUpdates, RemovedLightsMask, InPreUpdateMaxIndex, -1 }
			, PostUpdateChangeSet{TConstArrayView<FLightSceneId>(RemovedLightIds), TConstArrayView<FLightSceneId>(AddedLightIds), SceneLightInfoUpdates, RemovedLightsMask, InPreUpdateMaxIndex, -1 }
		{}
		FSceneLightInfoUpdates SceneLightInfoUpdates;
		FSceneRendererLightBitArray RemovedLightsMask;

		// These are the accessors to the data that are read only and can be passed around by pointer or reference as they are kept alive in the alloc.
		FLightSceneChangeSet PreUpdateChangeSet;
		FLightSceneChangeSet PostUpdateChangeSet;
	};
	// Allocate change set storage with graph builder lifetime such that we can safely pass it to async tasks.
	FFLightSceneChangeSetAllocation& ChangeSetAlloc = *GraphBuilder.AllocObject<FFLightSceneChangeSetAllocation>(MoveTemp(*SceneLightInfoUpdates), Lights.GetMaxIndex());
#if DO_CHECK
	FSceneLightInfoUpdates::FReadAccessScope ReadAccessScope(ChangeSetAlloc.SceneLightInfoUpdates);
#endif

	// Filter out removes & never-adds
	ChangeSetAlloc.SceneLightInfoUpdates.ForEachCommand([&](const FUpdateLightCommand& UpdateLightCommand)
	{
		FLightSceneId Id = UpdateLightCommand.GetSceneInfo()->Id;
		if (UpdateLightCommand.IsDelete())
		{
			if (Id.IsValid())
			{
				ChangeSetAlloc.RemovedLightIds.Add(Id);
				ChangeSetAlloc.RemovedLightsMask[Id] = true;
			}
			else
			{
				delete UpdateLightCommand.GetSceneInfo()->Proxy;
				delete UpdateLightCommand.GetSceneInfo();
			}
		}
	});
	ChangeSetAlloc.PreUpdateChangeSet.RemovedLightIds = ChangeSetAlloc.RemovedLightIds;
	ChangeSetAlloc.PostUpdateChangeSet.RemovedLightIds = ChangeSetAlloc.RemovedLightIds;

	SceneExtensionsUpdaters.PreLightsUpdate(GraphBuilder, ChangeSetAlloc.PreUpdateChangeSet);
	// This can't access the scene light data if done async since it happens before the actual removals.
	OnPreLightSceneInfoUpdate.Broadcast(GraphBuilder, ChangeSetAlloc.PreUpdateChangeSet);
	// Batch process all light removes
	for (auto LightId : ChangeSetAlloc.RemovedLightIds)
	{
		FLightSceneInfo* LightSceneInfo = Lights[LightId].LightSceneInfo;
		FScopeCycleCounter Context(LightSceneInfo->Proxy->GetStatId());
		RemoveLightSceneInfo_RenderThread(LightSceneInfo);
	}

	// Process all light adds & updates
	ChangeSetAlloc.SceneLightInfoUpdates.ForEachCommand(ESceneUpdateCommandFilter::AddedUpdated, [&](const FUpdateLightCommand& UpdateLightCommand)
	{
		FLightSceneInfo* LightSceneInfo = UpdateLightCommand.GetSceneInfo();
		FScopeCycleCounter Context(LightSceneInfo->Proxy->GetStatId());

		const FLightSceneId Id = LightSceneInfo->Id;
		const bool bHasId = Id.IsValid();
		check(bHasId == !UpdateLightCommand.IsAdd());
		// Directly process updates.
		if (FUpdateLightTransformParameters* TransformParameters = ChangeSetAlloc.SceneLightInfoUpdates.GetPayloadPtr<FUpdateLightTransformParameters>(UpdateLightCommand))
		{
			SCOPE_CYCLE_COUNTER(STAT_UpdateSceneLightTime);

			UpdateLightTransform_RenderThread(Id, LightSceneInfo, *TransformParameters);
		}

		if (FUpdateLightColorParameters* ColorParameters = ChangeSetAlloc.SceneLightInfoUpdates.GetPayloadPtr<FUpdateLightColorParameters>(UpdateLightCommand))
		{
			const FUpdateLightColorParameters& NewParameters = *ColorParameters;
			SCOPE_CYCLE_COUNTER(STAT_UpdateSceneLightTime);

			// Mobile renderer:
			// a light with no color/intensity can cause the light to be ignored when rendering.
			// thus, lights that change state in this way must update the draw lists.
			if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Mobile 
				&& LightSceneInfo->Proxy->GetLightType() == LightType_Directional 
				&& NewParameters.NewColor.IsAlmostBlack() != LightSceneInfo->Proxy->GetColor().IsAlmostBlack())
			{
				bScenesPrimitivesNeedStaticMeshElementUpdate = true;
				UE_CLOGF(!GIsEditor, LogRenderer, Log, "Forcing update for all mesh draw commands: Toggle directional light");
			}

			// Path Tracing: something about the light has changed, restart path traced accumulation
			InvalidatePathTracedOutput();

			LightSceneInfo->Proxy->SetColor(NewParameters.NewColor);
			LightSceneInfo->Proxy->IndirectLightingScale = NewParameters.NewIndirectLightingScale;
			LightSceneInfo->Proxy->VolumetricScatteringIntensity = NewParameters.NewVolumetricScatteringIntensity;

			// Also update the LightSceneInfoCompact (if it does not have an ID, it is being added)
			if (bHasId)
			{
				Lights[Id].Color = NewParameters.NewColor;
			}
		}

		// Perform Add after update, since that reduces redundant processing (e.g., Add + Move)
		if (UpdateLightCommand.IsAdd())
		{
			AddLightSceneInfo_RenderThread(LightSceneInfo);
			// Note: Id is set in AddLightSceneInfo_RenderThread so we must fetch it again
			ChangeSetAlloc.AddedLightIds.Add(LightSceneInfo->Id);
		}
	});
	ChangeSetAlloc.PostUpdateChangeSet.PostUpdateMaxIndex = Lights.GetMaxIndex();
	ChangeSetAlloc.PostUpdateChangeSet.AddedLightIds = ChangeSetAlloc.AddedLightIds;
	SceneExtensionsUpdaters.PostLightsUpdate(GraphBuilder, ChangeSetAlloc.PostUpdateChangeSet);
	OnPostLightSceneInfoUpdate.Broadcast(GraphBuilder, ChangeSetAlloc.PostUpdateChangeSet);
	GPUScene.OnPostLightSceneInfoUpdate(GraphBuilder, ChangeSetAlloc.PostUpdateChangeSet);
}

extern TAutoConsoleVariable<float> CVarReflectionCaptureRuntimeFadeInTime;
extern TAutoConsoleVariable<float> CVarReflectionCaptureRuntimeSmoothBlendTime;

template<class T>
static void CreateReflectionCaptureUniformBuffer(const TArray<FReflectionCaptureSortData>& SortedCaptures, TUniformBufferRef<T>& OutReflectionCaptureUniformBuffer)
{
	T SamplePositionsBuffer;
	for (int32 CaptureIndex = 0; CaptureIndex < SortedCaptures.Num(); CaptureIndex++)
	{
		SamplePositionsBuffer.PositionHighAndRadius[CaptureIndex] = FVector4f(SortedCaptures[CaptureIndex].Position.High, SortedCaptures[CaptureIndex].Radius);

		// PositionLow.w carries the packed refresh-blend state: integer = source cubemap array index, fractional = alpha.
		// A value of 0 means no active blend; the shader skips the cross-fade with a single compare.
		SamplePositionsBuffer.PositionLow[CaptureIndex] = FVector4f(SortedCaptures[CaptureIndex].Position.Low, SortedCaptures[CaptureIndex].SmoothBlendEncoded);

		SamplePositionsBuffer.CaptureProperties[CaptureIndex] = SortedCaptures[CaptureIndex].CaptureProperties;
		SamplePositionsBuffer.CaptureOffsetAndAverageBrightness[CaptureIndex] = SortedCaptures[CaptureIndex].CaptureOffsetAndAverageBrightness;
		SamplePositionsBuffer.BoxTransform[CaptureIndex] = SortedCaptures[CaptureIndex].BoxTransform;
		SamplePositionsBuffer.BoxScales[CaptureIndex] = SortedCaptures[CaptureIndex].BoxScales;
	}

	OutReflectionCaptureUniformBuffer = TUniformBufferRef<T>::CreateUniformBufferImmediate(SamplePositionsBuffer, UniformBuffer_MultiFrame);
}

void UpdateReflectionSceneData(FScene* Scene)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateReflectionSceneData)

	FReflectionEnvironmentSceneData& ReflectionSceneData = Scene->ReflectionSceneData;

	ReflectionSceneData.SortedCaptures.Reset(ReflectionSceneData.RegisteredReflectionCaptures.Num());
	ReflectionSceneData.NumBoxCaptures = 0;
	ReflectionSceneData.NumSphereCaptures = 0;

	const int32 MaxCubemaps = ReflectionSceneData.CubemapArray.GetMaxRegularCubemaps();
	int32_t PlatformMaxNumReflectionCaptures = FMath::Min(FMath::FloorToInt(GMaxTextureArrayLayers / 6.0f), GetMaxNumReflectionCaptures(Scene->GetShaderPlatform()));

	// Blend-in parameters for runtime captures.  When any capture is mid-fade we need to rebuild the uniform buffer
	// each frame so its CaptureProperties.a value advances even when nothing else has changed.
	const float FadeInDuration = FMath::Max(0.0f, CVarReflectionCaptureRuntimeFadeInTime.GetValueOnRenderThread());
	const double CurrentTime = FApp::GetCurrentTime();
	bool bAnyBlendInProgress = false;

	// Sweep refresh-blend slots: free any whose cross-fade has completed.  A slot's BlendStartTime is stamped by
	// StampRefreshBlendStartTime once the new render lands; before that the slot stays pinned (BlendStartTime < 0)
	// and the shader keeps the source at full strength.
	const float RefreshBlendDuration = FMath::Max(0.0f, CVarReflectionCaptureRuntimeSmoothBlendTime.GetValueOnRenderThread());
	for (FReflectionEnvironmentSceneData::FSmoothBlendEntry& Entry : ReflectionSceneData.RuntimeSmoothBlendSlots)
	{
		if (Entry.OwningComponent != nullptr && Entry.BlendStartTime >= 0.0)
		{
			const float Elapsed = (float)(CurrentTime - Entry.BlendStartTime);
			if (RefreshBlendDuration <= 0.0f || Elapsed >= RefreshBlendDuration)
			{
				Entry = {};
			}
		}
	}

	// Pack visible reflection captures into the uniform buffer, each with an index to its cubemap array entry.
	// GPUScene primitive data stores closest reflection capture as index into this buffer, so this index which must be invalidate every time OutSortData contents change.
	for (int32 ReflectionProxyIndex = 0; ReflectionProxyIndex < ReflectionSceneData.RegisteredReflectionCaptures.Num() && ReflectionSceneData.SortedCaptures.Num() < PlatformMaxNumReflectionCaptures; ReflectionProxyIndex++)
	{
		FReflectionCaptureProxy* CurrentCapture = ReflectionSceneData.RegisteredReflectionCaptures[ReflectionProxyIndex];

		FReflectionCaptureSortData NewSortEntry;

		NewSortEntry.CubemapIndex = -1;
		NewSortEntry.FadeAlpha = 1.0f;
		NewSortEntry.CaptureOffsetAndAverageBrightness = FVector4f(CurrentCapture->CaptureOffset, 1.0f);
		NewSortEntry.CaptureProxy = CurrentCapture;
		if (SupportsTextureCubeArray(Scene->GetFeatureLevel()))
		{
			FCaptureComponentSceneState* ComponentStatePtr = ReflectionSceneData.AllocatedReflectionCaptureState.Find(CurrentCapture->Component);
			if (!ComponentStatePtr)
			{
				// Skip reflection captures without built data to upload
				continue;
			}

			// Skip captures that haven't finished rendering for the current activation, to avoid sampling a stale cubemap slot.
			if (!ComponentStatePtr->bRenderedForShading)
			{
				continue;
			}

			NewSortEntry.CubemapIndex = ComponentStatePtr->CubemapIndex;
			check(NewSortEntry.CubemapIndex < MaxCubemaps || NewSortEntry.CubemapIndex == 0);
			NewSortEntry.CaptureOffsetAndAverageBrightness.W = ComponentStatePtr->AverageBrightness;

			// Match this capture against any active refresh-blend slot and pack (source cube index, alpha) into a
			// single float for PositionLow.w: integer part = SourceCubemapArrayIndex, fractional part = blend alpha.
			// Alpha is clamped strictly below 1 so floor() in the shader recovers the cube index even when the
			// fade has just started.  Setting any non-zero value forces a UB rebuild so the alpha advances per frame.
			for (int32 SlotIdx = 0; SlotIdx < ReflectionSceneData.RuntimeSmoothBlendSlots.Num(); SlotIdx++)
			{
				const FReflectionEnvironmentSceneData::FSmoothBlendEntry& Slot = ReflectionSceneData.RuntimeSmoothBlendSlots[SlotIdx];
				if (Slot.OwningComponent == CurrentCapture->Component && Slot.SourceCubemapArrayIndex >= 0)
				{
					float BlendAlpha = 1.0f;
					if (Slot.BlendStartTime >= 0.0)
					{
						BlendAlpha = (RefreshBlendDuration > 0.0f)
							? 1.0f - FMath::Clamp((float)(CurrentTime - Slot.BlendStartTime) / RefreshBlendDuration, 0.0f, 1.0f)
							: 0.0f;
					}
					BlendAlpha = FMath::Min(BlendAlpha, 0.99999f);
					NewSortEntry.SmoothBlendEncoded = (float)Slot.SourceCubemapArrayIndex + BlendAlpha;
					bAnyBlendInProgress = true;
					break;
				}
			}

			// Compute fade alpha from the (Start, Target, StartTime) triple.  Non-steady state forces a uniform buffer rebuild
			// so CaptureProperties.a advances each frame.  Once Alpha reaches Target we latch Start = Target so the next frame
			// can skip the rebuild if no fades are active.
			const float Alpha = ComponentStatePtr->ComputeCurrentFade(CurrentTime, FadeInDuration);
			NewSortEntry.FadeAlpha = Alpha;
			if (ComponentStatePtr->FadeStartValue != ComponentStatePtr->FadeTargetValue)
			{
				bAnyBlendInProgress = true;
				if (Alpha == ComponentStatePtr->FadeTargetValue)
				{
					ComponentStatePtr->FadeStartValue = ComponentStatePtr->FadeTargetValue;
				}
			}
		}

		NewSortEntry.Guid = CurrentCapture->Guid;
		NewSortEntry.Position = CurrentCapture->Position;
		NewSortEntry.Radius = CurrentCapture->InfluenceRadius;
		float ShapeTypeValue = (float)CurrentCapture->Shape;
		NewSortEntry.CaptureProperties = FVector4f(CurrentCapture->Brightness, NewSortEntry.CubemapIndex, ShapeTypeValue, NewSortEntry.FadeAlpha);

		if (CurrentCapture->Shape == EReflectionCaptureShape::Plane)
		{
			//planes count as boxes in the compute shader.
			++ReflectionSceneData.NumBoxCaptures;
			NewSortEntry.BoxTransform = FMatrix44f(
				FPlane4f(CurrentCapture->LocalReflectionPlane),
				FPlane4f((FVector4f)CurrentCapture->ReflectionXAxisAndYScale), // LWC_TODO: precision loss
				FPlane4f(0, 0, 0, 0),
				FPlane4f(0, 0, 0, 0));

			NewSortEntry.BoxScales = FVector4f(0);
		}
		else if (CurrentCapture->Shape == EReflectionCaptureShape::Sphere)
		{
			++ReflectionSceneData.NumSphereCaptures;
		}
		else
		{
			++ReflectionSceneData.NumBoxCaptures;
			NewSortEntry.BoxTransform = CurrentCapture->BoxTransform;
			NewSortEntry.BoxScales = FVector4f(CurrentCapture->BoxScales, CurrentCapture->BoxTransitionDistance);
		}

		ReflectionSceneData.SortedCaptures.Add(NewSortEntry);
	}

	ReflectionSceneData.SortedCaptures.Sort();

	for (int32 CaptureIndex = 0; CaptureIndex < ReflectionSceneData.SortedCaptures.Num(); CaptureIndex++)
	{
		ReflectionSceneData.SortedCaptures[CaptureIndex].CaptureProxy->SortedCaptureIndex = CaptureIndex;
	}

	// Create uniform buffers with sorted captures.  Also rebuild while any capture is mid blend-in, since
	// CaptureProperties.a changes every frame until the fade completes.
	if (ReflectionSceneData.bRegisteredReflectionCapturesHasChanged ||
		ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged ||
		bAnyBlendInProgress)
	{
		ReflectionSceneData.ReflectionCaptureUniformBuffer.SafeRelease();
		ReflectionSceneData.MobileReflectionCaptureUniformBuffer.SafeRelease();

		if (IsMobilePlatform(Scene->GetShaderPlatform()))
		{
			CreateReflectionCaptureUniformBuffer(ReflectionSceneData.SortedCaptures, ReflectionSceneData.MobileReflectionCaptureUniformBuffer);
		}
		else
		{
			CreateReflectionCaptureUniformBuffer(ReflectionSceneData.SortedCaptures, ReflectionSceneData.ReflectionCaptureUniformBuffer);
		}
	}

	// If SortedCaptures change, then in case of forward renderer all scene primitives need to be updated, as they 
	// store index into sorted reflection capture uniform buffer for the forward renderer.
	if (ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged)
	{
		if (IsForwardShadingEnabled(Scene->GetShaderPlatform()))
		{
			const int32 NumPrimitives = Scene->Primitives.Num();
			for (int32 PrimitiveIndex = 0; PrimitiveIndex < NumPrimitives; ++PrimitiveIndex)
			{
				Scene->PrimitivesNeedingUniformBufferUpdate[PrimitiveIndex] = true;
			}

			Scene->GPUScene.bUpdateAllPrimitives = true;
		}
		ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged = false;
	}

	// Mark all primitives for reflection proxy update
	if (Scene->ShouldCacheReflectionCaptures())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_MarkAllPrimitivesForReflectionProxyUpdate);

		// Mobile needs to re-cache all mesh commands when scene capture data has changed
		const bool bPlatformNeedsStaticMeshUpdate = GetFeatureLevelShadingPath(Scene->GetFeatureLevel()) == EShadingPath::Mobile;

		if (ReflectionSceneData.bRegisteredReflectionCapturesHasChanged)
		{
			// Check if the sorted capture list changed.  If it did, we need to update all primitives that point to a reflection capture.
			bool bSortedCapturesChanged = false;
			if (ReflectionSceneData.SortedCaptures.Num() != ReflectionSceneData.PreChangeSortOrder.Num())
			{
				bSortedCapturesChanged = true;
			}
			else
			{
				for (int32 CaptureIndex = 0; CaptureIndex < ReflectionSceneData.SortedCaptures.Num(); CaptureIndex++)
				{
					if (ReflectionSceneData.SortedCaptures[CaptureIndex] != ReflectionSceneData.PreChangeSortOrder[CaptureIndex])
					{
						bSortedCapturesChanged = true;
						break;
					}
				}
			}
			ReflectionSceneData.PreChangeSortOrder.Empty();

			// Mark all primitives as needing an update, that are affected by a capture that was updated.
			for (FPrimitiveSceneInfo* Primitive : Scene->Primitives)
			{
				if (bPlatformNeedsStaticMeshUpdate)
				{
					// If we have been flagged as requiring update, track that we need an update, even if CacheReflectionCaptures (which clears this flag)
					// returns false.  This handles the case where a primitive pointed to a modified reflection capture, but no longer does.  In that case,
					// CacheReflectionCaptures() will return false, since the capture pointers will already have been nulled out, and won't change.  We
					// need to generate new draw calls in that case, that don't use the reflection capture.
					bool bNeedsUpdate = Primitive->bNeedsCachedReflectionCaptureUpdate;

					// We need to call "CacheReflectionCaptures" for every primitive, even ones not flagged as affected by a changed capture, because it's
					// possible a change to the size or location of a capture causes the primitive to now intersect it.
					if (Primitive->CacheReflectionCaptures())
					{
						bNeedsUpdate = true;
					}

					// Finally, if the sorted captures changed, any primitive pointing to a capture whatsoever must be updated, since the sort indices
					// will have changed globally.
					if (bSortedCapturesChanged && Primitive->CachedReflectionCaptureProxy)
					{
						bNeedsUpdate = true;
					}

					if (bNeedsUpdate)
					{
						Primitive->RequestStaticMeshUpdate();
					}
				}
				else
				{
					// For platforms that don't need caching, remove the cache.
					Primitive->RemoveCachedReflectionCapture(nullptr);
					Primitive->RemoveCachedPlanarReflection(nullptr);
				}
			}

			ReflectionSceneData.bRegisteredReflectionCapturesHasChanged = false;
		}
		
		if (ReflectionSceneData.bSkylightHasChanged)
		{
			if (bPlatformNeedsStaticMeshUpdate)
			{
				// Mark all primitives as needing an update
				// Note: Only visible primitives will actually update their reflection proxy
				const int32 NumPrimitives = Scene->Primitives.Num();
				for (int32 PrimitiveIndex = 0; PrimitiveIndex < NumPrimitives; PrimitiveIndex++)
				{
					Scene->Primitives[PrimitiveIndex]->RequestStaticMeshUpdate();
				}
			}
			ReflectionSceneData.bSkylightHasChanged = false;
		}
	}
}

/**
 * Container for scene change set, that can live on beyond the update function. Note that the values are not safe to interpret at all points 
 * (e.g., a persistent ID of a removed item is not valid after the remove phase), but the arrays are valid as long as the RDG lives.
 */
struct FSceneUpdateChangeSetStorage
{

	FSceneUpdateChangeSetStorage()
		: PreUpdateChangeSet(FScenePreUpdateChangeSet {
			TConstArrayView<FPersistentPrimitiveIndex>(RemovedPrimitiveIds),
			TConstArrayView<FPrimitiveSceneInfo*>(RemovedPrimitiveSceneInfos),
			PrimitiveUpdates
			})
		, PostUpdateChangeSet(FScenePostUpdateChangeSet {
			TConstArrayView<FPersistentPrimitiveIndex>(AddedPrimitiveIds),
			TConstArrayView<FPrimitiveSceneInfo*>(AddedPrimitiveSceneInfos),
			PrimitiveUpdates
		})
	{
	}
	FSceneUpdateChangeSetStorage(FScenePrimitiveUpdates&& InPrimitiveUpdates, const FViewSceneChangeSet* InViewUpdateChangeSet)
		: PrimitiveUpdates(MoveTemp(InPrimitiveUpdates))
		, ViewUpdateChangeSet(InViewUpdateChangeSet)
		, PreUpdateChangeSet(FScenePreUpdateChangeSet {
			TConstArrayView<FPersistentPrimitiveIndex>(RemovedPrimitiveIds),
			TConstArrayView<FPrimitiveSceneInfo*>(RemovedPrimitiveSceneInfos),
			PrimitiveUpdates
			})
		, PostUpdateChangeSet(FScenePostUpdateChangeSet {
			TConstArrayView<FPersistentPrimitiveIndex>(AddedPrimitiveIds),
			TConstArrayView<FPrimitiveSceneInfo*>(AddedPrimitiveSceneInfos),
			PrimitiveUpdates
		})
	{
	}

	TArray<FPersistentPrimitiveIndex, SceneRenderingAllocator> RemovedPrimitiveIds;
	TArray<FPrimitiveSceneInfo*,SceneRenderingAllocator> RemovedPrimitiveSceneInfos;

	TArray<FPersistentPrimitiveIndex, SceneRenderingAllocator> AddedPrimitiveIds;
	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> AddedPrimitiveSceneInfos;
	FScenePrimitiveUpdates PrimitiveUpdates;
	const FViewSceneChangeSet* ViewUpdateChangeSet = nullptr;

	FScenePreUpdateChangeSet PreUpdateChangeSet;
	FScenePostUpdateChangeSet PostUpdateChangeSet;

	const FScenePreUpdateChangeSet& GetPreUpdateSet() 
	{
		PreUpdateChangeSet.RemovedPrimitiveIds = TConstArrayView<FPersistentPrimitiveIndex>(RemovedPrimitiveIds);
		PreUpdateChangeSet.RemovedPrimitiveSceneInfos =	TConstArrayView<FPrimitiveSceneInfo*>(RemovedPrimitiveSceneInfos);
		PreUpdateChangeSet.ViewUpdateChangeSet = ViewUpdateChangeSet;
		return PreUpdateChangeSet;
	}

	const FScenePostUpdateChangeSet& GetPostUpdateSet() 
	{
		PostUpdateChangeSet.AddedPrimitiveIds = TConstArrayView<FPersistentPrimitiveIndex>(AddedPrimitiveIds);
		PostUpdateChangeSet.AddedPrimitiveSceneInfos = TConstArrayView<FPrimitiveSceneInfo*>(AddedPrimitiveSceneInfos);
		PostUpdateChangeSet.ViewUpdateChangeSet = ViewUpdateChangeSet;
		return PostUpdateChangeSet;
	}
};

void FScene::UpdateAllPrimitiveSceneInfos(FRDGBuilder& GraphBuilder, EUpdateAllPrimitiveSceneInfosAsyncOps AsyncOps)
{
	FUpdateParameters Parameters;
	Parameters.AsyncOps = AsyncOps;
	Update(GraphBuilder, Parameters);
}

bool FScene::CanSampleSkyLightRealTimeCaptureData() const
{
	// We need a sky light, with bRealTimeCaptureEnabled (only true if supported by the platform settings) and if the captured data is ready.
	return SkyLight && SkyLight->bRealTimeCaptureEnabled && ConvolvedSkyRenderTargetReadyIndex >= 0;
}

FViewSceneChangeSet* FScene::ProcessViewChanges(FRDGBuilder& GraphBuilder, TConstArrayView<FViewInfo*> Views)
{
	FViewSceneChangeSet* ChangeSet = GraphBuilder.AllocObject<FViewSceneChangeSet>();
	ChangeSet->AddedViewIds.Reserve(Views.Num());
	ChangeSet->RemovedViewIds.Reserve(PersistentViewStateUniqueIDs.Num());

	uint64 NewViewStateDebugFlags = PersistentViewStateDebugFlags;

	// deferred cleanup of any view states that are no longer in the set of allocated IDs.
	for (auto It = PersistentViewStateUniqueIDs.CreateIterator(); It; ++It)
	{
		int32 ViewStateUniqueId = (*It);
		if (!FSceneViewState_ActiveUniqueIDs.Contains(ViewStateUniqueId))
		{
			ChangeSet->RemovedViewIds.Add(FPersistentViewId { It.GetIndex() });
			It.RemoveCurrent();

			NewViewStateDebugFlags &= ~(1ull << It.GetIndex());
		}
	}

	// helper to find existing persistent ID
	auto FindExistingPersistentId = [&](int32 InViewStateUniqueId)
	{
		for (auto It = PersistentViewStateUniqueIDs.CreateConstIterator(); It; ++It)
		{
			int32 ViewStateUniqueId = (*It);
			if (ViewStateUniqueId == InViewStateUniqueId)
			{
				return FPersistentViewId { It.GetIndex() };
			}
		}
		return FPersistentViewId{};
	};


	for (FViewInfo* View : Views)
	{
		FPersistentViewId PersistentViewId;
		if (View->ViewState != nullptr)
		{
			const int32 ViewStateUniqueId = View->ViewState->UniqueID;
			check(FSceneViewState_ActiveUniqueIDs.Contains(ViewStateUniqueId));
			PersistentViewId = FindExistingPersistentId(ViewStateUniqueId);

			if (!PersistentViewId.IsValid())
			{
				int32 StartIndex = 0;
				PersistentViewId = FPersistentViewId { PersistentViewStateUniqueIDs.EmplaceAtLowestFreeIndex(StartIndex, ViewStateUniqueId) };
				ChangeSet->AddedViewIds.Emplace(PersistentViewId); 
			}
			
			if (View->RequiresDebugMaterials())
			{
				NewViewStateDebugFlags |= (1ull << PersistentViewId.Index);
			}
			else
			{
				NewViewStateDebugFlags &= ~(1ull << PersistentViewId.Index);
			}
		}
		// Note: In a future where view updates are decoupled from the rendering properly, this association should happen as part of the rendering. 
		View->PersistentViewId = PersistentViewId;
	}

	// Track changes in combined state of view debug flags.
	ChangeSet->bIsRequiresDebugMaterialChanged = NewViewStateDebugFlags != PersistentViewStateDebugFlags;
	PersistentViewStateDebugFlags = NewViewStateDebugFlags;

	// Extract scene time from the view family. Views should always be non-empty here;
	// ProcessViewChanges is only called when SceneUpdateInputs provides rendered views.
	if (ensureMsgf(Views.Num() > 0, TEXT("ProcessViewChanges called with no views")))
	{
		ChangeSet->CurrentTime = Views[0]->Family->Time;
	}

	return ChangeSet;
}

void FScene::ProcessLevelCommands(FLevelCommand::EOp OpToProcess)
{
	// TODO: Explicit invalidation of various cached systems does not scale & is very error prone. 
	// In the future this should be changed to use the data manager & a tracked visibility state & any and all down stream systems 
	// (GPU scene, VSM, Occlusion culling) monitor changes to this.
	FVirtualShadowMapArrayCacheManager* VirtualShadowMapArrayCacheManager = GetExtensionPtr<FVirtualShadowMapArrayCacheManager>();
	UE::Renderer::Private::IShadowInvalidatingInstances* ShadowInvalidation = VirtualShadowMapArrayCacheManager ? VirtualShadowMapArrayCacheManager->GetInvalidatingInstancesInterface() : nullptr;

	auto FlushCachedShadowMaps = [&](FPrimitiveSceneInfo* PrimitiveSceneInfo)
	{
		FLightPrimitiveInteraction* NodeLightList = PrimitiveSceneInfo->LightList;
		while (NodeLightList)
		{
			NodeLightList->FlushCachedShadowMapData();
			NodeLightList = NodeLightList->GetNextLight();
		}
	};

	for (const FLevelCommand& Command : LevelCommands)
	{
		if (Command.Op == OpToProcess)
		{
			if (const TArray<FPrimitiveSceneInfo*>* LevelPrimitives = PrimitivesNeedingLevelUpdateNotification.Find(Command.Name))
			{
				for (FPrimitiveSceneInfo* PrimitiveSceneInfo : *LevelPrimitives)
				{
					// Skip primitives pending delete as they need no further action
					if (!PrimitiveSceneInfo->bPendingDelete)
					{
						if (OpToProcess == FLevelCommand::EOp::Add)
						{
							PrimitiveSceneInfo->Proxy->OnLevelAddedToWorld_RenderThread();
						}
						else
						{
							PrimitiveSceneInfo->Proxy->OnLevelRemovedFromWorld_RenderThread();
						}

						// Force hidden flag has changed, update it's dependent flags.
						PrimitiveSceneInfo->UpdateOcclusionFlags();
						PrimitiveFlagsCompact[PrimitiveSceneInfo->GetIndex()] = FPrimitiveFlagsCompact(PrimitiveSceneInfo->Proxy);

						// Note: this works because the explicitly queued invalidations are not filtered by primitive flags (otherwise the bIsForceHidden would kill it).
						if (ShadowInvalidation)
						{
							ShadowInvalidation->AddPrimitive(PrimitiveSceneInfo);
						}
						FlushCachedShadowMaps(PrimitiveSceneInfo);

						GPUScene.AddPrimitiveToUpdate(PrimitiveSceneInfo->GetPersistentIndex());
					}
				}
			}
		}		
	}
}

void FScene::Update(FRDGBuilder& GraphBuilder, const FUpdateParameters& Parameters)
{
	LLM_SCOPE(ELLMTag::SceneRender);
	SCOPE_CYCLE_COUNTER(STAT_UpdateScenePrimitiveRenderThreadTime);

	check(IsInRenderingThread());
	check(!UE::RenderCommandPipe::IsReplaying());

	// Wait for all outstanding RHI command lists so we can safely modify proxies.
	GraphBuilder.RHICmdList.ImmediateFlush(EImmediateFlushType::WaitForOutstandingTasksOnly);

	const FSkeletalMeshUpdater::FSubmitData* SkeletalMeshSubmitData = &FSkeletalMeshUpdater::FSubmitData::EmptyData;

	// Only allow async compute when we are updating as part of a render and are allowing async ops to occur.
	const ERHIPipeline GPUSkinCachePipeline = GPUSkinCache && Parameters.AsyncOps != EUpdateAllPrimitiveSceneInfosAsyncOps::None ? FGPUSkinCache::GetDispatchPipeline(GraphBuilder) : ERHIPipeline::Graphics;

	if (SkeletalMeshUpdater)
	{
		SkeletalMeshSubmitData = &SkeletalMeshUpdater->Submit(GraphBuilder, GPUSkinCachePipeline);
		GPUSkinUpdateTask = SkeletalMeshSubmitData->GetSkinCacheTask();
	}

	// Check if the skin cache is available here, because we might be midway through a feature level change
	if (GPUSkinCache && IsGPUSkinCacheAvailable(GetFeatureLevelShaderPlatform(GetFeatureLevel())))
	{
		GPUSkinCacheTask = GPUSkinCache->Dispatch(GraphBuilder, SkeletalMeshSubmitData->GetSkinCacheTask(), GPUSkinCachePipeline);
	}

	UE::Tasks::FTask UpdateUniformExpressionsTask;
	FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions(GraphBuilder.RHICmdList, EnumHasAnyFlags(Parameters.AsyncOps, EUpdateAllPrimitiveSceneInfosAsyncOps::CacheMaterialUniformExpressions) ? &UpdateUniformExpressionsTask : nullptr);

	RDG_EVENT_SCOPE(GraphBuilder, "UpdateAllPrimitiveSceneInfos");

	// Allocated with render graph lifetime, safe to reference from RDG tasks.
	FSceneUpdateChangeSetStorage& SceneUpdateChangeSetStorage = *GraphBuilder.AllocObject<FSceneUpdateChangeSetStorage>(MoveTemp(PrimitiveUpdates), Parameters.ViewUpdateChangeSet);
	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> DeletedPrimitiveSceneInfos;
	DeletedPrimitiveSceneInfos.Reserve(SceneUpdateChangeSetStorage.PrimitiveUpdates.NumCommands());
	SceneUpdateChangeSetStorage.AddedPrimitiveSceneInfos.Reserve(SceneUpdateChangeSetStorage.PrimitiveUpdates.NumCommands());
	SceneUpdateChangeSetStorage.RemovedPrimitiveSceneInfos.Reserve(SceneUpdateChangeSetStorage.PrimitiveUpdates.NumCommands());
	SceneUpdateChangeSetStorage.RemovedPrimitiveIds.Reserve(SceneUpdateChangeSetStorage.PrimitiveUpdates.NumCommands());

	// Retrieve the commonly used categories used in scene updates.
	// 1. RemovedPrimitiveSceneInfos: Primitives that exist in the scene before the update and will be removed.
	// 2. AddedPrimitiveSceneInfos: Primitives not in the scene before the update that will be added.
	// 3. DeletedPrimitiveSceneInfos: Primitives that needs to be deleted.
	SceneUpdateChangeSetStorage.PrimitiveUpdates.ForEachCommand([&](const FPrimitiveUpdateCommand& Cmd)
	{
		// Skip those that were added & removed in the same frame
		if (Cmd.IsAdd() && !Cmd.IsDelete())
		{
			SceneUpdateChangeSetStorage.AddedPrimitiveSceneInfos.Add(Cmd.GetSceneInfo());
		}
		if (Cmd.IsDelete())
		{
			FPrimitiveSceneInfo* SceneInfo = Cmd.GetSceneInfo();
			SceneInfo->bPendingDelete = true;

			DeletedPrimitiveSceneInfos.Add(SceneInfo);
			// Skip those that were added & removed in the same frame
			if (!Cmd.IsAdd())
			{
				SceneUpdateChangeSetStorage.RemovedPrimitiveSceneInfos.Add(Cmd.GetSceneInfo());
				SceneUpdateChangeSetStorage.RemovedPrimitiveIds.Add(Cmd.GetPersistentId());
			}
		}
	});

	const TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> &AddedPrimitiveSceneInfos = SceneUpdateChangeSetStorage.AddedPrimitiveSceneInfos;
	const TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> &RemovedPrimitiveSceneInfos = SceneUpdateChangeSetStorage.RemovedPrimitiveSceneInfos;

	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> RemovedLocalPrimitiveSceneInfos;
	RemovedLocalPrimitiveSceneInfos.Reserve(RemovedPrimitiveSceneInfos.Num());
	for (FPrimitiveSceneInfo* PrimitiveSceneInfo : RemovedPrimitiveSceneInfos)
	{
		RemovedLocalPrimitiveSceneInfos.Add(PrimitiveSceneInfo);
	}

	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> AddedLocalPrimitiveSceneInfos;
	AddedLocalPrimitiveSceneInfos.Reserve(AddedPrimitiveSceneInfos.Num());
	for (FPrimitiveSceneInfo* SceneInfo : AddedPrimitiveSceneInfos)
	{
		AddedLocalPrimitiveSceneInfos.Add(SceneInfo);
	}

	GPUScene.OnPreSceneUpdate(GraphBuilder, SceneUpdateChangeSetStorage.GetPreUpdateSet());

	FSceneExtensionsUpdaters& SceneExtensionsUpdaters = *GraphBuilder.AllocObject<FSceneExtensionsUpdaters>(*this);

	UpdateLights(GraphBuilder, SceneExtensionsUpdaters);

	// Remove ops shoud be processed before scene update to allow scene extensions PreSceneUpdate to handle effects before the scene is changed.
	// In particular VSM cache invalidation depends on this ordering.
	ProcessLevelCommands(FLevelCommand::EOp::Remove);

	SceneExtensionsUpdaters.PreSceneUpdate(GraphBuilder, SceneUpdateChangeSetStorage.GetPreUpdateSet());

#if RHI_RAYTRACING
	UpdateRayTracingGroupBounds_RemovePrimitives(RemovedPrimitiveSceneInfos);
	UpdateRayTracingGroupBounds_AddPrimitives(AddedPrimitiveSceneInfos);
#endif

	auto UpdatedInstances = SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateInstanceCommand>();
	auto UpdatedTransforms = SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateTransformCommand>();
	auto UpdatedInstanceCullDistance =  SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateInstanceCullDistanceData>();
	auto OverridenPreviousTransforms =  SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateOverridePreviousTransformData>();	
	RemovedLocalPrimitiveSceneInfos.Sort(FPrimitiveArraySortKey());
	
	AddedLocalPrimitiveSceneInfos.Sort(FPrimitiveArraySortKey());
	TArray<int32> RemovedPrimitiveIndices;
	RemovedPrimitiveIndices.SetNumUninitialized(RemovedLocalPrimitiveSceneInfos.Num());

	bool bNeedPathTracedInvalidation = false;
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(RemovePrimitiveSceneInfos);
		SCOPE_CYCLE_COUNTER(STAT_RemoveScenePrimitiveTime);

		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : RemovedLocalPrimitiveSceneInfos)
		{
			// clear it up, parent is getting removed
			SceneLODHierarchy.UpdateNodeSceneInfo(PrimitiveSceneInfo->PrimitiveComponentId, nullptr);
		}

		while (RemovedLocalPrimitiveSceneInfos.Num())
		{
			int32 StartIndex = RemovedLocalPrimitiveSceneInfos.Num() - 1;
			FPrimitiveSceneProxyType RemovedProxyType = FPrimitiveSceneProxyType(RemovedLocalPrimitiveSceneInfos[StartIndex]->Proxy);

			while (StartIndex > 0 && FPrimitiveSceneProxyType(RemovedLocalPrimitiveSceneInfos[StartIndex - 1]->Proxy) == RemovedProxyType)
			{
				StartIndex--;
			}

			int32 BroadIndex = -1;
			//broad phase search for a matching type
			for (BroadIndex = TypeOffsetTable.Num() - 1; BroadIndex >= 0; BroadIndex--)
			{
				// example how the prefix sum of the tails could look like
				// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,2,2,2,2,1,1,1,7,4,8]
				// TypeOffsetTable[3,8,12,15,16,17,18]

				if (TypeOffsetTable[BroadIndex].PrimitiveSceneProxyType == RemovedProxyType)
				{
					const int32 InsertionOffset = TypeOffsetTable[BroadIndex].Offset;
					const int32 PrevOffset = BroadIndex > 0 ? TypeOffsetTable[BroadIndex - 1].Offset : 0;
					for (int32 CheckIndex = StartIndex; CheckIndex < RemovedLocalPrimitiveSceneInfos.Num(); CheckIndex++)
					{
						int32 PrimitiveIndex = RemovedLocalPrimitiveSceneInfos[CheckIndex]->PackedIndex;
						checkfSlow(PrimitiveIndex >= PrevOffset && PrimitiveIndex < InsertionOffset, TEXT("PrimitiveIndex %d not in Bucket Range [%d, %d]"), PrimitiveIndex, PrevOffset, InsertionOffset);
					}
					break;
				}
			}

			{
				SCOPED_NAMED_EVENT(FScene_SwapPrimitiveSceneInfos, FColor::Turquoise);

				for (int32 CheckIndex = StartIndex; CheckIndex < RemovedLocalPrimitiveSceneInfos.Num(); CheckIndex++)
				{
					int32 SourceIndex = RemovedLocalPrimitiveSceneInfos[CheckIndex]->PackedIndex;

					for (int32 TypeIndex = BroadIndex; TypeIndex < TypeOffsetTable.Num(); TypeIndex++)
					{
						FTypeOffsetTableEntry& NextEntry = TypeOffsetTable[TypeIndex];
						int DestIndex = --NextEntry.Offset; //decrement and prepare swap 

						// example swap chain of removing X 
						// PrimitiveSceneProxies[0,0,0,6,X,6,6,6,2,2,2,2,1,1,1,7,4,8]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,X,2,2,2,1,1,1,7,4,8]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,2,2,2,X,1,1,1,7,4,8]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,2,2,2,1,1,1,X,7,4,8]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,2,2,2,1,1,1,7,X,4,8]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,2,2,2,1,1,1,7,4,X,8]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,2,2,2,1,1,1,7,4,8,X]

						if (DestIndex != SourceIndex)
						{
							checkfSlow(DestIndex > SourceIndex, TEXT("Corrupted Prefix Sum [%d, %d]"), DestIndex, SourceIndex);
							Primitives[DestIndex]->PackedIndex = SourceIndex;
							// Update (the dynamic/compacted) primitive ID for the swapped primitive (not moved), no need to do the other one since it is being removed.
							FPersistentPrimitiveIndex MovedPersisitentIndex = Primitives[DestIndex]->PersistentIndex;
							PersistentPrimitiveIdToIndexMap[MovedPersisitentIndex.Index] = SourceIndex;

							Primitives[SourceIndex]->PackedIndex = DestIndex;

							TArraySwapElements(Primitives, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveTransforms, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveSceneProxies, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveBounds, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveFlagsCompact, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveVisibilityIds, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveOctreeIndex, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveOcclusionFlags, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveComponentIds, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveOcclusionBounds, DestIndex, SourceIndex);
						#if WITH_EDITOR
							TBitArraySwapElements(PrimitivesSelected, DestIndex, SourceIndex);
						#endif
						#if RHI_RAYTRACING
							TArraySwapElements(PrimitiveRayTracingFlags, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveRayTracingDatas, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveRayTracingGroupIds, DestIndex, SourceIndex);
						#endif
							TBitArraySwapElements(PrimitivesNeedingStaticMeshUpdate, DestIndex, SourceIndex);
							TBitArraySwapElements(PrimitivesNeedingUniformBufferUpdate, DestIndex, SourceIndex);

							SourceIndex = DestIndex;
						}
					}
				}
			}

			const int32 PreviousOffset = BroadIndex > 0 ? TypeOffsetTable[BroadIndex - 1].Offset : 0;
			const int32 CurrentOffset = TypeOffsetTable[BroadIndex].Offset;

			checkfSlow(PreviousOffset <= CurrentOffset, TEXT("Corrupted Bucket [%d, %d]"), PreviousOffset, CurrentOffset);
			if (CurrentOffset - PreviousOffset == 0)
			{
				// remove empty OffsetTable entries e.g.
				// TypeOffsetTable[3,8,12,15,15,17,18]
				// TypeOffsetTable[3,8,12,15,17,18]
				TypeOffsetTable.RemoveAt(BroadIndex);
			}

			checkfSlow((TypeOffsetTable.Num() == 0 && Primitives.Num() == (RemovedLocalPrimitiveSceneInfos.Num() - StartIndex)) || TypeOffsetTable[TypeOffsetTable.Num() - 1].Offset == Primitives.Num() - (RemovedLocalPrimitiveSceneInfos.Num() - StartIndex), TEXT("Corrupted Tail Offset [%d, %d]"), TypeOffsetTable[TypeOffsetTable.Num() - 1].Offset, Primitives.Num() - (RemovedLocalPrimitiveSceneInfos.Num() - StartIndex));

			for (int32 RemoveIndex = StartIndex; RemoveIndex < RemovedLocalPrimitiveSceneInfos.Num(); RemoveIndex++)
			{
				FPrimitiveSceneInfo* PrimitiveSceneInfo = RemovedLocalPrimitiveSceneInfos[RemoveIndex];
				checkf(RemovedLocalPrimitiveSceneInfos[RemoveIndex]->PackedIndex >= Primitives.Num() - RemovedLocalPrimitiveSceneInfos.Num(), TEXT("Removed item should be at the end"));

				// Store the previous index for use later, and set the PackedIndex member to invalid.
				// FPrimitiveOctreeSemantics::SetOctreeNodeIndex will attempt to remove the node index from the 
				// PrimitiveOctreeIndex.  Since the elements have already been swapped, this will cause an invalid change to PrimitiveOctreeIndex.
				// Setting the packed index to INDEX_NONE prevents this from happening, but we also need to keep track of the old
				// index for use below.
				RemovedPrimitiveIndices[RemoveIndex] = RemovedLocalPrimitiveSceneInfos[RemoveIndex]->PackedIndex;
				PrimitiveSceneInfo->PackedIndex = INDEX_NONE;
			}

			//Remove all items from the location of StartIndex to the end of the arrays.
			int RemoveCount = RemovedLocalPrimitiveSceneInfos.Num() - StartIndex;
			int SourceIndex = Primitives.Num() - RemoveCount;

			Primitives.RemoveAt(SourceIndex, RemoveCount, EAllowShrinking::No);
			PrimitiveTransforms.Remove(RemoveCount, EAllowShrinking::No);
			PrimitiveSceneProxies.RemoveAt(SourceIndex, RemoveCount, EAllowShrinking::No);
			PrimitiveBounds.Remove(RemoveCount, EAllowShrinking::No);
			PrimitiveFlagsCompact.RemoveAt(SourceIndex, RemoveCount, EAllowShrinking::No);
			PrimitiveVisibilityIds.RemoveAt(SourceIndex, RemoveCount, EAllowShrinking::No);
			PrimitiveOctreeIndex.RemoveAt(SourceIndex, RemoveCount, EAllowShrinking::No);
			PrimitiveOcclusionFlags.RemoveAt(SourceIndex, RemoveCount, EAllowShrinking::No);
			PrimitiveComponentIds.RemoveAt(SourceIndex, RemoveCount, EAllowShrinking::No);
			PrimitiveOcclusionBounds.Remove(RemoveCount, EAllowShrinking::No);

			#if WITH_EDITOR
			PrimitivesSelected.RemoveAt(SourceIndex, RemoveCount);
			#endif
			#if RHI_RAYTRACING
			PrimitiveRayTracingFlags.RemoveAt(SourceIndex, RemoveCount);
			PrimitiveRayTracingDatas.RemoveAt(SourceIndex, RemoveCount);
			PrimitiveRayTracingGroupIds.RemoveAt(SourceIndex, RemoveCount);
			#endif
			PrimitivesNeedingStaticMeshUpdate.RemoveAt(SourceIndex, RemoveCount);
			PrimitivesNeedingUniformBufferUpdate.RemoveAt(SourceIndex, RemoveCount);

			CheckPrimitiveArrays();

			for (int32 RemoveIndex = StartIndex; RemoveIndex < RemovedLocalPrimitiveSceneInfos.Num(); RemoveIndex++)
			{
				FPrimitiveSceneInfo* PrimitiveSceneInfo = RemovedLocalPrimitiveSceneInfos[RemoveIndex];
				FPrimitiveSceneProxy* SceneProxy = PrimitiveSceneInfo->Proxy;
				FScopeCycleCounter Context(SceneProxy->GetStatId());

				// The removed items PrimitiveIndex has already been invalidated, but a backup is kept in RemovedPrimitiveIndices
				int32 PrimitiveIndex = RemovedPrimitiveIndices[RemoveIndex];

				if (PrimitiveSceneInfo->bRegisteredWithVelocityData)
				{
					// Remove primitive's motion blur information.
					VelocityData.RemoveFromScene(PrimitiveSceneInfo->PrimitiveComponentId, false);
				}

				PrimitiveComponentIdToInfoMap.Remove(PrimitiveSceneInfo->PrimitiveComponentId);

				// Unlink the primitive from its shadow parent.
				PrimitiveSceneInfo->UnlinkAttachmentGroup();

				// Unlink the LOD parent info if valid
				PrimitiveSceneInfo->UnlinkLODParentComponent();

				// Flush virtual textures touched by primitive
				PrimitiveSceneInfo->FlushRuntimeVirtualTexture();

				// Remove the primitive from the scene.
				PrimitiveSceneInfo->RemoveFromScene(true);

				PrimitiveSceneInfo->FreeGPUSceneInstances();

				DistanceFieldSceneData.RemovePrimitive(PrimitiveSceneInfo);
				LumenRemovePrimitive(PrimitiveSceneInfo, PrimitiveIndex);

#if RHI_RAYTRACING
				ensureMsgf(PrimitiveSceneInfo->bHasRayTracingRepresentation == SceneProxy->HasRayTracingRepresentation(),
					TEXT("Stale bHasRayTracingRepresentation detected. Primitive might have missed requests to update cached ray tracing state."));
				if (PrimitiveSceneInfo->bHasRayTracingRepresentation)
				{
					const RayTracing::FGeometryGroupHandle RayTracingGeometryGroupHandle = SceneProxy->GetRayTracingGeometryGroupHandle();
					if (!RayTracingGeometryGroupHandle.IsNull())
					{
						((FRayTracingGeometryManager*)GRayTracingGeometryManager)->UnregisterProxyWithCachedRayTracingState(SceneProxy, RayTracingGeometryGroupHandle);
					}

					if (SceneProxy->IsNaniteMesh())
					{
						Nanite::GRayTracingManager.Remove(PrimitiveSceneInfo);
					}
				}

				bNeedPathTracedInvalidation = bNeedPathTracedInvalidation || PrimitiveSceneInfo->bIsRelevantToPathTracing;
#endif

				const int32 PersistentIndex = PrimitiveSceneInfo->PersistentIndex.Index;
				PersistentPrimitiveIdAllocator.Free(PersistentIndex);
				PersistentPrimitiveIdToIndexMap[PersistentIndex] = INDEX_NONE;
			}

			RemovedLocalPrimitiveSceneInfos.RemoveAt(StartIndex, RemovedLocalPrimitiveSceneInfos.Num() - StartIndex, EAllowShrinking::No);
		}
	
	}

	const int32 SceneInfosContainerReservedSize = AddedPrimitiveSceneInfos.Num() + UpdatedTransforms.Num() + UpdatedInstances.Num();

	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>& SceneInfosWithAddToScene = *GraphBuilder.AllocObject<TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>>();
	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> SceneInfosWithFlushRuntimeVirtualTexture;
	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>& SceneInfosWithStaticDrawListUpdate = *GraphBuilder.AllocObject<TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>>();
	SceneInfosWithAddToScene.Reserve(SceneInfosContainerReservedSize);
	SceneInfosWithFlushRuntimeVirtualTexture.Reserve(SceneInfosContainerReservedSize);
	SceneInfosWithStaticDrawListUpdate.Reserve(SceneInfosContainerReservedSize);

	const auto QueueFlushRuntimeVirtualTexture = [&](FPrimitiveSceneInfo* SceneInfo) -> bool
	{
		if (SceneInfo->bWritesRuntimeVirtualTexture && !SceneInfo->bPendingFlushRuntimeVirtualTexture)
		{
			SceneInfo->bPendingFlushRuntimeVirtualTexture = true;
			SceneInfosWithFlushRuntimeVirtualTexture.Push(SceneInfo);
			return true;
		}
		return false;
	};

	const auto QueueAddToScene = [&](FPrimitiveSceneInfo* SceneInfo) -> bool
	{
		if (!SceneInfo->bPendingAddToScene)
		{
			SceneInfo->bPendingAddToScene = true;
			SceneInfosWithAddToScene.Push(SceneInfo);
			return true;
		}
		return false;
	};

	const auto QueueAddStaticMeshes = [&](FPrimitiveSceneInfo* SceneInfo)
	{
		if (SceneInfo->bPendingDelete)
		{
			UE_LOGF(LogRenderer, Fatal, "SceneInfo %ls is marked for deletion but is being queued for static mesh updates. This is a race condition and will crash.", *SceneInfo->Proxy->GetOwnerName().ToString());
		}

		if (!SceneInfo->bPendingAddStaticMeshes)
		{
			SceneInfo->bPendingAddStaticMeshes = 1;
			SceneInfosWithStaticDrawListUpdate.Push(SceneInfo);
			PrimitivesNeedingStaticMeshUpdate[SceneInfo->PackedIndex] = false;
			return true;
		}
		return false;
	};

	
	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> PendingAllocateInstanceIds;
	PendingAllocateInstanceIds.Reserve(UpdatedInstances.Num() + AddedLocalPrimitiveSceneInfos.Num());
	// All added primitive scene infos need to be allocated.
	PendingAllocateInstanceIds.Append(AddedLocalPrimitiveSceneInfos);

	// Perform instance releases before adds, such that allocator consolidation can happen only once.
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UpdatePrimitiveInstances);
		SCOPE_CYCLE_COUNTER(STAT_UpdatePrimitiveInstanceRenderThreadTime);

		for (const auto& UpdateInstance : UpdatedInstances)
		{
			FPrimitiveSceneProxy* PrimitiveSceneProxy = UpdateInstance.SceneInfo->Proxy;
			FScopeCycleCounter Context(PrimitiveSceneProxy->GetStatId());
			FPrimitiveSceneInfo* PrimitiveSceneInfo = UpdateInstance.SceneInfo;

			const FInstanceDataBufferHeader &InstanceDataBufferHeader = PrimitiveSceneInfo->GetInstanceDataHeader();
			const bool bInstanceCountChanged = PrimitiveSceneInfo->GetNumInstanceSceneDataEntries() != InstanceDataBufferHeader.NumInstances;
			const bool bInstancePayloadDataStrideChanged = InstanceDataBufferHeader.NumInstances > 0 && PrimitiveSceneInfo->GetInstancePayloadDataStride() != InstanceDataBufferHeader.PayloadDataStride;
			// Append to queue if not added (if it is also added it will already be queued up)
			if ((bInstanceCountChanged || bInstancePayloadDataStrideChanged) && PrimitiveSceneInfo->GetIndex() != INDEX_NONE)
			{
				PrimitiveSceneInfo->FreeGPUSceneInstances();
				PendingAllocateInstanceIds.Add(PrimitiveSceneInfo);
			}
		}
	}

	GPUScene.ConsolidateInstanceDataAllocations();

	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(AddPrimitiveSceneInfos);
		SCOPE_CYCLE_COUNTER(STAT_AddScenePrimitiveRenderThreadTime);

		PersistentPrimitiveIdAllocator.Consolidate();

		SceneUpdateChangeSetStorage.PrimitiveUpdates.ForEachCommand(ESceneUpdateCommandFilter::Added, [&](FPrimitiveUpdateCommand& Cmd)
		{
			// Skip those that were added & removed in the same frame
			if (Cmd.IsAdd() && !Cmd.IsDelete())
			{
				check(!Cmd.GetPersistentId().IsValid());
				check(!Cmd.GetSceneInfo()->PersistentIndex.IsValid());
				FPersistentPrimitiveIndex PersistentPrimitiveIndex{ PersistentPrimitiveIdAllocator.Allocate() };
				// Note: propagate to the command such that is it valid for subsequent passes without having to hit the primitive scene info.
				Cmd.SetPersistentId(PersistentPrimitiveIndex);
				Cmd.GetSceneInfo()->PersistentIndex = PersistentPrimitiveIndex;
			}
		});
		PersistentPrimitiveIdToIndexMap.SetNumUninitialized(GetMaxPersistentPrimitiveIndex());

		if (AddedLocalPrimitiveSceneInfos.Num())
		{
			SCOPED_NAMED_EVENT(FScene_AddPrimitiveSceneInfos_ReserveMemory, FColor::Green);
			Primitives.Reserve(Primitives.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveTransforms.Reserve(PrimitiveTransforms.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveSceneProxies.Reserve(PrimitiveSceneProxies.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveBounds.Reserve(PrimitiveBounds.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveFlagsCompact.Reserve(PrimitiveFlagsCompact.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveVisibilityIds.Reserve(PrimitiveVisibilityIds.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveOcclusionFlags.Reserve(PrimitiveOcclusionFlags.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveComponentIds.Reserve(PrimitiveComponentIds.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveOcclusionBounds.Reserve(PrimitiveOcclusionBounds.Num() + AddedLocalPrimitiveSceneInfos.Num());
		#if WITH_EDITOR
			PrimitivesSelected.Reserve(PrimitivesSelected.Num() + AddedLocalPrimitiveSceneInfos.Num());
		#endif
		#if RHI_RAYTRACING
			PrimitiveRayTracingFlags.Reserve(PrimitiveRayTracingFlags.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveRayTracingDatas.Reserve(PrimitiveRayTracingDatas.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitiveRayTracingGroupIds.Reserve(PrimitiveRayTracingGroupIds.Num() + AddedLocalPrimitiveSceneInfos.Num());
		#endif
			PrimitivesNeedingStaticMeshUpdate.Reserve(PrimitivesNeedingStaticMeshUpdate.Num() + AddedLocalPrimitiveSceneInfos.Num());
			PrimitivesNeedingUniformBufferUpdate.Reserve(PrimitivesNeedingUniformBufferUpdate.Num() + AddedLocalPrimitiveSceneInfos.Num());
		}

		while (AddedLocalPrimitiveSceneInfos.Num())
		{
			int32 StartIndex = AddedLocalPrimitiveSceneInfos.Num() - 1;
			FPrimitiveSceneProxyType InsertProxyType = FPrimitiveSceneProxyType(AddedLocalPrimitiveSceneInfos[StartIndex]->Proxy);

			while (StartIndex > 0 && FPrimitiveSceneProxyType(AddedLocalPrimitiveSceneInfos[StartIndex - 1]->Proxy) == InsertProxyType)
			{
				StartIndex--;
			}

			{
				SCOPED_NAMED_EVENT(FScene_AddPrimitiveSceneInfos, FColor::Turquoise);

				for (int32 AddIndex = StartIndex; AddIndex < AddedLocalPrimitiveSceneInfos.Num(); AddIndex++)
				{
					FPrimitiveSceneInfo* PrimitiveSceneInfo = AddedLocalPrimitiveSceneInfos[AddIndex];
					Primitives.Add(PrimitiveSceneInfo);
					const FMatrix LocalToWorld = PrimitiveSceneInfo->Proxy->GetLocalToWorld();
					PrimitiveTransforms.Add(LocalToWorld);
					PrimitiveSceneProxies.Add(PrimitiveSceneInfo->Proxy);
					PrimitiveBounds.AddUninitialized();
					PrimitiveFlagsCompact.AddUninitialized();
					PrimitiveVisibilityIds.AddUninitialized();
					PrimitiveOctreeIndex.Add(0);
					PrimitiveOcclusionFlags.AddUninitialized();
					PrimitiveComponentIds.AddUninitialized();
					PrimitiveOcclusionBounds.AddUninitialized();
				#if WITH_EDITOR
					PrimitivesSelected.Add(PrimitiveSceneInfo->Proxy->WantsEditorEffects() || PrimitiveSceneInfo->Proxy->IsSelected());
				#endif
				#if RHI_RAYTRACING
					PrimitiveRayTracingFlags.AddZeroed();
					PrimitiveRayTracingDatas.AddZeroed();
					PrimitiveRayTracingGroupIds.Add(Experimental::FHashElementId());
				#endif
					PrimitivesNeedingStaticMeshUpdate.Add(false);
					PrimitivesNeedingUniformBufferUpdate.Add(true);

					const int32 SourceIndex = PrimitiveSceneProxies.Num() - 1;
					PrimitiveSceneInfo->PackedIndex = SourceIndex;
					check(PrimitiveSceneInfo->PersistentIndex.IsValid());
					PersistentPrimitiveIdToIndexMap[PrimitiveSceneInfo->PersistentIndex.Index] = SourceIndex;
				}
			}

			bool EntryFound = false;
			int32 BroadIndex = -1;
			//broad phase search for a matching type
			for (BroadIndex = TypeOffsetTable.Num() - 1; BroadIndex >= 0; BroadIndex--)
			{
				// example how the prefix sum of the tails could look like
				// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,2,2,2,2,1,1,1,7,4,8]
				// TypeOffsetTable[3,8,12,15,16,17,18]

				if (TypeOffsetTable[BroadIndex].PrimitiveSceneProxyType == InsertProxyType)
				{
					EntryFound = true;
					break;
				}
			}

			// New type encountered
			if (!EntryFound)
			{
				BroadIndex = TypeOffsetTable.Num();
				if (BroadIndex)
				{
					uint32 NextTypeOffset = 0;
 					for (int32 TypeOffsetIndex = 0; TypeOffsetIndex < TypeOffsetTable.Num(); ++TypeOffsetIndex)
					{
						const FTypeOffsetTableEntry& TypeEntry = TypeOffsetTable[TypeOffsetIndex];
						if (PrimitiveSceneProxies[NextTypeOffset]->IsAlwaysVisible())
						{
							BroadIndex = TypeOffsetIndex;
							break;
						}

						NextTypeOffset = TypeEntry.Offset;
					}

					int32 PrevEntryOffset = BroadIndex > 0 ? TypeOffsetTable[BroadIndex - 1].Offset : 0;
					TypeOffsetTable.Insert(FTypeOffsetTableEntry(InsertProxyType, PrevEntryOffset), BroadIndex);
				}
				else
				{
					// Starting with an empty list and zero offset (offset will be incremented during the while loop)
					TypeOffsetTable.Push(FTypeOffsetTableEntry(InsertProxyType, 0));
				}
			}

			{
				SCOPED_NAMED_EVENT(FScene_SwapPrimitiveSceneInfos, FColor::Turquoise);

				for (int32 AddIndex = StartIndex; AddIndex < AddedLocalPrimitiveSceneInfos.Num(); AddIndex++)
				{
					int32 SourceIndex = AddedLocalPrimitiveSceneInfos[AddIndex]->PackedIndex;

					for (int32 TypeIndex = BroadIndex; TypeIndex < TypeOffsetTable.Num(); TypeIndex++)
					{
						FTypeOffsetTableEntry& NextEntry = TypeOffsetTable[TypeIndex];
						int32 DestIndex = NextEntry.Offset++; //prepare swap and increment

						// Example swap chain of inserting a type of 6 at the end
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,2,2,2,2,1,1,1,7,4,8,6]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,6,2,2,2,1,1,1,7,4,8,2]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,6,2,2,2,2,1,1,7,4,8,1]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,6,2,2,2,2,1,1,1,4,8,7]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,6,2,2,2,2,1,1,1,7,8,4]
						// PrimitiveSceneProxies[0,0,0,6,6,6,6,6,6,2,2,2,2,1,1,1,7,4,8]

						if (DestIndex != SourceIndex)
						{
							checkfSlow(SourceIndex > DestIndex, TEXT("Corrupted Prefix Sum [%d, %d]"), SourceIndex, DestIndex);
							Primitives[DestIndex]->PackedIndex = SourceIndex;
							Primitives[SourceIndex]->PackedIndex = DestIndex;

							// Update (the dynamic/compacted) primitive ID for the swapped primitives
							{
								FPersistentPrimitiveIndex PersistentIndex = Primitives[DestIndex]->PersistentIndex;
								PersistentPrimitiveIdToIndexMap[PersistentIndex.Index] = SourceIndex;
							}
							{
								FPersistentPrimitiveIndex PersistentIndex = Primitives[SourceIndex]->PersistentIndex;
								PersistentPrimitiveIdToIndexMap[PersistentIndex.Index] = DestIndex;
							}
							TArraySwapElements(Primitives, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveTransforms, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveSceneProxies, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveBounds, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveFlagsCompact, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveVisibilityIds, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveOctreeIndex, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveOcclusionFlags, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveComponentIds, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveOcclusionBounds, DestIndex, SourceIndex);
						#if WITH_EDITOR
							TBitArraySwapElements(PrimitivesSelected, DestIndex, SourceIndex);
						#endif
						#if RHI_RAYTRACING
							TArraySwapElements(PrimitiveRayTracingFlags, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveRayTracingDatas, DestIndex, SourceIndex);
							TArraySwapElements(PrimitiveRayTracingGroupIds, DestIndex, SourceIndex);
						#endif
							TBitArraySwapElements(PrimitivesNeedingStaticMeshUpdate, DestIndex, SourceIndex);
							TBitArraySwapElements(PrimitivesNeedingUniformBufferUpdate, DestIndex, SourceIndex);
						}
					}
				}
			}

			CheckPrimitiveArrays();

			for (int32 AddIndex = StartIndex; AddIndex < AddedLocalPrimitiveSceneInfos.Num(); AddIndex++)
			{
				FPrimitiveSceneInfo* PrimitiveSceneInfo = AddedLocalPrimitiveSceneInfos[AddIndex];
				FScopeCycleCounter Context(PrimitiveSceneInfo->Proxy->GetStatId());
				int32 PrimitiveIndex = PrimitiveSceneInfo->PackedIndex;

				// Add the primitive to its shadow parent's linked list of children.
				// Note: must happen before AddToScene because AddToScene depends on LightingAttachmentRoot
				PrimitiveSceneInfo->LinkAttachmentGroup();
			}

			for (int32 AddIndex = StartIndex; AddIndex < AddedLocalPrimitiveSceneInfos.Num(); AddIndex++)
			{
				FPrimitiveSceneInfo* PrimitiveSceneInfo = AddedLocalPrimitiveSceneInfos[AddIndex];
				int32 PrimitiveIndex = PrimitiveSceneInfo->PackedIndex;
				FPrimitiveSceneProxy* SceneProxy = PrimitiveSceneInfo->Proxy;
				SceneProxy->CreateUniformBuffer();

				if (ShouldPrimitiveOutputVelocity(SceneProxy, GetShaderPlatform()))
				{
					PrimitiveSceneInfo->bRegisteredWithVelocityData = true;
					// We must register the initial LocalToWorld with the velocity state. 
					// In the case of a moving component with MarkRenderStateDirty() called every frame, UpdateTransform will never happen.
					VelocityData.UpdateTransform(PrimitiveSceneInfo, PrimitiveTransforms[PrimitiveIndex], PrimitiveTransforms[PrimitiveIndex]);
				}

				DistanceFieldSceneData.AddPrimitive(PrimitiveSceneInfo);
				LumenAddPrimitive(PrimitiveSceneInfo);

#if RHI_RAYTRACING
				PrimitiveSceneInfo->bHasRayTracingRepresentation = SceneProxy->HasRayTracingRepresentation();
				if (PrimitiveSceneInfo->bHasRayTracingRepresentation)
				{
					// TODO: investigate registering proxy when data is actually cached (for example, in FPrimitiveSceneInfo::CacheRayTracingPrimitives(...))
					const RayTracing::FGeometryGroupHandle RayTracingGeometryGroupHandle = SceneProxy->GetRayTracingGeometryGroupHandle();
					if (!RayTracingGeometryGroupHandle.IsNull())
					{
						((FRayTracingGeometryManager*)GRayTracingGeometryManager)->RegisterProxyWithCachedRayTracingState(SceneProxy, RayTracingGeometryGroupHandle);
					}

					if (SceneProxy->IsNaniteMesh())
					{
						Nanite::GRayTracingManager.Add(PrimitiveSceneInfo);
					}
				}

				{
					bool bShouldRenderInMainPass = SceneProxy->ShouldRenderInMainPass();
					bool bIsRayTracingRelevant = SceneProxy->IsRayTracingRelevant();
					bool bIsVisibleInRayTracing = SceneProxy->IsVisibleInRayTracing();

					FScene::FPrimitiveRayTracingData& PrimitiveRayTracingData = PrimitiveRayTracingDatas[PrimitiveIndex];
					PrimitiveRayTracingData.bDrawInGame = SceneProxy->IsDrawnInGame();
					PrimitiveRayTracingData.bRayTracingFarField = SceneProxy->IsRayTracingFarField();
					PrimitiveRayTracingData.bRetainWhileHidden = SceneProxy->CastsHiddenShadow() || SceneProxy->AffectsIndirectLightingWhileHidden();
					PrimitiveRayTracingData.bIsVisibleInSceneCaptures = !SceneProxy->IsHiddenInSceneCapture();
					PrimitiveRayTracingData.bIsVisibleInSceneCapturesOnly = SceneProxy->IsVisibleInSceneCaptureOnly();
					PrimitiveRayTracingData.LightingChannelMask = SceneProxy->GetLightingChannelMask();
					PrimitiveRayTracingData.bCachedRaytracingDataDirty = true;
					PrimitiveRayTracingData.RayTracingGeometryGroupHandle = SceneProxy->GetRayTracingGeometryGroupHandle();
					PrimitiveRayTracingData.ProxyGeometryType = SceneProxy->GetRayTracingProxyType();

					// cache bIsRelevantToPathTracing in PrimitiveSceneInfo 
					// since we can't access the corresponding PrimitiveRayTracingData entry when primitive is removed
					PrimitiveSceneInfo->bIsRelevantToPathTracing =
						bIsRayTracingRelevant &&
						bIsVisibleInRayTracing &&
						(PrimitiveRayTracingData.bDrawInGame || PrimitiveRayTracingData.bRetainWhileHidden) &&
						bShouldRenderInMainPass;
				}

				bNeedPathTracedInvalidation = bNeedPathTracedInvalidation || PrimitiveSceneInfo->bIsRelevantToPathTracing;
#endif

				QueueFlushRuntimeVirtualTexture(PrimitiveSceneInfo);
				QueueAddToScene(PrimitiveSceneInfo);
				QueueAddStaticMeshes(PrimitiveSceneInfo);
			}
			AddedLocalPrimitiveSceneInfos.RemoveAt(StartIndex, AddedLocalPrimitiveSceneInfos.Num() - StartIndex, EAllowShrinking::No);
		}
	}
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UpdatePrimitiveTransform);
		SCOPE_CYCLE_COUNTER(STAT_UpdatePrimitiveTransformRenderThreadTime);


		for (const auto& Item : UpdatedTransforms)
		{
			FPrimitiveSceneProxy* PrimitiveSceneProxy = Item.SceneInfo->Proxy;

			const FBoxSphereBounds& WorldBounds = Item.Payload.WorldBounds;
			const FBoxSphereBounds& LocalBounds = Item.Payload.LocalBounds;
			const FMatrix& LocalToWorld = Item.Payload.LocalToWorld;
			const FVector& AttachmentRootPosition = Item.Payload.AttachmentRootPosition;

			check(PrimitiveSceneProxy->GetPrimitiveSceneInfo()->PackedIndex != INDEX_NONE);

			FScopeCycleCounter Context(PrimitiveSceneProxy->GetStatId());

			FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo();
			const bool bUpdateStaticDrawLists = !PrimitiveSceneProxy->StaticElementsAlwaysUseProxyPrimitiveUniformBuffer();

			if (QueueAddToScene(PrimitiveSceneInfo))
			{
				// Remove the primitive from the scene at its old location
				// (note that the octree update relies on the bounds not being modified yet).
				PrimitiveSceneInfo->RemoveFromScene(bUpdateStaticDrawLists);

				if (bUpdateStaticDrawLists)
				{
					QueueAddStaticMeshes(PrimitiveSceneInfo);
				}
			}

			QueueFlushRuntimeVirtualTexture(PrimitiveSceneInfo);

			if (ShouldPrimitiveOutputVelocity(PrimitiveSceneInfo->Proxy, GetShaderPlatform()))
			{
				PrimitiveSceneInfo->bRegisteredWithVelocityData = true;
				VelocityData.UpdateTransform(PrimitiveSceneInfo, LocalToWorld, PrimitiveSceneProxy->GetLocalToWorld());
			}

#if RHI_RAYTRACING
			bNeedPathTracedInvalidation = bNeedPathTracedInvalidation || (PrimitiveSceneInfo->bIsRelevantToPathTracing &&
				!PrimitiveTransforms[PrimitiveSceneInfo->PackedIndex].Equals(LocalToWorld, SMALL_NUMBER));
#endif

			// Update the primitive transform.
			PrimitiveSceneProxy->SetTransform(GraphBuilder.RHICmdList, LocalToWorld, WorldBounds, LocalBounds, AttachmentRootPosition);
			PrimitiveTransforms[PrimitiveSceneInfo->PackedIndex] = LocalToWorld;

			if (!RHISupportsVolumeTextures(GetFeatureLevel())
				&& (PrimitiveSceneProxy->IsMovable() || PrimitiveSceneProxy->NeedsUnbuiltPreviewLighting() || PrimitiveSceneProxy->GetLightmapType() == ELightmapType::ForceVolumetric))
			{
				PrimitiveSceneInfo->MarkIndirectLightingCacheBufferDirty();
			}

			DistanceFieldSceneData.UpdatePrimitive(PrimitiveSceneInfo);
			LumenUpdatePrimitive(PrimitiveSceneInfo);

			// If the primitive has static mesh elements, it should have returned true from ShouldRecreateProxyOnUpdateTransform!
			check(!(bUpdateStaticDrawLists && PrimitiveSceneInfo->StaticMeshes.Num()));
		}
#if RHI_RAYTRACING
		{
			UpdateRayTracingGroupBounds_UpdatePrimitives(UpdatedTransforms);
		}
#endif

		for (const auto& Transform : OverridenPreviousTransforms)
		{
			FPrimitiveSceneInfo* PrimitiveSceneInfo = Transform.SceneInfo;
			VelocityData.OverridePreviousTransform(PrimitiveSceneInfo->PrimitiveComponentId, Transform.Payload.Value);
		}
	}


	// Updating step of instance update. Note that instance IDs are not allocated at this point.
	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UpdatePrimitiveInstances);
		SCOPE_CYCLE_COUNTER(STAT_UpdatePrimitiveInstanceRenderThreadTime);

		for (const auto& Item : UpdatedInstances)
		{
			FPrimitiveSceneProxy* PrimitiveSceneProxy = Item.SceneInfo->Proxy;
			FPrimitiveSceneInfo* PrimitiveSceneInfo = Item.SceneInfo;
			const FUpdateInstanceCommand &UpdateInstance = Item.Payload;
			FScopeCycleCounter Context(PrimitiveSceneProxy->GetStatId());

			QueueFlushRuntimeVirtualTexture(PrimitiveSceneInfo);

			// TODO: no need to do this if only the payload size changed, we only need it because the MDC stores the instance count!
			//       Better yet: don't update MDCs on instance data change as we can pull it from elsewhere.
			const bool bInstanceDataAllocationChanged = PrimitiveSceneInfo->GetInstanceSceneDataOffset() == INDEX_NONE;
			const bool bUpdateStaticDrawLists = !PrimitiveSceneProxy->StaticElementsAlwaysUseProxyPrimitiveUniformBuffer() 
				// Re-cache if instance count changed & it is not promising to get instance count from the Scene OR it is Nanite (which does not have MDCs anyway and is GPU-Driven)
				|| (bInstanceDataAllocationChanged && !PrimitiveSceneProxy->DoesMeshBatchesUseSceneInstanceCount() && !PrimitiveSceneProxy->IsNaniteMesh())
				// In the mobile path, the call to UpdateInstances_RenderThread may/will update the vertex buffers, which leads to stale buffer references in the MDCs (TODO, make this not the case)
				|| !GPUScene.IsEnabled();

			if (QueueAddToScene(PrimitiveSceneInfo))
			{
				PrimitiveSceneInfo->RemoveFromScene(bUpdateStaticDrawLists);
			}

			// If it was not queued to add the static meshes, do so now and remove them (this may happen if e.g., a transform update happened in the same frame)
			if (bUpdateStaticDrawLists && !PrimitiveSceneInfo->bPendingAddStaticMeshes)
			{
				PrimitiveSceneInfo->RemoveStaticMeshes();
				QueueAddStaticMeshes(PrimitiveSceneInfo);
			}

#if RHI_RAYTRACING
			if (!PrimitiveSceneInfo->bPendingAddStaticMeshes)
			{
				UpdateCachedRayTracingState(PrimitiveSceneProxy);
			}
#endif

			// Update the Proxy's data.
			PrimitiveSceneProxy->UpdateInstances_RenderThread(GraphBuilder.RHICmdList, UpdateInstance.WorldBounds, UpdateInstance.LocalBounds);

			if (!RHISupportsVolumeTextures(GetFeatureLevel())
				&& (PrimitiveSceneProxy->IsMovable() || PrimitiveSceneProxy->NeedsUnbuiltPreviewLighting() || PrimitiveSceneProxy->GetLightmapType() == ELightmapType::ForceVolumetric))
			{
				PrimitiveSceneInfo->MarkIndirectLightingCacheBufferDirty();
			}

			if (bInstanceDataAllocationChanged)
			{
				DistanceFieldSceneData.RemovePrimitive(PrimitiveSceneInfo);
				DistanceFieldSceneData.AddPrimitive(PrimitiveSceneInfo);

				LumenRemovePrimitive(PrimitiveSceneInfo, PrimitiveSceneInfo->GetIndex());
				LumenAddPrimitive(PrimitiveSceneInfo);
			}
			else
			{
				// TODO: should modify the batched data to make this possible to discern
				GPUScene.AddPrimitiveToUpdate(PrimitiveSceneInfo->GetPersistentIndex(), EPrimitiveDirtyState::ChangedAll);

				DistanceFieldSceneData.UpdatePrimitive(PrimitiveSceneInfo);
				LumenUpdatePrimitive(PrimitiveSceneInfo);
			}

#if RHI_RAYTRACING
			bNeedPathTracedInvalidation = bNeedPathTracedInvalidation || PrimitiveSceneInfo->bIsRelevantToPathTracing;
#endif
		}

#if RHI_RAYTRACING
		{
			UpdateRayTracingGroupBounds_UpdatePrimitives(UpdatedInstances);
		}
#endif
	}

	// Determine the test visible vs. always visible primitive index ranges
	PrimitivesAlwaysVisibleOffset = ~0u;

// TODO: Support skip always visible in the editor (need to handle dynamic relevance)
#if !WITH_EDITOR
	if (GVisibilitySkipAlwaysVisible != 0)
	{
		uint32 NextTypeOffset = 0;
		for (int32 TypeOffsetIndex = 0; TypeOffsetIndex < TypeOffsetTable.Num(); ++TypeOffsetIndex)
		{
			const FTypeOffsetTableEntry& TypeEntry = TypeOffsetTable[TypeOffsetIndex];

		#if UE_BUILD_DEBUG
			// Sanity check
			checkSlow(Primitives[NextTypeOffset]->Proxy == PrimitiveSceneProxies[NextTypeOffset]);

			// Sanity check
			const FPrimitiveSceneProxyType Type = FPrimitiveSceneProxyType(PrimitiveSceneProxies[NextTypeOffset]);
			checkfSlow (Type == TypeEntry.PrimitiveSceneProxyType, TEXT("TypeHash: %zu not matching TypeOffsetTable, expected: %zu"), Type.ProxyTypeHash, TypeEntry.PrimitiveSceneProxyType.ProxyTypeHash);
		#endif

			if (PrimitiveSceneProxies[NextTypeOffset]->IsAlwaysVisible())
			{
				PrimitivesAlwaysVisibleOffset = NextTypeOffset;
				break;
			}

			NextTypeOffset = TypeEntry.Offset;
		}

	#if 0
		for (int32 Test = 0; Test < PrimitiveSceneProxies.Num(); ++Test)
		{
			const FPrimitiveSceneProxy* TestProxy = PrimitiveSceneProxies[Test];
			const bool AlwaysVisible = TestProxy->IsAlwaysVisible();
			const bool IsNanite = TestProxy->IsNaniteMesh();

			if (uint32(Test) < PrimitivesAlwaysVisibleOffset)
			{
				check(!AlwaysVisible);
			}
			else
			{
				check(AlwaysVisible);
				check(IsNanite);
			}
		}
	#endif

		// Align up to next full dword - this is to avoid having a single dword spanning "tested" and "always visible" primitives,
		// making the lockless parallel calculations much more efficient. This will push a few (<32) primitives from always visible
		// into the tested path, but this is not a big deal.
		if (PrimitivesAlwaysVisibleOffset != ~0u)
		{
			PrimitivesAlwaysVisibleOffset = (PrimitivesAlwaysVisibleOffset + uint32(NumBitsPerDWORD) - 1u) & ~(uint32(NumBitsPerDWORD) - 1u);
			if (int32(PrimitivesAlwaysVisibleOffset) >= Primitives.Num())
			{
				PrimitivesAlwaysVisibleOffset = ~0u;
			}
		}
	}
#endif // !WITH_EDITOR

	// Allocate all instance slots. Needs to happen after the instance data is updated since that may change the counts.
	FPrimitiveSceneInfo::AllocateGPUSceneInstances(this, PendingAllocateInstanceIds);

	if (SceneInfosWithAddToScene.Num() > 0)
	{
		FPrimitiveSceneInfo::AddToScene(this, SceneInfosWithAddToScene);
	}

	ProcessLevelCommands(FLevelCommand::EOp::Add);

	SceneUpdateChangeSetStorage.AddedPrimitiveIds.Reserve(AddedPrimitiveSceneInfos.Num());
	for (FPrimitiveSceneInfo* PrimitiveSceneInfo : AddedPrimitiveSceneInfos)
	{
		SceneUpdateChangeSetStorage.AddedPrimitiveIds.Add(PrimitiveSceneInfo->GetPersistentIndex());
	}

	GPUScene.OnPostSceneUpdate(GraphBuilder, SceneUpdateChangeSetStorage.GetPostUpdateSet());	

#if RHI_RAYTRACING
	RayTracingSBT.FlushAllocationsToClear(GraphBuilder.RHICmdList);
#endif // RHI_RAYTRACING

	SceneExtensionsUpdaters.PostSceneUpdate(GraphBuilder, SceneUpdateChangeSetStorage.GetPostUpdateSet());

	const bool bAsyncCacheMeshDrawCommands = EnumHasAnyFlags(Parameters.AsyncOps, EUpdateAllPrimitiveSceneInfosAsyncOps::CacheMeshDrawCommands) && GRHISupportsMultithreadedShaderCreation;

	auto UpdateCullDistances = [&UpdatedInstanceCullDistance]()
	{
		for (const auto& CullDistance : UpdatedInstanceCullDistance)
		{
			FPrimitiveSceneProxy* SceneProxy = CullDistance.SceneInfo->Proxy;
			float StartCullDistance = CullDistance.Payload.Value.X;
			float EndCullDistance = CullDistance.Payload.Value.Y;
		
			SceneProxy->SetInstanceCullDistance_RenderThread(StartCullDistance, EndCullDistance);
		}
	};

	if (GSafeCullDistanceUpdate)
	{
		UpdateCullDistances();
	}

	UE::Tasks::FTask AddStaticMeshesTask = GraphBuilder.AddCommandListSetupTask(
		[this, AddStaticMeshes = CopyTemp(SceneInfosWithStaticDrawListUpdate), SceneInfosWithFlushRuntimeVirtualTexture = MoveTemp(SceneInfosWithFlushRuntimeVirtualTexture), &SceneInfosWithAddToScene]
			(FRHICommandListBase& RHICmdList) mutable
	{
		SCOPED_NAMED_EVENT(StaticMeshUpdate, FColor::Emerald);

		if (AddStaticMeshes.Num() > 0)
		{
			FPrimitiveSceneInfo::AddStaticMeshes(RHICmdList, this, AddStaticMeshes, false);
		}

		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : SceneInfosWithFlushRuntimeVirtualTexture)
		{
			PrimitiveSceneInfo->FlushRuntimeVirtualTexture();
			PrimitiveSceneInfo->bPendingFlushRuntimeVirtualTexture = false;
		}

	}, MakeArrayView({ UpdateUniformExpressionsTask, SkeletalMeshSubmitData->GetInlineTask() }), UE::Tasks::ETaskPriority::High, bAsyncCacheMeshDrawCommands);

	UpdateReflectionSceneData(this);

	{
		SCOPED_NAMED_EVENT(UpdateStaticMeshes, FColor::Emerald);
		const bool bLastFrameShouldRenderSkylightInBasePass = bCachedShouldRenderSkylightInBasePass;
		bCachedShouldRenderSkylightInBasePass = ShouldRenderSkylightInBasePass(false);

		if (bCachedShouldRenderSkylightInBasePass != bLastFrameShouldRenderSkylightInBasePass)
		{
			// Mark the scene as needing static draw lists to be recreated if needed
			// The base pass chooses shaders based on whether there's a skylight in the scene, and that is cached in static draw lists
			UE_CLOGF(!GIsEditor, LogRenderer, Log, "Forcing update for all mesh draw commands: SkyLight change");
			bScenesPrimitivesNeedStaticMeshElementUpdate = true;
		}

		const bool bLastFrameSkyLightRealTimeCapture = bCachedSkyLightRealTimeCapture;
		bCachedSkyLightRealTimeCapture = CanSampleSkyLightRealTimeCaptureData();
		if (bCachedSkyLightRealTimeCapture != bLastFrameSkyLightRealTimeCapture)
		{
			// When real time capture on the sky light is toggled, we also need to recreate draw list for parameters.
			UE_CLOGF(!GIsEditor, LogRenderer, Log, "Forcing update for all mesh draw commands: SkyLight real-time capture change");
			bScenesPrimitivesNeedStaticMeshElementUpdate = true;
		}

		if (Parameters.ViewUpdateChangeSet && Parameters.ViewUpdateChangeSet->bIsRequiresDebugMaterialChanged)
		{
			UE_CLOGF(!GIsEditor, LogRenderer, Log, "Forcing update for all mesh draw commands: Scene view debug rendering change");
			bScenesPrimitivesNeedStaticMeshElementUpdate = true;
		}

		if (GVRSImageManager.GetNeedStaticMeshUpdate())
		{
			UE_CLOGF(!GIsEditor, LogRenderer, Log, "Forcing update for all mesh draw commands: Variable rate shading toggled");
			bScenesPrimitivesNeedStaticMeshElementUpdate = true;

			GVRSImageManager.SetNeedStaticMeshUpdate(false);
		}

		if (bScenesPrimitivesNeedStaticMeshElementUpdate || CachedDefaultBasePassDepthStencilAccess != DefaultBasePassDepthStencilAccess)
		{
			// Mark all primitives as needing an update
			PrimitivesNeedingStaticMeshUpdate.Init(true, PrimitivesNeedingStaticMeshUpdate.Num());

			bScenesPrimitivesNeedStaticMeshElementUpdate = false;
			CachedDefaultBasePassDepthStencilAccess = DefaultBasePassDepthStencilAccess;
		}

		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : SceneInfosWithStaticDrawListUpdate)
		{
			PrimitivesNeedingStaticMeshUpdate[PrimitiveSceneInfo->PackedIndex] = false;
		}

		for (TConstSetBitIterator<> BitIt(PrimitivesNeedingStaticMeshUpdate); BitIt; ++BitIt)
		{
			const int32 Index = BitIt.GetIndex();
			FPrimitiveSceneInfo* Primitive = Primitives[Index];

			Primitive->RemoveCachedMeshDrawCommands();
			Primitive->RemoveCachedNaniteMaterialBins();
		#if RHI_RAYTRACING
			Primitive->RemoveCachedRayTracingPrimitives();
		#endif
			SceneInfosWithStaticDrawListUpdate.Emplace(Primitive);
			PrimitivesNeedingStaticMeshUpdate[Index] = false;
		}
	}

	// LPI creation needs to launch after the static mesh update as it can call RequestStaticMeshUpdate() which modifies PrimitivesNeedingStaticMeshUpdate.
	CreateLightPrimitiveInteractionsTask = GraphBuilder.AddSetupTask([this, &SceneInfosWithAddToScene]
	{
		SCOPED_NAMED_EVENT(CreateLightPrimitiveInteractions, FColor::Emerald);

		bool bSkipNaniteLPIs = ShouldSkipNaniteLPIs(GetShaderPlatform());

		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : SceneInfosWithAddToScene)
		{
			FPrimitiveSceneProxy* Proxy = PrimitiveSceneInfo->Proxy;
			if (Proxy->GetLightingChannelMask() != 0)
			{
				// Don't create LPIs for Nanite
				if (bSkipNaniteLPIs && Proxy->IsNaniteMesh())
				{
					continue;
				}

				const FBoxSphereBounds& Bounds = Proxy->GetBounds();
				const FPrimitiveSceneInfoCompact PrimitiveSceneInfoCompact(PrimitiveSceneInfo);

				if(DoesPlatformNeedLocalLightPrimitiveInteraction(GetShaderPlatform()))
				{
					// Find local lights that affect the primitive in the light octree.
					LocalShadowCastingLightOctree.FindElementsWithBoundsTest(Bounds.GetBox(), [&PrimitiveSceneInfoCompact](const FLightSceneInfoCompact& LightSceneInfoCompact)
					{
						LightSceneInfoCompact.LightSceneInfo->CreateLightPrimitiveInteraction(LightSceneInfoCompact, PrimitiveSceneInfoCompact);
					});
				}
				// Also loop through non-local (directional) shadow-casting lights
				for (auto LightID : DirectionalShadowCastingLightIDs)
				{
					const FLightSceneInfoCompact& LightSceneInfoCompact = Lights[LightID];
					LightSceneInfoCompact.LightSceneInfo->CreateLightPrimitiveInteraction(LightSceneInfoCompact, PrimitiveSceneInfoCompact);
				}
			}			
		}

	}, EnumHasAnyFlags(Parameters.AsyncOps, EUpdateAllPrimitiveSceneInfosAsyncOps::CreateLightPrimitiveInteractions));

	if (bScenesPrimitivesNeedStaticMeshElementUpdate)
	{
		for (int32 PrimitiveIndex = 0; PrimitiveIndex < Primitives.Num(); PrimitiveIndex++)
		{
			// HACK: Update Nanite primitives that need re-caching in GPU Scene
			// TODO: Should be able to remove this after the move to compute materials.
			if (PrimitiveSceneProxies[PrimitiveIndex] && PrimitiveSceneProxies[PrimitiveIndex]->IsNaniteMesh())
			{
				GPUScene.AddPrimitiveToUpdate(Primitives[PrimitiveIndex]->GetPersistentIndex(), EPrimitiveDirtyState::ChangedOther);
			}
		}
	}

#if RHI_RAYTRACING
	if (IsRayTracingEnabled(GetShaderPlatform()))
	{
		// Nanite raytracing manager update must run before GPUScene update since it can modify primitive data
		Nanite::GRayTracingManager.Update(GraphBuilder.RHICmdList);
	}
#endif

	if (SceneInfosWithStaticDrawListUpdate.Num() > 0)
	{
		CacheMeshDrawCommandsTask = GraphBuilder.AddSetupTask([this, &SceneInfosWithStaticDrawListUpdate]
		{
			FPrimitiveSceneInfo::CacheMeshDrawCommands(this, SceneInfosWithStaticDrawListUpdate);

		}, MakeArrayView({ AddStaticMeshesTask, IsMobilePlatform(GetShaderPlatform()) ? CreateLightPrimitiveInteractionsTask : UE::Tasks::FTask() }), UE::Tasks::ETaskPriority::Normal, bAsyncCacheMeshDrawCommands);

		CacheNaniteMaterialBinsTask = GraphBuilder.AddSetupTask([this, &SceneInfosWithStaticDrawListUpdate]
		{
			FPrimitiveSceneInfo::CacheNaniteMaterialBins(this, SceneInfosWithStaticDrawListUpdate);

		}, AddStaticMeshesTask, UE::Tasks::ETaskPriority::Normal, bAsyncCacheMeshDrawCommands);

#if RHI_RAYTRACING
		CacheRayTracingPrimitivesTask = GraphBuilder.AddSetupTask([this, &SceneInfosWithStaticDrawListUpdate]
		{
			FPrimitiveSceneInfo::CacheRayTracingPrimitives(this, SceneInfosWithStaticDrawListUpdate);

		}, AddStaticMeshesTask, UE::Tasks::ETaskPriority::Normal, bAsyncCacheMeshDrawCommands);
#endif
	}

	for (const auto& Item : SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateCustomPrimitiveData>())
	{
		FPrimitiveSceneProxy* PrimitiveSceneProxy = Item.SceneInfo->Proxy;
		FScopeCycleCounter Context(PrimitiveSceneProxy->GetStatId());
		PrimitiveSceneProxy->CustomPrimitiveData = Item.Payload.Value;
		PrimitivesNeedingUniformBufferUpdate[Item.SceneInfo->PackedIndex] = true;
	}

	if (auto NaniteMaterialsUpdater = SceneExtensionsUpdaters.GetUpdaterPtr<Nanite::FMaterialsSceneExtension::FUpdater>())
	{
		NaniteMaterialsUpdater->PostCacheNaniteMaterialBins(GraphBuilder, SceneInfosWithStaticDrawListUpdate);
	}
	
	if (auto SkinningUpdater = SceneExtensionsUpdaters.GetUpdaterPtr<FSkinningSceneExtension::FUpdater>())
	{
		SkinningUpdater->PostMeshUpdate(GraphBuilder, SceneInfosWithStaticDrawListUpdate, SkeletalMeshSubmitData);
	}

	for (FPrimitiveSceneInfo* PrimitiveSceneInfo : AddedPrimitiveSceneInfos)
	{
		// Set LOD parent information if valid
		PrimitiveSceneInfo->LinkLODParentComponent();

		// Update scene LOD tree
		SceneLODHierarchy.UpdateNodeSceneInfo(PrimitiveSceneInfo->PrimitiveComponentId, PrimitiveSceneInfo);
	}

	auto UpdatedAttachmentRoots = SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateAttachmentRootData>();
	auto DistanceFieldSceneDataUpdates = SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateDistanceFieldSceneData>();
	auto UpdatedOcclusionBoundsSlacks = SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateOcclusionBoundsSlacksData>();
	auto UpdatedDrawDistance =  SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateDrawDistanceData>();

	for (const auto& Item : UpdatedAttachmentRoots)
	{
		FPrimitiveSceneInfo* PrimitiveSceneInfo = Item.SceneInfo;
		PrimitiveSceneInfo->UnlinkAttachmentGroup();
		PrimitiveSceneInfo->LightingAttachmentRoot = Item.Payload.Value;
		PrimitiveSceneInfo->LinkAttachmentGroup();
	}

	for (const auto& Item : DistanceFieldSceneDataUpdates)
	{
		DistanceFieldSceneData.UpdatePrimitive(Item.SceneInfo);
	}

	for (const auto& OccSlackDelta : UpdatedOcclusionBoundsSlacks)
	{
		const FPrimitiveSceneProxy* SceneProxy = OccSlackDelta.SceneInfo->Proxy;
		const FPrimitiveSceneInfo* SceneInfo = OccSlackDelta.SceneInfo;
		float Delta = OccSlackDelta.Payload.Value;

		FBoxSphereBounds NewOccBounds;
		if (SceneProxy->HasCustomOcclusionBounds())
		{
			NewOccBounds = SceneProxy->GetCustomOcclusionBounds();
		}
		else
		{
			NewOccBounds = SceneProxy->GetBounds();
		}

		PrimitiveOcclusionBounds[SceneInfo->PackedIndex] = NewOccBounds.ExpandBy(OCCLUSION_SLOP + Delta);
	}

	if (!GSafeCullDistanceUpdate)
	{
		UpdateCullDistances();
	}

	for (const auto& DrawDistance : UpdatedDrawDistance)
	{
		FPrimitiveSceneProxy* SceneProxy = DrawDistance.SceneInfo->Proxy;
		FPrimitiveSceneInfo* SceneInfo = DrawDistance.SceneInfo;
		float MinDrawDistance = DrawDistance.Payload.Value.X;
		float MaxDrawDistance = DrawDistance.Payload.Value.Y;
		float VirtualTextureMaxDrawDistance = DrawDistance.Payload.Value.Z;

		SceneProxy->SetDrawDistance_RenderThread(MinDrawDistance, MaxDrawDistance, VirtualTextureMaxDrawDistance);

		if (SceneInfo->PackedIndex != INDEX_NONE)
		{
			PrimitiveBounds[SceneInfo->PackedIndex].MinDrawDistance = SceneProxy->GetMinDrawDistance();
			PrimitiveBounds[SceneInfo->PackedIndex].MaxDrawDistance = SceneProxy->GetMaxDrawDistance();
			PrimitiveBounds[SceneInfo->PackedIndex].MaxCullDistance = SceneProxy->GetMaxDrawDistance();
		}

		// Update the primitive info in octree.
		if (SceneInfo->OctreeId.IsValidId())
		{
			FPrimitiveSceneInfoCompact& CompactPrimitiveSceneInfo = PrimitiveOctree.GetElementById(SceneInfo->OctreeId);
			CompactPrimitiveSceneInfo.MinDrawDistance = SceneProxy->GetMinDrawDistance();
			CompactPrimitiveSceneInfo.MaxDrawDistance = SceneProxy->GetMaxDrawDistance();
		}

		DistanceFieldSceneData.UpdatePrimitive(SceneInfo);
	}

	if (Parameters.Callbacks.PostStaticMeshUpdate)
	{
		Parameters.Callbacks.PostStaticMeshUpdate(AddStaticMeshesTask);
	}

	{
		SCOPED_NAMED_EVENT(UpdateUniformBuffers, FColor::Emerald);
		TArray<FPrimitiveSceneProxy*, SceneRenderingAllocator> ProxiesToUpdate;

		for (TConstSetBitIterator<> BitIt(PrimitivesNeedingUniformBufferUpdate); BitIt; ++BitIt)
		{
			const int32 Index = BitIt.GetIndex();
			FPrimitiveSceneInfo* Primitive = Primitives[Index];
			PrimitivesNeedingUniformBufferUpdate[Index] = false;
			ProxiesToUpdate.Emplace(Primitive->Proxy);
			GPUScene.AddPrimitiveToUpdate(Primitive->GetPersistentIndex(), EPrimitiveDirtyState::ChangedAll);
		}

		GraphBuilder.AddCommandListSetupTask([this, ProxiesToUpdate = MoveTemp(ProxiesToUpdate)](FRHICommandList& RHICmdList)
		{
			SCOPED_NAMED_EVENT(AsyncUpdateUniformBuffers, FColor::Emerald);
			FRHICommandListScopedFence ScopedFence(RHICmdList);

			for (FPrimitiveSceneProxy* Proxy : ProxiesToUpdate)
			{
				Proxy->UpdateUniformBuffer(RHICmdList);
			}
		});
	}

	if (auto SkinningUpdater = SceneExtensionsUpdaters.GetUpdaterPtr<FSkinningSceneExtension::FUpdater>())
	{
		SkinningUpdater->PreGPUSceneUpdate(GraphBuilder, Parameters.ViewUpdateChangeSet ? &Parameters.ViewUpdateChangeSet->CurrentTime : nullptr);
	}

	{
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, UpdateGPUScene);
		RDG_EVENT_SCOPE_STAT(GraphBuilder, GPUSceneUpdate, "GPUSceneUpdate");

		FRDGExternalAccessQueue ExternalAccessQueue;

		GPUScene.Update(GraphBuilder, ExternalAccessQueue, SceneUpdateChangeSetStorage.PrimitiveUpdates.GetRangeView<FUpdateInstanceFromComputeCommand>(), Parameters.GPUSceneUpdateTaskPrerequisites, SceneExtensionsUpdaters);

		ExternalAccessQueue.Submit(GraphBuilder);
	}

	SceneExtensionsUpdaters.PostGPUSceneUpdate(GraphBuilder);

	GraphBuilder.AddSetupTask([DeletedPrimitiveSceneInfos = MoveTemp(DeletedPrimitiveSceneInfos)]
	{
		SCOPED_NAMED_EVENT(FScene_DeletePrimitiveSceneInfo, FColor::Red);
		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : DeletedPrimitiveSceneInfos)
		{
			// It is possible that the HitProxies list isn't empty if PrimitiveSceneInfo was Added/Removed in same frame
			// Delete the PrimitiveSceneInfo on the game thread after the rendering thread has processed its removal.
			// This must be done on the game thread because the hit proxy references (and possibly other members) need to be freed on the game thread.
			struct DeferDeleteHitProxies : FDeferredCleanupInterface
			{
				DeferDeleteHitProxies(TArray<TRefCountPtr<HHitProxy>>&& InHitProxies) : HitProxies(MoveTemp(InHitProxies)) {}
				TArray<TRefCountPtr<HHitProxy>> HitProxies;
			};

			BeginCleanup(new DeferDeleteHitProxies(MoveTemp(PrimitiveSceneInfo->HitProxies)));
			// free the primitive scene proxy.
			delete PrimitiveSceneInfo->Proxy;
			delete PrimitiveSceneInfo;
		}
	});

	for (IComputeTaskWorker* ComputeTaskWorker : ComputeTaskWorkers)
	{
		if (ComputeTaskWorker->HasWork(ComputeTaskExecutionGroup::EndOfFrameUpdate))
		{
			FComputeContext Context
			{
				.GraphBuilder = GraphBuilder,
				.ExecutionGroupName = ComputeTaskExecutionGroup::EndOfFrameUpdate,
				.FeatureLevel = FeatureLevel,
				.Scene = this,
			};

			ComputeTaskWorker->SubmitWork(Context);
		}
	}

	AddStaticMeshesTask.Wait();

	if (bNeedPathTracedInvalidation)
	{
		InvalidatePathTracedOutput();
	}
	LevelCommands.Empty();

	LastUpdateFrameCounter = GFrameCounterRenderThread;

	TRACE_COUNTER_SET(SceneLights, Lights.Num());
	TRACE_COUNTER_SET(ScenePrimitives, Primitives.Num());
	TRACE_COUNTER_SET(ScenePrimitivesArrayMax, Primitives.Max());

#if DO_GUARD_SLOW
	for (const FPrimitiveSceneInfo* PrimitiveSceneInfo : Primitives)
	{
		checkSlow(PrimitiveSceneInfo->PackedIndex != INDEX_NONE);
		checkSlow(PrimitiveSceneInfo->PackedIndex < Primitives.Num());
		checkSlow(PrimitiveSceneInfo->PersistentIndex.Index != INDEX_NONE);
		checkSlow(PersistentPrimitiveIdToIndexMap[PrimitiveSceneInfo->PersistentIndex.Index] == PrimitiveSceneInfo->PackedIndex);
	}
#endif
}

bool FScene::ShouldRenderSkylightInBasePass(bool bIsTranslucent) const
{
	if (IsMobilePlatform(GetShaderPlatform()))
	{
		return false;
	}
	else
	{
		bool bRenderSkyLight = SkyLight && !SkyLight->bHasStaticLighting && !(ShouldRenderRayTracingSkyLight(SkyLight, GetShaderPlatform()) && !IsForwardShadingEnabled(GetShaderPlatform()));

		if (bIsTranslucent)
		{
			// Both stationary and movable skylights are applied in base pass for translucent materials
			bRenderSkyLight = bRenderSkyLight
				&& (FReadOnlyCVARCache::EnableStationarySkylight() || !SkyLight->bWantsStaticShadowing);
		}
		else
		{
			// For opaque materials, stationary skylight is applied in base pass but movable skylight
			// is applied in a separate render pass (bWantssStaticShadowing means stationary skylight)
			bRenderSkyLight = bRenderSkyLight
				&& ((FReadOnlyCVARCache::EnableStationarySkylight() && SkyLight->bWantsStaticShadowing)
					|| (!SkyLight->bWantsStaticShadowing
						&& IsForwardShadingEnabled(GetShaderPlatform())));
		}

		return bRenderSkyLight;
	}
}

void FScene::AddGPUSkinCacheAsyncComputeWait(FRDGBuilder& GraphBuilder) const
{
	if (GPUSkinCache)
	{
		GPUSkinCache->AddAsyncComputeWait(GraphBuilder);
	}
}

/**
 * Dummy NULL scene interface used by dedicated servers.
 */
class FNULLSceneInterface : public FSceneInterface
{
public:
	FNULLSceneInterface(UWorld* InWorld, bool bCreateFXSystem )
		:	FSceneInterface(GMaxRHIFeatureLevel)
		,	World( InWorld )
		,	FXSystem( nullptr )
	{
		World->Scene = this;

		if (bCreateFXSystem)
		{
			World->CreateFXSystem();
		}
		else
		{
			World->FXSystem = nullptr;
			SetFXSystem(nullptr);
		}
	}

	virtual void AddPrimitive(UPrimitiveComponent* Primitive) override {}
	virtual void RemovePrimitive(UPrimitiveComponent* Primitive) override {}
	virtual void ReleasePrimitive(UPrimitiveComponent* Primitive) override {}
	virtual void BatchAddPrimitives(TArrayView<UPrimitiveComponent*> InPrimitives) override {}
	virtual void BatchRemovePrimitives(TArrayView<UPrimitiveComponent*> InPrimitives) override {}
	virtual void BatchReleasePrimitives(TArrayView<UPrimitiveComponent*> InPrimitives) override {}
	virtual void UpdateAllPrimitiveSceneInfos(FRDGBuilder& GraphBuilder, EUpdateAllPrimitiveSceneInfosAsyncOps = EUpdateAllPrimitiveSceneInfosAsyncOps::None) override {}
	virtual FPrimitiveSceneInfo* GetPrimitiveSceneInfo(int32 PrimiteIndex) const final { return nullptr; }
	virtual FPrimitiveSceneInfo* GetPrimitiveSceneInfo(FPrimitiveComponentId PrimitiveId) const final { return nullptr; }
	virtual FPrimitiveSceneInfo* GetPrimitiveSceneInfo(const FPersistentPrimitiveIndex& PersistentPrimitiveIndex) const final { return nullptr; }

	/** Updates the transform of a primitive which has already been added to the scene. */
	virtual void UpdatePrimitiveTransform(UPrimitiveComponent* Primitive) override {}
	virtual void UpdatePrimitiveInstances(UPrimitiveComponent* Primitive) override {}
	virtual void UpdatePrimitiveInstancesFromCompute(FPrimitiveSceneDesc* Primitive, FGPUSceneWriteDelegate&& DataWriterGPU) override {}
	virtual void UpdatePrimitiveOcclusionBoundsSlack(UPrimitiveComponent* Primitive, float NewSlack) override {}
	virtual void UpdatePrimitiveDrawDistance(UPrimitiveComponent* Primitive, float MinDrawDistance, float MaxDrawDistance, float VirtualTextureMaxDrawDistance) override {}
	virtual void UpdateInstanceCullDistance(UPrimitiveComponent* Primitive, float StartCullDistance, float EndCullDistance) {}
	virtual void UpdatePrimitiveAttachment(UPrimitiveComponent* Primitive) override {}
	virtual void UpdateCustomPrimitiveData(UPrimitiveComponent* Primitive) override {}

	virtual void AddLight(ULightComponent* Light) override {}
	virtual void RemoveLight(ULightComponent* Light) override {}
	virtual void AddInvisibleLight(ULightComponent* Light) override {}
	virtual void BatchAddLights(TArrayView<FLightSceneDesc> InLights) override {}
	virtual void BatchAddInvisibleLights(TArrayView<FLightSceneDesc> InLights) override {}
	virtual void BatchRemoveLights(TArrayView<FLightSceneDesc> InLights) override {}
	virtual void SetSkyLight(FSkyLightSceneProxy* Light) override {}
	virtual void DisableSkyLight(FSkyLightSceneProxy* Light) override {}
	virtual bool HasSkyLightRequiringLightingBuild() const { return false; }
	virtual bool HasAtmosphereLightRequiringLightingBuild() const { return false; }

	virtual void AddDecal(UDecalComponent*) override {}
	virtual void RemoveDecal(UDecalComponent*) override {}
	virtual void UpdateDecalTransform(UDecalComponent* Decal) override {}
	virtual void UpdateDecalFadeOutTime(UDecalComponent* Decal) override {};
	virtual void UpdateDecalFadeInTime(UDecalComponent* Decal) override {};
	virtual void BatchUpdateDecals(TArray<FDeferredDecalUpdateParams>&& UpdateParams) override {}
	virtual void GetPrimitiveUniformShaderParameters_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool& bHasPrecomputedVolumetricLightmap, FMatrix& PreviousLocalToWorld, int32& SingleCaptureIndex, bool& OutputVelocity) const override 
	{
		PreviousLocalToWorld = FMatrix::Identity; 
		SingleCaptureIndex = 0;
		bHasPrecomputedVolumetricLightmap = OutputVelocity = false;
	}
	virtual void BuildPrimitiveUniformShaderParameters_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, FPrimitiveUniformShaderParametersBuilder& Builder) const override 
	{
		Builder.Defaults();
	};

	/** Updates the transform of a light which has already been added to the scene. */
	virtual void UpdateLightTransform(ULightComponent* Light) override {}
	virtual void UpdateLightColorAndBrightness(ULightComponent* Light) override {}

	virtual void UpdateLightProxy(ULightComponent* Light, TFunction<void(FLightSceneProxy*)>&& Func) override {}

	virtual void AddExponentialHeightFog(uint64 Id, const FExponentialHeightFogDynamicState& State) override {}
	virtual void RemoveExponentialHeightFog(uint64 Id) override {}
	virtual bool HasAnyExponentialHeightFog() const override { return false; }

	virtual void AddLocalFogVolume(class FLocalFogVolumeSceneProxy* FogProxy) override {}
	virtual void RemoveLocalFogVolume(class FLocalFogVolumeSceneProxy* FogProxy) override {}
	virtual bool HasAnyLocalFogVolume() const override { return false; }

	virtual void AddSkyAtmosphere(FSkyAtmosphereSceneProxy* SkyAtmosphereSceneProxy, bool bStaticLightingBuilt) override {}
	virtual void RemoveSkyAtmosphere(FSkyAtmosphereSceneProxy* SkyAtmosphereSceneProxy) override {}
	virtual FSkyAtmosphereRenderSceneInfo* GetSkyAtmosphereSceneInfo() override { return nullptr; }
	virtual const FSkyAtmosphereRenderSceneInfo* GetSkyAtmosphereSceneInfo() const override { return nullptr; }

	virtual void AddSparseVolumeTextureViewer(FSparseVolumeTextureViewerSceneProxy* SVTV) override {}
	virtual void RemoveSparseVolumeTextureViewer(FSparseVolumeTextureViewerSceneProxy* SVTV) override {}

	virtual IAnimBankTransformProvider* GetAnimBankTransformProvider() { return nullptr; }
	virtual IAnimRuntimeTransformProvider* GetAnimRuntimeTransformProvider() { return nullptr; }
	virtual IAnimSequenceTransformProvider* GetAnimSequenceTransformProvider() { return nullptr; }

	virtual void AddHairStrands(FHairStrandsInstance* Proxy) override {}
	virtual void RemoveHairStrands(FHairStrandsInstance* Proxy) override {}
	virtual void GetLightIESAtlasSlot(const FLightSceneProxy* Proxy, FLightRenderParameters* Out) override {}
	virtual void GetRectLightAtlasSlot(const FRectLightSceneProxy* Proxy, FLightRenderParameters* Out) override {}

	virtual void SetPhysicsField(FPhysicsFieldSceneProxy* PhysicsFieldSceneProxy) override {}
	virtual void ResetPhysicsField() override {}
	virtual void ShowPhysicsField() override {}
	virtual void UpdatePhysicsField(FRDGBuilder& GraphBuilder, FViewInfo& View) override {}

	virtual void AddVolumetricCloud(FVolumetricCloudSceneProxy* VolumetricCloudSceneProxy) override {}
	virtual void RemoveVolumetricCloud(FVolumetricCloudSceneProxy* VolumetricCloudSceneProxy) override {}
	virtual FVolumetricCloudRenderSceneInfo* GetVolumetricCloudSceneInfo() override { return NULL; }
	virtual const FVolumetricCloudRenderSceneInfo* GetVolumetricCloudSceneInfo() const override { return NULL; }

	virtual void AddWindSource(class UWindDirectionalSourceComponent* WindComponent) override {}
	virtual void RemoveWindSource(class UWindDirectionalSourceComponent* WindComponent) override {}
	virtual void UpdateWindSource(class UWindDirectionalSourceComponent* WindComponent) override {}
	virtual const TArray<class FWindSourceSceneProxy*>& GetWindSources_RenderThread() const override
	{
		static TArray<class FWindSourceSceneProxy*> NullWindSources;
		return NullWindSources;
	}
	virtual void GetWindParameters(const FVector& Position, FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const override { OutDirection = FVector(1.0f, 0.0f, 0.0f); OutSpeed = 0.0f; OutMinGustAmt = 0.0f; OutMaxGustAmt = 0.0f; }
	virtual void GetWindParameters_GameThread(const FVector& Position, FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const override { OutDirection = FVector(1.0f, 0.0f, 0.0f); OutSpeed = 0.0f; OutMinGustAmt = 0.0f; OutMaxGustAmt = 0.0f; }
	virtual void GetDirectionalWindParameters(FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const override { OutDirection = FVector(1.0f, 0.0f, 0.0f); OutSpeed = 0.0f; OutMinGustAmt = 0.0f; OutMaxGustAmt = 0.0f; }
	virtual void AddSpeedTreeWind(class FVertexFactory* VertexFactory, const class UStaticMesh* StaticMesh) override {}
	virtual void RemoveSpeedTreeWind_RenderThread(class FVertexFactory* VertexFactory, const class UStaticMesh* StaticMesh) override {}
	virtual void UpdateSpeedTreeWind(double CurrentTime) override {}
	virtual FRHIUniformBuffer* GetSpeedTreeUniformBuffer(const FVertexFactory* VertexFactory) const override { return nullptr; }

	virtual void Release() override {}

	/**
	 * Retrieves the lights interacting with the passed in primitive and adds them to the out array.
	 *
	 * @param	Primitive				Primitive to retrieve interacting lights for
	 * @param	RelevantLights	[out]	Array of lights interacting with primitive
	 */
	virtual void GetRelevantLights( UPrimitiveComponent* Primitive, TArray<const ULightComponent*>* RelevantLights ) const override {}

	virtual int32 GetRelevantLightCount(UPrimitiveComponent* Primitive) const override
	{
		return 0;
	}

	/**
	 * @return		true if hit proxies should be rendered in this scene.
	 */
	virtual bool RequiresHitProxies() const override
	{
		return false;
	}

	// Accessors.
	virtual class UWorld* GetWorld() const override
	{
		return World;
	}

	virtual FScene* GetRenderScene() override
	{
		return nullptr;
	}

	virtual const FScene* GetRenderScene() const override
	{
		return nullptr;
	}

	/**
	 * Sets the FX system associated with the scene.
	 */
	virtual void SetFXSystem( class FFXSystemInterface* InFXSystem ) override
	{
		FXSystem = InFXSystem;
	}

	/**
	 * Get the FX system associated with the scene.
	 */
	virtual class FFXSystemInterface* GetFXSystem() override
	{
		return FXSystem;
	}

	virtual bool HasAnyLights() const override { return false; }

	virtual TConstArrayView<FPrimitiveSceneProxy*> GetPrimitiveSceneProxies() const override
	{
		return TConstArrayView<FPrimitiveSceneProxy*>();
	}

	virtual TConstArrayView<FPrimitiveComponentId> GetScenePrimitiveComponentIds() const final
	{
		return TConstArrayView<FPrimitiveComponentId>();
	}

	virtual void AddPrimitive(FPrimitiveSceneDesc* Primitive) override {};
	virtual void RemovePrimitive(FPrimitiveSceneDesc* Primitive) override {};
	virtual void ReleasePrimitive(FPrimitiveSceneDesc* Primitive) override {};
	virtual void UpdatePrimitiveTransform(FPrimitiveSceneDesc* Primitive) override {};

	virtual void BatchAddPrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives) override {};
	virtual void BatchRemovePrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives) override {};
	virtual void BatchReleasePrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives) override {};

	virtual void BatchRemovePrimitives(TArray<FPrimitiveSceneProxy*>&& InPrimitives) override {};

	virtual void UpdateCustomPrimitiveData(FPrimitiveSceneDesc* Primitive, const FCustomPrimitiveData&) override {}
	virtual void UpdatePrimitiveInstances(FPrimitiveSceneDesc* Primitive) override {};

	virtual void UpdatePrimitivesIsDrawn_RenderThread(TArrayView<FPrimitiveSceneProxy*> PrimitiveSceneProxies, bool bIsDrawn) override {}

private:
	UWorld* World;
	class FFXSystemInterface* FXSystem;
};

FSceneInterface* FRendererModule::AllocateScene(UWorld* World, bool bInRequiresHitProxies, bool bCreateFXSystem, ERHIFeatureLevel::Type InFeatureLevel)
{
	LLM_SCOPE(ELLMTag::SceneRender);
	check(IsInGameThread());

	// Create a full fledged scene if we have something to render.
	if (GIsClient && FApp::CanEverRender() && !GUsingNullRHI)
	{
		FScene* NewScene = new FScene(World, bInRequiresHitProxies, GIsEditor && (!World || !World->IsGameWorld()), bCreateFXSystem, InFeatureLevel);
		AllocatedScenes.Add(NewScene);
		return NewScene;
	}
	// And fall back to a dummy/ NULL implementation for commandlets and dedicated server.
	else
	{
		return new FNULLSceneInterface(World, bCreateFXSystem);
	}
}

void FRendererModule::RemoveScene(FSceneInterface* Scene)
{
	check(IsInGameThread());
	SceneRemovedEvent.Broadcast(Scene);
	AllocatedScenes.Remove(Scene);
}

FSceneRemovedEvent::RegistrationType& FRendererModule::GetSceneRemovedEvent()
{
	return SceneRemovedEvent;
}

void FRendererModule::UpdateStaticDrawLists()
{
	ensureMsgf(false, TEXT("UpdateStaticDrawLists is no longer implemented and will be removed, see UpdateStaticMeshesForMaterials."));
}

void UpdateStaticMeshesForMaterials(const TArray<const FMaterial*>& MaterialResourcesToUpdate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateStaticMeshesForMaterials);

	TArray<UMaterialInterface*> UsedMaterials;
	TSet<UMaterialInterface*> UsedMaterialsDependencies;

	FObjectCacheContextScope ObjectCacheScope;

	TMap<FScene*, TArray<FPrimitiveSceneInfo*>> UsedPrimitives;
	
	for (IPrimitiveComponent* PrimitiveComponent: ObjectCacheScope.GetContext().GetPrimitiveComponents())
	{		
		if (PrimitiveComponent->IsRenderStateCreated() && PrimitiveComponent->GetSceneProxy() && PrimitiveComponent->GetSceneProxy()->GetPrimitiveSceneInfo()->IsIndexValid())
		{
			UsedMaterialsDependencies.Reset();
			UsedMaterials.Reset();

			// Note: relying on GetUsedMaterials to be accurate, or else we won't propagate to the right primitives and the renderer will crash later
			// FPrimitiveSceneProxy::VerifyUsedMaterial is used to make sure that all materials used for rendering are reported in GetUsedMaterials			
			PrimitiveComponent->GetUsedMaterials(UsedMaterials);

			for (UMaterialInterface* UsedMaterial : UsedMaterials)
			{
				if (UsedMaterial)
				{
					UsedMaterial->GetDependencies(UsedMaterialsDependencies);
				}
			}

			if (UsedMaterialsDependencies.Num() > 0)
			{
				for (const FMaterial* MaterialResourceToUpdate : MaterialResourcesToUpdate)
				{
					UMaterialInterface* UpdatedMaterialInterface = MaterialResourceToUpdate->GetMaterialInterface();

					if (UpdatedMaterialInterface)
					{
						if (UsedMaterialsDependencies.Contains(UpdatedMaterialInterface))
						{
							FPrimitiveSceneProxy* SceneProxy = PrimitiveComponent->GetSceneProxy();
							FPrimitiveSceneInfo* SceneInfo = SceneProxy->GetPrimitiveSceneInfo();
							FScene* Scene = SceneInfo->Scene;
							TArray<FPrimitiveSceneInfo*>& SceneInfos = UsedPrimitives.FindOrAdd(Scene);
							SceneInfos.Add(SceneInfo);
							break;
						}
					}
				}
			}
		}
	}

	UE::RenderCommandPipe::FSyncScope SyncScope;

	ENQUEUE_RENDER_COMMAND(FUpdateStaticMeshesForMaterials)(
	[UsedPrimitives = MoveTemp(UsedPrimitives)](FRHICommandListImmediate& RHICmdList) mutable
	{
		// Defer the caching until the next render tick, to make sure that all render components queued
		// for re-creation are processed. Otherwise, we may end up caching mesh commands from stale data.
		for (auto& SceneInfos: UsedPrimitives)
		{
			SceneInfos.Key->UpdateAllPrimitiveSceneInfos(RHICmdList);
		}
		for (auto& SceneInfos : UsedPrimitives)
		{
			TArray<FPrimitiveSceneInfo*>& SceneInfoArray = SceneInfos.Value;
			FPrimitiveSceneInfo::UpdateStaticMeshes(SceneInfos.Key, SceneInfoArray, EUpdateStaticMeshFlags::AllCommands, false);
		}
	});
}

void FRendererModule::UpdateStaticDrawListsForMaterials(const TArray<const FMaterial*>& Materials)
{
	// Update static meshes for a given set of materials in order to recache cached mesh draw commands.
	UpdateStaticMeshesForMaterials(Materials);
}

FSceneViewStateInterface* FRendererModule::AllocateViewState(ERHIFeatureLevel::Type FeatureLevel)
{
	return new FSceneViewState(FeatureLevel, nullptr);
}

FSceneViewStateInterface* FRendererModule::AllocateViewState(ERHIFeatureLevel::Type FeatureLevel, FSceneViewStateInterface* ShareOriginTarget)
{
	return new FSceneViewState(FeatureLevel, (FSceneViewState*)ShareOriginTarget);
}

TUniquePtr<ISceneRenderBuilder> FRendererModule::CreateSceneRenderBuilder(FSceneInterface* SceneInterface)
{
	return MakeUnique<FSceneRenderBuilder>(SceneInterface);
}

void FRendererModule::InvalidatePathTracedOutput(PathTracing::EInvalidateReason InvalidateReason)
{
	// AllocatedScenes is managed by the game thread

	// #jira UE-130700:
	// Because material updates call this function and could happen in parallel, we also allow the parallel game thread here.
	// We assume that no changes will be made to AllocatedScene during this time, otherwise locking would need to
	// be introduced (which could have performance implications). 

	check(IsInGameThread() || IsInParallelGameThread());
	for (FSceneInterface* Scene : AllocatedScenes)
	{
		Scene->InvalidatePathTracedOutput(InvalidateReason);
	}
}

uint32 FScene::GetFrameNumber() const
{
	if (IsInGameThread())
	{
		return SceneFrameNumber;
	}
	else
	{
		return SceneFrameNumberRenderThread;
	}
}

void FScene::IncrementFrameNumber()
{
	// Increment game-tread version
	++SceneFrameNumber;
	ENQUEUE_RENDER_COMMAND(SceneStartFrame)([this,NewNumber = SceneFrameNumber](FRHICommandListImmediate& RHICmdList)
	{
		SceneFrameNumberRenderThread = NewNumber;
	});
}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void FScene::DebugRender(TArrayView<FViewInfo> Views)
{
	// TODO: debug rendering should move to the renderer
	if (FShadowScene* ShadowScene = GetExtensionPtr<FShadowScene>())
	{
		ShadowScene->DebugRender(Views);
	}
}
#endif

bool FScene::AddCustomRenderPass(const FSceneViewFamily* ViewFamily, const FCustomRenderPassRendererInput& CustomRenderPassInput)
{
	CustomRenderPassRendererInputs.Add(CustomRenderPassInput);
	return true;
}

FVirtualShadowMapArrayCacheManager* FScene::GetVirtualShadowMapCache()
{
	return GetExtensionPtr<FVirtualShadowMapArrayCacheManager>();
}

void FScene::SwapMobileReflectionCaptureCachedShadowState()
{
	for (FLightSceneInfo* MobileDirectionalLightSceneInfo : MobileDirectionalLights)
	{
		if (MobileDirectionalLightSceneInfo && MobileDirectionalLightSceneInfo->Proxy->TimesliceAtlasFrame == ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFrame)
		{
			FReflectionEnvironmentCachedShadow& CachedShadow = ReflectionSceneData.RuntimeCaptureShadows.Shadows[MobileDirectionalLightSceneInfo->Proxy->TimesliceAtlasIndex[0]];

			// Generate payload for update command
			FUpdateLightTransformParameters TransformParameters;
			TransformParameters.LightToWorld = CachedShadow.LightToWorld;
			TransformParameters.Position     = CachedShadow.Position;

			// Store off the old transformation, so we can swap back to it later with a second call to this function
			CachedShadow.LightToWorld = MobileDirectionalLightSceneInfo->Proxy->LightToWorld;
			CachedShadow.Position     = MobileDirectionalLightSceneInfo->Proxy->Position;

			SceneLightInfoUpdates->Enqueue(MobileDirectionalLightSceneInfo, MoveTemp(TransformParameters));
		}
	}
}
