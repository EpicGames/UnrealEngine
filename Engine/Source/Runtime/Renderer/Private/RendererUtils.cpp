// Copyright Epic Games, Inc. All Rights Reserved.

#include "RendererUtils.h"
#include "RendererPrivateUtils.h"
#include "Nanite/NaniteRayTracing.h"
#include "RenderTargetPool.h"
#include "RHIDefinitions.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "VisualizeTexture.h"
#include "SystemTextures.h"
#include "UnifiedBuffer.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Substrate/Substrate.h"
#include "SceneTextures.h"

static int32 GSkipNaniteLPIs = 1;
static FAutoConsoleVariableRef CVarSkipNaniteLPIs(
	TEXT("r.SkipNaniteLPIs"),
	GSkipNaniteLPIs,
	TEXT("Skip Nanite primitives in the light-primitive interactions & the primitive octree as they perform GPU-driven culling separately.\n")
	TEXT(" Values:")
	TEXT("   1 - (auto, default) Skipping is auto-disabled if r.AllowStaticLighting is enabled for the project as it breaks some associated editor features otherwise.")
	TEXT("   2 - (forced) Skipping is always enabled regardless of r.AllowStaticLighting. May cause issues with static lighting. Use with care."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		// Needed because the primitives need to be re-added to the scene to be removed from the octree and to have existing LPIs cleaned up. And vice versa.
		// The cvar is not expected to be changed during runtime outside of testing.
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe);

bool ShouldSkipNaniteLPIs(EShaderPlatform ShaderPlatform)
{
	return (GSkipNaniteLPIs > 1 
		|| ( GSkipNaniteLPIs == 1 && !IsStaticLightingAllowed()))
		&& UseNanite(ShaderPlatform);
}

class FRTWriteMaskDecodeCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FRTWriteMaskDecodeCS);

	static const uint32 MaxRenderTargetCount = 4;
	static const uint32 ThreadGroupSizeX = 8;
	static const uint32 ThreadGroupSizeY = 8;

	class FNumRenderTargets : SHADER_PERMUTATION_RANGE_INT("NUM_RENDER_TARGETS", 1, MaxRenderTargetCount);
	using FPermutationDomain = TShaderPermutationDomain<FNumRenderTargets>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, ReferenceInput)
		SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(TextureMetadata, RTWriteMaskInputs, [MaxRenderTargetCount])
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<uint>, OutCombinedRTWriteMask)
	END_SHADER_PARAMETER_STRUCT()

	static bool IsSupported(uint32 NumRenderTargets)
	{
		return NumRenderTargets == 1 || NumRenderTargets == 3;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const uint32 NumRenderTargets = PermutationVector.Get<FNumRenderTargets>();

		if (!IsSupported(NumRenderTargets))
		{
			return false;
		}

		return RHISupportsRenderTargetWriteMask(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEX"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZEY"), ThreadGroupSizeY);
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	FRTWriteMaskDecodeCS() = default;
	FRTWriteMaskDecodeCS(const ShaderMetaType::CompiledShaderInitializerType & Initializer)
		: FGlobalShader(Initializer)
	{
		PlatformDataParam.Bind(Initializer.ParameterMap, TEXT("PlatformData"), SPF_Mandatory);
		BindForLegacyShaderParameters<FParameters>(this, Initializer.PermutationId, Initializer.ParameterMap);
	}

	// Shader parameter structs don't have a way to push variable sized data yet. So the we use the old shader parameter API.
	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const void* PlatformDataPtr, uint32 PlatformDataSize)
	{
		BatchedParameters.SetShaderParameter(PlatformDataParam.GetBufferIndex(), PlatformDataParam.GetBaseIndex(), PlatformDataSize, PlatformDataPtr);
	}

private:
	LAYOUT_FIELD(FShaderParameter, PlatformDataParam);
};

IMPLEMENT_GLOBAL_SHADER(FRTWriteMaskDecodeCS, "/Engine/Private/RTWriteMaskDecode.usf", "RTWriteMaskDecodeMain", SF_Compute);

void FRenderTargetWriteMask::Decode(
	FRHICommandListImmediate& RHICmdList,
	FGlobalShaderMap* ShaderMap,
	TArrayView<IPooledRenderTarget* const> InRenderTargets,
	TRefCountPtr<IPooledRenderTarget>& OutRTWriteMask,
	ETextureCreateFlags RTWriteMaskFastVRamConfig,
	const TCHAR* RTWriteMaskDebugName)
{
	FRDGBuilder GraphBuilder(RHICmdList);

	TArray<FRDGTextureRef, SceneRenderingAllocator> InputTextures;
	InputTextures.Reserve(InRenderTargets.Num());
	for (IPooledRenderTarget* RenderTarget : InRenderTargets)
	{
		InputTextures.Add(GraphBuilder.RegisterExternalTexture(RenderTarget));
	}

	FRDGTextureRef OutputTexture = nullptr;
	Decode(GraphBuilder, ShaderMap, InputTextures, OutputTexture, RTWriteMaskFastVRamConfig, RTWriteMaskDebugName);

	GraphBuilder.QueueTextureExtraction(OutputTexture, &OutRTWriteMask);
	GraphBuilder.Execute();
}

void FRenderTargetWriteMask::Decode(
	FRDGBuilder& GraphBuilder,
	FGlobalShaderMap* ShaderMap,
	TArrayView<FRDGTextureRef const> RenderTargets,
	FRDGTextureRef& OutRTWriteMask,
	ETextureCreateFlags RTWriteMaskFastVRamConfig,
	const TCHAR* RTWriteMaskDebugName)
{
	const uint32 NumRenderTargets = RenderTargets.Num();

	check(RHISupportsRenderTargetWriteMask(GMaxRHIShaderPlatform));
	checkf(FRTWriteMaskDecodeCS::IsSupported(NumRenderTargets), TEXT("FRenderTargetWriteMask::Decode does not currently support decoding %d render targets."), RenderTargets.Num());

	FRDGTextureRef Texture0 = RenderTargets[0];

	const FIntPoint RTWriteMaskDims(
		FMath::DivideAndRoundUp(Texture0->Desc.Extent.X, (int32)FRTWriteMaskDecodeCS::ThreadGroupSizeX),
		FMath::DivideAndRoundUp(Texture0->Desc.Extent.Y, (int32)FRTWriteMaskDecodeCS::ThreadGroupSizeY));

	// Allocate the Mask from the render target pool.
	const FRDGTextureDesc MaskDesc = FRDGTextureDesc::Create2D(
		RTWriteMaskDims,
		NumRenderTargets <= 2 ? PF_R8_UINT : PF_R16_UINT,
		FClearValueBinding::None,
		RTWriteMaskFastVRamConfig | TexCreate_UAV | TexCreate_RenderTargetable | TexCreate_ShaderResource);

	OutRTWriteMask = GraphBuilder.CreateTexture(MaskDesc, RTWriteMaskDebugName);

	auto* PassParameters = GraphBuilder.AllocParameters<FRTWriteMaskDecodeCS::FParameters>();
	PassParameters->ReferenceInput = Texture0;
	PassParameters->OutCombinedRTWriteMask = GraphBuilder.CreateUAV(OutRTWriteMask);

	for (uint32 Index = 0; Index < NumRenderTargets; ++Index)
	{
		PassParameters->RTWriteMaskInputs[Index] = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMetaData(RenderTargets[Index], ERDGTextureMetaDataAccess::CMask));
	}

	FRTWriteMaskDecodeCS::FPermutationDomain PermutationVector;
	PermutationVector.Set<FRTWriteMaskDecodeCS::FNumRenderTargets>(NumRenderTargets);
	TShaderMapRef<FRTWriteMaskDecodeCS> DecodeCS(ShaderMap, PermutationVector);

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("DecodeWriteMask[%d]", NumRenderTargets),
		PassParameters,
		ERDGPassFlags::Compute,
		[DecodeCS, PassParameters, ShaderMap, RTWriteMaskDims](FRDGAsyncTask, FRHIComputeCommandList& RHICmdList)
	{
		FRHITexture* Texture0RHI = PassParameters->ReferenceInput->GetRHI();

		// Retrieve the platform specific data that the decode shader needs.
		void* PlatformDataPtr = nullptr;
		uint32 PlatformDataSize = 0;
		Texture0RHI->GetWriteMaskProperties(PlatformDataPtr, PlatformDataSize);
		check(PlatformDataSize > 0);

		if (PlatformDataPtr == nullptr)
		{
			// If the returned pointer was null, the platform RHI wants us to allocate the memory instead.
			PlatformDataPtr = alloca(PlatformDataSize);
			Texture0RHI->GetWriteMaskProperties(PlatformDataPtr, PlatformDataSize);
		}

		SetComputePipelineState(RHICmdList, DecodeCS.GetComputeShader());

		SetShaderParametersMixedCS(RHICmdList, DecodeCS, *PassParameters, PlatformDataPtr, PlatformDataSize);

		RHICmdList.DispatchComputeShader(
			FMath::DivideAndRoundUp((uint32)RTWriteMaskDims.X, FRTWriteMaskDecodeCS::ThreadGroupSizeX),
			FMath::DivideAndRoundUp((uint32)RTWriteMaskDims.Y, FRTWriteMaskDecodeCS::ThreadGroupSizeY),
			1);
	});
}

FDepthBounds::FDepthBoundsValues FDepthBounds::CalculateNearFarDepthExcludingSky()
{
	FDepthBounds::FDepthBoundsValues Values;

	float SmallestFloatAbove0;

	if (GPixelFormats[PF_DepthStencil].bIs24BitUnormDepthStencil)
	{
		SmallestFloatAbove0 = 1.0f / 16777215.0f;			// 24bit norm depth
	}
	else
	{
		SmallestFloatAbove0 = 1.1754943508e-38f;				// 32bit float depth
	}

	Values.MinDepth = SmallestFloatAbove0;
	Values.MaxDepth = 1.f;

	return Values;
}

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FSubstratePublicGlobalUniformParameters, "SubstratePublic");

namespace Substrate
{
	// This version of the function is deprecated, as FScene no longer contains FSubstrateSceneData, and so is forced to return dummy data.
	RENDERER_API TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> GetPublicGlobalUniformBuffer(FRDGBuilder& GraphBuilder, FScene& Scene)
	{
		if (::Substrate::IsSubstrateEnabled())
		{
			return CreatePublicGlobalUniformBuffer(GraphBuilder, nullptr);
		}
		return nullptr;
	}

	TRDGUniformBufferRef<FSubstratePublicGlobalUniformParameters> GetPublicGlobalUniformBuffer(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures)
	{		
		if(::Substrate::IsSubstrateEnabled())
		{
			const FSubstrateSceneData* SubstrateScene = SceneTextures.SubstrateSceneData.Get();

			if (!SubstrateScene || SubstrateScene->SubstratePublicGlobalUniformParameters == nullptr)
			{
				return CreatePublicGlobalUniformBuffer(GraphBuilder, nullptr);//We are creating a dummy here so pass in null for the scene data.
			}
			return SubstrateScene->SubstratePublicGlobalUniformParameters;
		}

		return nullptr;
	}
}

namespace Nanite
{
	TRDGUniformBufferRef<FNaniteRayTracingUniformParameters> GetPublicGlobalRayTracingUniformBuffer()
	{
#if RHI_RAYTRACING
		return Nanite::GRayTracingManager.GetUniformBuffer();
#else
		return nullptr;
#endif
	}
}

void FBufferScatterUploader::UploadTo(FRDGBuilder& GraphBuilder, FRDGBuffer *DestBuffer, FRDGBuffer *ScatterOffsets, FRDGBuffer *Values, uint32 NumScatters, uint32 NumBytesPerElement, int32 NumValuesPerScatter, EPixelFormat ValuePixelFormat)
{
	FScatterCopyParams ScatterCopyParams { NumScatters, NumBytesPerElement, NumValuesPerScatter };
	ScatterCopyResource(GraphBuilder, DestBuffer, GraphBuilder.CreateSRV(ScatterOffsets), GraphBuilder.CreateSRV(Values, ValuePixelFormat), ScatterCopyParams);
}

void FBufferScatterUploader::UploadTo(FRDGBuilder& GraphBuilder, FRDGBuffer* DestBuffer, FRDGBuffer* ScatterOffsets, FRDGBuffer* Values, TFunction<uint64()>&& GetNumScatters, uint32 NumBytesPerElement, int32 NumValuesPerScatter)
{
	FAsyncScatterCopyParams ScatterCopyParams{ MoveTemp(GetNumScatters), NumBytesPerElement, NumValuesPerScatter };
	ScatterCopyResource(GraphBuilder, DestBuffer, GraphBuilder.CreateSRV(ScatterOffsets), GraphBuilder.CreateSRV(Values), ScatterCopyParams);
}

namespace UE::RendererPrivateUtils::Implementation
{

FPersistentBuffer::FPersistentBuffer(int32 InMinimumNumElementsReserved, const TCHAR *InName, bool bInRoundUpToPOT)
	: MinimumNumElementsReserved(InMinimumNumElementsReserved)
	, Name(InName)
	, bRoundUpToPOT(bInRoundUpToPOT)
{
}

FRDGBuffer* FPersistentBuffer::Register(FRDGBuilder& GraphBuilder) const
{ 
	return GraphBuilder.RegisterExternalBuffer(PooledBuffer); 
}

void FPersistentBuffer::Empty()
{
	PooledBuffer.SafeRelease();
}

FRDGBuffer* FPersistentBuffer::ResizeBufferIfNeeded(FRDGBuilder& GraphBuilder, const FRDGBufferDesc& BufferDesc)
{
	return ::ResizeBufferIfNeeded(GraphBuilder, PooledBuffer, BufferDesc, Name);
}

FRDGBuffer* FPersistentBuffer::ResizeAndClearBufferIfNeeded(FRDGBuilder& GraphBuilder, const FRDGBufferDesc& BufferDesc)
{
	uint32 PrevNumElements = PooledBuffer ? PooledBuffer->Desc.NumElements : 0u;
	FRDGBuffer* NewBuffer = ::ResizeBufferIfNeeded(GraphBuilder, PooledBuffer, BufferDesc, Name);
	if (NewBuffer->Desc.NumElements > PrevNumElements)
	{
		::MemsetResource(GraphBuilder, NewBuffer, FMemsetResourceParams { 0u, NewBuffer->Desc.NumElements - PrevNumElements,  PrevNumElements });
	}
	return NewBuffer;
}

} 

TGlobalResource<FTileTexCoordVertexBuffer> GOneTileQuadVertexBuffer(1);
TGlobalResource<FTileIndexBuffer> GOneTileQuadIndexBuffer(1);

RENDERER_API FBufferRHIRef& GetOneTileQuadVertexBuffer()
{
	return GOneTileQuadVertexBuffer.VertexBufferRHI;
}

RENDERER_API FBufferRHIRef& GetOneTileQuadIndexBuffer()
{
	return GOneTileQuadIndexBuffer.IndexBufferRHI;
}

class FClearIndirectDispatchArgsCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FClearIndirectDispatchArgsCS);
	SHADER_USE_PARAMETER_STRUCT(FClearIndirectDispatchArgsCS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, NumIndirectArgs)
		SHADER_PARAMETER(uint32, IndirectArgStride)
		SHADER_PARAMETER(FIntVector3, DimClearValue)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, OutIndirectArgsBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FClearIndirectDispatchArgsCS, "/Engine/Private/RendererUtils.usf", "ClearIndirectDispatchArgsCS", SF_Compute);

void AddClearIndirectDispatchArgsPass(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, FRDGBufferRef IndirectArgsRDG, const FIntVector3 &DimClearValue, uint32 NumIndirectArgs, uint32 IndirectArgStride)
{
	// Need room for XYZ dims at least.
	check(IndirectArgStride >= 3);

	FClearIndirectDispatchArgsCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FClearIndirectDispatchArgsCS::FParameters>();
	PassParameters->NumIndirectArgs = NumIndirectArgs;
	PassParameters->IndirectArgStride = IndirectArgStride;
	PassParameters->DimClearValue = DimClearValue; 
	PassParameters->OutIndirectArgsBuffer = GraphBuilder.CreateUAV(IndirectArgsRDG);

	auto ComputeShader = GetGlobalShaderMap(FeatureLevel)->GetShader<FClearIndirectDispatchArgsCS>();

	FComputeShaderUtils::AddPass(
		GraphBuilder,
		RDG_EVENT_NAME("ClearIndirectDispatchArgs"),
		ComputeShader,
		PassParameters,
		FComputeShaderUtils::GetGroupCount(NumIndirectArgs, 64)
	);
}

FRDGBufferRef CreateAndClearIndirectDispatchArgs(FRDGBuilder& GraphBuilder, ERHIFeatureLevel::Type FeatureLevel, const TCHAR* Name, const FIntVector3& DimClearValue, uint32 NumIndirectArgs, uint32 IndirectArgStride)
{
	FRDGBufferRef IndirectArgsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(NumIndirectArgs * IndirectArgStride), Name);
	AddClearIndirectDispatchArgsPass(GraphBuilder, FeatureLevel, IndirectArgsRDG, DimClearValue, NumIndirectArgs, IndirectArgStride);
	return IndirectArgsRDG;
}
