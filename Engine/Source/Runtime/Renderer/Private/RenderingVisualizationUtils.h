// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneRendering.h"
#include "ScreenPass.h"
#include "UnrealEngine.h"

template <typename EntryContainerType>
void AddLegendCanvasPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FScreenPassRenderTarget OutputTarget, const FString& HeaderLabel, FVector2f LegendAnchorPositionLowerLeft, FVector2f LegendMinSize, const EntryContainerType& LegendEntries)
{
	AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("Labels"), View, OutputTarget,
	[&](FCanvas& Canvas)
	{
		UFont* StatsFont = GetStatsFont();

		const float DPIScale = Canvas.GetDPIScale();
		Canvas.SetBaseTransform(FMatrix(FScaleMatrix(DPIScale) * Canvas.CalcBaseTransform2D(Canvas.GetViewRect().Width(), Canvas.GetViewRect().Height())));

		auto DrawColorTile = [&](float X, float Y, float Width, float Height, const FLinearColor& Color)
		{
			Canvas.DrawTile(X/DPIScale, Y/DPIScale, Width/DPIScale, Height/DPIScale, 0, 0, 0, 0, Color);
		};

		auto DrawShadowedString = [&](float X, float Y, FStringView Text, const FLinearColor& Color = FLinearColor::White)
		{
			Canvas.DrawShadowedString(X / DPIScale, Y / DPIScale, Text, StatsFont, Color);
		};
		
		const float YStride = 20.0f * DPIScale;

		FVector2f LegendPosition = LegendAnchorPositionLowerLeft;
		auto DrawLegendEntry = [&](const FString& Label, FLinearColor Color)
		{
			DrawColorTile(LegendPosition.X + 7.0f * DPIScale, LegendPosition.Y + 5.0f * DPIScale, 10.0f * DPIScale, 10.0f * DPIScale, Color);
			DrawShadowedString(LegendPosition.X + YStride, LegendPosition.Y + 2.0f * DPIScale, Label);
			LegendPosition.Y += YStride;
		};

		auto GetStringWidth = [&](FStringView StringView) -> float
		{
			int32 XL = 0;
			int32 YL = 0;
			StringSize(StatsFont, XL, YL, StringView);
			return float(XL);
		};

		float AutoSizeY = FMath::Max(LegendMinSize.Y, float(LegendEntries.Num() + 1) * 20.0f + 10.0f) * DPIScale;
		// using lower left positioning
		LegendPosition.Y -= AutoSizeY;

		float AutoSizeX = FMath::Max(LegendMinSize.X, GetStringWidth(HeaderLabel));
		for (const auto& Entry : LegendEntries)
		{
			AutoSizeX = FMath::Max(AutoSizeX, GetStringWidth(Entry.Label.ToString()));
		}
		AutoSizeX += 35.0f;
		AutoSizeX *= DPIScale;

		DrawColorTile(LegendPosition.X, LegendPosition.Y, AutoSizeX, AutoSizeY, FLinearColor(0.1f, 0.1f, 0.1f, 0.8f));
		DrawShadowedString(LegendPosition.X + 5.0f * DPIScale, LegendPosition.Y + 5.0f * DPIScale, HeaderLabel);
		LegendPosition.Y += YStride;

		for (const auto& Entry : LegendEntries)
		{
			DrawLegendEntry(Entry.Label.ToString(), Entry.Color);
		}
	});
}