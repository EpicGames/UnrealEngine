// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	
=============================================================================*/
#include "SceneCaptureRendering.h"
#include "Containers/ArrayView.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/App.h"
#include "Misc/MemStack.h"
#include "EngineDefines.h"
#include "RHIDefinitions.h"
#include "RHI.h"
#include "RenderingThread.h"
#include "Engine/Scene.h"
#include "SceneInterface.h"
#include "LegacyScreenPercentageDriver.h"
#include "GameFramework/Actor.h"
#include "GameFramework/WorldSettings.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "Shader.h"
#include "TextureResource.h"
#include "SceneUtils.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneCaptureComponent.h"
#include "Components/SceneCaptureComponent2D.h"
#include "Components/SceneCaptureComponentCube.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTargetCube.h"
#include "PostProcess/SceneRenderTargets.h"
#include "GlobalShader.h"
#include "SceneRenderTargetParameters.h"
#include "SceneRendering.h"
#include "SceneRenderingUtils.h"
#include "DeferredShadingRenderer.h"
#include "ScenePrivate.h"
#include "PostProcess/SceneFilterRendering.h"
#include "ScreenRendering.h"
#include "PipelineStateCache.h"
#include "LogRenderer.h"
#include "Rendering/MotionVectorSimulation.h"
#include "SceneViewExtension.h"
#include "GenerateMips.h"
#include "RectLightTexture.h"
#include "Materials/MaterialRenderProxy.h"
#include "Rendering/CustomRenderPass.h"
#include "DumpGPU.h"
#include "IRenderCaptureProvider.h"
#include "RenderCaptureInterface.h"
#include "CustomRenderPassSceneCapture.h"
#include "SceneRenderBuilder.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "Engine/PostProcessUtils.h"
#include "Components/ReflectionCaptureComponent.h"
#include "ReflectionEnvironmentCapture.h"
#include "SceneViewState.h"
#include "RenderTargetPool.h"

bool GSceneCaptureAllowRenderInMainRenderer = true;
static FAutoConsoleVariableRef CVarSceneCaptureAllowRenderInMainRenderer(
	TEXT("r.SceneCapture.AllowRenderInMainRenderer"),
	GSceneCaptureAllowRenderInMainRenderer,
	TEXT("Whether to allow SceneDepth & DeviceDepth scene capture to render in the main renderer as an optimization.\n")
	TEXT("0: render as an independent renderer.\n")
	TEXT("1: render as part of the main renderer if Render in Main Renderer is enabled on scene capture component.\n"),
	ECVF_Scalability);

bool GSceneCaptureCubeSinglePass = true;
static FAutoConsoleVariableRef CVarSceneCaptureCubeSinglePass(
	TEXT("r.SceneCapture.CubeSinglePass"),
	GSceneCaptureCubeSinglePass,
	TEXT("Whether to run all 6 faces of cube map capture in a single scene renderer pass."),
	ECVF_Scalability);

static int32 GRayTracingSceneCaptures = -1;
static FAutoConsoleVariableRef CVarRayTracingSceneCaptures(
	TEXT("r.RayTracing.SceneCaptures"),
	GRayTracingSceneCaptures,
	TEXT("Enable ray tracing in scene captures.\n")
	TEXT(" -1: Use scene capture settings (default) \n")
	TEXT(" 0: off \n")
	TEXT(" 1: on"),
	ECVF_Default);

extern TAutoConsoleVariable<bool> CVarMobileDiffuseFromCapture;

static TAutoConsoleVariable<bool> CVarDiffuseIrradianceReference(
	TEXT("r.SceneCapture.DiffuseIrradianceReference"),
	true,
	TEXT("Compute diffuse irradiance using reference implementation instead of from Spherical Harmonics.  Reference version looks\n")
	TEXT("much better, and is currently many times faster at the default 8x8 source mip resolution, because the original irradiance\n")
	TEXT("computation (designed for offline reflection capture builds) is totally unoptimized, and involves many passes.\n"),
	ECVF_RenderThreadSafe);

#if WITH_EDITOR
// All scene captures on the given render thread frame will be dumped
uint32 GDumpSceneCaptureMemoryFrame = INDEX_NONE;
void DumpSceneCaptureMemory()
{
	ENQUEUE_RENDER_COMMAND(DumpSceneCaptureMemory)(
		[](FRHICommandList& RHICmdList)
		{
			GDumpSceneCaptureMemoryFrame = GFrameNumberRenderThread;
		});
}

FAutoConsoleCommand CmdDumpSceneCaptureViewState(
	TEXT("r.SceneCapture.DumpMemory"),
	TEXT("Editor specific command to dump scene capture memory to log"),
	FConsoleCommandDelegate::CreateStatic(DumpSceneCaptureMemory)
);
#endif  // WITH_EDITOR

/** A pixel shader for capturing a component of the rendered scene for a scene capture.*/
class FSceneCapturePS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FSceneCapturePS);
	SHADER_USE_PARAMETER_STRUCT(FSceneCapturePS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSubstratePublicGlobalUniformParameters, SubstratePublic)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	enum class ESourceMode : uint32
	{
		ColorAndOpacity,
		ColorNoAlpha,
		ColorAndSceneDepth,
		SceneDepth,
		DeviceDepth,
		Normal,
		BaseColor,
		ColorOneAlpha,
		MAX
	};

	class FSourceModeDimension : SHADER_PERMUTATION_ENUM_CLASS("SOURCE_MODE", ESourceMode);
	class FEnable128BitRT : SHADER_PERMUTATION_BOOL("ENABLE_128_BIT");
	using FPermutationDomain = TShaderPermutationDomain<FSourceModeDimension, FEnable128BitRT>;

	static FPermutationDomain GetPermutationVector(ESceneCaptureSource CaptureSource, bool bUse128BitRT, bool bForwardShadingEnabled, bool bIsReflectionCapture)
	{
		ESourceMode SourceMode = ESourceMode::MAX;
		switch (CaptureSource)
		{
		case SCS_SceneColorHDR:
			SourceMode = ESourceMode::ColorAndOpacity;
			break;
		case SCS_SceneColorHDRNoAlpha:
			SourceMode = bIsReflectionCapture ? ESourceMode::ColorOneAlpha : ESourceMode::ColorNoAlpha;
			break;
		case SCS_SceneColorSceneDepth:
			SourceMode = ESourceMode::ColorAndSceneDepth;
			break;
		case SCS_SceneDepth:
			SourceMode = ESourceMode::SceneDepth;
			break;
		case SCS_DeviceDepth:
			SourceMode = ESourceMode::DeviceDepth;
			break;
		case SCS_Normal:
			SourceMode = ESourceMode::Normal;
			break;
		case SCS_BaseColor:
			SourceMode = ESourceMode::BaseColor;
			break;
		default:
			checkf(false, TEXT("SceneCaptureSource not implemented."));
		}

		if (bForwardShadingEnabled && (SourceMode == ESourceMode::Normal || SourceMode == ESourceMode::BaseColor))
		{
			SourceMode = ESourceMode::ColorAndOpacity;
		}
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FSourceModeDimension>(SourceMode);
		PermutationVector.Set<FEnable128BitRT>(bUse128BitRT);
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);

		auto SourceModeDim = PermutationVector.Get<FSourceModeDimension>();
		bool bPlatformRequiresExplicit128bitRT = FDataDrivenShaderPlatformInfo::GetRequiresExplicit128bitRT(Parameters.Platform);
		return (!PermutationVector.Get<FEnable128BitRT>() || bPlatformRequiresExplicit128bitRT);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		static const TCHAR* ShaderSourceModeDefineName[] =
		{
			TEXT("SOURCE_MODE_SCENE_COLOR_AND_OPACITY"),
			TEXT("SOURCE_MODE_SCENE_COLOR_NO_ALPHA"),
			TEXT("SOURCE_MODE_SCENE_COLOR_SCENE_DEPTH"),
			TEXT("SOURCE_MODE_SCENE_DEPTH"),
			TEXT("SOURCE_MODE_DEVICE_DEPTH"),
			TEXT("SOURCE_MODE_NORMAL"),
			TEXT("SOURCE_MODE_BASE_COLOR"),
			TEXT("SOURCE_MODE_SCENE_COLOR_ONE_ALPHA"),
		};
		static_assert(UE_ARRAY_COUNT(ShaderSourceModeDefineName) == (uint32)ESourceMode::MAX, "ESourceMode doesn't match define table.");

		const FPermutationDomain PermutationVector(Parameters.PermutationId);
		const uint32 SourceModeIndex = static_cast<uint32>(PermutationVector.Get<FSourceModeDimension>());
		OutEnvironment.SetDefine(ShaderSourceModeDefineName[SourceModeIndex], 1u);

		if (PermutationVector.Get<FEnable128BitRT>())
		{
			OutEnvironment.SetRenderTargetOutputFormat(0, PF_A32B32G32R32F);
		}

		if (IsMobilePlatform(Parameters.Platform))
		{
			OutEnvironment.FullPrecisionInPS = 1;
		}

	}
};

IMPLEMENT_GLOBAL_SHADER(FSceneCapturePS, "/Engine/Private/SceneCapturePixelShader.usf", "Main", SF_Pixel);

static bool CaptureNeedsSceneColor(ESceneCaptureSource CaptureSource)
{
	return CaptureSource != SCS_FinalColorLDR && CaptureSource != SCS_FinalColorHDR && CaptureSource != SCS_FinalToneCurveHDR;
}

using FSceneCaptureViewportSetterFunction = TFunction<void(FRHICommandList& RHICmdList, int32 ViewIndex)>;
class FSceneCaptureViewportSetterMap : public TMap<FRDGTexture*, FSceneCaptureViewportSetterFunction> {};

RDG_REGISTER_BLACKBOARD_STRUCT(FSceneCaptureViewportSetterMap);

void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ViewFamilyTexture,
	FRDGTextureRef ViewFamilyDepthTexture,
	const FSceneViewFamily& ViewFamily,
	TConstArrayView<FViewInfo> Views)
{
	TArray<const FViewInfo*> ViewPtrArray;
	for (const FViewInfo& View : Views)
	{
		ViewPtrArray.Add(&View);
	}
	CopySceneCaptureComponentToTarget(GraphBuilder, SceneTextures, ViewFamilyTexture, ViewFamilyDepthTexture, ViewFamily, ViewPtrArray);
}

void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ViewFamilyTexture,
	FRDGTextureRef ViewFamilyDepthTexture,
	const FSceneViewFamily& ViewFamily,
	const TArray<const FViewInfo*>& Views)
{
	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

	const bool bForwardShadingEnabled = IsForwardShadingEnabled(ViewFamily.GetShaderPlatform());
	int32 NumViews = Views.Num();
	for (int32 ViewIndex = 0; ViewIndex < NumViews; ViewIndex++)
	{
		const FViewInfo& View = *Views[ViewIndex];

		// If view has its own scene capture setting, use it over view family setting
		ESceneCaptureSource SceneCaptureSource = View.CustomRenderPass ? View.CustomRenderPass->GetSceneCaptureSource() : ViewFamily.SceneCaptureSource;
		if (bForwardShadingEnabled && (SceneCaptureSource == SCS_Normal || SceneCaptureSource == SCS_BaseColor))
		{
			SceneCaptureSource = SCS_SceneColorHDR;
		}
		if (!CaptureNeedsSceneColor(SceneCaptureSource))
		{
			continue;
		}

		RDG_EVENT_SCOPE(GraphBuilder, "CaptureSceneComponent_View[%d]", SceneCaptureSource);

		bool bIsCompositing = false;
		if (SceneCaptureSource == SCS_SceneColorHDR && ViewFamily.SceneCaptureCompositeMode == SCCM_Composite)
		{
			// Blend with existing render target color. Scene capture color is already pre-multiplied by alpha.
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_SourceAlpha, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
			bIsCompositing = true;
		}
		else if (SceneCaptureSource == SCS_SceneColorHDR && ViewFamily.SceneCaptureCompositeMode == SCCM_Additive)
		{
			// Add to existing render target color. Scene capture color is already pre-multiplied by alpha.
			GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_SourceAlpha>::GetRHI();
			bIsCompositing = true;
		}
		else
		{
			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		}

		const bool bUse128BitRT = PlatformRequires128bitRT(ViewFamilyTexture->Desc.Format);
		const FSceneCapturePS::FPermutationDomain PixelPermutationVector = FSceneCapturePS::GetPermutationVector(SceneCaptureSource, bUse128BitRT, bForwardShadingEnabled, View.IsRuntimeReflectionCapture());

		FSceneCapturePS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSceneCapturePS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->SceneTextures = SceneTextures.GetSceneTextureShaderParameters(ViewFamily.GetFeatureLevel());
		if (Substrate::IsSubstrateEnabled())
		{
			// CreatePublicGlobalUniformBuffer handles View.SubstrateViewData.SceneData==null.  Use null SceneData for depth captures, which won't
			// have produced Substrate data, to avoid an RDG validation error (even though technically the permutation doesn't use it).
			PassParameters->SubstratePublic = Substrate::CreatePublicGlobalUniformBuffer(GraphBuilder,
				SceneCaptureSource == SCS_SceneDepth || SceneCaptureSource == SCS_DeviceDepth ? nullptr : View.SubstrateViewData.SceneData);
		}
		PassParameters->RenderTargets[0] = FRenderTargetBinding(ViewFamilyTexture, bIsCompositing ? ERenderTargetLoadAction::ELoad : ERenderTargetLoadAction::ENoAction);

		TShaderMapRef<FScreenVS> VertexShader(View.ShaderMap);
		TShaderMapRef<FSceneCapturePS> PixelShader(View.ShaderMap, PixelPermutationVector);

		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;

		FIntPoint TargetSize;
		if (((const FViewFamilyInfo*)View.Family)->bIsSceneTextureSizedCapture)
		{
			// Scene texture sized target, use actual target extent for copy, and set correct extent for visualization debug feature
			TargetSize = ViewFamilyTexture->Desc.Extent;
			ViewFamilyTexture->EncloseVisualizeExtent(View.UnconstrainedViewRect.Max);
		}
		else
		{
			// Need to use the extent from the actual target texture for cube captures.  Although perhaps we should use the actual texture
			// extent across the board?  Would it ever be incorrect to do so?
			TargetSize = View.bIsSceneCaptureCube ? ViewFamilyTexture->Desc.Extent : View.UnconstrainedViewRect.Size();
		}

		TFunction<void(FRHICommandList& RHICmdList, int32 ViewIndex)> SetViewportLambda;

		if (FSceneCaptureViewportSetterMap* Map = GraphBuilder.Blackboard.GetMutable<FSceneCaptureViewportSetterMap>())
		{
			if (auto* Function = Map->Find(ViewFamilyTexture))
			{
				SetViewportLambda = MoveTemp(*Function);
			}
		}

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("View(%d)", ViewIndex),
			PassParameters,
			ERDGPassFlags::Raster,
			[PassParameters, GraphicsPSOInit, VertexShader, PixelShader, &View, ViewIndex, TargetSize, SetViewportLambda = MoveTemp(SetViewportLambda)] (FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			FGraphicsPipelineStateInitializer LocalGraphicsPSOInit = GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(LocalGraphicsPSOInit);
			SetGraphicsPipelineState(RHICmdList, LocalGraphicsPSOInit, 0);
			SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), *PassParameters);

			if (SetViewportLambda)
			{
				SetViewportLambda(RHICmdList, ViewIndex);
			}

			if (View.bIsSceneCaptureCube)
			{
				// When using timeslicing, cube capture faces are rendered over multiple frames.  In such cases, ViewRect is pushed to the origin,
				// while UnscaledViewRect is the actual offset where the output face should be written.  We don't want to use UnscaledViewRect in
				// the general case, as that will be incorrect if DRS has been inherited from the main view, but cube captures don't support that.
				DrawRectangle(
					RHICmdList,
					View.UnscaledViewRect.Min.X, View.UnscaledViewRect.Min.Y,
					View.UnscaledViewRect.Width(), View.UnscaledViewRect.Height(),
					View.ViewRect.Min.X, View.ViewRect.Min.Y,
					View.ViewRect.Width(), View.ViewRect.Height(),
					TargetSize,
					View.GetSceneTexturesConfig().Extent,
					VertexShader,
					EDRF_UseTriangleOptimization);
			}
			else
			{
				DrawRectangle(
					RHICmdList,
					View.ViewRect.Min.X, View.ViewRect.Min.Y,
					View.ViewRect.Width(), View.ViewRect.Height(),
					View.ViewRect.Min.X, View.ViewRect.Min.Y,
					View.ViewRect.Width(), View.ViewRect.Height(),
					TargetSize,
					View.GetSceneTexturesConfig().Extent,
					VertexShader,
					EDRF_UseTriangleOptimization);
			}
		});
	}

	if (ViewFamilyDepthTexture && ViewFamily.EngineShowFlags.SceneCaptureCopySceneDepth)
	{
		verify(SceneTextures.Depth.Target->Desc == ViewFamilyDepthTexture->Desc);
		AddCopyTexturePass(GraphBuilder, SceneTextures.Depth.Target, ViewFamilyDepthTexture);
	}
}

void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef ViewFamilyTexture,
	FRDGTextureRef ViewFamilyDepthTexture,
	const FSceneViewFamily& ViewFamily,
	TConstStridedView<FSceneView> Views)
{
	const FSceneView& View = Views[0];

	check(View.bIsViewInfo);
	const FMinimalSceneTextures& SceneTextures = static_cast<const FViewInfo&>(View).GetSceneTextures();

	TConstArrayView<FViewInfo> ViewInfos = MakeArrayView(static_cast<const FViewInfo*>(&Views[0]), Views.Num());

	CopySceneCaptureComponentToTarget(GraphBuilder, SceneTextures, ViewFamilyTexture, ViewFamilyDepthTexture, ViewFamily, ViewInfos);
}

void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ViewFamilyTexture,
	const FSceneViewFamily& ViewFamily,
	const TArray<const FViewInfo*>& Views)
{
	CopySceneCaptureComponentToTarget(GraphBuilder, SceneTextures, ViewFamilyTexture, nullptr, ViewFamily, Views);
}

void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	const FMinimalSceneTextures& SceneTextures,
	FRDGTextureRef ViewFamilyTexture,
	const FSceneViewFamily& ViewFamily,
	TConstArrayView<FViewInfo> Views)
{
	CopySceneCaptureComponentToTarget(GraphBuilder, SceneTextures, ViewFamilyTexture, nullptr, ViewFamily, Views);
}

void CopySceneCaptureComponentToTarget(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef ViewFamilyTexture,
	const FSceneViewFamily& ViewFamily,
	TConstStridedView<FSceneView> Views)
{
	CopySceneCaptureComponentToTarget(GraphBuilder, ViewFamilyTexture, nullptr, ViewFamily, Views);
}

static void UpdateSceneCaptureContent_RenderThread(
	FRDGBuilder& GraphBuilder,
	FSceneRenderer* SceneRenderer,
	const FSceneRenderUpdateInputs* SceneUpdateInputs,
	FRenderTarget* RenderTarget,
	FRDGTextureRef OutputTexture,
	TConstArrayView<FRHICopyTextureInfo> CopyInfos,
	bool bGenerateMips,
	const FGenerateMipsParams& GenerateMipsParams,
	bool bClearRenderTarget,
	bool bOrthographicCamera)
{
	const EShadingPath ShadingPath = GetFeatureLevelShadingPath(SceneRenderer->Scene->GetFeatureLevel());

	// TargetTexture (a.k.a. RenderTarget) is what gets rendered to, while OutputTexture is the final output.  For 2D scene captures, these
	// textures are the same.  For cube captures, OutputTexture will be a cube map, while TargetTexture will be a 2D render target containing
	// either one face of the cube map (when GSceneCaptureCubeSinglePass=0) or the six faces of the cube map tiled in a split screen configuration.
	FRDGTextureRef TargetTexture = RegisterExternalTexture(GraphBuilder, RenderTarget->GetRenderTargetTexture(), TEXT("SceneCaptureTarget"));

	if (bClearRenderTarget)
	{
		AddClearRenderTargetPass(GraphBuilder, TargetTexture, FLinearColor::Black, SceneRenderer->Views[0].UnscaledViewRect);
	}

	// The lambda below applies to tiled orthographic rendering, where the captured result is blitted from the origin in a scene texture
	// to a viewport on a larger output texture.  It specifically doesn't apply to cube maps, where the output texture has the same tiling
	// as the scene textures, and no viewport remapping is required.
	if (CopyInfos.Num() && !CopyInfos[0].Size.IsZero() && !OutputTexture->Desc.IsTextureCube())
	{
		// Fix for static analysis warning -- lambda lifetime exceeds lifetime of CopyInfos.  Technically, the lambda is consumed in the
		// scene render call below and not used afterwards, but static analysis doesn't know that, so we make a copy.
		TArray<FRHICopyTextureInfo, FConcurrentLinearArrayAllocator> CopyInfosLocal(CopyInfos);

		GraphBuilder.Blackboard.GetOrCreate<FSceneCaptureViewportSetterMap>().Emplace(TargetTexture, [CopyInfos = MoveTemp(CopyInfosLocal)](FRHICommandList& RHICmdList, int32 ViewIndex)
		{
			const FIntRect CopyDestRect = CopyInfos[ViewIndex].GetDestRect();

			RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
			RHICmdList.SetViewport
			(
				float(CopyDestRect.Min.X),
				float(CopyDestRect.Min.Y),
				0.0f,
				float(CopyDestRect.Max.X),
				float(CopyDestRect.Max.Y),
				1.0f
			);
		});
	}

	// Disable occlusion queries when in orthographic mode
	if (bOrthographicCamera)
	{
		FViewInfo& View = SceneRenderer->Views[0];
		View.bDisableQuerySubmissions = true;
		View.bIgnoreExistingQueries = true;
	}

	UE_CLOGF(FSceneCaptureLogUtils::bEnableSceneCaptureLogging, LogSceneCapture, Log, "Running UpdateSceneCaptureContent_RenderThread.");

	SceneRenderer->Render(GraphBuilder, SceneUpdateInputs);

	if (ShadingPath == EShadingPath::Mobile)
	{
		// Handles copying the SceneColor render target to the output if necessary (this happens inside the renderer for the deferred path).
		// Other scene captures are automatically written directly to the output, in which case this function returns and does nothing.
		const FRenderTarget* FamilyTarget = SceneRenderer->ViewFamily.RenderTarget;
		FRDGTextureRef FamilyTexture = RegisterExternalTexture(GraphBuilder, FamilyTarget->GetRenderTargetTexture(), TEXT("OutputTexture"));
		const FMinimalSceneTextures& SceneTextures = SceneRenderer->GetActiveSceneTextures();

		RDG_EVENT_SCOPE(GraphBuilder, "CaptureSceneColor");
		CopySceneCaptureComponentToTarget(
			GraphBuilder,
			SceneTextures,
			FamilyTexture,
			SceneRenderer->ViewFamily,
			SceneRenderer->Views);
	}

	// These copies become a no-op (function returns immediately) if TargetTexture and OutputTexture are the same, which
	// is true for 2D scene captures.  Actual copies only occur for cube captures, where copying is necessary to get
	// result data to specific slices.
	for (const FRHICopyTextureInfo& CopyInfo : CopyInfos)
	{
		AddCopyTexturePass(GraphBuilder, TargetTexture, OutputTexture, CopyInfo);
	}

	if (bGenerateMips)
	{
		FGenerateMips::Execute(GraphBuilder, SceneRenderer->FeatureLevel, OutputTexture, GenerateMipsParams);
	}
}

static void BuildOrthoMatrix(FIntPoint InRenderTargetSize, float InOrthoWidth, int32 InTileID, int32 InNumXTiles, int32 InNumYTiles, FMatrix& OutProjectionMatrix)
{
	float const XAxisMultiplier = 1.0f;
	float const YAxisMultiplier = InRenderTargetSize.X / float(InRenderTargetSize.Y);

	const float OrthoWidth = InOrthoWidth / 2.0f;
	const float OrthoHeight = InOrthoWidth / 2.0f * XAxisMultiplier / YAxisMultiplier;

	const float NearPlane = 0;
	const float FarPlane = UE_FLOAT_HUGE_DISTANCE / 4.0f;

	const float ZScale = 1.0f / (FarPlane - NearPlane);
	const float ZOffset = -NearPlane;

	if (InTileID == -1)
	{
		OutProjectionMatrix = FReversedZOrthoMatrix(
			OrthoWidth,
			OrthoHeight,
			ZScale,
			ZOffset
		);
		
		return;
	}

#if DO_CHECK
	check(InNumXTiles != 0 && InNumYTiles != 0);
	if (InNumXTiles == 0 || InNumYTiles == 0)
	{
		OutProjectionMatrix = FMatrix(EForceInit::ForceInitToZero);
		return;
	}
#endif

	const float XTileDividerRcp = 1.0f / float(InNumXTiles);
	const float YTileDividerRcp = 1.0f / float(InNumYTiles);

	const float TileX = float(InTileID % InNumXTiles);
	const float TileY = float(InTileID / InNumXTiles);

	float l = -OrthoWidth + TileX * InOrthoWidth * XTileDividerRcp;
	float r = l + InOrthoWidth * XTileDividerRcp;
	float t = OrthoHeight - TileY * InOrthoWidth * YTileDividerRcp;
	float b = t - InOrthoWidth * YTileDividerRcp;

	OutProjectionMatrix = FMatrix(
		FPlane(2.0f / (r-l), 0.0f, 0.0f, 0.0f),
		FPlane(0.0f, 2.0f / (t-b), 0.0f, 0.0f),
		FPlane(0.0f, 0.0f, -ZScale, 0.0f),
		FPlane(-((r+l)/(r-l)), -((t+b)/(t-b)), 1.0f - ZOffset * ZScale, 1.0f)
	);
}

void BuildProjectionMatrix(FIntPoint InRenderTargetSize, float InFOV, float InNearClippingPlane, float InFarClippingPlane, FMatrix& OutProjectionMatrix)
{
	float const XAxisMultiplier = 1.0f;
	float const YAxisMultiplier = InRenderTargetSize.X / float(InRenderTargetSize.Y);

	OutProjectionMatrix = FReversedZPerspectiveMatrix(
		InFOV,
		InFOV,
		XAxisMultiplier,
		YAxisMultiplier,
		InNearClippingPlane,
		InFarClippingPlane
		);
}

void GetShowOnlyAndHiddenComponents(USceneCaptureComponent* SceneCaptureComponent, TSet<FPrimitiveComponentId>& HiddenPrimitives, TOptional<TSet<FPrimitiveComponentId>>& ShowOnlyPrimitives)
{
	check(SceneCaptureComponent);
	for (auto It = SceneCaptureComponent->HiddenComponents.CreateConstIterator(); It; ++It)
	{
		// If the primitive component was destroyed, the weak pointer will return NULL.
		UPrimitiveComponent* PrimitiveComponent = It->Get();
		if (PrimitiveComponent)
		{
			HiddenPrimitives.Add(PrimitiveComponent->GetPrimitiveSceneId());
		}
	}

	for (auto It = SceneCaptureComponent->HiddenActors.CreateConstIterator(); It; ++It)
	{
		AActor* Actor = *It;

		if (Actor)
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
				{
					HiddenPrimitives.Add(PrimComp->GetPrimitiveSceneId());
				}
			}
		}
	}

	if (SceneCaptureComponent->PrimitiveRenderMode == ESceneCapturePrimitiveRenderMode::PRM_UseShowOnlyList)
	{
		ShowOnlyPrimitives.Emplace();

		for (auto It = SceneCaptureComponent->ShowOnlyComponents.CreateConstIterator(); It; ++It)
		{
			// If the primitive component was destroyed, the weak pointer will return NULL.
			UPrimitiveComponent* PrimitiveComponent = It->Get();
			if (PrimitiveComponent)
			{
				ShowOnlyPrimitives->Add(PrimitiveComponent->GetPrimitiveSceneId());
			}
		}

		for (auto It = SceneCaptureComponent->ShowOnlyActors.CreateConstIterator(); It; ++It)
		{
			AActor* Actor = *It;

			if (Actor)
			{
				for (UActorComponent* Component : Actor->GetComponents())
				{
					if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
					{
						ShowOnlyPrimitives->Add(PrimComp->GetPrimitiveSceneId());
					}
				}
			}
		}
	}
	else if (SceneCaptureComponent->ShowOnlyComponents.Num() > 0 || SceneCaptureComponent->ShowOnlyActors.Num() > 0)
	{
		static bool bWarned = false;

		if (!bWarned)
		{
			UE_LOGF(LogRenderer, Log, "Scene Capture has ShowOnlyComponents or ShowOnlyActors ignored by the PrimitiveRenderMode setting! %ls", *SceneCaptureComponent->GetPathName());
			bWarned = true;
		}
	}
}

namespace UE::Renderer::SceneCapture::Private
{

TArray<FSceneView*> SetupViewFamilyForSceneCapture(
	FSceneViewFamily& ViewFamily,
	USceneCaptureComponent* SceneCaptureComponent,
	const TArrayView<const FSceneCaptureViewInfo> Views,
	float MaxViewDistance,
	ESetupViewFamilyFlags Flags,
	FPostProcessSettings* PostProcessSettings,
	const FFinalPostProcessSettings* InheritedMainViewPostProcessSettings,
	float PostProcessBlendWeight,
	const AActor* ViewActor,
	int32 CubemapFaceIndex,
	int32 CubemapFaceOffset)
{
	check(!ViewFamily.GetScreenPercentageInterface());

	// For cube map capture, CubeMapFaceIndex takes precedence over view index, so we must have only one view for that case.
	// Or if CubemapFaceIndex == CubeFace_MAX (6), it's a renderer for all 6 cube map faces.
	check(CubemapFaceIndex == INDEX_NONE || Views.Num() == 1 || CubemapFaceIndex == CubeFace_MAX);

	// Initialize frame number
	ViewFamily.FrameNumber = ViewFamily.Scene->GetFrameNumber();
	ViewFamily.FrameCounter = GFrameCounter;

	ViewFamily.bOverrideVirtualTextureThrottle = EnumHasAnyFlags(Flags, ESetupViewFamilyFlags::OverrideVirtualTextureThrottle);

	TArray<FSceneView*> ViewPtrArray;
	ViewPtrArray.Reserve(Views.Num());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
	{
		const FSceneCaptureViewInfo& SceneCaptureViewInfo = Views[ViewIndex];

		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.SetViewRectangle(SceneCaptureViewInfo.ViewRect);
		ViewInitOptions.ViewFamily = &ViewFamily;
		ViewInitOptions.ViewActor = ViewActor;
		ViewInitOptions.ViewLocation = SceneCaptureViewInfo.ViewLocation;
		ViewInitOptions.ViewRotation = SceneCaptureViewInfo.ViewRotation;
		ViewInitOptions.ViewOrigin = SceneCaptureViewInfo.ViewOrigin;
		ViewInitOptions.ViewRotationMatrix = SceneCaptureViewInfo.ViewRotationMatrix;
		ViewInitOptions.BackgroundColor = FLinearColor::Black;
		ViewInitOptions.OverrideFarClippingPlaneDistance = MaxViewDistance;
		ViewInitOptions.StereoPass = SceneCaptureViewInfo.StereoPass;
		ViewInitOptions.StereoViewIndex = SceneCaptureViewInfo.StereoViewIndex;
		ViewInitOptions.ProjectionMatrix = SceneCaptureViewInfo.ProjectionMatrix;
		ViewInitOptions.SkylightScale = SceneCaptureComponent ? SceneCaptureComponent->SkylightScale : FLinearColor::White;
		ViewInitOptions.bIsSceneCapture = true;
		ViewInitOptions.bIsPlanarReflection = EnumHasAnyFlags(Flags, ESetupViewFamilyFlags::IsPlanarReflection);
		ViewInitOptions.bIsReflectionCapture = EnumHasAnyFlags(Flags, ESetupViewFamilyFlags::IsReflectionCapture);
		ViewInitOptions.FOV = SceneCaptureViewInfo.FOV;
		ViewInitOptions.DesiredFOV = SceneCaptureViewInfo.FOV;

		if (ViewFamily.Scene->GetWorld() != nullptr && ViewFamily.Scene->GetWorld()->GetWorldSettings() != nullptr)
		{
			ViewInitOptions.WorldToMetersScale = ViewFamily.Scene->GetWorld()->GetWorldSettings()->WorldToMeters;
		}

		if (EnumHasAnyFlags(Flags, ESetupViewFamilyFlags::CaptureSceneColor))
		{
			ViewFamily.EngineShowFlags.PostProcessing = 0;
			ViewInitOptions.OverlayColor = FLinearColor::Black;
		}

		if (SceneCaptureComponent)
		{
			// Use CubemapFaceIndex if in range [0..CubeFace_MAX), otherwise use ViewIndex.  Casting to unsigned treats -1 as a large value, choosing ViewIndex.
			ViewInitOptions.SceneViewStateInterface = SceneCaptureComponent->GetViewState((uint32)CubemapFaceIndex < CubeFace_MAX ? CubemapFaceIndex : ViewIndex + CubemapFaceOffset);
			ViewInitOptions.LODDistanceFactor = FMath::Clamp(SceneCaptureComponent->LODDistanceFactor, .01f, 100.0f);
			ViewInitOptions.bIsSceneCaptureCube = SceneCaptureComponent->IsCube();
			ViewInitOptions.bSceneCaptureUsesRayTracing = GRayTracingSceneCaptures == -1 ? SceneCaptureComponent->bUseRayTracingIfEnabled : GRayTracingSceneCaptures > 0;
			ViewInitOptions.bExcludeFromSceneTextureExtents = SceneCaptureComponent->bExcludeFromSceneTextureExtents;

			USceneCaptureComponent2D* SceneCaptureComponent2D = Cast<USceneCaptureComponent2D>(SceneCaptureComponent);
			if (IsValid(SceneCaptureComponent2D))
			{
				FMinimalViewInfo ViewInfo;
				SceneCaptureComponent2D->GetCameraView(0.0f, ViewInfo);
				ViewInitOptions.FirstPersonParams = FFirstPersonParameters(ViewInfo.CalculateFirstPersonFOVCorrectionFactor(), ViewInfo.FirstPersonScale, ViewInfo.bUseFirstPersonParameters);
			}
		}

		FSceneView* View = new FSceneView(ViewInitOptions);
		
		// Generate auto-exposure from all cube map faces.  Only affects cube captures with post processing enabled.  Adds 20% to the cost of post process
		// with the cheapest possible settings (only tonemap and FXAA enabled), or 2% of overall render time in a trival scene (0.09 ms on a high end card
		// at 1024 size).  If the performance hit was larger, we could consider an opt out CVar, but this seems fine.  Post processing for cube captures
		// was added in UE5.5, so there wouldn't be a lot of users of the feature affected by this minor perf impact.
		if (CubemapFaceIndex == CubeFace_MAX && ViewIndex == 0)
		{
			View->bEyeAdaptationAllViewPixels = true;
		}

		if (SceneCaptureComponent)
		{
			GetShowOnlyAndHiddenComponents(SceneCaptureComponent, View->HiddenPrimitives, View->ShowOnlyPrimitives);
		}
		
		ViewFamily.Views.Add(View);
		ViewPtrArray.Add(View);

		View->StartFinalPostprocessSettings(SceneCaptureViewInfo.ViewOrigin);

		if (InheritedMainViewPostProcessSettings)
		{
			View->FinalPostProcessSettings = *InheritedMainViewPostProcessSettings;
		}
		else
		{
			// Note: Future update to defaults should be reflected in the component constructors with backward-compatible serialization logic.

			// By default, Lumen is disabled in scene captures, but can be re-enabled with the post process settings in the component.
			View->FinalPostProcessSettings.DynamicGlobalIlluminationMethod = EDynamicGlobalIlluminationMethod::None;
			View->FinalPostProcessSettings.ReflectionMethod = EReflectionMethod::None;

			// Default surface cache to lower resolution for Scene Capture.  Can be overridden via post process settings.
			View->FinalPostProcessSettings.LumenSurfaceCacheResolution = 0.5f;

			if (SceneCaptureComponent && SceneCaptureComponent->IsCube())
			{
				// Disable vignette by default for cube maps -- darkened borders don't make sense for an omnidirectional projection.
				View->FinalPostProcessSettings.VignetteIntensity = 0.0f;

				// Disable screen traces by default for cube maps -- these don't blend well across face boundaries, creating major lighting seams.
				// Lumen lighting still has some seams with these disabled, but it's an order of magnitude better.
				View->FinalPostProcessSettings.LumenReflectionsScreenTraces = 0;
				View->FinalPostProcessSettings.LumenFinalGatherScreenTraces = 0;
			}
		}

		if (PostProcessSettings)
		{
			View->OverridePostProcessSettings(*PostProcessSettings, PostProcessBlendWeight);
		}
		View->EndFinalPostprocessSettings(ViewInitOptions);

		if (SceneCaptureComponent)
		{
			View->ViewLightingChannelMask = SceneCaptureComponent->ViewLightingChannels.GetMaskForStruct();
		}
	}

	return ViewPtrArray;
}

void SetupSceneViewExtensionsForSceneCapture(
	FSceneViewFamily& ViewFamily,
	TConstArrayView<FSceneView*> Views)
{
	for (const FSceneViewExtensionRef& Extension : ViewFamily.ViewExtensions)
	{
		Extension->SetupViewFamily(ViewFamily);
	}

	for (FSceneView* View : Views)
	{
		for (const FSceneViewExtensionRef& Extension : ViewFamily.ViewExtensions)
		{
			Extension->SetupView(ViewFamily, *View);
		}
	}
}

static FSceneRenderer* CreateSceneRendererForSceneCapture(
	ISceneRenderBuilder& SceneRenderBuilder,
	FScene* Scene,
	USceneCaptureComponent* SceneCaptureComponent,
	FRenderTarget* RenderTarget,
	FIntPoint RenderTargetSize,
	const FMatrix& ViewRotationMatrix,
	const FVector& ViewLocation,
	const FMatrix& ProjectionMatrix,
	float MaxViewDistance,
	float InFOV,
	ESceneRendererCreationFlags InFlags,
	FPostProcessSettings* PostProcessSettings,
	float PostProcessBlendWeight,
	const AActor* ViewActor,
	int32 CubemapFaceIndex = INDEX_NONE)
{
	FSceneCaptureViewInfo SceneCaptureViewInfo{};
	SceneCaptureViewInfo.ViewRotationMatrix = ViewRotationMatrix;
	SceneCaptureViewInfo.ViewOrigin = ViewLocation;
	SceneCaptureViewInfo.ViewLocation = ViewLocation;
	SceneCaptureViewInfo.ProjectionMatrix = ProjectionMatrix;
	SceneCaptureViewInfo.StereoPass = EStereoscopicPass::eSSP_FULL;
	SceneCaptureViewInfo.StereoViewIndex = INDEX_NONE;
	SceneCaptureViewInfo.ViewRect = FIntRect(0, 0, RenderTargetSize.X, RenderTargetSize.Y);
	SceneCaptureViewInfo.FOV = InFOV;

	bool bInheritMainViewScreenPercentage = false;
	const FFinalPostProcessSettings* InheritedMainViewPostProcessSettings = nullptr;
	USceneCaptureComponent2D* SceneCaptureComponent2D = Cast<USceneCaptureComponent2D>(SceneCaptureComponent);

	// Use camera position correction for ortho scene captures
	if(IsValid(SceneCaptureComponent2D))
	{
		const FSceneViewFamily* MainViewFamily = SceneCaptureComponent2D->MainViewFamily;

		if (!SceneCaptureViewInfo.IsPerspectiveProjection() && SceneCaptureComponent2D->bUpdateOrthoPlanes)
		{
			SceneCaptureViewInfo.UpdateOrthoPlanes(SceneCaptureComponent2D->bUseCameraHeightAsViewTarget);
		}

		if (SceneCaptureComponent2D->ShouldRenderWithMainViewResolution() && MainViewFamily && !SceneCaptureComponent2D->ShouldIgnoreScreenPercentage())
		{
			bInheritMainViewScreenPercentage = true;
		}

		if (SceneCaptureComponent2D->bInheritMainViewCameraPostProcessSettings && MainViewFamily)
		{
			InheritedMainViewPostProcessSettings = &MainViewFamily->Views[0]->FinalPostProcessSettings;
		}
	}
	else 
	{
		// These flags are reserved for 2D captures :
		check(!EnumHasAnyFlags(InFlags, ESceneRendererCreationFlags::CameraCut2D | ESceneRendererCreationFlags::CopyMainViewTemporalSettings2D));
	}

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		RenderTarget,
		Scene,
		SceneCaptureComponent->ShowFlags)
		.SetResolveScene(!EnumHasAnyFlags(InFlags, ESceneRendererCreationFlags::CaptureSceneColor))
		.SetRealtimeUpdate(SceneCaptureComponent->bCaptureEveryFrame || SceneCaptureComponent->bAlwaysPersistRenderingState));

	FSceneViewExtensionContext ViewExtensionContext(Scene);
	ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(ViewExtensionContext);

	ESetupViewFamilyFlags SetupViewFamilyFlags = ESetupViewFamilyFlags::None;
	if (EnumHasAnyFlags(InFlags, ESceneRendererCreationFlags::CaptureSceneColor))
	{
		SetupViewFamilyFlags |= ESetupViewFamilyFlags::CaptureSceneColor;
	}

	if (EnumHasAnyFlags(InFlags, ESceneRendererCreationFlags::OverrideVirtualTextureThrottle))
	{
		SetupViewFamilyFlags |= ESetupViewFamilyFlags::OverrideVirtualTextureThrottle;
	}

	TArray<FSceneView*> Views = SetupViewFamilyForSceneCapture(
		ViewFamily,
		SceneCaptureComponent,
		MakeArrayView(&SceneCaptureViewInfo, 1),
		MaxViewDistance, 
		SetupViewFamilyFlags,
		PostProcessSettings,
		InheritedMainViewPostProcessSettings,
		PostProcessBlendWeight,
		ViewActor,
		CubemapFaceIndex,
		0);

	// Scene capture source is used to determine whether to disable occlusion queries inside FSceneRenderer constructor
	ViewFamily.SceneCaptureSource = SceneCaptureComponent->CaptureSource;

	if (bInheritMainViewScreenPercentage)
	{
		ViewFamily.EngineShowFlags.ScreenPercentage = SceneCaptureComponent2D->MainViewFamily->EngineShowFlags.ScreenPercentage;
		ViewFamily.SetScreenPercentageInterface(SceneCaptureComponent2D->MainViewFamily->GetScreenPercentageInterface()->Fork_GameThread(ViewFamily));
	}
	else
	{
		// Screen percentage is still not supported in scene capture.
		ViewFamily.EngineShowFlags.ScreenPercentage = false;
		ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
			ViewFamily, /* GlobalResolutionFraction = */ 1.0f));
	}

	if (SceneCaptureComponent->IsUnlit())
	{
		bool bAllowAtmosphere =
			SceneCaptureComponent->CaptureSource == SCS_SceneColorHDR ||
			SceneCaptureComponent->CaptureSource == SCS_SceneColorHDRNoAlpha ||
			SceneCaptureComponent->CaptureSource == SCS_SceneColorSceneDepth;

		ViewFamily.EngineShowFlags.DisableFeaturesForUnlit(bAllowAtmosphere);
	}

	if (IsValid(SceneCaptureComponent2D))
	{
		// Scene capture 2D only support a single view
		check(Views.Num() == 1);

		// Ensure that the views for this scene capture reflect any simulated camera motion for this frame
		TOptional<FTransform> PreviousTransform = FMotionVectorSimulation::Get().GetPreviousTransform(SceneCaptureComponent2D);

		// Update views with scene capture 2d specific settings
		if (PreviousTransform.IsSet())
		{
			Views[0]->PreviousViewTransform = PreviousTransform.GetValue();
		}

		if (SceneCaptureComponent2D->bEnableClipPlane)
		{
			Views[0]->GlobalClippingPlane = FPlane(SceneCaptureComponent2D->ClipPlaneBase, SceneCaptureComponent2D->ClipPlaneNormal.GetSafeNormal());
			// Jitter can't be removed completely due to the clipping plane
			Views[0]->bAllowTemporalJitter = false;
		}

		Views[0]->bCameraCut = EnumHasAnyFlags(InFlags, ESceneRendererCreationFlags::CameraCut2D);

		if (EnumHasAnyFlags(InFlags, ESceneRendererCreationFlags::CopyMainViewTemporalSettings2D))
		{
			const FSceneViewFamily* MainViewFamily = SceneCaptureComponent2D->MainViewFamily;
			const FSceneView& SourceView = *MainViewFamily->Views[0];

			Views[0]->AntiAliasingMethod = SourceView.AntiAliasingMethod;
			Views[0]->PrimaryScreenPercentageMethod = SourceView.PrimaryScreenPercentageMethod;

			if (Views[0]->State && SourceView.State)
			{
				((FSceneViewState*)Views[0]->State)->TemporalAASampleIndex = ((const FSceneViewState*)SourceView.State)->TemporalAASampleIndex;
			}
		}

		// Append component-local view extensions to the view family
		for (int32 Index = 0; Index < SceneCaptureComponent2D->SceneViewExtensions.Num(); ++Index)
		{
			TSharedPtr<ISceneViewExtension, ESPMode::ThreadSafe> Extension = SceneCaptureComponent2D->SceneViewExtensions[Index].Pin();
			if (Extension.IsValid())
			{
				if (Extension->IsActiveThisFrame(ViewExtensionContext))
				{
					ViewFamily.ViewExtensions.Add(Extension.ToSharedRef());
				}
			}
			else
			{
				SceneCaptureComponent2D->SceneViewExtensions.RemoveAt(Index, EAllowShrinking::No);
				--Index;
			}
		}

		// For discoverability and backward compatibility, the unlit viewmode option is its own enum, rather than going through the
		// UnlitViewmode show flag.  The debug feature only works on non-shipping PC builds, and so going forward, the desired
		// default behavior for scene color Scene Captures running as Custom Render Passes is to disable the debug feature, for consistent
		// results across builds.  However, we can't change the default behavior for existing Scene Captures, as there may be licensees
		// always running PC development builds, using the debug behavior by design.
		//
		// The enum allows the creation of three states -- disabled across the board, enabled for captures, and enabled for both captures and
		// custom render passes, with the default being enabled only for captures.  This accomplishes both goals, and also allows licensees
		// using the debug feature by design to gain performance by switching to a CRP, and opting in to the debug feature there as well.
		//
		// Discoverability comes from the fact that setting the "Render In Main Renderer" flag (switching to a CRP) will toggle the debug
		// behavior, causing a visual change.  The "Unlit Viewmode" setting is immediately next to the flag the user just toggled,
		// and gives a clue as to what is happening, and allows them to choose a solution -- also enable the debug feature for the CRP, or
		// change their content to not assume the presence of the debug feature, and disable it across the board, depending on their goals.
		ViewFamily.EngineShowFlags.SetUnlitViewmode(SceneCaptureComponent2D->UnlitViewmode != ESceneCaptureUnlitViewmode::Disabled);
	}

	// Call SetupViewFamily & SetupView on scene view extensions before renderer creation
	SetupSceneViewExtensionsForSceneCapture(ViewFamily, Views);

	return SceneRenderBuilder.CreateSceneRenderer(&ViewFamily);
}

} // namespace UE::Renderer::SceneCapture::Private

FSceneCaptureCustomRenderPassUserData FSceneCaptureCustomRenderPassUserData::GDefaultData;

class FSceneCapturePass final : public FCustomRenderPassBase
{
public:
	IMPLEMENT_CUSTOM_RENDER_PASS(FSceneCapturePass);

	FSceneCapturePass(const FString& InDebugName, ERenderMode InRenderMode, ERenderOutput InRenderOutput, UTextureRenderTarget2D* InRenderTarget, USceneCaptureComponent2D* CaptureComponent, FIntPoint InRenderTargetSize)
		: FCustomRenderPassBase(InDebugName, InRenderMode, InRenderOutput, InRenderTargetSize)
		, SceneCaptureRenderTarget(InRenderTarget->GameThread_GetRenderTargetResource())
		, bAutoGenerateMips(InRenderTarget->bAutoGenerateMips)
	{
		FSceneCaptureCustomRenderPassUserData* UserData = new FSceneCaptureCustomRenderPassUserData();
		UserData->bMainViewFamily = CaptureComponent->ShouldRenderWithMainViewFamily();
		UserData->bMainViewResolution = CaptureComponent->ShouldRenderWithMainViewResolution();
		UserData->bMainViewCamera = CaptureComponent->ShouldRenderWithMainViewCamera();
		UserData->bIgnoreScreenPercentage = CaptureComponent->ShouldIgnoreScreenPercentage();
		UserData->bExcludeFromSceneTextureExtents = CaptureComponent->bExcludeFromSceneTextureExtents;
		UserData->SceneTextureDivisor = CaptureComponent->MainViewResolutionDivisor.ComponentMax(FIntPoint(1,1));
		UserData->UserSceneTextureBaseColor = CaptureComponent->UserSceneTextureBaseColor;
		UserData->UserSceneTextureNormal = CaptureComponent->UserSceneTextureNormal;
		UserData->UserSceneTextureSceneColor = CaptureComponent->UserSceneTextureSceneColor;
#if !UE_BUILD_SHIPPING
		CaptureComponent->GetOuter()->GetName(UserData->CaptureActorName);
#endif
		bSceneColorWithTranslucent = CaptureComponent->ShowFlags.Translucency;
		bSceneColorIsUserSceneTexture = !CaptureComponent->UserSceneTextureSceneColor.IsNone();

		SetUserData(TUniquePtr<FSceneCaptureCustomRenderPassUserData>(UserData));
	}

	virtual void OnPreRender(FRDGBuilder& GraphBuilder) override
	{
		// Resize the render resource if necessary -- render target size may have been overridden to the main view resolution, or later be changed back
		// to the resource resolution.  The resize call does nothing if the size already matches.
		((FTextureRenderTarget2DResource*)SceneCaptureRenderTarget)->Resize(GraphBuilder.RHICmdList, RenderTargetSize.X, RenderTargetSize.Y, bAutoGenerateMips);

		RenderTargetTexture = SceneCaptureRenderTarget->GetRenderTargetTexture(GraphBuilder);

		// If a Custom Render Pass output was accessed by Niagara, it will have converted it to external access, so switch it back to internal access.
		// If it's already in internal access mode (the common expected case), this is a nop.
		GraphBuilder.UseInternalAccessMode(RenderTargetTexture);
	}

	virtual void OnEndPass(FRDGBuilder& GraphBuilder) override
	{
		// Materials in the main view renderer will be using this render target, so we need RDG to transition it back to SRV now,
		// rather than at the end of graph execution.
		GraphBuilder.UseExternalAccessMode(RenderTargetTexture, ERHIAccess::SRVMask);
	}
	
	FRenderTarget* SceneCaptureRenderTarget = nullptr;
	bool bAutoGenerateMips = false;
};

ESceneRenderGroupFlags GetSceneCaptureGroupFlags(USceneCaptureComponent* CaptureComponent)
{
	ESceneRenderGroupFlags Flags = ESceneRenderGroupFlags::None;

	if (!CaptureComponent->bSuppressGpuCaptureOrDump)
	{
		if (CaptureComponent->bCaptureGpuNextRender)
		{
			CaptureComponent->bCaptureGpuNextRender = false;
			Flags |= ESceneRenderGroupFlags::GpuCapture;
		}

		if (CaptureComponent->bDumpGpuNextRender)
		{
			CaptureComponent->bDumpGpuNextRender = false;
			Flags |= ESceneRenderGroupFlags::GpuDump;
		}
	}

	CaptureComponent->bSuppressGpuCaptureOrDump = false;
	return Flags;
}

void FScene::UpdateSceneCaptureContents(USceneCaptureComponent2D* CaptureComponent, ISceneRenderBuilder& SceneRenderBuilder)
{
	using namespace UE::Renderer::SceneCapture::Private;
	check(CaptureComponent);

	const ESceneRenderGroupFlags SceneRenderGroupFlags = GetSceneCaptureGroupFlags(CaptureComponent);

	UTextureRenderTarget2D* TextureRenderTarget = CaptureComponent->TextureTarget;
	FTextureRenderTargetResource* TextureRenderTargetResource = TextureRenderTarget ? TextureRenderTarget->GameThread_GetRenderTargetResource() : nullptr;

	if (TextureRenderTargetResource)
	{
		FIntPoint CaptureSize;
		FVector ViewLocation;
		FMatrix ViewRotationMatrix;
		FMatrix ProjectionMatrix;
		bool bEnableOrthographicTiling;

		const bool bUseSceneColorTexture = CaptureNeedsSceneColor(CaptureComponent->CaptureSource);

		const int32 TileID = CaptureComponent->TileID;
		const int32 NumXTiles = CaptureComponent->GetNumXTiles();
		const int32 NumYTiles = CaptureComponent->GetNumYTiles();

		if (CaptureComponent->ShouldRenderWithMainViewResolution() && CaptureComponent->MainViewFamily)
		{
			CaptureSize = CaptureComponent->MainViewFamily->Views[0]->UnscaledViewRect.Size();
			CaptureSize = FIntPoint::DivideAndRoundUp(CaptureSize, CaptureComponent->MainViewResolutionDivisor.ComponentMax(FIntPoint(1,1)));

			// Main view resolution rendering doesn't support orthographic tiling
			bEnableOrthographicTiling = false;
		}
		else
		{
			CaptureSize = FIntPoint(TextureRenderTarget->GetSurfaceWidth(), TextureRenderTarget->GetSurfaceHeight());

			bEnableOrthographicTiling = (CaptureComponent->GetEnableOrthographicTiling() && CaptureComponent->ProjectionType == ECameraProjectionMode::Orthographic && bUseSceneColorTexture);

			if (CaptureComponent->GetEnableOrthographicTiling() && CaptureComponent->ProjectionType == ECameraProjectionMode::Orthographic && !bUseSceneColorTexture)
			{
				UE_LOGF(LogRenderer, Warning, "SceneCapture - Orthographic and tiling with CaptureSource not using SceneColor (i.e FinalColor) not compatible. SceneCapture render will not be tiled");
			}
		}

		if (CaptureComponent->ShouldRenderWithMainViewCamera() && CaptureComponent->MainViewFamily)
		{
			const FSceneView* MainView = CaptureComponent->MainViewFamily->Views[0];

			ViewLocation = MainView->ViewMatrices.GetViewOrigin();
			ViewRotationMatrix = MainView->ViewMatrices.GetWorldToView().RemoveTranslation();
			ProjectionMatrix = MainView->ViewMatrices.GetViewToClip();
		}
		else
		{
			FTransform Transform = CaptureComponent->GetComponentToWorld();
			ViewLocation = Transform.GetTranslation();

			// Remove the translation from Transform because we only need rotation.
			Transform.SetTranslation(FVector::ZeroVector);
			Transform.SetScale3D(FVector::OneVector);
			ViewRotationMatrix = Transform.ToInverseMatrixWithScale();

			// swap axis st. x=z,y=x,z=y (unreal coord space) so that z is up
			ViewRotationMatrix = ViewRotationMatrix * FMatrix(
				FPlane(0, 0, 1, 0),
				FPlane(1, 0, 0, 0),
				FPlane(0, 1, 0, 0),
				FPlane(0, 0, 0, 1));
			const float UnscaledFOV = CaptureComponent->FOVAngle * (float)PI / 360.0f;
			const float FOV = FMath::Atan((1.0f + CaptureComponent->Overscan) * FMath::Tan(UnscaledFOV));

			if (CaptureComponent->bUseCustomProjectionMatrix)
			{
				ProjectionMatrix = CaptureComponent->CustomProjectionMatrix;
			}
			else
			{
				if (CaptureComponent->ProjectionType == ECameraProjectionMode::Perspective)
				{
					const float NearClip = (CaptureComponent->bOverride_CustomNearClippingPlane) ? CaptureComponent->CustomNearClippingPlane : GNearClippingPlane;
					const float FarClip = CaptureComponent->MaxViewDistanceOverride > 0.0f && CaptureComponent->bFiniteFarPlane ?
						FMath::Max(CaptureComponent->MaxViewDistanceOverride, NearClip) : NearClip;

					BuildProjectionMatrix(CaptureSize, FOV, NearClip, FarClip, ProjectionMatrix);
				}
				else
				{
					if (bEnableOrthographicTiling)
					{
						BuildOrthoMatrix(CaptureSize, CaptureComponent->OrthoWidth, CaptureComponent->TileID, NumXTiles, NumYTiles, ProjectionMatrix);
						CaptureSize /= FIntPoint(NumXTiles, NumYTiles);
					}
					else
					{
						BuildOrthoMatrix(CaptureSize, CaptureComponent->OrthoWidth, -1, 0, 0, ProjectionMatrix);
					}
				}
			}
		}

		UMaterialParameterCollection* CollectionTransformTarget = CaptureComponent->CollectionTransform.Get();
		if (CollectionTransformTarget)
		{
			UMaterialParameterCollectionInstance* MaterialParameterCollectionInstance = World->GetParameterCollectionInstance(CollectionTransformTarget);
			bool bCollectionModified = false;

			// Find the parameters in the vector array
			int32 ParameterWorldToLocal = INDEX_NONE;
			int32 ParameterProjection = INDEX_NONE;
			for (int32 ParameterIndex = 0; ParameterIndex < CollectionTransformTarget->VectorParameters.Num(); ParameterIndex++)
			{
				FName ParameterName = CollectionTransformTarget->VectorParameters[ParameterIndex].ParameterName;
				if (!ParameterName.IsNone())
				{
					if (ParameterName == CaptureComponent->CollectionTransformWorldToLocal)
					{
						ParameterWorldToLocal = ParameterIndex;
					}
					else if (ParameterName == CaptureComponent->CollectionTransformProjection)
					{
						ParameterProjection = ParameterIndex;
					}
				}
			}

			// Structure to hold a matrix, plus LWC tile offset.
			struct FMatrixPlusTileOffset
			{
				FMatrix44f Matrix;
				FVector3f TileOffset;
				float Padding = 0.0f;
			};

			// Ensure there's space for 5 output vectors for the world to local matrix
			if (ParameterWorldToLocal != INDEX_NONE && ParameterWorldToLocal + 5 <= CollectionTransformTarget->VectorParameters.Num())
			{
				// Generate a view matrix (world space to local space) in LWC format.  This involves generating a tile origin using math
				// copied from FRelativeViewMatrices::Create, then subtracting that from the ViewLocation.  The tile offset gets subtracted
				// from the world position in the shader, before applying the matrix.
				const double TileSize = FLargeWorldRenderScalar::GetTileSize();
				const FVector ViewOriginTile = FLargeWorldRenderScalar::MakeQuantizedTile(ViewLocation, 8.0);

				FMatrix ViewMatrix = FTranslationMatrix(-(ViewLocation - ViewOriginTile * TileSize)) * ViewRotationMatrix;

				FMatrixPlusTileOffset MatrixPlusTileOffset;
				MatrixPlusTileOffset.Matrix = FMatrix44f(ViewMatrix);
				MatrixPlusTileOffset.TileOffset = (FVector3f)(ViewOriginTile);

				// Store the vectors to the collection instance
				const FLinearColor* MatrixPlusTileOffsetVectors = (const FLinearColor*)&MatrixPlusTileOffset;
				for (int32 ElementIndex = 0; ElementIndex < 5; ElementIndex++)
				{
					MaterialParameterCollectionInstance->SetVectorParameterValue(CollectionTransformTarget->VectorParameters[ParameterWorldToLocal + ElementIndex].ParameterName, MatrixPlusTileOffsetVectors[ElementIndex]);
				}

				bCollectionModified = true;
			}

			// Ensure there's space for 4 output vectors for the projection matrix
			if (ParameterProjection != INDEX_NONE && ParameterProjection + 4 <= CollectionTransformTarget->VectorParameters.Num())
			{
				FMatrix44f ProjectionMatrixf = FMatrix44f(ProjectionMatrix);

				// Store the vectors to the collection instance
				const FLinearColor* MatrixVectors = (const FLinearColor*)&ProjectionMatrixf;
				for (int32 ElementIndex = 0; ElementIndex < 4; ElementIndex++)
				{
					MaterialParameterCollectionInstance->SetVectorParameterValue(CollectionTransformTarget->VectorParameters[ParameterProjection + ElementIndex].ParameterName, MatrixVectors[ElementIndex]);
				}

				bCollectionModified = true;
			}

			if (bCollectionModified)
			{
				// Rendering runs after the world tick, where deferred material parameter collection updates normally occur,
				// so we need to manually update here, or the update will be delayed by a frame.
				MaterialParameterCollectionInstance->DeferredUpdateRenderState(false);
			}
		}

		// As optimization for depth capture modes, render scene capture as additional render passes inside the main renderer.
		if (GSceneCaptureAllowRenderInMainRenderer && CaptureComponent->ShouldRenderInMainRenderer())
		{
			FCustomRenderPassRendererInput PassInput;
			PassInput.ViewLocation = ViewLocation;
			PassInput.ViewRotationMatrix = ViewRotationMatrix;
			PassInput.ProjectionMatrix = ProjectionMatrix;
			PassInput.ViewActor = CaptureComponent->GetViewOwner();
			PassInput.bIsSceneCapture = true;

			FCustomRenderPassBase::ERenderMode RenderMode;
			FCustomRenderPassBase::ERenderOutput RenderOutput;
			const TCHAR* DebugName;

			bool bHasUserSceneTextureOutput = !CaptureComponent->UserSceneTextureBaseColor.IsNone() || !CaptureComponent->UserSceneTextureNormal.IsNone() || !CaptureComponent->UserSceneTextureSceneColor.IsNone();

			switch (CaptureComponent->CaptureSource)
			{
			case ESceneCaptureSource::SCS_SceneColorHDR:
				RenderMode = FCustomRenderPassBase::ERenderMode::DepthAndBasePass;
				RenderOutput = FCustomRenderPassBase::ERenderOutput::SceneColorAndAlpha;
				DebugName = TEXT("SceneCapturePass_SceneColorAndAlpha");
				break;
			case ESceneCaptureSource::SCS_SceneColorHDRNoAlpha:
				RenderMode = FCustomRenderPassBase::ERenderMode::DepthAndBasePass;
				RenderOutput = FCustomRenderPassBase::ERenderOutput::SceneColorNoAlpha;
				DebugName = TEXT("SceneCapturePass_SceneColorNoAlpha");
				break;
			case ESceneCaptureSource::SCS_SceneColorSceneDepth:
				RenderMode = FCustomRenderPassBase::ERenderMode::DepthAndBasePass;
				RenderOutput = FCustomRenderPassBase::ERenderOutput::SceneColorAndDepth;
				DebugName = TEXT("SceneCapturePass_SceneColorAndDepth");
				break;
			case ESceneCaptureSource::SCS_SceneDepth:
				if (bHasUserSceneTextureOutput)
				{
					// If a UserSceneTexture output is specified, the base pass needs to run to generate it.
					RenderMode = FCustomRenderPassBase::ERenderMode::DepthAndBasePass;
				}
				else
				{
					RenderMode = FCustomRenderPassBase::ERenderMode::DepthPass;
				}
				RenderOutput = FCustomRenderPassBase::ERenderOutput::SceneDepth;
				DebugName = TEXT("SceneCapturePass_SceneDepth");
				break;
			case ESceneCaptureSource::SCS_DeviceDepth:
				if (bHasUserSceneTextureOutput)
				{
					RenderMode = FCustomRenderPassBase::ERenderMode::DepthAndBasePass;
				}
				else
				{
					RenderMode = FCustomRenderPassBase::ERenderMode::DepthPass;
				}
				RenderOutput = FCustomRenderPassBase::ERenderOutput::DeviceDepth;
				DebugName = TEXT("SceneCapturePass_DeviceDepth");
				break;
			case ESceneCaptureSource::SCS_Normal:
				RenderMode = FCustomRenderPassBase::ERenderMode::DepthAndBasePass;
				RenderOutput = FCustomRenderPassBase::ERenderOutput::Normal;
				DebugName = TEXT("SceneCapturePass_Normal");
				break;
			case ESceneCaptureSource::SCS_BaseColor:
			default:
				RenderMode = FCustomRenderPassBase::ERenderMode::DepthAndBasePass;
				RenderOutput = FCustomRenderPassBase::ERenderOutput::BaseColor;
				DebugName = TEXT("SceneCapturePass_BaseColor");
				break;
			}

			FSceneCapturePass* CustomPass = new FSceneCapturePass(DebugName, RenderMode, RenderOutput, TextureRenderTarget, CaptureComponent, CaptureSize);
			PassInput.CustomRenderPass = CustomPass;

			GetShowOnlyAndHiddenComponents(CaptureComponent, PassInput.HiddenPrimitives, PassInput.ShowOnlyPrimitives);

			PassInput.EngineShowFlags = CaptureComponent->ShowFlags;
			PassInput.EngineShowFlags.DisableFeaturesForUnlit();

			if (CaptureComponent->UnlitViewmode == ESceneCaptureUnlitViewmode::CaptureOrCustomRenderPass)
			{
				PassInput.EngineShowFlags.SetUnlitViewmode(true);
			}

			if (CaptureComponent->PostProcessBlendWeight > 0.0f && CaptureComponent->PostProcessSettings.bOverride_UserFlags)
			{
				PassInput.PostVolumeUserFlags = CaptureComponent->PostProcessSettings.UserFlags;
				PassInput.bOverridesPostVolumeUserFlags = true;
			}

			// Caching scene capture info to be passed to the scene renderer.
			// #todo: We cannot (yet) guarantee for which ViewFamily this CRP will eventually be rendered since it will just execute the next time the scene is rendered by any FSceneRenderer. This seems quite problematic and could easily lead to unexpected behavior...
			AddCustomRenderPass(nullptr, PassInput);
			return;
		}

		ESceneRendererCreationFlags SceneRendererCreationFlags = ESceneRendererCreationFlags::None;
		if (bUseSceneColorTexture)
		{
			SceneRendererCreationFlags |= ESceneRendererCreationFlags::CaptureSceneColor;
		}

		// Copy temporal AA related settings for main view camera scene capture, to match jitter.  Don't match if resolution differs from main view,
		// if it's set to ignore screen percentage, or if it's final color, which will run its own AA.  For custom render passes (handled above),
		// computed jitter results are copied from the main view later in FSceneRenderer::PrepareViewStateForVisibility, but this doesn't work
		// for regular scene captures, because they run in a separate scene renderer before the main view, where the main view's results haven't
		// been computed yet.
		if ((CaptureComponent->MainViewFamily &&
			CaptureComponent->ShouldRenderWithMainViewCamera() &&
			CaptureComponent->ShouldRenderWithMainViewResolution() &&
			CaptureComponent->MainViewResolutionDivisor.X <= 1 && CaptureComponent->MainViewResolutionDivisor.Y <= 1 && !CaptureComponent->ShouldIgnoreScreenPercentage() &&
			CaptureComponent->CaptureSource != ESceneCaptureSource::SCS_FinalColorLDR &&
			CaptureComponent->CaptureSource != ESceneCaptureSource::SCS_FinalColorHDR &&
			CaptureComponent->CaptureSource != ESceneCaptureSource::SCS_FinalToneCurveHDR))
		{
			SceneRendererCreationFlags |= ESceneRendererCreationFlags::CopyMainViewTemporalSettings2D;
			if (CaptureComponent->MainViewFamily->Views[0]->bCameraCut)
			{
				SceneRendererCreationFlags |= ESceneRendererCreationFlags::CameraCut2D;
			}
		}
		else
		{
			if (CaptureComponent->bCameraCutThisFrame)
			{
				SceneRendererCreationFlags |= ESceneRendererCreationFlags::CameraCut2D;
			}
		}

		if (CaptureComponent->bOverrideVirtualTextureThrottle)
		{
			SceneRendererCreationFlags |= ESceneRendererCreationFlags::OverrideVirtualTextureThrottle;
		}

		FSceneRenderer* SceneRenderer = CreateSceneRendererForSceneCapture(
			SceneRenderBuilder,
			this, 
			CaptureComponent, 
			TextureRenderTarget->GameThread_GetRenderTargetResource(), 
			CaptureSize, 
			ViewRotationMatrix, 
			ViewLocation, 
			ProjectionMatrix, 
			CaptureComponent->MaxViewDistanceOverride, 
			CaptureComponent->FOVAngle,
			SceneRendererCreationFlags,
			&CaptureComponent->PostProcessSettings, 
			CaptureComponent->PostProcessBlendWeight,
			CaptureComponent->GetViewOwner());

		check(SceneRenderer != nullptr);

		SceneRenderer->Views[0].bSceneCaptureMainViewJitter = EnumHasAnyFlags(SceneRendererCreationFlags, ESceneRendererCreationFlags::CopyMainViewTemporalSettings2D);
		SceneRenderer->Views[0].bFogOnlyOnRenderedOpaque = CaptureComponent->bConsiderUnrenderedOpaquePixelAsFullyTranslucent;

		SceneRenderer->ViewFamily.SceneCaptureCompositeMode = CaptureComponent->CompositeMode;

		// Need view state interface to be allocated for Lumen, as it requires persistent data.  This means
		// "bCaptureEveryFrame" or "bAlwaysPersistRenderingState" must be enabled.
		FSceneViewStateInterface* ViewStateInterface = CaptureComponent->GetViewState(0);

		if (ViewStateInterface &&
			(SceneRenderer->Views[0].FinalPostProcessSettings.DynamicGlobalIlluminationMethod == EDynamicGlobalIlluminationMethod::Lumen ||
			 SceneRenderer->Views[0].FinalPostProcessSettings.ReflectionMethod == EReflectionMethod::Lumen))
		{
			// It's OK to call these every frame -- they are no-ops if the correct data is already there
			ViewStateInterface->AddLumenSceneData(this, SceneRenderer->Views[0].FinalPostProcessSettings.LumenSurfaceCacheResolution);
		}
		else if (ViewStateInterface)
		{
			ViewStateInterface->RemoveLumenSceneData(this);
		}

		// Reset scene capture's camera cut.
		CaptureComponent->bCameraCutThisFrame = false;

		FString EventName;
		if (!CaptureComponent->ProfilingEventName.IsEmpty())
		{
			EventName = CaptureComponent->ProfilingEventName;
		}
		else if (CaptureComponent->GetOwner())
		{
			// The label might be non-unique, so include the actor name as well
			EventName = CaptureComponent->GetOwner()->GetActorNameOrLabel();

			FName ActorName = CaptureComponent->GetOwner()->GetFName();
			if (ActorName != EventName)
			{
				EventName.Appendf(TEXT(" (%s)"), *ActorName.ToString());
			}
		}
		FName TargetName = TextureRenderTarget->GetFName();

		const bool bGenerateMips = TextureRenderTarget->bAutoGenerateMips;
		FGenerateMipsParams GenerateMipsParams{TextureRenderTarget->MipsSamplerFilter == TF_Nearest ? SF_Point : (TextureRenderTarget->MipsSamplerFilter == TF_Trilinear ? SF_Trilinear : SF_Bilinear),
			TextureRenderTarget->MipsAddressU == TA_Wrap ? AM_Wrap : (TextureRenderTarget->MipsAddressU == TA_Mirror ? AM_Mirror : AM_Clamp),
			TextureRenderTarget->MipsAddressV == TA_Wrap ? AM_Wrap : (TextureRenderTarget->MipsAddressV == TA_Mirror ? AM_Mirror : AM_Clamp)};

		const bool bOrthographicCamera = CaptureComponent->ProjectionType == ECameraProjectionMode::Orthographic;


		// If capturing every frame, only render to the GPUs that are actually being used
		// this frame. We can only determine this by querying the viewport back buffer on
		// the render thread, so pass that along if it exists.
		FRenderTarget* GameViewportRT = nullptr;
		if (CaptureComponent->bCaptureEveryFrame)
		{
			if (GEngine->GameViewport != nullptr)
			{
				GameViewportRT = GEngine->GameViewport->Viewport;
			}
		}

		// Compositing feature is only active when using SceneColor as the source
		bool bIsCompositing = (CaptureComponent->CompositeMode != SCCM_Overwrite) && (CaptureComponent->CaptureSource == SCS_SceneColorHDR);
#if WITH_EDITOR
		if (!CaptureComponent->CaptureMemorySize)
		{
			CaptureComponent->CaptureMemorySize = new FSceneCaptureMemorySize;
		}
		TRefCountPtr<FSceneCaptureMemorySize> CaptureMemorySize = CaptureComponent->CaptureMemorySize;
#else
		void* CaptureMemorySize = nullptr;
#endif

		ENQUEUE_RENDER_COMMAND(ResizeSceneCapture)([TextureRenderTargetResource, CaptureSize, bGenerateMips, GameViewportRT] (FRHICommandListImmediate& RHICmdList)
		{
			// Resize the render resource if necessary, either to the main viewport size overridden above (see ShouldRenderWithMainViewResolution()),
			// or the original size if we are changing back to that (the resize call does nothing if the size already matches).
			TextureRenderTargetResource->GetTextureRenderTarget2DResource()->Resize(RHICmdList, CaptureSize.X, CaptureSize.Y, bGenerateMips);

			if (GameViewportRT != nullptr)
			{
				TextureRenderTargetResource->SetActiveGPUMask(GameViewportRT->GetGPUMask(RHICmdList));
			}
			else
			{
				TextureRenderTargetResource->SetActiveGPUMask(FRHIGPUMask::All());
			}
		});

		SCENE_RENDER_GROUP_SCOPE(SceneRenderBuilder, MoveTemp(EventName), SceneRenderGroupFlags);

		SceneRenderBuilder.AddRenderer(
			SceneRenderer,
			[
				  TextureRenderTargetResource
				, TextureRenderTarget
				, TargetName
				, bGenerateMips
				, GenerateMipsParams
				, bEnableOrthographicTiling
				, bIsCompositing
				, bOrthographicCamera
				, NumXTiles
				, NumYTiles
				, TileID
				, CaptureMemorySize
				, CaptureSize
			] (FRDGBuilder& GraphBuilder, const FSceneRenderFunctionInputs& Inputs)
		{
			FRHICopyTextureInfo CopyInfo;

			if (bEnableOrthographicTiling)
			{
				const uint32 RTSizeX = TextureRenderTargetResource->GetSizeX() / NumXTiles;
				const uint32 RTSizeY = TextureRenderTargetResource->GetSizeY() / NumYTiles;
				const uint32 TileX = TileID % NumXTiles;
				const uint32 TileY = TileID / NumXTiles;
				CopyInfo.DestPosition.X = TileX * RTSizeX;
				CopyInfo.DestPosition.Y = TileY * RTSizeY;
				CopyInfo.Size.X = RTSizeX;
				CopyInfo.Size.Y = RTSizeY;
			}

			RectLightAtlas::FAtlasTextureInvalidationScope Invalidation(TextureRenderTarget);

			// Don't clear the render target when compositing, or in a tiling mode that fills in the render target in multiple passes.
			const bool bClearRenderTarget = !bIsCompositing && !bEnableOrthographicTiling;

			FRDGTextureRef OutputTexture = RegisterExternalTexture(GraphBuilder, TextureRenderTargetResource->TextureRHI, TEXT("SceneCaptureTexture"));
			UpdateSceneCaptureContent_RenderThread(
				GraphBuilder,
				Inputs.Renderer,
				Inputs.SceneUpdateInputs,
				TextureRenderTargetResource,
				OutputTexture,
				MakeArrayView({ CopyInfo }),
				bGenerateMips,
				GenerateMipsParams,
				bClearRenderTarget,
				bOrthographicCamera);

			GraphBuilder.SetTextureAccessFinal(OutputTexture, ERHIAccess::SRVMask);

		#if WITH_EDITOR
			if (const FSceneViewState* ViewState = Inputs.Renderer->Views[0].ViewState)
			{
				if (ViewState)
				{
					const bool bLogSizes = GDumpSceneCaptureMemoryFrame == GFrameNumberRenderThread;
					if (bLogSizes)
					{
						UE_LOGF(LogRenderer, Log, "LogSizes\tSceneCapture\t%ls\t%ls\t%dx%d", Inputs.FullPath, *TargetName.ToString(), TextureRenderTargetResource->GetSizeX(), TextureRenderTargetResource->GetSizeY());
					}
					CaptureMemorySize->Size = ViewState->GetGPUSizeBytes(bLogSizes);
				}
				else
				{
					CaptureMemorySize->Size = 0;
				}
			}
		#endif  // WITH_EDITOR

			return true;
		});
	}
}

// Split screen cube map faces are rendered as 3x2 tiles.
static const int32 GCubeFaceViewportOffsets[6][2] =
{
	{ 0,0 },
	{ 1,0 },
	{ 2,0 },
	{ 0,1 },
	{ 1,1 },
	{ 2,1 },
};

void FScene::UpdateSceneCaptureContents(USceneCaptureComponentCube* CaptureComponent, ISceneRenderBuilder& SceneRenderBuilder)
{
	using namespace UE::Renderer::SceneCapture::Private;
	ESceneRenderGroupFlags SceneRenderGroupFlags = GetSceneCaptureGroupFlags(CaptureComponent);

	struct FLocal
	{
		/** Creates a transformation for a cubemap face, following the D3D cubemap layout. */
		static FMatrix CalcCubeFaceTransform(ECubeFace Face)
		{
			static const FVector XAxis(1.f, 0.f, 0.f);
			static const FVector YAxis(0.f, 1.f, 0.f);
			static const FVector ZAxis(0.f, 0.f, 1.f);

			// vectors we will need for our basis
			FVector vUp(YAxis);
			FVector vDir;
			switch (Face)
			{
				case CubeFace_PosX:
					vDir = XAxis;
					break;
				case CubeFace_NegX:
					vDir = -XAxis;
					break;
				case CubeFace_PosY:
					vUp = -ZAxis;
					vDir = YAxis;
					break;
				case CubeFace_NegY:
					vUp = ZAxis;
					vDir = -YAxis;
					break;
				case CubeFace_PosZ:
					vDir = ZAxis;
					break;
				case CubeFace_NegZ:
					vDir = -ZAxis;
					break;
			}
			// derive right vector
			FVector vRight(vUp ^ vDir);
			// create matrix from the 3 axes
			return FBasisVectorMatrix(vRight, vUp, vDir, FVector::ZeroVector);
		}
	} ;

	check(CaptureComponent);

	FTransform Transform = CaptureComponent->GetComponentToWorld();
	const FVector ViewLocation = Transform.GetTranslation();

	if (CaptureComponent->bCaptureRotation)
	{
		// Remove the translation from Transform because we only need rotation.
		Transform.SetTranslation(FVector::ZeroVector);
		Transform.SetScale3D(FVector::OneVector);
	}

	bool bIsReflectionCapture = CaptureComponent->ReflectionComponent != nullptr;

	UTextureRenderTargetCube* TextureTarget = nullptr;
	FTextureRenderTargetCubeResource* TextureRenderTarget = nullptr;
	bool bGenerateMips;
	FGenerateMipsParams GenerateMipsParams;
	FIntPoint CaptureSize;

	if (bIsReflectionCapture)
	{
		uint32 ReflectionCaptureSize = UReflectionCaptureComponent::GetReflectionCaptureSize();
		bGenerateMips = true;
		CaptureSize = FIntPoint(ReflectionCaptureSize, ReflectionCaptureSize);
	}
	else if (CaptureComponent->TextureTarget)
	{
		TextureTarget = CaptureComponent->TextureTarget;
		TextureRenderTarget = static_cast<FTextureRenderTargetCubeResource*>(TextureTarget->GameThread_GetRenderTargetResource());
		bGenerateMips = TextureTarget->bAutoGenerateMips;
		GenerateMipsParams = FGenerateMipsParams{ TextureTarget->MipsSamplerFilter == TF_Nearest ? SF_Point : (TextureTarget->MipsSamplerFilter == TF_Trilinear ? SF_Trilinear : SF_Bilinear), AM_Clamp, AM_Clamp };
		CaptureSize = FIntPoint(TextureTarget->GetSurfaceWidth(), TextureTarget->GetSurfaceHeight());
	}
	else
	{
		return;
	}

	FString EventName;
	if (!CaptureComponent->ProfilingEventName.IsEmpty())
	{
		EventName = CaptureComponent->ProfilingEventName;
	}
	else
	{
		AActor* Owner = nullptr;
		if (bIsReflectionCapture)
		{
			// For reflection captures, use the reflection capture actor name (reflection captures use a singleton CaptureComponent)
			Owner = CaptureComponent->ReflectionComponent->GetOwner();
		}
		else
		{
			Owner = CaptureComponent->GetOwner();
		}

		if (Owner)
		{
			// The label might be non-unique, so include the actor name as well
			EventName = Owner->GetActorNameOrLabel();

			FName ActorName = Owner->GetFName();
			if (ActorName != EventName)
			{
				EventName.Appendf(TEXT(" (%s)"), *ActorName.ToString());
			}
		}
	}

	const float FOVInDegrees = 90.f;
	const float FOVInRadians = FOVInDegrees * (float)PI / 360.0f;

	auto ComputeProjectionMatrix = [CaptureComponent, Transform, CaptureSize, FOVInRadians](ECubeFace TargetFace, FMatrix& OutViewRotationMatrix, FMatrix& OutProjectionMatrix)
	{
		if (CaptureComponent->bCaptureRotation)
		{
			OutViewRotationMatrix = Transform.ToInverseMatrixWithScale() * FLocal::CalcCubeFaceTransform(TargetFace);
		}
		else
		{
			OutViewRotationMatrix = FLocal::CalcCubeFaceTransform(TargetFace);
		}
		const float FarClip = CaptureComponent->MaxViewDistanceOverride > 0.0f && CaptureComponent->bFiniteFarPlane ?
			FMath::Max(CaptureComponent->MaxViewDistanceOverride, GNearClippingPlane) : GNearClippingPlane;

		BuildProjectionMatrix(CaptureSize, FOVInRadians, GNearClippingPlane, FarClip, OutProjectionMatrix);
	};

	const FVector Location = CaptureComponent->GetComponentToWorld().GetTranslation();

	ESceneRendererCreationFlags SceneRendererCreationFlags = ESceneRendererCreationFlags::None;

	if (CaptureNeedsSceneColor(CaptureComponent->CaptureSource))
	{
		SceneRendererCreationFlags |= ESceneRendererCreationFlags::CaptureSceneColor;
	}

	if (CaptureComponent->bOverrideVirtualTextureThrottle)
	{
		SceneRendererCreationFlags |= ESceneRendererCreationFlags::OverrideVirtualTextureThrottle;
	}
	
	if (bIsReflectionCapture)
	{
		SceneRendererCreationFlags |= ESceneRendererCreationFlags::IsReflectionCapture;
	}

	SCENE_RENDER_GROUP_SCOPE(SceneRenderBuilder, MoveTemp(EventName), SceneRenderGroupFlags);

	// Reflection captures only support the single pass render case below, which uses a separately allocated render target for the six faces.  The reflection
	// environment cube map array isn't configured to be used as a render target, and direct individual face rendering would require that.
	if (GSceneCaptureCubeSinglePass == false && !bIsReflectionCapture)
	{
		for (int32 FaceIndex = 0; FaceIndex < (int32)ECubeFace::CubeFace_MAX; FaceIndex++)
		{
			const ECubeFace TargetFace = (ECubeFace)FaceIndex;

			FMatrix ViewRotationMatrix;
			FMatrix ProjectionMatrix;
			ComputeProjectionMatrix(TargetFace, ViewRotationMatrix, ProjectionMatrix);

			FSceneRenderer* SceneRenderer = CreateSceneRendererForSceneCapture(
				SceneRenderBuilder,
				this,
				CaptureComponent,
				TextureRenderTarget,
				CaptureSize,
				ViewRotationMatrix,
				Location,
				ProjectionMatrix,
				CaptureComponent->MaxViewDistanceOverride,
				FOVInDegrees,
				SceneRendererCreationFlags,
				&CaptureComponent->PostProcessSettings,
				CaptureComponent->PostProcessBlendWeight,
				CaptureComponent->GetViewOwner(),
				FaceIndex);

			SceneRenderBuilder.AddRenderer(SceneRenderer, FString::Printf(TEXT("CubeFace[%d]"), FaceIndex),
				[
					TextureRenderTarget
					, TargetFace
					, bGenerateMips
					, GenerateMipsParams
				] (FRDGBuilder& GraphBuilder, const FSceneRenderFunctionInputs& Inputs)
			{
				// We need to generate mips on last cube face
				const bool bLastCubeFace = ((int32)TargetFace == (int32)ECubeFace::CubeFace_MAX - 1);
				const bool bClearRenderTarget = true;
				const bool bOrthographicCamera = false;

				FRHICopyTextureInfo CopyInfo;
				CopyInfo.DestSliceIndex = TargetFace;

				FRDGTextureRef OutputTexture = RegisterExternalTexture(GraphBuilder, TextureRenderTarget->TextureRHI, TEXT("SceneCaptureTexture"));
				UpdateSceneCaptureContent_RenderThread(
					GraphBuilder,
					Inputs.Renderer,
					Inputs.SceneUpdateInputs,
					TextureRenderTarget,
					OutputTexture,
					MakeArrayView({ CopyInfo }),
					bGenerateMips && bLastCubeFace,
					GenerateMipsParams,
					bClearRenderTarget,
					bOrthographicCamera);

				GraphBuilder.SetTextureAccessFinal(OutputTexture, ERHIAccess::SRVMask);

#if WITH_EDITOR
				if (const FSceneViewState* ViewState = Inputs.Renderer->Views[0].ViewState)
				{
					const bool bLogSizes = GDumpSceneCaptureMemoryFrame == GFrameNumberRenderThread;
					if (bLogSizes)
					{
						UE_LOGF(LogRenderer, Log, "LogSizes\tSceneCaptureCube[%d]\t%ls\t%dx%d", TargetFace, Inputs.FullPath, TextureRenderTarget->GetSizeX(), TextureRenderTarget->GetSizeY());
						ViewState->GetGPUSizeBytes(bLogSizes);
					}
				}
#endif  // WITH_EDITOR

				return true;
			});
		}
	}
	else
	{
		int32 TimeSliceFirst = 0;
		int32 TimeSliceMask = (1 << CubeFace_MAX) - 1;		// All faces rendered
		if (bIsReflectionCapture)
		{
			int32 TimeSliceCount = FMath::Clamp(CaptureComponent->ReflectionTimeSliceCount, 1, CubeFace_MAX);
			TimeSliceFirst = FMath::Clamp(CaptureComponent->ReflectionTimeSliceFirst, 0, CubeFace_MAX - 1);
			TimeSliceMask &= ((1 << TimeSliceCount) - 1) << TimeSliceFirst;
		}

		TArray<FSceneCaptureViewInfo, TFixedAllocator<ECubeFace::CubeFace_MAX>> SceneCaptureViewInfos;
		for (int32 faceidx = 0; faceidx < (int32)ECubeFace::CubeFace_MAX; faceidx++)
		{
			if (!(TimeSliceMask & (1 << faceidx)))
			{
				continue;
			}

			const ECubeFace TargetFace = (ECubeFace)faceidx;

			FMatrix ViewRotationMatrix;
			FMatrix ProjectionMatrix;
			ComputeProjectionMatrix(TargetFace, ViewRotationMatrix, ProjectionMatrix);

			FIntPoint ViewportOffset(GCubeFaceViewportOffsets[faceidx][0] * CaptureSize.X, GCubeFaceViewportOffsets[faceidx][1] * CaptureSize.Y);

			FSceneCaptureViewInfo& SceneCaptureViewInfo = SceneCaptureViewInfos.AddDefaulted_GetRef();
			SceneCaptureViewInfo.ViewRotationMatrix = ViewRotationMatrix;
			SceneCaptureViewInfo.ViewOrigin = ViewLocation;
			SceneCaptureViewInfo.ProjectionMatrix = ProjectionMatrix;
			SceneCaptureViewInfo.StereoPass = EStereoscopicPass::eSSP_FULL;
			SceneCaptureViewInfo.StereoViewIndex = INDEX_NONE;
			SceneCaptureViewInfo.ViewRect = FIntRect(ViewportOffset.X, ViewportOffset.Y, ViewportOffset.X + CaptureSize.X, ViewportOffset.Y + CaptureSize.Y);
			SceneCaptureViewInfo.FOV = 90.f;
		}

		// Render target that includes all six tiled faces of the cube map
		class FCubeFaceRenderTarget final : public FRenderTarget
		{
		public:
			FCubeFaceRenderTarget(FTextureRenderTargetCubeResource* InTextureRenderTarget, TRefCountPtr<IPooledRenderTarget>* InExternalTarget, FIntPoint CaptureSize)
			{
				// Cache a pointer to the output texture so we can get the pixel format later (InitRHI may not have been called on InTextureRenderTarget)
				TextureRenderTarget = InTextureRenderTarget;
				ExternalTarget = InExternalTarget;

				// Assume last cube face viewport offset is the furthest corner of the tiled cube face render target.
				// Add one to include the dimensions of the tile in addition to the offset.
				FIntPoint Size;
				Size.X = CaptureSize.X * (GCubeFaceViewportOffsets[(int32)ECubeFace::CubeFace_MAX - 1][0] + 1);
				Size.Y = CaptureSize.Y * (GCubeFaceViewportOffsets[(int32)ECubeFace::CubeFace_MAX - 1][1] + 1);

				CubeFaceDesc = FPooledRenderTargetDesc::Create2DDesc(
					Size,
					PF_Unknown,							// Initialized in InitRHI below
					FClearValueBinding::Green,
					TexCreate_None,
					TexCreate_ShaderResource | TexCreate_RenderTargetable,
					false);
			}

			void InitRHI(FRHICommandListImmediate& RHICmdList)
			{
				// Set the format now that it's available.  If no render target was specified, we are rendering a reflection capture, which is 16-bit float RGBA.
				CubeFaceDesc.Format = TextureRenderTarget ? TextureRenderTarget->GetRenderTargetTexture()->GetFormat() : PF_FloatRGBA;

				// Reallocate the cached external target if the dimensions or format change.
				const bool bCanReuseExternalTarget = ExternalTarget
					&& ExternalTarget->IsValid()
					&& (*ExternalTarget)->GetDesc().Extent == CubeFaceDesc.Extent
					&& (*ExternalTarget)->GetDesc().Format == CubeFaceDesc.Format;

				if (bCanReuseExternalTarget)
				{
					RenderTarget = *ExternalTarget;
				}
				else
				{
					GRenderTargetPool.FindFreeElement(RHICmdList, CubeFaceDesc, RenderTarget, TEXT("SceneCaptureTarget"));
					check(RenderTarget);

					if (ExternalTarget)
					{
						*ExternalTarget = RenderTarget;
					}
				}

				RenderTargetTexture = RenderTarget->GetRHI();
			}

			// FRenderTarget interface
			const FTextureRHIRef& GetRenderTargetTexture() const override
			{
				return RenderTargetTexture;
			}

			FIntPoint GetSizeXY() const override { return CubeFaceDesc.Extent; }
			float GetDisplayGamma() const override { return 1.0f; }

		private:
			FTextureRenderTargetCubeResource* TextureRenderTarget;
			TRefCountPtr<IPooledRenderTarget>* ExternalTarget;
			FPooledRenderTargetDesc CubeFaceDesc;
			TRefCountPtr<IPooledRenderTarget> RenderTarget;
			FTextureRHIRef RenderTargetTexture;
		};

		TUniquePtr<FCubeFaceRenderTarget> CubeFaceTarget = MakeUnique<FCubeFaceRenderTarget>(TextureRenderTarget, bIsReflectionCapture ? &ReflectionSceneData.RuntimeCaptureTarget : nullptr, CaptureSize);

		// Copied from CreateSceneRendererForSceneCapture
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			CubeFaceTarget.Get(),
			this,
			CaptureComponent->ShowFlags)
			.SetResolveScene(!EnumHasAnyFlags(SceneRendererCreationFlags, ESceneRendererCreationFlags::CaptureSceneColor))
			.SetRealtimeUpdate(CaptureComponent->bCaptureEveryFrame || CaptureComponent->bAlwaysPersistRenderingState));

		FSceneViewExtensionContext ViewExtensionContext(this);
		ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(ViewExtensionContext);

		ESetupViewFamilyFlags SetupViewFamilyFlags = ESetupViewFamilyFlags::None;
		if (EnumHasAnyFlags(SceneRendererCreationFlags, ESceneRendererCreationFlags::CaptureSceneColor))
		{
			SetupViewFamilyFlags |= ESetupViewFamilyFlags::CaptureSceneColor;
		}

		if (EnumHasAnyFlags(SceneRendererCreationFlags, ESceneRendererCreationFlags::IsReflectionCapture))
		{
			SetupViewFamilyFlags |= ESetupViewFamilyFlags::IsReflectionCapture;
		}

		TArray<FSceneView*> Views = SetupViewFamilyForSceneCapture(
			ViewFamily,
			CaptureComponent,
			SceneCaptureViewInfos,
			CaptureComponent->MaxViewDistanceOverride,
			SetupViewFamilyFlags,
			&CaptureComponent->PostProcessSettings,
			nullptr,
			CaptureComponent->PostProcessBlendWeight,
			CaptureComponent->GetViewOwner(),
			(int32)ECubeFace::CubeFace_MAX,			// Passing max cube face count indicates a view family with all faces
			TimeSliceFirst);

		// Scene capture source is used to determine whether to disable occlusion queries inside FSceneRenderer constructor
		ViewFamily.SceneCaptureSource = CaptureComponent->CaptureSource;

		// Screen percentage is still not supported in scene capture.
		ViewFamily.EngineShowFlags.ScreenPercentage = false;
		ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
			ViewFamily, /* GlobalResolutionFraction = */ 1.0f));

		// Call SetupViewFamily & SetupView on scene view extensions before renderer creation
		SetupSceneViewExtensionsForSceneCapture(ViewFamily, Views);

		FSceneRenderer* SceneRenderer = SceneRenderBuilder.CreateSceneRenderer(&ViewFamily);

		// Need view state interface to be allocated for Lumen, as it requires persistent data.  This means
		// "bCaptureEveryFrame" or "bAlwaysPersistRenderingState" must be enabled.
		FSceneViewStateInterface* ViewStateInterface = CaptureComponent->GetViewState(0);

		if (ViewStateInterface &&
			(SceneRenderer->Views[0].FinalPostProcessSettings.DynamicGlobalIlluminationMethod == EDynamicGlobalIlluminationMethod::Lumen ||
				SceneRenderer->Views[0].FinalPostProcessSettings.ReflectionMethod == EReflectionMethod::Lumen))
		{
			// It's OK to call these every frame -- they are no-ops if the correct data is already there
			ViewStateInterface->AddLumenSceneData(this, SceneRenderer->Views[0].FinalPostProcessSettings.LumenSurfaceCacheResolution);
		}
		else if (ViewStateInterface)
		{
			ViewStateInterface->RemoveLumenSceneData(this);
		}

		SceneRenderBuilder.AddRenderer(SceneRenderer,
			[
				CubeFaceTarget = MoveTemp(CubeFaceTarget)
				, TextureRenderTarget
				, CaptureSize
				, bGenerateMips
				, GenerateMipsParams
				, ReflectionComponent = CaptureComponent->ReflectionComponent
				, TimeSliceMask
				, TimeSliceFirst
			] (FRDGBuilder& GraphBuilder, const FSceneRenderFunctionInputs& Inputs)
		{
			FReflectionEnvironmentSceneData& ReflectionSceneData = Inputs.Renderer->Scene->ReflectionSceneData;
			int32 ReflectionCaptureIndex = INDEX_NONE;
			if (ReflectionComponent)
			{
				// Assume allocation of the reflection state may have failed, and skip rendering
				FCaptureComponentSceneState* CaptureSceneState = ReflectionSceneData.AllocatedReflectionCaptureState.Find(ReflectionComponent);
				if (!CaptureSceneState)
				{
					return false;
				}
				ReflectionCaptureIndex = CaptureSceneState->CubemapIndex;
			}

			CubeFaceTarget->InitRHI(GraphBuilder.RHICmdList);

			const bool bIncludesLastFace = TimeSliceMask & (1 << (CubeFace_MAX - 1));

			TArray<FRHICopyTextureInfo, TFixedAllocator<ECubeFace::CubeFace_MAX>> CopyInfos;

			if (bIncludesLastFace)
			{
				for (int32 FaceIndex = 0; FaceIndex < (int32)ECubeFace::CubeFace_MAX; FaceIndex++)
				{
					FRHICopyTextureInfo& CopyInfo = CopyInfos.AddDefaulted_GetRef();
					CopyInfo.Size.X = CaptureSize.X;
					CopyInfo.Size.Y = CaptureSize.Y;
					CopyInfo.SourcePosition.X = GCubeFaceViewportOffsets[FaceIndex][0] * CaptureSize.X;
					CopyInfo.SourcePosition.Y = GCubeFaceViewportOffsets[FaceIndex][1] * CaptureSize.Y;
					CopyInfo.DestSliceIndex = FaceIndex;
				}
			}

			const bool bClearRenderTarget = (TimeSliceFirst == 0);
			const bool bOrthographicCamera = false;

			FRDGTextureRef OutputTexture;
			if (TextureRenderTarget)
			{
				OutputTexture = RegisterExternalTexture(GraphBuilder, TextureRenderTarget->TextureRHI, TEXT("SceneCaptureTexture"));
			}
			else
			{
				OutputTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::CreateCube(
					CaptureSize.X,
					PF_FloatRGBA,
					FClearValueBinding::None,
					ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV,
					ReflectionSceneData.CubemapArray.GetRenderTarget()->GetDesc().NumMips), TEXT("SceneCaptureTexture"));
			}

			UpdateSceneCaptureContent_RenderThread(
				GraphBuilder,
				Inputs.Renderer,
				Inputs.SceneUpdateInputs,
				CubeFaceTarget.Get(),
				OutputTexture,
				CopyInfos,
				bGenerateMips && bIncludesLastFace,
				GenerateMipsParams,
				bClearRenderTarget,
				bOrthographicCamera);

			if (ReflectionCaptureIndex >= 0)
			{
				// If we failed to properly render shadows due to the timesliced shadow atlas going invalid (typically due to an edit operation
				// causing the light render proxy to be recreated), skip copying it back to the reflection cube map array, to avoid a glitch.
				// jhoerner_todo:  Communicate this back to the game thread to indicate that it should retry the capture immediately, rather than
				// wait on its next timeslice.
				if (bIncludesLastFace && !Inputs.Renderer->Scene->ReflectionSceneData.RuntimeCaptureShadows.TimesliceAtlasFailed)
				{
					// Only generate diffuse irradiance mips if the rendering feature is enabled for the reflection pass.
					if (CVarMobileDiffuseFromCapture.GetValueOnAnyThread())
					{
						if (CVarDiffuseIrradianceReference.GetValueOnRenderThread())
						{
							// Uses exhaustive convolution to 8x8 cube map mips and below
							ComputeDiffuseIrradianceReference(GraphBuilder, Inputs.Renderer->FeatureLevel, Inputs.Renderer->Views[0].ShaderMap, OutputTexture, OutputTexture);
						}
						else
						{
							// Generates Spherical Harmonics from 32x32 mip, then evaluates that for the 8x8 cube map mips and below
							ComputeDiffuseIrradiance(GraphBuilder, Inputs.Renderer->Views[0].ShaderMap, OutputTexture, nullptr);
						}
					}

					FRHICopyTextureInfo ReflectionCopyInfo;
					ReflectionCopyInfo.DestSliceIndex = ReflectionCaptureIndex * 6;
					ReflectionCopyInfo.NumSlices = 6;
					ReflectionCopyInfo.NumMips = ReflectionSceneData.CubemapArray.GetRenderTarget()->GetDesc().NumMips;
					AddCopyTexturePass(GraphBuilder, OutputTexture, GraphBuilder.RegisterExternalTexture(ReflectionSceneData.CubemapArray.GetRenderTarget()), ReflectionCopyInfo);

					// The cubemap slot now holds freshly-rendered content, so the capture may be used for shading.  Re-find the state here
					// rather than reusing the earlier pointer, since the cache may have been mutated (and its TMap storage relocated) during
					// scene rendering above.  Mark that captures have changed, so SortedCaptures get rebuilt.
					if (FCaptureComponentSceneState* StateForShading = ReflectionSceneData.AllocatedReflectionCaptureState.Find(ReflectionComponent))
					{
						if (!StateForShading->bRenderedForShading)
						{
							StateForShading->bRenderedForShading = true;
							ReflectionSceneData.AllocatedReflectionCaptureStateHasChanged = true;
						}
					}
				}
			}
			else
			{
				GraphBuilder.SetTextureAccessFinal(OutputTexture, ERHIAccess::SRVMask);
			}

	#if WITH_EDITOR
			if (Inputs.Renderer->Views[0].ViewState)
			{
				const bool bLogSizes = GDumpSceneCaptureMemoryFrame == GFrameNumberRenderThread;
				if (bLogSizes)
				{
					UE_LOGF(LogRenderer, Log, "LogSizes\tSceneCaptureCube\t%ls\t%dx%d", Inputs.FullPath, CubeFaceTarget->GetSizeXY().X, CubeFaceTarget->GetSizeXY().Y);
					for (int32 FaceIdx = 0; FaceIdx < Inputs.Renderer->Views.Num(); FaceIdx++)
					{
						Inputs.Renderer->Views[FaceIdx].ViewState->GetGPUSizeBytes(bLogSizes);
					}
				}
			}
	#endif  // WITH_EDITOR

			return true;
		});
	}
}