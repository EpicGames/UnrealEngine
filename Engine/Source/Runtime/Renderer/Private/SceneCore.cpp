// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneCore.cpp: Core scene implementation.
=============================================================================*/

#include "SceneCore.h"
#include "SceneInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "Misc/ConfigCacheIni.h"
#include "DepthRendering.h"
#include "SceneHitProxyRendering.h"
#include "StateStream/ExponentialHeightFogStateStream.h"
#include "VelocityRendering.h"
#include "BasePassRendering.h"
#include "MobileBasePassRendering.h"
#include "RendererModule.h"
#include "ScenePrivate.h"
#include "Shadows/CSMSceneExtension.h"
#include "Containers/AllocatorFixedSizeFreeList.h"
#include "MaterialShared.h"
#include "HAL/LowLevelMemTracker.h"

int32 GUnbuiltPreviewShadowsInGame = 1;
FAutoConsoleVariableRef CVarUnbuiltPreviewShadowsInGame(
	TEXT("r.Shadow.UnbuiltPreviewInGame"),
	GUnbuiltPreviewShadowsInGame,
	TEXT("Whether to render unbuilt preview shadows in game.  When enabled and lighting is not built, expensive preview shadows will be rendered in game.  When disabled, lighting in game and editor won't match which can appear to be a bug."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

/**
 * Fixed Size pool allocator for FLightPrimitiveInteractions
 */
#define FREE_LIST_GROW_SIZE ( 16384 / sizeof(FLightPrimitiveInteraction) )
TAllocatorFixedSizeFreeList<sizeof(FLightPrimitiveInteraction), FREE_LIST_GROW_SIZE> GLightPrimitiveInteractionAllocator;


uint32 FRendererModule::GetNumDynamicLightsAffectingPrimitive(const FPrimitiveSceneInfo* PrimitiveSceneInfo,const FLightCacheInterface* LCI)
{
	uint32 NumDynamicLights = 0;

	FLightPrimitiveInteraction *LightList = PrimitiveSceneInfo->LightList;
	while ( LightList )
	{
		const FLightSceneInfo* LightSceneInfo = LightList->GetLight();

		// Determine the interaction type between the mesh and the light.
		FLightInteraction LightInteraction = FLightInteraction::Dynamic();
		if(LCI)
		{
			LightInteraction = LCI->GetInteraction(LightSceneInfo->Proxy);
		}

		// Don't count light-mapped or irrelevant lights.
		if(LightInteraction.GetType() != LIT_CachedIrrelevant && LightInteraction.GetType() != LIT_CachedLightMap)
		{
			++NumDynamicLights;
		}

		LightList = LightList->GetNextLight();
	}

	return NumDynamicLights;
}

/*-----------------------------------------------------------------------------
	FLightPrimitiveInteraction
-----------------------------------------------------------------------------*/

/**
 * Custom new
 */
void* FLightPrimitiveInteraction::operator new(size_t Size)
{
	// doesn't support derived classes with a different size
	checkSlow(Size == sizeof(FLightPrimitiveInteraction));
	return GLightPrimitiveInteractionAllocator.Allocate();
	//return FMemory::Malloc(Size);
}

/**
 * Custom delete
 */
void FLightPrimitiveInteraction::operator delete(void* RawMemory)
{
	GLightPrimitiveInteractionAllocator.Free(RawMemory);
	//FMemory::Free(RawMemory);
}

/**
 * Initialize the memory pool with a default size from the ini file.
 * Called at render thread startup. Since the render thread is potentially
 * created/destroyed multiple times, must make sure we only do it once.
 */
void FLightPrimitiveInteraction::InitializeMemoryPool()
{
	static bool bAlreadyInitialized = false;
	if (!bAlreadyInitialized)
	{
		bAlreadyInitialized = true;
		int32 InitialBlockSize = 0;
		GConfig->GetInt(TEXT("MemoryPools"), TEXT("FLightPrimitiveInteractionInitialBlockSize"), InitialBlockSize, GEngineIni);
		GLightPrimitiveInteractionAllocator.Grow(InitialBlockSize);
	}
}

/**
* Returns current size of memory pool
*/
uint32 FLightPrimitiveInteraction::GetMemoryPoolSize()
{
	return GLightPrimitiveInteractionAllocator.GetAllocatedSize();
}

FLightPrimitiveInteraction::FShouldCreateResult FLightPrimitiveInteraction::ShouldCreate(FLightSceneInfo* LightSceneInfo, FPrimitiveSceneInfo* PrimitiveSceneInfo)
{
	FShouldCreateResult Result;

	// Determine the light's relevance to the primitive.
	check(PrimitiveSceneInfo->Proxy && LightSceneInfo->Proxy);
	PrimitiveSceneInfo->Proxy->GetLightRelevance(LightSceneInfo->Proxy, Result.bDynamic, Result.bRelevant, Result.bIsLightMapped, Result.bShadowMapped);

	// Mobile renders stationary and dynamic local lights as dynamic
	Result.bDynamic |= (PrimitiveSceneInfo->Scene->GetShadingPath() == EShadingPath::Mobile && Result.bShadowMapped && LightSceneInfo->Proxy->IsLocalLight());

	if (Result.bRelevant && Result.bDynamic
		// Don't let lights with static shadowing or static lighting affect primitives that should use static lighting, but don't have valid settings (lightmap res 0, etc)
		// This prevents those components with invalid lightmap settings from causing lighting to remain unbuilt after a build
		&& !(LightSceneInfo->Proxy->HasStaticShadowing() && PrimitiveSceneInfo->Proxy->HasStaticLighting() && !PrimitiveSceneInfo->Proxy->HasValidSettingsForStaticLighting()))
	{
		Result.bTranslucentObjectShadow = LightSceneInfo->Proxy->CastsTranslucentShadows() && PrimitiveSceneInfo->Proxy->CastsVolumetricTranslucentShadow();
		Result.bInsetObjectShadow = 
			// Currently only supporting inset shadows on directional lights, but could be made to work with any whole scene shadows
			LightSceneInfo->Proxy->GetLightType() == LightType_Directional
			&& PrimitiveSceneInfo->Proxy->CastsInsetShadow();

		// Movable directional lights determine shadow relevance dynamically based on the view and CSM settings. Interactions are only required for per-object cases.
		if (LightSceneInfo->Proxy->GetLightType() != LightType_Directional || LightSceneInfo->Proxy->HasStaticShadowing() || Result.bTranslucentObjectShadow || Result.bInsetObjectShadow)
		{
			Result.bShouldCreate = true;
		}
	}

	return Result;
}

void FLightPrimitiveInteraction::Create(FLightSceneInfo* LightSceneInfo,FPrimitiveSceneInfo* PrimitiveSceneInfo)
{
	LLM_SCOPE(ELLMTag::SceneRender);

	FShouldCreateResult Result = ShouldCreate(LightSceneInfo, PrimitiveSceneInfo);
	if (Result.bShouldCreate)
	{
		FLightPrimitiveInteraction* Interaction = new FLightPrimitiveInteraction(LightSceneInfo, PrimitiveSceneInfo, Result.bDynamic, Result.bIsLightMapped, Result.bShadowMapped, Result.bTranslucentObjectShadow, Result.bInsetObjectShadow);
	} //-V773
}

void FLightPrimitiveInteraction::Destroy(FLightPrimitiveInteraction* LightPrimitiveInteraction)
{
	delete LightPrimitiveInteraction;
}

extern bool ShouldCreateObjectShadowForStationaryLight(const FLightSceneInfo* LightSceneInfo, const FPrimitiveSceneProxy* PrimitiveSceneProxy, bool bInteractionShadowMapped);

static bool MobileRequiresStaticMeshUpdateOnLocalLightChange(const FStaticShaderPlatform Platform)
{
	extern bool MobileLocalLightsUseSinglePermutation(EShaderPlatform);
	
	if (!IsMobileDeferredShadingEnabled(Platform))
	{
		return MobileForwardEnableLocalLights(Platform) && !MobileLocalLightsUseSinglePermutation(Platform);
	}
	return false;
}

FLightPrimitiveInteraction::FLightPrimitiveInteraction(
	FLightSceneInfo* InLightSceneInfo,
	FPrimitiveSceneInfo* InPrimitiveSceneInfo,
	bool bInIsDynamic,
	bool bInLightMapped,
	bool bInIsShadowMapped,
	bool bInHasTranslucentObjectShadow,
	bool bInHasInsetObjectShadow
	) :
	LightSceneInfo(InLightSceneInfo),
	PrimitiveSceneInfo(InPrimitiveSceneInfo),
	LightId(InLightSceneInfo->Id),
	bLightMapped(bInLightMapped),
	bIsDynamic(bInIsDynamic),
	bIsShadowMapped(bInIsShadowMapped),
	bUncachedStaticLighting(false),
	bHasTranslucentObjectShadow(bInHasTranslucentObjectShadow),
	bHasInsetObjectShadow(bInHasInsetObjectShadow),
	bSelfShadowOnly(false),
	bMobileDynamicLocalLight(false)
{
	// Determine whether this light-primitive interaction produces a shadow.
	if(PrimitiveSceneInfo->Proxy->HasStaticLighting())
	{
		const bool bHasStaticShadow =
			LightSceneInfo->Proxy->HasStaticShadowing() &&
			LightSceneInfo->Proxy->CastsStaticShadow() &&
			PrimitiveSceneInfo->Proxy->CastsStaticShadow();
		const bool bHasDynamicShadow =
			!LightSceneInfo->Proxy->HasStaticLighting() &&
			LightSceneInfo->Proxy->CastsDynamicShadow() &&
			PrimitiveSceneInfo->Proxy->CastsDynamicShadow();
		bCastShadow = bHasStaticShadow || bHasDynamicShadow;
	}
	else
	{
		bCastShadow = LightSceneInfo->Proxy->CastsDynamicShadow() && PrimitiveSceneInfo->Proxy->CastsDynamicShadow();
	}
	bNaniteMeshProxy = PrimitiveSceneInfo->Proxy->IsNaniteMesh();
	bProxySupportsGPUScene = PrimitiveSceneInfo->Proxy->SupportsGPUScene();

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if(bCastShadow && bIsDynamic)
	{
		// Determine the type of dynamic shadow produced by this light.
		if (PrimitiveSceneInfo->Proxy->HasStaticLighting()
			&& PrimitiveSceneInfo->Proxy->CastsStaticShadow()
			// Don't mark unbuilt for movable primitives which were built with lightmaps but moved into a new light's influence
			&& PrimitiveSceneInfo->Proxy->GetLightmapType() != ELightmapType::ForceSurface
			&& (LightSceneInfo->Proxy->HasStaticLighting() || (LightSceneInfo->Proxy->HasStaticShadowing() && !bInIsShadowMapped)))
		{
			// Update the game thread's counter of number of uncached static lighting interactions.
			bUncachedStaticLighting = true;

			if (!GUnbuiltPreviewShadowsInGame && !InLightSceneInfo->Scene->IsEditorScene())
			{
				bCastShadow = false;
			}
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			LightSceneInfo->NumUnbuiltInteractions++;
	#endif

	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			FPlatformAtomics::InterlockedIncrement(&PrimitiveSceneInfo->Scene->NumUncachedStaticLightingInteractions);
	#endif

#if WITH_EDITOR
			PrimitiveSceneInfo->Proxy->NumUncachedStaticLightingInteractions++;
#endif
		}
	}
#endif

	bSelfShadowOnly = PrimitiveSceneInfo->Proxy->CastsSelfShadowOnly();

	if (bIsDynamic)
	{
		// Add the interaction to the light's interaction list.
		PrevPrimitiveLink = PrimitiveSceneInfo->Proxy->IsMeshShapeOftenMoving() ? &LightSceneInfo->DynamicInteractionOftenMovingPrimitiveList : &LightSceneInfo->DynamicInteractionStaticPrimitiveList;

		// mobile local lights with dynamic lighting
		if (PrimitiveSceneInfo->Scene->GetShadingPath() == EShadingPath::Mobile && LightSceneInfo->ShouldRenderLightViewIndependent())
		{
			const uint8 LightType = LightSceneInfo->Proxy->GetLightType();

			const bool bIsValidLightType = 
				   LightType == LightType_Rect
				|| LightType == LightType_Point
				|| LightType == LightType_Spot;

			if( bIsValidLightType )
			{
				bMobileDynamicLocalLight = true;
				PrimitiveSceneInfo->NumMobileDynamicLocalLights++;
				if (PrimitiveSceneInfo->NumMobileDynamicLocalLights == 1 && 
					MobileRequiresStaticMeshUpdateOnLocalLightChange(PrimitiveSceneInfo->Scene->GetShaderPlatform()))
				{
					// Update static meshes to choose the shader permutation with local lights.
					PrimitiveSceneInfo->RequestStaticMeshUpdate();
				}
			} 

			if (LightSceneInfo->Proxy->CastsModulatedShadows() && !LightSceneInfo->Proxy->UseCSMForDynamicObjects() && LightSceneInfo->Proxy->HasStaticShadowing())
			{
				// Force bCastInsetShadow to be enabled to cast modulated shadow on mobile
				PrimitiveSceneInfo->Proxy->bCastInsetShadow = true;
				bHasInsetObjectShadow = true;
			}
		}
	}

	FlushCachedShadowMapData();

	NextPrimitive = *PrevPrimitiveLink;
	if(*PrevPrimitiveLink)
	{
		(*PrevPrimitiveLink)->PrevPrimitiveLink = &NextPrimitive;
	}
	*PrevPrimitiveLink = this;

	// Add the interaction to the primitives' interaction list.
	PrevLightLink = &PrimitiveSceneInfo->LightList;
	NextLight = *PrevLightLink;
	if(*PrevLightLink)
	{
		(*PrevLightLink)->PrevLightLink = &NextLight;
	}
	*PrevLightLink = this;

	if (HasShadow()
		&& LightSceneInfo->bRecordInteractionShadowPrimitives
		&& (HasTranslucentObjectShadow() || HasInsetObjectShadow() || ShouldCreateObjectShadowForStationaryLight(LightSceneInfo, PrimitiveSceneInfo->Proxy, IsShadowMapped())))
	{
		if (LightSceneInfo->InteractionShadowPrimitives.Num() < 16)
		{
			LightSceneInfo->InteractionShadowPrimitives.Add(this);
		}
		else
		{
			LightSceneInfo->bRecordInteractionShadowPrimitives = false;
			LightSceneInfo->InteractionShadowPrimitives.Empty();
		}
	}
}

FLightPrimitiveInteraction::~FLightPrimitiveInteraction()
{
	check(IsInRenderingThread());

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Update the game thread's counter of number of uncached static lighting interactions.
	if(bUncachedStaticLighting)
	{
		LightSceneInfo->NumUnbuiltInteractions--;
		FPlatformAtomics::InterlockedDecrement(&PrimitiveSceneInfo->Scene->NumUncachedStaticLightingInteractions);
#if WITH_EDITOR
		PrimitiveSceneInfo->Proxy->NumUncachedStaticLightingInteractions--;
#endif
	}
#endif

	FlushCachedShadowMapData();

	// Track mobile movable local light count
	if (bMobileDynamicLocalLight)
	{
		PrimitiveSceneInfo->NumMobileDynamicLocalLights--;
		if (PrimitiveSceneInfo->NumMobileDynamicLocalLights == 0 &&
			MobileRequiresStaticMeshUpdateOnLocalLightChange(PrimitiveSceneInfo->Scene->GetShaderPlatform()))
		{
			// Update static meshes to choose the shader permutation without local lights.
			PrimitiveSceneInfo->RequestStaticMeshUpdate();
		}
	}

	// Remove the interaction from the light's interaction list.
	if(NextPrimitive)
	{
		NextPrimitive->PrevPrimitiveLink = PrevPrimitiveLink;
	}
	*PrevPrimitiveLink = NextPrimitive;

	// Remove the interaction from the primitive's interaction list.
	if(NextLight)
	{
		NextLight->PrevLightLink = PrevLightLink;
	}
	*PrevLightLink = NextLight;

	LightSceneInfo->InteractionShadowPrimitives.RemoveSingleSwap(this);
}

void FLightPrimitiveInteraction::FlushCachedShadowMapData()
{
	if (LightSceneInfo && PrimitiveSceneInfo && PrimitiveSceneInfo->Proxy && PrimitiveSceneInfo->Scene)
	{
		if (bCastShadow && !PrimitiveSceneInfo->Proxy->IsMeshShapeOftenMoving())
		{
			TArray<FCachedShadowMapData>* CachedShadowMapDatas = PrimitiveSceneInfo->Scene->GetExtension<FCSMSceneExtension>().GetCachedShadowMapDatas(LightSceneInfo->Id);

			if (CachedShadowMapDatas)
			{
				for (auto& CachedShadowMapData : *CachedShadowMapDatas)
				{
					CachedShadowMapData.InvalidateCachedShadow();
				}
			}
		}
	}
}

/** Initialization constructor. */
FExponentialHeightFogSceneInfo::FExponentialHeightFogSceneInfo(uint64 InId, const FExponentialHeightFogDynamicState& State) :
	Id(InId),
	FogMaxOpacity(State.FogMaxOpacity),
	StartDistance(State.StartDistance),
	EndDistance(State.EndDistance),
	FogCutoffDistance(State.FogCutoffDistance),
	DirectionalInscatteringExponent(State.DirectionalInscatteringExponent),
	DirectionalInscatteringStartDistance(State.DirectionalInscatteringStartDistance),
	DirectionalInscatteringColor(State.DirectionalInscatteringLuminance)
{
	FogData[0].Height = State.Height;
	FogData[1].Height = State.Height + State.SecondFogData.FogHeightOffset;

	// Scale the densities back down to their real scale
	// Artists edit the densities scaled up so they aren't entering in minuscule floating point numbers
	FogData[0].Density = State.FogDensity / 1000.0f;	
	FogData[0].HeightFalloff = State.FogHeightFalloff / 1000.0f;
	FogData[1].Density = State.SecondFogData.FogDensity / 1000.0f;
	FogData[1].HeightFalloff = State.SecondFogData.FogHeightFalloff / 1000.0f;

	FogColor = State.InscatteringColorCubemap ? State.InscatteringTextureTint : State.FogInscatteringLuminance;
	InscatteringColorCubemap = State.InscatteringColorCubemap;
	SkyLightCaptureAffectsHeightFogRoughness = State.SkyLightCaptureAffectsHeightFogRoughness;
	SkyLightCaptureAffectsHeightFogStrength = State.SkyLightCaptureAffectsHeightFogStrength;
	InscatteringColorCubemapAngle = State.InscatteringColorCubemapAngle * (PI / 180.f);
	FullyDirectionalInscatteringColorDistance = State.FullyDirectionalInscatteringColorDistance;
	NonDirectionalInscatteringColorDistance = State.NonDirectionalInscatteringColorDistance;

	bEnableVolumetricFog = State.bEnableVolumetricFog;
	VolumetricFogScatteringDistribution = FMath::Clamp(State.VolumetricFogScatteringDistribution, -.99f, .99f);
	VolumetricFogAlbedo = FLinearColor(State.VolumetricFogAlbedo);
	VolumetricFogEmissive = State.VolumetricFogEmissive;

	// Apply a scale so artists don't have to work with tiny numbers.
	// The is only needed because emissive is by default not weighted by the height fog density distribution. 
	// When we run "HeightFog matches FVog" that is no longer needed.
	const float EmissiveUnitScale = DoesProjectSupportExpFogMatchesVolumetricFog() ? 1.0f : 1.0f / 10000.0f;
	VolumetricFogEmissive.R = FMath::Max(VolumetricFogEmissive.R * EmissiveUnitScale, 0.0f);
	VolumetricFogEmissive.G = FMath::Max(VolumetricFogEmissive.G * EmissiveUnitScale, 0.0f);
	VolumetricFogEmissive.B = FMath::Max(VolumetricFogEmissive.B * EmissiveUnitScale, 0.0f);
	VolumetricFogExtinctionScale = FMath::Max(State.VolumetricFogExtinctionScale, 0.0f);
	VolumetricFogDistance = FMath::Max(State.VolumetricFogStartDistance + State.VolumetricFogDistance, 0.0f);
	VolumetricFogStaticLightingScatteringIntensity = FMath::Max(State.VolumetricFogStaticLightingScatteringIntensity, 0.0f);
	bOverrideLightColorsWithFogInscatteringColors = State.bOverrideLightColorsWithFogInscatteringColors;
	bHoldout = State.bHoldout;
	bRenderInMainPass = State.bRenderInMainPass;
	bVisibleInReflectionCaptures = State.bVisibleInReflectionCaptures;
	bVisibleInRealTimeSkyCaptures = State.bVisibleInRealTimeSkyCaptures;

	VolumetricFogStartDistance = State.VolumetricFogStartDistance;
	VolumetricFogNearFadeInDistance = State.VolumetricFogNearFadeInDistance;

	bEnableFSSS = State.bEnableFSSS;
	FSSSSceneColorScatteringAmountScale = State.FSSSSceneColorScatteringAmountScale;
	FSSSSceneColorScatteringAmountPower = State.FSSSSceneColorScatteringAmountPower;
	FSSSSpreadScale = State.FSSSSpreadScale;
	FSSSBlurControl = State.FSSSBlurControl;

	SkyAtmosphereAmbientContributionColorScale = State.SkyAtmosphereAmbientContributionColorScale;
}
