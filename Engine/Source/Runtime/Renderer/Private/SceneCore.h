// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneCore.h: Core scene definitions.
=============================================================================*/

#pragma once

#include "HAL/Platform.h"
#include "Math/Color.h"
#include "RHIFwd.h"
#include "ScenePrivateBase.h"

class FLightSceneInfo;
class FPrimitiveSceneInfo;
class FScene;
class UTextureCube;
struct FExponentialHeightFogDynamicState;

/**
 * An interaction between a light and a primitive.
 */
class FLightPrimitiveInteraction
{
public:

	/** Creates an interaction for a light-primitive pair. */
	static void InitializeMemoryPool();

	struct FShouldCreateResult
	{
		bool bShouldCreate = false;
		bool bDynamic = true;
		bool bRelevant = false;
		bool bIsLightMapped = true;
		bool bShadowMapped = false;
		bool bTranslucentObjectShadow = false;
		bool bInsetObjectShadow = false;
	};

	static FShouldCreateResult ShouldCreate(FLightSceneInfo* LightSceneInfo,FPrimitiveSceneInfo* PrimitiveSceneInfo);
	static void Create(FLightSceneInfo* LightSceneInfo,FPrimitiveSceneInfo* PrimitiveSceneInfo);
	static void Destroy(FLightPrimitiveInteraction* LightPrimitiveInteraction);

	/** Returns current size of memory pool */
	static uint32 GetMemoryPoolSize();

	// Accessors.
	bool HasShadow() const { return bCastShadow; }
	bool IsLightMapped() const { return bLightMapped; }
	bool IsDynamic() const { return bIsDynamic; }
	bool IsShadowMapped() const { return bIsShadowMapped; }
	bool IsUncachedStaticLighting() const { return bUncachedStaticLighting; }
	bool HasTranslucentObjectShadow() const { return bHasTranslucentObjectShadow; }
	bool HasInsetObjectShadow() const { return bHasInsetObjectShadow; }
	bool CastsSelfShadowOnly() const { return bSelfShadowOnly; }
	bool IsMobileDynamicLocalLight() const { return bMobileDynamicLocalLight; }

	FORCEINLINE bool IsNaniteMeshProxy() const { return bNaniteMeshProxy; }
	FORCEINLINE bool ProxySupportsGPUScene() const { return bProxySupportsGPUScene; }


	FLightSceneInfo* GetLight() const { return LightSceneInfo; }
	FLightSceneId GetLightId() const { return LightId; }
	FPrimitiveSceneInfo* GetPrimitiveSceneInfo() const { return PrimitiveSceneInfo; }
	FLightPrimitiveInteraction* GetNextPrimitive() const { return NextPrimitive; }
	FLightPrimitiveInteraction* GetNextLight() const { return NextLight; }

	/** Hash function required for TMap support */
	friend uint32 GetTypeHash( const FLightPrimitiveInteraction* Interaction )
	{
		return GetTypeHash(Interaction->LightId);
	}

	/** Clears cached shadow maps, if possible */
	void FlushCachedShadowMapData();

	/** Custom new/delete */
	void* operator new(size_t Size);
	void operator delete(void* RawMemory);

private:
	/** The light which affects the primitive. */
	FLightSceneInfo* LightSceneInfo;

	/** The primitive which is affected by the light. */
	FPrimitiveSceneInfo* PrimitiveSceneInfo;

	/** A pointer to the NextPrimitive member of the previous interaction in the light's interaction list. */
	FLightPrimitiveInteraction** PrevPrimitiveLink;

	/** The next interaction in the light's interaction list. */
	FLightPrimitiveInteraction* NextPrimitive;

	/** A pointer to the NextLight member of the previous interaction in the primitive's interaction list. */
	FLightPrimitiveInteraction** PrevLightLink;

	/** The next interaction in the primitive's interaction list. */
	FLightPrimitiveInteraction* NextLight;

	/** The index into Scene->Lights of the light which affects the primitive. */
	FLightSceneId LightId;

	/** True if the primitive casts a shadow from the light. */
	uint32 bCastShadow : 1;

	/** True if the primitive has a light-map containing the light. */
	uint32 bLightMapped : 1;

	/** True if the interaction is dynamic. */
	uint32 bIsDynamic : 1;

	/** Whether the light's shadowing is contained in the primitive's static shadow map. */
	uint32 bIsShadowMapped : 1;

	/** True if the interaction is an uncached static lighting interaction. */
	uint32 bUncachedStaticLighting : 1;

	/** True if the interaction has a translucent per-object shadow. */
	uint32 bHasTranslucentObjectShadow : 1;

	/** True if the interaction has an inset per-object shadow. */
	uint32 bHasInsetObjectShadow : 1;

	/** True if the primitive only shadows itself. */
	uint32 bSelfShadowOnly : 1;

	/** True this is a mobile dynamic local light interaction. */
	uint32 bMobileDynamicLocalLight : 1;

	/** If true then all meshes drawn by the primitive scene proxy are Nanite meshes. Caches the result of FPrimitiveSceneProxy::IsNaniteMesh() */
	uint32 bNaniteMeshProxy : 1;

	/** If true then all meshes drawn by the primitive scene proxy supports GPU-Scene (and thus VSM shadows). */
	uint32 bProxySupportsGPUScene : 1;

	/** Initialization constructor. */
	FLightPrimitiveInteraction(FLightSceneInfo* InLightSceneInfo,FPrimitiveSceneInfo* InPrimitiveSceneInfo,
		bool bIsDynamic,bool bInLightMapped,bool bInIsShadowMapped, bool bInHasTranslucentObjectShadow, bool bInHasInsetObjectShadow);

	/** Hide dtor */
	~FLightPrimitiveInteraction();

};

/** The properties of a exponential height fog layer which are used for rendering. */
class FExponentialHeightFogSceneInfo
{
public:

	struct FExponentialHeightFogSceneData
	{
		float Height;
		float Density;
		float HeightFalloff;
	};

	/** Number of supported individual fog settings on this ExponentialHeightFog */
	static constexpr int NumFogs = 2;

	uint64 Id;
	FExponentialHeightFogSceneData FogData[NumFogs];
	float FogMaxOpacity;
	float StartDistance;
	float EndDistance;
	float FogCutoffDistance;
	FLinearColor FogColor;
	float DirectionalInscatteringExponent; 
	float DirectionalInscatteringStartDistance;
	FLinearColor DirectionalInscatteringColor;
	UTextureCube* InscatteringColorCubemap;
	float SkyLightCaptureAffectsHeightFogRoughness;
	float SkyLightCaptureAffectsHeightFogStrength;
	float InscatteringColorCubemapAngle;
	float FullyDirectionalInscatteringColorDistance;
	float NonDirectionalInscatteringColorDistance;

	bool bEnableVolumetricFog;
	float VolumetricFogScatteringDistribution;
	FLinearColor VolumetricFogAlbedo;
	FLinearColor VolumetricFogEmissive;
	float VolumetricFogExtinctionScale;
	float VolumetricFogDistance;
	float VolumetricFogStaticLightingScatteringIntensity;
	bool bOverrideLightColorsWithFogInscatteringColors;
	bool bHoldout;
	bool bRenderInMainPass;
	bool bVisibleInReflectionCaptures;
	bool bVisibleInRealTimeSkyCaptures;
	float VolumetricFogStartDistance;
	float VolumetricFogNearFadeInDistance;

	bool bEnableFSSS;
	float FSSSSceneColorScatteringAmountScale;
	float FSSSSceneColorScatteringAmountPower;
	float FSSSSpreadScale;
	float FSSSBlurControl;

	FLinearColor SkyAtmosphereAmbientContributionColorScale;

	/** Initialization constructor. */
	FExponentialHeightFogSceneInfo(uint64 InId, const FExponentialHeightFogDynamicState& State);
};

/** Returns true if the indirect lighting cache can be used at all. */
extern bool IsIndirectLightingCacheAllowed(ERHIFeatureLevel::Type InFeatureLevel);

/** Returns true if the indirect lighting cache can use the volume texture atlas on this feature level. */
extern bool CanIndirectLightingCacheUseVolumeTexture(ERHIFeatureLevel::Type InFeatureLevel);
