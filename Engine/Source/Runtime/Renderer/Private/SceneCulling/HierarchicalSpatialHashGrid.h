// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "SceneCullingDefinitions.h"
#include "Rendering/RenderingSpatialHash.h"
#include "SpanAllocator.h"
#include "Tasks/Task.h"

class FScenePreUpdateChangeSet;
class FScenePostUpdateChangeSet;

// Test implemented for perf comparison, may not scale exactly as size grows & not fully featured.
#define USE_STATIC_HASH_TABLE 0

#if USE_STATIC_HASH_TABLE 
template <typename InKeyType, typename InValueType>
class TSpatialHashMap
{
public:
	using KeyType = InKeyType;
	using ValueType = InValueType;
	struct FElement
	{
		KeyType Key;
		ValueType Value;
	};

	struct FElementId
	{
		inline FElementId(int32 InIndex = INDEX_NONE) 
		: Index(InIndex) 
		{
		}
		int32 Index = INDEX_NONE;

		int32 GetIndex() const { return Index; }
		bool IsValid() const { return Index != INDEX_NONE; }
	};

	using Hasher = TDefaultMapHashableKeyFuncs<KeyType, ValueType, false>;

	void Empty()
	{
		Elements.Reset();
		HashTable.Clear();
	}

	void Reserve(int32 Num)
	{
		Elements.Reserve(Num);
	}

	inline FElementId FindIdByHash(uint16 Hash, const KeyType& Location) const
	{
		for (uint16 Index = HashTable.First(Hash); HashTable.IsValid(Index); Index = HashTable.Next(Index))
		{
			if (Hasher::Matches(Elements[Index].Key, Location))
			{
				return FElementId{ int32(Index) };
			}
		}
		return FElementId{};
	}

	inline FElementId FindId(const KeyType& Location) const
	{
		uint16 Hash = uint16(Hasher::GetKeyHash(Location));
		return FindIdByHash(Hash, Location);
	}

	inline FElementId FindOrAddId(const KeyType& Location, const ValueType& Value)
	{
		uint16 Hash = uint16(Hasher::GetKeyHash(Location));
		FElementId Id = FindIdByHash(Hash, Location);
		if (!Id.IsValid())
		{
			int32 NewIndex = Elements.Num();
			Elements.Add(FElement{ Location, Value });
			HashTable.Add(Hash, NewIndex);
			Id = FElementId{ NewIndex };
		}
		return Id;
	}

	inline int32 GetMaxIndex() const { return Elements.Num(); }
	inline int32 Num() const { return Elements.Num(); }

	inline FElement& GetByElementId(FElementId Id) { return Elements[Id.Index]; }
	inline const FElement& GetByElementId(FElementId Id) const { return Elements[Id.Index]; }

	class FConstIteratorType
	{
		friend TSpatialHashMap<KeyType, ValueType>;

		const TSpatialHashMap<KeyType, ValueType>& TheMap;

		int32 Index;

		inline FConstIteratorType(const TSpatialHashMap<KeyType, ValueType>& InTheMap, bool bIsStartIterator) : TheMap(InTheMap)
		{
			if (bIsStartIterator)
			{
				Index = 0;
			}
			else
			{
				Index = TheMap.Elements.Num();
			}
		}

	public:
		inline bool operator ==(const FConstIteratorType& Rhs) const
		{
			return Index == Rhs.Index;
		}

		inline bool operator !=(const FConstIteratorType& Rhs) const
		{
			return Index != Rhs.Index;
		}

		inline const FElement& operator*() const
		{
			return TheMap.Elements[Index];
		}

		inline FConstIteratorType& operator++()
		{
			++Index;
			return *this;
		}

		inline FElementId GetElementId() const
		{
			return FElementId{ Index };
		}
	};

	inline FConstIteratorType begin() const
	{
		return FConstIteratorType(*this, true);
	}

	inline FConstIteratorType end() const
	{
		return FConstIteratorType(*this, false);
	}

	TArray<FElement> Elements;
	TStaticHashTable<32u * 1024u, 32u * 1024u> HashTable;
};

#endif



inline FInt64Vector FloorToInt64Vector(const FVector& Vector)
{
	return FInt64Vector(FMath::FloorToInt(Vector.X), FMath::FloorToInt(Vector.Y), FMath::FloorToInt(Vector.Z));
};

template <typename BlockTraitsType>
class THierarchicalSpatialHashGrid
{
public:
	using FBlockLoc = typename BlockTraitsType::FBlockLoc;

	static constexpr int32 CellBlockDimLog2 = BlockTraitsType::CellBlockDimLog2;
	static constexpr int32 kMaxLevel = 64;
	static constexpr int32 NumCellsPerBlockLog2 = CellBlockDimLog2 * 3; // number of bits needed for a local cell ID
	static constexpr int32 CellBlockDim = 1 << CellBlockDimLog2;
	static constexpr uint32 LocalCellCoordMask = uint32(CellBlockDim) - 1U;
	static constexpr int32 CellBlockSize = CellBlockDim * CellBlockDim * CellBlockDim;

	using FLocation64 = RenderingSpatialHash::TLocation<int64>;
	using FLocation32 = RenderingSpatialHash::TLocation<int32>;
	using FLocation8 = RenderingSpatialHash::TLocation<int8>;
	using FInt8Vector3 = UE::Math::TIntVector3<int8>;

	template <typename ScalarType>
	struct TFootprint
	{
		using FIntVector3 = UE::Math::TIntVector3<ScalarType>;

		template <typename LambdaType>
		inline void ForEach(LambdaType Lambda)
		{
			for (ScalarType Z = Min.Z; Z <= Max.Z; ++Z)
			{
				for (ScalarType Y = Min.Y; Y <= Max.Y; ++Y)
				{
					for (ScalarType X = Min.X; X <= Max.X; ++X)
					{
						RenderingSpatialHash::TLocation<ScalarType> L;
						L.Coord = typename RenderingSpatialHash::TLocation<ScalarType>::FIntVector3{ X, Y, Z };
						L.Level = Level;
						Lambda(L);
					}
				}
			}
		}

		FIntVector3 Min;
		FIntVector3 Max;
		int32 Level;
	};

	using FFootprint64 = TFootprint<int64>;
	using FFootprint32 = TFootprint<int32>;
	using FFootprint8 = TFootprint<int8>;

	struct FCellBlock
	{
		static uint32 constexpr CoarseCellMaskDimLog2 = 2U; // 2^2 == 4 -> 4x4x4 == 64
		static uint32 constexpr CoarseCellMaskDim = 1U << CoarseCellMaskDimLog2; // 2^2 == 4 -> 4x4x4 == 64
		static uint32 constexpr CoarseCellMaskCoordMask = (1U << CoarseCellMaskDimLog2) - 1u; // 2^2 == 4 -> 4x4x4 == 64

		inline static int32 CalcCellOffset(const FInt8Vector3& Loc)
		{
			return ((int32(Loc.Z) * CellBlockDim) + int32(Loc.Y)) * CellBlockDim + int32(Loc.X);
		};

		inline int32 GetCellGridOffset(const FInt8Vector3& Loc) const
		{
			return GridOffset + CalcCellOffset(Loc);
		}
		inline static uint32 CalcCellMaskOffset(const FInt8Vector3& MaskLoc)
		{
			return (((uint32(MaskLoc.Z) << CoarseCellMaskDimLog2) + uint32(MaskLoc.Y)) << CoarseCellMaskDimLog2) + uint32(MaskLoc.X);
		};

		inline static uint64 CalcCellMask(const FInt8Vector3& Loc)
		{
			FInt8Vector3 MaskLoc = Loc >> (CellBlockDimLog2 - CoarseCellMaskDimLog2);
			return 1ULL << CalcCellMaskOffset(MaskLoc);
		};

		inline static uint64 BuildFootPrintMask(const FFootprint8& FootprintInBlock)
		{
			FFootprint8 FootprintInCoarse = FootprintInBlock;
			FootprintInCoarse.Min = FootprintInCoarse.Min >> (CellBlockDimLog2 - CoarseCellMaskDimLog2);
			FootprintInCoarse.Max = FootprintInCoarse.Max >> (CellBlockDimLog2 - CoarseCellMaskDimLog2);

			uint64 Result = 0ULL;
			// TODO: make this not stupid... figure out bit intervals etc, do incrementally.
			FootprintInCoarse.ForEach([&](const FLocation8& Loc)
			{
				Result |= (1ULL << CalcCellMaskOffset(Loc.Coord));
			});

			return Result;
		}

		uint64 CoarseCellMask = 0ULL;
		int32 GridOffset = INDEX_NONE;
		int32 NumItemChunks = 0;
	};


	THierarchicalSpatialHashGrid() = default;

	THierarchicalSpatialHashGrid(double MinCellSize, double MaxCellSize)
	{
		// Subtract 1 to make sure that intuitively, when we set a cell size of exact POT e.g., 4096, that is what we get (not the correctly conservative 8192).
		FirstLevel = CalcLevel(MinCellSize - 1.0);
		LastLevel = CalcLevel(MaxCellSize - 1.0);
		check(GetCellSize(FirstLevel) >= MinCellSize && GetCellSize(FirstLevel - 1) < MinCellSize);
		check(GetCellSize(LastLevel) >= MaxCellSize && GetCellSize(LastLevel - 1) < MaxCellSize);

		LastLevelCellSize = GetCellSize(LastLevel);

		// TODO: figure out number of bits in coordinate type minus those in the smallest level,
		MaxCullingDistance = WORLD_MAX;

		for (int32 Level = 0; Level < kMaxLevel; ++Level)
		{
			RecCellSizes[Level] = 1.0 / GetCellSize(Level);
		}
	}

	void Empty()
	{
		HashMap.Empty();
	}

	inline int32 GetMaxNumBlocks() const { return HashMap.GetMaxIndex(); }

	static inline int32 CalcLevel(float Size)
	{
		return RenderingSpatialHash::CalcLevel(Size);
	};

	static inline int32 CalcLevelFromRadius(float Radius)
	{
		return CalcLevel(Radius * 2.0f);
	};

	static inline double GetCellSize(int32 Level)
	{
		return RenderingSpatialHash::GetCellSize(Level);
	};

	inline double GetRecCellSize(int32 Level) const
	{
		return RecCellSizes[Level];
	}

	inline FLocation64 ToCellLoc(int32 Level, const FVector& WorldPos) const
	{
		FLocation64 Result;
		double RecLevelCellSize = GetRecCellSize(Level);
		Result.Level = Level;
		Result.Coord = FloorToInt64Vector(WorldPos * RecLevelCellSize);
		return Result;
	};

	inline FFootprint64 CalcFootprintSphere(int32 Level, const FVector& Origin, double Radius) const
	{
		FFootprint64 Footprint;
		double RecLevelCellSize = GetRecCellSize(Level);
		Footprint.Level = Level;
		Footprint.Min = FloorToInt64Vector((Origin - FVector(Radius, Radius, Radius)) * RecLevelCellSize);
		Footprint.Max = FloorToInt64Vector((Origin + FVector(Radius, Radius, Radius)) * RecLevelCellSize);
		return Footprint;
	};

	static inline FFootprint64 CalcCellBlockFootprint(const FFootprint64& Footprint)
	{
		FFootprint64 BlockFootprint;
		BlockFootprint.Min = Footprint.Min >> CellBlockDimLog2;
		BlockFootprint.Max = Footprint.Max >> CellBlockDimLog2;
		BlockFootprint.Level = Footprint.Level + CellBlockDimLog2;
		return BlockFootprint;
	};

	inline FFootprint64 CalcLevelAndFootprint(const FBoxSphereBounds& BoxSphereBounds) const
	{
		// Can't be lower than this, or the footprint might be larger than 2x2x2, globally the same, can pre-calc.
		int32 Level = CalcLevelFromRadius(BoxSphereBounds.SphereRadius);

		// Clamp to lowest level represented in the grid
		Level = FMath::Max(FirstLevel, Level);

		double RecLevelCellSize = GetRecCellSize(Level);

		FFootprint64 Footprint;
		Footprint.Level = Level;
		Footprint.Min = FloorToInt64Vector((BoxSphereBounds.Origin - BoxSphereBounds.BoxExtent) * RecLevelCellSize);
		Footprint.Max = FloorToInt64Vector((BoxSphereBounds.Origin + BoxSphereBounds.BoxExtent) * RecLevelCellSize);
		return Footprint;
	};

	inline FLocation64 CalcLevelAndLocation(const FBoxSphereBounds& BoxSphereBounds) const
	{
		int32 Level = CalcLevelFromRadius(BoxSphereBounds.SphereRadius);

		// Clamp to lowest level represented in the grid
		Level = FMath::Max(FirstLevel, Level);

		return ToCellLoc(Level, BoxSphereBounds.Origin);
	};

	inline FLocation64 CalcLevelAndLocation(const FVector4d& Sphere) const
	{
		int32 Level = CalcLevelFromRadius(Sphere.W);

		// Clamp to lowest level represented in the grid
		Level = FMath::Max(FirstLevel, Level);

		return ToCellLoc(Level, FVector(Sphere));
	};

	inline bool IsValidLevel(int32 Level) const
	{
		return Level >= FirstLevel && Level <= LastLevel;
	}

	inline double GetLastLevelCellSize() const
	{
		return LastLevelCellSize;
	}

	inline double GetMaxCullingDistance() const
	{
		return MaxCullingDistance;
	}
	inline int32 NumLevels() const { return LastLevel - FirstLevel + 1; }

	// Padded by half a cell size because of loose bounds
	static FBox CalcCellBounds(const FLocation64& CellLoc)
	{
		FBox Box;
		double LevelCellSize = GetCellSize(CellLoc.Level);
		Box.Min = FVector(CellLoc.Coord) * LevelCellSize;
		Box.Max = Box.Min + LevelCellSize;

		// Extend extent by half a cell size in all directions
		Box.Min -= FVector(LevelCellSize * 0.5);
		Box.Max += FVector(LevelCellSize * 0.5);

		return Box;
	}

	static FVector CalcCellCenter(const FLocation64& CellLoc)
	{
		return RenderingSpatialHash::CalcWorldCellCenter(CellLoc);
	}

	// Padded by half a cell size since loose (NOT 1/2 block size!)
	inline FBox CalcBlockBounds(const FLocation64& BlockLoc) const
	{
		FBox Box;
		double BlockLevelSize = GetCellSize(BlockLoc.Level);
		Box.Min = FVector(BlockLoc.Coord) * BlockLevelSize;
		Box.Max = Box.Min + BlockLevelSize;

		// Extend extent by half a cell size in all directions
		double LevelCellSize = GetCellSize(BlockLoc.Level - CellBlockDimLog2);
		Box.Min -= FVector(LevelCellSize * 0.5);
		Box.Max += FVector(LevelCellSize * 0.5);

		return Box;
	}

	/**
	 * The reference location is always the minimum corner, unpadded by loose bounds expansion.
	 */
	inline FVector3d CalcBlockWorldPosition(const FLocation64& BlockLoc) const
	{
		return RenderingSpatialHash::CalcWorldPosition(BlockLoc);
	}

	/**
	 * The reference location is always the minimum corner, unpadded by loose bounds expansion.
	 */
	inline FVector3d CalcBlockWorldPosition(const FBlockLoc& BlockLoc) const
	{
		return BlockLoc.GetWorldPosition();
	}

	int32 GetFirstLevel() const { return FirstLevel; }

#if USE_STATIC_HASH_TABLE
	using FSpatialHashMap = ::TSpatialHashMap<FBlockLoc, FCellBlock>;
	using FHashElementId = typename FSpatialHashMap::FElementId;
	using FBlockId = typename FSpatialHashMap::FElementId;
#else

	struct FHasher : public TDefaultMapKeyFuncs<FBlockLoc, FCellBlock, false>
	{
		static FORCEINLINE uint32 GetKeyHash(const FBlockLoc& Key)
		{
			return Key.GetHash();
		}
	};

	using FSpatialHashMap = Experimental::TRobinHoodHashMap<FBlockLoc, FCellBlock, FHasher>;
	using FBlockId = Experimental::FHashElementId;
	using FHashElementId = Experimental::FHashElementId;

#endif


	FCellBlock &GetBlockById(const FBlockId &BlockId) { return HashMap.GetByElementId(BlockId).Value; }
	const FCellBlock &GetBlockById(const FBlockId &BlockId) const { return HashMap.GetByElementId(BlockId).Value; }
	
	FBlockLoc GetBlockLocById(const FBlockId &BlockId) const { return HashMap.GetByElementId(BlockId).Key; }

	const FSpatialHashMap &GetHashMap() const { return HashMap; };
	FSpatialHashMap &GetHashMap() { return HashMap; };

private:
	int32 FirstLevel;
	int32 LastLevel;
	double RecCellSizes[kMaxLevel];
	double LastLevelCellSize;
	double MaxCullingDistance;
	FSpatialHashMap HashMap;
};

