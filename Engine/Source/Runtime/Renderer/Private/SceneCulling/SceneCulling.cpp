// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneCulling.h"
#include "SceneCulling.inl"
#include "SceneCullingRenderer.h"
#include "ScenePrivate.h"
#include "ComponentRecreateRenderStateContext.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "InstanceDataSceneProxy.h"

#if !UE_BUILD_SHIPPING
#include "RenderCaptureInterface.h"
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "LogRenderer.h"
#include "DynamicPrimitiveDrawing.h"

  // Keep these in the ifdef to make it easier to iterate 
  // UE_DISABLE_OPTIMIZATION
  // #define SC_FORCEINLINE FORCENOINLINE
  #define SC_FORCEINLINE FORCEINLINE
#else

#define SC_FORCEINLINE FORCEINLINE

#endif

static_assert(sizeof(FPackedChunkAttributes) == SIZEOF_PACKED_CHUNK_ATTRIBUTES, "FPackedChunkAttributes has changed without update to SIZEOF_PACKED_CHUNK_ATTRIBUTES.");

#define OLA_TODO 0

#define SC_ENABLE_DETAILED_LOGGING 0 //(UE_BUILD_DEBUG)
#define SC_ENABLE_GPU_DATA_VALIDATION (DO_CHECK)
#define SC_ALLOW_ASYNC_TASKS 1

#if 0
#define SC_SCOPED_NAMED_EVENT_DETAIL SCOPED_NAMED_EVENT
#define SC_SCOPED_NAMED_EVENT_DETAIL_TCHAR SCOPED_NAMED_EVENT_TCHAR
#else
#define SC_SCOPED_NAMED_EVENT_DETAIL(...)
#define SC_SCOPED_NAMED_EVENT_DETAIL_TCHAR(...)
#endif

// TODO: this might be adding too much overhead in Development as well...
#define SC_ENABLE_DETAILED_BUILDER_STATS (!(UE_BUILD_SHIPPING || UE_BUILD_TEST))

DECLARE_STATS_GROUP(TEXT("SceneCulling"), STATGROUP_SceneCulling, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("Test"), STAT_SceneCulling_Test, STATGROUP_SceneCulling);
DECLARE_CYCLE_STAT(TEXT("Test Sphere"), STAT_SceneCulling_Test_Sphere, STATGROUP_SceneCulling);
DECLARE_CYCLE_STAT(TEXT("Test Convex"), STAT_SceneCulling_Test_Convex, STATGROUP_SceneCulling);

DECLARE_DWORD_COUNTER_STAT(TEXT("Test Sphere Blocks"), STAT_SceneCulling_TestSphereBlocks, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Test Sphere Cells"), STAT_SceneCulling_TestSphereCells, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Test Sphere Bounds"), STAT_SceneCulling_TestSphereBounds, STATGROUP_SceneCulling);

DECLARE_CYCLE_STAT(TEXT("Update Pre"), STAT_SceneCulling_Update_Pre, STATGROUP_SceneCulling);
DECLARE_CYCLE_STAT(TEXT("Update Post"), STAT_SceneCulling_Update_Post, STATGROUP_SceneCulling);
DECLARE_CYCLE_STAT(TEXT("Update Finalize"), STAT_SceneCulling_Update_FinalizeAndClear, STATGROUP_SceneCulling);

DECLARE_DWORD_COUNTER_STAT(TEXT("Removed Instances"),  STAT_SceneCulling_RemovedInstanceCount, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Updated Instances"),  STAT_SceneCulling_UpdatedInstanceCount, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Added Instances"),  STAT_SceneCulling_AddedInstanceCount, STATGROUP_SceneCulling);

DECLARE_DWORD_COUNTER_STAT(TEXT("Update Uploaded Chunks"), STAT_SceneCulling_UploadedChunks, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Update Uploaded Cells"), STAT_SceneCulling_UploadedCells, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Update Uploaded Items"), STAT_SceneCulling_UploadedItems, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Update Uploaded Blocks"), STAT_SceneCulling_UploadedBlocks, STATGROUP_SceneCulling);

DECLARE_DWORD_COUNTER_STAT(TEXT("Block Count"), STAT_SceneCulling_BlockCount, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Cell Count"), STAT_SceneCulling_CellCount, STATGROUP_SceneCulling);
// Number of chunk IDs that reference explicit chunks OR represents RLE an compressed range
DECLARE_DWORD_COUNTER_STAT(TEXT("Chunk ID Count"), STAT_SceneCulling_ItemChunkCount, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Explicit Chunk Count"), STAT_SceneCulling_ExplicitChunkCount, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Compressed Chunk Count"), STAT_SceneCulling_CompressedChunkCount, STATGROUP_SceneCulling);
// Size of the item buffer (where explicit chunk instance IDs are stored), not all are used.
DECLARE_DWORD_COUNTER_STAT(TEXT("Item Buffer Size"), STAT_SceneCulling_ItemBufferSize, STATGROUP_SceneCulling);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Chunk Occupancy"), STAT_SceneCulling_ChunkOccupancy, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Total Id Cache Size"), STAT_SceneCulling_IdCacheSize, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Scene Instance Count"), STAT_SceneCulling_SceneInstanceCount, STATGROUP_SceneCulling);

#if SC_ENABLE_DETAILED_BUILDER_STATS

DECLARE_DWORD_COUNTER_STAT(TEXT("Num Static Instances"), STAT_SceneCulling_NumStaticInstances, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Dynamic Instances"), STAT_SceneCulling_NumDynamicInstances, STATGROUP_SceneCulling);

// Detailed stat?
DECLARE_DWORD_COUNTER_STAT(TEXT("Non-Empty Cell Count"), STAT_SceneCulling_NonEmptyCellCount, STATGROUP_SceneCulling);

DECLARE_DWORD_COUNTER_STAT(TEXT("Ranges Added"), STAT_SceneCulling_RangeCount, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Num Comp. Ranges"), STAT_SceneCulling_CompRangeCount, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Update Visited Id Count"), STAT_SceneCulling_VisitedIdCount, STATGROUP_SceneCulling);
DECLARE_DWORD_COUNTER_STAT(TEXT("Update Copied Id Count"), STAT_SceneCulling_CopiedIdCount, STATGROUP_SceneCulling);

#endif

#if SC_ENABLE_DETAILED_BUILDER_STATS
	#define UPDATE_BUILDER_STAT(Builder, StatId, Delta) (Builder).Stats.StatId += Delta
#else
	#define UPDATE_BUILDER_STAT(Builder, StatId, Num) 
#endif

CSV_DEFINE_CATEGORY(SceneCulling, true);


LLM_DECLARE_TAG_API(SceneCulling, RENDERER_API);
DECLARE_LLM_MEMORY_STAT(TEXT("SceneCulling"), STAT_SceneCullingLLM, STATGROUP_LLMFULL);
DECLARE_LLM_MEMORY_STAT(TEXT("SceneCulling"), STAT_SceneCullingSummaryLLM, STATGROUP_LLM);
LLM_DEFINE_TAG(SceneCulling, NAME_None, NAME_None, GET_STATFNAME(STAT_SceneCullingLLM), GET_STATFNAME(STAT_SceneCullingLLM));

static TAutoConsoleVariable<int32> CVarSceneCullingPrecomputed(
	TEXT("r.SceneCulling.Precomputed"), 
	0, 
	TEXT("Enable/Disable precomputed spatial hashes for scene culling."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarSceneCullingAsyncUpdate(
	TEXT("r.SceneCulling.Async.Update"), 
	1, 
	TEXT("Enable/Disable async culling scene update."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSceneCullingAsyncQuery(
	TEXT("r.SceneCulling.Async.Query"), 
	1, 
	TEXT("Enable/Disable async culling scene queries."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarSceneCullingMinCellSize(
	TEXT("r.SceneCulling.MinCellSize"), 
	4096.0f, 
	TEXT("Set the minimum cell size & level in the hierarchy, rounded to nearest POT. Clamps the level for object footprints.\n")
	TEXT("  This trades culling effectiveness for construction cost and memory use.\n")
	TEXT("  The minimum cell size is (will be) used for precomputing the key for static ISMs.\n")
	TEXT("  Currently read-only as there is implementation to rebuild the whole structure on a change.\n"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<float> CVarSceneCullingMaxCellSize(
	TEXT("r.SceneCulling.MaxCellSize"), 
	UE_OLD_HALF_WORLD_MAX, 
	TEXT("Hierarchy max cell size. Objects with larger bounds will be classified as uncullable."), 
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

static TAutoConsoleVariable<int32> CVarTreatDynamicInstancedAsUncullable(
	TEXT("r.SceneCulling.TreatInstancedDynamicAsUnCullable"), 
	0, 
	TEXT("If this is turned on, dynamic primitives with instances are treated as uncullable (not put into the hierarchy and instead brute-forced on the GPU).")
	TEXT("  This significantly reduces the hierarchy update cost on the CPU and for scenes with a large proportion of static elements, does not increase the GPU cost."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarSmallFootprintSideThreshold(
	TEXT("r.SceneCulling.SmallFootprintSideThreshold"), 
	16, 
	TEXT("Queries with a smaller footprint (maximum) side (in number of cells in the lowest level) go down the footprint based path.\n") 
	TEXT("  The default (16) <=> a footprint of 16x16x16 cells or 8 blocks"), 
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarValidateAllInstanceAllocations(
	TEXT("r.SceneCulling.ValidateAllInstanceAllocations"), 
	0, 
	TEXT("Perform validation of all instance IDs stored in the grid. This is very slow."), 
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarSceneCullingUseForceRebuildExplicitChunkBounds(
	TEXT("r.SceneCulling.ForceRebuildExplicitChunkBounds"), 
	false, 
	TEXT("Forces a rebuild of the explicit chunk bounds each frame, for debugging only."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarSceneCullingHierarchicalCPUCulling(
	TEXT("r.SceneCulling.HierarchicalCPUCulling"), 
	true, 
	TEXT("Use to enable hierarchical CPU culling, which works for instanced things that have pre computed instance bounds."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarSceneCullingHierarchicalCPUCullingProjectEnabled(
	TEXT("r.SceneCulling.HierarchicalCPUCulling.ProjectEnabled"), 
	true, 
	TEXT("Read-only switch to turn off the CPU culling machinery for the project, allows the whole scene extension to be disabled for non-nanite platforms."),
	ECVF_RenderThreadSafe | ECVF_ReadOnly);

#if !UE_BUILD_SHIPPING

static int32 GCaptureNextSceneCullingUpdate = -1;
static FAutoConsoleVariableRef CVarCaptureNextSceneCullingUpdate(
	TEXT("r.CaptureNextSceneCullingUpdate"),
	GCaptureNextSceneCullingUpdate,
	TEXT("0 to capture the immideately next frame using e.g. RenderDoc or PIX.\n")
	TEXT(" > 0: N frames delay\n")
	TEXT(" < 0: disabled"),
	ECVF_RenderThreadSafe);

#endif

static TAutoConsoleVariable<bool> CVarLogCellSizes( TEXT("r.SceneCulling.LogCellSizes"), false, TEXT(""), ECVF_RenderThreadSafe);

#if SC_ENABLE_GPU_DATA_VALIDATION

static TAutoConsoleVariable<int32> CVarValidateGPUData(
	TEXT("r.SceneCulling.ValidateGPUData"), 
	0, 
	TEXT("Perform readback and validation of uploaded GPU-data against CPU copy. This is quite slow and forces CPU/GPU syncs."), 
	ECVF_RenderThreadSafe);


#endif

#if SC_ENABLE_DETAILED_LOGGING
static FSceneCullingBuilder *GBuilderForLogging = nullptr;

struct FLoggerGlobalScopeHelper
{
	FLoggerGlobalScopeHelper(FSceneCullingBuilder *InBuilderForLogging) : BuilderForLogging(InBuilderForLogging)
	{
		GBuilderForLogging = BuilderForLogging;
	}

	~FLoggerGlobalScopeHelper()
	{
		check(GBuilderForLogging == BuilderForLogging);
		GBuilderForLogging = nullptr;
	}

	FSceneCullingBuilder *BuilderForLogging = nullptr;
};

#define SC_DETAILED_LOGGING_SCOPE(_Builder_) FLoggerGlobalScopeHelper LoggerGlobalScopeHelper(_Builder_)


#define BUILDER_LOG(Fmt, ...) if (GBuilderForLogging) { GBuilderForLogging->AddLog(FString::Printf(TEXT(Fmt), ##__VA_ARGS__)); }

struct FIHLoggerScopeHelper
{
	FIHLoggerScopeHelper();
	~FIHLoggerScopeHelper();
};
struct FIHLoggerListScopeHelper
{
	FIHLoggerListScopeHelper(const FString &InListName);

	~FIHLoggerListScopeHelper();

	void Add(const FString &Item)
	{
		if(bEnabled)
		{
			LogStr.Appendf(TEXT("%s%s"), bFirst ? TEXT("") : TEXT(", "), *Item);
		}
		bFirst = false;
	}
	bool bEnabled = true;
	bool bFirst = true;
	FString LogStr;
};

#define BUILDER_LOG_SCOPE(Fmt, ...) if (GBuilderForLogging) { GBuilderForLogging->AddLog(FString::Printf(TEXT(Fmt), ##__VA_ARGS__)); } FIHLoggerScopeHelper UE_JOIN(IHLoggerScopeHelper, __LINE__)
#define BUILDER_LOG_LIST(Fmt, ...) FIHLoggerListScopeHelper IHLoggerListScopeHelper(FString::Printf(TEXT(Fmt), ##__VA_ARGS__))
#define BUILDER_LOG_LIST_APPEND(Fmt, ...) IHLoggerListScopeHelper.Add(FString::Printf(TEXT(Fmt), ##__VA_ARGS__))

static TAutoConsoleVariable<int32> CVarSceneCullingLogBuild(
	TEXT("r.SceneCulling.LogBuild"), 
	0, 
	TEXT("."), 
	ECVF_RenderThreadSafe);

#else
#define SC_DETAILED_LOGGING_SCOPE(_Builder_)
#define BUILDER_LOG(...)
#define BUILDER_LOG_SCOPE(...)
#define BUILDER_LOG_LIST(...)
#define BUILDER_LOG_LIST_APPEND(...)
#endif

IMPLEMENT_SCENE_EXTENSION(FSceneCulling);

/**
 * This conditional may need to move later, e.g, for when preparing data upstream.
 */
static bool UseSceneCulling(EShaderPlatform ShaderPlatform)
{
	return UseNanite(ShaderPlatform);
}

static bool UseHierarchicalCPUCulling()
{
	return CVarSceneCullingHierarchicalCPUCulling.GetValueOnAnyThread() && CVarSceneCullingHierarchicalCPUCullingProjectEnabled.GetValueOnAnyThread();
}

bool FSceneCulling::ShouldCreateExtension(FScene& Scene)
{
	// If CPU culling is enabled for the project, we must always create the extension such that we can toggle at runtime, since we don't track if ShouldCreateExtension changes.
	return CVarSceneCullingHierarchicalCPUCullingProjectEnabled.GetValueOnAnyThread() || DoesPlatformSupportNanite(GetFeatureLevelShaderPlatform(Scene.GetFeatureLevel()));
}

// doesn't exist in the global definitions for some reason
using FInt8Vector3 = UE::Math::TIntVector3<int8>;

namespace EUpdateFrequencyCategory
{
	enum EType
	{
		Static,
		Dynamic,
		Num,
	};

	const TCHAR* ToString(EType Category)
	{
		const TCHAR* Strs[Num] =
		{
			TEXT("Static"),
			TEXT("Dynamic"),
		};
		return Strs[Category];
	}

}

FString GSceneCullingDbgPattern;// = "Cube*";
FAutoConsoleVariableRef CVarSceneCullingDbgPattern(
	TEXT("r.SceneCulling.DbgPattern"),
	GSceneCullingDbgPattern,
	TEXT(""),
	ECVF_RenderThreadSafe
);


const FString &FSceneCulling::FPrimitiveState::ToString() const
{
	static FString Result;
	Result = TEXT("{ ");
	switch(State)
	{
	case Unknown:
		Result.Append(TEXT("Unknown"));
		break;
	case SingleCell:
		Result.Append(TEXT("SingleCell"));
		break;
	case Precomputed:
		Result.Append(TEXT("Precomputed"));
		break;
	case Dynamic:
		Result.Append(TEXT("Dynamic"));
		break;
	case Cached:
		Result.Append(TEXT("Cached"));
		break;
	case UnCullable:
		Result.Append(TEXT("UnCullable"));
		break;
	};
	Result.Appendf(TEXT(", InstanceDataOffset %d, NumInstances %d, bDynamic %d, Payload %d }"), InstanceDataOffset, NumInstances, bDynamic, Payload);
	return Result;
}

inline bool operator==(const FInstanceSceneDataBuffers::FCompressedSpatialHashItem A, const FInstanceSceneDataBuffers::FCompressedSpatialHashItem B)
{
	return A.Location == B.Location && A.NumInstances == B.NumInstances;
}

#if OLA_TODO

struct FSpatialHashNullDebugDrawer
{
	inline void OnBlockBegin(RenderingSpatialHash::FLocation64 BlockLoc) {}
	inline void DrawCell(bool bShouldDraw, bool bHighlight, const FVector3d& CellCenter, const FVector3d& CellBoundsExtent) {}
	inline void DrawBlock(bool bHighlight, const FBox& BlockBounds) {}

};

struct FSpatialHashDebugDrawer
{
	FLinearColor LevelColor;
	FLinearColor CellLevelColor;
	FViewElementPDI* DebugPDI;
	bool bDebugDrawCells;
	bool bDebugDrawBlocks;

	void OnBlockBegin(RenderingSpatialHash::FLocation64 BlockLoc)
	{
		LevelColor = FLinearColor::MakeRandomSeededColor(BlockLoc.Level);
		CellLevelColor = FLinearColor::MakeRandomSeededColor(BlockLoc.Level - FSceneCulling::FSpatialHash::CellBlockDimLog2);
	}
	void DrawCell(bool bShouldDraw, bool bHighlight, const FVector3d& CellCenter, const FVector3d& CellBoundsExtent)
	{
		if (bDebugDrawCells)
		{
			if (bShouldDraw)
			{
				if (bDebugDrawCells)
				{
					DrawWireBox(DebugPDI, FBox3d(CellCenter - CellBoundsExtent, CellCenter + CellBoundsExtent), CellLevelColor * (bHighlight ? 1.0f : 0.2f), SDPG_World);
				}
			}
		}
	}

	void DrawBlock(bool bHighlight, const FBox& BlockBounds)
	{
		if (bDebugDrawBlocks)
		{
			DrawWireBox(DebugPDI, BlockBounds, LevelColor * (bHighlight ? 1.0f : 0.2f), SDPG_World);
		}
	}
};
#endif


SC_FORCEINLINE FPackedCellHeader PackCellHeader(const FCellHeader& CellHeader)
{
	check(CellHeader.bIsValid);
	check(CellHeader.ItemChunksOffset < (1u << INSTANCE_HIERARCHY_CELL_HEADER_OFFSET_BITS));
	check(CellHeader.NumStaticChunks < (1u << INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS));
	// use 0 to represent an invalid cell
	check(CellHeader.NumDynamicChunks  < ((1u << INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS) - 1u));
	
	FPackedCellHeader Packed;

	uint32 NumDynamicChunks = CellHeader.bIsValid ? (CellHeader.NumDynamicChunks + 1u) : 0u;
	uint64& Bits = reinterpret_cast<uint64&>(Packed);
	Bits = (uint64(CellHeader.ItemChunksOffset) << (2u * INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS))
		| (uint64(CellHeader.NumStaticChunks) << INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS)
		| uint64(NumDynamicChunks);
	
	return Packed;
}

SC_FORCEINLINE FSceneCulling::FLocation8 ToBlockLocal(const FSceneCulling::FLocation64& ItemLoc, const FSceneCulling::FBlockLoc& BlockLoc)
{
	checkSlow(FSceneCulling::FSpatialHash::CellBlockDimLog2 == BlockLoc.GetLevel() - ItemLoc.Level);
	// Note: need to go to 64-bit here to avoid edge cases at the far LWC range... can probably reformulate to avoid the need.
	FInt64Vector3 BlockMin = FInt64Vector3(BlockLoc.GetCoord()) << FSceneCulling::FSpatialHash::CellBlockDimLog2;

	// This can be packed to very few bits if need be.
	FSceneCulling::FLocation8 LocalLoc;
	LocalLoc.Coord = FInt8Vector3(ItemLoc.Coord - BlockMin);
	LocalLoc.Level = ItemLoc.Level;

	checkSlow(LocalLoc.Coord.X >= 0 && LocalLoc.Coord.X < FSceneCulling::FSpatialHash::CellBlockDim);
	checkSlow(LocalLoc.Coord.Y >= 0 && LocalLoc.Coord.Y < FSceneCulling::FSpatialHash::CellBlockDim);
	checkSlow(LocalLoc.Coord.Z >= 0 && LocalLoc.Coord.Z < FSceneCulling::FSpatialHash::CellBlockDim);

	return LocalLoc;
};

inline FInt64Vector3 ToLevelRelative(const FInt64Vector3& Coord, int32 LevelDelta)
{
	if (LevelDelta > 0)
	{
		return Coord >> LevelDelta;
	}
	else if (LevelDelta < 0)
	{
		return Coord << -LevelDelta;
	}
	return Coord;
}

inline FSceneCulling::FLocation64 ToLevelRelative(const FSceneCulling::FLocation64& Loc, int32 LevelDelta)
{
	if (LevelDelta == 0)
	{
		return Loc;
	}
	FSceneCulling::FLocation64 ResLoc = Loc;
	ResLoc.Level += LevelDelta;
	ResLoc.Coord = ToLevelRelative(Loc.Coord, LevelDelta);
	return ResLoc;
};


template<int32 LevelDelta>
inline FInt64Vector3 ToLevelRelative(const FInt64Vector3& Coord)
{
	if (LevelDelta > 0)
	{
		return Coord >> LevelDelta;
	}
	else if (LevelDelta < 0)
	{
		return Coord << -LevelDelta;
	}
	return Coord;
}

template<int32 LevelDelta>
inline FSceneCulling::FLocation64 ToLevelRelative(const FSceneCulling::FLocation64& Loc)
{
	if (LevelDelta == 0)
	{
		return Loc;
	}
	FSceneCulling::FLocation64 ResLoc = Loc;
	ResLoc.Level += LevelDelta;
	ResLoc.Coord = ToLevelRelative<LevelDelta>(Loc.Coord);
	return ResLoc;
};

inline FSceneCulling::FFootprint64 ToLevelRelative(const FSceneCulling::FFootprint64& Footprint, int32 LevelDelta)
{
	FSceneCulling::FFootprint64 Result;
	Result.Min = ToLevelRelative(Footprint.Min, LevelDelta);
	Result.Max = ToLevelRelative(Footprint.Max, LevelDelta);
	Result.Level = Footprint.Level + LevelDelta;
	return Result;
};

/**
 * Clamp the level to the provided minimum. That is if the level of Loc is below MinLevel change it to MinLevel but not if already above.
 */
inline FSceneCulling::FLocation64 ClampToMinLevel(const FSceneCulling::FLocation64& Loc, int32 MinLevel)
{
	if (Loc.Level < MinLevel)
	{
		return RenderingSpatialHash::ToLevel(Loc, MinLevel);
	}
	return Loc;
};

#if 0
void FSceneCulling::TestConvexVolume(const FConvexVolume& ViewCullVolume, const FVector3d &WorldToVolumeTranslation, TArray<FCellDraw, SceneRenderingAllocator>& OutCellDraws, uint32 ViewGroupId, uint32& OutNumInstanceGroups) const
{
	LLM_SCOPE_BYTAG(SceneCulling);

	SCOPE_CYCLE_COUNTER(STAT_SceneCulling_Test_Convex);
#if OLA_TODO
	FSpatialHashNullDebugDrawer DebugDrawer;
#endif

	OutNumInstanceGroups += UncullableNumItemChunks;

	for (auto It = SpatialHash.GetHashMap().begin(); It != SpatialHash.GetHashMap().end(); ++It)
	{
		const auto& BlockItem = *It;
		int32 BlockIndex = It.GetElementId().GetIndex();
		const FSpatialHash::FCellBlock& Block = BlockItem.Value;
		FSpatialHash::FBlockLoc BlockLoc = BlockItem.Key;

#if OLA_TODO
		DebugDrawer.OnBlockBegin(BlockLoc);
#endif

		const double BlockLevelSize = SpatialHash.GetCellSize(BlockLoc.GetLevel());
		FVector3d BlockBoundsCenter = FVector3d(BlockLoc.GetCoord()) * BlockLevelSize + BlockLevelSize * 0.5;
		const double LevelCellSize = SpatialHash.GetCellSize(BlockLoc.GetLevel() - FSpatialHash::CellBlockDimLog2);
		// Extend extent by half a cell size in all directions
		FVector3d BlockBoundsExtent = FVector((BlockLevelSize + LevelCellSize) * 0.5);
		FOutcode BlockCullResult = ViewCullVolume.GetBoxIntersectionOutcode(BlockBoundsCenter + WorldToVolumeTranslation, BlockBoundsExtent);

		if (BlockCullResult.GetInside())
		{
			const bool bIsContained = !BlockCullResult.GetOutside();
			if (bIsContained)
			{
				// TODO: collect this in a per-block summary so we don't need to hit cell headers in this path
				OutNumInstanceGroups += Block.NumItemChunks;

				// Fully inside, just append non-empty cells
				int32 BitRangeEnd = Block.GridOffset + FSpatialHash::CellBlockSize;
				for (TConstSetBitIterator<> BitIt(CellOccupancyMask, Block.GridOffset); BitIt && BitIt.GetIndex() < BitRangeEnd; ++BitIt)
				{
					uint32 CellId = uint32(BitIt.GetIndex());
					OutCellDraws.Add(FCellDraw{ CellId, ViewGroupId });
				}
			}
			else
			{
				FVector3d CellBoundsExtent = FVector3d(LevelCellSize);
				FVector3d MinCellCenter = FVector3d(BlockLoc.GetCoord()) * BlockLevelSize + LevelCellSize * 0.5;

				int32 BitRangeEnd = Block.GridOffset + FSpatialHash::CellBlockSize;
				for (TConstSetBitIterator<> BitIt(CellOccupancyMask, Block.GridOffset); BitIt && BitIt.GetIndex() < BitRangeEnd; ++BitIt)
				{
					uint32 CellId = uint32(BitIt.GetIndex());
					FInt8Vector3 CellCoord;
					{
						uint32 BlockCellIndex = CellId - Block.GridOffset;
						CellCoord.X = BlockCellIndex & FSpatialHash::LocalCellCoordMask;
						BlockCellIndex = BlockCellIndex >> FSpatialHash::CellBlockDimLog2;
						CellCoord.Y = BlockCellIndex & FSpatialHash::LocalCellCoordMask;
						BlockCellIndex = BlockCellIndex >> FSpatialHash::CellBlockDimLog2;
						CellCoord.Z = BlockCellIndex;
					}
					// world-space double precision test, can equally translate to view local and do single precision test (with epsilons perhaps)
					FVector3d CellCenter = FVector3d(CellCoord) * LevelCellSize + MinCellCenter;

					// emit for intersecting cells
					bool bCellIntersects = ViewCullVolume.IntersectBox(CellCenter, WorldToVolumeTranslation, CellBoundsExtent);
					if (bCellIntersects)
					{
						FCellHeader CellHeader = UnpackCellHeader(CellHeaders[CellId]);
						check(IsValidCell(CellHeader));

						OutNumInstanceGroups += CellHeader.NumItemChunks;
						OutCellDraws.Add(FCellDraw{ CellId, ViewGroupId });
					}
#if OLA_TODO
					DebugDrawer.DrawCell(bCellIntersects && !bIsContained, bCellIntersects, CellCenter, CellBoundsExtent);
#endif
				}
#if OLA_TODO
				DebugDrawer.DrawBlock(bIsContained, BlockBounds);
#endif
			}
		}
	}
}

#endif

bool FSceneCulling::IsSmallCullingVolume(const FCullingVolume& CullingVolume) const
{
	if (CullingVolume.Sphere.W > 0.0f)
	{
		const float Level0CellSize = SpatialHash.GetCellSize(SpatialHash.GetFirstLevel());
		FFootprint64 LightFootprint = SpatialHash.CalcFootprintSphere(SpatialHash.GetFirstLevel(), CullingVolume.Sphere.Center, CullingVolume.Sphere.W + (Level0CellSize * 0.5f));

		if ((LightFootprint.Max - LightFootprint.Min).GetMax() <= int64(SmallFootprintCellSideThreshold))
		{
			return true;
		}
	}
	return false;
}

void FSceneCulling::Empty()
{
	LLM_SCOPE_BYTAG(SceneCulling);

	SpatialHash.Empty();
	PackedCellChunkData.Empty();
	CellChunkIdAllocator.Empty();
	PackedCellData.Empty();
	FreeChunks.Empty();
	CellHeaders.Empty();
	CellOccupancyMask.Empty();
	BlockLevelOccupancyMask.Empty();

	CellBlockData.Empty();
	UnCullablePrimitives.Empty();
	
	UncullableItemChunksOffset = INDEX_NONE;
	UncullableNumItemChunks = 0;

	PrimitiveStates.Empty();
	CellIndexCache.Empty();
	TotalCellIndexCacheItems = 0;
	NumStaticInstances = 0;
	NumDynamicInstances = 0;

	CellHeadersBuffer.Empty();
	ItemChunksBuffer.Empty();
	InstanceIdsBuffer.Empty();
	CellBlockDataBuffer.Empty();
	ExplicitChunkBoundsBuffer.Empty();
	ExplicitChunkCellIdsBuffer.Empty();
	UsedChunkIdMaskBuffer.SafeRelease();
}

class FComputeExplicitChunkBounds_CS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FComputeExplicitChunkBounds_CS);
	SHADER_USE_PARAMETER_STRUCT(FComputeExplicitChunkBounds_CS, FGlobalShader);

	static constexpr int32 NumThreadsPerGroup = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return DoesPlatformSupportNanite(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("USE_GLOBAL_GPU_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FGPUSceneResourceParameters, GPUSceneParameters)
		SHADER_PARAMETER(uint32, NumCellsPerBlockLog2)
		SHADER_PARAMETER(uint32, CellBlockDimLog2)
		SHADER_PARAMETER(uint32, LocalCellCoordMask) // (1 << NumCellsPerBlockLog2) - 1
		SHADER_PARAMETER(int32, FirstLevel)
		SHADER_PARAMETER(int32, DirtyChunkCount)

		SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< FCellBlockData >, InstanceHierarchyCellBlockData)
		SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< FPackedCellHeader >, InstanceHierarchyCellHeaders)
		SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< uint >, InstanceHierarchyItemChunks)
		SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< uint >, InstanceIds)
		SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< int2 >, DirtyChunkBoundsData)
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWByteAddressBuffer, OutExplicitChunkBoundsBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV( RWStructuredBuffer<uint>, OutExplicitChunkCellIdsBuffer)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FComputeExplicitChunkBounds_CS, "/Engine/Private/SceneCulling/SceneCullingBuildExplicitBounds.usf", "ComputeExplicitChunkBounds", SF_Compute);

/**
 * Produce a world-space bounding sphere for an instance given local bounds and transforms.
 */
SC_FORCEINLINE FVector4d TransformBounds(VectorRegister4f VecOrigin, VectorRegister4f VecExtent, const FRenderTransform& LocalToPrimitiveRelative, VectorRegister4Double PrimitiveToWorldTranslationVec)
{
	// 1. Matrix Concat and bounds all in one
	VectorRegister4f NewOrigin;
	VectorRegister4f NewExtent;

	{
		const VectorRegister4Float ARow = VectorLoadFloat3(&LocalToPrimitiveRelative.TransformRows[0]);
		NewOrigin = VectorMultiplyAdd(VectorReplicate(VecOrigin, 0), ARow, VectorLoadFloat3(&LocalToPrimitiveRelative.Origin));
		NewExtent = VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 0), ARow));
	}

	{
		const VectorRegister4Float ARow = VectorLoadFloat3(&LocalToPrimitiveRelative.TransformRows[1]);
		NewOrigin = VectorMultiplyAdd(VectorReplicate(VecOrigin, 1), ARow, NewOrigin);
		NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 1), ARow)));
	}

	{
		const VectorRegister4Float ARow = VectorLoadFloat3(&LocalToPrimitiveRelative.TransformRows[2]);
		NewOrigin = VectorMultiplyAdd(VectorReplicate(VecOrigin, 2), ARow, NewOrigin);
		NewExtent = VectorAdd(NewExtent, VectorAbs(VectorMultiply(VectorReplicate(VecExtent, 2), ARow)));
	}

	// Offset sphere and return
	float Radius = FMath::Sqrt(VectorDot3Scalar(NewExtent, NewExtent));
	const VectorRegister4Double VecCenterOffset = VectorAdd(PrimitiveToWorldTranslationVec, NewOrigin);

	FVector4d Result;
	VectorStoreAligned(VecCenterOffset, &Result);
	Result.W = Radius;
	return Result;
}

struct FBoundsTransformerBase
{
	SC_FORCEINLINE FBoundsTransformerBase(const FInstanceSceneDataBuffers& InInstanceSceneDataBuffers)
		: InstanceSceneDataBuffers(InInstanceSceneDataBuffers)
	{
		// Note: for reasons unknown VectorLoadFloat3 also does doubles...
		PrimitiveToWorldTranslationVec = VectorLoadFloat3(&InstanceSceneDataBuffers.GetPrimitiveWorldSpaceOffset());


		FInstanceSceneDataBuffers::FReadView InstanceDataView = InstanceSceneDataBuffers.GetReadView();
		if (InstanceDataView.InstanceToPrimitiveRelative.IsEmpty())
		{
			// Set up with dummy data array
			InstanceToPrimitiveRelativeArray = TConstArrayView<FRenderTransform>(&FRenderTransform::Identity, 1);
		}
		else
		{
			InstanceToPrimitiveRelativeArray = InstanceDataView.InstanceToPrimitiveRelative;
		}
	}

	// returns the clamped instance transform, to cover for OOB accesses
	SC_FORCEINLINE FRenderTransform GetInstanceToPrimitiveRelative(int32 InstanceIndex)
	{
		return InstanceToPrimitiveRelativeArray[FMath::Min(InstanceToPrimitiveRelativeArray.Num() - 1, InstanceIndex)];
	}
	VectorRegister4Double PrimitiveToWorldTranslationVec;

	const FInstanceSceneDataBuffers &InstanceSceneDataBuffers;
	TConstArrayView<FRenderTransform> InstanceToPrimitiveRelativeArray;
};

struct FBoundsTransformerUniqueBounds : public FBoundsTransformerBase
{
	SC_FORCEINLINE FBoundsTransformerUniqueBounds(const FInstanceSceneDataBuffers& InInstanceSceneDataBuffers)
		: FBoundsTransformerBase(InInstanceSceneDataBuffers)
	{
	}

	SC_FORCEINLINE FVector4d TransformBounds(int32 InstanceIndex)
	{
		const FRenderBounds InstanceBounds = InstanceSceneDataBuffers.GetInstanceLocalBounds(InstanceIndex);

		FRenderTransform InstanceToPrimitiveRelative = GetInstanceToPrimitiveRelative(InstanceIndex);
		const VectorRegister4f VecMin = VectorLoadFloat3(&InstanceBounds.Min);
		const VectorRegister4f VecMax = VectorLoadFloat3(&InstanceBounds.Max);
		const VectorRegister4f Half = VectorSetFloat1(0.5f); // VectorSetFloat1() can be faster than SetFloat3(0.5, 0.5, 0.5, 0.0). Okay if 4th element is 0.5, it's multiplied by 0.0 below and we discard W anyway.
		const VectorRegister4f VecOrigin = VectorMultiply(VectorAdd(VecMax, VecMin), Half);
		const VectorRegister4f VecExtent = VectorMultiply(VectorSubtract(VecMax, VecMin), Half);

		return ::TransformBounds(VecOrigin, VecExtent, InstanceToPrimitiveRelative, PrimitiveToWorldTranslationVec);
	}
};

struct FBoundsTransformerSharedBounds : public FBoundsTransformerBase
{
	SC_FORCEINLINE FBoundsTransformerSharedBounds(const FInstanceSceneDataBuffers& InInstanceSceneDataBuffers)
		: FBoundsTransformerBase(InInstanceSceneDataBuffers)
	{

		const FRenderBounds InstanceBounds = InstanceSceneDataBuffers.GetInstanceLocalBounds(0);
		const VectorRegister4f VecMin = VectorLoadFloat3(&InstanceBounds.Min);
		const VectorRegister4f VecMax = VectorLoadFloat3(&InstanceBounds.Max);
		const VectorRegister4f Half = VectorSetFloat1(0.5f); // VectorSetFloat1() can be faster than SetFloat3(0.5, 0.5, 0.5, 0.0). Okay if 4th element is 0.5, it's multiplied by 0.0 below and we discard W anyway.
		VecOrigin = VectorMultiply(VectorAdd(VecMax, VecMin), Half);
		VecExtent = VectorMultiply(VectorSubtract(VecMax, VecMin), Half);
	}

	SC_FORCEINLINE FVector4d TransformBounds(int32 InstanceIndex)
	{
		FRenderTransform InstanceToPrimitiveRelative = GetInstanceToPrimitiveRelative(InstanceIndex);
		return ::TransformBounds(VecOrigin, VecExtent, InstanceToPrimitiveRelative, PrimitiveToWorldTranslationVec);
	}

	VectorRegister4f VecOrigin;
	VectorRegister4f VecExtent;
};

template <typename BoundsTransformerType>
struct FHashLocationComputerFromBounds
{
	SC_FORCEINLINE FHashLocationComputerFromBounds(	const FInstanceSceneDataBuffers &InInstanceSceneDataBuffers, FSceneCulling::FSpatialHash& InSpatialHash)
		: BoundsTransformer(InInstanceSceneDataBuffers)
		, SpatialHash(InSpatialHash)
	{
	}

	SC_FORCEINLINE FSceneCulling::FLocation64 CalcLoc(int32 InstanceIndex)
	{
		FVector4d InstanceWorldBound = BoundsTransformer.TransformBounds(InstanceIndex);
		return SpatialHash.CalcLevelAndLocation(InstanceWorldBound);
	}

	BoundsTransformerType BoundsTransformer;
	FSceneCulling::FSpatialHash& SpatialHash;
};

class FSceneCullingBuilder
{
public:

	// Alias a bunch of types / definitions from the instance hierarchy
	using FSpatialHash = FSceneCulling::FSpatialHash;
	using FHashElementId = FSpatialHash::FHashElementId;
	static constexpr uint32 TempCellMarker = 0xFFFFFFFFu;

	static constexpr int32 CellBlockSize = FSpatialHash::CellBlockSize;
	static constexpr int32 CellBlockDimLog2 = FSpatialHash::CellBlockDimLog2;

	static constexpr int32 MaxChunkSize = int32(INSTANCE_HIERARCHY_MAX_CHUNK_SIZE);
	using FPrimitiveState = FSceneCulling::FPrimitiveState;
	using FCellIndexCacheEntry = FSceneCulling::FCellIndexCacheEntry;

	using FBlockLoc = FSceneCulling::FBlockLoc;

	enum class EExplicitBoundsUpdateMode
	{
		Disabled,
		Incremental,
		Full,
	};

	EExplicitBoundsUpdateMode ExplicitBoundsMode = EExplicitBoundsUpdateMode::Disabled;

	inline bool IsTempCell(int32 CellIndex)
	{
		return TempCellMask.IsValidIndex(CellIndex) && TempCellMask[CellIndex];
	}

	static constexpr FPackedCellHeader InvalidPackedCell = FPackedCellHeader{ 0u, 0u };

#if SC_ENABLE_DETAILED_BUILDER_STATS
	// Detail Stats
	struct FStats
	{
		int32 RangeCount = 0;
		int32 CompRangeCount = 0;
		int32 CopiedIdCount = 0;
		int32 VisitedIdCount = 0;
	};
	FStats Stats;
#endif 

	FSceneCullingBuilder(FSceneCulling& InSceneCulling, bool bAnySceneUpdatesExpected) 
		: SceneCulling(InSceneCulling)
		, SpatialHash(InSceneCulling.SpatialHash)
		, SpatialHashMap(InSceneCulling.SpatialHash.GetHashMap())
	{
		// re-used flag array for instances that are to be removed.
		RemovedInstanceFlags.SetNum(SceneCulling.Scene.GPUScene.GetNumInstances(), false);
		//RemovedInstanceCellFlags.SetNum(SceneCulling.CellHeaders.Num(), false);
		//RemovedInstanceCellInds.Reserve(SceneCulling.CellHeaders.Num());
		// TODO: this is not needed when we also call update for added primitives correctly, remove and replace with a check!
		SceneCulling.PrimitiveStates.SetNum(SceneCulling.Scene.GetMaxPersistentPrimitiveIndex());

		bUsePrecomputed = CVarSceneCullingPrecomputed.GetValueOnAnyThread() != 0;

		if (CVarTreatDynamicInstancedAsUncullable.GetValueOnRenderThread() != 0)
		{
			// Flip dynamic stuff into the uncullabe bucket. 
			DynamicInstancedPrimitiveState = FPrimitiveState::UnCullable;
		}

#if SC_ENABLE_DETAILED_LOGGING
		bIsLoggingEnabled = CVarSceneCullingLogBuild.GetValueOnRenderThread() != 0 && bAnySceneUpdatesExpected || CVarSceneCullingLogBuild.GetValueOnRenderThread() > 1;
		SceneTag.Appendf(TEXT("[%s]"), SceneCulling.Scene.IsEditorScene() ? TEXT("EditorScene") : TEXT(""));
		SceneTag.Append(SceneCulling.Scene.GetFullWorldName());
		AddLog(FString("Log-Scope-Begin - ") + SceneTag);
		LogIndent(1);
#endif
	}
	
	/**
	 * Allocate space for a range of chunk-offsets in the buffer if needed, and free the previous chunk if there was one.
	 */
	SC_FORCEINLINE uint32 ReallocateChunkRange(uint32 NewNumItemChunks, uint32 PrevItemChunksOffset, uint32 PrevNumItemChunks)
	{
		uint32 ItemChunksOffset = PrevItemChunksOffset;
		// TODO: round allocation size to POT, or some multiple to reduce reallocations & fragmentation?
		if (ItemChunksOffset != INDEX_NONE && PrevNumItemChunks != NewNumItemChunks)
		{
			BUILDER_LOG("Free Chunk Range: [%d,%d)", PrevItemChunksOffset, PrevItemChunksOffset + PrevNumItemChunks);
			SceneCulling.CellChunkIdAllocator.Free(PrevItemChunksOffset, PrevNumItemChunks);
			SceneCulling.UsedChunkIdMask.SetRange(PrevItemChunksOffset, PrevNumItemChunks, false);
			ItemChunksOffset = INDEX_NONE;
		}

		// Need a new chunk offset allocated
		if (NewNumItemChunks != 0 && ItemChunksOffset == INDEX_NONE)
		{
			ItemChunksOffset = SceneCulling.CellChunkIdAllocator.Allocate(NewNumItemChunks);
			SceneCulling.UsedChunkIdMask.PadToNum(ItemChunksOffset + NewNumItemChunks, false);
			SceneCulling.UsedChunkIdMask.SetRange(ItemChunksOffset, NewNumItemChunks, true);
			BUILDER_LOG("Allocate Chunk Range: [%d,%d)", ItemChunksOffset, ItemChunksOffset + NewNumItemChunks);
		} 
		return ItemChunksOffset;
	}

	struct FChunkBuilder
	{
		int32 CurrentChunkCount = MaxChunkSize;
		int32 CurrentChunkId = INDEX_NONE;
		

		SC_FORCEINLINE bool IsCurrentChunkEmpty() const
		{
			return CurrentChunkId == INDEX_NONE;
		}

		SC_FORCEINLINE void EmitCurrentChunkIfFull()
		{
			if (CurrentChunkId != INDEX_NONE && CurrentChunkCount >= MaxChunkSize)
			{
				PackedChunkIds.Add(uint32(CurrentChunkId) | (uint32(CurrentChunkCount) << INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT));
				CurrentChunkId = INDEX_NONE;
			}
		}

		SC_FORCEINLINE void EmitCurrentChunk()
		{
			if (CurrentChunkId != INDEX_NONE)
			{
				PackedChunkIds.Add(uint32(CurrentChunkId) | (uint32(CurrentChunkCount) << INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT));
			}
			CurrentChunkId = INDEX_NONE;
		}

		SC_FORCEINLINE void AddCompressedChunk(FSceneCullingBuilder& Builder, uint32 StartInstanceId)
		{
			Builder.TotalCellChunkDataCount += 1;
			// Add directly to chunk headers to not upset current chunk packing
			PackedChunkIds.Add(uint32(StartInstanceId) | INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_FLAG);

			//Builder->SceneCulling.InstanceIdToCellDataSlot[StartInstanceId] = FCellDataSlot { true, 0 };
		}

		SC_FORCEINLINE void Add(FSceneCullingBuilder& Builder, uint32 InstanceId)
		{
			TArray<uint32>& PackedCellData = Builder.SceneCulling.PackedCellData;
			if (CurrentChunkCount >= MaxChunkSize)
			{
				EmitCurrentChunk();
				// Allocate a new chunk
				CurrentChunkId = Builder.AllocateChunk();
				CurrentChunkCount = 0;
				Builder.TotalCellChunkDataCount += 1;
			}
			// Emit the instance ID directly to final destination
			int32 ItemOffset = CurrentChunkId * MaxChunkSize + CurrentChunkCount++;
			PackedCellData[ItemOffset] = InstanceId;
		}

		SC_FORCEINLINE void AddRange(FSceneCullingBuilder& Builder, int32 InInstanceIdOffset, int32 InInstanceIdCount)
		{
			// First add all compressible ranges.
			int32 InstanceIdOffset = InInstanceIdOffset;
			int32 InstanceIdCount = InInstanceIdCount;
			while (InstanceIdCount >= MaxChunkSize)
			{
				UPDATE_BUILDER_STAT(Builder, CompRangeCount, 1);

				AddCompressedChunk(Builder, InstanceIdOffset);
				InstanceIdOffset += MaxChunkSize;
				InstanceIdCount -= MaxChunkSize;
			}

			// add remainder individually
			for (int32 InstanceId = InstanceIdOffset; InstanceId < InstanceIdOffset + InstanceIdCount; ++InstanceId)
			{
				Add(Builder, InstanceId);
			}
		}

		SC_FORCEINLINE void AddExistingChunk(FSceneCullingBuilder& Builder, uint32 PackedChunkDesc)
		{
			Builder.TotalCellChunkDataCount += 1;
			// Add directly to chunk headers to not upset current chunk packing
			PackedChunkIds.Add(PackedChunkDesc);
		}

		SC_FORCEINLINE void OutputChunkIds(FSceneCullingBuilder &Builder, uint32 ItemChunksOffset)
		{
			BUILDER_LOG("OutputChunkIds Num: %u -> %d", PackedChunkIds.Num(), ItemChunksOffset);
			int32 NumIds = PackedChunkIds.Num();
			// 3. Copy the list of chunk IDs
			for (uint32 Index = 0; Index < uint32(NumIds); ++Index)
			{
				uint32 ChunkIndex = ItemChunksOffset + Index;
				uint32 ChunkData = PackedChunkIds[Index];
				Builder.SceneCulling.PackedCellChunkData[ChunkIndex] = ChunkData;
				// needs variable size scatter, this is using 1:1 int to scatter the data (an int).
				Builder.ItemChunkUploader.Add(ChunkData, ChunkIndex);
				BUILDER_LOG("ItemChunkUploader %u -> %d", ChunkData, ChunkIndex);

			}
		}

		SC_FORCEINLINE bool IsEmpty() const { return PackedChunkIds.IsEmpty(); }


		SC_FORCEINLINE int32 FinalizeChunks(FSceneCullingBuilder& Builder, int32 StartChunkOffset, int32 EndChunkOffset, int32& NumToRemove)
		{
			// Process removals if there are any left
			int32 ChunkOffset = StartChunkOffset;
			// Scan the chunks in the existing cell.
			for (; ChunkOffset < EndChunkOffset && NumToRemove > 0; ++ChunkOffset)
			{
				uint32 PackedChunkData = Builder.SceneCulling.PackedCellChunkData[ChunkOffset];

				const bool bIsCompressed = (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_FLAG) != 0u;
				const uint32 NumItems = bIsCompressed ? 64u : PackedChunkData >> INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT;
				const bool bIsFullChunk = NumItems == 64u;

				// 1. if it is a compressed chunk and contains any index, we may assume it is to be removed entirely.
				if (bIsCompressed)
				{
					uint32 FirstInstanceDataOffset = PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_PAYLOAD_MASK;
					if (!Builder.IsMarkedForRemove(FirstInstanceDataOffset))
					{
						BUILDER_LOG("Chunk-Retained (FirstInstanceDataOffset: %d)", FirstInstanceDataOffset);
						AddExistingChunk(Builder, PackedChunkData);
					}
					else
					{
						BUILDER_LOG("Chunk-Removed (FirstInstanceDataOffset: %d)", FirstInstanceDataOffset);
						NumToRemove -= 64;
					}
				}
				// 2. elsewise, loop over the items in the chunk and copy not-deleted ones.
				else
				{
					uint32 ChunkId =  (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_ID_MASK);
					uint32 ItemDataOffset = ChunkId * INSTANCE_HIERARCHY_MAX_CHUNK_SIZE;

					UPDATE_BUILDER_STAT(Builder, VisitedIdCount, NumItems);

					// 3.1. scan chunk for deleted items
					uint64 DeletionMask = 0ull;

					// Mark chunks data array as locked & return a pointer to this chunk, allocates a number of chunks of slack by default as it must not resize the underlying array during this.
					// Here, we know there can be at most one new chunk allocated.
					const uint32* RESTRICT ChunkDataPtr = Builder.SceneCulling.LockChunkCellData(ChunkId, 1);//PackedCellData[ItemDataOffset];

					{
						BUILDER_LOG_LIST("ScanChunk[ID:%d](%d):", ChunkId, NumItems);
						for (uint32 ItemIndex = 0u; ItemIndex < NumItems; ++ItemIndex)
						{
							uint32 InstanceId = ChunkDataPtr[ItemIndex];
							if (Builder.IsMarkedForRemove(InstanceId))
							{
								BUILDER_LOG_LIST_APPEND("RM:%d", InstanceId);

								DeletionMask |= 1ull << ItemIndex;
								NumToRemove -= 1;
							}
							else
							{
								BUILDER_LOG_LIST_APPEND("KP:%d", InstanceId);
							}
						}
					}

					// 3.2 If none were actually deleted, then re-emit the chunk (if it is full - otherwise we may end up with a lot of half filled chunks)
					if (DeletionMask == 0ull && bIsFullChunk)
					{
						AddExistingChunk(Builder, PackedChunkData);
					}
					else
					{
						// mask with 1 for each item to be kept
						uint64 RetainedMask = (~DeletionMask) & ((~0ull) >> (64u - NumItems));
						BUILDER_LOG_LIST("Retained(%d):", FMath::CountBits64(RetainedMask));
						
						// 3.3., otherwise, we must copy the surviving IDs
						while (RetainedMask != 0ull)
						{
							uint32 ItemIndex = FMath::CountTrailingZeros64(RetainedMask);
							RetainedMask &= RetainedMask - 1ull;
							uint32 InstanceId = ChunkDataPtr[ItemIndex];
							BUILDER_LOG_LIST_APPEND("K: %d", InstanceId);
							UPDATE_BUILDER_STAT(Builder, CopiedIdCount, 1);
							Add(Builder, InstanceId);
						}

						// Mark the chunk as not in use.
						Builder.SceneCulling.FreeChunk(ChunkId);
					}

					Builder.SceneCulling.UnLockChunkCellData(ChunkId);
				}
			}

			// If we have not reached the last chunk, there must be left overs that we can just copy
			if (ChunkOffset < EndChunkOffset)
			{
				// No removals, do a fast path
				int32 LastChunkIndex = EndChunkOffset - 1;
				uint32 PackedChunkData = Builder.SceneCulling.PackedCellChunkData[LastChunkIndex];
				const bool bIsLastCompressed = (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_FLAG) != 0u;
				const uint32 LastNumItems = bIsLastCompressed ? 64u : PackedChunkData >> INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT;

				// Conditionally flush the chunk in progress if it is full anyway.
				EmitCurrentChunkIfFull();

				// we can just copy the lot if there is no chunk in progress as well as if it is full.
				bool bBulkCopyLast = LastNumItems == 64u || IsCurrentChunkEmpty();
				int32 NumToBulkCopy = (EndChunkOffset - ChunkOffset) - (bBulkCopyLast ? 0 : 1);
				if (NumToBulkCopy > 0)
				{
					PackedChunkIds.Append(TConstArrayView<uint32>(&Builder.SceneCulling.PackedCellChunkData[ChunkOffset], NumToBulkCopy));
				}
				// need to rebuild the last chunk to continue adding to it.
				if (!bBulkCopyLast)
				{
					UPDATE_BUILDER_STAT(Builder, VisitedIdCount, LastNumItems);

					uint32 ChunkId =  (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_ID_MASK);
					uint32 ItemDataOffset = ChunkId * INSTANCE_HIERARCHY_MAX_CHUNK_SIZE;

					// 3. remove ids marked for delete
					for (uint32 ItemIndex = 0u; ItemIndex < LastNumItems; ++ItemIndex)
					{
						UPDATE_BUILDER_STAT(Builder, CopiedIdCount, 1);

						uint32 InstanceId = Builder.SceneCulling.PackedCellData[ItemDataOffset + ItemIndex];
						Add(Builder, InstanceId);
					}

					// Mark the chunk as not in use.
					Builder.SceneCulling.FreeChunk(ChunkId);
				}
			}

			// Flush the remainder into the last chunk.
			EmitCurrentChunk();

			return PackedChunkIds.Num();
		}

		TArray<uint32, TInlineAllocator<32, SceneRenderingAllocator>> PackedChunkIds;
	};

	/**
	 * The temp cell is used to record information about additions / removals for a grid cell during the update.
	 * An index to a temp cell is stored in the grid instead of the index to cell data when a cell is first accessed during update.
	 * At the end of the update all temp cells are processed and then removed.
	 */
	struct FTempCell
	{
		int32 CellOffset = INDEX_NONE;
		int32 ItemChunksOffset = INDEX_NONE;
		int32 RemovedInstanceCount[EUpdateFrequencyCategory::Num] = { 0, 0 };
		bool  bAnyChange[EUpdateFrequencyCategory::Num] = { false, false };

		FCellHeader PrevCellHeader = FCellHeader { false, 0u };

		SC_FORCEINLINE int32 GetRemovedInstanceCount() const
		{
			return RemovedInstanceCount[EUpdateFrequencyCategory::Static] + RemovedInstanceCount[EUpdateFrequencyCategory::Dynamic];
		}

		SC_FORCEINLINE void FinalizeChunks(FSceneCullingBuilder& Builder)
		{

			BUILDER_LOG_SCOPE("FinalizeChunks(Index: %d RemovedInstanceCount %d):", CellOffset, RemovedInstanceCount);
#if OLA_TODO
			// Handle complete removal case efficiently
			if (RemovedInstanceCount == TotalCellInstances)
			{
				ClearCell();
			}
#endif
			//check(IsValidCell(PrevCellHeader));
			const int32 PrevNumItemChunks = IsValidCell(PrevCellHeader) ? PrevCellHeader.NumItemChunks : 0;
			const int32 PrevNumStaticItemChunks = IsValidCell(PrevCellHeader) ? PrevCellHeader.NumStaticChunks : 0;
			const int32 PrevItemChunksOffset = IsValidCell(PrevCellHeader) ? PrevCellHeader.ItemChunksOffset : INDEX_NONE;

			int32 TotalItemChunks = 0;
			// 1. flush the dynamic stuff
			{
				int32 NumToRemove = RemovedInstanceCount[EUpdateFrequencyCategory::Dynamic];
				TotalItemChunks += Builders[EUpdateFrequencyCategory::Dynamic].FinalizeChunks(Builder, PrevItemChunksOffset + PrevNumStaticItemChunks, PrevItemChunksOffset + PrevNumItemChunks, NumToRemove);
				check(NumToRemove == 0);
			}

			// 2. And then the static.
			{
				int32 NumToRemove = RemovedInstanceCount[EUpdateFrequencyCategory::Static];
				TotalItemChunks += Builders[EUpdateFrequencyCategory::Static].FinalizeChunks(Builder, PrevItemChunksOffset, PrevItemChunksOffset + PrevNumStaticItemChunks, NumToRemove);
				check(NumToRemove == 0);
			}

			// Insert retained chunk info first.
			int32 BlockId = Builder.SceneCulling.CellIndexToBlockId(CellOffset);
			FSpatialHash::FCellBlock& Block = Builder.SpatialHashMap.GetByElementId(BlockId).Value;

			// update the delta to track total
			Block.NumItemChunks += TotalItemChunks - PrevNumItemChunks;
			check(Block.NumItemChunks >= 0);

			ItemChunksOffset = Builder.ReallocateChunkRange(TotalItemChunks, PrevItemChunksOffset, PrevNumItemChunks);

			// Must rebuild the chunk bounds if they moved as the indexing is based on the chunk index.
			if (PrevItemChunksOffset != ItemChunksOffset)
			{
				bAnyChange[EUpdateFrequencyCategory::Static] = true;
				bAnyChange[EUpdateFrequencyCategory::Dynamic] = true;
			}
		}

		// One Builders for static and one for dynamic
		FChunkBuilder Builders[EUpdateFrequencyCategory::Num];

		SC_FORCEINLINE bool IsEmpty() const
		{
			return Builders[EUpdateFrequencyCategory::Static].IsEmpty() && Builders[EUpdateFrequencyCategory::Dynamic].IsEmpty();
		}

		SC_FORCEINLINE int32 GetTotalItemChunks() const
		{
			return Builders[EUpdateFrequencyCategory::Static].PackedChunkIds.Num() + Builders[EUpdateFrequencyCategory::Dynamic].PackedChunkIds.Num();
		}

		SC_FORCEINLINE uint32 GetNumItemChunks(EUpdateFrequencyCategory::EType UpdateFrequencyCategory) const
		{
			return uint32(Builders[UpdateFrequencyCategory].PackedChunkIds.Num());
		}

		SC_FORCEINLINE void OutputChunkIds(FSceneCullingBuilder &Builder)
		{
			Builders[EUpdateFrequencyCategory::Static].OutputChunkIds(Builder, ItemChunksOffset);
			Builders[EUpdateFrequencyCategory::Dynamic].OutputChunkIds(Builder, ItemChunksOffset + Builders[EUpdateFrequencyCategory::Static].PackedChunkIds.Num());
		}
	};

	SC_FORCEINLINE FHashElementId FindOrAddBlock(const FBlockLoc& BlockLoc)
	{
		bool bAlreadyInMap = false;
		FHashElementId BlockId = SpatialHashMap.FindOrAddId(BlockLoc, FSpatialHash::FCellBlock{}, bAlreadyInMap);
		if (!bAlreadyInMap)
		{
			BUILDER_LOG("Allocated Block %d", BlockId.GetIndex());
		}

		return BlockId;
	}

	
	SC_FORCEINLINE FBlockLoc ToBlock(const FSceneCulling::FLocation64& Loc)
	{
		return FBlockLoc(RenderingSpatialHash::TLocation<int64>(ToLevelRelative<CellBlockDimLog2>(Loc.Coord), Loc.Level + CellBlockDimLog2));
	};

	SC_FORCEINLINE FTempCell& GetOrAddTempCell(const FSceneCulling::FLocation64& InstanceCellLoc)
	{
		// Address of the cell-block touched
		FBlockLoc BlockLoc = ToBlock(InstanceCellLoc);

		// TODO: queue fine-level work by block(?) and defer, better memory coherency, probably.
		// Possibly store compressed as we would have a fair bit of empty space & bit mask + prefix sum (can we use popc efficiently nowadays on CPU)
		// Doing that would require building the block as a two-pass process such that we know all occupied cells before allocating storage.
		// Also cheaper initialization if we don't add empty cells.
		FHashElementId BlockId = FindOrAddBlock(BlockLoc);
		FSpatialHash::FCellBlock& Block = SpatialHashMap.GetByElementId(BlockId).Value;

		if (Block.GridOffset == INDEX_NONE)
		{
			// Allocate space in the array of cell headers.
			// We keep a 1:1 mapping for now, maybe compact later? Blocks in the hash map can have holes.
			int32 StartIndex = BlockId.GetIndex() * CellBlockSize;
			Block.GridOffset = StartIndex;
			
			// Ensure enough space for the new cells
			int32 NewMinSize = FMath::Max(StartIndex + CellBlockSize, SceneCulling.CellHeaders.Num());
			SceneCulling.CellHeaders.SetNumUninitialized(NewMinSize, EAllowShrinking::No);
			// TODO: store the validity state in bit mask instead of with each item?
			for (int32 Index = 0; Index < CellBlockSize; ++Index)
			{
				SceneCulling.CellHeaders[StartIndex + Index] = InvalidPackedCell;
			}			
			// No need to set the bits as they are maintained incrementally (or new and cleared)
			SceneCulling.CellOccupancyMask.SetNum(NewMinSize, false);
			if (ExplicitBoundsMode == EExplicitBoundsUpdateMode::Incremental)
			{
				DirtyCellBoundsMaskStatic.SetNum(NewMinSize, false);
				DirtyCellBoundsMaskDynamic.SetNum(NewMinSize, false);
			}
			BUILDER_LOG("Alloc Cells [%d, %d], Block %d", StartIndex, StartIndex + CellBlockSize, BlockId.GetIndex());
		}

		const FSceneCulling::FLocation8 LocalCellLoc = ToBlockLocal(InstanceCellLoc, BlockLoc);
		const int32 CellOffset = Block.GetCellGridOffset(LocalCellLoc.Coord);
		Block.CoarseCellMask |= FSpatialHash::FCellBlock::CalcCellMask(LocalCellLoc.Coord);

		return GetOrAddTempCell(CellOffset);
	}

	SC_FORCEINLINE FTempCell& GetOrAddTempCell(int32 CellIndex)
	{
		FPackedCellHeader CellHeader = SceneCulling.CellHeaders[CellIndex];

		if (!IsTempCell(CellIndex))
		{
			// Mark the cell as a temp cell
			TempCellMask.PadToNum(SceneCulling.CellHeaders.Num(), false);
			TempCellMask[CellIndex] = true;

			int32 TempCellIndex = TempCells.AddDefaulted();
			BUILDER_LOG("Alloc Temp Cell %d / %d", CellIndex, TempCellIndex);
			// Store link back to the cell in question.
			FTempCell& TempCell = TempCells[TempCellIndex];

			TempCell.CellOffset = CellIndex;
			TempCell.PrevCellHeader = UnpackCellHeader(CellHeader);

			LogCell(TempCell.PrevCellHeader);

			// Hijack the items offset to store the index to the temp cell so we can add data there during construction.
			CellHeader.Packed0 = TempCellIndex;
			CellHeader.Packed1 = TempCellMarker;

			SceneCulling.CellHeaders[CellIndex] = CellHeader;

			return TempCell;
		}
		check(CellHeader.Packed1 == TempCellMarker);
		return TempCells[CellHeader.Packed0];
	}

	SC_FORCEINLINE int32 AddRange(const FSceneCulling::FLocation64& InstanceCellLoc, int32 InInstanceIdOffset, int32 InInstanceIdCount, EUpdateFrequencyCategory::EType UpdateFrequencyCategory)
	{
		UPDATE_BUILDER_STAT(*this, RangeCount, 1);

		FTempCell& TempCell = GetOrAddTempCell(InstanceCellLoc);

		TempCell.Builders[UpdateFrequencyCategory].AddRange(*this, InInstanceIdOffset, InInstanceIdCount);
		TempCell.bAnyChange[UpdateFrequencyCategory] = true;

#if SC_ENABLE_DETAILED_BUILDER_STATS
		if (UpdateFrequencyCategory == EUpdateFrequencyCategory::Dynamic )
		{
			SceneCulling.NumDynamicInstances += InInstanceIdCount;
		}
		else
		{
			SceneCulling.NumStaticInstances += InInstanceIdCount;
		}
#endif
		return TempCell.CellOffset;
	}

	SC_FORCEINLINE int32 AddToCell(const FSceneCulling::FLocation64& InstanceCellLoc, int32 InstanceId, EUpdateFrequencyCategory::EType UpdateFrequencyCategory)
	{
#if SC_ENABLE_DETAILED_BUILDER_STATS
		if (UpdateFrequencyCategory == EUpdateFrequencyCategory::Dynamic )
		{
			SceneCulling.NumDynamicInstances += 1;
		}
		else
		{
			SceneCulling.NumStaticInstances += 1;
		}
#endif
		FTempCell& TempCell = GetOrAddTempCell(InstanceCellLoc);
		TempCell.Builders[UpdateFrequencyCategory].Add(*this, uint32(InstanceId));
		TempCell.bAnyChange[UpdateFrequencyCategory] = true;
		return TempCell.CellOffset;
	}

	// Clamp the cell location to prevent overflows
	SC_FORCEINLINE FSceneCulling::FLocation64 ClampCellLoc(const FSceneCulling::FLocation64 &InLoc)
	{
		return FSceneCulling::FLocation64(ClampDim(InLoc.Coord, -FSceneCulling::FBlockTraits::MaxCellCoord, FSceneCulling::FBlockTraits::MaxCellCoord), InLoc.Level);
	}

	template <EUpdateFrequencyCategory::EType UpdateFrequencyCategory, typename HashLocationComputerType>
	SC_FORCEINLINE void BuildInstanceRange(int32 InstanceDataOffset, int32 NumInstances, HashLocationComputerType HashLocationComputer, FSceneCulling::FCellIndexCacheEntry &CellIndexCacheEntry)
	{
		BUILDER_LOG_LIST("BuildInstanceRange(%d, %d):", InstanceDataOffset, NumInstances);

		constexpr bool bCompressRLE = UpdateFrequencyCategory != EUpdateFrequencyCategory::Dynamic;
		CellIndexCacheEntry.bSingleInstanceOnly = !bCompressRLE;

		FSceneCulling::FLocation64 PrevInstanceCellLoc;
		int32 SameInstanceLocRunCount = 0;
		int32 StartRunInstanceInstanceId = -1;

		for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
		{
			const int32 InstanceId = InstanceDataOffset + InstanceIndex;

			FSceneCulling::FLocation64 InstanceCellLoc = ClampCellLoc(HashLocationComputer.CalcLoc(InstanceIndex));

			bool bSameLoc = bCompressRLE && SameInstanceLocRunCount > 0 && PrevInstanceCellLoc == InstanceCellLoc;

			if (bSameLoc)
			{
				++SameInstanceLocRunCount;
			}
			else
			{
				// If we have any accumulated same-cell instances, bulk-add those
				if (SameInstanceLocRunCount > 0)
				{
					int32 CellIndex = AddRange(PrevInstanceCellLoc, StartRunInstanceInstanceId, SameInstanceLocRunCount, UpdateFrequencyCategory);
					CellIndexCacheEntry.Add(CellIndex, SameInstanceLocRunCount);
					BUILDER_LOG_LIST_APPEND("(%d, Id: %d : %d)", CellIndex, StartRunInstanceInstanceId, SameInstanceLocRunCount);

				}
				// Start a new run
				PrevInstanceCellLoc = InstanceCellLoc;
				SameInstanceLocRunCount = 1;
				StartRunInstanceInstanceId = InstanceId;
			}
		}

		// Flush any outstanding ranges.
		if (SameInstanceLocRunCount > 0)
		{
			int32 CellIndex = AddRange(PrevInstanceCellLoc, StartRunInstanceInstanceId, SameInstanceLocRunCount, UpdateFrequencyCategory);
			CellIndexCacheEntry.Add(CellIndex, SameInstanceLocRunCount);
			BUILDER_LOG_LIST_APPEND("(%d, Id: %d : %d)", CellIndex, StartRunInstanceInstanceId, SameInstanceLocRunCount);
		}
	}

	SC_FORCEINLINE FHashElementId GetBlockId(const FBlockLoc& BlockLoc)
	{
		FHashElementId BlockId = SpatialHashMap.FindId(BlockLoc);
		check(BlockId.IsValid()); // TODO: checkslow?
		return BlockId;
	}

	SC_FORCEINLINE int32 GetCellIndex(const FSceneCulling::FLocation64 &CellLoc)
	{
		if (CachedCellIdIndex == INDEX_NONE || !(CachedCellLoc == CellLoc))
		{
			// Address of the cell-block touched
			FBlockLoc BlockLoc = ToBlock(CellLoc);

			FHashElementId BlockId = GetBlockId(BlockLoc);
			FSpatialHash::FCellBlock& Block = SpatialHashMap.GetByElementId(BlockId).Value;

			const FSceneCulling::FLocation8 LocalCellLoc = ToBlockLocal(CellLoc, BlockLoc);
			int32 CellIndex = Block.GetCellGridOffset(LocalCellLoc.Coord);

			CachedCellIdIndex = CellIndex;
			CachedCellLoc = CellLoc;
		}

		return CachedCellIdIndex;
	}

	SC_FORCEINLINE void FinalizeTempCellsAndUncullable()
	{
		SCOPED_NAMED_EVENT(BuildHierarchy_FinalizeGrid, FColor::Emerald);
		BUILDER_LOG_SCOPE("FinalizeTempCells: %d", TempCells.Num());

		{
			SCOPED_NAMED_EVENT(BuildHierarchy_Consolidate, FColor::Emerald);
			SceneCulling.CellChunkIdAllocator.Consolidate();
			// SceneCulling.UsedChunkIdMask.SetRange(PrevItemChunksOffset, PrevNumItemChunks, true);
		}

		CellHeaderUploader.Reserve(TempCells.Num());

		{
			SCOPED_NAMED_EVENT(BuildHierarchy_FinalizeChunks, FColor::Emerald);

		for (FTempCell& TempCell : TempCells)
		{
			TotalRemovedInstances += TempCell.GetRemovedInstanceCount();
			TempCell.FinalizeChunks(*this);
		}
		}

		{
			SCOPED_NAMED_EVENT(BuildHierarchy_FreeUncullable, FColor::Emerald);
		// Free all uncullable chunks
		for (int32 Index = 0; Index < SceneCulling.UncullableNumItemChunks; ++Index)
		{
			check(SceneCulling.UncullableItemChunksOffset != INDEX_NONE);
			uint32 PackedChunkData = SceneCulling.PackedCellChunkData[SceneCulling.UncullableItemChunksOffset + Index];
			const bool bIsCompressed = (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_FLAG) != 0u;
			if (!bIsCompressed)
			{
				uint32 ChunkId =  (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_ID_MASK);
				SceneCulling.FreeChunk(ChunkId);
			}
		}
		}

		{
			SCOPED_NAMED_EVENT(BuildHierarchy_ChunkBuilder, FColor::Emerald);

		FChunkBuilder ChunkBuilder;
		for (FPersistentPrimitiveIndex PersistentPrimitiveIndex : SceneCulling.UnCullablePrimitives)
		{
			const FSceneCulling::FPrimitiveState &PrimitiveState = SceneCulling.PrimitiveStates[PersistentPrimitiveIndex.Index];
			check(PrimitiveState.State == FPrimitiveState::UnCullable);
			if (PrimitiveState.InstanceDataOffset != INDEX_NONE && PrimitiveState.NumInstances > 0)
			{
				ChunkBuilder.AddRange(*this, PrimitiveState.InstanceDataOffset, PrimitiveState.NumInstances);
			}
		}
		ChunkBuilder.EmitCurrentChunk();
		SceneCulling.UncullableItemChunksOffset = ReallocateChunkRange(ChunkBuilder.PackedChunkIds.Num(), SceneCulling.UncullableItemChunksOffset, SceneCulling.UncullableNumItemChunks);
		SceneCulling.UncullableNumItemChunks = ChunkBuilder.PackedChunkIds.Num();

		SceneCulling.PackedCellChunkData.SetNum(SceneCulling.CellChunkIdAllocator.GetMaxSize());

		if (!ChunkBuilder.IsEmpty())
		{
			ChunkBuilder.OutputChunkIds(*this, SceneCulling.UncullableItemChunksOffset);
		}
		}
		{
			SCOPED_NAMED_EVENT(BuildHierarchy_OutputChunkIds, FColor::Emerald);

		
		DirtyBlocks.SetNum(SpatialHash.GetMaxNumBlocks(), false);
		NumDirtyBlocks = 0;
		for (FTempCell& TempCell : TempCells)
		{
			BUILDER_LOG_SCOPE("OutputChunkIds");
			FPackedCellHeader& OutPackedCellHeader = SceneCulling.CellHeaders[TempCell.CellOffset];
			check(IsTempCell(TempCell.CellOffset));
			check(OutPackedCellHeader.Packed1 == TempCellMarker);
			// mark block as dirty
			DirtyBlocks[SceneCulling.CellIndexToBlockId(TempCell.CellOffset)] = true;
			++NumDirtyBlocks;
			if (TempCell.IsEmpty())
			{
				BUILDER_LOG("Mark Empty Cell: %d", TempCell.CellOffset);
				SceneCulling.CellOccupancyMask[TempCell.CellOffset] = false;
				OutPackedCellHeader = InvalidPackedCell;
			}
			else
			{
				SceneCulling.CellOccupancyMask[TempCell.CellOffset] = true;
				// 2. Store final offsets
				FCellHeader CellHeader;
				CellHeader.bIsValid = true;
				CellHeader.ItemChunksOffset = TempCell.ItemChunksOffset;
				CellHeader.NumStaticChunks  = TempCell.GetNumItemChunks(EUpdateFrequencyCategory::Static);
				CellHeader.NumDynamicChunks = TempCell.GetNumItemChunks(EUpdateFrequencyCategory::Dynamic);
				CellHeader.NumItemChunks = CellHeader.NumDynamicChunks + CellHeader.NumStaticChunks;
				OutPackedCellHeader = PackCellHeader(CellHeader);

				// 3. Copy the list of chunk IDs
				TempCell.OutputChunkIds(*this);

				MarkCellBoundsDirty(TempCell.CellOffset, TempCell.bAnyChange[EUpdateFrequencyCategory::Static], TempCell.bAnyChange[EUpdateFrequencyCategory::Dynamic]);
			}
			LogCell(UnpackCellHeader(OutPackedCellHeader));
			CellHeaderUploader.Add(OutPackedCellHeader, TempCell.CellOffset);
			BUILDER_LOG("CellHeaderUploader { %u, %u, %u } -> %d", UnpackCellHeader(OutPackedCellHeader).ItemChunksOffset, UnpackCellHeader(OutPackedCellHeader).NumItemChunks, UnpackCellHeader(OutPackedCellHeader).NumDynamicChunks, TempCell.CellOffset);
		}
		}
		TempCells.Empty();
	}

	inline int32 AllocateCacheEntry()
	{
		int32 CacheIndex = SceneCulling.CellIndexCache.EmplaceAtLowestFreeIndex(LowestFreeCacheIndex);
		check(SceneCulling.CellIndexCache.IsValidIndex(CacheIndex));
		BUILDER_LOG("AllocateCacheEntry %d", CacheIndex);
		return CacheIndex;
	}

	inline void FreeCacheEntry(int32 CacheIndex)
	{
		BUILDER_LOG("FreeCacheEntry %d", CacheIndex);
		check(SceneCulling.CellIndexCache.IsValidIndex(CacheIndex));
		SceneCulling.TotalCellIndexCacheItems -= SceneCulling.CellIndexCache[CacheIndex].Items.Num();
		SceneCulling.CellIndexCache.RemoveAt(CacheIndex);
		LowestFreeCacheIndex = 0;
	}

	inline FCellIndexCacheEntry &GetCacheEntry(int32 CacheIndex)
	{
		check(SceneCulling.CellIndexCache.IsValidIndex(CacheIndex));
		return SceneCulling.CellIndexCache[CacheIndex];
	}

	SC_FORCEINLINE void LogCell(const FCellHeader &CellHeader)
	{
#if SC_ENABLE_DETAILED_LOGGING
		BUILDER_LOG_LIST("CellData[%d, %d]:", CellHeader.NumItemChunks, CellHeader.NumStaticChunks);
		if (IsValidCell(CellHeader))
		{
			for (uint32 ChunkOffset = CellHeader.ItemChunksOffset; ChunkOffset < CellHeader.ItemChunksOffset + CellHeader.NumItemChunks; ++ChunkOffset)
			{
				uint32 PackedChunkData = SceneCulling.PackedCellChunkData[ChunkOffset];
				const bool bIsCompressed = (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_FLAG) != 0u;
				const uint32 NumItems = bIsCompressed ? 64u : PackedChunkData >> INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT;
				const bool bIsFullChunk = NumItems == 64u;

				if (bIsCompressed)
				{
					uint32 FirstInstanceDataOffset = PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_PAYLOAD_MASK;
					BUILDER_LOG_LIST_APPEND("CMP: %d", FirstInstanceDataOffset);
				}
				else
				{
					uint32 ChunkId =  (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_ID_MASK);
					uint32 ItemDataOffset = ChunkId * INSTANCE_HIERARCHY_MAX_CHUNK_SIZE;

					BUILDER_LOG_LIST_APPEND("CNK: %d { ", ChunkId);
					for (uint32 ItemIndex = 0u; ItemIndex < NumItems; ++ItemIndex)
					{
						uint32 InstanceId = SceneCulling.PackedCellData[ItemDataOffset + ItemIndex];
						BUILDER_LOG_LIST_APPEND("Id:%d", InstanceId);
					}
					BUILDER_LOG_LIST_APPEND("}");
				}
			}
		}
#endif // SC_ENABLE_DETAILED_LOGGING
	}

	inline uint32 AllocateChunk()
	{
		uint32 ChunkId = SceneCulling.AllocateChunk();

		BUILDER_LOG("AllocateChunk (ChunkId: %d)", ChunkId);

		DirtyChunks.SetNum(FMath::Max(DirtyChunks.Num(), int32(ChunkId) + 1), false);
		// We know (as chunks are lazy allocated) that they need to be reuploaded if allocated during build, also because we double buffer (don't update in-place)
		// track chunk dirty state (could use an array and append ID instead), however, this guarantees that a reused chunk won't be uploaded twice
		DirtyChunks[ChunkId] = true;

		return ChunkId;
	}

	inline void FreeChunk(uint32 ChunkId)
	{
		BUILDER_LOG("FreeChunk (ChunkId: %d)", ChunkId);

		SceneCulling.FreeChunk(ChunkId);
	}

	SC_FORCEINLINE FPrimitiveState ComputePrimitiveState(const FPrimitiveBounds& Bounds, FPrimitiveSceneInfo* PrimitiveSceneInfo, int32 NumInstances, int32 InstanceDataOffset, FPrimitiveSceneProxy* SceneProxy, FInstanceDataFlags InstanceDataFlags, const FPrimitiveState &PrevState)
	{
		FPrimitiveState NewState;
		NewState.bDynamic = PrevState.bDynamic || SceneProxy->IsOftenMoving();
		NewState.NumInstances = NumInstances;
		NewState.InstanceDataOffset = InstanceDataOffset;

		if (SceneCulling.IsUncullable(Bounds, PrimitiveSceneInfo))
		{
			NewState.State = FPrimitiveState::UnCullable;
		}
		else if (InstanceDataOffset != INDEX_NONE)
		{
			if (NumInstances == 1 || SceneProxy->IsInstanceDataGPUOnly())
			{
				NewState.State = FPrimitiveState::SingleCell;
			}
			else
			{
				if (InstanceDataFlags.bHasCompressedSpatialHash && bUsePrecomputed)
				{
					NewState.State = FPrimitiveState::Precomputed;
				}
				else 
				{
					NewState.State = NewState.bDynamic ? DynamicInstancedPrimitiveState : FPrimitiveState::Cached;
				}		
			}
		}
		check(NewState.NumInstances > 0 || NewState.State == FPrimitiveState::Unknown || NewState.State == FPrimitiveState::UnCullable);

		return NewState;
	}


	template <EUpdateFrequencyCategory::EType UpdateFrequencyCategory>
	SC_FORCEINLINE int32 AddCachedOrDynamic(const FInstanceSceneDataBuffers *InstanceSceneDataBuffers, int32 CacheIndex, const bool bHasPerInstanceLocalBounds, int32 InstanceDataOffset, int32 NumInstances)
	{
		check(InstanceSceneDataBuffers != nullptr);

		FCellIndexCacheEntry &CellIndexCacheEntry = GetCacheEntry(CacheIndex);
		check(CellIndexCacheEntry.Items.IsEmpty());		

		if (bHasPerInstanceLocalBounds)
		{
			FHashLocationComputerFromBounds<FBoundsTransformerUniqueBounds> HashLocationComputer(*InstanceSceneDataBuffers, SpatialHash);
			BuildInstanceRange<UpdateFrequencyCategory>(InstanceDataOffset, NumInstances, HashLocationComputer, CellIndexCacheEntry);
		}
		else
		{
			FHashLocationComputerFromBounds<FBoundsTransformerSharedBounds> HashLocationComputer(*InstanceSceneDataBuffers, SpatialHash);
			BuildInstanceRange<UpdateFrequencyCategory>(InstanceDataOffset, NumInstances, HashLocationComputer, CellIndexCacheEntry);
		}

#if SC_ENABLE_DETAILED_LOGGING
		BUILDER_LOG_LIST("CellIndexCacheEntry(%d):", CellIndexCacheEntry.Items.Num());
		for (int32 ItemIndex = 0; ItemIndex < CellIndexCacheEntry.Items.Num(); ++ItemIndex)
		{
			FCellIndexCacheEntry::FItem Item = CellIndexCacheEntry.LoadAndStepItem(ItemIndex);
			BUILDER_LOG_LIST_APPEND("(%d, %d)", Item.CellIndex, Item.NumInstances);
		}
#endif

		SceneCulling.TotalCellIndexCacheItems += CellIndexCacheEntry.Items.Num();

		return CacheIndex;
	}

	// Helper to be able to capture stall time
	SC_FORCEINLINE const FInstanceSceneDataBuffers *GetInstanceSceneDataBuffers(FPrimitiveSceneInfo* PrimitiveSceneInfo)
	{
		SC_SCOPED_NAMED_EVENT_DETAIL(SceneCulling_GetInstanceSceneDataBuffers, FColor::Emerald);
		return PrimitiveSceneInfo->GetInstanceSceneDataBuffers();
	}

	template <typename HashCallbackFuncType>
	void ProcessPrecomputed(const TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem>& Hashes, int32 InstanceDataOffset, int32 MinLevel, HashCallbackFuncType HashCallbackFunc)
	{
		// This should be guaranteed by InstanceDataFlags.bHasCompressedSpatialHash never being true if there are no hashes.
		check(!Hashes.IsEmpty());
		int32 InstanceDataOffsetCurrent = InstanceDataOffset;
		int32 NumInstances = 0;
		RenderingSpatialHash::FLocation64 CurrentLoc = ClampToMinLevel(Hashes[0].Location, MinLevel);
		// Bias the the hash to the current min level and add all consecutive now-identical hashes 
		for (const FInstanceSceneDataBuffers::FCompressedSpatialHashItem& Item : Hashes)
		{
			RenderingSpatialHash::FLocation64 HML = ClampToMinLevel(Item.Location, MinLevel);

			if (HML != CurrentLoc)
			{
				HashCallbackFunc(CurrentLoc, InstanceDataOffsetCurrent, NumInstances);

				CurrentLoc = HML;
				InstanceDataOffsetCurrent += NumInstances;
				NumInstances = 0;

			}
			NumInstances += Item.NumInstances;
		}
		// Add remainder
		if (NumInstances != 0)
		{
			HashCallbackFunc(CurrentLoc, InstanceDataOffsetCurrent, NumInstances);
		}
	}

	SC_FORCEINLINE void AddPrecomputed(FPrimitiveState& NewPrimitiveState, FPersistentPrimitiveIndex PersistentPrimitiveId, int32 InstanceDataOffset, TSharedPtr<FInstanceSceneDataImmutable, ESPMode::ThreadSafe>&& InstanceSceneDataImmutable)
	{
		BUILDER_LOG_LIST("Add AddPrecomputed");
		const TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem>& Hashes = InstanceSceneDataImmutable->GetCompressedInstanceSpatialHashes();
		ProcessPrecomputed(Hashes, InstanceDataOffset, SceneCulling.SpatialHash.GetFirstLevel(), 
			[&](const RenderingSpatialHash::FLocation64& CurrentLoc, int32 InstanceDataOffsetCurrent, int32 NumInstances)
		{
			int32 CellIndex = AddRange(CurrentLoc, InstanceDataOffsetCurrent, NumInstances, EUpdateFrequencyCategory::Static);
			BUILDER_LOG_LIST_APPEND("(%d, %d, %d)", CellIndex, InstanceDataOffsetCurrent, NumInstances);
		});

		// Hang onto this so we can safely remove the thing if it changes in the future.
		NewPrimitiveState.InstanceSceneDataImmutable = MoveTemp(InstanceSceneDataImmutable);
	}

	SC_FORCEINLINE void AddInstances(FPersistentPrimitiveIndex PersistentId, FPrimitiveSceneInfo* PrimitiveSceneInfo)
	{
		FScene &Scene = SceneCulling.Scene;
		TArray<FPrimitiveState> &PrimitiveStates = SceneCulling.PrimitiveStates;
		const FPrimitiveState &PrevPrimitiveState = PrimitiveStates[PersistentId.Index];
		BUILDER_LOG_SCOPE("AddInstances: %d %s", PersistentId.Index, *PrevPrimitiveState.ToString());

		int32 PrimitiveIndex = Scene.GetPrimitiveIndex(PersistentId);
		check(PrimitiveIndex != INDEX_NONE);

		const int32 NumInstances = PrimitiveSceneInfo->GetNumInstanceSceneDataEntries();
		const int32 InstanceDataOffset = PrimitiveSceneInfo->GetInstanceSceneDataOffset();

		TotalAddedInstances += NumInstances;

		// leave in unknown state
		if (NumInstances == 0)
		{
			check(PrevPrimitiveState.State == FPrimitiveState::Unknown);
			return;
		}
		check(InstanceDataOffset >= 0);
		FPrimitiveSceneProxy* SceneProxy = PrimitiveSceneInfo->Proxy;
		const FInstanceSceneDataBuffers *InstanceSceneDataBuffers = GetInstanceSceneDataBuffers(PrimitiveSceneInfo);
		check((NumInstances == 1 && InstanceSceneDataBuffers == nullptr) || (InstanceSceneDataBuffers != nullptr && InstanceSceneDataBuffers->GetNumInstances() == NumInstances));
		FInstanceDataFlags InstanceDataFlags = InstanceSceneDataBuffers ? InstanceSceneDataBuffers->GetFlags() : FInstanceDataFlags();


		const FPrimitiveBounds& Bounds = Scene.PrimitiveBounds[PrimitiveIndex];
		FPrimitiveState NewPrimitiveState = ComputePrimitiveState(Bounds, PrimitiveSceneInfo, NumInstances, InstanceDataOffset, SceneProxy, InstanceDataFlags, PrevPrimitiveState);

		switch(NewPrimitiveState.State)
		{
			case FPrimitiveState::SingleCell:
			{
				FSceneCulling::FLocation64 PrimitiveCellLoc = SpatialHash.CalcLevelAndLocation(Bounds.BoxSphereBounds);
				int32 CellIndex = AddRange(PrimitiveCellLoc, InstanceDataOffset, NumInstances, NewPrimitiveState.bDynamic ? EUpdateFrequencyCategory::Dynamic : EUpdateFrequencyCategory::Static);
				NewPrimitiveState.Payload = CellIndex;
			}
			break;
			case FPrimitiveState::Precomputed:
			{
				check(InstanceSceneDataBuffers);
				SC_SCOPED_NAMED_EVENT_DETAIL(SceneCulling_Post_AddInstances_Precomputed, FColor::Emerald);
				TSharedPtr<FInstanceSceneDataImmutable, ESPMode::ThreadSafe> InstanceSceneDataImmutable = InstanceSceneDataBuffers->GetImmutable();
				AddPrecomputed(NewPrimitiveState, PersistentId, InstanceDataOffset, MoveTemp(InstanceSceneDataImmutable));
			}
			break;
			case FPrimitiveState::UnCullable:
			{
				SceneCulling.UnCullablePrimitives.Add(PersistentId);
			}
			break;
			case FPrimitiveState::Dynamic:
			{
				check(InstanceSceneDataBuffers);

				SC_SCOPED_NAMED_EVENT_DETAIL(SceneCulling_Post_AddInstances_Dynamic, FColor::Emerald);
				NewPrimitiveState.Payload = AddCachedOrDynamic<EUpdateFrequencyCategory::Dynamic>(InstanceSceneDataBuffers, AllocateCacheEntry(), InstanceDataFlags.bHasPerInstanceLocalBounds, InstanceDataOffset, NumInstances);
			}
			break;
			case FPrimitiveState::Cached:
			{
				check(InstanceSceneDataBuffers);

				SC_SCOPED_NAMED_EVENT_DETAIL(SceneCulling_Post_AddInstances_Cached, FColor::Emerald);
				NewPrimitiveState.Payload = AddCachedOrDynamic<EUpdateFrequencyCategory::Static>(InstanceSceneDataBuffers, AllocateCacheEntry(), InstanceDataFlags.bHasPerInstanceLocalBounds, InstanceDataOffset, NumInstances);
			}
			break;
		};

		check(NewPrimitiveState.NumInstances > 0 || NewPrimitiveState.State == FPrimitiveState::Unknown || NewPrimitiveState.State == FPrimitiveState::UnCullable);
		PrimitiveStates[PersistentId.Index] = NewPrimitiveState;
		BUILDER_LOG("PrimitiveState-end: %s", *NewPrimitiveState.ToString());
	}

	inline void MarkCellForRemove(int32 CellIndex, int32 NumInstances, EUpdateFrequencyCategory::EType UpdateFrequencyCategory)
	{
		BUILDER_LOG("MarkCellForRemove: %d, %d, %s", CellIndex, NumInstances, ToString(UpdateFrequencyCategory));
		FTempCell &TempCell = GetOrAddTempCell(CellIndex);
		// Should not mark remove for a cell that didn't have anything in it before...
		check(IsValidCell(TempCell.PrevCellHeader));
		// Track total to be removed.
		TempCell.RemovedInstanceCount[UpdateFrequencyCategory] += NumInstances;
		TempCell.bAnyChange[UpdateFrequencyCategory] = true;

#if SC_ENABLE_DETAILED_BUILDER_STATS
		if (UpdateFrequencyCategory == EUpdateFrequencyCategory::Dynamic )
		{
			SceneCulling.NumDynamicInstances -= NumInstances;
		}
		else
		{
			SceneCulling.NumStaticInstances -= NumInstances;
		}
#endif
	}

	inline void MarkForRemove(int32 CellIndex, int32 InstanceDataOffset, int32 NumInstances, EUpdateFrequencyCategory::EType UpdateFrequencyCategory)
	{
		BUILDER_LOG("MarkForRemove: %d, %d, %d, %s", CellIndex, InstanceDataOffset, NumInstances, ToString(UpdateFrequencyCategory));
		RemovedInstanceFlags.SetRange(InstanceDataOffset, NumInstances, true);
		MarkCellForRemove(CellIndex, NumInstances, UpdateFrequencyCategory);
	}

	SC_FORCEINLINE void MarkCellBoundsDirty(int32 CellIndex, bool bStaticChanged, bool bDynamicChanged)
	{
		if (ExplicitBoundsMode == EExplicitBoundsUpdateMode::Incremental)
		{
			if (!(DirtyCellBoundsMaskStatic[CellIndex] || DirtyCellBoundsMaskDynamic[CellIndex]))
			{
				DirtyCellBoundsIndices.Add(CellIndex);
			}
			DirtyCellBoundsMaskStatic[CellIndex] = bStaticChanged;
			DirtyCellBoundsMaskDynamic[CellIndex] = bDynamicChanged;
		}
	}

	SC_FORCEINLINE void MarkCellBoundsDirty(int32 CellIndex, EUpdateFrequencyCategory::EType UpdateFrequencyCategory)
	{
		MarkCellBoundsDirty(CellIndex, UpdateFrequencyCategory == EUpdateFrequencyCategory::Static, UpdateFrequencyCategory == EUpdateFrequencyCategory::Dynamic);
	}

	inline void RemovePrecomputed(FSceneCulling::FPrimitiveState& PrimitiveState, FPersistentPrimitiveIndex PersistentPrimitiveId)
	{
		BUILDER_LOG("FPrimitiveState::Precomputed");
		check(PrimitiveState.InstanceSceneDataImmutable);

		BUILDER_LOG_LIST("RemovePrecomputed");
		ProcessPrecomputed(PrimitiveState.InstanceSceneDataImmutable->GetCompressedInstanceSpatialHashes(), 0/*unused*/, SceneCulling.SpatialHash.GetFirstLevel(), 
			[&](const RenderingSpatialHash::FLocation64& CurrentLoc, int32 /*InstanceDataOffsetCurrent*/, int32 NumInstances)
		{
			int32 CellIndex = GetCellIndex(CurrentLoc);
			BUILDER_LOG_LIST_APPEND("(%d, %d)", CellIndex, NumInstances);
			MarkCellForRemove(CellIndex, NumInstances, EUpdateFrequencyCategory::Static);
		});
	}

	inline void MarkInstancesForRemoval(FPersistentPrimitiveIndex PersistentPrimitiveIndex, FPrimitiveSceneInfo *PrimitiveSceneInfo)
	{
		// Copy & Clear tracked state since it is being removed (ID may now be reused so we need to prevent state from surviving).
		FSceneCulling::FPrimitiveState PrimitiveState = MoveTemp(SceneCulling.PrimitiveStates[PersistentPrimitiveIndex.Index]);
		SceneCulling.PrimitiveStates[PersistentPrimitiveIndex.Index] = FPrimitiveState();
		BUILDER_LOG_SCOPE("MarkInstancesForRemoval: %d %s", PersistentPrimitiveIndex.Index, *PrimitiveState.ToString());
		check(PrimitiveState.NumInstances > 0 || PrimitiveState.State == FPrimitiveState::Unknown || PrimitiveState.State == FPrimitiveState::UnCullable);

		// TODO: this is not needed when we also call update for added primitives correctly, remove and replace with a check!
		if (PrimitiveState.State == FPrimitiveState::Unknown)
		{
			return;
		}

		if (PrimitiveState.State == FPrimitiveState::UnCullable)
		{
			// TODO: this could be somewhat more efficient, if there are many uncullables
			SceneCulling.UnCullablePrimitives.Remove(PersistentPrimitiveIndex);
			return;
		}

		INC_DWORD_STAT_BY(STAT_SceneCulling_RemovedInstanceCount, PrimitiveState.NumInstances);

		// 1: mark all instances that need clearing out
		// TODO: need only mark one bit for compressed chunks. Track knowledge of this?
		RemovedInstanceFlags.SetRange(PrimitiveState.InstanceDataOffset, PrimitiveState.NumInstances, true);
		BUILDER_LOG("RemovedInstanceFlags: %d, %d", PrimitiveState.InstanceDataOffset, PrimitiveState.NumInstances);

		// Handle singular case
		if (PrimitiveState.State == FPrimitiveState::SingleCell)
		{ 
			int32 CellIndex = PrimitiveState.Payload;
			MarkCellForRemove(CellIndex, PrimitiveState.NumInstances, PrimitiveState.bDynamic ? EUpdateFrequencyCategory::Dynamic : EUpdateFrequencyCategory::Static);
		}
		else if (PrimitiveState.State == FPrimitiveState::Precomputed)
		{
			RemovePrecomputed(PrimitiveState, PersistentPrimitiveIndex);
		}
		else if (PrimitiveState.State == FPrimitiveState::Cached || PrimitiveState.State == FPrimitiveState::Dynamic)
		{
			int32 CacheIndex = PrimitiveState.Payload;
			const FCellIndexCacheEntry &CellIndexCacheEntry = GetCacheEntry(CacheIndex);

			BUILDER_LOG_LIST("MarkForRemove(%d):", CellIndexCacheEntry.Items.Num());
			for (int32 ItemIndex = 0; ItemIndex < CellIndexCacheEntry.Items.Num(); ++ItemIndex)
			{
				FCellIndexCacheEntry::FItem Item = CellIndexCacheEntry.LoadAndStepItem(ItemIndex);
				BUILDER_LOG_LIST_APPEND("(%d, %d)", Item.CellIndex, Item.NumInstances);
				MarkCellForRemove(Item.CellIndex, Item.NumInstances, (PrimitiveState.State == FPrimitiveState::Dynamic) ? EUpdateFrequencyCategory::Dynamic : EUpdateFrequencyCategory::Static);
			}

			FreeCacheEntry(CacheIndex);
		}
		else
		{
			check(false);
		}
		BUILDER_LOG("PrimitiveState-end: %s", *PrimitiveState.ToString());
	}

	template <typename HashLocationComputerType>
	SC_FORCEINLINE void UpdateProcessDynamicInstances(HashLocationComputerType &HashLocationComputer, int32 InstanceDataOffset, int32 NumInstances, int32 PrevNumInstances, FSceneCulling::FCellIndexCacheEntry &CellIndexCacheEntry)
	{
		for (int32 InstanceIndex = 0; InstanceIndex < NumInstances; ++InstanceIndex)
		{
			const int32 InstanceId = InstanceDataOffset + InstanceIndex;
			FSceneCulling::FLocation64 InstanceCellLoc = ClampCellLoc(HashLocationComputer.CalcLoc(InstanceIndex));

			bool bNeedAdd = InstanceIndex >= PrevNumInstances;
			if (!bNeedAdd)
			{
				int32 PrevCellIndex =  CellIndexCacheEntry.LoadAndStepItem(InstanceIndex).CellIndex;
				FSceneCulling::FLocation64 PrevCellLoc = SceneCulling.GetCellLoc(PrevCellIndex);
				if (PrevCellLoc != InstanceCellLoc)
				{
					MarkForRemove(PrevCellIndex, InstanceId, 1, EUpdateFrequencyCategory::Dynamic);
					bNeedAdd = true;
				}
				else
				{
					MarkCellBoundsDirty(PrevCellIndex, EUpdateFrequencyCategory::Dynamic);
				}
			}
			if (bNeedAdd)
			{
				int32 CellIndex = AddToCell(InstanceCellLoc, InstanceId, EUpdateFrequencyCategory::Dynamic);
				CellIndexCacheEntry.Set(InstanceIndex, CellIndex);
			}
		}
	}

	// Mark those that need for remove, queue others for add

	SC_FORCEINLINE void UpdateInstances(FPersistentPrimitiveIndex PersistentPrimitiveIndex, FPrimitiveSceneInfo* PrimitiveSceneInfo)
	{
		FSceneCulling::FPrimitiveState PrevPrimitiveState = SceneCulling.PrimitiveStates[PersistentPrimitiveIndex.Index];
		BUILDER_LOG_SCOPE("UpdateInstances: %d [%s]", PersistentPrimitiveIndex.Index, *PrevPrimitiveState.ToString());
		check(PrevPrimitiveState.NumInstances > 0 || PrevPrimitiveState.State == FPrimitiveState::Unknown || PrevPrimitiveState.State == FPrimitiveState::UnCullable);

		const int32 InstanceDataOffset = PrimitiveSceneInfo->GetInstanceSceneDataOffset();
		const int32 NumInstances = PrimitiveSceneInfo->GetNumInstanceSceneDataEntries();

		TotalUpdatedInstances += NumInstances;
	 
		// Just leave on the list
		if (PrevPrimitiveState.State == FPrimitiveState::UnCullable)
		{
			PrevPrimitiveState.InstanceDataOffset = InstanceDataOffset;
			PrevPrimitiveState.NumInstances = NumInstances;
			SceneCulling.PrimitiveStates[PersistentPrimitiveIndex.Index] = PrevPrimitiveState;
			return;
		}

		int32 PrimitiveIndex = PrimitiveSceneInfo->GetIndex();
		const FPrimitiveBounds& Bounds = SceneCulling.Scene.PrimitiveBounds[PrimitiveIndex];
		FPrimitiveSceneProxy* SceneProxy = PrimitiveSceneInfo->Proxy;
		const FInstanceSceneDataBuffers *InstanceSceneDataBuffers = GetInstanceSceneDataBuffers(PrimitiveSceneInfo);
		check((NumInstances == 1 && InstanceSceneDataBuffers == nullptr) || (InstanceSceneDataBuffers != nullptr && InstanceSceneDataBuffers->GetNumInstances() == NumInstances));
		const FInstanceDataFlags InstanceDataFlags = InstanceSceneDataBuffers ? InstanceSceneDataBuffers->GetFlags() : FInstanceDataFlags();

		FPrimitiveState NewPrimitiveState = ComputePrimitiveState(Bounds, PrimitiveSceneInfo, NumInstances, InstanceDataOffset, SceneProxy, InstanceDataFlags, PrevPrimitiveState);
		const bool bStateChanged = NewPrimitiveState.State != PrevPrimitiveState.State;
		const bool bInstanceDataOffsetChanged = PrevPrimitiveState.InstanceDataOffset != NewPrimitiveState.InstanceDataOffset;
		const bool bInstanceCountChanged = PrevPrimitiveState.NumInstances != NewPrimitiveState.NumInstances;

		const FMatrix& PrimitiveToWorld = SceneCulling.Scene.PrimitiveTransforms[PrimitiveIndex];

		INC_DWORD_STAT_BY(STAT_SceneCulling_UpdatedInstanceCount, NumInstances);

		// If it was previously unknown it must now be added
		bool bNeedsAdd = PrevPrimitiveState.State == FPrimitiveState::Unknown && bStateChanged;

		// Handle singular case
		if (PrevPrimitiveState.State == FPrimitiveState::SingleCell)
		{ 			
			SC_SCOPED_NAMED_EVENT_DETAIL(SceneCulling_Post_UpdateInstances_SingleCell, FColor::Blue);

			int32 PrevCellIndex = PrevPrimitiveState.Payload;
			// If it is changed away from this state, just mark for remove and flag for re-add
			if (bStateChanged)
			{
				MarkForRemove(PrevCellIndex, PrevPrimitiveState.InstanceDataOffset, PrevPrimitiveState.NumInstances, PrevPrimitiveState.bDynamic ? EUpdateFrequencyCategory::Dynamic : EUpdateFrequencyCategory::Static);
				bNeedsAdd = true;
			}
			else
			{
				// Otherwise compute new cell location
				FSpatialHash::FLocation64 PrimitiveCellLoc = SpatialHash.CalcLevelAndLocation(Bounds.BoxSphereBounds);
				FSpatialHash::FLocation64 PrevCellLoc = SceneCulling.GetCellLoc(PrevCellIndex);

				// It is different, so need to remove/add
				if (bInstanceCountChanged || bInstanceDataOffsetChanged || PrevPrimitiveState.bDynamic != NewPrimitiveState.bDynamic || PrevCellLoc != PrimitiveCellLoc)
				{
					// mark for removal
					MarkForRemove(PrevCellIndex, PrevPrimitiveState.InstanceDataOffset, PrevPrimitiveState.NumInstances, PrevPrimitiveState.bDynamic ? EUpdateFrequencyCategory::Dynamic : EUpdateFrequencyCategory::Static);

					// It is still a single-instance prim (it might be an ISM which varies) re-add at once, since we have already computed the new PrimitiveCellLoc
					int32 CellIndex = AddRange(PrimitiveCellLoc, NewPrimitiveState.InstanceDataOffset, NewPrimitiveState.NumInstances, NewPrimitiveState.bDynamic ? EUpdateFrequencyCategory::Dynamic : EUpdateFrequencyCategory::Static);
					NewPrimitiveState.Payload = CellIndex;
				}
				else
				{
					// retain previous state.
					NewPrimitiveState.Payload = PrevCellIndex;
					MarkCellBoundsDirty(PrevCellIndex, PrevPrimitiveState.bDynamic ? EUpdateFrequencyCategory::Dynamic : EUpdateFrequencyCategory::Static);
				}
			}
		}
		else if (NewPrimitiveState.State == FPrimitiveState::Dynamic && !bStateChanged && !bInstanceDataOffsetChanged)
		{
			SC_SCOPED_NAMED_EVENT_DETAIL(SceneCulling_Post_UpdateInstances_DynamicUpdate, FColor::Red);
			check(InstanceSceneDataBuffers != nullptr);
			check(PrevPrimitiveState.bDynamic);

			// For dynamic instance batches we process individual instances since they can then more often be retained
			// Stored in the same data structure, just guaranteed to be singular instances
			FCellIndexCacheEntry &CellIndexCacheEntry = GetCacheEntry(PrevPrimitiveState.Payload);
			// retain previous state.
			NewPrimitiveState.Payload = PrevPrimitiveState.Payload;
			check(CellIndexCacheEntry.bSingleInstanceOnly);
			// Mark overflowing ones for remove (if the number of instances shrank)
			for (int32 ItemIndex = NumInstances; ItemIndex < CellIndexCacheEntry.Items.Num(); ++ItemIndex)
			{
				FCellIndexCacheEntry::FItem Item = CellIndexCacheEntry.LoadAndStepItem(ItemIndex);
				check(Item.NumInstances == 1);
				// Assumes 1:1 between index and ID
				MarkForRemove(Item.CellIndex, PrevPrimitiveState.InstanceDataOffset + ItemIndex, 1, EUpdateFrequencyCategory::Dynamic);
			}
			// Maintain the total accross all entries
			SceneCulling.TotalCellIndexCacheItems -= CellIndexCacheEntry.Items.Num();
			// Resize the cache entry to fit new IDs or trim excess ones.
			CellIndexCacheEntry.Items.SetNumZeroed(NumInstances, EAllowShrinking::No);
			SceneCulling.TotalCellIndexCacheItems += CellIndexCacheEntry.Items.Num();
			if (InstanceDataFlags.bHasPerInstanceLocalBounds)
			{
				FHashLocationComputerFromBounds<FBoundsTransformerUniqueBounds> HashLocationComputer(*InstanceSceneDataBuffers, SpatialHash);
				UpdateProcessDynamicInstances(HashLocationComputer, InstanceDataOffset, NumInstances, PrevPrimitiveState.NumInstances, CellIndexCacheEntry);
			}
			else
			{
				FHashLocationComputerFromBounds<FBoundsTransformerSharedBounds> HashLocationComputer(*InstanceSceneDataBuffers, SpatialHash);
				UpdateProcessDynamicInstances(HashLocationComputer, InstanceDataOffset, NumInstances, PrevPrimitiveState.NumInstances, CellIndexCacheEntry);
			}
		}
		else if (PrevPrimitiveState.State == FPrimitiveState::Precomputed)
		{
			SC_SCOPED_NAMED_EVENT_DETAIL(SceneCulling_Post_UpdateInstances_Precomputed, FColor::Emerald);
			BUILDER_LOG("FPrimitiveState::Precomputed");

			RemovedInstanceFlags.SetRange(PrevPrimitiveState.InstanceDataOffset, PrevPrimitiveState.NumInstances, true);
			BUILDER_LOG("RemovedInstanceFlags: %d, %d", PrevPrimitiveState.InstanceDataOffset, PrevPrimitiveState.NumInstances);

			RemovePrecomputed(PrevPrimitiveState, PersistentPrimitiveIndex);

			bNeedsAdd = true;
		}
		// In all other cases we have something that must be removed and is in either Cached or Dynamic state which are both removed in the same way.
		else if (PrevPrimitiveState.State != FPrimitiveState::Unknown)
		{
			SC_SCOPED_NAMED_EVENT_DETAIL_TCHAR(PrevPrimitiveState.State == FPrimitiveState::Cached ? TEXT("SceneCulling_Post_UpdateInstances_ReAddCached") : (InstanceDataOffset != PrevPrimitiveState.InstanceDataOffset ? TEXT("SceneCulling_Post_UpdateInstances_Dynamic_Offset") : TEXT("SceneCulling_Post_UpdateInstances_Dynamic_Other")), FColor::Emerald);

			check(PrevPrimitiveState.State == FPrimitiveState::Cached || (PrevPrimitiveState.State == FPrimitiveState::Dynamic && (InstanceDataOffset != PrevPrimitiveState.InstanceDataOffset || bStateChanged)));
			FCellIndexCacheEntry &CellIndexCacheEntry = GetCacheEntry(PrevPrimitiveState.Payload);

			RemovedInstanceFlags.SetRange(PrevPrimitiveState.InstanceDataOffset, PrevPrimitiveState.NumInstances, true);
			BUILDER_LOG("RemovedInstanceFlags: %d, %d", PrevPrimitiveState.InstanceDataOffset, PrevPrimitiveState.NumInstances);
			for (int32 ItemIndex = 0; ItemIndex < CellIndexCacheEntry.Items.Num(); ++ItemIndex)
			{
				FCellIndexCacheEntry::FItem Item = CellIndexCacheEntry.LoadAndStepItem(ItemIndex);
				MarkCellForRemove(Item.CellIndex, Item.NumInstances, PrevPrimitiveState.bDynamic ? EUpdateFrequencyCategory::Dynamic : EUpdateFrequencyCategory::Static);
			}
			
			// Reset the cache entry
			SceneCulling.TotalCellIndexCacheItems -= CellIndexCacheEntry.Items.Num();

			// If the primitive stays cached, we just hang on to the cache entry
			if (NewPrimitiveState.IsCachedState())
			{
				CellIndexCacheEntry.Reset();
			}
			else
			{
				// clean up cache entry if we're transitioning away from cached state
				FreeCacheEntry(PrevPrimitiveState.Payload);
			}

			bNeedsAdd = true;
		}
		
		// TODO: refactor to share more code with AddInstances
		if (bNeedsAdd)
		{
			SC_SCOPED_NAMED_EVENT_DETAIL(SceneCulling_Post_UpdateInstances_ReAdd, FColor::Emerald);

			switch(NewPrimitiveState.State)
			{
				case FPrimitiveState::Unknown:
				break;
				case FPrimitiveState::SingleCell:
				{
					FSceneCulling::FLocation64 PrimitiveCellLoc = SpatialHash.CalcLevelAndLocation(Bounds.BoxSphereBounds);
					int32 CellIndex = AddRange(PrimitiveCellLoc, InstanceDataOffset, NumInstances, NewPrimitiveState.bDynamic ? EUpdateFrequencyCategory::Dynamic : EUpdateFrequencyCategory::Static);
					NewPrimitiveState.Payload = CellIndex;
				}
				break;
				case FPrimitiveState::Precomputed:
				{
					check(InstanceSceneDataBuffers);
					SC_SCOPED_NAMED_EVENT_DETAIL(SceneCulling_Post_AddInstances_Precomputed, FColor::Emerald);
					TSharedPtr<FInstanceSceneDataImmutable, ESPMode::ThreadSafe> InstanceSceneDataImmutable = InstanceSceneDataBuffers->GetImmutable();
					AddPrecomputed(NewPrimitiveState, PersistentPrimitiveIndex, InstanceDataOffset, MoveTemp(InstanceSceneDataImmutable));
				}
				break;
				case FPrimitiveState::UnCullable:
				{
					SceneCulling.UnCullablePrimitives.Add(PersistentPrimitiveIndex);
				}
				break;
				case FPrimitiveState::Dynamic:
				{
					check(NewPrimitiveState.IsCachedState());
					NewPrimitiveState.Payload = AddCachedOrDynamic<EUpdateFrequencyCategory::Dynamic>(InstanceSceneDataBuffers, PrevPrimitiveState.IsCachedState() ? PrevPrimitiveState.Payload : AllocateCacheEntry(), InstanceDataFlags.bHasPerInstanceLocalBounds, InstanceDataOffset, NumInstances);
				}
				break;
				case FPrimitiveState::Cached:
				{
					check(NewPrimitiveState.IsCachedState());
					NewPrimitiveState.Payload = AddCachedOrDynamic<EUpdateFrequencyCategory::Static>(InstanceSceneDataBuffers, PrevPrimitiveState.IsCachedState() ? PrevPrimitiveState.Payload : AllocateCacheEntry(), InstanceDataFlags.bHasPerInstanceLocalBounds, InstanceDataOffset, NumInstances);
				}
				break;
			};
		}

		check(NewPrimitiveState.NumInstances > 0 || NewPrimitiveState.State == FPrimitiveState::Unknown || NewPrimitiveState.State == FPrimitiveState::UnCullable);

		BUILDER_LOG("PrimitiveState-end: %s", *NewPrimitiveState.ToString());
		// Update tracked state
		SceneCulling.PrimitiveStates[PersistentPrimitiveIndex.Index] = MoveTemp(NewPrimitiveState);
	}

	inline bool IsMarkedForRemove(uint32 InstanceId)
	{
		return RemovedInstanceFlags[InstanceId];
	}

	void PublishStats()
	{
#if SC_ENABLE_DETAILED_BUILDER_STATS
		INC_DWORD_STAT_BY(STAT_SceneCulling_RangeCount, Stats.RangeCount);
		INC_DWORD_STAT_BY(STAT_SceneCulling_CompRangeCount, Stats.CompRangeCount);
		INC_DWORD_STAT_BY(STAT_SceneCulling_CopiedIdCount, Stats.CopiedIdCount);
		INC_DWORD_STAT_BY(STAT_SceneCulling_VisitedIdCount, Stats.VisitedIdCount);
		Stats = FStats();
#endif 
		CSV_CUSTOM_STAT(SceneCulling, NumUpdatedInstances,  TotalUpdatedInstances, ECsvCustomStatOp::Accumulate);
		CSV_CUSTOM_STAT(SceneCulling, NumAddedInstances,  TotalAddedInstances, ECsvCustomStatOp::Accumulate);
		CSV_CUSTOM_STAT(SceneCulling, NumRemovedInstances,  TotalRemovedInstances, ECsvCustomStatOp::Accumulate);

		SceneCulling.PublishStats();
	}

	void UploadToGPU(FRDGBuilder& GraphBuilder)
	{
		// Early out if for some reason the platform has changed from underneath us.
		if (!UseSceneCulling(SceneCulling.Scene.GetShaderPlatform()))
		{
			return;
		}

		BUILDER_LOG("UploadToGPU %d", ItemChunkUploader.GetNumScatters());

		bool bValidToCapture = CellHeaderUploader.GetNumScatters() > 0;
#if !UE_BUILD_SHIPPING
		RenderCaptureInterface::FScopedCapture RenderCapture(bValidToCapture && GCaptureNextSceneCullingUpdate-- == 0, GraphBuilder, TEXT("SceneCulling.UploadToGPU"));
		// Prevent overflow every 2B frames.
		GCaptureNextSceneCullingUpdate = FMath::Max(-1, GCaptureNextSceneCullingUpdate);
#endif

		INC_DWORD_STAT_BY(STAT_SceneCulling_UploadedChunks, ItemChunkUploader.GetNumScatters())
		INC_DWORD_STAT_BY(STAT_SceneCulling_UploadedCells, CellHeaderUploader.GetNumScatters());
		INC_DWORD_STAT_BY(STAT_SceneCulling_UploadedItems, ItemChunkDataUploader.GetNumScatters());
		INC_DWORD_STAT_BY(STAT_SceneCulling_UploadedBlocks, BlockDataUploader.GetNumScatters());

		RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());

		//TODO: capture and return the (returned) registered buffers, probably need to do that elsewhere anyway?
		FRDGBuffer* CellBlockDataRDG = BlockDataUploader.ResizeAndUploadTo(GraphBuilder, SceneCulling.CellBlockDataBuffer, SceneCulling.CellBlockData.Num());
		FBufferScatterUploader::FScatterInfo UpdatedCellScatterInfo;
		FRDGBuffer* CellHeadersRDG = CellHeaderUploader.ResizeAndUploadTo(GraphBuilder, SceneCulling.CellHeadersBuffer, SceneCulling.CellHeaders.Num(), UpdatedCellScatterInfo);
		FRDGBuffer* InstanceIdsRDG = ItemChunkDataUploader.ResizeAndUploadTo(GraphBuilder, SceneCulling.InstanceIdsBuffer, SceneCulling.PackedCellData.Num());
		FRDGBuffer* ItemChunksRDG = ItemChunkUploader.ResizeAndUploadTo(GraphBuilder, SceneCulling.ItemChunksBuffer, SceneCulling.PackedCellChunkData.Num());
		
		// TODO: does this ever hit a case where it is zero?
		FRDGBufferRef UsedChunkIdMaskRDG = CreateStructuredBuffer(
			GraphBuilder,
			TEXT("SceneCulling.UsedChunkIdMask"),
			sizeof(uint32),
			FMath::Max(1, FMath::DivideAndRoundUp(SceneCulling.UsedChunkIdMask.Num(), 32)),
			SceneCulling.UsedChunkIdMask.GetData(),
			FMath::DivideAndRoundUp(SceneCulling.UsedChunkIdMask.Num(), 8)	// Size in bytes of initial data
		);
		SceneCulling.UsedChunkIdMaskBuffer = GraphBuilder.ConvertToExternalBuffer(UsedChunkIdMaskRDG);

		{
			RDG_EVENT_SCOPE(GraphBuilder, "SceneCulling_ComputeExplicitChunkBounds");
			check(ExplicitBoundsMode != EExplicitBoundsUpdateMode::Disabled);

			FRDGBuffer* ExplicitChunkBoundsBufferRDG = nullptr;
			{
				ExplicitChunkBoundsBufferRDG = SceneCulling.ExplicitChunkBoundsBuffer.ResizeBufferIfNeeded(GraphBuilder, SceneCulling.CellChunkIdAllocator.GetMaxSize());
				FRDGBuffer* ExplicitChunkCellIdsRDG = SceneCulling.ExplicitChunkCellIdsBuffer.ResizeBufferIfNeeded(GraphBuilder, SceneCulling.CellChunkIdAllocator.GetMaxSize());

				if (!DirtyChunkBoundsData.IsEmpty())
				{
					FComputeExplicitChunkBounds_CS::FParameters* PassParameters = GraphBuilder.AllocParameters<FComputeExplicitChunkBounds_CS::FParameters>();
					PassParameters->GPUSceneParameters = SceneCulling.Scene.GPUScene.GetShaderParameters(GraphBuilder);
					PassParameters->NumCellsPerBlockLog2 = FSpatialHash::NumCellsPerBlockLog2;
					PassParameters->CellBlockDimLog2 = FSpatialHash::CellBlockDimLog2;
					PassParameters->LocalCellCoordMask = (1U << FSpatialHash::CellBlockDimLog2) - 1U;
					PassParameters->FirstLevel = SceneCulling.SpatialHash.GetFirstLevel();
					PassParameters->InstanceHierarchyCellBlockData = GraphBuilder.CreateSRV(CellBlockDataRDG);
					PassParameters->InstanceHierarchyCellHeaders = GraphBuilder.CreateSRV(CellHeadersRDG);
					PassParameters->InstanceIds = GraphBuilder.CreateSRV(InstanceIdsRDG);
					PassParameters->InstanceHierarchyItemChunks = GraphBuilder.CreateSRV(ItemChunksRDG);
					int32 DirtyChunkCount = DirtyChunkBoundsData.Num();
					// Note: this Moves the DirtyCellBoundsIndices & thus implicitly clears it!
					PassParameters->DirtyChunkBoundsData = GraphBuilder.CreateSRV(CreateStructuredBuffer(GraphBuilder, TEXT("SceneCulling.DirtyChunkBoundsData"), MoveTemp(DirtyChunkBoundsData)));
					PassParameters->DirtyChunkCount = DirtyChunkCount;
					PassParameters->OutExplicitChunkBoundsBuffer = GraphBuilder.CreateUAV(ExplicitChunkBoundsBufferRDG);
					PassParameters->OutExplicitChunkCellIdsBuffer = GraphBuilder.CreateUAV(ExplicitChunkCellIdsRDG);

					auto ComputeShader = GetGlobalShaderMap(SceneCulling.Scene.GetShaderPlatform())->GetShader<FComputeExplicitChunkBounds_CS>();

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("ComputeExplicitChunkBounds"),
						ComputeShader,
						PassParameters,
						FComputeShaderUtils::GetGroupCountWrapped(DirtyChunkCount)
					);
				}
			}
		}

#if SC_ENABLE_GPU_DATA_VALIDATION
		if (CVarValidateGPUData.GetValueOnRenderThread() != 0)
		{
			SceneCulling.CellBlockDataBuffer.ValidateGPUData(GraphBuilder, TConstArrayView<FCellBlockData>(SceneCulling.CellBlockData), 
				[this](int32 Index, const FCellBlockData& HostValue, const FCellBlockData &GPUValue) 
				{
					check(GPUValue.LevelCellSize == HostValue.LevelCellSize); 
					check(GPUValue.WorldPos.GetVector3d() == HostValue.WorldPos.GetVector3d());
					check(GPUValue.Pad == HostValue.Pad); 
					check(GPUValue.Pad == 0xDeafBead); 				
				});

			TBitArray<> FreeChunkBits(false, SceneCulling.PackedCellData.Num() / INSTANCE_HIERARCHY_MAX_CHUNK_SIZE);
			for (int32 ChunkId : SceneCulling.FreeChunks)
			{
				FreeChunkBits[ChunkId] = true;
			}
			SceneCulling.InstanceIdsBuffer.ValidateGPUData(GraphBuilder, TConstArrayView<const uint32>(SceneCulling.PackedCellData), 
				[this, FreeChunkBits = MoveTemp(FreeChunkBits)](int32 Index, int32 HostValue, int32 GPUValue) 
				{
					if (!ensure(GPUValue == HostValue || FreeChunkBits[Index / INSTANCE_HIERARCHY_MAX_CHUNK_SIZE]))
					{
						UE_LOGF(LogTemp, Error, "PackedCellData[%d] != InstanceIdsBuffer[%d], %d, %d", Index, Index, HostValue, GPUValue);
					}

				});
			SceneCulling.CellHeadersBuffer.ValidateGPUData(GraphBuilder, TConstArrayView<FPackedCellHeader>(SceneCulling.CellHeaders), 
				[this](int32 Index, const FPackedCellHeader &HostValue, const FPackedCellHeader &GPUValue) 
				{
					// We don't upload unreferenced cells, so they can be just garbage on the GPU.
					// In the future (if we start doing GPU-side traversal that might need to change).
					if (IsValidCell(HostValue))
					{
						check(GPUValue.Packed0 == HostValue.Packed0); 
						check(GPUValue.Packed1 == HostValue.Packed1); 
					}
				});
			SceneCulling.ItemChunksBuffer.ValidateGPUData(GraphBuilder, TConstArrayView<const uint32>(SceneCulling.PackedCellChunkData), 
				[this](int32 Index, int32 HostValue, int32 GPUValue) 
				{
					check(GPUValue == HostValue); 
				});
		}
#endif
	}

	void ProcessPreSceneUpdate(const FScenePreUpdateChangeSet& ScenePreUpdateData)
	{
		{
			// trigger full upload if either feature has been turned on
			bool bFullUpload = SceneCulling.bForceFullExplictBoundsBuild
				|| !SceneCulling.ExplicitChunkBoundsBuffer.GetPooledBuffer().IsValid();

			ExplicitBoundsMode = bFullUpload ? EExplicitBoundsUpdateMode::Full : EExplicitBoundsUpdateMode::Incremental;
			
			// No need for dirty tracking if we are going to upload all of them
			if (ExplicitBoundsMode == EExplicitBoundsUpdateMode::Incremental)
			{
				DirtyCellBoundsMaskStatic.Init(false, SceneCulling.CellHeaders.Num());
				DirtyCellBoundsMaskDynamic.Init(false, SceneCulling.CellHeaders.Num());
			}
			// But we're still going to populate the index list
			DirtyCellBoundsIndices.Reset(SceneCulling.CellHeaders.Num());
			DirtyChunkBoundsData.Reset(SceneCulling.CellChunkIdAllocator.GetMaxSize());
		}

		for (int32 Index = 0; Index < ScenePreUpdateData.RemovedPrimitiveIds.Num(); ++Index)
		{
			MarkInstancesForRemoval(ScenePreUpdateData.RemovedPrimitiveIds[Index], ScenePreUpdateData.RemovedPrimitiveSceneInfos[Index]);
		}
	}

	void ProcessPostSceneUpdate(const FScenePostUpdateChangeSet& ScenePostUpdateData)
	{
		LLM_SCOPE_BYTAG(SceneCulling);
		SCOPED_NAMED_EVENT(SceneCulling_Update_Post, FColor::Emerald);
		SCOPE_CYCLE_COUNTER(STAT_SceneCulling_Update_Post);
		CSV_SCOPED_TIMING_STAT(SceneCulling, PostSceneUpdate);

		BUILDER_LOG_SCOPE("ProcessPostSceneUpdate: %d/%d", ScenePostUpdateData.PrimitiveUpdates.NumCommands(), ScenePostUpdateData.AddedPrimitiveIds.Num());

		SceneCulling.PrimitiveStates.SetNum(SceneCulling.Scene.GetMaxPersistentPrimitiveIndex(), EAllowShrinking::No);

		{
			SCOPED_NAMED_EVENT(SceneCulling_Post_UpdateInstances, FColor::Emerald);

			ScenePostUpdateData.PrimitiveUpdates.ForEachUpdateCommand(ESceneUpdateCommandFilter::AddedUpdated, EPrimitiveUpdateDirtyFlags::AllCulling, [&](const FPrimitiveUpdateCommand& Cmd)
			{
				if (Cmd.IsAdd())
				{
					AddInstances(Cmd.GetPersistentId(), Cmd.GetSceneInfo());
				}
				else
				{
					UpdateInstances(Cmd.GetPersistentId(), Cmd.GetSceneInfo());
				}
			});
		}
		FinalizeTempCellsAndUncullable();

		// TODO: count them while marking instead, perhaps.
		ItemChunkDataUploader.Reserve(DirtyChunks.CountSetBits());
		for (TConstSetBitIterator<SceneRenderingAllocator> BitIt(DirtyChunks); BitIt; ++BitIt)
		{
			uint32 ChunkId = BitIt.GetIndex();
			// NOTE: for the chunks we might allocate all new from a common block of memory and use an indirection on CPU to access them.
			//       Then the upload would be able to just reference that chunk.
			ItemChunkDataUploader.Add(TConstArrayView<uint32>(&SceneCulling.PackedCellData[ChunkId * INSTANCE_HIERARCHY_MAX_CHUNK_SIZE], INSTANCE_HIERARCHY_MAX_CHUNK_SIZE), ChunkId);
			BUILDER_LOG("ItemChunkDataUploader %u -> %d", ChunkId, ChunkId);
		}

		BlockDataUploader.Reserve(NumDirtyBlocks);
		SceneCulling.CellBlockData.SetNum(SpatialHashMap.GetMaxIndex());
		for (TConstSetBitIterator<SceneRenderingAllocator> BitIt(DirtyBlocks); BitIt; ++BitIt)
		{
			int32 BlockIndex = BitIt.GetIndex();
			const auto& BlockItem = SpatialHashMap.GetByElementId(BlockIndex);

			const FSpatialHash::FCellBlock& Block = BlockItem.Value;

			FCellBlockData BlockData;
			BlockData.Pad = 0xDeafBead;
			// Remove if empty
			if (Block.NumItemChunks == 0)
			{
#if DO_CHECK
				int32 BitRangeEnd = Block.GridOffset + FSpatialHash::CellBlockSize;

				// make sure we leave the data in a valid state
				for (int32 CellIndex = Block.GridOffset; CellIndex < Block.GridOffset + FSpatialHash::CellBlockSize; ++CellIndex)
				{
					check(!SceneCulling.CellOccupancyMask[CellIndex]);
					check(!IsValidCell(SceneCulling.CellHeaders[CellIndex]));
				}
#endif
				SpatialHashMap.RemoveByElementId(BlockIndex);
				BlockData.WorldPos = FDFVector3{};
				BlockData.LevelCellSize = 0.0f;
			}
			else
			{
				FSceneCulling::FBlockLoc BlockLoc = BlockItem.Key;
				FVector3d BlockWorldPos = SpatialHash.CalcBlockWorldPosition(BlockLoc);

				BlockData.WorldPos = FDFVector3{ BlockWorldPos };
				BlockData.LevelCellSize = SpatialHash.GetCellSize(BlockLoc.GetLevel() - FSpatialHash::CellBlockDimLog2);
			}
			// Keep host/GPU in sync (mostly for validation/debugging purposes)
			if (SceneCulling.CellBlockData.IsValidIndex(BlockIndex))
			{
				SceneCulling.CellBlockData[BlockIndex] = BlockData;
				BlockDataUploader.Add(BlockData, BlockIndex);
				BUILDER_LOG("BlockDataUploader {%0.0f, %0.0f, %0.0f, %0.0f} -> %d", BlockData.WorldPos.GetVector3d().X, BlockData.WorldPos.GetVector3d().Y, BlockData.WorldPos.GetVector3d().Z, BlockData.LevelCellSize, BlockIndex);
			}
			else
			{
				BUILDER_LOG("Invalid block index no upload or update: %d", BlockIndex);
			}
		}

		// Mark used levels to be able to skip empty ones when querying. Could be skipped if no blocks were added/removed.
		SceneCulling.BlockLevelOccupancyMask.Init(false, FSpatialHash::kMaxLevel);
		for (const auto &Item : SpatialHashMap)
		{
			SceneCulling.BlockLevelOccupancyMask[Item.Key.GetLevel()] = true;
		}
		
		// Note: this can be put into an own task as it can be waited on by the data upload.
		// If we're in full update mode, produce an array with all valid cell headers
		if (ExplicitBoundsMode == EExplicitBoundsUpdateMode::Full)
		{
			check(DirtyCellBoundsIndices.IsEmpty());
			for (int32 Index = 0; Index < SceneCulling.CellHeaders.Num(); ++Index)
			{
				if (IsValidCell(SceneCulling.CellHeaders[Index]))
				{
					DirtyCellBoundsIndices.Add(Index);
				}
			}
		}

		// expand into chunk indexes for the prospective cells to load balance better,
		// we could do this expansion on the GPU instead to save some CPU time (but waste some GPU time).
		if (!DirtyCellBoundsIndices.IsEmpty())
		{
			for (int32& CellIndex : DirtyCellBoundsIndices)
			{
				bool bStaticDirty = ExplicitBoundsMode == EExplicitBoundsUpdateMode::Full || DirtyCellBoundsMaskStatic[CellIndex];

				FCellHeader CellHeader = UnpackCellHeader(SceneCulling.CellHeaders[CellIndex]);
				check(IsValidCell(CellHeader));
				uint32 StartIndex = 0u;

				// If no static data was touched we don't update those chunks and instead reuse the already stored data in the cell.
				if (!bStaticDirty)
				{
					StartIndex += CellHeader.NumStaticChunks;
				}

				for (uint32 ChunkIndex = StartIndex; ChunkIndex < CellHeader.NumItemChunks; ++ChunkIndex)
				{
					DirtyChunkBoundsData.Add(FIntVector2(CellIndex, CellHeader.ItemChunksOffset + ChunkIndex));
				}

				// tag index with the flag & overwrite the index in DirtyCellBoundsIndices for upload to the GPU
				CellIndex = (CellIndex << 1u) | (bStaticDirty ? 1 : 0);
			}

		}


		// Note: the below could be put in a separate task, to cut down wait time for subsequent tasks
		// Releasing memory we're done with in the task saves some RT time
		DirtyChunks.Empty();
		DirtyBlocks.Empty();
		RemovedInstanceFlags.Empty();
		DirtyCellBoundsMaskDynamic.Empty();
		DirtyCellBoundsMaskStatic.Empty();
	}

	TBitArray<SceneRenderingAllocator> RemovedInstanceFlags;

	// Used for allocating new cache slots, important to reset/maintain whenever a cache slot is freed
	int32 LowestFreeCacheIndex = 0;

	FSceneCulling::FLocation64 CachedCellLoc;
	int32 CachedCellIdIndex = -1;

	TArray<FTempCell, SceneRenderingAllocator> TempCells;
	int32 TotalCellChunkDataCount = 0;

	FSceneCulling& SceneCulling;
	FSceneCulling::FSpatialHash &SpatialHash;
	FSceneCulling::FSpatialHash::FSpatialHashMap &SpatialHashMap;

	// Tracks current temp cells (cells where we store an index to a temp cell instead of proper cell data).
	TBitArray<SceneRenderingAllocator> TempCellMask;

	// Dirty state tracking for upload
	int32 NumDirtyBlocks = 0;
	TBitArray<SceneRenderingAllocator> DirtyBlocks;
	TBitArray<SceneRenderingAllocator> DirtyChunks;
	// Dirty tracking for explicit cell bounds
	TBitArray<SceneRenderingAllocator> DirtyCellBoundsMaskStatic;
	TBitArray<SceneRenderingAllocator> DirtyCellBoundsMaskDynamic;
	TArray<int32, SceneRenderingAllocator> DirtyCellBoundsIndices;
	TArray<FIntVector2, SceneRenderingAllocator> DirtyChunkBoundsData;

	TStructuredBufferScatterUploader<FCellBlockData> BlockDataUploader;
	TStructuredBufferScatterUploader<uint32, INSTANCE_HIERARCHY_MAX_CHUNK_SIZE> ItemChunkDataUploader;
	TStructuredBufferScatterUploader<FPackedCellHeader> CellHeaderUploader;
	TStructuredBufferScatterUploader<uint32> ItemChunkUploader;

	// The default for primitives that are dynamic & instanced is to be treated as dynamic, but to improve update performance they can instead be treated as "uncullable"
	FPrimitiveState::EState DynamicInstancedPrimitiveState = FPrimitiveState::Dynamic;

	int32 TotalUpdatedInstances = 0;
	int32 TotalAddedInstances = 0;
	int32 TotalRemovedInstances = 0;
	bool bUsePrecomputed = false;

#if SC_ENABLE_DETAILED_LOGGING
	bool bIsLoggingEnabled = false;
	FString SceneTag;
	int32 LogStrIndent = 0;
public:
	inline void LogIndent(int IndentDelta)
	{
		LogStrIndent += IndentDelta;
	}
	inline void AddLog(const FString &Item)
	{
		if (bIsLoggingEnabled)
		{
			FString Indent = FString::ChrN(LogStrIndent, TEXT(' '));
			UE_LOGF(LogTemp, Log, "%ls%ls", *Indent, *Item);
		}
	}
	inline void EndLogging()
	{
		if (bIsLoggingEnabled)
		{
			LogIndent(-1);
			AddLog(TEXT("Log-Scope-End"));
		}
		LogStrIndent = 0;
	}
#endif
};

template <typename T>
inline  bool operator<(const RenderingSpatialHash::TLocation<T>& Lhs, const RenderingSpatialHash::TLocation<T>& Rhs)
{
	if (Lhs.Level != Rhs.Level)
	{
		return (Lhs.Level < Rhs.Level);
	}
	if (Lhs.Coord.X != Rhs.Coord.X)
	{
		return Lhs.Coord.X < Rhs.Coord.X;
	}
	if (Lhs.Coord.Y != Rhs.Coord.Y)
	{
		return Lhs.Coord.Y < Rhs.Coord.Y;
	}

	return Lhs.Coord.Z < Rhs.Coord.Z;
}

void FSceneCulling::PublishStats()
{
	SET_DWORD_STAT(STAT_SceneCulling_BlockCount, CellBlockData.Num());
	SET_DWORD_STAT(STAT_SceneCulling_CellCount, CellHeaders.Num());
	int32 NumChunks = CellChunkIdAllocator.GetSparselyAllocatedSize();
	SET_DWORD_STAT(STAT_SceneCulling_ItemChunkCount, NumChunks);
	int32 NumExplicitChunks = PackedCellData.Num() / INSTANCE_HIERARCHY_MAX_CHUNK_SIZE - FreeChunks.Num();
	SET_DWORD_STAT(STAT_SceneCulling_ExplicitChunkCount, NumExplicitChunks);
	SET_DWORD_STAT(STAT_SceneCulling_CompressedChunkCount, NumChunks - NumExplicitChunks);
	SET_DWORD_STAT(STAT_SceneCulling_ItemBufferSize, PackedCellData.Num());
	float ChunkOccupancy = float(Scene.GPUScene.GetInstanceSceneDataAllocator().GetSparselyAllocatedSize()) /  float(NumChunks * INSTANCE_HIERARCHY_MAX_CHUNK_SIZE);
	SET_FLOAT_STAT(STAT_SceneCulling_ChunkOccupancy, ChunkOccupancy);
	SET_DWORD_STAT(STAT_SceneCulling_SceneInstanceCount, Scene.GPUScene.GetInstanceSceneDataAllocator().GetSparselyAllocatedSize());

	SET_DWORD_STAT(STAT_SceneCulling_IdCacheSize, TotalCellIndexCacheItems);

#if SC_ENABLE_DETAILED_BUILDER_STATS
	SET_DWORD_STAT(STAT_SceneCulling_NumStaticInstances, NumStaticInstances);
	SET_DWORD_STAT(STAT_SceneCulling_NumDynamicInstances, NumDynamicInstances);
	CSV_CUSTOM_STAT(SceneCulling, NumStaticInstances,  NumStaticInstances, ECsvCustomStatOp::Accumulate);
	CSV_CUSTOM_STAT(SceneCulling, NumDynamicInstances,  NumDynamicInstances, ECsvCustomStatOp::Accumulate);
#endif
}

FSceneCulling::FSceneCulling(FScene& InScene)
	: ISceneExtension(InScene)
	, bIsEnabled(UseSceneCulling(Scene.GetShaderPlatform()))
	, bHierarchicalCPUCullingEnabled(UseHierarchicalCPUCulling())
	, SpatialHash( CVarSceneCullingMinCellSize.GetValueOnAnyThread(), CVarSceneCullingMaxCellSize.GetValueOnAnyThread())
	, CellHeadersBuffer(16, TEXT("SceneCulling.CellHeaders"))
	, ItemChunksBuffer(16, TEXT("SceneCulling.ItemChunks"))
	, InstanceIdsBuffer(16, TEXT("SceneCulling.Items"))
	, CellBlockDataBuffer(16, TEXT("SceneCulling.CellBlockData"))
	, ExplicitChunkBoundsBuffer(16, TEXT("SceneCulling.ExplicitChunkBounds"))
	, ExplicitChunkCellIdsBuffer(16, TEXT("SceneCulling.ExplicitChunkBounds"))
{
}

ISceneExtensionUpdater* FSceneCulling::CreateUpdater() 
{ 
	return new FUpdater(*this); 
}

ISceneExtensionRenderer* FSceneCulling::CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags) 
{ 
	return new FSceneCullingRenderer(InSceneRenderer, *this); 
}

FSceneCulling::FUpdater::~FUpdater()
{
}

bool FSceneCulling::BeginUpdate(FRDGBuilder& GraphBuilder, bool bAnySceneUpdatesExpected)
{	
	check(!ActiveUpdaterImplementation.IsValid());

#if (!(UE_BUILD_SHIPPING || UE_BUILD_TEST))
	if (bIsEnabled && !UseNanite(Scene.GetShaderPlatform()))
	{
		FString ShaderPlatformString = LexToString(Scene.GetShaderPlatform());
		FString SceneWorldNameString = Scene.GetFullWorldName();
		UE_LOGF(LogRenderer, Log, "SceneCulling instance hierarchy is disabled as UseNanite(%ls) returned false, for scene: '%ls'.", *ShaderPlatformString, *SceneWorldNameString);
	}
#endif
	// Note: this only works in concert with the FGlobalComponentRecreateRenderStateContext on the CVarSceneCulling callback ensuring all geometry is re-registered
	bIsEnabled = UseSceneCulling(Scene.GetShaderPlatform());

	bForceFullExplictBoundsBuild = CVarSceneCullingUseForceRebuildExplicitChunkBounds.GetValueOnRenderThread();

	SmallFootprintCellSideThreshold = CVarSmallFootprintSideThreshold.GetValueOnRenderThread();
	bUseAsyncUpdate = CVarSceneCullingAsyncUpdate.GetValueOnRenderThread() != 0;
	bUseAsyncQuery = CVarSceneCullingAsyncQuery.GetValueOnRenderThread() != 0;

	if (bIsEnabled)
	{
		ActiveUpdaterImplementation = MakePimpl<FSceneCullingBuilder>(*this, bAnySceneUpdatesExpected);
	}
	else
	{
		Empty();
	}

	return bIsEnabled;
}

void FSceneCulling::FUpdater::PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
{
	LLM_SCOPE_BYTAG(SceneCulling);

	SceneCulling.FinalizeUpdateAndClear(GraphBuilder, false);

	if (SceneCulling.bHierarchicalCPUCullingEnabled)
	{
		// TODO, run async?
		bool bWasAnyRemoved = false;
		for (FPersistentPrimitiveIndex Id : ChangeSet.RemovedPrimitiveIds)
		{
			bWasAnyRemoved = SceneCulling.CPUCullingInfos.Remove(Id) || bWasAnyRemoved;

		}
		if (bWasAnyRemoved)
		{
			// Reset to trigger recompute
			SceneCulling.MaxHashGroupVisibilityWord = -1;
			SceneCulling.MaxHashGroupLODOffset = -1;
		}	
	}

	// TODO:
	bool bAnySceneUpdatesQueued = true;

	if (SceneCulling.BeginUpdate(GraphBuilder, bAnySceneUpdatesQueued))
	{
		SC_DETAILED_LOGGING_SCOPE(SceneCulling.ActiveUpdaterImplementation.Get());

		// Handle all removed primitives 
		// Step 1. Mark all removed instances
#if SC_ALLOW_ASYNC_TASKS
		PreUpdateTaskHandle = GraphBuilder.AddSetupTask([this, ChangeSet]()
#endif
		{
			SCOPED_NAMED_EVENT(SceneCulling_Update, FColor::Emerald);
			SCOPE_CYCLE_COUNTER(STAT_SceneCulling_Update_Pre);
			BUILDER_LOG_SCOPE("OnPreSceneUpdate: %d", ChangeSet.RemovedPrimitiveIds.Num());
			CSV_SCOPED_TIMING_STAT(SceneCulling, PreSceneUpdate);
#if DO_CHECK
			check(DebugTaskCounter++ == 0);
#endif
			SceneCulling.ActiveUpdaterImplementation->ProcessPreSceneUpdate(ChangeSet);
#if DO_CHECK
			check(--DebugTaskCounter == 0);
#endif
		}
#if SC_ALLOW_ASYNC_TASKS
		, SceneCulling.bUseAsyncUpdate);
#endif
	}
}

void FSceneCulling::FUpdater::PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
{
	LLM_SCOPE_BYTAG(SceneCulling);

	auto AddOrUpdateHierarchicalCPUCUlling = [&](FPersistentPrimitiveIndex PersistentPrimitiveId, FPrimitiveSceneInfo* PrimitiveSceneInfo) -> bool
	{
		if (PrimitiveSceneInfo->GetInstanceDataHeader().Flags.bHasCompressedSpatialHash && PrimitiveSceneInfo->GpuLodInstanceRadius > 0.0f && !PrimitiveSceneInfo->Proxy->IsNaniteMesh())
		{
			// Note: this causes stall, can we move precomputed ref to Header?
			if (const FInstanceSceneDataBuffers* InstanceSceneDataBuffers = PrimitiveSceneInfo->GetInstanceSceneDataBuffers())
			{
				if (TSharedPtr<FInstanceSceneDataImmutable, ESPMode::ThreadSafe> InstanceSceneDataImmutable = InstanceSceneDataBuffers->GetImmutable())
				{
					const TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem>& Hashes = InstanceSceneDataImmutable->GetCompressedInstanceSpatialHashes();
					FPrimitiveCullingInfo* PrimitiveCullingInfo = SceneCulling.CPUCullingInfos.FindOrAdd(PersistentPrimitiveId, FPrimitiveCullingInfo{});

					// Maintain reference to avoid keep Hashes alive
					PrimitiveCullingInfo->InstanceSceneDataImmutable = MoveTemp(InstanceSceneDataImmutable);
					PrimitiveCullingInfo->NumItems = Hashes.Num();
					PrimitiveCullingInfo->HashGroupVisibilityOffset = -1;
					PrimitiveCullingInfo->PersistentPrimitiveId = PersistentPrimitiveId;
					PrimitiveCullingInfo->Items = PrimitiveCullingInfo->InstanceSceneDataImmutable->GetCompressedInstanceSpatialHashes();
					
					FVector2f MinMaxDrawDistance;
					if (PrimitiveSceneInfo->Proxy->GetInstanceDrawDistanceMinMax(MinMaxDrawDistance))
					{
						PrimitiveCullingInfo->MaxInstanceDrawDistance = MinMaxDrawDistance.Y;
					}
					
					SceneCulling.MaxHashGroupVisibilityWord = -1;
					SceneCulling.MaxHashGroupLODOffset = -1;

					return true;
				}
			}
		}
		return false;
	};

	// Handle state changes
	bool bNewHierarchicalCPUCullingEnabled = UseHierarchicalCPUCulling();
	if (bNewHierarchicalCPUCullingEnabled && !SceneCulling.bHierarchicalCPUCullingEnabled)
	{
		// Make sure we are starting from empty
		check(SceneCulling.MaxHashGroupVisibilityWord == -1);
		check(SceneCulling.MaxHashGroupLODOffset == -1);
		check(SceneCulling.CPUCullingInfos.Num() == 0);

		for (FPrimitiveSceneInfo* PrimitiveSceneInfo : SceneCulling.Scene.Primitives)
		{
			AddOrUpdateHierarchicalCPUCUlling(PrimitiveSceneInfo->GetPersistentIndex(), PrimitiveSceneInfo);
		}
		SceneCulling.bHierarchicalCPUCullingEnabled = true;
	}
	else if (!bNewHierarchicalCPUCullingEnabled && SceneCulling.bHierarchicalCPUCullingEnabled)
	{
		// reset everything when it is turned off.
		SceneCulling.MaxHashGroupVisibilityWord = -1;
		SceneCulling.MaxHashGroupLODOffset = -1;
		SceneCulling.CPUCullingInfos.Empty();
		SceneCulling.bHierarchicalCPUCullingEnabled = false;
	}
	else if (SceneCulling.bHierarchicalCPUCullingEnabled)
	{
		// TODO, run async?
		ChangeSet.PrimitiveUpdates.ForEachUpdateCommand(ESceneUpdateCommandFilter::AddedUpdated, EPrimitiveUpdateDirtyFlags::All, [&](const FPrimitiveUpdateCommand& Cmd)
		{
			FPersistentPrimitiveIndex PersistentPrimitiveId = Cmd.GetPersistentId();
			FPrimitiveSceneInfo* PrimitiveSceneInfo = Cmd.GetSceneInfo();
			bool bWasAddedOrUpdated = false;
			if (!PrimitiveSceneInfo->Proxy->IsNaniteMesh())
			{
				bWasAddedOrUpdated = AddOrUpdateHierarchicalCPUCUlling(PersistentPrimitiveId, PrimitiveSceneInfo);
				// It was an update, but it no longer is a valid (e.g., spatial hashes were removed)
				if (!bWasAddedOrUpdated && !Cmd.IsAdd())
				{
					SceneCulling.CPUCullingInfos.Remove(PersistentPrimitiveId);
				}
			}
			// Make sure we're not leaving any that are not supposed to have an entry in the map
			check(bWasAddedOrUpdated || !SceneCulling.CPUCullingInfos.Find(PersistentPrimitiveId));
		});

	}
	
	// Derived state, regardless of previous state
	if (SceneCulling.bHierarchicalCPUCullingEnabled)
	{
		// Something has changed wrt the allocation of the bits in the bit vector, recompute the offsets as needed
		if (SceneCulling.MaxHashGroupVisibilityWord < 0)
		{
			uint32 Offset = 0u;
			uint32 LODOffset = 0u;
			for (auto& CPUCullingInfoItem : SceneCulling.CPUCullingInfos)
			{
				FPrimitiveCullingInfo& PrimitiveCullingInfo = CPUCullingInfoItem.Value;
				PrimitiveCullingInfo.HashGroupVisibilityOffset = Offset;
				PrimitiveCullingInfo.HashGroupLODOffset = LODOffset;
				Offset += PrimitiveCullingInfo.GetNumWords();
				LODOffset += PrimitiveCullingInfo.NumItems;
			}
			SceneCulling.MaxHashGroupVisibilityWord = Offset;
			SceneCulling.MaxHashGroupLODOffset = LODOffset;
		}
	}

	if (!SceneCulling.ActiveUpdaterImplementation.IsValid())
	{
		return;
	}
	SC_DETAILED_LOGGING_SCOPE(SceneCulling.ActiveUpdaterImplementation.Get());

#if SC_ALLOW_ASYNC_TASKS
	check(SceneCulling.PostUpdateTaskHandle.IsCompleted());
	SceneCulling.PostUpdateTaskHandle = GraphBuilder.AddSetupTask([this, ChangeSet]()
#endif
	{
#if DO_CHECK
		check(DebugTaskCounter++ == 0);
#endif
		SceneCulling.ActiveUpdaterImplementation->ProcessPostSceneUpdate(ChangeSet);
#if DO_CHECK
		check(--DebugTaskCounter == 0);
#endif
	}
#if SC_ALLOW_ASYNC_TASKS
	, nullptr, TArray<UE::Tasks::FTask>{ PreUpdateTaskHandle }, UE::Tasks::ETaskPriority::Normal, SceneCulling.bUseAsyncUpdate);
#endif
}

void FSceneCulling::FinalizeUpdateAndClear(FRDGBuilder& GraphBuilder, bool bPublishStats)
{
	LLM_SCOPE_BYTAG(SceneCulling);
	
	// if the updater is null, the task handle should not be potentially waiting for anything
	check(ActiveUpdaterImplementation.IsValid() || PostUpdateTaskHandle.IsCompleted());

	if (ActiveUpdaterImplementation.IsValid())
	{
		SC_DETAILED_LOGGING_SCOPE(ActiveUpdaterImplementation.Get());
		SCOPED_NAMED_EVENT(SceneCulling_Update_FinalizeAndClear, FColor::Emerald);
		SCOPE_CYCLE_COUNTER(STAT_SceneCulling_Update_FinalizeAndClear);
		//BUILDER_LOG_SCOPE("FinalizeAndClear %d", 1);

		PostUpdateTaskHandle.Wait();

		ActiveUpdaterImplementation->UploadToGPU(GraphBuilder);
		
		if (bPublishStats)
		{
			ActiveUpdaterImplementation->PublishStats();
		}

#if SC_ENABLE_DETAILED_LOGGING
		ActiveUpdaterImplementation->EndLogging();
#endif
		ActiveUpdaterImplementation.Reset();
	}
}

void FSceneCulling::EndUpdate(FRDGBuilder& GraphBuilder, bool bPublishStats)
{
	if (bIsEnabled)
	{
		FinalizeUpdateAndClear(GraphBuilder, bPublishStats);
#if DO_CHECK
		ValidateAllInstanceAllocations();
#endif
	}
}

void FSceneCulling::ValidateAllInstanceAllocations()
{
#if DO_CHECK
	if (CVarValidateAllInstanceAllocations.GetValueOnRenderThread() != 0)
	{
		for (TConstSetBitIterator<> BitIt(CellOccupancyMask); BitIt; ++BitIt)
		{
			uint32 CellId = uint32(BitIt.GetIndex());
			FCellHeader CellHeader = UnpackCellHeader(CellHeaders[CellId]);
			check(IsValidCell(CellHeader));

			for (uint32 ChunkIndex = 0; ChunkIndex < CellHeader.NumItemChunks; ++ChunkIndex)
			{
				uint32 PackedChunkData = PackedCellChunkData[CellHeader.ItemChunksOffset + ChunkIndex];
				const bool bIsCompressed = (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_FLAG) != 0u;
				const uint32 NumItems = bIsCompressed ? 64u : PackedChunkData >> INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT;
				const bool bIsFullChunk = NumItems == 64u;

				// 1. if it is a compressed chunk and contains any index, we may assume it is to be removed entirely.
				if (bIsCompressed)
				{
					uint32 InstanceDataOffset = PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_PAYLOAD_MASK;
					for (uint32 ItemIndex = 0u; ItemIndex < 64u; ++ItemIndex)
					{
						uint32 InstanceId = InstanceDataOffset + ItemIndex;
						check(!Scene.GPUScene.GetInstanceSceneDataAllocator().IsFree(InstanceId));
					}
				}
				else
				{
					uint32 ChunkId =  (PackedChunkData & INSTANCE_HIERARCHY_ITEM_CHUNK_ID_MASK);
					uint32 ItemDataOffset = ChunkId * INSTANCE_HIERARCHY_MAX_CHUNK_SIZE;

					// 3.1. scan chunk for deleted items
					uint64 DeletionMask = 0ull;

					for (uint32 ItemIndex = 0u; ItemIndex < NumItems; ++ItemIndex)
					{
						uint32 InstanceId = PackedCellData[ItemDataOffset + ItemIndex];
						check(!Scene.GPUScene.GetInstanceSceneDataAllocator().IsFree(InstanceId));
					}
				}
			}
		}
	}
#endif
}

UE::Tasks::FTask FSceneCulling::GetUpdateTaskHandle() const
{
	return ActiveUpdaterImplementation.IsValid() ? PostUpdateTaskHandle : UE::Tasks::FTask();
}

uint32 FSceneCulling::AllocateChunk()
{
	if (!FreeChunks.IsEmpty())
	{
		return FreeChunks.Pop(EAllowShrinking::No);
	}

	check(!bPackedCellDataLocked);
	uint32 NewChunkId = PackedCellData.Num() / uint32(INSTANCE_HIERARCHY_MAX_CHUNK_SIZE);
	PackedCellData.SetNumUninitialized(PackedCellData.Num() + int32(INSTANCE_HIERARCHY_MAX_CHUNK_SIZE), EAllowShrinking::No);
	return NewChunkId;
}

void FSceneCulling::FreeChunk(uint32 ChunkId)
{
	FreeChunks.Add(ChunkId);
}

inline int32 FSceneCulling::CellIndexToBlockId(int32 CellIndex)
{
	return CellIndex / FSpatialHash::CellBlockSize;
}

inline uint32* FSceneCulling::LockChunkCellData(uint32 ChunkId, int32 NumSlackChunksNeeded)
{
	check(!bPackedCellDataLocked);

	int32 NumNewNeeded = NumSlackChunksNeeded - FreeChunks.Num();
	if(NumNewNeeded > 0)
	{
		uint32 NewChunkId = PackedCellData.Num() / uint32(INSTANCE_HIERARCHY_MAX_CHUNK_SIZE);
		for (int32 Index = 0; Index < NumNewNeeded; ++Index)
		{
			FreeChunks.Add(NewChunkId + uint32(Index));
		}
		PackedCellData.SetNumUninitialized(PackedCellData.Num() + NumNewNeeded * int32(INSTANCE_HIERARCHY_MAX_CHUNK_SIZE));
	}

	bPackedCellDataLocked = true;
	return &PackedCellData[ChunkId * INSTANCE_HIERARCHY_MAX_CHUNK_SIZE];
}

inline void FSceneCulling::UnLockChunkCellData(uint32 ChunkId)
{
	check(bPackedCellDataLocked);
	bPackedCellDataLocked = false;
}

inline FSceneCulling::FLocation64 FSceneCulling::GetCellLoc(int32 CellIndex)
{
	FLocation64 Result;
	Result.Level = INT32_MIN;
	if (CellHeaders.IsValidIndex(CellIndex) && IsValidCell(CellHeaders[CellIndex]))
	{
		int32 BlockId = CellIndexToBlockId(CellIndex);
		FBlockLoc BlockLoc = SpatialHash.GetBlockLocById(BlockId);
		Result.Coord = FInt64Vector3(BlockLoc.GetCoord()) << FSpatialHash::CellBlockDimLog2;
		Result.Level = BlockLoc.GetLevel() - FSpatialHash::CellBlockDimLog2;
		// Offset by local coord.
		Result.Coord.X += CellIndex & FSpatialHash::LocalCellCoordMask;
		Result.Coord.Y += (CellIndex >> FSpatialHash::CellBlockDimLog2) & FSpatialHash::LocalCellCoordMask;
		Result.Coord.Z += (CellIndex >> (2u * FSpatialHash::CellBlockDimLog2)) & FSpatialHash::LocalCellCoordMask;
	}
	return Result;
}

SC_FORCEINLINE bool FSceneCulling::IsUncullable(const FPrimitiveBounds& Bounds, FPrimitiveSceneInfo* PrimitiveSceneInfo)
{
	// a primitive cannot be culled if it is too large, OR if it is so far away that it cannot be represented in the precision used in the hierarhcy
	return Bounds.BoxSphereBounds.SphereRadius * 2.0 >= SpatialHash.GetLastLevelCellSize()
		||  Bounds.BoxSphereBounds.Origin.SquaredLength() >= FMath::Square(SpatialHash.GetMaxCullingDistance() - Bounds.BoxSphereBounds.SphereRadius);
}


#if SC_ENABLE_DETAILED_LOGGING

FIHLoggerScopeHelper::FIHLoggerScopeHelper()
{
	if (GBuilderForLogging != nullptr)
	{
		GBuilderForLogging->LogIndent(1);
	}
}

FIHLoggerScopeHelper::~FIHLoggerScopeHelper()
{
	if (GBuilderForLogging != nullptr)
	{
		GBuilderForLogging->LogIndent(-1);
	}
}

FIHLoggerListScopeHelper::FIHLoggerListScopeHelper(const FString &InListName) 
	: bEnabled(GBuilderForLogging != nullptr)
	, LogStr(InListName) 
{
	if (bEnabled)
	{
		LogStr.Append(TEXT(": "));
	}
}

FIHLoggerListScopeHelper::~FIHLoggerListScopeHelper()
{
	if (bEnabled)
	{
		GBuilderForLogging->AddLog(LogStr);
	}
}

#endif
