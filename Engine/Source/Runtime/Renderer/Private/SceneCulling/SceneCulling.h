// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SpanAllocator.h"
#include "ConvexVolume.h"
#include "Rendering/RenderingSpatialHash.h"
#include "Tasks/Task.h"
#include "RendererInterface.h"
#include "RendererPrivateUtils.h"
#include "PrimitiveSceneInfo.h"

#include "SceneCullingDefinitions.h"
#include "HierarchicalSpatialHashGrid.h"
#include "InstanceDataSceneProxy.h"
#include "SceneExtensions.h"

class FScenePreUpdateChangeSet;
class FScenePostUpdateChangeSet;
class FScene;
struct FPrimitiveBounds;

/**
 * Represents either a set of planes, or a sphere,
 */
struct FCullingVolume
{
	FCullingVolume() = default;
	explicit FCullingVolume(const FConvexVolume& InConvexVolume) 
		: ConvexVolume(InConvexVolume)
	{
	}

	FConvexVolume ConvexVolume;
	// Negative translation to add to the tested location prior to testing the ConvexVolume.
	FVector3d WorldToVolumeTranslation = FVector3d::ZeroVector; 
	// Bounding sphere in world space, if radius is zero OR the footprint is <= r.SceneCulling.SmallFootprintSideThreshold, the ConvexVolume is used
	FSphere3d Sphere = FSphere(ForceInit);

	bool IntersectBox(const FVector& Origin, const FVector& BoxExtent) const
	{
		if (ConvexVolume.Planes.IsEmpty())
		{
			return FMath::SphereAABBIntersection(Sphere, FBox(Origin - BoxExtent, Origin + BoxExtent));
		}
		return ConvexVolume.IntersectBox(Origin + WorldToVolumeTranslation, BoxExtent);
	}
};


class FSceneCullingBuilder;

class FSceneCulling : public ISceneExtension
{
	DECLARE_SCENE_EXTENSION(RENDERER_API, FSceneCulling);
public:
	friend class FSceneCullingRenderer;
	friend class FSceneCullingRendererHierarchicalCullingQuery;

	FSceneCulling(FScene& InScene);

	/**
	 * This is just an empty shell  mostly which allows the builder to outlive RDG. This allows flushing the scene updates _next_ frame in case no one wanted them.
	 * Actual update tasks and suchlike use the RDG machinery and are thus not going to outlive the renderer / update, but we can't trigger an upload to GPU on delete.
	 */
	class FUpdater : public ISceneExtensionUpdater
	{
		DECLARE_SCENE_EXTENSION_UPDATER(FUpdater, FSceneCulling);
	public:

		FUpdater(FSceneCulling& InSceneCulling) : SceneCulling(InSceneCulling) {}

		virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) override;
		virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) override;

		~FUpdater();

	private:
		friend class FSceneCulling;

		UE::Tasks::FTask PreUpdateTaskHandle;

		FSceneCullingBuilder *Implementation = nullptr;
		FSceneCulling& SceneCulling;

#if DO_CHECK
		std::atomic<int32> DebugTaskCounter = 0;
#endif
	};
	// ISceneExtension API
	static bool ShouldCreateExtension(FScene& Scene);
	virtual ISceneExtensionUpdater* CreateUpdater();
	virtual ISceneExtensionRenderer* CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags);

	// end ISceneExtension

	/**
	 * Finalize update of hierarchy, should be done as late as possible, also performs update of RDG resources. 
	 * May be called multiple times, the first call does the work.
	 */
	void EndUpdate(FRDGBuilder& GraphBuilder, bool bPublishStats);

	UE::Tasks::FTask GetUpdateTaskHandle() const;

	bool IsEnabled() const { return bIsEnabled; }

	void PublishStats();

#if 0
	// This implementation is not currently used (and the data structures have changed somewhat) but is kept for reference.
	void TestConvexVolume(const FConvexVolume& ViewCullVolume, const FVector3d &WorldToVolumeTranslation, TArray<FCellDraw, SceneRenderingAllocator>& OutCellDraws, uint32 ViewGroupId, uint32& OutNumInstanceGroups) const;
#endif
	template <typename ResultConsumerType>
	void TestSphere(const FSphere& Sphere, ResultConsumerType& ResultConsumer) const;

	struct alignas(16) FBlockLocAligned
	{
		FORCEINLINE FBlockLocAligned() {}

		FORCEINLINE explicit FBlockLocAligned(const RenderingSpatialHash::TLocation<int64> &InLoc)
			: Data(int32(InLoc.Coord.X), int32(InLoc.Coord.Y), int32(InLoc.Coord.Z), int32(InLoc.Level))
		{
		}

		FORCEINLINE bool operator==(const FBlockLocAligned& BlockLocAligned) const
		{
			return Data == BlockLocAligned.Data;
		}

		FORCEINLINE void operator=(const FBlockLocAligned &BlockLocAligned)
		{
			Data = BlockLocAligned.Data;
		}
		FORCEINLINE int32 GetLevel() const { return Data.W; }

		FORCEINLINE FIntVector3 GetCoord() const { return FIntVector3(Data.X, Data.Y, Data.Z); }

		FORCEINLINE FVector3d GetWorldPosition() const
		{
			double LevelSize = RenderingSpatialHash::GetCellSize(Data.W);
			return FVector3d(GetCoord()) * LevelSize;
		}

		FORCEINLINE uint32 GetHash() const
		{
			using UIntType = std::make_unsigned_t<FIntVector4::IntType>;

			// TODO: Vectorize? Maybe convert to float vector & use dot product? Maybe not? (mul is easy, dot maybe not?)
			return uint32((UIntType)Data.X * 1150168907 + (UIntType)Data.Y * 1235029793 + (UIntType)Data.Z * 1282581571 + (UIntType)Data.W * 1264559321);
		}

		FIntVector4 Data;
	};

	using FBlockLoc = FBlockLocAligned;

	struct FBlockTraits
	{
		static constexpr int32 CellBlockDimLog2 = 3; // (8x8x8)
		using FBlockLoc = FBlockLoc;

		// The FBlockLocAligned represents the block locations as 32-bit ints.
		static constexpr int64 MaxCellBlockCoord = MAX_int32;
		// The cell coordinate may be larger by the block dimension and still can fit into a signed 32-bit integer
		static constexpr int64 MaxCellCoord = MaxCellBlockCoord << CellBlockDimLog2;
	};

	using FSpatialHash = THierarchicalSpatialHashGrid<FBlockTraits>;

	using FLocation64 = FSpatialHash::FLocation64;
	using FLocation32 = FSpatialHash::FLocation32;
	using FLocation8 = FSpatialHash::FLocation8;

	using FFootprint8 = FSpatialHash::FFootprint8;
	using FFootprint32 = FSpatialHash::FFootprint32;
	using FFootprint64 = FSpatialHash::FFootprint64;	

	/**
	 * Provides info to build cpu-side instance culling structures.
	 */
	class FPrimitiveCullingInfo
	{
	private:
		// Offset into a semi-compact range of bits (aligned to full word boundaries to avoid race conditions during async updates.
		uint32 HashGroupVisibilityOffset = 0u;
		uint32 HashGroupLODOffset = 0u;
		uint32 NumItems = 0u;
		float MaxInstanceDrawDistance = -1.0f;
		FPersistentPrimitiveIndex PersistentPrimitiveId;
		TSharedPtr<FInstanceSceneDataImmutable, ESPMode::ThreadSafe> InstanceSceneDataImmutable;
	public:
		friend class FSceneCulling;
		friend class FSceneCullingRenderer;
		friend class FSceneCullingRendererHierarchicalCullingQuery;

		TArrayView<const FInstanceSceneDataBuffers::FCompressedSpatialHashItem> Items;
		uint32 GetNumWords() const { return FMath::DivideAndRoundUp(NumItems, uint32(NumBitsPerDWORD));}
	};

	/**
	 * Scene culling info for a given primitive for CPU-side culling.
	 */
	FPrimitiveCullingInfo* GetCullingInfo(FPersistentPrimitiveIndex PersistentPrimitiveIndex)
	{
		return CPUCullingInfos.Find(PersistentPrimitiveIndex);			
	}
private:

	// Returns the number of bits required to store the state for all hierarchical culling ranges.
	int32 GetMaxHashGroupVisibilityOffset() const { return MaxHashGroupVisibilityWord * NumBitsPerDWORD; }

	// Returns the number of bits required to store the state for all hierarchical culling ranges.
	int32 GetMaxHashGroupLODOffset() const { return MaxHashGroupLODOffset; }

	bool IsSmallCullingVolume(const FCullingVolume& CullingVolume) const;

	void FinalizeUpdateAndClear(FRDGBuilder& GraphBuilder, bool bPublishStats);

	/**
	 * Set up update driver that can collect change sets and initiate async update. The updater (internals) has RDG scope.
	 */
	bool BeginUpdate(FRDGBuilder& GraphBuilder, bool bAnySceneUpdatesExpected);

	void ValidateAllInstanceAllocations();

	void Empty();

	/**
	 * The cache stores info about what cells the instances are inserted into in the grid, such that we can remove/update without needing to recompute the full transformation.
	 */
	struct FCellIndexCacheEntry
	{
		static constexpr uint32 SingleInstanceMask = 1u << 31;
		static constexpr uint32 CellIndexMask = (1u << 31) - 1u;
		static constexpr uint32 CellIndexMax = 1u << 31;

		struct FItem
		{
			int32 NumInstances;
			int32 CellIndex;
		};

		inline void Add(int32 CellIndex, int32 NumInstances)
		{
			check(uint32(CellIndex) < CellIndexMax);
			check(!bSingleInstanceOnly || NumInstances == 1);

			if (NumInstances > 1)
			{
				// Add RLE entry
				Items.Add(CellIndex);
				Items.Add(NumInstances);
			}
			else
			{
				// Mark as single-istance
				Items.Add(uint32(CellIndex) | SingleInstanceMask);
			}
		}

		/**
		 * Only possible if there is one single instance items in the list, otherwise we don't have a 1:1 mapping.
		 */
		inline void Set(int32 Index, int32 CellIndex)
		{
			check(bSingleInstanceOnly);
			check(uint32(CellIndex) < CellIndexMax);

			Items[Index] = uint32(CellIndex) | SingleInstanceMask;
		}

		FORCEINLINE void Reset() 
		{
			Items.Reset();
		}


		/**
		 * Load and unpack an item at a given ItemIndex. NOTE: advances ItemIndex if the item is RLE'd
		 */
		FItem LoadAndStepItem(int32 &InOutItemIndex) const
		{
			FItem Result; 
			uint32 PackedCellIndex = Items[InOutItemIndex];
			Result.CellIndex = int32(PackedCellIndex & FCellIndexCacheEntry::CellIndexMask);
			Result.NumInstances = 1;
			if ((PackedCellIndex & FCellIndexCacheEntry::SingleInstanceMask) == 0u)
			{
				Result.NumInstances = int32(Items[++InOutItemIndex]);
			}
			return Result;
		}

		bool bSingleInstanceOnly = false;
		TArray<uint32> Items;
	};

	/** 
	 * Tracking state of each added primitive, needed to be able to transition ones that change category when updated & correctly remove.
	 */
	struct FPrimitiveState
	{
		static constexpr int32 PayloadBits = 28;
		static constexpr uint32 InvalidPayload = (1u << PayloadBits) - 1u;
		FPrimitiveState() 
			: InstanceDataOffset(-1)
			, NumInstances(0)
			, State(Unknown)
			, bDynamic(false)
			, Payload(InvalidPayload)
		{
		}

		enum EState : uint32
		{
			Unknown,
			SingleCell,
			Precomputed,
			UnCullable,
			Dynamic,
			Cached,
		};

		inline bool IsCachedState() const { return State == Cached || State == Dynamic; }

		int32 InstanceDataOffset;
		int32 NumInstances;
		EState State : 3;
		// The bDynamic flag is used to record whether a primitive has been seen to be updated. This can happen, for example for a stationary primitive, if this happens it is transitioned to Dynamic.
		bool bDynamic : 1;
		// For SingleCell primitives the payload represents the cell index directly, whereas for cached, it is the offset into the CellIndexCache
		uint32 Payload :28;

		const FString &ToString() const;

		TSharedPtr<FInstanceSceneDataImmutable, ESPMode::ThreadSafe> InstanceSceneDataImmutable;
	};

	TArray<FPrimitiveState> PrimitiveStates;
	TSparseArray<FCellIndexCacheEntry> CellIndexCache;
	int32 TotalCellIndexCacheItems = 0;

	int32 NumDynamicInstances = 0;
	int32 NumStaticInstances = 0;
	

	friend class FSceneCullingBuilder;
	friend class FSceneInstanceCullingQuery;

	bool bIsEnabled = false;
	bool bHierarchicalCPUCullingEnabled = false;
	bool bForceFullExplictBoundsBuild = false;

	FSpatialHash SpatialHash;

	// Kept in the class for now, since we only want one active at a time.
	TPimplPtr<FSceneCullingBuilder> ActiveUpdaterImplementation;

	// A cell stores references to a list of chunks, that, in turn, reference units of 64 instances. 
	// This enables storing compressed chunks directly in the indirection, as well as simplifying allocation and movement of instance data lists.
	TArray<uint32> PackedCellChunkData;
	FSpanAllocator CellChunkIdAllocator;
	TArray<uint32> PackedCellData;
	TArray<uint32> FreeChunks;
	TArray<FPackedCellHeader> CellHeaders;
	TBitArray<> CellOccupancyMask;
	TBitArray<> BlockLevelOccupancyMask;
	// Bit marking each chunk ID as in use or not, complements the CellChunkIdAllocator.
	TBitArray<> UsedChunkIdMask;

	TArray<FCellBlockData> CellBlockData;
	TArray<FPersistentPrimitiveIndex> UnCullablePrimitives;
	int32 UncullableItemChunksOffset = INDEX_NONE;
	int32 UncullableNumItemChunks = 0;
	// Largest dimension length, in cells, at the finest level under which a footprint is considered "small" and should go down the direct footprint path
	int32 SmallFootprintCellSideThreshold = 16;
	bool bTestCellVsQueryBounds = true;
	bool bUseAsyncUpdate = true;
	bool bUseAsyncQuery = true;
	bool bPackedCellDataLocked = false;

	inline uint32 AllocateChunk();
	inline void FreeChunk(uint32 ChunkId);
	inline uint32* LockChunkCellData(uint32 ChunkId, int32 NumSlackChunksNeeded);
	inline void UnLockChunkCellData(uint32 ChunkId);
	inline int32 CellIndexToBlockId(int32 CellIndex);
	inline FLocation64 GetCellLoc(int32 CellIndex);
	inline bool IsUncullable(const FPrimitiveBounds& Bounds, FPrimitiveSceneInfo* PrimitiveSceneInfo);

	// Persistent GPU-representation
	TPersistentStructuredBuffer<FPackedCellHeader> CellHeadersBuffer;
	TPersistentStructuredBuffer<uint32> ItemChunksBuffer;
	TPersistentStructuredBuffer<uint32> InstanceIdsBuffer;
	TPersistentStructuredBuffer<FCellBlockData> CellBlockDataBuffer;
	// explicit chunk bounds, packed and quantized
	TPersistentByteAddressBuffer<FPackedChunkAttributes> ExplicitChunkBoundsBuffer;
	// parallel to the chunk bounds, stores ID of the cell they belong to.
	TPersistentStructuredBuffer<uint32> ExplicitChunkCellIdsBuffer;
	TRefCountPtr<FRDGPooledBuffer> UsedChunkIdMaskBuffer;

	UE::Tasks::FTask PostUpdateTaskHandle;

	// Used to allocate ranges of bits for the cpu-side instance culling state.
	int32 MaxHashGroupVisibilityWord = -1;
	int32 MaxHashGroupLODOffset = -1;
	Experimental::TRobinHoodHashMap<FPersistentPrimitiveIndex, FPrimitiveCullingInfo> CPUCullingInfos;
};
