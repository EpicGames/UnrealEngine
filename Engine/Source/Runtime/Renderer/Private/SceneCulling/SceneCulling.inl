// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneCulling.h"


template <typename ScalarType>
inline UE::Math::TIntVector3<ScalarType> ClampDim(const UE::Math::TIntVector3<ScalarType>& Vec, ScalarType MinValueInc, ScalarType MaxValueInc)
{
	return UE::Math::TIntVector3<ScalarType>(
		FMath::Clamp(Vec.X, MinValueInc, MaxValueInc),
		FMath::Clamp(Vec.Y, MinValueInc, MaxValueInc),
		FMath::Clamp(Vec.Z, MinValueInc, MaxValueInc));
};

inline FSceneCulling::FFootprint8 ToBlockLocal(const FSceneCulling::FFootprint64& ObjFootprint, const FSceneCulling::FLocation64& BlockLoc)
{
	FInt64Vector3 BlockMin = BlockLoc.Coord * FSceneCulling::FSpatialHash::CellBlockDim;
	FInt64Vector3 BlockMax = BlockMin + FInt64Vector3(FSceneCulling::FSpatialHash::CellBlockDim - 1);

	FInt64Vector3 BlockLocalMin = ClampDim(ObjFootprint.Min - BlockMin, 0ll, FSceneCulling::FSpatialHash::CellBlockDim - 1ll);
	FInt64Vector3 BlockLocalMax = ClampDim(ObjFootprint.Max - BlockMin, 0ll, FSceneCulling::FSpatialHash::CellBlockDim - 1ll);

	using FInt8Vector3 = UE::Math::TIntVector3<int8>;

	FSceneCulling::FFootprint8 LocalFp = {
		FInt8Vector3(BlockLocalMin),
		FInt8Vector3(BlockLocalMax),
		ObjFootprint.Level
	};
	return LocalFp;
};

template <typename ResultConsumerType>
void FSceneCulling::TestSphere(const FSphere& Sphere, ResultConsumerType& ResultConsumer) const
{
	const FSpatialHash::FSpatialHashMap &GlobalSpatialHash = SpatialHash.GetHashMap();

	// TODO[Opt]: Maybe specialized bit set since we have a fixed size & alignment guaranteed (64-bit words all in use)
	// TODO[Opt]: Add a per-view grid / cache that works like the VSM page table and allows skipping within the footprint?
	for (TConstSetBitIterator<> BitIt(BlockLevelOccupancyMask); BitIt; ++BitIt)
	{
		int32 BlockLevel = BitIt.GetIndex();
		int32 Level = BlockLevel - FSpatialHash::CellBlockDimLog2;
		// Note float size, this is intentional, the idea should be to never have cell sizes of unusual size 
		const float LevelCellSize = SpatialHash.GetCellSize(Level);

		// TODO[Opt]: may be computed as a relative from the previous level, needs to be adjusted for skipping levels:
		//    Expand by 1 (half a cell on the next level) before dividing to maintain looseness
		//      LightFootprint.Min -= FInt64Vector3(1);
		//      LightFootprint.Max += FInt64Vector3(1);
		//      LightFootprint = ToLevelRelative(LightFootprint, 1);
		FFootprint64 LightFootprint = SpatialHash.CalcFootprintSphere(Level, Sphere.Center, Sphere.W + (LevelCellSize * 0.5f));

		FFootprint64 BlockFootprint = SpatialHash.CalcCellBlockFootprint(LightFootprint);
		check(BlockFootprint.Level == BlockLevel);
		const float BlockSize = SpatialHash.GetCellSize(BlockFootprint.Level);

		// Loop over footprint
		BlockFootprint.ForEach([&](const FLocation64& BlockLoc)
		{
			// TODO[Opt]: Add cache for block ID lookups? The hash lookup is somewhat costly and we hit it quite a bit due to the loose footprint.
			//       Could be a 3d grid/level (or not?) with modulo and use the BlockLoc as key. Getting very similar to just using a cheaper hash...
			FSpatialHash::FBlockId BlockId = GlobalSpatialHash.FindId(FBlockLoc(BlockLoc));
			if (BlockId.IsValid())
			{
				const FSpatialHash::FCellBlock& Block = GlobalSpatialHash.GetByElementId(BlockId).Value;
				FVector3d BlockWorldPos = FVector3d(BlockLoc.Coord) * double(BlockSize);

				// relative query offset, float precision.
				// This is probably not important on PC, but on GPU the block world pos can be precomputed on host and this gets us out of large precision work
				// Expand by 1/2 cell size for loose
				FSphere3f BlockLocalSphere(FVector3f(Sphere.Center - BlockWorldPos), float(Sphere.W) + (LevelCellSize * 0.5f));

				FFootprint8 LightFootprintInBlock = ToBlockLocal(LightFootprint, BlockLoc);

				// Calc block mask 
				// TODO[Opt]: We can make a table of this and potentially save a bit of work here
				uint64 LightCellMask = FSpatialHash::FCellBlock::BuildFootPrintMask(LightFootprintInBlock);

				if ((Block.CoarseCellMask & LightCellMask) != 0ULL)
				{
					LightFootprintInBlock.ForEach([&](const FLocation8& CellSubLoc)
					{
						if ((Block.CoarseCellMask & FSpatialHash::FCellBlock::CalcCellMask(CellSubLoc.Coord)) != 0ULL)
						{
							// optionally test the cell bounds against the query
							// 1. Make local bounding box (we could do a global one but this is more GPU friendly)
							// Note: not expanded because the query is:
							FBox3f Box;
							Box.Min = FVector3f(CellSubLoc.Coord) * LevelCellSize;
							Box.Max = Box.Min + LevelCellSize;

							bool bIntersects = true;

							if (bTestCellVsQueryBounds)
							{
								if (!FMath::SphereAABBIntersection(BlockLocalSphere, Box))
								{
									bIntersects = false;
								}
							}
							if (bIntersects)
							{
								uint32 CellId = Block.GetCellGridOffset(CellSubLoc.Coord);
								ResultConsumer.OnCellOverlap(CellId);
							}
						}
					});
				}
			}
		});
	}
}

inline bool IsValidCell(const FCellHeader& CellHeader)
{
	return CellHeader.bIsValid;
}

inline bool IsValidCell(const FPackedCellHeader& CellHeader)
{
	// For a valid cell the value is always nonzero
	return CellHeader.Packed0 != 0u;
}

inline FCellHeader UnpackCellHeader(const FPackedCellHeader& Packed)
{
	const uint64& Bits = reinterpret_cast<const uint64&>(Packed);
	FCellHeader CellHeader;
	CellHeader.ItemChunksOffset = uint32(Bits >> (2u * INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS)) & ((1u << INSTANCE_HIERARCHY_CELL_HEADER_OFFSET_BITS) - 1u);
	CellHeader.NumStaticChunks  = uint32(Bits >> INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS) & ((1u << INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS) - 1u);
	CellHeader.NumDynamicChunks  = uint32(Bits) & ((1u << INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS) - 1u);
	CellHeader.bIsValid = CellHeader.NumDynamicChunks != 0u;
	if (CellHeader.bIsValid)
	{
		CellHeader.NumDynamicChunks -= 1u;
	}
	CellHeader.NumItemChunks = CellHeader.NumDynamicChunks + CellHeader.NumStaticChunks;
	return CellHeader;
}
