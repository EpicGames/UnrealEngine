// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneCullingRenderer.h"
#include "SceneCulling.h"
#include "SceneCulling.inl"
#include "RenderGraphUtils.h"
#include "SceneRendererInterface.h"
#include "SystemTextures.h"
#include "SceneRendering.h"
#include "ShaderPrintParameters.h"
#include "ScenePrivate.h"
#include "UnrealEngine.h"
#include "SceneManagement.h"

static TAutoConsoleVariable<int32> CVarSceneCullingDebugRenderMode(
	TEXT("r.SceneCulling.DebugRenderMode"), 
	0, 
	TEXT("SceneCulling debug render mode.\n")
	TEXT(" 0 = Disabled (default)\n")
	TEXT(" 1 = Enabled"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CvarSceneCullingCullChunkViewDistance(
	TEXT("r.SceneCulling.CullChunkViewDistance"), 
	false, 
	TEXT("Set to true (default is false) to enable per-chunk view distance culling.\n")
	TEXT("  For some scenes it may increase overhead due to poor load balancing in the cell culling. This may be mitigated by lowering r.SceneCulling.MinCellSize"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarUseExplicitBoundsForCulling(
	TEXT("r.SceneCulling.UseExplicitBoundsForCulling"),
	true,
	TEXT("When true, use explicit per cell bounds (if available) for CPU scene culling instead of implicit spatial hash cell bounds."),
	ECVF_RenderThreadSafe);

void FSceneCullingRenderer::PreVisibilityTasks(FRDGBuilder& GraphBuilder)
{
	bIsHierarchicalCPUCullingActive = SceneCulling.bHierarchicalCPUCullingEnabled && SceneCulling.GetMaxHashGroupLODOffset() > 0;
	bUseExplicitBounds = CVarUseExplicitBoundsForCulling.GetValueOnRenderThread();

	// Don't create anything if there's no work to do
	if (bIsHierarchicalCPUCullingActive)
	{
		FSceneRendererBase& SceneRenderer = GetSceneRenderer();
		// By definition the primary view culling queries are first in the array
		PrimaryCullingQueries.Reserve(SceneRenderer.Views.Num());
		for (const FViewInfo& View : SceneRenderer.Views)
		{
			PrimaryCullingQueries.Add(CreateHierarchicalCullingQuery(FCullingVolume(View.GetCullingFrustum())));
		}

		// Mark all primitives (using the compact ID so we don't have to translate before looking up in the culling loop).
		FScene& Scene = SceneCulling.Scene;
		PrimitivesWithHierachicalCullingInfoMap.SetNum(Scene.Primitives.Num(), false);

		for (const auto& InfoItem : SceneCulling.CPUCullingInfos)
		{
			FPersistentPrimitiveIndex PersistentPrimitiveIndex = InfoItem.Key;

			int32 Index = Scene.GetPrimitiveIndex(PersistentPrimitiveIndex);
			if (Scene.Primitives.IsValidIndex(Index) && !InfoItem.Value.Items.IsEmpty())
			{
				PrimitivesWithHierachicalCullingInfoMap[Index] = true;
			}
		}
	}
}

FInstanceHierarchyParameters& FSceneCullingRenderer::GetShaderParameters(FRDGBuilder& GraphBuilder) 
{ 
	// Sync any update that is in progress.
	SceneCulling.EndUpdate(GraphBuilder, true);

	// This should not need to be done more than once per frame
	if (CellHeadersRDG == nullptr)
	{
		CellBlockDataRDG = SceneCulling.CellBlockDataBuffer.Register(GraphBuilder);
		CellHeadersRDG = SceneCulling.CellHeadersBuffer.Register(GraphBuilder);
		ItemChunksRDG = SceneCulling.ItemChunksBuffer.Register(GraphBuilder);
		InstanceIdsRDG = SceneCulling.InstanceIdsBuffer.Register(GraphBuilder);
		UsedChunkIdMaskRDG = GraphBuilder.RegisterExternalBuffer(SceneCulling.UsedChunkIdMaskBuffer);
		ExplicitChunkBoundsRDG = SceneCulling.ExplicitChunkBoundsBuffer.Register(GraphBuilder);
		ExplicitChunkCellIdsRDG = SceneCulling.ExplicitChunkCellIdsBuffer.Register(GraphBuilder);

#if 0
		// Fully upload the buffers for debugging. 
		CellBlockDataRDG = CreateStructuredBuffer(GraphBuilder, TEXT("SceneCulling.BlockData"), SceneCulling.CellBlockData);
		CellHeadersRDG = CreateStructuredBuffer(GraphBuilder, TEXT("SceneCulling.CellHeaders"), SceneCulling.CellHeaders);
		ItemChunksRDG = CreateStructuredBuffer(GraphBuilder, TEXT("SceneCulling.ItemChunks"), SceneCulling.PackedCellChunkData);
		InstanceIdsRDG = CreateStructuredBuffer(GraphBuilder, TEXT("SceneCulling.Items"), SceneCulling.PackedCellData);
#endif

		ShaderParameters.NumCellsPerBlockLog2 = FSpatialHash::NumCellsPerBlockLog2;
		ShaderParameters.CellBlockDimLog2 = FSpatialHash::CellBlockDimLog2;
		ShaderParameters.LocalCellCoordMask = (1U << FSpatialHash::CellBlockDimLog2) - 1U;
		ShaderParameters.FirstLevel = SceneCulling.SpatialHash.GetFirstLevel();
		ShaderParameters.bCullChunkViewDistance = CvarSceneCullingCullChunkViewDistance.GetValueOnRenderThread();
		ShaderParameters.InstanceHierarchyCellBlockData = GraphBuilder.CreateSRV(CellBlockDataRDG);
		ShaderParameters.InstanceHierarchyCellHeaders = GraphBuilder.CreateSRV(CellHeadersRDG);
		ShaderParameters.InstanceIds = GraphBuilder.CreateSRV(InstanceIdsRDG);
		ShaderParameters.UsedChunkIdMask = GraphBuilder.CreateSRV(UsedChunkIdMaskRDG);
		ShaderParameters.InstanceHierarchyItemChunks = GraphBuilder.CreateSRV(ItemChunksRDG);
		ShaderParameters.ExplicitChunkBounds = GraphBuilder.CreateSRV(ExplicitChunkBoundsRDG);
		ShaderParameters.ExplicitChunkCellIds = GraphBuilder.CreateSRV(ExplicitChunkCellIdsRDG);
		ShaderParameters.NumAllocatedChunks = SceneCulling.CellChunkIdAllocator.GetMaxSize();
	}

	return ShaderParameters; 
}

FSceneInstanceCullingQuery* FSceneCullingRenderer::CullInstances(FRDGBuilder& GraphBuilder, const TConstArrayView<FConvexVolume>& ViewCullVolumes)
{
	SCOPED_NAMED_EVENT(FSceneCullingRenderer_CullInstances, FColor::Emerald);

	if (SceneCulling.IsEnabled())
	{
		FSceneInstanceCullingQuery* Query = GraphBuilder.AllocObject<FSceneInstanceCullingQuery>(*this);

		for (int32 Index = 0; Index < ViewCullVolumes.Num(); ++Index)
		{
			FCullingVolume CullingVolume;
			// Assume world-space
			CullingVolume.WorldToVolumeTranslation = FVector3d::ZeroVector;
			CullingVolume.ConvexVolume = ViewCullVolumes[Index];
			Query->Add(Index, 1, CullingVolume);
		}

		Query->Dispatch(GraphBuilder);

		return Query;
	}
	return nullptr;
}


class FSceneCullingDebugRender_CS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSceneCullingDebugRender_CS);
	SHADER_USE_PARAMETER_STRUCT(FSceneCullingDebugRender_CS, FGlobalShader);

	static constexpr int32 NumThreadsPerGroup = 64;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) 
	{ 
		return DoesPlatformSupportNanite(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), 1);
		OutEnvironment.SetDefine(TEXT("NUM_THREADS_PER_GROUP"), NumThreadsPerGroup);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE( ShaderPrint::FShaderParameters, ShaderPrintUniformBuffer )
		SHADER_PARAMETER_STRUCT_INCLUDE( FInstanceHierarchyParameters, InstanceHierarchyParameters )
		SHADER_PARAMETER(FVector3f, PickingRayStart)
		SHADER_PARAMETER(FVector3f, PickingRayEnd)
		SHADER_PARAMETER(int32, DebugMode)
		SHADER_PARAMETER(int32, MaxCells)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWStructuredBuffer<uint32>, RWDrawCellInfoCounter)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint32>, ValidCellsMask)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FSceneCullingDebugRender_CS, "/Engine/Private/SceneCulling/SceneCullingDebugRender.usf", "DebugRender", SF_Compute);


void FSceneCullingRenderer::DebugRender(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views)
{
#if !UE_BUILD_SHIPPING
	int32 MaxCellCount = SceneCulling.CellHeaders.Num();

	int32 DebugMode = CVarSceneCullingDebugRenderMode.GetValueOnRenderThread();
	if (DebugMode != 0 && MaxCellCount > 0)
	{
		// Force ShaderPrint on.
		ShaderPrint::SetEnabled(true); 

		// This lags by one frame, so may miss some in one frame, also overallocates since we will cull a lot.
		ShaderPrint::RequestSpaceForLines(MaxCellCount * 12 * Views.Num());

		// Note: we have to construct this as the GPU currently does not have a mapping of what cells are valid.
		//       Normally this comes from the CPU during the broad phase culling. Thus it is only needed here for debug purposes.
		TBitArray<> ValidCellsMask(false, MaxCellCount);
		for (int32 Index = 0; Index < MaxCellCount; ++Index)
		{
			ValidCellsMask[Index] =  IsValidCell(SceneCulling.CellHeaders[Index]); 
		}
		FRDGBuffer* ValidCellsMaskRdg = CreateStructuredBuffer(GraphBuilder, TEXT("SceneCulling.Debug.ValidCellsMaskRdg"), TConstArrayView<uint32>(ValidCellsMask.GetData(), FBitSet::CalculateNumWords(ValidCellsMask.Num())));

		for (auto &View : Views)
		{
			if (ShaderPrint::IsEnabled(View.ShaderPrintData))
			{	
				FRDGBufferRef DrawCellInfoCounterRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(4, 1), TEXT("SceneCulling.Debug.DrawCellInfoCounter"));
				FRDGBufferUAVRef DrawCellInfoCounterUAV = GraphBuilder.CreateUAV(DrawCellInfoCounterRDG);
				AddClearUAVPass(GraphBuilder, DrawCellInfoCounterUAV, 0u);

				FSceneCullingDebugRender_CS::FParameters* PassParameters = GraphBuilder.AllocParameters<FSceneCullingDebugRender_CS::FParameters>();
				ShaderPrint::SetParameters(GraphBuilder, PassParameters->ShaderPrintUniformBuffer);
				PassParameters->InstanceHierarchyParameters = GetShaderParameters(GraphBuilder);
				PassParameters->MaxCells = MaxCellCount;
				PassParameters->ValidCellsMask = GraphBuilder.CreateSRV(ValidCellsMaskRdg);
				PassParameters->DebugMode = DebugMode;
				FVector PickingRayStart(ForceInit);
				FVector PickingRayDir(ForceInit);
				FIntPoint CursorPos = View.CursorPos;
				if (CursorPos.GetMin() < 0)
				{
					CursorPos = View.ViewRect.Size() / 2;
				}
				View.DeprojectFVector2D(CursorPos, PickingRayStart, PickingRayDir);
				PassParameters->PickingRayStart = FVector3f(PickingRayStart);
				PassParameters->PickingRayEnd = FVector3f(PickingRayStart + PickingRayDir * WORLD_MAX);
				PassParameters->RWDrawCellInfoCounter = DrawCellInfoCounterUAV;

				auto ComputeShader = View.ShaderMap->GetShader<FSceneCullingDebugRender_CS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("SceneCullingDebugRender"),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCountWrapped(MaxCellCount, FSceneCullingDebugRender_CS::NumThreadsPerGroup)
				);
			}
		}
	}
#endif
}

FSceneInstanceCullingQuery* FSceneCullingRenderer::CreateInstanceQuery(FRDGBuilder& GraphBuilder)
{
	SCOPED_NAMED_EVENT(FSceneCullingRenderer_CullInstances, FColor::Emerald);

	if (SceneCulling.IsEnabled())
	{
		FSceneInstanceCullingQuery* Query = GraphBuilder.AllocObject<FSceneInstanceCullingQuery>(*this);
		return Query;
	}
	return nullptr;
}

int32 FSceneInstanceCullingQuery::AddViewDrawGroup(uint32 FirstPrimaryView, uint32 NumPrimaryViews)
{
	check(!CullingResult);
	check(!AsyncTaskHandle.IsValid());

	if (SceneCullingRenderer.IsEnabled())
	{
		FViewDrawGroup ViewDrawGroup;
		ViewDrawGroup.FirstView = FirstPrimaryView;
		ViewDrawGroup.NumViews = NumPrimaryViews;
		int32 ViewGroupId = ViewDrawGroups.Add(ViewDrawGroup);
		return ViewGroupId;
	}
	return INDEX_NONE;
}

void FSceneInstanceCullingQuery::Add(uint32 FirstPrimaryView, uint32 NumPrimaryViews, const FCullingVolume& CullingVolume)
{
	check(!CullingResult);
	check(!AsyncTaskHandle.IsValid());

	if (SceneCullingRenderer.IsEnabled())
	{
		FCullingJob Job;
		Job.CullingVolume = CullingVolume;
		Job.ViewGroupId = AddViewDrawGroup(FirstPrimaryView, NumPrimaryViews);
		CullingJobs.Add(Job);
	}
}

void FSceneInstanceCullingQuery::Dispatch(FRDGBuilder& GraphBuilder, bool bInAllowAsync)
{
	check(!CullingResult);
	check(!AsyncTaskHandle.IsValid());

	const bool bAllowAsync = SceneCullingRenderer.SceneCulling.bUseAsyncUpdate && bInAllowAsync;

	if (!CullingJobs.IsEmpty())
	{
		// Must wait if this query is not running async or we might race against the update task.
		UE::Tasks::FTask UpdateTaskHandle = SceneCullingRenderer.SceneCulling.GetUpdateTaskHandle();
		if (!bAllowAsync && UpdateTaskHandle.IsValid())
		{
			UpdateTaskHandle.Wait();
		}

		CullingResult = GraphBuilder.AllocObject<FSceneInstanceCullResult>();

		AsyncTaskHandle = GraphBuilder.AddSetupTask([this]()
		{
			ComputeResult();
		},
		nullptr, TArray<UE::Tasks::FTask>{ UpdateTaskHandle }, UE::Tasks::ETaskPriority::High, bAllowAsync);
	}
}

FSceneInstanceCullResult* FSceneInstanceCullingQuery::GetResult()
{
	SCOPED_NAMED_EVENT(FSceneInstanceCullingQuery_GetResult, FColor::Emerald);

	if (AsyncTaskHandle.IsValid())
	{
		AsyncTaskHandle.Wait();
	}

	return CullingResult;
}

void FSceneInstanceCullingQuery::ComputeResult()
{
	SCOPED_NAMED_EVENT(FSceneInstanceCullingQuery_ComputeResult, FColor::Emerald);

	const FSceneCulling& SceneCulling = SceneCullingRenderer.SceneCulling;
	// loop and append all results
	for (const FCullingJob& CullingJob : CullingJobs)
	{
		if (SceneCulling.IsSmallCullingVolume(CullingJob.CullingVolume))
		{
			struct FResultConsumer
			{
				FSceneInstanceCullResult* CullingResult;
				uint32 ViewGroupId;
				const TArray<FPackedCellHeader>& CellHeaders;

				void OnCellOverlap(uint32 CellId)
				{
					FCellHeader CellHeader = UnpackCellHeader(CellHeaders[CellId]);
					if (IsValidCell(CellHeader))
					{
						CullingResult->CellChunkDraws.Add(FCellChunkDraw{ CellHeader.ItemChunksOffset, ViewGroupId }, CellHeader.NumItemChunks);
					}
				}
			};

			FResultConsumer ResultConsumer { CullingResult, CullingJob.ViewGroupId, SceneCulling.CellHeaders };

			SceneCulling.TestSphere(CullingJob.CullingVolume.Sphere, ResultConsumer);
		}
		else 
		{
			// broad phase test, go wide over chunks will dispatch one thread per view group ID
			CullingResult->ChunkCullViewGroupIds.Add(CullingJob.ViewGroupId);
		}

	}

	CullingResult->CellChunkDraws.FinalizeBatches();
	CullingResult->NumAllocatedChunks = SceneCulling.CellChunkIdAllocator.GetMaxSize();
	// All chunks may possibly be occluded in the first pass (except the uncullable ones).
	CullingResult->MaxOccludedChunkDraws = SceneCulling.CellChunkIdAllocator.GetSparselyAllocatedSize() * CullingResult->ChunkCullViewGroupIds.Num()
		+ CullingResult->CellChunkDraws.GetTotalChildren();
	CullingResult->UncullableNumItemChunks = SceneCullingRenderer.SceneCulling.UncullableNumItemChunks;
	// All chunks (plus the uncullable, once per group) may potentially get through culling in the first pass.
	CullingResult->NumInstanceGroups += CullingResult->MaxOccludedChunkDraws 
		+ CullingResult->UncullableNumItemChunks * CullingJobs.Num();
	CullingResult->UncullableItemChunksOffset = SceneCullingRenderer.SceneCulling.UncullableItemChunksOffset;
	CullingResult->SceneCullingRenderer = &SceneCullingRenderer;
}

FSceneInstanceCullingQuery::FSceneInstanceCullingQuery(FSceneCullingRenderer& InSceneCullingRenderer)
	: SceneCullingRenderer(InSceneCullingRenderer)
{
}

TOptional<FLODMask> FSceneCullingRendererHierarchicalCullingQuery::ComputeCulling(int32 ScenePrimitiveIndex, const FBoxSphereBounds& PrimitiveBounds, const FSceneView& View, int32 ForcedLODLevel, const FDesiredLODLevel& DesiredLODLevel, float LODScale)
{
	const FScene& Scene = SceneCullingRenderer.SceneCulling.Scene;
	check(Scene.Primitives.IsValidIndex(ScenePrimitiveIndex));
	if (!SceneCullingRenderer.PrimitivesWithHierachicalCullingInfoMap[ScenePrimitiveIndex])
	{
		return TOptional<FLODMask>{};
	}

	// This ensures consistent view origin for LOD wrt ComputeLODForMeshes
	const FSceneView& LODView = GetLODView(View);
	// LODScale already incorporates static mesh lod scale 
	const FMatrix& ProjMatrix = LODView.ViewMatrices.GetViewToClip();
	const float ScreenMultipleForLOD = FMath::Max(ProjMatrix.M[0][0], ProjMatrix.M[1][1]) / LODScale;
	const FVector ViewOriginForLOD = LODView.ViewMatrices.GetViewOrigin();

	const FPrimitiveSceneInfo* PrimitiveSceneInfo = Scene.Primitives[ScenePrimitiveIndex];
	const float InstanceSphereRadius = PrimitiveSceneInfo->GpuLodInstanceRadius;
	
	// This is guaranteed by the setup, there should never be a culling info set up for a primitive without instance radius.
	check(InstanceSphereRadius > 0.0f);

	// Run per-cell frustum + distance culling: populates HashGroupVisibilityMask and HashGroupMinMaxFootprints.
	const FSceneCulling::FPrimitiveCullingInfo* CullingInfoPtr = GetCullingInfo(PrimitiveSceneInfo->GetPersistentIndex());
	check(CullingInfoPtr != nullptr);
	const FSceneCulling::FPrimitiveCullingInfo& CullingInfo = *CullingInfoPtr;
	check(!CullingInfo.Items.IsEmpty());

	bool bHasMinMaxDist = CullingInfo.MaxInstanceDrawDistance >= 0.0f;
	// NOTE: must match the GPU side instance distace cull, which is currently baking the ViewDistanceScale into the GPU scene data(!), so it only works if the primitives are uploaded after the scale is changed.
	const float MaxDrawDistSquared = bHasMinMaxDist ? FMath::Square(CullingInfo.MaxInstanceDrawDistance * GetCachedScalabilityCVars().ViewDistanceScale) : std::numeric_limits<float>::max();

	const bool bUseExplicit = SceneCullingRenderer.bUseExplicitBounds && CullingInfo.Items[0].ExplicitBounds.Min.X < CullingInfo.Items[0].ExplicitBounds.Max.X;
	float NearDist = std::numeric_limits<float>::max();
	float FarDist  = 0.0f;
	for (int32 ItemIndex = 0; ItemIndex < CullingInfo.Items.Num(); ++ItemIndex)
	{
		const auto& Item = CullingInfo.Items[ItemIndex];

		FVector3d CullCenter = FSceneCulling::FSpatialHash::CalcCellCenter(Item.Location);
		FVector3d CullExtent;
		float CullBaseRadius;

		if (bUseExplicit && Item.ExplicitBounds.Min.X < Item.ExplicitBounds.Max.X)
		{
			// Offset to explicit, relative location
			CullCenter += FVector3d(Item.ExplicitBounds.GetCenter());
			CullExtent = FVector3d(Item.ExplicitBounds.GetExtent());
			CullBaseRadius = Item.ExplicitBounds.GetExtent().Length();
		}
		else
		{
			// NOTE: We don't use the loose cell bounds as we have access to the instance radius, which means the max size for culling purposes is the base cell size + instance radius.
			// Note: can be relatively easily converted to single-precision if we rebase to e.g. primitive location. However, frustums are in world space.
			const float CellSize = float(FSceneCulling::FSpatialHash::GetCellSize(Item.Location.Level - 1));
			CullExtent = FVector(CellSize + InstanceSphereRadius);
			CullBaseRadius = CellSize * UE_SQRT_3;
		}

		// Min distance is if instance is positioned at the near edge of the bounding region
		float MinDistanceSquared = FMath::Max(1.0f, ComputeSquaredDistanceFromBoxToPoint(CullCenter - CullExtent, CullCenter + CullExtent, ViewOriginForLOD));

		bool bWithinMaxDrawDistance = MinDistanceSquared <= MaxDrawDistSquared;
		if (bWithinMaxDrawDistance && CullingVolume.IntersectBox(CullCenter, CullExtent))
		{
			HashGroupVisibilityMask[CullingInfo.HashGroupVisibilityOffset * NumBitsPerDWORD + ItemIndex] = true;

			// Compute conservative min/max instance screen-space radius for LOD pruning
			float MinDistance = FMath::Sqrt(MinDistanceSquared);
			float MaxDistance = MinDistance + CullBaseRadius * 2.0f;

			FVector2f Footprint = FVector2f(
				ScreenMultipleForLOD * InstanceSphereRadius / MaxDistance,
				ScreenMultipleForLOD * InstanceSphereRadius / MinDistance);

			HashGroupMinMaxFootprints[CullingInfo.HashGroupLODOffset + ItemIndex] = Footprint;

			NearDist = FMath::Min(NearDist, MinDistance);
			FarDist  = FMath::Max(FarDist,  MaxDistance);
		}
	}

	// No cells passed culling: suppress all draw calls for this primitive.
	if (FarDist <= 0.0f)
	{
		return FLODMask();
	}

	// TODO: This does not support the dithered LOD transition because that needs to take into account both potential origins to figure out the conservative min/max footprint.
	// TODO: Refactor the LOD compute functions to be more modular and pass footprint directly.
	const FVector ToPrimNorm = (PrimitiveBounds.Origin - ViewOriginForLOD).GetSafeNormal(UE_SMALL_NUMBER, /*return arbitrary direction since it is only used to compute a distance in the end. */ FVector(0.0, 0.0, 1.0));
	const FVector VirtualNearOrigin = ViewOriginForLOD + ToPrimNorm * NearDist;
	const FVector VirtualFarOrigin  = ViewOriginForLOD + ToPrimNorm * FarDist;

	// Compute LOD at each virtual origin and combine into a range covering all visible cells.
	float MeshScreenSizeSquared = 0.0f;
	const FLODMask MinLod = ComputeLODForMeshes(PrimitiveSceneInfo->StaticMeshRelevances, View, VirtualNearOrigin, InstanceSphereRadius, ForcedLODLevel, MeshScreenSizeSquared, DesiredLODLevel.LOD, LODScale);
	const FLODMask MaxLod = ComputeLODForMeshes(PrimitiveSceneInfo->StaticMeshRelevances, View, VirtualFarOrigin,  InstanceSphereRadius, ForcedLODLevel, MeshScreenSizeSquared, DesiredLODLevel.LOD, LODScale);
	FLODMask Result;
	Result.SetLODRange(MinLod.LODIndex0, MaxLod.LODIndex0);

	return Result;
}

FSceneCullingRendererHierarchicalCullingQuery* FSceneCullingRenderer::CreateHierarchicalCullingQuery(const FCullingVolume& InCullingVolume)
{
	if (UseHierarchicalCPUCulling())
	{
		check(SceneCulling.GetMaxHashGroupVisibilityOffset() > 0);
		return GetSceneRenderer().Allocator.Create<FSceneCullingRendererHierarchicalCullingQuery>(
			InCullingVolume, 
			*this
		);
	}
	return nullptr;
}

FSceneCullingRendererHierarchicalCullingQuery::FSceneCullingRendererHierarchicalCullingQuery(const FCullingVolume& InCullingVolume, FSceneCullingRenderer& InSceneCullingRenderer)
	: CullingVolume(InCullingVolume)
	, SceneCullingRenderer(InSceneCullingRenderer)
	, HashGroupVisibilityMask(false, SceneCullingRenderer.SceneCulling.GetMaxHashGroupVisibilityOffset())
{
	HashGroupMinMaxFootprints.SetNum(SceneCullingRenderer.SceneCulling.GetMaxHashGroupLODOffset());
}

const FSceneCulling::FPrimitiveCullingInfo* FSceneCullingRendererHierarchicalCullingQuery::GetCullingInfo(FPersistentPrimitiveIndex PersistentPrimitiveIndex)
{
	return SceneCullingRenderer.SceneCulling.GetCullingInfo(PersistentPrimitiveIndex);
}
