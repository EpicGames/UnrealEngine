// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneCulling.h"
#include "GPUWorkGroupLoadBalancer.h"
#include "SceneManagement.h"

class FSceneCulling;
class FSceneInstanceCullResult;
class FSceneInstanceCullingQuery;
class FSceneCullingRenderer;
class FSceneCullingRendererHierarchicalCullingQuery;
struct FDesiredLODLevel;

BEGIN_SHADER_PARAMETER_STRUCT( FInstanceHierarchyParameters, )
	SHADER_PARAMETER(uint32, NumCellsPerBlockLog2)
	SHADER_PARAMETER(uint32, CellBlockDimLog2)
	SHADER_PARAMETER(uint32, LocalCellCoordMask) // (1 << NumCellsPerBlockLog2) - 1
	SHADER_PARAMETER(int32, FirstLevel)
	SHADER_PARAMETER(uint32, bCullChunkViewDistance)
	SHADER_PARAMETER(uint32, NumAllocatedChunks)

	SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< FCellBlockData >, InstanceHierarchyCellBlockData)
	SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< FPackedCellHeader >, InstanceHierarchyCellHeaders)
	SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< uint >, InstanceHierarchyItemChunks)
	SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< uint >, InstanceIds)
	SHADER_PARAMETER_RDG_BUFFER_SRV( ByteAddressBuffer, ExplicitChunkBounds)
	SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< uint32 >, ExplicitChunkCellIds)
	SHADER_PARAMETER_RDG_BUFFER_SRV( StructuredBuffer< uint >, UsedChunkIdMask)
END_SHADER_PARAMETER_STRUCT()

/**
 * Renderer-lifetime functionality, provides scope for anything that should share life-time with a Scene Renderer, rather than Scene.
 */
class FSceneCullingRenderer : public ISceneExtensionRenderer
{
	DECLARE_SCENE_EXTENSION_RENDERER(FSceneCullingRenderer, FSceneCulling);
public:
	friend class FSceneInstanceCullingQuery;
	friend class FSceneCullingRendererHierarchicalCullingQuery;

	FSceneCullingRenderer(FSceneRendererBase& InSceneRenderer, FSceneCulling& InSceneCulling) : ISceneExtensionRenderer(InSceneRenderer), SceneCulling(InSceneCulling) {}

	//~ Begin ISceneExtensionRenderer Interface.
	virtual void PreVisibilityTasks(FRDGBuilder& GraphBuilder) override;
	//~ End ISceneExtensionRenderer Interface.

	inline bool IsEnabled() const { return SceneCulling.IsEnabled(); }

	/**
	 * Getting the shader parameters forces a sync wrt the hierarchy update, since we need to resize the GPU buffers at this point.
	 */
	FInstanceHierarchyParameters& GetShaderParameters(FRDGBuilder& GraphBuilder);
	
	/**
	 * Create and dispatch a culling query for a set of views that has a 1:1 mapping from culling volume to view index
	 * May run async.
	 */
	FSceneInstanceCullingQuery* CullInstances(FRDGBuilder& GraphBuilder, const TConstArrayView<FConvexVolume>& ViewCullVolumes);
	FSceneInstanceCullingQuery* CullInstances(FRDGBuilder& GraphBuilder, const FConvexVolume& ViewCullVolume) { return CullInstances(GraphBuilder, TConstArrayView<FConvexVolume>(&ViewCullVolume, 1)); }

	/**
	 * Create a query that is not immediately dispatched, such that jobs can be added first.
	 */
	FSceneInstanceCullingQuery* CreateInstanceQuery(FRDGBuilder& GraphBuilder);

	/**
	 */
	void DebugRender(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views);

	/**
	 * Returns true if the system is enabled and there exists any primitives providing hierarchical instance culling data.
	 */
	bool UseHierarchicalCPUCulling() const { return bIsHierarchicalCPUCullingActive; }

	/** 
	 * Create a query for hierarchical CPU instance culling with a lifetime managed by FSceneRenderer::Allocator.
	 * Safe to call from multiple threads.
	 */
	FSceneCullingRendererHierarchicalCullingQuery* CreateHierarchicalCullingQuery(const FCullingVolume& InCullingVolume);

	/**
	 * Primary (SceneRenderer.View) queries are always created if the data is available and can be accessed through the view index.
	 */
	FSceneCullingRendererHierarchicalCullingQuery* GetPrimaryViewHierarchicalCullingQuery(int32 ViewIndex)
	{
		if (PrimaryCullingQueries.IsValidIndex(ViewIndex))
		{
			return PrimaryCullingQueries[ViewIndex];
		}
		return nullptr;
	};

private:
	FSceneCulling& SceneCulling;
	using FSpatialHash = FSceneCulling::FSpatialHash;

	FInstanceHierarchyParameters ShaderParameters;
	FRDGBuffer* CellHeadersRDG = nullptr;
	FRDGBuffer* ItemChunksRDG = nullptr;
	FRDGBuffer* InstanceIdsRDG = nullptr;
	FRDGBuffer* CellBlockDataRDG = nullptr;
	FRDGBuffer* ExplicitChunkBoundsRDG = nullptr;
	FRDGBuffer* ExplicitChunkCellIdsRDG = nullptr;
	FRDGBuffer* UsedChunkIdMaskRDG = nullptr;

	TArray<FSceneCullingRendererHierarchicalCullingQuery*, TInlineAllocator<4, SceneRenderingAllocator>> PrimaryCullingQueries;
	// Bit map using non-persistent primitive index to determine if a primitive has any culling data.
	TBitArray<SceneRenderingAllocator> PrimitivesWithHierachicalCullingInfoMap;
	bool bIsHierarchicalCPUCullingActive = false;
	bool bUseExplicitBounds = false;
};

/**
 */
class FSceneInstanceCullingQuery
{
public:
	FSceneInstanceCullingQuery(FSceneCullingRenderer& InSceneCullingRenderer);

	/**
	 * Add a view-group without culling job to the query. The return value is the index of the view-group.
	 */
	int32 AddViewDrawGroup(uint32 FirstPrimaryView, uint32 NumPrimaryViews);

	/**
	 * Add a job for a view-group to the query.
	 */
	void Add(uint32 FirstPrimaryView, uint32 NumPrimaryViews, const FCullingVolume& CullingVolume);

	/**
	 * Run culling job.
	 */
	void Dispatch(FRDGBuilder& GraphBuilder, bool bAllowAsync = true);

	// Get GPU stuffs, TBD
	FSceneInstanceCullResult* GetResult();

	/**
	 * Get pointer to the result async, note that this means it may still be in the process of being filled in. 
	 * It is not safe to access anything in the result until the task has been waited on, either by calling GetResult()
	 * or GetAsyncTaskHandle().Wait().
	 * The data is always allocated on the RDG timeline so is safe to keep a pointer to in e.g., RDG setup tasks and other renderer-lifetime structures.
	 */
	FSceneInstanceCullResult* GetResultAsync() const { return CullingResult; }

	/**
	 * returns true if the task is running async
	 * NOTE: may return false for a task that has completed, even if it was spawned as an asyc task.
	 */
	bool IsAsync() const { return AsyncTaskHandle.IsValid() && !AsyncTaskHandle.IsCompleted(); }

	/**
	 * Get the task handle to be able to queue subsequent work, for example.
	 */
	UE::Tasks::FTask GetAsyncTaskHandle() const { return AsyncTaskHandle; }

	FSceneCullingRenderer& GetSceneCullingRenderer() { return SceneCullingRenderer; }
	
	TConstArrayView<FViewDrawGroup> GetViewDrawGroups() const
	{
		return ViewDrawGroups;
	}

private:
	void ComputeResult();

	FSceneCullingRenderer& SceneCullingRenderer;

	using FViewDrawGroups = TArray<FViewDrawGroup, SceneRenderingAllocator>;
	FViewDrawGroups ViewDrawGroups;

	struct FCullingJob
	{
		FCullingVolume CullingVolume;
		int32 ViewGroupId = 0;
	};

	TArray<FCullingJob, SceneRenderingAllocator> CullingJobs;

	FSceneInstanceCullResult *CullingResult = nullptr;

	UE::Tasks::FTask AsyncTaskHandle;
};


/**
 * TODO: This should be moved to Nanite & the testing interface generalized to allow this.
 */
class FSceneInstanceCullResult
{
public:
	using FCellChunkDraws = TGPUWorkGroupLoadBalancer<FCellChunkDraw>;
	// The list of cell/view-group pairs to feed to rendering
	FCellChunkDraws CellChunkDraws;
	// List of view group IDs (indexing into the query) that should be culled on a per chunk basis.
	using FChunkCullViewGroupIds = TArray<uint32, SceneRenderingAllocator>;
	FChunkCullViewGroupIds ChunkCullViewGroupIds;
	uint32 NumInstanceGroups = 0;
	// This is the number of occluded chunks that might be emitted (if everything is occluded in the Main pass).
	int32 MaxOccludedChunkDraws = 0;
	// This is the number of allocated chunks, we run a thread for each and skip those that are not currently in use.
	uint32 NumAllocatedChunks = 0u;
	FSceneCullingRenderer* SceneCullingRenderer = nullptr;
	uint32 UncullableItemChunksOffset = 0;
	uint32 UncullableNumItemChunks = 0;
};

/**
 * Query for hierarchical instance culling on the CPU against a FCullingVolume.
 */
class FSceneCullingRendererHierarchicalCullingQuery
{
public:
	FSceneCullingRendererHierarchicalCullingQuery(const FCullingVolume& InCullingVolume, FSceneCullingRenderer& InSceneCullingRenderer);
	
	const FCullingVolume CullingVolume;

	/**
	 * returns true if the primitive has per instance culling info & therefore requires culling to be run for each view (including shadow views).
	 */
	bool HasCullingInfo(int32 ScenePrimitiveIndex) const { return SceneCullingRenderer.PrimitivesWithHierachicalCullingInfoMap[ScenePrimitiveIndex]; }

	/**
	 * 
	 */
	const FSceneCulling::FPrimitiveCullingInfo* GetCullingInfo(FPersistentPrimitiveIndex PersistentPrimitiveIndex);

	bool IsVisible(const FSceneCulling::FPrimitiveCullingInfo& PrimitiveCullingInfo, int32 SubIndex) const
	{ 
		return HashGroupVisibilityMask[PrimitiveCullingInfo.HashGroupVisibilityOffset * NumBitsPerDWORD + SubIndex]; 
	}

	bool IsLODVisible(const FSceneCulling::FPrimitiveCullingInfo& PrimitiveCullingInfo, int32 SubIndex, const FVector2f& InCurrentLODRange)
	{
		const FVector2f& GroupMinMaxFootprint = HashGroupMinMaxFootprints[PrimitiveCullingInfo.HashGroupLODOffset + SubIndex];
		return GroupMinMaxFootprint.X <= InCurrentLODRange.Y && GroupMinMaxFootprint.Y >= InCurrentLODRange.X; 
	}

	/**
	 * Runs per-cell culling via ComputeCulling, then derives a tight LOD range from the union of
	 * visible-cell footprints rather than the conservative full primitive bounds.
	 * Returns empty if there is no per-cell culling data (caller should fall back to standard LOD path).
	 * Returns a set-but-invalid FLODMask if all cells are culled (suppresses all draw calls).
	 * Returns a set valid FLODMask with the tight LOD range otherwise.
	 * Thread safe in the case that no two threads call with the same primitive.
	 */
	TOptional<FLODMask> ComputeCulling(int32 ScenePrimitiveIndex, const FBoxSphereBounds& PrimitiveBounds, const FSceneView& View, int32 ForcedLODLevel, const FDesiredLODLevel& DesiredLODLevel, float LODScale);

private:
	FSceneCullingRenderer& SceneCullingRenderer;
	/**
	 * Allocated bits, initially cleared (hidden).
	 */
	TBitArray<SceneRenderingAllocator> HashGroupVisibilityMask;
	/**
	 * Allocated min/max footprints
	 */
	TArray<FVector2f, SceneRenderingAllocator> HashGroupMinMaxFootprints;
};
