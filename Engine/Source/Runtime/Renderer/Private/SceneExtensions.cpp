// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneExtensions.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "RenderGraphBuilder.h"
#include "Misc/ScopeLock.h"

FSceneExtensionRegistry* FSceneExtensionRegistry::GlobalRegistry = nullptr;

void FSceneExtensionRegistry::InitRegistry()
{
	if (!GlobalRegistry)
	{
		GlobalRegistry = new FSceneExtensionRegistry;
	}
}

void FSceneExtensionRegistry::Register(ISceneExtensionFactory& Factory)
{
	Factory.ExtensionID = Factories.Num();
	Factories.Add(&Factory);
}

TSparseArray<ISceneExtension*> FSceneExtensionRegistry::CreateExtensions(FScene& Scene)
{
	TSparseArray<ISceneExtension*> Extensions;
	Extensions.Empty(Factories.Num());

	int32 ExtensionID = 0;
	for (auto* Factory : Factories)
	{
		checkSlow(Factory->GetExtensionID() == ExtensionID); // sanity check
		if (auto Extension = Factory->CreateInstance(Scene))
		{
			Extensions.EmplaceAt(ExtensionID, Extension);
		}
		++ExtensionID;
	}

	return Extensions;
}


void FSceneExtensions::Init(FScene& Scene)
{
	Extensions = FSceneExtensionRegistry::Get().CreateExtensions(Scene);
	for (auto Extension : Extensions)
	{
		Extension->InitExtension(Scene);
	}
}

void FSceneExtensions::Reset()
{
	for (auto Extension : Extensions)
	{
		delete Extension;
	}
	Extensions.Reset();
}

void FSceneExtensions::CreateUpdaters(FUpdaterList& OutUpdaters)
{
	OutUpdaters.Empty(Extensions.Num());
	for (auto It = Extensions.CreateIterator(); It; ++It)
	{
		const int32 Index = It.GetIndex();
		check(Index <= FSceneExtensionRegistry::Get().GetMaxRegistrationID());
		if (auto Updater = Extensions[Index]->CreateUpdater())
		{
			OutUpdaters.EmplaceAt(Index, Updater);
		}
	}
}

void FSceneExtensions::CreateRenderers(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags, FRendererList& OutRenderers)
{
	OutRenderers.Empty(Extensions.Num());
	for (auto It = Extensions.CreateIterator(); It; ++It)
	{
		const int32 Index = It.GetIndex();
		check(Index <= FSceneExtensionRegistry::Get().GetMaxRegistrationID());
		if (auto Renderer = Extensions[Index]->CreateRenderer(InSceneRenderer, EngineShowFlags))
		{
			OutRenderers.EmplaceAt(Index, Renderer);
		}
	}
}


void FSceneExtensionsUpdaters::Begin(FScene& InScene)
{
	checkf(!IsUpdating(), TEXT("Detected FSceneExtensionsUpdater Begin() without matching End()"));
	
	Scene = &InScene;
	Scene->SceneExtensions.CreateUpdaters(Updaters);
	for (auto Updater : Updaters)
	{
		Updater->Begin(InScene);
	}
}

void FSceneExtensionsUpdaters::End()
{
	for (auto Updater : Updaters)
	{
		Updater->End();
		delete Updater;
	}
	Updaters.Reset();
	Scene = nullptr;
}


void FSceneExtensionsRenderers::Begin(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags, bool bInValidateCallbacks)
{
	bValidateCallbacks = false;//bInValidateCallbacks;
	ValidateAdvanceCallbackStage(ECallbackStage::Begin);
	checkf(!IsRendering(), TEXT("Detected FSceneExtensionsRenderer Begin() without matching End()"));

	SceneRenderer = &InSceneRenderer;
	check(SceneRenderer->Scene != nullptr);

	SceneRenderer->Scene->SceneExtensions.CreateRenderers(InSceneRenderer, EngineShowFlags, Renderers);
	for (auto Renderer : Renderers)
	{
		Renderer->Begin(&InSceneRenderer);
	}
}

void FSceneExtensionsRenderers::End()
{
	ValidateAdvanceCallbackStage(ECallbackStage::End);
	for (auto Renderer : Renderers)
	{
		Renderer->End();
		delete Renderer;
	}
	Renderers.Reset();
	SceneRenderer = nullptr;
}
