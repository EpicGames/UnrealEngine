// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RendererInterface.h"
#include "Tasks/Task.h"

class FScene;
class FSceneRendererBase;
class FRDGBuilder;
class FSceneUniformBuffer;
class FScenePreUpdateChangeSet;
class FScenePostUpdateChangeSet;
class FSceneExtensionRegistry;
class ISceneExtensionUpdater;
class ISceneExtensionRenderer;
struct FEngineShowFlags;
class FRendererViewDataManager;
struct FLightSceneChangeSet;

/** Abstract interface for an extension to the persistent data of a scene */
class ISceneExtension
{
public:
	// default fallback static method that can be overridden in child classes to predicate the creation of the extension
	static bool ShouldCreateExtension(FScene& Scene) { return true; }

	explicit ISceneExtension(FScene& InScene) : Scene(InScene) {}
	virtual ~ISceneExtension() = default;

	/**
	 * InitExtension is called after _all_ scene extensions have been created, and an extension can therefore query for other extensions here.
	 */
	virtual void InitExtension(FScene& InScene) { };
	virtual ISceneExtensionUpdater* CreateUpdater() { return nullptr; }
	virtual ISceneExtensionRenderer* CreateRenderer(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags) { return nullptr; }

protected:
	FScene& Scene;
};

/** Abstract interface to receive change sets to perform updates based on scene primitive data. */
class ISceneExtensionUpdater
{
public:
	virtual ~ISceneExtensionUpdater() = default;

	virtual void Begin(FScene& InScene) {}
	virtual void End() {}

	/**
	 * Callbacks that happen before & after lights are updated in the Scene
	 */
	virtual void PostLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet) {};
	virtual void PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet) {};

	// Some care and caution is needed when using the SceneUniforms passed in here.
	// These passes run outside of the context of the renderer so certain changes may not persist.
	// Additionally - particularly in the pre-scene update - only certain fields of the SceneUniforms will be populated (GPUScene notably).
	virtual void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet) {}

	virtual void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet) {}
	virtual void PostGPUSceneUpdate(FRDGBuilder& GraphBuilder) {}
};

/** Abstract interface for an extension to the scene renderer */
class ISceneExtensionRenderer
{
public:
	explicit ISceneExtensionRenderer(FSceneRendererBase& InSceneRenderer) : SceneRendererInternal(InSceneRenderer) { }
	virtual ~ISceneExtensionRenderer() = default;

	virtual void Begin(FSceneRendererBase* InRenderer) { }
	virtual void End() {}

	/**
	 * Called before LaunchVisibilityTasks to allow scene extensions to prepare data for use during these steps.
	 * Note that because of the visibility tasks being launched as early as possible, this can happen before the end of FScene::Update, 
	 * Notably, it happens (when done early) _after_ SceneExtensionsUpdaters.PostSceneUpdate and _before_ GPU-Scene update, in the callback Parameters.Callbacks.PostStaticMeshUpdate. 
	 * Thus, you can always rely on the updaters to have processed scene updates, but nothing after that sequencing point.
	 */
	virtual void PreVisibilityTasks(FRDGBuilder& GraphBuilder) {}

	/**
	 * Called before BeginInitViews to allow creating tasks work that have dependencies in InitViews.
	 */
	virtual void PreInitViews(FRDGBuilder& GraphBuilder) {}

	/**
	 * Perform any view dependent LOD calculations or similar to e.g., update instance state. 
	 * Called before UpdateSceneUniformBuffer.
	 */
	virtual void UpdateViewData(FRDGBuilder& GraphBuilder, const FRendererViewDataManager& ViewDataManager) {}

	// See the note in ISceneExtensionUpdater about the SceneUniforms.
	virtual void UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniforms) {}

	/**
	 * Called at the start of actual scene rendering, after scene update and view visibility.
	 */
	virtual void PreRender(FRDGBuilder& GraphBuilder) {}

	/**
	 * Called after all rendering has concluded
	 */
	virtual void PostRender(FRDGBuilder& GraphBuilder) {}

protected:
	FSceneRendererBase& GetSceneRenderer() { return SceneRendererInternal;}

private:
	FSceneRendererBase& SceneRendererInternal;
};

/** Abstract interface for creating an instance of a scene extension */
class ISceneExtensionFactory
{
	friend class FSceneExtensionRegistry;

public:
	virtual ~ISceneExtensionFactory() = default;
	virtual ISceneExtension* CreateInstance(FScene& Scene) = 0;

	const int32 GetExtensionID() const { return ExtensionID; }

private:
	int32 ExtensionID = INDEX_NONE;
};

/** Static class used to store a global registry of extension types */
class FSceneExtensionRegistry
{
public:
	static FSceneExtensionRegistry& Get()
	{
		InitRegistry();
		return *GlobalRegistry;
	}

	int32 GetMaxRegistrationID() const { return Factories.Num() - 1; }
	RENDERER_API void Register(ISceneExtensionFactory& Factory);
	TSparseArray<ISceneExtension*> CreateExtensions(FScene& Scene);

private:
	RENDERER_API static void InitRegistry();

	TArray<ISceneExtensionFactory*> Factories;

	RENDERER_API static FSceneExtensionRegistry* GlobalRegistry;
};

/** A collection of scene extensions */
class FSceneExtensions
{	
public:
	using FUpdaterList = TSparseArray<ISceneExtensionUpdater*, SceneRenderingSparseArrayAllocator>;
	using FRendererList = TSparseArray<ISceneExtensionRenderer*, SceneRenderingSparseArrayAllocator>;

	~FSceneExtensions() { Reset(); }

	void Init(FScene& Scene);
	void Reset();
	void CreateUpdaters(FUpdaterList& OutUpdaters);
	void CreateRenderers(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags, FRendererList& OutRenderers);

	template<typename TDerivedExtension>
	TDerivedExtension* GetExtensionPtr()
	{
		const int32 Index = TDerivedExtension::GetExtensionID();
		if (Extensions.IsValidIndex(Index))
		{
			return static_cast<TDerivedExtension*>(Extensions[Index]);
		}
		return nullptr;
	}

	template<typename TDerivedExtension>
	const TDerivedExtension* GetExtensionPtr() const
	{
		return const_cast<FSceneExtensions*>(this)->GetExtensionPtr<TDerivedExtension>();
	}
	
	template<typename TDerivedExtension>
	TDerivedExtension& GetExtension()
	{
		TDerivedExtension* Extension = this->GetExtensionPtr<TDerivedExtension>();
		check(Extension != nullptr);
		return *Extension;
	}
	
	template<typename TDerivedExtension>
	const TDerivedExtension& GetExtension() const
	{
		return const_cast<FSceneExtensions*>(this)->GetExtension<TDerivedExtension>();
	}

	template<typename TFunc>
	void ForEachExtension(const TFunc& F)
	{
		for(auto Ext : Extensions)
		{
			F(Ext);
		}
	}

private:
	TSparseArray<ISceneExtension*> Extensions;
};

/** Performs updates for the given scene extensions */
class FSceneExtensionsUpdaters
{
	friend class FSceneExtensions;

public:
	FSceneExtensionsUpdaters() {}
	explicit FSceneExtensionsUpdaters(FScene& InScene) { Begin(InScene); }
	~FSceneExtensionsUpdaters() { End(); }

	void Begin(FScene& InScene);
	void End();
	bool IsUpdating() const { return Scene != nullptr; }

	template<typename TUpdater>
	TUpdater* GetUpdaterPtr()
	{
		const int32 Index = TUpdater::FExtension::GetExtensionID();
		return Updaters.IsValidIndex(Index) ? static_cast<TUpdater*>(Updaters[Index]) : nullptr;
	}
	template<typename TUpdater>
	const TUpdater* GetUpdaterPtr() const { return const_cast<FSceneExtensionsUpdaters*>(this)->GetUpdaterPtr<TUpdater>(); }
	template<typename TUpdater>
	TUpdater& GetUpdater()
	{
		auto Updater = GetUpdaterPtr<TUpdater>();
		check(Updater != nullptr);
		return *Updater;
	}
	template<typename TUpdater>
	const TUpdater& GetUpdater() const { return const_cast<FSceneExtensionsUpdaters*>(this)->GetUpdater<TUpdater>(); }


	void PreLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet)
	{
		for (auto Updater : Updaters) { Updater->PreLightsUpdate(GraphBuilder, LightSceneChangeSet); }
	}

	void PostLightsUpdate(FRDGBuilder& GraphBuilder, const FLightSceneChangeSet& LightSceneChangeSet)
	{
		for (auto Updater : Updaters) { Updater->PostLightsUpdate(GraphBuilder, LightSceneChangeSet); }
	}

	void PreSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePreUpdateChangeSet& ChangeSet)
	{
		for (auto Updater : Updaters) { Updater->PreSceneUpdate(GraphBuilder, ChangeSet); }
	}

	void PostSceneUpdate(FRDGBuilder& GraphBuilder, const FScenePostUpdateChangeSet& ChangeSet)
	{
		for (auto Updater : Updaters) { Updater->PostSceneUpdate(GraphBuilder, ChangeSet); }
	}

	void PostGPUSceneUpdate(FRDGBuilder& GraphBuilder)
	{
		for (auto Updater : Updaters) { Updater->PostGPUSceneUpdate(GraphBuilder); }
	}

private:
	FScene* Scene = nullptr;
	FSceneExtensions::FUpdaterList Updaters;
};

/** Performs rendering for the given scene extensions */
class FSceneExtensionsRenderers
{
	friend class FSceneExtensions;

public:
	FSceneExtensionsRenderers() {}
	~FSceneExtensionsRenderers() { End(); }
	
	void Begin(FSceneRendererBase& InSceneRenderer, const FEngineShowFlags& EngineShowFlags, bool bInValidateCallbacks);
	void End();
	bool IsRendering() const { return SceneRenderer != nullptr; }

	template<typename TRenderer>
	TRenderer* GetRendererPtr()
	{
		const int32 Index = TRenderer::FExtension::GetExtensionID();
		return Renderers.IsValidIndex(Index) ? static_cast<TRenderer*>(Renderers[Index]) : nullptr;
	}
	template<typename TRenderer>
	const TRenderer* GetRendererPtr() const { return const_cast<FSceneExtensionsRenderers*>(this)->GetRendererPtr<TRenderer>(); }
	template<typename TRenderer>
	TRenderer& GetRenderer()
	{
		auto Renderer = GetRendererPtr<TRenderer>();
		check(Renderer != nullptr); 
		return *Renderer;
	}
	template<typename TRenderer>
	const TRenderer& GetRenderer() const { return const_cast<FSceneExtensionsRenderers*>(this)->GetRenderer<TRenderer>(); }
	
	/**
	 * Called before LaunchVisibilityTasks to allow scene extensions to prepare data for use during these steps.
	 * For details see ISceneExtensionRenderer::PreVisibilityTasks
	 */
	void PreVisibilityTasks(FRDGBuilder& GraphBuilder)
	{
		ValidateAdvanceCallbackStage(ECallbackStage::PreVisibilityTasks);
		for (auto Renderer : Renderers) { Renderer->PreVisibilityTasks(GraphBuilder); }
	}	

	/**
	 * Called before BeginInitViews to allow creating tasks work that have dependencies in InitViews.
	 */
	void PreInitViews(FRDGBuilder& GraphBuilder)
	{
		ValidateAdvanceCallbackStage(ECallbackStage::PreInitViews);
		for (auto Renderer : Renderers) { Renderer->PreInitViews(GraphBuilder); }
	}	

	void UpdateViewData(FRDGBuilder& GraphBuilder, const FRendererViewDataManager& ViewDataManager)
	{
		ValidateAdvanceCallbackStage(ECallbackStage::UpdateViewData);
		for (auto Renderer : Renderers) 
		{ 
			Renderer->UpdateViewData(GraphBuilder, ViewDataManager); 
		}
	}

	void UpdateSceneUniformBuffer(FRDGBuilder& GraphBuilder, FSceneUniformBuffer& SceneUniforms)
	{
		ValidateAdvanceCallbackStage(ECallbackStage::UpdateSceneUniformBuffer);
		for (auto Renderer : Renderers) { Renderer->UpdateSceneUniformBuffer(GraphBuilder, SceneUniforms); }
	}
	
	void PreRender(FRDGBuilder& GraphBuilder)
	{
		ValidateAdvanceCallbackStage(ECallbackStage::PreRender);
		for (auto Renderer : Renderers) { Renderer->PreRender(GraphBuilder); }
	}
	
	void PostRender(FRDGBuilder& GraphBuilder)
	{
		ValidateAdvanceCallbackStage(ECallbackStage::PostRender);
		for (auto Renderer : Renderers) { Renderer->PostRender(GraphBuilder); }
	}

private:

	// true if the callback order should be validated.
	bool bValidateCallbacks = false;
	// Just used to validate that the stages happen in the expected order and none are skipped.
	enum class ECallbackStage : int32
	{
		Begin,
		PreVisibilityTasks,
		PreInitViews,
		UpdateViewData,
		UpdateSceneUniformBuffer,
		PreRender,
		PostRender,
		End
	};
	ECallbackStage CurrentCallbackStage = ECallbackStage::Begin;

	void ValidateAdvanceCallbackStage(ECallbackStage InCallbackStage)
	{
		check(!bValidateCallbacks || InCallbackStage == CurrentCallbackStage);
		// Reset using the provided (called) stage as the subsequent stages might be ok.
		CurrentCallbackStage = static_cast<ECallbackStage>(static_cast<int32>(InCallbackStage) + 1);
	}

	FSceneRendererBase* SceneRenderer = nullptr;
	FSceneExtensions::FRendererList Renderers;
};

/** Helper to automatically register/unregister a factory implementation for a given ISceneExtension implementation */
template<typename TDerivedExtension>
class TSceneExtensionRegistration : public ISceneExtensionFactory
{
public:
	TSceneExtensionRegistration()
	{
		FSceneExtensionRegistry::Get().Register(*this);
	}

	virtual ~TSceneExtensionRegistration() = default;

	virtual ISceneExtension* CreateInstance(FScene& Scene) override
	{
		if (!TDerivedExtension::ShouldCreateExtension(Scene))
		{
			return nullptr;
		}
		return new TDerivedExtension(Scene);
	}
};

/** Use these macros in the class definitions of your extension. */
#define DECLARE_SCENE_EXTENSION(ModuleExport, ClassName) \
	public: \
		ModuleExport static int32 GetExtensionID(); \
	private: \
		ModuleExport static TSceneExtensionRegistration<ClassName> ExtensionRegistration

#define DECLARE_SCENE_EXTENSION_UPDATER(ClassName, SceneExtensionClassName) \
	public: \
		using FExtension = SceneExtensionClassName

#define DECLARE_SCENE_EXTENSION_RENDERER(ClassName, SceneExtensionClassName) \
	public: \
		using FExtension = SceneExtensionClassName

/** Use this macro in the implementation source file of your extension. */
#define IMPLEMENT_SCENE_EXTENSION(ClassName) \
	int32 ClassName::GetExtensionID() { return ExtensionRegistration.GetExtensionID();  } \
	TSceneExtensionRegistration<ClassName> ClassName::ExtensionRegistration
