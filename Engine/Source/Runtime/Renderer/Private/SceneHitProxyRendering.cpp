// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneHitProxyRendering.cpp: Scene hit proxy rendering.
=============================================================================*/

#include "SceneHitProxyRendering.h"
#include "CoreMinimal.h"
#include "BatchedElements.h"
#include "ClearQuad.h"
#include "Containers/ArrayView.h"
#include "Containers/IndirectArray.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DeferredShadingRenderer.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "FXSystem.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "GPUMessaging.h"
#include "GPUScene.h"
#include "GPUSortManager.h"
#include "HairStrands/HairStrandsData.h"
#include "HitProxies.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "Logging/LogMacros.h"
#include "Lumen/LumenSceneData.h"
#include "MaterialCache/MaterialCacheTagProvider.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShader.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "MeshBatch.h"
#include "MeshDrawCommands.h"
#include "MeshDrawShaderBindings.h"
#include "MeshMaterialShader.h"
#include "MeshMaterialShaderType.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "Nanite/NaniteCullRaster.h"
#include "Nanite/NaniteEditor.h"
#include "Nanite/NaniteShared.h"
#include "Nanite/NaniteVisibility.h"
#include "PixelShaderUtils.h"
#include "PostProcess/LensDistortion.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "RendererInterface.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/NaniteStreamingManager.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "RHIStaticStates.h"
#include "SceneExtensions.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneRendering.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneUniformBuffer.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "SceneVisibility.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "Shader.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterStruct.h"
#include "ShaderPrint.h"
#include "SimpleMeshDrawCommandPass.h"
#include "Stats/Stats.h"
#include "SystemTextures.h"
#include "VertexFactory.h"
#include "ViewData.h"
#include "VT/VirtualTextureSystem.h"

static bool RequiresEditorPermutations(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	const bool bSupportEditor = EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);
	const bool bSupportCookedEditor = IsPCPlatform(Parameters.Platform) && !IsMobilePlatform(Parameters.Platform);
	return bSupportEditor || bSupportCookedEditor;
}

class FHitProxyShaderElementData : public FMeshMaterialShaderElementData
{
public:
	FHitProxyShaderElementData(FHitProxyId InBatchHitProxyId, bool bInEditorSelectionPass = false)
		: BatchHitProxyId(InBatchHitProxyId)
		, bEditorSelectionPass(bInEditorSelectionPass)
	{
	}

	FHitProxyId BatchHitProxyId;
	bool bEditorSelectionPass; // When true, BatchHitProxyId is treated as OverlayColor
};

/**
 * A vertex shader for rendering the depth of a mesh.
 */
class FHitProxyVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FHitProxyVS,MeshMaterial);

public:
	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{

		// Only compile the hit proxy vertex shader on desktop platforms
		return RequiresEditorPermutations(Parameters)
			// and only compile for the default material or materials that are masked.
			&& (Parameters.MaterialParameters.bIsSpecialEngineMaterial ||
				!Parameters.MaterialParameters.bWritesEveryPixel ||
				Parameters.MaterialParameters.bMaterialMayModifyMeshPosition ||
				Parameters.MaterialParameters.bIsTwoSided);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FMeshMaterialShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings)
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

#if WITH_EDITOR
		const FColorVertexBuffer* HitProxyIdBuffer = PrimitiveSceneProxy ? PrimitiveSceneProxy->GetCustomHitProxyIdBuffer() : nullptr;
		if(HitProxyIdBuffer)
		{
			ShaderBindings.Add(VertexFetch_HitProxyIdBuffer, HitProxyIdBuffer->GetColorComponentsSRV());
		}
		else
		{
			ShaderBindings.Add(VertexFetch_HitProxyIdBuffer, GNullColorVertexBuffer.VertexBufferSRV);
		}
#endif
	}
protected:

	FHitProxyVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FMeshMaterialShader(Initializer)
	{
		VertexFetch_HitProxyIdBuffer.Bind(Initializer.ParameterMap, TEXT("VertexFetch_HitProxyIdBuffer"), SPF_Optional);
	}
	FHitProxyVS() {}

	LAYOUT_FIELD(FShaderResourceParameter, VertexFetch_HitProxyIdBuffer)
};

IMPLEMENT_MATERIAL_SHADER_TYPE(,FHitProxyVS,TEXT("/Engine/Private/HitProxyVertexShader.usf"),TEXT("Main"),SF_Vertex); 

/**
 * A pixel shader for rendering the HitProxyId of an object as a unique color in the scene.
 */
class FHitProxyPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FHitProxyPS,MeshMaterial);
public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		// Only compile the hit proxy vertex shader on editor platforms
		return RequiresEditorPermutations(Parameters)
			// and only compile for default materials or materials that are masked.
			&& (Parameters.MaterialParameters.bIsSpecialEngineMaterial ||
				!Parameters.MaterialParameters.bWritesEveryPixel ||
				Parameters.MaterialParameters.bMaterialMayModifyMeshPosition ||
				Parameters.MaterialParameters.bIsTwoSided) &&
				!Parameters.VertexFactoryType->SupportsNaniteRendering();
	}

	FHitProxyPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FMeshMaterialShader(Initializer)
	{
		HitProxyId.Bind(Initializer.ParameterMap,TEXT("HitProxyId"), SPF_Optional); // There is no way to guarantee that this parameter will be preserved in a material that kill()s all fragments as the optimiser can remove the global - this happens in various projects.
		WriteInstanceHitProxyId.Bind(Initializer.ParameterMap, TEXT("WriteInstanceHitProxyId"), SPF_Optional);
	}

	FHitProxyPS() {}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FHitProxyShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		FHitProxyId hitProxyId = ShaderElementData.BatchHitProxyId;

		// In the editor-selection pass, HitProxyId is actually the primitive's OverlayColor and must pass through untouched.
		if (!ShaderElementData.bEditorSelectionPass)
		{
#if WITH_EDITOR
			if (PrimitiveSceneProxy && PrimitiveSceneProxy->GetCustomHitProxyIdBuffer())
			{
				hitProxyId = FColor(0);
			}
			else
#endif
			if (PrimitiveSceneProxy && ShaderElementData.BatchHitProxyId == FHitProxyId())
			{
				hitProxyId = PrimitiveSceneProxy->GetPrimitiveSceneInfo()->DefaultDynamicHitProxyId;
			}

			// Per-instance hitproxies are supplied by the vertex factory.
			if (PrimitiveSceneProxy && PrimitiveSceneProxy->HasPerInstanceHitProxies())
			{
				hitProxyId = FColor(0);
			}
		}

		ShaderBindings.Add(HitProxyId, hitProxyId.GetColor().ReinterpretAsLinear());

		// This prevents the per-instance hit proxy ID from being added to the output when we're writing overlay color
		ShaderBindings.Add(WriteInstanceHitProxyId, ShaderElementData.bEditorSelectionPass ? 0.0f : 1.0f);
	}

private:
	LAYOUT_FIELD(FShaderParameter, HitProxyId)
	LAYOUT_FIELD(FShaderParameter, WriteInstanceHitProxyId)
};

IMPLEMENT_MATERIAL_SHADER_TYPE(,FHitProxyPS,TEXT("/Engine/Private/HitProxyPixelShader.usf"),TEXT("Main"),SF_Pixel);

#if WITH_EDITOR

void InitHitProxyRender(FRDGBuilder& GraphBuilder, FSceneRenderer* SceneRenderer, FRDGTextureRef& OutHitProxyTexture, FRDGTextureRef& OutHitProxyDepthTexture)
{
	auto& ViewFamily = SceneRenderer->ViewFamily;
	auto FeatureLevel = ViewFamily.Scene->GetFeatureLevel();

	// Ensure VirtualTexture resources are allocated
	if (UseVirtualTexturing(ViewFamily.Scene->GetShaderPlatform()))
	{
		FVirtualTextureUpdateSettings Settings;
		Settings.EnablePageRequests(false);

		FVirtualTextureSystem::Get().Update(GraphBuilder, FeatureLevel, nullptr, Settings);
		FMaterialCacheTagProvider::Get().Update(GraphBuilder);
	}

	// Initialize global system textures (pass-through if already initialized).
	GSystemTextures.InitializeTextures(GraphBuilder.RHICmdList, FeatureLevel);
	FRDGSystemTextures::Create(GraphBuilder);

	const FSceneTexturesConfig& SceneTexturesConfig = ViewFamily.SceneTexturesConfig;

	FMinimalSceneTextures::InitializeViewFamily(GraphBuilder, SceneRenderer->ViewFamily);
	const FMinimalSceneTextures& SceneTextures = ViewFamily.GetSceneTextures();

	// Create a texture to store the resolved light attenuation values, and a render-targetable surface to hold the unresolved light attenuation values.
	{
		FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(SceneTexturesConfig.Extent, PF_B8G8R8A8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource));
		OutHitProxyTexture = GraphBuilder.CreateTexture(Desc, TEXT("HitProxy"));

		// create non-MSAA version for hit proxies on PC if needed
		const EShaderPlatform CurrentShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];
		FRDGTextureDesc DepthDesc = SceneTextures.Depth.Target->Desc;

		if (DepthDesc.NumSamples > 1)
		{
			DepthDesc.NumSamples = 1;
			OutHitProxyDepthTexture = GraphBuilder.CreateTexture(DepthDesc, TEXT("NoMSAASceneDepthZ"));
		}
		else
		{
			OutHitProxyDepthTexture = SceneTextures.Depth.Target;
		}
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FHitProxyPassParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
	SHADER_PARAMETER_STRUCT_INCLUDE(FSceneTextureShaderParameters, SceneTextures)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

static void AddViewMeshElementsPass(const TIndirectArray<FMeshBatch> &MeshElements, FRDGBuilder& GraphBuilder, FHitProxyPassParameters* PassParameters, const FScene* Scene, const FViewInfo& View, const FMeshPassProcessorRenderState& DrawRenderState, FInstanceCullingManager& InstanceCullingManager)
{
	AddSimpleMeshPass(GraphBuilder, PassParameters, Scene, View, &InstanceCullingManager, RDG_EVENT_NAME("HitProxy::MeshElementsPass"), View.ViewRect,
		[&View, Scene, DrawRenderState, &MeshElements](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
		{
			FHitProxyMeshProcessor PassMeshProcessor(
				Scene,
				&View,
				View.bAllowTranslucentPrimitivesInHitProxy,
				DrawRenderState,
				DynamicMeshPassContext);

			const uint64 DefaultBatchElementMask = ~0ull;

			for (const FMeshBatch& MeshBatch : MeshElements)
			{
				PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
			}
		}
	);
}

class FHitProxyCopyPS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FHitProxyCopyPS);
	SHADER_USE_PARAMETER_STRUCT(FHitProxyCopyPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, UndistortingDisplacementTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState,  UndistortingDisplacementSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, HitProxyTexture)

		SHADER_PARAMETER(FScreenTransform, PassSvPositionToViewportUV)
		SHADER_PARAMETER(FScreenTransform, ViewportUVToHitProxyPixelPos)
		SHADER_PARAMETER(FIntPoint, HitProxyPixelPosMin)
		SHADER_PARAMETER(FIntPoint, HitProxyPixelPosMax)

		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FHitProxyCopyPS, "/Engine/Private/HitProxyCopy.usf", "MainPS", SF_Pixel);


static void DoRenderHitProxies(
	FRDGBuilder& GraphBuilder, 
	const FSceneRenderer* SceneRenderer, 
	FRDGTextureRef HitProxyTexture, 
	FRDGTextureRef HitProxyDepthTexture,
	const TArray<Nanite::FRasterResults, TInlineAllocator<2>>& NaniteRasterResults,
	FInstanceCullingManager& InstanceCullingManager)
{
	SceneRenderer->Scene->AddGPUSkinCacheAsyncComputeWait(GraphBuilder);

	auto& ViewFamily = SceneRenderer->ViewFamily;
	auto& Views = SceneRenderer->Views;
	const auto FeatureLevel = SceneRenderer->FeatureLevel;
	const FIntPoint HitProxyTextureExtent = HitProxyTexture->Desc.Extent;

	{
		auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(HitProxyTexture, ERenderTargetLoadAction::EClear);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(HitProxyDepthTexture, ERenderTargetLoadAction::EClear, ERenderTargetLoadAction::EClear, FExclusiveDepthStencil::DepthWrite_StencilWrite);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HitProxies::Clear"),
			PassParameters,
			ERDGPassFlags::Raster,
			[&Views, HitProxyTextureExtent](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			// Clear color for each view.
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				const FViewInfo& View = Views[ViewIndex];
				RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);
				DrawClearQuad(RHICmdList, true, FLinearColor::White, false, 0, false, 0, HitProxyTextureExtent, FIntRect());
				// Clear the depth buffer for each DPG.
				DrawClearQuad(RHICmdList, false, FLinearColor(), true, 0.f, true, 0, HitProxyTextureExtent, FIntRect());
			}
		});
	}

	// Nanite hit proxies
	if (NaniteRasterResults.Num() == Views.Num())
	{
		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			Nanite::DrawHitProxies(GraphBuilder, *SceneRenderer->Scene, Views[ViewIndex], NaniteRasterResults[ViewIndex], HitProxyTexture, HitProxyDepthTexture);
		}
	}

	// HairStrands hit proxies
	for (const FViewInfo& View : Views)
	{
		if (View.HairStrandsMeshElements.Num() > 0)
		{
			HairStrands::DrawHitProxies(GraphBuilder, *SceneRenderer->Scene, View, InstanceCullingManager, HitProxyTexture, HitProxyDepthTexture);
		}
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		FViewInfo& View = const_cast<FViewInfo&>(Views[ViewIndex]);
		FScene* LocalScene = SceneRenderer->Scene;
		View.BeginRenderView();

		auto* PassParameters = GraphBuilder.AllocParameters<FHitProxyPassParameters>();
		PassParameters->View = View.GetShaderParameters();

		// Adjust the visibility map for this view
		if (View.bAllowTranslucentPrimitivesInHitProxy)
		{
			if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::HitProxy])
			{
				Pass->BuildRenderingCommands(GraphBuilder, LocalScene->GPUScene, PassParameters->InstanceCullingDrawParams);
			}
		}
		else if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::HitProxyOpaqueOnly])
		{
			Pass->BuildRenderingCommands(GraphBuilder, LocalScene->GPUScene, PassParameters->InstanceCullingDrawParams);
		}

		// Need to setup dummy instance culling parameters if no hit proxy meshes are being rendered.
		if (!PassParameters->InstanceCullingDrawParams.Scene)
		{
			InstanceCullingManager.SetDummyCullingParams(GraphBuilder, PassParameters->InstanceCullingDrawParams);
		}

		PassParameters->RenderTargets[0] = FRenderTargetBinding(HitProxyTexture, ERenderTargetLoadAction::ELoad);
		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(HitProxyDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthWrite_StencilWrite);
		PassParameters->SceneTextures = CreateSceneTextureShaderParameters(GraphBuilder, View, ESceneTextureSetupMode::None);

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("HitProxies::Render"),
			PassParameters,
			ERDGPassFlags::Raster,
			[SceneRenderer, &View, LocalScene, FeatureLevel, PassParameters](FRHICommandList& RHICmdList)
		{
			FMeshPassProcessorRenderState DrawRenderState;

			// Set the device viewport for the view.
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

			// Depth tests + writes, no alpha blending.
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
			DrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());

			const bool bHitTesting = true;

			// Adjust the visibility map for this view
			if (View.bAllowTranslucentPrimitivesInHitProxy)
			{
				if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::HitProxy])
				{
					Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
				}
			}
			else if (auto* Pass = View.ParallelMeshDrawCommandPasses[EMeshPass::HitProxyOpaqueOnly])
			{
				Pass->Draw(RHICmdList, &PassParameters->InstanceCullingDrawParams);
			}

			DrawDynamicMeshPass(View, RHICmdList,
				[&View, &DrawRenderState, LocalScene](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FHitProxyMeshProcessor PassMeshProcessor(
					LocalScene,
					&View,
					View.bAllowTranslucentPrimitivesInHitProxy,
					DrawRenderState,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;

				for (int32 MeshIndex = 0; MeshIndex < View.DynamicEditorMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatchAndRelevance& MeshBatchAndRelevance = View.DynamicEditorMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(*MeshBatchAndRelevance.Mesh, DefaultBatchElementMask, MeshBatchAndRelevance.PrimitiveSceneProxy);
				}
			});

			View.SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::All, SDPG_World);
			View.SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::All, SDPG_Foreground);

			View.EditorSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::All, SDPG_World);
			View.EditorSimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::All, SDPG_Foreground);

			DrawDynamicMeshPass(View, RHICmdList,
				[&View, &DrawRenderState, LocalScene](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FHitProxyMeshProcessor PassMeshProcessor(
					LocalScene,
					&View,
					View.bAllowTranslucentPrimitivesInHitProxy,
					DrawRenderState,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;

				for (int32 MeshIndex = 0; MeshIndex < View.ViewMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatch& MeshBatch = View.ViewMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
				}
			});

			DrawDynamicMeshPass(View, RHICmdList,
				[&View, &DrawRenderState, LocalScene](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FHitProxyMeshProcessor PassMeshProcessor(
					LocalScene,
					&View,
					View.bAllowTranslucentPrimitivesInHitProxy,
					DrawRenderState,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;

				for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
				}
			});


			// Draw the view's batched simple elements(lines, sprites, etc).
			View.BatchedViewElements.Draw(RHICmdList, DrawRenderState, FeatureLevel, View, true);

			// Some elements should never be occluded (e.g. gizmos).
			// So we render those twice, first to overwrite potentially nearer objects,
			// then again to allows proper occlusion within those elements.
			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());

			DrawDynamicMeshPass(View, RHICmdList,
				[&View, &DrawRenderState, LocalScene](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FHitProxyMeshProcessor PassMeshProcessor(
					LocalScene,
					&View,
					View.bAllowTranslucentPrimitivesInHitProxy,
					DrawRenderState,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;

				for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
				}
			});

			View.TopBatchedViewElements.Draw(RHICmdList, DrawRenderState, FeatureLevel, View, true);

			DrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());

			DrawDynamicMeshPass(View, RHICmdList,
				[&View, &DrawRenderState, LocalScene](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FHitProxyMeshProcessor PassMeshProcessor(
					LocalScene,
					&View,
					View.bAllowTranslucentPrimitivesInHitProxy,
					DrawRenderState,
					DynamicMeshPassContext);

				const uint64 DefaultBatchElementMask = ~0ull;

				for (int32 MeshIndex = 0; MeshIndex < View.TopViewMeshElements.Num(); MeshIndex++)
				{
					const FMeshBatch& MeshBatch = View.TopViewMeshElements[MeshIndex];
					PassMeshProcessor.AddMeshBatch(MeshBatch, DefaultBatchElementMask, nullptr);
				}
			});

			View.TopBatchedViewElements.Draw(RHICmdList, DrawRenderState, FeatureLevel, View, true);
		});
	}

	FRDGTextureRef ViewFamilyTexture = TryCreateViewFamilyTexture(GraphBuilder, ViewFamily);
	check(ViewFamilyTexture);

	// Copy & Apply lens distortion of the hit proxy buffer into the view family's render target.
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];

		FHitProxyCopyPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHitProxyCopyPS::FParameters>();
		PassParameters->PassSvPositionToViewportUV = FScreenTransform::SvPositionToViewportUV(View.UnscaledViewRect);
		PassParameters->ViewportUVToHitProxyPixelPos = FScreenTransform::ChangeTextureBasisFromTo(
			FScreenPassTextureViewport(HitProxyTexture, View.ViewRect), FScreenTransform::ETextureBasis::ViewportUV, FScreenTransform::ETextureBasis::TexelPosition);
		PassParameters->HitProxyPixelPosMin = View.ViewRect.Min;
		PassParameters->HitProxyPixelPosMax = View.ViewRect.Max - 1;

		PassParameters->UndistortingDisplacementTexture = GSystemTextures.GetBlackDummy(GraphBuilder);
		PassParameters->UndistortingDisplacementSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		if (View.LensDistortionLUT.IsEnabled())
		{
			PassParameters->UndistortingDisplacementTexture = View.LensDistortionLUT.UndistortingDisplacementTexture;
		}
		PassParameters->HitProxyTexture = HitProxyTexture;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(ViewFamilyTexture, ERenderTargetLoadAction::ELoad);

		TShaderMapRef<FHitProxyCopyPS> PixelShader(View.ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("HitProxyCopy %dx%d", View.UnscaledViewRect.Width(), View.UnscaledViewRect.Height()),
			PixelShader,
			PassParameters,
			View.UnscaledViewRect);
	}
}
#endif

void FMobileSceneRenderer::RenderHitProxies(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs)
{
	IVisibilityTaskData* VisibilityTaskData = OnRenderBegin(GraphBuilder, SceneUpdateInputs);

	GPU_MESSAGE_SCOPE(GraphBuilder);

	FGPUSceneScopeBeginEndHelper GPUSceneScopeBeginEndHelper(GraphBuilder, Scene->GPUScene, GPUSceneDynamicContext);

#if WITH_EDITOR
	FSceneTexturesConfig& SceneTexturesConfig = GetActiveSceneTexturesConfig();
	FRDGTextureRef HitProxyTexture = nullptr;
	FRDGTextureRef HitProxyDepthTexture = nullptr;
	InitHitProxyRender(GraphBuilder, this, HitProxyTexture, HitProxyDepthTexture);

	GetSceneExtensionsRenderers().PreInitViews(GraphBuilder);

	FRendererViewDataManager& ViewDataManager = *GraphBuilder.AllocObject<FRendererViewDataManager>(GraphBuilder, *Scene, GetSceneUniforms(), AllViews);
	FInstanceCullingManager& InstanceCullingManager = *GraphBuilder.AllocObject<FInstanceCullingManager>(GraphBuilder, *Scene, GetSceneUniforms(), ViewDataManager);

	FInitViewTaskDatas InitViewTaskDatas(VisibilityTaskData);

	// Find the visible primitives.
	InitViews(GraphBuilder, SceneTexturesConfig, InstanceCullingManager, nullptr, InitViewTaskDatas);

	// Notify the FX system that the scene is about to be rendered.
	// TODO: These should probably be moved to scene extensions
	if (FXSystem && Views.IsValidIndex(0))
	{
		FGPUSortManager* GPUSortManager = FXSystem->GetGPUSortManager();
		FXSystem->PreRender(GraphBuilder, GetSceneViews(), GetSceneUniforms(), false);
		if (GPUSortManager)
		{
			GPUSortManager->OnPreRender(GraphBuilder);
		}
		// Call PostRenderOpaque now as this is irrelevant for when rendering hit proxies.
		// because we don't tick the particles in the render loop (see last param being "false").
		FXSystem->PostRenderOpaque(GraphBuilder, GetSceneViews(), GetSceneUniforms(), false /*bAllowGPUParticleUpdate*/);
		if (GPUSortManager)
		{
			GPUSortManager->OnPostRenderOpaque(GraphBuilder);
		}
	}

	{
		RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, UpdateGPUScene);

		for (int32 ViewIndex = 0; ViewIndex < AllViews.Num(); ViewIndex++)
		{
			FViewInfo& View = *AllViews[ViewIndex];
			Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, View);
			Scene->GPUScene.DebugRender(GraphBuilder, GetSceneUniforms(), View);
		}
	}
	
	GetSceneExtensionsRenderers().UpdateViewData(GraphBuilder, ViewDataManager);
	GetSceneExtensionsRenderers().UpdateSceneUniformBuffer(GraphBuilder, GetSceneUniforms());

	GetSceneExtensionsRenderers().PreRender(GraphBuilder);
	GEngine->GetPreRenderDelegateEx().Broadcast(GraphBuilder);

	TArray<Nanite::FRasterResults, TInlineAllocator<2>> NaniteRasterResults;
	::DoRenderHitProxies(GraphBuilder, this, HitProxyTexture, HitProxyDepthTexture, NaniteRasterResults, InstanceCullingManager);

	GEngine->GetPostRenderDelegateEx().Broadcast(GraphBuilder);
	GetSceneExtensionsRenderers().PostRender(GraphBuilder);

#endif

	OnRenderFinish(GraphBuilder, nullptr);
}

void FDeferredShadingSceneRenderer::RenderHitProxies(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs)
{
	const bool bNaniteEnabled = UseNanite(ShaderPlatform);

	IVisibilityTaskData* VisibilityTaskData = OnRenderBegin(GraphBuilder, SceneUpdateInputs);

	CommitFinalPipelineState();

	GPU_MESSAGE_SCOPE(GraphBuilder);

	FGPUSceneScopeBeginEndHelper GPUSceneScopeBeginEndHelper(GraphBuilder, Scene->GPUScene, GPUSceneDynamicContext);


#if WITH_EDITOR
	FSceneTexturesConfig& SceneTexturesConfig = GetActiveSceneTexturesConfig();
	FRDGTextureRef HitProxyTexture = nullptr;
	FRDGTextureRef HitProxyDepthTexture = nullptr;

	InitHitProxyRender(GraphBuilder, this, HitProxyTexture, HitProxyDepthTexture);

	const FIntPoint HitProxyTextureSize = HitProxyDepthTexture->Desc.Extent;

	GetSceneExtensionsRenderers().PreInitViews(GraphBuilder);

	FRendererViewDataManager& ViewDataManager = *GraphBuilder.AllocObject<FRendererViewDataManager>(GraphBuilder, *Scene, GetSceneUniforms(), AllViews);
	FInstanceCullingManager& InstanceCullingManager = *GraphBuilder.AllocObject<FInstanceCullingManager>(GraphBuilder, *Scene, GetSceneUniforms(), ViewDataManager);

	// Find the visible primitives.
	FLumenSceneFrameTemporaries LumenFrameTemporaries(Views);
	FInitViewTaskDatas InitViewTaskDatas(VisibilityTaskData);
	FRDGExternalAccessQueue ExternalAccessQueue;
	BeginInitViews(GraphBuilder, SceneTexturesConfig, InstanceCullingManager, ExternalAccessQueue, InitViewTaskDatas);

	extern TSet<IPersistentViewUniformBufferExtension*> PersistentViewUniformBufferExtensions;

	for (IPersistentViewUniformBufferExtension* Extension : PersistentViewUniformBufferExtensions)
	{
		Extension->BeginFrame();

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			// Must happen before RHI thread flush so any tasks we dispatch here can land in the idle gap during the flush
			Extension->PrepareView(&Views[ViewIndex]);
		}
	}

	ShaderPrint::BeginViews(GraphBuilder, Views);

	InitViewTaskDatas.VisibilityTaskData->FinishGatherDynamicMeshElements(FExclusiveDepthStencil::DepthWrite_StencilWrite, InstanceCullingManager, nullptr);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		Scene->GPUScene.UploadDynamicPrimitiveShaderDataForView(GraphBuilder, Views[ViewIndex]);
	}
	ViewDataManager.InitInstanceState(GraphBuilder);

	EndInitViews(GraphBuilder, LumenFrameTemporaries, InstanceCullingManager, InitViewTaskDatas);

	GetSceneExtensionsRenderers().UpdateViewData(GraphBuilder, ViewDataManager);

	GetSceneExtensionsRenderers().UpdateSceneUniformBuffer(GraphBuilder, GetSceneUniforms());

	ExternalAccessQueue.Submit(GraphBuilder);

	if (bNaniteEnabled)
	{
		Nanite::GGlobalResources.Update(GraphBuilder);
		Nanite::GStreamingManager.BeginAsyncUpdate(GraphBuilder);
		Nanite::GStreamingManager.EndAsyncUpdate(GraphBuilder);
	}

	GetSceneExtensionsRenderers().PreRender(GraphBuilder);
	GEngine->GetPreRenderDelegateEx().Broadcast(GraphBuilder);

	// Notify the FX system that the scene is about to be rendered.
	// TODO: These should probably be moved to scene extensions
	if (FXSystem && Views.IsValidIndex(0))
	{
		FGPUSortManager* GPUSortManager = FXSystem->GetGPUSortManager();
		FXSystem->PreRender(GraphBuilder, GetSceneViews(), GetSceneUniforms(), false);
		if (GPUSortManager)
		{
			GPUSortManager->OnPreRender(GraphBuilder);
		}
		// Call PostRenderOpaque now as this is irrelevant for when rendering hit proxies.
		// because we don't tick the particles in the render loop (see last param being "false").
		FXSystem->PostRenderOpaque(GraphBuilder, GetSceneViews(), GetSceneUniforms(), false /*bAllowGPUParticleUpdate*/);
		if (GPUSortManager)
		{
			GPUSortManager->OnPostRenderOpaque(GraphBuilder);
		}
	}

	TArray<Nanite::FRasterResults, TInlineAllocator<2>> NaniteRasterResults;
	if (bNaniteEnabled)
	{
		NaniteRasterResults.AddDefaulted(Views.Num());

		Nanite::FSharedContext SharedContext{};
		SharedContext.FeatureLevel = Scene->GetFeatureLevel();
		SharedContext.ShaderMap = GetGlobalShaderMap(SharedContext.FeatureLevel);
		SharedContext.Pipeline = Nanite::ERasterPipeline::HitProxy;

		FIntRect HitProxyTextureRect(0, 0, HitProxyTextureSize.X, HitProxyTextureSize.Y);

		const Nanite::FRasterContextInitParams RasterInitParams { .TextureSize = HitProxyTextureSize, .TextureRect = HitProxyTextureRect };
		Nanite::FRasterContext RasterContext = Nanite::InitRasterContext(GraphBuilder, SharedContext, ViewFamily, RasterInitParams);

		Nanite::FConfiguration CullingConfig = {0};

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];
			CullingConfig.SetViewFlags(View);

			auto NaniteRenderer = Nanite::IRenderer::Create(
				GraphBuilder,
				*Scene,
				View,
				GetSceneUniforms(),
				SharedContext,
				RasterContext,
				CullingConfig,
				FIntRect(),
				nullptr
			);

			Nanite::FPackedView PackedView = Nanite::CreatePackedViewFromViewInfo(View, HitProxyTextureSize, NANITE_VIEW_FLAG_HZBTEST | NANITE_VIEW_FLAG_NEAR_CLIP);
			NaniteRenderer->DrawGeometry(
				Scene->NaniteRasterPipelines[ENaniteMeshPass::BasePass],
				NaniteRasterResults[ViewIndex].VisibilityQuery,
				*Nanite::FPackedViewArray::Create(GraphBuilder, PackedView)
			);
			NaniteRenderer->ExtractResults( NaniteRasterResults[ViewIndex] );
		}
	}

	::DoRenderHitProxies(GraphBuilder, this, HitProxyTexture, HitProxyDepthTexture, NaniteRasterResults, InstanceCullingManager);

	if (NaniteBasePassVisibility.Visibility)
	{
		NaniteBasePassVisibility.Visibility->FinishVisibilityFrame();
	}

	ShaderPrint::EndViews(Views);

	GEngine->GetPostRenderDelegateEx().Broadcast(GraphBuilder);
	GetSceneExtensionsRenderers().PostRender(GraphBuilder);
#endif

	OnRenderFinish(GraphBuilder, nullptr);
}

#if WITH_EDITOR

bool FHitProxyMeshProcessor::TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial* Material)
{
	const bool bIsTranslucent = IsTranslucentBlendMode(*Material);
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = ComputeMeshCullMode(*Material, OverrideSettings);

	if (Material->WritesEveryPixel() && !Material->IsTwoSided() && !Material->MaterialModifiesMeshPosition_RenderThread())
	{
		// Default material doesn't handle masked, and doesn't have the correct bIsTwoSided setting.
		MaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
		check(MaterialRenderProxy);
		Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
	}

	check(Material && MaterialRenderProxy);

	bool bAddTranslucentPrimitive = bAllowTranslucentPrimitivesInHitProxy;

	// Check whether the primitive overrides the pass to force translucent hit proxies.
	if (!bAddTranslucentPrimitive)
	{
		FHitProxyId HitProxyId = MeshBatch.BatchHitProxyId;

		// Fallback to the primitive default hit proxy id if the mesh batch doesn't have one.
		if (MeshBatch.BatchHitProxyId == FHitProxyId() && PrimitiveSceneProxy)
		{
			if (const FPrimitiveSceneInfo* PrimitiveSceneInfo = PrimitiveSceneProxy->GetPrimitiveSceneInfo())
			{
				HitProxyId = PrimitiveSceneInfo->DefaultDynamicHitProxyId;
			}
		}

		if (const HHitProxy* HitProxy = GetHitProxyById(HitProxyId))
		{
			bAddTranslucentPrimitive = HitProxy->AlwaysAllowsTranslucentPrimitives();
		}
	}

	bool bResult = true;
	if (bAddTranslucentPrimitive || !bIsTranslucent)
	{
		bResult = Process(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *MaterialRenderProxy, *Material, MeshFillMode, MeshCullMode);
	}
	return bResult;
}

void FHitProxyMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (MeshBatch.BatchHitProxyId == FHitProxyId::InvisibleHitProxyId)
	{
		return;
	}

	if (MeshBatch.bUseForMaterial && MeshBatch.bSelectable && Scene->RequiresHitProxies() && (!PrimitiveSceneProxy || PrimitiveSceneProxy->IsSelectable()))
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material && Material->GetRenderingThreadShaderMap())
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

bool GetHitProxyPassShaders(
	const FMaterial& Material,
	FVertexFactoryType* VertexFactoryType,
	ERHIFeatureLevel::Type FeatureLevel,
	TShaderRef<FHitProxyVS>& VertexShader,
	TShaderRef<FHitProxyPS>& PixelShader)
{
	FMaterialShaderTypes ShaderTypes;

	ShaderTypes.AddShaderType<FHitProxyVS>();
	ShaderTypes.AddShaderType<FHitProxyPS>();

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryType, Shaders))
	{
		return false;
	}

	Shaders.TryGetVertexShader(VertexShader);
	Shaders.TryGetPixelShader(PixelShader);
	return true;
}

bool FHitProxyMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		FHitProxyVS,
		FHitProxyPS> HitProxyPassShaders;

	if (!GetHitProxyPassShaders(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		HitProxyPassShaders.VertexShader,
		HitProxyPassShaders.PixelShader))
	{
		return false;
	}

	FHitProxyShaderElementData ShaderElementData(MeshBatch.BatchHitProxyId);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(HitProxyPassShaders.VertexShader, HitProxyPassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		HitProxyPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

FHitProxyMeshProcessor::FHitProxyMeshProcessor(const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, bool InbAllowTranslucentPrimitivesInHitProxy, const FMeshPassProcessorRenderState& InRenderState, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::HitProxy, Scene, Scene->GetFeatureLevel(), InViewIfDynamicMeshCommand, InDrawListContext)
	, PassDrawRenderState(InRenderState)
	, bAllowTranslucentPrimitivesInHitProxy(InbAllowTranslucentPrimitivesInHitProxy)
{
}

FMeshPassProcessor* CreateHitProxyPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
	PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	return new FHitProxyMeshProcessor(Scene, InViewIfDynamicMeshCommand, true, PassDrawRenderState, InDrawListContext);
}

FMeshPassProcessor* CreateHitProxyOpaqueOnlyPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	FMeshPassProcessorRenderState PassDrawRenderState;
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual>::GetRHI());
	PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	return new FHitProxyMeshProcessor(Scene, InViewIfDynamicMeshCommand, false, PassDrawRenderState, InDrawListContext);
}

FRegisterPassProcessorCreateFunction RegisterHitProxyPass(&CreateHitProxyPassProcessor, EShadingPath::Deferred, EMeshPass::HitProxy, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
FRegisterPassProcessorCreateFunction RegisterHitProxyOpaqueOnlyPass(&CreateHitProxyOpaqueOnlyPassProcessor, EShadingPath::Deferred, EMeshPass::HitProxyOpaqueOnly, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
FRegisterPassProcessorCreateFunction RegisterMobileHitProxyPass(&CreateHitProxyPassProcessor, EShadingPath::Mobile, EMeshPass::HitProxy, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);
FRegisterPassProcessorCreateFunction RegisterMobileHitProxyOpaqueOnlyPass(&CreateHitProxyOpaqueOnlyPassProcessor, EShadingPath::Mobile, EMeshPass::HitProxyOpaqueOnly, EMeshPassFlags::CachedMeshCommands | EMeshPassFlags::MainView);

bool FEditorSelectionMeshProcessor::TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial* Material)
{
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = CM_None;

	if (Material->WritesEveryPixel() && !Material->IsTwoSided() && !Material->MaterialModifiesMeshPosition_RenderThread())
	{
		// Default material doesn't handle masked, and doesn't have the correct bIsTwoSided setting.
		MaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
		check(MaterialRenderProxy);
		Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
	}

	check(Material && MaterialRenderProxy);

	return Process(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *MaterialRenderProxy, *Material, MeshFillMode, MeshCullMode);
}

void FEditorSelectionMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (!PrimitiveSceneProxy)
	{
		return;
	}
	const bool bWantsEditorEffects = PrimitiveSceneProxy->WantsEditorEffects();
	const bool bWantsOutlineForSelection = PrimitiveSceneProxy->WantsSelectionOutline() && (PrimitiveSceneProxy->IsSelected() || PrimitiveSceneProxy->IsHovered());
	if (MeshBatch.bUseForMaterial 
		&& MeshBatch.bUseSelectionOutline 
		&& (bWantsEditorEffects || bWantsOutlineForSelection))
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material && Material->GetRenderingThreadShaderMap())
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

bool FEditorSelectionMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		FHitProxyVS,
		FHitProxyPS> HitProxyPassShaders;

	if (!GetHitProxyPassShaders(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		HitProxyPassShaders.VertexShader,
		HitProxyPassShaders.PixelShader))
	{
		return false;
	}

	const int32 StencilRef = GetStencilValue(ViewIfDynamicMeshCommand, PrimitiveSceneProxy);
	PassDrawRenderState.SetStencilRef(StencilRef);

	const FHitProxyId OverlayColor = PrimitiveSceneProxy->GetOverlayColor();
	FHitProxyShaderElementData ShaderElementData(OverlayColor, /*bEditorSelectionPass =*/ true);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(HitProxyPassShaders.VertexShader, HitProxyPassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		HitProxyPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

int32 FEditorSelectionMeshProcessor::GetStencilValue(const FSceneView* View, const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	const bool bActorSelectionColorIsSubdued = View->bHasSelectedComponents;

	const int32* ExistingStencilValue = PrimitiveSceneProxy->IsIndividuallySelected() ? ProxyToStencilIndex.Find(PrimitiveSceneProxy) : ActorNameToStencilIndex.Find(PrimitiveSceneProxy->GetOwnerName());

	static constexpr int BitsAvailable = 8; // Stencil buffer is 8-bit
	static constexpr int ColorBits = 3; // Can be changed
	static constexpr int UniqueIdBits = BitsAvailable - ColorBits;
	static constexpr int MaxColor = (1 << ColorBits);
	static constexpr int MaxUniqueId = (1 << UniqueIdBits);

	// Keep Selection Outline color index (0) just for selection.
	// It is handled differently by EncodeSelectionStencilValue lambda below.
	// Using 0 index for other colors might have the function generate the same stencil value for different elements,
	// preventing the shader from distinguishing them as separate objects.
	static constexpr int SelectionOutlineColorIndex = 0; // See FSceneView::SelectionOutlineColor
	static constexpr int SubduedSelectionOutlineColorIndex = 1; // See FSceneView::SubduedSelectionOutlineColor
	static constexpr int PreselectionOutlineColorIndex = SubduedSelectionOutlineColorIndex;

	auto EncodeSelectionStencilValue = [](int32 ColorIndex, int32 UniqueId) -> int32
	{
		uint8 Bits = 0;
		const int ColorShiftDistance = BitsAvailable - ColorBits;
		const uint8 ColorMask = (0xFFu >> ColorShiftDistance) << ColorShiftDistance;
		const uint8 UniqueIdMask = 0xFF >> (BitsAvailable - UniqueIdBits);
		Bits |= ((ColorIndex % MaxColor) << ColorShiftDistance) & ColorMask;
		// Allow all colors except one to use the full range of unreserved values
		if (ColorIndex == 0)
		{
			Bits |= (UniqueId % (MaxUniqueId - EEditorSelectionStencilValues::COUNT) + EEditorSelectionStencilValues::COUNT) & UniqueIdMask;
		}
		else
		{
			Bits |= (UniqueId % MaxUniqueId) & UniqueIdMask;
		}
		return Bits;
	};
	
	int32 StencilValue = EEditorSelectionStencilValues::NotSelected;

	if (PrimitiveSceneProxy->GetOwnerName() == NAME_BSP)
	{
		StencilValue = EEditorSelectionStencilValues::BSP;
	}
	else if (ExistingStencilValue != nullptr)
	{
		StencilValue = *ExistingStencilValue;
	}
	else if (PrimitiveSceneProxy->IsIndividuallySelected())
	{
		const int Color = 0;
		const int UniqueId = ProxyToStencilIndex.Num();
		StencilValue = EncodeSelectionStencilValue(Color, UniqueId);
		ProxyToStencilIndex.Add(PrimitiveSceneProxy, StencilValue);
	}
	else if (PrimitiveSceneProxy->IsParentSelected())
	{
		int Color = PrimitiveSceneProxy->GetSelectionOutlineColorIndex();
		if (bActorSelectionColorIsSubdued && (Color == SelectionOutlineColorIndex))
		{
			Color = SubduedSelectionOutlineColorIndex;
		}
		const int UniqueId = ActorNameToStencilIndex.Num();

		StencilValue = EncodeSelectionStencilValue(Color, UniqueId);
		ActorNameToStencilIndex.Add(PrimitiveSceneProxy->GetOwnerName(), StencilValue);
	}
	else if (PrimitiveSceneProxy->IsHovered())
	{
		const int UniqueId = ActorNameToStencilIndex.Num();
		StencilValue = EncodeSelectionStencilValue(PreselectionOutlineColorIndex, UniqueId);
		ActorNameToStencilIndex.Add(PrimitiveSceneProxy->GetOwnerName(), StencilValue);
	}

	return StencilValue;
}

FEditorSelectionMeshProcessor::FEditorSelectionMeshProcessor(const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::EditorSelection, Scene, Scene->GetFeatureLevel(), InViewIfDynamicMeshCommand, InDrawListContext)
{
	checkf(InViewIfDynamicMeshCommand, TEXT("Editor selection mesh process required dynamic mesh command mode."));

	ActorNameToStencilIndex.Add(NAME_BSP, EEditorSelectionStencilValues::BSP);

	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Keep, SO_Replace>::GetRHI());
	PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
}

FMeshPassProcessor* CreateEditorSelectionPassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FEditorSelectionMeshProcessor(Scene, InViewIfDynamicMeshCommand, InDrawListContext);
}

FRegisterPassProcessorCreateFunction RegisterEditorSelectionPass(&CreateEditorSelectionPassProcessor, EShadingPath::Deferred, EMeshPass::EditorSelection, EMeshPassFlags::MainView);
FRegisterPassProcessorCreateFunction RegisterMobileEditorSelectionPass(&CreateEditorSelectionPassProcessor, EShadingPath::Mobile, EMeshPass::EditorSelection, EMeshPassFlags::MainView);

void FEditorLevelInstanceMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId)
{
	if (MeshBatch.bUseForMaterial
		&& PrimitiveSceneProxy
		&& PrimitiveSceneProxy->IsEditingLevelInstanceChild())
	{
		const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		while (MaterialRenderProxy)
		{
			const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
			if (Material && Material->GetRenderingThreadShaderMap())
			{
				if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, StaticMeshId, MaterialRenderProxy, Material))
				{
					break;
				}
			}

			MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
		}
	}
}

bool FEditorLevelInstanceMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch, 
	uint64 BatchElementMask, 
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, 
	int32 StaticMeshId, 
	const FMaterialRenderProxy* MaterialRenderProxy, 
	const FMaterial* Material)
{
	// Determine the mesh's material and blend mode.
	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = CM_None;

	if (Material->WritesEveryPixel() && !Material->IsTwoSided() && !Material->MaterialModifiesMeshPosition_RenderThread())
	{
		// Default material doesn't handle masked, and doesn't have the correct bIsTwoSided setting.
		MaterialRenderProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
		check(MaterialRenderProxy);
		Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
	}

	check(Material && MaterialRenderProxy);

	return Process(MeshBatch, BatchElementMask, StaticMeshId, PrimitiveSceneProxy, *MaterialRenderProxy, *Material, MeshFillMode, MeshCullMode);
}

bool FEditorLevelInstanceMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	int32 StaticMeshId,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	ERasterizerFillMode MeshFillMode,
	ERasterizerCullMode MeshCullMode)
{
	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	TMeshProcessorShaders<
		FHitProxyVS,
		FHitProxyPS> HitProxyPassShaders;

	if (!GetHitProxyPassShaders(
		MaterialResource,
		VertexFactory->GetType(),
		FeatureLevel,
		HitProxyPassShaders.VertexShader,
		HitProxyPassShaders.PixelShader
	))
	{
		return false;
	}

	const int32 StencilRef = GetStencilValue(ViewIfDynamicMeshCommand, PrimitiveSceneProxy);
	PassDrawRenderState.SetStencilRef(StencilRef);

	FHitProxyId DummyId;
	FHitProxyShaderElementData ShaderElementData(DummyId);
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, StaticMeshId, false);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(HitProxyPassShaders.VertexShader, HitProxyPassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		MaterialRenderProxy,
		MaterialResource,
		PassDrawRenderState,
		HitProxyPassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);

	return true;
}

int32 FEditorLevelInstanceMeshProcessor::GetStencilValue(const FSceneView* View, const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	return PrimitiveSceneProxy->IsEditingLevelInstanceChild() ? EEditorSelectionStencilValues::VisualizeLevelInstances : EEditorSelectionStencilValues::NotSelected;
}

FEditorLevelInstanceMeshProcessor::FEditorLevelInstanceMeshProcessor(const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::EditorLevelInstance, Scene, Scene->GetFeatureLevel(), InViewIfDynamicMeshCommand, InDrawListContext)
{
	checkf(InViewIfDynamicMeshCommand, TEXT("Editor selection mesh process required dynamic mesh command mode."));

	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<true, CF_DepthNearOrEqual, true, CF_Always, SO_Keep, SO_Keep, SO_Replace>::GetRHI());
	PassDrawRenderState.SetBlendState(TStaticBlendStateWriteMask<CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI());
}

FMeshPassProcessor* CreateEditorLevelInstancePassProcessor(ERHIFeatureLevel::Type FeatureLevel, const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
{
	return new FEditorLevelInstanceMeshProcessor(Scene, InViewIfDynamicMeshCommand, InDrawListContext);
}

FRegisterPassProcessorCreateFunction RegisterEditorLevelInstancePass(&CreateEditorLevelInstancePassProcessor, EShadingPath::Deferred, EMeshPass::EditorLevelInstance, EMeshPassFlags::MainView);
FRegisterPassProcessorCreateFunction RegisterMobileEditorLevelInstancePass(&CreateEditorLevelInstancePassProcessor, EShadingPath::Mobile, EMeshPass::EditorLevelInstance, EMeshPassFlags::MainView);


#endif
