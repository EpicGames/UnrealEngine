// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneHitProxyRendering.h: Scene hit proxy rendering.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "HitProxies.h"
#include "MeshPassProcessor.h"

class FPrimitiveSceneProxy;
class FScene;
class FStaticMeshBatch;

#if WITH_EDITOR

class FHitProxyMeshProcessor : public FSceneRenderingAllocatorObject<FHitProxyMeshProcessor>, public FMeshPassProcessor
{
public:

	FHitProxyMeshProcessor(const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, bool InbAllowTranslucentPrimitivesInHitProxy, const FMeshPassProcessorRenderState& InRenderState, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;

	FMeshPassProcessorRenderState PassDrawRenderState;


private:
	bool TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial* Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	const bool bAllowTranslucentPrimitivesInHitProxy;
};


class FEditorSelectionMeshProcessor : public FSceneRenderingAllocatorObject<FEditorSelectionMeshProcessor>, public FMeshPassProcessor
{
public:

	FEditorSelectionMeshProcessor(const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;

	FMeshPassProcessorRenderState PassDrawRenderState;

private:
	bool TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial* Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	int32 GetStencilValue(const FSceneView* View, const FPrimitiveSceneProxy* PrimitiveSceneProxy);

	/** This map is needed to ensure that individually selected proxies rendered more than once a frame (if they have multiple sections) share a common outline */
	TMap<const FPrimitiveSceneProxy*, int32> ProxyToStencilIndex;
	/** This map is needed to ensure that proxies rendered more than once a frame (if they have multiple sections) share a common outline */
	TMap<FName, int32> ActorNameToStencilIndex;
};

class FEditorLevelInstanceMeshProcessor : public FSceneRenderingAllocatorObject<FEditorLevelInstanceMeshProcessor>, public FMeshPassProcessor
{
public:
	FEditorLevelInstanceMeshProcessor(const FScene* Scene, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final;

	FMeshPassProcessorRenderState PassDrawRenderState;

private:
	bool TryAddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId, const FMaterialRenderProxy* MaterialRenderProxy, const FMaterial* Material);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		int32 StaticMeshId,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		ERasterizerFillMode MeshFillMode,
		ERasterizerCullMode MeshCullMode);

	int32 GetStencilValue(const FSceneView* View, const FPrimitiveSceneProxy* PrimitiveSceneProxy);
};

#endif