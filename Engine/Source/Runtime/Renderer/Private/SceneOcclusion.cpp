// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneRendering.cpp: Scene rendering.
=============================================================================*/

#include "SceneOcclusion.h"
#include "EngineGlobals.h"
#include "SceneRendering.h"
#include "DeferredShadingRenderer.h"
#include "DynamicPrimitiveDrawing.h"
#include "Engine/Level.h"
#include "ScenePrivate.h"
#include "ScreenRendering.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PostProcess/PostProcessing.h"
#include "PlanarReflectionSceneProxy.h"
#include "PipelineStateCache.h"
#include "ShaderParameterStruct.h"
#include "VisualizeTexture.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "PixelShaderUtils.h"
#include "RenderCore.h"
#include "MobileBasePassRendering.h"
#include "Misc/Compression.h"
#include "SceneViewState.h"

/*-----------------------------------------------------------------------------
	Globals
-----------------------------------------------------------------------------*/

int32 GAllowPrecomputedVisibility = 1;
static FAutoConsoleVariableRef CVarAllowPrecomputedVisibility(
	TEXT("r.AllowPrecomputedVisibility"),
	GAllowPrecomputedVisibility,
	TEXT("If zero, precomputed visibility will not be used to cull primitives."),
	ECVF_RenderThreadSafe
	);

static int32 GShowPrecomputedVisibilityCells = 0;
static FAutoConsoleVariableRef CVarShowPrecomputedVisibilityCells(
	TEXT("r.ShowPrecomputedVisibilityCells"),
	GShowPrecomputedVisibilityCells,
	TEXT("If not zero, draw all precomputed visibility cells."),
	ECVF_RenderThreadSafe
	);

static int32 GShowRelevantPrecomputedVisibilityCells = 0;
static FAutoConsoleVariableRef CVarShowRelevantPrecomputedVisibilityCells(
	TEXT("r.ShowRelevantPrecomputedVisibilityCells"),
	GShowRelevantPrecomputedVisibilityCells,
	TEXT("If not zero, draw relevant precomputed visibility cells only."),
	ECVF_RenderThreadSafe
	);

int32 GOcclusionCullCascadedShadowMaps = 0;
FAutoConsoleVariableRef CVarOcclusionCullCascadedShadowMaps(
	TEXT("r.Shadow.OcclusionCullCascadedShadowMaps"),
	GOcclusionCullCascadedShadowMaps,
	TEXT("Whether to use occlusion culling on cascaded shadow maps.  Disabled by default because rapid view changes reveal new regions too quickly for latent occlusion queries to work with."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<bool> CVarMobileEnableOcclusionExtraFrame(
	TEXT("r.Mobile.EnableOcclusionExtraFrame"),
	true,
	TEXT("Whether to allow extra frame for occlusion culling (enabled by default)"),
	ECVF_RenderThreadSafe
	);

int32 GEnableComputeBuildHZB = 1;
static FAutoConsoleVariableRef CVarEnableComputeBuildHZB(
	TEXT("r.EnableComputeBuildHZB"),
	GEnableComputeBuildHZB,
	TEXT("If zero, build HZB using graphics pipeline."),
	ECVF_RenderThreadSafe
	);

int32 GDownsampledOcclusionQueries = 0;
static FAutoConsoleVariableRef CVarDownsampledOcclusionQueries(
	TEXT("r.DownsampledOcclusionQueries"),
	GDownsampledOcclusionQueries,
	TEXT("Whether to issue occlusion queries to a downsampled depth buffer"),
	ECVF_RenderThreadSafe
);

static int32 GOcclusionQueryDispatchOrder = 0;
static FAutoConsoleVariableRef CVarOcclusionQueryDispatchOrder(
	TEXT("r.OcclusionQueryDispatchOrder"),
	GOcclusionQueryDispatchOrder,
	TEXT("0: Grouped queries before individual (default)\n")
	TEXT("1: Individual queries before grouped"),
	ECVF_RenderThreadSafe
);

bool UseDownsampledOcclusionQueries()
{
	return GDownsampledOcclusionQueries != 0;
}

DEFINE_GPU_STAT(HZB);
DECLARE_GPU_DRAWCALL_STAT(BeginOcclusionTests);

/** Random table for occlusion **/
FOcclusionRandomStream GOcclusionRandomStream;

int32 FOcclusionQueryHelpers::GetNumBufferedFrames(ERHIFeatureLevel::Type FeatureLevel)
{
	int32 NumGPUS = 1;

	static const auto NumBufferedQueriesVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.NumBufferedOcclusionQueries"));
	EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

	int32 NumExtraMobileFrames = 0;
	if ((FeatureLevel <= ERHIFeatureLevel::ES3_1 || IsVulkanMobileSM5Platform(ShaderPlatform) || IsMetalMobileSM5Platform(ShaderPlatform)) && CVarMobileEnableOcclusionExtraFrame.GetValueOnAnyThread())
	{
		NumExtraMobileFrames++; // the mobile renderer just doesn't do much after the basepass, and hence it will be asking for the query results almost immediately; the results can't possibly be ready in 1 frame.
		
		bool bNeedsAnotherExtraMobileFrame = IsVulkanPlatform(ShaderPlatform) || IsMetalPlatform(ShaderPlatform) || IsOpenGLPlatform(ShaderPlatform);
		bNeedsAnotherExtraMobileFrame = bNeedsAnotherExtraMobileFrame || IsVulkanMobileSM5Platform(ShaderPlatform) || IsMetalMobileSM5Platform(ShaderPlatform);
		bNeedsAnotherExtraMobileFrame = bNeedsAnotherExtraMobileFrame || FDataDrivenShaderPlatformInfo::GetNeedsExtraMobileFrames(ShaderPlatform);
		bNeedsAnotherExtraMobileFrame = bNeedsAnotherExtraMobileFrame && IsRunningRHIInSeparateThread();

		if (bNeedsAnotherExtraMobileFrame)
		{
			// Android, unfortunately, requires the RHIThread to mediate the readback of queries. Therefore we need an extra frame to avoid a stall in either thread. 
			// The RHIT needs to do read back after the queries are ready and before the RT needs them to avoid stalls. The RHIT may be busy when the queries become ready, so this is all very complicated.
			NumExtraMobileFrames++;
		}
	}

	return FMath::Clamp<int32>(NumExtraMobileFrames + NumBufferedQueriesVar->GetValueOnAnyThread() * NumGPUS, 1, (int32)FOcclusionQueryHelpers::MaxBufferedOcclusionFrames);
}


// default, non-instanced shader implementation
IMPLEMENT_SHADER_TYPE(,FOcclusionQueryVS,TEXT("/Engine/Private/OcclusionQueryVertexShader.usf"),TEXT("Main"),SF_Vertex);
IMPLEMENT_SHADER_TYPE(,FOcclusionQueryPS,TEXT("/Engine/Private/OcclusionQueryPixelShader.usf"),TEXT("Main"),SF_Pixel);

static FGlobalBoundShaderState GOcclusionTestBoundShaderState;

/** 
 * Returns an array of visibility data for the given view position, or NULL if none exists. 
 * The data bits are indexed by VisibilityId of each primitive in the scene.
 * This method decompresses data if necessary and caches it based on the bucket and chunk index in the view state.
 * InScene is passed in, as the Scene pointer in the class itself may be null, if it was allocated without a scene.
 */
const uint8* FSceneViewState::ResolvePrecomputedVisibilityData(FViewInfo& View, const FScene* InScene)
{
	const uint8* PrecomputedVisibilityData = NULL;
	if (InScene->PrecomputedVisibilityHandler && GAllowPrecomputedVisibility && View.Family->EngineShowFlags.PrecomputedVisibility)
	{
		const FPrecomputedVisibilityHandler& Handler = *InScene->PrecomputedVisibilityHandler;
		FViewElementPDI VisibilityCellsPDI(&View, nullptr, nullptr);

		// Draw visibility cell bounds for debugging if enabled
		if ((GShowPrecomputedVisibilityCells || View.Family->EngineShowFlags.PrecomputedVisibilityCells) && !GShowRelevantPrecomputedVisibilityCells)
		{
			for (int32 BucketIndex = 0; BucketIndex < Handler.PrecomputedVisibilityCellBuckets.Num(); BucketIndex++)
			{
				for (int32 CellIndex = 0; CellIndex < Handler.PrecomputedVisibilityCellBuckets[BucketIndex].Cells.Num(); CellIndex++)
				{
					const FPrecomputedVisibilityCell& CurrentCell = Handler.PrecomputedVisibilityCellBuckets[BucketIndex].Cells[CellIndex];
					// Construct the cell's bounds
					const FBox CellBounds(CurrentCell.Min, CurrentCell.Min + FVector(Handler.PrecomputedVisibilityCellSizeXY, Handler.PrecomputedVisibilityCellSizeXY, Handler.PrecomputedVisibilityCellSizeZ));
					if (View.GetCullingFrustum().IntersectBox(CellBounds.GetCenter(), CellBounds.GetExtent()))
					{
						DrawWireBox(&VisibilityCellsPDI, CellBounds, FColor(50, 50, 255), SDPG_World);
					}
				}
			}
		}

		//Determine view origin
		FVector ViewOrigin = View.CullingOrigin;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (const FViewMatrices* FrozenViewMatrices = GetFrozenViewMatrices())
		{
			// Use the frozen view for culling so we can test that it's working
			ViewOrigin = FrozenViewMatrices->GetViewOrigin();
		}
#endif

		// Calculate the bucket that ViewOrigin falls into
		// Cells are hashed into buckets to reduce search time
		const float FloatOffsetX = (ViewOrigin.X - Handler.PrecomputedVisibilityCellBucketOriginXY.X) / Handler.PrecomputedVisibilityCellSizeXY;
		// FMath::TruncToInt rounds toward 0, we want to always round down
		const int32 BucketIndexX = FMath::Abs((FMath::TruncToInt(FloatOffsetX) - (FloatOffsetX < 0.0f ? 1 : 0)) / Handler.PrecomputedVisibilityCellBucketSizeXY % Handler.PrecomputedVisibilityNumCellBuckets);
		const float FloatOffsetY = (ViewOrigin.Y -Handler.PrecomputedVisibilityCellBucketOriginXY.Y) / Handler.PrecomputedVisibilityCellSizeXY;
		const int32 BucketIndexY = FMath::Abs((FMath::TruncToInt(FloatOffsetY) - (FloatOffsetY < 0.0f ? 1 : 0)) / Handler.PrecomputedVisibilityCellBucketSizeXY % Handler.PrecomputedVisibilityNumCellBuckets);
		const int32 PrecomputedVisibilityBucketIndex = BucketIndexY * Handler.PrecomputedVisibilityCellBucketSizeXY + BucketIndexX;

		check(PrecomputedVisibilityBucketIndex < Handler.PrecomputedVisibilityCellBuckets.Num());
		const FPrecomputedVisibilityBucket& CurrentBucket = Handler.PrecomputedVisibilityCellBuckets[PrecomputedVisibilityBucketIndex];
		for (int32 CellIndex = 0; CellIndex < CurrentBucket.Cells.Num(); CellIndex++)
		{
			const FPrecomputedVisibilityCell& CurrentCell = CurrentBucket.Cells[CellIndex];
			// Construct the cell's bounds
			const FBox CellBounds(CurrentCell.Min, CurrentCell.Min + FVector(Handler.PrecomputedVisibilityCellSizeXY, Handler.PrecomputedVisibilityCellSizeXY, Handler.PrecomputedVisibilityCellSizeZ));
			// Check if ViewOrigin is inside the current cell
			if (CellBounds.IsInside(ViewOrigin))
			{
				// Reuse a cached decompressed chunk if possible
				if (CachedVisibilityChunk
					&& CachedVisibilityHandlerId == InScene->PrecomputedVisibilityHandler->GetId()
					&& CachedVisibilityBucketIndex == PrecomputedVisibilityBucketIndex
					&& CachedVisibilityChunkIndex == CurrentCell.ChunkIndex)
				{
					checkSlow(CachedVisibilityChunk->Num() >= CurrentCell.DataOffset + CurrentBucket.CellDataSize);
					PrecomputedVisibilityData = &(*CachedVisibilityChunk)[CurrentCell.DataOffset];
				}
				else
				{
					const FCompressedVisibilityChunk& CompressedChunk = Handler.PrecomputedVisibilityCellBuckets[PrecomputedVisibilityBucketIndex].CellDataChunks[CurrentCell.ChunkIndex];
					CachedVisibilityBucketIndex = PrecomputedVisibilityBucketIndex;
					CachedVisibilityChunkIndex = CurrentCell.ChunkIndex;
					CachedVisibilityHandlerId = InScene->PrecomputedVisibilityHandler->GetId();

					if (CompressedChunk.bCompressed)
					{
						// Decompress the needed visibility data chunk
						DecompressedVisibilityChunk.Reset();
						DecompressedVisibilityChunk.AddUninitialized(CompressedChunk.UncompressedSize);
						verify(FCompression::UncompressMemory(
							NAME_Zlib, 
							DecompressedVisibilityChunk.GetData(),
							CompressedChunk.UncompressedSize,
							CompressedChunk.Data.GetData(),
							CompressedChunk.Data.Num()));
						CachedVisibilityChunk = &DecompressedVisibilityChunk;
					}
					else
					{
						CachedVisibilityChunk = &CompressedChunk.Data;
					}

					checkSlow(CachedVisibilityChunk->Num() >= CurrentCell.DataOffset + CurrentBucket.CellDataSize);
					// Return a pointer to the cell containing ViewOrigin's decompressed visibility data
					PrecomputedVisibilityData = &(*CachedVisibilityChunk)[CurrentCell.DataOffset];
				}

				if (GShowRelevantPrecomputedVisibilityCells)
				{
					// Draw the currently used visibility cell with green wireframe for debugging
					DrawWireBox(&VisibilityCellsPDI, CellBounds, FColor(50, 255, 50), SDPG_Foreground);
				}
				else
				{
					break;
				}
			}
			else if (GShowRelevantPrecomputedVisibilityCells)
			{
				// Draw all cells in the current visibility bucket as blue wireframe
				DrawWireBox(&VisibilityCellsPDI, CellBounds, FColor(50, 50, 255), SDPG_World);
			}
		}
	}
	return PrecomputedVisibilityData;
}

void FSceneViewState::TrimOcclusionHistory(float CurrentTime, float MinHistoryTime, float MinQueryTime, int32 FrameNumber)
{
	// Only trim every few frames, since stale entries won't cause problems
	if (FrameNumber % 6 == 0)
	{
		int32 NumBufferedFrames = FOcclusionQueryHelpers::GetNumBufferedFrames(GetFeatureLevel());

		for(TSet<FPrimitiveOcclusionHistory,FPrimitiveOcclusionHistoryKeyFuncs>::TIterator PrimitiveIt(Occlusion.PrimitiveOcclusionHistorySet);
			PrimitiveIt;
			++PrimitiveIt
			)
		{
			// If the primitive hasn't been considered for visibility recently, remove its history from the set.
			if (PrimitiveIt->LastConsideredTime < MinHistoryTime || PrimitiveIt->LastConsideredTime > CurrentTime)
			{
				PrimitiveIt.RemoveCurrent();
			}
		}
	}
}

bool FSceneViewState::IsShadowOccluded(FSceneViewState::FProjectedShadowKey ShadowKey, int32 NumBufferedFrames) const
{
	// Find the shadow's occlusion query from the previous frame.
	// Get the oldest occlusion query	
	const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryLookupIndex(PendingPrevFrameNumber, NumBufferedFrames);
	const FSceneViewState::ShadowKeyOcclusionQueryMap& ShadowOcclusionQueryMap = ShadowOcclusionQueryMaps[QueryIndex];	
	const FRHIPooledRenderQuery* Query = ShadowOcclusionQueryMap.Find(ShadowKey);

	// Read the occlusion query results.
	uint64 NumSamples = 0;
	const bool bWaitOnQuery = true;

	if (Query && RHIGetRenderQueryResult(Query->GetQuery(), NumSamples, bWaitOnQuery))
	{
		// If the shadow's occlusion query didn't have any pixels visible the previous frame, it's occluded.
		return NumSamples == 0;
	}
	else
	{
		// If the shadow wasn't queried the previous frame, it isn't occluded.

		return false;
	}
}

void FSceneViewState::Destroy()
{
	if (SplitScreenDebugViewState != nullptr)
	{
		SplitScreenDebugViewState->Destroy();
		SplitScreenDebugViewState = nullptr;
	}

	FSceneViewState* self = this;
	ENQUEUE_RENDER_COMMAND(FSceneViewState_Destroy)(
	[self](FRHICommandListImmediate& RHICmdList)
	{
		// Release the occlusion query data.
		self->ReleaseResource();
		// Defer deletion of the view state until the rendering thread is done with it.
		delete self;
	});
}

SIZE_T FSceneViewState::GetSizeBytes() const
{
	uint32 ShadowOcclusionQuerySize = ShadowOcclusionQueryMaps.GetAllocatedSize();
	for (int32 i = 0; i < ShadowOcclusionQueryMaps.Num(); ++i)
	{
		ShadowOcclusionQuerySize += ShadowOcclusionQueryMaps[i].GetAllocatedSize();
	}

	return sizeof(*this) 
		+ ShadowOcclusionQuerySize
		+ PrimitiveFadingStates.GetAllocatedSize()
		+ Occlusion.PrimitiveOcclusionHistorySet.GetAllocatedSize();
}

class FOcclusionQueryIndexBuffer : public FIndexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const uint32 MaxBatchedPrimitives = FOcclusionQueryBatcher::OccludedPrimitiveQueryBatchSize;
		const uint32 Stride = sizeof(uint16);
		const uint32 SizeInBytes = MaxBatchedPrimitives * NUM_CUBE_VERTICES * Stride;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex(TEXT("FOcclusionQueryIndexBuffer"), SizeInBytes, Stride)
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<uint16> Indices = RHICmdList.CreateBufferInitializer(CreateDesc);

		for (uint32 PrimitiveIndex = 0; PrimitiveIndex < MaxBatchedPrimitives; PrimitiveIndex++)
		{
			for(int32 Index = 0;Index < NUM_CUBE_VERTICES;Index++)
			{
				Indices[PrimitiveIndex * NUM_CUBE_VERTICES + Index] = PrimitiveIndex * 8 + GCubeIndices[Index];
			}
		}

		IndexBufferRHI = Indices.Finalize();
	}
};
TGlobalResource<FOcclusionQueryIndexBuffer> GOcclusionQueryIndexBuffer;

FRHIRenderQuery* FFrameBasedOcclusionQueryPool::AllocateQuery()
{
	FFrameOcclusionQueries& CurrentFrame = FrameQueries[CurrentFrameIndex];

	// If we have a free query in the current frame pool, just take it
	if (CurrentFrame.FirstFreeIndex < CurrentFrame.Queries.Num())
	{
		return CurrentFrame.Queries[CurrentFrame.FirstFreeIndex++];
	}

	// If current frame runs out of queries, try to get some from other frames
	for (uint32 Index = 0; Index < UE_ARRAY_COUNT(FrameQueries); ++Index)
	{
		if (Index != CurrentFrameIndex)
		{
			FFrameOcclusionQueries& OtherFrame = FrameQueries[Index];
			while (OtherFrame.FirstFreeIndex < OtherFrame.Queries.Num())
			{
				CurrentFrame.Queries.Add(OtherFrame.Queries.Pop(EAllowShrinking::No));
			}

			if (CurrentFrame.FirstFreeIndex < CurrentFrame.Queries.Num())
			{
				return CurrentFrame.Queries[CurrentFrame.FirstFreeIndex++];
			}
		}
	}

	// If all fails, create a new query
	FRenderQueryRHIRef NewQuery = RHICreateRenderQuery(RQT_Occlusion);
	if (NewQuery)
	{
		CurrentFrame.Queries.Add(MoveTemp(NewQuery));
		return CurrentFrame.Queries[CurrentFrame.FirstFreeIndex++];
	}
	else
	{
		return nullptr;
	}
}

void FFrameBasedOcclusionQueryPool::AdvanceFrame(uint32 InOcclusionFrameCounter, uint32 InNumBufferedFrames, bool bStereoRoundRobin)
{
	if (InOcclusionFrameCounter == OcclusionFrameCounter)
	{
		return;
	}

	OcclusionFrameCounter = InOcclusionFrameCounter;

	if (bStereoRoundRobin)
	{
		InNumBufferedFrames *= 2;
	}

	if (InNumBufferedFrames != NumBufferedFrames)
	{
		FFrameOcclusionQueries TmpFrameQueries[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames * 2];

		for (uint32 Index = 0; Index < NumBufferedFrames; ++Index)
		{
			FFrameOcclusionQueries& Frame = FrameQueries[Index];
			const uint32 NewIndex = FOcclusionQueryHelpers::GetQueryIssueIndex(Frame.OcclusionFrameCounter, InNumBufferedFrames);
			FFrameOcclusionQueries& NewFrame = TmpFrameQueries[NewIndex];

			if (Frame.OcclusionFrameCounter > NewFrame.OcclusionFrameCounter)
			{
				Frame.Queries.Append(MoveTemp(NewFrame.Queries));
				Swap(Frame, NewFrame);
			}
			else
			{
				NewFrame.Queries.Append(MoveTemp(Frame.Queries));
			}
		}

		Swap(FrameQueries, TmpFrameQueries);
		NumBufferedFrames = InNumBufferedFrames;
	}
	
	CurrentFrameIndex = FOcclusionQueryHelpers::GetQueryIssueIndex(OcclusionFrameCounter, NumBufferedFrames);
	check(CurrentFrameIndex < FOcclusionQueryHelpers::MaxBufferedOcclusionFrames * 2);

	FrameQueries[CurrentFrameIndex].FirstFreeIndex = 0;
	FrameQueries[CurrentFrameIndex].OcclusionFrameCounter = OcclusionFrameCounter;
}

FOcclusionQueryBatcher::FOcclusionQueryBatcher(class FSceneViewState* ViewState,uint32 InMaxBatchedPrimitives, uint32 InNumInstances)
:	CurrentBatchOcclusionQuery(NULL)
,	MaxBatchedPrimitives(InMaxBatchedPrimitives)
,	NumBatchedPrimitives(0)
,	OcclusionQueryPool(ViewState ? &ViewState->PrimitiveOcclusionQueryPool : NULL)
,	NumInstances(InNumInstances)
{}

FOcclusionQueryBatcher::~FOcclusionQueryBatcher()
{
	check(!BatchOcclusionQueries.Num());
}

void FOcclusionQueryBatcher::Flush(FRHICommandList& RHICmdList)
{
	if(BatchOcclusionQueries.Num())
	{
		// Create the indices for MaxBatchedPrimitives boxes.
		FRHIBuffer* IndexBufferRHI = GOcclusionQueryIndexBuffer.IndexBufferRHI;

		// Draw the batches.
		for(int32 BatchIndex = 0, NumBatches = BatchOcclusionQueries.Num();BatchIndex < NumBatches;BatchIndex++)
		{
			FOcclusionBatch& Batch = BatchOcclusionQueries[BatchIndex];
			FRHIRenderQuery* BatchOcclusionQuery = Batch.Query;
			FRHIBuffer* VertexBufferRHI = Batch.VertexAllocation.VertexBuffer->VertexBufferRHI;
			uint32 VertexBufferOffset = Batch.VertexAllocation.VertexOffset;
			const int32 NumPrimitivesThisBatch = (BatchIndex != (NumBatches-1)) ? MaxBatchedPrimitives : NumBatchedPrimitives;
				
			RHICmdList.BeginRenderQuery(BatchOcclusionQuery);
			RHICmdList.SetStreamSource(0, VertexBufferRHI, VertexBufferOffset);
			RHICmdList.DrawIndexedPrimitive(
				IndexBufferRHI,
				/*BaseVertexIndex=*/ 0,
				/*MinIndex=*/ 0,
				/*NumVertices=*/ 8 * NumPrimitivesThisBatch,
				/*StartIndex=*/ 0,
				/*NumPrimitives=*/ 12 * NumPrimitivesThisBatch,
				/*NumInstances=*/ NumInstances
				);
			RHICmdList.EndRenderQuery(BatchOcclusionQuery);
		}
		INC_DWORD_STAT_BY(STAT_OcclusionQueries,BatchOcclusionQueries.Num());

		// Reset the batch state.
		BatchOcclusionQueries.Empty(BatchOcclusionQueries.Num());
		CurrentBatchOcclusionQuery = NULL;
	}
}

FRHIRenderQuery* FOcclusionQueryBatcher::BatchPrimitive(const FVector& BoundsOrigin,const FVector& BoundsBoxExtent, FGlobalDynamicVertexBuffer& DynamicVertexBuffer)
{
	// Check if the current batch is full.
	if(CurrentBatchOcclusionQuery == NULL || NumBatchedPrimitives >= MaxBatchedPrimitives)
	{
		check(OcclusionQueryPool);
		CurrentBatchOcclusionQuery = new(BatchOcclusionQueries) FOcclusionBatch;
		CurrentBatchOcclusionQuery->Query = OcclusionQueryPool->AllocateQuery();
		CurrentBatchOcclusionQuery->VertexAllocation = DynamicVertexBuffer.Allocate(MaxBatchedPrimitives * 8 * sizeof(FVector3f));
		check(CurrentBatchOcclusionQuery->VertexAllocation.IsValid());
		NumBatchedPrimitives = 0;
	}

	// Add the primitive's bounding box to the current batch's vertex buffer.
	const FVector3f PrimitiveBoxMin = FVector3f(BoundsOrigin - BoundsBoxExtent);
	const FVector3f PrimitiveBoxMax = FVector3f(BoundsOrigin + BoundsBoxExtent);
	float* RESTRICT Vertices = (float*)CurrentBatchOcclusionQuery->VertexAllocation.Buffer;
	Vertices[ 0] = PrimitiveBoxMin.X; Vertices[ 1] = PrimitiveBoxMin.Y; Vertices[ 2] = PrimitiveBoxMin.Z;
	Vertices[ 3] = PrimitiveBoxMin.X; Vertices[ 4] = PrimitiveBoxMin.Y; Vertices[ 5] = PrimitiveBoxMax.Z;
	Vertices[ 6] = PrimitiveBoxMin.X; Vertices[ 7] = PrimitiveBoxMax.Y; Vertices[ 8] = PrimitiveBoxMin.Z;
	Vertices[ 9] = PrimitiveBoxMin.X; Vertices[10] = PrimitiveBoxMax.Y; Vertices[11] = PrimitiveBoxMax.Z;
	Vertices[12] = PrimitiveBoxMax.X; Vertices[13] = PrimitiveBoxMin.Y; Vertices[14] = PrimitiveBoxMin.Z;
	Vertices[15] = PrimitiveBoxMax.X; Vertices[16] = PrimitiveBoxMin.Y; Vertices[17] = PrimitiveBoxMax.Z;
	Vertices[18] = PrimitiveBoxMax.X; Vertices[19] = PrimitiveBoxMax.Y; Vertices[20] = PrimitiveBoxMin.Z;
	Vertices[21] = PrimitiveBoxMax.X; Vertices[22] = PrimitiveBoxMax.Y; Vertices[23] = PrimitiveBoxMax.Z;

	// Bump the batches buffer pointer.
	Vertices += 24;
	CurrentBatchOcclusionQuery->VertexAllocation.Buffer = (uint8*)Vertices;
	NumBatchedPrimitives++;

	return CurrentBatchOcclusionQuery->Query;
}

enum EShadowOcclusionQueryIntersectionMode
{
	SOQ_None,
	SOQ_LightInfluenceSphere,
	SOQ_NearPlaneVsShadowFrustum
};

static bool AllocateProjectedShadowOcclusionQuery(
	FViewInfo& View, 
	const FProjectedShadowInfo& ProjectedShadowInfo, 
	int32 NumBufferedFrames, 
	EShadowOcclusionQueryIntersectionMode IntersectionMode,
	FRHIRenderQuery*& ShadowOcclusionQuery)
{
	bool bIssueQuery = true;

	if (IntersectionMode == SOQ_LightInfluenceSphere)
	{
		FLightSceneProxy& LightProxy = *(ProjectedShadowInfo.GetLightSceneInfo().Proxy);

		// Make sure to perform the overlap test using the same geometry as will be used to render the sphere
		FVector4f StencilingSpherePosAndScale(ForceInit);
		StencilingGeometry::GStencilSphereVertexBuffer.CalcTransform(StencilingSpherePosAndScale, LightProxy.GetBoundingSphere(), View.ViewMatrices.GetPreViewTranslation());

		const bool bCameraInsideLightGeometry = 
			// Calculate overlap of the conservative sphere that will be rendered vs the camera origin, adding 2xNearClippingDistance of slack to avoid clipping issues
			FMath::Square(StencilingSpherePosAndScale.W + View.NearClippingDistance * 2.0f) >= StencilingSpherePosAndScale.SizeSquared3()
			// Always draw backfaces in ortho
			//@todo - accurate ortho camera / light intersection
			|| !View.IsPerspectiveProjection();

		bIssueQuery = !bCameraInsideLightGeometry;
	}
	else if (IntersectionMode == SOQ_NearPlaneVsShadowFrustum)
	{
		// The shadow transforms and view transforms are relative to different origins, so the world coordinates need to
		// be translated.
		const FVector4f PreShadowToPreViewTranslation(FVector3f(View.ViewMatrices.GetPreViewTranslation() - ProjectedShadowInfo.PreShadowTranslation),0);
	
		// If the shadow frustum is farther from the view origin than the near clipping plane,
		// it can't intersect the near clipping plane.
		const bool bIntersectsNearClippingPlane = ProjectedShadowInfo.ReceiverInnerFrustum.IntersectSphere(
			View.ViewMatrices.GetViewOrigin() + ProjectedShadowInfo.PreShadowTranslation,
			View.NearClippingDistance * FMath::Sqrt(3.0f)
			);

		bIssueQuery = !bIntersectsNearClippingPlane;
	}

	if (bIssueQuery)
	{
		FSceneViewState* ViewState = (FSceneViewState*)View.State;

		// Allocate an occlusion query for the primitive from the occlusion query pool.
		FSceneViewState::FProjectedShadowKey Key(ProjectedShadowInfo);
		const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryIssueIndex(ViewState->PendingPrevFrameNumber, NumBufferedFrames);
		FSceneViewState::ShadowKeyOcclusionQueryMap& ShadowOcclusionQueryMap = ViewState->ShadowOcclusionQueryMaps[QueryIndex];

		checkSlow(ShadowOcclusionQueryMap.Find(Key) == NULL);
		FRHIPooledRenderQuery PooledShadowOcclusionQuery = ViewState->OcclusionQueryPool->AllocateQuery();
		ShadowOcclusionQuery = PooledShadowOcclusionQuery.GetQuery();
		ShadowOcclusionQueryMap.Add(Key, MoveTemp(PooledShadowOcclusionQuery));
	}
	
	return bIssueQuery;
}

static void ExecutePointLightShadowOcclusionQuery(FRHICommandList& RHICmdList, FViewInfo& View, const FProjectedShadowInfo& ProjectedShadowInfo, const TShaderRef<FOcclusionQueryVS>& VertexShader, FRHIRenderQuery* ShadowOcclusionQuery)
{
	FLightSceneProxy& LightProxy = *(ProjectedShadowInfo.GetLightSceneInfo().Proxy);
	
	// Query one pass point light shadows separately because they don't have a shadow frustum, they have a bounding sphere instead.
	FSphere LightBounds = LightProxy.GetBoundingSphere();

	RHICmdList.BeginRenderQuery(ShadowOcclusionQuery);
	
	// Draw bounding sphere
	SetShaderParametersLegacyVS(RHICmdList, VertexShader, View, LightBounds);

	StencilingGeometry::DrawVectorSphere(RHICmdList);
		
	RHICmdList.EndRenderQuery(ShadowOcclusionQuery);
}

static void PrepareDirectionalLightShadowOcclusionQuery(uint32& BaseVertexIndex, TRHIBufferInitializer<FVector3f>& DestinationBuffer, const FViewInfo& View, const FProjectedShadowInfo& ProjectedShadowInfo)
{
	const FViewMatrices& ViewMatrices = View.GetShadowViewMatrices();

	const FMatrix& ViewMatrix = ViewMatrices.GetWorldToView();
	const FMatrix& ProjectionMatrix = ViewMatrices.GetViewToClip();
	const FVector CameraDirection = ViewMatrix.GetColumn(2);
	const float SplitNear = ProjectedShadowInfo.CascadeSettings.SplitNear;

	float AspectRatio = ProjectionMatrix.M[1][1] / ProjectionMatrix.M[0][0];
	float HalfFOV = ViewMatrices.IsPerspectiveProjection() ? FMath::Atan(1.0f / ProjectionMatrix.M[0][0]) : PI / 4.0f;

	// Build the camera frustum for this cascade
	const float StartHorizontalLength = SplitNear * FMath::Tan(HalfFOV);
	const FVector StartCameraRightOffset = ViewMatrix.GetColumn(0) * StartHorizontalLength;
	const float StartVerticalLength = StartHorizontalLength / AspectRatio;
	const FVector StartCameraUpOffset = ViewMatrix.GetColumn(1) * StartVerticalLength;

	FVector3f Verts[4] =
	{
		FVector3f(CameraDirection * SplitNear + StartCameraRightOffset + StartCameraUpOffset),
		FVector3f(CameraDirection * SplitNear + StartCameraRightOffset - StartCameraUpOffset),
		FVector3f(CameraDirection * SplitNear - StartCameraRightOffset - StartCameraUpOffset),
		FVector3f(CameraDirection * SplitNear - StartCameraRightOffset + StartCameraUpOffset)
	};

	DestinationBuffer[BaseVertexIndex + 0] = Verts[0];
	DestinationBuffer[BaseVertexIndex + 1] = Verts[3];
	DestinationBuffer[BaseVertexIndex + 2] = Verts[2];
	DestinationBuffer[BaseVertexIndex + 3] = Verts[0];
	DestinationBuffer[BaseVertexIndex + 4] = Verts[2];
	DestinationBuffer[BaseVertexIndex + 5] = Verts[1];
	BaseVertexIndex += 6;
}

static void ExecuteDirectionalLightShadowOcclusionQuery(FRHICommandList& RHICmdList, uint32& BaseVertexIndex, FRHIRenderQuery* ShadowOcclusionQuery)
{
	RHICmdList.BeginRenderQuery(ShadowOcclusionQuery);

	RHICmdList.DrawPrimitive(BaseVertexIndex, 2, 1);
	BaseVertexIndex += 6;

	RHICmdList.EndRenderQuery(ShadowOcclusionQuery);
}

static void PrepareProjectedShadowOcclusionQuery(uint32& BaseVertexIndex, TRHIBufferInitializer<FVector3f>& DestinationBuffer, const FViewInfo& View, const FProjectedShadowInfo& ProjectedShadowInfo)
{
	// The shadow transforms and view transforms are relative to different origins, so the world coordinates need to
	// be translated.
	const FVector4f PreShadowToPreViewTranslation(FVector3f(View.ViewMatrices.GetPreViewTranslation() - ProjectedShadowInfo.PreShadowTranslation), 0);

	// Generate vertices for the shadow's frustum.
	for (uint32 Z = 0; Z < 2; Z++)
	{
		for (uint32 Y = 0; Y < 2; Y++)
		{
			for (uint32 X = 0; X < 2; X++)
			{
				const FVector4f UnprojectedVertex = ProjectedShadowInfo.InvReceiverInnerMatrix.TransformFVector4(
					FVector4f(
						(X ? -1.0f : 1.0f),
						(Y ? -1.0f : 1.0f),
						(Z ?  0.0f : 1.0f),
						1.0f)
				);
				const FVector3f ProjectedVertex = UnprojectedVertex / UnprojectedVertex.W + PreShadowToPreViewTranslation;
				const uint16 CubeVertexIndex = GetCubeVertexIndex(X, Y, Z);

				DestinationBuffer.WriteValueAtIndex(BaseVertexIndex + CubeVertexIndex, ProjectedVertex);
			}
		}
	}

	BaseVertexIndex += 8;
}

static void ExecuteProjectedShadowOcclusionQuery(FRHICommandList& RHICmdList, uint32& BaseVertexIndex, FRHIRenderQuery* ShadowOcclusionQuery)
{	
	// Draw the primitive's bounding box, using the occlusion query.
	RHICmdList.BeginRenderQuery(ShadowOcclusionQuery);

	RHICmdList.DrawIndexedPrimitive(GCubeIndexBuffer.IndexBufferRHI, BaseVertexIndex, 0, 8, 0, 12, 1);
	BaseVertexIndex += 8;

	RHICmdList.EndRenderQuery(ShadowOcclusionQuery);
}

static bool AllocatePlanarReflectionOcclusionQuery(const FViewInfo& View, const FPlanarReflectionSceneProxy* SceneProxy, int32 NumBufferedFrames, FRHIRenderQuery*& OcclusionQuery)
{
	FSceneViewState* ViewState = (FSceneViewState*)View.State;
	
	bool bAllowBoundsTest = false;
	
	if (View.GetCullingFrustum().IntersectBox(SceneProxy->WorldBounds.GetCenter(), SceneProxy->WorldBounds.GetExtent()))
	{
		const FBoxSphereBounds OcclusionBounds(SceneProxy->WorldBounds);
		
		if (View.bHasNearClippingPlane)
		{
			bAllowBoundsTest = View.NearClippingPlane.PlaneDot(OcclusionBounds.Origin) <
			-(FVector::BoxPushOut(View.NearClippingPlane, OcclusionBounds.BoxExtent));
			
		}
		else if (!View.IsPerspectiveProjection())
		{
			// Transform parallel near plane
			bAllowBoundsTest = View.WorldToScreen(OcclusionBounds.Origin).Z - View.ViewMatrices.GetViewToClip().M[2][2] * OcclusionBounds.SphereRadius < 1;
		}
		else
		{
			bAllowBoundsTest = OcclusionBounds.SphereRadius < HALF_WORLD_MAX;
		}
	}
	
	uint32 OcclusionFrameCounter = ViewState->OcclusionFrameCounter;
	FIndividualOcclusionHistory& OcclusionHistory = ViewState->PlanarReflectionOcclusionHistories.FindOrAdd(SceneProxy->PlanarReflectionId);
	OcclusionHistory.ReleaseQuery(OcclusionFrameCounter, NumBufferedFrames);
	
	if (bAllowBoundsTest)
	{
		// Allocate an occlusion query for the primitive from the occlusion query pool.
		FRHIPooledRenderQuery PooledOcclusionQuery = ViewState->OcclusionQueryPool->AllocateQuery();
		OcclusionQuery = PooledOcclusionQuery.GetQuery();

		OcclusionHistory.SetCurrentQuery(OcclusionFrameCounter, MoveTemp(PooledOcclusionQuery), NumBufferedFrames);
	}
	else
	{
		OcclusionHistory.SetCurrentQuery(OcclusionFrameCounter, FRHIPooledRenderQuery(), NumBufferedFrames);
	}
	
	return bAllowBoundsTest;
}

static void PreparePlanarReflectionOcclusionQuery(uint32& BaseVertexIndex, TRHIBufferInitializer<FVector3f>& DestinationBuffer, const FViewInfo& View, const FPlanarReflectionSceneProxy* SceneProxy)
{
	const FVector3f PrimitiveBoxMin = FVector3f(SceneProxy->WorldBounds.Min + View.ViewMatrices.GetPreViewTranslation());
	const FVector3f PrimitiveBoxMax = FVector3f(SceneProxy->WorldBounds.Max + View.ViewMatrices.GetPreViewTranslation());

	DestinationBuffer.WriteValueAtIndex(BaseVertexIndex + 0, FVector3f(PrimitiveBoxMin.X, PrimitiveBoxMin.Y, PrimitiveBoxMin.Z));
	DestinationBuffer.WriteValueAtIndex(BaseVertexIndex + 1, FVector3f(PrimitiveBoxMin.X, PrimitiveBoxMin.Y, PrimitiveBoxMax.Z));
	DestinationBuffer.WriteValueAtIndex(BaseVertexIndex + 2, FVector3f(PrimitiveBoxMin.X, PrimitiveBoxMax.Y, PrimitiveBoxMin.Z));
	DestinationBuffer.WriteValueAtIndex(BaseVertexIndex + 3, FVector3f(PrimitiveBoxMin.X, PrimitiveBoxMax.Y, PrimitiveBoxMax.Z));
	DestinationBuffer.WriteValueAtIndex(BaseVertexIndex + 4, FVector3f(PrimitiveBoxMax.X, PrimitiveBoxMin.Y, PrimitiveBoxMin.Z));
	DestinationBuffer.WriteValueAtIndex(BaseVertexIndex + 5, FVector3f(PrimitiveBoxMax.X, PrimitiveBoxMin.Y, PrimitiveBoxMax.Z));
	DestinationBuffer.WriteValueAtIndex(BaseVertexIndex + 6, FVector3f(PrimitiveBoxMax.X, PrimitiveBoxMax.Y, PrimitiveBoxMin.Z));
	DestinationBuffer.WriteValueAtIndex(BaseVertexIndex + 7, FVector3f(PrimitiveBoxMax.X, PrimitiveBoxMax.Y, PrimitiveBoxMax.Z));

	BaseVertexIndex += 8;
}

static void ExecutePlanarReflectionOcclusionQuery(FRHICommandList& RHICmdList, uint32& BaseVertexIndex, FRHIRenderQuery* OcclusionQuery)
{
	// Draw the primitive's bounding box, using the occlusion query.
	RHICmdList.BeginRenderQuery(OcclusionQuery);

	RHICmdList.DrawIndexedPrimitive(GCubeIndexBuffer.IndexBufferRHI, BaseVertexIndex, 0, 8, 0, 12, 1);
	BaseVertexIndex += 8;

	RHICmdList.EndRenderQuery(OcclusionQuery);
}

FHZBOcclusionTester::FHZBOcclusionTester()
	: ResultsBuffer(nullptr)
	, ResultsBufferRowPitch(0)
{
	SetInvalidFrameNumber();
}

bool FHZBOcclusionTester::IsValidFrame(uint32 FrameNumber) const
{
	return (FrameNumber & FrameNumberMask) == ValidFrameNumber;
}

void FHZBOcclusionTester::SetValidFrameNumber(uint32 FrameNumber)
{
	ValidFrameNumber = FrameNumber & FrameNumberMask;

	checkSlow(!IsInvalidFrame());
}

bool FHZBOcclusionTester::IsInvalidFrame() const
{
	return ValidFrameNumber == InvalidFrameNumber;
}

void FHZBOcclusionTester::SetInvalidFrameNumber()
{
	// this number cannot be set by SetValidFrameNumber()
	ValidFrameNumber = InvalidFrameNumber;

	checkSlow(IsInvalidFrame());
}

void FHZBOcclusionTester::InitRHI(FRHICommandListBase& RHICmdList)
{
	if (GetFeatureLevel() >= ERHIFeatureLevel::SM5)
	{
		ResultsReadback.Reset(new FRHIGPUTextureReadback(TEXT("HZBGPUReadback")));
	}
}

void FHZBOcclusionTester::ReleaseRHI()
{
	if (GetFeatureLevel() >= ERHIFeatureLevel::SM5)
	{
		ResultsReadback.Reset();
	}
}

uint32 FHZBOcclusionTester::AddBounds( const FVector& BoundsCenter, const FVector& BoundsExtent )
{
	uint32 Index = Primitives.AddUninitialized();
	check( Index < SizeX * SizeY );
	Primitives[ Index ].Center = BoundsCenter;
	Primitives[ Index ].Extent = BoundsExtent;
	return Index;
}

void FHZBOcclusionTester::MapResults(FRHICommandListImmediate& RHICmdList)
{
	check( !ResultsBuffer );

	if (!IsInvalidFrame() )
	{
		// RHIMapStagingSurface will block until the results are ready (from the previous frame) so we need to consider this RT idle time
		FRenderThreadIdleScope IdleScope(ERenderThreadIdleTypes::WaitingForGPUQuery);

		SCOPED_GPU_MASK(RHICmdList, ResultsReadback->GetLastCopyGPUMask());

		int32 ResultBufferHeight = 0;
		ResultsBufferRowPitch = 0;
		ResultsBuffer = reinterpret_cast<const uint8*>(ResultsReadback->Lock(ResultsBufferRowPitch, &ResultBufferHeight));
		if (ResultsBuffer)
		{
			check(ResultsBufferRowPitch >= SizeX);
			check(ResultBufferHeight >= SizeY);
		}
	}
	
	// Can happen because of device removed, we might crash later but this occlusion culling system can behave gracefully.
	// It also happens if Submit has not been called yet, which can occur if there are no primitives in the scene.
	if( ResultsBuffer == nullptr )
	{
		static uint8 FirstFrameBuffer[] = { 255 };
		ResultsBuffer = FirstFrameBuffer;
		SetInvalidFrameNumber();
	}
}

void FHZBOcclusionTester::UnmapResults(FRHICommandListImmediate& RHICmdList)
{
	check( ResultsBuffer );
	if(!IsInvalidFrame())
	{
		SCOPED_GPU_MASK(RHICmdList, ResultsReadback->GetLastCopyGPUMask());
		ResultsReadback->Unlock();
	}
	ResultsBuffer = nullptr;
}

bool FHZBOcclusionTester::IsVisible( uint32 Index ) const
{
	checkSlow( ResultsBuffer );
	checkSlow( Index < SizeX * SizeY );
	
	// TODO shader compress to bits

#if 0
	return ResultsBuffer[ 4 * Index ] != 0;
#elif 0
	uint32 x = FMath::ReverseMortonCode2( Index >> 0 );
	uint32 y = FMath::ReverseMortonCode2( Index >> 1 );
	uint32 m = x + y * SizeX;
	return ResultsBuffer[ 4 * m ] != 0;
#else
	// TODO put block constants in class
	// TODO optimize
	const uint32 BlockSize = 8;
	const uint32 SizeInBlocksX = SizeX / BlockSize;
	const uint32 SizeInBlocksY = SizeY / BlockSize;

	const int32 BlockIndex = Index / (BlockSize * BlockSize);
	const int32 BlockX = BlockIndex % SizeInBlocksX;
	const int32 BlockY = BlockIndex / SizeInBlocksY;

	const int32 b = Index % (BlockSize * BlockSize);
	const int32 x = BlockX * BlockSize + b % BlockSize;
	const int32 y = BlockY * BlockSize + b / BlockSize;

	return ResultsBuffer[ 4 * (x + y * ResultsBufferRowPitch) ] != 0;
#endif
}


class FHZBTestPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHZBTestPS);
	SHADER_USE_PARAMETER_STRUCT(FHZBTestPS, FGlobalShader)

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_STRUCT_INCLUDE(FHZBParameters, HZBParameters)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BoundsCenterTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BoundsCenterSampler)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, BoundsExtentTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, BoundsExtentSampler)

		SHADER_PARAMETER(FVector2f, InvTestTargetSize)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}
};

IMPLEMENT_GLOBAL_SHADER(FHZBTestPS, "/Engine/Private/HZBOcclusion.usf", "HZBTestPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FHZBOcclusionUpdateTexturesParameters, )
	RDG_TEXTURE_ACCESS(BoundsCenterTexture, ERHIAccess::CopyDest)
	RDG_TEXTURE_ACCESS(BoundsExtentTexture, ERHIAccess::CopyDest)
END_SHADER_PARAMETER_STRUCT()

void FHZBOcclusionTester::Submit(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	RDG_EVENT_SCOPE(GraphBuilder, "SubmitHZB");

	FSceneViewState* ViewState = (FSceneViewState*)View.State;
	if( !ViewState )
	{
		return;
	}

	FRDGTextureRef BoundsCenterTexture = nullptr;
	FRDGTextureRef BoundsExtentTexture = nullptr;

	{
		const FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(FIntPoint(SizeX, SizeY), PF_A32B32G32R32F, FClearValueBinding::None, TexCreate_RenderTargetable | TexCreate_ShaderResource));
		BoundsCenterTexture = GraphBuilder.CreateTexture(Desc, TEXT("HZBBoundsCenter"));
		BoundsExtentTexture = GraphBuilder.CreateTexture(Desc, TEXT("HZBBoundsExtent"));
	}

	FRDGTextureRef ResultsTextureGPU = nullptr;
	{
		const FRDGTextureDesc Desc(FRDGTextureDesc::Create2D(FIntPoint(SizeX, SizeY), PF_B8G8R8A8, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_RenderTargetable));
		ResultsTextureGPU = GraphBuilder.CreateTexture(Desc, TEXT("HZBResultsGPU"));
	}

	{
		auto* PassParameters = GraphBuilder.AllocParameters<FHZBOcclusionUpdateTexturesParameters>();
		PassParameters->BoundsCenterTexture = BoundsCenterTexture;
		PassParameters->BoundsExtentTexture = BoundsExtentTexture;

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("UpdateTextures"),
			PassParameters,
			ERDGPassFlags::Copy,
			[Primitives = MoveTemp(Primitives), BoundsCenterTexture, BoundsExtentTexture](FRHICommandListImmediate& RHICmdList)
		{
			// Update in blocks to avoid large update
			const uint32 BlockSize = 8;
			const uint32 SizeInBlocksX = SizeX / BlockSize;
			const uint32 SizeInBlocksY = SizeY / BlockSize;
			const uint32 BlockStride = BlockSize * 4 * sizeof(float);

			float CenterBuffer[BlockSize * BlockSize][4];
			float ExtentBuffer[BlockSize * BlockSize][4];

			const uint32 NumPrimitives = Primitives.Num();
			for (uint32 i = 0; i < NumPrimitives; i += BlockSize * BlockSize)
			{
				const uint32 BlockEnd = FMath::Min(BlockSize * BlockSize, NumPrimitives - i);
				for (uint32 b = 0; b < BlockEnd; b++)
				{
					const FOcclusionPrimitive& Primitive = Primitives[i + b];

					CenterBuffer[b][0] = Primitive.Center.X;
					CenterBuffer[b][1] = Primitive.Center.Y;
					CenterBuffer[b][2] = Primitive.Center.Z;
					CenterBuffer[b][3] = 0.0f;

					ExtentBuffer[b][0] = Primitive.Extent.X;
					ExtentBuffer[b][1] = Primitive.Extent.Y;
					ExtentBuffer[b][2] = Primitive.Extent.Z;
					ExtentBuffer[b][3] = 1.0f;
				}

				// Clear rest of block
				if (BlockEnd < BlockSize * BlockSize)
				{
					FMemory::Memset((float*)CenterBuffer + BlockEnd * 4, 0, sizeof(CenterBuffer) - BlockEnd * 4 * sizeof(float));
					FMemory::Memset((float*)ExtentBuffer + BlockEnd * 4, 0, sizeof(ExtentBuffer) - BlockEnd * 4 * sizeof(float));
				}

				const int32 BlockIndex = i / (BlockSize * BlockSize);
				const int32 BlockX = BlockIndex % SizeInBlocksX;
				const int32 BlockY = BlockIndex / SizeInBlocksY;

				FUpdateTextureRegion2D Region(BlockX * BlockSize, BlockY * BlockSize, 0, 0, BlockSize, BlockSize);
				RHICmdList.UpdateTexture2D((FRHITexture*)BoundsCenterTexture->GetRHI(), 0, Region, BlockStride, (uint8*)CenterBuffer);
				RHICmdList.UpdateTexture2D((FRHITexture*)BoundsExtentTexture->GetRHI(), 0, Region, BlockStride, (uint8*)ExtentBuffer);
			}
		});
	}

	// Draw test
	{
		FHZBTestPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHZBTestPS::FParameters>();
		PassParameters->View = View.ViewUniformBuffer;
		PassParameters->BoundsCenterTexture = BoundsCenterTexture;
		PassParameters->BoundsExtentTexture = BoundsExtentTexture;
		PassParameters->RenderTargets[0] = FRenderTargetBinding(ResultsTextureGPU, ERenderTargetLoadAction::ENoAction);
		PassParameters->HZBParameters = GetHZBParameters(GraphBuilder, View, EHZBType::FurthestHZB);

		PassParameters->BoundsCenterSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		PassParameters->BoundsExtentSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

		PassParameters->InvTestTargetSize = FVector2f(1.0f / float(SizeX), 1.0f / float(SizeY));

		TShaderMapRef< FHZBTestPS >	PixelShader(View.ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			View.ShaderMap,
			RDG_EVENT_NAME("TestHZB"),
			PixelShader,
			PassParameters,
			FIntRect(0, 0, SizeX, SizeY),
			TStaticBlendState<>::GetRHI(),
			TStaticRasterizerState<>::GetRHI(),
			TStaticDepthStencilState<false, CF_Always>::GetRHI());
	}

	// Transfer memory GPU -> CPU
	AddEnqueueCopyPass(GraphBuilder, ResultsReadback.Get(), ResultsTextureGPU);
}

static void TrimAllOcclusionHistory(TArrayView<FViewInfo> Views)
{
	for (const FViewInfo& View : Views)
	{
		if (View.Family && View.ViewState)
		{
			const double RealTimeSeconds = View.Family->Time.GetRealTimeSeconds();
			View.ViewState->TrimOcclusionHistory(RealTimeSeconds, RealTimeSeconds - GEngine->PrimitiveProbablyVisibleTime, RealTimeSeconds, View.ViewState->OcclusionFrameCounter);
		}
	}
}

void AllocateOcclusionTests(const FScene* Scene, FViewOcclusionQueriesPerView& QueriesPerView, TArrayView<FViewInfo> Views, const FVisibleLightInfoArray& VisibleLightInfos)
{
	SCOPED_NAMED_EVENT(FSceneRenderer_AllocateOcclusionTests, FColor::Emerald);

	const ERHIFeatureLevel::Type FeatureLevel = Scene->GetFeatureLevel();
	const int32 NumBufferedFrames = FOcclusionQueryHelpers::GetNumBufferedFrames(FeatureLevel);

	bool bBatchedQueries = false;

	QueriesPerView.AddDefaulted(Views.Num());

	// Perform occlusion queries for each view
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		// Don't draw occlusion queries for both eyes in multiview
		FViewInfo& View = Views[ViewIndex];
		if (View.bIsSinglePassStereo && IStereoRendering::IsASecondaryPass(View.StereoPass))
		{
			continue;
		}

		FViewOcclusionQueries& ViewQuery = QueriesPerView[ViewIndex];
		FSceneViewState* ViewState = (FSceneViewState*)View.State;
		const FSceneViewFamily& ViewFamily = *View.Family;

		if (ViewState && !View.bDisableQuerySubmissions)
		{
			// Issue this frame's occlusion queries (occlusion queries from last frame may still be in flight)
			const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryIssueIndex(ViewState->PendingPrevFrameNumber, NumBufferedFrames);
			FSceneViewState::ShadowKeyOcclusionQueryMap& ShadowOcclusionQueryMap = ViewState->ShadowOcclusionQueryMaps[QueryIndex];

			// Clear primitives which haven't been visible recently out of the occlusion history, and reset old pending occlusion queries.
			ViewState->TrimOcclusionHistory(ViewFamily.Time.GetRealTimeSeconds(), ViewFamily.Time.GetRealTimeSeconds() - GEngine->PrimitiveProbablyVisibleTime, ViewFamily.Time.GetRealTimeSeconds(), ViewState->OcclusionFrameCounter);

			// Give back all these occlusion queries to the pool.
			ShadowOcclusionQueryMap.Reset();

			if (FeatureLevel > ERHIFeatureLevel::ES3_1)
			{
				for (auto LightIt = Scene->Lights.CreateConstIterator(); LightIt; ++LightIt)
				{
					const FVisibleLightInfo& VisibleLightInfo = VisibleLightInfos[LightIt.GetIndex()];

					for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightInfo.AllProjectedShadows.Num(); ShadowIndex++)
					{
						const FProjectedShadowInfo& ProjectedShadowInfo = *VisibleLightInfo.AllProjectedShadows[ShadowIndex];

						if (ProjectedShadowInfo.DependentView && ProjectedShadowInfo.DependentView != &View)
						{
							continue;
						}

						if (!IsShadowCacheModeOcclusionQueryable(ProjectedShadowInfo.CacheMode))
						{
							// Only query one of the cache modes for each shadow
							continue;
						}

						if (ProjectedShadowInfo.HasVirtualShadowMap())
						{
							// Skip virtual SMs, as they overlay the same light as physical SM and have their own culling methods anyway.
							continue;
						}
						if (ProjectedShadowInfo.IsWholeScenePointLightShadow())
						{
							FRHIRenderQuery* ShadowOcclusionQuery;
							if (AllocateProjectedShadowOcclusionQuery(View, ProjectedShadowInfo, NumBufferedFrames, SOQ_LightInfluenceSphere, ShadowOcclusionQuery))
							{
								ViewQuery.LocalLightQueryInfos.Add(&ProjectedShadowInfo);
								ViewQuery.LocalLightQueries.Add(ShadowOcclusionQuery);
								checkSlow(ViewQuery.LocalLightQueryInfos.Num() == ViewQuery.LocalLightQueries.Num());
								bBatchedQueries = true;
							}
						}
						else if (ProjectedShadowInfo.IsWholeSceneDirectionalShadow())
						{
							// Don't query the first cascade, it is always visible
							if (GOcclusionCullCascadedShadowMaps && ProjectedShadowInfo.CascadeSettings.ShadowSplitIndex > 0)
							{
								FRHIRenderQuery* ShadowOcclusionQuery;
								if (AllocateProjectedShadowOcclusionQuery(View, ProjectedShadowInfo, NumBufferedFrames, SOQ_None, ShadowOcclusionQuery))
								{
									ViewQuery.CSMQueryInfos.Add(&ProjectedShadowInfo);
									ViewQuery.CSMQueries.Add(ShadowOcclusionQuery);
									checkSlow(ViewQuery.CSMQueryInfos.Num() == ViewQuery.CSMQueries.Num());
									bBatchedQueries = true;
								}
							}
						}
						else if (
							// Don't query preshadows, since they are culled if their subject is occluded.
							!ProjectedShadowInfo.bPreShadow
							// Don't query if any subjects are visible because the shadow frustum will be definitely unoccluded
							&& !ProjectedShadowInfo.SubjectsVisible(View))
						{
							FRHIRenderQuery* ShadowOcclusionQuery;
							if (AllocateProjectedShadowOcclusionQuery(View, ProjectedShadowInfo, NumBufferedFrames, SOQ_NearPlaneVsShadowFrustum, ShadowOcclusionQuery))
							{
								ViewQuery.ShadowQueryInfos.Add(&ProjectedShadowInfo);
								ViewQuery.ShadowQueries.Add(ShadowOcclusionQuery);
								checkSlow(ViewQuery.ShadowQueryInfos.Num() == ViewQuery.ShadowQueries.Num());
								bBatchedQueries = true;
							}
						}
					}

					// Issue occlusion queries for all per-object projected shadows that we would have rendered but were occluded last frame.
					for (int32 ShadowIndex = 0; ShadowIndex < VisibleLightInfo.OccludedPerObjectShadows.Num(); ShadowIndex++)
					{
						const FProjectedShadowInfo& ProjectedShadowInfo = *VisibleLightInfo.OccludedPerObjectShadows[ShadowIndex];
						FRHIRenderQuery* ShadowOcclusionQuery;
						if (AllocateProjectedShadowOcclusionQuery(View, ProjectedShadowInfo, NumBufferedFrames, SOQ_NearPlaneVsShadowFrustum, ShadowOcclusionQuery))
						{
							ViewQuery.ShadowQueryInfos.Add(&ProjectedShadowInfo);
							ViewQuery.ShadowQueries.Add(ShadowOcclusionQuery);
							checkSlow(ViewQuery.ShadowQueryInfos.Num() == ViewQuery.ShadowQueries.Num());
							bBatchedQueries = true;
						}
					}
				}
			}

			if (FeatureLevel > ERHIFeatureLevel::ES3_1 &&
				!View.bIsPlanarReflection &&
				!View.bIsSceneCapture &&
				!View.bIsReflectionCapture)
			{
				// +1 to buffered frames because the query is submitted late into the main frame, but read at the beginning of a frame
				const int32 NumReflectionBufferedFrames = NumBufferedFrames + 1;

				for (int32 ReflectionIndex = 0; ReflectionIndex < Scene->PlanarReflections.Num(); ReflectionIndex++)
				{
					FPlanarReflectionSceneProxy* SceneProxy = Scene->PlanarReflections[ReflectionIndex];
					FRHIRenderQuery* ShadowOcclusionQuery;
					if (AllocatePlanarReflectionOcclusionQuery(View, SceneProxy, NumReflectionBufferedFrames, ShadowOcclusionQuery))
					{
						ViewQuery.ReflectionQueryInfos.Add(SceneProxy);
						ViewQuery.ReflectionQueries.Add(ShadowOcclusionQuery);
						checkSlow(ViewQuery.ReflectionQueryInfos.Num() == ViewQuery.ReflectionQueries.Num());
						bBatchedQueries = true;
					}
				}
			}

			bBatchedQueries |= (View.IndividualOcclusionQueries.HasBatches() || View.GroupedOcclusionQueries.HasBatches() || ViewQuery.bFlushQueries);
		}
	}

	// Return an empty array if no queries exist.
	if (!bBatchedQueries)
	{
		QueriesPerView.Empty();
	}
}

static void BeginOcclusionTests(
	FRHICommandList& RHICmdList,
	FViewInfo& View,
	const FViewOcclusionQueries& ViewQuery,
	ERHIFeatureLevel::Type FeatureLevel, 
	uint32 DownsampleFactor,
	FGraphicsPipelineStateInitializer* InGraphicsPSOInit = nullptr)
{
	check(RHICmdList.IsInsideRenderPass());

	SCOPE_CYCLE_COUNTER(STAT_BeginOcclusionTestsTime);
	SCOPED_DRAW_EVENT(RHICmdList, BeginOcclusionTests);

	FSceneViewState* ViewState = (FSceneViewState*)View.State;
	SCOPED_GPU_MASK(RHICmdList, View.GPUMask);

	FGraphicsPipelineStateInitializer StackGraphicsPSOInit;
	if (InGraphicsPSOInit == nullptr)
	{
		RHICmdList.ApplyCachedRenderTargets(StackGraphicsPSOInit);
		StackGraphicsPSOInit.PrimitiveType = PT_TriangleList;
		StackGraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
		// Depth tests, no depth writes, no color writes, opaque
		StackGraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
		StackGraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector3();
	}
	FGraphicsPipelineStateInitializer& GraphicsPSOInit = (InGraphicsPSOInit != nullptr) ? *InGraphicsPSOInit : StackGraphicsPSOInit;


	// We only need to render the front-faces of the culling geometry (this halves the amount of pixels we touch)
	GraphicsPSOInit.RasterizerState = View.bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI();

	// Set stereo viewports while accounting for the downsample factor
	if (View.IsInstancedStereo())
	{
		const FIntRect ViewRect = GetDownscaledRect(View.ViewRect, DownsampleFactor);
		const FIntRect SecondaryViewRect = GetDownscaledRect(View.GetInstancedView()->ViewRect, DownsampleFactor);
		RHICmdList.SetStereoViewport(
			ViewRect.Min.X,
			SecondaryViewRect.Min.X,
			ViewRect.Min.Y,
			SecondaryViewRect.Min.Y,
			0.0f, ViewRect.Max.X,
			SecondaryViewRect.Max.X,
			ViewRect.Max.Y,
			SecondaryViewRect.Max.Y,
			1.0f);
	}
	else
	{
		const FIntRect ViewRect = GetDownscaledRect(View.ViewRect, DownsampleFactor);
		RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);
	}

	// Lookup the vertex shader.
	TShaderMapRef<FOcclusionQueryVS> VertexShader(View.ShaderMap);
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

	if (View.Family->EngineShowFlags.OcclusionMeshes)
	{
		TShaderMapRef<FOcclusionQueryPS> PixelShader(View.ShaderMap);
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA>::GetRHI();
	}

	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	if (FeatureLevel > ERHIFeatureLevel::ES3_1)
	{
		SCOPED_DRAW_EVENT(RHICmdList, ShadowFrustumQueries);
		for (int i = 0; i < ViewQuery.LocalLightQueries.Num(); i++)
		{
			ExecutePointLightShadowOcclusionQuery(RHICmdList, View, *ViewQuery.LocalLightQueryInfos[i], VertexShader, ViewQuery.LocalLightQueries[i]);
		}
	}

	uint32 NumVertices = ViewQuery.CSMQueries.Num() * 6 // Plane 
		+ ViewQuery.ShadowQueries.Num() * 8 // Cube
		+ ViewQuery.ReflectionQueries.Num() * 8; // Cube

	if (NumVertices > 0)
	{
		uint32 BaseVertexOffset = 0;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector3f>(TEXT("ViewOcclusionTests"), NumVertices)
			.AddUsage(EBufferUsageFlags::Volatile)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FVector3f> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

		{
			for (FProjectedShadowInfo const* Query : ViewQuery.CSMQueryInfos)
			{
				PrepareDirectionalLightShadowOcclusionQuery(BaseVertexOffset, Initializer, View, *Query);
				checkSlow(BaseVertexOffset <= NumVertices);
			}

			for (FProjectedShadowInfo const* Query : ViewQuery.ShadowQueryInfos)
			{
				PrepareProjectedShadowOcclusionQuery(BaseVertexOffset, Initializer, View, *Query);
				checkSlow(BaseVertexOffset <= NumVertices);
			}

			for (FPlanarReflectionSceneProxy const* Query : ViewQuery.ReflectionQueryInfos)
			{
				PreparePlanarReflectionOcclusionQuery(BaseVertexOffset, Initializer, View, Query);
				checkSlow(BaseVertexOffset <= NumVertices);
			}
		}

		FBufferRHIRef VertexBufferRHI = Initializer.Finalize();

		{
			SCOPED_DRAW_EVENT(RHICmdList, ShadowFrustumQueries);
			SetShaderParametersLegacyVS(RHICmdList, VertexShader, View);
			RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
			BaseVertexOffset = 0;

			for (FRHIRenderQuery* const& Query : ViewQuery.CSMQueries)
			{
				ExecuteDirectionalLightShadowOcclusionQuery(RHICmdList, BaseVertexOffset, Query);
				checkSlow(BaseVertexOffset <= NumVertices);
			}

			for (FRHIRenderQuery* const& Query : ViewQuery.ShadowQueries)
			{
				ExecuteProjectedShadowOcclusionQuery(RHICmdList, BaseVertexOffset, Query);
				checkSlow(BaseVertexOffset <= NumVertices);
			}
		}

		if (FeatureLevel > ERHIFeatureLevel::ES3_1)
		{
			SCOPED_DRAW_EVENT(RHICmdList, PlanarReflectionQueries);
			for (FRHIRenderQuery* const& Query : ViewQuery.ReflectionQueries)
			{
				ExecutePlanarReflectionOcclusionQuery(RHICmdList, BaseVertexOffset, Query);
				check(BaseVertexOffset <= NumVertices);
			}
		}

		VertexBufferRHI.SafeRelease();
	}

	if (ViewQuery.bFlushQueries)
	{
		SetShaderParametersLegacyVS(RHICmdList, VertexShader, View);

		if (GOcclusionQueryDispatchOrder == 0)
		{
			{
				SCOPED_DRAW_EVENT(RHICmdList, GroupedQueries);
				View.GroupedOcclusionQueries.Flush(RHICmdList);
			}
			{
				SCOPED_DRAW_EVENT(RHICmdList, IndividualQueries);
				View.IndividualOcclusionQueries.Flush(RHICmdList);
			}
		}
		else
		{
			{
				SCOPED_DRAW_EVENT(RHICmdList, IndividualQueries);
				View.IndividualOcclusionQueries.Flush(RHICmdList);
			}
			{
				SCOPED_DRAW_EVENT(RHICmdList, GroupedQueries);
				View.GroupedOcclusionQueries.Flush(RHICmdList);
			}
		}

		if (FeatureLevel == ERHIFeatureLevel::ES3_1 &&
			View.ViewState && View.ViewState->OcclusionFeedback.IsInitialized())
		{
			SCOPED_DRAW_EVENT(RHICmdList, IndividualQueries);
			View.ViewState->OcclusionFeedback.SubmitOcclusionDraws(RHICmdList, View);
		}
	}
}

static void BeginOcclusionTests(
	FRHICommandList& RHICmdList,
	TArrayView<FViewInfo> Views,
	ERHIFeatureLevel::Type FeatureLevel,
	const FViewOcclusionQueriesPerView& QueriesPerView,
	uint32 DownsampleFactor)
{
	check(RHICmdList.IsInsideRenderPass());
	check(QueriesPerView.Num() == Views.Num());

	SCOPE_CYCLE_COUNTER(STAT_BeginOcclusionTestsTime);
	SCOPED_DRAW_EVENT(RHICmdList, BeginOcclusionTests);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;
	GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
	// Depth tests, no depth writes, no color writes, opaque
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector3();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		// Don't draw occlusion queries for both eyes in multiview
		FViewInfo& View = Views[ViewIndex];
		if (View.bIsSinglePassStereo && IStereoRendering::IsASecondaryPass(View.StereoPass))
		{
			continue;
		}

		SCOPED_DRAW_EVENTF(RHICmdList, ViewOcclusionTests, TEXT("ViewOcclusionTests %d"), ViewIndex);
		const FViewOcclusionQueries& ViewQuery = QueriesPerView[ViewIndex];

		BeginOcclusionTests(RHICmdList, View, ViewQuery, FeatureLevel, DownsampleFactor, &GraphicsPSOInit);
	}
}

void FDeferredShadingSceneRenderer::RenderOcclusion(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures& SceneTextures,
	bool bIsOcclusionTesting,
	const FBuildHZBAsyncComputeParams* BuildHZBAsyncComputeParams,
	Froxel::FRenderer& FroxelRenderer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDeferredShadingSceneRenderer::RenderOcclusion);

	if (bIsOcclusionTesting)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, HZB, "HZB");

		uint32 DownsampleFactor = 1;
		FRDGTextureRef SceneDepthTexture = SceneTextures.Depth.Target;
		FRDGTextureRef OcclusionDepthTexture = SceneDepthTexture;

		// Update the quarter-sized depth buffer with the current contents of the scene depth texture.
		// This needs to happen before occlusion tests, which makes use of the small depth buffer.
		if (UseDownsampledOcclusionQueries())
		{
			DownsampleFactor = SceneTextures.Config.SmallDepthDownsampleFactor;
			OcclusionDepthTexture = SceneTextures.SmallDepth;

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				const FViewInfo& View = Views[ViewIndex];
				RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

				const FScreenPassTexture SceneDepth(SceneDepthTexture, View.ViewRect);
				const FScreenPassRenderTarget SmallDepth(SceneTextures.SmallDepth, GetDownscaledRect(View.ViewRect, DownsampleFactor), GetLoadActionIfProduced(OcclusionDepthTexture, ERenderTargetLoadAction::ENoAction));
				AddDownsampleDepthPass(GraphBuilder, View, SceneDepth, SmallDepth, EDownsampleDepthFilter::Max);
			}
		}

		auto& QueriesPerView = *GraphBuilder.AllocObject<FViewOcclusionQueriesPerView>();
		auto* PassParameters = GraphBuilder.AllocParameters<FRenderTargetParameters>();

		GraphBuilder.AddSetupTask([this, PassParameters, &QueriesPerView]
		{
			// Issue occlusion queries. This is done after the downsampled depth buffer is created so that it can be used for issuing queries.
			AllocateOcclusionTests(Scene, QueriesPerView, Views, VisibleLightInfos);

			int32 NumQueriesForBatch = 0;

			{
				for (int32 ViewIndex = 0; ViewIndex < QueriesPerView.Num(); ViewIndex++)
				{
					const FViewOcclusionQueries& ViewQuery = QueriesPerView[ViewIndex];
					NumQueriesForBatch += ViewQuery.LocalLightQueries.Num();
					NumQueriesForBatch += ViewQuery.CSMQueries.Num();
					NumQueriesForBatch += ViewQuery.ShadowQueries.Num();
					NumQueriesForBatch += ViewQuery.ReflectionQueries.Num();

					FViewInfo& View = Views[ViewIndex];
					FSceneViewState* ViewState = (FSceneViewState*)View.State;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					if (!ViewState->bIsFrozen)
#endif
					{
						NumQueriesForBatch += View.IndividualOcclusionQueries.GetNumBatchOcclusionQueries();
						NumQueriesForBatch += View.GroupedOcclusionQueries.GetNumBatchOcclusionQueries();
					}
				}
			}

			PassParameters->RenderTargets.NumOcclusionQueries = NumQueriesForBatch;
		});

		PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(OcclusionDepthTexture, ERenderTargetLoadAction::ELoad, ERenderTargetLoadAction::ELoad, FExclusiveDepthStencil::DepthRead_StencilWrite);

		RDG_EVENT_SCOPE_STAT(GraphBuilder, BeginOcclusionTests, "BeginOcclusionTests");

		GraphBuilder.AddPass(
			RDG_EVENT_NAME("BeginOcclusionTests"),
			PassParameters,
			ERDGPassFlags::Raster | ERDGPassFlags::NeverCull,
			[this, &QueriesPerView, DownsampleFactor](FRDGAsyncTask, FRHICommandList& RHICmdList)
			{
				if (!QueriesPerView.IsEmpty())
				{
					BeginOcclusionTests(RHICmdList, Views, FeatureLevel, QueriesPerView, DownsampleFactor);
					QueriesPerView.Empty();
				}
			});
	}
	else
	{
		// Make sure the views are freeing their memory if we toggled occlusion off at any point
		TrimAllOcclusionHistory(Views);
	}

	const bool bUseHzbOcclusion = RenderHzb(GraphBuilder, SceneTextures.Depth.Resolve, BuildHZBAsyncComputeParams, FroxelRenderer);

	if (bIsOcclusionTesting)
	{
		FenceOcclusionTests(GraphBuilder);
	}

	if (bUseHzbOcclusion || bIsOcclusionTesting)
	{
		GraphBuilder.AddDispatchHint();
	}
}

void FMobileSceneRenderer::RenderOcclusion(FRHICommandList& RHICmdList, FViewInfo& View, const FViewOcclusionQueries& ViewOcclusionQueries)
{
	SCOPED_NAMED_EVENT(FMobileSceneRenderer_BeginOcclusionTests, FColor::Emerald);
	BeginOcclusionTests(RHICmdList, View, ViewOcclusionQueries, View.GetFeatureLevel(), 1.0f);
}

DECLARE_CYCLE_STAT(TEXT("OcclusionSubmittedFence Dispatch"), STAT_OcclusionSubmittedFence_Dispatch, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("OcclusionSubmittedFence Wait"), STAT_OcclusionSubmittedFence_Wait, STATGROUP_SceneRendering);

static uint32 GetViewStateUniqueID(const FSceneRenderer* SceneRenderer)
{
	return SceneRenderer->Views.Num() && SceneRenderer->Views[0].ViewState ? SceneRenderer->Views[0].ViewState->UniqueID : 0;
}

void FSceneRenderer::FenceOcclusionTests(FRDGBuilder& GraphBuilder)
{
	if (DoOcclusionQueries() && IsRunningRHIInSeparateThread())
	{
		AddPass(GraphBuilder, RDG_EVENT_NAME("FenceOcclusionTests"), [this](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			SCOPE_CYCLE_COUNTER(STAT_OcclusionSubmittedFence_Dispatch);

			// We implement a queue of buffered fences, so we can avoid waiting on queries for another view family that's rendering
			// in the same frame.  Here we push a new fence into the queue of buffered fences. We assume that the queue isn't full,
			// since WaitOcclusionTests (called earlier in the frame) will always pop at least one fence if the queue is full.
			check(OcclusionSubmittedFence[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames - 1].Fence == nullptr);

			for (int32 Dest = FOcclusionQueryHelpers::MaxBufferedOcclusionFrames - 1; Dest >= 1; Dest--)
			{
				CA_SUPPRESS(6385);
				OcclusionSubmittedFence[Dest] = OcclusionSubmittedFence[Dest - 1];
			}

			OcclusionSubmittedFence[0].Fence = RHICmdList.RHIThreadFence();
			OcclusionSubmittedFence[0].ViewStateUniqueID = GetViewStateUniqueID(this);
		});

		GraphBuilder.AddDispatchHint();
	}
}

void FSceneRenderer::WaitOcclusionTests(FRHICommandListImmediate& RHICmdList)
{
	if (IsRunningRHIInSeparateThread())
	{
		SCOPE_CYCLE_COUNTER(STAT_OcclusionSubmittedFence_Wait);

		// Wait for async execution tasks from the previous render frame, namely the FenceOcclusionTasks pass.
		FRDGBuilder::WaitForAsyncExecuteTask();

		// Count the number of fences in the queue for the view in question
		const uint32 ViewStateUniqueID = GetViewStateUniqueID(this);

		int32 ViewStateFenceCount = 0;
		for (int32 FenceIndex = 0; FenceIndex < FOcclusionQueryHelpers::MaxBufferedOcclusionFrames && OcclusionSubmittedFence[FenceIndex].Fence; FenceIndex++)
		{
			if (OcclusionSubmittedFence[FenceIndex].Fence && (OcclusionSubmittedFence[FenceIndex].ViewStateUniqueID == ViewStateUniqueID))
			{
				ViewStateFenceCount++;
			}
		}

		// Figure out how many fences are allowed to remain in the queue after the wait, which is one less than the number of buffered frames
		const int32 FencesAllowedInQueue = FOcclusionQueryHelpers::GetNumBufferedFrames(FeatureLevel) - 1;
		check(FencesAllowedInQueue >= 0);

		// Scan the fences in the queue in order (oldest will be at the end), and wait on them as appropriate.  If the queue is full,
		// we always want to wait on at least one fence, so we have room to push a new fence onto the queue if necessary.  Thus the
		// loop always runs at least once before breaking out.
		int32 FenceToWaitOn = FOcclusionQueryHelpers::MaxBufferedOcclusionFrames - 1;
		do
		{
			// If our fence counting logic is correct, we should already have broken out of the loop
			check(FenceToWaitOn >= 0);

			// Is there a fence in this queue entry?
			if (OcclusionSubmittedFence[FenceToWaitOn].Fence)
			{
				// If this is one of the fences for the current scene's view state, mark that we're waiting on it
				if (OcclusionSubmittedFence[FenceToWaitOn].ViewStateUniqueID == ViewStateUniqueID)
				{
					ViewStateFenceCount--;
				}

				// Wait on the current fence and clear it out
				FRHICommandListExecutor::WaitOnRHIThreadFence(OcclusionSubmittedFence[FenceToWaitOn].Fence);
				OcclusionSubmittedFence[FenceToWaitOn].Fence = nullptr;
				OcclusionSubmittedFence[FenceToWaitOn].ViewStateUniqueID = 0;
			}

			FenceToWaitOn--;

		} while (ViewStateFenceCount > FencesAllowedInQueue);
	}
}

FOcclusionFeedback::FOcclusionFeedback()
	: FRenderResource()
	, CurrentBufferIndex(0)
	, ResultsOcclusionFrameCounter(0)
{
	OcclusionVertexDeclarationRHI = nullptr;
}

FOcclusionFeedback::~FOcclusionFeedback()
{
}

void FOcclusionFeedback::InitRHI(FRHICommandListBase& RHICmdList)
{
	for (int32 i = 0; i < UE_ARRAY_COUNT(OcclusionBuffers); ++i)
	{
		OcclusionBuffers[i].ReadbackBuffer = new FRHIGPUBufferReadback(TEXT("FeedbackOcclusionBuffer.Readback"));
	}

	{
		FVertexDeclarationElementList Elements;
		Elements.Add(FVertexElement(0, 0,					VET_Float4, 0, sizeof(FVector4f)*1, false));
		Elements.Add(FVertexElement(1, 0,					VET_Float4, 1, sizeof(FVector4f)*2, true));
		Elements.Add(FVertexElement(1, sizeof(FVector4f),	VET_Float4, 2, sizeof(FVector4f)*2, true));
		OcclusionVertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
	}
}
void FOcclusionFeedback::ReleaseRHI()
{
	BatchOcclusionQueries.Empty();

	for (int32 i = 0; i < UE_ARRAY_COUNT(OcclusionBuffers); ++i)
	{
		delete OcclusionBuffers[i].ReadbackBuffer;
		OcclusionBuffers[i].ReadbackBuffer = nullptr;
		OcclusionBuffers[i].BatchedPrimitives.Empty();
	}
	CurrentBufferIndex = 0;
	OcclusionVertexDeclarationRHI.SafeRelease();
	GPUFeedbackBuffer = nullptr;
	LatestOcclusionResults.Empty();
}

void FOcclusionFeedback::BeginOcclusionScope(FRDGBuilder& GraphBuilder)
{
	check(GPUFeedbackBuffer == nullptr);

	FOcclusionBuffer& OcclusionBuffer = OcclusionBuffers[CurrentBufferIndex];
	uint32 NumPrimitives = FMath::Max(1, OcclusionBuffer.BatchedPrimitives.Num());

	FRDGBufferDesc Desc = FRDGBufferDesc::CreateStructuredDesc(sizeof(uint32), NumPrimitives);
	Desc.Usage |= EBufferUsageFlags::SourceCopy;
	GPUFeedbackBuffer = GraphBuilder.CreateBuffer(Desc, TEXT("FeedbackOcclusionBuffer"));
	AddClearUAVPass(GraphBuilder, GraphBuilder.CreateUAV(GPUFeedbackBuffer), 0u);
}

void FOcclusionFeedback::EndOcclusionScope(FRDGBuilder& GraphBuilder)
{
	FOcclusionBuffer& OcclusionBuffer = OcclusionBuffers[CurrentBufferIndex];

	FRHIGPUBufferReadback* ReadbackBuffer = OcclusionBuffer.ReadbackBuffer;
	check(ReadbackBuffer);
	uint32 ReadbackBytes = OcclusionBuffer.BatchedPrimitives.Num() * sizeof(uint32);
	AddEnqueueCopyPass(GraphBuilder, ReadbackBuffer, GPUFeedbackBuffer, ReadbackBytes);
	GPUFeedbackBuffer = nullptr;
}

void FOcclusionFeedback::AdvanceFrame(uint32 InOcclusionFrameCounter)
{
	constexpr uint32 NumBuffers = UE_ARRAY_COUNT(OcclusionBuffers);
	CurrentBufferIndex = (CurrentBufferIndex + 1u) % NumBuffers;
	
	FOcclusionBuffer& OcclusionBuffer = OcclusionBuffers[CurrentBufferIndex];
	OcclusionBuffer.BatchedPrimitives.Empty();
	OcclusionBuffer.OcclusionFrameCounter = InOcclusionFrameCounter;
}

void FOcclusionFeedback::ReadbackResults(FRHICommandListImmediate& RHICmdList)
{
	constexpr uint32 NumBuffers = UE_ARRAY_COUNT(OcclusionBuffers);
	TArray<uint32, TInlineAllocator<NumBuffers>> ReadyBuffers;

	// Find the most recent Ready buffer
	for (uint32 Index = 1u; Index <= NumBuffers; ++Index)
	{
		uint32 ReadbackBufferIndex = (CurrentBufferIndex + Index) % NumBuffers;
		FOcclusionBuffer& OcclusionBuffer = OcclusionBuffers[ReadbackBufferIndex];

		if (OcclusionBuffer.BatchedPrimitives.Num() == 0)
		{
			// No data to readback
			break;
		}
		
		if (OcclusionBuffer.OcclusionFrameCounter <= ResultsOcclusionFrameCounter)
		{
			// Already read
			continue;
		}

		if (!OcclusionBuffer.ReadbackBuffer->IsReady())
		{
			break;
		}

		ReadyBuffers.Add(ReadbackBufferIndex);
	}

	bool bReady = (ReadyBuffers.Num() != 0);
	if (!bReady)
	{
		uint32 TailBufferIndex = (CurrentBufferIndex + 1u) % NumBuffers;
		FOcclusionBuffer& TailOcclusionBuffer = OcclusionBuffers[TailBufferIndex];
		
		// Check if we have to wait for result
		if (TailOcclusionBuffer.BatchedPrimitives.Num() > 0 
			&& ResultsOcclusionFrameCounter < TailOcclusionBuffer.OcclusionFrameCounter)
		{
			// Wait for the results
			FRenderThreadIdleScope IdleScope(ERenderThreadIdleTypes::WaitingForGPUQuery);
			TailOcclusionBuffer.ReadbackBuffer->Wait(RHICmdList, FRHIGPUMask::All());
			bReady = true;

			ReadyBuffers.Add(TailBufferIndex);
		}
	}

	if (bReady)
	{
		uint32 ReadbackBufferIndex = ReadyBuffers.Last();
		FOcclusionBuffer& OcclusionBuffer = OcclusionBuffers[ReadbackBufferIndex];

		ResultsOcclusionFrameCounter = OcclusionBuffer.OcclusionFrameCounter;
		LatestOcclusionResults.Reset();

		uint32 NumPrimitives = OcclusionBuffer.BatchedPrimitives.Num();
		uint32 NumBytes = NumPrimitives * sizeof(uint32);
		uint32* Results = (uint32*)OcclusionBuffer.ReadbackBuffer->Lock(NumBytes);

		for (uint32 i = 0; i < NumPrimitives; ++i)
		{
			if (Results[i] == 0u)
			{
				LatestOcclusionResults.Add(OcclusionBuffer.BatchedPrimitives[i]);
			}
		}

		OcclusionBuffer.ReadbackBuffer->Unlock();
	}
}

void FOcclusionFeedback::AddPrimitive(const FPrimitiveOcclusionHistoryKey& PrimitiveKey, const FVector& BoundsOrigin, const FVector& BoundsBoxExtent, FGlobalDynamicVertexBuffer& DynamicVertexBuffer)
{
	constexpr uint32 MaxBatchedPrimitives = 512;
	constexpr uint32 PrimitiveStride = sizeof(FVector4f) * 2u;

	if (BatchOcclusionQueries.Num() == 0 ||
		BatchOcclusionQueries.Last().NumBatchedPrimitives >= MaxBatchedPrimitives)
	{
		FOcclusionBatch OcclusionBatch;
		OcclusionBatch.NumBatchedPrimitives = 0u;
		OcclusionBatch.VertexAllocation = DynamicVertexBuffer.Allocate(MaxBatchedPrimitives * PrimitiveStride);
		check(OcclusionBatch.VertexAllocation.IsValid());
		BatchOcclusionQueries.Add(OcclusionBatch);
	}

	FOcclusionBatch& OcclusionBatch = BatchOcclusionQueries.Last();

	// TODO: encode Tile into .w, to support LWC?
	FVector3f BoundsOrigin3f = FVector3f(BoundsOrigin);
	FVector3f BoundsBoxExtent3f = FVector3f(BoundsBoxExtent);

	FVector4f* RESTRICT PrimitiveData = (FVector4f*)OcclusionBatch.VertexAllocation.Buffer;
	PrimitiveData[0] = FVector4f(BoundsOrigin3f, 0.f);
	PrimitiveData[1] = FVector4f(BoundsBoxExtent3f, 0.f);

	OcclusionBatch.VertexAllocation.Buffer += PrimitiveStride;
	OcclusionBatch.NumBatchedPrimitives++;

	FOcclusionBuffer& OcclusionBuffer = OcclusionBuffers[CurrentBufferIndex];
	OcclusionBuffer.BatchedPrimitives.Add(PrimitiveKey);
}

class FOcclusionFeedbackVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOcclusionFeedbackVS);
	SHADER_USE_PARAMETER_STRUCT(FOcclusionFeedbackVS, FGlobalShader);

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FViewShaderParameters, View)
		SHADER_PARAMETER(uint32, StartIndex)
		SHADER_PARAMETER(uint32, InstancedViewIndex)
	END_SHADER_PARAMETER_STRUCT()
};

class FOcclusionFeedbackPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FOcclusionFeedbackPS);
	SHADER_USE_PARAMETER_STRUCT(FOcclusionFeedbackPS, FGlobalShader);

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsMobilePlatform(Parameters.Platform);
	}

	class FPixelDiscard : SHADER_PERMUTATION_BOOL("PERMUTATION_PIXEL_DISCARD");
	using FPermutationDomain = TShaderPermutationDomain<FPixelDiscard>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FMobileBasePassUniformParameters, MobileBasePass)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FOcclusionFeedbackVS,"/Engine/Private/OcclusionFeedbackShaders.usf","MainVS",SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FOcclusionFeedbackPS,"/Engine/Private/OcclusionFeedbackShaders.usf","MainPS",SF_Pixel);

int32 GOcclusionFeeback_Blending = 1;
static FAutoConsoleVariableRef CVarOcclusionFeeback_Blending(
	TEXT("r.OcclusionFeedback.Blending"),
	GOcclusionFeeback_Blending,
	TEXT("0: Opaque \n")
	TEXT("1: Enable blending  (default) \n")
	TEXT("2: Enable pixel discard"),
	ECVF_RenderThreadSafe
);

void FOcclusionFeedback::SubmitOcclusionDraws(FRHICommandList& RHICmdList, FViewInfo& View)
{
	if (BatchOcclusionQueries.Num() == 0)
	{
		return;
	}

	FRHIBuffer* CubeIndexBuffer = GetUnitCubeIndexBuffer();
	FRHIBuffer* CubeVertexBuffer = GetUnitCubeVertexBuffer();

	const FIntRect ViewRect = View.ViewRect;
	RHICmdList.SetViewport(ViewRect.Min.X, ViewRect.Min.Y, 0.0f, ViewRect.Max.X, ViewRect.Max.Y, 1.0f);

	TShaderMapRef<FOcclusionFeedbackVS> VertexShader(View.ShaderMap);

	FOcclusionFeedbackPS::FPermutationDomain PsPermutationVector;
	PsPermutationVector.Set<FOcclusionFeedbackPS::FPixelDiscard>(GOcclusionFeeback_Blending == 2);
	TShaderMapRef<FOcclusionFeedbackPS> PixelShader(View.ShaderMap, PsPermutationVector);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	GraphicsPSOInit.PrimitiveType = PT_TriangleList;

	if (GOcclusionFeeback_Blending == 1)
	{
		GraphicsPSOInit.BlendState = TStaticBlendState<
			CW_ALPHA, BO_Add, BF_One, BF_One, BO_Add, BF_Zero, BF_One,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero,
			CW_NONE, BO_Add, BF_One, BF_Zero, BO_Add, BF_One, BF_Zero>::GetRHI();
	}
	else
	{
		GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
	}

	// Depth tests, no depth writes, no color writes, opaque
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_DepthNearOrEqual>::GetRHI();
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = OcclusionVertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	// We only need to render the front-faces of the culling geometry (this halves the amount of pixels we touch)
	GraphicsPSOInit.RasterizerState = View.bReverseCulling ? TStaticRasterizerState<FM_Solid, CM_CW>::GetRHI() : TStaticRasterizerState<FM_Solid, CM_CCW>::GetRHI();
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	RHICmdList.SetStreamSource(0, CubeVertexBuffer, 0);

	uint32 PrimitiveStartIndex = 0u;
	// Draw the batches.
	for (int32 BatchIndex = 0, NumBatches = BatchOcclusionQueries.Num(); BatchIndex < NumBatches; BatchIndex++)
	{
		FOcclusionBatch& Batch = BatchOcclusionQueries[BatchIndex];
		
		FOcclusionFeedbackVS::FParameters VSParams;
		VSParams.View = View.GetShaderParameters();
		VSParams.StartIndex = PrimitiveStartIndex;
				
		FRHIBuffer* PrimitiveBufferRHI = Batch.VertexAllocation.VertexBuffer->VertexBufferRHI;
		uint32 VertexBufferOffset = Batch.VertexAllocation.VertexOffset;
		RHICmdList.SetStreamSource(1, PrimitiveBufferRHI, VertexBufferOffset);

		// Because the RHI does not yet support instance divisors, Occlusion Feedback supports instanced stereo 
		// by issuing a draw call for each eye. But both draw calls mark the same primitives, so it does not need to duplicate that state. 
		const uint32 InstanceCount = View.bIsInstancedStereoEnabled ? 2 : 1;
		for (uint32 InstancedViewIndex = 0; InstancedViewIndex < InstanceCount; ++InstancedViewIndex)
		{
			VSParams.InstancedViewIndex = InstancedViewIndex;
			SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParams);

			RHICmdList.DrawIndexedPrimitive(
				CubeIndexBuffer,
				/*BaseVertexIndex=*/ 0,
				/*MinIndex=*/ 0,
				/*NumVertices=*/ 8,
				/*StartIndex=*/ 0,
				/*NumPrimitives=*/ 12,
				/*NumInstances=*/ Batch.NumBatchedPrimitives
			);
		}

		PrimitiveStartIndex += Batch.NumBatchedPrimitives;
	}
	INC_DWORD_STAT_BY(STAT_OcclusionQueries,BatchOcclusionQueries.Num());

	// Reset the batch state.
	BatchOcclusionQueries.Empty(BatchOcclusionQueries.Num());
}
