// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShadowRendering.h"
#include "Engine/Engine.h"

class FProjectedShadowInfo;
class FPlanarReflectionSceneProxy;
class FRHIRenderQuery;

DECLARE_GPU_STAT_NAMED_EXTERN(HZB, TEXT("HZB"));

/*=============================================================================
	SceneOcclusion.h
=============================================================================*/

struct FViewOcclusionQueries
{
	using FProjectedShadowArray = TArray<FProjectedShadowInfo const*, SceneRenderingAllocator>;
	using FPlanarReflectionArray = TArray<FPlanarReflectionSceneProxy const*, SceneRenderingAllocator>;
	using FRenderQueryArray = TArray<FRHIRenderQuery*, SceneRenderingAllocator>;

	FProjectedShadowArray LocalLightQueryInfos;
	FProjectedShadowArray CSMQueryInfos;
	FProjectedShadowArray ShadowQueryInfos;
	FPlanarReflectionArray ReflectionQueryInfos;

	FRenderQueryArray LocalLightQueries;
	FRenderQueryArray CSMQueries;
	FRenderQueryArray ShadowQueries;
	FRenderQueryArray ReflectionQueries;

	bool bFlushQueries = true;
};

using FViewOcclusionQueriesPerView = TArray<FViewOcclusionQueries, TInlineAllocator<1, SceneRenderingAllocator>>;

extern void AllocateOcclusionTests(const FScene* Scene, FViewOcclusionQueriesPerView& QueriesPerView, TArrayView<class FViewInfo> Views, const FVisibleLightInfoArray& VisibleLightInfos);

/**
* A vertex shader for rendering a texture on a simple element.
*/
class FOcclusionQueryVS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FOcclusionQueryVS,Global);
public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		static auto* MobileUseHWsRGBEncodingCVAR = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.UseHWsRGBEncoding"));
		const bool bMobileUseHWsRGBEncoding = (MobileUseHWsRGBEncodingCVAR && MobileUseHWsRGBEncodingCVAR->GetValueOnAnyThread() == 1);

		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("OUTPUT_GAMMA_SPACE"), IsMobileHDR() == false && !bMobileUseHWsRGBEncoding);
		OutEnvironment.SetDefine(TEXT("OUTPUT_MOBILE_HDR"), IsMobileHDR() == true);
	}

	FOcclusionQueryVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer):
		FGlobalShader(Initializer)
	{
		StencilingGeometryParameters.Bind(Initializer.ParameterMap, TEXT("StencilingGeometryPosAndScale"));
	}

	FOcclusionQueryVS() {}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FViewInfo& View, const FSphere& BoundingSphere)
	{
		FVector4f StencilingSpherePosAndScale[2];
		StencilingGeometry::GStencilSphereVertexBuffer.CalcTransform(StencilingSpherePosAndScale[0], BoundingSphere, View.ViewMatrices.GetPreViewTranslation());
		if (View.bShouldBindInstancedViewUB)
		{
			if (const FViewInfo* InstancedView = View.GetInstancedView())
			{
				StencilingGeometry::GStencilSphereVertexBuffer.CalcTransform(StencilingSpherePosAndScale[1], BoundingSphere, InstancedView->ViewMatrices.GetPreViewTranslation());
			}
			else
			{
				StencilingSpherePosAndScale[1] = StencilingSpherePosAndScale[0];
			}
		}

		SetParametersInternal(BatchedParameters, View, StencilingSpherePosAndScale);
	}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FViewInfo& View)
	{
		// Don't transform if rendering frustum
		FVector4f InstancedOffsetScale[2] = { FVector4f(0, 0, 0, 1), FVector4f(0, 0, 0, 1) };
		if (View.bShouldBindInstancedViewUB)
		{
			// Occlusion queries are generated to be offset by the PreViewTranslation.
			// However for multiview, we still need to apply that offset to the right eye.
			// In that case we also need to undo the baked left eye translation.
			if (const FViewInfo* InstancedView = View.GetInstancedView())
			{
				InstancedOffsetScale[1] = FVector4f(FVector3f(InstancedView->ViewMatrices.GetPreViewTranslation() - View.ViewMatrices.GetPreViewTranslation()), 1.0f);
			}
		}
		SetParametersInternal(BatchedParameters, View, InstancedOffsetScale);
	}

private:
	void SetParametersInternal(FRHIBatchedShaderParameters& BatchedParameters, const FViewInfo& View, const FVector4f StencilingSpherePosAndScale[2])
	{
		// Mobile renderer sets View UB through static binding
		if (!IsMobilePlatform(View.GetShaderPlatform()))
		{
			FGlobalShader::SetParameters<FViewUniformShaderParameters>(BatchedParameters, View.ViewUniformBuffer);
		}
				
		if (!View.bShouldBindInstancedViewUB)
		{
			SetShaderValue(BatchedParameters, StencilingGeometryParameters, StencilingSpherePosAndScale[0]);
		}
		else
		{
			FGlobalShader::SetParameters<FInstancedViewUniformShaderParameters>(BatchedParameters, View.GetInstancedViewUniformBuffer());
			SetShaderValueArray(BatchedParameters, StencilingGeometryParameters, StencilingSpherePosAndScale, 2);
		}
	}

private:
	LAYOUT_FIELD(FShaderParameter, StencilingGeometryParameters)
};

/**
 * A pixel shader for rendering a texture on a simple element.
 */
class FOcclusionQueryPS : public FGlobalShader
{
	DECLARE_SHADER_TYPE(FOcclusionQueryPS, Global);
public:
	FOcclusionQueryPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) :
		FGlobalShader(Initializer) {}
	FOcclusionQueryPS() {}
};

// Returns whether occlusion queries should be downsampled.
extern RENDERER_API bool UseDownsampledOcclusionQueries();
