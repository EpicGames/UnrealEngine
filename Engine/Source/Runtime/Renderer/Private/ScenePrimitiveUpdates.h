// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SceneUpdateCommandQueue.h"
#include "PrimitiveSceneInfo.h"
#include "GPUSceneWriter.h"

struct FViewSceneChangeSet;

/**
 * Definitions of primitive scene update commands.
 */

enum class EPrimitiveUpdateDirtyFlags : uint32 
{
	None			=	0u,
	/** The Transform is modified by this command. */
	Transform		=	1u << 0,
	/** The (any) instance data is modified by this command. */
	InstanceData	=	1u << 1,
	/** 
	 * The culling bounds are modified by this command. 
	 * This means the bounds (instance, primitive or both) as used in the culling and should not be updated for any other case. 
	 * Thus, needs to be set for transform updates of all kinds.
	 */
	CullingBounds	=	1u << 2,
	/** 
	 * Culling distances or similar, affecting culling logic, but NOT the bounds.
	 */
	CullingLogic	=	1u << 3,
	/** Any state that either makes its way into GPU-Scene or the per primitive UB */
	GPUState		=	1u << 4,
	/** All culling-affecting changes */
	AllCulling		= CullingBounds | CullingLogic,
	All				=	GPUState |  Transform | InstanceData | CullingBounds | CullingLogic,
};
ENUM_CLASS_FLAGS(EPrimitiveUpdateDirtyFlags);

enum class EPrimitiveUpdateId : uint32 
{
	UpdateTransform,
	UpdateInstance,
	UpdateAttachmentRoot,
	CustomPrimitiveData,
	OcclusionBoundsSlacks,  
	InstanceCullDistance,  
	DrawDistance,
	DistanceFieldScene,
	OverridePreviousTransform,
	UpdateInstanceFromCompute,
	MAX
};

using FScenePrimitiveUpdates = TSceneUpdateCommandQueue<FPrimitiveSceneInfo, EPrimitiveUpdateDirtyFlags, EPrimitiveUpdateId>;

using FPrimitiveUpdateCommand = FScenePrimitiveUpdates::FUpdateCommand;

template <EPrimitiveUpdateId InId, EPrimitiveUpdateDirtyFlags InDirtyFlags>
using TPrimitiveUpdatePayloadBase = FScenePrimitiveUpdates::TPayloadBase<InId, InDirtyFlags>;

struct FUpdateTransformCommand : public TPrimitiveUpdatePayloadBase<EPrimitiveUpdateId::UpdateTransform, 
	EPrimitiveUpdateDirtyFlags::GPUState | EPrimitiveUpdateDirtyFlags::Transform | EPrimitiveUpdateDirtyFlags::CullingBounds>
{
	FBoxSphereBounds WorldBounds;
	FBoxSphereBounds LocalBounds; 
	FMatrix LocalToWorld; 
	FVector AttachmentRootPosition;
};

struct FUpdateInstanceCommand : public TPrimitiveUpdatePayloadBase<EPrimitiveUpdateId::UpdateInstance, 
	EPrimitiveUpdateDirtyFlags::GPUState | EPrimitiveUpdateDirtyFlags::Transform | EPrimitiveUpdateDirtyFlags::CullingBounds | EPrimitiveUpdateDirtyFlags::InstanceData>
{
	FPrimitiveSceneProxy* PrimitiveSceneProxy{ nullptr };
	FBoxSphereBounds WorldBounds;
	FBoxSphereBounds LocalBounds;
};

struct FUpdateInstanceFromComputeCommand : public TPrimitiveUpdatePayloadBase<EPrimitiveUpdateId::UpdateInstanceFromCompute, EPrimitiveUpdateDirtyFlags::CullingBounds | EPrimitiveUpdateDirtyFlags::InstanceData>
{
	FPrimitiveSceneProxy* PrimitiveSceneProxy{ nullptr };
	FGPUSceneWriteDelegate GPUSceneWriter;
};

/**
 * Helper for the update payloads that contain a single payload value.
 */
template <typename InPayloadDataType, EPrimitiveUpdateId InId, EPrimitiveUpdateDirtyFlags InPrimitiveDirtyFlags>
struct TSingleValuePrimitiveUpdatePayload : public TPrimitiveUpdatePayloadBase<InId, InPrimitiveDirtyFlags>
{
	TSingleValuePrimitiveUpdatePayload(const InPayloadDataType &InValue) : Value(InValue) {}
	InPayloadDataType Value;
};

using FUpdateAttachmentRootData = TSingleValuePrimitiveUpdatePayload<FPrimitiveComponentId, EPrimitiveUpdateId::UpdateAttachmentRoot,
	EPrimitiveUpdateDirtyFlags::None>; // No GPU side effect (?).
using FUpdateCustomPrimitiveData = TSingleValuePrimitiveUpdatePayload<FCustomPrimitiveData, EPrimitiveUpdateId::CustomPrimitiveData,
	EPrimitiveUpdateDirtyFlags::GPUState>; // Needs upload
using FUpdateOcclusionBoundsSlacksData = TSingleValuePrimitiveUpdatePayload<float, EPrimitiveUpdateId::OcclusionBoundsSlacks,  
	EPrimitiveUpdateDirtyFlags::None>; // Only affects primitive occlusion.
using FUpdateInstanceCullDistanceData = TSingleValuePrimitiveUpdatePayload<FVector2f, EPrimitiveUpdateId::InstanceCullDistance,  
	EPrimitiveUpdateDirtyFlags::GPUState | EPrimitiveUpdateDirtyFlags::CullingLogic>; // Affects GPU culling?
using FUpdateDrawDistanceData = TSingleValuePrimitiveUpdatePayload<FVector3f, EPrimitiveUpdateId::DrawDistance,  
	EPrimitiveUpdateDirtyFlags::CullingLogic>; // Only affects CPU culling.
using FUpdateDistanceFieldSceneData = TPrimitiveUpdatePayloadBase<EPrimitiveUpdateId::DistanceFieldScene,  
	EPrimitiveUpdateDirtyFlags::None>; // Only affects DF scene rep - candidate for using abstract type.
using FUpdateOverridePreviousTransformData = TSingleValuePrimitiveUpdatePayload<FMatrix, EPrimitiveUpdateId::OverridePreviousTransform,  
	EPrimitiveUpdateDirtyFlags::GPUState>;// Overrides the previous transform, which needs to be propagated to the GPU, but otherwise does not change anything on its own.

/**
 * Change set that is valid before removes are processed and the scene data modified.
 * The referenced arrays have RDG life-time and can be safely used in RDG tasks.
 * However, the referenced data (primitive/proxy) and meaning of the persistent ID is not generally valid past the call in which this is passed. 
 * Thus, care need to be excercised.
 */
class FScenePreUpdateChangeSet
{
public:
	TConstArrayView<FPersistentPrimitiveIndex> RemovedPrimitiveIds;
	TConstArrayView<FPrimitiveSceneInfo*> RemovedPrimitiveSceneInfos;
	const FScenePrimitiveUpdates &PrimitiveUpdates;
	const FViewSceneChangeSet* ViewUpdateChangeSet = nullptr;
};

/**
 * Change set that is valid before after adds are processed and the scene data is modified.
 * The referenced arrays have RDG life-time and can be safely used in RDG tasks.
 */
class FScenePostUpdateChangeSet
{
public:
	TConstArrayView<FPersistentPrimitiveIndex> AddedPrimitiveIds;
	TConstArrayView<FPrimitiveSceneInfo*> AddedPrimitiveSceneInfos;
	const FScenePrimitiveUpdates &PrimitiveUpdates;
	const FViewSceneChangeSet* ViewUpdateChangeSet = nullptr;
};
