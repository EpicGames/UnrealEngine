// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScenePrivate.h: Private scene manager definitions.
=============================================================================*/

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Set.h"
#include "Containers/Map.h"
#include "Containers/BitArray.h"
#include "Containers/SparseArray.h"
#include "Containers/SetUtilities.h"
#include "Containers/UnrealString.h"
#include "Containers/Queue.h"
#include "Templates/SharedPointerFwd.h"
#include "Templates/RefCounting.h"
#include "Async/TaskGraphFwd.h"
#include "RenderResource.h"
#include "RHIFwd.h"
#include "RHIFeatureLevel.h"
#include "RHIUtilities.h"
#include "Math/MathFwd.h"
#include "Math/DoubleFloat.h"
#include "Math/SHMath.h"
#include "Math/Float16Color.h"
#include "Math/Color.h"
#include "ShaderParameterMacros.h"
#include "UObject/ObjectPtr.h"
#include "RenderTransform.h"
#include "UnifiedBuffer.h"
#include "RenderGraphFwd.h"
#include "RenderGraphResources.h"
#include "GrowOnlySpanAllocator.h"
#include "TextureLayout3d.h"
#include "SceneRenderingAllocator.h"
#include "SceneUpdateCommandQueue.h"
#include "SceneManagement.h"
#include "SceneRendering.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "PrecomputedVolumetricLightmap.h"
#include "PrimitiveSceneProxy.h"
#include "LightSceneInfo.h"

#include "RenderGraphUtils.h"
#include "UObject/NameTypes.h"
#include "Tasks/Task.h"
#include "SceneInterface.h"
#include "MeshPassProcessor.h"
#include "Nanite/NaniteShared.h"
#include "Nanite/NaniteVisibility.h"
#include "ScenePrivateBase.h"
#include "PrimitiveComponentId.h"
#include "RayTracingGeometryManagerTypes.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "DepthRendering.h"
#include "RHIResources.h"
#include "Templates/Atomic.h"
#if RHI_RAYTRACING
#include "RayTracing/RayTracingInstanceMask.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracing/RayTracingShaderBindingTable.h"
#endif
#include "OIT/OIT.h"
#include "GPUScene.h"
#include "RayTracingMeshDrawCommands.h"
#include "TextureLayout.h"
#include "PrimitiveSceneInfo.h"
#include "LightFunctionAtlas.h"
#include "SpanAllocator.h"
#include "SceneExtensions.h"
#include "VirtualTextureEnum.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "PathTracingOutputInvalidateReason.h"
#include "PrimitiveDirtyState.h"
#include "ScenePrimitiveUpdates.h"
#include "MaterialDomain.h"
#include "PooledRenderTarget.h"
#include "RHIGPUReadback.h"
#include <atomic>

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "SceneViewState.h"
#include "Containers/SparseSet.h"
#include "Hash/xxhash.h"
#include "Misc/Guid.h"
#include "Math/RandomStream.h"
#include "Templates/PimplPtr.h"
#include "RendererModule.h"
#include "RHI.h"
#include "UniformBuffer.h"
#include "SceneView.h"
#include "RendererInterface.h"
#include "SceneUtils.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "RenderTargetPool.h"
#include "SceneHitProxyRendering.h"
#include "LightMapRendering.h"
#include "LightComponentId.h"
#include "VelocityRendering.h"
#include "VolumeRendering.h"
#include "CommonRenderResources.h"
#include "VisualizeTexture.h"
#include "DebugViewModeRendering.h"
#include "RayTracing/RaytracingOptions.h"
#include "Nanite/Nanite.h"
#include "Lumen/LumenViewState.h"
#include "MegaLights/MegaLightsViewState.h"
#include "StochasticLighting/StochasticLightingViewState.h"
#include "VolumetricRenderTargetViewStateData.h"
#include "TranslucentLightingViewState.h"
#include "DynamicBVH.h"
#include "ShadingEnergyConservation.h"
#include "Substrate/Substrate.h"
#include "Substrate/Glint/GlintShadingLUTs.h"
#include "GlobalDistanceField.h"
#include "Algo/RemoveIf.h"
#include "UObject/Package.h"
#include "HeterogeneousVolumes/HeterogeneousVolumes.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

class FRHICommandListBase;
struct IPooledRenderTarget;
class FReflectionCaptureProxy;
struct FGuid;
class UReflectionCaptureComponent;
class FReflectionCaptureShaderData;
class FMobileReflectionCaptureShaderData;
class ASceneCaptureCube;
class FScene;
struct FHairStrandsInstance;
struct FHairTransientResources;
class FPrecomputedVolumetricLightmap;
class FVolumetricLightmapInterpolation;
class FPrecomputedVolumetricLightmapData;
class FPrimitiveSceneInfo;
class FDistanceFieldVolumeData;
class FRHIGPUBufferReadback;
class FDistanceFieldObjectBuffers;
class FGlobalShaderMap;
struct FSceneRenderUpdateInputs;
class FIndirectLightingCacheAllocation;
class FSceneRenderer;
class FViewInfo;
class FRHITexture;
struct FHLODSceneNodeVisibilityState;
class FSceneViewState;
class FRenderTarget;
class FPixelInspectorRequest;
class FLumenSceneData;
class FMobileDirectionalLightShaderParameters;
class FMobileReflectionCaptureShaderParameters;
class FPrecomputedLightVolume;
namespace UE::Tasks
{
	class FTaskEvent;
}

/* Forward declarations for FScene */

class UWorld;
class FFXSystemInterface;
class FPrimitiveSceneProxy;
class FSkyLightSceneProxy;
class FLightSceneInfo;
class FDeferredDecalProxy;
class FPlanarReflectionSceneProxy;
class UPlanarReflectionComponent;
class FRayTracingPipelineState;
class FStaticMeshBatch;
class FSkyAtmosphereRenderSceneInfo;
class FSkyAtmosphereSceneProxy;
class FVolumetricCloudRenderSceneInfo;
class FVolumetricCloudSceneProxy;
class FLocalFogVolumeSceneProxy;
class FSparseVolumeTextureViewerSceneProxy;
class FPhysicsFieldSceneProxy;
class FWindSourceSceneProxy;
class UWindDirectionalSourceComponent;
class UStaticMesh;
struct FSpeedTreeWindComputation;
class FVertexFactory;
class FPrecomputedVisibilityHandler;
class FGPUSkinCache;
class FSkeletalMeshUpdater;
class IComputeTaskWorker;
class FRuntimeVirtualTextureSceneProxy;
class FRayTracingDynamicGeometryUpdateManager;
struct FViewSceneChangeSet;
class UPrimitiveComponent;
struct FPrimitiveSceneDesc;
class ULightComponent;
class UDecalComponent;
class USceneCaptureComponent2D;
class USceneCaptureComponentCube;
class USkyLightComponent;
class UTextureCube;
class FTexture;
class FFloat16Color;
struct FLinearColor;
class URuntimeVirtualTextureComponent;
class FLightSceneProxy;
struct FLightRenderParameters;
class FRectLightSceneProxy;
class IAnimBankTransformProvider;
class IAnimRuntimeTransformProvider;
class IAnimSequenceTransformProvider;
class FRHIUniformBuffer;
class FMaterialParameterCollectionInstanceResource;
class FVirtualShadowMapArrayCacheManager;
class AWorldSettings;
class FCachedShadowMapData;
class FSceneViewFamily;
class FInstanceCullingOcclusionQueryRenderer;
class IPrimitiveTransformUpdater;
class FSceneViewStateInterface;
class FGPUSceneWriteDelegate;
struct FPersistentPrimitiveIndex;
struct FDeferredDecalUpdateParams;
class FReflectionCaptureData;
class ISceneRenderBuilder;
class FInstanceCullingManager;
struct FPrimitiveUniformShaderParametersBuilder;
struct FExponentialHeightFogDynamicState;
class FOutputDevice;
struct FCustomPrimitiveData;
class FArchive;
class FRHICommandListImmediate;
struct FCustomRenderPassRendererInput;
class FSceneExtensionsUpdaters;
class FScenePreUpdateChangeSet;
class FScenePostUpdateChangeSet;
struct FUpdateInstanceCommand;

/** Extern GPU stats (used in multiple modules) **/
DECLARE_GPU_STAT_NAMED_EXTERN(ShadowProjection, TEXT("Shadow Projection"));
DECLARE_GPU_STAT_NAMED_EXTERN(ReflectionEnvironment, TEXT("Reflection Environment"));

/** Rendering resource class that manages a cubemap array for reflections. */
class FReflectionEnvironmentCubemapArray : public FRenderResource
{
public:

	FReflectionEnvironmentCubemapArray(ERHIFeatureLevel::Type InFeatureLevel)
		: FRenderResource(InFeatureLevel)
		, MaxCubemaps(0)
		, NumReservedCubemaps(0)
		, CubemapSize(0)
	{}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	/**
	 * Updates the maximum number of cubemaps that this array is allocated for.
	 * This reallocates the resource but does not copy over the old contents.
	 * InMaxCubemaps is the total slice count (regular + reserved); InNumReserved is the count of slots reserved
	 * at the tail for runtime-reflection-capture refresh blends (see r.ReflectionCapture.Runtime.RefreshBlendSlots).
	 */
	void UpdateMaxCubemaps(uint32 InMaxCubemaps, uint32 InNumReserved, int32 CubemapSize);

	/**
	* Updates the maximum number of cubemaps that this array is allocated for.
	* This reallocates the resource and copies over the old contents, preserving indices.
	* IndexRemapping is keyed by regular slot index 0..OldMaxRegularCubemaps, while ReservedTailRemapping is keyed by reserved
	* slot index 0..OldNumReservedCubemaps, or -1 if an element is discarded due to shrink of the number reserved.
	*/
	void ResizeCubemapArrayGPU(uint32 InMaxCubemaps, uint32 InNumReserved, int32 CubemapSize, const TArray<int32>& IndexRemapping, const TArray<int32>& ReservedTailRemapping);

	int32 GetMaxCubemaps() const { return MaxCubemaps; }
	int32 GetMaxRegularCubemaps() const { return (int32)MaxCubemaps - (int32)NumReservedCubemaps; }
	int32 GetNumReservedCubemaps() const { return (int32)NumReservedCubemaps; }
	int32 GetCubemapSize() const { return CubemapSize; }
	bool IsValid() const { return IsValidRef(ReflectionEnvs); }
	const TRefCountPtr<IPooledRenderTarget>& GetRenderTarget() const { return ReflectionEnvs; }
	void Reset();

protected:
	uint32 MaxCubemaps;

	// Number of slots at the tail of the cubemap array reserved as RuntimeRefreshBlendSlots blend sources.
	// Regular capture allocations live in [0, MaxCubemaps - NumReservedCubemaps); the reserved tail occupies
	// [MaxCubemaps - NumReservedCubemaps, MaxCubemaps).
	uint32 NumReservedCubemaps;

	int32 CubemapSize;
	TRefCountPtr<IPooledRenderTarget> ReflectionEnvs;

	void ReleaseCubeArray();
};

/** Per-component reflection capture state that needs to persist through a re-register. */
class FCaptureComponentSceneState
{
public:
	/** Index of the cubemap in the array for this capture component. */
	int32 CubemapIndex;

	float AverageBrightness;

	/**
	 * False until the cubemap slot has valid rendered content for the current activation of this component.  While false, the capture
	 * is excluded from SortedCaptures so shaders don't sample a stale slot.  Reset when the budget system (re)activates a runtime capture,
	 * and set once the scene capture's final face has been copied into the cubemap array (or baked data has been uploaded).
	 */
	bool bRenderedForShading;

	/**
	 * Render-thread mirror of UReflectionCaptureComponent's fade state.  The current blend value is evaluated each frame from these fields
	 * plus r.ReflectionCapture.Runtime.FadeInTime, and written into the uniform buffer's CaptureProperties.a.  Defaults (1,1,0) mean
	 * "steady at 1" -- no fade in or out.
	 */
	float FadeStartValue;
	float FadeTargetValue;
	double FadeStartTime;

	FCaptureComponentSceneState(int32 InCubemapIndex) :
		CubemapIndex(InCubemapIndex),
		AverageBrightness(0.0f),
		bRenderedForShading(false),
		FadeStartValue(1.0f),
		FadeTargetValue(1.0f),
		FadeStartTime(0.0)
	{}

	/** Compute the current fade value for the supplied time and fade duration.  Matches UReflectionCaptureComponent::ComputeCurrentRuntimeCaptureFade. */
	float ComputeCurrentFade(double Now, float Duration) const;

	bool operator==(const FCaptureComponentSceneState& Other) const
	{
		return CubemapIndex == Other.CubemapIndex;
	}
};

// Subset of FReflectionCaptureSortData, to support separate sorting of the key by itself.
struct FReflectionCaptureSortKey
{
	FXxHash64 Guid;
	float Radius;

	bool operator<(const FReflectionCaptureSortKey& Other) const
	{
		if (Radius != Other.Radius)
		{
			return Radius < Other.Radius;
		}
		else
		{
			return Guid < Other.Guid;
		}
	}

	bool operator!=(const FReflectionCaptureSortKey& Other) const
	{
		return Guid != Other.Guid || Radius != Other.Radius;
	}
};

struct FReflectionCaptureSortData : FReflectionCaptureSortKey
{
	FDFVector3 Position;
	FMatrix44f BoxTransform;
	int32 CubemapIndex;
	FVector4f CaptureProperties;
	FVector4f BoxScales;
	FVector4f CaptureOffsetAndAverageBrightness;
	FReflectionCaptureProxy* CaptureProxy;
	float FadeAlpha;

	// Packed smooth blend data written into PositionLow.w:
	// Integer part = source cubemap array index, fractional part = blend alpha (1=full source, 0=full new).
	// A value of 0 means no active blend.
	float SmoothBlendEncoded = 0.0f;
};


struct FReflectionCaptureCacheEntry
{
	int32 RefCount;
	FCaptureComponentSceneState SceneState;
};


struct FReflectionCaptureCache
{
public:

	const FCaptureComponentSceneState* Find(const FGuid& MapBuildDataId) const;
	FCaptureComponentSceneState* Find(const FGuid& MapBuildDataId);

	const FCaptureComponentSceneState* Find(const UReflectionCaptureComponent* Component) const;
	FCaptureComponentSceneState* Find(const UReflectionCaptureComponent* Component);
	const FCaptureComponentSceneState& FindChecked(const UReflectionCaptureComponent* Component) const;
	FCaptureComponentSceneState& FindChecked(const UReflectionCaptureComponent* Component);

	FCaptureComponentSceneState& Add(const UReflectionCaptureComponent* Component, const FCaptureComponentSceneState& Value);
	FCaptureComponentSceneState* AddReference(const UReflectionCaptureComponent* Component);
	bool Remove(const UReflectionCaptureComponent* Component);
	int32 Prune(const TSet<FGuid> KeysToKeep, TArray<int32>& ReleasedIndices);

	int32 GetKeys(TArray<FGuid>& OutKeys) const;
	int32 GetKeys(TSet<FGuid>& OutKeys) const;

	void Empty();

protected:

	bool RemapRegisteredComponentMapBuildDataId(const UReflectionCaptureComponent* Component);
	void RegisterComponentMapBuildDataId(const UReflectionCaptureComponent* Component);
	void UnregisterComponentMapBuildDataId(const UReflectionCaptureComponent* Component);

	// Different map build data id of a capture might share the same capture component while editing (e.g., when they move).
	// need to replace it with the new one.
	TMap<const UReflectionCaptureComponent*, FGuid> RegisteredComponentMapBuildDataIds;

	TMap<FGuid, FReflectionCaptureCacheEntry> CaptureData;
};

struct FReflectionEnvironmentCachedShadow
{
	uint32 X;
	uint32 Y;
	FVector4 Position;
	FMatrix WorldToLight;
	FMatrix LightToWorld;
};

struct FReflectionEnvironmentCachedShadowAtlas
{
	TRefCountPtr<IPooledRenderTarget> AtlasRenderTarget;
	FIntPoint AtlasSize = { 0, 0 };
	uint32 TimesliceAtlasFrame = 0;
	bool TimesliceAtlasFailed = false;							// Set to true if a valid shadow in the atlas couldn't be found
	TArray<FReflectionEnvironmentCachedShadow> Shadows;
};

/** Scene state used to manage the reflection environment feature. */
class FReflectionEnvironmentSceneData
{
public:

	/** 
	 * Set to true for one frame whenever RegisteredReflectionCaptures or the transforms of any registered reflection proxy has changed,
	 * Which allows one frame to update cached proxy associations.
	 */
	bool bRegisteredReflectionCapturesHasChanged;

	/** Set to true for one frame whenever Scene's Skylight changes. Which allows one frame to RequestStaticMeshUpdate for the Scene's Primitives.*/
	bool bSkylightHasChanged;
	
	/** True if AllocatedReflectionCaptureState has changed. Allows to update cached single capture id. */
	bool AllocatedReflectionCaptureStateHasChanged;

	/** The rendering thread's list of visible reflection captures in the scene. */
	TArray<FReflectionCaptureProxy*> RegisteredReflectionCaptures;
	TArray<FSphere> RegisteredReflectionCapturePositionAndRadius;

	/** 
	 * Cubemap array resource which contains the captured scene for each reflection capture.
	 * This is indexed by the value of AllocatedReflectionCaptureState.CaptureIndex.
	 */
	FReflectionEnvironmentCubemapArray CubemapArray;

	/** Rendering thread map from component to scene state.  This allows storage of RT state that needs to persist through a component re-register. */
	FReflectionCaptureCache AllocatedReflectionCaptureState;

	/** Rendering bitfield to track cubemap slots used. Needs to kept in sync with AllocatedReflectionCaptureState */
	TBitArray<> CubemapArraySlotsUsed;

	/** Sorted scene reflection captures for upload to the GPU. */
	TArray<FReflectionCaptureSortData> SortedCaptures;
	int32 NumBoxCaptures;
	int32 NumSphereCaptures;

	// Uniforms buffers with a sorted captures
	TUniformBufferRef<FReflectionCaptureShaderData> ReflectionCaptureUniformBuffer;
	TUniformBufferRef<FMobileReflectionCaptureShaderData> MobileReflectionCaptureUniformBuffer;

	/**
	 * As a mobile renderer optimization, when reflection captures change, we need to selectively update only primitives that are affected by the
	 * change.  Previously, a full primitive scene info and subsequent mesh draw command invalidation would occur for every static mesh, on any
	 * type of reflection capture edit.  This can drop the frame rate to low single digits depending on the complexity of scene loaded, and make
	 * dragging or slider adjusting values impossible.
	 *
	 * The cases where static meshes need to be updated on reflection capture changes for mobile are threefold.  First, a uniform buffer is referenced
	 * by static mesh draw calls, which contains a pointer to the texture for reflection captures that have a user defined texture, and settings for
	 * intensity and blending.  The proxy and its uniform buffer are recreated whenever a reflection capture is modified, and references need to be
	 * updated to the new uniform buffer.  Second, a reflection capture index, into the sorted reflection capture array, is stored on each primitive.
	 * The sort order can change if the capture is resized to a point where its radius changes relative to another capture, as radius is part of the
	 * sort key.  Finally, a change to the size or transform of a capture can cause it to intersect or no longer intersect a given primitive.
	 *
	 * Any static mesh that references, or previously referenced, a changed reflection capture needs to be updated.  A flag on the primitive
	 * scene proxy tracks whether its cached reflection capture has been modified.  Note that any property edit besides a transform will remove and
	 * recreate the render proxy, so we can just handle property changes in "remove".  Transform changes only matter for cases where a capture is newly
	 * or no longer intersected, which is detected in the call to CacheReflectionCaptures, which is run on all primitives regardless of their flags.
	 *
	 * Handling sort order changes is done by caching the previous sort order keys during a change, so it can be compared with the current sort order.
	 * On a sort order change, all primitives referencing ANY capture need to be updated, since their indices may have changed, even if they are not
	 * the specifically edited reflection capture.
	 */
	TArray<FReflectionCaptureSortKey> PreChangeSortOrder;

	/** 
	 * Game thread list of reflection components that have been allocated in the cubemap array. 
	 * These are not necessarily all visible or being rendered, but their scene state is stored in the cubemap array.
	 */
	TSparseArray<UReflectionCaptureComponent*> AllocatedReflectionCapturesGameThread;

	/** Game thread tracking of what size this scene has allocated for the cubemap array. */
	int32 MaxAllocatedReflectionCubemapsGameThread;

	/** Game thread tracking of what size cubemaps are in the cubemap array. */
	int32 ReflectionCaptureSizeGameThread;
	int32 DesiredReflectionCaptureSizeGameThread;

	/** Singleton actor used for runtime captures.  A scene capture is used, as these are more optimized than the editor reflection capture logic. */
	TObjectPtr<ASceneCaptureCube> RuntimeCaptureActor;

	/** All runtime capture components registered as candidates for the budget system. Managed on game thread. */
	TArray<UReflectionCaptureComponent*> RuntimeCaptureBudgetCandidates;

	/** Data for timesliced runtime reflection captures */
	UReflectionCaptureComponent* RuntimeCaptureTimesliceComponent;			/** Current component being timesliced (game thread) */
	int32 RuntimeCaptureTimesliceFaceCount_GT;								/** Number of faces rendered so far (game thread) */
	int32 RuntimeCaptureTimesliceFaceCount_RT;								/** Mirror of above (render thread) */
	TRefCountPtr<IPooledRenderTarget> RuntimeCaptureTarget;					/** Cached render target storing cube map faces (render thread) */
	FReflectionEnvironmentCachedShadowAtlas RuntimeCaptureShadows;			/** Cached shadow maps shared for all cube map faces (render thread) */

	/**
	 * State used by the fast-render path to detect teleports and level-load.  RuntimeFastRenderLastCameraLocation stores the previous
	 * frame's camera position so we can measure whether a camera jump counts as a teleport.  bRuntimeFastRenderPendingLevelLoad fires the
	 * initial load burst at startup, then stays false for the lifetime of the scene.
	 */
	FVector RuntimeFastRenderLastCameraLocation;
	bool bRuntimeFastRenderHasLastCameraLocation;
	bool bRuntimeFastRenderPendingLevelLoad;

	/**
	 * One entry per reserved smooth blend source slot in the cubemap array (the reserved tail just past MaxAllocatedReflectionCubemapsGameThread).
	 * Includes the slot index of the source cube map in the array, and the wall-clock time the blend started (negative until the timesliced
	 * capture completes, after which the fade clock starts).  An entry with OwningComponent == nullptr is free.
	 */
	struct FSmoothBlendEntry
	{
		UReflectionCaptureComponent* OwningComponent = nullptr;
		int32 SourceCubemapArrayIndex = INDEX_NONE;
		double BlendStartTime = -1.0;
	};
	TArray<FSmoothBlendEntry> RuntimeSmoothBlendSlots;

	FReflectionEnvironmentSceneData(ERHIFeatureLevel::Type InFeatureLevel) :
		bRegisteredReflectionCapturesHasChanged(true),
		bSkylightHasChanged(false),
		AllocatedReflectionCaptureStateHasChanged(false),
		CubemapArray(InFeatureLevel),
		MaxAllocatedReflectionCubemapsGameThread(0),
		ReflectionCaptureSizeGameThread(0),
		DesiredReflectionCaptureSizeGameThread(0),
		RuntimeCaptureTimesliceComponent(nullptr),
		RuntimeCaptureTimesliceFaceCount_GT(0),
		RuntimeCaptureTimesliceFaceCount_RT(0),
		RuntimeFastRenderLastCameraLocation(FVector::ZeroVector),
		bRuntimeFastRenderHasLastCameraLocation(false),
		bRuntimeFastRenderPendingLevelLoad(true)
	{}

	/** Set Data necessary to determine if GPU resources will need future updates */
	void SetGameThreadTrackingData(int32 MaxAllocatedCubemaps, int32 CaptureSize, int32 DesiredCaptureSize);

	/** Do the resources on the GPU match our desired state?  If not, reallocation will be necessary. */
	bool DoesAllocatedDataNeedUpdate(int32 DesiredMaxCubemaps, int32 DesiredCaptureSize) const;

	void ResizeCubemapArrayGPU(uint32 InMaxCubemaps, uint32 InNumReserved, int32 InCubemapSize);

	/** Resets the structure to empty, useful if you want to shrink the allocation. */
	void Reset(FScene* Scene);

	/** Mark reflection captures as changed, also initializing PreChangeSortOrder. */
	void SetRegisteredReflectionCapturesHasChanged();
};

/** Scene state used to manage hair strands. */
class FHairStrandsSceneData
{
public:
	// Ref-counted to guarantee instances stay alive while registered in the scene,
	// preventing use-after-free when the game thread destroys groom components
	// while the render thread is still accessing instances (e.g., during Sequencer scrubbing).
	TArray<TRefCountPtr<FHairStrandsInstance>> RegisteredProxies;
	FHairTransientResources* TransientResources = nullptr;
};

class FVolumetricLightmapInterpolation
{
public:
	FVector4f IndirectLightingSHCoefficients0[3];
	FVector4f IndirectLightingSHCoefficients1[3];
	FVector4f IndirectLightingSHCoefficients2;
	FVector4f IndirectLightingSHSingleCoefficient;
	FVector4f PointSkyBentNormal;
	float DirectionalLightShadowing;
	uint32 LastUsedSceneFrameNumber;
};

class FVolumetricLightmapSceneData
{
public:

	FVolumetricLightmapSceneData(FScene* InScene)
		: Scene(InScene)
	{
		GlobalVolumetricLightmap.Data = &GlobalVolumetricLightmapData;
	}

	bool HasData() const;

	void AddLevelVolume   (FRHICommandListImmediate& RHICmdList, const class FPrecomputedVolumetricLightmap* InVolume, EShadingPath ShadingPath, bool bIsPersistentLevel);
	void RemoveLevelVolume(FRHICommandListImmediate& RHICmdList, const class FPrecomputedVolumetricLightmap* InVolume);

	const FPrecomputedVolumetricLightmap* GetLevelVolumetricLightmap() const;

	TMap<FVector, FVolumetricLightmapInterpolation> CPUInterpolationCache;

	FPrecomputedVolumetricLightmapData GlobalVolumetricLightmapData;
private:
	FScene* Scene;
	FPrecomputedVolumetricLightmap GlobalVolumetricLightmap;
	const FPrecomputedVolumetricLightmap* PersistentLevelVolumetricLightmap = nullptr;
	TArray<const FPrecomputedVolumetricLightmap*> LevelVolumetricLightmaps;
};

class FPrimitiveAndInstance
{
public:
	FPrimitiveAndInstance(const FMatrix& InLocalToWorld, const FBox& InWorldBounds, FPrimitiveSceneInfo* InPrimitive, int32 InInstanceIndex)
	: Primitive(InPrimitive)
	, InstanceIndex(InInstanceIndex)
	{
		SetTransformAndBounds(InLocalToWorld, InWorldBounds);
	}

	FPrimitiveSceneInfo* Primitive;

	FVector Origin;
	FVector3f TransformRows[3];

	FRenderBounds WorldBoundsRelativeToOrigin;

	int32 InstanceIndex;

	FORCEINLINE void SetTransformAndBounds(const FMatrix& InLocalToWorld, const FBox& InWorldBounds)
	{
		TransformRows[0] = FVector3f((float)InLocalToWorld.M[0][0], (float)InLocalToWorld.M[0][1], (float)InLocalToWorld.M[0][2]);
		TransformRows[1] = FVector3f((float)InLocalToWorld.M[1][0], (float)InLocalToWorld.M[1][1], (float)InLocalToWorld.M[1][2]);
		TransformRows[2] = FVector3f((float)InLocalToWorld.M[2][0], (float)InLocalToWorld.M[2][1], (float)InLocalToWorld.M[2][2]);
		Origin = FVector(InLocalToWorld.M[3][0], InLocalToWorld.M[3][1], InLocalToWorld.M[3][2]);

		WorldBoundsRelativeToOrigin = InWorldBounds.ShiftBy(-Origin);
	}

	FORCEINLINE FMatrix GetLocalToWorld() const
	{
		FMatrix Matrix;
		Matrix.M[0][0] = TransformRows[0].X;
		Matrix.M[0][1] = TransformRows[0].Y;
		Matrix.M[0][2] = TransformRows[0].Z;
		Matrix.M[0][3] = 0.0f;
		Matrix.M[1][0] = TransformRows[1].X;
		Matrix.M[1][1] = TransformRows[1].Y;
		Matrix.M[1][2] = TransformRows[1].Z;
		Matrix.M[1][3] = 0.0f;
		Matrix.M[2][0] = TransformRows[2].X;
		Matrix.M[2][1] = TransformRows[2].Y;
		Matrix.M[2][2] = TransformRows[2].Z;
		Matrix.M[2][3] = 0.0f;
		Matrix.M[3][0] = Origin.X;
		Matrix.M[3][1] = Origin.Y;
		Matrix.M[3][2] = Origin.Z;
		Matrix.M[3][3] = 1.0f;
		return Matrix;
	}

	FORCEINLINE FBox GetWorldBounds() const
	{
		FBox WorldBoundsRelativeToOriginDoublePrecision = WorldBoundsRelativeToOrigin.ToBox();
		return WorldBoundsRelativeToOriginDoublePrecision.ShiftBy(Origin);
	}
};

class FPrimitiveRemoveInfo
{
public:
	FPrimitiveRemoveInfo(const FPrimitiveSceneInfo* InPrimitive)
	: Primitive(InPrimitive)
	, bOftenMoving(InPrimitive->Proxy->IsOftenMoving())
	, DistanceFieldInstanceIndices(Primitive->DistanceFieldInstanceIndices)
	{
		float SelfShadowBias;
		InPrimitive->Proxy->GetDistanceFieldAtlasData(DistanceFieldData, SelfShadowBias);
	}

	/** 
	 * Must not be dereferenced after creation, the primitive was removed from the scene and deleted
	 * Value of the pointer is still useful for map lookups
	 */
	const FPrimitiveSceneInfo* Primitive;

	bool bOftenMoving;

	TArray<int32, TInlineAllocator<1>> DistanceFieldInstanceIndices;

	const FDistanceFieldVolumeData* DistanceFieldData;
};

class FHeightFieldPrimitiveRemoveInfo : public FPrimitiveRemoveInfo
{
public:
	FHeightFieldPrimitiveRemoveInfo(const FPrimitiveSceneInfo* InPrimitive)
		: FPrimitiveRemoveInfo(InPrimitive)
	{
		const FBoxSphereBounds Bounds = InPrimitive->Proxy->GetBounds();
		WorldBounds = Bounds.GetBox();
	}

	FBox WorldBounds;
};

/** Identifies a mip of a distance field atlas. */
class FDistanceFieldAssetMipId
{
public:

	FDistanceFieldAssetMipId(FSetElementId InAssetId, int32 InReversedMipIndex = 0) :
		AssetId(InAssetId),
		ReversedMipIndex(InReversedMipIndex)
	{}

	FSetElementId AssetId;
	int32 ReversedMipIndex;
};

/** Stores distance field mip relocation data. */
class FDistanceFieldAssetMipRelocation
{
public:
	FDistanceFieldAssetMipRelocation(FIntVector InIndirectionDimensions, FIntVector InSrcPosition, FIntVector InDstPosition) :
		IndirectionDimensions(InIndirectionDimensions),
		SrcPosition(InSrcPosition),
		DstPosition(InDstPosition)
	{}

	FIntVector IndirectionDimensions;
	FIntVector SrcPosition;
	FIntVector DstPosition;
};

/** Stores state about a distance field mip that is tracked by the scene. */
class FDistanceFieldAssetMipState
{
public:

	FDistanceFieldAssetMipState() :
		IndirectionDimensions(FIntVector(0, 0, 0)),
		IndirectionTableOffset(-1),
		NumBricks(0)
	{}

	FIntVector IndirectionDimensions;
	int32 IndirectionTableOffset;
	FIntVector IndirectionAtlasOffset;
	int32 NumBricks;
	TArray<int32, TInlineAllocator<4>> AllocatedBlocks;
};

class FDistanceFieldAssetState
{
public:

	FDistanceFieldAssetState() :
		BuiltData(nullptr),
		RefCount(0),
		WantedNumMips(0)
	{}

	const FDistanceFieldVolumeData* BuiltData;
	int32 RefCount;
	int32 WantedNumMips;
	TArray<FDistanceFieldAssetMipState, TInlineAllocator<3>> ReversedMips;
};

struct TFDistanceFieldAssetStateFuncs : BaseKeyFuncs<FDistanceFieldAssetState, const FDistanceFieldVolumeData*, /* bInAllowDuplicateKeys = */ false>
{
	static FORCEINLINE KeyInitType GetSetKey(ElementInitType Element)
	{
		return Element.BuiltData;
	}
	static bool Matches(KeyInitType A,KeyInitType B)
	{
		return A == B;
	}
	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return PointerHash(Key);
	}
};

class FDistanceFieldBlockAllocator
{
public:
	void Allocate(int32 NumBlocks, TArray<int32, TInlineAllocator<4>>& OutBlocks);

	void Free(const TArray<int32, TInlineAllocator<4>>& ElementRange);

	int32 GetMaxSize() const 
	{ 
		return MaxNumBlocks; 
	}

	int32 GetAllocatedSize() const
	{
		return MaxNumBlocks - FreeBlocks.Num();
	}

private:
	int32 MaxNumBlocks = 0;
	TArray<int32, TInlineAllocator<4>> FreeBlocks;
};

struct FDistanceFieldReadRequest;
struct FDistanceFieldAsyncUpdateParameters;

/** Scene data used to manage distance field object buffers on the GPU. */
class FDistanceFieldSceneData
{
public:
	FDistanceFieldSceneData(FDistanceFieldSceneData&&);
	FDistanceFieldSceneData(EShaderPlatform ShaderPlatform);
	~FDistanceFieldSceneData();

	void AddPrimitive(FPrimitiveSceneInfo* InPrimitive);
	void UpdatePrimitive(FPrimitiveSceneInfo* InPrimitive);
	void RemovePrimitive(FPrimitiveSceneInfo* InPrimitive);
	void Release();
	void VerifyIntegrity();
	void ListMeshDistanceFields(bool bDumpAssetStats) const;

	void UpdateDistanceFieldObjectBuffers(
		FRDGBuilder& GraphBuilder,
		FRDGExternalAccessQueue& ExternalAccessQueue,
		FScene* Scene,
		TArray<FDistanceFieldAssetMipId>& DistanceFieldAssetAdds,
		TArray<FSetElementId>& DistanceFieldAssetRemoves);

	void UpdateDistanceFieldAtlas(
		FRDGBuilder& GraphBuilder,
		FRDGExternalAccessQueue& ExternalAccessQueue,
		const FSceneRenderUpdateInputs& SceneUpdateInputs,
		TArray<FDistanceFieldAssetMipId>& DistanceFieldAssetAdds,
		TArray<FSetElementId>& DistanceFieldAssetRemoves);

	bool HasPendingUploads() const
	{
		return IndicesToUpdateInObjectBuffers.Num() > 0;
	}

	bool HasPendingOperations() const
	{
		return PendingAddOperations.Num() > 0 || PendingUpdateOperations.Num() > 0 || PendingRemoveOperations.Num() > 0;
	}

	bool HasPendingHeightFieldOperations() const
	{
		return PendingHeightFieldAddOps.Num() > 0 || PendingHeightFieldRemoveOps.Num() > 0;
	}

	bool HasPendingRemovePrimitive(const FPrimitiveSceneInfo* Primitive) const
	{
		for (int32 RemoveIndex = 0; RemoveIndex < PendingRemoveOperations.Num(); ++RemoveIndex)
		{
			if (PendingRemoveOperations[RemoveIndex].Primitive == Primitive)
			{
				return true;
			}
		}

		return false;
	}

	bool HasPendingStreaming() const
	{
		return bHasPendingStreaming;
	}

	inline bool CanUse16BitObjectIndices() const
	{
		return bCanUse16BitObjectIndices && (NumObjectsInBuffer < (1 << 16));
	}

	const FDistanceFieldObjectBuffers* GetCurrentObjectBuffers() const
	{
		return ObjectBuffers;
	}

	const FDistanceFieldObjectBuffers* GetHeightFieldObjectBuffers() const
	{
		return HeightFieldObjectBuffers;
	}

	int32 NumObjectsInBuffer;
	FDistanceFieldObjectBuffers* ObjectBuffers;
	FDistanceFieldObjectBuffers* HeightFieldObjectBuffers;

	FRDGScatterUploadBuffer UploadHeightFieldDataBuffer;
	FRDGScatterUploadBuffer UploadHeightFieldBoundsBuffer;
	FRDGScatterUploadBuffer UploadDistanceFieldDataBuffer;
	FRDGScatterUploadBuffer UploadDistanceFieldBoundsBuffer;

	// track indices that need to be updated using both an array and a set
	// array is used for fast iteration and support ParallelFor
	// set is used to prevent duplicate indices
	TArray<int32> IndicesToUpdateInObjectBuffers;
	TSet<int32> IndicesToUpdateInObjectBuffersSet;

	TSparseSet<FDistanceFieldAssetState, TFDistanceFieldAssetStateFuncs> AssetStateArray;
	TRefCountPtr<FRDGPooledBuffer> AssetDataBuffer;
	FRDGScatterUploadBuffer AssetDataUploadBuffer;

	TArray<FRHIGPUBufferReadback*> StreamingRequestReadbackBuffers;
	uint32 MaxStreamingReadbackBuffers = 4;
	uint32 ReadbackBuffersWriteIndex = 0;
	uint32 ReadbackBuffersNumPending = 0;

	FGrowOnlySpanAllocator IndirectionTableAllocator;
	TRefCountPtr<FRDGPooledBuffer> IndirectionTable;
	FRDGAsyncScatterUploadBuffer IndirectionTableUploadBuffer;

	TRefCountPtr<IPooledRenderTarget> IndirectionAtlas;
	FTextureLayout3d IndirectionAtlasLayout;
	FReadBuffer IndirectionUploadIndicesBuffer;
	FReadBuffer IndirectionUploadDataBuffer;

	FDistanceFieldBlockAllocator DistanceFieldAtlasBlockAllocator;
	TRefCountPtr<IPooledRenderTarget> DistanceFieldBrickVolumeTexture;
	FIntVector BrickTextureDimensionsInBricks;
	FReadBuffer BrickUploadCoordinatesBuffer;
	FReadBuffer BrickUploadDataBuffer;

	TArray<FDistanceFieldReadRequest> ReadRequests;
	bool bHasPendingStreaming = false;

	/** Stores the primitive and instance index of every entry in the object buffer. */
	TArray<FPrimitiveAndInstance> PrimitiveInstanceMapping;
	TArray<FPrimitiveSceneInfo*> HeightfieldPrimitives;
	/** Pending operations on the object buffers to be processed next frame. */
	TSet<FPrimitiveSceneInfo*> PendingAddOperations;
	TSet<FPrimitiveSceneInfo*> PendingUpdateOperations;
	TArray<FPrimitiveRemoveInfo> PendingRemoveOperations;
	TArray<FBox> PrimitiveModifiedBounds[GDF_Num];

	TSet<FPrimitiveSceneInfo*> PendingHeightFieldAddOps;
	TArray<FHeightFieldPrimitiveRemoveInfo> PendingHeightFieldRemoveOps;

	int32 HeightFieldAtlasGeneration;
	int32 HFVisibilityAtlasGenerattion;

	bool bTrackAllPrimitives;
	bool bCanUse16BitObjectIndices;

private:

	void ProcessStreamingRequestsFromGPU(
		TArray<FDistanceFieldReadRequest>& NewReadRequests,
		TArray<FDistanceFieldAssetMipId>& AssetDataUploads);

	void ProcessReadRequests(
		TArray<FDistanceFieldAssetMipId>& AssetDataUploads,
		TArray<FDistanceFieldAssetMipId>& DistanceFieldAssetMipAdds,
		TArray<FDistanceFieldReadRequest>& ReadRequestsToUpload,
		TArray<FDistanceFieldReadRequest>& ReadRequestsToCleanUp);

	FRDGTexture* ResizeBrickAtlasIfNeeded(FRDGBuilder& GraphBuilder, FGlobalShaderMap* GlobalShaderMap);

	bool ResizeIndirectionAtlasIfNeeded(FRDGBuilder& GraphBuilder, FGlobalShaderMap* GlobalShaderMap, FRDGTexture*& OutTexture);

	void DefragmentIndirectionAtlas(FIntVector MinSize, TArray<FDistanceFieldAssetMipRelocation>& Relocations);

	void UploadAssetData(FRDGBuilder& GraphBuilder, const TArray<FDistanceFieldAssetMipId>& AssetDataUploads, FRDGBuffer* AssetDataBufferRDG);
	
	void UploadAllAssetData(FRDGBuilder& GraphBuilder, FRDGBuffer* AssetDataBufferRDG);

	void AsyncUpdate(FRHICommandListBase& RHICmdList, FDistanceFieldAsyncUpdateParameters& UpdateParameters);

	void GenerateStreamingRequests(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs& SceneUpdateInputs);

	friend class FDistanceFieldStreamingUpdateTask;
};

/**
 * Updates the global distance field view origin for all scene renderers.
 * Must be called before PrepareDistanceFieldScene(...).
 */
extern void UpdateGlobalDistanceFieldViewOrigin(const FSceneRenderUpdateInputs& SceneUpdateInputs);

/**
 * Prepares the distance field scene for all scene renderers.
 * Must be called after UpdateGlobalDistanceFieldViewOrigin(...).
 */
extern void PrepareDistanceFieldScene(FRDGBuilder& GraphBuilder, FRDGExternalAccessQueue& ExternalAccessQueue, const FSceneRenderUpdateInputs& SceneUpdateInputs);

/** Stores data for an allocation in the FIndirectLightingCache. */
class FIndirectLightingCacheBlock
{
public:

	FIndirectLightingCacheBlock() :
		MinTexel(FIntVector(0, 0, 0)),
		TexelSize(0),
		Min(FVector(0, 0, 0)),
		Size(FVector(0, 0, 0)),
		bHasEverBeenUpdated(false)
	{}

	FIntVector MinTexel;
	int32 TexelSize;
	FVector Min;
	FVector Size;
	bool bHasEverBeenUpdated;
};

/** Stores information about an indirect lighting cache block to be updated. */
class FBlockUpdateInfo
{
public:

	FBlockUpdateInfo(const FIndirectLightingCacheBlock& InBlock, FIndirectLightingCacheAllocation* InAllocation) :
		Block(InBlock),
		Allocation(InAllocation)
	{}

	FIndirectLightingCacheBlock Block;
	FIndirectLightingCacheAllocation* Allocation;
};

/** Information about the primitives that are attached together. */
class FAttachmentGroupSceneInfo
{
public:

	/** The parent primitive, which is the root of the attachment tree. */
	FPrimitiveSceneInfo* ParentSceneInfo;

	/** The primitives in the attachment group. */
	TArray<FPrimitiveSceneInfo*> Primitives;

	FAttachmentGroupSceneInfo() :
		ParentSceneInfo(nullptr)
	{}
};

struct FILCUpdatePrimTaskData
{
	FGraphEventRef TaskRef;
	TMap<FIntVector, FBlockUpdateInfo> OutBlocksToUpdate;
	TArray<FIndirectLightingCacheAllocation*> OutTransitionsOverTimeToUpdate;
	TArray<FPrimitiveSceneInfo*> OutPrimitivesToUpdateStaticMeshes;
};

/** 
 * Implements a volume texture atlas for caching indirect lighting on a per-object basis.
 * The indirect lighting is interpolated from Lightmass SH volume lighting samples.
 */
class FIndirectLightingCache : public FRenderResource
{
public:	

	/** true for the editor case where we want a better preview for object that have no valid lightmaps */
	FIndirectLightingCache(ERHIFeatureLevel::Type InFeatureLevel);

	// FRenderResource interface
	virtual void InitRHI(FRHICommandListBase& RHICmdList);
	virtual void ReleaseRHI();

	/** Allocates a block in the volume texture atlas for a primitive. */
	FIndirectLightingCacheAllocation* AllocatePrimitive(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bUnbuiltPreview);

	/** Releases the indirect lighting allocation for the given primitive. */
	void ReleasePrimitive(FPrimitiveComponentId PrimitiveId);

	FIndirectLightingCacheAllocation* FindPrimitiveAllocation(FPrimitiveComponentId PrimitiveId) const;	

	/** Updates indirect lighting in the cache based on visibility synchronously. */
	void UpdateCache(FRHICommandListBase& RHICmdList, FScene* Scene, FSceneRenderer& Renderer, bool bAllowUnbuiltPreview);

	/** Starts a task to update the cache primitives.  Results and task ref returned in the FILCUpdatePrimTaskData structure */
	void StartUpdateCachePrimitivesTask(FScene* Scene, FSceneRenderer& Renderer, bool bAllowUnbuiltPreview, FILCUpdatePrimTaskData& OutTaskData);

	/** Wait on a previously started task and complete any block updates and debug draw */
	void FinalizeCacheUpdates(FRHICommandListBase& RHICmdList, FScene* Scene, FSceneRenderer& Renderer, FILCUpdatePrimTaskData& TaskData);

	/** Force all primitive allocations to be re-interpolated. */
	void SetLightingCacheDirty(FScene* Scene, const FPrecomputedLightVolume* Volume);

	// Accessors
	FRHITexture* GetTexture0() { return Texture0->GetRHI(); }
	FRHITexture* GetTexture1() { return Texture1->GetRHI(); }
	FRHITexture* GetTexture2() { return Texture2->GetRHI(); }

private:
	/** Internal helper to determine if indirect lighting is enabled at all */
	bool IndirectLightingAllowed(FScene* Scene, FSceneRenderer& Renderer) const;

	void ProcessPrimitiveUpdate(FScene* Scene, FViewInfo& View, int32 PrimitiveIndex, bool bAllowUnbuiltPreview, bool bAllowVolumeSample, TMap<FIntVector, FBlockUpdateInfo>& OutBlocksToUpdate, TArray<FIndirectLightingCacheAllocation*>& OutTransitionsOverTimeToUpdate, TArray<FPrimitiveSceneInfo*>& OutPrimitivesToUpdateStaticMeshes);

	/** Internal helper to perform the work of updating the cache primitives.  Can be done on any thread as a task */
	void UpdateCachePrimitivesInternal(FScene* Scene, FSceneRenderer& Renderer, bool bAllowUnbuiltPreview, TMap<FIntVector, FBlockUpdateInfo>& OutBlocksToUpdate, TArray<FIndirectLightingCacheAllocation*>& OutTransitionsOverTimeToUpdate, TArray<FPrimitiveSceneInfo*>& OutPrimitivesToUpdateStaticMeshes);

	/** Internal helper to perform blockupdates and transition updates on the results of UpdateCachePrimitivesInternal.  Must be on render thread. */
	void FinalizeUpdateInternal_RenderThread(FRHICommandListBase& RHICmdList, FScene* Scene, FSceneRenderer& Renderer, TMap<FIntVector, FBlockUpdateInfo>& BlocksToUpdate, const TArray<FIndirectLightingCacheAllocation*>& TransitionsOverTimeToUpdate, TArray<FPrimitiveSceneInfo*>& PrimitivesToUpdateStaticMeshes);

	/** Internal helper which adds an entry to the update lists for this allocation, if needed (due to movement, etc). Returns true if the allocation was updated or will be udpated */
	bool UpdateCacheAllocation(
		const FBoxSphereBounds& Bounds, 
		int32 BlockSize,
		bool bPointSample,
		bool bUnbuiltPreview,
		FIndirectLightingCacheAllocation*& Allocation, 
		TMap<FIntVector, FBlockUpdateInfo>& BlocksToUpdate,
		TArray<FIndirectLightingCacheAllocation*>& TransitionsOverTimeToUpdate);	

	/** 
	 * Creates a new allocation if needed, caches the result in PrimitiveSceneInfo->IndirectLightingCacheAllocation, 
	 * And adds an entry to the update lists when an update is needed. 
	 */
	void UpdateCachePrimitive(
		const TMap<FPrimitiveComponentId, FAttachmentGroupSceneInfo>& AttachmentGroups,
		FPrimitiveSceneInfo* PrimitiveSceneInfo,
		bool bAllowUnbuiltPreview, 
		bool bAllowVolumeSample, 
		TMap<FIntVector, FBlockUpdateInfo>& BlocksToUpdate, 
		TArray<FIndirectLightingCacheAllocation*>& TransitionsOverTimeToUpdate,
		TArray<FPrimitiveSceneInfo*>& PrimitivesToUpdateStaticMeshes);

	/** Updates the contents of the volume texture blocks in BlocksToUpdate. */
	void UpdateBlocks(FRHICommandListBase& RHICmdList, FScene* Scene, FViewInfo* DebugDrawingView, TMap<FIntVector, FBlockUpdateInfo>& BlocksToUpdate);

	/** Updates any outstanding transitions with a new delta time. */
	void UpdateTransitionsOverTime(const TArray<FIndirectLightingCacheAllocation*>& TransitionsOverTimeToUpdate, float DeltaWorldTime) const;

	/** Creates an allocation to be used outside the indirect lighting cache and a block to be used internally. */
	FIndirectLightingCacheAllocation* CreateAllocation(int32 BlockSize, const FBoxSphereBounds& Bounds, bool bPointSample, bool bUnbuiltPreview);	

	/** Block accessors. */
	FIndirectLightingCacheBlock& FindBlock(FIntVector TexelMin);
	const FIndirectLightingCacheBlock& FindBlock(FIntVector TexelMin) const;

	/** Block operations. */
	void DeallocateBlock(FIntVector Min, int32 Size);
	bool AllocateBlock(int32 Size, FIntVector& OutMin);

	/**
	 * Updates an allocation block in the cache, by re-interpolating values and uploading to the cache volume texture.
	 * @param DebugDrawingView can be 0
	 */
	void UpdateBlock(FRHICommandListBase& RHICmdList, FScene* Scene, FViewInfo* DebugDrawingView, FBlockUpdateInfo& Block);

	/** Interpolates a single SH sample from all levels. */
	void InterpolatePoint(
		FScene* Scene, 
		const FIndirectLightingCacheBlock& Block,
		float& OutDirectionalShadowing, 
		FSHVectorRGB3& OutIncidentRadiance,
		FVector& OutSkyBentNormal);

	/** Interpolates SH samples for a block from all levels. */
	void InterpolateBlock(
		FScene* Scene, 
		const FIndirectLightingCacheBlock& Block, 
		TArray<float>& AccumulatedWeight, 
		TArray<FSHVectorRGB2>& AccumulatedIncidentRadiance);

	/** 
	 * Normalizes, adjusts for SH ringing, and encodes SH samples into a texture format.
	 * @param DebugDrawingView can be 0
	 */
	void EncodeBlock(
		FViewInfo* DebugDrawingView,
		const FIndirectLightingCacheBlock& Block, 
		const TArray<float>& AccumulatedWeight, 
		const TArray<FSHVectorRGB2>& AccumulatedIncidentRadiance,
		TArray<FFloat16Color>& Texture0Data,
		TArray<FFloat16Color>& Texture1Data,
		TArray<FFloat16Color>& Texture2Data		
	);

	/** Helper that calculates an effective world position min and size given a bounds. */
	void CalculateBlockPositionAndSize(const FBoxSphereBounds& Bounds, int32 TexelSize, FVector& OutMin, FVector& OutSize) const;

	/** Helper that calculates a scale and add to convert world space position into volume texture UVs for a given block. */
	void CalculateBlockScaleAndAdd(FIntVector InTexelMin, int32 AllocationTexelSize, FVector InMin, FVector InSize, FVector& OutScale, FVector& OutAdd, FVector& OutMinUV, FVector& OutMaxUV) const;

	/** true: next rendering we update all entries no matter if they are visible to avoid further hitches */
	bool bUpdateAllCacheEntries;

	/** Size of the volume texture cache. */
	int32 CacheSize;

	/** Volume textures that store SH indirect lighting, interpolated from Lightmass volume samples. */
	TRefCountPtr<IPooledRenderTarget> Texture0;
	TRefCountPtr<IPooledRenderTarget> Texture1;
	TRefCountPtr<IPooledRenderTarget> Texture2;

	/** Tracks the allocation state of the atlas. */
	TMap<FIntVector, FIndirectLightingCacheBlock> VolumeBlocks;

	/** Tracks used sections of the volume texture atlas. */
	FTextureLayout3d BlockAllocator;

	int32 NextPointId;

	/** Tracks primitive allocations by component, so that they persist across re-registers. */
	TMap<FPrimitiveComponentId, FIndirectLightingCacheAllocation*> PrimitiveAllocations;

	friend class FUpdateCachePrimitivesTask;
};

/**
 * Bounding information used to cull primitives in the scene.
 */
struct FPrimitiveBounds
{
	FBoxSphereBounds BoxSphereBounds;
	/** Square of the minimum draw distance for the primitive. */
	float MinDrawDistance;
	/** Maximum draw distance for the primitive. */
	float MaxDrawDistance;
	/** Maximum cull distance for the primitive. This is only different from the MaxDrawDistance for HLOD.*/
	float MaxCullDistance;
};

/**
 * Precomputed primitive visibility ID.
 */
struct FPrimitiveVisibilityId
{
	/** Index in to the byte where precomputed occlusion data is stored. */
	int32 ByteIndex;
	/** Mast of the bit where precomputed occlusion data is stored. */
	uint8 BitMask;
};

/**
 * Flags that affect how primitives are occlusion culled.
 */
namespace EOcclusionFlags
{
	enum Type
	{
		/** No flags. */
		None = 0x0,
		/** Indicates the primitive can be occluded. */
		CanBeOccluded = 0x1,
		/** Allow the primitive to be batched with others to determine occlusion. */
		AllowApproximateOcclusion = 0x4,
		/** Indicates the primitive has a valid ID for precomputed visibility. */
		HasPrecomputedVisibility = 0x8,
		/** Indicates the primitive supports subprimitive occlusion queries. */
		HasSubprimitiveQueries = 0x10,
		/** Indicates that the primitive is forced hidden and does not need to be occlusion tested. */
		IsForceHidden = 0x20,
	};
};

/** Velocity state for a single component. */
class FComponentVelocityData
{
public:

	FPrimitiveSceneInfo* PrimitiveSceneInfo;
	FMatrix LocalToWorld;
	FMatrix PreviousLocalToWorld;
	mutable uint64 LastFrameUsed;
	uint64 LastFrameUpdated;
	bool bPreviousLocalToWorldValid = false;
};

/**
 * Tracks primitive transforms so they will be persistent across rendering state recreates.
 */
class FSceneVelocityData
{
public:

	/**
	 * Must be called once per frame, even when there are multiple EndDrawingViewports.
	 */
	void StartFrame(FScene* Scene);

	/** 
	 * Looks up the PreviousLocalToWorld state for the given component.  Returns false if none is found (the primitive has never been moved). 
	 */
	inline bool GetComponentPreviousLocalToWorld(FPrimitiveComponentId PrimitiveComponentId, FMatrix& OutPreviousLocalToWorld) const
	{
		const FComponentVelocityData* VelocityData = ComponentData.Find(PrimitiveComponentId);

		if (VelocityData && VelocityData->PrimitiveSceneInfo)
		{
			check(VelocityData->bPreviousLocalToWorldValid);
			VelocityData->LastFrameUsed = InternalFrameIndex;
			OutPreviousLocalToWorld = VelocityData->PreviousLocalToWorld;
			return true;
		}

		return false;
	}

	/** 
	 * Looks up the PreviousLocalToWorld state for the given component.  
	 * Returns null if none is found (the primitive has never been moved). 
	 */
	const FMatrix* RESTRICT GetComponentPreviousLocalToWorld(FPrimitiveComponentId PrimitiveComponentId) const
	{
		const FComponentVelocityData* VelocityData = ComponentData.Find(PrimitiveComponentId);

		if (VelocityData && VelocityData->PrimitiveSceneInfo)
		{
			check(VelocityData->bPreviousLocalToWorldValid);
			VelocityData->LastFrameUsed = InternalFrameIndex;
			return &VelocityData->PreviousLocalToWorld;
		}

		return nullptr;
	}

	/** 
	 * Updates a primitives current LocalToWorld state.
	 */
	void UpdateTransform(FPrimitiveSceneInfo* PrimitiveSceneInfo, const FMatrix& LocalToWorld, const FMatrix& PreviousLocalToWorld)
	{
		check(PrimitiveSceneInfo->Proxy->HasDynamicTransform());

		FComponentVelocityData& VelocityData = ComponentData.FindOrAdd(PrimitiveSceneInfo->PrimitiveComponentId);
		VelocityData.LocalToWorld = LocalToWorld;
		VelocityData.LastFrameUsed = InternalFrameIndex;
		VelocityData.LastFrameUpdated = InternalFrameIndex;
		VelocityData.PrimitiveSceneInfo = PrimitiveSceneInfo;

		// If this transform state is newly added, use the passed in PreviousLocalToWorld for this frame
		if (!VelocityData.bPreviousLocalToWorldValid)
		{
			VelocityData.PreviousLocalToWorld = PreviousLocalToWorld;
			VelocityData.bPreviousLocalToWorldValid = true;
		}
	}

	void RemoveFromScene(FPrimitiveComponentId PrimitiveComponentId, bool bImmediate)
	{
		if (bImmediate)
		{
			ComponentData.Remove(PrimitiveComponentId);
		}
		else
		{
			FComponentVelocityData* VelocityData = ComponentData.Find(PrimitiveComponentId);

			if (VelocityData)
			{
				VelocityData->PrimitiveSceneInfo = nullptr;
			}
		}
	}

	/** 
	 * Overrides a primitive's previous LocalToWorld matrix for this frame only
	 */
	void OverridePreviousTransform(FPrimitiveComponentId PrimitiveComponentId, const FMatrix& PreviousLocalToWorld)
	{
		FComponentVelocityData* VelocityData = ComponentData.Find(PrimitiveComponentId);
		if (VelocityData)
		{
			VelocityData->PreviousLocalToWorld = PreviousLocalToWorld;
			VelocityData->bPreviousLocalToWorldValid = true;
		}
	}

	void ApplyOffset(FVector Offset)
	{
		for (TMap<FPrimitiveComponentId, FComponentVelocityData>::TIterator It(ComponentData); It; ++It)
		{
			FComponentVelocityData& VelocityData = It.Value();
			VelocityData.LocalToWorld.SetOrigin(VelocityData.LocalToWorld.GetOrigin() + Offset);
			VelocityData.PreviousLocalToWorld.SetOrigin(VelocityData.PreviousLocalToWorld.GetOrigin() + Offset);
		}
	}

private:

	uint64 InternalFrameIndex = 0;
	TMap<FPrimitiveComponentId, FComponentVelocityData> ComponentData;
};

class FLODSceneTree
{
public:
	FLODSceneTree(FScene* InScene)
		: Scene(InScene)
	{
	}

	/** Information about the primitives that are attached together. */
	struct FLODSceneNode
	{
		/** Children scene infos. */
		TArray<FPrimitiveSceneInfo*> ChildrenSceneInfos;

		/** The primitive. */
		FPrimitiveSceneInfo* SceneInfo;

		FLODSceneNode()
			: SceneInfo(nullptr)
		{
		}

		void AddChild(FPrimitiveSceneInfo* NewChild)
		{
			if (NewChild)
			{
				ChildrenSceneInfos.AddUnique(NewChild);
			}
		}

		void RemoveChild(FPrimitiveSceneInfo* ChildToDelete)
		{
			if (ChildToDelete)
			{
				ChildrenSceneInfos.Remove(ChildToDelete);
			}
		}
	};

	void AddChildNode(FPrimitiveComponentId ParentId, FPrimitiveSceneInfo* ChildSceneInfo);
	void RemoveChildNode(FPrimitiveComponentId ParentId, FPrimitiveSceneInfo* ChildSceneInfo);

	void UpdateNodeSceneInfo(FPrimitiveComponentId NodeId, FPrimitiveSceneInfo* SceneInfo);
	void UpdateVisibilityStates(FViewInfo& View, UE::Tasks::FTaskEvent& FlushCachedShadowsTaskEvent);

	void ClearVisibilityState(FViewInfo& View);

	bool IsActive() const { return (SceneNodes.Num() > 0); }

private:

	/** Scene this Tree belong to */
	FScene* Scene;

	/** The LOD groups in the scene.  The map key is the current primitive who has children. */
	TMap<FPrimitiveComponentId, FLODSceneNode> SceneNodes;

	/** Recursive state updates */
	void ApplyNodeFadingToChildren(FSceneViewState* ViewState, FLODSceneNode& Node, FHLODSceneNodeVisibilityState& NodeVisibility, const bool bIsFading, const bool bIsFadingOut);
	void HideNodeChildren(FSceneViewState* ViewState, FLODSceneNode& Node);
};

#if WITH_EDITOR
class FPixelInspectorData
{
public:
	FPixelInspectorData();

	void InitializeBuffers(FRenderTarget* BufferFinalColor, FRenderTarget* BufferSceneColor, FRenderTarget* BufferDepth, FRenderTarget* BufferHDR, FRenderTarget* BufferA, FRenderTarget* BufferBCDEF, int32 bufferIndex);

	bool AddPixelInspectorRequest(FPixelInspectorRequest *PixelInspectorRequest);

	//Hold the buffer array
	TMap<FVector2f, FPixelInspectorRequest *> Requests;

	FRenderTarget* RenderTargetBufferDepth[2];
	FRenderTarget* RenderTargetBufferFinalColor[2];
	FRenderTarget* RenderTargetBufferHDR[2];
	FRenderTarget* RenderTargetBufferSceneColor[2];
	FRenderTarget* RenderTargetBufferA[2];
	FRenderTarget* RenderTargetBufferBCDEF[2];
};

// Reserved values for the stencil buffer that carry specific meaning
namespace EEditorSelectionStencilValues
{
	enum Type : int32
	{
		NotSelected = 0,
		BSP = 1, // The outlines of all BSPs should be merged
		VisualizeLevelInstances = 2,
		Nanite = 3, // Nanite can use only a single value

		COUNT,
	};
}

#endif //WITH_EDITOR

class FPersistentUniformBuffers
{
public:
	FPersistentUniformBuffers() = default;

	void Initialize();
	void Clear();


	/** Mobile Directional Lighting uniform buffers, one for each lighting channel 
	  * The first is used for primitives with no lighting channels set.
	  */
	TUniformBufferRef<FMobileDirectionalLightShaderParameters> MobileDirectionalLightUniformBuffers[NUM_LIGHTING_CHANNELS+1];
	TUniformBufferRef<FMobileReflectionCaptureShaderParameters> MobileSkyReflectionUniformBuffer;
};

#if RHI_RAYTRACING

enum class ERayTracingType : uint8;

#endif

struct FLumenSceneDataKey
{
	uint32 ViewUniqueId;		// Zero if not view specific
	uint32 GPUIndex;			// INDEX_NONE if not GPU specific

	friend FORCEINLINE bool operator == (const FLumenSceneDataKey& A, const FLumenSceneDataKey& B)
	{
		return A.ViewUniqueId == B.ViewUniqueId && A.GPUIndex == B.GPUIndex;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FLumenSceneDataKey& Key)
	{
		return HashCombine(GetTypeHash(Key.ViewUniqueId), GetTypeHash(Key.GPUIndex));
	}
};

typedef TMap<FLumenSceneDataKey, FLumenSceneData*> FLumenSceneDataMap;

class FLumenSceneDataIterator
{
public:
	FLumenSceneDataIterator(const FScene* InScene);
	FLumenSceneDataIterator& operator++();

	FORCEINLINE explicit operator bool() const { return LumenSceneData != nullptr; }
	FORCEINLINE bool operator !() const { return LumenSceneData == nullptr; }
	FORCEINLINE FLumenSceneData* operator->() const { return LumenSceneData; }
	FORCEINLINE FLumenSceneData& operator*() const { return *LumenSceneData; }

private:
	const FScene* Scene;
	FLumenSceneData* LumenSceneData;
	FLumenSceneDataMap::TConstIterator NextSceneData;
};

/**
 * Definitions for light scene updates.
 */

enum class ELightDirtyFlags : uint32 
{
	None			=	0u
};
ENUM_CLASS_FLAGS(ELightDirtyFlags);

enum class ELightUpdateId : uint32 
{
	Invalidate,
	Transform,
	Color,
	MAX
};

using FSceneLightInfoUpdates = TSceneUpdateCommandQueue<FLightSceneInfo, ELightDirtyFlags, ELightUpdateId>;
using FUpdateLightCommand = FSceneLightInfoUpdates::FUpdateCommand;

using FUpdateLightInvalidateGPUState = FSceneLightInfoUpdates::TPayloadBase<ELightUpdateId::Invalidate, ELightDirtyFlags::None>;

struct FUpdateLightTransformParameters : public FSceneLightInfoUpdates::TPayloadBase<ELightUpdateId::Transform, ELightDirtyFlags::None>
{
	FMatrix LightToWorld;
	FVector4 Position;
};

struct FUpdateLightColorParameters: public FSceneLightInfoUpdates::TPayloadBase<ELightUpdateId::Color, ELightDirtyFlags::None>
{
	FLinearColor NewColor;
	float NewIndirectLightingScale;
	float NewVolumetricScatteringIntensity;
};

/**
 * Describes all light modifications to the scene by recording the light scene IDs.
 */
struct FLightSceneChangeSet
{
	// IDs of all lights before they were removed, IDs in this array may not be valid at all times when the change-set is used (depends on whether the callback site is before or after the given lights are removed from the scene).
	TConstArrayView<FLightSceneId> RemovedLightIds;
	// IDs of all lights added to the scene, only available after all lights are added to the scene, may contain the same ID's as removed, as they may be reused.
	TConstArrayView<FLightSceneId> AddedLightIds;
	FSceneLightInfoUpdates& SceneLightInfoUpdates;
	// Bit-array of at least size PreUpdateMaxIndex, with a bit set for each removed light
	const FSceneRendererLightBitArray& RemovedLightsMask;
	int32 PreUpdateMaxIndex = 0;
	int32 PostUpdateMaxIndex = 0;
};

struct FViewSceneChangeSet
{
	TArray<FPersistentViewId, SceneRenderingAllocator> AddedViewIds;
	TArray<FPersistentViewId> RemovedViewIds;
	FGameTime CurrentTime;
	bool bIsRequiresDebugMaterialChanged = false;
};


/** 
 * Renderer scene which is private to the renderer module.
 * Ordinarily this is the renderer version of a UWorld, but an FScene can be created for previewing in editors which don't have a UWorld as well.
 * The scene stores renderer state that is independent of any view or frame, with the primary actions being adding and removing of primitives and lights.
 */
class FScene : public FSceneInterface
{
public:

	/** An optional world associated with the scene. */
	UWorld* World;

	/** An optional FX system associated with the scene. */
	class FFXSystemInterface* FXSystem;

	/** List of view states associated with the scene. */
	TArray<FSceneViewState*> ViewStates;

	/**
	 * Unique IDs of view states that are tracked by the (ever been rendered & not deleted), indexed by their persistent ID. 
	 * The sparse array defines a semi-compact range that can be used to maintain data for each view state in e.g., scene extensions.
	 */
	TSparseArray<int32> PersistentViewStateUniqueIDs;

	/** 
	 * Get the upper bound on the persistent view index. 
	 */
	int32 GetMaxPersistentViewId() const { return PersistentViewStateUniqueIDs.GetMaxIndex(); }

	FPersistentUniformBuffers UniformBuffers;

	/** Instancing state buckets.  These are stored on the scene as they are precomputed at FPrimitiveSceneInfo::AddToScene time. */
	FStateBucketMap CachedMeshDrawCommandStateBuckets[EMeshPass::Num];
	FCachedPassMeshDrawList CachedDrawLists[EMeshPass::Num];

#if RHI_RAYTRACING
	/** Force a refresh of all cached ray tracing data in the scene (when path tracing mode changes or coarse mesh streaming for example). */
	void RefreshCachedRayTracingData();

	/** Release unused persistent ray tracing SBTs at end of frame */
	void ReleaseUnusedPersistentSBTs();
#endif

	/** Nanite shading material commands. These are stored on the scene as they are computed at FPrimitiveSceneInfo::AddToScene time. */
	FNaniteShadingCommands NaniteShadingCommands[ENaniteMeshPass::Num];

	/** Nanite raster and shading pipelines. These are stored on the scene as they are computed at FPrimitiveSceneInfo::AddToScene time. */
	FNaniteRasterPipelines  NaniteRasterPipelines[ENaniteMeshPass::Num];
	FNaniteShadingPipelines NaniteShadingPipelines[ENaniteMeshPass::Num];

	/** Nanite material visibility references. These are stored on the scene as they are computed at FPrimitiveSceneInfo::AddToScene time. */
	FNaniteVisibility NaniteVisibility[ENaniteMeshPass::Num];

	struct FPrimitiveUpdateParams
	{
		FScene* Scene;
		FPrimitiveSceneProxy* PrimitiveSceneProxy;
		FBoxSphereBounds WorldBounds;
		FBoxSphereBounds LocalBounds;
		FMatrix LocalToWorld;
		TOptional<FTransform> PreviousTransform;
		FVector AttachmentRootPosition;
	};

	struct FPrimitiveTransformUpdater : public IPrimitiveTransformUpdater
	{
		std::atomic_int32_t Index = 0;
		TArray<FPrimitiveUpdateParams> Params;
	};

	/**
	 * The following arrays are densely packed primitive data needed by various
	 * rendering passes. PrimitiveSceneInfo->PackedIndex maintains the index
	 * where data is stored in these arrays for a given primitive.
	 */

	/** Index into primitive arrays where the always visible partition starts. */
	uint32 PrimitivesAlwaysVisibleOffset = ~0u;

	/** Packed array of primitives in the scene. */
	TArray<FPrimitiveSceneInfo*> Primitives;
	/** Packed array of all transforms in the scene. */
	TScenePrimitiveArray<FMatrix> PrimitiveTransforms;
	/** Packed array of primitive scene proxies in the scene. */
	TArray<FPrimitiveSceneProxy*> PrimitiveSceneProxies;
	/** Packed array of primitive bounds. */
	TScenePrimitiveArray<FPrimitiveBounds> PrimitiveBounds;
	/** Packed array of primitive flags. */
	TArray<FPrimitiveFlagsCompact> PrimitiveFlagsCompact;
	/** Packed array of precomputed primitive visibility IDs. */
	TArray<FPrimitiveVisibilityId> PrimitiveVisibilityIds;
	/**Array of primitive octree node index**/
	TArray<uint32> PrimitiveOctreeIndex;
	/** Packed array of primitive occlusion flags. See EOcclusionFlags. */
	TArray<uint8> PrimitiveOcclusionFlags;
	/** Packed array of primitive occlusion bounds. */
	TScenePrimitiveArray<FBoxSphereBounds> PrimitiveOcclusionBounds;
	/** Packed array of primitive components associated with the primitive. */
	TArray<FPrimitiveComponentId> PrimitiveComponentIds;
	/** Map from FPrimitiveComponentId to FPrimitiveSceneInfo*. */
	TMap<FPrimitiveComponentId, FPrimitiveSceneInfo*> PrimitiveComponentIdToInfoMap;
#if RHI_RAYTRACING
	/** Packed array of ray tracing primitive caching flags*/
	TArray<ERayTracingPrimitiveFlags> PrimitiveRayTracingFlags;
	/** Packed array of primitive ray tracing data - fits in 8 bytes */
	struct FPrimitiveRayTracingData
	{
		bool bDrawInGame : 1;
		bool bRayTracingFarField : 1;
		bool bRetainWhileHidden : 1;
		bool bIsVisibleInSceneCaptures : 1;
		bool bIsVisibleInSceneCapturesOnly : 1;
		uint8 LightingChannelMask : 3;
		bool bCachedRaytracingDataDirty : 1 = true;
		ERayTracingProxyType ProxyGeometryType : 7;
		RayTracing::FGeometryGroupHandle RayTracingGeometryGroupHandle;
	};
	static_assert(sizeof(FPrimitiveRayTracingData) == 8, "Increasing the size of FPrimitiveRayTracingData will negatively affect performance of ray tracing instance gathering.");
	TArray<FPrimitiveRayTracingData> PrimitiveRayTracingDatas;
	/** Packed array of ray tracing primitive group id hash indices. */
	TArray<Experimental::FHashElementId> PrimitiveRayTracingGroupIds;
	/** Aggregate bounds for all primitives which share a ray tracing group id. */
	struct FRayTracingCullingGroup
	{
		FBoxSphereBounds Bounds;
		float MinDrawDistance = 0.0f;
		TArray<FPrimitiveSceneInfo*> Primitives;
	};
	Experimental::TRobinHoodHashMap<int32, FRayTracingCullingGroup> PrimitiveRayTracingGroups;
#endif

	TMap<FName, TArray<FPrimitiveSceneInfo*>> PrimitivesNeedingLevelUpdateNotification;

#if WITH_EDITOR
	/** Packed bit array of primitives which are selected in the editor. */
	TBitArray<> PrimitivesSelected;
#endif

	TBitArray<> PrimitivesNeedingStaticMeshUpdate;
	TBitArray<> PrimitivesNeedingUniformBufferUpdate;

	TArray<int32> PersistentPrimitiveIdToIndexMap;

	/**
	 * Defines a bucket "type" in the sorted order of the primitive arrays, as defined by the type-offset table.
	 */
	struct FPrimitiveSceneProxyType
	{
		FPrimitiveSceneProxyType(const FPrimitiveSceneProxy *PrimitiveSceneProxy);
		bool operator ==(const FPrimitiveSceneProxyType&) const = default;

		SIZE_T ProxyTypeHash; 
		bool bIsAlwaysVisible;
	};

	struct FTypeOffsetTableEntry
	{
		FTypeOffsetTableEntry(const FPrimitiveSceneProxyType &InPrimitiveSceneProxyType, uint32 InOffset) : PrimitiveSceneProxyType(InPrimitiveSceneProxyType), Offset(InOffset) {}
		FPrimitiveSceneProxyType PrimitiveSceneProxyType;
		uint32 Offset; //(e.g. prefix sum where the next type starts)
	};
	/* During insertion and deletion, used to skip large chunks of items of the same type */
	TArray<FTypeOffsetTableEntry> TypeOffsetTable;

	/** The lights in the scene. */
	FSceneLightInfoArray Lights;

	/** 
	 * Lights in the scene which are invisible, but still needed by the editor for previewing. 
	 * Lights in this array cannot be in the Lights array.  They also are not fully set up, as AddLightSceneInfo_RenderThread is not called for them.
	 */
	FSceneLightInfoArray InvisibleLights;

	/** Shadow casting lights that couldn't get a shadowmap channel assigned and therefore won't have valid dynamic shadows, forward renderer only. */
	TArray<FString> OverflowingDynamicShadowedLights;

	/** Early Z pass mode. */
	EDepthDrawingMode EarlyZPassMode;

	/** Early Z pass movable. */
	bool bEarlyZPassMovable;

	/** Default base pass depth stencil access. */
	FExclusiveDepthStencil::Type DefaultBasePassDepthStencilAccess;

	/** Default base pass depth stencil access used to cache mesh draw commands. */
	FExclusiveDepthStencil::Type CachedDefaultBasePassDepthStencilAccess;

	/** Previous frame SkyLight state. */
	bool bCachedShouldRenderSkylightInBasePass;

	/** Previous frame was using skylight realtime capture or not */
	bool bCachedSkyLightRealTimeCapture;

	/** True if a change to SkyLight / Lighting has occurred that requires static draw lists to be updated. */
	bool bScenesPrimitivesNeedStaticMeshElementUpdate;

	/** This counter will be incremented anytime something about the scene changed that should invalidate the path traced accumulation buffers. */
	TAtomic<uint32> PathTracingInvalidationCounter;

#if RHI_RAYTRACING
	/** What mode was the cached RT commands prepared for last? */
	ERayTracingType CachedRayTracingMeshCommandsType;
#endif

	/** The scene's sky light, if any. */
	FSkyLightSceneProxy* SkyLight;

	/** Contains the sky env map irradiance as spherical harmonics. */
	TRefCountPtr<FRDGPooledBuffer> SkyIrradianceEnvironmentMap;

	/** The SkyView LUT used when rendering sky material sampling this lut into the realtime capture sky env map. It must be generated at the skylight position*/
	TRefCountPtr<IPooledRenderTarget> RealTimeReflectionCaptureSkyAtmosphereViewLutTexture;

	/** The Camera 360 AP is used when rendering sky material sampling this lut or volumetric clouds into the realtime capture sky env map. It must be generated at the skylight position*/
	TRefCountPtr<IPooledRenderTarget> RealTimeReflectionCaptureCamera360APLutTexture;

	/** If sky light bRealTimeCaptureEnabled is true, used to render the sky env map (sky, sky dome mesh or clouds). */
	TRefCountPtr<IPooledRenderTarget> CapturedSkyRenderTarget;	// Needs to be a IPooledRenderTarget because it must be created before the View uniform buffer is created.

	/** These store the result of the sky env map GGX specular convolution. */
	TRefCountPtr<IPooledRenderTarget> ConvolvedSkyRenderTarget[2];

	/** The index of the ConvolvedSkyRenderTarget to use when rendering meshes. -1 when not initialised. */
	int32 ConvolvedSkyRenderTargetReadyIndex;

	/** Return true if the real time sky light captured data is ready to be sampled */
	bool CanSampleSkyLightRealTimeCaptureData() const;

	struct FRealTimeSlicedReflectionCapture
	{
		/** We always enforce a complete one the first frame even with time slicing for correct start up lighting.*/
		enum class EFirstFrameState
		{
			INIT = 0,
			FIRST_FRAME = 1,
			BEYOND_FIRST_FRAME = 2,
		} FirstFrameState = EFirstFrameState::INIT;

		/** The current progress of the real time reflection capture when time sliced. */
		int32 State = -1;

		/** The current progress of each sub step of a state capture of the real time reflection capture when time sliced. */
		int32 StateSubStep = 0;

		/** Keeps track of which GPUs have been initialized with a full cube map */
		uint32 GpusWithFullCube = 0;

		/** Keeps track of which GPUs calculations have been done in the frame */
		uint32 GpusHandledThisFrame = 0;

		/** Cache of Frame Number, used to detect first viewfamily */
		uint64 FrameNumber = uint64(-1);
	} RealTimeSlicedReflectionCapture;

	/**
	 * The path tracer uses its own representation of the skylight. These textures
	 * are updated lazily by the path tracer when missing. Any code that modifies
	 * the skylight appearance should simply reset these pointers.
	 * 
	 * We also remember the last used color so we can detect changes that would require rebuilding the tables
	 */
	TRefCountPtr<IPooledRenderTarget> PathTracingSkylightTexture;
	TRefCountPtr<IPooledRenderTarget> PathTracingSkylightPdf;
	FLinearColor PathTracingSkylightColor;

	/** Used to track the order that skylights were enabled in. */
	TArray<FSkyLightSceneProxy*> SkyLightStack;

	/** Mobile renderer: used to readback the sky irradiance in order to set it onto the view constant UB */
	TQueue<TSharedPtr<FRHIGPUBufferReadback>> MobileSkyLightRealTimeCaptureIrradianceReadBackQueries;
	/** Mobile renderer: store a ready-to-copy to view UB sky irradiance environment map */
	FVector4f MobileSkyLightRealTimeCaptureIrradianceEnvironmentMap[SKY_IRRADIANCE_ENVIRONMENT_MAP_VEC4_COUNT];

	/** The directional light to use for simple dynamic lighting, if any. */
	FLightSceneInfo* SimpleDirectionalLight;

	/** For the mobile renderer, the first directional light in each lighting channel. */
	FLightSceneInfo* MobileDirectionalLights[NUM_LIGHTING_CHANNELS];

	/** The light sources for atmospheric effects, if any. */
	FLightSceneInfo* AtmosphereLights[NUM_ATMOSPHERE_LIGHTS];

	TArray<FLightSceneInfo*, TInlineAllocator<4>> DirectionalLights;

	/** The decals in the scene. */
	TArray<FDeferredDecalProxy*> Decals;

	/** Potential capsule shadow casters registered to the scene. */
	TArray<FPrimitiveSceneInfo*> DynamicIndirectCasterPrimitives; 

	TArray<class FPlanarReflectionSceneProxy*> PlanarReflections;
	TArray<class UPlanarReflectionComponent*> PlanarReflections_GameThread;

	/** State needed for the reflection environment feature. */
	FReflectionEnvironmentSceneData ReflectionSceneData;

	/** The hair strands in the scene. */
	FHairStrandsSceneData HairStrandsSceneData;

	/** The OIT resources in the scene. */
	FOITSceneData OITSceneData;

	/** 
	 * Precomputed lighting volumes in the scene, used for interpolating dynamic object lighting.
	 * These are typically one per streaming level and they store volume lighting samples computed by Lightmass. 
	 */
	TArray<const FPrecomputedLightVolume*> PrecomputedLightVolumes;

	/** Interpolates and caches indirect lighting for dynamic objects. */
	FIndirectLightingCache IndirectLightingCache;

	FVolumetricLightmapSceneData VolumetricLightmapSceneData;
	
	FGPUScene GPUScene;

#if RHI_RAYTRACING
	/** Persistently-allocated ray tracing scene data. */
	FRayTracingScene RayTracingScene;
	FRayTracingScene HeterogeneousVolumesRayTracingScene;

	FRayTracingMeshCommandStorage CachedRayTracingMeshCommands;
	
	// Last valid RTPSO is saved, so it could be used as fallback in future frames if background PSO compilation is enabled.
	// This RTPSO can be used only if the only difference from previous PSO is the material hit shaders.
	FRayTracingPipelineStateSignature LastRayTracingMaterialPipelineSignature;

    bool bLastFrameUsedFallback = false;
	
	/** SBT object to be used with hardware ray tracing */
	FRayTracingShaderBindingTable RayTracingSBT;

	/** State tracking for persistent ray tracing SBTs to support multi-viewfamily scenarios */
	struct FRayTracingPersistentSBTState
	{
		struct FTrackedSBT
		{
			FRayTracingPersistentShaderBindingTableID ID = INDEX_NONE;
			bool bUsedThisFrame = false;
			FRayTracingPipelineState* LastValidPipelineState = nullptr;  // Cached fallback pipeline state from last ViewFamily that created it

			/** Resets the state, call after associated SBT is released. */
			void Reset()
			{
				ID = INDEX_NONE;
				bUsedThisFrame = false;
				LastValidPipelineState = nullptr;
			}
		};

		FTrackedSBT MaterialSBT;
		FTrackedSBT PathTracingSBT;
		FTrackedSBT LumenSBT;
		FTrackedSBT InlineSBT;
	};

	FRayTracingPersistentSBTState RayTracingPersistentSBTState;
#endif // RHI_RAYTRACING

	/** Distance field object scene data. */
	FDistanceFieldSceneData DistanceFieldSceneData;

	FLumenSceneData* DefaultLumenSceneData;
	FLumenSceneDataMap PerViewOrGPULumenSceneData;

	/** Maps regular landscape component IDs to their Nanite proxies. */
	TMap<FPrimitiveComponentId, FPrimitiveSceneInfo*> LandscapeToNaniteProxyMap;

	/** Atlas HZB textures from the previous render. */
	TArray<TRefCountPtr<IPooledRenderTarget>>	PrevAtlasHZBs;
	TArray<TRefCountPtr<IPooledRenderTarget>>	PrevAtlasCompleteHZBs;

	TRefCountPtr<IPooledRenderTarget> PreShadowCacheDepthZ;

	/** Preshadows that are currently cached in the PreshadowCache render target. */
	TArray<TRefCountPtr<FProjectedShadowInfo> > CachedPreshadows;

	/** Texture layout that tracks current allocations in the PreshadowCache render target. */
	FTextureLayout PreshadowCacheLayout;

	/** The static meshes in the scene. */
	TSparseArray<FStaticMeshBatch*> StaticMeshes;

	/** The exponential fog components in the scene. */
	TArray<FExponentialHeightFogSceneInfo> ExponentialFogs;

	/** The sky/atmosphere components of the scene. */
	FSkyAtmosphereRenderSceneInfo* SkyAtmosphere;

	/** A value representing the state of the SkyAtmosphere to know if we need to update some psecific LUTs */
	uint32 CachedTransmittanceAndMultiScatteringLUTsVersion;

	/** Used to track the order that skylights were enabled in. */
	TArray<FSkyAtmosphereSceneProxy*> SkyAtmosphereStack;

	/** The sky/atmosphere components of the scene. */
	FVolumetricCloudRenderSceneInfo* VolumetricCloud;

	/** Used to track the order that skylights were enabled in. */
	TArray<FVolumetricCloudSceneProxy*> VolumetricCloudStack;

	TArray<FLocalFogVolumeSceneProxy*> LocalFogVolumes;

	TArray<FSparseVolumeTextureViewerSceneProxy*> SparseVolumeTextureViewers;

	/** Global Field Manager */
	class FPhysicsFieldSceneProxy* PhysicsField = nullptr;

	/** The wind sources in the scene. */
	TArray<class FWindSourceSceneProxy*> WindSources;

	/** Wind source components, tracked so the game thread can also access wind parameters */
	TArray<UWindDirectionalSourceComponent*> WindComponents_GameThread;

	/** SpeedTree wind objects in the scene. FLocalVertexFactoryShaderParametersBase needs to lookup by FVertexFactory, but wind objects are per tree (i.e. per UStaticMesh)*/
	TMap<const UStaticMesh*, struct FSpeedTreeWindComputation*> SpeedTreeWindComputationMap;
	TMap<FVertexFactory*, const UStaticMesh*> SpeedTreeVertexFactoryMap;

	/** The attachment groups in the scene.  The map key is the attachment group's root primitive. */
	TMap<FPrimitiveComponentId,FAttachmentGroupSceneInfo> AttachmentGroups;

	/** Precomputed visibility data for the scene. */
	const FPrecomputedVisibilityHandler* PrecomputedVisibilityHandler;

	/** An octree containing the shadow-casting local lights in the scene. */
	FSceneLightOctree LocalShadowCastingLightOctree;
	/** An array containing IDs of shadow-casting directional lights in the scene. */
	TArray<FLightSceneId> DirectionalShadowCastingLightIDs;

	/** An octree containing the primitives in the scene. */
	FScenePrimitiveOctree PrimitiveOctree;

	/** Indicates whether this scene requires hit proxy rendering. */
	bool bRequiresHitProxies;

	/** Whether this is an editor scene. */
	bool bIsEditorScene;

	/** Determines whether primitives that draw to a runtime virtual texture should also be drawn in the main pass. */
	bool bRuntimeVirtualTexturePrimitiveHideEditor;
	bool bRuntimeVirtualTexturePrimitiveHideGame;

	/** Set by the rendering thread to signal to the game thread that the scene needs a static lighting build. */
	volatile mutable int32 NumUncachedStaticLightingInteractions;

	volatile mutable int32 NumUnbuiltReflectionCaptures;

	/** Track numbers of various lights types on mobile, used to show warnings for disabled shader permutations. */
	int32 NumMobileStaticAndCSMLights_RenderThread;
	int32 NumMobileMovableDirectionalLights_RenderThread;

	FSceneVelocityData VelocityData;

	/** GPU Skinning cache, if enabled */
	class FGPUSkinCache* GPUSkinCache;
	class FSkeletalMeshUpdater* SkeletalMeshUpdater;

	/* Array of registered compute work schedulers*/
	TArray<class IComputeTaskWorker*> ComputeTaskWorkers;

	/** Uniform buffers for parameter collections with the corresponding Ids. */
	TMap<FGuid, FUniformBufferRHIRef> ParameterCollections;

	/** LOD Tree Holder for massive LOD system */
	FLODSceneTree SceneLODHierarchy;

	/** The runtime virtual textures in the scene. */
	TSparseArray<FRuntimeVirtualTextureSceneProxy*> RuntimeVirtualTextures;

	LightFunctionAtlas::FLightFunctionAtlasSceneData LightFunctionAtlasSceneData;

	float DefaultMaxDistanceFieldOcclusionDistance;

	float GlobalDistanceFieldViewDistance;

	float DynamicIndirectShadowsSelfShadowingIntensity;

	FSpanAllocator PersistentPrimitiveIdAllocator;

#if WITH_EDITOR
	/** Editor Pixel inspector */
	FPixelInspectorData PixelInspectorData;
#endif //WITH_EDITOR

#if RHI_RAYTRACING
	class FRayTracingDynamicGeometryUpdateManager* RayTracingDynamicGeometryUpdateManager;
#endif

	/** Collection of scene render extensions. */
	FSceneExtensions SceneExtensions;

	/** List of all the custom render passes that will run the next time the scene is rendered. */
	TArray<FCustomRenderPassRendererInput> CustomRenderPassRendererInputs;

	/** Initialization constructor. */
	FScene(UWorld* InWorld, bool bInRequiresHitProxies,bool bInIsEditorScene, bool bCreateFXSystem, ERHIFeatureLevel::Type InFeatureLevel);

	virtual ~FScene();

	FString GetFullWorldName() const { return FullWorldName; }

	using FSceneInterface::UpdateAllPrimitiveSceneInfos;

	struct FUpdateParameters
	{
		FViewSceneChangeSet* ViewUpdateChangeSet = nullptr;
		EUpdateAllPrimitiveSceneInfosAsyncOps AsyncOps = EUpdateAllPrimitiveSceneInfosAsyncOps::None;
		UE::Tasks::FTask GPUSceneUpdateTaskPrerequisites;
		bool bDestruction = false;

		struct
		{
			TFunction<void(const UE::Tasks::FTask&)> PostStaticMeshUpdate;

		} Callbacks;
	};


	/**
	 * Determines added & removed view states based on the rendered views passed in & the RemovedViews.
	 */
	FViewSceneChangeSet* ProcessViewChanges(FRDGBuilder& GraphBuilder, TConstArrayView<FViewInfo*> AllViews);

	void Update(FRDGBuilder& GraphBuilder, const FUpdateParameters& Parameters);

	// FSceneInterface interface.
	virtual void AddPrimitive(UPrimitiveComponent* Primitive) override;
	virtual void RemovePrimitive(UPrimitiveComponent* Primitive) override;
	virtual void ReleasePrimitive(UPrimitiveComponent* Primitive) override;
	virtual void BatchAddPrimitives(TArrayView<UPrimitiveComponent*> InPrimitives) override;
	virtual void BatchRemovePrimitives(TArrayView<UPrimitiveComponent*> InPrimitives) override;
	virtual void BatchReleasePrimitives(TArrayView<UPrimitiveComponent*> InPrimitives) override;
	virtual void UpdateAllPrimitiveSceneInfos(FRDGBuilder& GraphBuilder, EUpdateAllPrimitiveSceneInfosAsyncOps AsyncOps = EUpdateAllPrimitiveSceneInfosAsyncOps::None) override;
	virtual void UpdatePrimitiveTransform(UPrimitiveComponent* Primitive) override;
	virtual void UpdatePrimitiveInstances(UPrimitiveComponent* Primitive) override;
	virtual void UpdatePrimitiveInstancesFromCompute(FPrimitiveSceneDesc* Primitive, FGPUSceneWriteDelegate&& DataWriterGPU) override;
	virtual void UpdatePrimitiveOcclusionBoundsSlack(UPrimitiveComponent* Primitive, float NewSlack) override;
	virtual void UpdatePrimitiveDrawDistance(UPrimitiveComponent* Primitive, float MinDrawDistance, float MaxDrawDistance, float VirtualTextureMaxDrawDistance) override;
	virtual void UpdateInstanceCullDistance(UPrimitiveComponent* Primitive, float StartCullDistance, float EndCullDistance) override;
	virtual void UpdatePrimitiveAttachment(UPrimitiveComponent* Primitive) override;
	virtual void UpdateCustomPrimitiveData(UPrimitiveComponent* Primitive) override;
	virtual void UpdatePrimitiveDistanceFieldSceneData_GameThread(UPrimitiveComponent* Primitive) override;
	virtual FPrimitiveSceneInfo* GetPrimitiveSceneInfo(int32 PrimitiveIndex) const final;
	virtual FPrimitiveSceneInfo* GetPrimitiveSceneInfo(FPrimitiveComponentId PrimitiveId) const final;
	virtual FPrimitiveSceneInfo* GetPrimitiveSceneInfo(const FPersistentPrimitiveIndex& PersistentPrimitiveIndex) const final;
	virtual bool GetPreviousLocalToWorld(const FPrimitiveSceneInfo* PrimitiveSceneInfo, FMatrix& OutPreviousLocalToWorld) const override;
	virtual void AddLight(ULightComponent* Light) override;
	virtual void RemoveLight(ULightComponent* Light) override;
	virtual void AddInvisibleLight(ULightComponent* Light) override;
	virtual void BatchAddLights(TArrayView<FLightSceneDesc> InLights) override;
	virtual void BatchAddInvisibleLights(TArrayView<FLightSceneDesc> InLights) override;
	virtual void BatchRemoveLights(TArrayView<FLightSceneDesc> InLights) override;
	virtual void SetSkyLight(FSkyLightSceneProxy* Light) override;
	virtual void DisableSkyLight(FSkyLightSceneProxy* Light) override;
	virtual bool HasSkyLightRequiringLightingBuild() const override;
	virtual bool HasAtmosphereLightRequiringLightingBuild() const override;
	virtual void AddDecal(UDecalComponent* Component) override;
	virtual void RemoveDecal(UDecalComponent* Component) override;
	virtual void UpdateDecalTransform(UDecalComponent* Decal) override;
	virtual void UpdateDecalFadeOutTime(UDecalComponent* Decal) override;
	virtual void UpdateDecalFadeInTime(UDecalComponent* Decal) override;
	virtual void BatchUpdateDecals(TArray<FDeferredDecalUpdateParams>&& UpdateParams) override;
	virtual void AddReflectionCapture(UReflectionCaptureComponent* Component) override;
	virtual void RemoveReflectionCapture(UReflectionCaptureComponent* Component) override;
	bool ShouldCacheReflectionCaptures() const { return GetShadingPath() == EShadingPath::Mobile || IsForwardShadingEnabled(GetShaderPlatform()); }
	void CreateReflectionCaptureProxy(UReflectionCaptureComponent* Component);
	void DestroyReflectionCaptureProxy(UReflectionCaptureComponent* Component);
	virtual void GetReflectionCaptureData(UReflectionCaptureComponent* Component, class FReflectionCaptureData& OutCaptureData) override;
	virtual void UpdateReflectionCaptureTransform(UReflectionCaptureComponent* Component) override;
	virtual void ReleaseReflectionCubemap(UReflectionCaptureComponent* CaptureComponent) override;
	virtual void AddPlanarReflection(class UPlanarReflectionComponent* Component) override;
	virtual void RemovePlanarReflection(UPlanarReflectionComponent* Component) override;
	virtual void UpdatePlanarReflectionTransform(UPlanarReflectionComponent* Component) override;
	virtual void UpdateSceneCaptureContents(class USceneCaptureComponent2D* CaptureComponent, ISceneRenderBuilder& SceneRenderBuilder) override;
	virtual void UpdateSceneCaptureContents(class USceneCaptureComponentCube* CaptureComponent, ISceneRenderBuilder& SceneRenderBuilder) override;
	virtual void UpdatePlanarReflectionContents(UPlanarReflectionComponent* CaptureComponent, FSceneRenderer& MainSceneRenderer, ISceneRenderBuilder& SceneRenderBuilder) override;
	virtual void AllocateReflectionCaptures(const TArray<UReflectionCaptureComponent*>& NewCaptures, const TCHAR* CaptureReason, bool bVerifyOnlyCapturing, bool bCapturingForMobile, bool bInsideTick) override;
	virtual void ResetReflectionCaptures(bool bOnlyIfOOM) override;
	virtual void UpdateSkyCaptureContents(const USkyLightComponent* CaptureComponent, bool bCaptureEmissiveOnly, UTextureCube* SourceCubemap, FTexture* OutProcessedTexture, float& OutAverageBrightness, FSHVectorRGB3& OutIrradianceEnvironmentMap, TArray<FFloat16Color>* OutRadianceMap, FLinearColor* SpecifiedCubemapColorScale) override;
	virtual void AllocateAndCaptureFrameSkyEnvMap(FRDGBuilder& GraphBuilder, FSceneRenderer& SceneRenderer, FViewInfo& MainView, bool bShouldRenderSkyAtmosphere, bool bShouldRenderVolumetricCloud, FInstanceCullingManager& InstanceCullingManager, FRDGExternalAccessQueue& ExternalAccessQueue) override;
	virtual void ValidateSkyLightRealTimeCapture(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef SceneColorTexture) override;
	virtual void ProcessAndRenderIlluminanceMeter(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> Views, FRDGTextureRef SceneColorTexture);
	virtual void AddPrecomputedLightVolume(const class FPrecomputedLightVolume* Volume) override;
	virtual void RemovePrecomputedLightVolume(const class FPrecomputedLightVolume* Volume) override;
	virtual bool HasPrecomputedVolumetricLightmap_RenderThread() const override;
	virtual void AddPrecomputedVolumetricLightmap(const class FPrecomputedVolumetricLightmap* Volume, bool bIsPersistentLevel) override;
	virtual void RemovePrecomputedVolumetricLightmap(const class FPrecomputedVolumetricLightmap* Volume) override;
	virtual void AddRuntimeVirtualTexture(class URuntimeVirtualTextureComponent* Component) override;
	virtual void RemoveRuntimeVirtualTexture(class URuntimeVirtualTextureComponent* Component) override;
	virtual void GetRuntimeVirtualTextureHidePrimitiveMask(uint8& bHideMaskEditor, uint8& bHideMaskGame) const override;
	virtual void InvalidateRuntimeVirtualTexture(class URuntimeVirtualTextureComponent* Component, FBoxSphereBounds const& WorldBounds, EVTInvalidatePriority InvalidatePriority) override;
	virtual void RequestPreloadRuntimeVirtualTexture(class URuntimeVirtualTextureComponent* Component, FBoxSphereBounds const& WorldBounds, int32 Level) override;
	virtual void InvalidatePathTracedOutput(PathTracing::EInvalidateReason InvalidateReason = PathTracing::EInvalidateReason::Uncategorized) override;
	virtual void InvalidateLumenSurfaceCache_GameThread(UPrimitiveComponent* Component) override;
	virtual void GetPrimitiveUniformShaderParameters_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool& bHasPrecomputedVolumetricLightmap, FMatrix& PreviousLocalToWorld, int32& SingleCaptureIndex, bool& bOutputVelocity) const override;
	virtual void BuildPrimitiveUniformShaderParameters_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, FPrimitiveUniformShaderParametersBuilder& Builder) const override;
	virtual void UpdateLightTransform(ULightComponent* Light) override;
	virtual void UpdateLightColorAndBrightness(ULightComponent* Light) override;
	virtual void UpdateLightProxy(ULightComponent* Light, TFunction<void(FLightSceneProxy*)>&& Func) override;
	virtual void AddExponentialHeightFog(uint64 Id, const FExponentialHeightFogDynamicState& State) override;
	virtual void RemoveExponentialHeightFog(uint64 Id) override;
	virtual bool HasAnyExponentialHeightFog() const override;

	virtual void AddHairStrands(FHairStrandsInstance* Proxy) override;
	virtual void RemoveHairStrands(FHairStrandsInstance* Proxy) override;

	virtual void GetLightIESAtlasSlot(const FLightSceneProxy* Proxy, FLightRenderParameters* Out) override;
	virtual void GetRectLightAtlasSlot(const FRectLightSceneProxy* Proxy, FLightRenderParameters* Out) override;

	virtual void AddLocalFogVolume(class FLocalFogVolumeSceneProxy* FogProxy) override;
	virtual void RemoveLocalFogVolume(class FLocalFogVolumeSceneProxy* FogProxy) override;
	virtual bool HasAnyLocalFogVolume() const override;

	virtual void AddSkyAtmosphere(FSkyAtmosphereSceneProxy* SkyAtmosphereSceneProxy, bool bStaticLightingBuilt) override;
	virtual void RemoveSkyAtmosphere(FSkyAtmosphereSceneProxy* SkyAtmosphereSceneProxy) override;
	virtual FSkyAtmosphereRenderSceneInfo* GetSkyAtmosphereSceneInfo() override { return SkyAtmosphere; }
	virtual const FSkyAtmosphereRenderSceneInfo* GetSkyAtmosphereSceneInfo() const override { return SkyAtmosphere; }

	inline uint32 GetCachedTransmittanceAndMultiScatteringLUTsVersion() const
	{
		return CachedTransmittanceAndMultiScatteringLUTsVersion;
	}
	inline void SetCachedTransmittanceAndMultiScatteringLUTsVersion(uint32 NewTransmittanceAndMultiScatteringLUTsVersion)
	{
		CachedTransmittanceAndMultiScatteringLUTsVersion = NewTransmittanceAndMultiScatteringLUTsVersion;
	}

	virtual void AddSparseVolumeTextureViewer(FSparseVolumeTextureViewerSceneProxy* SVTV) override;
	virtual void RemoveSparseVolumeTextureViewer(FSparseVolumeTextureViewerSceneProxy* SVTV) override;

	virtual IAnimBankTransformProvider* GetAnimBankTransformProvider() override;
	virtual IAnimRuntimeTransformProvider* GetAnimRuntimeTransformProvider() override;
	virtual IAnimSequenceTransformProvider* GetAnimSequenceTransformProvider() override;

	virtual void SetPhysicsField(class FPhysicsFieldSceneProxy* PhysicsFieldSceneProxy) override;
	virtual void ResetPhysicsField() override;
	virtual void ShowPhysicsField() override;
	virtual void UpdatePhysicsField(FRDGBuilder& GraphBuilder, FViewInfo& View) override;

	virtual void AddVolumetricCloud(FVolumetricCloudSceneProxy* VolumetricCloudSceneProxy) override;
	virtual void RemoveVolumetricCloud(FVolumetricCloudSceneProxy* VolumetricCloudSceneProxy) override;
	virtual FVolumetricCloudRenderSceneInfo* GetVolumetricCloudSceneInfo() override { return VolumetricCloud; }
	virtual const FVolumetricCloudRenderSceneInfo* GetVolumetricCloudSceneInfo() const override { return VolumetricCloud; }

	virtual void AddWindSource(UWindDirectionalSourceComponent* WindComponent) override;
	virtual void RemoveWindSource(UWindDirectionalSourceComponent* WindComponent) override;
	virtual void UpdateWindSource(UWindDirectionalSourceComponent* WindComponent) override;
	virtual const TArray<FWindSourceSceneProxy*>& GetWindSources_RenderThread() const override;
	virtual void GetWindParameters(const FVector& Position, FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const override;
	virtual void GetWindParameters_GameThread(const FVector& Position, FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const override;
	virtual void GetDirectionalWindParameters(FVector& OutDirection, float& OutSpeed, float& OutMinGustAmt, float& OutMaxGustAmt) const override;
	virtual void AddSpeedTreeWind(FVertexFactory* VertexFactory, const UStaticMesh* StaticMesh) override;
	virtual void RemoveSpeedTreeWind_RenderThread(FVertexFactory* VertexFactory, const UStaticMesh* StaticMesh) override;
	virtual void UpdateSpeedTreeWind(double CurrentTime) override;
	virtual FRHIUniformBuffer* GetSpeedTreeUniformBuffer(const FVertexFactory* VertexFactory) const override;
	virtual void DumpUnbuiltLightInteractions( FOutputDevice& Ar ) const override;
	virtual void UpdateParameterCollections(const TArray<FMaterialParameterCollectionInstanceResource*>& InParameterCollections) override;

	virtual bool RequestGPUSceneUpdate(FPrimitiveSceneInfo& PrimitiveSceneInfo, EPrimitiveDirtyState PrimitiveDirtyState) override;
	virtual bool RequestUniformBufferUpdate(FPrimitiveSceneInfo& PrimitiveSceneInfo) override;

	virtual void RefreshNaniteRasterBins(FPrimitiveSceneInfo& PrimitiveSceneInfo) override;
	virtual void ReloadNaniteFixedFunctionBins() override;

	FVirtualShadowMapArrayCacheManager* GetVirtualShadowMapCache();

	FLumenSceneData* FindLumenSceneData(uint32 ViewKey, uint32 GPUIndex) const;
	static uint32 GetShareOriginViewKey(const FSceneViewState& View);
	inline FLumenSceneData* GetLumenSceneData(const FViewInfo& View) const
	{
		if (View.ViewLumenSceneData)
		{
			return View.ViewLumenSceneData;
		}
		else
		{
			return FindLumenSceneData(View.ViewState ? GetShareOriginViewKey(*View.ViewState) : 0, View.GPUMask.GetFirstIndex());
		}
	}
	inline FLumenSceneData* GetLumenSceneData(const FSceneView& View) const
	{
		// Should we assert that this is only called for FViewInfo (meaning inside scene renderer)?
		if (View.bIsViewInfo)
		{
			return GetLumenSceneData((const FViewInfo&)View);
		}
		else
		{
			return FindLumenSceneData(View.State ? GetShareOriginViewKey(*(const FSceneViewState*)View.State) : 0, View.GPUMask.GetFirstIndex());
		}
	}

	virtual void AddPrimitive(FPrimitiveSceneDesc* Primitive) override;
	virtual void RemovePrimitive(FPrimitiveSceneDesc* Primitive) override;
	virtual void ReleasePrimitive(FPrimitiveSceneDesc* Primitive) override;
	virtual void UpdatePrimitiveTransform(FPrimitiveSceneDesc* Primitive) override;

	virtual void BatchAddPrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives) override;
	virtual void BatchRemovePrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives) override;
	virtual void BatchReleasePrimitives(TArrayView<FPrimitiveSceneDesc*> InPrimitives) override;

	virtual void BatchRemovePrimitives(TArray<FPrimitiveSceneProxy*>&& InPrimitives) override;
		
	virtual void UpdateCustomPrimitiveData(FPrimitiveSceneDesc* Primitive, const FCustomPrimitiveData& CustomPrimitiveData) override;

	virtual void UpdatePrimitiveInstances(FPrimitiveSceneDesc* Primitive) override;

	bool HasSkyAtmosphere() const
	{
		return (SkyAtmosphere != nullptr);
	}
	bool HasVolumetricCloud() const
	{
		return (VolumetricCloud != nullptr);
	}

	bool IsSecondAtmosphereLightEnabled()
	{
		// If the second light is not null then we enable the second light.
		// We do not do any light1 to light0 remapping if light0 is null.
		return AtmosphereLights[1] != nullptr;
	}

	bool IsSecondAtmosphereLightEnabledForCloud()
	{
		if (IsSecondAtmosphereLightEnabled())
		{
			// Check the scattering scale of the second light to make sure it can contribute to the clouds before enabling it, as the second atmosphere light is an expensive feature.
			FLightSceneInfo* AtmosphericLight1Info = AtmosphereLights[1];
			FLightSceneProxy* AtmosphericLight1 = AtmosphericLight1Info ? AtmosphericLight1Info->Proxy : nullptr;
			if (AtmosphericLight1)
			{
				FLinearColor ColorScale = AtmosphericLight1->GetCloudScatteredLuminanceScale();
				if (ColorScale.R <= 0.0f && ColorScale.G <= 0.0f && ColorScale.B <= 0.0f)
				{
					return false;
				}
				return true;
			}
		}
		return false;
	}

	// Reset all the light to default state "not being affected by atmosphere". Should only be called from render side.
	void ResetAtmosphereLightsProperties();

	/**
	 * Retrieves the lights interacting with the passed in primitive and adds them to the out array.
	 *
	 * @param	Primitive				Primitive to retrieve interacting lights for
	 * @param	RelevantLights	[out]	Array of lights interacting with primitive
	 */
	virtual void GetRelevantLights( UPrimitiveComponent* Primitive, TArray<const ULightComponent*>* RelevantLights ) const override;

	/** Retrieves the number of lights interacting with the passed in primitive. */
	virtual int32 GetRelevantLightCount(UPrimitiveComponent* Primitive) const override;

	/** Sets the precomputed visibility handler for the scene, or NULL to clear the current one. */
	virtual void SetPrecomputedVisibility(const FPrecomputedVisibilityHandler* InPrecomputedVisibilityHandler) override;

	/** Update render states that possibly cached inside renderer, like mesh draw commands. More lightweight than re-registering the scene proxy. */
	virtual void UpdateCachedRenderStates(FPrimitiveSceneProxy* SceneProxy) override;

	/** Updates PrimitivesSelected array for this PrimitiveSceneInfo */
	virtual void UpdatePrimitiveSelectedState_RenderThread(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bIsSelected) override;
	virtual void UpdatePrimitiveVelocityState_RenderThread(FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bIsBeingMoved) override;

	virtual void UpdateEarlyZPassMode() override;

	virtual void Release() override;
	virtual UWorld* GetWorld() const override { return World; }

	/** Finds the closest reflection capture to a point in space, accounting influence radius */
	const FReflectionCaptureProxy* FindClosestReflectionCapture(FVector Position) const;

	const class FPlanarReflectionSceneProxy* FindClosestPlanarReflection(const FBoxSphereBounds& Bounds) const;

	const class FPlanarReflectionSceneProxy* GetForwardPassGlobalPlanarReflection() const;

	/**
	 * Get the default base pass depth stencil access
	 */
	static FExclusiveDepthStencil::Type GetDefaultBasePassDepthStencilAccess(ERHIFeatureLevel::Type InFeatureLevel);

	/**
	 * Get the default base pass depth stencil access
	 */
	static void GetEarlyZPassMode(ERHIFeatureLevel::Type InFeatureLevel, EDepthDrawingMode& OutZPassMode, bool& bOutEarlyZPassMovable);

	/**
	 * @return		true if hit proxies should be rendered in this scene.
	 */
	virtual bool RequiresHitProxies() const override;

	SIZE_T GetSizeBytes() const;

	/**
	* Return the scene to be used for rendering
	*/
	virtual FScene* GetRenderScene() final { return this; }
	virtual const FScene* GetRenderScene() const final { return this; }

	virtual void OnWorldCleanup() override;


	virtual void UpdateSceneSettings(AWorldSettings* WorldSettings) override;

	virtual class FGPUSkinCache* GetGPUSkinCache() override
	{
		return GPUSkinCache;
	}

	virtual class FSkeletalMeshUpdater* GetSkeletalMeshUpdater() override
	{
		return SkeletalMeshUpdater;
	}

	virtual void GetComputeTaskWorkers(TArray<class IComputeTaskWorker*>& OutWorkers) const override
	{
		OutWorkers = ComputeTaskWorkers;
	}

#if RHI_RAYTRACING
	virtual void UpdateCachedRayTracingState(class FPrimitiveSceneProxy* SceneProxy, EPrimitiveDirtyRayTracingState PrimitiveDirtyState) override;
	using FSceneInterface::UpdateCachedRayTracingState;

	FRayTracingDynamicGeometryUpdateManager* GetRayTracingDynamicGeometryUpdateManager()
	{
		return RayTracingDynamicGeometryUpdateManager;
	}
#endif

	/**
	 * Sets the FX system associated with the scene.
	 */
	virtual void SetFXSystem( class FFXSystemInterface* InFXSystem ) override;

	/**
	 * Get the FX system associated with the scene.
	 */
	virtual class FFXSystemInterface* GetFXSystem() override;

	/**
	 * Exports the scene.
	 *
	 * @param	Ar		The Archive used for exporting.
	 **/
	virtual void Export( FArchive& Ar ) const override;

	FRHIUniformBuffer* GetParameterCollectionBuffer(const FGuid& InId) const
	{
		const FUniformBufferRHIRef* ExistingUniformBuffer = ParameterCollections.Find(InId);

		if (ExistingUniformBuffer)
		{
			return *ExistingUniformBuffer;
		}

		return nullptr;
	}

	virtual void ApplyWorldOffset(const FVector& InOffset) override;

	virtual void OnLevelAddedToWorld(const FName& InLevelName, UWorld* InWorld, bool bIsLightingScenario) override;
	virtual void OnLevelRemovedFromWorld(const FName& LevelRemovedName, UWorld* InWorld, bool bIsLightingScenario) override;

	virtual bool HasAnyLights() const override 
	{ 
		check(IsInGameThread());
		return NumVisibleLights_AnyThread > 0 || NumEnabledSkylights_GameThread > 0;
	}

	virtual bool IsEditorScene() const override { return bIsEditorScene; }

	bool ShouldRenderSkylightInBasePass(bool bIsTranslucent) const;

	virtual TConstArrayView<FPrimitiveSceneProxy*> GetPrimitiveSceneProxies() const final
	{
		return PrimitiveSceneProxies;
	}

	virtual TConstArrayView<FPrimitiveComponentId> GetScenePrimitiveComponentIds() const final
	{
		return PrimitiveComponentIds;
	}

	/** Flush any dirty runtime virtual texture pages */
	void FlushDirtyRuntimeVirtualTextures();

#if WITH_EDITOR
	virtual bool InitializePixelInspector(FRenderTarget* BufferFinalColor, FRenderTarget* BufferSceneColor, FRenderTarget* BufferDepth, FRenderTarget* BufferHDR, FRenderTarget* BufferA, FRenderTarget* BufferBCDEF, int32 BufferIndex) override;

	virtual bool AddPixelInspectorRequest(FPixelInspectorRequest *PixelInspectorRequest) override;
#endif //WITH_EDITOR

	virtual void StartFrame() override
	{
		VelocityData.StartFrame(this);
	}

	virtual void EndFrame(FRHICommandListImmediate& RHICmdList) override;

	/**
	 * Returns the current "FrameNumber" where frame corresponds to how many times FRendererModule::BeginRenderingViewFamilies has been called.
	 * Thread safe, and returns a different copy for game/render thread. GetFrameNumberRenderThread can only be called from the render thread. 
	 */
	virtual uint32 GetFrameNumber() const override;
	inline uint32 GetFrameNumberRenderThread() const { return SceneFrameNumberRenderThread; }

	virtual void IncrementFrameNumber() override;

	void DumpMeshDrawCommandMemoryStats();

	/**
	 * Maximum used persistent Primitive Index, use to size arrays that store primitive data indexed by FPrimitiveSceneInfo::PersistentIndex.
	 * Only changes during UpdateAllPrimitiveSceneInfos.
	 */
	inline int32 GetMaxPersistentPrimitiveIndex() const { return PersistentPrimitiveIdAllocator.GetMaxSize(); }

	FORCEINLINE int32 GetPrimitiveIndex(const FPersistentPrimitiveIndex& PersistentPrimitiveIndex) const
	{ 
		if (PersistentPrimitiveIndex.IsValid() && PersistentPrimitiveIndex.Index < PersistentPrimitiveIdToIndexMap.Num())
		{
			return PersistentPrimitiveIdToIndexMap[PersistentPrimitiveIndex.Index];
		}
		return INDEX_NONE;
	}

	bool GetForceNoPrecomputedLighting() const
	{
		return bForceNoPrecomputedLighting;
	}

	FLumenSceneDataIterator GetLumenSceneDataIterator() const
	{
		return FLumenSceneDataIterator(this);
	}

	void WaitForCreateLightPrimitiveInteractionsTask()
	{
		CSV_SCOPED_SET_WAIT_STAT(LightPrimitiveInteractions);
		CreateLightPrimitiveInteractionsTask.Wait();
	}

	const UE::Tasks::FTask& GetCreateLightPrimitiveInteractionsTask() const
	{
		return CreateLightPrimitiveInteractionsTask;
	}

	const UE::Tasks::FTask& GetGPUSkinUpdateTask() const
	{
		return GPUSkinUpdateTask;
	}

	const UE::Tasks::FTask& GetGPUSkinCacheTask() const
	{
		return GPUSkinCacheTask;
	}

	void AddGPUSkinCacheAsyncComputeWait(FRDGBuilder& GraphBuilder) const;

	void WaitForCacheMeshDrawCommandsTask()
	{
		CSV_SCOPED_SET_WAIT_STAT(CacheMeshDrawCommands);
		CacheMeshDrawCommandsTask.Wait();
	}

	const UE::Tasks::FTask& GetCacheMeshDrawCommandsTask() const
	{
		return CacheMeshDrawCommandsTask;
	}

	void WaitForCacheNaniteMaterialBinsTask()
	{
		CSV_SCOPED_SET_WAIT_STAT(CacheNaniteMaterialBins);
		CacheNaniteMaterialBinsTask.Wait();
	}

	const UE::Tasks::FTask& GetCacheNaniteMaterialBinsTask() const
	{
		return CacheNaniteMaterialBinsTask;
	}

#if RHI_RAYTRACING
	void WaitForCacheRayTracingPrimitivesTask()
	{
		CSV_SCOPED_SET_WAIT_STAT(CacheRayTracingPrimitives);
		CacheRayTracingPrimitivesTask.Wait();
	}

	const UE::Tasks::FTask& GetCacheRayTracingPrimitivesTask()
	{
		return CacheRayTracingPrimitivesTask;
	}
#endif

	void LumenAddPrimitive(FPrimitiveSceneInfo* InPrimitive);
	void LumenUpdatePrimitive(FPrimitiveSceneInfo* InPrimitive);
	void LumenInvalidateSurfaceCacheForPrimitive(FPrimitiveSceneInfo* InPrimitive);
	void LumenRemovePrimitive(FPrimitiveSceneInfo* InPrimitive, int32 PrimitiveIndex);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	void DebugRender(TArrayView<FViewInfo> Views);
#endif

	template<typename TExtension>
	TExtension* GetExtensionPtr() { return SceneExtensions.GetExtensionPtr<TExtension>(); }
	template<typename TExtension>
	const TExtension* GetExtensionPtr() const { return SceneExtensions.GetExtensionPtr<TExtension>(); }
	template<typename TExtension>
	TExtension& GetExtension() { return SceneExtensions.GetExtension<TExtension>(); }
	template<typename TExtension>
	const TExtension& GetExtension() const { return SceneExtensions.GetExtension<TExtension>(); }

	virtual bool AddCustomRenderPass(const FSceneViewFamily* ViewFamily, const FCustomRenderPassRendererInput& CustomRenderPassInput);

	class FInstanceCullingOcclusionQueryRenderer* InstanceCullingOcclusionQueryRenderer = nullptr;

	/**
	 * Light scene change delegates, may be used to hook in subsystems that need to respond to light scene changes.
	 * Note, all the light scene changes are applied _before_ all the primitive scene infos are updated.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FSceneLightSceneInfoUpdateDelegate, FRDGBuilder& , const FLightSceneChangeSet&);
	/**
	 * This delegate is invoked during the scene update phase _before_ the scene has had any light changes applied.
	 * Thus, AddedLightIds is not valid in the change set as the added lights do not have assigned IDs yet.
	 * IF using this to drive an async task, care must be taken as the (light) scene will be modified directly after.
	 */
	FSceneLightSceneInfoUpdateDelegate OnPreLightSceneInfoUpdate;
	/**
	 * This delegate is invoked during the scene update phase _after_ all light changes are applied.
	 * Thus, RemovedLightIds may contain ID's that are no longer valid or are now referencing newly added lights.
	 * IF using this to drive an async task, the core light scene info may be used, but primitive scene updates will still be ongoing (e.g., light/primitive interactions may change).
	 */
	FSceneLightSceneInfoUpdateDelegate OnPostLightSceneInfoUpdate;

	/** Returns the mobile directional light in the given channel index, if enabled for the given view */
	FLightSceneInfo* GetMobileDirectionalLightForView(int32 ChannelIdx, const FViewInfo& View) const;

	/**
	 * Retrieves the lights interacting with the passed in primitive and adds them to the out array.
	 * Render thread version of function.
	 * @param	PrimitiveSceneProxy		Proxy of Primitive to retrieve interacting lights for
	 * @param	RelevantLights	[out]	Array of lights interacting with primitive
	 */
	void GetRelevantLights_RenderThread( const FPrimitiveSceneProxy* PrimitiveSceneProxy, TArray<const FLightSceneProxy*> &OutRelevantLights ) const;

	/** Updates a primitive's transform, called on the rendering thread. */
	void UpdatePrimitiveTransform_RenderThread(FPrimitiveSceneProxy* PrimitiveSceneProxy, const FBoxSphereBounds& WorldBounds, const FBoxSphereBounds& LocalBounds, const FMatrix& LocalToWorld, const FVector& OwnerPosition, const TOptional<FTransform>& PreviousTransform);

	/** Returns true if the scene requires the use of debug material permutations. */
	bool RequiresDebugMaterials() const { return PersistentViewStateDebugFlags != 0; }

     /** Update the DrawnInGame and the DrawnInEditor flags on the provided scene proxies */
	virtual void UpdatePrimitivesIsDrawn_RenderThread(TArrayView<FPrimitiveSceneProxy*> PrimitiveSceneProxies, bool bIsDrawn) override;

	/** Swaps cached shadow state for a timesliced runtime reflection capture, to avoid artifacts if the shadow is moving during the timeslice. */
	void SwapMobileReflectionCaptureCachedShadowState();

protected:

private:
	template<class T> 	
	void BatchAddPrimitivesInternal(TArrayView<T*> InPrimitives);

	template<class T> 	
	void BatchRemovePrimitivesInternal(TArrayView<T*> InPrimitives);

	template<class T> 	
	void BatchReleasePrimitivesInternal(TArrayView<T*> InPrimitives);	

	template<class T> 	
	void UpdatePrimitiveTransformInternal(T* Primitive);

	[[nodiscard]] virtual IPrimitiveTransformUpdater* CreatePrimitiveTransformUpdater(int32 MaxPrimitives) override;
	virtual void UpdatePrimitiveTransforms(IPrimitiveTransformUpdater* PrimitiveTransformUpdater) override;
	
	void RemoveViewLumenSceneData_RenderThread(FSceneViewStateInterface* ViewState);
	void RemoveViewState_RenderThread(FSceneViewStateInterface*);

	/**
	 * Ensures the packed primitive arrays contain the same number of elements.
	 */
	void CheckPrimitiveArrays(int MaxTypeOffsetIndex = -1);

	/**
	 * Adds a primitive to the scene.  Called in the rendering thread by AddPrimitive.
	 * @param PrimitiveSceneInfo - The primitive being added.
	 */
	void AddPrimitiveSceneInfo_RenderThread(FPrimitiveSceneInfo* PrimitiveSceneInfo, const TOptional<FTransform>& PreviousTransform);

	/**
	 * Removes a primitive from the scene.  Called in the rendering thread by RemovePrimitive.
	 * @param PrimitiveSceneInfo - The primitive being removed.
	 */
	void RemovePrimitiveSceneInfo_RenderThread(FPrimitiveSceneInfo* PrimitiveSceneInfo);


	void UpdatePrimitiveOcclusionBoundsSlack_RenderThread(const FPrimitiveSceneProxy* PrimitiveSceneProxy, float NewSlack);

	void UpdateCustomPrimitiveData(FPrimitiveSceneProxy* SceneProxy, const FCustomPrimitiveData& CustomPrimitiveData);

	/** Updates a single primitive's lighting attachment root. */
	void UpdatePrimitiveLightingAttachmentRoot(UPrimitiveComponent* Primitive);

	void AssignAvailableShadowMapChannelForLight(FLightSceneInfo* LightSceneInfo);

	/**
	 * Adds a light to the scene.  Called in the rendering thread by AddLight.
	 * @param LightSceneInfo - The light being added.
	 */
	void AddLightSceneInfo_RenderThread(FLightSceneInfo* LightSceneInfo);

	/**
	 * Adds a decal to the scene.  Called in the rendering thread by AddDecal or RemoveDecal.
	 * @param Component - The object that should being added or removed.
	 * @param bAdd true:add, FASLE:remove
	 */
	void AddOrRemoveDecal_RenderThread(FDeferredDecalProxy* Component, bool bAdd);

	/**
	 * Removes a light from the scene.  Called in the rendering thread by RemoveLight.
	 * @param LightSceneInfo - The light being removed.
	 */
	void RemoveLightSceneInfo_RenderThread(FLightSceneInfo* LightSceneInfo);

	void UpdateLightTransform_RenderThread(FLightSceneId LightId, FLightSceneInfo* LightSceneInfo, const struct FUpdateLightTransformParameters& Parameters);

	/** 
	* Updates the contents of the given reflection capture by rendering the scene. 
	* This must be called on the game thread.
	*/
	void CaptureOrUploadReflectionCapture(UReflectionCaptureComponent* CaptureComponent, int32 ReflectionCaptureSize, bool bVerifyOnlyCapturing, bool bCapturingForMobile, bool bInsideTick);

	/** Updates the contents of all reflection captures in the scene.  Must be called from the game thread. */
	void UpdateAllReflectionCaptures(const TCHAR* CaptureReason, int32 ReflectionCaptureSize, bool bVerifyOnlyCapturing, bool bCapturingForMobile, bool bInsideTick);

	/**
	 * Shifts scene data by provided delta
	 * Called on world origin changes
	 * 
	 * @param	InOffset	Delta to shift scene by
	 */
	void ApplyWorldOffset_RenderThread(FRHICommandListBase& RHICmdList, const FVector& InOffset);

	void ProcessAtmosphereLightRemoval_RenderThread(FLightSceneInfo* LightSceneInfo);
	void ProcessAtmosphereLightAddition_RenderThread(FLightSceneInfo* LightSceneInfo);

	/**
	 * Process all scene updates for lights.
	 */
	void UpdateLights(FRDGBuilder& GraphBuilder, FSceneExtensionsUpdaters& SceneExtensionsUpdaters);

private:
	template <typename UpdatePayloadType>
	void UpdatePrimitiveInternal(FPrimitiveSceneProxy* SceneProxy, UpdatePayloadType &&InUpdatePayload);

	template <typename UpdatePayloadType>
	void UpdateLightInternal(FLightSceneProxy* LightSceneProxy, UpdatePayloadType &&InUpdatePayload);

	template <typename UpdatePayloadType>
	void UpdateLightInternal_RenderThread(FLightSceneProxy* LightSceneProxy, UpdatePayloadType&& InUpdatePayload);

	FString FullWorldName;
#if RHI_RAYTRACING
	void UpdateRayTracingGroupBounds_AddPrimitives(const TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>& PrimitiveSceneInfos);
	void UpdateRayTracingGroupBounds_RemovePrimitives(const TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator>& PrimitiveSceneInfos);
	template<typename RangeType>
	inline void UpdateRayTracingGroupBounds_UpdatePrimitives(const RangeType& UpdatedTransforms);
#endif

	void UpdatePrimitiveInstances(FUpdateInstanceCommand& UpdateParams);

	struct FLevelCommand
	{
		enum class EOp
		{
			Add,
			Remove
		};

		FName Name;
		EOp Op;
	};

	/**
	 * Helper function to process level commands during scene update, only process command with the given EOp.
	 */
	void ProcessLevelCommands(FLevelCommand::EOp OpToProcess);

	FScenePrimitiveUpdates PrimitiveUpdates;
	TArray<FLevelCommand> LevelCommands;

	UE::Tasks::FTask CreateLightPrimitiveInteractionsTask;
	UE::Tasks::FTask GPUSkinUpdateTask;
	UE::Tasks::FTask GPUSkinCacheTask;
	UE::Tasks::FTask CacheMeshDrawCommandsTask;
	UE::Tasks::FTask CacheNaniteMaterialBinsTask;
#if RHI_RAYTRACING
	UE::Tasks::FTask CacheRayTracingPrimitivesTask;
#endif

	FSceneLightInfoUpdates *SceneLightInfoUpdates;

	/** Bit mask of debug request flags for all the views in PersistentViewStateUniqueIDs. */
	uint64 PersistentViewStateDebugFlags;

	/** 
	 * The number of visible lights in the scene
	 */
	std::atomic<int32> NumVisibleLights_AnyThread;

	/** 
	 * Whether the scene has a valid sky light.
	 * Note: This is tracked on the game thread!
	 */
	int32 NumEnabledSkylights_GameThread;

	/** Frame number incremented per-family (except if there are multiple view families in one render call) viewing this scene. */
	uint32 SceneFrameNumber;
	uint32 SceneFrameNumberRenderThread;

	uint32 LastUpdateFrameCounter = UINT32_MAX;

	/** Whether world settings has bForceNoPrecomputedLighting set */
	bool bForceNoPrecomputedLighting;

	friend class FSceneViewState;
};

inline bool ShouldIncludeDomainInMeshPass(EMaterialDomain Domain)
{
	// Non-Surface domains can be applied to static meshes for thumbnails or material editor preview
	// Volume domain materials however must only be rendered in the voxelization pass
	return Domain != MD_Volume;
}

