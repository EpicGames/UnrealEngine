// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScenePrivate.h: Private scene manager definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "SceneRenderingAllocator.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "SceneViewOcclusionHistory.h"
#include "RendererInterface.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

typedef TBitArray<SceneRenderingBitArrayAllocator> FSceneBitArray;
typedef TConstSetBitIterator<SceneRenderingBitArrayAllocator> FSceneSetBitIterator;
typedef TConstDualSetBitIterator<SceneRenderingBitArrayAllocator,SceneRenderingBitArrayAllocator> FSceneDualSetBitIterator;

class FLightSceneId
{
public:
	FLightSceneId() = default;
	explicit FLightSceneId(int32 InIndex) : Index(InIndex) {}

	bool IsValid() const { return Index != INDEX_NONE; }

	friend uint32 GetTypeHash(FLightSceneId Id)
	{
		return GetTypeHash(Id.Index);
	}

	bool operator==(const FLightSceneId& B) const = default;

	int32 GetIndex() const { return Index; }

protected:
	int32 Index = INDEX_NONE;
};

// Forward declarations.
class FScene;

/** A simple chunked array representation for scene primitives data arrays. */
template <typename T>
class TScenePrimitiveArray
{
	static const int32 NumElementsPerChunk = 1024;
public:
	TScenePrimitiveArray() = default;

	T& Add(const T& Element)
	{
		return *(new (&AddUninitialized()) T(Element));
	}

	T& AddUninitialized()
	{
		if (NumElements % NumElementsPerChunk == 0)
		{
			ChunkType* Chunk = new ChunkType;
			Chunk->Reserve(NumElementsPerChunk);
			Chunks.Emplace(Chunk);
		}

		NumElements++;
		ChunkType& Chunk = *Chunks.Last();
		Chunk.AddUninitialized();
		return Chunk.Last();
	}

	void Remove(uint32 Count, EAllowShrinking AllowShrinking)
	{
		check(Count <= NumElements);
		const uint32 NumElementsNew = NumElements - Count;
		while (NumElements != NumElementsNew)
		{
			--NumElements;

			ChunkType& Chunk = *Chunks.Last();
			Chunk.Pop(EAllowShrinking::No);

			if (Chunk.IsEmpty())
			{
				Chunks.Pop(EAllowShrinking::No);
			}
		}
	}

	void Reserve(int32 Count)
	{
		Chunks.Reserve(NumChunks(Count));
	}

	T& Get(int32 ElementIndex)
	{
		const uint32 ChunkIndex        = ElementIndex / NumElementsPerChunk;
		const uint32 ChunkElementIndex = ElementIndex % NumElementsPerChunk;
		return (*Chunks[ChunkIndex])[ChunkElementIndex];
	}

	const T& Get(int32 ElementIndex) const
	{
		const uint32 ChunkIndex        = ElementIndex / NumElementsPerChunk;
		const uint32 ChunkElementIndex = ElementIndex % NumElementsPerChunk;
		return (*Chunks[ChunkIndex])[ChunkElementIndex];
	}

	FORCEINLINE T& operator[] (int32 Index) { return Get(Index); }
	FORCEINLINE const T& operator[] (int32 Index) const { return Get(Index); }

	bool IsValidIndex(int32 Index) const { return static_cast<uint32>(Index) < NumElements; }

	int32 Num() const { return NumElements; }

private:
	static constexpr uint32 NumChunks(uint32 NumElements)
	{
		return (NumElements + NumElementsPerChunk - 1u) / NumElementsPerChunk;
	}

	using ChunkType = TArray<T>;
	TArray<TUniquePtr<ChunkType>> Chunks;
	uint32 NumElements = 0;
};

template<typename IndexType, typename Allocator = FDefaultBitArrayAllocator>
class TIdBasedSetBitIterator;

template<typename IndexType, typename Allocator = FDefaultBitArrayAllocator>
class TIdBasedBitArray : protected TBitArray<Allocator>
{
public:
	using FIndex = IndexType;
	using Super = TBitArray<Allocator>;
	friend TIdBasedSetBitIterator<IndexType,Allocator>;
	using FIterator = TIdBasedSetBitIterator<IndexType,Allocator>;

	using TBitArray<Allocator>::TBitArray;

	using Super::SetNum;
	using Super::IsEmpty;

	void PadToId(FIndex Index, bool bPadValue = false)
	{
		Super::PadToNum(Index.GetIndex() + 1, bPadValue);
	}

	[[nodiscard]] UE_FORCEINLINE_HINT FBitReference operator[](FIndex Index)
	{
		return Super::operator[](Index.GetIndex());
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const FConstBitReference operator[](FIndex Index) const
	{
		return Super::operator[](Index.GetIndex());
	}

	bool IsValidIndex(FIndex Index) const { return Super::IsValidIndex(Index.GetIndex()); }

	[[nodiscard]] int32 CountSetBits(FIndex FromIndex = FIndex { 0 }, FIndex ToIndex = FIndex { INDEX_NONE } ) const
	{
		return Super::CountSetBits(FromIndex.GetIndex(), ToIndex.GetIndex());
	}

};

template<typename IndexType, typename Allocator>
class TIdBasedSetBitIterator : public TConstSetBitIterator<Allocator>
{
public:
	using FIndex = IndexType;

	FIndex GetIndex() const
	{
		return FIndex { TConstSetBitIterator<Allocator>::GetIndex() };
	}

	[[nodiscard]] explicit TIdBasedSetBitIterator(const TIdBasedBitArray<IndexType, Allocator>& InArray UE_LIFETIMEBOUND, FIndex StartIndex)
		: TConstSetBitIterator<Allocator>(InArray, StartIndex.GetIndex())
	{
	}

	[[nodiscard]] explicit TIdBasedSetBitIterator(const TIdBasedBitArray<IndexType, Allocator>& InArray UE_LIFETIMEBOUND)
		: TConstSetBitIterator<Allocator>(InArray, 0)
	{
	}
};

template<typename IndexType, typename ItemType, typename Allocator = FDefaultAllocator>
class TIdBasedArray : public TArray<ItemType, Allocator>
{
public:
	using FIndex = IndexType;

	/**
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ItemType& operator[](FIndex Index) UE_LIFETIMEBOUND
	{
		check(Index.IsValid());
		return TArray<ItemType, Allocator>::operator[](Index.GetIndex());
	}

	/**
	 */
	[[nodiscard]] UE_REWRITE const ItemType& operator[](FIndex Index) const UE_LIFETIMEBOUND
	{
		return (*const_cast<TIdBasedArray*>(this))[Index];
	}

	bool IsValidIndex(FIndex Index) const
	{
		return Index.IsValid() && TArray<ItemType, Allocator>::IsValidIndex(Index.GetIndex());
	}
};

template<typename IndexType, typename ItemType, typename Allocator = FDefaultSparseArrayAllocator>
class TIdBasedSparseArray : protected TSparseArray<ItemType, Allocator>
{
public:
	using Super = TSparseArray<ItemType, Allocator>;
	using FIndex = IndexType;

	// Explicitly hoist functions that we want since the base is protected to avoid accidental use of raw index.
	using Super::GetAllocatedSize;
	using Super::begin;
	using Super::end;
	using Super::GetMaxIndex;
	using Super::Num;

	/**
	 */
	UE_NODEBUG [[nodiscard]] UE_FORCEINLINE_HINT ItemType& operator[](FIndex Index) UE_LIFETIMEBOUND
	{
		check(IsValidIndex(Index));
		return Super::operator[](Index.GetIndex());
	}

	/**
	 */
	[[nodiscard]] UE_REWRITE const ItemType& operator[](FIndex Index) const UE_LIFETIMEBOUND
	{
		return (*const_cast<TIdBasedSparseArray*>(this))[Index];
	}

	bool IsValidIndex(FIndex Index) const
	{
		return Index.IsValid() && Super::IsValidIndex(Index.GetIndex());
	}

	void RemoveAt(FIndex Index)
	{
		check(IsValidIndex(Index));
		Super::RemoveAt(Index.GetIndex());
	}

	FIndex Add(const ItemType& Item)
	{
		return FIndex { Super::Add(Item) };
	}

	void EmplaceAt(FIndex Index, ItemType&& Item)
	{
		check(Index.IsValid());
		check(!Super::IsValidIndex(Index.GetIndex()));
		Super::EmplaceAt(Index.GetIndex(), MoveTemp(Item));
	}

	struct FConstIterator : public Super::TConstIterator
	{
		using Super::TConstIterator::TConstIterator;

		FIndex GetIndex() const 
		{
			return FIndex { Super::TConstIterator::GetIndex() };
		}
	};

	[[nodiscard]] FConstIterator CreateConstIterator() const
	{
		return FConstIterator(*this);
	}

};

// TODO: Specializations for Scene / Renderer data arrays using FPersistentPrimitiveIndex

//template<typename ItemType, typename Allocator = FDefaultAllocator>
//using TScenePersistentPrimitiveArray = TIdBasedArray<FPersistentPrimitiveIndex, ItemType, Allocator>;
//template<typename Allocator = FDefaultBitArrayAllocator>
//using TScenePersistentPrimitiveBitArray = TIdBasedBitArray<FPersistentPrimitiveIndex, Allocator>;


// Specializations for Scene / Renderer data arrays using FLightSceneId

template<typename ItemType, typename Allocator = FDefaultAllocator>
using TScenePeristentLightArray = TIdBasedArray<FLightSceneId, ItemType, Allocator>;

template<typename ItemType, typename Allocator = FDefaultSparseArrayAllocator>
using TLightSparseArray = TIdBasedSparseArray<FLightSceneId, ItemType, Allocator>;

/**
 * Template Array type for data indexed by FLightSceneId for use in the SceneRenderer - note uses SceneRenderingAllocator.
 */
template<typename ItemType>
using TSceneRendererLightArray = TIdBasedArray<FLightSceneId, ItemType, SceneRenderingAllocator>;

/**
 * BitArray indexed by FLightSceneId for use in the SceneRenderer - note uses SceneRenderingAllocator.
 */
using FSceneRendererLightBitArray = TIdBasedBitArray<FLightSceneId, SceneRenderingAllocator>;

/**
 * BitArray indexed by FLightSceneId for use in the Scene - uses persistent allocation FDefaultBitArrayAllocator.
 */
using FSceneLightBitArray = TIdBasedBitArray<FLightSceneId, FDefaultBitArrayAllocator>;

class FVisibleLightInfo;
/**
 * Array type for FVisibleLightInfo for use in the SceneRenderer - note, TSceneRendererLightArray uses SceneRenderingAllocator.
 */
using FVisibleLightInfoArray = TSceneRendererLightArray<FVisibleLightInfo>;
