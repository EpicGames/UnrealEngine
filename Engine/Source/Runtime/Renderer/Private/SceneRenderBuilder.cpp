// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneRenderBuilder.h"
#include "RenderCaptureInterface.h"
#include "VisualizeTexture.h"
#include "SceneRendering.h"
#include "GPUDebugCrashUtils.h"
#include "ScenePrivate.h"
#include "DumpGPU.h"
#include "Materials/MaterialRenderProxy.h"
#include "SceneViewExtension.h"
#include "Containers/IntrusiveDoubleLinkedList.h"
#include "TextureResource.h"
#include "DeferredShadingRenderer.h"
#include "FXSystem.h"
#include "Renderer/ViewSnapshotCache.h"
#include "SceneViewState.h"

//////////////////////////////////////////////////////////////////////////

#if !UE_BUILD_SHIPPING

static TAutoConsoleVariable<int32> CVarSplitScreenDebugEnable(
	TEXT("r.SplitScreenDebug.Enable"),
	0,
	TEXT("Debug feature to replace the main view with a pair of split screen views for testing purposes."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarSplitScreenDebugVertical(
	TEXT("r.SplitScreenDebug.Vertical"),
	0,
	TEXT("Split screen debug use vertical split (two panes vertically stacked).  If false, uses horizontal split (two panes side by side)."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarSplitScreenDebugFOVZoom(
	TEXT("r.SplitScreenDebug.FOVZoom"),
	1.0f,
	TEXT("Amount to zoom FOV.  Split screen expands the FOV for the new aspect.  This setting can counteract that expansion."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarSplitScreenDebugRotate0(
	TEXT("r.SplitScreenDebug.Rotate0"),
	0,
	TEXT("Rotate first split screen view by this amount.  Values [-1..1] are rotations in view space by fraction of horizontal FOV, outside that range are yaw rotation in degrees."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarSplitScreenDebugRotate1(
	TEXT("r.SplitScreenDebug.Rotate1"),
	0,
	TEXT("Rotate second split screen view by this amount.  Values [-1..1] are rotations in view space by fraction of horizontal FOV, outside that range are yaw rotation in degrees."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarSplitScreenDebugOrbit(
	TEXT("r.SplitScreenDebug.Orbit"),
	1,
	TEXT("When rotating by yaw, orbit around camera target actor, to keep third person character visible."),
	ECVF_Default);

static TAutoConsoleVariable<float> CVarSplitScreenDebugLetterbox(
	TEXT("r.SplitScreenDebug.Letterbox"),
	0,
	TEXT("When non-zero, letterboxes away this percent of screen (rounds up to nearest multiple of 8 pixels, max 50%)."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarSplitScreenDebugLumenScene(
	TEXT("r.SplitScreenDebug.LumenScene"),
	1,
	TEXT("For split screen debugging, allocate a separate Lumen scene for the second view."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarSplitScreenDebugMultiViewFamily(
	TEXT("r.SplitScreenDebug.MultiViewFamily"),
	0,
	TEXT("Uses two renderers with one view each rather than two views."),
	ECVF_Default);

static bool IsSplitScreenDebugEnabled(TConstArrayView<const FSceneViewFamily*> ViewFamilies)
{
	return CVarSplitScreenDebugEnable.GetValueOnGameThread() > 0 && ViewFamilies.Num() == 1 && ViewFamilies[0]->bSplitScreenDebugAllowed && ViewFamilies[0]->Views.Num() == 1;
}

static void CreateSplitScreenDebugViewFamilies(const FSceneViewFamily& InFamily, TConstArrayView<FSceneViewFamily*>& OutFamilies, TArray<FSceneViewFamily*, TFixedAllocator<2>>& OutFamiliesStorage)
{
	// We either generate a single family with 2 views, or two families with 1 view each for MGPU
	int32 NumFamilies;
	int32 NumViewsPerFamily;
	if (CVarSplitScreenDebugMultiViewFamily.GetValueOnGameThread())
	{
		NumFamilies = 2;
		NumViewsPerFamily = 1;
	}
	else
	{
		NumFamilies = 1;
		NumViewsPerFamily = 2;
	}

	TArray<FSceneViewFamily*, TFixedAllocator<2>> Families;
	TArray<const FSceneView**, TFixedAllocator<2>> ViewPointers;
	TArray<FSceneViewFamily*, TFixedAllocator<2>> ViewParents;

	for (int32 FamilyIndex = 0; FamilyIndex < NumFamilies; FamilyIndex++)
	{
		FSceneViewFamily* Family = new FSceneViewFamily(InFamily);

		Families.Add(Family);
		Family->SetScreenPercentageInterface_Unchecked(InFamily.GetScreenPercentageInterface()->Fork_GameThread(InFamily));
		Family->Views.SetNumZeroed(NumViewsPerFamily);

		for (int32 ViewIndex = 0; ViewIndex < NumViewsPerFamily; ViewIndex++)
		{
			ViewPointers.Add(&Family->Views[ViewIndex]);
			ViewParents.Add(Family);
		}
	}

	FIntRect OriginalViewRect = InFamily.Views[0]->SceneViewInitOptions.ViewRect;

	int32 SplitVertical = CVarSplitScreenDebugVertical.GetValueOnGameThread();
	float FOVZoom = CVarSplitScreenDebugFOVZoom.GetValueOnGameThread();
	float FOVScaleX = FOVZoom;
	float FOVScaleY = FOVZoom;

	float Letterbox = FMath::Clamp(CVarSplitScreenDebugLetterbox.GetValueOnGameThread(), 0.0f, 50.0f);
	int32 LetterboxPixels;

	if (SplitVertical)
	{
		// Double FOV X
		FOVScaleX *= 0.5f;

		// Convert letterbox from percentage to a multiple of 8 pixels, then reduce FOV by the relative pixel size
		LetterboxPixels = FMath::CeilToInt((Letterbox / 100.0f) * OriginalViewRect.Size().X * 0.125f) * 8;
		FOVScaleX = FOVScaleX * OriginalViewRect.Size().X / (OriginalViewRect.Size().X - LetterboxPixels);
	}
	else
	{
		// Double FOV Y
		FOVScaleY *= 0.5f;

		// Convert letterbox from percentage to a multiple of 8 pixels, then reduce FOV by the relative pixel size
		LetterboxPixels = FMath::CeilToInt((Letterbox / 100.0f) * OriginalViewRect.Size().Y * 0.125f) * 8;
		FOVScaleY = FOVScaleY * OriginalViewRect.Size().Y / (OriginalViewRect.Size().Y - LetterboxPixels);
	}

	for (int32 ViewIndex = 0; ViewIndex < 2; ViewIndex++)
	{
		FSceneViewInitOptions InitOptions = InFamily.Views[0]->SceneViewInitOptions;

		// Adjust projection
		InitOptions.ProjectionMatrix *= FMatrix(FVector(FOVScaleX, 0.0, 0.0), FVector(0.0, FOVScaleY, 0.0), FVector(0.0, 0.0, 1.0), FVector(0.0, 0.0, 0.0));

		// Adjust view matrix rotation
		double Rotate = ViewIndex == 0 ? CVarSplitScreenDebugRotate0.GetValueOnGameThread() : CVarSplitScreenDebugRotate1.GetValueOnGameThread();
		if (Rotate)
		{
			if (FMath::Abs(Rotate) <= 1.0)
			{
				// Rotation in view space (post multiply) as a fraction of horizontal FOV.  This mode is useful for creating views
				// that line up exactly along an edge with each other, without needing to do complex FOV calculations.  For example,
				// setting the left pane to -0.5 and right pane to 0.5 rotates the views away from each other by half the FOV,
				// producing a matching frustum edge at the middle of the screen (setting the right pane to 1.0 is another example).
				double FOV = FMath::RadiansToDegrees(FMath::Atan(1.0 / InitOptions.ProjectionMatrix.M[0][0]) * 2.0);
				Rotate *= FOV;

				InitOptions.ViewRotationMatrix = InitOptions.ViewRotationMatrix * UE::Math::TRotationMatrix<double>::Make(FRotator(Rotate, 0.0, 0.0));
			}
			else
			{
				// Rotate by degrees in Yaw
				FMatrix YawRotation = UE::Math::TRotationMatrix<double>::Make(FRotator(0.0, Rotate, 0.0));
				InitOptions.ViewRotationMatrix = YawRotation * InitOptions.ViewRotationMatrix;

#if !WITH_STATE_STREAM // TODO: Fix. ViewActor does not exist on render side
				// And optionally orbit the position around the player
				if (CVarSplitScreenDebugOrbit.GetValueOnGameThread() && InitOptions.ViewActor)
				{
					FVector TargetTranslation = InitOptions.ViewActor->GetTransform().GetTranslation();
					InitOptions.ViewOrigin = YawRotation.GetTransposed().TransformVector(InitOptions.ViewOrigin - TargetTranslation) + TargetTranslation;
					InitOptions.ViewLocation = InitOptions.ViewOrigin;
				}
#endif
			}

			// Convert adjusted matrix back to a rotation
			InitOptions.ViewRotation = InitOptions.ViewRotationMatrix.Rotator();
		}
		
		// Make view rectangles half the width / height and adjust opposite dimension for letterbox
		FIntRect ViewRect = OriginalViewRect;
		if (SplitVertical)
		{
			if (ViewIndex == 0)
			{
				ViewRect.Max.Y = (ViewRect.Min.Y + ViewRect.Max.Y) / 2;
			}
			else
			{
				ViewRect.Min.Y = (ViewRect.Min.Y + ViewRect.Max.Y) / 2;
			}
			ViewRect.Min.X += LetterboxPixels / 2;
			ViewRect.Max.X -= LetterboxPixels / 2;
		}
		else
		{
			if (ViewIndex == 0)
			{
				ViewRect.Max.X = (ViewRect.Min.X + ViewRect.Max.X) / 2;
			}
			else
			{
				ViewRect.Min.X = (ViewRect.Min.X + ViewRect.Max.X) / 2;
			}
			ViewRect.Min.Y += LetterboxPixels / 2;
			ViewRect.Max.Y -= LetterboxPixels / 2;
		}

		InitOptions.SetViewRectangle(ViewRect);

		// Set view family to dynamically allocated copy
		InitOptions.ViewFamily = ViewParents[ViewIndex];

		// Use new static view state for second view
		if (ViewIndex == 1)
		{
			FSceneViewState* SourceViewState = InFamily.Views[0]->State->GetConcreteViewState();
			
			if (SourceViewState->SplitScreenDebugViewState == nullptr)
			{
				SourceViewState->SplitScreenDebugViewState = new FSceneViewState(InFamily.GetFeatureLevel(), nullptr);
			}

			FSceneViewState* SecondViewState = SourceViewState->SplitScreenDebugViewState;

			// Propagate this user writable field between FSceneViewState
			SecondViewState->SequencerState = SourceViewState->SequencerState;

			// Add or remove optional Lumen scene for second view state
			if (CVarSplitScreenDebugLumenScene.GetValueOnGameThread())
			{
				SecondViewState->AddLumenSceneData(ViewParents[ViewIndex]->Scene, 1.0f);
			}
			else
			{
				SecondViewState->RemoveLumenSceneData(ViewParents[ViewIndex]->Scene);
			}

			InitOptions.SceneViewStateInterface = SecondViewState;
		}

		FSceneView* View = new FSceneView(InitOptions);
		View->PrimaryViewIndex = NumViewsPerFamily == 1 ? 0 : ViewIndex;
		View->FinalPostProcessSettings = InFamily.Views[0]->FinalPostProcessSettings;

		*ViewPointers[ViewIndex] = View;

		// Set up second view for multi-GPU
		if (NumFamilies > 1 && ViewIndex == 1)
		{
			FSceneViewFamily* Family = ViewParents[ViewIndex];

			// Prevent the render target from being cleared
			Family->bAdditionalViewFamily = true;

			if (GNumExplicitGPUsForRendering > 1)
			{
				// Enable cross GPU transfers
				Family->bMultiGPUForkAndJoin = true;

				// Set the view to run on the second GPU
				View->bOverrideGPUMask = true;
				View->GPUMask = FRHIGPUMask::FromIndex(ViewIndex);
			}
		}
	}

	// Copy the view families to the const output array
	for (FSceneViewFamily* Family : Families)
	{
		OutFamiliesStorage.Add(Family);
	}
	OutFamilies = OutFamiliesStorage;
}

static void DestroySplitScreenDebugViewFamilies(TConstArrayView<const FSceneViewFamily*> ViewFamilies)
{
	for (const FSceneViewFamily* Family : ViewFamilies)
	{
		for (int32 ViewIndex = 0; ViewIndex < Family->Views.Num(); ViewIndex++)
		{
			delete Family->Views[ViewIndex];
		}
		delete Family;
	}
}

#endif  // !UE_BUILD_SHIPPING

//////////////////////////////////////////////////////////////////////////

static int32 GSceneRenderCleanUpMode = 1;
static FAutoConsoleVariableRef CVarSceneRenderCleanUpMode(
	TEXT("r.SceneRender.CleanUpMode"),
	GSceneRenderCleanUpMode,
	TEXT("Controls when to perform clean up of the scene renderer.\n")
	TEXT(" 0: clean up is performed immediately after render on the render thread.\n")
	TEXT(" 1: clean up is performed asynchronously in a task. (default)\n"),
	ECVF_RenderThreadSafe
);

enum class ESceneRenderCleanUpMode : uint8
{
	Immediate,
	Async
};

inline ESceneRenderCleanUpMode GetSceneRenderCleanUpMode()
{
	if (IsRunningRHIInSeparateThread() && GSceneRenderCleanUpMode == 1)
	{
		return ESceneRenderCleanUpMode::Async;
	}
	return ESceneRenderCleanUpMode::Immediate;
}

//////////////////////////////////////////////////////////////////////////

/** This class is responsible for processing a batch of scene renderers. */
class FSceneRenderProcessor
{
	struct FRenderNode;

	struct FGroupNode : public TIntrusiveDoubleLinkedListNode<FGroupNode>
	{
		FGroupNode(FString&& InName, ESceneRenderGroupFlags InFlags)
			: Name(Forward<FString&&>(InName))
			, Flags(InFlags)
		{}

		TIntrusiveDoubleLinkedList<FRenderNode> RenderNodes;
		FString Name;
		ESceneRenderGroupFlags Flags;
	};

	struct FRenderNode : public TIntrusiveDoubleLinkedListNode<FRenderNode>
	{
		FRenderNode(FSceneRenderer* InRenderer, FString&& InName, FSceneRenderFunction&& InFunction, FGroupNode* InGroup)
			: Renderer(InRenderer)
			, Name(Forward<FString&&>(InName))
			, Function(Forward<FSceneRenderFunction&&>(InFunction))
			, Group(InGroup)
		{}

		FSceneRenderer* Renderer;
		FString Name;
		FSceneRenderFunction Function;
		FGroupNode* Group;
	};

	struct FOp
	{
		enum class EType
		{
			Render,
			FunctionCall,
			BeginGroup,
			EndGroup
		};

		static FOp Render(FRenderNode* Data)
		{
			check(Data);
			FOp Op;
			Op.Type = EType::Render;
			Op.Data_Render = Data;
			return Op;
		}

		static FOp FunctionCall(TUniqueFunction<void()>* Data)
		{
			check(Data);
			FOp Op;
			Op.Type = EType::FunctionCall;
			Op.Data_FunctionCall = Data;
			return Op;
		}

		static FOp BeginGroup(FGroupNode* Data)
		{
			check(Data);
			FOp Op;
			Op.Type = EType::BeginGroup;
			Op.Data_Group = Data;
			return Op;
		}
		
		static FOp EndGroup(FGroupNode* Data)
		{
			check(Data);
			FOp Op;
			Op.Type = EType::EndGroup;
			Op.Data_Group = Data;
			return Op;
		}

		EType Type;

		union
		{
			FRenderNode* Data_Render = nullptr;
			TUniqueFunction<void()>* Data_FunctionCall;
			FGroupNode* Data_Group;
		};
	};

	enum class FGroupEventLocation
	{
		GroupCommand,
		SceneRenderCommand
	};

	class FGroupEvent
	{
	public:
		void Begin(FGroupNode* Group, FRHICommandList& RHICmdList, FGroupEventLocation Location)
		{
#if WITH_RHI_BREADCRUMBS
			if (Group && IsLocationActive(Location))
			{
				// User specified an explicit group name.
				if (!Group->Name.IsEmpty())
				{
					Event.Emplace(RHICmdList, RHI_BREADCRUMB_DESC_FORWARD_VALUES(TEXT("SceneRenderGroup"), TEXT("%s"), RHI_GPU_STAT_ARGS_NONE)(Group->Name));
				}
				// User didn't specify a name, but we have more than one renderer, so a group event is useful.
				else if (!Group->RenderNodes.IsEmpty() && Group->RenderNodes.GetHead() != Group->RenderNodes.GetTail())
				{
					Event.Emplace(RHICmdList, RHI_BREADCRUMB_DESC_FORWARD_VALUES(TEXT("SceneRenderGroup"), nullptr, RHI_GPU_STAT_ARGS_NONE)());
				}
			}
#endif
		};

		void End(FRHICommandList& RHICmdList, FGroupEventLocation Location)
		{
#if WITH_RHI_BREADCRUMBS
			if (IsLocationActive(Location) && Event)
			{
				Event->End(RHICmdList);
				Event.Reset();
			}
#endif
		};

	private:
#if WITH_RHI_BREADCRUMBS
		bool IsLocationActive(FGroupEventLocation Location) const
		{
			// When the render command channel is enabled, we push a unique group scope for each scene render command.
			// Otherwise we push a scope inside of the group begin / end commands. This makes scopes behave properly.
			const bool bRenderCommandsChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(RenderCommandsChannel);
			const bool bGroupCommandLocation = Location == FGroupEventLocation::SceneRenderCommand;
			return (bRenderCommandsChannelEnabled == bGroupCommandLocation);
		};

		TOptional<FRHIBreadcrumbEventManual> Event;
#endif
	};

public:
	FSceneRenderProcessor(FScene* InScene)
		: Scene(InScene)
	{}

	TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> CreateSceneRenderers(
		TConstArrayView<FSceneViewFamily*> ViewFamilies,
		FHitProxyConsumer* HitProxyConsumer,
		bool bAllowSplitScreenDebug) const
	{
		if (!ViewFamilies.Num())
		{
			return {};
		}

	#if !UE_BUILD_SHIPPING
		bool bSplitScreenDebug = false;
		TArray<FSceneViewFamily*, TFixedAllocator<2>> DebugViewFamilyStorage;
		if (bAllowSplitScreenDebug && IsSplitScreenDebugEnabled(ViewFamilies))
		{
			CreateSplitScreenDebugViewFamilies(*ViewFamilies[0], ViewFamilies, DebugViewFamilyStorage);
			bSplitScreenDebug = true;
		}
	#endif

		const FSceneInterface* SceneInterface = ViewFamilies[0]->Scene;
		check(SceneInterface);
		check(SceneInterface->GetRenderScene() == Scene);

		const EShadingPath ShadingPath = GetFeatureLevelShadingPath(SceneInterface->GetFeatureLevel());

		TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> OutRenderers;
		OutRenderers.Reserve(ViewFamilies.Num());

		for (FSceneViewFamily* ViewFamily : ViewFamilies)
		{
			check(ViewFamily);
			check(ViewFamily->Scene == SceneInterface);

			for (auto& ViewExtension : ViewFamily->ViewExtensions)
			{
				ViewExtension->BeginRenderViewFamily(*ViewFamily);
			}

			if (ShadingPath == EShadingPath::Deferred)
			{
				OutRenderers.Add(new FDeferredShadingSceneRenderer(ViewFamily, HitProxyConsumer));
			}
			else
			{
				check(ShadingPath == EShadingPath::Mobile);
				OutRenderers.Add(new FMobileSceneRenderer(ViewFamily, HitProxyConsumer));
			}

			for (auto& ViewExtension : ViewFamily->ViewExtensions)
			{
				ViewExtension->PostCreateSceneRenderer(*ViewFamily, OutRenderers.Last());
			}
		}

		// Cache the FXSystem for the duration of the scene render
		// UWorld::CleanupWorldInternal() will mark the system as pending kill on the GameThread and then enqueue a delete command
		//-TODO: The call to IsPendingKill should no longer be required as we are caching & using within a single render command
		FFXSystemInterface* FXSystem = nullptr;
		if (Scene && Scene->FXSystem && !Scene->FXSystem->IsPendingKill())
		{
			FXSystem = Scene->FXSystem;
		}

		for (FSceneRenderer* Renderer : OutRenderers)
		{
			Renderer->Link.Head = OutRenderers[0];
			Renderer->FXSystem  = FXSystem;
		}

		for (int32 Index = 1; Index < OutRenderers.Num(); ++Index)
		{
			OutRenderers[Index - 1]->Link.Next = OutRenderers[Index];
		}

	#if !UE_BUILD_SHIPPING
		if (bSplitScreenDebug)
		{
			DestroySplitScreenDebugViewFamilies(ViewFamilies);
		}
	#endif

		return OutRenderers;
	}

	void AddCommand(TUniqueFunction<void()>&& Function)
	{
		Ops.Emplace(FOp::FunctionCall(Allocator.Create<TUniqueFunction<void()>>(MoveTemp(Function))));
	}

	void AddRenderer(FSceneRenderer* Renderer, FString&& Name, FSceneRenderFunction&& Function)
	{
		check(Renderer);
		check(Renderer->Scene == Scene);
		check(Renderer->ViewFamily.Views.Num() > 0);
		check(Renderer->ViewFamily.Views[0]);
		checkf(IsCompatible(Renderer->ViewFamily.EngineShowFlags),
			TEXT("Renderer contains show flags that are not compatible with other renderers that were previously added. ")
			TEXT("Use IsCompatible(...) to check if the show flags are compatible"));

		if (Renderers.IsEmpty())
		{
			CommonShowFlags |= Renderer->ViewFamily.EngineShowFlags.HitProxies ? ESceneRenderCommonShowFlags::HitProxies : ESceneRenderCommonShowFlags::None;
			CommonShowFlags |= Renderer->ViewFamily.EngineShowFlags.PathTracing ? ESceneRenderCommonShowFlags::PathTracing : ESceneRenderCommonShowFlags::None;
		}

		FGroupNode* TailGroup = GroupNodes.GetTail();

		FRenderNode* RenderNode = Allocator.Create<FRenderNode>(
			Renderer,
			Forward<FString&&>(Name),
			Forward<FSceneRenderFunction&&>(Function),
			TailGroup);

		if (TailGroup)
		{
			TailGroup->RenderNodes.AddTail(RenderNode);
		}

		Ops.Emplace(FOp::Render(RenderNode));
		Renderers.Emplace(Renderer);

		if (Renderer->ViewFamily.EngineShowFlags.Rendering)
		{
			ActiveRenderers.Emplace(Renderer);
			ActiveViewFamilies.Emplace(&Renderer->ViewFamily);
			ActiveViews.Reserve(ActiveViews.Num() + Renderer->Views.Num());
			for (FViewInfo& View : Renderer->Views)
			{
				ActiveViews.Emplace(&View);
			}
		}
	}

	void BeginGroup(FString&& Name, ESceneRenderGroupFlags Flags)
	{
		checkf(!bInsideGroup, TEXT("SceneRenderBuilderGroup scope %s is being nested with the group %s. Groups do not currently support nesting."), *GroupNodes.GetTail()->Name, *Name);
		FGroupNode* Group = Allocator.Create<FGroupNode>(Forward<FString&&>(Name), Flags);
		GroupNodes.AddTail(Group);
		Ops.Emplace(FOp::BeginGroup(Group));
		bInsideGroup = true;
	}

	void EndGroup()
	{
		checkf(bInsideGroup, TEXT("EndGroup called without a matching BeginGroup"));
		Ops.Emplace(FOp::EndGroup(GroupNodes.GetTail()));
		bInsideGroup = false;
	}

	void Execute();

	FConcurrentLinearBulkObjectAllocator& GetAllocator()
	{
		return Allocator;
	}

	bool IsCompatible(const FEngineShowFlags& EngineShowFlags)
	{
		if (Renderers.IsEmpty())
		{
			return true;
		}

		return EnumHasAnyFlags(CommonShowFlags, ESceneRenderCommonShowFlags::HitProxies)  == EngineShowFlags.HitProxies
			&& EnumHasAnyFlags(CommonShowFlags, ESceneRenderCommonShowFlags::PathTracing) == EngineShowFlags.PathTracing;
	}
	
	static void WaitForAsyncCleanupTask()
	{
		check(IsInRenderingThread());
		if (AsyncTasks.Cleanup)
		{
			AsyncTasks.Cleanup->Wait();
			AsyncTasks.Cleanup = {};
		}
	}

	static void WaitForAsyncDeleteTask()
	{
		check(IsInRenderingThread());
		if (AsyncTasks.Delete)
		{
			AsyncTasks.Delete->Wait();
			AsyncTasks = {};
		}
	}

	static const FGraphEventRef& GetAsyncCleanupTask()
	{
		return AsyncTasks.Cleanup;
	}

private:
	struct FAsyncTasks
	{
		FGraphEventRef Cleanup;
		FGraphEventRef Delete;
	};

	static FAsyncTasks AsyncTasks;

	void Cleanup(FRHICommandListImmediate& RHICmdList, FSceneRenderer* SceneRenderer);

	FScene* Scene;
	TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> Renderers;
	TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> ActiveRenderers;
	TArray<FViewFamilyInfo*, FConcurrentLinearArrayAllocator> ActiveViewFamilies;
	TArray<FViewInfo*, FConcurrentLinearArrayAllocator> ActiveViews;
	TIntrusiveDoubleLinkedList<FGroupNode> GroupNodes;
	TArray<FOp, FConcurrentLinearArrayAllocator> Ops;
	FConcurrentLinearBulkObjectAllocator Allocator;
	ESceneRenderCommonShowFlags CommonShowFlags = ESceneRenderCommonShowFlags::None;

	struct
	{
		FGroupNode* Group = nullptr;
		FGroupEvent GroupEvent;
		FString FullPath;
		bool bSceneUpdateConsumed = false;

	} RenderState;

	bool bInsideGroup : 1 = false;
};

FSceneRenderProcessor::FAsyncTasks FSceneRenderProcessor::AsyncTasks;

static void CleanupSceneRenderer(FSceneRenderer* Renderer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CleanupSceneRenderer);

	for (auto* Pass : Renderer->DispatchedShadowDepthPasses)
	{
		Pass->Cleanup();
	}

	for (FViewInfo* View : Renderer->AllViews)
	{
		View->Cleanup();
	}

	ViewSnapshotCache::Destroy();
}

void FSceneRenderProcessor::Cleanup(FRHICommandListImmediate& RHICmdList, FSceneRenderer* Renderer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Cleanup);

	// We need to sync async uniform expression cache updates since we're about to start deleting material proxies.
	FUniformExpressionCacheAsyncUpdateScope::WaitForTask();

	const ESceneRenderCleanUpMode SceneRenderCleanUpMode = GetSceneRenderCleanUpMode();

	if (SceneRenderCleanUpMode == ESceneRenderCleanUpMode::Async)
	{
		Renderer->GPUSceneDynamicContext.Release();

		// Wait for the last renderer's cleanup tasks so that snapshot deallocation and destruction don't overlap.
		if (AsyncTasks.Cleanup)
		{
			AsyncTasks.Cleanup->Wait(ENamedThreads::GetRenderThread_Local());
		}

		ViewSnapshotCache::Deallocate();

		AsyncTasks.Cleanup = FFunctionGraphTask::CreateAndDispatchWhenReady([Renderer]
		{
			CleanupSceneRenderer(Renderer);

		}, TStatId(), &GRHICommandList.WaitOutstandingTasks);
	}
	else // Immediate
	{
		WaitForAsyncDeleteTask(); // This is to handle cases where as switch from async to immediate.
		RHICmdList.ImmediateFlush(EImmediateFlushType::WaitForOutstandingTasksOnly);
		ViewSnapshotCache::Deallocate();
		CleanupSceneRenderer(Renderer);
	}

	GlobalDynamicBuffer::GarbageCollect();
	GPrimitiveIdVertexBufferPool.DiscardAll();
	FGraphicsMinimalPipelineStateId::ResetLocalPipelineIdTableSize();
}

void FSceneRenderProcessor::Execute()
{
	checkf(!bInsideGroup, TEXT("FSceneRenderBuilder::Execute called within scene render group scope %s. You must end the scope first."), *GroupNodes.GetTail()->Name);

#if WITH_GPUDEBUGCRASH
	if (GRHIGlobals.TriggerGPUCrash != ERequestedGPUCrash::None)
	{
		ENQUEUE_RENDER_COMMAND(ScheduleGPUDebugCrash)([] (FRHICommandListImmediate& RHICmdList)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(TriggerGPUCrash);
			FRDGBuilder GraphBuilder(RHICmdList);
			ScheduleGPUDebugCrash(GraphBuilder);
			GraphBuilder.Execute();
		});
	}
#endif

	UE::RenderCommandPipe::FSyncScope SyncScope;
	FUniformExpressionCacheAsyncUpdateScope AsyncUpdateScope;
	TOptional<UE::RenderCore::DumpGPU::FDumpScope> GpuDumpScope;
	TOptional<RenderCaptureInterface::FScopedCapture> GpuCaptureScope;

	for (FOp Op : Ops)
	{
		switch (Op.Type)
		{
		case FOp::EType::BeginGroup:
		{
			FGroupNode* Group = Op.Data_Group;

			if (EnumHasAnyFlags(Group->Flags, ESceneRenderGroupFlags::GpuCapture))
			{
				GpuCaptureScope.Emplace(true, *Group->Name);
			}

			if (EnumHasAnyFlags(Group->Flags, ESceneRenderGroupFlags::GpuDump))
			{
				GpuDumpScope.Emplace();
			}

			ENQUEUE_RENDER_COMMAND(SceneRenderBuilder_BeginGroup)([this, Group] (FRHICommandListImmediate& RHICmdList)
			{
				RenderState.Group = Group;
				RenderState.GroupEvent.Begin(Group, RHICmdList, FGroupEventLocation::GroupCommand);
				RenderState.FullPath += Group->Name;
			});
		}
		break;
		case FOp::EType::EndGroup:
		{
			ENQUEUE_RENDER_COMMAND(SceneRenderBuilder_EndGroup)([this](FRHICommandListImmediate& RHICmdList) mutable
			{
				RenderState.GroupEvent.End(RHICmdList, FGroupEventLocation::GroupCommand);
				RenderState.Group = nullptr;
				RenderState.FullPath.Reset();
			});

			GpuCaptureScope.Reset();
			GpuDumpScope.Reset();
		}
		break;
		case FOp::EType::FunctionCall:
		{
			(*Op.Data_FunctionCall)();
		}
		break;
		case FOp::EType::Render:
		{
			FRenderNode& RenderNode = *Op.Data_Render;

			ENQUEUE_RENDER_COMMAND(SceneRenderBuilder_Render)([this, &RenderNode] (FRHICommandListImmediate& RHICmdList)
			{
				LLM_SCOPE(ELLMTag::SceneRender);
				RenderState.GroupEvent.Begin(RenderState.Group, RHICmdList, FGroupEventLocation::SceneRenderCommand);

				ON_SCOPE_EXIT
				{
					RenderState.GroupEvent.End(RHICmdList, FGroupEventLocation::SceneRenderCommand);
				};

				RHI_BREADCRUMB_EVENT_CONDITIONAL_F(RHICmdList, !RenderNode.Name.IsEmpty(), "SceneRender", "SceneRender - %s", RenderNode.Name);
				RHI_BREADCRUMB_EVENT_CONDITIONAL(RHICmdList, RenderNode.Name.IsEmpty(), "SceneRender");

				FSceneRenderer& Renderer = *RenderNode.Renderer;
				FDeferredUpdateResource::UpdateResources(RHICmdList);

				if (!RenderNode.Name.IsEmpty())
				{
					RenderState.FullPath += TEXT("/");
					RenderState.FullPath += RenderNode.Name;
				}

				const bool bFirstRenderer = !ActiveRenderers.IsEmpty() && RenderNode.Renderer == ActiveRenderers[0];
				const bool bLastRenderer  = !ActiveRenderers.IsEmpty() && RenderNode.Renderer == ActiveRenderers.Last();

				TOptional<FSceneRenderUpdateInputs> SceneUpdateInputs;

				if (Renderer.ViewFamily.EngineShowFlags.Rendering && !RenderState.bSceneUpdateConsumed)
				{
					SceneUpdateInputs.Emplace();
					SceneUpdateInputs->Scene = Scene;
					SceneUpdateInputs->FXSystem = Scene->FXSystem;
					SceneUpdateInputs->FeatureLevel = Scene->GetFeatureLevel();
					SceneUpdateInputs->ShaderPlatform = Scene->GetShaderPlatform();
					SceneUpdateInputs->GlobalShaderMap = GetGlobalShaderMap(SceneUpdateInputs->ShaderPlatform);
					SceneUpdateInputs->Renderers = ActiveRenderers;
					SceneUpdateInputs->ViewFamilies = ActiveViewFamilies;
					SceneUpdateInputs->Views = ActiveViews;
					SceneUpdateInputs->CommonShowFlags = CommonShowFlags;
				}

				const FSceneRenderFunctionInputs FunctionInputs(&Renderer, SceneUpdateInputs ? &SceneUpdateInputs.GetValue() : nullptr, *RenderNode.Name, *RenderState.FullPath);

				FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("%s", FunctionInputs.FullPath), ERDGBuilderFlags::Parallel, Scene->GetShaderPlatform());
				FSceneRendererBase::SetActiveInstance(GraphBuilder, &Renderer);

			#if WITH_MGPU
				if (Renderer.ViewFamily.bForceCopyCrossGPU)
				{
					GraphBuilder.EnableForceCopyCrossGPU();
				}
			#endif

				if (!Renderer.ViewFamily.EngineShowFlags.HitProxies)
				{
					VISUALIZE_TEXTURE_BEGIN_VIEW(Scene->GetFeatureLevel(), Renderer.Views[0].GetViewKey(), FunctionInputs.FullPath, Renderer.Views[0].bIsSceneCapture, Renderer.Views[0].bIsReflectionCapture);
				}
	
				bool bRenderCalled = false;

				if (Renderer.ViewFamily.EngineShowFlags.Rendering)
				{
					bRenderCalled = RenderNode.Function(GraphBuilder, FunctionInputs);
				}

				if (SceneUpdateInputs)
				{
					RenderState.bSceneUpdateConsumed |= bRenderCalled;
				}

				if (!Renderer.ViewFamily.EngineShowFlags.HitProxies)
				{
					VISUALIZE_TEXTURE_END_VIEW();
				}

				if (!RenderNode.Name.IsEmpty())
				{
					RenderState.FullPath.LeftChopInline(RenderNode.Name.Len() + 1, EAllowShrinking::No);
				}

				// The final graph builder is responsible for flushing resources.
				if (RenderNode.Renderer == Renderers.Last())
				{
					GraphBuilder.SetFlushResourcesRHI();
				}

				GraphBuilder.Execute();

				Cleanup(RHICmdList, &Renderer);
			});
		}
		break;
		}
	}

	ENQUEUE_RENDER_COMMAND(SceneRenderBuilder_End)([this](FRHICommandListImmediate& RHICmdList) mutable
	{
		const auto DeleteLambda = [this]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FSceneRenderProcessor::DeleteSceneRenderers);
			for (FSceneRenderer* Renderer : Renderers)
			{
				delete Renderer;
			}
			delete this;
		};

		if (GetSceneRenderCleanUpMode() == ESceneRenderCleanUpMode::Async)
		{
			FGraphEventArray Prereqs;
			Prereqs.Add(AsyncTasks.Cleanup);
			Prereqs.Add(AsyncTasks.Delete);

			AsyncTasks.Delete = FFunctionGraphTask::CreateAndDispatchWhenReady(DeleteLambda, TStatId(), &Prereqs);
		}
		else
		{
			DeleteLambda();
		}
	});

	// NOTE: 'this' is queued for deletion and is no longer valid!
}

//////////////////////////////////////////////////////////////////////////////

struct FSceneRenderBuilder::FPersistentState
{
#if DO_CHECK
	TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> RenderersToAdd;
#endif
};

TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> FSceneRenderBuilder::CreateSceneRenderers(
	TConstArrayView<FSceneViewFamily*> ViewFamilies,
	FHitProxyConsumer* HitProxyConsumer,
	bool bAllowSplitScreenDebug)
{
	LazyInit();
	TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> Renderers = Processor->CreateSceneRenderers(ViewFamilies, HitProxyConsumer, bAllowSplitScreenDebug);

#if DO_CHECK
	PersistentState->RenderersToAdd.Append(Renderers);
#endif

	return Renderers;
}

FSceneRenderer* FSceneRenderBuilder::CreateSceneRenderer(FSceneViewFamily* ViewFamily)
{
	const bool bAllowSplitScreenDebug = false;
	return CreateSceneRenderers(MakeArrayView({ ViewFamily }), nullptr, bAllowSplitScreenDebug)[0];
}

TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> FSceneRenderBuilder::CreateLinkedSceneRenderers(
	TConstArrayView<FSceneViewFamily*> ViewFamilies,
	FHitProxyConsumer* HitProxyConsumer)
{
	const bool bAllowSplitScreenDebug = true;
	return CreateSceneRenderers(ViewFamilies, HitProxyConsumer, bAllowSplitScreenDebug);
}

FSceneRenderBuilder::FSceneRenderBuilder(FSceneInterface* InScene)
	: Scene(InScene->GetRenderScene())
	, PersistentState(new FPersistentState)
{}

FSceneRenderBuilder::~FSceneRenderBuilder()
{
	if (PersistentState)
	{
#if DO_CHECK
		checkf(PersistentState->RenderersToAdd.IsEmpty(), TEXT("FSceneRenderBuilder::Execute called but %d renderers were not added."), PersistentState->RenderersToAdd.Num());
#endif
		delete PersistentState;
		PersistentState = nullptr;
	}

	if (Processor)
	{
	#if !USE_NULL_RHI
		checkf(false, TEXT("SceneRenderBuilder is being destructed without having called Execute."));
	#endif

		delete Processor;
		Processor = nullptr;
	}
}

void FSceneRenderBuilder::LazyInit()
{
	if (!Processor)
	{
		Processor = new FSceneRenderProcessor(Scene);
	}
}

void FSceneRenderBuilder::AddCommand(TUniqueFunction<void()>&& Function)
{
#if !USE_NULL_RHI
	LazyInit();
	Processor->AddCommand(MoveTemp(Function));
#endif
}

void FSceneRenderBuilder::AddRenderer(FSceneRenderer* Renderer, FString&& Name, FSceneRenderFunction&& Function)
{
#if !USE_NULL_RHI
	LazyInit();

#if DO_CHECK
	bool bFoundRenderer = false;
	for (int32 Index = 0; Index < PersistentState->RenderersToAdd.Num(); ++Index)
	{
		if (PersistentState->RenderersToAdd[Index] == Renderer)
		{
			PersistentState->RenderersToAdd.RemoveAtSwap(Index, EAllowShrinking::No);
			bFoundRenderer = true;
			break;
		}
	}
	checkf(bFoundRenderer, TEXT("Renderer being added was not created with this scene render builder or is being added twice."));
#endif

	Processor->AddRenderer(Renderer, Forward<FString&&>(Name), MoveTemp(Function));
#endif
}

void FSceneRenderBuilder::BeginGroup(FString&& Name, ESceneRenderGroupFlags Flags)
{
#if !USE_NULL_RHI
	// If user sets both capture and dump flags, prefer capturing over dumping (or clear flag if dumping is not available or we are currently dumping already).
	if (EnumHasAnyFlags(Flags, ESceneRenderGroupFlags::GpuCapture) || !(WITH_ENGINE && WITH_DUMPGPU) || FRDGBuilder::IsDumpingFrame())
	{
		EnumRemoveFlags(Flags, ESceneRenderGroupFlags::GpuDump);
	}

	LazyInit();
	Processor->BeginGroup(Forward<FString&&>(Name), Flags);
#endif
}

void FSceneRenderBuilder::EndGroup()
{
#if !USE_NULL_RHI
	checkf(Processor, TEXT("EndGroup called on an empty scene render builder."));
	Processor->EndGroup();
#endif
}

void FSceneRenderBuilder::Execute()
{
#if !USE_NULL_RHI
	if (Processor)
	{
		Processor->Execute();
		Processor = nullptr;
	}
#endif
}

FConcurrentLinearBulkObjectAllocator& FSceneRenderBuilder::GetAllocator()
{
	LazyInit();
	return Processor->GetAllocator();
}

bool FSceneRenderBuilder::IsCompatible(const FEngineShowFlags& EngineShowFlags) const
{
	if (Processor)
	{
		return Processor->IsCompatible(EngineShowFlags);
	}
	return true;
}

void FSceneRenderBuilder::WaitForAsyncDeleteTask()
{
	FSceneRenderProcessor::WaitForAsyncDeleteTask();
}

void FSceneRenderBuilder::WaitForAsyncCleanupTask()
{
	FSceneRenderProcessor::WaitForAsyncCleanupTask();
}

const FGraphEventRef& FSceneRenderBuilder::GetAsyncCleanupTask()
{
	return FSceneRenderProcessor::GetAsyncCleanupTask();
}