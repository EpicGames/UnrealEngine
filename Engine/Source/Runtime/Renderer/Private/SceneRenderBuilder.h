// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneRenderBuilderInterface.h"
#include "Templates/PimplPtr.h"

class FSceneRenderProcessor;
class FSceneRenderer;
class FScene;

class FSceneRenderBuilder final : public ISceneRenderBuilder
{
public:
	RENDERER_API FSceneRenderBuilder(FSceneInterface* InScene);
	FSceneRenderBuilder(const FSceneRenderBuilder&) = delete;
	FSceneRenderBuilder& operator=(const FSceneRenderBuilder&) = delete;

	FSceneRenderBuilder(FSceneRenderBuilder&& RHS)
	{
		*this = MoveTemp(RHS);
	}

	FSceneRenderBuilder& operator=(FSceneRenderBuilder&& RHS)
	{
		// Scene stays persistent while the processor instance moves.
		Scene               = RHS.Scene;
		Processor           = RHS.Processor;
		PersistentState     = RHS.PersistentState;
		RHS.Processor       = nullptr;
		RHS.PersistentState = nullptr;
		return *this;
	}

	RENDERER_API ~FSceneRenderBuilder() override;

	//////////////////////////////////////////////////////////////////////////
	// ISceneRenderBuilder Overrides

	RENDERER_API FSceneRenderer* CreateSceneRenderer(FSceneViewFamily* ViewFamily) override;
	RENDERER_API TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> CreateLinkedSceneRenderers(
		TConstArrayView<FSceneViewFamily*> ViewFamilies,
		FHitProxyConsumer* HitProxyConsumer) override;
	RENDERER_API void AddCommand(TUniqueFunction<void()>&& Function) override;
	RENDERER_API void AddRenderer(FSceneRenderer* Renderer, FString&& Name, FSceneRenderFunction&& Function) override;
	using ISceneRenderBuilder::AddRenderer;
	RENDERER_API bool IsCompatible(const FEngineShowFlags& EngineShowFlags) const override;
	RENDERER_API void Execute() override;
	RENDERER_API FConcurrentLinearBulkObjectAllocator& GetAllocator() override;

	//////////////////////////////////////////////////////////////////////////

	void FlushIfIncompatible(const FEngineShowFlags& EngineShowFlags)
	{
		if (!IsCompatible(EngineShowFlags))
		{
			Execute();
		}
	}

	static void WaitForAsyncCleanupTask();
	static void WaitForAsyncDeleteTask();

	static const FGraphEventRef& GetAsyncCleanupTask();

private:
	void LazyInit();
	void BeginGroup(FString&& Name, ESceneRenderGroupFlags Flags) override;
	void EndGroup() override;

	TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> CreateSceneRenderers(
		TConstArrayView<FSceneViewFamily*> ViewFamilies,
		FHitProxyConsumer* HitProxyConsumer,
		bool bAllowSplitScreenDebug);

	struct FPersistentState;

	FScene* Scene;
	FSceneRenderProcessor* Processor = nullptr;
	FPersistentState* PersistentState = nullptr;
};