// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneRendering.cpp: Scene rendering.
=============================================================================*/

#include "SceneRendering.h"
#include "CoreMinimal.h"
#include "Async/Mutex.h"
#include "Async/TaskGraphInterfaces.h"
#include "BatchedElements.h"
#include "BlueNoise.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "CommonRenderResources.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Components/ReflectionCaptureComponent.h"
#include "Containers/AllowShrinking.h"
#include "Containers/ArrayView.h"
#include "Containers/ChunkedArray.h"
#include "Containers/IndirectArray.h"
#include "Containers/SetUtilities.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "ContentStreaming.h"
#include "CustomRenderPassSceneCapture.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DebugViewModeHelpers.h"
#include "DecalRenderingShared.h"
#include "DepthRendering.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DistanceFieldLightingShared.h"
#include "DumpGPU.h"
#include "DynamicBufferAllocator.h"
#include "DynamicPrimitiveDrawing.h"
#include "DynamicRenderScaling.h"
#include "DynamicResolutionState.h"
#include "DynamicRHI.h"
#include "Engine/Engine.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/EngineTypes.h"
#include "Engine/Font.h"
#include "Engine/Scene.h"
#include "Engine/SpecularProfile.h"
#include "Engine/StreamableRenderAsset.h"
#include "Engine/SubsurfaceProfile.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureDefines.h"
#include "Engine/TextureLODSettings.h"
#include "Engine/ToonProfile.h"
#include "Engine/VolumeTexture.h"
#include "Engine/World.h"
#include "EnvironmentComponentsFlags.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Field/FieldSystemTypes.h"
#include "FinalPostProcessSettings.h"
#include "FrontLayerTranslucency.h"
#include "Froxel/FroxelGridUtils.h"
#include "FXSystem.h"
#include "GameTime.h"
#include "GlobalDistanceFieldParameters.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "GPUScene.h"
#include "GPUSkinCache.h"
#include "GPUSkinCacheVisualizationData.h"
#include "GroomVisualizationData.h"
#include "HairStrands/HairStrandsRendering.h"
#include "HairStrands/HairStrandsVisibility.h"
#include "HairStrandsInterface.h"
#include "HAL/CriticalSection.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HdrCustomResolveShaders.h"
#include "HitProxies.h"
#include "IESTextureManager.h"
#include "IHeadMountedDisplay.h"
#include "InstanceCulling/InstanceCullingContext.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "IXRCamera.h"
#include "IXRTrackingSystem.h"
#include "LegacyScreenPercentageDriver.h"
#include "LightFunctionAtlas.h"
#include "LightSceneInfo.h"
#include "LightSceneProxy.h"
#include "LocalFogVolumeRendering.h"
#include "Logging/LogMacros.h"
#include "LogRenderer.h"
#include "Lumen/Lumen.h"
#include "MaterialCache/MaterialCacheTagProvider.h"
#include "Materials/MaterialRenderProxy.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "Math/DoubleFloat.h"
#include "Math/SHMath.h"
#include "Math/UnrealPlatformMathSSE.h"
#include "MegaFiltersInterface.h"
#include "MegaLights/MegaLights.h"
#include "MeshBatch.h"
#include "MeshDrawCommands.h"
#include "MeshDrawCommandStats.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.h"
#include "Misc/AutomationTest.h"
#include "Misc/AlignedElement.h"
#include "Misc/App.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Misc/NotNull.h"
#include "Misc/ScopeLock.h"
#include "MultiGPU.h"
#include "Nanite/NaniteRayTracing.h"
#include "Nanite/NaniteShared.h"
#include "NaniteVisualizationData.h"
#include "OIT/OIT.h"
#include "PathTracing.h"
#include "PhysicsField/PhysicsFieldComponent.h"
#include "PipelineStateCache.h"
#include "PostProcess/DiaphragmDOF.h"
#include "PostProcess/LensDistortion.h"
#include "PostProcess/PostProcessEyeAdaptation.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "PostProcess/PostProcessTonemap.h"
#include "PostProcess/PostProcessUpscale.h"
#include "PostProcess/SceneFilterRendering.h"
#include "PrecomputedVolumetricLightmap.h"
#include "PrimitiveSceneInfo.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveSceneShaderData.h"
#include "PrimitiveViewRelevance.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "ProfilingDebugging/TagTrace.h"
#include "Quantization.h"
#include "RayTracing/RayTracing.h"
#include "RayTracing/RaytracingOptions.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracingGeometryManagerInterface.h"
#include "ReadOnlyCVARCache.h"
#include "RectLightTextureManager.h"
#include "ReflectionEnvironment.h"
#include "RenderCommandTag.h"
#include "RenderCore.h"
#include "RenderCounters.h"
#include "Renderer/ViewSnapshotCache.h"
#include "RendererInterface.h"
#include "RendererModule.h"
#include "RendererOnScreenNotification.h"
#include "RendererUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "Rendering/CustomRenderPass.h"
#include "Rendering/NaniteCoarseMeshStreamingManager.h"
#include "Rendering/NaniteResources.h"
#include "Rendering/NaniteStreamingManager.h"
#include "Rendering/SkyAtmosphereCommonData.h"
#if WITH_EDITOR
#include "Rendering/StaticLightingSystemInterface.h"
#endif
#include "RenderingThread.h"
#include "RenderResource.h"
#include "RenderUtils.h"
#include "RenderViewportFeedback.h"
#include "ResolveShader.h"
#include "RHI.h"
#include "RHIBreadcrumbs.h"
#include "RHICommandList.h"
#include "RHIContext.h"
#include "RHIPipeline.h"
#include "RHIResources.h"
#include "RHIShaderParameters.h"
#include "RHIStaticStates.h"
#include "RHIUtilities.h"
#include "SceneCapture/SceneCaptureInternal.h"
#include "SceneCore.h"
#include "SceneCulling/SceneCullingRenderer.h"
#include "SceneExtensions.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "ScenePrivate.h"
#include "SceneProxies/SkyAtmosphereSceneProxy.h"
#include "SceneProxies/SkyLightSceneProxy.h"
#include "SceneRenderBuilder.h"
#include "SceneRenderBuilderInterface.h"
#include "SceneRendererInterface.h"
#include "SceneTextures.h"
#include "SceneTexturesConfig.h"
#include "SceneUniformBuffer.h"
#include "SceneUtils.h"
#include "SceneView.h"
#include "SceneViewExtension.h"
#include "SceneVisibility.h"
#include "ScreenPass.h"
#include "Shader.h"
#include "ShaderCompiler.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterUtils.h"
#include "ShaderPrintParameters.h"
#include "ShadingEnergyConservation.h"
#include "ShadowRendering.h"
#include "Shadows/ShadowSceneRenderer.h"
#include "ShowFlags.h"
#include "SingleLayerWaterRendering.h"
#include "SkyAtmosphereRendering.h"
#include "Stats/Stats.h"
#include "StereoRendering.h"
#include "Substrate/Glint/GlintShadingLUTs.h"
#include "Substrate/Substrate.h"
#include "SystemTextures.h"
#include "Tasks/Task.h"
#include "TemporalUpscaler.h"
#include "TextureFallbacks.h"
#include "TextureResource.h"
#include "TranslucentLighting.h"
#include "UniformBuffer.h"
#include "UnrealClient.h"
#include "UObject/ObjectPtr.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "VirtualShadowMaps/VirtualShadowMapCacheManager.h"
#include "VirtualShadowMaps/VirtualShadowMapClipmap.h"
#include "VirtualTextureEnum.h"
#include "VirtualTexturing.h"
#include "VisualizeTexturePresent.h"
#include "VolumetricCloudProxy.h"
#include "VolumetricCloudRendering.h"
#include "VolumetricFog.h"
#include "VolumetricRenderTargetViewStateData.h"
#include "VT/VirtualTextureFeedbackResource.h"
#include "VT/VirtualTextureProducer.h"
#include "VT/VirtualTextureScalability.h"
#include "VT/VirtualTextureSystem.h"
#include "WideCustomResolveShaders.h"
#include "SceneViewState.h"
#include <type_traits>
#include "MaterialCache/IMaterialCacheVirtualTextureAllocation.h"
#include "MaterialCache/MaterialCacheVirtualTextureAllocator.h"

/*-----------------------------------------------------------------------------
	Globals
-----------------------------------------------------------------------------*/

DEFINE_LOG_CATEGORY(LogSceneCapture);

// This is an experimental optimization switch to render pre-pass depth for scene capture without calling the entire FDeferredShadingSceneRenderer::Render()
int32 GSceneCaptureDepthPrepassOptimization = 0;
static FAutoConsoleVariableRef CVarSceneCaptureDepthPrepassOptimization(
	TEXT("r.SceneCapture.DepthPrepassOptimization"),
	GSceneCaptureDepthPrepassOptimization,
	TEXT("Whether to apply optimized render path when capturing depth prepass for scene capture 2D. Experimental!\n"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static int32 GAsyncCreateLightPrimitiveInteractions = 1;
static FAutoConsoleVariableRef CVarAsyncCreateLightPrimitiveInteractions(
	TEXT("r.AsyncCreateLightPrimitiveInteractions"),
	GAsyncCreateLightPrimitiveInteractions,
	TEXT("Light primitive interactions are created off the render thread in an async task."),
	ECVF_RenderThreadSafe);

static int32 GAsyncCacheMeshDrawCommands = 1;
static FAutoConsoleVariableRef CVarAsyncMeshDrawCommands(
	TEXT("r.AsyncCacheMeshDrawCommands"),
	GAsyncCacheMeshDrawCommands,
	TEXT("Mesh draw command caching is offloaded to an async task."),
	ECVF_RenderThreadSafe);

static int32 GAsyncCacheMaterialUniformExpressions = 1;
static FAutoConsoleVariableRef CVarAsyncMaterialUniformExpressions(
	TEXT("r.AsyncCacheMaterialUniformExpressions"),
	GAsyncCacheMaterialUniformExpressions,
	TEXT("Material uniform expression caching is offloaded to an async task."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarCachedMeshDrawCommands(
	TEXT("r.MeshDrawCommands.UseCachedCommands"),
	1,
	TEXT("Whether to render from cached mesh draw commands (on vertex factories that support it), or to generate draw commands every frame."),
	ECVF_RenderThreadSafe);

bool UseCachedMeshDrawCommands()
{
	return CVarCachedMeshDrawCommands.GetValueOnRenderThread() > 0;
}

bool UseCachedMeshDrawCommands_AnyThread()
{
	return CVarCachedMeshDrawCommands.GetValueOnAnyThread() > 0;
}

static TAutoConsoleVariable<int32> CVarMeshDrawCommandsDynamicInstancing(
	TEXT("r.MeshDrawCommands.DynamicInstancing"),
	1,
	TEXT("Whether to dynamically combine multiple compatible visible Mesh Draw Commands into one instanced draw on vertex factories that support it."),
	ECVF_RenderThreadSafe);

bool FSceneCaptureLogUtils::bEnableSceneCaptureLogging = false;

FAutoConsoleVariableRef CVarEnableSceneCaptureLogging(
	TEXT("r.SceneCapture.EnableLogging"),
	FSceneCaptureLogUtils::bEnableSceneCaptureLogging,
	TEXT("Enable logging of scene captures."));

bool IsDynamicInstancingEnabled(ERHIFeatureLevel::Type FeatureLevel)
{
	return CVarMeshDrawCommandsDynamicInstancing.GetValueOnRenderThread() > 0
		&& UseGPUScene(GShaderPlatformForFeatureLevel[FeatureLevel]);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
int32 GetMaxNumReflectionCaptures(EShaderPlatform ShaderPlatform)
{
	return IsMobilePlatform(ShaderPlatform) ? GMobileMaxNumReflectionCaptures : GMaxNumReflectionCaptures;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

int32 GDumpInstancingStats = 0;
FAutoConsoleVariableRef CVarDumpInstancingStats(
	TEXT("r.MeshDrawCommands.LogDynamicInstancingStats"),
	GDumpInstancingStats,
	TEXT("Whether to log dynamic instancing stats on the next frame"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

int32 GDumpMeshDrawCommandMemoryStats = 0;
FAutoConsoleVariableRef CVarDumpMeshDrawCommandMemoryStats(
	TEXT("r.MeshDrawCommands.LogMeshDrawCommandMemoryStats"),
	GDumpMeshDrawCommandMemoryStats,
	TEXT("Whether to log mesh draw command memory stats on the next frame"),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<float> CVarDemosaicVposOffset(
	TEXT("r.DemosaicVposOffset"),
	0.0f,
	TEXT("This offset is added to the rasterized position used for demosaic in the mobile tonemapping shader. It exists to workaround driver bugs on some Android devices that have a half-pixel offset."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarDecalDepthBias(
	TEXT("r.DecalDepthBias"),
	0.005f,
	TEXT("Global depth bias used by mesh decals. Default is 0.005 for perspective. Scaled by the PerProjectionDepthThicknessScale for Ortho"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRefractionQuality(
	TEXT("r.RefractionQuality"),
	2,
	TEXT("Defines the distorion/refraction quality which allows to adjust for quality or performance.\n")
	TEXT("<=0: off (fastest)\n")
	TEXT("  1: low quality (not yet implemented)\n")
	TEXT("  2: normal quality (default)\n")
	TEXT("  3: high quality (e.g. color fringe, not yet implemented)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

// HZB reallocate it's RT when dynamic resolution changes. While this behaviour needs to be addressed we don't want to pay the allocation cost on some platforms keeping the size fixed.
#if PLATFORM_DESKTOP
static constexpr int32 GHZBUseUpperBoundViewRectDefault = 1;
#else
static constexpr int32 GHZBUseUpperBoundViewRectDefault = 0;
#endif

static TAutoConsoleVariable<int32> CVarHZBUseUpperBoundViewRect(
	TEXT("r.HZB.UseUpperBoundViewRect"),
	GHZBUseUpperBoundViewRectDefault,
	TEXT(" 0: HZB sized to View.ViewRect (variable, smaller HZB, reallocation cost).\n")
	TEXT(" 1: HZB sized to View.UpperBoundViewRect (fixed, larger HZB, no reallocation cost).\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarInstancedStereo(
	TEXT("vr.InstancedStereo"),
	0,
	TEXT("0 to disable instanced stereo (default), 1 to enable."),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMobileMultiView(
	TEXT("vr.MobileMultiView"),
	0,
	TEXT("0 to disable mobile multi-view, 1 to enable.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarRoundRobinOcclusion(
	TEXT("vr.RoundRobinOcclusion"),
	0,
	TEXT("0 to disable round-robin occlusion queries for stereo rendering (default), 1 to enable."),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarViewRectUseScreenBottom(
	TEXT("r.ViewRectUseScreenBottom"),
	0,
	TEXT("WARNING: This is an experimental, unsupported feature and does not work with all postprocesses (e.g DOF and DFAO)\n")
	TEXT("If enabled, the view rectangle will use the bottom left corner instead of top left"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingSceneUpdateOnce(
	TEXT("r.RayTracing.SceneUpdateOnce"),
	0,
	TEXT("Experimental:  Improves GPU perf by updating ray tracing scene once, but may cause artifacts (mainly for nDisplay)\n"),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarAllowTranslucencyAfterDOF(
	TEXT("r.SeparateTranslucency"),
	1,
	TEXT("Allows to disable the separate translucency feature (all translucency is rendered in separate RT and composited\n")
	TEXT("after DOF, if not specified otherwise in the material).\n")
	TEXT(" 0: off (translucency is affected by depth of field)\n")
	TEXT(" 1: on costs GPU performance and memory but keeps translucency unaffected by Depth of Field. (default)"),
	ECVF_Scalability | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarTranslucencyStandardSeparated(
	TEXT("r.Translucency.StandardSeparated"),
	0,
	TEXT("Render translucent meshes in separate buffer from the scene color.\n")
	TEXT("This prevent those meshes from self refracting and leaking scnee color behind over edges when it should be affect by colored transmittance.\n")
	TEXT("Forced disabled when r.SeparateTranslucency is 0.\n"),
	ECVF_RenderThreadSafe | ECVF_Default);

static TAutoConsoleVariable<int32> CVarTSRForceSeparateTranslucency(
	TEXT("r.TSR.ForceSeparateTranslucency"), 1,
	TEXT("Overrides r.SeparateTranslucency whenever TSR is enabled (enabled by default).\n"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarViewHasTileOffsetData(
	TEXT("r.ViewHasTileOffsetData"),
	1,
	TEXT("1 to upload lower-precision tileoffset view data to gpu, 0 to use only higher-precision double float.\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarPrimitiveAlphaHoldoutSupport(
	TEXT("r.Deferred.SupportPrimitiveAlphaHoldout"),
	false,
	TEXT("True to enable deferred renderer support for primitive alpha holdout (disabled by default).\n"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
static TAutoConsoleVariable<float> CVarGeneralPurposeTweak(
	TEXT("r.GeneralPurposeTweak"),
	1.0f,
	TEXT("Useful for low level shader development to get quick iteration time without having to change any c++ code.\n")
	TEXT("Value maps to Frame.GeneralPurposeTweak inside the shaders.\n")
	TEXT("Example usage: Multiplier on some value to tweak, toggle to switch between different algorithms (Default: 1.0)\n")
	TEXT("DON'T USE THIS FOR ANYTHING THAT IS CHECKED IN. Compiled out in SHIPPING to make cheating a bit harder."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarGeneralPurposeTweak2(
	TEXT("r.GeneralPurposeTweak2"),
	1.0f,
	TEXT("Useful for low level shader development to get quick iteration time without having to change any c++ code.\n")
	TEXT("Value maps to Frame.GeneralPurposeTweak2 inside the shaders.\n")
	TEXT("Example usage: Multiplier on some value to tweak, toggle to switch between different algorithms (Default: 1.0)\n")
	TEXT("DON'T USE THIS FOR ANYTHING THAT IS CHECKED IN. Compiled out in SHIPPING to make cheating a bit harder."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDisplayInternals(
	TEXT("r.DisplayInternals"),
	0,
	TEXT("Allows to enable screen printouts that show the internals on the engine/renderer\n")
	TEXT("This is mostly useful to be able to reason why a screenshots looks different.\n")
	TEXT(" 0: off (default)\n")
	TEXT(" 1: enabled"),
	ECVF_RenderThreadSafe | ECVF_Cheat);
#endif

/**
 * Console variable controlling the maximum number of shadow cascades to render with.
 *   DO NOT READ ON THE RENDERING THREAD. Use FSceneView::MaxShadowCascades.
 */
static TAutoConsoleVariable<int32> CVarMaxShadowCascades(
	TEXT("r.Shadow.CSM.MaxCascades"),
	10,
	TEXT("The maximum number of cascades with which to render dynamic directional light shadows."),
	ECVF_Scalability | ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<float> CVarNormalCurvatureToRoughnessBias(
	TEXT("r.NormalCurvatureToRoughnessBias"),
	0.0f,
	TEXT("Biases the roughness resulting from screen space normal changes for materials with NormalCurvatureToRoughness enabled.  Valid range [-1, 1]"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarNormalCurvatureToRoughnessExponent(
	TEXT("r.NormalCurvatureToRoughnessExponent"),
	0.333f,
	TEXT("Exponent on the roughness resulting from screen space normal changes for materials with NormalCurvatureToRoughness enabled."),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<float> CVarNormalCurvatureToRoughnessScale(
	TEXT("r.NormalCurvatureToRoughnessScale"),
	1.0f,
	TEXT("Scales the roughness resulting from screen space normal changes for materials with NormalCurvatureToRoughness enabled.  Valid range [0, 2]"),
	ECVF_RenderThreadSafe | ECVF_Scalability);

static TAutoConsoleVariable<int32> CVarEnableMultiGPUForkAndJoin(
	TEXT("r.EnableMultiGPUForkAndJoin"),
	1,
	TEXT("Whether to allow unused GPUs to speedup rendering by sharing work.\n"),
	ECVF_Default
	);

static TAutoConsoleVariable<float> CVarLensDistortionAffectScreenPercentage(
	TEXT("r.LensDistortion.AffectScreenPercentage"),
	0.0f,
	TEXT("Whether the screen percentage is automatically increased to avoid any upscaling due to the distortion. Disabled by default as this affect render target sizes, and is dependent of the upscaling factor that migth be animated (different FOV or distortion settings for instance)."),
	ECVF_RenderThreadSafe);

/*-----------------------------------------------------------------------------
	FParallelCommandListSet
-----------------------------------------------------------------------------*/

TAutoConsoleVariable<int32> CVarRHICmdMinDrawsPerParallelCmdList(
	TEXT("r.RHICmdMinDrawsPerParallelCmdList"),
	64,
	TEXT("The minimum number of draws per cmdlist. If the total number of draws is less than this, then no parallel work will be done at all. This can't always be honored or done correctly."));

static TAutoConsoleVariable<int32> CVarWideCustomResolve(
	TEXT("r.WideCustomResolve"),
	0,
	TEXT("Use a wide custom resolve filter when MSAA is enabled")
	TEXT("0: Disabled [hardware box filter]")
	TEXT("1: Wide (r=1.25, 12 samples)")
	TEXT("2: Wider (r=1.4, 16 samples)")
	TEXT("3: Widest (r=1.5, 20 samples)"),
	ECVF_RenderThreadSafe | ECVF_Scalability
	);

static TAutoConsoleVariable<int32> CVarFilmGrain(
	TEXT("r.FilmGrain"), 1,
	TEXT("Whether to enable film grain."),
	ECVF_RenderThreadSafe);

#if !UE_BUILD_SHIPPING

static TAutoConsoleVariable<int32> CVarTestInternalViewRectOffset(
	TEXT("r.Test.ViewRectOffset"),
	0,
	TEXT("Moves the view rect within the renderer's internal render target.\n")
	TEXT(" 0: disabled (default);"));

static TAutoConsoleVariable<int32> CVarTestCameraCut(
	TEXT("r.Test.CameraCut"),
	0,
	TEXT("Force enabling camera cut for testing purposes.\n")
	TEXT(" 0: disabled (default); 1: enabled."));

static TAutoConsoleVariable<float> CVarTestViewRollAngle(
	TEXT("r.Test.ViewRollAngle"), 0.0f,
	TEXT("Roll the camera in degrees, for testing motion vector upscaling precision. (disabled by default)"));

static TAutoConsoleVariable<int32> CVarTestScreenPercentageInterface(
	TEXT("r.Test.DynamicResolutionHell"),
	0,
	TEXT("Override the screen percentage interface for all view family with dynamic resolution hell.\n")
	TEXT(" 0: off (default);\n")
	TEXT(" 1: Dynamic resolution hell."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarTestPrimaryScreenPercentageMethodOverride(
	TEXT("r.Test.PrimaryScreenPercentageMethodOverride"),
	0,
	TEXT("Override the screen percentage method for all view family.\n")
	TEXT(" 0: view family's screen percentage interface choose; (default)\n")
	TEXT(" 1: old fashion upscaling pass at the very end right before before UI;\n")
	TEXT(" 2: TemporalAA upsample."));

static TAutoConsoleVariable<int32> CVarTestSecondaryUpscaleOverride(
	TEXT("r.Test.SecondaryUpscaleOverride"),
	0,
	TEXT("Override the secondary upscale.\n")
	TEXT(" 0: disabled; (default)\n")
	TEXT(" 1: use secondary view fraction = 0.5 with nearest secondary upscale."));

#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

static TAutoConsoleVariable<int32> CVarNaniteShowUnsupportedError(
	TEXT("r.Nanite.ShowUnsupportedError"),
	1,
	TEXT("Specify behavior of Nanite unsupported screen error message.\n")
	TEXT(" 0: disabled\n")
	TEXT(" 1: show error if Nanite is present in the scene but unsupported, and fallback meshes are not used for rendering; (default)")
	TEXT(" 2: show error if Nanite is present in the scene but unsupported, even if fallback meshes are used for rendering")
);

#endif

static TAutoConsoleVariable<float> CVarTranslucencyAutoBeforeDOF(
	TEXT("r.Translucency.AutoBeforeDOF"), 0.5f,
	TEXT("Automatically bin After DOF translucency before DOF if behind focus distance (Experimental)"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarCrossGPUTransferOption(
	TEXT("r.MultiGPU.Transfer"),
	2,
	TEXT("Mode to use for cross GPU transfers when multiple nDisplay views are active\n")
	TEXT(" 0: immediate pull transfer\n")
	TEXT(" 1: optimized push transfer (source GPU runs copy, with deferred fence wait on destination GPU)\n")
	TEXT(" 2: optimized pull transfer (destination GPU runs copy, with transfers delayed to last view's render); (default)"),
	ECVF_Default);

FOcclusionSubmittedFenceState FSceneRenderer::OcclusionSubmittedFence[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames];

// cleanup OcclusionSubmittedFence to avoid undefined order of destruction that can destroy it after its allocator
void CleanupOcclusionSubmittedFence()
{
	for (FOcclusionSubmittedFenceState& FenceState : FSceneRenderer::OcclusionSubmittedFence)
	{
		FenceState.Fence = nullptr;
	}
}

extern int32 GetTranslucencyLightingVolumeDim();
extern bool GetSubstrateEnabledUseRoughRefraction();

DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer ViewExtensionPostRenderView"), STAT_FDeferredShadingSceneRenderer_ViewExtensionPostRenderView, STATGROUP_SceneRendering);
DECLARE_CYCLE_STAT(TEXT("DeferredShadingSceneRenderer ViewExtensionPreRenderView"), STAT_FDeferredShadingSceneRenderer_ViewExtensionPreRenderView, STATGROUP_SceneRendering);

#define FASTVRAM_CVAR(Name,DefaultValue) static TAutoConsoleVariable<int32> CVarFastVRam_##Name(TEXT("r.FastVRam."#Name), DefaultValue, TEXT(""))

FASTVRAM_CVAR(GBufferA, 0);
FASTVRAM_CVAR(GBufferB, 1);
FASTVRAM_CVAR(GBufferC, 0);
FASTVRAM_CVAR(GBufferD, 0);
FASTVRAM_CVAR(GBufferE, 0);
FASTVRAM_CVAR(GBufferF, 0);
FASTVRAM_CVAR(GBufferVelocity, 0);
FASTVRAM_CVAR(HZB, 1);
FASTVRAM_CVAR(SceneDepth, 1);
FASTVRAM_CVAR(SceneColor, 1);
FASTVRAM_CVAR(Bloom, 1);
FASTVRAM_CVAR(BokehDOF, 1);
FASTVRAM_CVAR(CircleDOF, 1);
FASTVRAM_CVAR(CombineLUTs, 1);
FASTVRAM_CVAR(Downsample, 1);
FASTVRAM_CVAR(EyeAdaptation, 1);
FASTVRAM_CVAR(Histogram, 1);
FASTVRAM_CVAR(HistogramReduce, 1);
FASTVRAM_CVAR(VelocityFlat, 1);
FASTVRAM_CVAR(VelocityMax, 1);
FASTVRAM_CVAR(MotionBlur, 1);
FASTVRAM_CVAR(Tonemap, 1);
FASTVRAM_CVAR(Upscale, 1);
FASTVRAM_CVAR(DistanceFieldNormal, 1);
FASTVRAM_CVAR(DistanceFieldAOHistory, 1);
FASTVRAM_CVAR(DistanceFieldAODownsampledBentNormal, 1); 
FASTVRAM_CVAR(DistanceFieldAOBentNormal, 0); 
FASTVRAM_CVAR(DistanceFieldIrradiance, 0); 
FASTVRAM_CVAR(DistanceFieldShadows, 1);
FASTVRAM_CVAR(Distortion, 1);
FASTVRAM_CVAR(ScreenSpaceShadowMask, 1);
FASTVRAM_CVAR(VolumetricFog, 1);
FASTVRAM_CVAR(SeparateTranslucency, 0); 
FASTVRAM_CVAR(SeparateTranslucencyModulate, 0);
FASTVRAM_CVAR(ScreenSpaceAO,0);
FASTVRAM_CVAR(SSR, 0);
FASTVRAM_CVAR(DBufferA, 0);
FASTVRAM_CVAR(DBufferB, 0);
FASTVRAM_CVAR(DBufferC, 0); 
FASTVRAM_CVAR(DBufferMask, 0);
FASTVRAM_CVAR(DOFSetup, 1);
FASTVRAM_CVAR(DOFReduce, 1);
FASTVRAM_CVAR(DOFPostfilter, 1);
FASTVRAM_CVAR(PostProcessMaterial, 1);

FASTVRAM_CVAR(CustomDepth, 0);
FASTVRAM_CVAR(ShadowPointLight, 0);
FASTVRAM_CVAR(ShadowPerObject, 0);
FASTVRAM_CVAR(ShadowCSM, 0);

FASTVRAM_CVAR(DistanceFieldCulledObjectBuffers, 1);
FASTVRAM_CVAR(DistanceFieldTileIntersectionResources, 1);
FASTVRAM_CVAR(DistanceFieldAOScreenGridResources, 1);
FASTVRAM_CVAR(ForwardLightingCullingResources, 1);
FASTVRAM_CVAR(GlobalDistanceFieldCullGridBuffers, 1);

TSharedPtr<FVirtualShadowMapClipmap> FVisibleLightInfo::FindVirtualShadowMapShadowClipmapForView(const FViewInfo* View) const
{
	for (const auto& Clipmap : VirtualShadowMapClipmaps)
	{
		if (Clipmap->GetDependentView() == View)
		{
			return Clipmap;
		}
	}
	
	// This has to mirror the if (IStereoRendering::IsAPrimaryView(View)) test in ShadowSetup.cpp, which ensures only one view dependent shadow is set up for a stereo pair.
	// TODO: this should very much be explicitly linked.
	if (!IStereoRendering::IsAPrimaryView(*View) && VirtualShadowMapClipmaps.Num() > 0)
	{
		return VirtualShadowMapClipmaps[0];
	}

	return TSharedPtr<FVirtualShadowMapClipmap>();
}

int32 FVisibleLightInfo::GetVirtualShadowMapId(const FViewInfo* View) const
{
	if (VirtualShadowMapClipmaps.Num())
	{
		TSharedPtr<FVirtualShadowMapClipmap> Clipmap = FindVirtualShadowMapShadowClipmapForView(View);
		return Clipmap.IsValid() ? Clipmap->GetVirtualShadowMapId() : INDEX_NONE;
	}
	else
	{
		// Look through local light projected shadows
		// TODO: With ForceOnlyVSM there should only be one
		for (int32 ShadowIndex = 0; ShadowIndex < AllProjectedShadows.Num(); ShadowIndex++)
		{
			const FProjectedShadowInfo* ProjectedShadowInfo = AllProjectedShadows[ShadowIndex];
			if (ProjectedShadowInfo->VirtualShadowMapPerLightCacheEntry.IsValid())
			{
				// VSM ID should be allocated by the time anyone calls this function
				check(ProjectedShadowInfo->VirtualShadowMapPerLightCacheEntry->GetVirtualShadowMapId() >= 0);
				return ProjectedShadowInfo->VirtualShadowMapPerLightCacheEntry->GetVirtualShadowMapId();
			}
		}

		return INDEX_NONE;
	}
}

bool FVisibleLightInfo::ContainsOnlyVirtualShadowMaps() const
{
	for (int32 ShadowIndex=0; ShadowIndex < AllProjectedShadows.Num(); ++ShadowIndex)
	{
		// Simple test for now, but sufficient
		const FProjectedShadowInfo* ProjectedShadowInfo = AllProjectedShadows[ShadowIndex];
		if (ProjectedShadowInfo->bAllocated && !ProjectedShadowInfo->HasVirtualShadowMap())
		{
			return false;
		}
	}
	return true;
}


#if !UE_BUILD_SHIPPING
namespace
{

/*
 * Screen percentage interface that is just constantly changing res to test resolution changes.
 */
class FScreenPercentageHellDriver : public ISceneViewFamilyScreenPercentage
{
public:

	FScreenPercentageHellDriver(const FSceneViewFamily& InViewFamily)
		: ViewFamily(InViewFamily)
	{ 
		if (InViewFamily.GetTemporalUpscalerInterface())
		{
			MinResolutionFraction = InViewFamily.GetTemporalUpscalerInterface()->GetMinUpsampleResolutionFraction();
			MaxResolutionFraction = InViewFamily.GetTemporalUpscalerInterface()->GetMaxUpsampleResolutionFraction();
		}

		check(MinResolutionFraction <= MaxResolutionFraction);
		check(MinResolutionFraction > 0.0f);
		check(MaxResolutionFraction > 0.0f);
	}

	virtual DynamicRenderScaling::TMap<float> GetResolutionFractionsUpperBound() const override
	{
		DynamicRenderScaling::TMap<float> ResolutionFractions;
		if (ViewFamily.EngineShowFlags.ScreenPercentage)
		{
			ResolutionFractions[GDynamicPrimaryResolutionFraction] = MaxResolutionFraction;
		}
		return ResolutionFractions;
	}

	virtual ISceneViewFamilyScreenPercentage* Fork_GameThread(const class FSceneViewFamily& ForkedViewFamily) const override
	{
		check(IsInGameThread());

		if (ForkedViewFamily.Views[0]->State)
		{
			return new FScreenPercentageHellDriver(ForkedViewFamily);
		}

		return new FLegacyScreenPercentageDriver(
			ForkedViewFamily, /* GlobalResolutionFraction = */ MaxResolutionFraction);
	}

	virtual DynamicRenderScaling::TMap<float> GetResolutionFractions_RenderThread() const override
	{
		check(IsInParallelRenderingThread());

		uint32 FrameId = 0;

		const FSceneViewState* ViewState = static_cast<const FSceneViewState*>(ViewFamily.Views[0]->State);
		if (ViewState)
		{
			FrameId = ViewState->GetFrameIndex(8);
		}

		DynamicRenderScaling::TMap<float> ResolutionFractions;
		if (ViewFamily.EngineShowFlags.ScreenPercentage)
		{
			ResolutionFractions[GDynamicPrimaryResolutionFraction] =
				FrameId == 0 ? MaxResolutionFraction : FMath::Lerp(MinResolutionFraction, MaxResolutionFraction, 0.5f + 0.5f * FMath::Cos((FrameId + 0.25) * PI / 8));
		}
		return ResolutionFractions;
	}

private:
	// View family to take care of.
	const FSceneViewFamily& ViewFamily;
	float MinResolutionFraction = 0.5f;
	float MaxResolutionFraction = 1.0f;
};

} // namespace
#endif // !UE_BUILD_SHIPPING

FFastVramConfig::FFastVramConfig()
{
	FMemory::Memset(*this, 0);
}

void FFastVramConfig::Update()
{
	bDirty = false;
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_GBufferA, GBufferA);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_GBufferB, GBufferB);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_GBufferC, GBufferC);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_GBufferD, GBufferD);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_GBufferE, GBufferE);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_GBufferF, GBufferF);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_GBufferVelocity, GBufferVelocity);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_HZB, HZB);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_SceneDepth, SceneDepth);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_SceneColor, SceneColor);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_Bloom, Bloom);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_BokehDOF, BokehDOF);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_CircleDOF, CircleDOF);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_CombineLUTs, CombineLUTs);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_Downsample, Downsample);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_EyeAdaptation, EyeAdaptation);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_Histogram, Histogram);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_HistogramReduce, HistogramReduce);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_VelocityFlat, VelocityFlat);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_VelocityMax, VelocityMax);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_MotionBlur, MotionBlur);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_Tonemap, Tonemap);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_Upscale, Upscale);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DistanceFieldNormal, DistanceFieldNormal);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DistanceFieldAOHistory, DistanceFieldAOHistory);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DistanceFieldAODownsampledBentNormal, DistanceFieldAODownsampledBentNormal);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DistanceFieldAOBentNormal, DistanceFieldAOBentNormal);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DistanceFieldIrradiance, DistanceFieldIrradiance);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DistanceFieldShadows, DistanceFieldShadows);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_Distortion, Distortion);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_ScreenSpaceShadowMask, ScreenSpaceShadowMask);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_VolumetricFog, VolumetricFog);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_SeparateTranslucency, SeparateTranslucency);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_SeparateTranslucencyModulate, SeparateTranslucencyModulate);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_ScreenSpaceAO, ScreenSpaceAO);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_SSR, SSR);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DBufferA, DBufferA);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DBufferB, DBufferB);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DBufferC, DBufferC);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DBufferMask, DBufferMask);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DOFSetup, DOFSetup);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DOFReduce, DOFReduce);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_DOFPostfilter, DOFPostfilter);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_CustomDepth, CustomDepth);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_ShadowPointLight, ShadowPointLight);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_ShadowPerObject, ShadowPerObject);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_ShadowCSM, ShadowCSM);
	bDirty |= UpdateTextureFlagFromCVar(CVarFastVRam_PostProcessMaterial, PostProcessMaterial);

	bDirty |= UpdateBufferFlagFromCVar(CVarFastVRam_DistanceFieldCulledObjectBuffers, DistanceFieldCulledObjectBuffers);
	bDirty |= UpdateBufferFlagFromCVar(CVarFastVRam_DistanceFieldTileIntersectionResources, DistanceFieldTileIntersectionResources);
	bDirty |= UpdateBufferFlagFromCVar(CVarFastVRam_DistanceFieldAOScreenGridResources, DistanceFieldAOScreenGridResources);
	bDirty |= UpdateBufferFlagFromCVar(CVarFastVRam_ForwardLightingCullingResources, ForwardLightingCullingResources);
	bDirty |= UpdateBufferFlagFromCVar(CVarFastVRam_GlobalDistanceFieldCullGridBuffers, GlobalDistanceFieldCullGridBuffers);
}

bool FFastVramConfig::UpdateTextureFlagFromCVar(TAutoConsoleVariable<int32>& CVar, ETextureCreateFlags& InOutValue)
{
	ETextureCreateFlags OldValue = InOutValue;
	int32 CVarValue = CVar.GetValueOnRenderThread();
	InOutValue = TexCreate_None;
	if (CVarValue == 1)
	{
		InOutValue = TexCreate_FastVRAM;
	}
	else if (CVarValue == 2)
	{
		InOutValue = TexCreate_FastVRAM | TexCreate_FastVRAMPartialAlloc;
	}
	return OldValue != InOutValue;
}

bool FFastVramConfig::UpdateBufferFlagFromCVar(TAutoConsoleVariable<int32>& CVar, EBufferUsageFlags& InOutValue)
{
	EBufferUsageFlags OldValue = InOutValue;
	InOutValue = CVar.GetValueOnRenderThread() ? ( BUF_FastVRAM ) : BUF_None;
	return OldValue != InOutValue;
}

FFastVramConfig GFastVRamConfig;

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void FRDGParallelCommandListSet::SetStateOnCommandList(FRHICommandList& RHICmdList)
{
	FParallelCommandListSet::SetStateOnCommandList(RHICmdList);
	Bindings.SetOnCommandList(RHICmdList);
	if (bHasRenderPasses)
	{
		FSceneRenderer::SetStereoViewport(RHICmdList, View, ViewportScale);
	}
}

FParallelCommandListSet::FParallelCommandListSet(const FRDGPass* InPass, const FViewInfo& InView, FRHICommandListImmediate& InParentCmdList, bool bInHasRenderPasses)
	: Pass(InPass)
	, View(InView)
	, ParentCmdList(InParentCmdList)
	, NumAlloc(0)
	, bHasRenderPasses(bInHasRenderPasses)
{
	Width = CVarRHICmdWidth.GetValueOnRenderThread();
	MinDrawsPerCommandList = CVarRHICmdMinDrawsPerParallelCmdList.GetValueOnRenderThread();
	QueuedCommandLists.Reserve(Width * 8);
}

FRHICommandList* FParallelCommandListSet::AllocCommandList()
{
	NumAlloc++;
	return new FRHICommandList(ParentCmdList.GetGPUMask());
}

void FParallelCommandListSet::Dispatch(bool /*bHighPriority*/)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FParallelCommandListSet_Dispatch);
	check(QueuedCommandLists.Num() == NumAlloc);

	// We should not be submitting work off a parent command list if it's still in the middle of a renderpass.
	// This is a bit weird since we will (likely) end up opening one in the parallel translate case but until we have
	// a cleaner way for the RHI to specify parallel passes this is what we've got.
	check(ParentCmdList.IsOutsideRenderPass());

	NumAlloc -= QueuedCommandLists.Num();
	ParentCmdList.QueueAsyncCommandListSubmit(QueuedCommandLists);
	QueuedCommandLists.Reset();
}

FParallelCommandListSet::~FParallelCommandListSet()
{
	checkf(QueuedCommandLists.Num() == 0, TEXT("Derived class of FParallelCommandListSet did not call Dispatch in virtual destructor"));
	checkf(NumAlloc == 0, TEXT("Derived class of FParallelCommandListSet did not call Dispatch in virtual destructor"));
}

FRHICommandList* FParallelCommandListSet::NewParallelCommandList()
{
	FRHICommandList* Result = AllocCommandList();
	
	// Command lists used with FParallelCommandListSet are graphics pipe by default.
	Result->SwitchPipeline(ERHIPipeline::Graphics);

	SetStateOnCommandList(*Result);
	return Result;
}

void FParallelCommandListSet::AddParallelCommandList(FRHICommandList* CmdList)
{
	QueuedCommandLists.Emplace(CmdList);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool IsHMDHiddenAreaMaskActive()
{
	// Query if we have a custom HMD post process mesh to use
	static const auto* const HiddenAreaMaskCVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("vr.HiddenAreaMask"));

	return
		HiddenAreaMaskCVar != nullptr &&
		// Any thread is used due to FViewInfo initialization.
		HiddenAreaMaskCVar->GetValueOnAnyThread() == 1 &&
		GEngine &&
		GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice() &&
		GEngine->XRSystem->GetHMDDevice()->HasVisibleAreaMesh();
}

/*-----------------------------------------------------------------------------
	FViewInfo
-----------------------------------------------------------------------------*/

/** 
 * Initialization constructor. Passes all parameters to FSceneView constructor
 */
FViewInfo::FViewInfo(const FSceneViewInitOptions& InitOptions)
	:	FSceneView(InitOptions)
	,	IndividualOcclusionQueries((FSceneViewState*)InitOptions.SceneViewStateInterface, 1, bIsInstancedStereoEnabled ? 2 : 1)
	,	GroupedOcclusionQueries((FSceneViewState*)InitOptions.SceneViewStateInterface, FOcclusionQueryBatcher::OccludedPrimitiveQueryBatchSize, bIsInstancedStereoEnabled ? 2 : 1)
	,	CustomVisibilityQuery(nullptr)
{
	Init();
}

/** 
 * Initialization constructor. 
 * @param InView - copy to init with
 */
FViewInfo::FViewInfo(const FSceneView* InView)
	:	FSceneView(*InView)
	,	IndividualOcclusionQueries((FSceneViewState*)InView->State,1, bIsInstancedStereoEnabled ? 2 : 1)
	,	GroupedOcclusionQueries((FSceneViewState*)InView->State,FOcclusionQueryBatcher::OccludedPrimitiveQueryBatchSize, bIsInstancedStereoEnabled ? 2 : 1)
	,	CustomVisibilityQuery(nullptr)
{
	Init();
}

void FViewInfo::Init()
{
	ViewRect = FIntRect(0, 0, 0, 0);

	CachedViewUniformShaderParameters = nullptr;
	bHasNoVisiblePrimitive = false;
	bHasTranslucentViewMeshElements = 0;
	bPrevTransformsReset = false;
	bIgnoreExistingQueries = false;
	bDisableQuerySubmissions = false;
	bDisableDistanceBasedFadeTransitions = false;	
	ShadingModelMaskInView = 0;
	bSceneHasSkyMaterial = 0;
	bHasSingleLayerWaterMaterial = 0;
	AutoBeforeDOFTranslucencyBoundary = 0.0f;
	bUsesSecondStageDepthPass = 0;
	bSceneCaptureMainViewJitter = 0;

	NumVisibleStaticMeshElements = 0;
	PrecomputedVisibilityData = 0;

	bIsViewInfo = true;
	
	bStatePrevViewInfoIsReadOnly = true;
	bUsesGlobalDistanceField = false;
	bUsesLightingChannels = false;
	bTranslucentSurfaceLighting = false;
	bFogOnlyOnRenderedOpaque = false;

	ExponentialFogParameters = FVector4f(0,1,1,0);
	ExponentialFogParameters2 = FVector4f(0, 1, 0, 0);
	ExponentialFogColor = FVector3f::ZeroVector;
	FogMaxOpacity = 1;
	ExponentialFogParameters3 = FVector4f(0, 0, 0, 0);
	SinCosInscatteringColorCubemapRotation = FVector2f::ZeroVector;
	FogEndDistance = 0.0f;
	FogInscatteringColorCubemap = nullptr;
	FogInscatteringTextureParameters = FVector::ZeroVector;
	VolumetricFogStartDistance = false;
	VolumetricFogStartDistance = 0.0f;
	VolumetricFogNearFadeInDistanceInv = 100000000.0f;
	VolumetricFogAlbedo = FVector3f::Zero();
	VolumetricFogPhaseG = 0.0f;

	SkyAtmosphereCameraAerialPerspectiveVolume = nullptr;
	SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly = nullptr;
	SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly = nullptr;
	SkyAtmosphereUniformShaderParameters = nullptr;

	bUseDirectionalInscattering = false;
	DirectionalInscatteringExponent = 0;
	DirectionalInscatteringStartDistance = 0;
	InscatteringLightDirection = FVector(0);
	DirectionalInscatteringColor = FLinearColor(ForceInit);

	for (int32 CascadeIndex = 0; CascadeIndex < TVC_MAX; CascadeIndex++)
	{
		TranslucencyLightingVolumeMin[CascadeIndex] = FVector(0);
		TranslucencyVolumeVoxelSize[CascadeIndex] = 0;
		TranslucencyLightingVolumeSize[CascadeIndex] = FVector(0);
	}

	const bool bCubeShareCascades = bIsSceneCaptureCube && IsSceneCaptureCubeShareCascadesEnabled(GetShaderPlatform());

	// When cascade shadows are shared across all cube faces, we only need a single shadow atlas regardless of the number of views,
	// so the view count is ignored for this calculation of the number of cascade shadows that can be rendered.
	const int32 MaxMobileShadowCascadeCount = MAX_MOBILE_SHADOWCASCADES / (bCubeShareCascades ? 1 : FMath::Max(Family->Views.Num(), 1));
	const int32 MaxShadowCascadeCountUpperBound = GetFeatureLevel() >= ERHIFeatureLevel::SM5 ? 10 : MaxMobileShadowCascadeCount;

	MaxShadowCascades = FMath::Clamp<int32>(CVarMaxShadowCascades.GetValueOnAnyThread(), 0, MaxShadowCascadeCountUpperBound);

	ShaderMap = GetGlobalShaderMap(FeatureLevel);

	ViewState = (FSceneViewState*)State;
	bHMDHiddenAreaMaskActive = IsHMDHiddenAreaMaskActive();
	bUseComputePasses = IsPostProcessingWithComputeEnabled(FeatureLevel);
	bHasCustomDepthPrimitives = false;
	bHasDistortionPrimitives = false;
	bAllowStencilDither = false;
	bCustomDepthStencilValid = false;
	bUsesCustomDepth = false;
	bUsesCustomStencil = false;
	bUsesMotionVectorWorldOffset = false;
	bSLWUsesMotionVectorWorldOffset = false;

	// Sky dome, or any emissive, materials can result in high luminance values, e.g. the sun disk. 
	// This Min here is to we make sure pre-exposed luminance remains within the boundaries of fp10 and not cause NaN on some platforms.
	// We also half that range to also make sure we have room for other additive elements such as bloom, clouds or particle visual effects.
	const static float Max10BitsFloat = 64512.0f;
	MaterialMaxEmissiveValue = Max10BitsFloat * 0.5f;

	NumBoxReflectionCaptures = 0;
	NumSphereReflectionCaptures = 0;
	FurthestReflectionCaptureDistance = 0;

	TemporalSourceView = nullptr;
	TemporalJitterSequenceLength = 1;
	TemporalJitterIndex = 0;
	TemporalJitterPixels = FVector2D::ZeroVector;

	PreExposure = 1.0f;

	// Cache TEXTUREGROUP filter settings for the render thread to create shared samplers.
	if (IsInGameThread())
	{
		const UTextureLODSettings* TextureLODSettings = UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings();
		WorldTextureGroupSamplerFilter = (ESamplerFilter)TextureLODSettings->GetSamplerFilter(TEXTUREGROUP_World);
		TerrainWeightmapTextureGroupSamplerFilter = (ESamplerFilter)TextureLODSettings->GetSamplerFilter(TEXTUREGROUP_Terrain_Weightmap);
		WorldTextureGroupMaxAnisotropy = TextureLODSettings->GetTextureLODGroup(TEXTUREGROUP_World).MaxAniso;
		bIsValidTextureGroupSamplerFilters = true;
	}
	else
	{
		bIsValidTextureGroupSamplerFilters = false;
	}

	PrimitiveSceneDataTextureOverrideRHI = nullptr;

	DitherFadeInUniformBuffer = nullptr;
	DitherFadeOutUniformBuffer = nullptr;

	for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; ++PassIndex)
	{
		NumVisibleDynamicMeshElements[PassIndex] = 0;
	}

	NumVisibleDynamicPrimitives = 0;
	NumVisibleDynamicEditorPrimitives = 0;

	SubstrateViewData.Reset();

	LocalFogVolumeViewData = FLocalFogVolumeViewData();

	SceneRendererPrimaryViewId = INDEX_NONE; // Initialized later in the FSceneRenderer constructor.

	// Filled in by FDeferredShadingSceneRenderer::UpdateLumenScene
	ViewLumenSceneData = nullptr;

	VolumetricCloudSkyAO = nullptr;
	for (int LightIndex = 0; LightIndex < NUM_ATMOSPHERE_LIGHTS; ++LightIndex)
	{
		VolumetricCloudShadowRenderTarget[LightIndex] = nullptr;
		VolumetricCloudShadowExtractedRenderTarget[LightIndex] = nullptr;
		if (ViewState != nullptr)
		{
			ViewState->VolumetricCloudShadowRenderTarget[LightIndex].Reset();
		}
	}
}

FParallelMeshDrawCommandPass* FViewInfo::CreateMeshPass(EMeshPass::Type MeshPass)
{
	check(!ParallelMeshDrawCommandPasses[MeshPass]);
	return ParallelMeshDrawCommandPasses[MeshPass] = Allocator.Create<FParallelMeshDrawCommandPass>();
}

void FViewInfo::Cleanup()
{
	for (int32 MeshDrawIndex = 0; MeshDrawIndex < EMeshPass::Num; MeshDrawIndex++)
	{
		if (auto* Pass = ParallelMeshDrawCommandPasses[MeshDrawIndex])
		{
			Pass->Cleanup();
		}
	}
}

FViewInfo::~FViewInfo()
{
	for(int32 ResourceIndex = 0;ResourceIndex < DynamicResources.Num();ResourceIndex++)
	{
		DynamicResources[ResourceIndex]->ReleasePrimitiveResource();
	}
	if (CustomVisibilityQuery)
	{
		CustomVisibilityQuery->Release();
	}

	//this uses memstack allocation for strongrefs, so we need to manually empty to get the destructor called to not leak the uniformbuffers stored here.
	TranslucentSelfShadowUniformBufferMap.Empty();
}

#if RHI_RAYTRACING
bool FViewInfo::HasRayTracingScene() const
{
	check(Family);
	FScene* Scene = Family->Scene ? Family->Scene->GetRenderScene() : nullptr;
	if (Scene)
	{
		return Scene->RayTracingScene.IsCreated();
	}
	return false;
}

FRHIRayTracingScene* FViewInfo::GetRayTracingSceneChecked(ERayTracingSceneLayer Layer) const
{
	check(Family);
	if (Family->Scene)
	{
		if (FScene* Scene = Family->Scene->GetRenderScene())
		{
			FRHIRayTracingScene* Result = Scene->RayTracingScene.GetRHIRayTracingScene(Layer, GetRayTracingSceneViewHandle());
			checkf(Result, TEXT("Ray tracing scene is expected to be created at this point."));
			return Result;
		}
	}
	return nullptr;
}

FRDGBufferSRVRef FViewInfo::GetRayTracingSceneLayerViewChecked(ERayTracingSceneLayer Layer) const
{
	FRDGBufferSRVRef Result = nullptr;
	check(Family);
	if (Family->Scene)
	{
		if (FScene* Scene = Family->Scene->GetRenderScene())
		{
			Result = Scene->RayTracingScene.GetLayerView(Layer, GetRayTracingSceneViewHandle());
		}
	}
	checkf(Result, TEXT("Ray tracing scene SRV is expected to be created at this point."));
	return Result;
}

FRDGBufferUAVRef FViewInfo::GetRayTracingInstanceHitCountUAV(FRDGBuilder& GraphBuilder) const
{
	check(Family);
	if (Family->Scene)
	{
		if (FScene* Scene = Family->Scene->GetRenderScene())
		{
			return Scene->RayTracingScene.GetInstanceHitCountBufferUAV(ERayTracingSceneLayer::Base, GetRayTracingSceneViewHandle());
		}
	}	
	return nullptr;
}

const FRayTracingScene::FViewHandle& FViewInfo::GetRayTracingSceneViewHandle() const
{
	if (UNLIKELY(IStereoRendering::IsStereoEyePass(StereoPass) && IStereoRendering::IsASecondaryPass(StereoPass)))
	{
		return GetPrimaryView()->RayTracingSceneViewHandle;
	}

	return RayTracingSceneViewHandle;
}

#endif // RHI_RAYTRACING

#if DO_CHECK || USING_CODE_ANALYSIS
bool FViewInfo::VerifyMembersChecks() const
{
	FSceneView::VerifyMembersChecks();

	check(ViewState == State);

	return true;
}
#endif

void UpdateNoiseTextureParameters(FViewUniformShaderParameters& ViewUniformShaderParameters)
{
	if (GSystemTextures.PerlinNoiseGradient.GetReference())
	{
		ViewUniformShaderParameters.PerlinNoiseGradientTexture = GSystemTextures.PerlinNoiseGradient->GetRHI();
		SetBlack2DIfNull(ViewUniformShaderParameters.PerlinNoiseGradientTexture);
	}
	check(ViewUniformShaderParameters.PerlinNoiseGradientTexture);
	ViewUniformShaderParameters.PerlinNoiseGradientTextureSampler = TStaticSamplerState<SF_Point, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

	if (GSystemTextures.PerlinNoise3D.GetReference())
	{
		ViewUniformShaderParameters.PerlinNoise3DTexture = GSystemTextures.PerlinNoise3D->GetRHI();
		SetBlack3DIfNull(ViewUniformShaderParameters.PerlinNoise3DTexture);
	}
	check(ViewUniformShaderParameters.PerlinNoise3DTexture);
	ViewUniformShaderParameters.PerlinNoise3DTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

	if (GSystemTextures.SobolSampling.GetReference())
	{
		ViewUniformShaderParameters.SobolSamplingTexture = GSystemTextures.SobolSampling->GetRHI();
		SetBlack2DIfNull(ViewUniformShaderParameters.SobolSamplingTexture);
	}
	check(ViewUniformShaderParameters.SobolSamplingTexture);
}

void SetupPrecomputedVolumetricLightmapUniformBufferParameters(const FScene* Scene, FEngineShowFlags EngineShowFlags, FViewUniformShaderParameters& ViewUniformShaderParameters)
{
	if (Scene && Scene->VolumetricLightmapSceneData.HasData() && EngineShowFlags.VolumetricLightmap)
	{
		const FPrecomputedVolumetricLightmapData* VolumetricLightmapData = Scene->VolumetricLightmapSceneData.GetLevelVolumetricLightmap()->Data;

		FVector BrickDimensions;
		const FVolumetricLightmapBasicBrickDataLayers* BrickData = nullptr;

#if WITH_EDITOR
		if (FStaticLightingSystemInterface::GetPrecomputedVolumetricLightmap(Scene->GetWorld()))
		{
			BrickDimensions = FVector(VolumetricLightmapData->BrickDataDimensions);
			BrickData = &VolumetricLightmapData->BrickData;
		}
		else
#endif
		{
			BrickDimensions = FVector(GVolumetricLightmapBrickAtlas.TextureSet.BrickDataDimensions);
			BrickData = &GVolumetricLightmapBrickAtlas.TextureSet;
		}

		ViewUniformShaderParameters.VolumetricLightmapIndirectionTexture = OrBlack3DUintIfNull(VolumetricLightmapData->IndirectionTexture.Texture);
		ViewUniformShaderParameters.VolumetricLightmapBrickAmbientVector = OrBlack3DIfNull(BrickData->AmbientVector.Texture);
		ViewUniformShaderParameters.VolumetricLightmapBrickSHCoefficients0 = OrBlack3DIfNull(BrickData->SHCoefficients[0].Texture);
		ViewUniformShaderParameters.VolumetricLightmapBrickSHCoefficients1 = OrBlack3DIfNull(BrickData->SHCoefficients[1].Texture);
		ViewUniformShaderParameters.VolumetricLightmapBrickSHCoefficients2 = OrBlack3DIfNull(BrickData->SHCoefficients[2].Texture);
		ViewUniformShaderParameters.VolumetricLightmapBrickSHCoefficients3 = OrBlack3DIfNull(BrickData->SHCoefficients[3].Texture);
		ViewUniformShaderParameters.VolumetricLightmapBrickSHCoefficients4 = OrBlack3DIfNull(BrickData->SHCoefficients[4].Texture);
		ViewUniformShaderParameters.VolumetricLightmapBrickSHCoefficients5 = OrBlack3DIfNull(BrickData->SHCoefficients[5].Texture);
		ViewUniformShaderParameters.SkyBentNormalBrickTexture = OrBlack3DIfNull(BrickData->SkyBentNormal.Texture);
		ViewUniformShaderParameters.DirectionalLightShadowingBrickTexture = OrBlack3DIfNull(BrickData->DirectionalLightShadowing.Texture);

		const FBox VolumeBounds = VolumetricLightmapData->GetBounds();
		const FVector VolumeSize = VolumeBounds.GetSize();
		const FVector InvVolumeSize = VolumeSize.Reciprocal();

		const FVector InvBrickDimensions = BrickDimensions.Reciprocal();

		ViewUniformShaderParameters.VolumetricLightmapWorldToUVScale = (FVector3f)InvVolumeSize;
		ViewUniformShaderParameters.VolumetricLightmapWorldToUVAdd = FVector3f(-VolumeBounds.Min * InvVolumeSize);
		ViewUniformShaderParameters.VolumetricLightmapIndirectionTextureSize = FVector3f(VolumetricLightmapData->IndirectionTextureDimensions);
		ViewUniformShaderParameters.VolumetricLightmapBrickSize = VolumetricLightmapData->BrickSize;
		ViewUniformShaderParameters.VolumetricLightmapBrickTexelSize = (FVector3f)InvBrickDimensions;
	}
	else
	{
		// Resources are initialized in FViewUniformShaderParameters ctor, only need to set defaults for non-resource types

		ViewUniformShaderParameters.VolumetricLightmapWorldToUVScale = FVector3f::ZeroVector;
		ViewUniformShaderParameters.VolumetricLightmapWorldToUVAdd = FVector3f::ZeroVector;
		ViewUniformShaderParameters.VolumetricLightmapIndirectionTextureSize = FVector3f::ZeroVector;
		ViewUniformShaderParameters.VolumetricLightmapBrickSize = 0;
		ViewUniformShaderParameters.VolumetricLightmapBrickTexelSize = FVector3f::ZeroVector;
	}
}

void SetupPhysicsFieldUniformBufferParameters(const FScene* Scene, FEngineShowFlags EngineShowFlags, FViewUniformShaderParameters& ViewUniformShaderParameters)
{
	if (Scene && Scene->PhysicsField && Scene->PhysicsField->FieldResource)
	{
		FPhysicsFieldResource* FieldResource = Scene->PhysicsField->FieldResource;
		if (FieldResource->FieldInfos.bBuildClipmap)
		{
			ViewUniformShaderParameters.PhysicsFieldClipmapBuffer = FieldResource->ClipmapBuffer.SRV.GetReference();
			ViewUniformShaderParameters.PhysicsFieldNodesParams = FieldResource->NodesParams.SRV.GetReference();
			ViewUniformShaderParameters.PhysicsFieldNodesOffsets = FieldResource->NodesOffsets.SRV.GetReference();
			ViewUniformShaderParameters.PhysicsFieldTargetsOffsets = FieldResource->TargetsOffsets.SRV.GetReference();
		}
		else
		{
			ViewUniformShaderParameters.PhysicsFieldClipmapBuffer = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
			ViewUniformShaderParameters.PhysicsFieldNodesParams = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
			ViewUniformShaderParameters.PhysicsFieldNodesOffsets = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
			ViewUniformShaderParameters.PhysicsFieldTargetsOffsets = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
		}
		ViewUniformShaderParameters.PhysicsFieldClipmapCenter = (FVector3f)FieldResource->FieldInfos.ClipmapCenter;
		ViewUniformShaderParameters.PhysicsFieldClipmapDistance = FieldResource->FieldInfos.ClipmapDistance;
		ViewUniformShaderParameters.PhysicsFieldClipmapResolution = FieldResource->FieldInfos.ClipmapResolution;
		ViewUniformShaderParameters.PhysicsFieldClipmapExponent = FieldResource->FieldInfos.ClipmapExponent;
		ViewUniformShaderParameters.PhysicsFieldClipmapCount = FieldResource->FieldInfos.ClipmapCount;
		ViewUniformShaderParameters.PhysicsFieldTargetCount = FieldResource->FieldInfos.TargetCount;

		const TArray<EFieldPhysicsType> VectorTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Vector);
		const TArray<EFieldPhysicsType> ScalarTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Scalar);
		const TArray<EFieldPhysicsType> IntegerTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Integer);

		for (int32 Index = 0; Index < MAX_PHYSICS_FIELD_TARGETS; ++Index)
		{
			ViewUniformShaderParameters.PhysicsFieldTargets[Index].X = FieldResource->FieldInfos.VectorTargets[Index];
			ViewUniformShaderParameters.PhysicsFieldTargets[Index].Y = FieldResource->FieldInfos.ScalarTargets[Index];
			ViewUniformShaderParameters.PhysicsFieldTargets[Index].Z = FieldResource->FieldInfos.IntegerTargets[Index];
			ViewUniformShaderParameters.PhysicsFieldTargets[Index].W = 0; // Padding

			ViewUniformShaderParameters.PhysicsFieldTargetMappings[Index].X = Index < VectorTypes.Num() ? VectorTypes[Index] : 0;
			ViewUniformShaderParameters.PhysicsFieldTargetMappings[Index].Y = Index < ScalarTypes.Num() ? ScalarTypes[Index] : 0;
			ViewUniformShaderParameters.PhysicsFieldTargetMappings[Index].Z = Index < IntegerTypes.Num() ? IntegerTypes[Index] : 0;
			ViewUniformShaderParameters.PhysicsFieldTargetMappings[Index].W = 0;
		}
	}
	else
	{
		TStaticArray<UE::Core::TAlignedElement<FIntVector4, 16>, MAX_PHYSICS_FIELD_TARGETS> EmptyTargets = {};
		ViewUniformShaderParameters.PhysicsFieldClipmapBuffer = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
		ViewUniformShaderParameters.PhysicsFieldNodesParams = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
		ViewUniformShaderParameters.PhysicsFieldNodesOffsets = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
		ViewUniformShaderParameters.PhysicsFieldTargetsOffsets = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
		ViewUniformShaderParameters.PhysicsFieldClipmapCenter = FVector3f::ZeroVector;
		ViewUniformShaderParameters.PhysicsFieldClipmapDistance = 1.0;
		ViewUniformShaderParameters.PhysicsFieldClipmapResolution = 2;
		ViewUniformShaderParameters.PhysicsFieldClipmapExponent = 1;
		ViewUniformShaderParameters.PhysicsFieldClipmapCount = 1;
		ViewUniformShaderParameters.PhysicsFieldTargetCount = 0;
		ViewUniformShaderParameters.PhysicsFieldTargets = EmptyTargets;
		ViewUniformShaderParameters.PhysicsFieldTargetMappings = EmptyTargets;
	}
}

FIntPoint FViewInfo::GetSecondaryViewRectSize() const
{
	return FIntPoint(
		FMath::CeilToInt(UnscaledViewRect.Width() * Family->SecondaryViewFraction * SceneViewInitOptions.OverscanResolutionFraction),
		FMath::CeilToInt(UnscaledViewRect.Height() * Family->SecondaryViewFraction * SceneViewInitOptions.OverscanResolutionFraction));
}

FIntRect FViewInfo::GetSecondaryViewCropRect() const
{
	const FIntPoint SecondaryVewRectSize = GetSecondaryViewRectSize();

	// Clamp the crop fraction to sensible values to ensure crop rect is always a valid rectangle
	const FVector4f CropFrac = FVector4f(
		FMath::Clamp(SceneViewInitOptions.AsymmetricCropFraction.X * SceneViewInitOptions.CropFraction, 0.0f, 1.0f),
		FMath::Clamp(SceneViewInitOptions.AsymmetricCropFraction.Y * SceneViewInitOptions.CropFraction, 0.0f, 1.0f),
		FMath::Clamp(SceneViewInitOptions.AsymmetricCropFraction.Z * SceneViewInitOptions.CropFraction, 0.0f, 1.0f),
		FMath::Clamp(SceneViewInitOptions.AsymmetricCropFraction.W * SceneViewInitOptions.CropFraction, 0.0f, 1.0f));
	
	FIntRect CropRect;
	CropRect.Min = FIntPoint(
			FMath::FloorToInt(0.5f * (1.0f - CropFrac.X) * SecondaryVewRectSize.X),
			FMath::FloorToInt(0.5f * (1.0f - CropFrac.Z) * SecondaryVewRectSize.Y));
		
	CropRect.Max = FIntPoint(
		FMath::CeilToInt(0.5f * (1.0f + CropFrac.Y) * SecondaryVewRectSize.X),
		FMath::CeilToInt(0.5f * (1.0f + CropFrac.W) * SecondaryVewRectSize.Y));

	return CropRect;
}

/** Creates the view's uniform buffers given a set of view transforms. */
void FViewInfo::SetupUniformBufferParameters(
	const FViewMatrices& InViewMatrices,
	const FViewMatrices& InPrevViewMatrices,
	FBox* OutTranslucentCascadeBoundsArray,
	int32 NumTranslucentCascades,
	FViewUniformShaderParameters& ViewUniformShaderParameters) const
{
	check(Family);

	const FSceneTexturesConfig& SceneTexturesConfig = GetSceneTexturesConfig();

	// Create the view's uniform buffer.

	// Mobile multi-view is not side by side
	const FIntRect EffectiveViewRect = GetMultiViewCount() > 1 ? FIntRect(0, 0, ViewRect.Width(), ViewRect.Height()) : ViewRect;

	// Scene render targets may not be created yet; avoids NaNs.
	FIntPoint EffectiveBufferSize = SceneTexturesConfig.Extent;
	EffectiveBufferSize.X = FMath::Max(EffectiveBufferSize.X, 1);
	EffectiveBufferSize.Y = FMath::Max(EffectiveBufferSize.Y, 1);

	// TODO: We should use a view and previous view uniform buffer to avoid code duplication and keep consistency
	SetupCommonViewUniformBufferParameters(
		ViewUniformShaderParameters,
		EffectiveBufferSize,
		SceneTexturesConfig.NumSamples,
		EffectiveViewRect,
		InViewMatrices,
		InPrevViewMatrices
	);

	const bool bCheckerboardSubsurfaceRendering = IsSubsurfaceCheckerboardFormat(SceneTexturesConfig.ColorFormat, *this);
	ViewUniformShaderParameters.bCheckerboardSubsurfaceProfileRendering = bCheckerboardSubsurfaceRendering ? 1.0f : 0.0f;

	ViewUniformShaderParameters.IndirectLightingCacheShowFlag = Family->EngineShowFlags.IndirectLightingCache;

	FScene* Scene = nullptr;

	if (Family->Scene)
	{
		Scene = Family->Scene->GetRenderScene();
	}

	ERHIFeatureLevel::Type RHIFeatureLevel = Scene == nullptr ? GMaxRHIFeatureLevel : Scene->GetFeatureLevel();
	EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[RHIFeatureLevel];

	const FVector DefaultSunDirection(0.0f, 0.0f, 1.0f); // Up vector so that the AtmosphericLightVector node always output a valid direction.
	auto ClearAtmosphereLightData = [&](uint32 Index)
	{
		check(Index < NUM_ATMOSPHERE_LIGHTS);
		ViewUniformShaderParameters.AtmosphereLightDiscCosHalfApexAngle_PPTrans[Index] = FVector4f(1.0f, 0.0f, 0.0f, 0.0f);
		ViewUniformShaderParameters.AtmosphereLightDiscLuminance[Index] = FLinearColor::Black;
		ViewUniformShaderParameters.AtmosphereLightIlluminanceOnGroundPostTransmittance[Index] = FLinearColor::Black;
		ViewUniformShaderParameters.AtmosphereLightIlluminanceOnGroundPostTransmittance[Index].A = 0.0f;
		ViewUniformShaderParameters.AtmosphereLightIlluminanceOuterSpace[Index] = FLinearColor::Black;

		// We must set a default atmospheric light0 direction because this is use for instance by the height fog directional lobe. And we do not want to add an in shader test for that.
		ViewUniformShaderParameters.AtmosphereLightDirection[Index] = FVector3f(Index == 0 && Scene && Scene->SimpleDirectionalLight && Scene->SimpleDirectionalLight->Proxy ? -Scene->SimpleDirectionalLight->Proxy->GetDirection() : DefaultSunDirection);
	};

	if (Scene)
	{
		if (Scene->SimpleDirectionalLight)
		{
			ViewUniformShaderParameters.DirectionalLightColor = Scene->SimpleDirectionalLight->Proxy->GetAtmosphereTransmittanceTowardSun() * Scene->SimpleDirectionalLight->Proxy->GetColor() / PI;
			ViewUniformShaderParameters.DirectionalLightDirection = -(FVector3f)Scene->SimpleDirectionalLight->Proxy->GetDirection();
		}
		else
		{
			ViewUniformShaderParameters.DirectionalLightColor = FLinearColor::Black;
			ViewUniformShaderParameters.DirectionalLightDirection = FVector3f::ZeroVector;
		}

#if RHI_RAYTRACING
		const FRayTracingScene::FViewHandle& LocalRayTracingSceneViewHandle = GetRayTracingSceneViewHandle();
		const FDFVector3& RayTracingScenePreViewTranslation = LocalRayTracingSceneViewHandle.IsValid() ? Scene->RayTracingScene.GetPreViewTranslation(LocalRayTracingSceneViewHandle) : FDFVector3{};
		ViewUniformShaderParameters.TLASPreViewTranslationHigh = RayTracingScenePreViewTranslation.High;
		ViewUniformShaderParameters.TLASPreViewTranslationLow = RayTracingScenePreViewTranslation.Low;
#endif

		// Set default atmosphere lights parameters
		FLightSceneInfo* SunLight = Scene->AtmosphereLights[0];	// Atmospheric fog only takes into account the a single sun light with index 0.
		const float SunLightDiskHalfApexAngleRadian = SunLight ? SunLight->Proxy->GetSunLightHalfApexAngleRadian() : FLightSceneProxy::GetSunOnEarthHalfApexAngleRadian();
		const float UsePerPixelAtmosphereTransmittance = 0.0f; // The default sun light should not use per pixel transmitance without an atmosphere.

		ViewUniformShaderParameters.AtmosphereLightDiscCosHalfApexAngle_PPTrans[0] = FVector4f(FMath::Cos(SunLightDiskHalfApexAngleRadian), UsePerPixelAtmosphereTransmittance, 0.0f, 0.0f);
		//Added check so atmospheric light color and vector can use a directional light without needing an atmospheric fog actor in the scene
		ViewUniformShaderParameters.AtmosphereLightDiscLuminance[0] = SunLight ? SunLight->Proxy->GetOuterSpaceLuminance() : FLinearColor::Black;
		ViewUniformShaderParameters.AtmosphereLightIlluminanceOnGroundPostTransmittance[0] = SunLight ? SunLight->Proxy->GetColor() : FLinearColor::Black;
		ViewUniformShaderParameters.AtmosphereLightIlluminanceOnGroundPostTransmittance[0].A = 0.0f;
		ViewUniformShaderParameters.AtmosphereLightIlluminanceOuterSpace[0] = ViewUniformShaderParameters.AtmosphereLightIlluminanceOnGroundPostTransmittance[0];
		ViewUniformShaderParameters.AtmosphereLightIlluminanceOuterSpace[0].A = 0.0f;
		ViewUniformShaderParameters.AtmosphereLightDirection[0] = FVector3f(SunLight ? -SunLight->Proxy->GetDirection() : DefaultSunDirection);

		// Do not clear the first AtmosphereLight data, it has been setup above
		for (uint8 Index = 1; Index < NUM_ATMOSPHERE_LIGHTS; ++Index)
		{
			ClearAtmosphereLightData(Index);
		}
	}

	ViewUniformShaderParameters.BufferToSceneTextureScale = FVector2f(1.0f, 1.0f);

	FRHITexture* TransmittanceLutTextureFound = nullptr;
	FRHITexture* SkyViewLutTextureFound = nullptr;
	FRHITexture* CameraAerialPerspectiveVolumeFound = nullptr;
	FRHITexture* CameraAerialPerspectiveVolumeMieOnlyFound = nullptr;
	FRHITexture* CameraAerialPerspectiveVolumeRayOnlyFound = nullptr;
	FRHIShaderResourceView* DistantSkyLightLutBufferSRVFound = nullptr;
	FRHIShaderResourceView* MobileDistantSkyLightLutBufferSRVFound = nullptr;
	if (ShouldRenderSkyAtmosphere(Scene, Family->EngineShowFlags))
	{
		ViewUniformShaderParameters.SkyAtmospherePresentInScene = 1.0f;

		FSkyAtmosphereRenderSceneInfo* SkyAtmosphere = Scene->SkyAtmosphere;
		const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyAtmosphere->GetSkyAtmosphereSceneProxy();

		// Get access to texture resource if we have valid pointer.
		// (Valid pointer checks are needed because some resources might not have been initialized when coming from FCanvasTileRendererItem or FCanvasTriangleRendererItem)

		const TRefCountPtr<IPooledRenderTarget>& PooledTransmittanceLutTexture = SkyAtmosphere->GetTransmittanceLutTexture();
		if (PooledTransmittanceLutTexture.IsValid())
		{
			TransmittanceLutTextureFound = PooledTransmittanceLutTexture->GetRHI();
		}

		DistantSkyLightLutBufferSRVFound = SkyAtmosphere->GetDistantSkyLightLutBufferSRV();
		MobileDistantSkyLightLutBufferSRVFound = SkyAtmosphere->GetMobileDistantSkyLightLutBufferSRV();

		if (this->SkyAtmosphereCameraAerialPerspectiveVolume.IsValid())
		{
			CameraAerialPerspectiveVolumeFound = this->SkyAtmosphereCameraAerialPerspectiveVolume->GetRHI();
		}
		if (this->SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly.IsValid())
		{
			CameraAerialPerspectiveVolumeMieOnlyFound = this->SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly->GetRHI();
		}
		if (this->SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly.IsValid())
		{
			CameraAerialPerspectiveVolumeRayOnlyFound = this->SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly->GetRHI();
		}

		float SkyViewLutWidth = 1.0f;
		float SkyViewLutHeight = 1.0f;
		if (this->SkyAtmosphereViewLutTexture.IsValid())
		{
			SkyViewLutTextureFound = this->SkyAtmosphereViewLutTexture->GetRHI();
			SkyViewLutWidth = float(this->SkyAtmosphereViewLutTexture->GetDesc().GetSize().X);
			SkyViewLutHeight = float(this->SkyAtmosphereViewLutTexture->GetDesc().GetSize().Y);
		}
		ViewUniformShaderParameters.SkyViewLutSizeAndInvSize = FVector4f(SkyViewLutWidth, SkyViewLutHeight, 1.0f / SkyViewLutWidth, 1.0f / SkyViewLutHeight);

		// Now initialize remaining view parameters.

		const FAtmosphereSetup& AtmosphereSetup = SkyAtmosphereSceneProxy.GetAtmosphereSetup();
		ViewUniformShaderParameters.SkyAtmosphereBottomRadiusKm = AtmosphereSetup.BottomRadiusKm;
		ViewUniformShaderParameters.SkyAtmosphereTopRadiusKm = AtmosphereSetup.TopRadiusKm;

		FSkyAtmosphereViewSharedUniformShaderParameters OutParameters;
		SetupSkyAtmosphereViewSharedUniformShaderParameters(*this, SkyAtmosphereSceneProxy, OutParameters);
		ViewUniformShaderParameters.SkyAtmosphereAerialPerspectiveStartDepthKm = OutParameters.AerialPerspectiveStartDepthKm;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize = OutParameters.CameraAerialPerspectiveVolumeSizeAndInvSize;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution = OutParameters.CameraAerialPerspectiveVolumeDepthResolution;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv = OutParameters.CameraAerialPerspectiveVolumeDepthResolutionInv;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm = OutParameters.CameraAerialPerspectiveVolumeDepthSliceLengthKm;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv = OutParameters.CameraAerialPerspectiveVolumeDepthSliceLengthKmInv;
		ViewUniformShaderParameters.SkyAtmosphereApplyCameraAerialPerspectiveVolume = OutParameters.ApplyCameraAerialPerspectiveVolume;
		ViewUniformShaderParameters.SkyAtmosphereSkyLuminanceFactor = SkyAtmosphereSceneProxy.GetSkyLuminanceFactor();
		ViewUniformShaderParameters.SkyAtmosphereHeightFogContribution = SkyAtmosphereSceneProxy.GetHeightFogContribution();

		// Fill atmosphere lights shader parameters
		for (uint8 Index = 0; Index < NUM_ATMOSPHERE_LIGHTS; ++Index)
		{
			FLightSceneInfo* Light = Scene->AtmosphereLights[Index];
			if (Light)
			{
				const float UsePerPixelAtmosphereTransmittance = Light->Proxy->GetUsePerPixelAtmosphereTransmittance() ? 1.0f : 0.0f;
				ViewUniformShaderParameters.AtmosphereLightDiscCosHalfApexAngle_PPTrans[Index] = FVector4f(FMath::Cos(Light->Proxy->GetSunLightHalfApexAngleRadian()), UsePerPixelAtmosphereTransmittance, 0.0f, 0.0f);
				ViewUniformShaderParameters.AtmosphereLightDiscLuminance[Index] = Light->Proxy->GetOuterSpaceLuminance();
				ViewUniformShaderParameters.AtmosphereLightIlluminanceOnGroundPostTransmittance[Index] = Light->Proxy->GetSunIlluminanceOnGroundPostTransmittance();
				ViewUniformShaderParameters.AtmosphereLightIlluminanceOnGroundPostTransmittance[Index].A = 1.0f; // interactions with HeightFogComponent
				ViewUniformShaderParameters.AtmosphereLightIlluminanceOuterSpace[Index] = Light->Proxy->GetOuterSpaceIlluminance();
				ViewUniformShaderParameters.AtmosphereLightIlluminanceOuterSpace[Index].A = 1.0f;
				ViewUniformShaderParameters.AtmosphereLightDirection[Index] = FVector3f(SkyAtmosphereSceneProxy.GetAtmosphereLightDirection(Index, -Light->Proxy->GetDirection()));
			}
			else
			{
				ClearAtmosphereLightData(Index);
			}
		}

		// Regular view sampling of the SkyViewLUT. This is only changed when sampled from a sky material for the real time reflection capture around sky light position)
		FVector3f SkyCameraTranslatedWorldOrigin;
		FMatrix44f SkyViewLutReferential;
		FVector4f TempSkyPlanetData;
		AtmosphereSetup.ComputeViewData(
			InViewMatrices.GetViewOrigin(), InViewMatrices.GetPreViewTranslation(), ViewUniformShaderParameters.ViewForward, ViewUniformShaderParameters.ViewRight,
			SkyCameraTranslatedWorldOrigin, TempSkyPlanetData, SkyViewLutReferential);
		// LWC_TODO: Precision loss
		ViewUniformShaderParameters.SkyPlanetTranslatedWorldCenterAndViewHeight = FVector4f(TempSkyPlanetData);
		ViewUniformShaderParameters.SkyCameraTranslatedWorldOrigin = SkyCameraTranslatedWorldOrigin;
		ViewUniformShaderParameters.SkyViewLutReferential = SkyViewLutReferential;
	}
	else
	{
		ViewUniformShaderParameters.SkyAtmospherePresentInScene = 0.0f;
		ViewUniformShaderParameters.SkyAtmosphereHeightFogContribution = 0.0f;
		ViewUniformShaderParameters.SkyViewLutSizeAndInvSize = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
		ViewUniformShaderParameters.SkyAtmosphereBottomRadiusKm = 1.0f;
		ViewUniformShaderParameters.SkyAtmosphereTopRadiusKm = 1.0f;
		ViewUniformShaderParameters.SkyAtmosphereSkyLuminanceFactor = FLinearColor::White;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize = FVector4f(1.0f, 1.0f, 1.0f, 1.0f);
		ViewUniformShaderParameters.SkyAtmosphereAerialPerspectiveStartDepthKm = 1.0f;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution = 1.0f;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv = 1.0f;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm = 1.0f;
		ViewUniformShaderParameters.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv = 1.0f;
		ViewUniformShaderParameters.SkyAtmosphereApplyCameraAerialPerspectiveVolume = 0.0f;
		ViewUniformShaderParameters.SkyCameraTranslatedWorldOrigin = ViewUniformShaderParameters.TranslatedWorldCameraOrigin;
		ViewUniformShaderParameters.SkyPlanetTranslatedWorldCenterAndViewHeight = FVector4f(ForceInitToZero);
		ViewUniformShaderParameters.SkyViewLutReferential = FMatrix44f::Identity;

		if(Scene)
		{
			// Fill atmosphere lights shader parameters even without any SkyAtmosphere component.
			// This is to always make these parameters usable, for instance by the VolumetricCloud component.
			for (uint8 Index = 0; Index < NUM_ATMOSPHERE_LIGHTS; ++Index)
			{
				FLightSceneInfo* Light = Scene->AtmosphereLights[Index];
				if (Light)
				{
					ViewUniformShaderParameters.AtmosphereLightDiscCosHalfApexAngle_PPTrans[Index] = FVector4f(1.0f, 0.0f, 0.0f, 0.0f);
					ViewUniformShaderParameters.AtmosphereLightDiscLuminance[Index] = FLinearColor::Black;
					ViewUniformShaderParameters.AtmosphereLightIlluminanceOnGroundPostTransmittance[Index] = Light->Proxy->GetColor();
					ViewUniformShaderParameters.AtmosphereLightIlluminanceOnGroundPostTransmittance[Index].A = 0.0f; // no interactions with HeightFogComponent
					ViewUniformShaderParameters.AtmosphereLightIlluminanceOuterSpace[Index] = Light->Proxy->GetColor();
					ViewUniformShaderParameters.AtmosphereLightIlluminanceOuterSpace[0].A = 0.0f;
					ViewUniformShaderParameters.AtmosphereLightDirection[Index] = FVector3f(-Light->Proxy->GetDirection());
				}
				else
				{
					ClearAtmosphereLightData(Index);
				}
			}
		}
		else if (!Scene)
		{
			for (uint8 Index = 0; Index < NUM_ATMOSPHERE_LIGHTS; ++Index)
			{
				ClearAtmosphereLightData(Index);
			}
		}
	}

	ViewUniformShaderParameters.TransmittanceLutTexture = OrWhite2DIfNull(TransmittanceLutTextureFound);
	ViewUniformShaderParameters.TransmittanceLutTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	if(DistantSkyLightLutBufferSRVFound != nullptr)
	{
		ViewUniformShaderParameters.DistantSkyLightLutBufferSRV = DistantSkyLightLutBufferSRVFound;
	}
	else
	{
		ViewUniformShaderParameters.DistantSkyLightLutBufferSRV = GBlackFloat4StructuredBufferWithSRV->ShaderResourceViewRHI.GetReference();
	}
	if(MobileDistantSkyLightLutBufferSRVFound != nullptr)
	{
		ViewUniformShaderParameters.MobileDistantSkyLightLutBufferSRV = MobileDistantSkyLightLutBufferSRVFound;
	}
	else
	{
		ViewUniformShaderParameters.MobileDistantSkyLightLutBufferSRV = GBlackFloat4VertexBufferWithSRV->ShaderResourceViewRHI.GetReference();
	}
	ViewUniformShaderParameters.SkyViewLutTexture = OrBlack2DIfNull(SkyViewLutTextureFound);
	ViewUniformShaderParameters.SkyViewLutTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	ViewUniformShaderParameters.CameraAerialPerspectiveVolume = OrBlack3DAlpha1IfNull(CameraAerialPerspectiveVolumeFound);
	ViewUniformShaderParameters.CameraAerialPerspectiveVolumeMieOnly = OrBlack3DAlpha1IfNull(CameraAerialPerspectiveVolumeMieOnlyFound);
	ViewUniformShaderParameters.CameraAerialPerspectiveVolumeRayOnly = OrBlack3DAlpha1IfNull(CameraAerialPerspectiveVolumeRayOnlyFound);
	ViewUniformShaderParameters.CameraAerialPerspectiveVolumeSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	ViewUniformShaderParameters.CameraAerialPerspectiveVolumeMieOnlySampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	ViewUniformShaderParameters.CameraAerialPerspectiveVolumeRayOnlySampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	ViewUniformShaderParameters.AtmosphereTransmittanceTexture = OrBlack2DIfNull(AtmosphereTransmittanceTexture);
	ViewUniformShaderParameters.AtmosphereIrradianceTexture = OrBlack2DIfNull(AtmosphereIrradianceTexture);
	ViewUniformShaderParameters.AtmosphereInscatterTexture = OrBlack3DIfNull(AtmosphereInscatterTexture);

	ViewUniformShaderParameters.AtmosphereTransmittanceTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	ViewUniformShaderParameters.AtmosphereIrradianceTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	ViewUniformShaderParameters.AtmosphereInscatterTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();

	// Upload environment holdout flags
	ViewUniformShaderParameters.EnvironmentComponentsFlags = FIntVector4(EForceInit::ForceInitToZero);
	if (Scene)
	{
		int32 Flags = 0;
		if (ShouldRenderSkyAtmosphere(Scene, Family->EngineShowFlags))
		{
			FSkyAtmosphereRenderSceneInfo* SkyAtmosphere = Scene->SkyAtmosphere;
			const FSkyAtmosphereSceneProxy& SkyAtmosphereSceneProxy = SkyAtmosphere->GetSkyAtmosphereSceneProxy();

			Flags |= (SkyAtmosphereSceneProxy.IsHoldout() && Family->EngineShowFlags.AllowPrimitiveAlphaHoldout) ? ENVCOMP_FLAG_SKYATMOSPHERE_HOLDOUT : 0;
			Flags |= SkyAtmosphereSceneProxy.IsRenderedInMainPass() ? ENVCOMP_FLAG_SKYATMOSPHERE_RENDERINMAIN : 0;
		}

		if (ShouldRenderVolumetricCloud(Scene, Family->EngineShowFlags))
		{
			FVolumetricCloudRenderSceneInfo* VolumetricCloud = Scene->VolumetricCloud;
			const FVolumetricCloudSceneProxy& VolumetricCloudSceneProxy = VolumetricCloud->GetVolumetricCloudSceneProxy();

			Flags |= (VolumetricCloudSceneProxy.bHoldout && Family->EngineShowFlags.AllowPrimitiveAlphaHoldout) ? ENVCOMP_FLAG_VOLUMETRICCLOUD_HOLDOUT : 0;
			Flags |= VolumetricCloudSceneProxy.bRenderInMainPass ? ENVCOMP_FLAG_VOLUMETRICCLOUD_RENDERINMAIN : 0;
		}

		if (Scene->ExponentialFogs.Num() > 0)
		{
			FExponentialHeightFogSceneInfo& Fog = Scene->ExponentialFogs[0];

			Flags |= (Fog.bHoldout && Family->EngineShowFlags.AllowPrimitiveAlphaHoldout) ? ENVCOMP_FLAG_EXPONENTIALFOG_HOLDOUT : 0;
			Flags |= Fog.bRenderInMainPass ? ENVCOMP_FLAG_EXPONENTIALFOG_RENDERINMAIN : 0;
		}

		ViewUniformShaderParameters.EnvironmentComponentsFlags.X = Flags;
	}

	ViewUniformShaderParameters.MaterialMaxEmissiveValue = MaterialMaxEmissiveValue;
	ViewUniformShaderParameters.PostVolumeUserFlags = FinalPostProcessSettings.UserFlags;

	// This should probably be in SetupCommonViewUniformBufferParameters, but drags in too many dependencies
	UpdateNoiseTextureParameters(ViewUniformShaderParameters);

	SetupDefaultGlobalDistanceFieldUniformBufferParameters(ViewUniformShaderParameters);

	int32 VolumetricFogViewGridPixelSize;
	int32 VolumetricFogResourceGridPixelSize;
	const FIntVector VolumetricFogResourceGridSize = GetVolumetricFogResourceGridSize(*this, VolumetricFogResourceGridPixelSize);
	const FIntVector VolumetricFogViewGridSize = GetVolumetricFogViewGridSize(*this, VolumetricFogViewGridPixelSize);
	const FVector2f ViewRectSize = FVector2f(ViewRect.Size());

	SetupVolumetricFogUniformBufferParameters(ViewUniformShaderParameters);
	ViewUniformShaderParameters.VolumetricFogViewGridUVToPrevViewRectUV = FVector2f::One();
	ViewUniformShaderParameters.VolumetricFogPrevViewGridRectUVToResourceUV = FVector2f::One();
	ViewUniformShaderParameters.VolumetricFogPrevUVMax = FVector2f::One();
	ViewUniformShaderParameters.VolumetricFogPrevUVMaxForTemporalBlend = FVector2f::One();
	ViewUniformShaderParameters.VolumetricFogPrevResourceGridSize = FVector3f(VolumetricFogResourceGridSize);
	if (ViewState)
	{
		// Compute LightScatteringViewGridUVToViewRectVolumeUV, for the current frame resolution and volume texture resolution according to grid size.
		FVector2f LightScatteringViewGridUVToViewRectVolumeUV = ViewRectSize / (FVector2f(VolumetricFogViewGridSize.X, VolumetricFogViewGridSize.Y) * VolumetricFogViewGridPixelSize);

		// Due to dynamic resolution scaling, the previous frame might have had a different screen to volume UV due to padding not being aligned on resolution changes.
		// This effectively correct history samples to account for the change as a ratio of current volume UV to history volume UV.
		ViewUniformShaderParameters.VolumetricFogViewGridUVToPrevViewRectUV = ViewState->PrevLightScatteringViewGridUVToViewRectVolumeUV / LightScatteringViewGridUVToViewRectVolumeUV;

		ViewUniformShaderParameters.VolumetricFogPrevViewGridRectUVToResourceUV = ViewState->VolumetricFogPrevViewGridRectUVToResourceUV;
		ViewUniformShaderParameters.VolumetricFogPrevUVMax = ViewState->VolumetricFogPrevUVMax;
		ViewUniformShaderParameters.VolumetricFogPrevUVMaxForTemporalBlend = ViewState->VolumetricFogPrevUVMaxForTemporalBlend;
		ViewUniformShaderParameters.VolumetricFogPrevResourceGridSize = FVector3f(ViewState->VolumetricFogPrevResourceGridSize);
	}
	ViewUniformShaderParameters.VolumetricFogScreenToResourceUV = FVector2f(VolumetricFogViewGridSize.X, VolumetricFogViewGridSize.Y) / FVector2f(VolumetricFogResourceGridSize.X, VolumetricFogResourceGridSize.Y);
	ViewUniformShaderParameters.VolumetricFogUVMax = GetFroxelGridUVMaxForSampling(ViewRectSize, VolumetricFogResourceGridSize, VolumetricFogResourceGridPixelSize);

	SetupPrecomputedVolumetricLightmapUniformBufferParameters(Scene, Family->EngineShowFlags, ViewUniformShaderParameters);

	SetupPhysicsFieldUniformBufferParameters(Scene, Family->EngineShowFlags, ViewUniformShaderParameters);

	// Setup view's shared sampler for material texture sampling.
	float FinalMaterialTextureMipBias;
	{
		const float GlobalMipBias = UTexture2D::GetGlobalMipMapLODBias();

		FinalMaterialTextureMipBias = GlobalMipBias;

		if (bIsValidTextureGroupSamplerFilters && !FMath::IsNearlyZero(MaterialTextureMipBias))
		{
			ViewUniformShaderParameters.MaterialTextureMipBias = MaterialTextureMipBias;
			ViewUniformShaderParameters.MaterialTextureDerivativeMultiply = FMath::Pow(2.0f, MaterialTextureMipBias);

			FinalMaterialTextureMipBias += MaterialTextureMipBias;
		}

		// Protect access to the view state sampler caches when called from multiple tasks.
		static FCriticalSection CS;
		FScopeLock Lock(&CS);

		FSamplerStateRHIRef WrappedSampler = nullptr;
		FSamplerStateRHIRef ClampedSampler = nullptr;

		if (FMath::Abs(FinalMaterialTextureMipBias - GlobalMipBias) < KINDA_SMALL_NUMBER)
		{
			WrappedSampler = Wrap_WorldGroupSettings->SamplerStateRHI;
			ClampedSampler = Clamp_WorldGroupSettings->SamplerStateRHI;
		}
		else if (ViewState && FMath::Abs(ViewState->MaterialTextureCachedMipBias - FinalMaterialTextureMipBias) < KINDA_SMALL_NUMBER)
		{
			WrappedSampler = ViewState->MaterialTextureBilinearWrapedSamplerCache;
			ClampedSampler = ViewState->MaterialTextureBilinearClampedSamplerCache;
		}
		else
		{
			check(bIsValidTextureGroupSamplerFilters);

			WrappedSampler = RHICreateSamplerState(FSamplerStateInitializerRHI(WorldTextureGroupSamplerFilter, AM_Wrap,  AM_Wrap,  AM_Wrap,  FinalMaterialTextureMipBias, WorldTextureGroupMaxAnisotropy));
			ClampedSampler = RHICreateSamplerState(FSamplerStateInitializerRHI(WorldTextureGroupSamplerFilter, AM_Clamp, AM_Clamp, AM_Clamp, FinalMaterialTextureMipBias, WorldTextureGroupMaxAnisotropy));
		}

		// At this point, a sampler must be set.
		check(WrappedSampler.IsValid());
		check(ClampedSampler.IsValid());

		ViewUniformShaderParameters.MaterialTextureBilinearWrapedSampler = WrappedSampler;
		ViewUniformShaderParameters.MaterialTextureBilinearClampedSampler = ClampedSampler;

		// Update view state's cached sampler.
		if (ViewState && ViewState->MaterialTextureBilinearWrapedSamplerCache != WrappedSampler)
		{
			ViewState->MaterialTextureCachedMipBias = FinalMaterialTextureMipBias;
			ViewState->MaterialTextureBilinearWrapedSamplerCache = WrappedSampler;
			ViewState->MaterialTextureBilinearClampedSamplerCache = ClampedSampler;
		}

		// Landscape global resources
		{
			FSamplerStateRHIRef WeightmapSampler = nullptr;
			if (ViewState && FMath::Abs(ViewState->LandscapeCachedMipBias - FinalMaterialTextureMipBias) < KINDA_SMALL_NUMBER)
			{
				// use cached sampler
				WeightmapSampler = ViewState->LandscapeWeightmapSamplerCache;
			}
			else
			{
				// create a new one
				ESamplerFilter Filter = bIsValidTextureGroupSamplerFilters ? TerrainWeightmapTextureGroupSamplerFilter : SF_AnisotropicPoint;
				WeightmapSampler = RHICreateSamplerState(FSamplerStateInitializerRHI(Filter, AM_Clamp, AM_Clamp, AM_Clamp, FinalMaterialTextureMipBias));
			}
			check(WeightmapSampler.IsValid());
			ViewUniformShaderParameters.LandscapeWeightmapSampler = WeightmapSampler;

			if (ViewState)
			{
				ViewState->LandscapeCachedMipBias = FinalMaterialTextureMipBias;
				ViewState->LandscapeWeightmapSamplerCache = WeightmapSampler;
			}
		}
	}

	{
		ensureMsgf(TemporalJitterSequenceLength == 1 || IsTemporalAccumulationBasedMethod(AntiAliasingMethod) || (CustomRenderPass && FSceneCaptureCustomRenderPassUserData::Get(CustomRenderPass).bMainViewResolution),
			TEXT("TemporalJitterSequenceLength = %i is invalid"), TemporalJitterSequenceLength);
		ensureMsgf(TemporalJitterIndex >= 0 && TemporalJitterIndex < TemporalJitterSequenceLength,
			TEXT("TemporalJitterIndex = %i is invalid (TemporalJitterSequenceLength = %i)"), TemporalJitterIndex, TemporalJitterSequenceLength);
		ViewUniformShaderParameters.TemporalAAParams = FVector4f(
			TemporalJitterIndex, 
			TemporalJitterSequenceLength,
			TemporalJitterPixels.X,
			TemporalJitterPixels.Y);
	}

	{
		float ResolutionFraction = float(ViewRect.Width()) / float(UnscaledViewRect.Width());
		
		// When the primary upscaler is not temporal AA, the resolution is the same between the rendering pass
		// and the post processing pass after upscaling. No need to modify the ResolutionFractionAndInv.
		// When ResolutionFraction > 1.0, it is used for other purpose, e.g., in Shadow. 
		if (PrimaryScreenPercentageMethod != EPrimaryScreenPercentageMethod::TemporalUpscale
			&& ResolutionFraction < 1.0f)
		{
			ResolutionFraction = 1.0f;
		}

		ViewUniformShaderParameters.ResolutionFractionAndInv.X = ResolutionFraction;
		ViewUniformShaderParameters.ResolutionFractionAndInv.Y = 1.0f / ResolutionFraction;
	}

	uint32 FrameIndex = 0;
	uint32 OutputFrameIndex = 0;
	if (ViewState)
	{
		FrameIndex = ViewState->GetFrameIndex();
		OutputFrameIndex = ViewState->GetOutputFrameIndex();
	}

	// TODO(GA): kill StateFrameIndexMod8 because this is only a scalar bit mask with StateFrameIndex anyway.
	ViewUniformShaderParameters.StateFrameIndexMod8 = FrameIndex % 8;
	ViewUniformShaderParameters.StateFrameIndex = FrameIndex;
	ViewUniformShaderParameters.StateOutputFrameIndex = OutputFrameIndex;

	{
		// If rendering in stereo, the other stereo passes uses the left eye's translucency lighting volume.
		const FViewInfo* PrimaryView = GetPrimaryView();
		PrimaryView->CalcTranslucencyLightingVolumeBounds(OutTranslucentCascadeBoundsArray, NumTranslucentCascades);

		const int32 TranslucencyLightingVolumeDim = GetTranslucencyLightingVolumeDim();
		for (int32 CascadeIndex = 0; CascadeIndex < NumTranslucentCascades; CascadeIndex++)
		{
			const float VolumeVoxelSize = (OutTranslucentCascadeBoundsArray[CascadeIndex].Max.X - OutTranslucentCascadeBoundsArray[CascadeIndex].Min.X) / TranslucencyLightingVolumeDim;
			const FVector VolumeWorldMin = OutTranslucentCascadeBoundsArray[CascadeIndex].Min;
			const FVector3f VolumeSize = FVector3f(OutTranslucentCascadeBoundsArray[CascadeIndex].Max - VolumeWorldMin);
			const FVector3f VolumeTranslatedWorldMin = FVector3f(VolumeWorldMin + PrimaryView->ViewMatrices.GetPreViewTranslation());

			ViewUniformShaderParameters.TranslucencyLightingVolumeMin[CascadeIndex] = FVector4f(VolumeTranslatedWorldMin, 1.0f / TranslucencyLightingVolumeDim);
			ViewUniformShaderParameters.TranslucencyLightingVolumeInvSize[CascadeIndex] = FVector4f(FVector3f(1.0f) / VolumeSize, VolumeVoxelSize);
		}
	}
	
	ViewUniformShaderParameters.PreExposure = PreExposure;
	ViewUniformShaderParameters.OneOverPreExposure = 1.f / PreExposure;

	ViewUniformShaderParameters.DepthOfFieldFocalDistance = FinalPostProcessSettings.DepthOfFieldFocalDistance;
	ViewUniformShaderParameters.DepthOfFieldSensorWidth = FinalPostProcessSettings.DepthOfFieldSensorWidth;
	ViewUniformShaderParameters.DepthOfFieldFocalRegion = FinalPostProcessSettings.DepthOfFieldFocalRegion;
	// clamped to avoid div by 0 in shader
	ViewUniformShaderParameters.DepthOfFieldNearTransitionRegion = FMath::Max(0.01f, FinalPostProcessSettings.DepthOfFieldNearTransitionRegion);
	// clamped to avoid div by 0 in shader
	ViewUniformShaderParameters.DepthOfFieldFarTransitionRegion = FMath::Max(0.01f, FinalPostProcessSettings.DepthOfFieldFarTransitionRegion);
	ViewUniformShaderParameters.DepthOfFieldScale = FinalPostProcessSettings.DepthOfFieldScale;
	ViewUniformShaderParameters.DepthOfFieldFocalLength = 50.0f;

	// Subsurface
	{
		ViewUniformShaderParameters.bSubsurfacePostprocessEnabled = IsSubsurfaceEnabled(Family->EngineShowFlags) ? 1.0f : 0.0f;

		// Subsurface shading model
		{
			static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.SSS.SubSurfaceColorAsTansmittanceAtDistance"));
			const float SSSDistanceInMeters = CVar ? FMath::Clamp(CVar->GetValueOnRenderThread(), 0.05f, 1.0f) : 0.15f; // Default 0.15 normalized unit
			ViewUniformShaderParameters.SubSurfaceColorAsTransmittanceAtDistanceInMeters = SSSDistanceInMeters;
		}

		// Profiles
		{
			FRHITexture* Texture = SubsurfaceProfile::GetSubsurfaceProfileTextureWithFallback();
			FIntVector TextureSize = Texture->GetSizeXYZ();
			ViewUniformShaderParameters.SSProfilesTextureSizeAndInvSize = FVector4f(TextureSize.X, TextureSize.Y, 1.0f / TextureSize.X, 1.0f / TextureSize.Y);
			ViewUniformShaderParameters.SSProfilesTexture = Texture;
			ViewUniformShaderParameters.SSProfilesSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			ViewUniformShaderParameters.SSProfilesTransmissionSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}

		// Pre-integrated profiles
		{
			FRHITexture* Texture = SubsurfaceProfile::GetSSProfilesPreIntegratedTextureWithFallback();
			FIntVector TextureSize = Texture->GetSizeXYZ();
			ViewUniformShaderParameters.SSProfilesPreIntegratedTextureSizeAndInvSize = FVector4f(TextureSize.X, TextureSize.Y, 1.0f / TextureSize.X, 1.0f / TextureSize.Y);
			ViewUniformShaderParameters.SSProfilesPreIntegratedTexture = Texture;
			ViewUniformShaderParameters.SSProfilesPreIntegratedSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
		}
	}

	// Specular Profiles
	{
		FRHITexture* Texture = SpecularProfile::GetSpecularProfileTextureAtlasWithFallback();
		FIntVector TextureSize = Texture->GetSizeXYZ();
		ViewUniformShaderParameters.SpecularProfileTextureSizeAndInvSize = FVector4f(TextureSize.X, TextureSize.Y, 1.0f / TextureSize.X, 1.0f / TextureSize.Y);
		ViewUniformShaderParameters.SpecularProfileTexture = Texture;
		ViewUniformShaderParameters.SpecularProfileSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}

	// Toon Profiles
	{
		FRHITexture* Texture = ToonProfile::GetToonProfileTextureAtlasWithFallback();
		FIntVector TextureSize = Texture->GetSizeXYZ();
		ViewUniformShaderParameters.ToonProfileTextureSizeAndInvSize = FVector4f(TextureSize.X, TextureSize.Y, 1.0f / TextureSize.X, 1.0f / TextureSize.Y);

		// See ToonProfile.usf for the baking/sampling description.
		const float  ToonProfileDitherPatternFullResolution = ViewUniformShaderParameters.ToonProfileTextureSizeAndInvSize.X;
		const float  ToonProfileDitherPatternBakeResolution = ToonProfileDitherPatternFullResolution - 2.0f;
		const float  ToonProfileDitherPatternBakeResolutionInv = 1.0f / ToonProfileDitherPatternBakeResolution;
		ViewUniformShaderParameters.ToonProfileDitherPatternSizeAndInvSize = FVector2f(ToonProfileDitherPatternBakeResolution, ToonProfileDitherPatternBakeResolutionInv);

		ViewUniformShaderParameters.ToonProfileTexture = Texture;
		ViewUniformShaderParameters.ToonProfileSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}

	{
		// This is the CVar default
		float Value = 1.0f;
		float Value2 = 1.0f;

		// Compiled out in SHIPPING to make cheating a bit harder.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		Value = CVarGeneralPurposeTweak.GetValueOnRenderThread();
		Value2 = CVarGeneralPurposeTweak2.GetValueOnRenderThread();
#endif

		ViewUniformShaderParameters.GeneralPurposeTweak = Value;
		ViewUniformShaderParameters.GeneralPurposeTweak2 = Value2;
	}

	ViewUniformShaderParameters.DemosaicVposOffset = 0.0f;
	{
		ViewUniformShaderParameters.DemosaicVposOffset = CVarDemosaicVposOffset.GetValueOnRenderThread();
	}

	ViewUniformShaderParameters.DecalDepthBias = CVarDecalDepthBias.GetValueOnRenderThread() * InViewMatrices.GetPerProjectionDepthThicknessScale();

	ViewUniformShaderParameters.IndirectLightingColorScale = FVector3f(FinalPostProcessSettings.IndirectLightingColor.R * FinalPostProcessSettings.IndirectLightingIntensity,
		FinalPostProcessSettings.IndirectLightingColor.G * FinalPostProcessSettings.IndirectLightingIntensity,
		FinalPostProcessSettings.IndirectLightingColor.B * FinalPostProcessSettings.IndirectLightingIntensity);

	ViewUniformShaderParameters.PrecomputedIndirectLightingColorScale = ViewUniformShaderParameters.IndirectLightingColorScale;

	// If Lumen Dynamic GI is enabled then we don't want GI from Lightmaps
	// Note: this has the side effect of removing direct lighting from Static Lights
	if (ShouldRenderLumenDiffuseGI(Scene, *this))
	{
		ViewUniformShaderParameters.PrecomputedIndirectLightingColorScale = FVector3f::ZeroVector;
	}

	ViewUniformShaderParameters.PrecomputedIndirectSpecularColorScale = ViewUniformShaderParameters.IndirectLightingColorScale;

	// If Lumen Reflections are enabled then we don't want precomputed reflections from reflection captures
	// Note: this has the side effect of removing direct specular from Static Lights
	if (ShouldRenderLumenReflections(*this, false, false, /* bIncludeStandalone */ false))
	{
		ViewUniformShaderParameters.PrecomputedIndirectSpecularColorScale = FVector3f::ZeroVector;
	}

	ViewUniformShaderParameters.NormalCurvatureToRoughnessScaleBias.X = FMath::Clamp(CVarNormalCurvatureToRoughnessScale.GetValueOnAnyThread(), 0.0f, 2.0f);
	ViewUniformShaderParameters.NormalCurvatureToRoughnessScaleBias.Y = FMath::Clamp(CVarNormalCurvatureToRoughnessBias.GetValueOnAnyThread(), -1.0f, 1.0f);
	ViewUniformShaderParameters.NormalCurvatureToRoughnessScaleBias.Z = FMath::Clamp(CVarNormalCurvatureToRoughnessExponent.GetValueOnAnyThread(), .05f, 20.0f);

	ViewUniformShaderParameters.RenderingReflectionCaptureMask = bIsReflectionCapture ? 1.0f : 0.0f;
	ViewUniformShaderParameters.RealTimeReflectionCapture = 0.0f;
	ViewUniformShaderParameters.RealTimeReflectionCapturePreExposure = EyeAdaptation::GetCachedLightingPreExposure();

	ViewUniformShaderParameters.bPrimitiveAlphaHoldoutEnabled = IsPrimitiveAlphaHoldoutEnabled(*this);

	ViewUniformShaderParameters.AmbientCubemapTint = FinalPostProcessSettings.AmbientCubemapTint;
	ViewUniformShaderParameters.AmbientCubemapIntensity = FinalPostProcessSettings.AmbientCubemapIntensity;

	ViewUniformShaderParameters.CircleDOFParams = DiaphragmDOF::CircleDofHalfCoc(*this);

	ViewUniformShaderParameters.SceneColorTextureFormatQuantizationError = ComputePixelFormatQuantizationError(SceneTexturesConfig.ColorFormat);

	if (Scene && Scene->SkyLight)
	{
		FSkyLightSceneProxy* SkyLight = Scene->SkyLight;

		float SkyPreExposureInv = 1.0f;
		if (SkyLight->bRealTimeCaptureEnabled)
		{
			SkyPreExposureInv = 1.0f / EyeAdaptation::GetCachedLightingPreExposure();
		}

		// Setup the sky color mulitpler, and use it to nullify the sky contribution in case SkyLighting is disabled.
		// Note: we cannot simply select the base pass shader permutation skylight=0 because we would need to trigger bScenesPrimitivesNeedStaticMeshElementUpdate.
		// However, this would need to be done per view (showflag is per view) and this is not possible today as it is selected within the scene. 
		// So we simply nullify the sky light diffuse contribution. Reflection are handled by the indirect lighting render pass.
		ViewUniformShaderParameters.SkyLightColor = Family->EngineShowFlags.SkyLighting ? SkyLight->GetEffectiveLightColor() * SkyPreExposureInv * SkylightScale : FLinearColor::Black;

		bool bApplyPrecomputedBentNormalShadowing = 
			SkyLight->bCastShadows 
			&& SkyLight->bWantsStaticShadowing;

		ViewUniformShaderParameters.SkyLightApplyPrecomputedBentNormalShadowingFlag = bApplyPrecomputedBentNormalShadowing ? 1.0f : 0.0f;
		ViewUniformShaderParameters.SkyLightAffectReflectionFlag = SkyLight->bAffectReflection ? 1.0f : 0.0f;
		ViewUniformShaderParameters.SkyLightAffectGlobalIlluminationFlag = SkyLight->bAffectGlobalIllumination ? 1.0f : 0.0f;
		ViewUniformShaderParameters.SkyLightVolumetricScatteringIntensity = SkyLight->VolumetricScatteringIntensity;
	}
	else
	{
		ViewUniformShaderParameters.SkyLightColor = FLinearColor::Black;
		ViewUniformShaderParameters.SkyLightApplyPrecomputedBentNormalShadowingFlag = 0.0f;
		ViewUniformShaderParameters.SkyLightAffectReflectionFlag = 0.0f;
		ViewUniformShaderParameters.SkyLightAffectGlobalIlluminationFlag = 0.0f;
		ViewUniformShaderParameters.SkyLightVolumetricScatteringIntensity = 0.0f;
	}

	if (RHIFeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		// Make sure there's no padding since we're going to cast to FVector4f*
		static_assert(sizeof(ViewUniformShaderParameters.MobileSkyIrradianceEnvironmentMap) == sizeof(FVector4f) * SKY_IRRADIANCE_ENVIRONMENT_MAP_VEC4_COUNT, "unexpected sizeof ViewUniformShaderParameters.MobileSkyIrradianceEnvironmentMap");

		const bool bSetupSkyIrradiance = Scene
			&& Scene->SkyLight
			// Skylights with static lighting already had their diffuse contribution baked into lightmaps
			&& (!Scene->SkyLight->bHasStaticLighting || !IsStaticLightingAllowed())
			&& Family->EngineShowFlags.SkyLighting;

		const bool bMobileRealTimeSkyLightCapture = Scene && Scene->CanSampleSkyLightRealTimeCaptureData() && Family->EngineShowFlags.SkyLighting;

		if (bMobileRealTimeSkyLightCapture)
		{
			FMemory::Memcpy((void*)&ViewUniformShaderParameters.MobileSkyIrradianceEnvironmentMap, (void*)&Scene->MobileSkyLightRealTimeCaptureIrradianceEnvironmentMap, sizeof(Scene->MobileSkyLightRealTimeCaptureIrradianceEnvironmentMap));
		}
		else if (bSetupSkyIrradiance)
		{
			const FSHVectorRGB3& SkyIrradiance = Scene->SkyLight->IrradianceEnvironmentMap;
			SetupSkyIrradianceEnvironmentMapConstantsFromSkyIrradiance((FVector4f*)&ViewUniformShaderParameters.MobileSkyIrradianceEnvironmentMap, SkyIrradiance);
			ViewUniformShaderParameters.MobileSkyIrradianceEnvironmentMap[7].X = Scene->SkyLight->AverageBrightness;
		}
		else
		{
			FMemory::Memzero((FVector4f*)&ViewUniformShaderParameters.MobileSkyIrradianceEnvironmentMap, sizeof(FVector4f) * SKY_IRRADIANCE_ENVIRONMENT_MAP_VEC4_COUNT);
		}
	}
	else
	{
		if (Scene && Scene->SkyIrradianceEnvironmentMap)
		{
			ViewUniformShaderParameters.SkyIrradianceEnvironmentMap = Scene->SkyIrradianceEnvironmentMap->GetSRV();
		}
		else
		{
			ViewUniformShaderParameters.SkyIrradianceEnvironmentMap = GIdentityPrimitiveBuffer.SkyIrradianceEnvironmentMapSRV;
		}
	}
	ViewUniformShaderParameters.MobilePreviewMode =
		(GIsEditor &&
		(RHIFeatureLevel == ERHIFeatureLevel::ES3_1) &&
		GMaxRHIFeatureLevel > ERHIFeatureLevel::ES3_1) ? 1.0f : 0.0f;

	// Padding between the left and right eye may be introduced by an HMD, which instanced stereo needs to account for.
	if ((IStereoRendering::IsStereoEyePass(StereoPass)) && (Family->Views.Num() > 1))
	{
		check(Family->Views.Num() >= 2);

		// The static_cast<const FViewInfo*> is fine because when executing this method, we know that
		// Family::Views point to multiple FViewInfo, since of them is <this>.
		const float StereoViewportWidth = float(
			static_cast<const FViewInfo*>(Family->Views[1])->ViewRect.Max.X - 
			static_cast<const FViewInfo*>(Family->Views[0])->ViewRect.Min.X);
		const float EyePaddingSize = float(
			static_cast<const FViewInfo*>(Family->Views[1])->ViewRect.Min.X -
			static_cast<const FViewInfo*>(Family->Views[0])->ViewRect.Max.X);

		ViewUniformShaderParameters.HMDEyePaddingOffset = (StereoViewportWidth - EyePaddingSize) / StereoViewportWidth;
	}
	else
	{
		ViewUniformShaderParameters.HMDEyePaddingOffset = 1.0f;
	}

	ViewUniformShaderParameters.ReflectionCubemapMaxMip = FMath::FloorLog2(UReflectionCaptureComponent::GetReflectionCaptureSize());

	ViewUniformShaderParameters.ShowDecalsMask = Family->EngineShowFlags.Decals ? 1.0f : 0.0f;

	extern int32 GDistanceFieldAOSpecularOcclusionMode;
	ViewUniformShaderParameters.DistanceFieldAOSpecularOcclusionMode = GDistanceFieldAOSpecularOcclusionMode;

	ViewUniformShaderParameters.IndirectCapsuleSelfShadowingIntensity = Scene ? Scene->DynamicIndirectShadowsSelfShadowingIntensity : 1.0f;

	extern FVector GetReflectionEnvironmentRoughnessMixingScaleBiasAndLargestWeight();
	ViewUniformShaderParameters.ReflectionEnvironmentRoughnessMixingScaleBiasAndLargestWeight = (FVector3f)GetReflectionEnvironmentRoughnessMixingScaleBiasAndLargestWeight();

	ViewUniformShaderParameters.StereoPassIndex = StereoViewIndex != INDEX_NONE ? StereoViewIndex : 0;

	{
		auto XRCamera = GEngine->XRSystem ? GEngine->XRSystem->GetXRCamera() : nullptr;
		TArray<FVector2D> CameraUVs;
		if (XRCamera.IsValid() && XRCamera->GetPassthroughCameraUVs_RenderThread(CameraUVs) && CameraUVs.Num() == 4)
		{
			ViewUniformShaderParameters.XRPassthroughCameraUVs[0] = FVector4f(FVector2f(CameraUVs[0]), FVector2f(CameraUVs[1]));
			ViewUniformShaderParameters.XRPassthroughCameraUVs[1] = FVector4f(FVector2f(CameraUVs[2]), FVector2f(CameraUVs[3]));
		}
		else
		{
			ViewUniformShaderParameters.XRPassthroughCameraUVs[0] = FVector4f(0, 0, 0, 1);
			ViewUniformShaderParameters.XRPassthroughCameraUVs[1] = FVector4f(1, 0, 1, 1);
		}
	}

	if (DrawDynamicFlags & EDrawDynamicFlags::FarShadowCascade)
	{
		extern ENGINE_API int32 GFarShadowStaticMeshLODBias;
		ViewUniformShaderParameters.FarShadowStaticMeshLODBias = GFarShadowStaticMeshLODBias;
	}
	else
	{
		ViewUniformShaderParameters.FarShadowStaticMeshLODBias = 0;
	}

	if (GEngine->PreIntegratedSkinBRDFTexture)
	{
		const FTextureResource* TextureResource = GEngine->PreIntegratedSkinBRDFTexture->GetResource();
		if (TextureResource)
		{
			ViewUniformShaderParameters.PreIntegratedBRDF = TextureResource->TextureRHI;
		}
	}

	const uint32 VirtualTextureFrameIndex = ViewState ? ViewState->GetFrameIndex() : Family->FrameNumber;
	const uint32 VirtualTextureFeedbackTileSize = Family->VirtualTextureFeedbackFactor;
	VirtualTexture::FFeedbackShaderParams Params;
	VirtualTexture::GetFeedbackShaderParams(VirtualTextureFrameIndex, VirtualTextureFeedbackTileSize, Params);
	VirtualTexture::UpdateViewUniformShaderParameters(Params, ViewUniformShaderParameters);

	ViewUniformShaderParameters.GlobalVirtualTextureMipBias = FVirtualTextureSystem::Get().GetGlobalMipBias();

	// GGX/Sheen LTC (used as BSDF or for rect light integration)
	if (GSystemTextures.GGXLTCMat.IsValid() && GSystemTextures.GGXLTCAmp.IsValid())
	{
		ViewUniformShaderParameters.GGXLTCMatTexture = GSystemTextures.GGXLTCMat->GetRHI();
		ViewUniformShaderParameters.GGXLTCAmpTexture = GSystemTextures.GGXLTCAmp->GetRHI();
	}
	if (GSystemTextures.SheenLTC.IsValid())
	{
		ViewUniformShaderParameters.SheenLTCTexture = GSystemTextures.SheenLTC->GetRHI();
	}
	ViewUniformShaderParameters.GGXLTCMatTexture	= OrBlack2DIfNull(ViewUniformShaderParameters.GGXLTCMatTexture);
	ViewUniformShaderParameters.GGXLTCAmpTexture	= OrBlack2DIfNull(ViewUniformShaderParameters.GGXLTCAmpTexture);
	ViewUniformShaderParameters.SheenLTCTexture		= OrBlack2DIfNull(ViewUniformShaderParameters.SheenLTCTexture);
	ViewUniformShaderParameters.GGXLTCMatSampler	= TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	ViewUniformShaderParameters.GGXLTCAmpSampler	= TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	ViewUniformShaderParameters.SheenLTCSampler		= TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Rect light. atlas
	{
		FRHITexture* AtlasTexture = RectLightAtlas::GetAtlasTexture();
		if (!AtlasTexture && GSystemTextures.BlackDummy.IsValid())
		{
			AtlasTexture = GSystemTextures.BlackDummy->GetRHI();
		}
				
		if (AtlasTexture)
		{
			const FIntVector AtlasSize = AtlasTexture->GetSizeXYZ();
			ViewUniformShaderParameters.RectLightAtlasTexture = AtlasTexture;
			ViewUniformShaderParameters.RectLightAtlasSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			ViewUniformShaderParameters.RectLightAtlasMaxMipLevel = AtlasTexture->GetNumMips() - 1;
			ViewUniformShaderParameters.RectLightAtlasSizeAndInvSize = FVector4f(AtlasSize.X, AtlasSize.Y, 1.0f / AtlasSize.X, 1.0f / AtlasSize.Y);
		}
		ViewUniformShaderParameters.RectLightAtlasTexture = OrBlack2DIfNull(ViewUniformShaderParameters.RectLightAtlasTexture);
	}

	// IES atlas
	{
		FRHITexture* AtlasTexture = IESAtlas::GetAtlasTexture();
		if (!AtlasTexture && GSystemTextures.BlackArrayDummy.IsValid())
		{
			AtlasTexture = GSystemTextures.BlackArrayDummy->GetRHI();
		}
				
		if (AtlasTexture)
		{
			const FIntVector AtlasSize = AtlasTexture->GetSizeXYZ();
			ViewUniformShaderParameters.IESAtlasTexture = AtlasTexture;
			ViewUniformShaderParameters.IESAtlasSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			ViewUniformShaderParameters.IESAtlasSizeAndInvSize = FVector4f(AtlasSize.X, AtlasSize.Y, 1.0f / AtlasSize.X, 1.0f / AtlasSize.Y);
		}
		ViewUniformShaderParameters.IESAtlasTexture = OrBlack2DArrayIfNull(ViewUniformShaderParameters.IESAtlasTexture);
	}

	// Hair global resources 
	SetUpViewHairRenderInfo(*this, ViewUniformShaderParameters.HairRenderInfo, ViewUniformShaderParameters.HairRenderInfoBits, ViewUniformShaderParameters.HairComponents);
	ViewUniformShaderParameters.HairScatteringLUTTexture = nullptr;
	if (GSystemTextures.HairLUT0.IsValid() && GSystemTextures.HairLUT0->GetRHI())
	{
		ViewUniformShaderParameters.HairScatteringLUTTexture = GSystemTextures.HairLUT0->GetRHI();
	}
	ViewUniformShaderParameters.HairScatteringLUTTexture = OrBlack3DIfNull(ViewUniformShaderParameters.HairScatteringLUTTexture);
	ViewUniformShaderParameters.HairScatteringLUTSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Shading energy conservation
	{
		const FShadingEnergyConservationData ShadingEnergyConservationData = ShadingEnergyConservation::GetData(*this);
		ViewUniformShaderParameters.bShadingEnergyConservation		= ShadingEnergyConservationData.bEnergyConservation ? 1u : 0u;
		ViewUniformShaderParameters.bShadingEnergyPreservation		= ShadingEnergyConservationData.bEnergyPreservation ? 1u : 0u;
		ViewUniformShaderParameters.ShadingEnergyGGXSpecTexture		= ShadingEnergyConservationData.GGXSpecEnergyTexture ? ShadingEnergyConservationData.GGXSpecEnergyTexture->GetRHI() : nullptr;
		ViewUniformShaderParameters.ShadingEnergyGGXMaxSpecTexture	= ShadingEnergyConservationData.GGXMaxSpecEnergyTexture ? ShadingEnergyConservationData.GGXMaxSpecEnergyTexture->GetRHI() : nullptr;
		ViewUniformShaderParameters.ShadingEnergyGGXGlassTexture	= ShadingEnergyConservationData.GGXGlassEnergyTexture ?ShadingEnergyConservationData.GGXGlassEnergyTexture->GetRHI() : nullptr;
		ViewUniformShaderParameters.ShadingEnergyClothSpecTexture	= ShadingEnergyConservationData.ClothEnergyTexture ?   ShadingEnergyConservationData.ClothEnergyTexture->GetRHI() : nullptr;
		ViewUniformShaderParameters.ShadingEnergyDiffuseTexture		= ShadingEnergyConservationData.DiffuseEnergyTexture ? ShadingEnergyConservationData.DiffuseEnergyTexture->GetRHI() : nullptr;
	}
	ViewUniformShaderParameters.ShadingEnergyGGXSpecTexture		 = OrBlack2DIfNull(ViewUniformShaderParameters.ShadingEnergyGGXSpecTexture);
	ViewUniformShaderParameters.ShadingEnergyGGXMaxSpecTexture	 = OrBlack2DIfNull(ViewUniformShaderParameters.ShadingEnergyGGXMaxSpecTexture);
	ViewUniformShaderParameters.ShadingEnergyGGXGlassTexture	 = OrBlack3DIfNull(ViewUniformShaderParameters.ShadingEnergyGGXGlassTexture);
	ViewUniformShaderParameters.ShadingEnergyClothSpecTexture	 = OrBlack2DIfNull(ViewUniformShaderParameters.ShadingEnergyClothSpecTexture);
	ViewUniformShaderParameters.ShadingEnergyDiffuseTexture		 = OrBlack2DIfNull(ViewUniformShaderParameters.ShadingEnergyDiffuseTexture);
	ViewUniformShaderParameters.ShadingEnergySampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	// Glint
	ViewUniformShaderParameters.GlintSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	{
		const FGlintShadingLUTsStateData GlintData = FGlintShadingLUTsStateData::GetParameters(*this);
		ViewUniformShaderParameters.GlintTexture = GlintData.RHIGlintShadingLUTs;
		ViewUniformShaderParameters.GlintLUTParameters0 = FVector4f(
			GlintData.Dictionary_Alpha,
			*reinterpret_cast<const float*>(&GlintData.Dictionary_N),
			*reinterpret_cast<const float*>(&GlintData.Dictionary_NLevels),
			Substrate::GlintLevelBias());
		ViewUniformShaderParameters.GlintLUTParameters1 = FVector4f(
			Substrate::GlintLevelMin(),
			0.0f, 0.0f, 0.0f);
	}
	ViewUniformShaderParameters.GlintTexture = OrBlack2DArrayIfNull(ViewUniformShaderParameters.GlintTexture);

	ViewUniformShaderParameters.SimpleVolumeTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	ViewUniformShaderParameters.SimpleVolumeEnvTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();

	if (GEngine->SimpleVolumeTexture)
	{
		const FTextureResource* SimpleVolumeTextureResources = GEngine->SimpleVolumeTexture->GetResource();
		if (SimpleVolumeTextureResources)
		{
			ViewUniformShaderParameters.SimpleVolumeTexture = SimpleVolumeTextureResources->TextureRHI->GetTexture3D();
		}
	}
	ViewUniformShaderParameters.SimpleVolumeTexture = OrBlack3DIfNull(ViewUniformShaderParameters.SimpleVolumeTexture);

	if (GEngine->SimpleVolumeEnvTexture)
	{
		const FTextureResource* SimpleVolumeEnvTextureResources = GEngine->SimpleVolumeEnvTexture->GetResource();
		if (SimpleVolumeEnvTextureResources)
		{
			ViewUniformShaderParameters.SimpleVolumeEnvTexture = SimpleVolumeEnvTextureResources->TextureRHI->GetTexture3D();
		}
	}
	ViewUniformShaderParameters.SimpleVolumeEnvTexture = OrBlack3DIfNull(ViewUniformShaderParameters.SimpleVolumeEnvTexture);

	// Water global resources
	if (WaterDataBuffer.IsValid() && WaterIndirectionBuffer.IsValid())
	{
		ViewUniformShaderParameters.WaterIndirection = WaterIndirectionBuffer.GetReference();
		ViewUniformShaderParameters.WaterData = WaterDataBuffer.GetReference();
	}
	else
	{
		ViewUniformShaderParameters.WaterIndirection = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
		ViewUniformShaderParameters.WaterData = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
	}
	ViewUniformShaderParameters.WaterInfoTextureViewIndex = WaterInfoTextureViewIndex;

	if (LandscapePerComponentDataBuffer.IsValid() && LandscapeIndirectionBuffer.IsValid())
	{
		ViewUniformShaderParameters.LandscapeIndirection = LandscapeIndirectionBuffer.GetReference();
		ViewUniformShaderParameters.LandscapePerComponentData = LandscapePerComponentDataBuffer.GetReference();
	}
	else
	{
		ViewUniformShaderParameters.LandscapeIndirection = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
		ViewUniformShaderParameters.LandscapePerComponentData = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
	}

	ViewUniformShaderParameters.GPUSceneViewId = SceneRendererPrimaryViewId; // TODO: GPUSceneViewId should be deprecated and renamed to SceneRendererPrimaryViewId

	{
		FBlueNoiseParameters BlueNoiseParam = GetBlueNoiseParametersForView();
		ViewUniformShaderParameters.BlueNoiseScalarTexture	= BlueNoiseParam.ScalarTexture;
		ViewUniformShaderParameters.BlueNoiseDimensions		= BlueNoiseParam.Dimensions;
		ViewUniformShaderParameters.BlueNoiseModuloMasks	= BlueNoiseParam.ModuloMasks;
	}
}

void FViewInfo::InitRHIResources(uint32 OverrideNumMSAASamples)
{
	FBox VolumeBounds[TVC_MAX];

	check(IsInRenderingThread());

	if (!CachedViewUniformShaderParameters)
	{
		CachedViewUniformShaderParameters = MakeUnique<FViewUniformShaderParameters>();
	}

	SetupUniformBufferParameters(
		VolumeBounds,
		TVC_MAX,
		*CachedViewUniformShaderParameters);

	if (OverrideNumMSAASamples > 0)
	{
		CachedViewUniformShaderParameters->NumSceneColorMSAASamples = OverrideNumMSAASamples;
	}

	CreateViewUniformBuffers(*CachedViewUniformShaderParameters);

	const int32 TranslucencyLightingVolumeDim = GetTranslucencyLightingVolumeDim();

	for (int32 CascadeIndex = 0; CascadeIndex < TVC_MAX; CascadeIndex++)
	{
		TranslucencyLightingVolumeMin[CascadeIndex] = VolumeBounds[CascadeIndex].Min;
		TranslucencyVolumeVoxelSize[CascadeIndex] = (VolumeBounds[CascadeIndex].Max.X - VolumeBounds[CascadeIndex].Min.X) / TranslucencyLightingVolumeDim;
		TranslucencyLightingVolumeSize[CascadeIndex] = VolumeBounds[CascadeIndex].Max - VolumeBounds[CascadeIndex].Min;
	}
}

void FViewInfo::CreateViewUniformBuffers(const FViewUniformShaderParameters& Params)
{
	ViewUniformBuffer = TUniformBufferRef<FViewUniformShaderParameters>::CreateUniformBufferImmediate(Params, UniformBuffer_SingleFrame);
	if (bShouldBindInstancedViewUB)
	{
		FInstancedViewUniformShaderParameters LocalInstancedViewUniformShaderParameters;
		//always copy the left/primary view in array index 0
		InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, Params, 0);

		if (const FViewInfo* InstancedView = GetInstancedView())
		{
			// Copy instanced view (usually right view) into array index 1
			checkf(InstancedView->CachedViewUniformShaderParameters.IsValid(), TEXT("Instanced view should have had its RHI resources initialized first. Check InitViews order."));
			InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, *InstancedView->CachedViewUniformShaderParameters, 1);
		}
		else
		{
			// If we don't render this view in stereo, we simply initialize index 1 with the existing contents from primary view
			InstancedViewParametersUtils::CopyIntoInstancedViewParameters(LocalInstancedViewUniformShaderParameters, Params, 1);
		}

		InstancedViewUniformBuffer = TUniformBufferRef<FInstancedViewUniformShaderParameters>::CreateUniformBufferImmediate(LocalInstancedViewUniformShaderParameters, UniformBuffer_SingleFrame);
	}
}

extern TSet<IPersistentViewUniformBufferExtension*> PersistentViewUniformBufferExtensions;

FIntRect FViewInfo::GetFamilyViewRect() const
{
	if (IsInstancedStereo())
	{
		return ViewRectWithSecondaryViews;
	}

	FIntRect FamilyRect = {};
	for (uint64 ViewIdx = 0, NumViews = (uint64)Family->Views.Num(); ViewIdx < NumViews; ++ViewIdx)
	{
		FamilyRect.Union(static_cast<const FViewInfo*>(Family->Views[ViewIdx])->ViewRect);
	}
	return FamilyRect;
}

FIntRect FViewInfo::GetUnscaledFamilyViewRect() const
{
	FIntRect FamilyRect = {};
	for (uint64 ViewIdx = 0, NumViews = (uint64)Family->Views.Num(); ViewIdx < NumViews; ++ViewIdx)
	{
		FamilyRect.Union(static_cast<const FViewInfo*>(Family->Views[ViewIdx])->UnscaledViewRect);

		if (IsInstancedStereo())
		{
			for (const FSceneView* SecondaryView : GetSecondaryViews())
			{
				const FViewInfo& InstancedView = static_cast<const FViewInfo&>(*SecondaryView);
				FamilyRect.Union(InstancedView.UnscaledViewRect);
			}
		}
	}
	return FamilyRect;
}

void FViewInfo::BeginRenderView() const
{
	const bool bShouldWaitForPersistentViewUniformBufferExtensionsJobs = true;

	// Let the implementation of each extension decide whether it can cache the result for CachedView
	for (IPersistentViewUniformBufferExtension* Extension : PersistentViewUniformBufferExtensions)
	{
		Extension->BeginRenderView(this, bShouldWaitForPersistentViewUniformBufferExtensionsJobs);
	}
}

FViewShaderParameters FViewInfo::GetShaderParameters() const
{
	FViewShaderParameters Parameters;
	Parameters.View = ViewUniformBuffer;
	Parameters.InstancedView = InstancedViewUniformBuffer;
	// if we're a part of the stereo pair, make sure that the pointer isn't bogus
	checkf(InstancedViewUniformBuffer.IsValid() || !bShouldBindInstancedViewUB, TEXT("A view that is a part of the stereo pair has bogus state for InstancedView."));
	return Parameters;
}

const FViewInfo* FViewInfo::GetPrimaryView() const
{
	// It is valid for this function to return itself if it's already the primary view.
	if (Family && Family->Views.IsValidIndex(PrimaryViewIndex))
	{
		const FSceneView* PrimaryView = Family->Views[PrimaryViewIndex];
		check(PrimaryView->bIsViewInfo);
		return static_cast<const FViewInfo*>(PrimaryView);
	}
	return this;
}

extern TUniformBufferRef<FMobileDirectionalLightShaderParameters>& GetNullMobileDirectionalLightShaderParameters();

FViewInfo* FViewInfo::CreateSnapshot(EViewSnapshotType Type) const
{
	return ViewSnapshotCache::Create(this, Type);
}

FInt32Range FViewInfo::GetDynamicMeshElementRange(uint32 PrimitiveIndex) const
{
	// DynamicMeshEndIndices contains valid values only for visible primitives with bDynamicRelevance.
	if (PrimitiveVisibilityMap[PrimitiveIndex])
	{
		const FPrimitiveViewRelevance& ViewRelevance = PrimitiveViewRelevanceMap[PrimitiveIndex];
		if (ViewRelevance.bDynamicRelevance)
		{

			return FInt32Range(DynamicMeshElementRanges[PrimitiveIndex].X, DynamicMeshElementRanges[PrimitiveIndex].Y);
		}
	}

	return FInt32Range::Empty();
}

FRDGTextureRef FViewInfo::GetVolumetricCloudTexture(FRDGBuilder& GraphBuilder) const
{
	if (State)
	{
		return State->GetVolumetricCloudTexture(GraphBuilder);
	}
	return nullptr;
}

FSceneViewState* FViewInfo::GetEyeAdaptationViewState() const
{
	return static_cast<FSceneViewState*>(EyeAdaptationViewState);
}

FRDGPooledBuffer* FViewInfo::GetEyeAdaptationBuffer(FRDGBuilder& GraphBuilder) const
{
	if (FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState())
	{
		return EffectiveViewState->GetCurrentEyeAdaptationBuffer(GraphBuilder);
	}
	return nullptr;
}

void FViewInfo::SwapEyeAdaptationBuffers() const
{
	if (FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState())
	{
		EffectiveViewState->SwapEyeAdaptationBuffers();
	}
}

void FViewInfo::UpdateEyeAdaptationLastExposureFromBuffer() const
{
	if (FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState())
	{
		EffectiveViewState->UpdateEyeAdaptationLastExposureFromBuffer();
	}
}

void FViewInfo::EnqueueEyeAdaptationExposureBufferReadback(FRDGBuilder& GraphBuilder) const
{
	if (FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState())
	{
		EffectiveViewState->EnqueueEyeAdaptationExposureBufferReadback(GraphBuilder);
	}
}

bool FViewInfo::ShouldUpdateEyeAdaptationBuffer() const
{
	// This code should only be reached if eye adaptation is enabled (calling code should check HasEyeAdaptationViewState())
	check(EyeAdaptationViewState);

	// If this view owns its eye adaptation view state (equal to main view state), it should update
	if (EyeAdaptationViewState == reinterpret_cast<const FSceneViewStateInterface*>(ViewState))
	{
		return true;
	}

	// Otherwise, update the eye adaptation view state if none is available whatsoever
	return EyeAdaptationViewState->HasValidEyeAdaptationBuffer() == false;
}

float FViewInfo::GetLastEyeAdaptationExposure() const
{
	if (const FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState())
	{
		return EffectiveViewState->GetLastEyeAdaptationExposure();
	}
	return 0.0f; // Invalid exposure
}

float FViewInfo::GetLastAverageLocalExposure() const
{
	if (const FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState())
	{
		return EffectiveViewState->GetLastAverageLocalExposure();
	}
	return 1.0f; // Default to "local exposure disabled"
}

float FViewInfo::GetLastAverageSceneLuminance() const
{
	if (const FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState())
	{
		return EffectiveViewState->GetLastAverageSceneLuminance();
	}
	return 0.0f; // Invalid scene luminance
}

void FViewInfo::SetValidTonemappingLUT() const
{
	if (FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState())
	{
		EffectiveViewState->SetValidTonemappingLUT();
	}
}

IPooledRenderTarget* FViewInfo::GetTonemappingLUT() const
{
	FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState();
	if (EffectiveViewState && EffectiveViewState->HasValidTonemappingLUT())
	{
		return EffectiveViewState->GetTonemappingLUT();
	}
	return nullptr;
};

IPooledRenderTarget* FViewInfo::GetTonemappingLUT(FRHICommandList& RHICmdList, const int32 LUTSize, const bool bNeedFloatOutput) const 
{
	FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState();
	if (EffectiveViewState)
	{
		return EffectiveViewState->GetTonemappingLUT(RHICmdList, LUTSize, bNeedFloatOutput);
	}
	return nullptr;
}

IPooledRenderTarget* FViewInfo::GetInnerTonemappingLUT(FRHICommandList& RHICmdList, const int32 LUTSize, const bool bNeedFloatOutput) const
{
	FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState();
	if (EffectiveViewState)
	{
		return EffectiveViewState->GetInnerTonemappingLUT(RHICmdList, LUTSize, bNeedFloatOutput);
	}
	return nullptr;
}

void FViewInfo::ClearInnerTonemappingLUT() const
{
	FSceneViewState* EffectiveViewState = GetEyeAdaptationViewState();
	if (EffectiveViewState)
	{
		EffectiveViewState->ClearInnerTonemappingLUT();
	}
}

bool FViewInfo::RequiresDebugMaterials() const
{
	// We can only use debug materials in ODSC environments.
	static const bool bCanUseDebugMaterials = ShouldCompileODSCOnlyShaders();
	// Add other debug modes here as required.
	return bCanUseDebugMaterials && 
		(Family != nullptr && Family->EngineShowFlags.VisualizeVirtualTexture);
}

void FDisplayInternalsData::Setup(UWorld *World)
{
	DisplayInternalsCVarValue = 0;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	DisplayInternalsCVarValue = CVarDisplayInternals.GetValueOnGameThread();

	if(IsValid())
	{
#if WITH_AUTOMATION_TESTS
		// this variable is defined inside WITH_AUTOMATION_TESTS, 
		extern ENGINE_API uint32 GStreamAllResourcesStillInFlight;
		NumPendingStreamingRequests = GStreamAllResourcesStillInFlight;
#endif
	}
#endif
}

void FSortedShadowMaps::Release()
{
	for (int32 AtlasIndex = 0; AtlasIndex < ShadowMapAtlases.Num(); AtlasIndex++)
	{
		ShadowMapAtlases[AtlasIndex].RenderTargets.Release();
	}

	for (int32 AtlasIndex = 0; AtlasIndex < ShadowMapCubemaps.Num(); AtlasIndex++)
	{
		ShadowMapCubemaps[AtlasIndex].RenderTargets.Release();
	}

	PreshadowCache.RenderTargets.Release();
}

static bool PreparePostProcessSettingTextureForRenderer(const FViewInfo& View, UTexture2D* Texture2D, const TCHAR* TextureUsageName)
{
	check(IsInGameThread());

	bool bIsValid = Texture2D != nullptr;

	if (bIsValid)
	{
		const int32 CinematicTextureGroups = 0;
		const float Seconds = 5.0f;
		Texture2D->SetForceMipLevelsToBeResident(Seconds, CinematicTextureGroups);
	}

	const uint32 FramesPerWarning = 15;

	if (bIsValid && (Texture2D->IsFullyStreamedIn() == false || Texture2D->HasPendingInitOrStreaming()))
	{
		if ((View.Family->FrameNumber % FramesPerWarning) == 0)
		{
			UE_LOGF(LogRenderer, Warning, "The %ls texture is not streamed in.", TextureUsageName);
		}

		bIsValid = false;
	}

	if (bIsValid && Texture2D->bHasStreamingUpdatePending == true)
	{
		if ((View.Family->FrameNumber % FramesPerWarning) == 0)
		{
			UE_LOGF(LogRenderer, Warning, "The %ls texture has pending update.", TextureUsageName);
		}

		bIsValid = false;
	}

#if WITH_EDITOR
	if (bIsValid && Texture2D->IsDefaultTexture())
#else
	if (bIsValid && (!Texture2D->GetResource() || Texture2D->GetResource()->IsProxy()))
#endif
	{
		if ((View.Family->FrameNumber % FramesPerWarning) == 0)
		{
			UE_LOGF(LogRenderer, Warning, "The %ls texture is still using the default texture proxy.", TextureUsageName);
		}

		bIsValid = false;
	}

	return bIsValid;
};

template <typename T>
inline T* CheckPointer(T* Ptr)
{
	check(Ptr != nullptr);
	return Ptr;
}

FViewFamilyInfo::FViewFamilyInfo(const FSceneViewFamily& InViewFamily)
:	FSceneViewFamily(InViewFamily)
{
	bIsViewFamilyInfo = true;

	SceneTextures = new FSceneTextures;
	SceneTextures->Owner = this;
}

// Constructor that shares scene textures with a MainViewFamily.  Used to create a separate FViewFamilyInfo for custom render passes, so
// they can have distinct EngineShowFlags from the view family they are rendering with.
FViewFamilyInfo::FViewFamilyInfo(const FSceneViewFamily::ConstructionValues& CVS, const FViewFamilyInfo& MainViewFamily)
:	FSceneViewFamily(CVS)
{
	bIsViewFamilyInfo = true;

	SceneTextures = MainViewFamily.SceneTextures;
}

FViewFamilyInfo::~FViewFamilyInfo()
{
	if (SceneTextures && SceneTextures->Owner == this)
	{
		delete SceneTextures;
	}
}

FSceneRenderer::FCustomRenderPassInfo::FCustomRenderPassInfo(const FSceneViewFamily::ConstructionValues& CVS)
	: ViewFamily(CVS)
{
}

FSceneRenderer::FCustomRenderPassInfo::FCustomRenderPassInfo(const FSceneViewFamily::ConstructionValues& CVS, const FViewFamilyInfo& MainViewFamily)
	: ViewFamily(CVS, MainViewFamily)
{
}

TGlobalResource<FGlobalDynamicReadBuffer> FSceneRenderer::DynamicReadBufferForInitViews;
TGlobalResource<FGlobalDynamicReadBuffer> FSceneRenderer::DynamicReadBufferForRayTracing;
TGlobalResource<FGlobalDynamicReadBuffer> FSceneRenderer::DynamicReadBufferForShadows;

/*-----------------------------------------------------------------------------
	FSceneRenderer
-----------------------------------------------------------------------------*/

struct FSceneUniformBufferBlackboardStruct
{
	FSceneRendererBase* SceneRenderer = nullptr;
};

RDG_REGISTER_BLACKBOARD_STRUCT(FSceneUniformBufferBlackboardStruct);

void FSceneRendererBase::SetActiveInstance(FRDGBuilder& GraphBuilder, FSceneRendererBase* SceneRenderer)
{
	GraphBuilder.Blackboard.GetOrCreate<FSceneUniformBufferBlackboardStruct>().SceneRenderer = SceneRenderer;
}

FSceneRendererBase* FSceneRendererBase::GetActiveInstance(FRDGBuilder& GraphBuilder)
{
	if (const FSceneUniformBufferBlackboardStruct* Struct = GraphBuilder.Blackboard.Get<FSceneUniformBufferBlackboardStruct>())
	{
		return Struct->SceneRenderer;
	}
	return nullptr;
}

FIntRect FSceneRendererBase::GetHZBViewRect(const FViewInfo& View)
{
	return CVarHZBUseUpperBoundViewRect.GetValueOnRenderThread() != 0 ? View.UpperBoundViewRect : View.ViewRect;
}

FSceneRenderer::FSceneRenderer(const FSceneViewFamily* InViewFamily, FHitProxyConsumer* HitProxyConsumer)
:	FSceneRendererBase(*CheckPointer(CheckPointer(CheckPointer(InViewFamily)->Scene)->GetRenderScene()))
,	DepthPass(GetDepthPassInfo(Scene))
,	ViewFamily(*InViewFamily)
,	VirtualShadowMapArray(*Scene)
,	bHasRequestedToggleFreeze(false)
,	bUsedPrecomputedVisibility(false)
,	bGPUMasksComputed(false)
,	FamilySize(0, 0)
,	GPUSceneDynamicContext(CheckPointer(Scene)->GPUScene)
,	bShadowDepthRenderCompleted(false)
{
	check(Scene != nullptr);

	check(IsInGameThread());

	ViewFamily.SetSceneRenderer(this);

	// Copy the individual views.
	bool bAnyViewIsLocked = false;
	Views.Empty(InViewFamily->Views.Num());
	for (int32 ViewIndex = 0;ViewIndex < InViewFamily->Views.Num();ViewIndex++)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		for (int32 ViewIndex2 = 0;ViewIndex2 < InViewFamily->Views.Num();ViewIndex2++)
		{
			if (ViewIndex != ViewIndex2 && InViewFamily->Views[ViewIndex]->State != nullptr)
			{
				// Verify that each view has a unique view state, as the occlusion query mechanism depends on it.
				check(InViewFamily->Views[ViewIndex]->State != InViewFamily->Views[ViewIndex2]->State);
			}
		}
#endif

		// Construct a FViewInfo with the FSceneView properties.
		FViewInfo* ViewInfo = &Views.Emplace_GetRef(InViewFamily->Views[ViewIndex]);
		ViewFamily.Views[ViewIndex] = ViewInfo;
		ViewInfo->Family = &ViewFamily;
		bAnyViewIsLocked |= ViewInfo->bIsLocked;

		// Must initialize to have a GPUScene connected to be able to collect dynamic primitives.
		ViewInfo->DynamicPrimitiveCollector = FGPUScenePrimitiveCollector(&GPUSceneDynamicContext);
		ViewInfo->RayTracingDynamicPrimitiveCollector = FGPUScenePrimitiveCollector(&GPUSceneDynamicContext);

#if !UE_BUILD_SHIPPING
		if (float ViewRollAngle = CVarTestViewRollAngle.GetValueOnGameThread())
		{
			FViewMatrices& CurrentMatrices = ViewInfo->ViewMatrices;

			FRotator Rotate(/* InPitch = */ 0.0, /* InYaw = */ ViewRollAngle, /* Roll = */ 0.0);
			FMatrix Rotation = FRotationMatrix::Make(Rotate);

			FViewMatrices::FMinimalInitializer NewMatrices;
			NewMatrices.ViewRotationMatrix  = CurrentMatrices.GetWorldToView().RemoveTranslation() * Rotation;
			NewMatrices.ProjectionMatrix    = CurrentMatrices.GetViewToClip();
			NewMatrices.ViewOrigin          = CurrentMatrices.GetViewOrigin();
			NewMatrices.ConstrainedViewRect = ViewInfo->CameraConstrainedViewRect;
			NewMatrices.CameraToViewTarget  = CurrentMatrices.GetCameraToViewTarget();

			CurrentMatrices = FViewMatrices(NewMatrices);
		}
#endif

		check(ViewInfo->ViewRect.Area() == 0);

#if WITH_EDITOR
		// Should we allow the user to select translucent primitives?
		ViewInfo->bAllowTranslucentPrimitivesInHitProxy =
			GEngine->AllowSelectTranslucent() ||		// User preference enabled?
			!ViewInfo->IsPerspectiveProjection();		// Is orthographic view?
#endif

		// Batch the view's elements for later rendering.
		if (ViewInfo->Drawer)
		{
			FViewElementPDI ViewElementPDI(ViewInfo, HitProxyConsumer, &ViewInfo->DynamicPrimitiveCollector);
			ViewInfo->Drawer->Draw(ViewInfo, &ViewElementPDI);
		}

#if !UE_BUILD_SHIPPING
		if (CVarTestCameraCut.GetValueOnGameThread())
		{
			ViewInfo->bCameraCut = true;
		}
#endif

#if WITH_DUMPGPU
		if (UE::RenderCore::DumpGPU::ShouldCameraCut())
		{
			ViewInfo->bCameraCut = true;
		}
#endif

		const bool LoadVector2BlueNoiseTexture =
			ShouldRenderLumenDiffuseGI(Scene, *ViewInfo) ||
			ShouldRenderLumenReflections(*ViewInfo) ||
			ShouldRenderVolumetricCloudWithBlueNoise_GameThread(Scene, *ViewInfo) ||
			UseVirtualShadowMaps(Scene->GetShaderPlatform(), Scene->GetFeatureLevel()) ||
			Substrate::IsGlintEnabled(ViewInfo->GetShaderPlatform()) ||
			IsHairStrandsSupported(EHairStrandsShaderType::Strands, ViewInfo->GetShaderPlatform()) ||
			IsTranslucencyLightingVolumeUsingBlueNoise() ||
			GetSubstrateEnabledUseRoughRefraction();
		GEngine->LoadBlueNoiseTexture(LoadVector2BlueNoiseTexture);
		GEngine->LoadHairLUTTextures();

		if (Substrate::IsGlintEnabled(ViewInfo->GetShaderPlatform()))
		{
			GEngine->LoadGlintTextures();
		}

		if (Substrate::IsSubstrateEnabled())
		{
			GEngine->LoadSimpleVolumeTextures();
		}

		if (InViewFamily->Views[ViewIndex]->AntiAliasingMethod == AAM_SMAA)
		{
			GEngine->LoadSMAATextures();
		}

		// Handle the FFT bloom kernel texture
		if (ViewInfo->FinalPostProcessSettings.BloomMethod == EBloomMethod::BM_FFT && ViewInfo->ViewState != nullptr)
		{
			UTexture2D* BloomConvolutionTexture = ViewInfo->FinalPostProcessSettings.BloomConvolutionTexture;
			if (BloomConvolutionTexture == nullptr)
			{
				GEngine->LoadDefaultBloomTexture();

				BloomConvolutionTexture = GEngine->DefaultBloomKernelTexture;
			}

			bool bIsValid = PreparePostProcessSettingTextureForRenderer(*ViewInfo, BloomConvolutionTexture, TEXT("convolution bloom"));

			if (bIsValid)
			{
				const FTextureResource* TextureResource = BloomConvolutionTexture->GetResource();
				if (TextureResource)
				{
					ViewInfo->FFTBloomKernelTexture = TextureResource->GetTexture2DResource();
					ViewInfo->FinalPostProcessSettings.BloomConvolutionTexture = BloomConvolutionTexture;
				}
				else
				{
					ViewInfo->FinalPostProcessSettings.BloomConvolutionTexture = nullptr;
				}
			}
		}

		// Handle the film grain texture
		if (ViewInfo->FinalPostProcessSettings.FilmGrainIntensity > 0.0f &&
			ViewFamily.EngineShowFlags.Grain &&
			CVarFilmGrain.GetValueOnGameThread() != 0 &&
			SupportsFilmGrain(ViewFamily.GetShaderPlatform()))
		{
			UTexture2D* FilmGrainTexture = ViewInfo->FinalPostProcessSettings.FilmGrainTexture;
			if (FilmGrainTexture == nullptr)
			{
				GEngine->LoadDefaultFilmGrainTexture();
				FilmGrainTexture = GEngine->DefaultFilmGrainTexture;
			}

			bool bIsValid = PreparePostProcessSettingTextureForRenderer(*ViewInfo, FilmGrainTexture, TEXT("film grain"));

			if (bIsValid)
			{
				const FTextureResource* TextureResource = FilmGrainTexture->GetResource();
				if (TextureResource)
				{
					ViewInfo->FilmGrainTexture = TextureResource->GetTexture2DResource();
				}
			}
		}

		if (CVarTranslucencyAutoBeforeDOF.GetValueOnGameThread() >= 0.0f && DiaphragmDOF::IsEnabled(*ViewInfo))
		{
			ViewInfo->AutoBeforeDOFTranslucencyBoundary = ViewInfo->FinalPostProcessSettings.DepthOfFieldFocalDistance / FMath::Clamp(1.0f - CVarTranslucencyAutoBeforeDOF.GetValueOnGameThread(), 0.01f, 1.0f);
		}
	}

	// Catches inconsistency one engine show flags for screen percentage and whether it is supported or not.
	checkf(!(ViewFamily.EngineShowFlags.ScreenPercentage && !ViewFamily.SupportsScreenPercentage()), TEXT("Screen percentage is not supported, but show flag was incorectly set to true."));

	// Disable occlusion queries for scene capture depth optimization mode
	if (GetRendererOutput() == ERendererOutput::DepthPrepassOnly)
	{
		ViewFamily.EngineShowFlags.SetDisableOcclusionQueries(true);
	}

	// Fork the plugin interfaces of the view family.
	{
		{
			check(InViewFamily->ScreenPercentageInterface);
			ViewFamily.ScreenPercentageInterface = nullptr;
			ViewFamily.SetScreenPercentageInterface(InViewFamily->ScreenPercentageInterface->Fork_GameThread(ViewFamily));
		}

		if (ViewFamily.TemporalUpscalerInterface)
		{
			ViewFamily.TemporalUpscalerInterface = nullptr;
			ViewFamily.SetTemporalUpscalerInterface(InViewFamily->TemporalUpscalerInterface->Fork_GameThread(ViewFamily));

			if (ViewFamily.EngineShowFlags.TemporalAA)
			{
				for (FViewInfo& View : Views)
				{
					View.AntiAliasingMethod = AAM_TemporalAA;
					View.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::TemporalUpscale;
#if DO_CHECK || USING_CODE_ANALYSIS
					check(View.VerifyMembersChecks());
#endif
				}
			}
		}

		if (ViewFamily.MegaFiltersInterface)
		{
			ViewFamily.MegaFiltersInterface = nullptr;
			ViewFamily.SetMegaFiltersInterface(InViewFamily->MegaFiltersInterface->Fork_GameThread(ViewFamily));
		}

		if (ViewFamily.PrimarySpatialUpscalerInterface)
		{
			ViewFamily.PrimarySpatialUpscalerInterface = nullptr;
			ViewFamily.SetPrimarySpatialUpscalerInterface(InViewFamily->PrimarySpatialUpscalerInterface->Fork_GameThread(ViewFamily));
		}

		if (ViewFamily.SecondarySpatialUpscalerInterface)
		{
			ViewFamily.SecondarySpatialUpscalerInterface = nullptr;
			ViewFamily.SetSecondarySpatialUpscalerInterface(InViewFamily->SecondarySpatialUpscalerInterface->Fork_GameThread(ViewFamily));
		}
	}

#if !UE_BUILD_SHIPPING
	// Override screen percentage interface.
	if (int32 OverrideId = CVarTestScreenPercentageInterface.GetValueOnGameThread())
	{
		check(ViewFamily.ScreenPercentageInterface);

		// Replaces screen percentage interface with dynamic resolution hell's driver.
		if (OverrideId == 1 && ViewFamily.Views[0]->State)
		{
			delete ViewFamily.ScreenPercentageInterface;
			ViewFamily.ScreenPercentageInterface = nullptr;
			ViewFamily.EngineShowFlags.ScreenPercentage = true;
			ViewFamily.SetScreenPercentageInterface(new FScreenPercentageHellDriver(ViewFamily));
		}
	}

	// Override secondary screen percentage for testing purpose.
	if (CVarTestSecondaryUpscaleOverride.GetValueOnGameThread() > 0 && !ViewFamily.Views[0]->bIsReflectionCapture)
	{
		ViewFamily.SecondaryViewFraction = 1.0 / float(CVarTestSecondaryUpscaleOverride.GetValueOnGameThread());
		ViewFamily.SecondaryScreenPercentageMethod = ESecondaryScreenPercentageMethod::NearestSpatialUpscale;
	}
#endif

	// If any viewpoint has been locked, set time to zero to avoid time-based
	// rendering differences in materials.
	if (bAnyViewIsLocked)
	{
		ViewFamily.Time = FGameTime::CreateDilated(0.0, ViewFamily.Time.GetDeltaRealTimeSeconds(), 0.0, ViewFamily.Time.GetDeltaWorldTimeSeconds());
	}

	// copy off the requests
	if (ensure(InViewFamily->RenderTarget))
	{
		// (I apologize for the const_cast, but didn't seem worth refactoring just for the freezerendering command)
		if (const_cast<FRenderTarget*>(InViewFamily->RenderTarget)->HasToggleFreezeCommand())
		{
			bHasRequestedToggleFreeze = true;
		}
	}

	// launch custom visibility queries for views
	if (GCustomCullingImpl)
	{
		for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
		{
			FViewInfo& ViewInfo = Views[ViewIndex];
			ViewInfo.CustomVisibilityQuery = GCustomCullingImpl->CreateQuery(ViewInfo);
		}
	}

	// Prepare custom render passes and their views:
	CustomRenderPassInfos.Empty(Scene->CustomRenderPassRendererInputs.Num());

	// If multiple CRPs are in use with MSAA active, we share the non-MSAA version of scene textures across them.
	FCustomRenderPassInfo* CustomRenderPassInfoNoMSAA = nullptr;

	int32 NumAdditionalViews = 0;
	int32 IncrementIfNotRemoved = 1;
	for (int32 i = 0; i < Scene->CustomRenderPassRendererInputs.Num(); i+=IncrementIfNotRemoved)
	{
		const FScene::FCustomRenderPassRendererInput& PassInput = Scene->CustomRenderPassRendererInputs[i];
		FCustomRenderPassBase* CustomRenderPass = PassInput.CustomRenderPass;
		check(CustomRenderPass);

		const FSceneCaptureCustomRenderPassUserData& SceneCaptureUserData = FSceneCaptureCustomRenderPassUserData::Get(CustomRenderPass);

		if (SceneCaptureUserData.bMainViewFamily && !ViewFamily.bIsMainViewFamily)
		{
			// If the custom render pass is flagged as rendering with the main view family, and this isn't the main view family, skip it.
			IncrementIfNotRemoved = 1;
			continue;
		}
		else
		{
			IncrementIfNotRemoved = 0;
		}

		// We construct from scratch, rather than copying, as we don't want to copy interfaces attached to the view family
		// (ScreenPercentageInterface, TemporalUpscalerInterface, etc), which can assert or double free if copied.  Those aren't
		// relevant for custom render passes anyway.
		FSceneViewFamily::ConstructionValues FamilyCVS(ViewFamily.RenderTarget, Scene, PassInput.bUseMainViewFamilyShowFlags ? ViewFamily.EngineShowFlags : PassInput.EngineShowFlags);

		if (PassInput.bUseMainViewFamilyShowFlags)
		{
			// PassInput.EngineShowFlags will already have had this called at construction, but show flags copied from the ViewFamily will not
			FamilyCVS.EngineShowFlags.DisableFeaturesForUnlit();
		}

		// Conditionally enable translucency.  Custom render passes have their own translucency flag, with the assumption that by default
		// they shouldn't have translucency.  Also, depending on the output of the CRP, the translucent pass may be writing to the scene
		// color where it isn't used, and translucency should be disabled as an unnecessary perf cost.
		FamilyCVS.EngineShowFlags.SetTranslucency(CustomRenderPass->IsTranslucentIncluded());

		// The GetDefaultMSAACount call below must match the logic that initializes NumSamples in FSceneTexturesConfig::Init.  Downstream
		// code uses NumSamples to detect MSAA, after the config has been initialized.
		FCustomRenderPassInfo* CustomRenderPassInfo;
		if (SceneCaptureUserData.bExcludeFromSceneTextureExtents)
		{
			// If excluded from scene texture extents, generate its own scene textures struct, by not passing a source view family.
			CustomRenderPassInfo = &CustomRenderPassInfos.Emplace_GetRef(FamilyCVS);
		}
		else if (GetDefaultMSAACount(Scene->GetFeatureLevel(), GDynamicRHI->RHIGetPlatformTextureMaxSampleCount()) > 1)
		{
			// If main pass is MSAA, create a view family with its own FSceneTextures, by calling the constructor that doesn't accept another view family.
			// MSAA isn't supported for custom render passes, so we'll initialize this one without MSAA (see FSceneRenderer::OnRenderBegin, where we set
			// NumSamples to 1 for Custom Render Pass scene texture configs).  First custom render pass gets its own, subsequent custom render passes share
			// with the first custom render pass.
			if (!CustomRenderPassInfoNoMSAA)
			{
				CustomRenderPassInfo = &CustomRenderPassInfos.Emplace_GetRef(FamilyCVS);
				CustomRenderPassInfoNoMSAA = CustomRenderPassInfo;
			}
			else
			{
				CustomRenderPassInfo = &CustomRenderPassInfos.Emplace_GetRef(FamilyCVS, CustomRenderPassInfoNoMSAA->ViewFamily);
			}
		}
		else
		{
			// Passing a View Family to the constructor shares its FSceneTextures struct.
			CustomRenderPassInfo = &CustomRenderPassInfos.Emplace_GetRef(FamilyCVS, ViewFamily);
		}

		CustomRenderPassInfo->CustomRenderPass = CustomRenderPass;
		CustomRenderPassInfo->ViewFamily.Time = ViewFamily.Time;
		CustomRenderPassInfo->ViewFamily.FrameNumber = ViewFamily.FrameNumber;
		CustomRenderPassInfo->ViewFamily.FrameCounter = ViewFamily.FrameCounter;
		CustomRenderPassInfo->ViewFamily.SetSceneRenderer(this);
		CustomRenderPassInfo->ViewFamily.bIsSceneTextureSizedCapture = SceneCaptureUserData.bMainViewResolution;

		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.SceneViewStateInterface = PassInput.ViewStateInterface;
		ViewInitOptions.SetViewRectangle(FIntRect(0, 0, CustomRenderPass->GetRenderTargetSize().X, CustomRenderPass->GetRenderTargetSize().Y));
		ViewInitOptions.ViewOrigin = PassInput.ViewLocation;
		ViewInitOptions.ViewRotationMatrix = PassInput.ViewRotationMatrix;
		ViewInitOptions.ProjectionMatrix = PassInput.ProjectionMatrix;
		ViewInitOptions.bIsSceneCapture = PassInput.bIsSceneCapture;
		ViewInitOptions.ViewFamily = &CustomRenderPassInfo->ViewFamily;
		ViewInitOptions.ViewActor = PassInput.ViewActor;
		ViewInitOptions.ShowOnlyPrimitives = PassInput.ShowOnlyPrimitives;
		ViewInitOptions.HiddenPrimitives = PassInput.HiddenPrimitives;
		ViewInitOptions.bExcludeFromSceneTextureExtents = SceneCaptureUserData.bExcludeFromSceneTextureExtents;

		FSceneView NewView(ViewInitOptions);
		FViewInfo* ViewInfo = &CustomRenderPassInfo->Views.Emplace_GetRef(&NewView);
		CustomRenderPassInfo->ViewFamily.Views.Add(ViewInfo);

		if (PassInput.bOverridesPostVolumeUserFlags)
		{
			ViewInfo->FinalPostProcessSettings.UserFlags = PassInput.PostVolumeUserFlags;
		}
		else
		{
			// Arbitrarily use the post process UserFlags from the first view.
			ViewInfo->FinalPostProcessSettings.UserFlags = Views[0].FinalPostProcessSettings.UserFlags;
		}

		// Must initialize to have a GPUScene connected to be able to collect dynamic primitives.
		ViewInfo->DynamicPrimitiveCollector = FGPUScenePrimitiveCollector(&GPUSceneDynamicContext);
		ViewInfo->bDisableQuerySubmissions = true;
		ViewInfo->bIgnoreExistingQueries = true;
		ViewInfo->CustomRenderPass = CustomRenderPass;
		ViewInfo->ViewRect = ViewInfo->UnscaledViewRect;
		ViewInfo->ViewRectWithSecondaryViews = ViewInfo->UnscaledViewRect;
		CustomRenderPass->Views.Add(ViewInfo);

		NumAdditionalViews++;

		Scene->CustomRenderPassRendererInputs.RemoveAt(i, EAllowShrinking::No);
	}

	AllViews.Empty(Views.Num() + NumAdditionalViews);
	for (int32 i = 0; i < Views.Num(); ++i)
	{
		AllViews.Add(&Views[i]);
	}
	for (FCustomRenderPassInfo& PassInfo : CustomRenderPassInfos)
	{
		for (FViewInfo& View : PassInfo.Views)
		{
			AllViews.Add(&View);
		}
	}

	// Set a unique id on each view in this scene renderer
	for (int32 i = 0; i < AllViews.Num(); ++i)
	{
		AllViews[i]->SceneRendererPrimaryViewId = i;
	}

#if !UE_BUILD_SHIPPING
	// Validate the views
	TSet<FSceneViewStateInterface*> UniqueViewStates;
	for (FViewInfo* View : AllViews)
	{
		if (View->State != nullptr)
		{
			checkf(!UniqueViewStates.Contains(View->State), TEXT("2 views sharing a view state is currently forbidden, please make sure each FViewInfo is using a separate FSceneViewStateInterface or none at all"));
			UniqueViewStates.Add(View->State);
		}
	}
#endif // !UE_BUILD_SHIPPING


	// Check if the translucency are allowed to be rendered after DOF, if not, translucency after DOF will be rendered in standard translucency.
	{
		bool SeparateTranslucencyEnabled = ViewFamily.EngineShowFlags.PostProcessing // Used for reflection captures.
			&& !ViewFamily.UseDebugViewPS()
			&& ViewFamily.EngineShowFlags.SeparateTranslucency;

		const bool bIsMobile = ViewFamily.GetFeatureLevel() == ERHIFeatureLevel::ES3_1;
		if (bIsMobile)
		{
			const bool bMobileMSAA = GetDefaultMSAACount(ERHIFeatureLevel::ES3_1) > 1;
			SeparateTranslucencyEnabled &= (IsMobileHDR() && !bMobileMSAA); // on <= ES3_1 separate translucency requires HDR on and MSAA off
		}

		ViewFamily.bAllowTranslucencyAfterDOF = SeparateTranslucencyEnabled && CVarAllowTranslucencyAfterDOF.GetValueOnAnyThread() != 0;

		if (!ViewFamily.bAllowTranslucencyAfterDOF && !bIsMobile && CVarTSRForceSeparateTranslucency.GetValueOnAnyThread() != 0)
		{
			for (FViewInfo* View : AllViews)
			{
				// Need to also check PostProcessing flag, as scene captures may run with temporal AA jitter matching the main view, but post processing disabled.
				// Without this, translucency doesn't show up, because the renderer assumes post processing will composite in the translucency.
				if (View->AntiAliasingMethod == AAM_TSR && View->Family->EngineShowFlags.PostProcessing)
				{
					ViewFamily.bAllowTranslucencyAfterDOF = true;
					break;
				}
			}
		}

		// We do not allow separated translucency on mobile
		// When MSAA sample count is >1 it works, but hair has not been properly tested so far due to other issues, so MSAA cannot use separted standard translucent for now.
		uint32 MSAASampleCount = GetDefaultMSAACount(ViewFamily.GetFeatureLevel());
		ViewFamily.bAllowStandardTranslucencySeparated = SeparateTranslucencyEnabled && MSAASampleCount == 1 && !bIsMobile && CVarTranslucencyStandardSeparated.GetValueOnAnyThread() != 0;
	}

	check(!ViewFamily.AllViews.Num());
	ViewFamily.AllViews.Append(AllViews);

	// Mirror AllViews across CustomRenderPass view families
	for (FCustomRenderPassInfo& PassInfo : CustomRenderPassInfos)
	{
		PassInfo.ViewFamily.AllViews = ViewFamily.AllViews;
	}

	FeatureLevel = Scene->GetFeatureLevel();
	ShaderPlatform = Scene->GetShaderPlatform();

	bDumpMeshDrawCommandInstancingStats = !!GDumpInstancingStats;
	GDumpInstancingStats = 0;
}

// static
FIntPoint FSceneRenderer::ApplyResolutionFraction(const FIntPoint& UnscaledViewSize, float ResolutionFraction)
{
	FIntPoint ViewSize;

	// CeilToInt so tha view size is at least 1x1 if ResolutionFraction == ISceneViewFamilyScreenPercentage::kMinResolutionFraction.
	ViewSize.X = FMath::CeilToInt(UnscaledViewSize.X * ResolutionFraction);
	ViewSize.Y = FMath::CeilToInt(UnscaledViewSize.Y * ResolutionFraction);

	check(ViewSize.GetMin() > 0);

	return ViewSize;
}

// static
FIntPoint FSceneRenderer::QuantizeViewRectMin(const FIntPoint& ViewRectMin)
{
	FIntPoint Out;
	// some features (e.g. Nanite, Substrate, Lumen, MegaLights) require that view rect is aligned on 8x8 boundary.
	QuantizeSceneBufferSize(ViewRectMin, Out, 8);
	return Out;
}

// static
FIntPoint FSceneRenderer::GetDesiredInternalBufferSize(const FSceneViewFamily& ViewFamily)
{
	// If this function is called for a CustomRenderPass view family, it means it is getting its own scene textures,
	// separate from the main renderer, and only its own views should be counted, not AllViews.
	if (ViewFamily.Views[0]->CustomRenderPass)
	{
		FIntPoint FamilySizeUpperBound(0, 0);
		for (const FSceneView* View : ViewFamily.Views)
		{
			FamilySizeUpperBound.X = FMath::Max(FamilySizeUpperBound.X, View->UnscaledViewRect.Max.X);
			FamilySizeUpperBound.Y = FMath::Max(FamilySizeUpperBound.Y, View->UnscaledViewRect.Max.Y);
		}

		FIntPoint DesiredBufferSize;
		QuantizeSceneBufferSize(FamilySizeUpperBound, DesiredBufferSize);
		return DesiredBufferSize;
	}

	// If not supporting screen percentage, bypass all computation.
	// This bypasses not only resolution scaling, but also quantizing the buffer size.
	// The only configurations that do not support screen percentage are Forward Mobile non-HDR and Forward Mobile w/ Tonemap Subpass.
	// These configurations do not support any features that require buffer quantization, so it's safe to skip.
	if (!ViewFamily.SupportsScreenPercentage())
	{
		FIntPoint FamilySizeUpperBound(0, 0);

		for (const FSceneView* View : ViewFamily.AllViews)
		{
			// If a custom render pass has bExcludeFromSceneTextureExtents set, it will get its own scene textures, and shouldn't be counted in the main view family.
			if (View->CustomRenderPass && View->bExcludeFromSceneTextureExtents)
			{
				continue;
			}

			FamilySizeUpperBound.X = FMath::Max(FamilySizeUpperBound.X, View->UnscaledViewRect.Max.X);
			FamilySizeUpperBound.Y = FMath::Max(FamilySizeUpperBound.Y, View->UnscaledViewRect.Max.Y);
		}

		FIntPoint DesiredBufferSize;
		QuantizeSceneBufferSize(FamilySizeUpperBound, DesiredBufferSize);
		return DesiredBufferSize;
	}

	// Compute final resolution fraction.
	float ResolutionFractionUpperBound = 1.f;
	if (ISceneViewFamilyScreenPercentage const* ScreenPercentageInterface = ViewFamily.GetScreenPercentageInterface())
	{
		DynamicRenderScaling::TMap<float> DynamicResolutionUpperBounds = ScreenPercentageInterface->GetResolutionFractionsUpperBound();
		const float PrimaryResolutionFractionUpperBound = DynamicResolutionUpperBounds[GDynamicPrimaryResolutionFraction];
		ResolutionFractionUpperBound = PrimaryResolutionFractionUpperBound * ViewFamily.SecondaryViewFraction;
	}

	if (ViewFamily.Views[0]->bIsViewInfo)
	{
		const FViewInfo& View = static_cast<const FViewInfo&>(*ViewFamily.Views[0]);
		if (View.LensDistortionLUT.IsEnabled())
		{
			float AffectScreenPercentage = CVarLensDistortionAffectScreenPercentage.GetValueOnRenderThread();
			ResolutionFractionUpperBound *= FMath::Lerp(1.0, View.LensDistortionLUT.ResolutionFraction, AffectScreenPercentage);
		}
	}

	FIntPoint FamilySizeUpperBound(0, 0);

	// For multiple views, use the maximum overscan fraction to ensure that enough space is allocated so that any overscanned views
	// do not encroach into the space of other views
	float MaxOverscanResolutionFraction = 1.0f;
	for (const FSceneView* View : ViewFamily.AllViews)
	{
		// If a custom render pass has bExcludeFromSceneTextureExtents set, it will get its own scene textures, and shouldn't be counted in the main view family.
		if (View->CustomRenderPass && View->bExcludeFromSceneTextureExtents)
		{
			continue;
		}

		MaxOverscanResolutionFraction = FMath::Max(MaxOverscanResolutionFraction, View->SceneViewInitOptions.OverscanResolutionFraction);
	}

	ResolutionFractionUpperBound *= MaxOverscanResolutionFraction;
	
	for (const FSceneView* View : ViewFamily.AllViews)
	{
		// If a custom render pass has bExcludeFromSceneTextureExtents set, it will get its own scene textures, and shouldn't be counted in the main view family.
		if (View->CustomRenderPass && View->bExcludeFromSceneTextureExtents)
		{
			continue;
		}

		// Note: This ensures that custom passes (rendered with the main renderer) ignore screen percentage, like regular scene captures.
		const float AdjustedResolutionFractionUpperBounds = View->CustomRenderPass ? 1.0f : (View->SceneViewInitOptions.OverridePrimaryResolutionFraction > 0.0 ? (View->SceneViewInitOptions.OverridePrimaryResolutionFraction * ViewFamily.SecondaryViewFraction)  : ResolutionFractionUpperBound);
		
		FIntPoint ViewSize = ApplyResolutionFraction(View->UnconstrainedViewRect.Size(), AdjustedResolutionFractionUpperBounds);
		FIntPoint ViewRectMin = QuantizeViewRectMin(FIntPoint(
			FMath::CeilToInt(View->UnconstrainedViewRect.Min.X * AdjustedResolutionFractionUpperBounds),
			FMath::CeilToInt(View->UnconstrainedViewRect.Min.Y * AdjustedResolutionFractionUpperBounds)));

		FamilySizeUpperBound.X = FMath::Max(FamilySizeUpperBound.X, ViewRectMin.X + ViewSize.X);
		FamilySizeUpperBound.Y = FMath::Max(FamilySizeUpperBound.Y, ViewRectMin.Y + ViewSize.Y);
	}

	check(FamilySizeUpperBound.GetMin() > 0);

	FIntPoint DesiredBufferSize;
	QuantizeSceneBufferSize(FamilySizeUpperBound, DesiredBufferSize);

#if !UE_BUILD_SHIPPING
	{
		// Increase the size of desired buffer size by 2 when testing for view rectangle offset.
		static const auto CVar = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Test.ViewRectOffset"));
		if (CVar->GetValueOnAnyThread() > 0)
		{
			DesiredBufferSize *= 2;
		}
	}
#endif

	return DesiredBufferSize;
}

FSceneRenderer::ERendererOutput FSceneRenderer::GetRendererOutput() const
{
	if (!Views[0].bIsSceneCapture)
	{
		return ERendererOutput::FinalSceneColor;
	}
	if (ViewFamily.SceneCaptureSource == ESceneCaptureSource::SCS_SceneDepth || ViewFamily.SceneCaptureSource == ESceneCaptureSource::SCS_DeviceDepth)
	{
		if (GSceneCaptureDepthPrepassOptimization)
		{
			return ERendererOutput::DepthPrepassOnly;
		}
	}
	return ERendererOutput::FinalSceneColor;
}

void FSceneRenderer::PrepareViewRectsForRendering()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PrepareViewRectsForRendering);

	// If we support screen percentage, update the dynamic resolution state with our current temporal upscaler, which clamps the screen percentage to its supported range.
	if (ViewFamily.SupportsScreenPercentage())
	{
		IDynamicResolutionState* DynamicResolutionState = GEngine->GetDynamicResolutionState();
		if (DynamicResolutionState)
		{
			DynamicResolutionState->SetTemporalUpscaler(ViewFamily.GetTemporalUpscalerInterface());
		}
	}

	// Read the resolution data.
	{
		check(ViewFamily.ScreenPercentageInterface);
		DynamicResolutionUpperBounds = ViewFamily.ScreenPercentageInterface->GetResolutionFractionsUpperBound();
		DynamicResolutionFractions = ViewFamily.ScreenPercentageInterface->GetResolutionFractions_RenderThread();
	}

	// Checks that view rects were still not initialized.
	for (FViewInfo& View : Views)
	{
		// Make sure there was no attempt to configure ViewRect and screen percentage method before.
		check(View.ViewRect.Area() == 0);

		// Fallback to no anti aliasing.
		{
			const bool bWillApplyTemporalAA = (IsPostProcessingEnabled(View) || View.bIsPlanarReflection || View.bSceneCaptureMainViewJitter)
#if RHI_RAYTRACING
				// path tracer does its own anti-aliasing (unless it specifically requests it, such as for the debug mode)
				&& (!ViewFamily.EngineShowFlags.PathTracing || PathTracing::NeedsAntiAliasing(View))
#endif
			;

			if (!bWillApplyTemporalAA && !(View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::RawOutput))
			{
				// Disable anti-aliasing if we are not going to be able to apply final post process effects
				View.AntiAliasingMethod = AAM_None;
				View.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::SpatialUpscale;
			}
		}
	}

	// If not supporting screen percentage, bypass all computation.
	// This bypasses not only resolution scaling, but also quantizing the ViewRect and shifting letterboxed ViewRects to the top left for post-processing.
	// The only configurations that do not support screen percentage are Forward Mobile non-HDR and Forward Mobile w/ Tonemap Subpass.
	// These configurations do not support post-processing or any features that require ViewRect quantization (e.g. Nanite, Substrate, Lumen, MegaLights), so it's safe to skip.
	if (!ViewFamily.SupportsScreenPercentage())
	{
		DynamicResolutionFractions[GDynamicPrimaryResolutionFraction] = 1.0f;

		// The base pass have to respect FSceneView::UnscaledViewRect.
		for (FViewInfo& View : Views)
		{
			View.ViewRect = View.UpperBoundViewRect = View.UnscaledViewRect;
		}

		ComputeFamilySize();

		return;
	}

	const float PrimaryResolutionFraction = DynamicResolutionFractions[GDynamicPrimaryResolutionFraction];
	const float PrimaryMaxResolutionFraction = DynamicResolutionUpperBounds[GDynamicPrimaryResolutionFraction];
	{
		// Ensure screen percentage show flag is respected. Prefer to check() rather rendering at a differen screen percentage
		// to make sure the renderer does not lie how a frame as been rendering to a dynamic resolution heuristic.
		if (!ViewFamily.EngineShowFlags.ScreenPercentage)
		{
			checkf(PrimaryResolutionFraction == 1.0f, TEXT("It is illegal to set ResolutionFraction != 1 if screen percentage show flag is disabled."));
		}

		// Make sure the screen percentage interface has not lied to the renderer about the upper bound.
		checkf(PrimaryResolutionFraction <= DynamicResolutionUpperBounds[GDynamicPrimaryResolutionFraction],
			TEXT("ISceneViewFamilyScreenPercentage::GetPrimaryResolutionFractionUpperBound() should not lie to the renderer."));

#if DO_CHECK || USING_CODE_ANALYSIS
		check(ISceneViewFamilyScreenPercentage::IsValidResolutionFraction(PrimaryResolutionFraction));
#endif
	}

	float LensDistortionResolutionFraction = 1.0f;
	if (Views[0].LensDistortionLUT.IsEnabled())
	{
		float AffectScreenPercentage = CVarLensDistortionAffectScreenPercentage.GetValueOnRenderThread();
		LensDistortionResolutionFraction = FMath::Lerp(1.0, Views[0].LensDistortionLUT.ResolutionFraction, AffectScreenPercentage);
	}

	// For multiple views, we must find the maximum overscan resolution so that views can be offset appropriately to avoid overscanned
	// views encroaching into other views' buffer space
	float MaxOverscanResolutionFraction = 1.0f;
	for (const FSceneView* View : ViewFamily.AllViews)
	{
		MaxOverscanResolutionFraction = FMath::Max(MaxOverscanResolutionFraction, View->SceneViewInitOptions.OverscanResolutionFraction);
	}
	
	// Compute final resolution fraction.
	const float ResolutionFraction = PrimaryResolutionFraction * ViewFamily.SecondaryViewFraction * LensDistortionResolutionFraction;
	const float ResolutionMaxFraction = PrimaryMaxResolutionFraction * ViewFamily.SecondaryViewFraction * LensDistortionResolutionFraction;

	// Checks that view rects are correctly initialized.
	for (int32 i = 0; i < Views.Num(); i++)
	{
		FViewInfo& View = Views[i];

		auto InitViewRect = [&View, &MaxOverscanResolutionFraction, ViewFamilySecondaryViewFraction = ViewFamily.SecondaryViewFraction](float PositionResFraction, float SizeResFraction)
			{
				auto GetViewResFraction = [&](float Fraction)
				{
					return View.SceneViewInitOptions.OverridePrimaryResolutionFraction > 0.0 ? (View.SceneViewInitOptions.OverridePrimaryResolutionFraction * ViewFamilySecondaryViewFraction) : Fraction;
				};

				const float ViewPositionFraction = GetViewResFraction(PositionResFraction);
				const float ViewSizeFraction = GetViewResFraction(SizeResFraction);

				FIntPoint ViewSize = ApplyResolutionFraction(View.UnscaledViewRect.Size(), ViewSizeFraction * View.SceneViewInitOptions.OverscanResolutionFraction);
				FIntPoint ViewRectMin = QuantizeViewRectMin(FIntPoint(
					FMath::CeilToInt(View.UnscaledViewRect.Min.X * ViewPositionFraction * MaxOverscanResolutionFraction),
					FMath::CeilToInt(View.UnscaledViewRect.Min.Y * ViewPositionFraction * MaxOverscanResolutionFraction)));

				// Use the bottom-left view rect if requested, instead of top-left
				if (CVarViewRectUseScreenBottom.GetValueOnRenderThread())
				{
					ViewRectMin.Y = FMath::CeilToInt(View.UnscaledViewRect.Max.Y * ViewFamilySecondaryViewFraction) - ViewSize.Y;
				}

				return FIntRect(ViewRectMin, ViewRectMin + ViewSize);
			};

		if (Views.Num() > 1)
		{
			// In split-screen, multiple views share a depth buffer.  Use ResolutionMaxFraction for position so that adjacent views' UpperBoundViewRects don't overlap. 
			View.ViewRect = InitViewRect(ResolutionMaxFraction, ResolutionFraction);
			View.UpperBoundViewRect = InitViewRect(ResolutionMaxFraction, ResolutionMaxFraction);
		}
		else
		{
			View.ViewRect = InitViewRect(ResolutionFraction, ResolutionFraction);
			View.UpperBoundViewRect = InitViewRect(ResolutionMaxFraction, ResolutionMaxFraction);
		}

		#if !UE_BUILD_SHIPPING
		// For testing purpose, override the screen percentage method.
		{
			switch (CVarTestPrimaryScreenPercentageMethodOverride.GetValueOnRenderThread())
			{
			case 1: View.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::SpatialUpscale; break;
			case 2: View.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::TemporalUpscale; break;
			case 3: View.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::RawOutput; break;
			}
		}
		#endif

		// Automatic screen percentage fallback.
		{
			// Tenmporal upsample is supported only if TAA is turned on.
			if (View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::TemporalUpscale &&
				(!IsTemporalAccumulationBasedMethod(View.AntiAliasingMethod) ||
				 ViewFamily.EngineShowFlags.VisualizeBuffer || 
				 ViewFamily.EngineShowFlags.VisualizeSSS))
			{
				View.PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::SpatialUpscale;
			}
		}

		check(View.ViewRect.Area() != 0);
		check(View.UpperBoundViewRect.Area() != 0);
#if DO_CHECK || USING_CODE_ANALYSIS
		check(View.VerifyMembersChecks());
#endif

		if (!ViewFamily.EngineShowFlags.HitProxies)
		{
			FIntPoint SecondaryViewRect = View.GetSecondaryViewRectSize();

			const float ViewResolutionFraction = View.SceneViewInitOptions.OverridePrimaryResolutionFraction > 0.0 ? (View.SceneViewInitOptions.OverridePrimaryResolutionFraction * ViewFamily.SecondaryViewFraction) : ResolutionFraction;
			GPixelRenderCounters.AddViewStatistics(ViewResolutionFraction, View.ViewRect.Size(), SecondaryViewRect.X * SecondaryViewRect.Y);
		}
	}

	// Shifts all view rects layout to the top left corner of the buffers, since post processing will just output the final
	// views in FSceneViewFamily::RenderTarget whereever it was requested with FSceneView::UnscaledViewRect.
	{
		FIntPoint TopLeftShift = Views[0].ViewRect.Min;
		for (int32 i = 1; i < Views.Num(); i++)
		{
			TopLeftShift.X = FMath::Min(TopLeftShift.X, Views[i].ViewRect.Min.X);
			TopLeftShift.Y = FMath::Min(TopLeftShift.Y, Views[i].ViewRect.Min.Y);
		}
		for (int32 i = 0; i < Views.Num(); i++)
		{
			Views[i].ViewRect -= TopLeftShift;
			Views[i].UpperBoundViewRect -= TopLeftShift;
		}
	}

	#if !UE_BUILD_SHIPPING
	{
		int32 ViewRectOffset = CVarTestInternalViewRectOffset.GetValueOnRenderThread();

		if (Views.Num() == 1 && ViewRectOffset > 0)
		{
			FViewInfo& View = Views[0];

			if (!View.bIsSceneCapture && !View.bIsReflectionCapture)
			{
				FIntPoint DesiredBufferSize = GetDesiredInternalBufferSize(ViewFamily);
				FIntPoint Offset = (DesiredBufferSize - View.ViewRect.Size()) / 2;
				FIntPoint NewViewRectMin(0, 0);

				switch (ViewRectOffset)
				{
					// Move to the center of the buffer.
				case 1: NewViewRectMin = Offset; break;

					// Move to top left.
				case 2: break;

					// Move to top right.
				case 3: NewViewRectMin = FIntPoint(2 * Offset.X, 0); break;

					// Move to bottom right.
				case 4: NewViewRectMin = FIntPoint(0, 2 * Offset.Y); break;

					// Move to bottom left.
				case 5: NewViewRectMin = FIntPoint(2 * Offset.X, 2 * Offset.Y); break;
				}

				const FIntPoint NewOffset = QuantizeViewRectMin(NewViewRectMin) - View.ViewRect.Min;

				View.ViewRect += NewOffset;
				View.UpperBoundViewRect += NewOffset;

#if DO_CHECK || USING_CODE_ANALYSIS
				check(View.VerifyMembersChecks());
#endif
			}
		}
	}
	#endif

	ComputeFamilySize();

	for (FCustomRenderPassInfo& PassInfo : CustomRenderPassInfos)
	{
		for (FViewInfo& View : PassInfo.Views)
		{
			const FSceneCaptureCustomRenderPassUserData& SceneCaptureUserData = FSceneCaptureCustomRenderPassUserData::Get(PassInfo.CustomRenderPass);

			if (SceneCaptureUserData.bMainViewResolution)
			{
				if (SceneCaptureUserData.bIgnoreScreenPercentage)
				{
					const FIntRect SourceViewRect = FIntRect(FInt32Point::ZeroValue, Views[0].UnscaledViewRect.Size());
					
					View.ViewRect = GetDownscaledViewRect(SourceViewRect, Views[0].GetUnscaledFamilyViewRect().Size(), SceneCaptureUserData.SceneTextureDivisor);
				}
				else
				{
					View.ViewRect = GetDownscaledViewRect(Views[0].ViewRect, Views[0].GetFamilyViewRect().Max, SceneCaptureUserData.SceneTextureDivisor);

					// Share temporal AA offset if this is coincident with main view camera
					if (SceneCaptureUserData.bMainViewCamera && SceneCaptureUserData.SceneTextureDivisor == FIntPoint(1,1))
					{
						View.TemporalSourceView = &Views[0];
					}
				}
				View.UpperBoundViewRect = View.UnconstrainedViewRect = View.ViewRect;
			}
			else
			{
				View.UpperBoundViewRect = View.ViewRect = View.UnscaledViewRect;
			}
		}	
	}
}

#if WITH_MGPU
void FSceneRenderer::ComputeGPUMasks(FRHICommandListImmediate* RHICmdList)
{
	if (bGPUMasksComputed)
	{
		return;
	}

	RenderTargetGPUMask = FRHIGPUMask::GPU0();
	
	// Scene capture render targets should be propagated to all GPUs the render target exists on.  For other render targets
	// (like nDisplay outputs), we default them to only be copied to GPU0, for performance.
	//
	// TODO:  we should remove this conditional, and set the GPU mask for the source render targets, but the goal is to have
	// a minimal scope CL for the 5.1.1 hot fix.  This effectively reverts the change from CL 20540730, just for scene captures.
	if ((GNumExplicitGPUsForRendering > 1) && ViewFamily.RenderTarget && Views[0].bIsSceneCapture)
	{
		check(RHICmdList);
		RenderTargetGPUMask = ViewFamily.RenderTarget->GetGPUMask(*RHICmdList);
	}

	// First check whether we are in multi-GPU and if fork and join cross-gpu transfers are enabled.
	// Otherwise fallback on rendering the whole view family on each relevant GPU using broadcast logic.
	if (GNumExplicitGPUsForRendering > 1 && CVarEnableMultiGPUForkAndJoin.GetValueOnAnyThread() != 0)
	{
		// Start iterating from RenderTargetGPUMask and then wrap around. This avoids an
		// unnecessary cross-gpu transfer in cases where you only have 1 view and the
		// render target is located on a GPU other than GPU 0.
		FRHIGPUMask::FIterator GPUIterator(RenderTargetGPUMask);
		for (FViewInfo& ViewInfo : Views)
		{
			// Only handle views that are to be rendered (this excludes instance stereo).
			if (ViewInfo.ShouldRenderView())
			{
				// TODO:  should reflection captures run on one GPU and transfer, like all other rendering?
				if (ViewInfo.bIsReflectionCapture)
				{
					ViewInfo.GPUMask = FRHIGPUMask::All();
				}
				else
				{
					if (!ViewInfo.bOverrideGPUMask)
					{
						ViewInfo.GPUMask = FRHIGPUMask::FromIndex(*GPUIterator);
					}

					ViewFamily.bMultiGPUForkAndJoin |= (ViewInfo.GPUMask != RenderTargetGPUMask);

					// Increment and wrap around if we reach the last index.
					++GPUIterator;
					if (!GPUIterator)
					{
						GPUIterator = FRHIGPUMask::FIterator(RenderTargetGPUMask);
					}
				}
			}
		}
	}
	else
	{
		for (FViewInfo& ViewInfo : Views)
		{
			if (ViewInfo.ShouldRenderView())
			{
				ViewInfo.GPUMask = RenderTargetGPUMask;
			}
		}
	}

	AllViewsGPUMask = Views[0].GPUMask;
	for (int32 ViewIndex = 1; ViewIndex < Views.Num(); ++ViewIndex)
	{
		AllViewsGPUMask |= Views[ViewIndex].GPUMask;
	}

	bGPUMasksComputed = true;
}
#endif // WITH_MGPU

#if WITH_MGPU
DECLARE_GPU_STAT_NAMED(CrossGPUTransfers, TEXT("Cross GPU Transfer"));
DECLARE_GPU_STAT_NAMED(CrossGPUSync, TEXT("Cross GPU Sync"));

struct FCrossGPUTransfer
{
	FIntRect TransferRect;
	int32 SrcGPUIndex;
	int32 DestGPUIndex;
	FTransferResourceFenceData* DelayedFence;

	FCrossGPUTransfer(const FIntRect& InTransferRect, uint32 InSrcGPUIndex, uint32 InDestGPUIndex)
		: TransferRect(InTransferRect), SrcGPUIndex(InSrcGPUIndex), DestGPUIndex(InDestGPUIndex), DelayedFence(nullptr)
	{
		// Empty
	}
};

struct FCrossGPUTarget
{
	const FRenderTarget* RenderTarget = nullptr;
	TArray<FCrossGPUTransfer> Transfers;
};

class FCrossGPUTransfersDeferred : public UE::Private::FQueryableRefCountedObject
{
public:
	TArray<FCrossGPUTarget> Targets;
};

static void GetCrossGPUTransfers(FSceneRenderer* SceneRenderer, TArray<FCrossGPUTransfer>& OutTransfers, TArrayView<FViewInfo> InViews, const FIntPoint RenderTargetSize, FRHIGPUMask RenderTargetGPUMask)
{
	check(SceneRenderer->bGPUMasksComputed);

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ++ViewIndex)
	{
		const FViewInfo& ViewInfo = InViews[ViewIndex];
		if (ViewInfo.bAllowCrossGPUTransfer && ViewInfo.GPUMask != RenderTargetGPUMask)
		{
			// Clamp the view rect by the rendertarget rect to prevent issues when resizing the viewport.
			const FIntRect TransferRect(ViewInfo.UnscaledViewRect.Min.ComponentMin(RenderTargetSize), ViewInfo.UnscaledViewRect.Max.ComponentMin(RenderTargetSize));
			if (TransferRect.Width() > 0 && TransferRect.Height() > 0)
			{
				for (uint32 RenderTargetGPUIndex : RenderTargetGPUMask)
				{
					if (!ViewInfo.GPUMask.Contains(RenderTargetGPUIndex))
					{
						OutTransfers.Add(FCrossGPUTransfer(TransferRect, ViewInfo.GPUMask.GetFirstIndex(), RenderTargetGPUIndex));

						// If multiple families write to the same render target via MGPU, mask cross GPU copies after the first view family
						// to the view rect.
						SceneRenderer->EnumerateLinkedViewFamilies([&] (const FSceneViewFamily& ViewFamily)
						{
							if (&ViewFamily == &SceneRenderer->ViewFamily)
							{
								// Exit if we found the current view family
								return false;
							}
							else if (ViewFamily.RenderTarget == SceneRenderer->ViewFamily.RenderTarget)
							{
								// We found another view family writing to the same target, set the TransferRect
								OutTransfers.Last().TransferRect = ViewInfo.UnscaledViewRect;
								return false;
							}
							return true;
						});
					}
				}
			}
		}
	}
}
#endif // WITH_MGPU

void FSceneRenderer::PreallocateCrossGPUFences(TConstArrayView<FSceneRenderer*> SceneRenderers)
{
#if WITH_MGPU
	if (SceneRenderers.Num() > 1 && GNumExplicitGPUsForRendering > 1)
	{
		int32 CrossGPUOption = CVarCrossGPUTransferOption.GetValueOnAnyThread();
		if (CrossGPUOption == 1)
		{
			// Allocated fences to wait on are placed in the last scene renderer
			TArray<FCrossGPUTransferFence*>& LastRendererFencesWait = SceneRenderers.Last()->CrossGPUTransferFencesWait;

			check(LastRendererFencesWait.IsEmpty());

			// Each prior renderer allocates fences and also adds them to last renderer
			for (int32 RendererIndex = 0; RendererIndex < SceneRenderers.Num() - 1; RendererIndex++)
			{
				FSceneRenderer* SceneRenderer = SceneRenderers[RendererIndex];

				check(SceneRenderer->CrossGPUTransferFencesDefer.IsEmpty());

				SceneRenderer->ComputeGPUMasks(nullptr);

				if (SceneRenderer->ViewFamily.bMultiGPUForkAndJoin)
				{
					// Check if we can do optimized transfers, which requires a single index
					if (SceneRenderer->AllViewsGPUMask.HasSingleIndex())
					{
						TArray<FCrossGPUTransfer> Transfers;
						GetCrossGPUTransfers(SceneRenderer, Transfers, SceneRenderer->Views, SceneRenderer->ViewFamily.RenderTarget->GetSizeXY(), SceneRenderer->RenderTargetGPUMask);

						SceneRenderer->CrossGPUTransferFencesDefer.SetNumUninitialized(Transfers.Num());

						for (int32 TransferIndex = 0; TransferIndex < Transfers.Num(); TransferIndex++)
						{
							FCrossGPUTransferFence* FenceData = RHICreateCrossGPUTransferFence();

							SceneRenderer->CrossGPUTransferFencesDefer[TransferIndex] = FenceData;
							LastRendererFencesWait.Add(FenceData);
						}
					}
				}
			}
		}
		else if (CrossGPUOption == 2)
		{
			TRefCountPtr<FCrossGPUTransfersDeferred> TransfersDeferred = new FCrossGPUTransfersDeferred;
			for (FSceneRenderer* SceneRenderer : SceneRenderers)
			{
				// Each scene renderer will add transfers to the shared structure, then the last will emit the transfers
				SceneRenderer->CrossGPUTransferDeferred = TransfersDeferred;
			}
		}
	}
#endif
}

void FSceneRenderer::DoCrossGPUTransfers(FRDGBuilder& GraphBuilder, FRDGTextureRef RenderTargetTexture, TArrayView<FViewInfo> InViews, bool bCrossGPUTransferFencesDefer, FRHIGPUMask InRenderTargetGPUMask, class FCrossGPUTransfersDeferred* TransfersDeferred)
{
#if WITH_MGPU
	// Must be all GPUs because context redirector only supports single or all GPUs
	RDG_GPU_MASK_SCOPE(GraphBuilder, FRHIGPUMask::All());
	RDG_EVENT_SCOPE_STAT(GraphBuilder, CrossGPUTransfers, "CrossGPUTransfers");

	// Need to use this structure as an intermediate, because the RHI texture reference isn't available yet,
	// and must be fetched inside the pass.
	TArray<FCrossGPUTransfer> Transfers;
	GetCrossGPUTransfers(this, Transfers, InViews, RenderTargetTexture->Desc.Extent, InRenderTargetGPUMask);

	if (TransfersDeferred)
	{
		// Accumulate transfers from each scene renderer
		if (Transfers.Num() > 0)
		{
			TransfersDeferred->Targets.Add({ ViewFamily.RenderTarget, MoveTemp(Transfers) });
		}
	}
	else if (Transfers.Num() > 0)
	{
		if (bCrossGPUTransferFencesDefer)
		{
			// Optimized push transfer code path, with delay for the cross GPU transfer fence wait
			// A readback pass is the closest analog to what this is doing. There isn't a way to express cross-GPU transfers via the RHI barrier API.
			AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("CrossGPUTransfers"), RenderTargetTexture,
				[this, RenderTargetTexture, LocalTransfers = MoveTemp(Transfers), PostTransferFences = MoveTemp(CrossGPUTransferFencesDefer)](FRHICommandListImmediate& RHICmdList)
			{
				TArray<FTransferResourceParams> TransferParams;
				for (const FCrossGPUTransfer& Transfer : LocalTransfers)
				{
					TransferParams.Add(FTransferResourceParams(RenderTargetTexture->GetRHI(), Transfer.SrcGPUIndex, Transfer.DestGPUIndex, false, false));
					TransferParams.Last().SetRect(Transfer.TransferRect);
				}

				// Transition resources on destination GPU and signal when transition has finished
				TArray<FCrossGPUTransferFence*> PreTransferFences;
				RHIGenerateCrossGPUPreTransferFences(TransferParams, PreTransferFences);
				RHICmdList.CrossGPUTransferSignal(TransferParams, PreTransferFences);

				// Then do the actual transfer
				RHICmdList.CrossGPUTransfer(TransferParams, PreTransferFences, PostTransferFences);
			});
		}
		else
		{
			// A readback pass is the closest analog to what this is doing. There isn't a way to express cross-GPU transfers via the RHI barrier API.
			AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("CrossGPUTransfers"), RenderTargetTexture,
				[this, RenderTargetTexture, LocalTransfers = MoveTemp(Transfers)](FRHICommandListImmediate& RHICmdList)
			{
				TArray<FTransferResourceParams> TransferParams;
				for (const FCrossGPUTransfer& Transfer : LocalTransfers)
				{
					TransferParams.Add(FTransferResourceParams(RenderTargetTexture->GetRHI(), Transfer.SrcGPUIndex, Transfer.DestGPUIndex, true, false));
					TransferParams.Last().SetRect(Transfer.TransferRect);
				}

				RHICmdList.TransferResources(TransferParams);
			});
		}
	}
#endif // WITH_MGPU
}

#if WITH_MGPU
BEGIN_SHADER_PARAMETER_STRUCT(FFlushCrossGPUTransfersParameters, )
	RDG_TEXTURE_ACCESS_ARRAY(Textures)
END_SHADER_PARAMETER_STRUCT()
#endif

void FSceneRenderer::FlushCrossGPUTransfers(FRDGBuilder& GraphBuilder)
{
#if WITH_MGPU
	if (CrossGPUTransferDeferred)
	{
		// If this is the last scene renderer, flush the transfers
		if (CrossGPUTransferDeferred->GetRefCount() == 1 && CrossGPUTransferDeferred->Targets.Num())
		{
			auto* PassParameters = GraphBuilder.AllocParameters<FFlushCrossGPUTransfersParameters>();
			PassParameters->Textures.Reserve(CrossGPUTransferDeferred->Targets.Num());

			// Create RDG textures for each render target
			for (FCrossGPUTarget& Target : CrossGPUTransferDeferred->Targets)
			{
				FRHITexture* TextureRHI = Target.RenderTarget->GetRenderTargetTexture();
				check(TextureRHI);
				PassParameters->Textures.Emplace(RegisterExternalTexture(GraphBuilder, TextureRHI, TEXT("CrossGPUTexture")), ERHIAccess::CopySrc);
			}

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("CrossGPUTransfers"),
				PassParameters,
				ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
				[LocalTransfers = CrossGPUTransferDeferred, PassParameters](FRHICommandList& RHICmdList)
			{
				TArray<FTransferResourceParams> TransferParams;
				for (int32 TargetIndex = 0; TargetIndex < LocalTransfers->Targets.Num(); TargetIndex++)
				{
					const FCrossGPUTarget& Target = LocalTransfers->Targets[TargetIndex];
					for (const FCrossGPUTransfer& Transfer : Target.Transfers)
					{
						TransferParams.Add(FTransferResourceParams(PassParameters->Textures[TargetIndex].GetTexture()->GetRHI(), Transfer.SrcGPUIndex, Transfer.DestGPUIndex, true, true));
						TransferParams.Last().SetRect(Transfer.TransferRect);
					}
				}

				RHICmdList.TransferResources(TransferParams);
			});
		}

		// Remove reference to the deferred transfers in the flush for each scene
		CrossGPUTransferDeferred = nullptr;
	}
#endif // WITH_MGPU
}

void FSceneRenderer::FlushCrossGPUFences(FRDGBuilder& GraphBuilder)
{
#if WITH_MGPU
	if (CrossGPUTransferFencesWait.Num() > 0)
	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, CrossGPUSync, "CrossGPUSync");

		AddPass(GraphBuilder, RDG_EVENT_NAME("CrossGPUTransferSync"),
			[LocalFenceDatas = MoveTemp(CrossGPUTransferFencesWait)](FRDGAsyncTask, FRHICommandList& RHICmdList)
		{
			RHICmdList.CrossGPUTransferWait(LocalFenceDatas);
		});
	}
#endif // WITH_MGPU
}


void FSceneRenderer::ComputeFamilySize()
{
	check(FamilySize.X == 0);
	check(IsInParallelRenderingThread());

	// Calculate the screen extents of the view family.
	bool bInitializedExtents = false;
	float MaxFamilyX = 0;
	float MaxFamilyY = 0;

	for (FViewInfo& View : Views)
	{
		float FinalViewMaxX = (float)View.ViewRect.Max.X;
		float FinalViewMaxY = (float)View.ViewRect.Max.Y;

		// Derive the amount of scaling needed for screenpercentage from the scaled / unscaled rect
		const float XScale = FinalViewMaxX / (float)View.UnscaledViewRect.Max.X;
		const float YScale = FinalViewMaxY / (float)View.UnscaledViewRect.Max.Y;

		if (!bInitializedExtents)
		{
			// Note: using the unconstrained view rect to compute family size
			// In the case of constrained views (black bars) this means the scene render targets will fill the whole screen
			// Which is needed for mobile paths where we render directly to the backbuffer, and the scene depth buffer has to match in size
			MaxFamilyX = View.UnconstrainedViewRect.Max.X * XScale;
			MaxFamilyY = View.UnconstrainedViewRect.Max.Y * YScale;
			bInitializedExtents = true;
		}
		else
		{
			MaxFamilyX = FMath::Max(MaxFamilyX, View.UnconstrainedViewRect.Max.X * XScale);
			MaxFamilyY = FMath::Max(MaxFamilyY, View.UnconstrainedViewRect.Max.Y * YScale);
		}

		// floating point imprecision could cause MaxFamilyX to be less than View->ViewRect.Max.X after integer truncation.
		// since this value controls rendertarget sizes, we don't want to create rendertargets smaller than the view size.
		MaxFamilyX = FMath::Max(MaxFamilyX, FinalViewMaxX);
		MaxFamilyY = FMath::Max(MaxFamilyY, FinalViewMaxY);

		View.ViewRectWithSecondaryViews = View.ViewRect;
		if (View.IsInstancedStereo())
		{
			for (const FSceneView* SecondaryView : View.GetSecondaryViews())
			{
				const FViewInfo& InstancedView = static_cast<const FViewInfo&>(*SecondaryView);
				View.ViewRectWithSecondaryViews.Union(InstancedView.ViewRect);
			}
		}
	}

	// We render to the actual position of the viewports so with black borders we need the max.
	// We could change it by rendering all to left top but that has implications for splitscreen. 
	FamilySize.X = FMath::TruncToInt(MaxFamilyX);
	FamilySize.Y = FMath::TruncToInt(MaxFamilyY);

	check(FamilySize.X != 0);
	check(bInitializedExtents);
}

FSceneRenderer::~FSceneRenderer()
{
	// Manually release references to TRefCountPtrs that are allocated on the mem stack, which doesn't call dtors
	SortedShadowsForShadowDepthPass.Release();

	for (FCustomRenderPassInfo& Info : CustomRenderPassInfos)
	{
		if (Info.CustomRenderPass)
		{
			delete Info.CustomRenderPass;
		}
	}
}

IVisibilityTaskData* FSceneRenderer::OnRenderBegin(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs)
{
	check(!FDeferredUpdateResource::IsUpdateNeeded());

	// This is called prior to scene update to avoid a race condition with the MDC caching task.
	FVirtualTextureSystem::Get().CallPendingCallbacks();
	FMaterialCacheTagProvider::Get().CallPendingCallbacks();

	// This is called prior to scene update
	OIT::OnRenderBegin(Scene->OITSceneData);

	const bool bIsMobilePlatform = IsMobilePlatform(ShaderPlatform);

	EUpdateAllPrimitiveSceneInfosAsyncOps AsyncOps = EUpdateAllPrimitiveSceneInfosAsyncOps::None;

	if (GAsyncCreateLightPrimitiveInteractions > 0)
	{
		AsyncOps |= EUpdateAllPrimitiveSceneInfosAsyncOps::CreateLightPrimitiveInteractions;
	}

	if (GAsyncCacheMeshDrawCommands > 0)
	{
		AsyncOps |= EUpdateAllPrimitiveSceneInfosAsyncOps::CacheMeshDrawCommands;
	}

	if (GAsyncCacheMaterialUniformExpressions > 0 && !bIsMobilePlatform)
	{
		AsyncOps |= EUpdateAllPrimitiveSceneInfosAsyncOps::CacheMaterialUniformExpressions;
	}

	IVisibilityTaskData* VisibilityTaskData = nullptr;

	FScene::FUpdateParameters SceneUpdateParameters;
	SceneUpdateParameters.AsyncOps = AsyncOps;

	UE::Tasks::FTaskEvent GPUSceneUpdateTaskPrerequisites{ UE_SOURCE_LOCATION };
	SceneUpdateParameters.GPUSceneUpdateTaskPrerequisites = GPUSceneUpdateTaskPrerequisites;

	UE::Tasks::FTask PrepareSceneTexturesConfigTask;

	if (SceneUpdateInputs)
	{
		PrepareSceneTexturesConfigTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [SceneUpdateInputs]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrepareViewRects);
			FTaskTagScope TagScope(ETaskTag::EParallelRenderingThread);
		
			for (FSceneRenderer* Renderer : SceneUpdateInputs->Renderers)
			{
				Renderer->PrepareViewRectsForRendering();
	
				InitializeSceneTexturesConfig(Renderer->ViewFamily.SceneTexturesConfig, Renderer->ViewFamily);
				const FSceneTexturesConfig& SceneTexturesConfig = Renderer->GetActiveSceneTexturesConfig();

				// Custom render passes have their own view family structure, so they can have separate EngineShowFlags, so the SceneTexturesConfig
				// needs to be copied.  The FSceneTextures structure itself is pointer shared, and doesn't need to be copied.
				for (FCustomRenderPassInfo& CustomRenderPass : Renderer->CustomRenderPassInfos)
				{
					// If this is excluded from scene texture extents, it computes its own scene texture config, rather than copying.
					if (CustomRenderPass.ViewFamily.Views[0]->bExcludeFromSceneTextureExtents)
					{
						InitializeSceneTexturesConfig(CustomRenderPass.ViewFamily.SceneTexturesConfig, CustomRenderPass.ViewFamily);
					}
					else
					{
						CustomRenderPass.ViewFamily.SceneTexturesConfig = Renderer->ViewFamily.SceneTexturesConfig;
					}

					// Custom Render Passes don't support MSAA.  If MSAA is enabled, the first Custom Render Pass will allocate a separate non-MSAA
					// FSceneTextures, initialized using this config (see logic in the FSceneRenderer constructor that fills in CustomRenderPassInfos).
					CustomRenderPass.ViewFamily.SceneTexturesConfig.NumSamples = 1;
					CustomRenderPass.ViewFamily.SceneTexturesConfig.EditorPrimitiveNumSamples = 1;
				}
			}

		}, UE::Tasks::ETaskPriority::Normal, bIsMobilePlatform ? UE::Tasks::EExtendedTaskPriority::Inline : UE::Tasks::EExtendedTaskPriority::None);
	}

	SceneUpdateParameters.Callbacks.PostStaticMeshUpdate = [&] (const UE::Tasks::FTask& StaticMeshUpdateTask)
	{
		PrepareSceneTexturesConfigTask.Wait();

#if RHI_RAYTRACING
		if (SceneUpdateInputs)
		{
			RayTracing::OnRenderBegin(*SceneUpdateInputs);

			for (FSceneRenderer* Renderer : SceneUpdateInputs->Renderers)
			{
				Renderer->InitializeRayTracingFlags_RenderThread();
			}
		}
#endif

		if (!ViewFamily.ViewExtensions.IsEmpty())
		{
			RDG_CSV_STAT_EXCLUSIVE_SCOPE(GraphBuilder, PreRender);
			SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_ViewExtensionPreRenderView);

			for (auto& ViewExtension : ViewFamily.ViewExtensions)
			{
				ViewExtension->PreRenderViewFamily_RenderThread(GraphBuilder, ViewFamily);

				for (FViewInfo* View : AllViews)
				{
					ViewExtension->PreRenderView_RenderThread(GraphBuilder, *View);
				}
			}
		}

		if (SceneUpdateInputs)
		{
			for (FSceneRenderer* Renderer : SceneUpdateInputs->Renderers)
			{
				const FSceneTexturesConfig& SceneTexturesConfig = Renderer->GetActiveSceneTexturesConfig();
				Renderer->PrepareViewStateForVisibility(SceneTexturesConfig);
			}
		}

		// This must be done after ViewExtension->PreRenderView_RenderThread performs any updates to the camera position
		PrepareViewStatePrevFrameViewInfo();

		if (ViewFamily.EngineShowFlags.LensDistortion && FPaniniProjectionConfig::IsEnabledByCVars())
		{
			const FPaniniProjectionConfig PaniniProjection = FPaniniProjectionConfig::ReadCVars();

			for (FViewInfo& View : Views)
			{
				if (View.ViewMatrices.IsPerspectiveProjection())
				{
					View.LensDistortionLUT = PaniniProjection.GenerateLUTPasses(GraphBuilder, View);
				}
			}
		}

		// Run Groom LOD selection prior to visibility for selecting appropriate LOD & geometry type
		if (IsGroomEnabled())
		{
			if (Views.Num() > 0 && !ViewFamily.EngineShowFlags.HitProxies)
			{
				FHairStrandsBookmarkParameters Parameters;
				CreateHairStrandsBookmarkParameters(Scene, Views, AllViews, Parameters, false/*bComputeVisibleInstances*/);
				if (Parameters.HasInstances())
				{
					// 1. Select appropriate LOD & geometry type
					RunHairStrandsBookmark(GraphBuilder, EHairStrandsBookmark::ProcessLODSelection, Parameters);
				}
			}
		}

		// Notify the atmosphere system to update lights that are needed for this scene and all its renders.
		SkyAtmosphereOnRenderBegin(*Scene, ViewFamily.EngineShowFlags);
	
		// Lighting is skipped when running ERendererOutput::DepthPrepassOnly
		if (GetRendererOutput() == ERendererOutput::FinalSceneColor)
		{
			LightFunctionAtlas::OnRenderBegin(LightFunctionAtlas, *Scene, Views, ViewFamily);
		}

		// Notify the volumetric fog system that we start a new frame, deal with its state, etc.
		VolumetricFogOnRenderBegin(*Scene, Views);

		// Allow scene extensions to set up data that may be needed in the visibility task processing.
		GetSceneExtensionsRenderers().PreVisibilityTasks(GraphBuilder);

		VisibilityTaskData = LaunchVisibilityTasks(GraphBuilder.RHICmdList, *this, StaticMeshUpdateTask);

		if (GraphBuilder.IsParallelSetupEnabled())
		{
			GPUSceneUpdateTaskPrerequisites.AddPrerequisites(VisibilityTaskData->GetComputeRelevanceTask());
		}
		GPUSceneUpdateTaskPrerequisites.Trigger();
	};

	if (SceneUpdateInputs)
	{
		// Note: in the future persistent views should be added/removed as other scene primitives such that the updates are deferred and so on. 
		//       right now there's no explicit mechanism for this, so we discover added views here & pass the change set to the scene update.

		SceneUpdateParameters.ViewUpdateChangeSet = Scene->ProcessViewChanges(GraphBuilder, SceneUpdateInputs->Views);
		Scene->Update(GraphBuilder, SceneUpdateParameters);
	}
	else
	{
		SceneUpdateParameters.Callbacks.PostStaticMeshUpdate(UE::Tasks::FTask{});
	}

	FSceneTexturesConfig::Set(GetActiveSceneTexturesConfig());

	// Notify StereoRenderingDevice about new ViewRects
	if (GEngine->StereoRenderingDevice.IsValid() && ViewFamily.EngineShowFlags.StereoRendering)
	{
		for (const FViewInfo& View : Views)
		{
			// if we have an upscale pass, the final rect is _unscaled_ for the compositor
			const FIntRect OutputViewRect =
				(View.PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::RawOutput) ? View.ViewRect : View.UnscaledViewRect;

			if (IStereoRendering::IsStereoEyePass(View.StereoPass))
			{
				GEngine->StereoRenderingDevice->SetFinalViewRect(GraphBuilder.RHICmdList, View.StereoViewIndex, OutputViewRect);
			}
		}
	}

	return VisibilityTaskData;
}

static void SetupDebugViewModes(TConstArrayView<FSceneRenderer*> Renderers)
{
#if WITH_DEBUG_VIEW_MODES
	check(!Renderers.IsEmpty());
	FScene* Scene = Renderers[0]->Scene;

	if (AllowDebugViewShaderMode(DVSM_VisualizeGPUSkinCache, Scene->GetShaderPlatform(), Scene->GetFeatureLevel()))
	{
		bool UpdatedGPUSkinCacheVisualization = false;

		for (FSceneRenderer* Renderer : Renderers)
		{
			FViewInfo& View = Renderer->Views[0];
			FGPUSkinCacheVisualizationData& VisualizationData = GetGPUSkinCacheVisualizationData();

			// Only run visualization update once, but set debug flags for all view families if the mode is active
			// Note VisualizationData.Update needs to be called per frame, as || lazy evaluation is used, so need to do it before evaluating VisualizeGPUSkinCache flag
			if (UpdatedGPUSkinCacheVisualization || VisualizationData.Update(View.CurrentGPUSkinCacheVisualizationMode) || Renderer->ViewFamily.EngineShowFlags.VisualizeGPUSkinCache)
			{
				// When activating visualization from the command line, enable VisualizeGPUSkinCache.
				Renderer->ViewFamily.EngineShowFlags.SetVisualizeGPUSkinCache(true);
				Renderer->ViewFamily.SetDebugViewShaderMode(DVSM_VisualizeGPUSkinCache);
				UpdatedGPUSkinCacheVisualization = true;
			}
		}
	}
#endif  // WITH_DEBUG_VIEW_MODES
}

/** 
* Finishes the view family rendering.
*/
void FSceneRenderer::OnRenderFinish(FRDGBuilder& GraphBuilder, FRDGTextureRef ViewFamilyTexture)
{
	RDG_EVENT_SCOPE(GraphBuilder, "RenderFinish");

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (ViewFamilyTexture)
	{
		bool bShowPrecomputedVisibilityWarning = false;
		static const auto* CVarPrecomputedVisibilityWarning = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.PrecomputedVisibilityWarning"));
		if (CVarPrecomputedVisibilityWarning && CVarPrecomputedVisibilityWarning->GetValueOnRenderThread() == 1)
		{
			bShowPrecomputedVisibilityWarning = !bUsedPrecomputedVisibility;
		}

		bool bShowDemotedLocalMemoryWarning = false;
		static const auto* CVarDemotedLocalMemoryWarning = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DemotedLocalMemoryWarning"));
		if (CVarDemotedLocalMemoryWarning && CVarDemotedLocalMemoryWarning->GetValueOnRenderThread() == 1)
		{
			bShowDemotedLocalMemoryWarning = GDemotedLocalMemorySize > 0;
		}

		bool bShowGlobalClipPlaneWarning = false;

		if (Scene->PlanarReflections.Num() > 0)
		{
			static const auto* CVarClipPlane = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.AllowGlobalClipPlane"));
			
			if (CVarClipPlane && CVarClipPlane->GetValueOnRenderThread() == 0)
			{
				bShowGlobalClipPlaneWarning = true;
			}
		}

		FGPUSkinCache* SkinCache = Scene->GetGPUSkinCache();

		const bool bMeshDistanceFieldEnabled = DoesProjectSupportDistanceFields();
		extern bool UseDistanceFieldAO();
		const bool bShowDFAODisabledWarning = !UseDistanceFieldAO() && (ViewFamily.EngineShowFlags.VisualizeDistanceFieldAO);
		const bool bShowDFDisabledWarning = !bMeshDistanceFieldEnabled && (ViewFamily.EngineShowFlags.VisualizeMeshDistanceFields || ViewFamily.EngineShowFlags.VisualizeGlobalDistanceField || ViewFamily.EngineShowFlags.VisualizeDistanceFieldAO);

		const bool bShowNoSkyAtmosphereComponentWarning = !Scene->HasSkyAtmosphere() && ViewFamily.EngineShowFlags.VisualizeSkyAtmosphere;

		const bool bMobile = (FeatureLevel <= ERHIFeatureLevel::ES3_1);
		const bool bStationarySkylight = Scene->SkyLight && Scene->SkyLight->bWantsStaticShadowing;
		bool bShowSkylightWarning = bStationarySkylight && !FReadOnlyCVARCache::EnableStationarySkylight();
		if (bMobile)
		{
			// For mobile EnableStationarySkylight has to be enabled in a projects with StaticLighting to support Stationary or Movable skylights
			bShowSkylightWarning = IsStaticLightingAllowed() && !FReadOnlyCVARCache::EnableStationarySkylight() && (bStationarySkylight || (Scene->SkyLight && Scene->SkyLight->IsMovable()));
		}

		const bool bRealTimeSkyCaptureButNothingToCapture = Scene->SkyLight && Scene->SkyLight->bRealTimeCaptureEnabled && (!Scene->HasSkyAtmosphere() && !Scene->HasVolumetricCloud() && (Views.Num() > 0 && !Views[0].bSceneHasSkyMaterial));

		// Point light shadows are disabled by default on mobile platforms.
		const bool bShowPointLightWarning = !IsMobilePlatform(ShaderPlatform) ? bUsedPointLightWholeSceneShadow && !FReadOnlyCVARCache::EnablePointLightShadows(ShaderPlatform) : false;
		const bool bShowShadowedLightOverflowWarning = Scene->OverflowingDynamicShadowedLights.Num() > 0;

		const bool bLocalFogVolumeInSceneButProjectDisabled = Scene->HasAnyLocalFogVolume() && !ProjectSupportsLocalFogVolumes();
		
		const bool bLumenHasWarnings = Lumen::WriteWarnings(Scene, ViewFamily.EngineShowFlags, Views, /*FScreenMessageWriter*/ nullptr);

		bool bNaniteEnabledButDisabledInProject = false;
		bool bLocalExposureEnabledOnAnyView = false;

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{	
			FViewInfo& View = Views[ViewIndex];

			bNaniteEnabledButDisabledInProject = bNaniteEnabledButDisabledInProject || (WouldRenderNanite(Scene, View, /*bCheckForAtomicSupport*/ false, /*bCheckForProjectSetting*/ false) && !WouldRenderNanite(Scene, View, /*bCheckForAtomicSupport*/ false, /*bCheckForProjectSetting*/ true));

			if (IsPostProcessingEnabled(View)
				&& (!FMath::IsNearlyEqual(View.FinalPostProcessSettings.LocalExposureHighlightContrastScale, 1.0f)
					|| !FMath::IsNearlyEqual(View.FinalPostProcessSettings.LocalExposureShadowContrastScale, 1.0f)
					|| View.FinalPostProcessSettings.LocalExposureShadowContrastCurve
					|| View.FinalPostProcessSettings.LocalExposureHighlightContrastCurve
					|| !FMath::IsNearlyEqual(View.FinalPostProcessSettings.LocalExposureDetailStrength, 1.0f)))
			{
				bLocalExposureEnabledOnAnyView = true;
			}
		}

		const bool bShowLocalExposureDisabledWarning = ViewFamily.EngineShowFlags.VisualizeLocalExposure && !bLocalExposureEnabledOnAnyView;

		const int32 NaniteShowError = CVarNaniteShowUnsupportedError.GetValueOnRenderThread();
		// 0: disabled
		// 1: show error if Nanite is present in the scene but unsupported, and fallback meshes are not used for rendering
		// 2: show error if Nanite is present in the scene but unsupported, even if fallback meshes are used for rendering

		static const auto NaniteProxyRenderModeVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite.ProxyRenderMode"));
		const int32 NaniteProxyRenderMode = (NaniteProxyRenderModeVar != nullptr) ? (NaniteProxyRenderModeVar->GetInt() != 0) : 0;
		// 0: Fall back to rendering Nanite proxy meshes if Nanite is unsupported.
		// 1: Disable rendering if Nanite is enabled on a mesh but is unsupported
		// 2: Disable rendering if Nanite is enabled on a mesh but is unsupported, except for static mesh editor toggle

		bool bNaniteEnabledButNoAtomics = false;

		bool bNaniteCheckError = (NaniteShowError == 1 && NaniteProxyRenderMode != 0) || (NaniteShowError == 2);
		if (bNaniteCheckError && !NaniteAtomicsSupported())
		{
			// We want to know when Nanite would've been rendered regardless of atomics being supported or not.
			const bool bCheckForAtomicSupport = false;

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				FViewInfo& View = Views[ViewIndex];
				bNaniteEnabledButNoAtomics |= ::ShouldRenderNanite(Scene, View, bCheckForAtomicSupport);
			}
		}

		bool bNaniteDisabledButNoFallbackMeshes = !UseNanite(Scene->GetShaderPlatform()) && !AreNaniteFallbackMeshesEnabledForPlatform(Scene->GetShaderPlatform());

		static const auto ContactShadowNonCastingIntensityCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ContactShadows.NonShadowCastingIntensity"));
		const bool bContactShadowIntensityCvarUsed = ContactShadowNonCastingIntensityCVar && ContactShadowNonCastingIntensityCVar->GetFloat() != 0.0f;

		// Mobile-specific warnings
		const bool bShowMobileLowQualityLightmapWarning = bMobile && !FReadOnlyCVARCache::EnableLowQualityLightmaps() && IsStaticLightingAllowed();
		const bool bShowMobileDynamicCSMWarning = bMobile && Scene->NumMobileStaticAndCSMLights_RenderThread > 0 && !(FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers() && FReadOnlyCVARCache::MobileAllowDistanceFieldShadows());
		const bool bMobileMissingSkyMaterial = (bMobile && Scene->HasSkyAtmosphere() && (Views.Num() > 0 && !Views[0].bSceneHasSkyMaterial));

		const bool bSingleLayerWaterWarning = ShouldRenderSingleLayerWaterSkippedRenderEditorNotification(Views);

		const bool bLightFunctionAtlasOutOfSlotWarning = LightFunctionAtlas.IsLightFunctionAtlasEnabled() ? LightFunctionAtlas.IsOutOfSlots() : false;

		bool bShowWaitingSkylight = false;
		bool bExpenssiveSkyLightRealTimeCaptureWithCloud = false;
#if WITH_EDITOR
		FSkyLightSceneProxy* SkyLight = Scene->SkyLight;
		if (SkyLight && !SkyLight->bRealTimeCaptureEnabled)
		{
			bShowWaitingSkylight = SkyLight->bCubemapSkyLightWaitingForCubeMapTexture || SkyLight->bCaptureSkyLightWaitingForShaders || SkyLight->bCaptureSkyLightWaitingForMeshesOrTextures;
		}

		if (SkyLight && SkyLight->bRealTimeCaptureEnabled && SkyLight->CaptureCubeMapResolution >= 512 && Scene->HasVolumetricCloud())
		{
			static const auto* CVarSkyCloudCubeFacePerFrame = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SkyLight.RealTimeReflectionCapture.TimeSlice.SkyCloudCubeFacePerFrame"));
			static const auto* CVarDisableExpenssiveCaptureMessage = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.SkyLight.RealTimeReflectionCapture.DisableExpenssiveCaptureMessage"));

			bExpenssiveSkyLightRealTimeCaptureWithCloud = 
				CVarSkyCloudCubeFacePerFrame && CVarSkyCloudCubeFacePerFrame->GetValueOnRenderThread() == 6 
				&& CVarDisableExpenssiveCaptureMessage && CVarDisableExpenssiveCaptureMessage->GetValueOnRenderThread() <= 0;
		}
#endif

		FFXSystemInterface* FXInterface = Scene->GetFXSystem();
		const bool bFxDebugDraw = FXInterface && FXInterface->ShouldDebugDraw_RenderThread();

		const bool bHasDelegateWarnings = OnGetOnScreenMessages.IsBound();

		const bool bAnyWarning = bShowPrecomputedVisibilityWarning || bShowDemotedLocalMemoryWarning || bShowGlobalClipPlaneWarning || bShowSkylightWarning || bShowPointLightWarning
			|| bShowDFAODisabledWarning || bShowShadowedLightOverflowWarning || bShowMobileDynamicCSMWarning || bShowMobileLowQualityLightmapWarning
			|| bMobileMissingSkyMaterial || bSingleLayerWaterWarning || bLightFunctionAtlasOutOfSlotWarning || bShowDFDisabledWarning || bShowNoSkyAtmosphereComponentWarning || bFxDebugDraw
			|| bLumenHasWarnings || bNaniteEnabledButNoAtomics || bNaniteEnabledButDisabledInProject || bNaniteDisabledButNoFallbackMeshes
			|| bRealTimeSkyCaptureButNothingToCapture || bShowWaitingSkylight || bExpenssiveSkyLightRealTimeCaptureWithCloud
			|| bShowLocalExposureDisabledWarning || bHasDelegateWarnings || bContactShadowIntensityCvarUsed || bLocalFogVolumeInSceneButProjectDisabled
			;

		for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
		{	
			FViewInfo& View = Views[ViewIndex];
			if (!View.bIsReflectionCapture && !View.bIsSceneCapture )
			{
				const FScreenPassRenderTarget Output(ViewFamilyTexture, View.UnconstrainedViewRect, ERenderTargetLoadAction::ELoad);

				// display a message saying we're frozen
				FSceneViewState* ViewState = (FSceneViewState*)View.State;
				bool bIsFrozen = ViewState && (ViewState->bIsFrozen);
				bool bLocked = View.bIsLocked;
				const bool bStereoView = IStereoRendering::IsStereoEyeView(View);
				const bool bGPUSkinCacheVisualzationMode = SkinCache && ViewFamily.EngineShowFlags.VisualizeGPUSkinCache && View.CurrentGPUSkinCacheVisualizationMode != NAME_None;
				const bool bMegaLightsHasWarning = MegaLights::HasWarning(View);
				const bool bEyeAdaptationHasWarning = EyeAdaptation::WriteWarnings(Scene, View, /*Writer*/ nullptr);

				// display a warning if an ambient cubemap uses non-angular mipmap filtering
				bool bShowAmbientCubemapMipGenSettingsWarning = false;

#if WITH_EDITORONLY_DATA
				for (FFinalPostProcessSettings::FCubemapEntry ContributingCubemap : View.FinalPostProcessSettings.ContributingCubemaps)
				{
					// platform configuration can't be loaded from the rendering thread, therefore the warning wont be displayed for TMGS_FromTextureGroup settings
					if (ContributingCubemap.AmbientCubemap &&
						ContributingCubemap.AmbientCubemap->MipGenSettings != TMGS_FromTextureGroup &&
						ContributingCubemap.AmbientCubemap->MipGenSettings != TMGS_Angular)
					{
						bShowAmbientCubemapMipGenSettingsWarning = true;
						break;
					}
				}
#endif
				const bool bViewHasWarnings = bAnyWarning
					|| bIsFrozen
					|| bLocked
					|| bStereoView
					|| bShowAmbientCubemapMipGenSettingsWarning
					|| bGPUSkinCacheVisualzationMode
					|| bMegaLightsHasWarning
					|| bEyeAdaptationHasWarning;

				if ((GAreScreenMessagesEnabled && !GEngine->bSuppressMapWarnings) && bViewHasWarnings)
				{
					RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);

					const bool bPrimaryStereoView = IStereoRendering::IsAPrimaryView(View);
					const bool bIsInstancedStereo = View.IsInstancedStereo();
					const bool bIsMobileMultiViewEnabled = View.UsesMobileMultiViewShaders();

					AddDrawCanvasPass(GraphBuilder, {}, View, Output,
						[this, &View, SkinCache, bGPUSkinCacheVisualzationMode, ViewState,
						bLocked, bShowPrecomputedVisibilityWarning, bShowDemotedLocalMemoryWarning, bShowGlobalClipPlaneWarning, bShowDFAODisabledWarning, bShowDFDisabledWarning,
						bIsFrozen, bShowSkylightWarning, bShowPointLightWarning, bShowShadowedLightOverflowWarning,
						bShowMobileLowQualityLightmapWarning, bShowMobileDynamicCSMWarning, bMobileMissingSkyMaterial, 
						bSingleLayerWaterWarning, bLightFunctionAtlasOutOfSlotWarning, bShowNoSkyAtmosphereComponentWarning, bFxDebugDraw, FXInterface,
						bShowLocalExposureDisabledWarning, bNaniteEnabledButNoAtomics, bNaniteEnabledButDisabledInProject, bNaniteDisabledButNoFallbackMeshes, bRealTimeSkyCaptureButNothingToCapture, bShowWaitingSkylight,
						bShowAmbientCubemapMipGenSettingsWarning, bLocalFogVolumeInSceneButProjectDisabled, bLumenHasWarnings, bMegaLightsHasWarning, bEyeAdaptationHasWarning,
						bStereoView, bPrimaryStereoView, bIsInstancedStereo, bIsMobileMultiViewEnabled, bContactShadowIntensityCvarUsed, bExpenssiveSkyLightRealTimeCaptureWithCloud]
						(FCanvas& Canvas)
					{
						// so it can get the screen size
						FScreenMessageWriter Writer(Canvas, 130);

						if (bIsFrozen)
						{
							static const FText StateText = NSLOCTEXT("SceneRendering", "RenderingFrozen", "Rendering frozen...");
							Writer.DrawLine(StateText, 10, FLinearColor(0.8f, 1.0f, 0.2f, 1.0f));
						}
						if (bShowPrecomputedVisibilityWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "NoPrecomputedVisibility", "NO PRECOMPUTED VISIBILITY");
							Writer.DrawLine(Message);
						}
						if (bShowGlobalClipPlaneWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "NoGlobalClipPlane", "PLANAR REFLECTION REQUIRES GLOBAL CLIP PLANE PROJECT SETTING ENABLED TO WORK PROPERLY");
							Writer.DrawLine(Message);
						}
						if (bShowDFAODisabledWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "DFAODisabled", "Distance Field AO is disabled through scalability");
							Writer.DrawLine(Message);
						}
						if (bShowDFDisabledWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "DFDisabled", "Mesh distance fields generation is disabled by project settings, cannot visualize DFAO, mesh or global distance field.");
							Writer.DrawLine(Message);
						}

						if (bShowNoSkyAtmosphereComponentWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "SkyAtmosphere", "There is no SkyAtmosphere component to visualize.");
							Writer.DrawLine(Message);
						}
						if (bShowSkylightWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "SkylightNotSuppported", "PROJECT DOES NOT SUPPORT STATIONARY SKYLIGHT: ");
							Writer.DrawLine(Message);
						}
						if (bExpenssiveSkyLightRealTimeCaptureWithCloud)
						{
							// This can happen because cloud in the real time capture are rendered at the face capture resolution
							static const FText Message = NSLOCTEXT("Renderer", "SkylightResolutionTooHigh", "The sky light resolution is too high for 6 faces captured per frame with volumetric cloud tracing (done at face resolution).\n\
																							It could take a long time and cause a GPU TDR or crash on older GPUs.\n\
																							Please reduce the sky light resolution to less than 512 or the r.SkyLight.RealTimeReflectionCapture.TimeSlice.SkyCloudCubeFacePerFrame to less than 6.\n\
																							You may use r.SkyLight.RealTimeReflectionCapture.DisableExpenssiveCaptureMessage 1 to disable that message.");
							Writer.DrawLine(Message);
						}
						if (bRealTimeSkyCaptureButNothingToCapture)
						{
							static const FText Message = NSLOCTEXT("Renderer", "SkylightRequiresSkyAtmosphere", "A sky light with real-time capture enable is in the scene. It requires at least a SkyAtmosphere component, A volumetricCloud component or a mesh with a material tagged as IsSky. Otherwise it will be black");
							Writer.DrawLine(Message);
						}
						if (bShowPointLightWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "PointLight", "A shadow casting point light is present but the project does not support whole scene point light shadows");
							Writer.DrawLine(Message);
						}
						if (bShowShadowedLightOverflowWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "ShadowedLightOverflow", "TOO MANY OVERLAPPING SHADOWED MOVABLE LIGHTS, SHADOW CASTING DISABLED: ");
							Writer.DrawLine(Message);

							for (const FString& LightName : Scene->OverflowingDynamicShadowedLights)
							{
								Writer.DrawLine(FText::FromString(LightName));
							}
						}
						if (bShowMobileLowQualityLightmapWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "MobileLQLightmap", "MOBILE PROJECTS SUPPORTING STATIC LIGHTING MUST HAVE LQ LIGHTMAPS ENABLED");
							Writer.DrawLine(Message);
						}
						if (bShowMobileDynamicCSMWarning)
						{
							static const FText Message = (!FReadOnlyCVARCache::MobileEnableStaticAndCSMShadowReceivers())
								? NSLOCTEXT("Renderer", "MobileDynamicCSM", "PROJECT HAS MOBILE CSM SHADOWS FROM STATIONARY DIRECTIONAL LIGHTS DISABLED")
								: NSLOCTEXT("Renderer", "MobileDynamicCSMDistFieldShadows", "MOBILE CSM+STATIC REQUIRES DISTANCE FIELD SHADOWS ENABLED FOR PROJECT");
							Writer.DrawLine(Message);
						}

						if (bMobileMissingSkyMaterial)
						{
							static const FText Message = NSLOCTEXT("Renderer", "MobileMissingSkyMaterial", "On mobile the SkyAtmosphere component needs a mesh with a material tagged as IsSky and using the SkyAtmosphere nodes to visualize the Atmosphere.");
							Writer.DrawLine(Message);
						}

						if (bGPUSkinCacheVisualzationMode)
						{
							SkinCache->DrawVisualizationInfoText(View.CurrentGPUSkinCacheVisualizationMode, Writer);
						}

						if (bShowLocalExposureDisabledWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "LocalExposureDisabled", "Local Exposure is disabled.");
							Writer.DrawLine(Message);
						}

						if (bLocked)
						{
							static const FText Message = NSLOCTEXT("Renderer", "ViewLocked", "VIEW LOCKED");
							Writer.DrawLine(Message, 10, FLinearColor(0.8f, 1.0f, 0.2f, 1.0f));
						}

						if (bSingleLayerWaterWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "SingleLayerWater", "r.Water.SingleLayer rendering is disabled with a view containing mesh(es) using water material. Meshes are not visible.");
							Writer.DrawLine(Message);
						}

						if (bLightFunctionAtlasOutOfSlotWarning)
						{
							Writer.DrawLine(FText::FromString(LightFunctionAtlas.GetOutOfSlotWarningMessage()));
						}

						if (bLumenHasWarnings)
						{
							Lumen::WriteWarnings(Scene, ViewFamily.EngineShowFlags, Views, &Writer);
						}

						if (bMegaLightsHasWarning)
						{
							MegaLights::WriteWarnings(View, Writer);
						}

						if (bEyeAdaptationHasWarning)
						{
							EyeAdaptation::WriteWarnings(Scene, View, &Writer);
						}

						if (bNaniteEnabledButNoAtomics)
						{
							FString NaniteError = "Nanite is used in the scene but not supported by your graphics hardware and/or driver. Meshes will not render using Nanite.";
							Writer.DrawLine(FText::FromString(NaniteError));
						}

						if (bNaniteEnabledButDisabledInProject)
						{
							static const FText Message = NSLOCTEXT("Renderer", "NaniteDisabledForProject", "Nanite is enabled but cannot render, because the project has Nanite disabled in an ini (r.Nanite.ProjectEnabled = 0)");
							Writer.DrawLine(Message);
						}

						if (bNaniteDisabledButNoFallbackMeshes)
						{
							static const FText Message = NSLOCTEXT("Renderer", "NaniteDisabledButNoFallbackMeshes", "Nanite is disabled but fallback meshes were stripped during cooking for this platform due to project settings. Meshes might not render correctly.");
							Writer.DrawLine(Message);
						}

						if (bShowDemotedLocalMemoryWarning)
						{
							FString String = FString::Printf(TEXT("Video memory has been exhausted (%.3f MB over budget). Expect extremely poor performance."), float(GDemotedLocalMemorySize) / 1048576.0f);
							Writer.DrawLine(FText::FromString(String));
						}

						if (bShowAmbientCubemapMipGenSettingsWarning)
						{
							static const FText Message = NSLOCTEXT("Renderer", "AmbientCubemapMipGenSettings", "Ambient cubemaps should use 'Angular' Mip Gen Settings.");
							Writer.DrawLine(Message);
						}

						if (bContactShadowIntensityCvarUsed)
						{
							static const FText Message = NSLOCTEXT("Renderer", "ContactShadowsIntensityCvar", "r.ContactShadows.NonShadowCastingIntensity is set but ignored. Use setting on the Light Component instead.");
							Writer.DrawLine(Message);
						}

						if (bLocalFogVolumeInSceneButProjectDisabled)
						{
							static const FText Message = NSLOCTEXT("Renderer", "LocalFogVolumeDisabled", "There are Local Fog Volumes in the scene, but your project does not support rendering them. This can be enabled from the project settings panel (r.SupportLocalFogVolumes).");
							Writer.DrawLine(Message);
						}

#if !UE_BUILD_SHIPPING
						if (bStereoView)
						{
							const TCHAR* SecondaryOrInstanced = bIsInstancedStereo || bIsMobileMultiViewEnabled ? TEXT("Instanced") : TEXT("Secondary");
							FString ViewIdString = FString::Printf(TEXT("StereoView: %s"), bPrimaryStereoView ? TEXT("Primary") : SecondaryOrInstanced);
							Writer.DrawLine(FText::FromString(ViewIdString));

							// Display information (in the primary view only) about the particular method used
							if (bPrimaryStereoView)
							{
								const TCHAR* Technique = TEXT("Splitscreen-like");

								if (bIsInstancedStereo)
								{
									Technique = TEXT("Instanced-Stereo");
								}
								else if (bIsMobileMultiViewEnabled)
								{
									Technique = TEXT("Multi-view (mobile)");
								}

								FString TechniqueString = FString::Printf(TEXT("Stereo rendering method: %s"), Technique);
								Writer.DrawLine(FText::FromString(TechniqueString));
							}
						}
#endif

#if WITH_EDITOR
						FSkyLightSceneProxy* SkyLight = Scene->SkyLight;
						if (bShowWaitingSkylight && SkyLight)
						{
							const FLinearColor OrangeColor = FColor::Orange;

							FString String = "Sky Light waiting on ";
							bool bAddComma = false;
							if (SkyLight->bCubemapSkyLightWaitingForCubeMapTexture)
							{
								String += TEXT("CubeMap");
								bAddComma = true;
							}
							if (SkyLight->bCaptureSkyLightWaitingForShaders)
							{
								String += bAddComma ? TEXT(", ") : TEXT("");
								String += TEXT("Shaders");
								bAddComma = true;
							}
							if (SkyLight->bCaptureSkyLightWaitingForMeshesOrTextures)
							{
								String += bAddComma ? TEXT(", ") : TEXT("");
								String += TEXT("Meshes, Textures");
							}
							String += TEXT(" for final capture.");
							Writer.DrawLine(FText::FromString(String), 10, OrangeColor);
						}
#endif
						OnGetOnScreenMessages.Broadcast(Writer);
					});
					if (bFxDebugDraw)
					{
						FXInterface->DrawDebug_RenderThread(GraphBuilder, (const FSceneView&)View, Output);
					}
				}
			}
		}
	}
	
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	// Save the post-occlusion visibility stats for the frame and freezing info
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		INC_DWORD_STAT_BY(STAT_VisibleStaticMeshElements, View.NumVisibleStaticMeshElements);
		INC_DWORD_STAT_BY(STAT_VisibleDynamicPrimitives, View.NumVisibleDynamicPrimitives);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// update freezing info
		FSceneViewState* ViewState = (FSceneViewState*)View.State;
		if (ViewState)
		{
			// if we're finished freezing, now we are frozen
			if (ViewState->bIsFreezing)
			{
				ViewState->bIsFreezing = false;
				ViewState->bIsFrozen = true;
				ViewState->bIsFrozenViewMatricesCached = true;
				ViewState->CachedViewMatrices = View.ViewMatrices;
			}

			// handle freeze toggle request
			if (bHasRequestedToggleFreeze)
			{
				// do we want to start freezing or stop?
				ViewState->bIsFreezing = !ViewState->bIsFrozen;
				ViewState->bIsFrozen = false;
				ViewState->bIsFrozenViewMatricesCached = false;
				ViewState->FrozenPrimitives.Empty();
			}
		}
#endif
	}

#if SUPPORTS_VISUALIZE_TEXTURE
	// clear the commands
	bHasRequestedToggleFreeze = false;

	if(ViewFamily.EngineShowFlags.OnScreenDebug && ViewFamilyTexture)
	{
		for(int32 ViewIndex = 0;ViewIndex < Views.Num();ViewIndex++)
		{
			const FViewInfo& View = Views[ViewIndex];

			if(!View.IsPerspectiveProjection())
			{
				continue;
			}

			const FScreenPassRenderTarget Output(ViewFamilyTexture, View.UnconstrainedViewRect, ERenderTargetLoadAction::ELoad);

			FVisualizeTexturePresent::PresentContent(GraphBuilder, View, Output);
		}
	}
#endif //SUPPORTS_VISUALIZE_TEXTURE

	{
		SCOPE_CYCLE_COUNTER(STAT_FDeferredShadingSceneRenderer_ViewExtensionPostRenderView);
		for(int32 ViewExt = 0; ViewExt < ViewFamily.ViewExtensions.Num(); ++ViewExt)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "ViewFamilyExtension(%d)", ViewExt);
			ISceneViewExtension& ViewExtension = *ViewFamily.ViewExtensions[ViewExt];
			ViewExtension.PostRenderViewFamily_RenderThread(GraphBuilder, ViewFamily);

			for(int32 ViewIndex = 0; ViewIndex < ViewFamily.Views.Num(); ++ViewIndex)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "ViewExtension(%d)", ViewIndex);
				ViewExtension.PostRenderView_RenderThread(GraphBuilder, Views[ViewIndex]);
			}
		}
	}

	GraphBuilder.AddPostExecuteCallback([this]
	{
		if (GDumpMeshDrawCommandMemoryStats)
		{
			GDumpMeshDrawCommandMemoryStats = 0;
			Scene->DumpMeshDrawCommandMemoryStats();
		}
	});

#if RHI_RAYTRACING
	GraphBuilder.AddPostExecuteCallback([this]
		{
			Scene->RayTracingScene.Reset();;
		});
#endif // RHI_RAYTRACING

	// Sync all mesh pass setup tasks prior to starting RDG execution.
	FGraphEventArray TasksToWait;

	for (FParallelMeshDrawCommandPass* DispatchedShadowDepthPass : DispatchedShadowDepthPasses)
	{
		if (FGraphEventRef Event = DispatchedShadowDepthPass->AcquireTaskEvent())
		{
			TasksToWait.Emplace(MoveTemp(Event));
		}
	}

	for (const FViewInfo* View : AllViews)
	{
		for (FParallelMeshDrawCommandPass* Pass : View->ParallelMeshDrawCommandPasses)
		{
			if (Pass)
			{
				if (FGraphEventRef Event = Pass->AcquireTaskEvent())
				{
					TasksToWait.Emplace(MoveTemp(Event));
				}
			}
		}
	}

	FTaskGraphInterface::Get().WaitUntilTasksComplete(TasksToWait, ENamedThreads::GetRenderThread_Local());
}

void FSceneRenderer::SetupMeshPass(FViewInfo& View, FExclusiveDepthStencil::Type BasePassDepthStencilAccess, FViewCommands& ViewCommands, FInstanceCullingManager& InstanceCullingManager)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSceneRenderer::SetupMeshPass);

	const EShadingPath ShadingPath = GetFeatureLevelShadingPath(Scene->GetFeatureLevel());
	FSceneCullingRenderer* SceneCullingRenderer = GetSceneExtensionsRenderers().GetRendererPtr<FSceneCullingRenderer>();

	for (int32 PassIndex = 0; PassIndex < EMeshPass::Num; PassIndex++)
	{
		const EMeshPass::Type PassType = (EMeshPass::Type)PassIndex;

		if ((FPassProcessorManager::GetPassFlags(ShadingPath, PassType) & EMeshPassFlags::MainView) != EMeshPassFlags::None)
		{
			// Mobile: BasePass and MobileBasePassCSM lists need to be merged and sorted after shadow pass.
			if (ShadingPath == EShadingPath::Mobile && (PassType == EMeshPass::BasePass || PassType == EMeshPass::MobileBasePassCSM))
			{
				continue;
			}

			if (ViewFamily.UseDebugViewPS() && ShadingPath == EShadingPath::Deferred)
			{
				switch (PassType)
				{
					case EMeshPass::DepthPass:
					case EMeshPass::CustomDepth:
					case EMeshPass::DebugViewMode:
#if WITH_EDITOR
					case EMeshPass::HitProxy:
					case EMeshPass::HitProxyOpaqueOnly:
					case EMeshPass::EditorSelection:
					case EMeshPass::EditorLevelInstance:
#endif
						break;
					default:
						continue;
				}
			}

			if (ViewCommands.MeshCommands[PassIndex].IsEmpty() && View.NumVisibleDynamicMeshElements[PassType] == 0 && ViewCommands.NumDynamicMeshCommandBuildRequestElements[PassType] == 0)
			{
				continue;
			}

			FMeshPassProcessor* MeshPassProcessor = FPassProcessorManager::CreateMeshPassProcessor(ShadingPath, PassType, Scene->GetFeatureLevel(), Scene, &View, nullptr);

			FParallelMeshDrawCommandPass& Pass = *View.CreateMeshPass(PassType);

			if (ShouldDumpMeshDrawCommandInstancingStats())
			{
				Pass.SetDumpInstancingStats(GetMeshPassName(PassType));
			}

			TArray<int32, TInlineAllocator<2> > ViewIds;
			ViewIds.Add(View.SceneRendererPrimaryViewId);
			// Only apply instancing for ISR to main view passes
			const bool bIsMainViewPass = (FPassProcessorManager::GetPassFlags(ShadingPath, PassType) & EMeshPassFlags::MainView) != EMeshPassFlags::None;

			EInstanceCullingMode InstanceCullingMode = bIsMainViewPass && View.IsInstancedStereo() ? EInstanceCullingMode::Stereo : EInstanceCullingMode::Normal;
			if (InstanceCullingMode == EInstanceCullingMode::Stereo)
			{
				check(View.GetInstancedView() != nullptr);
				ViewIds.Add(View.GetInstancedView()->SceneRendererPrimaryViewId);
			}
			
			EInstanceCullingFlags CullingFlags = EInstanceCullingFlags::None;

#if WITH_EDITOR
			if (PassType == EMeshPass::EditorSelection)
			{
				CullingFlags |= EInstanceCullingFlags::DrawOnlySelected;
			}
#endif

			// TODO: Maybe this should be configured somewhere else?
			const bool bAllowInstanceOcclusionCulling = PassType != EMeshPass::CustomDepth;

			FSceneCullingRendererHierarchicalCullingQuery* HierarchicalCullingQuery = SceneCullingRenderer ? SceneCullingRenderer->GetPrimaryViewHierarchicalCullingQuery(View.SceneRendererPrimaryViewId) : nullptr;
			Pass.DispatchPassSetup(
				Scene,
				View,
				FInstanceCullingContext(GetMeshPassName(PassType), ShaderPlatform, &InstanceCullingManager, ViewIds, bAllowInstanceOcclusionCulling ? View.PrevViewInfo.HZB : nullptr, InstanceCullingMode, CullingFlags, EBatchProcessingMode::UnCulled, HierarchicalCullingQuery),
				PassType,
				BasePassDepthStencilAccess,
				MeshPassProcessor,
				View.DynamicMeshElements,
				&View.DynamicMeshElementsPassRelevance,
				View.NumVisibleDynamicMeshElements[PassType],
				ViewCommands.DynamicMeshCommandBuildRequests[PassType],
				ViewCommands.DynamicMeshCommandBuildFlags[PassType],
				ViewCommands.NumDynamicMeshCommandBuildRequestElements[PassType],
				ViewCommands.MeshCommands[PassIndex]);
		}
	}
}

bool FSceneRenderer::ShouldCompositeEditorPrimitives(const FViewInfo& View)
{
	const FEngineShowFlags& ShowFlags = View.Family->EngineShowFlags;
	if ((ShowFlags.VisualizeHDR && !ShowFlags.Wireframe) ||				// VisualizeHDR is skipped with in Wireframe mode.
		ShowFlags.VisualizeSkyLightIlluminance ||
		ShowFlags.VisualizePostProcessStack || View.Family->UseDebugViewPS())
	{
		// certain visualize modes get obstructed too much
		return false;
	}

	if (ShowFlags.Wireframe || ShowFlags.MeshEdges)
	{
		// Wireframe is drawn to EditorPrimitives buffer because it uses MSAA, and so it requires the composition step
		return true;
	}
	else if (ShowFlags.CompositeEditorPrimitives)
	{
	    // Any elements that needed compositing were drawn then compositing should be done
	    if (View.ViewMeshElements.Num() 
		    || View.TopViewMeshElements.Num() 
		    || View.BatchedViewElements.HasPrimsToDraw() 
		    || View.TopBatchedViewElements.HasPrimsToDraw() 
		    || View.NumVisibleDynamicEditorPrimitives > 0
			|| IsMobileColorsRGB())
	    {
		    return true;
	    }
	}

	return false;
}

void FSceneRenderer::UpdatePrimitiveIndirectLightingCacheBuffers(FRHICommandListBase& RHICmdList)
{
	// Use a bit array to prevent primitives from being updated more than once.
	FSceneBitArray UpdatedPrimitiveMap;
	UpdatedPrimitiveMap.Init(false, Scene->Primitives.Num());

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{		
		FViewInfo& View = Views[ViewIndex];

		for (int32 Index = 0; Index < View.DirtyIndirectLightingCacheBufferPrimitives.Num(); ++Index)
		{
			FPrimitiveSceneInfo* PrimitiveSceneInfo = View.DirtyIndirectLightingCacheBufferPrimitives[Index];

			FBitReference bInserted = UpdatedPrimitiveMap[PrimitiveSceneInfo->GetIndex()];
			if (!bInserted)
			{
				PrimitiveSceneInfo->UpdateIndirectLightingCacheBuffer(RHICmdList);
				bInserted = true;
			}
			else
			{
				// This will prevent clearing it twice.
				View.DirtyIndirectLightingCacheBufferPrimitives[Index] = nullptr;
			}
		}
	}

	const uint32 CurrentSceneFrameNumber = Scene->GetFrameNumber();

	// Trim old CPUInterpolationCache entries occasionally
	if (CurrentSceneFrameNumber % 10 == 0)
	{
		for (TMap<FVector, FVolumetricLightmapInterpolation>::TIterator It(Scene->VolumetricLightmapSceneData.CPUInterpolationCache); It; ++It)
		{
			FVolumetricLightmapInterpolation& Interpolation = It.Value();

			if (Interpolation.LastUsedSceneFrameNumber < CurrentSceneFrameNumber - 100)
			{
				It.RemoveCurrent();
			}
		}
	}
}

/*-----------------------------------------------------------------------------
	FRendererModule
-----------------------------------------------------------------------------*/

TRDGUniformBufferRef<FSceneUniformParameters> GetSceneUniformBufferRef(FRDGBuilder& GraphBuilder, const FSceneView& View)
{
	ISceneRenderer* SceneRender = View.Family ? View.Family->GetSceneRenderer() : nullptr;
	return SceneRender ? SceneRender->GetSceneUniformBufferRef(GraphBuilder) : nullptr;
}

UE::Renderer::Private::IShadowInvalidatingInstances *FSceneRenderer::GetShadowInvalidatingInstancesInterface(const FSceneView *SceneView)
{
	checkf(IsInRenderingThread(), TEXT("Accessing the ShadowInvalidatingInstancesInterface should only be allowed from the rendering thread!"));
	if (FShadowSceneRenderer* ShadowSceneRenderer = GetSceneExtensionsRenderers().GetRendererPtr<FShadowSceneRenderer>())
	{
		return ShadowSceneRenderer->GetInvalidatingInstancesInterface(SceneView);
	}
	return nullptr;
}

void ResetAndShrinkModifiedBounds(TArray<FBox>& Bounds)
{
	const int32 MaxAllocatedSize = FMath::RoundUpToPowerOfTwo(FMath::Max<uint32>(DistanceField::MinPrimitiveModifiedBoundsAllocation, Bounds.Num()));

	if (Bounds.Max() > MaxAllocatedSize)
	{
		Bounds.Empty(MaxAllocatedSize);
	}

	Bounds.Reset();
}

static void RenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneRenderer* Renderer, const FSceneRenderUpdateInputs* SceneUpdateInputs)
{
	FSceneViewFamily& ViewFamily = Renderer->ViewFamily;

	LLM_SCOPE(ELLMTag::SceneRender);
	SCOPE_CYCLE_COUNTER(STAT_TotalSceneRenderingTime);
	SCOPED_NAMED_EVENT_TCHAR_CONDITIONAL(*ViewFamily.ProfileDescription, FColor::Red, !ViewFamily.ProfileDescription.IsEmpty());

	if (ViewFamily.EngineShowFlags.HitProxies)
	{
		Renderer->RenderHitProxies(GraphBuilder, SceneUpdateInputs);
	}
	else
	{
		Renderer->Render(GraphBuilder, SceneUpdateInputs);
	}

	Renderer->FlushCrossGPUFences(GraphBuilder);
}

static void CleanupViewFamilies_RenderThread(FRHICommandListImmediate& RHICmdList, TConstArrayView<FSceneRenderer*> SceneRenderers)
{
	LLM_SCOPE(ELLMTag::SceneRender);

	FScene* Scene = SceneRenderers[0]->Scene;

#if MESH_DRAW_COMMAND_STATS
	if (FMeshDrawCommandStatsManager* Instance = FMeshDrawCommandStatsManager::Get())
	{
		Instance->QueueCustomDrawIndirectArgsReadback(RHICmdList);
	}
#endif

	{
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(PostRenderCleanUp);

		if (IsHairStrandsEnabled(EHairStrandsShaderType::All, Scene->GetShaderPlatform()) && (SceneRenderers[0]->AllViews.Num() > 0))
		{
			FHairStrandsBookmarkParameters Parameters;
			CreateHairStrandsBookmarkParameters(Scene, SceneRenderers[0]->Views, SceneRenderers[0]->AllViews, Parameters, false /*bComputeVisibleInstances*/);
			if (Parameters.HasInstances())
			{
				RunHairStrandsBookmark(EHairStrandsBookmark::ProcessEndOfFrame, Parameters);
			}
		}

		// Only reset per-frame scene state once all views have processed their frame, including those in planar reflections
		for (int32 CacheType = 0; CacheType < UE_ARRAY_COUNT(Scene->DistanceFieldSceneData.PrimitiveModifiedBounds); CacheType++)
		{
			ResetAndShrinkModifiedBounds(Scene->DistanceFieldSceneData.PrimitiveModifiedBounds[CacheType]);
		}

		// Immediately issue EndFrame() for all extensions in case any of the outstanding tasks they issued getting out of this frame
		extern TSet<IPersistentViewUniformBufferExtension*> PersistentViewUniformBufferExtensions;

		for (IPersistentViewUniformBufferExtension* Extension : PersistentViewUniformBufferExtensions)
		{
			Extension->EndFrame();
		}
	}

#if RHI_RAYTRACING
	Scene->RayTracingScene.EndFrame();
	Scene->RayTracingSBT.EndFrame();
	Nanite::GRayTracingManager.EndFrame();
#endif

	// Update scene memory stats that couldn't be tracked continuously
	SET_MEMORY_STAT(STAT_RenderingSceneMemory, Scene->GetSizeBytes());

	SIZE_T ViewStateMemory = 0;
	for (FSceneRenderer* SceneRenderer : SceneRenderers)
	{
		for (FViewInfo& View : SceneRenderer->Views)
		{
#if WITH_EDITOR
			if (View.ViewportFeedback)
			{
				View.ViewportFeedback->EndFrameRenderThread();
			}
#endif

			// Copy relevant data from ViewInfo to ViewState->PrevFrameViewInfo
			if (View.ViewState)
			{
				View.ViewState->PrevFrameViewInfo.bUsesGlobalDistanceField = View.bUsesGlobalDistanceField;

			#if STATS
				ViewStateMemory += View.ViewState->GetSizeBytes();
			#endif
			}
		}
	}
	SET_MEMORY_STAT(STAT_ViewStateMemory, ViewStateMemory);
	SET_MEMORY_STAT(STAT_LightInteractionMemory, FLightPrimitiveInteraction::GetMemoryPoolSize());

#if STATS
	if (FPlatformProperties::SupportsWindowedMode() == false)
	{
		/** Update STATS with the total GPU time taken to render the last frame. */
		SET_CYCLE_COUNTER(STAT_TotalGPUFrameTime, RHIGetGPUFrameCycles());
	}
#endif

#if !UE_BUILD_SHIPPING
	// Update on screen notifications.
	FRendererOnScreenNotification::Get().Broadcast();
#endif
}

void OnChangeCVarRequiringRecreateRenderState(IConsoleVariable* Var)
{
	// Propgate cvar change to static draw lists
	FGlobalComponentRecreateRenderStateContext Context;
}

FRendererModule::FRendererModule()
{
	static auto EarlyZPassVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.EarlyZPass"));
	EarlyZPassVar->SetOnChangedCallback(FConsoleVariableDelegate::CreateStatic(&OnChangeCVarRequiringRecreateRenderState));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	void InitDebugViewModeInterface();
	InitDebugViewModeInterface();
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

void FRendererModule::CreateAndInitSingleView(FRHICommandListImmediate& RHICmdList, class FSceneViewFamily* ViewFamily, const struct FSceneViewInitOptions* ViewInitOptions)
{
	// Create and add the new view
	FViewInfo* NewView = new FViewInfo(*ViewInitOptions);
	ViewFamily->Views.Add(NewView);
	FViewInfo* View = (FViewInfo*)ViewFamily->Views[0];
	View->ViewRect = View->UnscaledViewRect;
	View->InitRHIResources();
}

extern CORE_API bool GRenderThreadPollingOn;

void FRendererModule::BeginRenderingViewFamily(FCanvas* Canvas, FSceneViewFamily* ViewFamily)
{
	BeginRenderingViewFamilies(Canvas, MakeArrayView({ ViewFamily }));
}

void FRendererModule::BeginRenderingViewFamilies(FCanvas* Canvas, TConstArrayView<FSceneViewFamily*> ViewFamilies)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BeginRenderingViewFamily);
	check(Canvas);
	for (FSceneViewFamily* ViewFamily : ViewFamilies)
	{
		check(ViewFamily);
		check(ViewFamily->Scene == ViewFamilies[0]->Scene);
	}

	UWorld* World = nullptr;

	FScene* const Scene = ViewFamilies[0]->Scene ? ViewFamilies[0]->Scene->GetRenderScene() : nullptr;
	if (Scene)
	{
		World = Scene->GetWorld();
		if (World)
		{
			UE::Stats::FThreadIdleStats::BeginCriticalPath();
			{
				// Guarantee that all render proxies are up to date before kicking off a BeginRenderViewFamily.
				World->SendAllEndOfFrameUpdates();
			}
			UE::Stats::FThreadIdleStats::EndCriticalPath();

			GetNaniteVisualizationData().Pick(World);

		#if WITH_STATE_STREAM
			ENQUEUE_RENDER_COMMAND(UpdateStateStream)([StateStreamManager = static_cast<FStateStreamManagerImpl*>(World->GetStateStreamManager()), RealTimeSeconds = World->RealTimeSeconds] (FRHICommandListImmediate&)
			{
				StateStreamManager->Render_Update(RealTimeSeconds);
				StateStreamManager->Render_GarbageCollect(true);
			});
		#endif
		}
	}

	ENQUEUE_RENDER_COMMAND(SetRtWaitCriticalPath)(
		[](FRHICommandList& RHICmdList)
		{
			// Rendering is up and running now, so waits are considered part of the RT critical path
			UE::Stats::FThreadIdleStats::BeginCriticalPath();
		});

	FUniformExpressionCacheAsyncUpdateScope AsyncUpdateScope;

	ENQUEUE_RENDER_COMMAND(UpdateFastVRamConfig)(
		[](FRHICommandList& RHICmdList)
		{
			GFastVRamConfig.Update();
		});

	UE::RenderCommandPipe::FSyncScope SyncScope;

	// Flush the canvas first.
	Canvas->Flush_GameThread();

	if (Scene)
	{
		// We allow caching of per-frame, per-scene data
		if (ViewFamilies[0]->bIsFirstViewInMultipleViewFamily)
		{
			Scene->IncrementFrameNumber();
		}
		for (FSceneViewFamily* ViewFamily : ViewFamilies)
		{
			ViewFamily->FrameNumber = Scene->GetFrameNumber();
		}
	}
	else
	{
		// this is passes to the render thread, better access that than GFrameNumberRenderThread
		for (FSceneViewFamily* ViewFamily : ViewFamilies)
		{
			ViewFamily->FrameNumber = GFrameNumber;
		}
	}

	// Add streaming view origins
	const uint32 StreamingViewCount = IStreamingManager::Get().GetNumViews();
	for (FSceneViewFamily* ViewFamily : ViewFamilies)
	{
		ViewFamily->StreamingViewOrigins.Empty(StreamingViewCount);
		for (uint32 StreamingViewIndex = 0; StreamingViewIndex < StreamingViewCount; ++StreamingViewIndex)
		{
			ViewFamily->StreamingViewOrigins.Add(IStreamingManager::Get().GetViewInformation(StreamingViewIndex).ViewOrigin);
		}
	}

	for (FSceneViewFamily* ViewFamily : ViewFamilies)
	{
		ViewFamily->FrameCounter = GFrameCounter;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		{
			extern TSharedRef<ISceneViewExtension, ESPMode::ThreadSafe> GetRendererViewExtension();

			ViewFamily->ViewExtensions.Add(GetRendererViewExtension());
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

		// Force the upscalers to be set no earlier than ISceneViewExtension::BeginRenderViewFamily();
		check(ViewFamily->GetTemporalUpscalerInterface() == nullptr);
		check(ViewFamily->GetPrimarySpatialUpscalerInterface() == nullptr);
		check(ViewFamily->GetSecondarySpatialUpscalerInterface() == nullptr);
		checkf(!(ViewFamily->GetTemporalUpscalerInterface() != nullptr && ViewFamily->GetPrimarySpatialUpscalerInterface() != nullptr),
			TEXT("Conflict setting up a third party primary spatial upscaler or temporal upscaler."));
	}

	if (Scene)
	{
		// Set the world's "needs full lighting rebuild" flag if the scene has any uncached static lighting interactions.
		if (World)
		{
			// Note: reading NumUncachedStaticLightingInteractions on the game thread here which is written to by the rendering thread
			// This is reliable because the RT uses interlocked mechanisms to update it
			World->SetMapNeedsLightingFullyRebuilt(Scene->NumUncachedStaticLightingInteractions, Scene->NumUnbuiltReflectionCaptures);
		}

	#if CSV_PROFILER && !CSV_PROFILER_MINIMAL
		ENQUEUE_RENDER_COMMAND(SetDrawSceneCommand_StartDelay)([DrawSceneEnqueue = FPlatformTime::Cycles64()] (FRHICommandListImmediate&)
		{
			const uint64 SceneRenderStart = FPlatformTime::Cycles64();
			const float StartDelayMillisec = FPlatformTime::ToMilliseconds64(SceneRenderStart - DrawSceneEnqueue);
			CSV_CUSTOM_STAT_GLOBAL(DrawSceneCommand_StartDelay, StartDelayMillisec, ECsvCustomStatOp::Set);
		});
	#endif

		FSceneRenderBuilder SceneRenderBuilder(Scene);

		// Update deferred scene captures before creating the main view scene renderers, so custom render passes are available during scene renderer construction
		bool bShowHitProxies = (Canvas->GetHitProxyConsumer() != nullptr);
		if (!bShowHitProxies)
		{
			SceneCaptureUpdateDeferredCapturesInternal(Scene, ViewFamilies, SceneRenderBuilder);
		}

		TArray<FSceneRenderer*, FConcurrentLinearArrayAllocator> SceneRenderers = SceneRenderBuilder.CreateLinkedSceneRenderers(ViewFamilies, Canvas->GetHitProxyConsumer());
		SetupDebugViewModes(SceneRenderers);

		if (!bShowHitProxies)
		{
			for (int32 ReflectionIndex = 0; ReflectionIndex < Scene->PlanarReflections_GameThread.Num(); ReflectionIndex++)
			{
				UPlanarReflectionComponent* ReflectionComponent = Scene->PlanarReflections_GameThread[ReflectionIndex];
				for (FSceneRenderer* SceneRenderer : SceneRenderers)
				{
					if (HasRayTracedOverlay(SceneRenderer->ViewFamily))
					{
						continue;
					}
					Scene->UpdatePlanarReflectionContents(ReflectionComponent, *SceneRenderer, SceneRenderBuilder);
				}
			}
		}

		FSceneRenderer::PreallocateCrossGPUFences(SceneRenderers);

		// Flush if the current show flags can't be merged with the current set renderers already added.
		SceneRenderBuilder.FlushIfIncompatible(ViewFamilies[0]->EngineShowFlags);

		for (FSceneRenderer* SceneRenderer : SceneRenderers)
		{
			SceneRenderer->ViewFamily.DisplayInternalsData.Setup(World);

			SceneRenderBuilder.AddRenderer(SceneRenderer, bShowHitProxies ? TEXT("HitProxies") : TEXT("ViewFamilies"),
				[] (FRDGBuilder& GraphBuilder, const FSceneRenderFunctionInputs& Inputs)
			{
				RenderViewFamily_RenderThread(GraphBuilder, Inputs.Renderer, Inputs.SceneUpdateInputs);
				return true;
			});
		}

		SceneRenderBuilder.AddRenderCommand([SceneRenderers = MoveTemp(SceneRenderers)](FRHICommandListImmediate& RHICmdList)
		{
			CleanupViewFamilies_RenderThread(RHICmdList, SceneRenderers);
		});

		SceneRenderBuilder.Execute();

		// Force kick the RT if we've got RT polling on.
		// This saves us having to wait until the polling period before the scene draw starts executing.
		if (GRenderThreadPollingOn)
		{
			FTaskGraphInterface::Get().WakeNamedThread(ENamedThreads::GetRenderThread());
		}
	}
}

void FRendererModule::PostRenderAllViewports()
{
	// Increment FrameNumber before render the scene. Wrapping around is no problem.
	// This is the only spot we change GFrameNumber, other places can only read.
	++GFrameNumber;

#if RHI_RAYTRACING
	UE::RenderCommandPipe::FSyncScope SyncScope;

	ENQUEUE_RENDER_COMMAND(PostRenderAllViewports_RenderThread)(
 		[](FRHICommandListImmediate& RHICmdList)
 		{
 			GRayTracingGeometryManager->Tick(RHICmdList);
 		});
#endif //#if RHI_RAYTRACING
}

void FRendererModule::PerFrameCleanupIfSkipRenderer()
{
	UE::RenderCommandPipe::FSyncScope SyncScope;

	// Some systems (e.g. Slate) can still draw (via FRendererModule::DrawTileMesh for example) when scene renderer is not used
	ENQUEUE_RENDER_COMMAND(CmdPerFrameCleanupIfSkipRenderer)(
		[](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
		PipelineStateCache::FlushResources();
		FSceneRenderBuilder::WaitForAsyncDeleteTask();
		GPrimitiveIdVertexBufferPool.DiscardAll();
	});
}

void FRendererModule::UpdateMapNeedsLightingFullyRebuiltState(UWorld* World)
{
	World->SetMapNeedsLightingFullyRebuilt(World->Scene->GetRenderScene()->NumUncachedStaticLightingInteractions, World->Scene->GetRenderScene()->NumUnbuiltReflectionCaptures);
}

void FRendererModule::DrawRectangle(
		FRHICommandList& RHICmdList,
		float X,
		float Y,
		float SizeX,
		float SizeY,
		float U,
		float V,
		float SizeU,
		float SizeV,
		FIntPoint TargetSize,
		FIntPoint TextureSize,
		const TShaderRef<FShader>& VertexShader,
		EDrawRectangleFlags Flags
		)
{
	::DrawRectangle( RHICmdList, X, Y, SizeX, SizeY, U, V, SizeU, SizeV, TargetSize, TextureSize, VertexShader, Flags );
}

FDelegateHandle FRendererModule::RegisterPostOpaqueRenderDelegate(const FPostOpaqueRenderDelegate& InPostOpaqueRenderDelegate)
{
	return PostOpaqueRenderDelegate.Add(InPostOpaqueRenderDelegate);
}

void FRendererModule::RemovePostOpaqueRenderDelegate(FDelegateHandle InPostOpaqueRenderDelegate)
{
	PostOpaqueRenderDelegate.Remove(InPostOpaqueRenderDelegate);
}

FDelegateHandle FRendererModule::RegisterOverlayRenderDelegate(const FPostOpaqueRenderDelegate& InOverlayRenderDelegate)
{
	return OverlayRenderDelegate.Add(InOverlayRenderDelegate);
}

void FRendererModule::RemoveOverlayRenderDelegate(FDelegateHandle InOverlayRenderDelegate)
{
	OverlayRenderDelegate.Remove(InOverlayRenderDelegate);
}

void FRendererModule::RenderPostOpaqueExtensions(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FSceneTextures& SceneTextures)
{
	if (PostOpaqueRenderDelegate.IsBound())
	{
		RDG_EVENT_SCOPE(GraphBuilder, "PostOpaqueExtensions");

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			check(IsInRenderingThread());
			FPostOpaqueRenderParameters RenderParameters;
			RenderParameters.ViewMatrix = View.ViewMatrices.GetWorldToView();
			RenderParameters.ProjMatrix = View.ViewMatrices.GetViewToClip();
			RenderParameters.ColorTexture = SceneTextures.Color.Target;
			RenderParameters.DepthTexture = SceneTextures.Depth.Target;
			RenderParameters.NormalTexture = SceneTextures.GBufferA;
			RenderParameters.VelocityTexture = SceneTextures.Velocity;
			RenderParameters.SmallDepthTexture = SceneTextures.SmallDepth;
			RenderParameters.ViewUniformBuffer = View.ViewUniformBuffer;
			RenderParameters.SceneTexturesUniformParams = SceneTextures.UniformBuffer;
			RenderParameters.MobileSceneTexturesUniformParams = SceneTextures.MobileUniformBuffer;
			RenderParameters.GlobalDistanceFieldParams = &View.GlobalDistanceFieldInfo.ParameterData;

			RenderParameters.ViewportRect = View.ViewRect;
			RenderParameters.GraphBuilder = &GraphBuilder;

			RenderParameters.Uid = (void*)(&View);
			RenderParameters.View = &View;
			PostOpaqueRenderDelegate.Broadcast(RenderParameters);
		}
	}
}

void FRendererModule::RenderOverlayExtensions(
	FRDGBuilder& GraphBuilder,
	TArrayView<const FViewInfo> Views,
	const FSceneTextures& SceneTextures)
{
	if (OverlayRenderDelegate.IsBound())
	{
		RDG_EVENT_SCOPE(GraphBuilder, "OverlayExtensions");

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ++ViewIndex)
		{
			const FViewInfo& View = Views[ViewIndex];
			RDG_EVENT_SCOPE_CONDITIONAL(GraphBuilder, Views.Num() > 1, "View%d", ViewIndex);
			RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

			FPostOpaqueRenderParameters RenderParameters;
			RenderParameters.ViewMatrix = View.ViewMatrices.GetWorldToView();
			RenderParameters.ProjMatrix = View.ViewMatrices.GetViewToClip();
			RenderParameters.ColorTexture = SceneTextures.Color.Target;
			RenderParameters.DepthTexture = SceneTextures.Depth.Target;
			RenderParameters.SmallDepthTexture = SceneTextures.SmallDepth;

			RenderParameters.ViewportRect = View.ViewRect;
			RenderParameters.GraphBuilder = &GraphBuilder;

			RenderParameters.Uid = (void*)(&View);
			RenderParameters.View = &View;
			OverlayRenderDelegate.Broadcast(RenderParameters);
		}
	}
}

void FRendererModule::RenderPostResolvedSceneColorExtension(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures)
{
	if (PostResolvedSceneColorCallbacks.IsBound())
	{
		PostResolvedSceneColorCallbacks.Broadcast(GraphBuilder, SceneTextures);
	}
}

class FScenePrimitiveRenderingContext : public IScenePrimitiveRenderingContext
{
public:
	FScenePrimitiveRenderingContext(FRDGBuilder& InGraphBuilder, FScene& Scene, FSceneViewFamily* InViewFamily = nullptr, EBeginScenePrimitiveRenderingFlags BeginScenePrimitiveRenderingFlags = EBeginScenePrimitiveRenderingFlags::All) :
		GraphBuilder(InGraphBuilder),
		GPUScene(Scene.GPUScene),
		GPUSceneDynamicContext(GPUScene),
		ViewFamily(InViewFamily)
	{
		Renderer.Scene = &Scene;
		Renderer.InitSceneExtensionsRenderers(ViewFamily ? ViewFamily->EngineShowFlags : FEngineShowFlags(ESFIM_Game));
		if (ViewFamily)
		{
			ViewFamily->SetSceneRenderer(&Renderer);
		}

		if (EnumHasAllFlags(BeginScenePrimitiveRenderingFlags, EBeginScenePrimitiveRenderingFlags::UpdateScenePrimitives))
		{
			Scene.UpdateAllPrimitiveSceneInfos(GraphBuilder);
		}
		
		GPUScene.BeginRender(GraphBuilder, GPUSceneDynamicContext);

		GPUScene.FillSceneUniformBuffer(GraphBuilder, Renderer.GetSceneUniforms());
		Renderer.GetSceneExtensionsRenderers().UpdateSceneUniformBuffer(GraphBuilder, Renderer.GetSceneUniforms());

		FSceneRendererBase::SetActiveInstance(GraphBuilder, &Renderer);
	}

	virtual ~FScenePrimitiveRenderingContext()
	{
		FSceneRendererBase::SetActiveInstance(GraphBuilder, nullptr);
		GPUScene.EndRender();
		if (ViewFamily)
		{
			ViewFamily->SetSceneRenderer(nullptr);
		}
	}

	virtual ISceneRenderer* GetSceneRenderer()
	{
		return &Renderer;
	}

	FRDGBuilder& GraphBuilder;
	FSceneRendererBase Renderer;
	FGPUScene& GPUScene;
	FGPUSceneDynamicContext GPUSceneDynamicContext;
	FSceneViewFamily* ViewFamily;
};


IScenePrimitiveRenderingContext* FRendererModule::BeginScenePrimitiveRendering(FRDGBuilder &GraphBuilder, FSceneViewFamily* ViewFamily, EBeginScenePrimitiveRenderingFlags BeginScenePrimitiveRenderingFlags)
{
	check(ViewFamily);
	check(ViewFamily->Scene);
	FScene* Scene = ViewFamily->Scene->GetRenderScene();
	check(Scene);

	FScenePrimitiveRenderingContext* ScenePrimitiveRenderingContext = new FScenePrimitiveRenderingContext(GraphBuilder, *Scene, ViewFamily, BeginScenePrimitiveRenderingFlags);

	return ScenePrimitiveRenderingContext;
}

IScenePrimitiveRenderingContext* FRendererModule::BeginScenePrimitiveRendering(FRDGBuilder &GraphBuilder, FSceneInterface& InScene, EBeginScenePrimitiveRenderingFlags BeginScenePrimitiveRenderingFlags)
{
	FScene* Scene = InScene.GetRenderScene();
	check(Scene);

	FScenePrimitiveRenderingContext* ScenePrimitiveRenderingContext = new FScenePrimitiveRenderingContext(GraphBuilder, *Scene, nullptr, BeginScenePrimitiveRenderingFlags);

	return ScenePrimitiveRenderingContext;
}

IMaterialCacheTagProvider* FRendererModule::GetMaterialCacheTagProvider()
{
	return &FMaterialCacheTagProvider::Get();
}

IMaterialCacheVirtualTextureAllocator* FRendererModule::GetMaterialCacheVirtualTextureAllocator()
{
	return &FMaterialCacheVirtualTextureAllocator::Get();
}

IAllocatedVirtualTexture* FRendererModule::AllocateVirtualTexture(FRHICommandListBase& RHICmdList, const FAllocatedVTDescription& Desc)
{
	return FVirtualTextureSystem::Get().AllocateVirtualTexture(RHICmdList, Desc);
}

void FRendererModule::DestroyVirtualTexture(IAllocatedVirtualTexture* AllocatedVT)
{
	FVirtualTextureSystem::Get().DestroyVirtualTexture(AllocatedVT);
}

IAdaptiveVirtualTexture* FRendererModule::AllocateAdaptiveVirtualTexture(FRHICommandListBase& RHICmdList, const FAdaptiveVTDescription& AdaptiveVTDesc, const FAllocatedVTDescription& AllocatedVTDesc)
{
	return FVirtualTextureSystem::Get().AllocateAdaptiveVirtualTexture(RHICmdList, AdaptiveVTDesc, AllocatedVTDesc);
}

void FRendererModule::DestroyAdaptiveVirtualTexture(IAdaptiveVirtualTexture* AdaptiveVT)
{
	FVirtualTextureSystem::Get().DestroyAdaptiveVirtualTexture(AdaptiveVT);
}

FVirtualTextureProducerHandle FRendererModule::RegisterVirtualTextureProducer(FRHICommandListBase& RHICmdList, const FVTProducerDescription& Desc, IVirtualTexture* Producer)
{
	return FVirtualTextureSystem::Get().RegisterProducer(RHICmdList, Desc, Producer);
}

void FRendererModule::ReleaseVirtualTextureProducer(const FVirtualTextureProducerHandle& Handle)
{
	FVirtualTextureSystem::Get().ReleaseProducer(Handle);
}

void FRendererModule::ReleaseVirtualTexturePendingResources()
{
	FVirtualTextureSystem::Get().ReleasePendingResources();
}

void FRendererModule::AddVirtualTextureProducerDestroyedCallback(const FVirtualTextureProducerHandle& Handle, FVTProducerDestroyedFunction* Function, void* Baton)
{
	FVirtualTextureSystem::Get().AddProducerDestroyedCallback(Handle, Function, Baton);
}

uint32 FRendererModule::RemoveAllVirtualTextureProducerDestroyedCallbacks(const void* Baton)
{
	return FVirtualTextureSystem::Get().RemoveAllProducerDestroyedCallbacks(Baton);
}

void FRendererModule::RequestVirtualTextureTiles(const FVector2D& InScreenSpaceSize, int32 InMipLevel)
{
	FVirtualTextureSystem::Get().RequestTiles(InScreenSpaceSize, InMipLevel);
}

void FRendererModule::RequestVirtualTextureTiles(const FMaterialRenderProxy* InMaterialRenderProxy, const FVector2D& InScreenSpaceSize, ERHIFeatureLevel::Type InFeatureLevel)
{
	FVirtualTextureSystem::Get().RequestTiles(InMaterialRenderProxy, InScreenSpaceSize, InFeatureLevel);
}

void FRendererModule::RequestVirtualTextureTiles(IAllocatedVirtualTexture* AllocatedVT, const FVector2D& InScreenSpaceSize, const FVector2D& InViewportPosition, const FVector2D& InViewportSize, const FVector2D& InUV0, const FVector2D& InUV1, int32 InMipLevel)
{
	FVirtualTextureSystem::Get().RequestTiles(AllocatedVT, InScreenSpaceSize, InViewportPosition, InViewportSize, InUV0, InUV1, InMipLevel);
}

IVirtualTexture* FRendererModule::FindProducer(const FVirtualTextureProducerHandle& Handle)
{
	if (FVirtualTextureProducer* Producer = FVirtualTextureSystem::Get().FindProducer(Handle))
	{
		return Producer->GetVirtualTexture();
	}

	return nullptr;
}

void FRendererModule::LoadPendingVirtualTextureTiles(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel)
{
	FRDGBuilder GraphBuilder(RHICmdList);
	FVirtualTextureSystem::Get().LoadPendingTiles(GraphBuilder, FeatureLevel);
	GraphBuilder.Execute();
}

void FRendererModule::LockVirtualTextureTiles(FVirtualTextureProducerHandle ProducerHandle, int32 InMipLevel)
{
	FVirtualTextureSystem::Get().SetMipLevelToLock(ProducerHandle, InMipLevel);
}

void FRendererModule::SetVirtualTextureRequestRecordBuffer(uint64 Handle)
{
#if WITH_EDITOR
	FVirtualTextureSystem::Get().SetRequestRecordBuffer(Handle);
#endif
}

uint64 FRendererModule::GetVirtualTextureRequestRecordBuffer(TMap<uint64, uint32>& OutRequestData)
{
#if WITH_EDITOR
	return FVirtualTextureSystem::Get().GetRequestRecordBuffer(OutRequestData);
#else
	return (uint64)-1;
#endif
}

void FRendererModule::PackVirtualTextureRequestsToStream(TMap<uint64, uint32> const& InPageRequests, TArray<uint32>& OutRequestStream)
{
#if WITH_EDITOR
	FVirtualTextureSystem::PackRequestsToStream(InPageRequests, OutRequestStream);
#else
	OutRequestStream.Reset();
#endif
}

void FRendererModule::RequestVirtualTextureTiles(TArray<uint32>&& InRequestStream, FIntPoint const& InCaptureResolution)
{
	FVirtualTextureSystem::Get().RequestRecordedTiles(MoveTemp(InRequestStream), InCaptureResolution);
}

void FRendererModule::FlushVirtualTextureCache()
{
	FVirtualTextureSystem::Get().FlushCache();
}

void FRendererModule::FlushVirtualTextureCache(IAllocatedVirtualTexture* AllocatedVT, const FVector2f& InUV0, const FVector2f& InUV1)
{
	if (AllocatedVT != nullptr)
	{
		const uint32 NumLayers = AllocatedVT->GetNumTextureLayers();
		const uint32 SpaceID = AllocatedVT->GetSpaceID();
		const uint32 Width = AllocatedVT->GetBlockWidthInTiles() * AllocatedVT->GetVirtualTileSize();
		const uint32 Height = AllocatedVT->GetBlockHeightInTiles() * AllocatedVT->GetVirtualTileSize();
		const FIntPoint Texel0 = FIntPoint(FMath::FloorToInt32(InUV0.X * Width), FMath::FloorToInt32(InUV0.Y * Height));
		const FIntPoint Texel1 = FIntPoint(FMath::CeilToInt32(InUV1.X * Width), FMath::CeilToInt32(InUV1.Y * Height));
		const FIntRect TextureRect(Texel0, Texel1);
		const uint32 MaxLevel = AllocatedVT->GetMaxLevel();
		const uint32 MaxAgeToKeepMapped = VirtualTextureScalability::GetKeepDirtyPageMappedFrameThreshold();

		for (uint32 LayerIndex = 0; LayerIndex < NumLayers;  ++LayerIndex)
		{
			FVirtualTextureSystem::Get().FlushCache(AllocatedVT->GetProducerHandle(LayerIndex), SpaceID, TextureRect, MaxLevel, MaxAgeToKeepMapped, EVTInvalidatePriority::Normal);
		}
	}
}

void FRendererModule::SyncVirtualTextureUpdates(FRHICommandListImmediate& RHICmdList, ERHIFeatureLevel::Type FeatureLevel)
{
	FRDGBuilder GraphBuilder(RHICmdList);
	FVirtualTextureUpdateSettings Settings;
	Settings.EnableThrottling(false);
	FVirtualTextureSystem::Get().Update(GraphBuilder, FeatureLevel, nullptr, Settings);
	GraphBuilder.Execute();
}

uint64 FRendererModule::GetNaniteRequestRecordBuffer(TMap<uint64, uint32>& OutPageRequests)
{
#if WITH_EDITOR
	return Nanite::GStreamingManager.GetRequestRecordBuffer(OutPageRequests);
#else
	return (uint64)-1;
#endif
}

void FRendererModule::SetNaniteRequestRecordBuffer(uint64 Handle)
{
#if WITH_EDITOR
	Nanite::GStreamingManager.SetRequestRecordBuffer(Handle);
#endif
}

void FRendererModule::PackNaniteRequestsToStream(TMap<uint64, uint32> const& InPageRequests, TArray<uint32>& OutRequestStream)
{
#if WITH_EDITOR
	Nanite::FStreamingManager::PackRequestsToStream(InPageRequests, OutRequestStream);
#else
	OutRequestStream.Reset();
#endif 
}

void FRendererModule::RequestNanitePages(TArray<uint32>&& InRequestStream)
{
	Nanite::GStreamingManager.RequestNanitePages(MoveTemp(InRequestStream));
}

void FRendererModule::PrefetchNaniteResource(const Nanite::FResources* Resource, uint32 NumFramesUntilRender)
{
	Nanite::GStreamingManager.PrefetchResource(Resource, NumFramesUntilRender);
}

const FViewMatrices& FRendererModule::GetPreviousViewMatrices(const FSceneView& View)
{
	if (ensure(View.bIsViewInfo))
	{
		return static_cast<const FViewInfo&>(View).PrevViewInfo.ViewMatrices;
	}
	return View.ViewMatrices;
}

const FGlobalDistanceFieldParameterData* FRendererModule::GetGlobalDistanceFieldParameterData(const FSceneView& View)
{
	if (ensure(View.bIsViewInfo))
	{
		return &static_cast<const FViewInfo&>(View).GlobalDistanceFieldInfo.ParameterData;
	}
	return nullptr;
}

void FRendererModule::RequestStaticMeshUpdate(FPrimitiveSceneInfo* Info)
{
	if (Info)
	{
		Info->RequestStaticMeshUpdate();
	}
}

void FRendererModule::AddMeshBatchToGPUScene(FGPUScenePrimitiveCollector* Collector, FMeshBatch& MeshBatch)
{
	for (FMeshBatchElement& Element : MeshBatch.Elements)
	{
		if (const TUniformBuffer<FPrimitiveUniformShaderParameters>* PrimitiveUniformBufferResource = Element.PrimitiveUniformBufferResource)
		{
			Element.PrimitiveIdMode = PrimID_DynamicPrimitiveShaderData;
			Collector->Add(
				Element.DynamicPrimitiveData,
				*reinterpret_cast<const FPrimitiveUniformShaderParameters*>(PrimitiveUniformBufferResource->GetContents()),
				Element.NumInstances,
				Element.DynamicPrimitiveIndex,
				Element.DynamicPrimitiveInstanceSceneDataOffset);
		}
	}
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

class FConsoleVariableAutoCompleteVisitor 
{
public:
	// @param Name must not be 0
	// @param CVar must not be 0
	static void OnConsoleVariable(const TCHAR *Name, IConsoleObject* CObj, uint32* Crc)
	{
		IConsoleVariable* CVar = CObj->AsVariable();
		if(CVar)
		{
			if(CObj->TestFlags(ECVF_Scalability) || CObj->TestFlags(ECVF_ScalabilityGroup))
			{
				// float should work on int32 as well
				float Value = CVar->GetFloat();
				*Crc = FCrc::MemCrc32(&Value, sizeof(Value), *Crc);
			}
		}
	}
};
static uint32 ComputeScalabilityCVarHash()
{
	uint32 Ret = 0;

	IConsoleManager::Get().ForEachConsoleObjectThatStartsWith(FConsoleObjectVisitor::CreateStatic(&FConsoleVariableAutoCompleteVisitor::OnConsoleVariable, &Ret));

	return Ret;
}

static void DisplayInternals(FRDGBuilder& GraphBuilder, FViewInfo& InView)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	auto Family = InView.Family;
	// if r.DisplayInternals != 0
	if(Family->EngineShowFlags.OnScreenDebug && Family->DisplayInternalsData.IsValid())
	{
		FRDGTextureRef OutputTexture = GraphBuilder.FindExternalTexture(Family->RenderTarget->GetRenderTargetTexture());
		FScreenPassRenderTarget Output = FScreenPassRenderTarget::CreateViewFamilyOutput(OutputTexture, InView);
		AddDrawCanvasPass(GraphBuilder, RDG_EVENT_NAME("DisplayInternals"), InView, Output, [Family, &InView](FCanvas& Canvas)
		{
			// could be 0
			auto State = InView.ViewState;

			Canvas.SetRenderTargetRect(FIntRect(0, 0, Family->RenderTarget->GetSizeXY().X, Family->RenderTarget->GetSizeXY().Y));


			FRHIRenderPassInfo RenderPassInfo(Family->RenderTarget->GetRenderTargetTexture(), ERenderTargetActions::Load_Store);

			// further down to not intersect with "LIGHTING NEEDS TO BE REBUILT"
			FVector2D Pos(30, 140);
			const int32 FontSizeY = 14;

			// dark background
			const uint32 BackgroundHeight = 30;
			Canvas.DrawTile(Pos.X - 4, Pos.Y - 4, 500 + 8, FontSizeY * BackgroundHeight + 8, 0, 0, 1, 1, FLinearColor(0,0,0,0.6f), 0, true);

			UFont* Font = GEngine->GetSmallFont();
			FCanvasTextItem SmallTextItem( Pos, FText::GetEmpty(), GEngine->GetSmallFont(), FLinearColor::White );

			SmallTextItem.SetColor(FLinearColor::White);
			SmallTextItem.Text = FText::FromString(FString::Printf(TEXT("r.DisplayInternals = %d"), Family->DisplayInternalsData.DisplayInternalsCVarValue));
			Canvas.DrawItem(SmallTextItem, Pos);
			SmallTextItem.SetColor(FLinearColor::Gray);
			Pos.Y += 2 * FontSizeY;

			FViewInfo& ViewInfo = (FViewInfo&)InView;
	#define CANVAS_HEADER(txt) \
			{ \
				SmallTextItem.SetColor(FLinearColor::Gray); \
				SmallTextItem.Text = FText::FromString(txt); \
				Canvas.DrawItem(SmallTextItem, Pos); \
				Pos.Y += FontSizeY; \
			}
	#define CANVAS_LINE(bHighlight, txt, ... ) \
			{ \
				SmallTextItem.SetColor(bHighlight ? FLinearColor::Red : FLinearColor::Gray); \
				SmallTextItem.Text = FText::FromString(FString::Printf(txt, __VA_ARGS__)); \
				Canvas.DrawItem(SmallTextItem, Pos); \
				Pos.Y += FontSizeY; \
			}

			CANVAS_HEADER(TEXT("command line options:"))
			{
				bool bHighlight = !(FApp::UseFixedTimeStep() && FApp::bUseFixedSeed);
				CANVAS_LINE(bHighlight, TEXT("  -UseFixedTimeStep: %u"), FApp::UseFixedTimeStep())
				CANVAS_LINE(bHighlight, TEXT("  -FixedSeed: %u"), FApp::bUseFixedSeed)
				CANVAS_LINE(false, TEXT("  -gABC= (changelist): %d"), GetChangeListNumberForPerfTesting())
			}

			CANVAS_HEADER(TEXT("Global:"))
			CANVAS_LINE(false, TEXT("  FrameNumberRT: %u"), GFrameNumberRenderThread)
			CANVAS_LINE(false, TEXT("  Scalability CVar Hash: %x (use console command \"Scalability\")"), ComputeScalabilityCVarHash())
			//not really useful as it is non deterministic and should not be used for rendering features:  CANVAS_LINE(false, TEXT("  FrameNumberRT: %u"), GFrameNumberRenderThread)
			CANVAS_LINE(false, TEXT("  FrameCounter: %llu"), (uint64)GFrameCounter)
			CANVAS_LINE(false, TEXT("  rand()/SRand: %x/%x"), FMath::Rand(), FMath::GetRandSeed())
			{
				bool bHighlight = Family->DisplayInternalsData.NumPendingStreamingRequests != 0;
				CANVAS_LINE(bHighlight, TEXT("  FStreamAllResourcesLatentCommand: %d"), bHighlight)
			}
			{
				static auto* Var = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Streaming.FramesForFullUpdate"));
				int32 Value = Var->GetValueOnRenderThread();
				bool bHighlight = Value != 0;
				CANVAS_LINE(bHighlight, TEXT("  r.Streaming.FramesForFullUpdate: %u%s"), Value, bHighlight ? TEXT(" (should be 0)") : TEXT(""));
			}

			if(State)
			{
				CANVAS_HEADER(TEXT("State:"))
				CANVAS_LINE(false, TEXT("  TemporalAASample: %u"), State->GetCurrentTemporalAASampleIndex())
				CANVAS_LINE(false, TEXT("  FrameIndexMod8: %u"), State->GetFrameIndex(8))
				CANVAS_LINE(false, TEXT("  LODTransition: %.2f"), State->GetTemporalLODTransition())
			}

			CANVAS_HEADER(TEXT("Family:"))
			CANVAS_LINE(false, TEXT("  Time (Real/World/DeltaWorld): %.2f/%.2f/%.2f"), Family->Time.GetRealTimeSeconds(), Family->Time.GetWorldTimeSeconds(), Family->Time.GetDeltaWorldTimeSeconds())
			CANVAS_LINE(false, TEXT("  FrameNumber: %u"), Family->FrameNumber)
			CANVAS_LINE(false, TEXT("  ExposureSettings: %s"), *Family->ExposureSettings.ToString())

			CANVAS_HEADER(TEXT("View:"))
			CANVAS_LINE(false, TEXT("  TemporalJitter: %.2f/%.2f"), ViewInfo.TemporalJitterPixels.X, ViewInfo.TemporalJitterPixels.Y)
			CANVAS_LINE(false, TEXT("  ViewProjectionMatrix Hash: %x"), InView.ViewMatrices.GetWorldToClip().ComputeHash())
			CANVAS_LINE(false, TEXT("  ViewLocation: %s"), *InView.ViewLocation.ToString())
			CANVAS_LINE(false, TEXT("  ViewRotation: %s"), *InView.ViewRotation.ToString())
			CANVAS_LINE(false, TEXT("  ViewRect: %s"), *ViewInfo.ViewRect.ToString())

			CANVAS_LINE(false, TEXT("  DynMeshElements/TranslPrim: %d/%d"), ViewInfo.DynamicMeshElements.Num(), ViewInfo.TranslucentPrimCount.NumPrims())

	#undef CANVAS_LINE
	#undef CANVAS_HEADER
		});
	}
#endif

}

TSharedRef<ISceneViewExtension, ESPMode::ThreadSafe> GetRendererViewExtension()
{
	class FRendererViewExtension : public ISceneViewExtension
	{
	public:
		virtual ~FRendererViewExtension() = default;
		virtual void PostRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
		{
			FViewInfo& View = static_cast<FViewInfo&>(InView);
			DisplayInternals(GraphBuilder, View);
		}
	};
	TSharedRef<FRendererViewExtension, ESPMode::ThreadSafe> ref(new FRendererViewExtension);
	return StaticCastSharedRef<ISceneViewExtension>(ref);
}

#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

void FSceneRenderer::SetStereoViewport(FRHICommandList& RHICmdList, const FViewInfo& View, float ViewportScale)
{
	if (View.IsInstancedStereo() && IStereoRendering::IsAPrimaryView(View))
	{
		const FViewInfo& LeftView = View;
		const uint32 LeftMinX = LeftView.ViewRect.Min.X * ViewportScale;
		const uint32 LeftMaxX = LeftView.ViewRect.Max.X * ViewportScale;
		const uint32 LeftMaxY = LeftView.ViewRect.Max.Y * ViewportScale;

		const FViewInfo& RightView = static_cast<const FViewInfo&>(*View.GetInstancedView());
		const uint32 RightMinX = RightView.ViewRect.Min.X * ViewportScale;
		const uint32 RightMaxX = RightView.ViewRect.Max.X * ViewportScale;
		const uint32 RightMaxY = RightView.ViewRect.Max.Y * ViewportScale;

		RHICmdList.SetStereoViewport(LeftMinX, RightMinX, 0, 0, 0.0f, LeftMaxX, RightMaxX, LeftMaxY, RightMaxY, 1.0f);
	}
	else
	{
		RHICmdList.SetViewport(View.ViewRect.Min.X * ViewportScale, View.ViewRect.Min.Y * ViewportScale, 0.0f, View.ViewRect.Max.X * ViewportScale, View.ViewRect.Max.Y * ViewportScale, 1.0f);
	}
}

/**
* Saves a previously rendered scene color target
*/

class FDummySceneColorResolveBuffer : public FVertexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector4f>(TEXT("FDummySceneColorResolveBuffer"), 3)
			.AddUsage(EBufferUsageFlags::Static)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionZeroData();

		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}
};

TGlobalResource<FDummySceneColorResolveBuffer> GResolveDummyVertexBuffer;

BEGIN_SHADER_PARAMETER_STRUCT(FResolveSceneColorParameters, )
	RDG_TEXTURE_ACCESS(SceneColor, ERHIAccess::SRVGraphics)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, SceneColorFMask)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

namespace
{
	template<typename ShaderType>
	void TSetColorResolveShader(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, const FViewInfo& View, bool bArrayResolve, FRHITexture* SceneColorTargetableRHI)
	{
		TShaderMapRef<ShaderType> ShaderRef(View.ShaderMap);
		FRHIPixelShader* Shader = ShaderRef.GetPixelShader();
		check(Shader);
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = Shader;
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
		SetShaderParametersLegacyPS(RHICmdList, ShaderRef, SceneColorTargetableRHI);
	}

	template<typename ShaderType, typename ShaderArrayType>
	void TChooseColorResolveShader(FRHICommandList& RHICmdList, FGraphicsPipelineStateInitializer& GraphicsPSOInit, const FViewInfo& View, bool bArrayResolve, FRHITexture* SceneColorTargetableRHI)
	{
		if (UNLIKELY(bArrayResolve))
		{
			TSetColorResolveShader<ShaderArrayType>(RHICmdList, GraphicsPSOInit, View, bArrayResolve, SceneColorTargetableRHI);
		}
		else
		{
			TSetColorResolveShader<ShaderType>(RHICmdList, GraphicsPSOInit, View, bArrayResolve, SceneColorTargetableRHI);
		}
	}

	template<typename ShaderType>
	FRHIVertexShader* GetTypedVS(const FViewInfo& View)
	{
		TShaderMapRef<ShaderType> ShaderRef(View.ShaderMap);
		return ShaderRef.GetVertexShader();
	}
}

void AddResolveSceneColorPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureMSAA SceneColor)
{
	check(SceneColor.IsValid());

	const uint32 NumSamples = SceneColor.Target->Desc.NumSamples;
	const EShaderPlatform CurrentShaderPlatform = GetFeatureLevelShaderPlatform(View.FeatureLevel);

	if (NumSamples == 1 || !SceneColor.IsSeparate()
		|| EnumHasAnyFlags(SceneColor.Target->Desc.Flags, TexCreate_Memoryless))
	{
		return;
	}

	FRDGTextureSRVRef SceneColorFMask = nullptr;

	if (GRHISupportsExplicitFMask)
	{
		SceneColorFMask = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMetaData(SceneColor.Target, ERDGTextureMetaDataAccess::FMask));
	}

	FResolveSceneColorParameters* PassParameters = GraphBuilder.AllocParameters<FResolveSceneColorParameters>();
	PassParameters->SceneColor = SceneColor.Target;
	PassParameters->SceneColorFMask = SceneColorFMask;
	PassParameters->RenderTargets[0] = FRenderTargetBinding(SceneColor.Resolve, SceneColor.Resolve->HasBeenProduced() ? ERenderTargetLoadAction::ELoad : ERenderTargetLoadAction::ENoAction);

	FRDGTextureRef SceneColorTargetable = SceneColor.Target;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ResolveSceneColor"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, SceneColorTargetable, SceneColorFMask, NumSamples](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FRHITexture* SceneColorTargetableRHI = SceneColorTargetable->GetRHI();
		SceneColorTargetable->MarkResourceAsUsed();

		FRHIShaderResourceView* SceneColorFMaskRHI = nullptr;
		if (SceneColorFMask)
		{
			SceneColorFMask->MarkResourceAsUsed();
			SceneColorFMaskRHI = SceneColorFMask->GetRHI();
		}

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

		const FIntPoint SceneColorExtent = SceneColorTargetable->Desc.Extent;

		// Resolve views individually. In the case of adaptive resolution, the view family will be much larger than the views individually.
		RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, SceneColorExtent.X, SceneColorExtent.Y, 1.0f);
		RHICmdList.SetScissorRect(true, View.ViewRectWithSecondaryViews.Min.X, View.ViewRectWithSecondaryViews.Min.Y, View.ViewRectWithSecondaryViews.Max.X, View.ViewRectWithSecondaryViews.Max.Y);

		int32 ResolveWidth = CVarWideCustomResolve.GetValueOnRenderThread();

		if (NumSamples <= 1)
		{
			ResolveWidth = 0;
		}

		if (ResolveWidth != 0)
		{
			ResolveFilterWide(RHICmdList, GraphicsPSOInit, View.FeatureLevel, SceneColorTargetableRHI, SceneColorFMaskRHI, FIntPoint(0, 0), NumSamples, ResolveWidth, GResolveDummyVertexBuffer.VertexBufferRHI);
		}
		else
		{
			bool bArrayResolve = SceneColorTargetableRHI->GetDesc().IsTextureArray();

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = bArrayResolve ? GetTypedVS<FHdrCustomResolveArrayVS>(View) : GetTypedVS<FHdrCustomResolveVS>(View);
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;

			if (SceneColorFMaskRHI)
			{
				checkf(!bArrayResolve, TEXT("Array MSAA resolve is not supported for the FMask path"));

				if (NumSamples == 2)
				{
					TShaderMapRef<FHdrCustomResolveFMask2xPS> PixelShader(View.ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, PixelShader, SceneColorTargetableRHI, SceneColorFMaskRHI);
				}
				else if (NumSamples == 4)
				{
					TShaderMapRef<FHdrCustomResolveFMask4xPS> PixelShader(View.ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, PixelShader, SceneColorTargetableRHI, SceneColorFMaskRHI);
				}
				else if (NumSamples == 8)
				{
					TShaderMapRef<FHdrCustomResolveFMask8xPS> PixelShader(View.ShaderMap);
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
					SetShaderParametersLegacyPS(RHICmdList, PixelShader, SceneColorTargetableRHI, SceneColorFMaskRHI);
				}
				else
				{
					// Everything other than 2,4,8 samples is not implemented.
					checkNoEntry();
				}
			}
			else
			{
				if (NumSamples == 2)
				{
					TChooseColorResolveShader<FHdrCustomResolve2xPS, FHdrCustomResolveArray2xPS>(RHICmdList, GraphicsPSOInit, View, bArrayResolve, SceneColorTargetableRHI);
				}
				else if (NumSamples == 4)
				{
					TChooseColorResolveShader<FHdrCustomResolve4xPS, FHdrCustomResolveArray4xPS>(RHICmdList, GraphicsPSOInit, View, bArrayResolve, SceneColorTargetableRHI);
				}
				else if (NumSamples == 8)
				{
					TChooseColorResolveShader<FHdrCustomResolve8xPS, FHdrCustomResolveArray8xPS>(RHICmdList, GraphicsPSOInit, View, bArrayResolve, SceneColorTargetableRHI);
				}
				else
				{
					// Everything other than 2,4,8 samples is not implemented.
					checkNoEntry();
				}
			}

			RHICmdList.SetStreamSource(0, GResolveDummyVertexBuffer.VertexBufferRHI, 0);
			RHICmdList.DrawPrimitive(0, 1, 1);
		}

		RHICmdList.SetScissorRect(false, 0, 0, 0, 0);
	});
}

void AddResolveSceneColorPass(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureMSAA SceneColor)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		if (View.ShouldRenderView())
		{
			AddResolveSceneColorPass(GraphBuilder, View, SceneColor);
		}
	}
}

BEGIN_SHADER_PARAMETER_STRUCT(FResolveSceneDepthParameters, )
	RDG_TEXTURE_ACCESS(SceneDepth, ERHIAccess::SRVGraphics)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()

namespace
{
	template<typename ShaderType>
	FRHIPixelShader* GetDepthResolveShader(const FViewInfo& View, FShaderResourceParameter& UnresolvedSurfaceParameter)
	{
		TShaderMapRef<ShaderType> ShaderRef(View.ShaderMap);
		UnresolvedSurfaceParameter = ShaderRef->UnresolvedSurface;
		return ShaderRef.GetPixelShader();
	};

	template<typename ShaderType>
	FRHIVertexShader* GetDepthResolveVS(const FViewInfo& View, TShaderRef<FResolveVS>& OutShaderMapRef)
	{
		TShaderMapRef<ShaderType> ShaderRef(View.ShaderMap);
		OutShaderMapRef = ShaderRef;
		return ShaderRef.GetVertexShader();
	}
}

void AddResolveSceneDepthPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureMSAA SceneDepth)
{
	check(SceneDepth.IsValid());

	const uint32 NumSamples = SceneDepth.Target->Desc.NumSamples;
	const EShaderPlatform CurrentShaderPlatform = GetFeatureLevelShaderPlatform(View.FeatureLevel);

	if (NumSamples == 1 || !SceneDepth.IsSeparate() 
		|| EnumHasAnyFlags(SceneDepth.Target->Desc.Flags, TexCreate_Memoryless))
	{
		return;
	}

	FResolveRect ResolveRect(View.ViewRectWithSecondaryViews);
	const FIntPoint DepthExtent = SceneDepth.Resolve->Desc.Extent;

	FResolveSceneDepthParameters* PassParameters = GraphBuilder.AllocParameters<FResolveSceneDepthParameters>();
	PassParameters->SceneDepth = SceneDepth.Target;
	PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(SceneDepth.Resolve, ERenderTargetLoadAction::ENoAction, ERenderTargetLoadAction::ENoAction, FExclusiveDepthStencil::DepthWrite_StencilWrite);

	FRDGTextureRef SourceTexture = SceneDepth.Target;

	GraphBuilder.AddPass(
		RDG_EVENT_NAME("ResolveSceneDepth"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, SourceTexture, NumSamples, DepthExtent, ResolveRect](FRDGAsyncTask, FRHICommandList& RHICmdList)
	{
		FRHITexture* SourceTextureRHI = SourceTexture->GetRHI();
		SourceTexture->MarkResourceAsUsed();

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<FM_Solid, CM_None>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<true, CF_Always, true, CF_Always, SO_Zero, SO_Zero, SO_Zero, true, CF_Always, SO_Zero, SO_Zero, SO_Zero>::GetRHI();

		RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, DepthExtent.X, DepthExtent.Y, 1.0f);

		bool bArrayResolve = SourceTextureRHI->GetDesc().IsTextureArray();
		ensureMsgf(!bArrayResolve || (RHISupportsVertexShaderLayer(View.GetShaderPlatform()) && GRHISupportsArrayIndexFromAnyShader),
			TEXT("Resolving scene depth array requires support for outputting SV_RenderTargetArrayIndex from any shader."));

		/** Chooses one of many ResolvePS variants */
		auto ChoosePixelShader = [](const FViewInfo& View, bool bIsArrayResolve, int32 NumSamples, FShaderResourceParameter& UnresolvedSurfaceParameter) -> FRHIPixelShader*
		{
			if (LIKELY(!bIsArrayResolve))
			{
				switch (NumSamples)
				{
					case 2:
						return GetDepthResolveShader<FResolveDepth2XPS>(View, UnresolvedSurfaceParameter);
					case 4:
						return GetDepthResolveShader<FResolveDepth4XPS>(View, UnresolvedSurfaceParameter);
					case 8:
						return GetDepthResolveShader<FResolveDepth8XPS>(View, UnresolvedSurfaceParameter);
					default:
						ensureMsgf(false, TEXT("Unsupported depth resolve for samples: %i.  Dynamic loop method isn't supported on all platforms.  Please add specific case."), NumSamples);
						return GetDepthResolveShader<FResolveDepthPS>(View, UnresolvedSurfaceParameter);
				}
			}
			else
			{
				switch (NumSamples)
				{
					case 2:
						return GetDepthResolveShader<FResolveDepthArray2XPS>(View, UnresolvedSurfaceParameter);
					case 4:
						return GetDepthResolveShader<FResolveDepthArray4XPS>(View, UnresolvedSurfaceParameter);
					case 8:
						return GetDepthResolveShader<FResolveDepthArray8XPS>(View, UnresolvedSurfaceParameter);
					default:
						ensureMsgf(false, TEXT("Unsupported depth resolve for samples: %i (texture array case).  Dynamic loop method isn't supported on all platforms.  Please add specific case."), NumSamples);
						return GetDepthResolveShader<FResolveDepthPS>(View, UnresolvedSurfaceParameter);
				}
			}
			// unreachable
		};

		FShaderResourceParameter UnresolvedSurfaceParameter;
		FRHIPixelShader* ResolvePixelShader = ChoosePixelShader(View, bArrayResolve, NumSamples, UnresolvedSurfaceParameter);

		TShaderRef<FResolveVS> ResolveVertexShader;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = bArrayResolve ? GetDepthResolveVS<FResolveArrayVS>(View, ResolveVertexShader) : GetDepthResolveVS<FResolveVS>(View, ResolveVertexShader);
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ResolvePixelShader;
		GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
		RHICmdList.SetBlendFactor(FLinearColor::White);

		FRHIBatchedShaderParameters& BatchedParameters = RHICmdList.GetScratchShaderParameters();
		SetTextureParameter(BatchedParameters, UnresolvedSurfaceParameter, SourceTextureRHI);
		RHICmdList.SetBatchedShaderParameters(ResolvePixelShader, BatchedParameters);

		SetShaderParametersLegacyVS(RHICmdList, ResolveVertexShader, ResolveRect, ResolveRect, DepthExtent.X, DepthExtent.Y);

		RHICmdList.SetStreamSource(0, nullptr, 0);
		RHICmdList.DrawPrimitive(0, 2, bArrayResolve ? 2 : 1);
	});
}

void AddResolveSceneDepthPass(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureMSAA SceneDepth)
{
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		const FViewInfo& View = Views[ViewIndex];
		if (View.ShouldRenderView())
		{
			AddResolveSceneDepthPass(GraphBuilder, View, SceneDepth);
		}
	}
}

void VirtualTextureFeedbackBegin(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FIntPoint SceneTextureExtent)
{
	const uint32 FeedbackTileSize = Views.Num() > 0 ? Views[0].Family->VirtualTextureFeedbackFactor : 0;
	const ERHIFeatureLevel::Type FeatureLevel = Views.Num() > 0 ? Views[0].GetFeatureLevel() : GMaxRHIFeatureLevel;
	static const bool bCanUseDebugMaterials = ShouldCompileODSCOnlyShaders();
	const bool bExtendFeedbackForDebug = Views.Num() > 0 ? Views[0].Family->EngineShowFlags.VisualizeVirtualTexture && bCanUseDebugMaterials : false;
	VirtualTexture::BeginFeedback(GraphBuilder, SceneTextureExtent, FeedbackTileSize, bExtendFeedbackForDebug, FeatureLevel);
}

static TAutoConsoleVariable<int32> CVarHalfResDepthNoFastClear(
	TEXT("r.HalfResDepthNoFastClear"),
	1,
	TEXT("Remove fast clear on half resolution depth buffer (checkerboard and minmax)"),
	ECVF_RenderThreadSafe);

FRDGTextureRef CreateHalfResolutionDepthCheckerboardMinMax(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureRef SceneDepthTexture)
{
	const uint32 DownscaleFactor = 2;
	const FIntPoint SmallDepthExtent = GetDownscaledExtent(SceneDepthTexture->Desc.Extent, DownscaleFactor);

	const ETextureCreateFlags NoFastClearFlags = (CVarHalfResDepthNoFastClear.GetValueOnAnyThread() != 0) ? TexCreate_NoFastClear : TexCreate_None;

	const FRDGTextureDesc SmallDepthDesc = FRDGTextureDesc::Create2D(SmallDepthExtent, PF_DepthStencil, FClearValueBinding::None, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource | NoFastClearFlags);
	FRDGTextureRef SmallDepthTexture = GraphBuilder.CreateTexture(SmallDepthDesc, TEXT("HalfResolutionDepthCheckerboardMinMax"));

	for (const FViewInfo& View : Views)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		const FScreenPassTexture SceneDepth(SceneDepthTexture, View.ViewRect);
		const FScreenPassRenderTarget SmallDepth(SmallDepthTexture, GetDownscaledRect(View.ViewRect, DownscaleFactor), View.DecayLoadAction(ERenderTargetLoadAction::ENoAction));
		AddDownsampleDepthPass(GraphBuilder, View, SceneDepth, SmallDepth, EDownsampleDepthFilter::Checkerboard);
	}

	return SmallDepthTexture;
}

FRDGTextureRef CreateQuarterResolutionDepthMinAndMax(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureRef InputDepthTexture)
{
	const FIntPoint SmallDepthExtent = GetDownscaledExtent(InputDepthTexture->Desc.Extent, 2);
	const ETextureCreateFlags NoFastClearFlags = (CVarHalfResDepthNoFastClear.GetValueOnAnyThread() != 0) ? TexCreate_NoFastClear : TexCreate_None;
	const FRDGTextureDesc SmallTextureDesc = FRDGTextureDesc::Create2D(SmallDepthExtent, PF_G16R16F, FClearValueBinding::None, TexCreate_RenderTargetable | TexCreate_ShaderResource | NoFastClearFlags);
	FRDGTextureRef SmallTexture = GraphBuilder.CreateTexture(SmallTextureDesc, TEXT("HalfResolutionDepthMinAndMax"));

	for (const FViewInfo& View : Views)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		const FScreenPassTexture InputDepth(InputDepthTexture, GetDownscaledRect(View.ViewRect, 2));
		const FScreenPassRenderTarget SmallTextureRT(SmallTexture, GetDownscaledRect(InputDepth.ViewRect, 2), View.DecayLoadAction(ERenderTargetLoadAction::ENoAction));
		AddDownsampleDepthPass(GraphBuilder, View, InputDepth, SmallTextureRT, EDownsampleDepthFilter::MinAndMaxDepth);
	}

	return SmallTexture;
}

void CreateQuarterResolutionDepthMinAndMaxFromDepthTexture(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureRef DepthTexture, FRDGTextureRef& OutHalfResMinMax, FRDGTextureRef& OutQuarterResMinMax)
{
	const uint32 DownscaleFactor = 2;
	const ETextureCreateFlags NoFastClearFlags = (CVarHalfResDepthNoFastClear.GetValueOnAnyThread() != 0) ? TexCreate_NoFastClear : TexCreate_None;

	const FIntPoint HalfResMinMaxDepthExtent	= GetDownscaledExtent(DepthTexture->Desc.Extent, DownscaleFactor);
	const FIntPoint QuarterResMinMaxDepthExtent	= GetDownscaledExtent(HalfResMinMaxDepthExtent, DownscaleFactor);

	const FRDGTextureDesc HalfMinMaxDepthDesc	= FRDGTextureDesc::Create2D(HalfResMinMaxDepthExtent, PF_G16R16F, FClearValueBinding::None, TexCreate_RenderTargetable | TexCreate_ShaderResource | NoFastClearFlags);
	const FRDGTextureDesc QuarterMinMaxDepthDesc= FRDGTextureDesc::Create2D(QuarterResMinMaxDepthExtent, PF_G16R16F, FClearValueBinding::None, TexCreate_RenderTargetable | TexCreate_ShaderResource | NoFastClearFlags);

	OutHalfResMinMax	= GraphBuilder.CreateTexture(HalfMinMaxDepthDesc, TEXT("HalfResMinMaxDepthTexture"));
	OutQuarterResMinMax	= GraphBuilder.CreateTexture(QuarterMinMaxDepthDesc, TEXT("QuarterResMinMaxDepthTexture"));

	for (const FViewInfo& View : Views)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);
		
		const FScreenPassTexture DepthPassTexture(DepthTexture, View.ViewRect);
		const FScreenPassRenderTarget HalfResDepthTexture(OutHalfResMinMax, GetDownscaledRect(View.ViewRect, DownscaleFactor), View.DecayLoadAction(ERenderTargetLoadAction::ENoAction));
		AddDownsampleDepthPass(GraphBuilder, View, DepthPassTexture, HalfResDepthTexture, EDownsampleDepthFilter::MinAndMaxDepth);
	}

	for (const FViewInfo& View : Views)
	{
		RDG_GPU_MASK_SCOPE(GraphBuilder, View.GPUMask);

		const FScreenPassTexture HalfResDepthPassTexture(OutHalfResMinMax, GetDownscaledRect(View.ViewRect, DownscaleFactor));
		const FScreenPassRenderTarget QuarterResDepthTexture(OutQuarterResMinMax, GetDownscaledRect(GetDownscaledRect(View.ViewRect, DownscaleFactor), DownscaleFactor), View.DecayLoadAction(ERenderTargetLoadAction::ENoAction));
		AddDownsampleDepthPass(GraphBuilder, View, HalfResDepthPassTexture, QuarterResDepthTexture, EDownsampleDepthFilter::MinAndMaxDepthFromMinAndMaxDepth);
	}
}

bool IsDeferredPrimitiveAlphaHoldoutSupported()
{
	// Read-only cvar is expected to match SUPPORT_PRIMITIVE_ALPHA_HOLDOUT shader definition
	return CVarPrimitiveAlphaHoldoutSupport.GetValueOnRenderThread();
}

bool IsPrimitiveAlphaHoldoutEnabled(const FViewInfo& View)
{
	// Note: r.Deferred.SupportPrimitiveAlphaHoldout excludes the path tracer
	const bool bSupportPrimitiveAlphaHoldout = View.Family->EngineShowFlags.PathTracing ? true : CVarPrimitiveAlphaHoldoutSupport.GetValueOnRenderThread();

	return bSupportPrimitiveAlphaHoldout
		&& (GetFeatureLevelShadingPath(View.GetFeatureLevel()) != EShadingPath::Mobile) 
		&& IsPostProcessingWithAlphaChannelSupported()
		&& View.Family->EngineShowFlags.AllowPrimitiveAlphaHoldout
		&& !View.bIsReflectionCapture; // Force-disable primitive alpha holdout during reflection captures
}

bool IsPrimitiveAlphaHoldoutEnabledForAnyView(TArrayView<const FViewInfo> Views)
{
	for (const FViewInfo& View : Views)
	{
		if (IsPrimitiveAlphaHoldoutEnabled(View))
		{
			return true;
		}
	}
	
	return false;
}

bool SceneCaptureRequiresAlphaChannel(const FSceneView& View)
{
	// Planar reflections and scene captures use scene color alpha to keep track of where content has been rendered, for compositing into a different scene later
	if (View.bIsPlanarReflection)
	{
		return true;
	}

	if (View.bIsSceneCapture)
	{
		// Depth capture modes do not require alpha channel
		if (View.CustomRenderPass)
		{
			return View.CustomRenderPass->GetRenderOutput() != FCustomRenderPassBase::ERenderOutput::SceneDepth
				&& View.CustomRenderPass->GetRenderOutput() != FCustomRenderPassBase::ERenderOutput::DeviceDepth
				&& View.CustomRenderPass->GetRenderOutput() != FCustomRenderPassBase::ERenderOutput::SceneColorNoAlpha;
		}
		else if(View.Family)
		{
			return View.Family->SceneCaptureSource != SCS_SceneDepth 
				&& View.Family->SceneCaptureSource != SCS_DeviceDepth
				&& View.Family->SceneCaptureSource != SCS_SceneColorHDRNoAlpha;
		}
	}
	return false;
}

bool DoMaterialAndPrimitiveModifyMeshPosition(const FMaterial& Material, const FPrimitiveSceneProxy* PrimitiveSceneProxy)
{
	const bool bMaterialModifiesMeshPosition = Material.MaterialModifiesMeshPosition_RenderThread();
	const bool bPrimitiveAllowsWPOEvaluation = !ShouldOptimizedWPOAffectNonNaniteShaderSelection() || (PrimitiveSceneProxy && PrimitiveSceneProxy->EvaluateWorldPositionOffset());
	const bool bIsFirstPerson = PrimitiveSceneProxy && PrimitiveSceneProxy->IsFirstPerson();
	// First person primitives have special logic that modifies vertex positions after WPO has been applied. If Material.HasFirstPersonOutput() is true, then the material has custom logic controlling this,
	// but even without that custom logic, optimized position-only shaders for depth-only rendering do not support first person, so by checking the bIsFirstPerson flag here, we prevent first person primitives
	// from being rendered with such incompatible shaders.
	return (bMaterialModifiesMeshPosition && bPrimitiveAllowsWPOEvaluation) || bIsFirstPerson;
}