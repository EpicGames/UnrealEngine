// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneRendering.h: Scene rendering definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Async/Mutex.h"
#include "Async/TaskGraphFwd.h"
#include "Async/TaskGraphInterfaces.h"
#include "BatchedElements.h"
#include "Containers/ArrayView.h"
#include "Containers/ChunkedArray.h"
#include "Containers/IndirectArray.h"
#include "Containers/SparseSet.h"
#include "Containers/StaticArray.h"
#include "Containers/StridedView.h"
#include "DepthRendering.h"
#include "DynamicRHI.h"
#include "DynamicRenderScaling.h"
#include "Engine/Scene.h"
#include "Engine/World.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "ExtensibleUniformBuffer.h"
#include "FrontLayerTranslucency.h"
#include "GPUScene.h"
#include "GlobalDistanceField.h"
#include "GlobalDistanceFieldParameters.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "HairStrands/HairStrandsData.h"
#include "HitProxies.h"
#include "InstanceCulling/InstanceCullingManager.h"
#include "LightFunctionAtlas.h"
#include "LightSceneProxy.h"
#include "LocalFogVolumeRendering.h"
#include "Logging/LogMacros.h"
#include "Lumen/LumenTranslucencyVolumeLighting.h"
#include "MaterialShared.h"
#include "Math/Color.h"
#include "MegaLights/MegaLights.h"
#include "MeshBatch.h"
#include "MeshDrawCommands.h"
#include "MeshElementCollector.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.h"
#include "Misc/App.h"
#include "Misc/LargeWorldCoordinates.h"
#include "Misc/MemStack.h"
#include "MultiGPU.h"
#include "Nanite/NaniteShared.h"
#include "PostProcess/LensDistortion.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "PrimitiveSceneInfo.h"
#include "RHICommandList.h"
#include "RHIContext.h"
#include "RHIDefinitions.h"
#include "RHIFeatureLevel.h"
#include "RHIFwd.h"
#include "RHIResources.h"
#include "RHIShaderPlatform.h"
#include "RayTracing/RayTracingScene.h"
#include "RayTracingMeshDrawCommands.h"
#include "RenderGraphBuilder.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphFwd.h"
#include "RenderGraphPass.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "RenderResource.h"
#include "RendererInterface.h"
#include "RendererUtils.h"
#include "RenderingThread.h"
#include "Stats/Stats.h"
#include "Tasks/Task.h"
#include "SceneManagement.h"
#include "SceneTexturesConfig.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "StereoRendering.h"
#include "ShaderParameterMacros.h"
#include "ShaderParameterMetadata.h"
#include "StaticBoundShaderState.h"
#include "SceneTextures.h"
#include "TranslucentPassResource.h"
#include "SceneExtensions.h"
#include "ScenePrivateBase.h"
#include "SceneVisibility.h"
#include "Substrate/Substrate.h"
#include "VirtualShadowMaps/VirtualShadowMapArray.h"
#include "SceneRendererInterface.h"
#include "SceneUniformBuffer.h"
#include "ShaderPrintParameters.h"
#include "TemporalUpscaler.h"
#include "MegaFiltersInterface.h"
#include "LightShaftRendering.h"
#include "SceneViewOcclusionHistory.h"
#include "VisualizeLightShape.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "RHI.h"
#include "UniformBuffer.h"
#include "PrimitiveViewRelevance.h"
#include "LightComponentId.h"
#include "Templates/UniquePtr.h"
#include "PostProcess/PostProcessAmbientOcclusionMobile.h"
#include "VirtualShadowMaps/VirtualShadowMapProjection.h"
#include "RenderCore.h"
#include "DBufferTextures.h"
#include "TranslucencyPass.h"
#include "TextureFallbacks.h"
#include "SceneInterface.h"
#include "DecalRenderingShared.h"
#include "DecalRenderingCommon.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#if RHI_RAYTRACING
#include "RayTracing/RayTracingScene.h"
#endif // RHI_RAYTRACING

// Forward declarations.
class FScene;
class FSceneViewState;
class FViewInfo;
struct FILCUpdatePrimTaskData;
class FPostprocessContext;
struct FVisibleDecal;
struct FDecalViewInfo;
class FDecalVisibilityTaskData;
class FRayTracingLightGrid;
class FRayTracingDecals;
class FRayTracingLocalShaderBindingWriter;
class FVirtualShadowMapClipmap;
class FShadowProjectionPassParameters;
class FSceneTextureShaderParameters;
class FLumenSceneData;
class FShadowSceneRenderer;
class FGlobalShaderMap;
class FVirtualTextureUpdater;
class FExponentialHeightFogSceneInfo;
struct FCloudRenderContext;
struct FSceneWithoutWaterTextures;
struct FHairStrandsVisibilityViews;
struct FSortedLightSetSceneInfo;
enum class EVelocityPass : uint32;
enum class ERayTracingSceneLayer : uint8;
class FTransientLightFunctionTextureAtlas;
struct FSceneTexturesConfig;
struct FMinimalSceneTextures;
struct FSceneTextures;
struct FCustomDepthTextures;
struct FDynamicShadowsTaskData;
class FAtmosphereUniformShaderParameters;
struct FSkyAtmosphereRenderContext;
class FTexture2DResource;
class FSimpleLightArray;
struct FScreenMessageWriter;
struct FVolumetricFogIntegrationParameterData;
class FLumenHardwareRayTracingUniformBufferParameters;
struct FDBufferTextures;
class FDistanceFieldCulledObjectBufferParameters;
class FTileIntersectionParameters;
class FDistanceFieldAOParameters;
struct FMobileBasePassTextures;
struct FDBufferTextures;
class FFXSystemInterface;
struct FPrimitiveViewRelevance;
struct FNaniteShadingCommands;
namespace Nanite
{
	struct FInstanceDraw;
}
namespace UE::Renderer::Private
{
	class ITemporalUpscaler;
	class IMegaFilters;
}
enum class EDecalRenderStage : uint8;

struct FPersistentViewId
{
	bool IsValid() const { return Index != INDEX_NONE; }
	int32 Index = INDEX_NONE;
};

DECLARE_LOG_CATEGORY_EXTERN(LogSceneCapture, Log, All);

struct FSceneCaptureLogUtils
{
	static bool bEnableSceneCaptureLogging;
};

extern bool ShouldUseStereoLumenOptimizations();

DECLARE_GPU_STAT_NAMED_EXTERN(Postprocessing, TEXT("Postprocessing"));
DECLARE_GPU_STAT_NAMED_EXTERN(CustomRenderPasses, TEXT("CustomRenderPasses"));

/** Mobile only. Information used to determine whether static meshes will be rendered with CSM shaders or not. */
class FMobileCSMVisibilityInfo
{
public:
	/** true if there are any primitives affected by CSM subjects */
	uint32 bMobileDynamicCSMInUse : 1;

	// true if all draws should be forced to use CSM shaders.
	uint32 bAlwaysUseCSM : 1;

	/** Visibility lists for static meshes that will use expensive CSM shaders. */
	FSceneBitArray MobilePrimitiveCSMReceiverVisibilityMap;
	FSceneBitArray MobileCSMStaticMeshVisibilityMap;

	/** Visibility lists for static meshes that will use the non CSM shaders. */
	FSceneBitArray MobileNonCSMStaticMeshVisibilityMap;

	/** Initialization constructor. */
	FMobileCSMVisibilityInfo() : bMobileDynamicCSMInUse(false), bAlwaysUseCSM(false)
	{}
};

/** Stores a list of CSM shadow casters. Used by mobile renderer for culling primitives receiving static + CSM shadows. */
class FMobileCSMSubjectPrimitives
{
public:
	/** Adds a subject primitive */
	void AddSubjectPrimitive(const FPrimitiveSceneInfo* PrimitiveSceneInfo, int32 PrimitiveId)
	{
		checkSlow(PrimitiveSceneInfo->GetIndex() == PrimitiveId);
		const int32 PrimitiveIndex = PrimitiveSceneInfo->GetIndex();
		if (!ShadowSubjectPrimitivesEncountered[PrimitiveId])
		{
			ShadowSubjectPrimitives.Add(PrimitiveSceneInfo);
			ShadowSubjectPrimitivesEncountered[PrimitiveId] = true;
		}
	}

	/** Returns the list of subject primitives */
	const TArray<const FPrimitiveSceneInfo*, SceneRenderingAllocator>& GetShadowSubjectPrimitives() const
	{
		return ShadowSubjectPrimitives;
	}

	/** Used to initialize the ShadowSubjectPrimitivesEncountered bit array
	  * to prevent shadow primitives being added more than once. */
	void InitShadowSubjectPrimitives(int32 PrimitiveCount)
	{
		ShadowSubjectPrimitivesEncountered.Init(false, PrimitiveCount);
	}

protected:
	/** List of this light's shadow subject primitives. */
	FSceneBitArray ShadowSubjectPrimitivesEncountered;
	TArray<const FPrimitiveSceneInfo*, SceneRenderingAllocator> ShadowSubjectPrimitives;
};

/** Information about a visible light which is specific to the view it's visible in. */
class FVisibleLightViewInfo
{
public:

	/** Whether each shadow in the corresponding FVisibleLightInfo::AllProjectedShadows array is visible. */
	FSceneBitArray ProjectedShadowVisibilityMap;

	/** The view relevance of each shadow in the corresponding FVisibleLightInfo::AllProjectedShadows array. */
	TArray<FPrimitiveViewRelevance,SceneRenderingAllocator> ProjectedShadowViewRelevanceMap;

	/** true if this light in the view frustum (dir/sky lights always are). */
	uint32 bInViewFrustum : 1;
	/** true if the light didn't get distance culled. */
	uint32 bInDrawRange : 1;

	/** List of CSM shadow casters. Used by mobile renderer for culling primitives receiving static + CSM shadows */
	FMobileCSMSubjectPrimitives MobileCSMSubjectPrimitives;

	/** Initialization constructor. */
	FVisibleLightViewInfo()
	:	bInViewFrustum(false)
	,	bInDrawRange(false)
	{}
};

/** Information about a visible light which isn't view-specific. */
class FVisibleLightInfo
{
public:

	/** All visible projected shadows, output of shadow setup.  Not all of these will be rendered. */
	TArray<FProjectedShadowInfo*,SceneRenderingAllocator> AllProjectedShadows;

	/** Shadows to project for each feature that needs special handling. */
	TArray<FProjectedShadowInfo*,SceneRenderingAllocator> ShadowsToProject;
	TArray<FProjectedShadowInfo*,SceneRenderingAllocator> CapsuleShadowsToProject;

	/** All visible projected preshdows.  These are not allocated on the mem stack so they are refcounted. */
	TArray<TRefCountPtr<FProjectedShadowInfo>,SceneRenderingAllocator> ProjectedPreShadows;

	/** A list of per-object shadows that were occluded. We need to track these so we can issue occlusion queries for them. */
	TArray<FProjectedShadowInfo*,SceneRenderingAllocator> OccludedPerObjectShadows;

	TArray<TSharedPtr<FVirtualShadowMapClipmap>, SceneRenderingAllocator> VirtualShadowMapClipmaps;

	/** 
	* Returns true if the light has only virtual shadow maps (or no shadows at all), i.e. no conventional shadow maps that are allocated
	*/
	bool ContainsOnlyVirtualShadowMaps() const;

	TSharedPtr<FVirtualShadowMapClipmap> FindVirtualShadowMapShadowClipmapForView(const FViewInfo* View) const;

	/**
	* Prefer this to direct access of the VirtualShadowMapId member when a view is known.
	* For directional lights this will attempt to find a clipmap associated with the given view,
	* while the VirtualShadowMapId variable will simply be an arbitrary one of them if multiple exist.
	*/
	int32 GetVirtualShadowMapId( const FViewInfo* View ) const;
};

// Stores the primitive count of each translucency pass (redundant, could be computed after sorting but this way we touch less memory)
struct FTranslucenyPrimCount
{
private:
	uint32 Count[ETranslucencyPass::TPT_MAX];
	bool UseSceneColorCopyPerPass[ETranslucencyPass::TPT_MAX];

public:
	// constructor
	FTranslucenyPrimCount()
	{
		for(uint32 i = 0; i < ETranslucencyPass::TPT_MAX; ++i)
		{
			Count[i] = 0;
			UseSceneColorCopyPerPass[i] = false;
		}
	}

	// interface similar to TArray but here we only store the count of Prims per pass
	void Append(const FTranslucenyPrimCount& InSrc)
	{
		for(uint32 i = 0; i < ETranslucencyPass::TPT_MAX; ++i)
		{
			Count[i] += InSrc.Count[i];
			UseSceneColorCopyPerPass[i] |= InSrc.UseSceneColorCopyPerPass[i];
		}
	}

	// interface similar to TArray but here we only store the count of Prims per pass
	void Add(ETranslucencyPass::Type InPass, bool bUseSceneColorCopy)
	{
		++Count[InPass];
		UseSceneColorCopyPerPass[InPass] |= bUseSceneColorCopy;
	}

	int32 Num(ETranslucencyPass::Type InPass) const
	{
		return Count[InPass];
	}

	int32 NumPrims() const
	{
		int32 NumTotal = 0;
		for (uint32 PassIndex = 0; PassIndex < ETranslucencyPass::TPT_MAX; ++PassIndex)
		{
			NumTotal += Count[PassIndex];
		}
		return NumTotal;
	}

	bool UseSceneColorCopy(ETranslucencyPass::Type InPass) const
	{
		return UseSceneColorCopyPerPass[InPass];
	}
};

/** A batched occlusion primitive. */
struct FOcclusionPrimitive
{
	FVector Center;
	FVector Extent;
};

// An occlusion query pool with frame based lifetime management
class FFrameBasedOcclusionQueryPool
{
public:
	FFrameBasedOcclusionQueryPool() = default;

	FRHIRenderQuery* AllocateQuery();

	// Recycle queries that are (OcclusionFrameCounter - NumBufferedFrames) old or older
	void AdvanceFrame(uint32 InOcclusionFrameCounter, uint32 InNumBufferedFrames, bool bStereoRoundRobin);

private:
	struct FFrameOcclusionQueries
	{
		TArray<FRenderQueryRHIRef> Queries;
		int32 FirstFreeIndex;
		uint32 OcclusionFrameCounter;

		FFrameOcclusionQueries()
			: FirstFreeIndex(0)
			, OcclusionFrameCounter(0)
		{}
	};

	FFrameOcclusionQueries FrameQueries[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames * 2];
	uint32 CurrentFrameIndex = 0;
	uint32 OcclusionFrameCounter = -1;
	uint32 NumBufferedFrames = 0;
};

/**
 * Combines consecutive primitives which use the same occlusion query into a single DrawIndexedPrimitive call.
 */
class FOcclusionQueryBatcher
{
public:

	/** The maximum number of consecutive previously occluded primitives which will be combined into a single occlusion query. */
	enum { OccludedPrimitiveQueryBatchSize = 16 };

	/** Initialization constructor. */
	FOcclusionQueryBatcher(class FSceneViewState* ViewState, uint32 InMaxBatchedPrimitives, uint32 InNumInstances);

	/** Destructor. */
	~FOcclusionQueryBatcher();

	/** @returns True if the batcher has any outstanding batches, otherwise false. */
	bool HasBatches(void) const { return (NumBatchedPrimitives > 0); }

	/** Renders the current batch and resets the batch state. */
	void Flush(FRHICommandList& RHICmdList);

	/**
	 * Batches a primitive's occlusion query for rendering.
	 * @param Bounds - The primitive's bounds.
	 */
	FRHIRenderQuery* BatchPrimitive(const FVector& BoundsOrigin, const FVector& BoundsBoxExtent, FGlobalDynamicVertexBuffer& DynamicVertexBuffer);
	inline int32 GetNumBatchOcclusionQueries() const
	{
		return BatchOcclusionQueries.Num();
	}

private:

	struct FOcclusionBatch
	{
		FRHIRenderQuery* Query;
		FGlobalDynamicVertexBuffer::FAllocation VertexAllocation;
	};

	/** The pending batches. */
	TArray<FOcclusionBatch,SceneRenderingAllocator> BatchOcclusionQueries;

	/** The batch new primitives are being added to. */
	FOcclusionBatch* CurrentBatchOcclusionQuery;

	/** The maximum number of primitives in a batch. */
	const uint32 MaxBatchedPrimitives;

	/** The number of primitives in the current batch. */
	uint32 NumBatchedPrimitives;

	/** The pool to allocate occlusion queries from. */
	FFrameBasedOcclusionQueryPool* OcclusionQueryPool;

	/** The number of instances for instanced stereo rendering */
	uint32 NumInstances;
};

class FHZBOcclusionTester : public FRenderResource
{
public:
	FHZBOcclusionTester();
	~FHZBOcclusionTester() = default;

	// FRenderResource interface
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
	
	uint32 GetNum() const { return Primitives.Num(); }

	uint32 AddBounds( const FVector& BoundsOrigin, const FVector& BoundsExtent );
	void Submit(FRDGBuilder& GraphBuilder, const FViewInfo& View);

	void MapResults(FRHICommandListImmediate& RHICmdList);
	void UnmapResults(FRHICommandListImmediate& RHICmdList);
	bool IsVisible( uint32 Index ) const;

	bool IsValidFrame(uint32 FrameNumber) const;

	void SetValidFrameNumber(uint32 FrameNumber);

	uint64 GetGPUSizeBytes(bool bLogSizes) const;

private:
	enum { SizeX = 256 };
	enum { SizeY = 256 };
	enum { FrameNumberMask = 0x7fffffff };
	enum { InvalidFrameNumber = 0xffffffff };

	TArray< FOcclusionPrimitive, SceneRenderingAllocator >	Primitives;

	const uint8*						ResultsBuffer;
	int32								ResultsBufferRowPitch;
	TUniquePtr<FRHIGPUTextureReadback>	ResultsReadback;

	bool IsInvalidFrame() const;

	// set ValidFrameNumber to a number that cannot be set by SetValidFrameNumber so IsValidFrame will return false for any frame number
	void SetInvalidFrameNumber();

	uint32 ValidFrameNumber;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/** Helper class to marshal data from your RDG pass into the parallel command list set. */
class UE_DEPRECATED(5.5, "Use GraphBuilder.AddDispatchPass instead") FParallelCommandListBindings
{
public:
	template <typename ParameterStructType>
	FParallelCommandListBindings(ParameterStructType* ParameterStruct)
		: StaticUniformBuffers(GetStaticUniformBuffers(ParameterStruct))
		, bHasRenderPassInfo(HasRenderPassInfo(ParameterStruct))
	{
		if (bHasRenderPassInfo)
		{
			RenderPassInfo = GetRenderPassInfo(ParameterStruct);
		}
	}

	inline void SetOnCommandList(FRHICommandList& RHICmdList) const
	{
		if (bHasRenderPassInfo)
		{
			RHICmdList.BeginRenderPass(RenderPassInfo, TEXT("Parallel"));
		}

		RHICmdList.SetStaticUniformBuffers(StaticUniformBuffers);
	}

	FRHIRenderPassInfo RenderPassInfo;
	FUniformBufferStaticBindings StaticUniformBuffers;
	bool bHasRenderPassInfo;
};

class UE_DEPRECATED(5.5, "Use GraphBuilder.AddDispatchPass instead") FParallelCommandListSet
{
public:
	const FRDGPass* Pass;
	const FViewInfo& View;
	FRHICommandListImmediate& ParentCmdList;
	int32 Width;
	int32 NumAlloc;
	int32 MinDrawsPerCommandList;

private:
	TArray<FRHICommandListImmediate::FQueuedCommandList, SceneRenderingAllocator> QueuedCommandLists;

protected:
	//this must be called by deriving classes virtual destructor because it calls the virtual SetStateOnCommandList.
	//C++ will not do dynamic dispatch of virtual calls from destructors so we can't call it in the base class.
	void Dispatch(bool bHighPriority = false);
	FRHICommandList* AllocCommandList();
	bool bHasRenderPasses;

public:
	FParallelCommandListSet(const FRDGPass* InPass, const FViewInfo& InView, FRHICommandListImmediate& InParentCmdList, bool bHasRenderPasses = true);

	FParallelCommandListSet(const FRDGPass* InPass, TStatId InExecuteStat, const FViewInfo& InView, FRHICommandListImmediate& InParentCmdList, bool bHasRenderPasses = true)
		: FParallelCommandListSet(InPass, InView, InParentCmdList, bHasRenderPasses)
	{}

	virtual ~FParallelCommandListSet();

	int32 NumParallelCommandLists() const
	{
		return QueuedCommandLists.Num();
	}

	FRHICommandList* NewParallelCommandList();

	FORCEINLINE FGraphEventArray* GetPrereqs()
	{
		return nullptr;
	}

	void AddParallelCommandList(FRHICommandList* CmdList);

	virtual void SetStateOnCommandList(FRHICommandList& CmdList) {}
};

class UE_DEPRECATED(5.5, "Use GraphBuilder.AddDispatchPass instead") FRDGParallelCommandListSet final : public FParallelCommandListSet
{
public:
	FRDGParallelCommandListSet(
		const FRDGPass* InPass,
		FRHICommandListImmediate& InParentCmdList,
		const FViewInfo& InView,
		const FParallelCommandListBindings& InBindings,
		float InViewportScale = 1.0f)
		: FParallelCommandListSet(InPass, InView, InParentCmdList, InBindings.bHasRenderPassInfo)
		, Bindings(InBindings)
		, ViewportScale(InViewportScale)
	{}

	FRDGParallelCommandListSet(const FRDGPass* InPass, FRHICommandListImmediate& InParentCmdList, TStatId InStatId, const FViewInfo& InView, const FParallelCommandListBindings& InBindings, float InViewportScale = 1.0f)
		: FRDGParallelCommandListSet(InPass, InParentCmdList, InView, InBindings, InViewportScale)
	{}

	~FRDGParallelCommandListSet() override
	{
		Dispatch(bHighPriority);
	}

	void SetStateOnCommandList(FRHICommandList& RHICmdList) override;

	void SetHighPriority()
	{
		bHighPriority = true;
	}

private:
	FParallelCommandListBindings Bindings;
	float ViewportScale;
	bool bHighPriority = false;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

enum EVolumeUpdateType
{
	VUT_MeshDistanceFields = 1,
	VUT_Heightfields = 2,
	VUT_All = VUT_MeshDistanceFields | VUT_Heightfields
};

class FVolumeUpdateRegion
{
public:

	FVolumeUpdateRegion() :
		UpdateType(VUT_All)
	{}

	/** World space bounds. */
	FBox Bounds;

	/** Number of texels in each dimension to update. */
	FIntVector CellsSize;

	EVolumeUpdateType UpdateType;
};

class FClipmapUpdateBounds
{
public:
	FClipmapUpdateBounds()
		: Center(0.0f, 0.0f, 0.0f)
		, bExpandByInfluenceRadius(false)
		, Extent(0.0f, 0.0f, 0.0f)
	{
	}

	FClipmapUpdateBounds(const FVector& InCenter, const FVector& InExtent, bool bInExpandByInfluenceRadius)
		: Center(InCenter)
		, bExpandByInfluenceRadius(bInExpandByInfluenceRadius)
		, Extent(InExtent)
	{
	}

	FVector Center;
	bool bExpandByInfluenceRadius;
	FVector Extent;
};

enum class EGlobalSDFFullRecaptureReason
{
	None,
	TooManyUpdateBounds,
	HeightfieldStreaming,
	MeshSDFStreaming,
	NoViewState
};

class FGlobalDistanceFieldClipmap
{
public:
	/** World space bounds. */
	FBox Bounds;

	/** Offset applied to UVs so that only new or dirty areas of the volume texture have to be updated. */
	FVector ScrollOffset;

	EGlobalSDFFullRecaptureReason FullRecaptureReason = EGlobalSDFFullRecaptureReason::None;

	// Bounds in the volume texture to update.
	TArray<FClipmapUpdateBounds, TInlineAllocator<64>> UpdateBounds;
};

class FGlobalDistanceFieldInfo
{
public:
	bool bInitialized = false;

	TArray<FGlobalDistanceFieldClipmap> MostlyStaticClipmaps;
	TArray<FGlobalDistanceFieldClipmap> Clipmaps;

	FGlobalDistanceFieldParameterData ParameterData;

	FRDGPooledBuffer* PageFreeListAllocatorBuffer = nullptr;
	FRDGPooledBuffer* PageFreeListBuffer = nullptr;
	IPooledRenderTarget* PageAtlasTexture = nullptr;
	IPooledRenderTarget* CoverageAtlasTexture = nullptr;
	FRDGPooledBuffer* PageObjectGridBuffer = nullptr;
	IPooledRenderTarget* PageTableCombinedTexture = nullptr;
	IPooledRenderTarget* PageTableLayerTextures[GDF_Num] = {};
	IPooledRenderTarget* MipTexture = nullptr;

	void UpdateParameterData(float MaxOcclusionDistance, bool bLumenEnabled, float LumenSceneViewDistance, FVector PreViewTranslation);
};

const int32 GMaxForwardShadowCascades = 4;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT_WITH_CONSTRUCTOR(FForwardLightUniformParameters, )
	SHADER_PARAMETER(uint32, NumLocalLights)
	SHADER_PARAMETER(uint32, NumDirectionalLights)
	SHADER_PARAMETER(uint32, NumReflectionCaptures)
	SHADER_PARAMETER(uint32, HasDirectionalLight)
	SHADER_PARAMETER(uint32, NumGridCells)
	SHADER_PARAMETER(uint32, LightNameDwords)
	SHADER_PARAMETER(FIntVector, CulledGridSize)
	SHADER_PARAMETER(uint32, MaxCulledLightsPerCell)
	SHADER_PARAMETER(uint32, LightGridPixelSizeShift)
	SHADER_PARAMETER(FVector3f, LightGridZParams)
	SHADER_PARAMETER(FVector3f, DirectionalLightDirection)
	SHADER_PARAMETER(float, DirectionalLightSourceRadius)
	SHADER_PARAMETER(float, DirectionalLightSoftSourceRadius)
	SHADER_PARAMETER(FVector3f, DirectionalLightColor)
	SHADER_PARAMETER(float, DirectionalLightVolumetricScatteringIntensity)
	SHADER_PARAMETER(float, DirectionalLightSpecularScale)
	SHADER_PARAMETER(float, DirectionalLightDiffuseScale)
	SHADER_PARAMETER(uint32, DirectionalLightSceneInfoExtraDataPacked)
	SHADER_PARAMETER(FVector2f, DirectionalLightDistanceFadeMAD)
	SHADER_PARAMETER(int32, DirectionalLightVSM)
	SHADER_PARAMETER(uint32, DirectionalLightHandledByMegaLights)
	SHADER_PARAMETER(uint32, DirectionalMegaLightsSupportedStartIndex)
	SHADER_PARAMETER(uint32, DirectLightingShowFlag)
	SHADER_PARAMETER(uint32, CulledBufferOffsetISR)
	SHADER_PARAMETER(uint32, DirectionalLightLightFunctionAtlasLightIndex)
	SHADER_PARAMETER(uint32, DirectionalLightAffectsTranslucentLighting)
	SHADER_PARAMETER(FVector4f, PreViewTranslationOffsetISR)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float4>, ForwardLightBuffer)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, NumCulledLightsGrid)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, CulledLightDataGrid32Bit)
	SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, CulledLightDataGrid16Bit)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<uint>, DirectionalLightIndices)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FLightViewData>, LightViewData)
	SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, LightNamesBuffer)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT_WITH_CONSTRUCTOR(FForwardDirectionalLightShadowMapParameters, )
	SHADER_PARAMETER(uint32, NumDirectionalLightCascades)
	SHADER_PARAMETER(FVector4f, CascadeEndDepths)
	SHADER_PARAMETER_ARRAY(FMatrix44f, DirectionalLightTranslatedWorldToShadowMatrix, [GMaxForwardShadowCascades])
	SHADER_PARAMETER_ARRAY(FVector4f, DirectionalLightShadowmapMinMax, [GMaxForwardShadowCascades])
	SHADER_PARAMETER(FVector4f, DirectionalLightShadowmapAtlasBufferSize)
	SHADER_PARAMETER(float, DirectionalLightDepthBias)
	SHADER_PARAMETER(uint32, DirectionalLightUseStaticShadowing)
	SHADER_PARAMETER(FVector4f, DirectionalLightStaticShadowBufferSize)
	SHADER_PARAMETER(FMatrix44f, DirectionalLightTranslatedWorldToStaticShadow)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, DirectionalLightShadowmapAtlas)
	SHADER_PARAMETER_SAMPLER(SamplerState, ShadowmapSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, DirectionalLightStaticShadowmap)
	SHADER_PARAMETER_SAMPLER(SamplerState, StaticShadowmapSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

extern TRDGUniformBufferRef<FForwardLightUniformParameters> CreateDummyForwardLightUniformBuffer(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform);
extern TRDGUniformBufferRef<FForwardDirectionalLightShadowMapParameters> CreateDummyForwardDirLightShadowUniformBuffer(FRDGBuilder& GraphBuilder);
extern void SetDummyForwardLightUniformBufferOnViews(FRDGBuilder& GraphBuilder, EShaderPlatform ShaderPlatform, TArray<FViewInfo>& Views);

class FForwardLightingViewResources
{
public:
	void SetUniformBuffer(TRDGUniformBufferRef<FForwardLightUniformParameters> UniformBuffer)
	{
		check(UniformBuffer);
		ForwardLightUniformBuffer = UniformBuffer;
		ForwardLightUniformParameters = UniformBuffer->GetContents();
	}

	void SetShadowUniformBuffer(TRDGUniformBufferRef<FForwardDirectionalLightShadowMapParameters> UniformBuffer)
	{
		check(UniformBuffer);
		ForwardDirLightShadowUniformBuffer = UniformBuffer;
		ForwardDirLightShadowParameters = UniformBuffer->GetContents();
	}

	const FForwardLightUniformParameters* ForwardLightUniformParameters = nullptr;
	TRDGUniformBufferRef<FForwardLightUniformParameters> ForwardLightUniformBuffer = nullptr;

	const FForwardDirectionalLightShadowMapParameters* ForwardDirLightShadowParameters = nullptr;
	TRDGUniformBufferRef<FForwardDirectionalLightShadowMapParameters> ForwardDirLightShadowUniformBuffer = nullptr;

	const FLightSceneProxy* SelectedForwardDirectionalLightProxy = nullptr;
	int32 SelectedForwardDirectionalLightIndex = INDEX_NONE;

	// Buffers shared between primary and secondary view in single-pass stereo
	FRDGBufferSRVRef CulledLightDataGridSRV;
	FRDGBufferUAVRef CulledLightDataGridUAV;
	FRDGBufferSRVRef NumCulledLightsGridSRV;
	FRDGBufferUAVRef NumCulledLightsGridUAV;

	FRDGBufferRef LightPowerHistoryRatioBuffer = nullptr;

	TConstArrayView<VisualizeLightShape::FLightInfo> VisualizeLightShapeLightInfos;
	FRDGBufferRef VisualizeLightShapeLightInfoBuffer = nullptr;
};

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT_WITH_CONSTRUCTOR(FVolumetricFogGlobalData,) 
	SHADER_PARAMETER(FIntVector,ViewGridSizeInt)
	SHADER_PARAMETER(FVector3f, ViewGridSize)
	SHADER_PARAMETER(FIntVector,ResourceGridSizeInt)
	SHADER_PARAMETER(FVector3f, ResourceGridSize)
	SHADER_PARAMETER(FVector3f, GridZParams)
	SHADER_PARAMETER(FVector2f, SVPosToVolumeUV)
	SHADER_PARAMETER(float, MaxDistance)
	SHADER_PARAMETER(float, LightSoftFading)
	SHADER_PARAMETER(FVector3f, HeightFogInscatteringColor)
	SHADER_PARAMETER(FVector3f, HeightFogDirectionalLightInscatteringColor)
	SHADER_PARAMETER(FIntPoint, FogGridToPixelXY)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

extern void SetupVolumetricFogGlobalData(const FViewInfo& View, FVolumetricFogGlobalData& Parameters);

struct FTransientLightFunctionTextureAtlasTile
{
	bool bIsDefault;		// If true, then the atlas item generation can be skipped
	FRDGTextureRef Texture;
	FIntRect RectBound;
	FVector4f MinMaxUvBound;
};

struct FVolumetricFogLocalLightFunctionInfo
{
	FTransientLightFunctionTextureAtlasTile AtlasTile;
	FMatrix44f LightFunctionTranslatedWorldToLightMatrix;
};

class FVolumetricFogViewResources
{
public:
	TUniformBufferRef<FVolumetricFogGlobalData> VolumetricFogGlobalData;

	FRDGTextureRef IntegratedLightScatteringTexture = nullptr;

	FVolumetricFogViewResources()
	{}

	void Release()
	{
		IntegratedLightScatteringTexture = nullptr;
	}
};

struct FVolumetricMeshBatch
{
	const FMeshBatch* Mesh;
	const FPrimitiveSceneProxy* Proxy;

	FVolumetricMeshBatch(const FMeshBatch* M, const FPrimitiveSceneProxy* P)
		: Mesh(M)
		, Proxy(P)
	{}

	FORCEINLINE bool operator==(const FVolumetricMeshBatch& rhs) const
	{
		return (Mesh->MeshIdInPrimitive == rhs.Mesh->MeshIdInPrimitive) && (Proxy == rhs.Proxy);
	}
};

struct FSkyMeshBatch
{
	const FMeshBatch* Mesh;
	const FPrimitiveSceneProxy* Proxy;
	bool bVisibleInMainPass : 1;
	bool bVisibleInRealTimeSkyCapture : 1;
};

struct FSortedTrianglesMeshBatch
{
	const FMeshBatch* Mesh = nullptr;
	const FPrimitiveSceneProxy* Proxy = nullptr;
};

// DX11 maximum 2d texture array size is D3D11_REQ_TEXTURE2D_ARRAY_AXIS_DIMENSION = 2048, and 2048/6 = 341.33.
UE_DEPRECATED(5.4, "Using GMaxNumReflectionCaptures directly is deprecated. Please use GetMaxNumReflectionCaptures(EShaderPlatform) instead")
static const int32 GMaxNumReflectionCaptures = 341;

// Maximum number of concurrent runtime-reflection-capture smooth blends (cubemap-array slots reserved as blend
// sources, plus the corresponding shader-side blend table size).  Actual blend slot count is defined in
// r.ReflectionCapture.Runtime.SmoothBlendSlots, and clamped to this value.
static constexpr int32 GMaxRuntimeReflectionCaptureSmoothBlendSlots = 4;

/** Per-reflection capture data needed by the shader. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FReflectionCaptureShaderData,)
	SHADER_PARAMETER_ARRAY(FVector4f,PositionHighAndRadius,[GMaxNumReflectionCaptures])
	// W is smooth blend info for runtime captures -- integer portion is source cube map index, fractional is blend, with 0 representing no active blend
	SHADER_PARAMETER_ARRAY(FVector4f,PositionLow,[GMaxNumReflectionCaptures])
	// R is brightness, G is array index, B is shape, A is the fade-in alpha (0..1) for runtime captures
	SHADER_PARAMETER_ARRAY(FVector4f,CaptureProperties,[GMaxNumReflectionCaptures])
	SHADER_PARAMETER_ARRAY(FVector4f,CaptureOffsetAndAverageBrightness,[GMaxNumReflectionCaptures])
	// Stores the box transform for a box shape, other data is packed for other shapes
	SHADER_PARAMETER_ARRAY(FMatrix44f,BoxTransform,[GMaxNumReflectionCaptures])
	SHADER_PARAMETER_ARRAY(FVector4f,BoxScales,[GMaxNumReflectionCaptures])
END_GLOBAL_SHADER_PARAMETER_STRUCT()
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UE_DEPRECATED(5.4, "Using GMobileMaxNumReflectionCaptures directly is deprecated. Please use GetMaxNumReflectionCaptures(EShaderPlatform) instead")
static const int32 GMobileMaxNumReflectionCaptures = 100;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FMobileReflectionCaptureShaderData, )
	SHADER_PARAMETER_ARRAY(FVector4f,PositionHighAndRadius,[GMobileMaxNumReflectionCaptures])
	// W is smooth blend info for runtime captures -- integer portion is source cube map index, fractional is blend, with 0 representing no active blend
	SHADER_PARAMETER_ARRAY(FVector4f,PositionLow,[GMobileMaxNumReflectionCaptures])
	// R is brightness, G is array index, B is shape, A is the fade-in alpha (0..1) for runtime captures
	SHADER_PARAMETER_ARRAY(FVector4f,CaptureProperties,[GMobileMaxNumReflectionCaptures])
	SHADER_PARAMETER_ARRAY(FVector4f,CaptureOffsetAndAverageBrightness,[GMobileMaxNumReflectionCaptures])
	// Stores the box transform for a box shape, other data is packed for other shapes
	SHADER_PARAMETER_ARRAY(FMatrix44f,BoxTransform,[GMobileMaxNumReflectionCaptures])
	SHADER_PARAMETER_ARRAY(FVector4f,BoxScales,[GMobileMaxNumReflectionCaptures])
END_GLOBAL_SHADER_PARAMETER_STRUCT()
PRAGMA_ENABLE_DEPRECATION_WARNINGS

extern int32 GetMaxNumReflectionCaptures(EShaderPlatform ShaderPlatform);

// Structure in charge of storing all information about TAA's history.
struct FTemporalAAHistory
{
	// Number of render target in the history.
	static constexpr int32 kRenderTargetCount = 2;

	// Render targets holding's pixel history.
	//  scene color's RGBA are in OutputRT[0].
	TStaticArray<TRefCountPtr<IPooledRenderTarget>, kRenderTargetCount> RT;

	// Reference size of RT. Might be different than RT's actual size to handle down res.
	FIntPoint ReferenceBufferSize;

	// Viewport coordinate of the history in RT according to ReferenceBufferSize.
	FIntRect ViewportRect;

	// Returns the slice index that contains the output in RT[0].
	int32 OutputSliceIndex = 0;


	void SafeRelease()
	{
		*this = FTemporalAAHistory();
	}

	bool IsValid() const
	{
		return RT[0].IsValid();
	}

	uint64 GetGPUSizeBytes(bool bLogSizes) const;
};

// Structure in charge of storing all information about TSR's history.
struct FTSRHistory
{
	// Output resolution.
	TRefCountPtr<IPooledRenderTarget> ColorArray;
	TRefCountPtr<IPooledRenderTarget> MetadataArray;

	// Input resolution representation of the output
	TRefCountPtr<IPooledRenderTarget> GuideArray;
	TRefCountPtr<IPooledRenderTarget> MoireArray;
	TRefCountPtr<IPooledRenderTarget> CoverageArray;

	// Frame's input and output resolution.
	FIntRect InputViewportRect;
	FIntRect OutputViewportRect;

	// Format of the history for auto camera cut when setting change.
	uint32 FormatBit = 0;

	// Number of frame in history.
	int32 FrameStorageCount = 1;
	int32 FrameStoragePeriod = 1;
	int32 AccumulatedFrameCount = 1;
	int32 LastFrameRollingIndex = 0;

	// All the information the previous frames for resurrection.
	TArray<FViewMatrices> ViewMatrices;
	TArray<float> SceneColorPreExposures;
	TArray<FIntRect> InputViewportRects;
	TArray<TRefCountPtr<IPooledRenderTarget>> DistortingDisplacementTextures;


	void SafeRelease()
	{
		*this = FTSRHistory();
	}

	bool IsValid() const
	{
		return MetadataArray.IsValid();
	}

	uint64 GetGPUSizeBytes(bool bLogSizes) const;
};

/** Temporal history for a denoiser. */
struct FScreenSpaceDenoiserHistory
{
	// Number of history render target to store.
	static constexpr int32 RTCount = 3;

	// Scissors of valid data in the render target (can be multiple if there are split screen views)
	TArray<FIntRect, TInlineAllocator<1>> Scissors;

	// Render target specific to the history.
	TStaticArray<TRefCountPtr<IPooledRenderTarget>, RTCount> RT;

	// The texture for tile classification.
	TRefCountPtr<IPooledRenderTarget> TileClassification;


	void SafeRelease()
	{
		for (int32 i = 0; i < RTCount; i++)
			RT[i].SafeRelease();
		TileClassification.SafeRelease();
	}

	bool IsValid() const
	{
		return RT[0].IsValid();
	}

	uint64 GetGPUSizeBytes(bool bLogSizes) const;
};



// Structure for storing a frame of GTAO history.
struct FGTAOTAAHistory
{
	// Render targets holding a frame's pixel history.
	//  scene color's RGBA are in RT[0].
	TRefCountPtr<IPooledRenderTarget> RT;

	// Reference size of RT. Might be different than RT's actual size to handle down res.
	FIntPoint ReferenceBufferSize;

	// Viewport coordinate of the history in RT according to ReferenceBufferSize.
	FIntRect ViewportRect;

	void SafeRelease()
	{
		RT.SafeRelease();
	}

	bool IsValid() const
	{
		return RT.IsValid();
	}
};

// Structure that hold all information related to previous frame.
struct FPreviousViewInfo
{
	// View rect
	FIntRect ViewRect;

	// View matrices.
	FViewMatrices	ViewMatrices;

	// Scene color's PreExposure.
	float SceneColorPreExposure = 1.0f;

	bool bUsesGlobalDistanceField = false;

	// Depth buffer and Normals of the previous frame generating this history entry for bilateral kernel rejection.
	TRefCountPtr<IPooledRenderTarget> DepthBuffer;
	TRefCountPtr<IPooledRenderTarget> GBufferA;
	TRefCountPtr<IPooledRenderTarget> GBufferB;
	TRefCountPtr<IPooledRenderTarget> GBufferC;

	TRefCountPtr<IPooledRenderTarget> HZB;
	TRefCountPtr<IPooledRenderTarget> NaniteHZB;

	// Distorting displacement texture applied.
	TRefCountPtr<IPooledRenderTarget> DistortingDisplacementTexture;

	// Bit mask used to interpret per-instance occlusion query results for this view.
	// Expected to contain a single active bit or zero if instance occlusion query data is not available.
	// See FInstanceCullingOcclusionQueryRenderer.
	uint32 InstanceOcclusionQueryMask = 0;

	// Compressed scene textures for bandwidth efficient bilateral kernel rejection.
	// DeviceZ as float16, and normal in view space.
	TRefCountPtr<IPooledRenderTarget> CompressedDepthViewNormal;

	// 16bit compressed depth buffer with opaque only.
	TRefCountPtr<IPooledRenderTarget> CompressedOpaqueDepth;

	// R8_UINT Shading model ID with opaque only.
	TRefCountPtr<IPooledRenderTarget> CompressedOpaqueShadingModel;

	// Bleed free scene color to use for screen space ray tracing.
	TRefCountPtr<IPooledRenderTarget> ScreenSpaceRayTracingInput;

	// Temporal AA result of last frame
	FTemporalAAHistory TemporalAAHistory;

	// Temporal Super Resolution result of last frame
	FTSRHistory TSRHistory;

	// Custom Temporal AA result of last frame, used by plugins
	TRefCountPtr<UE::Renderer::Private::ITemporalUpscaler::IHistory> ThirdPartyTemporalUpscalerHistory;

	// History of all mega filters.
	TRefCountPtr<UE::Renderer::Private::IMegaFilters::IHistory> MegaFiltersHistory;

	// Half resolution version temporal AA result of last frame
	TRefCountPtr<IPooledRenderTarget> HalfResTemporalAAHistory;

	// Temporal AA history for diaphragm DOF.
	FTemporalAAHistory DOFSetupHistory;
	
	// Temporal AA history for SSR
	FTemporalAAHistory SSRHistory;
	FTemporalAAHistory WaterSSRHistory;

	// Temporal AA history for Rough refraction
	FTemporalAAHistory RoughRefractionHistory;

	// Temporal AA history for Fog Screen Space Scattering
	FTemporalAAHistory FSSSHistory;

	// Temporal AA history for Hair
	FTemporalAAHistory HairHistory;

#if UE_ENABLE_DEBUG_DRAWING
	// Temporal AA history for the editor primitive depth upsampling
	FTemporalAAHistory CompositePrimitiveDepthHistory;
#endif

	// Scene color input for SSR, that can be different from TemporalAAHistory.RT[0] if there is a SSR
	// input post process material.
	FTemporalAAHistory CustomSSRInput;

	// History for the reflections
	FScreenSpaceDenoiserHistory ReflectionsHistory;
	FScreenSpaceDenoiserHistory WaterReflectionsHistory;
	
	// History for the ambient occlusion
	FScreenSpaceDenoiserHistory AmbientOcclusionHistory;

	// History for GTAO
	FGTAOTAAHistory				 GTAOHistory;

	// History for global illumination
	FScreenSpaceDenoiserHistory DiffuseIndirectHistory;

	// History for sky light
	FScreenSpaceDenoiserHistory SkyLightHistory;

	// History for reflected sky light
	FScreenSpaceDenoiserHistory ReflectedSkyLightHistory;

	// History for shadow denoising.
	TMap<FLightComponentId, TSharedPtr<FScreenSpaceDenoiserHistory>> ShadowHistories;

	// History for denoising all lights penumbra at once.
	FScreenSpaceDenoiserHistory PolychromaticPenumbraHarmonicsHistory;

	// History for the final back buffer luminance
	TRefCountPtr<IPooledRenderTarget> LuminanceHistory;

	// History for the final back buffer luminance view rect
	FIntRect LuminanceViewRectHistory;

	// Mobile bloom setup eye adaptation surface.
	TRefCountPtr<IPooledRenderTarget> MobileBloomSetup_EyeAdaptation;

	// View size of the EyeAdaptation texture
	FIntRect MobileBloomSetup_EyeAdaptationViewSize;

	// Scene color used for reprojecting next frame to verify the motion vector reprojects correctly.
	TRefCountPtr<IPooledRenderTarget> VisualizeMotionVectors;
	FIntRect VisualizeMotionVectorsRect;
	bool bIsVisualizeMotionVectorsDistorted = false;

	uint64 GetGPUSizeBytes(bool bLogSizes) const;
};

#if RHI_RAYTRACING

namespace RayTracing
{
	enum class ECullingMode : uint8;
};

struct FRayTracingCullingParameters
{
	RayTracing::ECullingMode CullingMode;
	float CullingRadius;
	float FarFieldCullingRadius;
	float CullAngleThreshold;
	float AngleThresholdRatio;
	float AngleThresholdRatioSq;
	FVector ViewOrigin;
	FVector ViewDirection;
	FVector3f TranslatedViewOrigin;
	bool bCullAllObjects;
	bool bCullByRadiusOrDistance;
	bool bIsRayTracingFarField;
	bool bCullUsingGroupIds;
	bool bCullMinDrawDistance;
	bool bUseInstanceCulling;
	bool bUseDistanceToBox;

	void Init(FViewInfo& View);
};

#endif

struct FPrimitiveInstanceRange
{
	int32 PrimitiveIndex;
	int32 InstanceSceneDataOffset;
	int32 NumInstances;
};

enum class EViewSnapshotType
{
	// The copy contains all members for rendering. Must be done on the rendering thread.
	Main,

	// The copy contains only members relevant for shadow views.
	Shadow
};

/** A FSceneView with additional state used by the scene renderer. */
class FViewInfo : public FSceneView
{
public:
	FSceneRenderingBulkObjectAllocator Allocator;

	/* Final position of the view in the final render target (in pixels), potentially scaled by ScreenPercentage */
	FIntRect ViewRect;
	
	/* Final position of the view in the final render target (in pixels), potentially scaled by the Upper Bound of ScreenPercentage */
	FIntRect UpperBoundViewRect;

	/** 
	 * The view's state, or NULL if no state exists.
	 * This should be used internally to the renderer module to avoid having to cast View.State to an FSceneViewState*
	 */
	FSceneViewState* ViewState;

	/** Cached view uniform shader parameters, to allow recreating the view uniform buffer without having to fill out the entire struct. */
	TUniquePtr<FViewUniformShaderParameters> CachedViewUniformShaderParameters;

	/** A map from primitive ID to a boolean visibility value. */
	FSceneBitArray PrimitiveVisibilityMap;

	/** A map from primitive ID to a boolean ray tracing visibility value. */
	FSceneBitArray PrimitiveRayTracingVisibilityMap;

	/** Bit set when a primitive is known to be un-occluded. */
	FSceneBitArray PrimitiveDefinitelyUnoccludedMap;

	/** A map from primitive ID to a boolean is fading value. */
	FSceneBitArray PotentiallyFadingPrimitiveMap;

	/** Primitive fade uniform buffers, indexed by packed primitive index. */
	TArray<FRHIUniformBuffer*,SceneRenderingAllocator> PrimitiveFadeUniformBuffers;

	/**  Bit set when a primitive has a valid fade uniform buffer. */
	FSceneBitArray PrimitiveFadeUniformBufferMap;

	/** One frame dither fade in uniform buffer. */
	FUniformBufferRHIRef DitherFadeInUniformBuffer;

	/** One frame dither fade out uniform buffer. */
	FUniformBufferRHIRef DitherFadeOutUniformBuffer;

	/** A map from primitive ID to the primitive's view relevance. */
	TArray<FPrimitiveViewRelevance,SceneRenderingAllocator> PrimitiveViewRelevanceMap;

	/** A map from static mesh ID to a boolean visibility value. */
	FSceneBitArray StaticMeshVisibilityMap;

	/** A map from static mesh ID to a boolean dithered LOD fade out value. */
	FSceneBitArray StaticMeshFadeOutDitheredLODMap;

	/** A map from static mesh ID to a boolean dithered LOD fade in value. */
	FSceneBitArray StaticMeshFadeInDitheredLODMap;

	/** Will only contain relevant primitives for view and/or shadow */
	TArray<FLODMask, SceneRenderingAllocator> PrimitivesLODMask;

	/** The dynamic primitives with simple lights visible in this view. */
	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> VisibleDynamicPrimitivesWithSimpleLights;

	/** Number of dynamic primitives visible in this view. */
	int32 NumVisibleDynamicPrimitives;

	/** Number of dynamic editor primitives visible in this view. */
	int32 NumVisibleDynamicEditorPrimitives;

	/** Number of dynamic mesh elements per mesh pass (inside FViewInfo::DynamicMeshElements). */
	int32 NumVisibleDynamicMeshElements[EMeshPass::Num];

	/** List of visible primitives with dirty indirect lighting cache buffers */
	TArray<FPrimitiveSceneInfo*,SceneRenderingAllocator> DirtyIndirectLightingCacheBufferPrimitives;
	UE::FMutex DirtyIndirectLightingCacheBufferPrimitivesMutex; 

	/** Maps a single primitive to it's per view translucent self shadow uniform buffer. */
	FTranslucentSelfShadowUniformBufferMap TranslucentSelfShadowUniformBufferMap;

	/** View dependent global distance field clipmap info. */
	FGlobalDistanceFieldInfo& GlobalDistanceFieldInfo = *Allocator.Create<FGlobalDistanceFieldInfo>();

	/** Count of translucent prims for this view. */
	FTranslucenyPrimCount TranslucentPrimCount;
	
	bool bHasDistortionPrimitives;
	bool bHasCustomDepthPrimitives;

    /** Get all stencil values written into the custom depth pass */
	TSet<uint32, DefaultKeyFuncs<uint32>, SceneRenderingSetAllocator> CustomDepthStencilValues;

	/** Get the GPU Scene instance ranges of visible Nanite primitives writing custom depth. */
	TArray<FPrimitiveInstanceRange, SceneRenderingAllocator> NaniteCustomDepthInstances;

	/** Mesh batches with a volumetric material. */
	TArray<FVolumetricMeshBatch, SceneRenderingAllocator> VolumetricMeshBatches;

	/** Mesh batches for heterogeneous volumes rendering. */
	TArray<FVolumetricMeshBatch, SceneRenderingAllocator> HeterogeneousVolumesMeshBatches;

	/** Mesh batches with a sky material. */
	TArray<FSkyMeshBatch, SceneRenderingAllocator> SkyMeshBatches;

	/** Mesh batches with a triangle sorting. */
	TArray<FSortedTrianglesMeshBatch, SceneRenderingAllocator> SortedTrianglesMeshBatches;

	/** A map from light ID to a boolean visibility value. */
	TSceneRendererLightArray<FVisibleLightViewInfo> VisibleLightInfos;

	/** Tracks the list of visible reflection capture lights that need to add meshes to the view. */
	TArray<const FLightSceneProxy*, SceneRenderingAllocator> VisibleReflectionCaptureLights;

	/** The view's batched elements. */
	FBatchedElements& BatchedViewElements = *Allocator.Create<FBatchedElements>();

	/** The view's batched elements, above all other elements, for gizmos that should never be occluded. */
	FBatchedElements& TopBatchedViewElements = *Allocator.Create<FBatchedElements>();

	/** The view's mesh elements. */
	TIndirectArray<FMeshBatch, SceneRenderingAllocator> ViewMeshElements;

	/** The view's mesh elements for the foreground (editor gizmos and primitives )*/
	TIndirectArray<FMeshBatch, SceneRenderingAllocator> TopViewMeshElements;

	/** The dynamic resources used by the view elements. */
	TArray<FDynamicPrimitiveResource*, SceneRenderingAllocator> DynamicResources;

	/** Gathered in initviews from all the primitives with dynamic view relevance, used in each mesh pass. */
	TArray<FMeshBatchAndRelevance,SceneRenderingAllocator> DynamicMeshElements;

	/* [PrimitiveIndex] = end index index in DynamicMeshElements[], to support GetDynamicMeshElementRange(). Contains valid values only for visible primitives with bDynamicRelevance. */
	TArray<FInt32Vector2, SceneRenderingAllocator> DynamicMeshElementRanges;

	/** Hair strands & cards dynamic mesh element. */
	TArray<FMeshBatchAndRelevance, SceneRenderingAllocator> HairStrandsMeshElements;
	TArray<FMeshBatchAndRelevance, SceneRenderingAllocator> HairCardsMeshElements;

	/* Mesh pass relevance for gathered dynamic mesh elements. */
	TArray<FMeshPassMask, SceneRenderingAllocator> DynamicMeshElementsPassRelevance;

	/** Gathered in UpdateRayTracingWorld from all the primitives with dynamic view relevance, used in each mesh pass. */
	TArray<FMeshBatchAndRelevance, SceneRenderingAllocator> RayTracedDynamicMeshElements;

	TArray<FMeshBatchAndRelevance,SceneRenderingAllocator> DynamicEditorMeshElements;

	FSimpleElementCollector& SimpleElementCollector = *Allocator.Create<FSimpleElementCollector>();

	FSimpleElementCollector& EditorSimpleElementCollector = *Allocator.Create<FSimpleElementCollector>();;

#if UE_ENABLE_DEBUG_DRAWING
	//Separate DebugSimpleElementCollector to not conflate any other simple element collector draws which may have been added from non-debug draw passes (e.g. non-opaque draws)
	FSimpleElementCollector& DebugSimpleElementCollector = *Allocator.Create<FSimpleElementCollector>();;
#endif

	FParallelMeshDrawCommandPass* CreateMeshPass(EMeshPass::Type MeshPass);

	TStaticArray<FParallelMeshDrawCommandPass*, EMeshPass::Num> ParallelMeshDrawCommandPasses = TStaticArray<FParallelMeshDrawCommandPass*, EMeshPass::Num>(InPlace, nullptr);

#if RHI_RAYTRACING 
	FRayTracingShaderBindingDataOneFrameArray VisibleRayTracingShaderBindings;
	FDynamicRayTracingMeshCommandStorage DynamicRayTracingMeshCommandStorage;
	TSet<FRDGBuffer*> DynamicRayTracingRDGBuffers;

	FRayTracingCullingParameters RayTracingCullingParameters;

	TArray<UE::Tasks::FTask> AddDynamicRayTracingMeshBatchTaskList;
	TArray<FRayTracingShaderBindingDataOneFrameArray*, SceneRenderingAllocator> DynamicRayTracingShaderBindingsPerTask;
	TArray<FDynamicRayTracingMeshCommandStorage*, SceneRenderingAllocator> DynamicRayTracingMeshCommandStoragePerTask;
#endif // RHI_RAYTRACING

	// Used by mobile renderer to determine whether static meshes will be rendered with CSM shaders or not.
	FMobileCSMVisibilityInfo MobileCSMVisibilityInfo;

	FSubstrateViewData& SubstrateViewData = *Allocator.Create<FSubstrateViewData>();

	FLocalFogVolumeViewData& LocalFogVolumeViewData = *Allocator.Create<FLocalFogVolumeViewData>();

	FHairStrandsViewData& HairStrandsViewData = *Allocator.Create<FHairStrandsViewData>();

	LightFunctionAtlas::FLightFunctionAtlasViewData LightFunctionAtlasViewData;

	/** Parameters for exponential height fog. */
	FVector4f ExponentialFogParameters;
	FVector4f ExponentialFogParameters2;
	FVector3f ExponentialFogColor;
	float FogMaxOpacity;
	FVector4f ExponentialFogParameters3;
	FVector4f SkyAtmosphereAmbientContributionColorScale;
	float FogEndDistance;
	bool bEnableVolumetricFog;
	float VolumetricFogStartDistance;
	float VolumetricFogNearFadeInDistanceInv;
	FVector3f VolumetricFogAlbedo;
	float VolumetricFogPhaseG;
	FVector2f SinCosInscatteringColorCubemapRotation;

	UTexture* FogInscatteringColorCubemap;
	FVector FogInscatteringTextureParameters;

	/** Parameters for directional inscattering of exponential height fog. */
	bool bUseDirectionalInscattering;
	float DirectionalInscatteringExponent;
	float DirectionalInscatteringStartDistance;
	FVector InscatteringLightDirection;
	FLinearColor DirectionalInscatteringColor;

	/** Translucency lighting volume properties. */
	FVector TranslucencyLightingVolumeMin[TVC_MAX];
	float TranslucencyVolumeVoxelSize[TVC_MAX];
	FVector TranslucencyLightingVolumeSize[TVC_MAX];

	/** Optional source view for temporal AA, to handle custom render passes and scene captures sharing the main view's camera (jitter needs to match) */
	FViewInfo* TemporalSourceView;

	/** Number of samples in the temporal AA sequqnce */
	int32 TemporalJitterSequenceLength;

	/** Index of the temporal AA jitter in the sequence. */
	int32 TemporalJitterIndex;

	/** Temporal AA jitter at the pixel scale. */
	FVector2D TemporalJitterPixels;

	/** Whether FSceneViewState::PrevFrameViewInfo can be updated with this view. */
	uint32 bStatePrevViewInfoIsReadOnly : 1;

	/** true if all PrimitiveVisibilityMap's bits are set to false. */
	uint32 bHasNoVisiblePrimitive : 1;

	/** true if the view has at least one mesh with a translucent material. */
	uint32 bHasTranslucentViewMeshElements : 1;
	/** Whether we should ignore queries from last frame (useful to ignoring occlusions on the first frame after a large camera movement). */
	uint32 bIgnoreExistingQueries : 1;
	/** Whether we should submit new queries this frame. (used to disable occlusion queries completely. */
	uint32 bDisableQuerySubmissions : 1;
	/** Whether the view has any materials that use the global distance field. */
	uint32 bUsesGlobalDistanceField : 1;
	uint32 bUsesLightingChannels : 1;
	uint32 bTranslucentSurfaceLighting : 1;
	uint32 bCustomDepthStencilValid : 1;
	uint32 bUsesCustomDepth : 1;
	uint32 bUsesCustomStencil : 1;

	/** Whether the view has any materials that uses motion vector offset to apply per pixel modification of velocities*/
	uint32 bUsesMotionVectorWorldOffset : 1;

	/** Whether the view has any Single Layer Water materials that use motion vector world offset. */
	uint32 bSLWUsesMotionVectorWorldOffset : 1;

	/** Whether fog should only be computed on rendered opaque pixels or not. */
	uint32 bFogOnlyOnRenderedOpaque : 1;

	/**
	 * true if the scene has at least one mesh with a material tagged as sky. 
	 * This is used to skip the sky rendering part during the SkyAtmosphere pass on non mobile platforms.
	 */
	uint32 bSceneHasSkyMaterial : 1;
	/**
	 * true if the scene has at least one mesh with a material tagged as water visible in a view.
	 */
	uint32 bHasSingleLayerWaterMaterial : 1;
	/**
	 * True if the scene has at least one mesh that needs to sample form the first stage depth buffer.
	 * And as such will need to render in the second stage depth buffer after the first stage depth buffer is copied.
	 * The first stage depth buffer is usually used for depth buffer collision and projection of Niagara's particles.
	 */
	uint32 bUsesSecondStageDepthPass : 1;

	/**
	 * Set to true if this is a scene capture sharing temporal AA jitter with the main view camera.  Needed to force temporal jitter
	 * logic to run when post processing is disabled for the scene capture, which otherwise disables jitter.
	 */
	uint32 bSceneCaptureMainViewJitter : 1;

	/** Whether post DOF translucency should be rendered before DOF if primitive bounds behind DOF's focus distance. */
	float AutoBeforeDOFTranslucencyBoundary;

	/** Bitmask of all shading models used by primitives in this view */
	uint16 ShadingModelMaskInView;

	/** Informations from the previous frame to use for this view. */
	FPreviousViewInfo& PrevViewInfo = *Allocator.Create<FPreviousViewInfo>();

	/** An intermediate number of visible static meshes.  Doesn't account for occlusion until after FinishOcclusionQueries is called. */
	int32 NumVisibleStaticMeshElements;

	/** Frame's exposure. Always > 0. */
	float PreExposure;

	/** Precomputed visibility data, the bits are indexed by VisibilityId of a primitive component. */
	const uint8* PrecomputedVisibilityData;

	FOcclusionQueryBatcher IndividualOcclusionQueries;
	FOcclusionQueryBatcher GroupedOcclusionQueries;

	// Furthest and closest Hierarchical Z Buffer
	FRDGTextureRef HZB = nullptr;
	FRDGTextureRef ClosestHZB = nullptr;

	struct FTranslucencyVolumeMarkData
	{
		FRDGTextureRef MarkTexture = nullptr;
		FRDGBufferRef VoxelAllocator = nullptr;
		FRDGBufferRef VoxelData = nullptr;
		FRDGBufferRef VoxelIndirectArgs = nullptr;
	};

	TStaticArray<FTranslucencyVolumeMarkData, TVC_MAX> TranslucencyVolumeMarkData = {};

	int32 NumBoxReflectionCaptures;
	int32 NumSphereReflectionCaptures;
	float FurthestReflectionCaptureDistance;
	TUniformBufferRef<FReflectionCaptureShaderData> ReflectionCaptureUniformBuffer;
	TUniformBufferRef<FMobileReflectionCaptureShaderData> MobileReflectionCaptureUniformBuffer;

	// Sky / Atmosphere textures (transient owned by this view info) and pointer to constants owned by SkyAtmosphere proxy.
	TRefCountPtr<IPooledRenderTarget> SkyAtmosphereCameraAerialPerspectiveVolume;
	TRefCountPtr<IPooledRenderTarget> SkyAtmosphereCameraAerialPerspectiveVolumeMieOnly;
	TRefCountPtr<IPooledRenderTarget> SkyAtmosphereCameraAerialPerspectiveVolumeRayOnly;
	TRefCountPtr<IPooledRenderTarget> SkyAtmosphereViewLutTexture;
	const FAtmosphereUniformShaderParameters* SkyAtmosphereUniformShaderParameters;

	FRDGTextureRef VolumetricCloudSkyAO = nullptr;
	TUniformBufferRef<FViewUniformShaderParameters> VolumetricRenderTargetViewUniformBuffer;
	// The effective cloud shadow target this frame independently of the fact that a view can have a state (primary view) or not (sky light reflection capture)
	FRDGTextureRef VolumetricCloudShadowRenderTarget[NUM_ATMOSPHERE_LIGHTS] = {};
	// We need to extract that RDG resource because the RHI must be accessed to setup FTranslucentLightingInjectPS & TVolumetricFogLightScatteringCS
	TRefCountPtr<IPooledRenderTarget> VolumetricCloudShadowExtractedRenderTarget[NUM_ATMOSPHERE_LIGHTS] = {};

	FForwardLightingViewResources ForwardLightingResources;
	FVolumetricFogViewResources VolumetricFogResources;

	bool bLightGridHasRectLights = false;
	bool bLightGridHasTexturedLights = false;
	uint8 LightGridLightingChannelAndMask = 0xFF;

	FRDGTextureRef HeterogeneousVolumeRadiance = nullptr;
	FRDGTextureRef HeterogeneousVolumeHoldout = nullptr;
	FRDGTextureRef HeterogeneousVolumeBeerShadowMap = nullptr;

	// Size of the HZB's mipmap 0
	// NOTE: the mipmap 0 is downsampled version of the depth buffer
	FIntPoint HZBMipmap0Size;

	/** Used by occlusion for percent unoccluded calculations. */
	float OneOverNumPossiblePixels;

	TOptional<FMobileLightShaftInfo> MobileLightShaft;

	FGlobalShaderMap* ShaderMap;

	// Whether this view should use compute passes where appropriate.
	bool bUseComputePasses = false;

	// Optional stencil dithering optimization during prepasses
	bool bAllowStencilDither;

	// Indicates whether previous frame transforms were reset this frame for any reason.
	bool bPrevTransformsReset = false;

	// Max emissive luminance ouput by any material for this view.
	float MaterialMaxEmissiveValue;

	/** Custom visibility query for view */
	ICustomVisibilityQuery* CustomVisibilityQuery;

	const FTexture2DResource* FFTBloomKernelTexture = nullptr;
	const FTexture2DResource* FilmGrainTexture = nullptr;

	TArray<FPrimitiveSceneInfo*, SceneRenderingAllocator> IndirectShadowPrimitives;

	/** Only one of the resources(TextureBuffer or Texture2D) will be used depending on the Mobile.UseGPUSceneTexture cvar */

	FTextureRHIRef PrimitiveSceneDataTextureOverrideRHI;

	FLensDistortionLUT LensDistortionLUT;

	FShaderPrintData ShaderPrintData;

private:
	FLumenTranslucencyGIVolume& LumenTranslucencyGIVolume = *Allocator.Create<FLumenTranslucencyGIVolume>();
	FMegaLightsVolume MegaLightsVolume;

public:
	FFrontLayerTranslucency FrontLayerTranslucency;

	inline const FLumenTranslucencyGIVolume& GetLumenTranslucencyGIVolume() const
	{
		if (UNLIKELY(bIsInstancedStereoEnabled && ShouldUseStereoLumenOptimizations() && IStereoRendering::IsASecondaryPass(StereoPass)))
		{
			return GetPrimaryView()->LumenTranslucencyGIVolume;
		}

		return LumenTranslucencyGIVolume;
	}

	inline FLumenTranslucencyGIVolume& GetOwnLumenTranslucencyGIVolume()
	{
		return LumenTranslucencyGIVolume;
	}

	inline const FMegaLightsVolume& GetMegaLightsVolume() const
	{
		return MegaLightsVolume;
	}

	inline FMegaLightsVolume& GetOwnMegaLightsVolume()
	{
		return MegaLightsVolume;
	}

	FLumenSceneData* ViewLumenSceneData;

#if RHI_RAYTRACING
private:
	FRayTracingScene::FViewHandle RayTracingSceneViewHandle;

public:
	RENDERER_API bool HasRayTracingScene() const;
	RENDERER_API FRHIRayTracingScene* GetRayTracingSceneChecked(ERayTracingSceneLayer Layer) const;
	RENDERER_API FRDGBufferSRVRef GetRayTracingSceneLayerViewChecked(ERayTracingSceneLayer Layer) const;

	FRDGBufferUAVRef GetRayTracingInstanceHitCountUAV(FRDGBuilder& GraphBuilder) const;

	RENDERER_API const FRayTracingScene::FViewHandle& GetRayTracingSceneViewHandle() const;

	inline void SetRayTracingSceneViewHandle(const FRayTracingScene::FViewHandle& InHandle)
	{
		checkf(!IStereoRendering::IsStereoEyePass(StereoPass) || !IStereoRendering::IsASecondaryPass(StereoPass), TEXT("Stereo views should share a single ray tracing scene view."));
		RayTracingSceneViewHandle = InHandle;
	}

	virtual FRDGBufferRef GetInlineRayTracingBindingDataBuffer() const override
	{
		return InlineRayTracingBindingDataBuffer;
	}

	struct FRayTracingData
	{
		FRayTracingPipelineState* PipelineState = nullptr;
		FShaderBindingTableRHIRef ShaderBindingTable;

		TArray<FRayTracingLocalShaderBindingWriter*, SceneRenderingAllocator> MaterialBindings; // One per binding task
		TArray<FRayTracingLocalShaderBindingWriter*, SceneRenderingAllocator> CallableBindings; // One per binding task

		FMemStackBase MaterialBindingsMemory; //< Optional stacked based alloc for binding data
	};

	FRayTracingData MaterialRayTracingData;
	FRayTracingData LumenRayTracingData;
	bool bOwnsShaderBindingTables = true;
	bool bTransientShaderBindingTables = false;

	// Buffer to the shader binding data used for inline raytracing - only valid when inline raytracing is enabled
	FRDGBufferRef InlineRayTracingBindingDataBuffer = nullptr;

	// Buffer that stores the hit group data for Lumen passes that use MinimalPayload and inline ray tracing.
	FRDGBufferRef LumenHardwareRayTracingHitDataBuffer = nullptr;

	// Global Lumen parameters for CHS, AHS and inline
	TUniformBufferRef<FLumenHardwareRayTracingUniformBufferParameters> LumenHardwareRayTracingUniformBuffer;

	// Common resources used for lighting in ray tracing effects
	TRDGUniformBufferRef<FRayTracingLightGrid>			RayTracingLightGridUniformBuffer;
	TRDGUniformBufferRef<FRayTracingDecals>				RayTracingDecalUniformBuffer;
	bool												bHasRayTracingDecals = false;

	int32 PathTracingVolumetricCloudCallableShaderIndex = -1;

	bool bHasAnyRayTracingPass = false;
	bool bHasRayTracingShadows = false;
	bool bRayTracingFeedbackEnabled = false;
#endif // RHI_RAYTRACING

	/**
	 * Index of the view in the AllViews array on the FSceneRenderer. This view ID is transient and only valid during this frame.
	 * Identical to the ID of the view in the GPU instance culling manager. Among other things, used to fetch the culled draw commands.
	 */
	int32 SceneRendererPrimaryViewId;

	/**
	 */
	FPersistentViewId PersistentViewId;

	/* View rect for all instanced views laid out side-by-side. Only primary view will have it populated. 
	 *
	 * This may be different than FamilySize if we're using adaptive resolution stereo rendering. In that case, FamilySize represents the maximum size of
	 * the family to ensure the backing render targets don't change between frames as the view size varies.
	 */
	FIntRect ViewRectWithSecondaryViews;

	/** 
	 * Initialization constructor. Passes all parameters to FSceneView constructor
	 */
	RENDERER_API FViewInfo(const FSceneViewInitOptions& InitOptions);

	/** 
	* Initialization constructor. 
	* @param InView - copy to init with
	*/
	explicit FViewInfo(const FSceneView* InView);

	/** 
	* Destructor. 
	*/
	RENDERER_API virtual ~FViewInfo();

#if DO_CHECK || USING_CODE_ANALYSIS
	/** Verifies all the assertions made on members. */
	bool VerifyMembersChecks() const;
#endif

	/** Returns the size of view rect after primary upscale ( == only with secondary screen percentage). */
	RENDERER_API FIntPoint GetSecondaryViewRectSize() const;

	/** Returns the rectangle to crop to during the secondary upscale */
	RENDERER_API FIntRect GetSecondaryViewCropRect() const;
	
	/** Returns whether the view requires a secondary upscale. */
	bool RequiresSecondaryUpscale() const
	{
		return UnscaledViewRect.Size() != GetSecondaryViewRectSize() || GetSecondaryViewCropRect().Size() != GetSecondaryViewRectSize();
	}

	/** Compute the pre-exposure of internal renderer resources. */
	void UpdatePreExposure();

	/** Creates ViewUniformShaderParameters given a set of view transforms. */
	RENDERER_API void SetupUniformBufferParameters(
		const FViewMatrices& InViewMatrices,
		const FViewMatrices& InPrevViewMatrices,
		FBox* OutTranslucentCascadeBoundsArray, 
		int32 NumTranslucentCascades,
		FViewUniformShaderParameters& ViewUniformShaderParameters) const;

	/** Recreates ViewUniformShaderParameters, taking the view transform from the View Matrices */
	inline void SetupUniformBufferParameters(
		FBox* OutTranslucentCascadeBoundsArray,
		int32 NumTranslucentCascades,
		FViewUniformShaderParameters& ViewUniformShaderParameters) const
	{
		SetupUniformBufferParameters(
			ViewMatrices,
			PrevViewInfo.ViewMatrices,
			OutTranslucentCascadeBoundsArray,
			NumTranslucentCascades,
			ViewUniformShaderParameters);
	}

	void SetupDefaultGlobalDistanceFieldUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters) const;
	void SetupGlobalDistanceFieldUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters) const;
	void SetupVolumetricFogUniformBufferParameters(FViewUniformShaderParameters& ViewUniformShaderParameters) const;

	/** Initializes the RHI resources used by this view. OverrideNumMSAASamples can optionally override NumSceneColorMSAASamples (needed for editor views) */
	void InitRHIResources(uint32 OverrideNumMSAASamples = 0);

	/** Creates both ViewUniformBuffer and InstancedViewUniformBuffer (if needed). */
	void CreateViewUniformBuffers(const FViewUniformShaderParameters& Params);

	/** Determines distance culling and fades if the state changes */
	bool IsDistanceCulled(float DistanceSquared, float MinDrawDistance, float InMaxDrawDistance, const FPrimitiveSceneInfo* PrimitiveSceneInfo);

	bool IsDistanceCulled_AnyThread(float DistanceSquared, float MinDrawDistance, float InMaxDrawDistance, const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool& bOutMayBeFading, bool& bOutFadingIn) const;

	/** @return - whether this primitive has completely faded out */
	bool UpdatePrimitiveFadingState(const FPrimitiveSceneInfo* PrimitiveSceneInfo, bool bFadingIn);

	/** Allocates and returns the current eye adaptation buffer. */
	using FSceneView::GetEyeAdaptationBuffer;
	FRDGPooledBuffer* GetEyeAdaptationBuffer(FRDGBuilder& GraphBuilder) const;

	/** Get the last valid exposure value for eye adaptation. */
	float GetLastEyeAdaptationExposure() const;

	/** Get the last valid average local exposure value. */
	float GetLastAverageLocalExposure() const;

	/** Get the last valid average scene luminance for eye adaptation (exposure compensation curve). */
	float GetLastAverageSceneLuminance() const;

	/**Swap the order of the two eye adaptation targets in the double buffer system */
	void SwapEyeAdaptationBuffers() const;

	/** Update Last Exposure with the most recent available value */
	void UpdateEyeAdaptationLastExposureFromBuffer() const;

	/** Enqueue a pass to readback current exposure */
	void EnqueueEyeAdaptationExposureBufferReadback(FRDGBuilder& GraphBuilder) const;

	/** Returns whether the current view should update the eye adaptation state */
	bool ShouldUpdateEyeAdaptationBuffer() const;
	
	/** Informs sceneinfo that tonemapping LUT has queued commands to compute it at least once */
	void SetValidTonemappingLUT() const;

	/** Gets the tonemapping LUT texture, previously computed by the CombineLUTS post process,
	* for stereo rendering, this will force the post-processing to use the same texture for both eyes*/
	IPooledRenderTarget* GetTonemappingLUT() const;

	/** Gets the rendertarget that will be populated by CombineLUTS post process 
	* for stereo rendering, this will force the post-processing to use the same render target for both eyes*/
	IPooledRenderTarget* GetTonemappingLUT(FRHICommandList& RHICmdList, const int32 LUTSize, const bool bNeedFloatOutput) const;

	/** Gets the other rendertarget that will be populated by CombineLUTS post process */
	IPooledRenderTarget* GetInnerTonemappingLUT(FRHICommandList& RHICmdList, const int32 LUTSize, const bool bNeedFloatOutput) const;

	void ClearInnerTonemappingLUT() const;

	bool IsFirstInFamily() const
	{
		return Family->Views[0] == this;
	}

	bool IsLastInFamily() const
	{
		return Family->Views.Last() == this;
	}

	ERenderTargetLoadAction DecayLoadAction(ERenderTargetLoadAction RequestedLoadAction) const
	{
		return IsFirstInFamily() || Family->bMultiGPUForkAndJoin ? RequestedLoadAction : ERenderTargetLoadAction::ELoad;
	}

	/** Instanced stereo and multi-view only need to render the left eye. */
	bool ShouldRenderView() const 
	{
		if (bHasNoVisiblePrimitive)
		{
			return false;
		}
		else if (!bIsSinglePassStereo)
		{
			return true;
		}
		else if (bIsSinglePassStereo && !IStereoRendering::IsASecondaryPass(StereoPass))
		{
			return true;
		}
		else
		{
			return false;
		}
	}

	/** Returns view rect for all views in the family (usable for atlased views only). Normally a union of their view rects */
	FIntRect GetFamilyViewRect() const;

	/** Similar to above, but returns the unscaled rect, ignoring dynamic resolution scaling */
	FIntRect GetUnscaledFamilyViewRect() const;

	/** Prepares the view shader parameters for rendering and calls the persistent uniform buffer hooks. */
	void BeginRenderView() const;

	/** Returns the set of view uniform buffers representing this view. */
	FViewShaderParameters GetShaderParameters() const;

	/** Returns the primary view associated with the input view, or null if none exists. */
	RENDERER_API const FViewInfo* GetPrimaryView() const;

	/** Returns the instanced view associated with the input view, or null if none exists. */
	inline const FViewInfo* GetInstancedView() const { return static_cast<const FViewInfo*>(GetInstancedSceneView()); }

	/** Create a snapshot of this view info on the scene allocator. */
	FViewInfo* CreateSnapshot(EViewSnapshotType Type = EViewSnapshotType::Main) const;

	void Cleanup();

	// Get the range in DynamicMeshElements[] for a given PrimitiveIndex
	// @return range (start is inclusive, end is exclusive)
	FInt32Range GetDynamicMeshElementRange(uint32 PrimitiveIndex) const;

	/** Get scene textures or config from the view family associated with this view */
	const FSceneTexturesConfig& GetSceneTexturesConfig() const;
	const FSceneTextures& GetSceneTextures() const;
	const FSceneTextures* GetSceneTexturesChecked() const;

	FSceneUniformBuffer& GetSceneUniforms() const;

	RENDERER_API FRDGTextureRef GetVolumetricCloudTexture(FRDGBuilder& GraphBuilder) const;

	/** Return true if this view requires the use of debug material permutations. */
	bool RequiresDebugMaterials() const;

	/**
	 * Collector for view-dependent data.
	 */
	FGPUScenePrimitiveCollector DynamicPrimitiveCollector;
	FGPUScenePrimitiveCollector RayTracingDynamicPrimitiveCollector;

private:
	// Cache of TEXTUREGROUP_World to create view's samplers on render thread.
	// may not have a valid value if FViewInfo is created on the render thread.
	ESamplerFilter WorldTextureGroupSamplerFilter;
	ESamplerFilter TerrainWeightmapTextureGroupSamplerFilter;
	int32 WorldTextureGroupMaxAnisotropy;
	bool bIsValidTextureGroupSamplerFilters;

	FSceneViewState* GetEyeAdaptationViewState() const;

	/** Initialization that is common to the constructors. */
	void Init();

	/** Calculates bounding boxes for the translucency lighting volume cascades. */
	void CalcTranslucencyLightingVolumeBounds(FBox* InOutCascadeBoundsArray, int32 NumCascades) const;
};


/**
 * Masks indicating for which views a primitive needs to have a certain operation on.
 * One entry per primitive in the scene.
 */
typedef TArray<uint8, SceneRenderingAllocator> FPrimitiveViewMasks;

class FShadowMapRenderTargetsRefCounted
{
public:
	// This structure gets included in FCachedShadowMapData, so avoid SceneRenderingAllocator use!
	TArray<TRefCountPtr<IPooledRenderTarget>, TInlineAllocator<4>> ColorTargets;
	TRefCountPtr<IPooledRenderTarget> DepthTarget;

	bool IsValid() const
	{
		if (DepthTarget)
		{
			return true;
		}
		else 
		{
			return ColorTargets.Num() > 0;
		}
	}

	FIntPoint GetSize() const
	{
		const FPooledRenderTargetDesc* Desc = NULL;

		if (DepthTarget)
		{
			Desc = &DepthTarget->GetDesc();
		}
		else 
		{
			check(ColorTargets.Num() > 0);
			Desc = &ColorTargets[0]->GetDesc();
		}

		return Desc->Extent;
	}

	int64 ComputeMemorySize() const
	{
		int64 MemorySize = 0;

		for (int32 i = 0; i < ColorTargets.Num(); i++)
		{
			MemorySize += ColorTargets[i]->ComputeMemorySize();
		}

		if (DepthTarget)
		{
			MemorySize += DepthTarget->ComputeMemorySize();
		}

		return MemorySize;
	}

	void Release()
	{
		for (int32 i = 0; i < ColorTargets.Num(); i++)
		{
			ColorTargets[i] = NULL;
		}

		ColorTargets.Empty();

		DepthTarget = NULL;
	}
};

struct FSortedShadowMapAtlas
{
	FShadowMapRenderTargetsRefCounted RenderTargets;
	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> Shadows;
};

struct FSortedShadowMaps
{
	/** Visible shadows sorted by their shadow depth map render target. */
	TArray<FSortedShadowMapAtlas,SceneRenderingAllocator> ShadowMapAtlases;

	TArray<FSortedShadowMapAtlas,SceneRenderingAllocator> ShadowMapCubemaps;

	FSortedShadowMapAtlas PreshadowCache;

	TArray<FSortedShadowMapAtlas,SceneRenderingAllocator> TranslucencyShadowMapAtlases;

	TArray<FProjectedShadowInfo*, SceneRenderingAllocator> VirtualShadowMapShadows;

	TArray<FSortedShadowMapAtlas,SceneRenderingAllocator> CompleteShadowMapAtlases;

	void Release();

	int64 ComputeMemorySize() const
	{
		int64 MemorySize = 0;

		for (int i = 0; i < ShadowMapAtlases.Num(); i++)
		{
			MemorySize += ShadowMapAtlases[i].RenderTargets.ComputeMemorySize();
		}

		for (int i = 0; i < ShadowMapCubemaps.Num(); i++)
		{
			MemorySize += ShadowMapCubemaps[i].RenderTargets.ComputeMemorySize();
		}

		MemorySize += PreshadowCache.RenderTargets.ComputeMemorySize();

		for (int i = 0; i < TranslucencyShadowMapAtlases.Num(); i++)
		{
			MemorySize += TranslucencyShadowMapAtlases[i].RenderTargets.ComputeMemorySize();
		}

		return MemorySize;
	}

	bool IsEmpty() const
	{
		return ShadowMapAtlases.IsEmpty() && ShadowMapCubemaps.IsEmpty() && PreshadowCache.Shadows.IsEmpty() && TranslucencyShadowMapAtlases.IsEmpty() &&
			VirtualShadowMapShadows.IsEmpty() && CompleteShadowMapAtlases.IsEmpty();
	}
};

struct FOcclusionSubmittedFenceState
{
	FGraphEventRef	Fence;
	uint32			ViewStateUniqueID;
};

/**
 * View family plus associated transient scene textures.
 */
class FViewFamilyInfo : public FSceneViewFamily
{
public:
	explicit FViewFamilyInfo(const FSceneViewFamily& InViewFamily);
	explicit FViewFamilyInfo(const FSceneViewFamily::ConstructionValues& CVS, const FViewFamilyInfo& MainViewFamily);
	virtual ~FViewFamilyInfo();

	FSceneTexturesConfig SceneTexturesConfig;

	/** Set to true if this is a scene capture sized to scene texture size */
	bool bIsSceneTextureSizedCapture = false;

	/** Get scene textures associated with this view family -- asserts or checks that they have been initialized */
	inline FSceneTextures& GetSceneTextures()
	{
		checkf(bIsViewFamilyInfo && SceneTextures->bIsSceneTexturesInitialized, TEXT("FSceneTextures was not initialized. Call FSceneTextures::InitializeViewFamily() first."));
		return *SceneTextures;
	}

	inline const FSceneTextures& GetSceneTextures() const
	{
		checkf(bIsViewFamilyInfo && SceneTextures->bIsSceneTexturesInitialized, TEXT("FSceneTextures was not initialized. Call FSceneTextures::InitializeViewFamily() first."));
		return *SceneTextures;
	}

	inline FSceneTextures* GetSceneTexturesChecked()
	{
		return bIsViewFamilyInfo && SceneTextures->bIsSceneTexturesInitialized ? SceneTextures : nullptr;
	}

	inline const FSceneTextures* GetSceneTexturesChecked() const
	{
		return bIsViewFamilyInfo && SceneTextures->bIsSceneTexturesInitialized ? SceneTextures : nullptr;
	}

	inline bool IsSceneTexturesOwner() const
	{
		return bIsViewFamilyInfo && SceneTextures->Owner == this;
	}

private:
	friend struct FMinimalSceneTextures;
	friend struct FSceneTextures;

	// Structure may be pointed to by multiple FViewFamilyInfo during scene rendering, through CustomRenderPasses.  The Owner
	// (pointed to by FSceneTextures) handles deleting the structure when the scene renderer is destroyed.  TRefCountPtr doesn't
	// work, because the structure is also copied by value, and the copy constructor is disabled for reference counted structures.
	FSceneTextures* SceneTextures;
};

struct FComputeLightGridOutput
{
	FRDGPassRef CompactLinksPass = {};
};

struct FSceneRenderUpdateInputs;
class FDeferredShadingSceneRenderer;

class FSceneRendererBase : public ISceneRenderer
{
public:
	/** The scene being rendered. */
	FScene* Scene = nullptr;

	/** Linear bulk allocator with a lifetime tied to the scene renderer. */
	FSceneRenderingBulkObjectAllocator Allocator;

	FSceneRendererBase() {}
	FSceneRendererBase(FScene& InScene) : Scene(&InScene) {}

	// Helper functions to retrieve the currently active scene renderer from the GraphBuilder instance.
	static void SetActiveInstance(FRDGBuilder& GraphBuilder, FSceneRendererBase* SceneRenderer);
	static FSceneRendererBase* GetActiveInstance(FRDGBuilder& GraphBuilder);

	// ISceneRenderer interface
	FScene* GetScene() final override { return Scene; }
	const FSceneUniformBuffer& GetSceneUniforms() const final override { return SceneUniforms; }
	FSceneUniformBuffer& GetSceneUniforms() final override { return SceneUniforms; }

	TRDGUniformBufferRef<FSceneUniformParameters> GetSceneUniformBufferRef(FRDGBuilder& GraphBuilder) override final
	{
		return GetSceneUniforms().GetBuffer(GraphBuilder);
	}

	void InitSceneExtensionsRenderers(const FEngineShowFlags& EngineShowFlags, bool bValidateCallbacks = false)
	{
		SceneExtensionsRenderers.Begin(*this, EngineShowFlags, bValidateCallbacks);
	}
	FSceneExtensionsRenderers& GetSceneExtensionsRenderers() { return SceneExtensionsRenderers; }
	const FSceneExtensionsRenderers& GetSceneExtensionsRenderers() const { return SceneExtensionsRenderers; }

	/** The views being rendered. */
	TArray<FViewInfo> Views;

	/** 
	 * The view family being rendered. This references the Views array, if it exsists. 
	 * A view family is not always set up by all rendering paths, notably not the VT rendering path.
	 */
	virtual FViewFamilyInfo* GetViewFamily() { return nullptr; }

	/**
	 * This is a workaround to allow initialization of scene extension (renderers) that depend on data not yet migrated to other extensions.
	 * Be very careful to not build in new dependencies on this unless 
	 * 1. required to make something new better, 
	 * 2. you intend to fix this later, pinky promise!
	 */
	virtual FDeferredShadingSceneRenderer* GetDeferredShadingSceneRenderer() { return nullptr; }

	/**
	* Utility function to retrieve from a View the appropriate ViewRect used to create an HZB
	*/
	static FIntRect GetHZBViewRect(const FViewInfo& View);

private:
	FSceneUniformBuffer SceneUniforms;
	FSceneExtensionsRenderers SceneExtensionsRenderers;
};

/**
 * Used as the scope for scene rendering functions.
 * It is initialized in the game thread by FSceneViewFamily::BeginRender, and then passed to the rendering thread.
 * The rendering thread calls Render(), and deletes the scene renderer when it returns.
 */
class FSceneRenderer : public FSceneRendererBase
{
public:
	/** Defines which objects we want to render in the EarlyZPass. */
	FDepthPassInfo DepthPass;

	/** The view family being rendered.  This references the Views array. */
	FViewFamilyInfo ViewFamily;

	/** Information of a custom render pass that renders as part of the main renderer. */
	struct FCustomRenderPassInfo
	{
		FCustomRenderPassInfo(const FSceneViewFamily::ConstructionValues& CVS);
		FCustomRenderPassInfo(const FSceneViewFamily::ConstructionValues& CVS, const FViewFamilyInfo& MainViewFamily);

		/** Custom render pass that render as part of the main renderer. */
		FCustomRenderPassBase* CustomRenderPass;
		/** View family used by custom render pass. NOT treated as a linked view. Required to allow different EngineShowFlags from main renderer ViewFamily. */
		FViewFamilyInfo ViewFamily;
		/** Views used to render the custom render pass. */
		TArray<FViewInfo> Views;
		FNaniteShadingCommands NaniteBasePassShadingCommands;
	};
	TArray<FCustomRenderPassInfo> CustomRenderPassInfos;

	/** All views include main camera views and custom render pass views. */
	TArray<FViewInfo*> AllViews;

	//////////////////////////////////////////////////////////////////////////
	// Provides access to properties of linked scene renderers.

	struct
	{
		FSceneRenderer* Head = nullptr;
		FSceneRenderer* Next = nullptr;
	
	} Link;

	bool IsHeadLink() const { return Link.Head == this; }

	template <typename LambdaType>
	void EnumerateLinkedViews(LambdaType&& Lambda, const FSceneView* ViewToSkip = nullptr)
	{
		for (FSceneRenderer* Renderer = Link.Head; Renderer; Renderer = Renderer->Link.Next)
		{
			for (FViewInfo& View : Renderer->Views)
			{
				if (ViewToSkip != &View)
				{
					if (!Lambda(View))
					{
						return;
					}
				}
			}
		}
	}

	template <typename LambdaType>
	void EnumerateLinkedViewFamilies(LambdaType&& Lambda, const FSceneViewFamily* ViewFamilyToSkip = nullptr)
	{
		for (FSceneRenderer* Renderer = Link.Head; Renderer; Renderer = Renderer->Link.Next)
		{
			if (ViewFamilyToSkip != &Renderer->ViewFamily)
			{
				if (!Lambda(Renderer->ViewFamily))
				{
					return;
				}
			}
		}
	}

	//////////////////////////////////////////////////////////////////////////

	/** All the dynamic scaling informations */
	DynamicRenderScaling::TMap<float> DynamicResolutionFractions;
	DynamicRenderScaling::TMap<float> DynamicResolutionUpperBounds;

	/** Information about the visible lights. */
	FVisibleLightInfoArray VisibleLightInfos;

	/** Array of dispatched parallel shadow depth passes. */
	UE::FMutex DispatchedShadowDepthPassesMutex;
	TArray<FParallelMeshDrawCommandPass*, SceneRenderingAllocator> DispatchedShadowDepthPasses;

	FSortedShadowMaps SortedShadowsForShadowDepthPass;

	FVirtualShadowMapArray VirtualShadowMapArray;

	LightFunctionAtlas::FLightFunctionAtlas LightFunctionAtlas;

	/** If a freeze request has been made */
	bool bHasRequestedToggleFreeze;

	/** True if precomputed visibility was used when rendering the scene. */
	bool bUsedPrecomputedVisibility;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	// Set to true if a whole scene point light shadow is present
	// Used for warnings about unsupported features
	bool bUsedPointLightWholeSceneShadow = false;
#endif
	/** Feature level being rendered */
	ERHIFeatureLevel::Type FeatureLevel;
	EShaderPlatform ShaderPlatform;

	bool bGPUMasksComputed;
	FRHIGPUMask RenderTargetGPUMask;

public:

	FSceneRenderer(const FSceneViewFamily* InViewFamily, FHitProxyConsumer* HitProxyConsumer);
	virtual ~FSceneRenderer();

	// ISceneRenderer interface
	virtual UE::Renderer::Private::IShadowInvalidatingInstances *GetShadowInvalidatingInstancesInterface(const FSceneView *SceneView) override;

	// FSceneRendererBase interface
	virtual FViewFamilyInfo* GetViewFamily() override { return &ViewFamily; }

	// FSceneRenderer interface
	virtual void Render(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs) = 0;
	virtual void RenderHitProxies(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs) {}
	virtual bool ShouldRenderVelocities() const { return false; }
	virtual bool ShouldRenderPrePass() const { return false; }
	virtual bool ShouldRenderNanite() const { return false; }
	virtual bool AllowSimpleLights() const;

	/** Setups FViewInfo::ViewRect according to ViewFamilly's ScreenPercentageInterface. */
	void PrepareViewRectsForRendering();

#if WITH_MGPU
	/**
	  * Assigns the view GPU masks and initializes RenderTargetGPUMask.  RHICmdList is only required if alternate frame rendering
	  * is active, and must be called in the render thread in that case, otherwise it can be called earlier.  Computing the masks
	  * early is used to optimize handling of cross GPU fences (see PreallocateCrossGPUFences).
	  */
	void ComputeGPUMasks(FRHICommandListImmediate* RHICmdList);
#endif

	/** Logic to update render targets across all GPUs */
	static void PreallocateCrossGPUFences(TConstArrayView<FSceneRenderer*> SceneRenderers);
	void DoCrossGPUTransfers(FRDGBuilder& GraphBuilder, FRDGTextureRef ViewFamilyTexture, TArrayView<FViewInfo> InViews, bool bCrossGPUTransferFencesDefer, FRHIGPUMask RenderTargetGPUMask, class FCrossGPUTransfersDeferred* TransfersDeferred);
	void FlushCrossGPUTransfers(FRDGBuilder& GraphBuilder);
	void FlushCrossGPUFences(FRDGBuilder& GraphBuilder);

	bool DoOcclusionQueries() const;

	void FenceOcclusionTests(FRDGBuilder& GraphBuilder);
	void WaitOcclusionTests(FRHICommandListImmediate& RHICmdList);

	// fences to make sure the rhi thread has digested the occlusion query renders before we attempt to read them back async
	static FOcclusionSubmittedFenceState OcclusionSubmittedFence[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames];

	bool ShouldDumpMeshDrawCommandInstancingStats() const { return bDumpMeshDrawCommandInstancingStats; }

	/** bound shader state for occlusion test prims */
	static FGlobalBoundShaderState OcclusionTestBoundShaderState;
	
	/**
	* Whether or not to composite editor objects onto the scene as a post processing step
	*
	* @param View The view to test against
	*
	* @return true if compositing is needed
	*/
	static bool ShouldCompositeEditorPrimitives(const FViewInfo& View);

	/** Apply the ResolutionFraction on ViewSize, taking into account renderer's requirements. */
	static FIntPoint ApplyResolutionFraction(const FIntPoint& UnscaledViewSize, float ResolutionFraction);

	/** Quantize the ViewRect.Min according to various renderer's downscale requirements. */
	static FIntPoint QuantizeViewRectMin(const FIntPoint& ViewRectMin);

	/** Get the desired buffer size from the view family's ResolutionFraction upperbound.
	 * Can be called on game thread or render thread. 
	 */
	static FIntPoint GetDesiredInternalBufferSize(const FSceneViewFamily& ViewFamily);

	/** Exposes renderer's privilege to fork view family's screen percentage interface. */
	static ISceneViewFamilyScreenPercentage* ForkScreenPercentageInterface(
		const ISceneViewFamilyScreenPercentage* ScreenPercentageInterface, FSceneViewFamily& ForkedViewFamily)
	{
		return ScreenPercentageInterface->Fork_GameThread(ForkedViewFamily);
	}

	static int32 GetRefractionQuality(const FSceneViewFamily& ViewFamily);

	/** Common function to render a sky using shared LUT resources from any view point (if not using the SkyView and AerialPerspective textures). */
	void RenderSkyAtmosphereInternal(
		FRDGBuilder& GraphBuilder,
		const FSceneTextureShaderParameters& SceneTextures,
		FSkyAtmosphereRenderContext& SkyRenderContext);

	/** Common function to render a cloud layer using shared LUT resources. */
	void  RenderVolumetricCloudsInternal(FRDGBuilder& GraphBuilder, FCloudRenderContext& CloudRC, FInstanceCullingManager& InstanceCullingManager, const FIntPoint& CloudViewportSize);

	/** Sets the stereo-compatible RHI viewport. If the view doesn't requires stereo rendering, the standard viewport is set. */
	static void SetStereoViewport(FRHICommandList& RHICmdList, const FViewInfo& View, float ViewportScale = 1.0f);

	FGPUSceneDynamicContext& GetGPUSceneDynamicContext() { return GPUSceneDynamicContext; }

	void VoxelizeFogVolumePrimitives(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		const FVolumetricFogIntegrationParameterData& IntegrationData,
		FIntVector VolumetricFogGridSize,
		FVector GridZParams,
		float VolumetricFogDistance,
		bool bVoxelizeEmissive);

	void RenderLightFunctionForVolumetricFog(
		FRDGBuilder& GraphBuilder,
		FViewInfo& View,
		const FSceneTextures& SceneTextures,
		FIntVector VolumetricFogGridSize,
		float VolumetricFogMaxDistance,
		FLightSceneInfo* DirectionalLightSceneInfo,
		FMatrix44f& OutLightFunctionTranslatedWorldToShadow,
		FRDGTexture*& OutLightFunctionTexture);

	void RenderLocalLightsForVolumetricFog(
		FRDGBuilder& GraphBuilder,
		FViewInfo& View, int32 ViewIndex,
		bool bUseTemporalReprojection,
		const struct FVolumetricFogIntegrationParameterData& IntegrationData,
		const FExponentialHeightFogSceneInfo& FogInfo,
		FIntVector VolumetricFogGridSize,
		FVector GridZParams,
		const FRDGTextureDesc& VolumeDesc,
		FRDGTextureRef ConservativeDepthTexture,
		TConstArrayView<const FLightSceneInfo*> LightsToInject,
		TConstArrayView<const FLightSceneInfo*> RayTracedLightsToInject,
		FRDGTexture*& OutLocalShadowedLightScattering);

	/** Renders capsule shadows for movable skylights, using the cone of visibility (bent normal) from DFAO. */
	void RenderCapsuleShadowsForMovableSkylight(
		FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FRDGTextureRef& BentNormalOutput) const;

	/** Render Ambient Occlusion using mesh distance fields on a screen based grid. */
	void RenderDistanceFieldAOScreenGrid(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		const FViewInfo& View,
		const FDistanceFieldCulledObjectBufferParameters& CulledObjectBufferParameters,
		FRDGBufferRef ObjectTilesIndirectArguments,
		const FTileIntersectionParameters& TileIntersectionParameters,
		const FDistanceFieldAOParameters& Parameters,
		FRDGTextureRef DistanceFieldNormal,
		FRDGTextureRef& OutDynamicBentNormalAO);

	/** Render Ambient Occlusion using mesh distance fields and the surface cache, which supports dynamic rigid meshes. */
	void RenderDistanceFieldLighting(
		FRDGBuilder& GraphBuilder,
		const FSceneTextures& SceneTextures,
		const class FDistanceFieldAOParameters& Parameters,
		TArray<FRDGTextureRef>& OutDynamicBentNormalAOTextures,
		bool bModulateToSceneColor,
		bool bVisualizeAmbientOcclusion,
		bool bModulateToScreenSpaceAO = false);

	void SetupVolumetricFog();

	bool ShouldRenderVolumetricFog() const;
	void ComputeVolumetricFog(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

	virtual bool IsLumenEnabled(const FViewInfo& View) const { return false; }
	virtual bool IsLumenGIEnabled(const FViewInfo& View) const { return false; }
	virtual bool AnyViewHasGIMethodSupportingDFAO() const { return true; }

	/** Gets a readable light name for use with a draw event. */
	static void GetLightNameForDrawEvent(const FLightSceneProxy* LightProxy, FString& LightNameWithLevel);

	FORCEINLINE FSceneTextures& GetActiveSceneTextures() { return ViewFamily.GetSceneTextures(); }
	FORCEINLINE FSceneTexturesConfig& GetActiveSceneTexturesConfig() { return ViewFamily.SceneTexturesConfig; }

	FORCEINLINE const FSceneTextures& GetActiveSceneTextures() const { return ViewFamily.GetSceneTextures(); }
	FORCEINLINE const FSceneTexturesConfig& GetActiveSceneTexturesConfig() const { return ViewFamily.SceneTexturesConfig; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FSceneOnScreenMessagesDelegate, FScreenMessageWriter&);
	FSceneOnScreenMessagesDelegate OnGetOnScreenMessages;

	inline TConstStridedView<FSceneView> GetSceneViews() const { return MakeStridedViewOfBase<const FSceneView>(MakeArrayView(Views)); }

	static TGlobalResource<FGlobalDynamicReadBuffer> DynamicReadBufferForInitViews;
	static TGlobalResource<FGlobalDynamicReadBuffer> DynamicReadBufferForRayTracing;
	static TGlobalResource<FGlobalDynamicReadBuffer> DynamicReadBufferForShadows;

	bool IsRenderingStereo() const
	{
		return Views.Num() == 2 && IStereoRendering::IsASecondaryView(Views[1]);
	}

protected:

	/** Size of the family. */
	FIntPoint FamilySize;

#if WITH_MGPU
	/**
	 * Fences for cross GPU render target transfers.  We defer the wait on cross GPU fences until the last scene renderer,
	 * to avoid needless stalls in the middle of the frame, improving performance.  The "Defer" array holds fences issued
	 * by each prior scene renderer, while the "Wait" array holds fences to be waited on in the last scene renderer
	 * (a collection of all the fences from prior scene renderers).  The function "PreallocateCrossGPUFences" initializes
	 * these arrays.
	 */
	TArray<FCrossGPUTransferFence*> CrossGPUTransferFencesDefer;
	TArray<FCrossGPUTransferFence*> CrossGPUTransferFencesWait;

	/** Deferred transfers to be executed in the last scene renderer */
	TRefCountPtr<class FCrossGPUTransfersDeferred> CrossGPUTransferDeferred;

	FRHIGPUMask AllViewsGPUMask;
	bool IsShadowCached(FProjectedShadowInfo* ProjectedShadowInfo) const;
	FRHIGPUMask GetGPUMaskForShadow(FProjectedShadowInfo* ProjectedShadowInfo) const;
#endif

	/** The cached FXSystem which could be released while we are rendering. */
	class FFXSystemInterface* FXSystem = nullptr;

	bool bDumpMeshDrawCommandInstancingStats;

	// Shared functionality between all scene renderers

	enum class ERendererOutput
	{
		DepthPrepassOnly,	// Only render depth prepass and its related code paths
		FinalSceneColor		// Render the whole pipeline
	};

	ERendererOutput GetRendererOutput() const;

	FDynamicShadowsTaskData* BeginInitDynamicShadows(FRDGBuilder& GraphBuilder, bool bRunningEarly, IVisibilityTaskData* VisibilityTaskData, FInstanceCullingManager& InstanceCullingManager);
	void FinishInitDynamicShadows(FRDGBuilder& GraphBuilder, FDynamicShadowsTaskData* TaskData);
	void FinishDynamicShadowMeshPassSetup(FRDGBuilder& GraphBuilder, FDynamicShadowsTaskData* TaskData);
	FDynamicShadowsTaskData* InitDynamicShadows(FRDGBuilder& GraphBuilder, FInstanceCullingManager& InstanceCullingManager);

	void CreateDynamicShadows(FDynamicShadowsTaskData& TaskData);
	void FilterDynamicShadows(FDynamicShadowsTaskData& TaskData);
	friend struct FGatherShadowPrimitivesPrepareTask;

	void SetupMeshPass(FViewInfo& View, FExclusiveDepthStencil::Type BasePassDepthStencilAccess, FViewCommands& ViewCommands, FInstanceCullingManager& InstanceCullingManager);

	void RenderShadowProjections(
		FRDGBuilder& GraphBuilder,
		FRDGTextureRef OutputTexture,
		const FMinimalSceneTextures& SceneTextures,
		const FLightSceneProxy* LightSceneProxy,
		TArrayView<const FProjectedShadowInfo* const> Shadows,
		bool bSubPixelShadow,
		bool bProjectingForForwardShading);


	void RenderShadowProjections(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		FRDGTextureRef ScreenShadowMaskTexture,
		FRDGTextureRef ScreenShadowMaskSubPixelTexture,
		const FLightSceneInfo* LightSceneInfo,
		bool bProjectingForForwardShading);

	void BeginAsyncDistanceFieldShadowProjections(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		const FDynamicShadowsTaskData* TaskData) const;

	/** Finds a matching cached preshadow, if one exists. */
	TRefCountPtr<FProjectedShadowInfo> GetCachedPreshadow(
		const FLightPrimitiveInteraction* InParentInteraction,
		const FProjectedShadowInitializer& Initializer,
		const FBoxSphereBounds& Bounds,
		uint32 InResolutionX);

	/** Creates a per object projected shadow for the given interaction. */
	void CreatePerObjectProjectedShadow(
		FDynamicShadowsTaskData& TaskData,
		FLightPrimitiveInteraction* Interaction,
		bool bCreateTranslucentObjectShadow,
		bool bCreateInsetObjectShadow,
		const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ViewDependentWholeSceneShadows,
		TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& OutPreShadows);

	/** Creates shadows for the given interaction. */
	void SetupInteractionShadows(
		FDynamicShadowsTaskData& TaskData,
		FLightPrimitiveInteraction* Interaction,
		FVisibleLightInfo& VisibleLightInfo,
		bool bStaticSceneOnly,
		const TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ViewDependentWholeSceneShadows,
		TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& PreShadows);

	/** Generates FProjectedShadowInfos for all wholesceneshadows on the given light.*/
	void AddViewDependentWholeSceneShadowsForView(
		TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ShadowInfos, 
		TArray<FProjectedShadowInfo*, SceneRenderingAllocator>& ShadowInfosThatNeedCulling, 
		FVisibleLightInfo& VisibleLightInfo, 
		FLightSceneInfo& LightSceneInfo,
		int64 CachedShadowMapsSize,
		uint32& NumCSMCachesUpdatedThisFrame);

	void AllocateShadowDepthTargets(FDynamicShadowsTaskData& TaskData);
	
	void AllocateAtlasedShadowDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> Shadows, TArray<FSortedShadowMapAtlas,SceneRenderingAllocator>& OutAtlases);

	void AllocateCachedShadowDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> CachedShadows);

	void AllocateCSMDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> WholeSceneDirectionalShadows, TArray<FSortedShadowMapAtlas, SceneRenderingAllocator>& OutAtlases);

	void AllocateOnePassPointLightDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> WholeScenePointShadows);

	void AllocateTranslucentShadowDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> TranslucentShadows);

	void AllocateMobileCSMAndSpotLightShadowDepthTargets(FRHICommandListBase& RHICmdList, TConstArrayView<FProjectedShadowInfo*> MobileCSMAndSpotLightShadows);
	/**
	* Used by RenderLights to figure out if projected shadows need to be rendered to the attenuation buffer.
	* Or to render a given shadowdepth map for forward rendering.
	*
	* @param LightSceneInfo Represents the current light
	* @return true if anything needs to be rendered
	*/
	bool CheckForProjectedShadows(const FLightSceneInfo* LightSceneInfo) const;

	/** Gathers the list of primitives used to draw various shadow types */
	void BeginGatherShadowPrimitives(FDynamicShadowsTaskData* TaskData, IVisibilityTaskData* VisibilityTaskData);
	void FinishGatherShadowPrimitives(FDynamicShadowsTaskData* TaskData);

	void RenderShadowDepthMaps(FRDGBuilder& GraphBuilder, FDynamicShadowsTaskData* DynamicShadowsTaskData, FInstanceCullingManager& InstanceCullingManager, FRDGExternalAccessQueue& ExternalAccessQueue);

	void RenderShadowDepthMapAtlases(FRDGBuilder& GraphBuilder);

	/**
	* Creates a projected shadow for all primitives affected by a light.
	* @param LightSceneInfo - The light to create a shadow for.
	*/
	void CreateWholeSceneProjectedShadow(FDynamicShadowsTaskData& TaskData, FLightSceneInfo* LightSceneInfo, int64 CachedShadowMapsSize, uint32& NumPointShadowCachesUpdatedThisFrame, uint32& NumSpotShadowCachesUpdatedThisFrame);

	/** Updates the preshadow cache, allocating new preshadows that can fit and evicting old ones. */
	void UpdatePreshadowCache();

	/** Gathers simple lights from visible primtives in the passed in views. */
	static void GatherSimpleLights(const FSceneViewFamily& ViewFamily, const TArray<FViewInfo>& Views, FSimpleLightArray& SimpleLights);

	/** Calculates projected shadow visibility. */
	void InitProjectedShadowVisibility(FDynamicShadowsTaskData& TaskData);

	/** Adds a debug shadow frustum to the views primitive draw interface. */
	void DrawDebugShadowFrustum(FViewInfo& View, FProjectedShadowInfo& ProjectedShadowInfo);

	/** Gathers dynamic mesh elements for all shadows. */
	void GatherShadowDynamicMeshElements(FDynamicShadowsTaskData& TaskData);

	/** Renders capsule shadows for all per-object shadows using it for the given light. */
	bool RenderCapsuleDirectShadows(
		FRDGBuilder& GraphBuilder,
		const FLightSceneInfo& LightSceneInfo,
		FRDGTextureRef ScreenShadowMaskTexture,
		TArrayView<const FProjectedShadowInfo* const> CapsuleShadows,
		bool bProjectingForForwardShading) const;

	/** Performs once per frame setup prior to visibility determination. */
	void PreVisibilityFrameSetup(FRDGBuilder& GraphBuilder);

	void GatherReflectionCaptureLightMeshElements();

	/** Performs once per frame setup after to visibility determination. */
	void PostVisibilityFrameSetup(FILCUpdatePrimTaskData*& OutILCTaskData);

	/** Initialized the fog constants for each view. */
	void InitFogConstants();

	/** Returns whether there are translucent primitives to be rendered. */
	bool ShouldRenderTranslucency() const;
	static bool ShouldRenderTranslucency(ETranslucencyPass::Type TranslucencyPass, TArrayView<FViewInfo> InViews);

	/** Called at the begin / finish of the scene render. */
	IVisibilityTaskData* OnRenderBegin(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs);
	void OnRenderFinish(FRDGBuilder& GraphBuilder, FRDGTextureRef ViewFamilyTexture);

	bool RenderCustomDepthPass(
		FRDGBuilder& GraphBuilder,
		FCustomDepthTextures& CustomDepthTextures,
		const FSceneTextureShaderParameters& SceneTextures,
		TConstArrayView<Nanite::FRasterResults> PrimaryNaniteRasterResults,
		TConstArrayView<Nanite::FPackedView> PrimaryNaniteViews);

	void UpdatePrimitiveIndirectLightingCacheBuffers(FRHICommandListBase& RHICmdList);

	void RenderPlanarReflection(class FPlanarReflectionSceneProxy* ReflectionSceneProxy);

	/** Initialise sky atmosphere resources.*/
	void InitSkyAtmosphereForViews(FRHICommandListImmediate& RHICmdList, FRDGBuilder& GraphBuilder);
	
	/** Render the sky atmosphere look up table needed for this frame.*/
	void RenderSkyAtmosphereLookUpTables(FRDGBuilder& GraphBuilder, class FSkyAtmospherePendingRDGResources& PendingRDGResources, FDynamicShadowsTaskData* FilterDynamicShadowTaskData = nullptr);

	/** Render the sky atmosphere over the scene.*/
	void RenderSkyAtmosphere(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures);

	/** Initialise volumetric cloud resources.*/
	void InitVolumetricCloudsForViews(FRDGBuilder& GraphBuilder, bool bShouldRenderVolumetricCloud, FInstanceCullingManager& InstanceCullingManager);

	/** Render volumetric cloud. */
	bool RenderVolumetricCloud(
		FRDGBuilder& GraphBuilder,
		const FMinimalSceneTextures& SceneTextures,
		bool bSkipVolumetricRenderTarget,
		bool bSkipPerPixelTracing,
		FRDGTextureRef HalfResolutionDepthCheckerboardMinMaxTexture,
		FRDGTextureRef QuarterResolutionDepthMinMaxTexture,
		bool bAsyncCompute,
		FInstanceCullingManager& InstanceCullingManager);

	/** Render notification to artist when a sky material is used but it might comtains the camera (and then the sky/background would look black).*/
	void RenderSkyAtmosphereEditorNotifications(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FRDGTextureRef SceneColorTexture) const;

	/** We should render on screen notification only if any of the scene contains a mesh using a sky material.*/
	static bool ShouldRenderSkyAtmosphereEditorNotifications(TArrayView<FViewInfo> Views);

	/**
	 * Rounds up lights and sorts them according to what type of renderer supports them. The result is stored in OutSortedLights 
	 * NOTE: Also extracts the SimpleLights AND adds them to the sorted range (first sub-range). 
	 */
	void GatherAndSortLights(FSortedLightSetSceneInfo& OutSortedLights, bool bShadowedLightsInClustered = false);

	/**
	 * Register all lights with the light function atlas and updates it.
	 */
	void UpdateLightFunctionAtlasTaskFunction();
	
	/** 
	 * Culls local lights and reflection probes to a grid in frustum space, builds one light list and grid per view in the current Views.  
	 * Needed for forward shading or translucency using the Surface lighting mode, and clustered deferred shading. 
	 */
	template <typename T>
	using FComputeLightGridArray = TArray<T, TInlineAllocator<2, SceneRenderingAllocator>>;
	FComputeLightGridOutput ComputeLightGrid(FRDGBuilder& GraphBuilder, bool bCullLightsToGrid, const FSortedLightSetSceneInfo& SortedLightSet, FComputeLightGridArray<FForwardLightUniformParameters*>& PerViewForwardLightUniformParameters, FComputeLightGridArray<FForwardDirectionalLightShadowMapParameters*>& PerViewForwardDirLightShadowParameters);

	/**
	 * Prepare ForwardLightingResources for the Views (including building light grids)
	 */
	FComputeLightGridOutput PrepareForwardLightData(FRDGBuilder& GraphBuilder, bool bCullLightsToGrid, const FSortedLightSetSceneInfo& SortedLightSet);

	/**
	* Used by RenderLights to figure out if light functions need to be rendered to the attenuation buffer.
	*
	* @param LightSceneInfo Represents the current light
	* @return true if anything got rendered
	*/
	bool CheckForLightFunction(const FLightSceneInfo* LightSceneInfo) const;

	void SetupSceneReflectionCaptureBuffer(FRHICommandListImmediate& RHICmdList);

	void RenderVelocities(
		FRDGBuilder& GraphBuilder,
		TArrayView<FViewInfo> InViews,
		const FSceneTextures& SceneTextures,
		EVelocityPass VelocityPass,
		bool bForceVelocity,
		bool bBindRenderTarget = true);

	void RenderVelocities(
		FRDGBuilder& GraphBuilder,
		TArrayView<FViewInfo> InViews,
		const FSceneTextures& SceneTextures,
		EVelocityPass VelocityPass,
		bool bForceVelocity,
		bool bBindRenderTarget,
		FRDGTextureRef TargetVelocity,
		FRDGTextureRef TargetDepth);

	void RenderMeshDistanceFieldVisualization(FRDGBuilder& GraphBuilder, const FMinimalSceneTextures& SceneTextures);

#if !UE_BUILD_SHIPPING
	static void CreateSplitScreenDebugViewFamilies(const FSceneViewFamily& InFamily, TArray<const FSceneViewFamily*>& OutViewFamilies, TArray<const FSceneViewFamily*, TFixedAllocator<2>>& OutFamiliesStorage);
	static void DestroySplitScreenDebugViewFamilies(TConstArrayView<const FSceneViewFamily*> ViewFamilies);
#endif

protected:
	FGPUSceneDynamicContext GPUSceneDynamicContext;

	void CheckShadowDepthRenderCompleted() const
	{
		checkf(bShadowDepthRenderCompleted, TEXT("Shadow depth rendering was not done before shadow projections, this will cause severe shadow artifacts and indicates an engine bug (pass ordering)"));
	}

	virtual void ComputeLightVisibility();

	void ComputeMegaLightsLightPowerDelta(FRDGBuilder& GraphBuilder);

private:
	void ComputeFamilySize();

	friend class FVisibilityTaskData;

	void PrepareViewStateForVisibility(const FSceneTexturesConfig& SceneTexturesConfig);
	void PrepareViewStatePrevFrameViewInfo();

	bool bShadowDepthRenderCompleted;

#if RHI_RAYTRACING
	virtual void InitializeRayTracingFlags_RenderThread() {}
#endif

	friend class FRendererModule;
	friend class FSceneRenderProcessor;
	friend class FMobileSceneRenderer;
};

/** A set of show flags thare are guaranteed to be common among all renderers in the set. */
enum class ESceneRenderCommonShowFlags
{
	None             = 0,
	HitProxies       = 1 << 0,
	PathTracing      = 1 << 1
};
ENUM_CLASS_FLAGS(ESceneRenderCommonShowFlags);

/** An optional set of all renderers, view families, and views for operations that are performed once for a
 *  batch of scene renderers. Only the first scene renderer will have this applied.
 */
struct FSceneRenderUpdateInputs
{
	TConstArrayView<const FSceneView*> GetAsSceneViews() const
	{
		return MakeArrayView(reinterpret_cast<const FSceneView* const*>(Views.GetData()), Views.Num());
	}

	TConstArrayView<const FSceneViewFamily*> GetAsSceneViewFamilies() const
	{
		return MakeArrayView(reinterpret_cast<const FSceneViewFamily* const*>(ViewFamilies.GetData()), ViewFamilies.Num());
	}

	template <typename LambdaType>
	bool HasAnyShowFlags(LambdaType&& Lambda) const
	{
		for (FViewFamilyInfo* Family : ViewFamilies)
		{
			if (Lambda(Family->EngineShowFlags))
			{
				return true;
			}
		}
		return false;
	}

	template <typename LambdaType>
	void ForEachView(LambdaType&& Lambda) const
	{
		for (FSceneRenderer* Renderer : Renderers)
		{
			for (FViewInfo& View : Renderer->Views)
			{
				if (!Lambda(Renderer, View))
				{
					return;
				}
			}
		}
	}

	EShaderPlatform ShaderPlatform;
	ERHIFeatureLevel::Type FeatureLevel;
	FScene* Scene;
	FFXSystemInterface* FXSystem;
	FGlobalShaderMap* GlobalShaderMap;
	TConstArrayView<FSceneRenderer*> Renderers;
	TConstArrayView<FViewFamilyInfo*> ViewFamilies;
	TConstArrayView<FViewInfo*> Views;
	ESceneRenderCommonShowFlags CommonShowFlags;
};

template <typename LambdaType, typename PrerequisiteTaskCollectionType>
UE::Tasks::FTask LaunchSceneRenderTask(const TCHAR* DebugName, LambdaType&& Lambda, PrerequisiteTaskCollectionType&& Prerequisites, bool bExecuteInParallelCondition = true, UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High)
{
	const bool bExecuteInParallel = bExecuteInParallelCondition && FApp::ShouldUseThreadingForPerformance() && GIsThreadedRendering;

	return UE::Tasks::Launch(DebugName, [Lambda = Forward<LambdaType&&>(Lambda)] () mutable
		{
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
			Lambda();
		}, 
		Forward<PrerequisiteTaskCollectionType&&>(Prerequisites),
		TaskPriority,
		bExecuteInParallel ? UE::Tasks::EExtendedTaskPriority::None : UE::Tasks::EExtendedTaskPriority::Inline
	);
}

template <typename LambdaType>
UE::Tasks::FTask LaunchSceneRenderTask(const TCHAR* DebugName, LambdaType&& Lambda, bool bExecuteInParallelCondition = true, UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High)
{
	const bool bExecuteInParallel = bExecuteInParallelCondition && FApp::ShouldUseThreadingForPerformance() && GIsThreadedRendering;

	return UE::Tasks::Launch(DebugName, [Lambda = Forward<LambdaType&&>(Lambda)] () mutable
		{
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
			Lambda();
		},
		TaskPriority,
		bExecuteInParallel ? UE::Tasks::EExtendedTaskPriority::None : UE::Tasks::EExtendedTaskPriority::Inline
	);
}

template <typename ReturnType, typename LambdaType, typename PrerequisiteTaskCollectionType>
UE::Tasks::TTask<ReturnType> LaunchSceneRenderTask(const TCHAR* DebugName, LambdaType&& Lambda, PrerequisiteTaskCollectionType&& Prerequisites, bool bExecuteInParallelCondition = true, UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::High)
{
	const bool bExecuteInParallel = bExecuteInParallelCondition && FApp::ShouldUseThreadingForPerformance() && GIsThreadedRendering;

	return UE::Tasks::Launch(DebugName, [Lambda = Forward<LambdaType&&>(Lambda)] () mutable
		{
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
			return Lambda();
		},
		Forward<PrerequisiteTaskCollectionType&&>(Prerequisites),
		TaskPriority,
		bExecuteInParallel ? UE::Tasks::EExtendedTaskPriority::None : UE::Tasks::EExtendedTaskPriority::Inline
	);
}

// Creates a FGraphEventRef from UE::Task::FTask prerequisites.
template <typename PrerequisiteTaskCollectionType>
FGraphEventRef CreateCompatibilityGraphEvent(PrerequisiteTaskCollectionType&& Prerequisites)
{
	FGraphEventRef GraphEvent = FGraphEvent::CreateGraphEvent();
	UE::Tasks::Launch(UE_SOURCE_LOCATION, [GraphEvent]
	{
		GraphEvent->DispatchSubsequents();

	}, Prerequisites, UE::Tasks::ETaskPriority::High, UE::Tasks::EExtendedTaskPriority::Inline);
	return GraphEvent;
}

typedef TArray<FRDGTextureRef, TInlineAllocator<6>> FColorTargets;

/**
 * Renderer that implements simple forward shading and associated features.
 */
class FMobileSceneRenderer : public FSceneRenderer
{
public:

	FMobileSceneRenderer(const FSceneViewFamily* InViewFamily, FHitProxyConsumer* HitProxyConsumer);

	// FSceneRenderer interface

	virtual void Render(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs) override;

	virtual void RenderHitProxies(FRDGBuilder& GraphBuilder, const FSceneRenderUpdateInputs* SceneUpdateInputs) override;

	virtual bool ShouldRenderVelocities() const override;

	virtual bool ShouldRenderPrePass() const override;

	virtual bool AllowSimpleLights() const override;

	/** Whether platform requires multiple render-passes for SceneColor rendering */
	static bool RequiresMultiPass(int32 NumMSAASamples, EShaderPlatform ShaderPlatform);

	/** Check if custom depth pass should be rendered with current view info */
	static bool ShouldRenderCustomDepth(const FViewInfo& View);

protected:
	/** Finds the visible dynamic shadows for each view. */
	FDynamicShadowsTaskData* InitDynamicShadows(FRDGBuilder& GraphBuilder, FInstanceCullingManager& FInstanceCullingManager);

	void PrepareViewVisibilityLists();

	/** Build visibility lists on CSM receivers and non-csm receivers. */
	void BuildCSMVisibilityState(FLightSceneInfo* LightSceneInfo);

	struct FInitViewTaskDatas
	{
		FInitViewTaskDatas(IVisibilityTaskData* InVisibilityTaskData)
			: VisibilityTaskData(InVisibilityTaskData)
		{}

		IVisibilityTaskData* VisibilityTaskData;
		FDynamicShadowsTaskData* DynamicShadows = nullptr;
	};

	void InitViews(
		FRDGBuilder& GraphBuilder,
		FSceneTexturesConfig& SceneTexturesConfig,
		FInstanceCullingManager& InstanceCullingManager,
		FVirtualTextureUpdater* VirtualTextureUpdater,
		FInitViewTaskDatas& TaskDatas);

	void RenderPrePass(FRHICommandList& RHICmdList, const FViewInfo& View, const FInstanceCullingDrawParams* InstanceCullingDrawParams);
	void RenderMaskedPrePass(FRHICommandList& RHICmdList, const FViewInfo& View, const FInstanceCullingDrawParams* DepthPassInstanceCullingDrawParams);
	void RenderFullDepthPrepass(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager, bool bIsSceneCaptureRenderPass=false);

	void RenderVelocityPass(FRHICommandList& RHICmdList, const FViewInfo& View, const FInstanceCullingDrawParams* InstanceCullingDrawParams);

	void RenderCustomRenderPassBasePass(FRDGBuilder& GraphBuilder, TArrayView<FViewInfo> InViews, FRDGTextureRef ViewFamilyTexture, FSceneTextures& SceneTextures, FMobileBasePassTextures& BasePassTextures, FInstanceCullingManager& InstanceCullingManager, bool bIncludeTranslucent);

	/** Renders the opaque base pass for mobile. */
	void RenderMobileBasePass(FRHICommandList& RHICmdList, const FViewInfo& View, const FInstanceCullingDrawParams* InstanceCullingDrawParams, const FInstanceCullingDrawParams* SkyPassInstanceCullingDrawParams);

	void PostRenderBasePass(FRHICommandList& RHICmdList, FViewInfo& View);

	void RenderMobileEditorPrimitives(FRHICommandList& RHICmdList, const FViewInfo& View, const FMeshPassProcessorRenderState& DrawRenderState, const FInstanceCullingDrawParams* InstanceCullingDrawParams);

#if UE_ENABLE_DEBUG_DRAWING
	/** Render debug primitives in the base pass, for MobileHDR=false */
	void RenderMobileDebugPrimitives(FRHICommandList& RHICmdList, const FViewInfo& View);
#endif

	/** Renders the debug view pass for mobile in its own render pass */
	void RenderMobileDebugViewMode(
		FRDGBuilder& GraphBuilder,
		FViewInfo& View,
		FSceneTextures& SceneTextures,
		FInstanceCullingManager& InstanceCullingManager,
		const class FMobileRenderPassParameters& BasePassParameters);

	/** Render modulated shadow projections in to the scene, loops over any unrendered shadows until all are processed.*/
	void RenderModulatedShadowProjections(FRHICommandList& RHICmdList, int32 ViewIndex, const FViewInfo& View);

	/** Issues occlusion queries */
	void RenderOcclusion(FRHICommandList& RHICmdList, FViewInfo& View, const struct FViewOcclusionQueries& ViewOcclusionQueries);

	bool ShouldRenderHZB(TArrayView<FViewInfo> InViews);


	/** Generate HZB */
	void RenderHZB(FRHICommandListImmediate& RHICmdList, const TRefCountPtr<IPooledRenderTarget>& SceneDepthZ);
	void RenderHZB(FRDGBuilder& GraphBuilder, FRDGTextureRef SceneDepthTexture);

	/** Computes how many queries will be issued this frame */
	int32 ComputeNumOcclusionQueriesToBatch() const;


	/** Renders decals. */
	TConstArrayView<const FVisibleDecal*> GetRelevantDeferredDecals(int32 DecalViewOffset, int32 ViewIndex, EDecalRenderStage DecalRenderStage);
	void RenderDecals(FRHICommandList& RHICmdList, FViewInfo& View, const FDecalViewInfo& DecalViewInfo, const FInstanceCullingDrawParams* InstanceCullingDrawParams);
	void RenderDeferredDecalsMobile(FRHICommandList& RHICmdList, const FViewInfo& View, const FDecalViewInfo& DecalViewInfo);
	void RenderDBuffer(FRDGBuilder& GraphBuilder, TArray<FViewInfo>& InViews, FSceneTextures& SceneTextures, FDBufferTextures& DBufferTextures, FInstanceCullingManager& InstanceCullingManager);

	/** Utility to fetch DecalViewOffset to pass to decal related rendering functions */
	int32 GetDecalViewOffset(TConstArrayView<FViewInfo> InViews) const;

	/** Renders the atmospheric and height fog */
	void RenderFog(FRHICommandList& RHICmdList, const FViewInfo& View);

	/** Renders the base pass for translucency. */
	static void RenderTranslucency(FRHICommandList& RHICmdList, const FViewInfo& View, TArrayView<FViewInfo> FamilyViewInfos, ETranslucencyPass::Type InStandardTranslucencyPass, EMeshPass::Type InStandardTranslucencyMeshPass, const FInstanceCullingDrawParams* InTranslucencyInstanceCullingDrawParams);

	/** On chip pre-tonemap before scene color MSAA resolve (iOS only) */
	void PreTonemapMSAA(FRHICommandList& RHICmdList, const FMinimalSceneTextures& SceneTextures);

	void SetupMobileBasePassAfterShadowInit(FExclusiveDepthStencil::Type BasePassDepthStencilAccess, TArrayView<FViewCommands> ViewCommandsPerView, FInstanceCullingManager& InstanceCullingManager);

	void UpdateDirectionalLightUniformBuffers(FRDGBuilder& GraphBuilder, const FViewInfo& View);
	void UpdateSkyReflectionUniformBuffer(FRHICommandListBase& RHICmdList);
	
	void RenderForward(FRDGBuilder& GraphBuilder, FRDGTextureRef ViewFamilyTexture, FSceneTextures& SceneTextures, FMobileBasePassTextures& BasePassTextures, FInstanceCullingManager& InstanceCullingManager);
	void RenderForwardSinglePass(FRDGBuilder& GraphBuilder, class FMobileRenderPassParameters* PassParameters, struct FRenderViewContext& ViewContext, FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager, struct FViewOcclusionQueries* ViewOcclusionQueries);
	void RenderForwardMultiPass(FRDGBuilder& GraphBuilder, class FMobileRenderPassParameters* PassParameters, struct FRenderViewContext& ViewContext, FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager, struct FViewOcclusionQueries* ViewOcclusionQueries);
	
	void RenderDeferredSinglePass(FRDGBuilder& GraphBuilder, FSceneTextures& SceneTextures, const FSortedLightSetSceneInfo& SortedLightSet, FMobileBasePassTextures& BasePassTextures, FInstanceCullingManager& InstanceCullingManager);
	void RenderDeferredMultiPass(FRDGBuilder& GraphBuilder, FSceneTextures& SceneTextures, const FSortedLightSetSceneInfo& SortedLightSet, FMobileBasePassTextures& BasePassTextures, FInstanceCullingManager& InstanceCullingManager);
	void RenderDeferredMultiPassDecals(FRDGBuilder& GraphBuilder, FViewInfo& View, int32 DecalViewOffset, uint32 ViewIndex, FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager);

	void RenderDistortion(FRDGBuilder& GraphBuilder,
		const FViewInfo& View,
		FRDGTextureRef SceneColorTexture,
		FRDGTextureRef SceneDepthTexture);

	void RenderStereoMotionVectors(FRDGBuilder& GraphBuilder, FViewInfo& View, const FSceneTextures& SceneTextures, FInstanceCullingManager& InstanceCullingManager);
	bool HasStereoMotionVectorRenderTargets(const FSceneTextures& SceneTextures);

	FRDGTextureRef RenderAmbientOcclusion(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

	void ClearMobileScreenSpaceShadows(FRDGBuilder& GraphBuilder, const FViewInfo& View, const FLightSceneProxy* LightSceneProxy, FRDGTextureRef ScreenShadowMaskTexture);
	void RenderMobileScreenSpaceShadows(FRDGBuilder& GraphBuilder, FIntPoint Extent, FRDGTextureRef ScreenShadowMaskTexture);
	FRDGTextureRef RenderMobileShadowProjections(FRDGBuilder& GraphBuilder, FIntPoint Extent, bool bIsMobileMultiView);

	FRenderTargetBindingSlots InitRenderTargetBindings_Deferred(FSceneTextures& SceneTextures, FColorTargets& ColorTargets);
	FRenderTargetBindingSlots InitRenderTargetBindings_Forward(FRDGTextureRef ViewFamilyTexture, FSceneTextures& SceneTextures);
	FColorTargets GetColorTargets_Deferred(FSceneTextures& SceneTextures);

	void GatherShadowPointLights(const FViewInfo& View, const FSortedLightSetSceneInfo& SortedLightSet, TArray<int32, TInlineAllocator<8>>& OutShadowPointLightIndexes);

private:
	const bool bGammaSpace;
	const bool bDeferredShading;
	const bool bRequiresDBufferDecals;
	const bool bUseVirtualTexturing;
	const bool bSupportsSimpleLights;
	bool bTonemapSubpass;
	bool bTonemapSubpassInline;
	int32 NumMSAASamples;
	bool bRenderToSceneColor;
	bool bRequiresMultiPass;
	bool bKeepDepthContent;
	bool bModulatedShadowsInUse;
	bool bShouldRenderCustomDepth;
	bool bShouldRenderVelocities;
	bool bShouldRenderHZB;
	bool bIsFullDepthPrepassEnabled;
	bool bIsMaskedOnlyDepthPrepassEnabled;
	bool bRequiresSceneDepthAux;
	bool bEnableClusteredLocalLights;
	bool bEnableClusteredReflections;
	bool bRequiresShadowProjections;
	bool bAdrenoOcclusionMode;
	bool bEnableDistanceFieldAO;
	bool bShouldRenderVolumetricFog;
	ETranslucencyPass::Type StandardTranslucencyPass;
	EMeshPass::Type StandardTranslucencyMeshPass;
	FDecalVisibilityTaskData* DecalVisibilityTaskData = nullptr;
	const FViewInfo* CachedView = nullptr;
};

// The noise textures need to be set in Slate too.
RENDERER_API void UpdateNoiseTextureParameters(FViewUniformShaderParameters& ViewUniformShaderParameters);

struct FFastVramConfig
{
	FFastVramConfig();
	void Update();
	void OnCVarUpdated();
	void OnSceneRenderTargetsAllocated();

	ETextureCreateFlags GBufferA;
	ETextureCreateFlags GBufferB;
	ETextureCreateFlags GBufferC;
	ETextureCreateFlags GBufferD;
	ETextureCreateFlags GBufferE;
	ETextureCreateFlags GBufferF;
	ETextureCreateFlags GBufferVelocity;
	ETextureCreateFlags HZB;
	ETextureCreateFlags SceneDepth;
	ETextureCreateFlags SceneColor;
	ETextureCreateFlags Bloom;
	ETextureCreateFlags BokehDOF;
	ETextureCreateFlags CircleDOF;
	ETextureCreateFlags CombineLUTs;
	ETextureCreateFlags Downsample;
	ETextureCreateFlags EyeAdaptation;
	ETextureCreateFlags Histogram;
	ETextureCreateFlags HistogramReduce;
	ETextureCreateFlags VelocityFlat;
	ETextureCreateFlags VelocityMax;
	ETextureCreateFlags MotionBlur;
	ETextureCreateFlags Tonemap;
	ETextureCreateFlags Upscale;
	ETextureCreateFlags DistanceFieldNormal;
	ETextureCreateFlags DistanceFieldAOHistory;
	ETextureCreateFlags DistanceFieldAOBentNormal;
	ETextureCreateFlags DistanceFieldAODownsampledBentNormal;
	ETextureCreateFlags DistanceFieldShadows;
	ETextureCreateFlags DistanceFieldIrradiance;
	ETextureCreateFlags DistanceFieldAOConfidence;
	ETextureCreateFlags Distortion;
	ETextureCreateFlags ScreenSpaceShadowMask;
	ETextureCreateFlags VolumetricFog;
	ETextureCreateFlags SeparateTranslucency;
	ETextureCreateFlags SeparateTranslucencyModulate;
	ETextureCreateFlags ScreenSpaceAO;
	ETextureCreateFlags SSR;
	ETextureCreateFlags DBufferA;
	ETextureCreateFlags DBufferB;
	ETextureCreateFlags DBufferC;
	ETextureCreateFlags DBufferMask;
	ETextureCreateFlags DOFSetup;
	ETextureCreateFlags DOFReduce;
	ETextureCreateFlags DOFPostfilter;
	ETextureCreateFlags PostProcessMaterial;

	ETextureCreateFlags CustomDepth;
	ETextureCreateFlags ShadowPointLight;
	ETextureCreateFlags ShadowPerObject;
	ETextureCreateFlags ShadowCSM;

	// Buffers
	EBufferUsageFlags DistanceFieldCulledObjectBuffers;
	EBufferUsageFlags DistanceFieldTileIntersectionResources;
	EBufferUsageFlags DistanceFieldAOScreenGridResources;
	EBufferUsageFlags ForwardLightingCullingResources;
	EBufferUsageFlags GlobalDistanceFieldCullGridBuffers;
	bool bDirty;

private:
	bool UpdateTextureFlagFromCVar(TAutoConsoleVariable<int32>& CVar, ETextureCreateFlags& InOutValue);
	bool UpdateBufferFlagFromCVar(TAutoConsoleVariable<int32>& CVar, EBufferUsageFlags& InOutValue);
};

extern FFastVramConfig GFastVRamConfig;

/** Returns the array of shadows with distance fields. Call only after finishing shadow initialization. */
extern TConstArrayView<FProjectedShadowInfo*> GetProjectedDistanceFieldShadows(const FDynamicShadowsTaskData* TaskData);

/** Triggers shadow gather dynamic mesh elements task graph to start processing. */
extern void BeginShadowGatherDynamicMeshElements(FDynamicShadowsTaskData* TaskData);

extern bool UseCachedMeshDrawCommands();
extern bool UseCachedMeshDrawCommands_AnyThread();
extern bool IsDynamicInstancingEnabled(ERHIFeatureLevel::Type FeatureLevel);
enum class EGPUSkinCacheTransition
{
	FrameSetup,
	Renderer,
};

/** Resolves the view rect of scene color or depth using either a custom resolve or hardware resolve. */
void AddResolveSceneColorPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureMSAA SceneColor);
void AddResolveSceneDepthPass(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureMSAA SceneDepth);

/** Resolves all views for scene color / depth. */
void AddResolveSceneColorPass(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureMSAA SceneColor);
void AddResolveSceneDepthPass(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureMSAA SceneDepth);

/** Prepares virtual textures for feedback updates. */
void VirtualTextureFeedbackBegin(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FIntPoint SceneTextureExtent);

/** Creates a half resolution checkerboard min / max depth buffer from the input full resolution depth buffer. */
FRDGTextureRef CreateHalfResolutionDepthCheckerboardMinMax(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureRef SceneDepth);

/** Creates a half resolution depth buffer storing the min and max depth for each 2x2 pixel quad of a half resolution buffer. */
FRDGTextureRef CreateQuarterResolutionDepthMinAndMax(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureRef DepthTexture);

/** Creates a quarter resolution depth buffer storing the min and max depth for each 4x4 pixel quad of the depth buffer. */
void CreateQuarterResolutionDepthMinAndMaxFromDepthTexture(FRDGBuilder& GraphBuilder, TArrayView<const FViewInfo> Views, FRDGTextureRef DepthTexture, FRDGTextureRef& OutHalfResMinMax, FRDGTextureRef& OutQuarterResMinMax);

inline const FSceneTexturesConfig& FViewInfo::GetSceneTexturesConfig() const
{
	// TODO:  We are refactoring away use of the FSceneTexturesConfig::Get() global singleton, but need this workaround for now to avoid crashes
	return Family->bIsViewFamilyInfo ? ((const FViewFamilyInfo*)Family)->SceneTexturesConfig : FSceneTexturesConfig::Get();
}

inline const FSceneTextures& FViewInfo::GetSceneTextures() const
{
	return ((FViewFamilyInfo*)Family)->GetSceneTextures();
}

inline const FSceneTextures* FViewInfo::GetSceneTexturesChecked() const
{
	return ((FViewFamilyInfo*)Family)->GetSceneTexturesChecked();
}

inline FSceneUniformBuffer& FViewInfo::GetSceneUniforms() const
{
	return Family->GetSceneRenderer()->GetSceneUniforms();
}

/**
  * Returns a family from an array of views, with the assumption that all point to the same view family, which will be
  * true for the "Views" array in the scene renderer.  There are some utility functions that receive the Views array,
  * rather than the renderer itself, and this avoids confusing code that accesses Views[0], in addition to validating
  * the assumption that all Views have the same Family.  FViewFamilyInfo is used to access FSceneTextures|Config.
  */
inline FViewFamilyInfo& GetViewFamilyInfo(const TArray<FViewInfo>& Views)
{
	check(Views.Num() == 1 || Views[0].Family == Views.Last().Family);
	return *(FViewFamilyInfo*)Views[0].Family;
}

inline const FViewFamilyInfo& GetViewFamilyInfo(const TArray<const FViewInfo>& Views)
{
	check(Views.Num() == 1 || Views[0].Family == Views.Last().Family);
	return *(const FViewFamilyInfo*)Views[0].Family;
}

inline FViewFamilyInfo& GetViewFamilyInfo(const TArrayView<FViewInfo>& Views)
{
	check(Views.Num() == 1 || Views[0].Family == Views.Last().Family);
	return *(FViewFamilyInfo*)Views[0].Family;
}

inline const FViewFamilyInfo& GetViewFamilyInfo(const TArrayView<const FViewInfo>& Views)
{
	check(Views.Num() == 1 || Views[0].Family == Views.Last().Family);
	return *(const FViewFamilyInfo*)Views[0].Family;
}

/** Checks whether deferred primitive alpha holdout is supported in shaders.*/
bool IsDeferredPrimitiveAlphaHoldoutSupported();

/** Checks whether primitive alpha holdout is enabled.*/
bool IsPrimitiveAlphaHoldoutEnabled(const FViewInfo& View);

/** Checks whether primitive alpha holdout is enabled for any view.*/
bool IsPrimitiveAlphaHoldoutEnabledForAnyView(TArrayView<const FViewInfo> Views);

bool SceneCaptureRequiresAlphaChannel(const FSceneView& View);

/** Checks whether the material and scene proxy combination will modify the mesh position. Useful for determining whether the material can be substituted with a default material for depth rendering etc. */
bool DoMaterialAndPrimitiveModifyMeshPosition(const FMaterial& Material, const FPrimitiveSceneProxy* PrimitiveSceneProxy);
