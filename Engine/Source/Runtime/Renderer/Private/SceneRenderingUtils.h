// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
//#include "SceneManagement.h"
#include "SceneRendering.h"

template <typename PassParametersType>
inline void AddSimpleElementCollectorPass(const FSimpleElementCollector& SimpleElementCollector, FRDGBuilder& GraphBuilder, PassParametersType* PassParameters, const FViewInfo& View, const FMeshPassProcessorRenderState& DrawRenderState)
{
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("EditorPrimitives"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, &SimpleElementCollector, DrawRenderState](FRHICommandListImmediate& RHICmdList)
		{
			FSceneRenderer::SetStereoViewport(RHICmdList, View, 1.0f);

			SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_World, View.GetStereoPassInstanceFactor());
			SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SDPG_Foreground, View.GetStereoPassInstanceFactor());
		}
	);
}

template <typename PassParametersType>
inline void AddSimpleElementCollectorPass(const FSimpleElementCollector& SimpleElementCollector, FRDGBuilder& GraphBuilder, PassParametersType* PassParameters, const FViewInfo& View, const FMeshPassProcessorRenderState& DrawRenderState, ESceneDepthPriorityGroup SceneDepthPriorityGroup)
{
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("EditorPrimitives"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, &SimpleElementCollector, DrawRenderState, SceneDepthPriorityGroup](FRHICommandListImmediate& RHICmdList)
		{
			FSceneRenderer::SetStereoViewport(RHICmdList, View, 1.0f);

			SimpleElementCollector.DrawBatchedElements(RHICmdList, DrawRenderState, View, EBlendModeFilter::OpaqueAndMasked, SceneDepthPriorityGroup, View.GetStereoPassInstanceFactor());
		}
	);
}

template <typename PassParametersType>
inline void AddBatchedElementsPass(const FBatchedElements& BatchedElements, FRDGBuilder& GraphBuilder, PassParametersType* PassParameters, const FViewInfo& View, const FMeshPassProcessorRenderState& DrawRenderState)
{
	GraphBuilder.AddPass(
		RDG_EVENT_NAME("BatchedElements"),
		PassParameters,
		ERDGPassFlags::Raster,
		[&View, &BatchedElements, DrawRenderState](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.SetViewport(View.ViewRect.Min.X, View.ViewRect.Min.Y, 0.0f, View.ViewRect.Max.X, View.ViewRect.Max.Y, 1.0f);

			// Draw the view's batched simple elements(lines, sprites, etc).
			BatchedElements.Draw(RHICmdList, DrawRenderState, View.FeatureLevel, View, false);
		}
	);
}

namespace UE::Renderer::SceneCapture::Private
{
enum class ESetupViewFamilyFlags
{
	None = 0,
	CaptureSceneColor = (1 << 0),
	IsPlanarReflection = (1 << 1),
	OverrideVirtualTextureThrottle = (1 << 2),
	IsReflectionCapture = (1 << 3),
};
ENUM_CLASS_FLAGS(ESetupViewFamilyFlags);

enum class ESceneRendererCreationFlags
{
	None = 0,
	CaptureSceneColor = (1 << 0),
	CameraCut2D = (1 << 1),	// Only valid for 2D captures
	CopyMainViewTemporalSettings2D = (1 << 2), // Only valid for 2D captures
	OverrideVirtualTextureThrottle = (1 << 3),
	IsReflectionCapture = (1 << 4), // Only valid for runtime reflection cube captures
};
ENUM_CLASS_FLAGS(ESceneRendererCreationFlags);
} // namespace UE::Renderer::SceneCapture::Private