// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneTextureParameters.h"
#include "SceneRendering.h"
#include "SceneTextures.h"
#include "SystemTextures.h"

FSceneTextureParameters GetSceneTextureParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View)
{
	return GetSceneTextureParameters(GraphBuilder, View.GetSceneTextures());
}

FSceneTextureParameters GetSceneTextureParameters(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures)
{
	const auto& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	FSceneTextureParameters Parameters;

	// Should always have a depth buffer around allocated, since early z-pass is first.
	Parameters.SceneDepthTexture = SceneTextures.Depth.Resolve;
	Parameters.SceneStencilTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(Parameters.SceneDepthTexture, PF_X24_G8));

	// Registers all the scene texture from the scene context. No fallback is provided to catch mistake at shader parameter validation time
	// when a pass is trying to access a resource before any other pass actually created it.
	Parameters.GBufferVelocityTexture = GetIfProduced(SceneTextures.Velocity);
	Parameters.GBufferATexture = GetIfProduced(SceneTextures.GBufferA);
	Parameters.GBufferBTexture = GetIfProduced(SceneTextures.GBufferB);
	Parameters.GBufferCTexture = GetIfProduced(SceneTextures.GBufferC);
	Parameters.GBufferDTexture = GetIfProduced(SceneTextures.GBufferD);
	Parameters.GBufferETexture = GetIfProduced(SceneTextures.GBufferE);
	Parameters.GBufferFTexture = GetIfProduced(SceneTextures.GBufferF, SystemTextures.MidGrey);
	Parameters.GBufferSGGXTexture = GetIfProduced(SceneTextures.GBufferSGGX);

	return Parameters;
}

FSceneTextureParameters GetSceneTextureParameters(FRDGBuilder& GraphBuilder, TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextureUniformBuffer)
{
	FSceneTextureParameters Parameters;
	Parameters.SceneDepthTexture = (*SceneTextureUniformBuffer)->SceneDepthTexture;
	if (Parameters.SceneDepthTexture)
	{
		Parameters.SceneStencilTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(Parameters.SceneDepthTexture, PF_X24_G8));
	}
	Parameters.GBufferATexture = (*SceneTextureUniformBuffer)->GBufferATexture;
	Parameters.GBufferBTexture = (*SceneTextureUniformBuffer)->GBufferBTexture;
	Parameters.GBufferCTexture = (*SceneTextureUniformBuffer)->GBufferCTexture;
	Parameters.GBufferDTexture = (*SceneTextureUniformBuffer)->GBufferDTexture;
	Parameters.GBufferETexture = (*SceneTextureUniformBuffer)->GBufferETexture;
	Parameters.GBufferFTexture = (*SceneTextureUniformBuffer)->GBufferFTexture;
	Parameters.GBufferVelocityTexture = (*SceneTextureUniformBuffer)->GBufferVelocityTexture;
	Parameters.GBufferSGGXTexture = (*SceneTextureUniformBuffer)->GBufferSGGXTexture;
	return Parameters;
}

FSceneLightingChannelParameters GetSceneLightingChannelParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef LightingChannelsTexture)
{
	FSceneLightingChannelParameters Parameters;

	// Lighting channels might be disabled when all lights are on the same channel.
	// Also need to explictly check View.bUsesLightingChannels since other systems can cause LightingChannelsTexture to be created
	// (see FDeferredShadingSceneRenderer::CopyStencilToLightingChannelTexture(...)
	if (LightingChannelsTexture && View.bUsesLightingChannels)
	{
		Parameters.SceneLightingChannels = LightingChannelsTexture;
		Parameters.bSceneLightingChannelsValid = true;
	}
	else
	{
		Parameters.SceneLightingChannels = GSystemTextures.GetZeroUIntDummy(GraphBuilder);
		Parameters.bSceneLightingChannelsValid = false;
	}

	return Parameters;
}

FRDGBufferRef GetEyeAdaptationBuffer(FRDGBuilder& GraphBuilder, const FSceneView& View)
{
	if (View.HasValidEyeAdaptationBuffer())
	{
		return GraphBuilder.RegisterExternalBuffer(View.GetEyeAdaptationBuffer(), ERDGBufferFlags::MultiFrame);
	}
	else
	{
		return GSystemTextures.GetDefaultStructuredBuffer(GraphBuilder, sizeof(FVector4f), FVector4f::One());
	}
}