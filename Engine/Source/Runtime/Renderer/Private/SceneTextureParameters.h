// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneRenderTargetParameters.h"

struct FSceneTextures;

/** Contains reference to scene GBuffer textures. */
BEGIN_SHADER_PARAMETER_STRUCT(FSceneTextureParameters, )
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, SceneDepthTexture)
	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint2>, SceneStencilTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferATexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferBTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferCTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferDTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferETexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferFTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferVelocityTexture)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D, GBufferSGGXTexture)
END_SHADER_PARAMETER_STRUCT()

/** Constructs scene texture parameters from the scene context. */
FSceneTextureParameters GetSceneTextureParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View);

/** Constructs scene texture parameters from the scene texture blackboard struct. */
FSceneTextureParameters GetSceneTextureParameters(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures);

/** Constructs scene texture parameters from the scene texture uniform buffer. Useful if you prefer to use loose parameters. */
FSceneTextureParameters GetSceneTextureParameters(FRDGBuilder& GraphBuilder, TRDGUniformBufferRef<FSceneTextureUniformParameters> SceneTextureUniformBuffer);

BEGIN_SHADER_PARAMETER_STRUCT(FSceneLightingChannelParameters, )
	// SceneLightingChannels needs to be accessed with SceneLightingChannels.Load(), so a shader accessing
	// needs to know when it not valid since SceneLightingChannels could end up being a dummy system texture.
	SHADER_PARAMETER(uint32, bSceneLightingChannelsValid)
	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint>, SceneLightingChannels)
END_SHADER_PARAMETER_STRUCT()

/** Constructs lighting channel parameters from the scene context. */
FSceneLightingChannelParameters GetSceneLightingChannelParameters(FRDGBuilder& GraphBuilder, const FViewInfo& View, FRDGTextureRef LightingChannelsTexture);

/** Returns a render graph buffer resource reference onto the eye adaptation or fallback. */
RENDERER_API FRDGBufferRef GetEyeAdaptationBuffer(FRDGBuilder& GraphBuilder, const FSceneView& View);
