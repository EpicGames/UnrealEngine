// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SceneRendering.cpp: Scene rendering.
=============================================================================*/

#include "SceneTextureReductions.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderGraph.h"
#include "PixelShaderUtils.h"
#include "SceneTextureParameters.h"
#include "PostProcess/PostProcessing.h" // for FPostProcessVS
#include "Froxel/Froxel.h"
#include "SceneRendering.h"

static TAutoConsoleVariable<int32> CVarHZBBuildUseCompute(
	TEXT("r.HZB.BuildUseCompute"), 1,
	TEXT("Selects whether HZB should be built with compute."),
	ECVF_RenderThreadSafe);



BEGIN_SHADER_PARAMETER_STRUCT(FSharedHZBParameters, )
	SHADER_PARAMETER(FVector4f, DispatchThreadIdToBufferUV)
	SHADER_PARAMETER(FIntVector4, PixelViewPortMinMax)
	SHADER_PARAMETER(FVector4f, InvDeviceZToWorldZTransform)
	SHADER_PARAMETER(FVector2f, InputViewportMaxBound)
	SHADER_PARAMETER(FVector2f, InvSize)
	SHADER_PARAMETER(float, SceneDepthBias)

	SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, ParentTextureMip)
	SHADER_PARAMETER_SAMPLER(SamplerState, ParentTextureMipSampler)

	SHADER_PARAMETER_RDG_TEXTURE(Texture2D<uint2>, VisBufferTexture)
END_SHADER_PARAMETER_STRUCT()


class FHZBBuildPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHZBBuildPS);
	SHADER_USE_PARAMETER_STRUCT(FHZBBuildPS, FGlobalShader)

	class FDimUseMipIndex : SHADER_PERMUTATION_BOOL("DIM_USE_MIPINDEX");
	using FPermutationDomain = TShaderPermutationDomain<FDimUseMipIndex>;

	BEGIN_SHADER_PARAMETER_STRUCT( FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSharedHZBParameters, Shared)
		SHADER_PARAMETER(int32, SourceMipIndex)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetRenderTargetOutputFormat(0, PF_R32_FLOAT);
	}
};

class FHZBBuildCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FHZBBuildCS);
	SHADER_USE_PARAMETER_STRUCT(FHZBBuildCS, FGlobalShader)

	static constexpr int32 kMaxMipBatchSize = 4;

	class FDimVisBufferFormat : SHADER_PERMUTATION_INT("VIS_BUFFER_FORMAT", 5);
	class FDimFurthest : SHADER_PERMUTATION_BOOL("DIM_FURTHEST");
	class FDimClosest : SHADER_PERMUTATION_BOOL("DIM_CLOSEST");
	class FDimFroxels : SHADER_PERMUTATION_BOOL("DIM_FROXELS");
	class FDimMipLevelCount : SHADER_PERMUTATION_RANGE_INT("DIM_MIP_LEVEL_COUNT", 1, kMaxMipBatchSize);
	using FPermutationDomain = TShaderPermutationDomain<FDimFurthest, FDimClosest, FDimFroxels, FDimMipLevelCount, FDimVisBufferFormat>;

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FSharedHZBParameters, Shared)
		SHADER_PARAMETER_STRUCT_INCLUDE(Froxel::FBuilderParameters, FroxelBuilder)		
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<float>, FurthestHZBOutput, [kMaxMipBatchSize])
		SHADER_PARAMETER_RDG_TEXTURE_UAV_ARRAY(RWTexture2D<float>, ClosestHZBOutput, [kMaxMipBatchSize])
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		FPermutationDomain PermutationVector(Parameters.PermutationId);
		
		// No need for froxels on non-VSM platforms
		if (PermutationVector.Get<FDimFroxels>() && !DoesPlatformSupportVirtualShadowMaps(Parameters.Platform))
		{
			return false;
		}
		// Necessarily reduce at least closest or furthest.
		if (!PermutationVector.Get<FDimFurthest>() && !PermutationVector.Get<FDimClosest>())
		{
			return false;
		}

		return true;
	}
};

IMPLEMENT_GLOBAL_SHADER(FHZBBuildPS, "/Engine/Private/HZB.usf", "HZBBuildPS", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FHZBBuildCS, "/Engine/Private/HZB.usf", "HZBBuildCS", SF_Compute);

static bool HZBBuildUseCompute(EShaderPlatform ShaderPlatform)
{
	return CVarHZBBuildUseCompute.GetValueOnRenderThread() == 1;
}

void BuildHZB(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef SceneDepth,
	FRDGTextureRef VisBufferTexture,
	const FIntRect ViewRect,
	ERHIFeatureLevel::Type FeatureLevel,
	EShaderPlatform ShaderPlatform,
	const TCHAR* ClosestHZBName,
	FRDGTextureRef* OutClosestHZBTexture,
	const TCHAR* FurthestHZBName,
	FRDGTextureRef* OutFurthestHZBTexture,
	EPixelFormat Format,
	const FBuildHZBAsyncComputeParams* AsyncComputeParams,
	const Froxel::FViewData* OutFroxelViewData,
	FExtraParameters ExtraParameters)
{
	RDG_EVENT_SCOPE(GraphBuilder, "BuildHZB");
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BuildHZB);

	uint32 SizeShift = 1u;
	if (ExtraParameters.bLevel0Unscaled)
	{
		SizeShift = 0u;
	}

	FIntPoint HZBSize;
	HZBSize.X = FMath::Max( FPlatformMath::RoundUpToPowerOfTwo( ViewRect.Width() ) >> SizeShift, 1u );
	HZBSize.Y = FMath::Max( FPlatformMath::RoundUpToPowerOfTwo( ViewRect.Height() ) >> SizeShift, 1u );

	int32 NumMips = FMath::Max(FMath::FloorToInt(FMath::Log2(FMath::Max<float>(HZBSize.X, HZBSize.Y))), 1);

	bool bReduceClosestDepth = OutClosestHZBTexture != nullptr;
	// Force compute when a VisBuffer is provided, since HZBBuildPS does not support sampling it.
	bool bUseCompute = bReduceClosestDepth || (VisBufferTexture != nullptr) || HZBBuildUseCompute(ShaderPlatform);

	FRDGTextureDesc HZBDesc = FRDGTextureDesc::Create2D(
		HZBSize, Format,
		FClearValueBinding::None,
		TexCreate_ShaderResource | (bUseCompute ? TexCreate_UAV : TexCreate_RenderTargetable),
		NumMips);
	HZBDesc.Flags |= GFastVRamConfig.HZB;

	/** Closest and furthest HZB are intentionally in separate render target, because majority of the case you only one or the other.
	 * Keeping them separate avoid doubling the size in cache for this cases, to avoid performance regression.
	 */
	FRDGTextureRef FurthestHZBTexture = GraphBuilder.CreateTexture(HZBDesc, FurthestHZBName);

	FRDGTextureRef ClosestHZBTexture = nullptr;
	if (bReduceClosestDepth)
	{
		ClosestHZBTexture = GraphBuilder.CreateTexture(HZBDesc, ClosestHZBName);
	}

	int32 MaxMipBatchSize = bUseCompute ? FHZBBuildCS::kMaxMipBatchSize : 1;


	int32 MaxMipBatchSizeFirstPass = MaxMipBatchSize;

	// Avoid overflowing MaxSimultaneousUAVs by reducing the number of levels in the first pass (the only one that writes both near/far textures at once)
	if (bUseCompute && bReduceClosestDepth && OutFroxelViewData != nullptr && GRHIGlobals.MaxSimultaneousUAVs < 10)
	{
		// 6 UAVs for the ouput mip levels + 2 from the froxel builder
		MaxMipBatchSizeFirstPass = 3;
	}

	auto ReduceMips = [&](
		FRDGTextureSRVRef ParentTextureMip, FIntPoint SrcSize,
		int32 MipBatchSize,
		int32 StartDestMip, 
		FVector4f DispatchThreadIdToBufferUV, 
		FVector2D InputViewportMaxBound,
		FIntVector4 PixelViewPortMinMax, 
		bool bOutputClosest, bool bOutputFurthest, bool bProduceFroxelData, bool bLevel0Copy)
	{

		FSharedHZBParameters ShaderParameters;
		ShaderParameters.InvSize = FVector2f(1.0f / SrcSize.X, 1.0f / SrcSize.Y);
		ShaderParameters.InputViewportMaxBound = FVector2f(InputViewportMaxBound);	// LWC_TODO: Precision loss
		ShaderParameters.DispatchThreadIdToBufferUV = DispatchThreadIdToBufferUV;
		ShaderParameters.PixelViewPortMinMax = PixelViewPortMinMax;
		ShaderParameters.VisBufferTexture = VisBufferTexture;
		ShaderParameters.ParentTextureMip = ParentTextureMip;
		ShaderParameters.ParentTextureMipSampler = TStaticSamplerState<SF_Point>::GetRHI();
		ShaderParameters.SceneDepthBias = ExtraParameters.SceneDepthBias;
		ShaderParameters.InvDeviceZToWorldZTransform = ExtraParameters.InvDeviceZToWorldZTransform;

		FIntPoint DstSize = FIntPoint::DivideAndRoundUp(HZBSize, 1 << StartDestMip);

		if (bUseCompute)
		{
			int32 EndDestMip = FMath::Min(StartDestMip + MipBatchSize, NumMips);

			FHZBBuildCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHZBBuildCS::FParameters>();
			PassParameters->Shared = ShaderParameters;

			for (int32 DestMip = StartDestMip; DestMip < EndDestMip; DestMip++)
			{
				if (bOutputFurthest)
				{
					PassParameters->FurthestHZBOutput[DestMip - StartDestMip] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(FurthestHZBTexture, DestMip));
				}
				if (bOutputClosest)
				{
					PassParameters->ClosestHZBOutput[DestMip - StartDestMip] = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(ClosestHZBTexture, DestMip));
				}
			}

			int32 VisBufferFormat = 0;
			if( VisBufferTexture && StartDestMip == 0 )
			{
				const int32 VisBufferFormatR32G32 = 1;
				const int32 VisBufferFormatR32 = 2;
				const int32 VisBufferFormatR64 = 3;
				const int32 VisBufferFormatR32Copy = 4;

				if (VisBufferTexture->Desc.Format == PF_R32_UINT)
				{
					VisBufferFormat = bLevel0Copy ? VisBufferFormatR32Copy : VisBufferFormatR32;
				}
				else if (VisBufferTexture->Desc.Format == PF_R64_UINT)
				{
					// Vulkan uses a single UlongType that can't be viewed as a R32G32
					VisBufferFormat = VisBufferFormatR64;
				}
				else
				{
					VisBufferFormat = VisBufferFormatR32G32;
				}
			}

			if (bProduceFroxelData)
			{
				PassParameters->FroxelBuilder = OutFroxelViewData->GetBuilderParameters(GraphBuilder);
			}

			FHZBBuildCS::FPermutationDomain PermutationVector;
			PermutationVector.Set<FHZBBuildCS::FDimMipLevelCount>(EndDestMip - StartDestMip);
			PermutationVector.Set<FHZBBuildCS::FDimFurthest>(bOutputFurthest);
			PermutationVector.Set<FHZBBuildCS::FDimClosest>(bOutputClosest);
			PermutationVector.Set<FHZBBuildCS::FDimFroxels>(bProduceFroxelData);
			PermutationVector.Set<FHZBBuildCS::FDimVisBufferFormat>(VisBufferFormat);

			const bool bAsyncCompute = AsyncComputeParams != nullptr;
			const ERDGPassFlags PassFlags = bAsyncCompute ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute;

			TShaderMapRef<FHZBBuildCS> ComputeShader(GetGlobalShaderMap(FeatureLevel), PermutationVector);
			FRDGPassRef ReduceHZBPass = FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("ReduceHZB(mips=[%d;%d]%s%s%s) %dx%d",
					StartDestMip, EndDestMip - 1,
					bOutputClosest ? TEXT(" Closest") : TEXT(""),
					bOutputFurthest ? TEXT(" Furthest") : TEXT(""),
					bProduceFroxelData ? TEXT(" Froxels") : TEXT(""),
					DstSize.X, DstSize.Y),
				PassFlags,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(DstSize, 8));

			if (AsyncComputeParams && AsyncComputeParams->Prerequisite)
			{
				GraphBuilder.AddPassDependency(AsyncComputeParams->Prerequisite, ReduceHZBPass);
			}
		}
		else
		{
			check(!bProduceFroxelData);
			check(bOutputFurthest);
			check(!bOutputClosest);

			FHZBBuildPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FHZBBuildPS::FParameters>();
			PassParameters->Shared = ShaderParameters;
			PassParameters->SourceMipIndex = GRHISupportsTextureViews ? 0 : FMath::Max(0, StartDestMip - 1);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(FurthestHZBTexture, ERenderTargetLoadAction::ENoAction, StartDestMip);

			FHZBBuildPS::FPermutationDomain PermutationVector;
			// Use fallback with sampling specific HZB mip index if RHI does not support texture views (OpenGL)
			PermutationVector.Set<FHZBBuildPS::FDimUseMipIndex>(GRHISupportsTextureViews ? false : true);

			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(FeatureLevel);
			TShaderMapRef<FHZBBuildPS> PixelShader(GlobalShaderMap, PermutationVector);

			FPixelShaderUtils::AddFullscreenPass(
				GraphBuilder,
				GlobalShaderMap,
				RDG_EVENT_NAME("DownsampleHZB(mip=%d) %dx%d", StartDestMip, DstSize.X, DstSize.Y),
				PixelShader,
				PassParameters,
				FIntRect(0, 0, DstSize.X, DstSize.Y));
		}
	};

	// Reduce first mips Closest and furtherest are done at same time.
	{
		FRDGTextureSRVRef ParentTextureMip = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SceneDepth));
		FIntPoint SrcSize = (VisBufferTexture && !ExtraParameters.bLevel0Unscaled) ? VisBufferTexture->Desc.Extent : ParentTextureMip->Desc.Texture->Desc.Extent;
		
		FVector4f DispatchThreadIdToBufferUV;
		DispatchThreadIdToBufferUV.X = 2.0f / float(SrcSize.X);
		DispatchThreadIdToBufferUV.Y = 2.0f / float(SrcSize.Y);
		DispatchThreadIdToBufferUV.Z = ViewRect.Min.X / float(SrcSize.X);
		DispatchThreadIdToBufferUV.W = ViewRect.Min.Y / float(SrcSize.Y);

		FVector2D InputViewportMaxBound = FVector2D(
			float(ViewRect.Max.X - 0.5f) / float(SrcSize.X),
			float(ViewRect.Max.Y - 0.5f) / float(SrcSize.Y));
		FIntVector4 PixelViewPortMinMax = FIntVector4(
			ViewRect.Min.X, ViewRect.Min.Y, 
			ViewRect.Max.X - 1, ViewRect.Max.Y - 1);

		ReduceMips(
			ParentTextureMip, SrcSize,
			MaxMipBatchSizeFirstPass, /* StartDestMip = */ 0, 
			DispatchThreadIdToBufferUV, InputViewportMaxBound, PixelViewPortMinMax,
			/* bOutputClosest = */ bReduceClosestDepth, /* bOutputFurthest = */ true,
			OutFroxelViewData != nullptr,
			ExtraParameters.bLevel0Unscaled);
	}

	// Reduce the next mips
	for (int32 StartDestMip = MaxMipBatchSizeFirstPass; StartDestMip < NumMips; StartDestMip += MaxMipBatchSize)
	{
		FIntPoint SrcSize = FIntPoint::DivideAndRoundUp(HZBSize, 1 << int32(StartDestMip - 1));

		FVector4f DispatchThreadIdToBufferUV;
		DispatchThreadIdToBufferUV.X = 2.0f / float(SrcSize.X);
		DispatchThreadIdToBufferUV.Y = 2.0f / float(SrcSize.Y);
		DispatchThreadIdToBufferUV.Z = 0.0f;
		DispatchThreadIdToBufferUV.W = 0.0f;

		FVector2D InputViewportMaxBound(1.0f, 1.0f);
		FIntVector4 PixelViewPortMinMax = FIntVector4(0, 0, SrcSize.X - 1, SrcSize.Y - 1);
		
		{
			FRDGTextureSRVRef ParentTextureMip = GraphBuilder.CreateSRV(GRHISupportsTextureViews ? FRDGTextureSRVDesc::CreateForMipLevel(FurthestHZBTexture, StartDestMip - 1) : FRDGTextureSRVDesc::Create(FurthestHZBTexture));
			ReduceMips(ParentTextureMip, SrcSize,
				MaxMipBatchSize, StartDestMip, 
				DispatchThreadIdToBufferUV, InputViewportMaxBound, PixelViewPortMinMax,
				/* bOutputClosest = */ false, /* bOutputFurthest = */ true, false, false);
		}

		if (bReduceClosestDepth)
		{
			check(ClosestHZBTexture)
			FRDGTextureSRVRef ParentTextureMip = GraphBuilder.CreateSRV(GRHISupportsTextureViews ? FRDGTextureSRVDesc::CreateForMipLevel(ClosestHZBTexture, StartDestMip - 1) : FRDGTextureSRVDesc::Create(ClosestHZBTexture));
			ReduceMips(ParentTextureMip, SrcSize,
				MaxMipBatchSize, StartDestMip, 
				DispatchThreadIdToBufferUV, InputViewportMaxBound, PixelViewPortMinMax,
				/* bOutputClosest = */ true, /* bOutputFurthest = */ false, false, false);
		}
	}

	check(OutFurthestHZBTexture);
	*OutFurthestHZBTexture = FurthestHZBTexture;

	if (OutClosestHZBTexture)
	{
		*OutClosestHZBTexture = ClosestHZBTexture;
	}
}

void BuildHZBFurthest(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef SceneDepth,
	FRDGTextureRef VisBufferTexture,
	const FIntRect ViewRect,
	ERHIFeatureLevel::Type FeatureLevel,
	EShaderPlatform ShaderPlatform,
	const TCHAR* FurthestHZBName,
	FRDGTextureRef* OutFurthestHZBTexture,
	EPixelFormat Format,
	const FBuildHZBAsyncComputeParams* AsyncComputeParams,
	FExtraParameters ExtraParameters
)
{
	BuildHZB(
		GraphBuilder,
		SceneDepth,
		VisBufferTexture,
		ViewRect,
		FeatureLevel,
		ShaderPlatform,
		TEXT("HZBClosest"),
		nullptr,
		FurthestHZBName,
		OutFurthestHZBTexture,
		Format,
		AsyncComputeParams,
		nullptr,
		ExtraParameters);
}
