// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelFormat.h"
#include "RenderGraphFwd.h"
#include "RHIFwd.h"
#include "Math/Vector4.h"

namespace Froxel
{
	struct FViewData;
}

struct FBuildHZBAsyncComputeParams
{
	FRDGPassRef Prerequisite = nullptr;
};

static constexpr EPixelFormat BuildHZBDefaultPixelFormat = PF_R16F;

struct FExtraParameters
{
	FVector4f InvDeviceZToWorldZTransform = FVector4f(0.0f, 0.0f, 1.0f, 0.0f);
	float SceneDepthBias = 0.0f;
	bool bLevel0Unscaled = false;
};

void BuildHZB(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef SceneDepth,
	FRDGTextureRef VisBufferTexture,
	const FIntRect ViewRect,
	ERHIFeatureLevel::Type FeatureLevel,
	EShaderPlatform ShaderPlatform,
	const TCHAR* ClosestHZBName,
	FRDGTextureRef* OutClosestHZBTexture,
	const TCHAR* FurthestHZBName,
	FRDGTextureRef* OutFurthestHZBTexture,
	EPixelFormat Format = BuildHZBDefaultPixelFormat,
	const FBuildHZBAsyncComputeParams* AsyncComputeParams = nullptr,
	const Froxel::FViewData* OutFroxelData = nullptr,
	FExtraParameters ExtraParameters = {});


// Build only the furthest HZB
void BuildHZBFurthest(
	FRDGBuilder& GraphBuilder,
	FRDGTextureRef SceneDepth,
	FRDGTextureRef VisBufferTexture,
	const FIntRect ViewRect,
	ERHIFeatureLevel::Type FeatureLevel,
	EShaderPlatform ShaderPlatform,
	const TCHAR* FurthestHZBName,
	FRDGTextureRef* OutFurthestHZBTexture,
	EPixelFormat Format = BuildHZBDefaultPixelFormat,
	const FBuildHZBAsyncComputeParams* AsyncComputeParams = nullptr,
	FExtraParameters ExtraParameters = {});
