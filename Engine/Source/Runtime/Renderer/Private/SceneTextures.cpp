// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneTextures.h"
#include "Shader.h"
#include "SceneUtils.h"
#include "SceneRenderTargetParameters.h"
#include "SceneTextureParameters.h"
#include "VelocityRendering.h"
#include "RenderUtils.h"
#include "EngineGlobals.h"
#include "UnrealEngine.h"
#include "LogRenderer.h"
#include "SceneRendering.h"
#include "StereoRendering.h"
#include "StereoRenderTargetManager.h"
#include "CompositionLighting/PostProcessAmbientOcclusion.h"
#include "PostProcess/PostProcessCompositeEditorPrimitives.h"
#include "ShaderCompiler.h"
#include "SystemTextures.h"
#include "PostProcess/PostProcessing.h"
#include "PostProcess/PostProcessAmbientOcclusionMobile.h"
#include "IHeadMountedDisplayModule.h"
#include "Substrate/Substrate.h"
#include "VisualizeTexture.h"
#include "PostProcess/PostProcessSubsurface.h"
#include "DBufferTextures.h"

static TAutoConsoleVariable<int32> CVarSceneTargetsResizeMethod(
	TEXT("r.SceneRenderTargetResizeMethod"),
	0,
	TEXT("Control the scene render target resize method:\n")
	TEXT("(This value is only used in game mode and on windowing platforms unless 'r.SceneRenderTargetsResizingMethodForceOverride' is enabled.)\n")
	TEXT("0: Resize to match requested render size (Default) (Least memory use, can cause stalls when size changes e.g. ScreenPercentage)\n")
	TEXT("1: Fixed to screen resolution.\n")
	TEXT("2: Expands to encompass the largest requested render dimension. (Most memory use, least prone to allocation stalls.)"),	
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarSceneTargetsResizeMethodForceOverride(
	TEXT("r.SceneRenderTargetResizeMethodForceOverride"),
	0,
	TEXT("Forces 'r.SceneRenderTargetResizeMethod' to be respected on all configurations.\n")
	TEXT("0: Disabled.\n")
	TEXT("1: Enabled.\n"),
	ECVF_RenderThreadSafe
	);

static TAutoConsoleVariable<int32> CVarMSAACount(
	TEXT("r.MSAACount"),
	4,
	TEXT("Number of MSAA samples to use with the forward renderer.  Only used when MSAA is enabled in the rendering project settings.\n")
	TEXT("0: MSAA disabled (Temporal AA enabled)\n")
	TEXT("1: MSAA disabled\n")
	TEXT("2: Use 2x MSAA\n")
	TEXT("4: Use 4x MSAA")
	TEXT("8: Use 8x MSAA"),
	ECVF_RenderThreadSafe | ECVF_Scalability
	);

static TAutoConsoleVariable<int32> CVarGBufferFormat(
	TEXT("r.GBufferFormat"),
	1,
	TEXT("Defines the memory layout used for the GBuffer.\n")
	TEXT("(affects performance, mostly through bandwidth, quality of normals and material attributes).\n")
	TEXT(" 0: lower precision (8bit per component, for profiling)\n")
	TEXT(" 1: low precision (default)\n")
	TEXT(" 3: high precision normals encoding\n")
	TEXT(" 5: high precision"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarDefaultBackBufferPixelFormat(
	TEXT("r.DefaultBackBufferPixelFormat"),
	4,
	TEXT("Defines the default back buffer pixel format.\n")
	TEXT(" 0: 8bit RGBA\n")
	TEXT(" 1: 16bit RGBA\n")
	TEXT(" 2: Float RGB\n")
	TEXT(" 3: Float RGBA\n")
	TEXT(" 4: 10bit RGB, 2bit Alpha\n"),
	ECVF_ReadOnly);

RDG_REGISTER_BLACKBOARD_STRUCT(FSceneTextures);

EPixelFormat FSceneTextures::GetGBufferFFormatAndCreateFlags(ETextureCreateFlags& OutCreateFlags)
{
	const int32 GBufferFormat = CVarGBufferFormat.GetValueOnAnyThread();
	const bool bHighPrecisionGBuffers = (GBufferFormat >= EGBufferFormat::Force16BitsPerChannel);
	const bool bEnforce8BitPerChannel = (GBufferFormat == EGBufferFormat::Force8BitsPerChannel);
	EPixelFormat NormalGBufferFormat = bHighPrecisionGBuffers ? PF_FloatRGBA : PF_B8G8R8A8;

	if (bEnforce8BitPerChannel)
	{
		NormalGBufferFormat = PF_B8G8R8A8;
	}
	else if (GBufferFormat == EGBufferFormat::HighPrecisionNormals)
	{
		NormalGBufferFormat = PF_FloatRGBA;
	}

	OutCreateFlags = TexCreate_RenderTargetable | TexCreate_ShaderResource | GFastVRamConfig.GBufferF;
	return NormalGBufferFormat;
}

static IStereoRenderTargetManager* FindStereoRenderTargetManager()
{
	if (!GEngine->StereoRenderingDevice.IsValid() || !GEngine->StereoRenderingDevice->IsStereoEnabled())
	{
		return nullptr;
	}

	return GEngine->StereoRenderingDevice->GetRenderTargetManager();
}

static TRefCountPtr<FRHITexture> FindStereoDepthTexture(uint32 bSupportsXRDepth, FIntPoint TextureExtent, ETextureCreateFlags RequestedCreateFlags, uint8 NumSamples)
{
	if (bSupportsXRDepth == 1)
	{
		if (IStereoRenderTargetManager* StereoRenderTargetManager = FindStereoRenderTargetManager())
		{
			TRefCountPtr<FRHITexture> DepthTex, SRTex;
			StereoRenderTargetManager->AllocateDepthTexture(0, TextureExtent.X, TextureExtent.Y, PF_DepthStencil, 1, RequestedCreateFlags, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead, DepthTex, SRTex, NumSamples);
			return MoveTemp(SRTex);
		}
	}
	return nullptr;
}

static bool FindStereoMotionVectorTexture(FIntPoint& OutVelocityTextureSize, FTextureRHIRef& OutVelocityTexture, FTextureRHIRef& OutVelocityDepthTexture)
{
	if (IStereoRenderTargetManager* StereoRenderTargetManager = FindStereoRenderTargetManager())
	{
		if (StereoRenderTargetManager->GetRecommendedMotionVectorTextureSize(OutVelocityTextureSize))
		{
			if (StereoRenderTargetManager->GetMotionVectorTexture(0, OutVelocityTextureSize, PF_FloatRGBA, 1, TexCreate_RenderTargetable | TexCreate_ResolveTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead, OutVelocityTexture))
			{
				StereoRenderTargetManager->GetMotionVectorDepthTexture(0, OutVelocityTextureSize, PF_DepthStencil, 1, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead, OutVelocityDepthTexture);
				return true;
			}
		}
	}
	
	OutVelocityTexture = nullptr;
	OutVelocityDepthTexture = nullptr;
	return false;
}

/** Helper class used to track and compute a suitable scene texture extent for the renderer based on history / global configuration. */
class FSceneTextureExtentState
{
public:
	static FSceneTextureExtentState& Get()
	{
		static FSceneTextureExtentState Instance;
		return Instance;
	}

	FIntPoint Compute(const FSceneViewFamily& ViewFamily)
	{
		enum ESizingMethods { RequestedSize, ScreenRes, Grow, VisibleSizingMethodsCount };
		ESizingMethods SceneTargetsSizingMethod = Grow;

		bool bIsSceneCapture = false;
		bool bIsReflectionCapture = false;
		bool bIsVRScene = false;
		// If explicitly requested, don't include this in history tracking for future requests.  This extent request can still be adjusted based on existing history.
		bool bExcludeFromHistoryUpdate = false;

		// If this function is called for a CustomRenderPass view family, it means it is getting its own scene textures,
		// separate from the main renderer, and only its own views should be counted, not AllViews.
		if (ViewFamily.Views[0]->CustomRenderPass)
		{
			for (const FSceneView* View : ViewFamily.Views)
			{
				bIsSceneCapture |= View->bIsSceneCapture;
				bIsReflectionCapture |= View->bIsReflectionCapture;
				bExcludeFromHistoryUpdate |= View->bExcludeFromSceneTextureExtents;
			}
		}
		else
		{
			for (const FSceneView* View : ViewFamily.AllViews)
			{
				// If a custom render pass has bExcludeFromSceneTextureExtents set, it will get its own scene textures, and shouldn't be counted in the main view family.
				if (View->CustomRenderPass && View->bExcludeFromSceneTextureExtents)
				{
					continue;
				}

				bIsSceneCapture |= View->bIsSceneCapture;
				bIsReflectionCapture |= View->bIsReflectionCapture;
				bIsVRScene |= (IStereoRendering::IsStereoEyeView(*View) && GEngine->XRSystem.IsValid());
				bExcludeFromHistoryUpdate |= View->bExcludeFromSceneTextureExtents;
			}
		}

		FIntPoint DesiredExtent = FIntPoint::ZeroValue;
		FIntPoint DesiredFamilyExtent = FSceneRenderer::GetDesiredInternalBufferSize(ViewFamily);

		{
			bool bUseResizeMethodCVar = true;

			if (CVarSceneTargetsResizeMethodForceOverride.GetValueOnRenderThread() != 1)
			{
				if (!FPlatformProperties::SupportsWindowedMode() || bIsVRScene)
				{
					if (bIsVRScene)
					{
						if (!bIsSceneCapture && !bIsReflectionCapture)
						{
							// If this is VR, but not a capture (only current XR capture is for Planar Reflections), then we want
							// to use the requested size. Ideally, capture targets will be able to 'grow' into the VR extents.
							if (DesiredFamilyExtent.X != LastStereoExtent.X || DesiredFamilyExtent.Y != LastStereoExtent.Y)
							{
								LastStereoExtent = DesiredFamilyExtent;
								UE_LOGF(LogRenderer, Warning, "Resizing VR buffer to %d by %d", DesiredFamilyExtent.X, DesiredFamilyExtent.Y);
							}
							SceneTargetsSizingMethod = RequestedSize;
						}
						else
						{
							// If this is a VR scene capture (i.e planar reflection), and it's smaller than the VR view size, then don't re-allocate buffers, just use the "grow" method.
							// If it's bigger than the VR view, then log a warning, and use resize method.
							if (DesiredFamilyExtent.X > LastStereoExtent.X || DesiredFamilyExtent.Y > LastStereoExtent.Y)
							{
								if (LastStereoExtent.X > 0 && bIsSceneCapture)
								{
									static bool DisplayedCaptureSizeWarning = false;
									if (!DisplayedCaptureSizeWarning)
									{
										DisplayedCaptureSizeWarning = true;
										UE_LOGF(LogRenderer, Warning, 
											"Scene capture of %d by %d is larger than the current VR target. If this is deliberate for a capture that is being done for multiple frames, consider the performance and memory implications. "
											"To disable this warning and ensure optimal behavior with this path, set r.SceneRenderTargetResizeMethod to 2, and r.SceneRenderTargetResizeMethodForceOverride to 1.", 
											DesiredFamilyExtent.X, DesiredFamilyExtent.Y);
									}
								}
								SceneTargetsSizingMethod = RequestedSize;
							}
							else
							{
								SceneTargetsSizingMethod = Grow;
							}
						}
					}
					else
					{
						// Force ScreenRes on non windowed platforms.
						SceneTargetsSizingMethod = RequestedSize;
					}
					bUseResizeMethodCVar = false;
				}
				else if (GIsEditor)
				{
					// Always grow scene render targets in the editor.
					SceneTargetsSizingMethod = Grow;
					bUseResizeMethodCVar = false;
				}
			}

			if (bUseResizeMethodCVar)
			{
				// Otherwise use the setting specified by the console variable.
				// #jira UE-156400: The 'clamp()' includes min and max values, so the range is [0 .. Count-1]
				// The checkNoEntry() macro is called from 'default:' in the switch() below, when the SceneTargetsSizingMethod is out of the supported range.
				SceneTargetsSizingMethod = (ESizingMethods)FMath::Clamp(CVarSceneTargetsResizeMethod.GetValueOnRenderThread(), 0, (int32)VisibleSizingMethodsCount - 1);
			}
		}

		switch (SceneTargetsSizingMethod)
		{
		case RequestedSize:
			DesiredExtent = DesiredFamilyExtent;
			break;

		case ScreenRes:
			DesiredExtent = FIntPoint(GSystemResolution.ResX, GSystemResolution.ResY);
			break;

		case Grow:
			DesiredExtent = FIntPoint(
				FMath::Max((int32)LastExtent.X, DesiredFamilyExtent.X),
				FMath::Max((int32)LastExtent.Y, DesiredFamilyExtent.Y));
			break;

		default:
			checkNoEntry();
		}

		const uint32 FrameNumber = ViewFamily.FrameNumber;
		if (ThisFrameNumber != FrameNumber)
		{
			ThisFrameNumber = FrameNumber;
			if (++DesiredExtentIndex == ExtentHistoryCount)
			{
				DesiredExtentIndex -= ExtentHistoryCount;
			}
			// This allows the extent to shrink each frame (in game)
			LargestDesiredExtents[DesiredExtentIndex] = FIntPoint::ZeroValue;
			HistoryFlags[DesiredExtentIndex] = ERenderTargetHistory::None;
		}

		// this allows The extent to not grow below the SceneCapture requests (happen before scene rendering, in the same frame with a Grow request)
		FIntPoint& LargestDesiredExtentThisFrame = LargestDesiredExtents[DesiredExtentIndex];
		bool bIsHighResScreenshot = GIsHighResScreenshot;

		if (!bExcludeFromHistoryUpdate)
		{
			LargestDesiredExtentThisFrame = LargestDesiredExtentThisFrame.ComponentMax(DesiredExtent);
			UpdateHistoryFlags(HistoryFlags[DesiredExtentIndex], bIsSceneCapture, bIsReflectionCapture, bIsHighResScreenshot);
		}

		// We want to shrink the buffer but as we can have multiple scene captures per frame we have to delay that a frame to get all size requests
		// We have to use the requested size when allocating an hmd depth target to ensure it matches the hmd allocated render target size.
		bool bAllowDelayResize = !GIsHighResScreenshot && !bIsVRScene;

		// Don't consider the history buffer when the aspect ratio changes, the existing buffers won't make much sense at all.
		// This prevents problems when orientation changes on mobile in particular.
		// bIsReflectionCapture is explicitly checked on all platforms to prevent aspect ratio change detection from forcing the immediate buffer resize.
		// This ensures that 1) buffers are not resized spuriously during reflection rendering 2) all cubemap faces use the same render target size.
		if (bAllowDelayResize && !bIsReflectionCapture && !bExcludeFromHistoryUpdate && !AnyCaptureRenderedRecently<ExtentHistoryCount>(ERenderTargetHistory::MaskAll))
		{
			const bool bAspectRatioChanged =
				!LastExtent.Y ||
				!FMath::IsNearlyEqual(
					(float)LastExtent.X / LastExtent.Y,
					(float)DesiredExtent.X / DesiredExtent.Y);

			if (bAspectRatioChanged)
			{
				bAllowDelayResize = false;

				// At this point we're assuming a simple output resize and forcing a hard swap so clear the history.
				// If we don't the next frame will fail this check as the allocated aspect ratio will match the new
				// frame's forced size so we end up looking through the history again, finding the previous old size
				// and reallocating. Only after a few frames can the results actually settle when the history clears 
				for (int32 i = 0; i < ExtentHistoryCount; ++i)
				{
					LargestDesiredExtents[i] = FIntPoint::ZeroValue;
					HistoryFlags[i] = ERenderTargetHistory::None;
				}
			}
		}
		const bool bAnyHighresScreenshotRecently = AnyCaptureRenderedRecently<ExtentHistoryCount>(ERenderTargetHistory::HighresScreenshot);
		if (bAnyHighresScreenshotRecently != GIsHighResScreenshot)
		{
			bAllowDelayResize = false;
		}

		if (bAllowDelayResize)
		{
			for (int32 i = 0; i < ExtentHistoryCount; ++i)
			{
				DesiredExtent = DesiredExtent.ComponentMax(LargestDesiredExtents[i]);
			}
		}

		check(DesiredExtent.X > 0 && DesiredExtent.Y > 0);
		QuantizeSceneBufferSize(DesiredExtent, DesiredExtent);
		if (!bExcludeFromHistoryUpdate)
		{
			LastExtent = DesiredExtent;
		}
		return DesiredExtent;
	}

	void ResetHistory()
	{
		LastStereoExtent = FIntPoint(0, 0);
		LastExtent = FIntPoint(0, 0);
	}

private:
	enum class ERenderTargetHistory
	{
		None				= 0,
		SceneCapture		= 1 << 0,
		ReflectionCapture	= 1 << 1,
		HighresScreenshot	= 1 << 2,
		MaskAll				= 1 << 3,
	};
	FRIEND_ENUM_CLASS_FLAGS(ERenderTargetHistory);

	static void UpdateHistoryFlags(ERenderTargetHistory& Flags, bool bIsSceneCapture, bool bIsReflectionCapture, bool bIsHighResScreenShot)
	{
		Flags |= bIsSceneCapture ? ERenderTargetHistory::SceneCapture : ERenderTargetHistory::None;
		Flags |= bIsReflectionCapture ? ERenderTargetHistory::ReflectionCapture : ERenderTargetHistory::None;
		Flags |= bIsHighResScreenShot ? ERenderTargetHistory::HighresScreenshot : ERenderTargetHistory::None;
	}

	template <uint32 EntryCount>
	bool AnyCaptureRenderedRecently(ERenderTargetHistory Mask) const
	{
		ERenderTargetHistory Result = ERenderTargetHistory::None;
		for (uint32 EntryIndex = 0; EntryIndex < EntryCount; ++EntryIndex)
		{
			Result |= HistoryFlags[EntryIndex] & Mask;
		}
		return Result != ERenderTargetHistory::None;
	}

	FSceneTextureExtentState()
	{
		FMemory::Memset(LargestDesiredExtents, 0);
		FMemory::Memset(HistoryFlags, 0, sizeof(HistoryFlags));
	}

	FIntPoint LastStereoExtent = FIntPoint(0, 0);
	FIntPoint LastExtent = FIntPoint(0, 0);

	/** as we might get multiple extent requests each frame for SceneCaptures and we want to avoid reallocations we can only go as low as the largest request */
	static const uint32 ExtentHistoryCount = 3;
	uint32 DesiredExtentIndex = 0;
	FIntPoint LargestDesiredExtents[ExtentHistoryCount];
	ERenderTargetHistory HistoryFlags[ExtentHistoryCount];

	/** to detect when LargestDesiredSizeThisFrame is outdated */
	uint32 ThisFrameNumber = 0;
};

void ResetSceneTextureExtentHistory()
{
	FSceneTextureExtentState::Get().ResetHistory();
}

ENUM_CLASS_FLAGS(FSceneTextureExtentState::ERenderTargetHistory);

void InitializeSceneTexturesConfig(FSceneTexturesConfig& Config, const FSceneViewFamily& ViewFamily, FIntPoint ExtentOverride)
{
	FIntPoint Extent;
	if (ExtentOverride.X > 0)
	{
#if DO_CHECK
		for (const FSceneView* View : ViewFamily.Views)
		{
			check(View->UnscaledViewRect.Max.X <= ExtentOverride.X && View->UnscaledViewRect.Max.Y <= ExtentOverride.Y);
		}
#endif
		Extent = ExtentOverride;
	}
	else
	{
		Extent = FSceneTextureExtentState::Get().Compute(ViewFamily);
	}
	EShadingPath ShadingPath = GetFeatureLevelShadingPath(ViewFamily.GetFeatureLevel());

	bool bRequiresAlphaChannel = ShadingPath == EShadingPath::Mobile ? IsMobilePropagateAlphaEnabled(ViewFamily.GetShaderPlatform()) : IsPostProcessingWithAlphaChannelSupported();
	int32 NumberOfViewsWithMultiviewEnabled = 0;
	
	for (const FSceneView* View : ViewFamily.AllViews)
	{
		bRequiresAlphaChannel|= SceneCaptureRequiresAlphaChannel(*View);
		NumberOfViewsWithMultiviewEnabled += View->bIsMobileMultiViewEnabled ? 1 : 0;
	}

	ensureMsgf(NumberOfViewsWithMultiviewEnabled == 0 || NumberOfViewsWithMultiviewEnabled == ViewFamily.AllViews.Num(),
		TEXT("Either all or no views in a view family should have multiview enabled. Mixing views with enabled and disabled is not allowed."));

	const bool bAllViewsHaveMultiviewEnabled = NumberOfViewsWithMultiviewEnabled == ViewFamily.AllViews.Num();

	const bool bNeedsStereoAlloc = ViewFamily.AllViews.ContainsByPredicate([](const FSceneView* View)
		{
			return (IStereoRendering::IsStereoEyeView(*View) && (FindStereoRenderTargetManager() != nullptr));
		});

	FSceneTexturesConfigInitSettings SceneTexturesConfigInitSettings;
	SceneTexturesConfigInitSettings.FeatureLevel = ViewFamily.GetFeatureLevel();
	SceneTexturesConfigInitSettings.Extent = Extent;
	SceneTexturesConfigInitSettings.bRequireMultiView = ViewFamily.bRequireMultiView && bAllViewsHaveMultiviewEnabled;
	SceneTexturesConfigInitSettings.bRequiresAlphaChannel = bRequiresAlphaChannel;
	SceneTexturesConfigInitSettings.bSupportsXRTargetManagerDepthAlloc = bNeedsStereoAlloc ? 1 : 0;
	SceneTexturesConfigInitSettings.ExtraSceneColorCreateFlags = GFastVRamConfig.SceneColor;
	SceneTexturesConfigInitSettings.ExtraSceneDepthCreateFlags = GFastVRamConfig.SceneDepth;
	Config.Init(SceneTexturesConfigInitSettings);
}

static bool UseMSAAStereoDepthTextureDirectly()
{
	static int Mode = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.Mobile.XRMSAAMode"))->GetValueOnAnyThread();
	return Mode == 2;
}

void FMinimalSceneTextures::InitializeViewFamily(FRDGBuilder& GraphBuilder, FViewFamilyInfo& ViewFamily)
{
	checkf(ViewFamily.SceneTextures->Owner == &ViewFamily, TEXT("Scene Textures should only be initialized by their owning view family -- possible duplicate initialization"));

	const FSceneTexturesConfig& Config = ViewFamily.SceneTexturesConfig;
	FSceneTextures& SceneTextures = *ViewFamily.SceneTextures;

	checkf(Config.IsValid(), TEXT("Attempted to create scene textures with an empty config."));

	SceneTextures.Config = Config;

	// Scene Depth

	// If not using MSAA, we need to make sure to grab the stereo depth texture if appropriate.
	FTextureRHIRef StereoDepthRHI;
	bool bUseDepthTextureDirectly = Config.NumSamples == 1 || UseMSAAStereoDepthTextureDirectly();
	if (bUseDepthTextureDirectly && (StereoDepthRHI = FindStereoDepthTexture(Config.bSupportsXRTargetManagerDepthAlloc, Config.Extent, ETextureCreateFlags::None, Config.NumSamples)) != nullptr)

	{
		SceneTextures.Depth = RegisterExternalTexture(GraphBuilder, StereoDepthRHI, TEXT("SceneDepthZ"));
		SceneTextures.Stencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(SceneTextures.Depth.Target, PF_X24_G8));
	}
	else
	{
		FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(SceneTextures.Config.Extent, PF_DepthStencil, Config.DepthClearValue, Config.DepthCreateFlags, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
		Desc.NumSamples = Config.NumSamples;
		Desc.AliasableFormats.Add(PF_X24_G8);

		SceneTextures.Depth = GraphBuilder.CreateTexture(Desc, TEXT("SceneDepthZ"));

		if (Desc.NumSamples > 1)
		{
			Desc.NumSamples = 1;

			if ((StereoDepthRHI = FindStereoDepthTexture(Config.bSupportsXRTargetManagerDepthAlloc, Config.Extent, ETextureCreateFlags::DepthStencilResolveTarget, Desc.NumSamples)) != nullptr)
			{
				ensureMsgf(Desc.ArraySize == StereoDepthRHI->GetDesc().ArraySize, TEXT("Resolve texture does not agree in dimensionality with Target (Resolve.ArraySize=%d, Target.ArraySize=%d)"),
					Desc.ArraySize, StereoDepthRHI->GetDesc().ArraySize);
				SceneTextures.Depth.Resolve = RegisterExternalTexture(GraphBuilder, StereoDepthRHI, TEXT("SceneDepthZ"));
			}
			else if (Config.bKeepDepthContent)
			{
				SceneTextures.Depth.Resolve = GraphBuilder.CreateTexture(Desc, TEXT("SceneDepthZ"));
			}
		}

		SceneTextures.Stencil = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateWithPixelFormat(SceneTextures.Depth.Target, PF_X24_G8));
	}

	// Scene Color
	{
		const bool bIsMobilePlatform = Config.ShadingPath == EShadingPath::Mobile;
		const ETextureCreateFlags sRGBFlag = (bIsMobilePlatform && IsMobileColorsRGB()) ? TexCreate_SRGB : TexCreate_None;

		// Create the scene color.
		FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Config.Extent, Config.ColorFormat, Config.ColorClearValue, Config.ColorCreateFlags, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
		Desc.NumSamples = Config.NumSamples;
		SceneTextures.Color = CreateTextureMSAA(GraphBuilder, Desc, TEXT("SceneColorMS"), TEXT("SceneColor"), GFastVRamConfig.SceneColor | sRGBFlag);
	}

	// Custom Depth
	SceneTextures.CustomDepth = FCustomDepthTextures::Create(GraphBuilder, Config.Extent, Config.ShaderPlatform, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);

	SceneTextures.bIsSceneTexturesInitialized = true;
}

FSceneTextureShaderParameters FMinimalSceneTextures::GetSceneTextureShaderParameters(ERHIFeatureLevel::Type FeatureLevel) const
{
	FSceneTextureShaderParameters OutSceneTextureShaderParameters;
	if (FeatureLevel >= ERHIFeatureLevel::SM5)
	{
		OutSceneTextureShaderParameters.SceneTextures = UniformBuffer;
	}
	else
	{
		OutSceneTextureShaderParameters.MobileSceneTextures = MobileUniformBuffer;
	}
	return OutSceneTextureShaderParameters;
}

FRDGTextureRef FMinimalSceneTextures::FindOrAddUserSceneTexture(FRDGBuilder& GraphBuilder, int32 ViewIndex, FName Name, FIntPoint ResolutionDivisor, bool& bOutFirstRender, const UMaterialInterface* MaterialInterface, const FIntRect& OutputRect) const
{
	check(ResolutionDivisor.X >= 1 && ResolutionDivisor.Y >= 1);

	bool bFound = false;

	TArray<FTransientUserSceneTexture>* TransientTextures = UserSceneTextures.Find(Name);
	if (TransientTextures)
	{
		if ((*TransientTextures)[0].ResolutionDivisor == ResolutionDivisor)
		{
			bFound = true;
		}

		for (int32 TextureIndex = 1; TextureIndex < TransientTextures->Num(); ++TextureIndex)
		{
			if ((*TransientTextures)[TextureIndex].ResolutionDivisor == ResolutionDivisor)
			{
				// Swap found item to front of array and return it -- next material render should use the most recently written item
				TransientTextures->Swap(0, TextureIndex);
				bFound = true;
				break;
			}
		}
	}
	else
	{
		TransientTextures = &UserSceneTextures.Add(Name);
	}

	if (!bFound)
	{
		// We didn't find an existing item, need to allocate a new one
		FTransientUserSceneTexture TransientUserTexture;
		TransientUserTexture.ResolutionDivisor = ResolutionDivisor;
		TransientUserTexture.AllocationOrder = TransientTextures->Num();
		TransientUserTexture.bUsed = false;
		TransientUserTexture.ViewMask = 0;

		FIntPoint Extent = OutputRect.Size();

		FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Extent, Config.ColorFormat, Config.ColorClearValue, Config.ColorCreateFlags, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// Generate a heap allocated debug name for the resource, required to make UserSceneTextures accessible in VisualizeTexture,
		// GPU captures, and GPU dumps, as otherwise every texture will end up with the same debug name.
		FString DebugName = TransientUserTexture.AllocationOrder == 0 ?
			FString::Printf(TEXT("UST.%s"), *Name.ToString()) :
			FString::Printf(TEXT("UST.%s[%d]"), *Name.ToString(), TransientUserTexture.AllocationOrder);
		TCHAR* HeapDebugName = new TCHAR[DebugName.Len() + 1];
		FCString::Strncpy(HeapDebugName, *DebugName, DebugName.Len() + 1);

		TransientUserTexture.Texture = GraphBuilder.CreateTexture(Desc, HeapDebugName);
		TransientUserTexture.Texture->SetDebugNameIsHeapAllocated();
#else
		TransientUserTexture.Texture = GraphBuilder.CreateTexture(Desc, TEXT("TransientUserTexture"));
#endif

		TransientTextures->Add(TransientUserTexture);

		// Swap newly allocated item to front of array if it's not the only item
		if (TransientTextures->Num() > 1)
		{
			TransientTextures->Swap(0, TransientTextures->Num() - 1);
		}
	}

#if !(UE_BUILD_SHIPPING)
	UserSceneTextureEvents.Add({ EUserSceneTextureEvent::Output, Name, (*TransientTextures)[0].AllocationOrder, (uint16)ViewIndex, MaterialInterface, OutputRect.Size() });
#endif

	// If out of mask range, treat it as the first render.  This may result in certain transparent post process materials writing to UserSceneTextures
	// rendering incorrectly if there are more than 32 views, but we don't expect to see any real world situation with that number of views anyway
	// (and transparent post process materials themselves aren't common to begin with).
	if (ViewIndex > 31)
	{
		bOutFirstRender = true;
	}
	else
	{
		bOutFirstRender = ((*TransientTextures)[0].ViewMask & (1u << ViewIndex)) == 0;
		(*TransientTextures)[0].ViewMask |= (1u << ViewIndex);
	}

	return (*TransientTextures)[0].Texture;
}

FScreenPassTextureSlice FMinimalSceneTextures::GetUserSceneTexture(FRDGBuilder& GraphBuilder, const FViewInfo& View, int32 ViewIndex, FName Name, const UMaterialInterface* MaterialInterface) const
{
	TArray<FTransientUserSceneTexture>* TransientTextures = UserSceneTextures.Find(Name);
	if (TransientTextures)
	{
		FScreenPassTextureSlice TransientTextureSlice(
			GraphBuilder.CreateSRV(FRDGTextureSRVDesc((*TransientTextures)[0].Texture)),
			GetDownscaledViewRect(View.UnconstrainedViewRect, View.GetFamilyViewRect().Max, (*TransientTextures)[0].ResolutionDivisor));

#if !(UE_BUILD_SHIPPING)
		(*TransientTextures)[0].bUsed = true;
		UserSceneTextureEvents.Add({ EUserSceneTextureEvent::FoundInput, Name, (*TransientTextures)[0].AllocationOrder, (uint16)ViewIndex, MaterialInterface });
#endif

		return TransientTextureSlice;
	}
	else
	{
#if !(UE_BUILD_SHIPPING)
		UserSceneTextureEvents.Add({ EUserSceneTextureEvent::MissingInput, Name, 0, (uint16)ViewIndex, MaterialInterface });
#endif

		return FScreenPassTextureSlice();
	}
}

FIntPoint FMinimalSceneTextures::GetUserSceneTextureDivisor(FName Name) const
{
	TArray<FTransientUserSceneTexture>* TransientTextures = UserSceneTextures.Find(Name);
	if (TransientTextures)
	{
		return (*TransientTextures)[0].ResolutionDivisor;
	}
	else
	{
		return FIntPoint(1, 1);
	}
}

#if !(UE_BUILD_SHIPPING)
const FTransientUserSceneTexture* FMinimalSceneTextures::FindUserSceneTextureByEvent(const FUserSceneTextureEventData& Event) const
{
	const TArray<FTransientUserSceneTexture>* TransientTextures = UserSceneTextures.Find(Event.Name);
	if (!TransientTextures)
	{
		return nullptr;
	}

	for (const FTransientUserSceneTexture& TransientTexture : *TransientTextures)
	{
		if (TransientTexture.AllocationOrder == Event.AllocationOrder)
		{
			return &TransientTexture;
		}
	}

	return nullptr;
}
#endif

void FSceneTextures::InitializeViewFamily(FRDGBuilder& GraphBuilder, FViewFamilyInfo& ViewFamily, FIntPoint FamilySize)
{
	const FSceneTexturesConfig& Config = ViewFamily.SceneTexturesConfig;
	FSceneTextures& SceneTextures = *ViewFamily.SceneTextures;

	FMinimalSceneTextures::InitializeViewFamily(GraphBuilder, ViewFamily);

	if (Config.ShadingPath == EShadingPath::Deferred)
	{
		// Screen Space Ambient Occlusion
		SceneTextures.ScreenSpaceAO = CreateScreenSpaceAOTexture(GraphBuilder, ViewFamily.GetFeatureLevel(), Config.Extent);

		// Small Depth
		const FIntPoint SmallDepthExtent = GetDownscaledExtent(Config.Extent, Config.SmallDepthDownsampleFactor);
		const FRDGTextureDesc SmallDepthDesc(FRDGTextureDesc::Create2D(SmallDepthExtent, PF_DepthStencil, FClearValueBinding::None, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource));
		SceneTextures.SmallDepth = GraphBuilder.CreateTexture(SmallDepthDesc, TEXT("SmallDepthZ"));
	}

	// Velocity
	SceneTextures.Velocity = GraphBuilder.CreateTexture(FVelocityRendering::GetRenderTargetDesc(Config.ShaderPlatform, Config.Extent, Config.bRequireMultiView), TEXT("SceneVelocity"));

	if (Config.bIsUsingGBuffers)
	{
		ETextureCreateFlags FlagsToAdd = TexCreate_None;
		const FGBufferBindings& Bindings = Config.GBufferBindings[GBL_Default];

		if (Bindings.GBufferA.Index >= 0)
		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Config.Extent, Bindings.GBufferA.Format, FClearValueBinding::Transparent, Bindings.GBufferA.Flags | FlagsToAdd | GFastVRamConfig.GBufferA, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
			SceneTextures.GBufferA = GraphBuilder.CreateTexture(Desc, TEXT("GBufferA"));
		}

		if (Bindings.GBufferB.Index >= 0)
		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Config.Extent, Bindings.GBufferB.Format, FClearValueBinding::Transparent, Bindings.GBufferB.Flags | FlagsToAdd | GFastVRamConfig.GBufferB, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
			SceneTextures.GBufferB = GraphBuilder.CreateTexture(Desc, TEXT("GBufferB"));
		}

		if (Bindings.GBufferC.Index >= 0)
		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Config.Extent, Bindings.GBufferC.Format, FClearValueBinding::Transparent, Bindings.GBufferC.Flags | FlagsToAdd | GFastVRamConfig.GBufferC, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
			SceneTextures.GBufferC = GraphBuilder.CreateTexture(Desc, TEXT("GBufferC"));
		}

		if (Bindings.GBufferD.Index >= 0)
		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Config.Extent, Bindings.GBufferD.Format, FClearValueBinding::Transparent, Bindings.GBufferD.Flags | FlagsToAdd | GFastVRamConfig.GBufferD, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
			SceneTextures.GBufferD = GraphBuilder.CreateTexture(Desc, TEXT("GBufferD"));
		}

		if (Bindings.GBufferE.Index >= 0)
		{
			const FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Config.Extent, Bindings.GBufferE.Format, FClearValueBinding::Transparent, Bindings.GBufferE.Flags | FlagsToAdd | GFastVRamConfig.GBufferE, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
			SceneTextures.GBufferE = GraphBuilder.CreateTexture(Desc, TEXT("GBufferE"));
		}

		// GBufferF is not yet part of the data driven GBuffer info.
		if (Config.ShadingPath == EShadingPath::Deferred)
		{
			ETextureCreateFlags GBufferFCreateFlags;
			EPixelFormat GBufferFPixelFormat = GetGBufferFFormatAndCreateFlags(GBufferFCreateFlags);
			const FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Config.Extent, GBufferFPixelFormat, FClearValueBinding({ 0.5f, 0.5f, 0.5f, 0.5f }), GBufferFCreateFlags | FlagsToAdd, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
			SceneTextures.GBufferF = GraphBuilder.CreateTexture(Desc, TEXT("GBufferF"));
		}
	}

	if (Config.bRequiresDepthAux)
	{
		const float FarDepth = 0.f;
		const FLinearColor FarDepthColor(FarDepth, FarDepth, FarDepth, FarDepth);
		ETextureCreateFlags MemorylessFlag = TexCreate_None;
		if (IsMobileDeferredShadingEnabled(Config.ShaderPlatform) || (Config.NumSamples > 1 && Config.bMemorylessMSAA))
		{
		// hotfix for a crash on a Mac mobile preview, proper fix is in 5.2
		#if !PLATFORM_MAC
			MemorylessFlag = TexCreate_Memoryless;
		#endif
		}

		EPixelFormat DepthAuxFormat = Config.bPreciseDepthAux ? PF_R32_FLOAT : PF_R16F;
		FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Config.Extent, DepthAuxFormat, FClearValueBinding(FarDepthColor), TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_InputAttachmentRead | MemorylessFlag, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
		Desc.NumSamples = Config.NumSamples;
		SceneTextures.DepthAux = CreateTextureMSAA(GraphBuilder, Desc, TEXT("SceneDepthAuxMS"), TEXT("SceneDepthAux"));
	}

	if (ViewFamily.EngineShowFlags.StereoMotionVectors && PlatformSupportsOpenXRMotionVectors(ViewFamily.GetShaderPlatform()))
	{
		FTextureRHIRef StereoMotionVectorRHI, StereoMotionVectorDepthRHI;
		FIntPoint StereoMotionVectorSize;
		if (FindStereoMotionVectorTexture(StereoMotionVectorSize, StereoMotionVectorRHI, StereoMotionVectorDepthRHI))
		{
			SceneTextures.StereoMotionVectors = RegisterExternalTexture(GraphBuilder, StereoMotionVectorRHI, TEXT("ExternalStereoMotionVectors"));
			if (StereoMotionVectorDepthRHI)
			{
				SceneTextures.StereoMotionVectorDepth = RegisterExternalTexture(GraphBuilder, StereoMotionVectorDepthRHI, TEXT("ExternalStereoMotionVectorDepth"));
			}
		}
	}

	if (Config.ShadingPath == EShadingPath::Deferred &&
		UseSeparatedSubsurfaceDiffuse(Config.ShaderPlatform))
	{
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(Config.Extent, PF_R16F, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV);
		SceneTextures.SeparatedSubsurfaceDiffuse = GraphBuilder.CreateTexture(Desc, TEXT("SeparatedSubsurfaceDiffuse"));
	}

#if WITH_EDITOR
	{
		const FRDGTextureDesc ColorDesc(FRDGTextureDesc::Create2D(Config.Extent, PF_B8G8R8A8, FClearValueBinding::Transparent, TexCreate_ShaderResource | TexCreate_RenderTargetable, 1, Config.EditorPrimitiveNumSamples));
		SceneTextures.EditorPrimitiveColor = GraphBuilder.CreateTexture(ColorDesc, TEXT("Editor.PrimitivesColor"));

		FRDGTextureDesc DepthDesc(FRDGTextureDesc::Create2D(Config.Extent, PF_DepthStencil, FClearValueBinding::DepthFar, TexCreate_ShaderResource | TexCreate_DepthStencilTargetable | TexCreate_InputAttachmentRead, 1, Config.EditorPrimitiveNumSamples));
		DepthDesc.AliasableFormats.Add(PF_X24_G8);
		SceneTextures.EditorPrimitiveDepth = GraphBuilder.CreateTexture(DepthDesc, TEXT("Editor.PrimitivesDepth"));
	}
#endif

	SceneTextures.DBufferTextures = MakePimpl<FDBufferTextures, EPimplPtrMode::DeepCopy>();
	SceneTextures.SubstrateSceneData = MakePimpl<FSubstrateSceneData, EPimplPtrMode::DeepCopy>();

#if NANITE_USE_ANALYTIC_SGGX
	if (DoesPlatformSupportNanite(Config.ShaderPlatform))
	{
		ETextureCreateFlags FlagsToAdd = TexCreate_None;
		const FGBufferBindings& Bindings = Config.GBufferBindings[GBL_Default];

		const FRDGTextureDesc Desc = FRDGTextureDesc::CreateRenderTargetTextureDesc(Config.Extent, PF_A2B10G10R10, FClearValueBinding::Black, Bindings.GBufferA.Flags | FlagsToAdd | GFastVRamConfig.GBufferA, Config.bRequireMultiView, Config.MobileMultiViewRenderTargetNumLayers);
		SceneTextures.GBufferSGGX = GraphBuilder.CreateTexture(Desc, TEXT("GBufferSGGX"));
	}
#endif

#if WITH_DEBUG_VIEW_MODES
	{
		FIntPoint DebugAuxTextureExtent { 1, 1 };
		EPixelFormat DebugAuxTextureFormat = PF_R32_UINT;

		EDebugViewShaderMode DebugViewMode = ViewFamily.GetDebugViewShaderMode();
		if (DebugViewMode == DVSM_QuadComplexity || DebugViewMode == DVSM_ShaderComplexityContainedQuadOverhead)
		{
			DebugAuxTextureExtent.X = 2 * FMath::Max<uint32>((Config.Extent.X + 1) / 2, 1); // The size is time 2 since left side is QuadDescriptor, and right side QuadComplexity.
			DebugAuxTextureExtent.Y =     FMath::Max<uint32>((Config.Extent.Y + 1) / 2, 1);
		}
		else if (DebugViewMode == DVSM_ShadowCasters)
		{
			DebugAuxTextureExtent = Config.Extent;
		}

		const FRDGTextureDesc DebugAuxTextureDesc(FRDGTextureDesc::Create2D(DebugAuxTextureExtent, DebugAuxTextureFormat, FClearValueBinding::None, TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_UAV));
		SceneTextures.DebugAux = GraphBuilder.CreateTexture(DebugAuxTextureDesc, TEXT("DebugAuxTexture"));
	}
#endif

#if SUPPORTS_VISUALIZE_TEXTURE
	// This function can be called for a Custom Render Pass view family.  Skip those, by checking if this is the
	// main view family (the first view is the same as the first one in "AllViews").
	if (GVisualizeTexture.IsRequestedView() && ViewFamily.Views[0] == ViewFamily.AllViews[0])
	{
		TArray<FIntRect> FamilyViewRects;
		FamilyViewRects.SetNumUninitialized(ViewFamily.Views.Num());
		for (int32 ViewIndex = 0; ViewIndex < ViewFamily.Views.Num(); ViewIndex++)
		{
			FamilyViewRects[ViewIndex] = ViewFamily.Views[ViewIndex]->UnconstrainedViewRect;
		}

		GVisualizeTexture.SetSceneTextures(SceneTextures.EnumerateSceneTextures(), FamilySize, FamilyViewRects);
	}
#endif
}

uint32 FSceneTextures::GetGBufferRenderTargets(
	TArrayView<FTextureRenderTargetBinding> RenderTargets,
	EGBufferLayout Layout) const
{
	uint32 RenderTargetCount = 0;

	// All configurations use scene color in the first slot.
	RenderTargets[RenderTargetCount++] = FTextureRenderTargetBinding(Color.Target);

	if (Config.bIsUsingGBuffers)
	{
		struct FGBufferEntry
		{
			FGBufferEntry(const TCHAR* InName, FRDGTextureRef InTexture, int32 InIndex)
				: Name(InName)
				, Texture(InTexture)
				, Index(InIndex)
			{}

			const TCHAR* Name;
			FRDGTextureRef Texture;
			int32 Index;
		};

		const FGBufferBindings& Bindings = Config.GBufferBindings[Layout];
		const FGBufferEntry GBufferEntries[] =
		{
			{ TEXT("GBufferA"), GBufferA, Bindings.GBufferA.Index },
			{ TEXT("GBufferB"), GBufferB, Bindings.GBufferB.Index },
			{ TEXT("GBufferC"), GBufferC, Bindings.GBufferC.Index },
			{ TEXT("GBufferD"), GBufferD, Bindings.GBufferD.Index },
			{ TEXT("GBufferE"), GBufferE, Bindings.GBufferE.Index },
			{ TEXT("Velocity"), Velocity, Bindings.GBufferVelocity.Index }
		};

		for (const FGBufferEntry& Entry : GBufferEntries)
		{
			checkf(Entry.Index <= 0 || Entry.Texture != nullptr, TEXT("Texture '%s' was requested by FGBufferInfo, but it is null."), Entry.Name);
			if (Entry.Index > 0)
			{
				RenderTargets[Entry.Index] = FTextureRenderTargetBinding(Entry.Texture);
				RenderTargetCount = FMath::Max(RenderTargetCount, uint32(Entry.Index + 1));
			}
		}
	}
	// Forward shading path
	else if (IsUsingBasePassVelocity(Config.ShaderPlatform))
	{
		RenderTargets[RenderTargetCount++] = FTextureRenderTargetBinding(Velocity);
	}

	return RenderTargetCount;
}

uint32 FSceneTextures::GetGBufferRenderTargets(
	ERenderTargetLoadAction LoadAction,
	FRenderTargetBindingSlots& RenderTargetBindingSlots,
	EGBufferLayout Layout) const
{
	TStaticArray<FTextureRenderTargetBinding, MaxSimultaneousRenderTargets> RenderTargets;
	const uint32 RenderTargetCount = GetGBufferRenderTargets(RenderTargets, Layout);
	for (uint32 Index = 0; Index < RenderTargetCount; ++Index)
	{
		RenderTargetBindingSlots[Index] = FRenderTargetBinding(RenderTargets[Index].Texture, LoadAction);
	}
	return RenderTargetCount;
}

static void AddTextureIfNonNull(FRDGTextureRef Texture, TArray<FRDGTextureRef>& OutTextures)
{
	if (Texture)
	{
		OutTextures.Add(Texture);
	}
}

static void AddTextureIfNonNull(const FRDGTextureMSAA& Texture, TArray<FRDGTextureRef>& OutTextures)
{
	if (Texture.Target)
	{
		OutTextures.Add(Texture.Target);
	}
}

TArray<FRDGTextureRef> FSceneTextures::EnumerateSceneTextures() const
{
	TArray<FRDGTextureRef> Results;
	Results.Reserve(21);

	AddTextureIfNonNull(Color, Results);
	AddTextureIfNonNull(Depth, Results);
	AddTextureIfNonNull(PartialDepth, Results);
	AddTextureIfNonNull(CustomDepth.Depth, Results);
	AddTextureIfNonNull(SmallDepth, Results);
	AddTextureIfNonNull(GBufferA, Results);
	AddTextureIfNonNull(GBufferB, Results);
	AddTextureIfNonNull(GBufferC, Results);
	AddTextureIfNonNull(GBufferD, Results);
	AddTextureIfNonNull(GBufferE, Results);
	AddTextureIfNonNull(GBufferF, Results);
	AddTextureIfNonNull(DitheredHalfResDepth, Results);
	AddTextureIfNonNull(GBufferSGGX, Results);
	AddTextureIfNonNull(DepthAux, Results);
	AddTextureIfNonNull(Velocity, Results);
	AddTextureIfNonNull(ScreenSpaceAO, Results);
	AddTextureIfNonNull(DebugAux, Results);
#if WITH_EDITOR
	AddTextureIfNonNull(EditorPrimitiveColor, Results);
	AddTextureIfNonNull(EditorPrimitiveDepth, Results);
#endif

	return Results;
}

FRHITexture* FSceneTextureExtracts::GetDepthTexture() const
{
	return Depth ? Depth->GetRHI() : nullptr;
}

void FSceneTextureExtracts::QueueExtractions(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures)
{
	// Free up the memory for reuse during the RDG execution phase.
	Release();

	ESceneTextureSetupMode SetupMode = ESceneTextureSetupMode::None;

	const auto ExtractIfProduced = [&](FRDGTextureRef Texture, TRefCountPtr<IPooledRenderTarget>& OutTarget)
	{
		if (HasBeenProduced(Texture) && !EnumHasAnyFlags(Texture->Desc.Flags, TexCreate_Memoryless))
		{
			GraphBuilder.QueueTextureExtraction(Texture, &OutTarget, ERDGResourceExtractionFlags::AllowTransient);
		}
	};

	if (EnumHasAnyFlags(SceneTextures.Config.Extracts, ESceneTextureExtracts::Depth))
	{
		SetupMode |= ESceneTextureSetupMode::SceneDepth;
		ExtractIfProduced(SceneTextures.Depth.Resolve, Depth);
		ExtractIfProduced(SceneTextures.PartialDepth.Resolve, PartialDepth);
	}

	if (EnumHasAnyFlags(SceneTextures.Config.Extracts, ESceneTextureExtracts::CustomDepth))
	{
		SetupMode |= ESceneTextureSetupMode::CustomDepth;
		ExtractIfProduced(SceneTextures.CustomDepth.Depth, CustomDepth);
	}

	// Create and extract a scene texture uniform buffer for RHI code outside of the main render graph instance. This
	// uniform buffer will reference all extracted textures. No transitions will be required since the textures are left
	// in a shader resource state.
	auto* PassParameters = GraphBuilder.AllocParameters<FSceneTextureShaderParameters>();
	*PassParameters = CreateSceneTextureShaderParameters(GraphBuilder, &SceneTextures, SceneTextures.Config.FeatureLevel, SetupMode);

	// We want these textures in a SRV Compute | Raster state.
	const ERDGPassFlags PassFlags = ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass | ERDGPassFlags::Compute | ERDGPassFlags::NeverCull;

	GraphBuilder.AddPass(RDG_EVENT_NAME("ExtractUniformBuffer"), PassParameters, PassFlags,
		[this, PassParameters, ShadingPath = SceneTextures.Config.ShadingPath](FRHICommandList&)
	{
		if (ShadingPath == EShadingPath::Deferred)
		{
			UniformBuffer = PassParameters->SceneTextures->GetRHIRef();
		}
		else
		{
			MobileUniformBuffer = PassParameters->MobileSceneTextures->GetRHIRef();
		}
	});
}

void FSceneTextureExtracts::Release()
{
	Depth = {};
	CustomDepth = {};
	UniformBuffer = {};
	MobileUniformBuffer = {};
}

static TGlobalResource<FSceneTextureExtracts> GSceneTextureExtracts;

const FSceneTextureExtracts& GetSceneTextureExtracts()
{
	return GSceneTextureExtracts;
}

void QueueSceneTextureExtractions(FRDGBuilder& GraphBuilder, const FSceneTextures& SceneTextures)
{
	return GSceneTextureExtracts.QueueExtractions(GraphBuilder, SceneTextures);
}

FRDGTextureRef GetSceneTexture(const FSceneTextures& SceneTextures, ESceneTexture InSceneTexture)
{
	switch (InSceneTexture)
	{
	case ESceneTexture::Color:          return SceneTextures.Color.Resolve;
	case ESceneTexture::Depth:          return SceneTextures.Depth.Resolve;
	case ESceneTexture::SmallDepth:     return SceneTextures.SmallDepth;
	case ESceneTexture::Velocity:       return SceneTextures.Velocity;
	case ESceneTexture::GBufferA:       return SceneTextures.GBufferA;
	case ESceneTexture::GBufferB:       return SceneTextures.GBufferB;
	case ESceneTexture::GBufferC:       return SceneTextures.GBufferC;
	case ESceneTexture::GBufferD:       return SceneTextures.GBufferD;
	case ESceneTexture::GBufferE:       return SceneTextures.GBufferE;
	case ESceneTexture::GBufferF:       return SceneTextures.GBufferF;
	case ESceneTexture::GBufferSGGX:	return SceneTextures.GBufferSGGX;
	case ESceneTexture::SSAO:           return SceneTextures.ScreenSpaceAO;
	case ESceneTexture::CustomDepth:	return SceneTextures.CustomDepth.Depth;
	default:
		checkNoEntry();
		return nullptr;
	}
}

void SetupSceneTextureUniformParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	ERHIFeatureLevel::Type FeatureLevel,
	ESceneTextureSetupMode SetupMode,
	FSceneTextureUniformParameters& SceneTextureParameters)
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	SceneTextureParameters.PointClampSampler = TStaticSamplerState<SF_Point>::GetRHI();
	SceneTextureParameters.SceneColorTexture = SystemTextures.Black;
	SceneTextureParameters.SceneDepthTexture = SystemTextures.DepthDummy;
	SceneTextureParameters.ScenePartialDepthTexture = SystemTextures.DepthDummy;
	SceneTextureParameters.GBufferATexture = SystemTextures.Black;
	SceneTextureParameters.GBufferBTexture = SystemTextures.Black;
	SceneTextureParameters.GBufferCTexture = SystemTextures.Black;
	SceneTextureParameters.GBufferDTexture = SystemTextures.Black;
	SceneTextureParameters.GBufferETexture = SystemTextures.Black;
	SceneTextureParameters.GBufferFTexture = SystemTextures.MidGrey;
	SceneTextureParameters.GBufferVelocityTexture = SystemTextures.Black;
	SceneTextureParameters.GBufferSGGXTexture = SystemTextures.Black;
	SceneTextureParameters.ScreenSpaceAOTexture = GetScreenSpaceAOFallback(SystemTextures);
	SceneTextureParameters.CustomDepthTexture = SystemTextures.DepthDummy;
	SceneTextureParameters.CustomStencilTexture = SystemTextures.StencilDummySRV;

	if (SceneTextures)
	{
		const EShaderPlatform ShaderPlatform = SceneTextures->Config.ShaderPlatform;

		if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::SceneColor))
		{
			SceneTextureParameters.SceneColorTexture = SceneTextures->Color.Resolve;
		}

		if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::SceneDepth))
		{
			SceneTextureParameters.SceneDepthTexture = SceneTextures->Depth.Resolve;
			SceneTextureParameters.ScenePartialDepthTexture = SceneTextures->PartialDepth.Resolve;
		}

		if (IsUsingGBuffers(ShaderPlatform))
		{
			if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::GBufferA) && HasBeenProduced(SceneTextures->GBufferA))
			{
				SceneTextureParameters.GBufferATexture = SceneTextures->GBufferA;
			}

			if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::GBufferB) && HasBeenProduced(SceneTextures->GBufferB))
			{
				SceneTextureParameters.GBufferBTexture = SceneTextures->GBufferB;
			}

			if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::GBufferC) && HasBeenProduced(SceneTextures->GBufferC))
			{
				SceneTextureParameters.GBufferCTexture = SceneTextures->GBufferC;
			}

			if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::GBufferD) && HasBeenProduced(SceneTextures->GBufferD))
			{
				SceneTextureParameters.GBufferDTexture = SceneTextures->GBufferD;
			}

			if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::GBufferE) && HasBeenProduced(SceneTextures->GBufferE))
			{
				SceneTextureParameters.GBufferETexture = SceneTextures->GBufferE;
			}

			if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::GBufferF) && HasBeenProduced(SceneTextures->GBufferF))
			{
				SceneTextureParameters.GBufferFTexture = SceneTextures->GBufferF;
			}
		}

		if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::SceneVelocity) && HasBeenProduced(SceneTextures->Velocity))
		{
			SceneTextureParameters.GBufferVelocityTexture = SceneTextures->Velocity;
		}

		if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::SSAO) && HasBeenProduced(SceneTextures->ScreenSpaceAO))
		{
			SceneTextureParameters.ScreenSpaceAOTexture = SceneTextures->ScreenSpaceAO;
		}

		if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::SGGX) && HasBeenProduced(SceneTextures->GBufferSGGX))
		{
			SceneTextureParameters.GBufferSGGXTexture = SceneTextures->GBufferSGGX;
		}

		if (EnumHasAnyFlags(SetupMode, ESceneTextureSetupMode::CustomDepth))
		{
			const FCustomDepthTextures& CustomDepthTextures = SceneTextures->CustomDepth;

			if (HasBeenProduced(CustomDepthTextures.Depth))
			{
				SceneTextureParameters.CustomDepthTexture = CustomDepthTextures.Depth;
				SceneTextureParameters.CustomStencilTexture = CustomDepthTextures.Stencil;
			}
		}
	}
}

TRDGUniformBufferRef<FSceneTextureUniformParameters> CreateSceneTextureUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	ERHIFeatureLevel::Type FeatureLevel,
	ESceneTextureSetupMode SetupMode)
{
	FSceneTextureUniformParameters* SceneTexturesParameters = GraphBuilder.AllocParameters<FSceneTextureUniformParameters>();
	SetupSceneTextureUniformParameters(GraphBuilder, SceneTextures, FeatureLevel, SetupMode, *SceneTexturesParameters);
	return GraphBuilder.CreateUniformBuffer(SceneTexturesParameters);
}

TRDGUniformBufferRef<FSceneTextureUniformParameters> CreateSceneTextureUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FSceneView& View,
	ESceneTextureSetupMode SetupMode)
{
	if (View.bIsViewInfo)
	{
		return CreateSceneTextureUniformBuffer(GraphBuilder, ((const FViewInfo&)View).GetSceneTexturesChecked(), View.GetFeatureLevel(), SetupMode);
	}

	return nullptr;
}

EMobileSceneTextureSetupMode Translate(ESceneTextureSetupMode InSetupMode)
{
	EMobileSceneTextureSetupMode OutSetupMode = EMobileSceneTextureSetupMode::None;
	if (EnumHasAnyFlags(InSetupMode, ESceneTextureSetupMode::GBuffers))
	{
		OutSetupMode |= EMobileSceneTextureSetupMode::SceneColor;
	}
	if (EnumHasAnyFlags(InSetupMode, ESceneTextureSetupMode::CustomDepth))
	{
		OutSetupMode |= EMobileSceneTextureSetupMode::CustomDepth;
	}
	return OutSetupMode;
}

void SetupMobileSceneTextureUniformParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	EMobileSceneTextureSetupMode SetupMode,
	FMobileSceneTextureUniformParameters& SceneTextureParameters)
{
	const FRDGSystemTextures& SystemTextures = FRDGSystemTextures::Get(GraphBuilder);

	SceneTextureParameters.SceneColorTexture = SystemTextures.Black;
	SceneTextureParameters.SceneColorTextureSampler = TStaticSamplerState<>::GetRHI();
	SceneTextureParameters.SceneDepthTexture = SystemTextures.DepthDummy;
	SceneTextureParameters.SceneDepthTextureArray = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, EPixelFormat::PF_B8G8R8A8, FClearValueBinding::Black);
	SceneTextureParameters.SceneDepthTextureSampler = TStaticSamplerState<>::GetRHI();
	SceneTextureParameters.ScenePartialDepthTexture = SystemTextures.DepthDummy;
	SceneTextureParameters.ScenePartialDepthTextureSampler = TStaticSamplerState<>::GetRHI();
	// CustomDepthTexture is a color texture on mobile, with DeviceZ values
	SceneTextureParameters.CustomDepthTexture = SystemTextures.Black;
	SceneTextureParameters.CustomDepthTextureArray = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, PF_DepthStencil, FClearValueBinding::Black);
	SceneTextureParameters.CustomDepthTextureSampler = TStaticSamplerState<>::GetRHI();
	SceneTextureParameters.CustomStencilTexture = SystemTextures.StencilDummySRV;
	SceneTextureParameters.CustomStencilTextureArray = SystemTextures.StencilDummySRV;
	SceneTextureParameters.SceneVelocityTexture = SystemTextures.Black;
	SceneTextureParameters.SceneVelocityTextureSampler = TStaticSamplerState<>::GetRHI();
	SceneTextureParameters.GBufferATexture = SystemTextures.Black;
	SceneTextureParameters.GBufferBTexture = SystemTextures.Black;
	SceneTextureParameters.GBufferCTexture = SystemTextures.Black;
	SceneTextureParameters.GBufferDTexture = SystemTextures.Black;
	// SceneDepthAuxTexture is a color texture on mobile, with DeviceZ values
	SceneTextureParameters.SceneDepthAuxTexture = SystemTextures.Black;
	SceneTextureParameters.SceneDepthAuxTextureArray = GSystemTextures.GetDefaultTexture(GraphBuilder, ETextureDimension::Texture2DArray, EPixelFormat::PF_B8G8R8A8, FClearValueBinding::Black);
	SceneTextureParameters.LocalLightTextureA = SystemTextures.Black;
	SceneTextureParameters.LocalLightTextureB = SystemTextures.Black;
	SceneTextureParameters.GBufferATextureSampler = TStaticSamplerState<>::GetRHI();
	SceneTextureParameters.GBufferBTextureSampler = TStaticSamplerState<>::GetRHI();
	SceneTextureParameters.GBufferCTextureSampler = TStaticSamplerState<>::GetRHI();
	SceneTextureParameters.GBufferDTextureSampler = TStaticSamplerState<>::GetRHI();
	SceneTextureParameters.SceneDepthAuxTextureSampler = TStaticSamplerState<>::GetRHI();

	if (SceneTextures)
	{
		if (EnumHasAnyFlags(SetupMode, EMobileSceneTextureSetupMode::SceneColor) && HasBeenProduced(SceneTextures->Color.Resolve))
		{
			SceneTextureParameters.SceneColorTexture = SceneTextures->Color.Resolve;
		}

		if (EnumHasAnyFlags(SetupMode, EMobileSceneTextureSetupMode::SceneDepth) && 
			HasBeenProduced(SceneTextures->Depth.Resolve) && 
			!EnumHasAnyFlags(SceneTextures->Depth.Resolve->Desc.Flags, TexCreate_Memoryless))
		{
			SceneTextureParameters.SceneDepthTexture = SceneTextures->Depth.Resolve;
			SceneTextureParameters.SceneDepthTextureArray = SceneTextures->Depth.Resolve;
		}

		if (EnumHasAnyFlags(SetupMode, EMobileSceneTextureSetupMode::SceneDepth) &&
			HasBeenProduced(SceneTextures->PartialDepth.Resolve) &&
			!EnumHasAnyFlags(SceneTextures->PartialDepth.Resolve->Desc.Flags, TexCreate_Memoryless))
		{
			SceneTextureParameters.ScenePartialDepthTexture = SceneTextures->PartialDepth.Resolve;
		}

		if (SceneTextures->Config.bIsUsingGBuffers)
		{
			if (HasBeenProduced(SceneTextures->GBufferA) &&
				!EnumHasAnyFlags(SceneTextures->GBufferA->Desc.Flags, TexCreate_Memoryless))
			{
				SceneTextureParameters.GBufferATexture = SceneTextures->GBufferA;
			}

			if (HasBeenProduced(SceneTextures->GBufferB) &&
				!EnumHasAnyFlags(SceneTextures->GBufferB->Desc.Flags, TexCreate_Memoryless))
			{
				SceneTextureParameters.GBufferBTexture = SceneTextures->GBufferB;
			}

			if (HasBeenProduced(SceneTextures->GBufferC) &&
				!EnumHasAnyFlags(SceneTextures->GBufferC->Desc.Flags, TexCreate_Memoryless))
			{
				SceneTextureParameters.GBufferCTexture = SceneTextures->GBufferC;
			}

			if (HasBeenProduced(SceneTextures->GBufferD) &&
				!EnumHasAnyFlags(SceneTextures->GBufferD->Desc.Flags, TexCreate_Memoryless))
			{
				SceneTextureParameters.GBufferDTexture = SceneTextures->GBufferD;
			}
		}

		if (EnumHasAnyFlags(SetupMode, EMobileSceneTextureSetupMode::SceneDepthAux))
		{
			if (HasBeenProduced(SceneTextures->DepthAux.Resolve) &&
				!EnumHasAnyFlags(SceneTextures->DepthAux.Resolve->Desc.Flags, TexCreate_Memoryless))
			{
				SceneTextureParameters.SceneDepthAuxTexture = SceneTextures->DepthAux.Resolve;
				SceneTextureParameters.SceneDepthAuxTextureArray = SceneTextures->DepthAux.Resolve;
			}
		}

		if (EnumHasAnyFlags(SetupMode, EMobileSceneTextureSetupMode::CustomDepth))
		{
			const FCustomDepthTextures& CustomDepthTextures = SceneTextures->CustomDepth;

			bool bCustomDepthProduced = HasBeenProduced(CustomDepthTextures.Depth);
			SceneTextureParameters.CustomDepthTexture = bCustomDepthProduced ? CustomDepthTextures.Depth : SystemTextures.DepthDummy;
			SceneTextureParameters.CustomDepthTextureArray = bCustomDepthProduced ? CustomDepthTextures.Depth : SystemTextures.DepthDummy;
			SceneTextureParameters.CustomStencilTexture = bCustomDepthProduced ? CustomDepthTextures.Stencil : SystemTextures.StencilDummySRV;
			SceneTextureParameters.CustomStencilTextureArray = bCustomDepthProduced ? CustomDepthTextures.Stencil : SystemTextures.StencilDummySRV;
		}

		if (EnumHasAnyFlags(SetupMode, EMobileSceneTextureSetupMode::SceneVelocity))
		{
			if (HasBeenProduced(SceneTextures->Velocity))
			{
				SceneTextureParameters.SceneVelocityTexture = SceneTextures->Velocity;
			}
		}
	}
}

TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> CreateMobileSceneTextureUniformBuffer(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	EMobileSceneTextureSetupMode SetupMode)
{
	FMobileSceneTextureUniformParameters* SceneTexturesParameters = GraphBuilder.AllocParameters<FMobileSceneTextureUniformParameters>();
	SetupMobileSceneTextureUniformParameters(GraphBuilder, SceneTextures, SetupMode, *SceneTexturesParameters);
	return GraphBuilder.CreateUniformBuffer(SceneTexturesParameters);
}

TRDGUniformBufferRef<FMobileSceneTextureUniformParameters> CreateMobileSceneTextureUniformBuffer(FRDGBuilder& GraphBuilder, const FSceneView& View, EMobileSceneTextureSetupMode SetupMode)
{
	if (View.bIsViewInfo)
	{
		return CreateMobileSceneTextureUniformBuffer(GraphBuilder, ((const FViewInfo&)View).GetSceneTexturesChecked(), SetupMode);
	}

	return nullptr;
}

FSceneTextureShaderParameters CreateSceneTextureShaderParameters(
	FRDGBuilder& GraphBuilder,
	const FSceneTextures* SceneTextures,
	ERHIFeatureLevel::Type FeatureLevel,
	ESceneTextureSetupMode SetupMode)
{
	FSceneTextureShaderParameters Parameters;
	if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Deferred)
	{
		Parameters.SceneTextures = CreateSceneTextureUniformBuffer(GraphBuilder, SceneTextures, FeatureLevel, SetupMode);
	}
	else if (GetFeatureLevelShadingPath(FeatureLevel) == EShadingPath::Mobile)
	{
		Parameters.MobileSceneTextures = CreateMobileSceneTextureUniformBuffer(GraphBuilder, SceneTextures, Translate(SetupMode));
	}
	return Parameters;
}

FSceneTextureShaderParameters CreateSceneTextureShaderParameters(FRDGBuilder& GraphBuilder, const FSceneView& View, ESceneTextureSetupMode SetupMode)
{
	FSceneTextureShaderParameters Parameters;
	if (GetFeatureLevelShadingPath(View.FeatureLevel) == EShadingPath::Deferred)
	{
		Parameters.SceneTextures = CreateSceneTextureUniformBuffer(GraphBuilder, View, SetupMode);
	}
	else if (GetFeatureLevelShadingPath(View.FeatureLevel) == EShadingPath::Mobile)
	{
		Parameters.MobileSceneTextures = CreateMobileSceneTextureUniformBuffer(GraphBuilder, View, Translate(SetupMode));
	}
	return Parameters;
}

FSceneTextureShaderParameters GetSceneTextureShaderParameters(const FSceneView& View)
{
	check(View.bIsViewInfo);
	const FMinimalSceneTextures& SceneTextures = static_cast<const FViewInfo&>(View).GetSceneTextures();
	return SceneTextures.GetSceneTextureShaderParameters(View.GetFeatureLevel());
}

TRDGUniformBufferRef<FSceneTextureUniformParameters> GetSceneTextureUnformBuffer(const FSceneView& View)
{
	if (const FSceneTextures* SceneTextures = static_cast<const FViewFamilyInfo*>(View.Family)->GetSceneTexturesChecked())
	{
		return SceneTextures->UniformBuffer;
	}

	return TRDGUniformBufferRef<FSceneTextureUniformParameters>{};
}

bool IsSceneTexturesValid()
{
	return FSceneTexturesConfig::Get().IsValid();
}

FIntPoint GetSceneTextureExtent()
{
	return FSceneTexturesConfig::Get().Extent;
}

FIntPoint GetSceneTextureExtentFromView(const FViewInfo& View)
{
	return View.GetSceneTexturesConfig().Extent;
}

ERHIFeatureLevel::Type GetSceneTextureFeatureLevel()
{
	return FSceneTexturesConfig::Get().FeatureLevel;
}

void CreateSystemTextures(FRDGBuilder& GraphBuilder)
{
	FRDGSystemTextures::Create(GraphBuilder);
}
