// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Experimental/Containers/RobinHoodHashTable.h"
#include "RendererInterface.h"

enum class ESceneUpdateCommandFilter : uint32
{
	Added			= 1u << 0,
	Deleted			= 1u << 1,
	Updated			= 1u << 2,
	AddedUpdated	= Added | Updated,
	All				= Added | Deleted | Updated
};
ENUM_CLASS_FLAGS(ESceneUpdateCommandFilter);

/**
 * An unordered queue for sending scene object updates (agnostic to the object type InSceneInfoType/FSceneInfo). 
 * Several update payloads can be enqueued for each object but only the last of each type will have effect.
 * The update payloads are stored in a typed compact array, but are not themselves required to have virtual destructors or even be of any particular type.
 * TPayloadBase is a helper & example that can be used as a base class to implement payload data types.
 * Update payload types are identified by an ID that comes from the template argument enum EInId.
 * While iterating the FUpdateCommand it is possible to access each type of update payload safely, or one can interate the payload types in a continuous fashion.
 */
template <typename InSceneInfoType, typename EInDirtyFlagsType, typename EInId>
class TSceneUpdateCommandQueue
{
private:
	using FAllocator = FDefaultAllocator;

	struct FBasePayloadArray;
	template <typename PayloadType>
	struct TPayloadArray;

public:
	using EDirtyFlags = EInDirtyFlagsType;
	using FSceneInfo = InSceneInfoType;
	using FSceneInfoPersistentId = typename InSceneInfoType::FPersistentId;

	using EId = EInId;

	static constexpr int32 MaxId = int32(EId::MAX);
	static_assert(MaxId <= 64, "The max update ID must fit in the 64 bits we use to store the mask.");

	/**
	 * Each command represents all the updates for a given scene object. Add/Delete/AttributeUpdate. 
	 * Associated with a command are zero or more payloads which are arbitrarily typed data packets.
	 */
	struct FUpdateCommand
	{
		FUpdateCommand(FSceneInfo *InSceneInfo, FSceneInfoPersistentId InPersistentId) : SceneInfo(InSceneInfo), PersistentId(InPersistentId) { }

		template <typename PayloadType>
		int32 GetPayloadOffset() const
		{
			if (PayloadType::IdBit & PayloadMask)
			{
				int32 Index = FPlatformMath::CountBits64(PayloadMask& PayloadType::ExclusiveIdMask);

				return PayloadDataSlots[Index];
			}
			return INDEX_NONE;
		}

		template <typename PayloadType>
		void SetOrAddPayloadOffset(int32 PayloadOffset, EDirtyFlags InDirtyFlags)
		{
			DirtyFlags |= InDirtyFlags;

			int32 Index = FPlatformMath::CountBits64(PayloadMask& PayloadType::ExclusiveIdMask);
			// previously set, replace pointer
			if (PayloadType::IdBit& PayloadMask)
			{
				PayloadDataSlots[Index] = PayloadOffset;
			}
			else
			{
				PayloadMask |= PayloadType::IdBit;
				PayloadDataSlots.Insert(PayloadOffset, Index);
			}
		}

		FSceneInfo* GetSceneInfo() const { return SceneInfo; }
		FSceneInfoPersistentId GetPersistentId() const { return PersistentId; }
		bool IsDelete() const { return bDeleted; }
		bool IsAdd() const { return bAdded; }

		// Should only be called for added objects after the ID has been allocated.
		void SetPersistentId(FSceneInfoPersistentId InId)
		{
			check(bAdded);
			PersistentId = InId;
		}

	private:
		FSceneInfo* SceneInfo = nullptr;
		uint64 PayloadMask = 0ull;
		FSceneInfoPersistentId PersistentId;
		EDirtyFlags DirtyFlags = EDirtyFlags::None;
		bool bDeleted = false;
		bool bAdded = false;
		// offsets to the stored data (in bit-order)
		TArray<int32, TInlineAllocator<8, FAllocator>> PayloadDataSlots;
		friend TSceneUpdateCommandQueue;
	};

	TSceneUpdateCommandQueue() 
		: PayloadArrays(InPlace)
	{
	}

	// Prevent copy constructor to avoid issues with the owned payload arrays
	TSceneUpdateCommandQueue(const TSceneUpdateCommandQueue&) = delete;
	TSceneUpdateCommandQueue(TSceneUpdateCommandQueue&& Other)
		: PayloadArrays(MoveTemp(Other.PayloadArrays))
		, Commands(MoveTemp(Other.Commands))
	{
#if DO_CHECK
		check(Other.RaceGuard == 0);
#endif
		Other.CommandSlots.Empty();
	}

	~TSceneUpdateCommandQueue()
	{
#if DO_CHECK
		check(RaceGuard == 0);
#endif
	}

	bool IsEmpty() const { return Commands.IsEmpty(); }

	int32 NumCommands() const { return Commands.Num(); }

	bool HasCommand(FSceneInfo* SceneInfo) const
	{
		return CommandSlots.Find(SceneInfo) != nullptr;
	}

	const FUpdateCommand* FindCommand(FSceneInfo* SceneInfo) const
	{
		Experimental::FHashElementId Id = CommandSlots.FindId(SceneInfo);
		if (Id.IsValid())
		{
			return &Commands[Id.GetIndex()];
		}
		return nullptr;
	}

	/**
	 * Enqueue a Delete command. This will mark the command for the scene object as deleted, but does not remove the command or associated updates.
	 * Thus a command may have both add/delete flags and update payloads. It is up to the consumer to handle these appropriately.
	 */
	void EnqueueDelete(FSceneInfo* SceneInfo)
	{
#if DO_CHECK
		check(RaceGuard == 0);
#endif

		int32 CommandSlot = GetOrAddCommandSlot(SceneInfo);
		FUpdateCommand& Command = Commands[CommandSlot];

		Command.bDeleted = true;
		// We leave any update data in place, as the alternative is doing a remove-swap and then fixing up all the indexes, which seems not-worth it, instead these updates should be skipped.
	}

	/**
	 * Enqueue an Add command. This will mark the command for the scene object as added, and must always be the first command for the object.
	 */
	void EnqueueAdd(FSceneInfo* SceneInfo)
	{
#if DO_CHECK
		check(RaceGuard == 0);
#endif

		// Add should always be the first command.
		check(CommandSlots.Find(SceneInfo) == nullptr);
		// It should not have been added to the scene already
		int32 CommandSlot = GetOrAddCommandSlot(SceneInfo);
		FUpdateCommand& Command = Commands[CommandSlot];
		// It should not even be possible for this to happen
		check(!Command.bAdded);
		check(!Command.bDeleted);
		Command.bAdded = true;
	}

	/**
	 * Enqueue an update with a data payload.
	 */
	template <typename PayloadType>
	void Enqueue(FSceneInfo* SceneInfo, PayloadType&& Payload)
	{
#if DO_CHECK
		check(RaceGuard == 0);
#endif

		int32 CommandSlot = GetOrAddCommandSlot(SceneInfo);
		FUpdateCommand& Command = Commands[CommandSlot];

		if (PayloadArrays[int32(PayloadType::Id)] == nullptr)
		{
			PayloadArrays[int32(PayloadType::Id)] = MakeUnique<TPayloadArray<PayloadType>>();
		}
		TPayloadArray<PayloadType>* Payloads = GetPayloadArray<PayloadType>();
		check(Payloads != nullptr);
		int32 PrevPayloadOffset = Command.template GetPayloadOffset<PayloadType>();

		// Update existing payload (maybe we want to disallow this?)
		if (PrevPayloadOffset != INDEX_NONE)
		{
			check(Payloads->CommandSlots[PrevPayloadOffset] == CommandSlot);
			Payloads->PayloadData[PrevPayloadOffset] = MoveTemp(Payload);
		}
		else // New payload for this command
		{
			int32 PayloadOffset = Payloads->CommandSlots.Num();
			Commands[CommandSlot].template SetOrAddPayloadOffset<PayloadType>(PayloadOffset, Payload.GetDirtyFlags());
			Payloads->CommandSlots.Emplace(CommandSlot);
			Payloads->PayloadData.Emplace(MoveTemp(Payload));
		}
	}

	/**
	 * Retriev a pointer to the PayloadType data for the given command. Returs nullptr if no such data exists.
	 */
	template <typename PayloadType>
	PayloadType* GetPayloadPtr(const FUpdateCommand& Command)
	{
		if (PayloadArrays[int32(PayloadType::Id)] == nullptr)
		{
			return nullptr;
		}

		TPayloadArray<PayloadType>* Payloads = GetPayloadArray<PayloadType>();
		check(Payloads != nullptr);
		int32 PayloadOffset = Command.template GetPayloadOffset<PayloadType>();

		// Update existing payload (maybe we want to disallow this?)
		if (PayloadOffset != INDEX_NONE)
		{
			// cross check
			check(Payloads->CommandSlots[PayloadOffset] != INDEX_NONE);
			check(&Commands[Payloads->CommandSlots[PayloadOffset]] == &Command);
			return &Payloads->PayloadData[PayloadOffset];
		}
		return nullptr;
	}

	/**
	 * Reset the command and payload data stored in the buffer, leaving allocations unchanged.
	 */
	void Reset()
	{
#if DO_CHECK
		check(RaceGuard == 0);
#endif

		CommandSlots.Empty();
		Commands.Reset();
		for (const auto& PayloadArray : PayloadArrays)
		{
			if (PayloadArray != nullptr)
			{
				PayloadArray->Reset();
			}
		}
	}

	/**
	 */
	template <typename CallbackFuncType>
	void ForEachCommand(ESceneUpdateCommandFilter CommandFilter, CallbackFuncType CallbackFunc)
	{
		for (FUpdateCommand& Command : Commands)
		{
			if (IsFilterIncludingCommand(Command, CommandFilter))
			{
				CallbackFunc(Command);
			}
		}
	}

	template <typename CallbackFuncType>
	void ForEachCommand(CallbackFuncType CallbackFunc)
	{
		return ForEachCommand(ESceneUpdateCommandFilter::All, CallbackFunc);
	}

	/**
	 * Filter on ESceneUpdateCommandFilter and _updates_ on payload mask. I.e., the payload mask only matters if the Command is an update.
	 *   E.g., ForEachUpdateCommand(ESceneUpdateCommandFilter::Added,...) will return _all_ added commands regardless of payload mask.
	 *   E.g., ForEachUpdateCommand(ESceneUpdateCommandFilter::Added | ESceneUpdateCommandFilter::Updated,...) will return _all_ added commands regardless of UpdatePayloadMask AND all updates that match the UpdatePayloadMask.
	 */
	template <typename CallbackFuncType>
	void ForEachUpdateCommand(ESceneUpdateCommandFilter CommandFilter, uint64 UpdatePayloadMask, CallbackFuncType CallbackFunc) const
	{
		for (const FUpdateCommand& Command : Commands)
		{
			if (!IsFilterIncludingCommand(Command, CommandFilter))
			{
				continue;
			}

			// Include a command if it is added/deleted OR matches the mask (the former can be excluded in the CommandFilter)
			if (Command.bAdded || Command.bDeleted || (UpdatePayloadMask& Command.PayloadMask) != 0)
			{
				CallbackFunc(Command);
			}
		}
	}

	/**
	 * Filter on ESceneUpdateCommandFilter and _updates_ on DirtyFlags mask. I.e., the DirtyFlags mask only matters if the Command is an update.
	 *   E.g., ForEachUpdateCommand(ESceneUpdateCommandFilter::Added,...) will return _all_ added commands regardless of payload mask.
	 *   E.g., ForEachUpdateCommand(ESceneUpdateCommandFilter::Added | ESceneUpdateCommandFilter::Updated,...) will return _all_ added commands regardless of DirtyFlags AND all updates that match the DirtyFlags.
	 */
	template <typename CallbackFuncType>
	void ForEachUpdateCommand(ESceneUpdateCommandFilter CommandFilter, EDirtyFlags DirtyFlags, CallbackFuncType CallbackFunc) const
	{
		for (const FUpdateCommand& Command : Commands)
		{
			if (!IsFilterIncludingCommand(Command, CommandFilter))
			{
				continue;
			}

			if (Command.bAdded || Command.bDeleted || EnumHasAnyFlags(Command.DirtyFlags, DirtyFlags))
			{
				CallbackFunc(Command);
			}
		}
	}

	/**
	 * Iterator to loop over a particular type of payload.
	 * Used to implement the (typically) more convenient TPayloadRangeView, see GetRangeView() below.
	 * Deleted items are automatically skipped.
	 */
	template <typename PayloadType>
	struct TConstPayloadIterator
	{
		struct FItem
		{
			const PayloadType& Payload;
			FSceneInfo* SceneInfo;
		};

		TConstPayloadIterator(const TPayloadArray<PayloadType>* InPayloads, const TArray<FUpdateCommand, FAllocator>& InCommands, int32 InIndex = 0) 
			: Payloads(InPayloads)
			, Commands(InCommands)
			, Index(InIndex)
		{
			SkipDeleted();
		}

		void SkipDeleted()
		{
			while (Payloads && Index < Payloads->CommandSlots.Num() && Commands[Payloads->CommandSlots[Index]].bDeleted)
			{
				++Index;
			}
		}

		void operator++() 
		{ 
			++Index;
			SkipDeleted();
		}

		FItem operator*() const 
		{ 
			check(Payloads && Index < Payloads->CommandSlots.Num());
			const FUpdateCommand& Command = Commands[Payloads->CommandSlots[Index]];
			check(!Command.bDeleted);
			return { Payloads->PayloadData[Index], Command.SceneInfo };
		}

		bool operator != (const TConstPayloadIterator& It ) const 
		{ 
			check(Payloads == It.Payloads);
			return Index != It.Index; 
		}

		explicit operator bool() const { return Payloads!= nullptr && Index < Payloads->CommandSlots.Num(); }

	private:
		const TPayloadArray<PayloadType>* Payloads;
		const TArray<FUpdateCommand, FAllocator>& Commands;
		int32 Index = 0;
	};

	/**
	 * Get an iterator to iterate updates with a given payload type.
	 */
	template <typename PayloadType>
	TConstPayloadIterator<PayloadType> GetIterator() const
	{
		const TPayloadArray<PayloadType>* Payloads = GetPayloadArray<PayloadType>();
		return TConstPayloadIterator<PayloadType>(Payloads, Commands, 0);
	}

	/**
	 * Get the number of updates with a given payload type.
	 */
	template <typename PayloadType>
	int32 GetNumItems() const
	{
		const TPayloadArray<PayloadType>* Payloads = GetPayloadArray<PayloadType>();
		return Payloads != nullptr ? Payloads->PayloadData.Num() : 0;
	}

	template <typename PayloadType>
	struct TPayloadRangeView
	{
		TPayloadRangeView(TSceneUpdateCommandQueue& InUpdateBuffer, int32 InNumItems) 
			: UpdateBuffer(InUpdateBuffer)
			, NumItems(InNumItems)
		{
#if DO_CHECK
			UpdateBuffer.BeginReadAccess();
#endif
		}
		~TPayloadRangeView()
		{
#if DO_CHECK
			UpdateBuffer.EndReadAccess();
#endif
		}

		TConstPayloadIterator<PayloadType> begin() const { return UpdateBuffer.GetIterator<PayloadType>(); }
		TConstPayloadIterator<PayloadType> end() const { return UpdateBuffer.GetEndIterator<PayloadType>(); }

		int32 Num() const { return NumItems; }

		bool IsEmpty() const { return NumItems == 0; }

	private:
		TSceneUpdateCommandQueue& UpdateBuffer;
		int32 NumItems = 0;
	};

	/**
	 * Get a "range" that can be used in a range for loop to access updates of a single payload type.
	 * e.g. for (auto& Item : Buffer.GetRangeView<MyUpdatePayloadType>());
	 * Deleted items are automatically skipped.
	 */
	template <typename PayloadType>
	TPayloadRangeView<PayloadType> GetRangeView()
	{
		return TPayloadRangeView<PayloadType>(*this, GetNumItems<PayloadType>());
	}

	/**
	 * Helper payload base templace, that defines the expected id flags & masks.
	 * Not required, as any struct that has the same interface can be used.
	 */
	template <EId InId, EDirtyFlags InDirtyFlags>
	struct TPayloadBase
	{
		// unique ID for the type of update, can be made to support registration like the SceneExtensions to rather than be compile time.
		static constexpr EId Id = InId;
		static constexpr uint64 IdBit = 1ull << uint32(Id);
		static constexpr uint64 ExclusiveIdMask = (1ull << uint32(Id)) - 1ull;
		static constexpr EDirtyFlags DirtyFlags = InDirtyFlags;
	
		// Allow static polymorphism to implement per-payload runtime variable flags.
		EDirtyFlags GetDirtyFlags() const { return DirtyFlags; }
	};
	
#if DO_CHECK
	void BeginReadAccess()
	{
		++RaceGuard;
	}

	void EndReadAccess()
	{
		--RaceGuard;
	}

	struct FReadAccessScope
	{
		FReadAccessScope(TSceneUpdateCommandQueue& InUpdateQueue) : 
			UpdateQueue(InUpdateQueue)
		{
			UpdateQueue.BeginReadAccess();
		}
		~FReadAccessScope()
		{
			UpdateQueue.EndReadAccess();
		}
	protected:
		TSceneUpdateCommandQueue& UpdateQueue;
	};
#endif

private:
	/**
	 * Get an iterator that points to the end of a payload array with a given payload type.
	 */
	template <typename PayloadType>
	TConstPayloadIterator<PayloadType> GetEndIterator() const
	{
		const TPayloadArray<PayloadType>* Payloads = GetPayloadArray<PayloadType>();
		return TConstPayloadIterator<PayloadType>(Payloads, Commands, Payloads != nullptr ? Payloads->PayloadData.Num() : 0);
	}

	struct FBasePayloadArray
	{
		FBasePayloadArray(int32 InPayloadByteSize) : PayloadByteSize(InPayloadByteSize) {}
		virtual ~FBasePayloadArray() = default;
		virtual void Reset() = 0;
		
		int32 PayloadByteSize = 0;
	};

	template <typename PayloadType>
	struct TPayloadArray : public FBasePayloadArray
	{
		TArray<PayloadType, FAllocator> PayloadData;
		TArray<int32, FAllocator> CommandSlots;
		
		TPayloadArray() : FBasePayloadArray(sizeof(PayloadType)) {}

		virtual void Reset() override
		{
			PayloadData.Reset();
			CommandSlots.Reset();
		}
	};

	int32 GetOrAddCommandSlot(FSceneInfo* SceneInfo)
	{
		bool bWasAlreadyInSet = false;
		Experimental::FHashElementId Id = CommandSlots.FindOrAddId(SceneInfo, bWasAlreadyInSet);
		int32 CommandSlot = Id.GetIndex();
		if (!bWasAlreadyInSet)
		{
			// Since we only add to this thing the slots should always go at the end.
			// Note: since we always grow this table it could easily have a much simpler hash table.
			check(CommandSlot == Commands.Num());
			Commands.Emplace(SceneInfo, SceneInfo->GetPersistentIndex());
		}
		check(Commands.IsValidIndex(CommandSlot));
		return CommandSlot;
	}

	bool IsFilterIncludingCommand(const FUpdateCommand& Command, ESceneUpdateCommandFilter CommandFilter) const
	{
		if (Command.bDeleted)
		{
			return EnumHasAnyFlags(CommandFilter, ESceneUpdateCommandFilter::Deleted);
		}

		if (Command.bAdded)
		{
			return EnumHasAnyFlags(CommandFilter, ESceneUpdateCommandFilter::Added);
		}
		return EnumHasAnyFlags(CommandFilter, ESceneUpdateCommandFilter::Updated);
	}

	template <typename PayloadType>
	TPayloadArray<PayloadType>* GetPayloadArray()
	{
		check(PayloadArrays[int32(PayloadType::Id)] == nullptr || PayloadArrays[int32(PayloadType::Id)]->PayloadByteSize == sizeof(PayloadType));
		return static_cast<TPayloadArray<PayloadType>*>(PayloadArrays[int32(PayloadType::Id)].Get());
	}
	template <typename PayloadType>
	const TPayloadArray<PayloadType>* GetPayloadArray() const
	{
		return const_cast<TSceneUpdateCommandQueue*>(this)->GetPayloadArray<PayloadType>();
	}

	using FPayloadArrays = TStaticArray<TUniquePtr<FBasePayloadArray>, 64>;
	FPayloadArrays PayloadArrays = FPayloadArrays(InPlace);
	TArray<FUpdateCommand, FAllocator> Commands;
	Experimental::TRobinHoodHashSet<FSceneInfo*> CommandSlots;

#if DO_CHECK
	std::atomic<int> RaceGuard = 0;
#endif
};
