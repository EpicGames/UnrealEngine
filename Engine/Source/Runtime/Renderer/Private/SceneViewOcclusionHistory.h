// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PrimitiveComponentId.h"
#include "RHIResources.h"
#include "RHIFeatureLevel.h"
#include "Math/MathFwd.h"
#include "Containers/Set.h"
#include "Containers/SetUtilities.h"
#include "GlobalRenderResources.h"
#include "RHIFwd.h"
#include "RenderGraphFwd.h"
#include "RenderResource.h"

/** Factor by which to grow occlusion tests **/
#define OCCLUSION_SLOP (1.0f)

class FViewInfo;
class FRHIGPUBufferReadback;

class FOcclusionQueryHelpers
{
public:

	enum
	{
		MaxBufferedOcclusionFrames = 4
	};

	// get the system-wide number of frames of buffered occlusion queries.
	static int32 GetNumBufferedFrames(ERHIFeatureLevel::Type FeatureLevel);

	// get the index of the oldest query based on the current frame and number of buffered frames.
	static uint32 GetQueryLookupIndex(int32 CurrentFrame, int32 NumBufferedFrames)
	{
		// queries are currently always requested earlier in the frame than they are issued.
		// thus we can always overwrite the oldest query with the current one as we never need them
		// to coexist.  This saves us a buffer entry.
		const uint32 QueryIndex = CurrentFrame % NumBufferedFrames;
		return QueryIndex;
	}

	// get the index of the query to overwrite for new queries.
	static uint32 GetQueryIssueIndex(int32 CurrentFrame, int32 NumBufferedFrames)
	{
		// queries are currently always requested earlier in the frame than they are issued.
		// thus we can always overwrite the oldest query with the current one as we never need them
		// to coexist.  This saves us a buffer entry.
		const uint32 QueryIndex = CurrentFrame % NumBufferedFrames;
		return QueryIndex;
	}
};

/** Holds information about a single primitive's occlusion. */
class FPrimitiveOcclusionHistory
{
public:
	/** The primitive the occlusion information is about. */
	FPrimitiveComponentId PrimitiveId;

	/** The occlusion query which contains the primitive's pending occlusion results. */
	FRHIRenderQuery* PendingOcclusionQuery[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames];
	uint32 PendingOcclusionQueryFrames[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames]; 

	uint32 LastTestFrameNumber;
	uint32 LastConsideredFrameNumber;
	uint32 HZBTestIndex;

	/** The last time the primitive was visible. */
	float LastProvenVisibleTime;

	/** The last time the primitive was in the view frustum. */
	float LastConsideredTime;

	/** 
	 *	The pixels that were rendered the last time the primitive was drawn.
	 *	It is the ratio of pixels unoccluded to the resolution of the scene.
	 */
	float LastPixelsPercentage;

	/**
	* For things that have subqueries (foliage), this is the non-zero
	*/
	int32 CustomIndex;

	/** When things first become eligible for occlusion, then might be sweeping into the frustum, we are going to leave them at visible for a few frames, then start real queries.  */
	uint8 BecameEligibleForQueryCooldown : 6;

	uint8 WasOccludedLastFrame : 1;
	uint8 OcclusionStateWasDefiniteLastFrame : 1;

	/** whether or not this primitive was grouped the last time it was queried */
	bool bGroupedQuery[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames];

private:
	/**
	 *	Whether or not we need to linearly search the history for a past entry. Scanning may be necessary if for every frame there
	 *	is a hole in PendingOcclusionQueryFrames in the same spot (ex. if for every frame PendingOcclusionQueryFrames[1] is null).
	 *	This could lead to overdraw for the frames that attempt to read these holes by getting back nothing every time.
	 *	This can occur when round robin occlusion queries are turned on while NumBufferedFrames is even.
	 */
	bool bNeedsScanOnRead;

	/**
	 *	Scan for the oldest non-stale (<= LagTolerance frames old) in the occlusion history by examining their corresponding frame numbers.
	 *	Conditions where this is needed to get a query for read-back are described for bNeedsScanOnRead.
	 *	Returns -1 if no such query exists in the occlusion history.
	 */
	inline int32 ScanOldestNonStaleQueryIndex(uint32 FrameNumber, int32 NumBufferedFrames, int32 LagTolerance) const
	{
		uint32 OldestFrame = UINT32_MAX;
		int32 OldestQueryIndex = -1;
		for (int Index = 0; Index < NumBufferedFrames; ++Index)
		{
			const uint32 ThisFrameNumber = PendingOcclusionQueryFrames[Index];
			const int32 LaggedFrames = FrameNumber - ThisFrameNumber;
			// Queries older than LagTolerance are invalid. They may have already been reused and will give incorrect results if read
			if (PendingOcclusionQuery[Index] && LaggedFrames <= LagTolerance && ThisFrameNumber < OldestFrame)
			{
				OldestFrame = ThisFrameNumber;
				OldestQueryIndex = Index;
			}
		}
		return OldestQueryIndex;
	}

public:
	/** Initialization constructor. */
	inline FPrimitiveOcclusionHistory(FPrimitiveComponentId InPrimitiveId, int32 SubQuery)
		: PrimitiveId(InPrimitiveId)
		, LastTestFrameNumber(~0u)
		, LastConsideredFrameNumber(~0u)
		, HZBTestIndex(0)
		, LastProvenVisibleTime(0.0f)
		, LastConsideredTime(0.0f)
		, LastPixelsPercentage(0.0f)
		, CustomIndex(SubQuery)
		, BecameEligibleForQueryCooldown(0)
		, WasOccludedLastFrame(false)
		, OcclusionStateWasDefiniteLastFrame(false)
		, bNeedsScanOnRead(false)
	{
		for (int32 Index = 0; Index < FOcclusionQueryHelpers::MaxBufferedOcclusionFrames; Index++)
		{
			PendingOcclusionQuery[Index] = nullptr;
			PendingOcclusionQueryFrames[Index] = 0;
			bGroupedQuery[Index] = false;
		}
	}

	inline FPrimitiveOcclusionHistory()
		: LastTestFrameNumber(~0u)
		, LastConsideredFrameNumber(~0u)
		, HZBTestIndex(0)
		, LastProvenVisibleTime(0.0f)
		, LastConsideredTime(0.0f)
		, LastPixelsPercentage(0.0f)
		, CustomIndex(0)
		, BecameEligibleForQueryCooldown(0)
		, WasOccludedLastFrame(false)
		, OcclusionStateWasDefiniteLastFrame(false)
		, bNeedsScanOnRead(false)
	{
		for (int32 Index = 0; Index < FOcclusionQueryHelpers::MaxBufferedOcclusionFrames; Index++)
		{
			PendingOcclusionQuery[Index] = nullptr;
			PendingOcclusionQueryFrames[Index] = 0;
			bGroupedQuery[Index] = false;
		}
	}

	inline FRHIRenderQuery* GetQueryForReading(uint32 FrameNumber, int32 NumBufferedFrames, int32 LagTolerance, bool& bOutGrouped) const
	{
		const int32 OldestQueryIndex = bNeedsScanOnRead ? ScanOldestNonStaleQueryIndex(FrameNumber, NumBufferedFrames, LagTolerance)
														: FOcclusionQueryHelpers::GetQueryLookupIndex(FrameNumber, NumBufferedFrames);
		const int32 LaggedFrames = FrameNumber - PendingOcclusionQueryFrames[OldestQueryIndex];
		// Nenever read from queries are older than LagTolerance. They may have already been reused and will give incorrect results
		if (OldestQueryIndex == -1 || !PendingOcclusionQuery[OldestQueryIndex] || LaggedFrames > LagTolerance)
		{
			bOutGrouped = false;
			return nullptr;
		}
		bOutGrouped = bGroupedQuery[OldestQueryIndex];
		return PendingOcclusionQuery[OldestQueryIndex];
	}

	inline void SetCurrentQuery(uint32 FrameNumber, FRHIRenderQuery* NewQuery, int32 NumBufferedFrames, bool bGrouped, bool bNeedsScan)
	{
		// Get the current occlusion query
		const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryIssueIndex(FrameNumber, NumBufferedFrames);
		PendingOcclusionQuery[QueryIndex] = NewQuery;
		PendingOcclusionQueryFrames[QueryIndex] = FrameNumber;
		bGroupedQuery[QueryIndex] = bGrouped;

		bNeedsScanOnRead = bNeedsScan;
	}

	inline uint32 LastQuerySubmitFrame() const
	{
		uint32 Result = 0;

		for (int32 Index = 0; Index < FOcclusionQueryHelpers::MaxBufferedOcclusionFrames; Index++)
		{
			if (!bGroupedQuery[Index])
			{
				Result = FMath::Max(Result, PendingOcclusionQueryFrames[Index]);
			}
		}

		return Result;
	}
};

struct FPrimitiveOcclusionHistoryKey
{
	FPrimitiveComponentId PrimitiveId;
	int32 CustomIndex;

	explicit FPrimitiveOcclusionHistoryKey(const FPrimitiveOcclusionHistory& Element)
		: PrimitiveId(Element.PrimitiveId)
		, CustomIndex(Element.CustomIndex)
	{
	}
	FPrimitiveOcclusionHistoryKey(FPrimitiveComponentId InPrimitiveId, int32 InCustomIndex)
		: PrimitiveId(InPrimitiveId)
		, CustomIndex(InCustomIndex)
	{
	}
};

inline uint32 GetTypeHash(const FPrimitiveOcclusionHistoryKey& Key)
{
	return GetTypeHash(Key.PrimitiveId.PrimIDValue) ^ (GetTypeHash(Key.CustomIndex) >> 20);
}

inline bool operator==(const FPrimitiveOcclusionHistoryKey& A, const FPrimitiveOcclusionHistoryKey& B)
{
	return A.PrimitiveId == B.PrimitiveId && A.CustomIndex == B.CustomIndex;
}

/** Defines how the hash set indexes the FPrimitiveOcclusionHistory objects. */
struct FPrimitiveOcclusionHistoryKeyFuncs : BaseKeyFuncs<FPrimitiveOcclusionHistory,FPrimitiveOcclusionHistoryKey>
{
	typedef FPrimitiveOcclusionHistoryKey KeyInitType;

	static KeyInitType GetSetKey(const FPrimitiveOcclusionHistory& Element)
	{
		return FPrimitiveOcclusionHistoryKey(Element);
	}

	static bool Matches(KeyInitType A,KeyInitType B)
	{
		return A == B;
	}

	static uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};

class FIndividualOcclusionHistory
{
	FRHIPooledRenderQuery PendingOcclusionQuery[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames];
	uint32 PendingOcclusionQueryFrames[FOcclusionQueryHelpers::MaxBufferedOcclusionFrames]; // not initialized...this is ok

public:

	inline void ReleaseStaleQueries(uint32 FrameNumber, int32 NumBufferedFrames)
	{
		for (uint32 DeltaFrame = NumBufferedFrames; DeltaFrame > 0; DeltaFrame--)
		{
			if (FrameNumber >= (DeltaFrame - 1))
			{
				uint32 TestFrame = FrameNumber - (DeltaFrame - 1);
				const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryLookupIndex(TestFrame, NumBufferedFrames);
				if (PendingOcclusionQueryFrames[QueryIndex] != TestFrame)
				{
					PendingOcclusionQuery[QueryIndex].ReleaseQuery();
				}
			}
		}
	}
	inline void ReleaseQuery(uint32 FrameNumber, int32 NumBufferedFrames)
	{
		const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryLookupIndex(FrameNumber, NumBufferedFrames);
		PendingOcclusionQuery[QueryIndex].ReleaseQuery();
	}

	inline FRHIRenderQuery* GetPastQuery(uint32 FrameNumber, int32 NumBufferedFrames)
	{
		// Get the oldest occlusion query
		const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryLookupIndex(FrameNumber, NumBufferedFrames);
		if (PendingOcclusionQuery[QueryIndex].GetQuery() && PendingOcclusionQueryFrames[QueryIndex] == FrameNumber - uint32(NumBufferedFrames))
		{
			return PendingOcclusionQuery[QueryIndex].GetQuery();
		}
		return nullptr;
	}

	inline void SetCurrentQuery(uint32 FrameNumber, FRHIPooledRenderQuery&& NewQuery, int32 NumBufferedFrames)
	{
		// Get the current occlusion query
		const uint32 QueryIndex = FOcclusionQueryHelpers::GetQueryIssueIndex(FrameNumber, NumBufferedFrames);
		PendingOcclusionQuery[QueryIndex] = MoveTemp(NewQuery);
		PendingOcclusionQueryFrames[QueryIndex] = FrameNumber;
	}
};

class FOcclusionFeedback : public FRenderResource
{
public:
	FOcclusionFeedback();
	~FOcclusionFeedback();

	// FRenderResource interface
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;

	void AddPrimitive(const FPrimitiveOcclusionHistoryKey& PrimitiveKey, const FVector& BoundsOrigin, const FVector& BoundsBoxExtent, FGlobalDynamicVertexBuffer& DynamicVertexBuffer);

	void BeginOcclusionScope(FRDGBuilder& GraphBuilder);
	void EndOcclusionScope(FRDGBuilder& GraphBuilder);

	/** Renders the current batch and resets the batch state. */
	void SubmitOcclusionDraws(FRHICommandList& RHICmdList, FViewInfo& View);

	void ReadbackResults(FRHICommandListImmediate& RHICmdList);
	void AdvanceFrame(uint32 OcclusionFrameCounter);

	inline FRDGBuffer* GetGPUFeedbackBuffer() const
	{
		return GPUFeedbackBuffer;
	}

	inline bool IsOccluded(const FPrimitiveOcclusionHistoryKey& PrimitiveKey) const
	{
		return LatestOcclusionResults.Contains(PrimitiveKey);
	}

private:
	struct FOcclusionBatch
	{
		FGlobalDynamicVertexBuffer::FAllocation VertexAllocation;
		uint32 NumBatchedPrimitives;
	};

	/** The pending batches. */
	TArray<FOcclusionBatch, TInlineAllocator<3>> BatchOcclusionQueries;

	FRDGBuffer* GPUFeedbackBuffer{};

	struct FOcclusionBuffer
	{
		TArray<FPrimitiveOcclusionHistoryKey> BatchedPrimitives;
		FRHIGPUBufferReadback* ReadbackBuffer = nullptr;
		uint32 OcclusionFrameCounter = 0u;
	};

	FOcclusionBuffer OcclusionBuffers[3];
	uint32 CurrentBufferIndex;

	TSet<FPrimitiveOcclusionHistoryKey> LatestOcclusionResults;
	uint32 ResultsOcclusionFrameCounter;

	//
	FVertexDeclarationRHIRef OcclusionVertexDeclarationRHI;
};

